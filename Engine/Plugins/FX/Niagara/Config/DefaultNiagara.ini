[CoreRedirects]

+ClassRedirects=(OldName="NiagaraEmitterProperties", NewName="/Script/Niagara.NiagaraEmitter")
+ClassRedirects=(OldName="NiagaraEffect", NewName="/Script/Niagara.NiagaraSystem")
+ClassRedirects=(OldName="NiagaraEffectRendererProperties", NewName="/Script/Niagara.NiagaraRendererProperties")
+ClassRedirects=(OldName="NiagaraDataInterfaceTexture2D", NewName="/Script/Niagara.NiagaraDataInterfaceTexture")
+ClassRedirects=(OldName="NiagaraDataInterfacePhysicsAsset", NewName="/Script/Niagara.NiagaraDataInterfacePhysicsAsset")

+EnumRedirects=(OldName="ENiagaraScriptUsage",ValueChanges=(("ENiagaraScriptUsage::SpawnScript","ENiagaraScriptUsage::ParticleSpawnScript"), ("ENiagaraScriptUsage::SpawnScriptInterpolated","ENiagaraScriptUsage::ParticleSpawnScriptInterpolated"),("ENiagaraScriptUsage::UpdateScript","ENiagaraScriptUsage::ParticleUpdateScript"), ("ENiagaraScriptUsage::EventScript","ENiagaraScriptUsage::ParticleEventScript"), ("ENiagaraScriptUsage::SystemScript","ENiagaraScriptUsage::SystemSpawnScript")) )
+EnumRedirects=(OldName="/Niagara/Functions/Localspace/ENiagaraCoordinateSpace.ENiagaraCoordinateSpace",NewName="/Script/Niagara.ENiagaraCoordinateSpace",OverrideClassName="/Script/CoreUObject.Enum",ValueChanges=(("NewEnumerator0","Simulation"),("NewEnumerator1","World"),("NewEnumerator2","Local"),("NewEnumerator3","Max")))
+EnumRedirects=(OldName="/Niagara/Functions/Localspace/ENiagaraOrientationAxis.ENiagaraOrientationAxis",NewName="/Script/Niagara.ENiagaraOrientationAxis",OverrideClassName="/Script/CoreUObject.Enum",ValueChanges=(("NewEnumerator0","XAxis"),("NewEnumerator1","YAxis"),("NewEnumerator2","ZAxis"),("NewEnumerator3","Max")))
+EnumRedirects=(OldName="/Niagara/Enums/ENiagaraCoordinateSpace.ENiagaraCoordinateSpace",NewName="/Script/Niagara.ENiagaraCoordinateSpace",OverrideClassName="/Script/CoreUObject.Enum",ValueChanges=(("NewEnumerator0","Simulation"),("NewEnumerator1","World"),("NewEnumerator2","Local"),("NewEnumerator3","Max")))
+EnumRedirects=(OldName="/Niagara/Enums/ENiagaraOrientationAxis.ENiagaraOrientationAxis",NewName="/Script/Niagara.ENiagaraOrientationAxis",OverrideClassName="/Script/CoreUObject.Enum",ValueChanges=(("NewEnumerator0","XAxis"),("NewEnumerator1","YAxis"),("NewEnumerator2","ZAxis"),("NewEnumerator3","Max")))
+EnumRedirects=(OldName="/Niagara/Enums/ENiagaraExecutionStateManagement.ENiagaraExecutionStateManagement",NewName="/Script/Niagara.ENiagaraExecutionStateManagement",OverrideClassName="/Script/CoreUObject.Enum",ValueChanges=(("NewEnumerator0","Awaken"),("NewEnumerator1","SleepAndLetParticlesFinish"),("NewEnumerator2","SleepAndClearParticles"),("NewEnumerator3","KillImmediately"),("NewEnumerator4","KillAfterParticlesFinish")))

+PackageRedirects=(OldName="/Niagara/Enums/ENiagaraExecutionStateManagement", NewName="/Script/Niagara")
+PackageRedirects=(OldName="/Niagara/DefaultAssets/Templates/Emitters/Empty", NewName="/Niagara/DefaultAssets/Templates/Emitters/Minimal")
+PackageRedirects=(OldName="/Niagara/DefaultAssets/Templates/Emitters/Empty.Empty", NewName="/Niagara/DefaultAssets/Templates/Emitters/Minimal.Minimal")

+StructRedirects=(OldName="NiagaraScalabilitySettings",NewName="/Script/Niagara.NiagaraSystemScalabilitySettings")
+StructRedirects=(OldName="NiagaraScalabilityOverrides",NewName="/Script/Niagara.NiagaraSystemScalabilityOverride")

+PropertyRedirects=(OldName="NiagaraPlatformSet.EffectsQualityMask",NewName="QualityLevelMask")
+PropertyRedirects=(OldName="NiagaraDeviceProfileStateEntry.EffectsQualityMask",NewName="QualityLevelMask")
+PropertyRedirects=(OldName="NiagaraDeviceProfileStateEntry.SetEffectsQualityMask",NewName="SetQualityLevelMask")
+PropertyRedirects=(OldName="FNiagaraPlatformSetConflictEntry.EffectsQualityMask",NewName="QualityLevelMask")

+PropertyRedirects=(OldName="NiagaraDataInterfaceSkeletalMesh.DefaultMesh", NewName="NiagaraDataInterfaceSkeletalMesh.PreviewMesh")

;+EnumRedirects=(OldName="ENiagaraSortMode",ValueChanges=(("ENiagaraSortMode::SortNone","ENiagaraSortMode::None"), ("ENiagaraSortMode::SortViewDepth","ENiagaraSortMode::ViewDepth"), ("ENiagaraSortMode::SortViewDistance","ENiagaraSortMode::ViewDistance")) )

+PropertyRedirects=(OldName="NiagaraDataInterfaceSkeletalMesh.SpecificBones", NewName="NiagaraDataInterfaceSkeletalMesh.FilteredBones")
+PropertyRedirects=(OldName="NiagaraDataInterfaceSkeletalMesh.SpecificSockets", NewName="NiagaraDataInterfaceSkeletalMesh.FilteredSockets")

+PropertyRedirects=(OldName="PinGuidsForPath.InputAPinGuid", NewName="PinGuidsForPath.InputTruePinGuid")
+PropertyRedirects=(OldName="PinGuidsForPath.InputBPinGuid", NewName="PinGuidsForPath.InputFalsePinGuid")

+PropertyRedirects=(OldName="NiagaraRibbonRendererProperties.RibbonDistanceFromStart", NewName="NiagaraRibbonRendererProperties.RibbonUVDistance")

+PropertyRedirects=(OldName="NiagaraParameterBinding.Parameter", NewName="NiagaraParameterBinding.ResolvedParameter")

+EnumRedirects=(OldName="ENiagaraMessageSeverity",ValueChanges=(("ENiagaraMessageSeverity::CustomNote", "ENiagaraMessageSeverity::Info")) )
+EnumRedirects=(OldName="EStackIssueSeverity",ValueChanges=(("EStackIssueSeverity::CustomNote", "EStackIssueSeverity::Info")) )

+ClassRedirects=(OldName="NDIRenderTargetVolumeSimCacheData", NewName="/Script/Niagara.NDIRenderTargetSimCacheData")
+StructRedirects=(OldName="NDIRenderTargetVolumeSimCacheFrame", NewName="/Script/Niagara.NDIRenderTargetSimCacheFrame")

; Flipbook to Baker
+ClassRedirects=(OldName="NiagaraFlipbookSettings", NewName="/Script/Niagara.NiagaraBakerSettings")
+PropertyRedirects=(OldName="NiagaraSystem.FlipbookSettings", NewName="NiagaraSystem.BakerSettings")
+PropertyRedirects=(OldName="NiagaraSystem.FlipbookGeneratedSettings", NewName="NiagaraSystem.BakerGeneratedSettings")
+StructRedirects=(OldName="NiagaraFlipbookTextureSettings", NewName="/Script/Niagara.NiagaraBakerTextureSettings")
+StructRedirects=(OldName="NiagaraFlipbookTextureSource", NewName="/Script/Niagara.NiagaraBakerTextureSource")
; ParticleNeighborQuery -> NeighborQuery rename
+ClassRedirects=(OldName="NiagaraDataInterfaceParticleNeighborQuery", NewName="/Script/Niagara.NiagaraDataInterfaceNeighborQuery")

; Data Hierarchy Editor Redirects
+StructRedirects=(OldName="/Script/NiagaraEditor.NiagaraHierarchyIdentity",NewName="/Script/DataHierarchyEditor.HierarchyElementIdentity")
+ClassRedirects=(OldName="/Script/NiagaraEditor.NiagaraHierarchyItemBase",NewName="/Script/DataHierarchyEditor.HierarchyElement")
+ClassRedirects=(OldName="/Script/NiagaraEditor.NiagaraHierarchyRoot",NewName="/Script/DataHierarchyEditor.HierarchyRoot")
+ClassRedirects=(OldName="/Script/NiagaraEditor.NiagaraHierarchyItem",NewName="/Script/DataHierarchyEditor.HierarchyItem")
+ClassRedirects=(OldName="/Script/NiagaraEditor.NiagaraHierarchySection",NewName="/Script/DataHierarchyEditor.HierarchySection")
+ClassRedirects=(OldName="/Script/NiagaraEditor.NiagaraHierarchyCategory",NewName="/Script/DataHierarchyEditor.HierarchyCategory")

[/Script/Niagara.NiagaraBakerSettings]
BakeQualityLevel=Cinematic

[/Script/Niagara.NiagaraSettings]
+AdditionalParameterEnums=/Niagara/Enums/ENiagaraCoordinateSpace.ENiagaraCoordinateSpace
+AdditionalParameterEnums=/Niagara/Enums/ENiagaraOrientationAxis.ENiagaraOrientationAxis
+AdditionalParameterEnums=/Niagara/Enums/ENiagaraRandomnessMode.ENiagaraRandomnessMode
+QualityLevels=NSLOCTEXT("[/Script/Niagara]", "3D45F0C54145F29D6947A1880E359F0A", "Low")
+QualityLevels=NSLOCTEXT("[/Script/Niagara]", "E13C61484F7E1E9BCDE0BCA1DF7D32E6", "Medium")
+QualityLevels=NSLOCTEXT("[/Script/Niagara]", "F5CA410D441DB51D4A80B1B4B07071CB", "High")
+QualityLevels=NSLOCTEXT("[/Script/Niagara]", "D56BFF7C4D755EFCB5B350BDE094625A", "Epic")
+QualityLevels=NSLOCTEXT("[/Script/Niagara]", "1E2BA6CD40ADCA8512BC70AC59AACAD7", "Cinematic")
ComponentRendererWarningsPerClass=(("PostProcessComponent", NSLOCTEXT("[/Script/Niagara]", "022421AD4D0A1767041BEE86A524E12D", "The post process component has a separate \"Enabled\" flag in PostProcessVolume that you need to overwrite\n if you want to disable a component when a particle dies. Otherwise it will still affect the scene\n when it goes back into the component pool.")))
DefaultPSOPrecacheSettings=(DelayMode=DelayRendering,PreCachePriorityBoost=Highest,PrecachingTime=OnInitialize,bCacheOnLoad=False)

[/Script/NiagaraEditor.NiagaraEditorSettings]
DefaultSystem=None
DefaultEmitter=/Niagara/DefaultAssets/DefaultEmitter.DefaultEmitter
DefaultScript=/Niagara/DefaultAssets/DefaultModule.DefaultModule
DefaultDynamicInputScript=/Niagara/DefaultAssets/DefaultDynamicInputs.DefaultDynamicInputs
DefaultFunctionScript=/Niagara/DefaultAssets/DefaultFunction.DefaultFunction
DefaultModuleScript=/Niagara/DefaultAssets/DefaultModule.DefaultModule
DefaultEmptyEmitter=/Niagara/DefaultAssets/Templates/Emitters/Minimal.Minimal
DefaultMeshRendererMesh=/Niagara/DefaultAssets/S_Gnomon.S_Gnomon
DefaultDecalRendererMaterial=/Niagara/DefaultAssets/DefaultDecalMaterial.DefaultDecalMaterial
DefaultRibbonRendererMaterial=/Niagara/DefaultAssets/DefaultRibbonMaterial.DefaultRibbonMaterial
DefaultSpriteRendererMaterial=/Niagara/DefaultAssets/DefaultSpriteMaterial.DefaultSpriteMaterial
SystemAssetWizardConfiguration=/Niagara/DefaultAssets/TABC_SystemWizard.TABC_SystemWizard
EmitterAssetWizardConfiguration=/Niagara/DefaultAssets/TABC_EmitterWizard.TABC_EmitterWizard
RequiredSystemUpdateScript=/Niagara/Modules/System/SystemState.SystemState
+GraphCreationShortcuts=(Name="Numeric::Add",Input=(Key=A,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Numeric::Divide",Input=(Key=D,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Numeric::Pow",Input=(Key=E,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Numeric::Subtract",Input=(Key=S,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Select",Input=(Key=I,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Numeric::Multiply",Input=(Key=M,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Numeric::Normalize",Input=(Key=N,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Numeric::One Minus",Input=(Key=O,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Lerp",Input=(Key=L,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="float",Input=(Key=One,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="bool",Input=(Key=B,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Vector 2D",Input=(Key=Two,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Vector",Input=(Key=Three,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Vector 4",Input=(Key=Four,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+GraphCreationShortcuts=(Name="Linear Color",Input=(Key=C,bShift=False,bCtrl=False,bAlt=False,bCmd=False))
+CurveTemplates=(DisplayNameOverride="",CurveAsset=/Niagara/DefaultAssets/Curves/Templates/LinearRampUp.LinearRampUp)
+CurveTemplates=(DisplayNameOverride="",CurveAsset=/Niagara/DefaultAssets/Curves/Templates/LinearRampDown.LinearRampDown)
+CurveTemplates=(DisplayNameOverride="",CurveAsset=/Niagara/DefaultAssets/Curves/Templates/DropOff.DropOff)
+CurveTemplates=(DisplayNameOverride="",CurveAsset=/Niagara/DefaultAssets/Curves/Templates/EaseIn.EaseIn)
+CurveTemplates=(DisplayNameOverride="",CurveAsset=/Niagara/DefaultAssets/Curves/Templates/PulseOut.PulseOut)
+CurveTemplates=(DisplayNameOverride="",CurveAsset=/Niagara/DefaultAssets/Curves/Templates/SmoothRampUp.SmoothRampUp)
+CurveTemplates=(DisplayNameOverride="",CurveAsset=/Niagara/DefaultAssets/Curves/Templates/SmoothRampDown.SmoothRampDown)
+CurveTemplates=(DisplayNameOverride="",CurveAsset=/Niagara/DefaultAssets/Curves/Templates/RampUpDown.RampUpDown)
+PlaybackSpeeds=0.010000
+PlaybackSpeeds=0.100000
+PlaybackSpeeds=0.250000
+PlaybackSpeeds=0.500000
+PlaybackSpeeds=1.000000
+PlaybackSpeeds=1.500000
+PlaybackSpeeds=2.000000
ActionColors=(NiagaraColor=(R=0.494792,G=0.445313,B=0.445313,A=1.000000),GameColor=(R=0.510417,G=0.300158,B=0.051042,A=1.000000),PluginColor=(R=0.048438,G=0.461547,B=0.484375,A=1.000000),DeveloperColor=(R=0.070312,G=0.468750,B=0.100547,A=1.000000))
DefaultNoteColor=(R=0.000000,G=0.8,B=0.21,A=1.000000)
HLSLKeywordReplacements=(("AppendStructuredBuffer", "APPENDSTRUCTUREDBUFFER_VAR"),("asm", "ASSEMBLY_VAR"),("asm_fragment", "ASSEMBLY_FRAGMENT_VAR"),("BlendState", "BLEND_STATE_VAR"),("bool", "BOOL_VAR"),("bool1", "BOOL1_VAR"),("bool2", "BOOL2_VAR"),("bool3", "BOOL3_VAR"),("bool4", "BOOL4_VAR"),("break", "BREAK_VAR"),("Buffer", "BUFFER_VAR"),("ByteAddressBuffer", "BYTE_ADDRESS_BUFFER_VAR"),("case", "CASE_VAR"),("cbuffer", "CBUFFER_VAR"),("centroid", "CENTROID_VAR"),("class", "CLASS_VAR"),("column_major", "COLUMN_MAJOR_VAR"),("compile", "COMPILE_VAR"),("compile_fragment", "COMPILE_FRAGMENT_VAR"),("CompileShader", "COMPILE_SHADER_VAR"),("const", "CONST_VAR"),("continue", "CONTINUE_VAR"),("ComputeShader", "COMPUTE_SHADER_VAR"),("ConsumeStructuredBuffer", "CONSUME_STRUCTURED_BUFFER"),("default", "DEFAULT_VAR"),("DepthStencilState", "DEPTHSTENCILSTATE_VAR"),("DepthStencilView", "DEPTHSTENCILVIEW_VAR"),("discard", "DISCARD_VAR"),("do", "DO_VAR"),("double", "DOUBLE_VAR"),("double1", "DOUBLE1_VAR"),("double2", "DOUBLE2_VAR"),("double3", "DOUBLE3_VAR"),("double4", "DOUBLE4_VAR"),("DomainShader", "DOMAINSHADER_VAR"),("dword", "DWORD_VAR"),("else", "ELSE_VAR"),("export", "EXPORT_VAR"),("extern", "EXTERN_VAR"),("false", "FALSE_VAR"),("float", "FLOAT_VAR"),("float1", "FLOAT1_VAR"),("float2", "FLOAT2_VAR"),("float3", "FLOAT3_VAR"),("float4", "FLOAT4_VAR"),("for", "FOR_VAR"),("fxgroup", "FXGROUP_VAR"),("GeometryShader", "GEOMETRYSHADER_VAR"),("groupshared", "GROUPSHARED_VAR"),("half", "HALF_VAR"),("half1", "HALF1_VAR"),("half2", "HALF2_VAR"),("half3", "HALF3_VAR"),("half4", "HALF4_VAR"),("Hullshader", "HULLSHADER_VAR"),("if", "IF_VAR"),("in", "IN_VAR"),("inline", "INLINE_VAR"),("inout", "INOUT_VAR"),("InputPatch", "INPUTPATCH_VAR"),("int", "INT_VAR"),("int1", "INT1_VAR"),("int2", "INT2_VAR"),("int3", "INT3_VAR"),("int4", "INT4_VAR"),("interface", "INTERFACE_VAR"),("line", "LINE_VAR"),("lineadj", "LINEADJ_VAR"),("linear", "LINEAR_VAR"),("LineStream", "LINESTREAM_VAR"),("matrix", "MATRIX_VAR"),("min16float", "MIN16FLOAT_VAR"),("min10float", "MIN10FLOAT_VAR"),("min16int", "MIN16INT_VAR"),("min12int", "MIN12INT_VAR"),("min16uint", "MIN16UINT_VAR"),("namespace", "NAMESPACE_VAR"),("nointerpolation", "NOINTERPOLATION_VAR"),("noperspective", "NOPERSPECTIVE_VAR"),("NULL", "NULL_VAR"),("out", "OUT_VAR"),("OutputPatch", "OUTPUTPATCH_VAR"),("packoffset", "PACKOFFSET,_VAR"),("pass", "PASS,_VAR"),("pixelfragment", "PIXELFRAGMENT_VAR"),("PixelShader", "PIXELSHADER_VAR"),("point", "POINT_VAR"),("PointStream", "POINTSTREAM_VAR"),("precise", "PRECISE_VAR"),("RasterizerState", "RASTERIZERSTATE_VAR"),("RenderTargetView", "RENDERTARGETVIEW_VAR"),("return", "RETURN_VAR"),("register", "REGISTER_VAR"),("row_major", "ROW_MAJOR_VAR"),("RWBuffer", "RWBUFFER_VAR"),("RWByteAddressBuffer", "RWBYTEADDRESSBUFFER_VAR"),("RWStructuredBuffer", "RWSTRUCTUREDBUFFER_VAR"),("RWTexture1D", "RWTEXTURE1D_VAR"),("RWTexture1DArray", "RWTEXTURE1DARRAY_VAR"),("RWTexture2D", "RWTEXTURE2D_VAR"),("RWTexture2DArray", "RWTEXTURE2DARRAY_VAR"),("RWTexture3D", "RWTEXTURE3D_VAR"),("sample", "SAMPLE_VAR"),("sampler", "SAMPLER_VAR"),("SamplerState", "SAMPLERSTATE_VAR"),("SamplerComparisonState", "SAMPLERCOMPARISONSTATE_VAR"),("shared", "SHARED_VAR"),("snorm", "SNORM_VAR"),("stateblock", "STATEBLOCK_VAR"),("stateblock_state", "STATEBLOCK_STATE_VAR"),("static", "STATIC_VAR"),("string", "STRING_VAR"),("struct", "STRUCT_VAR"),("switch", "SWITCH_VAR"),("StructuredBuffer", "STRUCTUREDBUFFER_VAR"),("tbuffer", "TBUFFER_VAR"),("technique", "TECHNIQUE_VAR"),("technique10", "TECHNIQUE10_VAR"),("technique11", "TECHNIQUE11_VAR"),("texture", "TEXTURE_VAR"),("Texture1D", "TEXTURE1D_VAR"),("Texture1DArray", "TEXTURE1DARRAY_VAR"),("Texture2D", "TEXTURE2D_VAR"),("Texture2DArray", "TEXTURE2DARRAY_VAR"),("Texture2DMS", "TEXTURE2DMS_VAR"),("Texture2DMSArray", "TEXTURE2DMSARRAY_VAR"),("Texture3D", "TEXTURE3D_VAR"),("TextureCube", "TEXTURECUBE_VAR"),("TextureCubeArray", "TEXTURECUBEARRAY_VAR"),("true", "TRUE_VAR"),("typedef", "TYPEDEF_VAR"),("triangle", "TRIANGLE_VAR"),("triangleadj", "TRIANGLEADJ_VAR"),("TriangleStream", "TRIANGLESTREAM_VAR"),("uint", "UINT_VAR"),("uint1", "UINT1_VAR"),("uint2", "UINT2_VAR"),("uint3", "UINT3_VAR"),("uint4", "UINT4_VAR"),("uniform", "UNIFORM_VAR"),("unorm", "UNORM_VAR"),("unsigned", "UNSIGNED_VAR"),("vector", "VECTOR_VAR"),("vertexfragment", "VERTEXFRAGMENT_VAR"),("VertexShader", "VERTEXSHADER_VAR"),("void", "VOID_VAR"),("volatile", "VOLATILE_VAR"),("while", "WHILE_VAR"),("attribute", "ATTRIBUTE_VAR"),("varying", "VARYING_VAR"),("layout", "LAYOUT_VAR"),("mat2", "MAT2_VAR"),("mat3", "MAT3_VAR"),("mat4", "MAT4_VAR"),("mat2x2", "MAT2X2_VAR"),("mat2x3", "MAT2X3_VAR"),("mat2x4", "MAT2X4_VAR"),("mat3x2", "MAT3X2_VAR"),("mat3x3", "MAT3X3_VAR"),("mat3x4", "MAT3X4_VAR"),("mat4x2", "MAT4X2_VAR"),("mat4x3", "MAT4X3_VAR"),("mat4x4", "MAT4X4_VAR"),("vec2", "VEC2_VAR"),("vec3", "VEC3_VAR"),("vec4", "VEC4_VAR"),("ivec2", "IVEC2_VAR"),("ivec3", "IVEC3_VAR"),("ivec4", "IVEC4_VAR"),("bvec2", "BVEC2_VAR"),("bvec3", "BVEC3_VAR"),("bvec4", "BVEC4_VAR"),("uvec2", "UVEC2_VAR"),("uvec3", "UVEC3_VAR"),("uvec4", "UVEC4_VAR"),("lowp", "LOWP_VAR"),("mediump", "MEDIUMP_VAR"),("highp", "HIGHP_VAR"),("precision", "PRECISION_VAR"),("sampler1D", "SAMPLER1D_VAR"),("sampler2D", "SAMPLER2D_VAR"),("sampler3D", "SAMPLER3D_VAR"),("samplerCube", "SAMPLERCUBE_VAR"),("sampler1DShadow", "SAMPLER1DSHADOW_VAR"),("sampler2DShadow", "SAMPLER2DSHADOW_VAR"),("samplerCubeShadow", "SAMPLERCUBESHADOW_VAR"),("sampler1DArray", "SAMPLER1DARRAY_VAR"),("sampler2DArray", "SAMPLER2DARRAY_VAR"),("sampler1DArrayShadow", "SAMPLER1DARRAYSHADOW_VAR"),("sampler2DArrayShadow", "SAMPLER2DARRAYSHADOW_VAR"),("isampler1D", "ISAMPLER1D_VAR"),("isampler2D", "ISAMPLER2D_VAR"),("isampler3D", "ISAMPLER3D_VAR"),("isamplerCube", "ISAMPLERCUBE_VAR"),("isampler1DArray", "ISAMPLER1DARRAY_VAR"),("isampler2DArray", "ISAMPLER2DARRAY_VAR"),("usampler1D", "USAMPLER1D_VAR"),("usampler2D", "USAMPLER2D_VAR"),("usampler3D", "USAMPLER3D_VAR"),("usamplerCube", "USAMPLERCUBE_VAR"),("usampler1DArray", "USAMPLER1DARRAY_VAR"),("usampler2DArray", "USAMPLER2DARRAY_VAR"),("sampler2DRect", "SAMPLER2DRECT_VAR"),("sampler2DRectShadow", "SAMPLER2DRECTSHADOW_VAR"),("isampler2DRect", "ISAMPLER2DRECT_VAR"),("usampler2DRect", "USAMPLER2DRECT_VAR"),("samplerBuffer", "SAMPLERBUFFER_VAR"),("isamplerBuffer", "ISAMPLERBUFFER_VAR"),("usamplerBuffer", "USAMPLERBUFFER_VAR"),("struct", "STRUCT_VAR"),("flat", "FLAT_VAR"),("smooth", "SMOOTH_VAR"),("invariant", "INVARIANT_VAR"),("union", "UNION_VAR"),("enum", "ENUM_VAR"),("typedef", "TYPEDEF_VAR"),("template", "TEMPLATE_VAR"),("this", "THIS_VAR"),("packed", "PACKED_VAR"),("goto", "GOTO_VAR"),("noinline", "NOINLINE_VAR"),("public", "PUBLIC_VAR"),("external", "EXTERNAL_VAR"),("long short", "LONG SHORT_VAR"),("superp", "SUPERP_VAR"),("input", "INPUT_VAR"),("output", "OUTPUT_VAR"),("hvec2", "HVEC2_VAR"),("hvec3", "HVEC3_VAR"),("hvec4", "HVEC4_VAR"),("dvec2", "DVEC2_VAR"),("dvec3", "DVEC3_VAR"),("dvec4", "DVEC4_VAR"),("fvec2", "FVEC2_VAR"),("fvec3", "FVEC3_VAR"),("fvec4", "FVEC4_VAR"),("sampler3DRect", "SAMPLER3DRECT_VAR"),("filter", "FILTER_VAR"),("image1D", "IMAGE1D_VAR"),("image2D", "IMAGE2D_VAR"),("image3D", "IMAGE3D_VAR"),("imageCube", "IMAGECUBE_VAR"),("iimage1D", "IIMAGE1D_VAR"),("iimage2D", "IIMAGE2D_VAR"),("iimage3D", "IIMAGE3D_VAR"),("iimageCub", "IIMAGECUBE_VAR"),("uimage1D", "UIMAGE1D_VAR"),("uimage2D", "UIMAGE2D_VAR"),("uimage3D", "UIMAGE3D_VAR"),("uimageCube", "UIMAGECUBE_VAR"),("image1DArray", "IMAGE1DARRAY_VAR"),("image2DArray", "IMAGE2DARRAY_VAR"),("iimage1DArray", "IIMAGE1DARRAY_VAR"),("iimage2DArray", "IIMAGE2DARRAY_VAR"),("uimage1DArray", "UIMAGE1DARRAY_VAR"),("uimage2DArray", "UIMAGE2DARRAY_VAR"),("image1DShadow", "IMAGE1DSHADOW_VAR"),("image2DShadow", "IMAGE2DSHADOW_VAR"),("image1DArrayShadow", "IMAGE1DARRAYSHADOW_VAR"),("image2DArrayShadow", "IMAGE2DARRAYSHADOW_VAR"),("imageBuffer", "IMAGEBUFFER_VAR"),("iimageBuffer", "IIMAGEBUFFER_VAR"),("uimageBuffer", "UIMAGEBUFFER_VAR"),("sizeof", "SIZEOF_VAR"),("cast", "CAST_VAR"),("using", "USING_VAR"),("row_major", "ROW_MAJOR_VAR"),("int8_t", "INT8_T_VAR"),("unsigned char", "UNSIGNED_CHAR_VAR"),("uchar", "UCHAR_VAR"),("uint8_t", "UINT8_T_VAR"),("short", "SHORT_VAR"),("int16_t", "INT16_T_VAR"),("unsigned shorT", "UNSIGNED_SHORT_VAR"),("ushort", "USHORT_VAR"),("unit16_t", "UNIT16_T_VAR"),("int32_t", "INT32_T_VAR"),("unsigned int", "UNSIGNED_INT_VAR"),("uint32_t", "UINT32_T_VAR"),("smooth", "SMOOTH_VAR"),("invariant", "INVARIANT_VAR"),("union", "UNION_VAR"),("enum", "ENUM_VAR"),("typedef", "TYPEDEF_VAR"),("template", "TEMPLATE_VAR"),("this", "THIS_VAR"),("packed", "PACKED_VAR"),("goto", "GOTO_VAR"),("noinline", "NOINLINE_VAR"),("public", "PUBLIC_VAR"),("external", "EXTERNAL_VAR"),(" long short", "LONG_SHORT_VAR"),("superp", "SUPERP_VAR"),("input", "INPUT_VAR"),("output", "OUTPUT_VAR"),("hvec2", "HVEC2_VAR"),("hvec3", "HVEC3_VAR"),("hvec4", "HVEC4_VAR"),("dvec2", "DVEC2_VAR"),("dvec3", "DVEC3_VAR"),("dvec4", "DVEC4_VAR"),("fvec2", "FVEC2_VAR"),("fvec3", "FVEC3_VAR"),("fvec4", "FVEC4_VAR"),("sampler3DRect", "SAMPLER3DRECT_VAR"),("filter", "FILTER_VAR"),("image2D", "IMAGE2D_VAR"),("image3D", "IMAGE3D_VAR"),("imageCube", "IMAGECUBE_VAR"),("iimage1D", "IIMAGE1D_VAR"),("iimage2D", "IIMAGE2D_VAR"),("iimage3D", "IIMAGE3D_VAR"),("iimageCube", "IIMAGECUBE_VAR"),("uimage1D", "UIMAGE1D_VAR"),("uimage2D", "UIMAGE2D_VAR"),("uimage3D", "UIMAGE3D_VAR"),("uimageCube", "UIMAGECUBE_VAR"),("image1DArray", "IMAGE1DARRAY_VAR"),("image2DArray", "IMAGE2DARRAY_VAR"),("iimage1DArray", "IIMAGE1DARRAY_VAR"),("iimage2DArray", "IIMAGE2DARRAY_VAR"),("uimage1DArray", "UIMAGE1DARRAY_VAR"),("uimage2DArray", "UIMAGE2DARRAY_VAR"),("image1DShadow", "IMAGE1DSHADOW_VAR"),("image2DShadow", "IMAGE2DSHADOW_VAR"),("image1DArrayShadow", "IMAGE1DARRAYSHADOW_VAR"),("image2DArrayShadow", "IMAGE2DARRAYSHADOW_VAR"),("imageBuffer", "IMAGEBUFFER_VAR"),("iimageBuffer", "IIMAGEBUFFER_VAR"),("uimageBuffer", "UIMAGEBUFFER_VAR"),("sizeof", "SIZEOF_VAR"),("cast", "CAST_VAR"),("using", "USING_VAR"),("row_major", "ROW_MAJOR_VAR"),("int8_t", "INT8_T_VAR"),("unsigned char", "UNSIGNED_CHAR_VAR"),("uchar", "UCHAR_VAR"),("uint8_t", "UINT8_T_VAR"),("short", "SHORT_VAR"),("int16_t", "INT16_T_VAR"),("unsigned short", "UNSIGNED_SHORT_VAR"),("ushort", "USHORT_VAR"),("unit16_t", "UNIT16_T_VAR"),("int32_t", "INT32_T_VAR"),("unsigned int", "UNSIGNED_INT_VAR"),("uint32_t", "UINT32_T_VAR"))
bShowGridInViewport=True
bShowInstructionsCount=False
bShowParticleCountsInViewport=False
bShowEmitterExecutionOrder=False
bDisplayPreviewMoviesInTooltips=True
TooltipPreviewMovieSize=(X=128.000000,Y=128.000000)
