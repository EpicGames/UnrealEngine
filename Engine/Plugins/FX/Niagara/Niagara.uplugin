{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Niagara",
	"Description" : "Niagara effect systems.",
	"Category" : "FX",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : true,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Plugins": [
		{
			"Name": "PythonScriptPlugin",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "NiagaraCore",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "Niagara",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "NiagaraAnimNotifies",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "NiagaraShader",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "NiagaraVertexFactories",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "NiagaraBlueprintNodes",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NiagaraEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NiagaraEditorWidgets",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	] 
}
