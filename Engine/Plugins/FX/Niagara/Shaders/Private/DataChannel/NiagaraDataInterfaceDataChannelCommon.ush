// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**

Shader code common to all instances and functions of Niagara Data Channel DI.

*/

//TODO: Bindless buffers would allow us to read from any number of separate buffers without having to do a bunch of copying around.
#define NIAGARA_DATACHANNEL_BINDLESSBUFFERS 0

//TOOD: Add support for N multiple buffers
//TODO: Add support for bindless buffers on platforms that support it.

void NiagaraDataChannelRead_Float(Buffer<float> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out float OutValue)
{
	if(RegisterIdx != -1)
	{
		OutValue = InBuffer[RegisterIdx * Stride + ParticleIdx];
		++RegisterIdx;
	}
	else
	{
		OutValue = 0.0;
	}
}

void NiagaraDataChannelRead_Float(Buffer<float> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out float2 OutValue)
{
	NiagaraDataChannelRead_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.x);
	NiagaraDataChannelRead_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.y);
}

void NiagaraDataChannelRead_Float(Buffer<float> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out float3 OutValue)
{
	NiagaraDataChannelRead_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.x);
	NiagaraDataChannelRead_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.y);
	NiagaraDataChannelRead_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.z);
}

void NiagaraDataChannelRead_Float(Buffer<float> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out float4 OutValue)
{
	NiagaraDataChannelRead_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.x);
	NiagaraDataChannelRead_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.y);
	NiagaraDataChannelRead_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.z);
	NiagaraDataChannelRead_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.w);
}

void NiagaraDataChannelRead_Int32(Buffer<int> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out int OutValue)
{
	if(RegisterIdx != -1)
	{
		OutValue = InBuffer[RegisterIdx * Stride + ParticleIdx];
		++RegisterIdx;
	}
	else
	{
		OutValue = 0;
	}
}

void NiagaraDataChannelRead_Int32(Buffer<int> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out int2 OutValue)
{
	NiagaraDataChannelRead_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.x);
	NiagaraDataChannelRead_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.y);
}

void NiagaraDataChannelRead_Int32(Buffer<int> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out int3 OutValue)
{
	NiagaraDataChannelRead_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.x);
	NiagaraDataChannelRead_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.y);
	NiagaraDataChannelRead_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.z);
}

void NiagaraDataChannelRead_Int32(Buffer<int> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out int4 OutValue)
{
	NiagaraDataChannelRead_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.x);
	NiagaraDataChannelRead_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.y);
	NiagaraDataChannelRead_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.z);
	NiagaraDataChannelRead_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.w);
}

void NiagaraDataChannelRead_Bool(Buffer<int> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out bool OutValue)
{
	if(RegisterIdx != -1)
	{
		int IntVal = InBuffer[RegisterIdx * Stride + ParticleIdx];
		OutValue = IntVal == 0 ? false : true;
		++RegisterIdx;
	}
	else
	{
		OutValue = false;
	}
}

/*
//TODO: Half Support | 
void NiagaraDataChannelRead_Half(Buffer<half> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out half OutValue)
{
	if(RegisterIdx != -1)
	{
		OutValue = InBuffer[RegisterIdx * Stride + ParticleIdx];
		++RegisterIdx;
	}
	else
	{
		OutValue = 0.0;
	}
}

void NiagaraDataChannelRead_Half(Buffer<half> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out half2 OutValue)
{
	NiagaraDataChannelRead_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.x);
	NiagaraDataChannelRead_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.y);
}

void NiagaraDataChannelRead_Half(Buffer<half> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out half3 OutValue)
{
	NiagaraDataChannelRead_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.x);
	NiagaraDataChannelRead_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.y);
	NiagaraDataChannelRead_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.z);
}

void NiagaraDataChannelRead_Half(Buffer<half> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, out half4 OutValue)
{
	NiagaraDataChannelRead_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.x);
	NiagaraDataChannelRead_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.y);
	NiagaraDataChannelRead_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.z);
	NiagaraDataChannelRead_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, OutValue.w);
}
*/

// Write accessors 

void NiagaraDataChannelWrite_Float(RWBuffer<float> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, float InValue)
{
	if(RegisterIdx != -1)
	{
		InBuffer[RegisterIdx * Stride + ParticleIdx] = InValue;
		++RegisterIdx;
	}
}

void NiagaraDataChannelWrite_Float(RWBuffer<float> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, float2 InValue)
{
	NiagaraDataChannelWrite_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.x);
	NiagaraDataChannelWrite_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.y);
}

void NiagaraDataChannelWrite_Float(RWBuffer<float> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, float3 InValue)
{
	NiagaraDataChannelWrite_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.x);
	NiagaraDataChannelWrite_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.y);
	NiagaraDataChannelWrite_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.z);
}

void NiagaraDataChannelWrite_Float(RWBuffer<float> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, float4 InValue)
{
	NiagaraDataChannelWrite_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.x);
	NiagaraDataChannelWrite_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.y);
	NiagaraDataChannelWrite_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.z);
	NiagaraDataChannelWrite_Float(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.w);
}

void NiagaraDataChannelWrite_Int32(RWBuffer<int> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, int InValue)
{
	if(RegisterIdx != -1)
	{
		InBuffer[RegisterIdx * Stride + ParticleIdx] = InValue;
		++RegisterIdx;
	}
}

void NiagaraDataChannelWrite_Int32(RWBuffer<int> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, int2 InValue)
{
	NiagaraDataChannelWrite_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.x);
	NiagaraDataChannelWrite_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.y);
}

void NiagaraDataChannelWrite_Int32(RWBuffer<int> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, int3 InValue)
{
	NiagaraDataChannelWrite_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.x);
	NiagaraDataChannelWrite_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.y);
	NiagaraDataChannelWrite_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.z);
}

void NiagaraDataChannelWrite_Int32(RWBuffer<int> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, int4 InValue)
{
	NiagaraDataChannelWrite_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.x);
	NiagaraDataChannelWrite_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.y);
	NiagaraDataChannelWrite_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.z);
	NiagaraDataChannelWrite_Int32(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.w);
}

void NiagaraDataChannelWrite_Bool(RWBuffer<int> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, bool InValue)
{
	if(RegisterIdx != -1)
	{
		InBuffer[RegisterIdx * Stride + ParticleIdx] = InValue ? 0xFFFFFFFF : 0;
		++RegisterIdx;
	}
}

/*
//TODO: Half Support | 
void NiagaraDataChannelWrite_Half(RWBuffer<half> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, half InValue)
{
	if(RegisterIdx != -1)
	{
		InBuffer[RegisterIdx * Stride + ParticleIdx] = InValue;
		++RegisterIdx;
	}
}

void NiagaraDataChannelWrite_Half(RWBuffer<half> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, half2 InValue)
{
	NiagaraDataChannelWrite_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.x);
	NiagaraDataChannelWrite_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.y);
}

void NiagaraDataChannelWrite_Half(RWBuffer<half> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, half3 InValue)
{
	NiagaraDataChannelWrite_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.x);
	NiagaraDataChannelWrite_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.y);
	NiagaraDataChannelWrite_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.z);
}

void NiagaraDataChannelWrite_Half(RWBuffer<half> InBuffer, uint Stride, in out uint RegisterIdx, uint ParticleIdx, half4 InValue)
{
	NiagaraDataChannelWrite_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.x);
	NiagaraDataChannelWrite_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.y);
	NiagaraDataChannelWrite_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.z);
	NiagaraDataChannelWrite_Half(InBuffer, Stride, RegisterIdx, ParticleIdx, InValue.w);
}
*/

//TODO: More base type accessors.

