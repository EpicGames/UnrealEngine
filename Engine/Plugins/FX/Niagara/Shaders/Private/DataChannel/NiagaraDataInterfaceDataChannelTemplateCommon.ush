// Copyright Epic Games, Inc. All Rights Reserved.

/**
Template hlsl for Data Channel DI functions to apply to their own that is shared accross all functions for the DI.
*/

//Offset of each parameter accessed by this script.
//Indices for accessing this table are embeded directly into the HLSL and we generate the table on the CPU accordingly.
//TODO: Can probably pack this into a single global buffer rather than one per DI.
Buffer<uint> {ParameterName}_ParamOffsetTable;

int {ParameterName}_ParameterOffsetTableIndex;
int {ParameterName}_FloatStride;
int {ParameterName}_Int32Stride;
//TODO: Half Support | int {ParameterName}_HalfStride;

// Outputs the starting offset of each component in the channel data for a partiuclar parameter.
bool GetParameterLayoutInfo_{ParameterName}(uint ParameterIndex, out uint OutFloatRegister, out uint OutInt32Register)//TODO: Half Support | , out uint OutHalfRegister)
{
	int ParameterTableStart = {ParameterName}_ParameterOffsetTableIndex;
	if(ParameterTableStart != -1)
	{
		const int NumComponents = 2;
		uint ParamLayoutStart = ParameterTableStart + NumComponents * ParameterIndex;
		OutFloatRegister = {ParameterName}_ParamOffsetTable[ParamLayoutStart];
		OutInt32Register = {ParameterName}_ParamOffsetTable[ParamLayoutStart + 1];
		//TODO: Half Support | OutHalfRegister = {ParameterName}_ParamOffsetTable[ParamLayoutStart + 2];
		return OutFloatRegister != -1 || OutInt32Register != -1;//If we have no valid mappings then this parameter is not valid.
	}
	else
	{
		OutFloatRegister = -1;
		OutInt32Register = -1;
		//TODO: Half Support | OutHalfRegister = -1;
		return false;
	}
}
