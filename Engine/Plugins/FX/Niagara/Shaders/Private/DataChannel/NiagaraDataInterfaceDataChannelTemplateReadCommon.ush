// Copyright Epic Games, Inc. All Rights Reserved.

/**
Template hlsl for Data Channel Read DI functions to apply to their own that is shared accross all functions for the DI.
*/

Buffer<float> {ParameterName}_DataFloat;
Buffer<int> {ParameterName}_DataInt32;
//TODO: Half Support | Buffer<half> {ParameterName}_DataHalf;

//Instance count offset for the buffer we're reading.
//Must be done via an instance count slot as data can come from CPU and GPU.
int {ParameterName}_InstanceCountOffset;

//Instance count offset for tracking consume calls.
int {ParameterName}_ConsumeInstanceCountOffset;

int {ParameterName}_BufferSize;

int {ParameterName}_NDCElementCountAtSpawn;

//Buffer containing back mapping from spawn particle exec index to the NDC index that spawned that particle.
//Arraged into power of two buckets so that we can avoid an entry per particle.
Buffer<int> {ParameterName}_NDCSpawnDataBuffer;

//Helper struct with various info needed for reading that can be passed around more easily in generated code.
struct FNDCAccessContext_{ParameterName}
{
	bool bValidParameter;
	bool bValidReadIndex;

	//Index at which to read data.
	int ReadIndex;
		
	//Component start offests for the current parameter.
	int ParameterComponentStart_Float;
	int ParameterComponentStart_Int32;
	//TODO: Half Support | int ParameterComponentStart_Half;

	//Current registe indices for each of the GPU write buffers
	int Register_Float;
	int Register_Int32;
	//TODO: Half Support | int Register_Half;

	int GetFloatStride() { return {ParameterName}_FloatStride; }
	int GetIntStride() { return {ParameterName}_Int32Stride; }
	//TODO: Half Support | int GetHalfStride() { return {ParameterName}_HalfStride; }
		
	Buffer<float> GetBuffer_Float() { return {ParameterName}_DataFloat; }
	Buffer<int> GetBuffer_Int32() { return {ParameterName}_DataInt32; }
	//TODO: Half Support | Buffer<half> GetBuffer_Half() { return {ParameterName}_DataHalf; }
	
	//Instance count used for the written number of elements in the NDC buffers.
	int GetInstanceCountOffset() { return {ParameterName}_InstanceCountOffset; }

	//Instance count used to get a read index when calling Consume().
	int GetConsumeInstanceCountOffset() { return {ParameterName}_ConsumeInstanceCountOffset; }
	
	//Return the actual buffer size allocated.
	int GetBufferSize() { return {ParameterName}_BufferSize; }

	//Actual number of NDC items in the buffers. Must come from an indirect instance count instead of a uniform as the GPU could have written to this buffer.
	int Num() { return RWInstanceCounts[ GetInstanceCountOffset() ]; }

	int GetConsumeIndex()
	{
		//TODO: Add GS and Wave op path to reduce atomic contention.
		//TODO: Right now doing the simple dumb thing.

		int OutIndex = -1;
		if(GetConsumeInstanceCountOffset() != -1)
		{
			uint OrigVal = 0;
			InterlockedAdd(RWInstanceCounts[GetConsumeInstanceCountOffset()], 1U, OrigVal);
			int MaxSize = min(Num(), GetBufferSize());
			if(OrigVal >= 0 && OrigVal < (uint)MaxSize)
			{
				OutIndex = (int)OrigVal;
			}
			else
			{
				OutIndex = -1;
				InterlockedMin(RWInstanceCounts[GetConsumeInstanceCountOffset()], (uint)MaxSize);
			}
		}

		return OutIndex;
	}
	
	bool InitDirect(int InReadIndex)
	{
		ReadIndex = InReadIndex;
		bValidReadIndex = ReadIndex >= 0 && ReadIndex < Num() && ReadIndex < GetBufferSize();
		return bValidReadIndex;
	}

	bool InitConsume()
	{
		ReadIndex = GetConsumeIndex();
		bValidReadIndex = ReadIndex >= 0 && ReadIndex < Num() && ReadIndex < GetBufferSize();
		return bValidReadIndex;
	}

	bool InitForParameter(int ParameterIndex)
	{
		bValidParameter = GetParameterLayoutInfo_{ParameterName}(ParameterIndex, ParameterComponentStart_Float, ParameterComponentStart_Int32);//TODO: Half Support | ParameterComponentStart_Half);
		
		Register_Float = ParameterComponentStart_Float;
		Register_Int32 = ParameterComponentStart_Int32;
		//TODO: Half Support | Register_Half = ParameterComponentStart_Half;
	
		return bValidParameter;
	}

	void Read_Float(out float Value){ NiagaraDataChannelRead_Float(GetBuffer_Float(), GetFloatStride(), Register_Float, ReadIndex, Value); }
	void Read_Float(out float2 Value){ NiagaraDataChannelRead_Float(GetBuffer_Float(), GetFloatStride(), Register_Float, ReadIndex, Value); }
	void Read_Float(out float3 Value){ NiagaraDataChannelRead_Float(GetBuffer_Float(), GetFloatStride(), Register_Float, ReadIndex, Value); }
	void Read_Float(out float4 Value){ NiagaraDataChannelRead_Float(GetBuffer_Float(), GetFloatStride(), Register_Float, ReadIndex, Value); }

	void Read_Int32(out int Value){ NiagaraDataChannelRead_Int32(GetBuffer_Int32(), GetIntStride(), Register_Int32, ReadIndex, Value); }
	void Read_Int32(out int2 Value){ NiagaraDataChannelRead_Int32(GetBuffer_Int32(), GetIntStride(), Register_Int32, ReadIndex, Value); }
	void Read_Int32(out int3 Value){ NiagaraDataChannelRead_Int32(GetBuffer_Int32(), GetIntStride(), Register_Int32, ReadIndex, Value); }
	void Read_Int32(out int4 Value){ NiagaraDataChannelRead_Int32(GetBuffer_Int32(), GetIntStride(), Register_Int32, ReadIndex, Value); }

	void Read_Bool(out bool Value){ NiagaraDataChannelRead_Bool(GetBuffer_Int32(), GetIntStride(), Register_Int32, ReadIndex, Value); }

	//TODO: Half Support | void Read_Half(out half Value){ NiagaraDataChannelRead_Half(GetBuffer_Half(), GetHalfStride(), Register_Half, ReadIndex, Value); }
	//TODO: Half Support | void Read_Half(out half2 Value){ NiagaraDataChannelRead_Half(GetBuffer_Half(), GetHalfStride(), Register_Half, ReadIndex, Value); }
	//TODO: Half Support | void Read_Half(out half3 Value){ NiagaraDataChannelRead_Half(GetBuffer_Half(), GetHalfStride(), Register_Half, ReadIndex, Value); }
	//TODO: Half Support | void Read_Half(out half4 Value){ NiagaraDataChannelRead_Half(GetBuffer_Half(), GetHalfStride(), Register_Half, ReadIndex, Value); }
};

void Num_{ParameterName}(out int OutNum)
{
	if({ParameterName}_InstanceCountOffset != -1)
	{
		OutNum = RWInstanceCounts[{ParameterName}_InstanceCountOffset];
	}
	else
	{
		OutNum = 0;
	}
}

void GetNDCSpawnData_{ParameterName}(NiagaraEmitterID InEmitterID, int InExecIndex, out int OutNDCIndex, out int OutNDCSpawnIndex, out int OutNDCSpawnCount)
{
	/*
	Look up the various bits of spawn data from the NDC that spawned this particle.
	The first section of the NDCSpawnDataBuffer contains a SpawnCount for every NDC Item from which we can calcualte NDCSpawnCount.
	The next section is a series of buckets containing all NDC indices that contain enough particles to spawn 1 << Bucket particles.
	ie. if an NDC spawns 10 particles it will have an entry in the 8 and 2 buckets and none in any other bucket.
	From this we can map the main ExecIndex back into an NDCIndex that generated it. However ofc this mapping is by bucket and so particles from one NDC may not be contiuous in the spawn script execution/particle data.
	From the per NDCSpawnCount we can also determine if an NDC has an entry in each bucket and so calculate the SpawnIndex of each particle within the NDC Spawn.

	This is a little convoluted but means we can access the spawning NDC Index when spawning in a single large SpawnInfo/Execution.
	This power of 2 bucket system also means we can keep the indirection table size reasonable for large and small particle counts in exchange for some ALU work.

	|| NDCSpawnDataBuffer |
	|| Bucket Sizes	                                           || Spawn Counts for each NDC         || 2^15 Bucket   || 2^14 Bucket || ... || 2^1 Bucket || 2^0 Bucket   ||
	|| Bucket 0 Size | Bucket 1 Size | ... | Bucket 15 Size    || NDC 2 | ... | NDC N               || 0 | 0 | 5 | 7 || 0 |.........|| ... || 0 | 1 | 5  ||              || 
	
	Above shows the layout for NDCSpawnDataBuffer.
	All Emitters are packed into the same buffer. The first part of the buffer is a lookup table for each emitter (by emitter ID/Index) into the rest of the data.

	As we proceed through each ExecIndex in the spawn execution, we iterate through each bucket and can know for example, we can allocate the first 2^15 particles to NDC 0 as that has an entry in the 2^15 bucket.
	The 2^15 bucket is special in that it can have multiple entries for the same NDC as it's the final bucket. All other buckets have only one as higher particle counts just move the NDC into other buckets. 
	Less fills up lower buckets, more with fill up this and lower buckets. 2x will push it into the bucket above. Except in the final bucket if we hit 2x or more 2^15 then we just have to add another entry to that bucket.
	So in the above example the first and second 2^15 instances of the execution are given to NDC 0.
	The next 2^15 are given to 5, then 7. Now we've processed the 2^15 bucket, further instances take from the next bucket and so on until we're at the 2^0 bucket and we've handled all instances.
	*/
	
	OutNDCIndex = -1;
	OutNDCSpawnIndex = -1;
	OutNDCSpawnCount = -1;

	if(InExecIndex < 0)
	{
		return;
	}

	uint ExecIdx = (uint)InExecIndex;

	int NumNDCEntries = {ParameterName}_NDCElementCountAtSpawn;

	//First part of the buffer is the per emitter offsets into the data.
	//Grab the right base offset for this emitter.
	uint NDCSpawnDataOffset = { ParameterName }_NDCSpawnDataBuffer[InEmitterID.ID];
	if(NDCSpawnDataOffset == 0 || NumNDCEntries <= 0)
	{
		OutNDCIndex = -1;
		OutNDCSpawnIndex = -1;
		OutNDCSpawnCount = 0;

		return;
	}

	uint NDCSpawnData_BucketSizeStart = NDCSpawnDataOffset;
	uint NDCSpawnData_NDCCountsStart = NDCSpawnData_BucketSizeStart + 16;
	uint NDCSPawnData_NDCIndexBucketsStart = NDCSpawnData_NDCCountsStart + NumNDCEntries;
	
	uint MaxBucketExecIndex = 0;
	uint BucketEntryStart = NDCSPawnData_NDCIndexBucketsStart;

	UNROLL
	for(int BucketIdx = 0; BucketIdx < 16; ++BucketIdx)
	{
		uint BucketSize = (1U << 15U) >> BucketIdx;
		uint NumEntriesInBucket = {ParameterName}_NDCSpawnDataBuffer[NDCSpawnData_BucketSizeStart + BucketIdx];
		uint MinBucketExecIndex = MaxBucketExecIndex; 
		MaxBucketExecIndex += BucketSize * NumEntriesInBucket;
		
		if(ExecIdx < MaxBucketExecIndex)
		{
			//We found our bucket.
			//Find our NDC entry. There is one entry for each bucket load of instances. So we divide our current adjusted exec index in this bucket by the bucket size.
			//As buckets are power of 2 we can do this faster by just shifting down.
			uint NDCIndexEntry = (ExecIdx - MinBucketExecIndex) >> (15U - BucketIdx);			
			OutNDCIndex = {ParameterName}_NDCSpawnDataBuffer[BucketEntryStart + NDCIndexEntry];
			break;
		}
		
		BucketEntryStart += NumEntriesInBucket;
	}
	
	if(OutNDCIndex >= 0 && OutNDCIndex < (int)NumNDCEntries)
	{
		//Pull the NDC Spawn count from the first part of the buffer.
		OutNDCSpawnCount = {ParameterName}_NDCSpawnDataBuffer[NDCSpawnData_NDCCountsStart + OutNDCIndex];

		//Do another pass to calculate our SpawnIndex for this NDC within the total count for this NDC.
		OutNDCSpawnIndex = 0;
		MaxBucketExecIndex = 0;
		uint Count = OutNDCSpawnCount;
		UNROLL
		for(int BucketIdx = 0; BucketIdx < 16; ++BucketIdx)
		{
			uint BucketSize = (1U << 15U) >> BucketIdx;
			uint Mask = (0xFFFFU >> (BucketIdx + 1U));
			uint CountMasked = Count & ~Mask;
			Count &= Mask;
			uint NumNDCEntriesInBucket = CountMasked >> (15U - BucketIdx);
			uint NumEntriesInBucket = {ParameterName}_NDCSpawnDataBuffer[NDCSpawnData_BucketSizeStart + BucketIdx];
			uint NumNDCInstancesInBucket = NumNDCEntriesInBucket * BucketSize;

			uint MinBucketExecIndex = MaxBucketExecIndex;
			MaxBucketExecIndex += BucketSize * NumEntriesInBucket;
			if (ExecIdx < MaxBucketExecIndex && NumNDCInstancesInBucket > 0)
			{
				uint NDCIndexEntry = (ExecIdx - MinBucketExecIndex) >> (15U - BucketIdx);
				uint MinNDCBucketExecIndex = MinBucketExecIndex + (BucketSize * NDCIndexEntry);

				OutNDCSpawnIndex += (ExecIdx - MinNDCBucketExecIndex);
				break;
			}
			else
			{
				OutNDCSpawnIndex += NumNDCInstancesInBucket;
			}
		}
	}
	else
	{
		OutNDCIndex = -1;
		OutNDCSpawnIndex = -1;
		OutNDCSpawnCount = -1;
	}
}
