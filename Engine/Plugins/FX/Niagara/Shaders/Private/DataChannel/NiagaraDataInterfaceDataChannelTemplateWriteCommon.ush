// Copyright Epic Games, Inc. All Rights Reserved.

/**
Template hlsl for Data Channel Write DI functions to apply to their own that is shared accross all functions for the DI.
*/

//Data bound for other readers on the GPU is written into one large shared buffer
//We index into this using atomic ops on an instance count in RWCountsBuffer.
RWBuffer<float> {ParameterName}_GPUBufferFloat;
RWBuffer<int> {ParameterName}_GPUBufferInt32;
//TODO: Half Support | RWBuffer<half> {ParameterName}_GPUBufferHalf;

//Buffer and offset for our instance count value.
//We'll atomically update this accordingly if we're write/appending.
int {ParameterName}_GPUInstanceCountOffset;
int {ParameterName}_GPUBufferSize;

/////////////////////////////////////////

//Data bounds back to the CPU for game and CPU sims is written into a separate specific buffer indexed directly by the thread index.
RWBuffer<float> {ParameterName}_CPUBufferFloat;
RWBuffer<int> {ParameterName}_CPUBufferInt32;
//TODO: Half Support | RWBuffer<half> {ParameterName}_CPUBufferHalf;

//Buffer and offset for our instance count value.
//We'll atomically update this accordingly if we're write/appending.
int {ParameterName}_CPUInstanceCountOffset;
int {ParameterName}_CPUBufferSize;
int {ParameterName}_CPUFloatStride;
int {ParameterName}_CPUInt32Stride;
//TODO: Half Support | int {ParameterName}_CPUHalfStride;

//Helper struct with various info needed for writing that can be passed around more easily in generated code.
struct FNDCAccessContext_{ParameterName}
{
	bool bValidWriteIndex;
	bool bValidParameter;

	//Indices we'll be writing to each buffer.
	int WriteIndex;

	int WriteIndexCPU;
	int WriteIndexGPU;
		
	//Component start offests for the current parameter.
	int ParameterComponentStart_Float;
	int ParameterComponentStart_Int32;
	//TODO: Half Support | int ParameterComponentStart_Half;

	//Current register indices for each component buffer
	int CurrRegister_Float;
	int CurrRegister_Int32;
	//TODO: Half Support | int CurrRegister_Half;

	bool bReadGPUOrCPU;

	int GetFloatStride() { return bReadGPUOrCPU ? {ParameterName}_FloatStride : {ParameterName}_CPUFloatStride; }
	int GetInt32Stride() { return bReadGPUOrCPU ? {ParameterName}_Int32Stride : {ParameterName}_CPUInt32Stride; }
	//TODO: Half Support | int GetHalfStride() { return bReadGPUOrCPU ? {ParameterName}_HalfStride : {ParameterName}_CPUHalfStride; }
	
	RWBuffer<float> GetGPUBuffer_Float() { return {ParameterName}_GPUBufferFloat; }
	RWBuffer<int> GetGPUBuffer_Int32() { return {ParameterName}_GPUBufferInt32; }
	//TODO: Half Support | RWBuffer<half> GetGPUBuffer_Half() { return {ParameterName}_GPUBufferHalf; }
	int GetGPUInstanceCountOffset() { return {ParameterName}_GPUInstanceCountOffset; }
	int GetGPUBufferSize() { return {ParameterName}_GPUBufferSize; }

	RWBuffer<float> GetCPUBuffer_Float() { return {ParameterName}_CPUBufferFloat; }
	RWBuffer<int> GetCPUBuffer_Int32() { return {ParameterName}_CPUBufferInt32; }
	//TODO: Half Support | RWBuffer<half> GetCPUBuffer_Half() { return {ParameterName}_CPUBufferHalf; }
	int GetCPUInstanceCountOffset() { return {ParameterName}_CPUInstanceCountOffset; }
	int GetCPUBufferSize() { return {ParameterName}_CPUBufferSize; }

	RWBuffer<float> GetCurrBuffer_Float() { return bReadGPUOrCPU ? GetGPUBuffer_Float() : GetCPUBuffer_Float(); }
	RWBuffer<int> GetCurrBuffer_Int32() { return bReadGPUOrCPU ? GetGPUBuffer_Int32() : GetCPUBuffer_Int32(); }
	//TODO: Half Support | RWBuffer<half> GetCurrBuffer_Half() { return bReadGPUOrCPU ? GetGPUBuffer_Half() : GetCPUBuffer_Half(); }
	
	bool ShouldWriteGPUData() { return bValidParameter && bValidWriteIndex && GetGPUInstanceCountOffset() != -1; }
	bool ShouldWriteCPUData() { return bValidParameter && bValidWriteIndex && GetCPUInstanceCountOffset() != -1; }

	int GetAppendIndex(int InstanceCountOffset, int BufferSize)
	{
		//TODO: Add GS and Wave op path to reduce atomic contention.
		//TODO: Right now doing the simple dumb thing.

		int OutIndex = -1;
		if(InstanceCountOffset != -1)
		{
			//Note: This allows the index to temporarily exceed the max limits so is unsafe if we access this concurrently anywhere else without checking the limits.
			//Note: However it does avoid a more expensive looping compare exchange.
			uint OrigVal = 0;
			InterlockedAdd(RWInstanceCounts[InstanceCountOffset], 1U, OrigVal);
			if(OrigVal >= 0 && OrigVal < (uint)BufferSize)
			{ 
				OutIndex = (int)OrigVal;
			}
			else
			{
				OutIndex = -1;
				InterlockedMin(RWInstanceCounts[InstanceCountOffset], (uint)BufferSize);
			}
		}

		return OutIndex;
	}
	
	//Initialize our append indices. Called at the start of the append funciton.
	bool InitAppend()
	{
		WriteIndexGPU = GetAppendIndex(GetGPUInstanceCountOffset(), GetGPUBufferSize());
		WriteIndexCPU = GetAppendIndex(GetCPUInstanceCountOffset(), GetCPUBufferSize());
	
		bValidWriteIndex = (WriteIndexGPU >= 0 && WriteIndexGPU < GetGPUBufferSize()) ||
				(WriteIndexCPU  >= 0 && WriteIndexCPU < GetCPUBufferSize());
	
		return bValidWriteIndex;
	}

	//Initialize for the given parameter index.
	bool InitForParameter(int ParameterIndex)
	{
		bValidParameter = GetParameterLayoutInfo_{ParameterName}(ParameterIndex, ParameterComponentStart_Float, ParameterComponentStart_Int32);//TODO: Half Support | , ParameterComponentStart_Half);
		return bValidParameter;
	}
	
	//Initialize for wrting the current parameter to the GPU buffers
	bool InitForGPUWrite()
	{
		bReadGPUOrCPU = true;
		WriteIndex = WriteIndexGPU;	
		CurrRegister_Float = ParameterComponentStart_Float;
		CurrRegister_Int32 = ParameterComponentStart_Int32;
		//TODO: Half Support | CurrRegister_Half = ParameterComponentStart_Half;
		
		return ShouldWriteGPUData();
	}

	//Initialize for wrting the current parameter to the "CPU" buffers
	bool InitForCPUWrite()
	{
		bReadGPUOrCPU = false;
		WriteIndex = WriteIndexCPU;	
		CurrRegister_Float = ParameterComponentStart_Float;
		CurrRegister_Int32 = ParameterComponentStart_Int32;
		//TODO: Half Support | CurrRegister_Half = ParameterComponentStart_Half;
		
		return ShouldWriteCPUData();
	}

	void Write_Float(float Value){ NiagaraDataChannelWrite_Float(GetCurrBuffer_Float(), GetFloatStride(), CurrRegister_Float, WriteIndex, Value);	}
	void Write_Float(float2 Value){	NiagaraDataChannelWrite_Float(GetCurrBuffer_Float(), GetFloatStride(), CurrRegister_Float, WriteIndex, Value);	}
	void Write_Float(float3 Value){	NiagaraDataChannelWrite_Float(GetCurrBuffer_Float(), GetFloatStride(), CurrRegister_Float, WriteIndex, Value);	}
	void Write_Float(float4 Value){	NiagaraDataChannelWrite_Float(GetCurrBuffer_Float(), GetFloatStride(), CurrRegister_Float, WriteIndex, Value);	}

	void Write_Int32(int Value){ NiagaraDataChannelWrite_Int32(GetCurrBuffer_Int32(), GetInt32Stride(), CurrRegister_Int32, WriteIndex, Value);	}
	void Write_Int32(int2 Value){ NiagaraDataChannelWrite_Int32(GetCurrBuffer_Int32(), GetInt32Stride(), CurrRegister_Int32, WriteIndex, Value);	}
	void Write_Int32(int3 Value){ NiagaraDataChannelWrite_Int32(GetCurrBuffer_Int32(), GetInt32Stride(), CurrRegister_Int32, WriteIndex, Value);	}
	void Write_Int32(int4 Value){ NiagaraDataChannelWrite_Int32(GetCurrBuffer_Int32(), GetInt32Stride(), CurrRegister_Int32, WriteIndex, Value);	}

	void Write_Bool(bool Value){ NiagaraDataChannelWrite_Bool(GetCurrBuffer_Int32(), GetInt32Stride(), CurrRegister_Int32, WriteIndex, Value);	}

	//TODO: Half Support | void Write_Half(half Value){ NiagaraDataChannelWrite_Half(GetCurrBuffer_Half(), GetHalfStride(), CurrRegister_Half, WriteIndex, Value); }
	//TODO: Half Support | void Write_Half(half2 Value){ NiagaraDataChannelWrite_Half(GetCurrBuffer_Half(), GetHalfStride(), CurrRegister_Half, WriteIndex, Value); }
	//TODO: Half Support | void Write_Half(half3 Value){ NiagaraDataChannelWrite_Half(GetCurrBuffer_Half(), GetHalfStride(), CurrRegister_Half, WriteIndex, Value); }
	//TODO: Half Support | void Write_Half(half4 Value){ NiagaraDataChannelWrite_Half(GetCurrBuffer_Half(), GetHalfStride(), CurrRegister_Half, WriteIndex, Value); }
};