// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Template shader code for the Append function for the NiagaraDataInterfaceDataChannel.
 * Contained in it's own shader file to allow reuse between different invocations with different params.
 */

{PerParameterFunctionDefinitions}

void {FunctionSymbol}(bool bEmit, {FunctionInputParameters}, out bool bOutSuccess)
{
	bOutSuccess = false;
	
	if(bEmit)
	{
		FNDCAccessContext_{ParameterName} Context;
		if(Context.InitAppend())
		{
			bOutSuccess = true;
	
			//This is generated in the DI Cpp depending on the function parameters.
{PerFunctionParameterShaderCode}
		}
	}
}