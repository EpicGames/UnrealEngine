// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Template shader code for the Consume function for the NiagaraDataInterfaceDataChannel.
 * Contained in it's own shader file to allow reuse between different invocations with different params.
 */

{PerParameterFunctionDefinitions}

void {FunctionSymbol}(bool bInConsume, out bool bOutSuccess, out int OutIndex, {FunctionOutputParameters})
{
	//Default outputs in case of failure.
	{DefaultOutputsShaderCode}

	bOutSuccess = false;
	OutIndex = -1;
	
	if(bInConsume)
	{
		FNDCAccessContext_{ParameterName} Context;
		if(Context.InitConsume())
		{
			bOutSuccess = true;
			OutIndex = Context.ReadIndex;

			//This is generated in the DI Cpp depending on the function parameters.
			{PerFunctionParameterShaderCode}
		}
	}
}

