// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Template shader code for the Write function for the NiagaraDataInterfaceDataChannel.
 * Contained in it's own shader file to allow reuse between different invocations with different params.
 */

{PerParameterFunctionDefinitions}

void {FunctionSymbol}(bool bEmit, int ElementIndex, {FunctionInputParameters}, out bool bOutSuccess)
{
	//Write is deprecated on the GPU.
	//All writes now done with append as a/ it's less useful, b/ it's brittle as multiple things could change internally, c/ Already can't work on GPU as we're wll writing concurrently to a single shared buffer.
	//TODO: Remove
}