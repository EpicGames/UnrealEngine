// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Plugin/FX/Niagara/Private/NiagaraBaryCentricUtils.ush"

struct UvMappingParameters
{
	int				NumUVs;
	Buffer<uint>	IndexBuffer;
	Buffer<float2>	UVBuffer;
	Buffer<int>		UvMappingBuffer;
	uint			UvMappingBufferLength;
	uint			UvMappingSet;
};

#define UvMapping_ElementCountOffset 4
#define UvMapping_TriangleIndexOffset 5
#define UvMapping_MinValidUvMappingBufferSize 24 // minimum size (in bytes) for a quadtree to have valid data

bool UvMapping_NormalizedAabbTriangleIntersection(float2 A, float2 B, float2 C)
{
	float2 TriAabbMin = float2(min3(A.x, B.x, C.x), min3(A.y, B.y, C.y));
	float2 TriAabbMax = float2(max3(A.x, B.x, C.x), max3(A.y, B.y, C.y));

	if (any(TriAabbMin > 1.0f) || any(TriAabbMax < 0.0f))
	{
		return false;
	}

	#define AXIS_COUNT 3
	float2 TriangleEdges[AXIS_COUNT] = { C - B, A - C, B - A };

	for (int i = 0; i < AXIS_COUNT; ++i)
	{
		float2 Axis = TriangleEdges[i].yx * (float2(-1.0f, 0.0f));
		float AabbSegmentMin = min(0.0f, min3(Axis.x, Axis.y, Axis.x + Axis.y));
		float AabbSegmentMax = max(0.0f, max3(Axis.x, Axis.y, Axis.x + Axis.y));
		float TriangleSegmentMin = min3(dot(A, Axis), dot(B, Axis), dot(C, Axis));
		float TriangleSegmentMax = max3(dot(A, Axis), dot(B, Axis), dot(C, Axis));
		if (AabbSegmentMin > TriangleSegmentMax || AabbSegmentMax < TriangleSegmentMin)
		{
			return false;
		}
	}

	return true;
}

// Evaluates QuadTree for a single point
bool UvMapping_QuadTreeTriangleOverlap(
	UvMappingParameters MappingParam,
	float2 Uv,
	float Tolerance,
	int TriangleIndex,
	out float3 OutBaryCoord)
{
	if (MappingParam.NumUVs > 0)
	{
        uint IndexBufferOffset = TriangleIndex * 3;
		uint VertexIndex0 = MappingParam.IndexBuffer[IndexBufferOffset  ];
		uint VertexIndex1 = MappingParam.IndexBuffer[IndexBufferOffset+1];
		uint VertexIndex2 = MappingParam.IndexBuffer[IndexBufferOffset+2];

		uint Stride = MappingParam.NumUVs;
		float2 UV0 = MappingParam.UVBuffer[VertexIndex0 * Stride + MappingParam.UvMappingSet];
		float2 UV1 = MappingParam.UVBuffer[VertexIndex1 * Stride + MappingParam.UvMappingSet];
		float2 UV2 = MappingParam.UVBuffer[VertexIndex2 * Stride + MappingParam.UvMappingSet];

		OutBaryCoord = GetBaryCentric2D(Uv, UV0, UV1, UV2);

		if (all(OutBaryCoord > -Tolerance) && all(OutBaryCoord < (1.0f + Tolerance)))
		{
			return true;
		}
	}

	return false;
}

bool UvMapping_QuadTreeNodeContains(
	UvMappingParameters MappingParam,
	int NodeBufferOffset,
	float2 Uv,
	float Tolerance,
	out int OutTriangleIndex,
	out float3 OutBaryCoord)
{
	OutTriangleIndex = -1;

	if (NodeBufferOffset == -1)
	{
		return false;
	}

	const int TriangleCount = MappingParam.UvMappingBuffer[NodeBufferOffset + UvMapping_ElementCountOffset];
	for (int TriangleIt = 0; TriangleIt < TriangleCount; ++TriangleIt)
	{
		int TriangleIndex = MappingParam.UvMappingBuffer[NodeBufferOffset + UvMapping_TriangleIndexOffset + TriangleIt];
		if (UvMapping_QuadTreeTriangleOverlap(MappingParam, Uv, Tolerance, TriangleIndex, OutBaryCoord))
		{
			OutTriangleIndex = TriangleIndex;
			return true;
		}
	}

	return false;
}

void UvMapping_GetTriangleCoordAtUV(
	UvMappingParameters MappingParam,
	bool Enabled,
	float2 InUv,
	float InTolerance,
	out int OutTriangle,
	out float3 OutBaryCoord,
	out bool OutIsValid)
{
	OutTriangle = -1;
	OutBaryCoord = float3(0.0f, 0.0f, 0.0f);
	OutIsValid = false;

	if (!Enabled || MappingParam.UvMappingBufferLength < UvMapping_MinValidUvMappingBufferSize)
	{
		return;
	}

	int NodeBufferOffset = 0;

	float2 MidPoint = float2(0.5f, 0.5f);
	float2 Extent = float2(0.5f, 0.5f);

	while (NodeBufferOffset != -1)
	{
		if (UvMapping_QuadTreeNodeContains(MappingParam, NodeBufferOffset, InUv, InTolerance, OutTriangle, OutBaryCoord))
		{
			OutIsValid = true;
			return;
		}

		Extent *= 0.5f;

		const bool LessX = InUv.x < MidPoint.x;
		const bool LessY = InUv.y < MidPoint.y;

		MidPoint += Extent * float2(LessX ? -1.0f : 1.0f, LessY ? -1.0f : 1.0f);
		int ChildIndex = LessX ? (LessY ? 2 : 0) : (LessY ? 3 : 1);

		NodeBufferOffset = MappingParam.UvMappingBuffer[NodeBufferOffset + ChildIndex];
	}
}

// Evaluates QuadTree for an AABB region
bool UvMapping_QuadTreeTriangleAabbOverlap(
	UvMappingParameters MappingParam,
	float2 UvRef,
	float2 UvExtentScale,
	float2 UvExtentBias,
	int TriangleIndex,
	out float3 OutBaryCoord)
{
	if (MappingParam.NumUVs > 0)
	{
        uint IndexBufferOffset = TriangleIndex * 3;
		uint VertexIndex0 = MappingParam.IndexBuffer[IndexBufferOffset  ];
		uint VertexIndex1 = MappingParam.IndexBuffer[IndexBufferOffset+1];
		uint VertexIndex2 = MappingParam.IndexBuffer[IndexBufferOffset+2];

		uint Stride = MappingParam.NumUVs;
		float2 UV0 = MappingParam.UVBuffer[VertexIndex0 * Stride + MappingParam.UvMappingSet];
		float2 UV1 = MappingParam.UVBuffer[VertexIndex1 * Stride + MappingParam.UvMappingSet];
		float2 UV2 = MappingParam.UVBuffer[VertexIndex2 * Stride + MappingParam.UvMappingSet];

		// evaluate whether our triangle intersects with our AABB, if it does, then generate the barycentric coordinate for the center of the AABB
		const bool Intersection = UvMapping_NormalizedAabbTriangleIntersection(
				UV0 * UvExtentScale + UvExtentBias,
				UV1 * UvExtentScale + UvExtentBias,
				UV2 * UvExtentScale + UvExtentBias);

		if (Intersection)
		{
			OutBaryCoord = GetClosestBaryCentric2D(UvRef, UV0, UV1, UV2);
			return true;
		}
	}

	return false;
}

bool UvMapping_QuadTreeNodeContainsAabb(
	UvMappingParameters MappingParam,
	int NodeBufferOffset,
	float2 UvMin,
	float2 UvMax,
	out int OutTriangleIndex,
	out float3 OutBaryCoord)
{
	OutTriangleIndex = -1;

	if (NodeBufferOffset == -1)
	{
		return false;
	}

	const int TriangleCount = MappingParam.UvMappingBuffer[NodeBufferOffset + UvMapping_ElementCountOffset];
	if (TriangleCount == 0)
	{
		return false;
	}

	const float2 UvRef = 0.5f * (UvMin + UvMax);
	const float2 UvExtentScale = 1.0f / (UvMax - UvMin);
	const float2 UvExtentBias = 1.0f - UvMax * UvExtentScale;

	for (int TriangleIt = 0; TriangleIt < TriangleCount; ++TriangleIt)
	{
		float3 BaryCoord;
		int TriangleIndex = MappingParam.UvMappingBuffer[NodeBufferOffset + UvMapping_TriangleIndexOffset + TriangleIt];
		if (UvMapping_QuadTreeTriangleAabbOverlap(MappingParam, UvRef, UvExtentScale, UvExtentBias, TriangleIndex, BaryCoord))
		{
			OutTriangleIndex = TriangleIndex;
			OutBaryCoord = BaryCoord;
			return true;
		}
	}

	return false;
}

void UvMapping_GetTriangleCoordInAabb(
	UvMappingParameters MappingParam,
	bool Enabled,
	float2 InUvMin,
	float2 InUvMax,
	out int OutTriangle,
	out float3 OutBaryCoord,
	out bool OutIsValid)
{
	OutTriangle = -1;
	OutBaryCoord = float3(0.0f, 0.0f, 0.0f);
	OutIsValid = false;

	if (!Enabled || MappingParam.UvMappingBufferLength < UvMapping_MinValidUvMappingBufferSize)
	{
		return;
	}

	int NodeBufferOffset = 0;

	const float2 UvRegionCenter = 0.5f * (InUvMin + InUvMax);
	float2 MidPoint = float2(0.5f, 0.5f);
	float2 Extent = float2(0.5f, 0.5f);

	while (NodeBufferOffset != -1)
	{
		if (UvMapping_QuadTreeNodeContainsAabb(MappingParam, NodeBufferOffset, InUvMin, InUvMax, OutTriangle, OutBaryCoord))
		{
			OutIsValid = true;
			return;
		}

		Extent *= 0.5f;

		const bool LessX = UvRegionCenter.x < MidPoint.x;
		const bool LessY = UvRegionCenter.y < MidPoint.y;

		MidPoint += Extent * float2(LessX ? -1.0f : 1.0f, LessY ? -1.0f : 1.0f);
		int ChildIndex = LessX ? (LessY ? 2 : 0) : (LessY ? 3 : 1);

		NodeBufferOffset = MappingParam.UvMappingBuffer[NodeBufferOffset + ChildIndex];
	}
}