// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/Common.ush"

struct FNiagaraAsyncGpuTrace
{
	float3 Origin;
	float TFar;
	float3 Direction;
	uint CollisionGroup;
};

struct FNiagaraAsyncGpuTraceResult
{
	float3 WorldPosition;
	float HitT;

	float3 WorldNormal;

	float _Pad0; // padding to force 16 byte alignment to meet requirements for VK
};

bool IsHit(FNiagaraAsyncGpuTraceResult Result)
{
	return Result.HitT > 0;
}