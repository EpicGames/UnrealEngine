// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/Common.ush"
#include "NiagaraAsyncGpuTraceCommon.ush"

#if !defined(THREADGROUP_SIZE_X)
	#define THREADGROUP_SIZE_X 32
#endif

RWStructuredBuffer<FNiagaraAsyncGpuTraceResult> Results;

uint TraceCount;
uint ResultsOffset;

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void NiagaraClearAsyncGpuTraceCS(uint GroupIndex : SV_GroupIndex)
{
	if (GroupIndex >= TraceCount)
	{
		return;
	}

	FNiagaraAsyncGpuTraceResult Result = (FNiagaraAsyncGpuTraceResult) 0;
	Result.HitT = -1.0f;

	Results[ResultsOffset + GroupIndex] = Result;
}