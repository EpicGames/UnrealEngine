// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// from FMath::GetBaryCentric2D()
float3 GetBaryCentric2D(float2 P, float2 A, float2 B, float2 C)
{
	const float s = ((B.y-C.y)*(P.x-C.x) + (C.x-B.x)*(P.y-C.y)) / ((B.y-C.y)*(A.x-C.x) + (C.x-B.x)*(A.y-C.y));
	const float t = ((C.y-A.y)*(P.x-C.x) + (A.x-C.x)*(P.y-C.y)) / ((B.y-C.y)*(A.x-C.x) + (C.x-B.x)*(A.y-C.y));

	return float3(s, t, 1.0f - s - t);
}

float3 GetClosestBaryCentric2D(float2 P, float2 A, float2 B, float2 C)
{
	float3 Coord = GetBaryCentric2D(P, A, B, C);

	// check if the point is outside of the triangle, if it is then we need to adjust the barycentric coordinate
	if (any(Coord != saturate(Coord)))
	{
		if (Coord.x >= 0.0f && Coord.y < 0.0f)
		{
			// AB or AC
			float2 AP = P - A;
			float2 AB = B - A;

			if (Coord.z < 0.0f && dot(AP, AB) > 0.0f)
			{
				// AB
				Coord.y = saturate(dot(AP, AB) / dot(AB, AB));
				Coord.z = 0.0f;
				Coord.x = 1.0f - Coord.y;
			}
			else
			{
				float2 AC = C - A;

				// AC
				Coord.y = 0.0f;
				Coord.z = saturate(dot(AP, AC) / dot(AC, AC));
				Coord.x = 1.0f - Coord.z;
			}
		}
		else if (Coord.y >= 0.0f && Coord.z < 0.0f)
		{
			// BC or BA
			float2 BP = P - B;
			float2 BC = C - B;

			if (Coord.x < 0.0f && dot(BP, BC) > 0.0f)
			{
				// BC
				Coord.x = 0.0f;
				Coord.z = saturate(dot(BP, BC) / dot(BC, BC));
				Coord.y = 1.0f - Coord.z;
			}
			else
			{
				float2 BA = A - B;

				// BA
				Coord.x = saturate(dot(BP, BA) / dot(BA, BA));
				Coord.y = 1.0f - Coord.x;
				Coord.z = 0.0f;
			}
		}
		else if (Coord.z >= 0.0f && Coord.x < 0.0f)
		{
			// CA or CB
			float2 CP = P - C;
			float2 CA = A - C;

			if (Coord.y < 0.0f && dot(CP, CA) > 0.0f)
			{
				// CA
				Coord.x = saturate(dot(CP, CA)/ dot(CA, CA));
				Coord.y = 0.0f;
				Coord.z = 1.0f - Coord.x;
			}
			else
			{
				float2 CB = B - C;

				// CB
				Coord.x = 0.0f;
				Coord.y = saturate(dot(CP, CB)/dot(CB, CB));
				Coord.z = 1.0f - Coord.y;
			}
		}
	}

	return Coord;
}