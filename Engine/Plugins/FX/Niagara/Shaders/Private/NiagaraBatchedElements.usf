// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
//#include "ColorUtils.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"
#include "NiagaraCommon.ush"
//#include "NiagaraDebugDraw.ush"

////////////////////////////////////////////////////////////////////////////////////
#if NIAGARA_2DARRAY_ATTRIBUTE_PS

#define INDEX_NONE -1

Texture2DArray	InTexture;
SamplerState	InTextureSampler;
int4			InAttributeSlices;
float4x4		InColorWeights;
float			InGamma;

void MainPS(
	in float2 InTextureCoordinate : TEXCOORD0,
	in float4 InColor : TEXCOORD1,
	in float4 InHitProxyId : TEXCOORD2,
	in float4 InPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
#if WRITE_TO_GBUFFER
	,out float4 OutWorldNormal : SV_Target1 
#endif
	)
{
	float4 Sample = 0;
	for ( int i=0; i < 4; ++i )
	{
		//Sample[i] = InAttributeSlices[i] == INDEX_NONE ? 0.0f : InTexture.Load(int3(Input.Position.x, Input.Position.y, InAttributeSlices[i])).x;
		Sample[i] = InAttributeSlices[i] == INDEX_NONE ? 0.0f : Texture2DArraySample(InTexture, InTextureSampler, float3(InTextureCoordinate.x, InTextureCoordinate.y, InAttributeSlices[i])).x;
	}

	float4 Color;
	Color.r = dot(Sample, InColorWeights[0]);
	Color.g = dot(Sample, InColorWeights[1]);
	Color.b = dot(Sample, InColorWeights[2]);
	Color.a = dot(Sample, InColorWeights[3]);
	Color *= InColor;

	if( InGamma != 1.0 )
	{
		Color.rgb = ApplyGammaCorrection(saturate(Color.rgb), 2.2 / (1.0 / InGamma));
	}

	OutColor = RETURN_COLOR(Color);

#if WRITE_TO_GBUFFER
	// Set the G buffer bits that indicate that deferred lighting and image reflections are not enabled
	OutWorldNormal = 0;
#endif
}

#endif //NIAGARA_2DARRAY_ATTRIBUTE_PS

////////////////////////////////////////////////////////////////////////////////////

#if NIAGARA_VOLUME_ATTRIBUTE_PS

Texture3D		InTexture;
SamplerState	InTextureSampler;
int				InNumAttributes;
float4			InAttributeUVAndChannel[4];
float2			InTileUV;
float4x4		InColorWeights;
float			InGamma;

void MainPS(
	in float2 InTextureCoordinate : TEXCOORD0,
	in float4 InColor : TEXCOORD1,
	in float4 InHitProxyId : TEXCOORD2,
	in float4 InPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
#if WRITE_TO_GBUFFER
	,out float4 OutWorldNormal : SV_Target1 
#endif
	)
{
	float4 Sample = 0;
	float3 TileUV = float3(InTextureCoordinate * InTileUV, 0);
	for ( int i=0; i < InNumAttributes; ++i )
	{
		int Channel = asint(InAttributeUVAndChannel[i].w);
		Sample[i] = Texture3DSample(InTexture, InTextureSampler, InAttributeUVAndChannel[i].xyz + TileUV)[Channel];
	}

	float4 Color;
	Color.r = dot(Sample, InColorWeights[0]);
	Color.g = dot(Sample, InColorWeights[1]);
	Color.b = dot(Sample, InColorWeights[2]);
	Color.a = dot(Sample, InColorWeights[3]);
	Color *= InColor;

	if( InGamma != 1.0 )
	{
		Color.rgb = ApplyGammaCorrection(saturate(Color.rgb), 2.2 / (1.0 / InGamma));
	}

	OutColor = RETURN_COLOR(Color);

#if WRITE_TO_GBUFFER
	// Set the G buffer bits that indicate that deferred lighting and image reflections are not enabled
	OutWorldNormal = 0;
#endif
}

#endif //NIAGARA_VOLUME_ATTRIBUTE_PS

////////////////////////////////////////////////////////////////////////////////////

#if NIAGARA_SIMPLE_PS

Texture2D		InTexture;
SamplerState	InTextureSampler;
float4x4		InColorWeights;
float			InGamma;

void MainPS(
	in float2 InTextureCoordinate : TEXCOORD0,
	in float4 InColor : TEXCOORD1,
	in float4 InHitProxyId : TEXCOORD2,
	in float4 InPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
#if WRITE_TO_GBUFFER
	,out float4 OutWorldNormal : SV_Target1 
#endif
	)
{
	float4 Sample = Texture2DSample(InTexture, InTextureSampler, InTextureCoordinate);
	float4 Color;
	Color.r = dot(Sample, InColorWeights[0]);
	Color.g = dot(Sample, InColorWeights[1]);
	Color.b = dot(Sample, InColorWeights[2]);
	Color.a = dot(Sample, InColorWeights[3]);
	Color *= InColor;

	if( InGamma != 1.0 )
	{
		Color.rgb = ApplyGammaCorrection(saturate(Color.rgb), 2.2 / (1.0 / InGamma));
	}

	OutColor = RETURN_COLOR(Color);

#if WRITE_TO_GBUFFER
	// Set the G buffer bits that indicate that deferred lighting and image reflections are not enabled
	OutWorldNormal = 0;
#endif
}

#endif //NIAGARA_SIMPLE_PS

////////////////////////////////////////////////////////////////////////////////////
