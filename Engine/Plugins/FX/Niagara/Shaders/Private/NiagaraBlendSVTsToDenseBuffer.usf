// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/SparseVolumeTexture/SparseVolumeTextureCommon.ush"
RWTexture3D<float4>	DestinationBuffer;

SamplerState		TileDataTextureSampler_A;
Texture3D<uint>		SparseVolumeTexturePageTable_A;
Texture3D			SparseVolumeTextureA_A;
uint4				PackedSVTUniforms0_A;
uint4				PackedSVTUniforms1_A;
int3				TextureSize_A;
int					MipLevels_A;

SamplerState		TileDataTextureSampler_B;
Texture3D<uint>		SparseVolumeTexturePageTable_B;
Texture3D			SparseVolumeTextureA_B;
uint4				PackedSVTUniforms0_B;
uint4				PackedSVTUniforms1_B;
int3				TextureSize_B;
int					MipLevels_B;

float				LerpAmount;

[numthreads(4, 4, 4)]
void PerformBlendCS(uint3 Index : SV_DispatchThreadID)
{
	const int MipLevel = 0; // We want to sample level 0 but the page table can actually point to lower resolution mip level voxels	
	const int3 VirtualCoord = Index >> MipLevel;	

	const FSparseVolumeTextureUniforms Uniforms_A = SparseVolumeTextureUnpackUniforms(PackedSVTUniforms0_A, PackedSVTUniforms1_A);		
	const int3 VoxelCoordA = SparseVolumeTextureLoadPageTable(SparseVolumeTexturePageTable_A, Uniforms_A, VirtualCoord, MipLevel);
	float4 ValueA = SparseVolumeTextureA_A.Load(int4(VoxelCoordA, 0));

	const FSparseVolumeTextureUniforms Uniforms_B = SparseVolumeTextureUnpackUniforms(PackedSVTUniforms0_B, PackedSVTUniforms1_B);		
	const int3 VoxelCoordB = SparseVolumeTextureLoadPageTable(SparseVolumeTexturePageTable_B, Uniforms_B, VirtualCoord, MipLevel);
	float4 ValueB = SparseVolumeTextureA_B.Load(int4(VoxelCoordB, 0));
	
	DestinationBuffer[Index] = lerp(ValueA, ValueB, LerpAmount);
}
