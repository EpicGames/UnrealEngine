// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

#if NIAGARA_CLEAR_COUNTS_INT_CS

RWBuffer<int>	CountBuffer;
Buffer<int>		CountsAndValuesBuffer;
uint			NumCounts;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void ClearCountsIntCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint Index = DispatchThreadId.x;
	if(Index < NumCounts)
	{
		int ClearIndex = CountsAndValuesBuffer[Index * 2];
		int ClearValue = CountsAndValuesBuffer[(Index * 2) + 1];
		CountBuffer[ClearIndex] = ClearValue;
	}
}

#endif

#if NIAGARA_CLEAR_COUNTS_UINT_CS

RWBuffer<uint>	CountBuffer;
Buffer<uint>	CountsAndValuesBuffer;
uint			NumCounts;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void ClearCountsUIntCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint Index = DispatchThreadId.x;
	if(Index < NumCounts)
	{
		uint ClearIndex = CountsAndValuesBuffer[Index * 2];
		uint ClearValue = CountsAndValuesBuffer[(Index * 2) + 1];
		CountBuffer[ClearIndex] = ClearValue;
	}
}

#endif
