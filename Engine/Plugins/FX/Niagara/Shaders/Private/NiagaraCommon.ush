// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

float3x3 NiagaraQuatTo3x3(float4 Q)
{
	float x2 = Q.x + Q.x;  float y2 = Q.y + Q.y;  float z2 = Q.z + Q.z;
	float xx = Q.x * x2;   float xy = Q.x * y2;   float xz = Q.x * z2;
	float yy = Q.y * y2;   float yz = Q.y * z2;   float zz = Q.z * z2;
	float wx = Q.w * x2;   float wy = Q.w * y2;   float wz = Q.w * z2;

	float3x3 Mat = {
		1.0f - (yy + zz),	xy + wz,			xz - wy,
		xy - wz,			1.0f - (xx + zz),	yz + wx,
		xz + wy,			yz - wx,			1.0f - (xx + yy),
	};

	return Mat;
}

float4x4 NiagaraComposeTransformMatrix(float3 S, float3x3 R, float3 T)
{
	float3x3 ScaleMat = float3x3(
		S.x, 0.0f, 0.0f,
		0.0f, S.y, 0.0f,
		0.0f, 0.0f, S.z);

	float3x3 ScaleRot = mul(ScaleMat, R);

	return float4x4(
		ScaleRot[0], 0.0f,
		ScaleRot[1], 0.0f,
		ScaleRot[2], 0.0f,
		T, 1.0f);
}

FLWCMatrix NiagaraComposeTransformMatrix(float3 S, float3x3 R, FLWCVector3 T)
{
	return MakeLWCMatrix(LWCGetTile(T), NiagaraComposeTransformMatrix(S, R, T.Offset));
}

float4x4 NiagaraComposeInvTransformMatrix(float3 S, float3x3 R, float3 T)
{
	float3x3 InvScaleMat = float3x3(
		1.0f / S.x, 0.0f, 0.0f,
		0.0f, 1.0f / S.y, 0.0f,
		0.0f, 0.0f, 1.0f / S.z);

	float3x3 InvRotScale = mul(transpose(R), InvScaleMat);

	float4x4 InvRotScale44 = float4x4(
		InvRotScale[0], 0.0f,
		InvRotScale[1], 0.0f,
		InvRotScale[2], 0.0f,
		0.0f, 0.0f, 0.0f, 1.0f
	);

	float4x4 InvTransMat = float4x4(
		1.0f, 0.0f, 0.0f, 0.0f,
		0.0f, 1.0f, 0.0f, 0.0f,
		0.0f, 0.0f, 1.0f, 0.0f,
		-T, 1.0f);

	return mul(InvTransMat, InvRotScale44);
}

FLWCInverseMatrix NiagaraComposeInvTransformMatrix(float3 S, float3x3 R, FLWCVector3 T)
{
	return MakeLWCInverseMatrix(LWCGetTile(T), NiagaraComposeInvTransformMatrix(S, R, T.Offset));
}

float3x3 NiagaraFindRotationBetweenNormals(float3 A, float3 B)
{
	float W = dot(A, B);

	float4 Quat;

	if (W >= 1e-6f)
	{
		Quat = float4(A.y * B.z - A.z * B.y,
					   A.z * B.x - A.x * B.z,
					   A.x * B.y - A.y * B.x,
					   W);
	}
	else
	{
		W = 0.f;
		Quat = abs(A.x) > abs(A.y)
				? float4(-A.z, 0.f, A.x, W)
				: float4(0.f, -A.z, A.y, W);
	}

	return NiagaraQuatTo3x3(Quat);
}

float4 NiagaraQuatMul(float4 Quat1, float4 Quat2)
{
	return float4(
		Quat2.xyz * Quat1.w + Quat1.xyz * Quat2.w + cross(Quat1.xyz, Quat2.xyz),
		Quat1.w * Quat2.w - dot(Quat1.xyz, Quat2.xyz)
	);
}

float3 NiagaraQuatRotateVector(float4 Quat, float3 Vector)
{
	float4 Rc = Quat * float4(-1, -1, -1, 1);
	return NiagaraQuatMul(Quat, NiagaraQuatMul(float4(Vector, 0), Rc)).xyz;
}

