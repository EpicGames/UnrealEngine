// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

// ID mapping table produced by the simulate step.
Buffer<int> IDToIndexTable;

// Output list of free IDs.
RWBuffer<int> RWFreeIDList;

// Buffer containing the current size of each list that's currently being processed.
// Multiple invocations of this shader might be running concurrently, for independent emitters.
RWBuffer<int> RWFreeIDListSizes;
// Index of the free list that's being computed by this invocation. The current size of the
// list is at RWFreeIDListSizes[FreeIDListIndex].
uint FreeIDListIndex;

#if USE_WAVE_INTRINSICS

// This happens to be fixed on PS4 and XB1, but we'll need to configure it externally
[numthreads(THREADGROUP_SIZE, 1, 1)]
void ComputeFreeIDs(uint3 GroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint InputStart = GroupId.x * WaveGetLaneCount();
	uint Thread = GroupThreadId.x;

	uint ParticleID = InputStart + Thread;
	uint IsFree = (IDToIndexTable[ParticleID] == -1) ? 1 : 0;
	
	// Count how many predicates are true across all lanes.
	uint NumGroupFreeIDs = WaveActiveCountBits(IsFree);

	// Compute a prefix sum on the predicate bit mask. This gives us the
	// local write offset for each lane.
	uint WriteOffset = WavePrefixCountBits(IsFree);

	// Skip the whole wave if all the IDs are in use.
	if(NumGroupFreeIDs > 0)
	{
		uint GroupWriteStart;
		if (WaveIsFirstLane())
		{
			// Add to the global write offset. The previous offset is where we start writing in the output buffer.
			// We can reduce the number of atomic adds by processing multiple sub-groups in the shader, such that each lane
			// computes the offsets and counts of a sub-group. However, this doesn't seem to be the bottleneck right now,
			// so it's not worth the effort.
			InterlockedAdd(RWFreeIDListSizes[FreeIDListIndex], NumGroupFreeIDs, GroupWriteStart);
		}

		// Broadcast the global write offset to all lanes.
		GroupWriteStart = WaveReadLaneFirst(GroupWriteStart);

		if(IsFree)
		{
			RWFreeIDList[GroupWriteStart + WriteOffset] = ParticleID;
		}
	}
}

#else

// Temporary buffer where we compute the number of elements to write before each input element.
groupshared uint GroupWriteOffsets[2*THREADGROUP_SIZE];
// The offset where this thread group will start writing its output.
groupshared uint GroupWriteStart;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void ComputeFreeIDs(uint3 GroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint InputStart = GroupId.x * THREADGROUP_SIZE;
	uint Thread = GroupThreadId.x;
	uint ParticleID = InputStart + Thread;

	int OutBuffer = 0, InBuffer = 1;

	// Evaluate predicate for each input slot, then shift down by one position because we need an exclusive sum.
	if(Thread > 0)
	{
		uint IsFree = (IDToIndexTable[ParticleID - 1] == -1) ? 1 : 0;
		GroupWriteOffsets[Thread] = IsFree;
	}
	else
	{
		GroupWriteOffsets[0] = 0;
	}
	GroupMemoryBarrierWithGroupSync();

	// Compute prefix sum.
	[unroll]
	for(uint Offset = 1; Offset < THREADGROUP_SIZE; Offset = Offset << 1)
	{
		// Swap the input and output buffers.
		OutBuffer = 1 - OutBuffer;
		InBuffer = 1 - InBuffer;
		if(Thread >= Offset)
		{
			GroupWriteOffsets[OutBuffer*THREADGROUP_SIZE + Thread] = GroupWriteOffsets[InBuffer*THREADGROUP_SIZE + Thread - Offset] + GroupWriteOffsets[InBuffer*THREADGROUP_SIZE + Thread];
		}
		else
		{
			GroupWriteOffsets[OutBuffer*THREADGROUP_SIZE + Thread] = GroupWriteOffsets[InBuffer*THREADGROUP_SIZE + Thread];
		}
		GroupMemoryBarrierWithGroupSync();
	}

	if(Thread == 0)
	{
		// Add the value of the last predicate to get the total number of IDs that this group will write.
		uint LastIsFree = (IDToIndexTable[InputStart + THREADGROUP_SIZE - 1] == -1) ? 1 : 0;
		uint NumGroupFreeIDs = GroupWriteOffsets[(OutBuffer + 1)*THREADGROUP_SIZE - 1] + LastIsFree;
		// Add to the global write offset. The previous offset is where we start writing in the output buffer.
		InterlockedAdd(RWFreeIDListSizes[FreeIDListIndex], NumGroupFreeIDs, GroupWriteStart);
	}
	GroupMemoryBarrierWithGroupSync();

	if(IDToIndexTable[ParticleID] == -1)
	{
		// This is a free ID, output it.
		uint WriteOffset = GroupWriteStart + GroupWriteOffsets[OutBuffer*THREADGROUP_SIZE + Thread];
		// TODO: consider buffering the results before writing to the output UAV.
		RWFreeIDList[WriteOffset] = ParticleID;
	}
}

#endif
