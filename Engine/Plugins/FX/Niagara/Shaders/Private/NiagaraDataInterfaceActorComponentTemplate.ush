// Copyright Epic Games, Inc. All Rights Reserved.

uint			{ParameterName}_Valid;
float4x4		{ParameterName}_Matrix;
float4			{ParameterName}_Rotation;
float3			{ParameterName}_Scale;
float3			{ParameterName}_Velocity;

void GetMatrix_{ParameterName}(out bool bOutValid, out float4x4 OutMatrix)
{
	bOutValid = {ParameterName}_Valid != 0;
	OutMatrix = {ParameterName}_Matrix;
}

void GetTransform_{ParameterName}(out bool bOutValid, out float3 OutPosition, out float4 OutRotation, out float3 OutScale)
{
	bOutValid = {ParameterName}_Valid != 0;
	OutPosition = {ParameterName}_Matrix[3].xyz;
	OutRotation = {ParameterName}_Rotation;
	OutScale = {ParameterName}_Scale;
}

void GetVelocity_{ParameterName}(out bool bOutValid, out float3 OutVelocity)
{
	bOutValid = {ParameterName}_Valid != 0;
	OutVelocity = {ParameterName}_Velocity;
}

