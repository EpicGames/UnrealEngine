// Copyright Epic Games, Inc. All Rights Reserved.

int					{ParameterName}_Length;
ByteAddressBuffer	{ParameterName}_BuiltTableData;

void Length_{ParameterName}(out int OutLength)
{
	OutLength = {ParameterName}_Length;
}

void IsValidIndex_{ParameterName}(in int InIndex, out bool OutIsValid)
{
	OutIsValid = InIndex >= 0 && InIndex < {ParameterName}_Length;
}

void LastIndex_{ParameterName}(out int OutLastIndex)
{
	OutLastIndex = {ParameterName}_Length - 1;
}

void Get_{ParameterName}(in int InIndex, out NDIDistributionIntArrayEntry OutEntry)
{
	const int ClampedIndex = clamp(InIndex, 0, {ParameterName}_Length - 1);
	const int Offset = ClampedIndex * 16; 
	const uint2 Data = {ParameterName}_BuiltTableData.Load2(Offset + 8);
	OutEntry.Value = asint(Data.x);
	OutEntry.Weight = asfloat(Data.y);
}

void GetProbabilityAlias_{ParameterName}(in int InIndex, out float Prob, out int Alias)
{
	const int ClampedIndex = clamp(InIndex, 0, {ParameterName}_Length - 1);
	const int Offset = ClampedIndex * 16; 
	const uint2 Data = {ParameterName}_BuiltTableData.Load2(Offset);
	Prob = asfloat(Data.x);
	Alias = asint(Data.y);
}

void GetRandomValue_{ParameterName}(NiagaraRandInfo RandInfo, out int OutValue)
{
	int Index = NiagaraRandomInt(RandInfo, {ParameterName}_Length);
	const uint2 ProbAlias = {ParameterName}_BuiltTableData.Load2(Index * 16);

	const float RandT0 = NiagaraRandomFloat(RandInfo);
	Index = RandT0 > asfloat(ProbAlias.x) ? asint(ProbAlias.y) : Index;

	OutValue = {ParameterName}_BuiltTableData.Load((Index * 16) + 8);
}
