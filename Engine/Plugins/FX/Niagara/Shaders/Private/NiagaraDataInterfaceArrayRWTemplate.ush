// Copyright Epic Games, Inc. All Rights Reserved.
// RWInstanceCounts come from NiagaraEmitterInstanceSimulation.ush as we use counts from that buffer
// VariableType		- Variable type, i.e. float, float2, etc
// RWBufferType		- RW Buffer type, generally float or int since we don't support typed UAV loads across all platforms
// RWBufferRead		- Hlsl code to read from the buffer, i.e Value = BUFFER_NAME[Index]
// RWBufferWrite	- Hlsl code to write to the buffer, i.e BUFFER_NAME[Index] = Value

RWBuffer<{RWBufferType}>	{ParameterName}_ArrayRWBuffer;
int2						{ParameterName}_ArrayBufferParams;				// Where x=CountOffset & y=BufferCapacity

int GetCountOffset_{ParameterName}() { return {ParameterName}_ArrayBufferParams.x; }
int GetCapacity_{ParameterName}() { return {ParameterName}_ArrayBufferParams.y; }
int GetLength_{ParameterName}() { return RWInstanceCounts[GetCountOffset_{ParameterName}()]; }

void Length_{ParameterName}(out int ArrayLength)
{
	ArrayLength = GetLength_{ParameterName}();
}

void IsValidIndex_{ParameterName}(int Index, out bool bIsValid)
{
	bIsValid = Index >= 0 && Index < GetLength_{ParameterName}();
}

void LastIndex_{ParameterName}(out int Index)
{
	Index = GetLength_{ParameterName}() - 1;
}

void Get_{ParameterName}(int Index, out {VariableType} Value)
{
	int Length = GetLength_{ParameterName}();
	Index = Length > 0 ? clamp(Index, 0, Length - 1) : GetCapacity_{ParameterName}();
	#define BUFFER_NAME {ParameterName}_ArrayRWBuffer
	{RWBufferRead}
	#undef BUFFER_NAME
}

void SetArrayElem_{ParameterName}_UEImpureCall(bool bSkip, int Index, {VariableType} Value)
{
	bool bIsValidIndex;
	IsValidIndex_{ParameterName}(Index, bIsValidIndex);
	if ( !bSkip && bIsValidIndex )
	{
		#define BUFFER_NAME {ParameterName}_ArrayRWBuffer
		{RWBufferWrite}
		#undef BUFFER_NAME
	}
}

void Add_{ParameterName}_UEImpureCall(bool bSkip, {VariableType} Value)
{
	if ( bSkip == true )
	{
		return;
	}

	uint Index;
	InterlockedAdd(RWInstanceCounts[GetCountOffset_{ParameterName}()], 1, Index);
	if ( Index < GetCapacity_{ParameterName}() )
	{
		#define BUFFER_NAME {ParameterName}_ArrayRWBuffer
		{RWBufferWrite}
		#undef BUFFER_NAME
	}
	else
	{
		InterlockedAdd(RWInstanceCounts[GetCountOffset_{ParameterName}()], uint(-1));
	}
}

void RemoveLastElem_{ParameterName}_UEImpureCall(bool bSkip, out {VariableType} Value, out bool bOutValid)
{
	uint Index;
	bOutValid = false;

	if ( bSkip == false )
	{
		InterlockedAdd(RWInstanceCounts[GetCountOffset_{ParameterName}()], uint(-1), Index);
		--Index;

		bOutValid = Index < GetCapacity_{ParameterName}();
		if ( !bOutValid )
		{
			InterlockedAdd(RWInstanceCounts[GetCountOffset_{ParameterName}()], 1);
		}
	}

	Index = bOutValid ? Index : GetCapacity_{ParameterName}();
	#define BUFFER_NAME {ParameterName}_ArrayRWBuffer
	{RWBufferRead}
	#undef BUFFER_NAME
}

#if {bSupportsAtomicOps}

void AtomicAdd_{ParameterName}_UEImpureCall(bool bSkip, int Index, int Value, out {VariableType} PrevValue, out {VariableType} CurrValue)
{
	bool bIsValidIndex;
	IsValidIndex_{ParameterName}(Index, bIsValidIndex);
	if ( !bSkip && bIsValidIndex )
	{
		InterlockedAdd({ParameterName}_ArrayRWBuffer[Index], Value, PrevValue);
		CurrValue = PrevValue + Value;
	}
	else
	{
		PrevValue = 0;
		CurrValue = 0;
	}
}

void AtomicMin_{ParameterName}_UEImpureCall(bool bSkip, int Index, int Value, out {VariableType} PrevValue, out {VariableType} CurrValue)
{
	bool bIsValidIndex;
	IsValidIndex_{ParameterName}(Index, bIsValidIndex);
	if ( !bSkip && bIsValidIndex )
	{
		InterlockedMin({ParameterName}_ArrayRWBuffer[Index], Value, PrevValue);
		CurrValue = min(PrevValue, Value);
	}
	else
	{
		PrevValue = 0;
		CurrValue = 0;
	}
}

void AtomicMax_{ParameterName}_UEImpureCall(bool bSkip, int Index, int Value, out {VariableType} PrevValue, out {VariableType} CurrValue)
{
	bool bIsValidIndex;
	IsValidIndex_{ParameterName}(Index, bIsValidIndex);
	if ( !bSkip && bIsValidIndex )
	{
		InterlockedMax({ParameterName}_ArrayRWBuffer[Index], Value, PrevValue);
		CurrValue = max(PrevValue, Value);
	}
	else
	{
		PrevValue = 0;
		CurrValue = 0;
	}
}

#endif
