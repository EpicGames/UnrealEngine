// Copyright Epic Games, Inc. All Rights Reserved.
// ParameterName	- Unique name sent from compiler
// VariableType		- Variable type, i.e. float, float2, etc
// ReadBufferType	- Read Buffer Type, i.e. float, float2, etc
// ReadBufferRead	- Hlsl code to read from the buffer, i.e Value = BUFFER_NAME[Index]

Buffer<{ReadBufferType}>	{ParameterName}_ArrayReadBuffer;
int2						{ParameterName}_ArrayBufferParams;				// Where x=Length & y=max(Length-1,0)

int GetLength_{ParameterName}() { return {ParameterName}_ArrayBufferParams.x; }
int GetLengthMinusOne_{ParameterName}() { return {ParameterName}_ArrayBufferParams.y; }

void Length_{ParameterName}(out int ArrayLength)
{
	ArrayLength = GetLength_{ParameterName}();
}

void IsValidIndex_{ParameterName}(int Index, out bool bIsValid)
{
	bIsValid = Index >= 0 && Index < GetLength_{ParameterName}();
}

void LastIndex_{ParameterName}(out int Index)
{
	Index = GetLengthMinusOne_{ParameterName}();
}

void Get_{ParameterName}(int Index, out {VariableType} Value)
{
	Index = clamp(Index, 0, GetLengthMinusOne_{ParameterName}());
	#define BUFFER_NAME {ParameterName}_ArrayReadBuffer
	{ReadBufferRead}
	#undef BUFFER_NAME
}
