// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraAsyncGpuTraceCommon.ush"

void NDIAsyncGpuTrace_IssueAsyncRayTrace(
	uint MaxRayTraceCount,
	RWStructuredBuffer<FNiagaraAsyncGpuTrace> RWRayRequests,
	uint RayRequestOffset,
	RWBuffer<uint> RWRayTraceCounts,
	uint RayTraceCountsOffset,
	float3 In_LWCTile,
	int In_QueryID,
	float3 In_TraceStart,
	float3 In_TraceEnd,
	int In_CollisionGroup,
	out bool Out_IsQueryValid)
{
	Out_IsQueryValid = false;

	if (In_QueryID >= 0 && uint(In_QueryID) < MaxRayTraceCount)
	{
		FLWCVector3 LwcStartPos = MakeLWCVector3(In_LWCTile, In_TraceStart);
		FLWCVector3 LwcEndPos = MakeLWCVector3(In_LWCTile, In_TraceEnd);
		
		float3 Trace = LWCToFloat(LWCSubtract(LwcEndPos, LwcStartPos));
		float TraceLength = length(Trace);

		//Offset query into the actual ray buffer.
		uint RayIndex = RayRequestOffset + In_QueryID;
		RWRayRequests[RayIndex].Origin = LWCToFloat(LwcStartPos); // TODO: remove LWCToFloat here when FNiagaraAsyncGpuTrace supports LWC
		RWRayRequests[RayIndex].Direction = normalize(Trace);
		RWRayRequests[RayIndex].TFar = TraceLength;
		RWRayRequests[RayIndex].CollisionGroup = In_CollisionGroup;
		Out_IsQueryValid = true;

		// we want to keep track of the maximum index used
		InterlockedMax(RWRayTraceCounts[RayTraceCountsOffset + 0], In_QueryID + 1);
		RWRayTraceCounts[RayTraceCountsOffset + 1] = 1;
		RWRayTraceCounts[RayTraceCountsOffset + 2] = 1;
	}
}

void NDIAsyncGpuTrace_ReserveRayTraceIndex(uint MaxRayTraceCount, RWBuffer<uint> RWRayTraceCounts, uint RayTraceCountsOffset, int In_TraceCount, out int Out_StartQueryID, out bool Out_IndicesValid)
{
	Out_StartQueryID = -1;
	Out_IndicesValid = false;

	if ((In_TraceCount + RWRayTraceCounts[RayTraceCountsOffset + 0]) <= MaxRayTraceCount)
	{
		uint RequestIndex = 0;
		InterlockedAdd(RWRayTraceCounts[RayTraceCountsOffset + 0], In_TraceCount, RequestIndex);
		RWRayTraceCounts[RayTraceCountsOffset + 1] = 1;
		RWRayTraceCounts[RayTraceCountsOffset + 2] = 1;

		if ((In_TraceCount + RequestIndex) <= MaxRayTraceCount)
		{
			Out_StartQueryID = int(RequestIndex);
			Out_IndicesValid = true;
		}
		else
		{
			InterlockedMax(RWRayTraceCounts[RayTraceCountsOffset + 0], MaxRayTraceCount);
		}
	}
}

void NDIAsyncGpuTrace_CreateAsyncRayTrace(	uint MaxRayTraceCount, RWStructuredBuffer<FNiagaraAsyncGpuTrace> RWRayRequests, uint RayRequestsOffset, RWBuffer<uint> RWRayTraceCounts, uint RayTraceCountsOffset, float3 In_LWCTile,
											float3 In_TraceStart, float3 In_TraceEnd, int In_CollisionGroup, out int Out_QueryID, out bool Out_IsQueryValid)
{
	Out_IsQueryValid = false;
	Out_QueryID = -1;

	NDIAsyncGpuTrace_ReserveRayTraceIndex(MaxRayTraceCount, RWRayTraceCounts, RayTraceCountsOffset, 1, Out_QueryID, Out_IsQueryValid);

	if (Out_IsQueryValid)
	{
		FLWCVector3 LwcStartPos = MakeLWCVector3(In_LWCTile, In_TraceStart);
		FLWCVector3 LwcEndPos = MakeLWCVector3(In_LWCTile, In_TraceEnd);
		
		float3 Trace = LWCToFloat(LWCSubtract(LwcEndPos, LwcStartPos));
		float TraceLength = length(Trace);

		uint RayIndex = Out_QueryID + RayRequestsOffset;
		RWRayRequests[RayIndex].Origin = LWCToFloat(LwcStartPos); // TODO: remove LWCToFloat here when FNiagaraAsyncGpuTrace supports LWC
		RWRayRequests[RayIndex].Direction = normalize(Trace);
		RWRayRequests[RayIndex].TFar = TraceLength;

		//TODO: We can pack collision group and trace channels into the same int and test both in the RG Shader.
		RWRayRequests[RayIndex].CollisionGroup = In_CollisionGroup;
	}
}

void NDIAsyncGpuTrace_ReadAsyncRayTrace(uint MaxRayTraceCount, StructuredBuffer<FNiagaraAsyncGpuTraceResult> IntersectionResults, uint IntersectionRestultsOffest, float3 In_LWCTile, int In_PreviousFrameQueryID, out bool Out_CollisionValid, out float Out_CollisionDistance, out float3 Out_CollisionPosWorld, out float3 Out_CollisionNormal)
{
	Out_CollisionValid = false;
	Out_CollisionDistance = 0.0f;
	Out_CollisionPosWorld = float3(0.0f, 0.0f, 0.0f);
	Out_CollisionNormal = float3(0.0f, 0.0f, 0.0f);

	if (In_PreviousFrameQueryID >= 0 && uint(In_PreviousFrameQueryID) < MaxRayTraceCount)
	{
		FNiagaraAsyncGpuTraceResult HitResult = IntersectionResults[In_PreviousFrameQueryID + IntersectionRestultsOffest];

		Out_CollisionValid = IsHit(HitResult);
		if (Out_CollisionValid)
		{
			Out_CollisionDistance = HitResult.HitT;
			Out_CollisionPosWorld = LWCToFloat(MakeLWCVector3(-In_LWCTile, HitResult.WorldPosition)); // TODO: HitResult.WorldPosition should support LWC properly
			Out_CollisionNormal = HitResult.WorldNormal;
		}
	}
}