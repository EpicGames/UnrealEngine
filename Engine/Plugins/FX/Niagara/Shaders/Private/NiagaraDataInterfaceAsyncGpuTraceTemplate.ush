// Copyright Epic Games, Inc. All Rights Reserved.

/** Niagara ray tracing uses shared buffers between dispatches so each buffer has a buffer + and offset into that buffer for the current dispatch. */
uint											{ParameterName}_MaxRayTraceCount;
RWStructuredBuffer<FNiagaraAsyncGpuTrace>		{ParameterName}_RWRayRequests;
uint											{ParameterName}_RayRequestsOffset;
StructuredBuffer<FNiagaraAsyncGpuTraceResult>	{ParameterName}_IntersectionResults;
uint											{ParameterName}_IntersectionResultsOffset;
RWBuffer<uint>									{ParameterName}_RWRayTraceCounts;
uint											{ParameterName}_RayTraceCountsOffset;

void IssueAsyncRayTraceGpu_{ParameterName}_UEImpureCall(bool bExecute, int In_QueryID, float3 In_TraceStart, float3 In_TraceEnd, int In_CollisionGroup, out bool Out_IsQueryValid)
{
	Out_IsQueryValid = false;
	if ( bExecute )
	{
		NDIAsyncGpuTrace_IssueAsyncRayTrace({ParameterName}_MaxRayTraceCount, {ParameterName}_RWRayRequests, {ParameterName}_RayRequestsOffset, {ParameterName}_RWRayTraceCounts, {ParameterName}_RayTraceCountsOffset, GetEngineOwnerLWCTile(), In_QueryID, In_TraceStart, In_TraceEnd, In_CollisionGroup, Out_IsQueryValid);
	}
}

void ReserveAsyncRayTraceGpu_{ParameterName}_UEImpureCall(bool bExecute, int In_TraceCount, out int Out_StartQueryID, out bool Out_IndicesValid)
{
	Out_StartQueryID = -1;
	Out_IndicesValid = false;
	if (bExecute)
	{
		NDIAsyncGpuTrace_ReserveRayTraceIndex({ParameterName}_MaxRayTraceCount, {ParameterName}_RWRayTraceCounts, {ParameterName}_RayTraceCountsOffset, In_TraceCount, Out_StartQueryID, Out_IndicesValid);
	}
}

void CreateAsyncRayTraceGpu_{ParameterName}_UEImpureCall(bool bExecute, float3 In_TraceStart, float3 In_TraceEnd, int In_CollisionGroup, out int Out_QueryID, out bool Out_IsQueryValid)
{
	Out_QueryID			= -1;
	Out_IsQueryValid	= false;
	if (bExecute)
	{
		NDIAsyncGpuTrace_CreateAsyncRayTrace({ParameterName}_MaxRayTraceCount, {ParameterName}_RWRayRequests, {ParameterName}_RayRequestsOffset, {ParameterName}_RWRayTraceCounts, {ParameterName}_RayTraceCountsOffset, GetEngineOwnerLWCTile(), In_TraceStart, In_TraceEnd, In_CollisionGroup, Out_QueryID, Out_IsQueryValid);
	}
}

void ReadAsyncRayTraceGpu_{ParameterName}(bool bExecute, int In_PreviousFrameQueryID, out bool Out_CollisionValid, out float Out_CollisionDistance, out float3 Out_CollisionPosWorld, out float3 Out_CollisionNormal)
{
	Out_CollisionValid		= false;
	Out_CollisionDistance	= 0.0f;
	Out_CollisionPosWorld	= float3(0.0f, 0.0f, 0.0f);
	Out_CollisionNormal		= float3(0.0f, 0.0f, 0.0f);
	if (bExecute)
	{
		NDIAsyncGpuTrace_ReadAsyncRayTrace({ParameterName}_MaxRayTraceCount, {ParameterName}_IntersectionResults, {ParameterName}_IntersectionResultsOffset, GetEngineOwnerLWCTile(), In_PreviousFrameQueryID, Out_CollisionValid, Out_CollisionDistance, Out_CollisionPosWorld, Out_CollisionNormal);
	}
}
