// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

float SampleInterleavedFloatBuffer(Buffer<float> InBuffer, int ChannelCount, int ChannelResolution, float NormalizedPosition, int ChannelIndex)
{
	float FrameIndex = saturate(NormalizedPosition) * (ChannelResolution - 1);
	int LowerIndex = max(0, floor(FrameIndex));
	int UpperIndex = min(LowerIndex + 1, ChannelResolution - 1);
	ChannelIndex = clamp(ChannelIndex, 0, ChannelCount);

	float Fraction = FrameIndex - LowerIndex;

	float LowerValue = InBuffer.Load(LowerIndex * ChannelCount + ChannelIndex);
	float UpperValue = InBuffer.Load(UpperIndex * ChannelCount + ChannelIndex);

	return lerp(LowerValue, UpperValue, Fraction);
}

float SamplePlanarFloatBuffer(Buffer<float> InBuffer, int ChannelCount, int ChannelResolution, float NormalizedPosition, int ChannelIndex)
{
	float FrameIndex = saturate(NormalizedPosition) * (ChannelResolution - 1);
	int LowerIndex = max(0, floor(FrameIndex));
	int UpperIndex = min(LowerIndex + 1, ChannelResolution - 1);
	ChannelIndex = clamp(ChannelIndex, 0, ChannelCount);

	float Fraction = FrameIndex - LowerIndex;

	float LowerValue = InBuffer.Load(ChannelIndex * ChannelCount + LowerIndex);
	float UpperValue = InBuffer.Load(ChannelIndex * ChannelCount + UpperIndex);

	return lerp(LowerValue, UpperValue, Fraction);
}

