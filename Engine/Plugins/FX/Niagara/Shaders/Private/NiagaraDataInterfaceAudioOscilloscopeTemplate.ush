// Copyright Epic Games, Inc. All Rights Reserved.

Buffer<float> {ParameterName}_OscilloscopeBuffer;
int {ParameterName}_ChannelCount;
int {ParameterName}_Resolution;

void SampleAudioBuffer_{ParameterName}(float In_NormalizedPosition, int In_ChannelIndex, out float Out_Val)
{
	Out_Val = SampleInterleavedFloatBuffer(
		{ParameterName}_OscilloscopeBuffer,
		{ParameterName}_ChannelCount,
		{ParameterName}_Resolution,
		In_NormalizedPosition,
		In_ChannelIndex);
}

void GetAudioBufferNumChannels_{ParameterName}(out int Out_Val)
{
	Out_Val = {ParameterName}_ChannelCount;
}

