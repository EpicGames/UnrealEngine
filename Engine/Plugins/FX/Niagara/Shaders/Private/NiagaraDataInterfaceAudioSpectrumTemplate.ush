// Copyright Epic Games, Inc. All Rights Reserved.

Buffer<float> {ParameterName}_SpectrumBuffer;
int {ParameterName}_ChannelCount;
int {ParameterName}_Resolution;

void AudioSpectrum_{ParameterName}(float In_NormalizedPosition, int In_ChannelIndex, out float Out_Val)
{
	Out_Val = SamplePlanarFloatBuffer(
		{ParameterName}_SpectrumBuffer,
		{ParameterName}_ChannelCount,
		{ParameterName}_Resolution,
		In_NormalizedPosition,
		In_ChannelIndex);
}

void GetNumChannels_{ParameterName}(out int Out_Val)
{
	Out_Val = {ParameterName}_ChannelCount;
}
