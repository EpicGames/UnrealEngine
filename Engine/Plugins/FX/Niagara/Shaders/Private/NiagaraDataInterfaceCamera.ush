// Copyright Epic Games, Inc. All Rights Reserved.

int {ParameterName}_SplitscreenMode;

void GetFieldOfView_{ParameterName}(out float Out_FieldOfViewAngle)
{
	Out_FieldOfViewAngle = degrees(View.FieldOfViewWideAngles.x);
}

void GetSplitscreenInfo_{ParameterName}(out int Out_CurrentSplitscreenMode)
{
	Out_CurrentSplitscreenMode = {ParameterName}_SplitscreenMode;
}

void GetTAAJitter_{ParameterName}(out float2 Out_CurrentJitter, out float2 Out_PreviousJitter)
{
	Out_CurrentJitter = View.TemporalAAJitter.xy;
	Out_PreviousJitter = View.TemporalAAJitter.zw;
}

void GetCameraPropertiesCPUASC47GPU_{ParameterName}(out float3 Out_CameraPositionWorld, out float3 Out_ViewForwardVector, out float3 Out_ViewUpVector, out float3 Out_ViewRightVector, out float2 Out_ViewportSize, out float Out_NearClipPlane)
{
	FLWCVector3 CameraPos = PrimaryView.TileOffset.WorldCameraOrigin;
	LWCSetTile(CameraPos, LWCGetTile(CameraPos) - GetEngineOwnerLWCTile()); // convert to simulation space
	Out_CameraPositionWorld = LWCToFloat(CameraPos);
	Out_ViewForwardVector = View.ViewForward;
	Out_ViewUpVector = View.ViewUp;
	Out_ViewRightVector = View.ViewRight;
	Out_ViewportSize = View.ViewSizeAndInvSize.xy;
	Out_NearClipPlane = View.NearPlane;
}

void GetViewPropertiesGPU_{ParameterName}(out float3 Out_ViewPositionWorld, out float3 Out_ViewForwardVector, out float3 Out_ViewUpVector, out float3 Out_ViewRightVector, out float4 Out_ViewSizeAndInverseSize,
	out float4 Out_ScreenToViewSpace, out float2 Out_Current_TAAJitter, out float2 Out_Previous_TAAJitter, out float3 Out_PreViewTranslation, out float4 Out_BufferSizeAndInverseSize, out float2 Out_ViewportOffset,
	out float Out_NearPlane, out float2 Out_FOVCurrent, out float2 Out_FOVPrevious)
{
	FLWCVector3 ViewPos = PrimaryView.TileOffset.WorldViewOrigin;
	LWCSetTile(ViewPos, LWCGetTile(ViewPos) - GetEngineOwnerLWCTile()); // convert to simulation space
	Out_ViewPositionWorld.xyz = LWCToFloat(ViewPos);
	Out_ViewForwardVector = View.ViewForward;
	Out_ViewUpVector = View.ViewUp;
	Out_ViewRightVector = View.ViewRight;
	Out_ViewSizeAndInverseSize = View.ViewSizeAndInvSize;
	Out_ScreenToViewSpace = View.ScreenToViewSpace;
	Out_Current_TAAJitter = View.TemporalAAJitter.xy;
	Out_Previous_TAAJitter = View.TemporalAAJitter.zw;
	Out_PreViewTranslation = DFHackToFloat(PrimaryView.PreViewTranslation);
	Out_BufferSizeAndInverseSize = View.BufferSizeAndInvSize;
	Out_ViewportOffset = View.ViewRectMin.xy;
	Out_NearPlane = View.NearPlane;
	Out_FOVCurrent = View.FieldOfViewWideAngles.xy;
	Out_FOVPrevious = View.PrevFieldOfViewWideAngles.xy;
}

void GetClipSpaceTransformsGPU_{ParameterName}(out float4x4 Out_WorldToClipTransform, out float4x4 Out_TranslatedWorldToClipTransform, out float4x4 Out_ClipToWorldTransform, out float4x4 Out_ClipToViewTransform,
	out float4x4 Out_ClipToTranslatedWorldTransform, out float4x4 Out_ScreenToWorldTransform, out float4x4 Out_ScreenToTranslatedWorldTransform, out float4x4 Out_ClipToPreviousClipTransform)
{
	Out_WorldToClipTransform = DFHackToFloat(PrimaryView.WorldToClip);
	Out_TranslatedWorldToClipTransform = View.TranslatedWorldToClip;
	Out_ClipToWorldTransform = DFHackToFloat(PrimaryView.ClipToWorld);
	Out_ClipToViewTransform = View.ClipToView;
	Out_ClipToTranslatedWorldTransform = View.ClipToTranslatedWorld;
	Out_ScreenToWorldTransform = DFHackToFloat(PrimaryView.ScreenToWorld);
	Out_ScreenToTranslatedWorldTransform = View.ScreenToTranslatedWorld;
	Out_ClipToPreviousClipTransform = View.ClipToPrevClip;
}

void GetViewSpaceTransformsGPU_{ParameterName}(out float4x4 Out_TranslatedWorldToViewTransform, out float4x4 Out_ViewToTranslatedWorldTransform, out float4x4 Out_TranslatedWorldToCameraViewTransform,
	out float4x4 Out_CameraViewToTranslatedWorldTransform, out float4x4 Out_ViewToClipTransform, out float4x4 Out_ViewToClipNoAATransform)
{
	Out_TranslatedWorldToViewTransform = View.TranslatedWorldToView;
	Out_ViewToTranslatedWorldTransform = View.ViewToTranslatedWorld;
	Out_TranslatedWorldToCameraViewTransform = View.TranslatedWorldToCameraView;
	Out_CameraViewToTranslatedWorldTransform = View.CameraViewToTranslatedWorld;
	Out_ViewToClipTransform = View.ViewToClip;
	Out_ViewToClipNoAATransform = View.ViewToClipNoAA;
}

void ApplyPreViewTranslationToPositionGPU_{ParameterName}(in float3 In_WorldPosition, out float4 Out_TranslatedPosition)
{
	FLWCVector3 LwcPos = MakeLWCVector3(GetEngineOwnerLWCTile(), In_WorldPosition);
	float3 ResolvedLwcPos = LWCToFloat(LWCAdd(LwcPos, PrimaryView.TileOffset.PreViewTranslation));
	Out_TranslatedPosition = float4(ResolvedLwcPos, 1);
}

void RemovePreViewTranslationFromPositionGPU_{ParameterName}(in float4 In_TranslatedPosition, out float3 Out_WorldPosition)
{
	FLWCVector3 TranslatedPos = LWCPromote(In_TranslatedPosition.xyz / In_TranslatedPosition.w);
	FLWCVector3 WorldPos = LWCSubtract(TranslatedPos, PrimaryView.TileOffset.PreViewTranslation);
	WorldPos.Tile -= GetEngineOwnerLWCTile();
	Out_WorldPosition = LWCToFloat(WorldPos);
}
