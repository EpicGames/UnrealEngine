// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/DeferredShadingCommon.ush"

#define NEEDS_SCENE_TEXTURES 1
#define SCENE_TEXTURES_DISABLED 0
#define SubstratePublicStruct SubstratePublic
#include "/Engine/Public/SubstratePublic.ush"

// Mobile requires multi-pass deferred to access GBuffer
#define NDICOLLISIONQUERY_USE_GBUFFER_NORMAL    (FEATURE_LEVEL >= FEATURE_LEVEL_SM5) || (MOBILE_DEFERRED_SHADING && !ALLOW_FRAMEBUFFER_FETCH)

float NDICollisionQuery_GetCustomSceneDepth(float2 ScreenUV)
{
#if (FEATURE_LEVEL <= FEATURE_LEVEL_ES3_1)
	return ConvertFromDeviceZ(Texture2DSampleLevel(MobileSceneTextures.CustomDepthTexture, MobileSceneTextures.CustomDepthTextureSampler, ScreenUV, 0).r);
#else
	return ConvertFromDeviceZ(Texture2DSampleLevel(SceneTexturesStruct.CustomDepthTexture, SceneTexturesStruct_CustomDepthTextureSampler, ScreenUV, 0).r);
#endif
}

float NDICollisionQuery_GetScenePartialDepth(float2 ScreenUV)
{
#if (FEATURE_LEVEL <= FEATURE_LEVEL_ES3_1)
	return ConvertFromDeviceZ(Texture2DSampleLevel(MobileSceneTextures.ScenePartialDepthTexture, MobileSceneTextures.ScenePartialDepthTextureSampler, ScreenUV, 0).r);
#else
	return ConvertFromDeviceZ(Texture2DSampleLevel(SceneTexturesStruct.ScenePartialDepthTexture, SceneTexturesStruct_ScenePartialDepthTextureSampler, ScreenUV, 0).r);
#endif
}

void NDICollisionQuery_QuerySceneDepthGPU(in float3 In_SamplePos, in float3 In_LWCTile, out float Out_SceneDepth, out float3 Out_CameraPosWorld, out bool Out_IsInsideView, out float3 Out_WorldPos, out float3 Out_WorldNormal)
{
	Out_SceneDepth = -1;
	Out_WorldPos = float3(0.0, 0.0, 0.0);
	Out_WorldNormal = float3(0.0, 0.0, 1.0);
	Out_IsInsideView = true;
	FLWCVector3 CameraPos = PrimaryView.TileOffset.WorldCameraOrigin;
	LWCSetTile(CameraPos, LWCGetTile(CameraPos) - In_LWCTile); // convert to simulation space
	Out_CameraPosWorld = LWCToFloat(CameraPos);

	FLWCVector3 LwcSamplePos = MakeLWCVector3(In_LWCTile, In_SamplePos);
	float4 SamplePosition = float4(LWCToFloat(LWCAdd(LwcSamplePos, PrimaryView.TileOffset.PreViewTranslation)), 1); // TODO[mg]: LWCToFloat here?
	float4 ClipPosition = mul(SamplePosition, View.TranslatedWorldToClip);
	float2 ScreenPosition = ClipPosition.xy / ClipPosition.w;

	// Check if the sample is inside the view.
	if (all(abs(ScreenPosition.xy) <= float2(1, 1)))
	{
		// Sample the depth buffer to get a world position near the sample position.
		float2 ScreenUV = ScreenPosition * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;
		float SceneDepth = CalcSceneDepth(ScreenUV);
		FLWCVector3 WorldPosition = WorldPositionFromSceneDepth(ScreenPosition.xy, SceneDepth);

#if NDICOLLISIONQUERY_USE_GBUFFER_NORMAL
#if SUBSTRATE_GBUFFER_FORMAT==1
		uint2 ScreenPos = uint2(ScreenUV * View.BufferSizeAndInvSize.xy);
		float3 WorldNormal = SubstratePublic_GetWorldNormal(ScreenPos);
#else
		FGBufferData GBuffer = GetGBufferData(ScreenUV, false);
		float3 WorldNormal = GBuffer.WorldNormal;
#endif
#else
		float CollisionDepthBounds = 500.0f;
		float SceneDepth0 = CalcSceneDepth(ScreenUV + float2(View.BufferSizeAndInvSize.z, 0.0));
		float SceneDepth1 = CalcSceneDepth(ScreenUV + float2(0.0, View.BufferSizeAndInvSize.w));
		// When using the forward shading, the normal of the pixel is approximated by the derivative of the world position
		// of the pixel. But in on the visible edge this derivative can become very high, making CollisionPlane almost
		// perpendicular to the view plane. In these case the particle may collide the visible edges of the diferent meshes
		// in the view frustum. To avoid this, we disable the collision test if one of the derivate is above a threshold.
		if (max(abs(SceneDepth - SceneDepth0), abs(SceneDepth - SceneDepth1)) > CollisionDepthBounds)
		{
			return;
		}

		FLWCVector3 WorldPosition0 = WorldPositionFromSceneDepth(ScreenPosition.xy + float2(2 * View.ViewSizeAndInvSize.z, 0.0), SceneDepth0);
		FLWCVector3 WorldPosition1 = WorldPositionFromSceneDepth(ScreenPosition.xy - float2(0.0, 2 * View.ViewSizeAndInvSize.w), SceneDepth1);
		float3 WorldNormal = normalize(cross(LWCSubtract(WorldPosition0, WorldPosition).Offset, LWCSubtract(WorldPosition1, WorldPosition).Offset));
#endif

		// convert LWC position back to simulation space
		LWCSetTile(WorldPosition, LWCGetTile(WorldPosition) - In_LWCTile);
		
		// Set outputs
		Out_SceneDepth = SceneDepth;
		Out_WorldPos = LWCToFloat(WorldPosition);
		Out_WorldNormal = WorldNormal;
	}
	else
	{
		Out_IsInsideView = false;
	}
}

void NDICollisionQuery_QueryScenePartialDepthGPU(in float3 In_SamplePos, in float3 In_LWCTile, out float Out_SceneDepth, out float3 Out_CameraPosWorld, out bool Out_IsInsideView, out float3 Out_WorldPos, out float3 Out_WorldNormal)
{
	Out_SceneDepth = -1;
	Out_WorldPos = float3(0.0, 0.0, 0.0);
	Out_WorldNormal = float3(0.0, 0.0, 1.0);
	Out_IsInsideView = true;
	FLWCVector3 CameraPos = PrimaryView.TileOffset.WorldCameraOrigin;
	LWCSetTile(CameraPos, LWCGetTile(CameraPos) - In_LWCTile); // convert to simulation space
	Out_CameraPosWorld = LWCToFloat(CameraPos);

	FLWCVector3 LwcSamplePos = MakeLWCVector3(In_LWCTile, In_SamplePos);
	float4 SamplePosition = float4(LWCToFloat(LWCAdd(LwcSamplePos, PrimaryView.TileOffset.PreViewTranslation)), 1); // TODO[mg]: LWCToFloat here?
	float4 ClipPosition = mul(SamplePosition, View.TranslatedWorldToClip);
	float2 ScreenPosition = ClipPosition.xy / ClipPosition.w;

	// Check if the sample is inside the view.
	if (all(abs(ScreenPosition.xy) <= float2(1, 1)))
	{
		// Sample the depth buffer to get a world position near the sample position.
		float2 ScreenUV = ScreenPosition * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;

		float SceneDepth = NDICollisionQuery_GetScenePartialDepth(ScreenUV);

		FLWCVector3 WorldPosition = WorldPositionFromSceneDepth(ScreenPosition.xy, SceneDepth);

		// We cannot sample the normal from the GBuffer when using partial depth collision queries.
		// => It must be derived form the partial depth buffer itself to get a match.

		float CollisionDepthBounds = 500.0f;
		float SceneDepth0 = NDICollisionQuery_GetScenePartialDepth(ScreenUV + float2(View.BufferSizeAndInvSize.z, 0.0));
		float SceneDepth1 = NDICollisionQuery_GetScenePartialDepth(ScreenUV + float2(0.0, View.BufferSizeAndInvSize.w));
		// When using the forward shading, the normal of the pixel is approximated by the derivative of the world position
		// of the pixel. But in on the visible edge this derivative can become very high, making CollisionPlane almost
		// perpendicular to the view plane. In these case the particle may collide the visible edges of the diferent meshes
		// in the view frustum. To avoid this, we disable the collision test if one of the derivate is above a threshold.
		if (max(abs(SceneDepth - SceneDepth0), abs(SceneDepth - SceneDepth1)) > CollisionDepthBounds)
		{
			return;
		}

		FLWCVector3 WorldPosition0 = WorldPositionFromSceneDepth(ScreenPosition.xy + float2(2 * View.ViewSizeAndInvSize.z, 0.0), SceneDepth0);
		FLWCVector3 WorldPosition1 = WorldPositionFromSceneDepth(ScreenPosition.xy - float2(0.0, 2 * View.ViewSizeAndInvSize.w), SceneDepth1);
		float3 WorldNormal = normalize(cross(LWCSubtract(WorldPosition0, WorldPosition).Offset, LWCSubtract(WorldPosition1, WorldPosition).Offset));

		// convert LWC position back to simulation space
		LWCSetTile(WorldPosition, LWCGetTile(WorldPosition) - In_LWCTile);

		// Set outputs
		Out_SceneDepth = SceneDepth;
		Out_WorldPos = LWCToFloat(WorldPosition);
		Out_WorldNormal = WorldNormal;
	}
	else
	{
		Out_IsInsideView = false;
	}
}

void NDICollisionQuery_QueryCustomDepthGPU(in float3 In_SamplePos, in float3 In_LWCTile, out float Out_SceneDepth, out float3 Out_CameraPosWorld, out bool Out_IsInsideView, out float3 Out_WorldPos, out float3 Out_WorldNormal)
{
	Out_SceneDepth = -1;
	Out_WorldPos = float3(0.0, 0.0, 0.0);
	Out_WorldNormal = float3(0.0, 0.0, 1.0);
	Out_IsInsideView = true;
	FLWCVector3 CameraPos = PrimaryView.TileOffset.WorldCameraOrigin;
	LWCSetTile(CameraPos, LWCGetTile(CameraPos) - In_LWCTile); // convert to simulation space
	Out_CameraPosWorld = LWCToFloat(CameraPos);

	FLWCVector3 LwcSamplePos = MakeLWCVector3(In_LWCTile, In_SamplePos);
	float4 SamplePosition = float4(LWCToFloat(LWCAdd(LwcSamplePos, PrimaryView.TileOffset.PreViewTranslation)), 1); // TODO[mg]: LWCToFloat here?
	float4 ClipPosition = mul(SamplePosition, View.TranslatedWorldToClip);
	float2 ScreenPosition = ClipPosition.xy / ClipPosition.w;

	// Check if the sample is inside the view.
	if (all(abs(ScreenPosition.xy) <= float2(1, 1)))
	{
		// Sample the depth buffer to get a world position near the sample position.
		float2 ScreenUV = ScreenPosition * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;
		float SceneDepth = NDICollisionQuery_GetCustomSceneDepth(ScreenUV);
		FLWCVector3 WorldPosition = WorldPositionFromSceneDepth(ScreenPosition.xy, SceneDepth);

		float CollisionDepthBounds = 500.0f;
		float SceneDepth0 = NDICollisionQuery_GetCustomSceneDepth(ScreenUV + float2(View.BufferSizeAndInvSize.z, 0.0));
		float SceneDepth1 = NDICollisionQuery_GetCustomSceneDepth(ScreenUV + float2(0.0, View.BufferSizeAndInvSize.w));
		// When using the forward shading, the normal of the pixel is approximated by the derivative of the world position
		// of the pixel. But in on the visible edge this derivative can become very high, making CollisionPlane almost
		// perpendicular to the view plane. In these case the particle may collide the visible edges of the diferent meshes
		// in the view frustum. To avoid this, we disable the collision test if one of the derivate is above a threshold.
		if (max(abs(SceneDepth - SceneDepth0), abs(SceneDepth - SceneDepth1)) > CollisionDepthBounds)
		{
			return;
		}

		FLWCVector3 WorldPosition0 = WorldPositionFromSceneDepth(ScreenPosition.xy + float2(2 * View.ViewSizeAndInvSize.z, 0.0), SceneDepth0);
		FLWCVector3 WorldPosition1 = WorldPositionFromSceneDepth(ScreenPosition.xy - float2(0.0, 2 * View.ViewSizeAndInvSize.w), SceneDepth1);
		float3 WorldNormal = normalize(cross(LWCSubtract(WorldPosition0, WorldPosition).Offset, LWCSubtract(WorldPosition1, WorldPosition).Offset));

		// convert LWC position back to simulation space
		LWCSetTile(WorldPosition, LWCGetTile(WorldPosition) - In_LWCTile);
		
		// Set outputs
		Out_SceneDepth = SceneDepth;
		Out_WorldPos = LWCToFloat(WorldPosition);
		Out_WorldNormal = WorldNormal;
	}
	else
	{
		Out_IsInsideView = false;
	}
}

void NDICollisionQuery_QueryMeshDistanceFieldGPU(in float3 In_SamplePos, in float3 In_LWCTile, out float Out_DistanceToNearestSurface, out float3 Out_FieldGradient, out bool Out_IsDistanceFieldValid)
{
#if PLATFORM_SUPPORTS_DISTANCE_FIELDS
	FLWCVector3 LwcSamplePos = MakeLWCVector3(In_LWCTile, In_SamplePos);
	float3 SamplePosition = LWCToFloat(LWCAdd(LwcSamplePos, PrimaryView.TileOffset.PreViewTranslation));
	
	Out_DistanceToNearestSurface = GetDistanceToNearestSurfaceGlobal(SamplePosition);
	Out_FieldGradient = GetDistanceFieldGradientGlobal(SamplePosition);
	Out_IsDistanceFieldValid = (MaxGlobalDFAOConeDistance > 0) && !(Out_DistanceToNearestSurface > 0 && all(Out_FieldGradient == float3(0,0,0.001f)));
#else
	Out_DistanceToNearestSurface = 0;
	Out_FieldGradient = (float3)0;
	Out_IsDistanceFieldValid = false;
#endif
}
