// Copyright Epic Games, Inc. All Rights Reserved.

void QuerySceneDepthGPU_{ParameterName}(in float3 In_SamplePos, out float Out_SceneDepth, out float3 Out_CameraPosWorld, out bool Out_IsInsideView, out float3 Out_WorldPos, out float3 Out_WorldNormal)
{
	NDICollisionQuery_QuerySceneDepthGPU(In_SamplePos, GetEngineOwnerLWCTile(), Out_SceneDepth, Out_CameraPosWorld, Out_IsInsideView, Out_WorldPos, Out_WorldNormal);
}

void QueryScenePartialDepthGPU_{ParameterName}(in float3 In_SamplePos, out float Out_SceneDepth, out float3 Out_CameraPosWorld, out bool Out_IsInsideView, out float3 Out_WorldPos, out float3 Out_WorldNormal)
{
	NDICollisionQuery_QueryScenePartialDepthGPU(In_SamplePos, GetEngineOwnerLWCTile(), Out_SceneDepth, Out_CameraPosWorld, Out_IsInsideView, Out_WorldPos, Out_WorldNormal);
}

void QueryCustomDepthGPU_{ParameterName}(in float3 In_SamplePos, out float Out_SceneDepth, out float3 Out_CameraPosWorld, out bool Out_IsInsideView, out float3 Out_WorldPos, out float3 Out_WorldNormal)
{
	NDICollisionQuery_QueryCustomDepthGPU(In_SamplePos, GetEngineOwnerLWCTile(), Out_SceneDepth, Out_CameraPosWorld, Out_IsInsideView, Out_WorldPos, Out_WorldNormal);
}

void QueryMeshDistanceFieldGPU_{ParameterName}(in float3 In_SamplePos, out float Out_DistanceToNearestSurface, out float3 Out_FieldGradient, out bool Out_IsDistanceFieldValid)
{
	NDICollisionQuery_QueryMeshDistanceFieldGPU(In_SamplePos, GetEngineOwnerLWCTile(), Out_DistanceToNearestSurface, Out_FieldGradient, Out_IsDistanceFieldValid);
}

// DEPRECATE_BEGIN
void IssueAsyncRayTraceGpu_{ParameterName}_UEImpureCall(int In_QueryID, float3 In_TraceStart, float3 In_TraceEnd, int In_TraceChannel, out bool Out_IsQueryValid)
{
	Out_IsQueryValid = false;
}

void ReserveRayTraceIndex_{ParameterName}_UEImpureCall(int In_TraceCount, out int Out_StartQueryID, out bool Out_IndicesValid)
{
	Out_StartQueryID = -1;
	Out_IndicesValid = false;
}

void CreateAsyncRayTraceGpu_{ParameterName}_UEImpureCall(float3 In_TraceStart, float3 In_TraceEnd, int In_TraceChannel, out int Out_QueryID, out bool Out_IsQueryValid)
{
	Out_QueryID = -1;
	Out_IsQueryValid = false;
}

void ReadAsyncRayTraceGpu_{ParameterName}(int In_PreviousFrameQueryID, out bool Out_CollisionValid, out float Out_CollisionDistance, out float3 Out_CollisionPosWorld, out float3 Out_CollisionNormal)
{
	Out_CollisionDistance = 0.0f;
	Out_CollisionPosWorld = float3(0.0f, 0.0f, 0.0f);
	Out_CollisionNormal = float3(0.0f, 0.0f, 0.0f);
	Out_CollisionValid = false;
}
// DEPRECATE_END
