// Copyright Epic Games, Inc. All Rights Reserved.

int2			{ParameterName}_TextureSize;
TextureCube		{ParameterName}_Texture;
SamplerState	{ParameterName}_TextureSampler;

void SampleCubeTexture_{ParameterName}(in float3 UVW, in float MipLevel, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UVW, MipLevel);
}

void TextureDimensions_{ParameterName}(out int OutWidth, out int OutHeight)
{
	OutWidth = {ParameterName}_TextureSize.x;
	OutHeight = {ParameterName}_TextureSize.y;
}
