// Copyright Epic Games, Inc. All Rights Reserved.

float			{ParameterName}_MinTime;
float			{ParameterName}_MaxTime;
float			{ParameterName}_InvTimeRange;
uint			{ParameterName}_CurveLUTNumMinusOne;
uint			{ParameterName}_LUTOffset;
Buffer<float>	{ParameterName}_CurveLUT;

void GetCurveLUTIndices_{ParameterName}(float Time, out uint IndexA, out uint IndexB, out float Fraction)
{
	float RemappedTime = saturate((Time - {ParameterName}_MinTime) * {ParameterName}_InvTimeRange) * float({ParameterName}_CurveLUTNumMinusOne);
	IndexA = floor(RemappedTime);
	IndexB = min(IndexA + 1, {ParameterName}_CurveLUTNumMinusOne);
	Fraction = frac(RemappedTime);
}

float SampleCurveLUT_{ParameterName}(uint index)
{
#if {UseUniqueBuffer} == 1
	return {ParameterName}_CurveLUT[index];
#else
	return StaticInputFloat[{ParameterName}_LUTOffset + index];
#endif
}

#if {NumElements} == 1
void {CurveSampleFunctionName}_{ParameterName}(float Time, out float Value)
{
	uint IndexA, IndexB;
	float Fraction;
	GetCurveLUTIndices_{ParameterName}(Time, IndexA, IndexB, Fraction);

	IndexA *= {NumElements};
	IndexB *= {NumElements};

	[unroll]
	for ( int i=0; i < {NumElements}; ++i )
	{
		Value = lerp(SampleCurveLUT_{ParameterName}(IndexA + i), SampleCurveLUT_{ParameterName}(IndexB + i), Fraction);
	}
}
#else
void {CurveSampleFunctionName}_{ParameterName}(float Time, out float{NumElements} Value)
{
	uint IndexA, IndexB;
	float Fraction;
	GetCurveLUTIndices_{ParameterName}(Time, IndexA, IndexB, Fraction);

	IndexA *= {NumElements};
	IndexB *= {NumElements};

	[unroll]
	for ( int i=0; i < {NumElements}; ++i )
	{
		Value[i] = lerp(SampleCurveLUT_{ParameterName}(IndexA + i), SampleCurveLUT_{ParameterName}(IndexB + i), Fraction);
	}
}
#endif
