// Copyright Epic Games, Inc. All Rights Reserved.

int					{ParameterName}_NumRows;
uint				{ParameterName}_RowStride;
uint				{ParameterName}_InvalidRowReadOffset;
int					{ParameterName}_NumFilteredRows;
ByteAddressBuffer	{ParameterName}_TableDataBuffer;
Buffer<uint>		{ParameterName}_FilteredRowDataOffsetBuffer;
//uint4				{ParameterName}_AttributeReadOffset[];

uint GetReadOffset_{ParameterName}(uint RowOffset, uint Column)
{
	const uint ColumnOffset = {ParameterName}_AttributeReadOffset[Column / 4][Column % 4];
	return ColumnOffset == 0xffffffff ? {ParameterName}_InvalidRowReadOffset : RowOffset + ColumnOffset;
}

void ReadValue_{ParameterName}(uint RowOffset, uint Column, out bool   Value) {	Value = asint({ParameterName}_TableDataBuffer.Load(GetReadOffset_{ParameterName}(RowOffset, Column))) != 0; }
void ReadValue_{ParameterName}(uint RowOffset, uint Column, out int    Value) {	Value = asint({ParameterName}_TableDataBuffer.Load(GetReadOffset_{ParameterName}(RowOffset, Column))); }
void ReadValue_{ParameterName}(uint RowOffset, uint Column, out float  Value) { Value = asfloat({ParameterName}_TableDataBuffer.Load(GetReadOffset_{ParameterName}(RowOffset, Column))); }
void ReadValue_{ParameterName}(uint RowOffset, uint Column, out float2 Value) { Value = asfloat({ParameterName}_TableDataBuffer.Load2(GetReadOffset_{ParameterName}(RowOffset, Column))); }
void ReadValue_{ParameterName}(uint RowOffset, uint Column, out float3 Value) { Value = asfloat({ParameterName}_TableDataBuffer.Load3(GetReadOffset_{ParameterName}(RowOffset, Column))); }
void ReadValue_{ParameterName}(uint RowOffset, uint Column, out float4 Value) { Value = asfloat({ParameterName}_TableDataBuffer.Load4(GetReadOffset_{ParameterName}(RowOffset, Column))); }

uint GetFilteredRowReadOffset_{ParameterName}(int RowIndex, out bool bIsValid)
{
	bIsValid = RowIndex >= 0 && RowIndex < {ParameterName}_NumFilteredRows;

	RowIndex = bIsValid ? RowIndex : {ParameterName}_NumFilteredRows;
	return {ParameterName}_FilteredRowDataOffsetBuffer.Load(RowIndex);
}

uint GetRowReadOffset_{ParameterName}(int RowIndex, out bool bIsValid)
{
	bIsValid = RowIndex >= 0 && RowIndex < {ParameterName}_NumRows;
	return bIsValid ? uint(RowIndex) * {ParameterName}_RowStride : {ParameterName}_InvalidRowReadOffset;
}

void IsValid_{ParameterName}(out bool bIsValid)
{
	bIsValid = {ParameterName}_NumRows > 0;
}

void GetNumRows_{ParameterName}(out int OutNumRows)
{
	OutNumRows = {ParameterName}_NumRows;
}

void GetNumFilteredRows_{ParameterName}(out int OutNumRows)
{
	OutNumRows = {ParameterName}_NumFilteredRows;
}

// Example of generate output function
//void ReadFilteredRow_{ParameterName}(int RowIndex, out bool bIsValid, out int MyInt, out float MyFloat)//, ...)
//{
//	const uint RowReadOffset = GetRowReadOffset_{ParameterName}(RowIndex, bIsValid);
//	ReadValue_{ParameterName}(RowReadOffset, 0, MyInt);
//	ReadValue_{ParameterName}(RowReadOffset, 1, MyFloat);
//	ReadValue_{ParameterName}(RowReadOffset, 2, MyVector);
//}
