// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.ush"
#include "NiagaraQuaternionUtils.ush"
#include "NiagaraDebugDraw.ush"

RWBuffer<uint>		RWNDIDebugDrawArgs;
RWBuffer<uint>		RWNDIDebugDrawLineVertex;
uint				NDIDebugDrawLineMaxInstances;

void NDIDebugDraw_DrawBox(bool bDraw, float3 Location, float4 Rotation, float3 Extents, float4 Color)
{
	if ( bDraw && (NDIDebugDrawLineMaxInstances > 0) )
	{
		int NumLines = 12;

		uint InstanceIndex = 0;
		InterlockedAdd(RWNDIDebugDrawArgs[1], NumLines, InstanceIndex);
		if ( InstanceIndex + NumLines <= NDIDebugDrawLineMaxInstances )
		{
			float3x3 RotMatrix = NiagaraQuatTo3x3(Rotation);
			float3 Points[] =
			{
				Location + mul(float3( Extents.x,  Extents.y,  Extents.z), RotMatrix),
				Location + mul(float3(-Extents.x,  Extents.y,  Extents.z), RotMatrix),
				Location + mul(float3(-Extents.x, -Extents.y,  Extents.z), RotMatrix),
				Location + mul(float3( Extents.x, -Extents.y,  Extents.z), RotMatrix),
				Location + mul(float3( Extents.x,  Extents.y, -Extents.z), RotMatrix),
				Location + mul(float3(-Extents.x,  Extents.y, -Extents.z), RotMatrix),
				Location + mul(float3(-Extents.x, -Extents.y, -Extents.z), RotMatrix),
				Location + mul(float3( Extents.x, -Extents.y, -Extents.z), RotMatrix),
			};
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 0, Points[0], Points[1], Color);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 1, Points[1], Points[2], Color);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 2, Points[2], Points[3], Color);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 3, Points[3], Points[0], Color);

			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 4, Points[4], Points[5], Color);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 5, Points[5], Points[6], Color);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 6, Points[6], Points[7], Color);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 7, Points[7], Points[4], Color);

			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 8, Points[0], Points[4], Color);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 9, Points[1], Points[5], Color);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex +10, Points[2], Points[6], Color);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex +11, Points[3], Points[7], Color);
		}
		else
		{
			// Not enough space, reduce count back to original
			InterlockedAdd(RWNDIDebugDrawArgs[1], -NumLines);
		}
	}
}

void NDIDebugDraw_Circle(bool bDraw, float3 Location, float3 XAxis, float3 YAxis, float Scale, int Segments, float4 Color)
{
	if ( bDraw && (NDIDebugDrawLineMaxInstances > 0) )
	{
		Segments = clamp(Segments, 4, 16);

		uint InstanceIndex = 0;
		InterlockedAdd(RWNDIDebugDrawArgs[1], Segments, InstanceIndex);
		if ( InstanceIndex + Segments <= NDIDebugDrawLineMaxInstances )
		{
			float3 X = XAxis * Scale;
			float3 Y = YAxis * Scale;

			float d = 2.0f * PI / float(Segments);
			float u = 0.0f;

			float3 LastPoint = Location + (X * cos(u)) + (Y * sin(u));
			for ( int i=0; i < Segments; ++i )
			{
				u += d;
				float3 CurrPoint = Location + (X * cos(u)) + (Y * sin(u));
				PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + i, LastPoint, CurrPoint, Color);
				LastPoint = CurrPoint;
			}
		}
		else
		{
			// Not enough space, reduce count back to original
			InterlockedAdd(RWNDIDebugDrawArgs[1], -Segments);
		}
	}
}

void NDIDebugDraw_DrawRectangle(bool bDraw, float3 Location, float3 XAxis, float3 YAxis, float2 Extents, int SegmentsX, int SegmentsY, float4 Color, bool bUnbounded)
{
	if ( bDraw && (NDIDebugDrawLineMaxInstances > 0) )
	{		
		int LowerBound = bUnbounded ? 1 : 0;
		int UpperBoundX = bUnbounded ? max(SegmentsX - 1, 1) : SegmentsX;
		int UpperBoundY = bUnbounded ? max(SegmentsY - 1, 1) : SegmentsY;

		int TotalSegments = (UpperBoundX - LowerBound + 1) + (UpperBoundY - LowerBound + 1);

		uint InstanceIndex = 0;
		InterlockedAdd(RWNDIDebugDrawArgs[1], TotalSegments, InstanceIndex);
		if (InstanceIndex + TotalSegments <= NDIDebugDrawLineMaxInstances)
		{
			float3 XAxisNorm = normalize(XAxis);
			float3 YAxisNorm = normalize(YAxis);

			float3 StartPosition = Location - ((XAxisNorm * Extents.x) + (YAxisNorm * Extents.y));
			float2 Step = (Extents * 2) / float2(SegmentsX, SegmentsY);
			float3 StepX = XAxisNorm * Step.x;
			float3 StepY = YAxisNorm * Step.y;

			// Add all Y axis lines
			for (int X = LowerBound; X <= UpperBoundX; X++)
			{
				float3 LineStart = StartPosition + (StepX * X);
				float3 LineEnd = LineStart + (StepY * SegmentsY);

				PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, LineStart, LineEnd, Color);
			}

			// Add all X axis lines
			for (int Y = LowerBound; Y <= UpperBoundY; Y++)
			{				
				float3 LineStart = StartPosition + (StepY * Y);
				float3 LineEnd = LineStart + (StepX * SegmentsX);

				PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, LineStart, LineEnd, Color);
			}
		}
		else
		{
			// Not enough space, reduce count back to original
			InterlockedAdd(RWNDIDebugDrawArgs[1], -TotalSegments);
		}
	}
}

void NDIDebugDraw_CoordinateSystem(bool bDraw, float3 Location, float4 Rotation, float Scale)
{
	if ( bDraw && (NDIDebugDrawLineMaxInstances > 0) )
	{
		int NumLines = 3;

		uint InstanceIndex = 0;
		InterlockedAdd(RWNDIDebugDrawArgs[1], NumLines, InstanceIndex);
		if ( InstanceIndex + NumLines <= NDIDebugDrawLineMaxInstances )
		{
			float3x3 RotMatrix = NiagaraQuatTo3x3(Rotation);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 0, Location, Location + (RotMatrix[0] * Scale), 0xff0000ff);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 1, Location, Location + (RotMatrix[1] * Scale), 0x00ff00ff);
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex + 2, Location, Location + (RotMatrix[2] * Scale), 0x0000ffff);
		}
		else
		{
			// Not enough space, reduce count back to original
			InterlockedAdd(RWNDIDebugDrawArgs[1], -NumLines);
		}
	}
}

void NDIDebugDraw_Grid2D(bool bDraw, float3 Center, float4 Rotation, float2 Extents, int2 NumCells, float4 Color)
{
	if ( bDraw && (NDIDebugDrawLineMaxInstances > 0) && (NumCells.x > 0) && (NumCells.y > 0) )
	{
		int NumLines = (NumCells.x + 1) * (NumCells.y + 1) * 2;

		uint InstanceIndex = 0;
		InterlockedAdd(RWNDIDebugDrawArgs[1], NumLines, InstanceIndex);
		if ( InstanceIndex + NumLines <= NDIDebugDrawLineMaxInstances )
		{
			float3x3 RotMatrix = NiagaraQuatTo3x3(Rotation);
			float3 Corner = Center - mul(float3(Extents, 0), RotMatrix);
			float3 XLength = mul(float3(Extents.x * 2.0f, 0.0f, 0.0f), RotMatrix);
			float3 YLength = mul(float3(0.0f, Extents.y * 2.0f, 0.0f), RotMatrix);
			float3 XDelta = XLength / float(NumCells.x);
			float3 YDelta = YLength / float(NumCells.y);

			for (int X=0; X <= NumCells.x; ++X)
			{
				float3 XOffset = XDelta * float(X);
				for (int Y=0; Y <= NumCells.y; ++Y)
				{
					float3 YOffset = YDelta * float(Y);
					PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, Corner + XOffset, Corner + XOffset + YLength, Color);
					PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, Corner + YOffset, Corner + YOffset + XLength, Color);
				}
			}
		}
		else
		{
			// Not enough space, reduce count back to original
			InterlockedAdd(RWNDIDebugDrawArgs[1], -NumLines);
		}
	}
}

void NDIDebugDraw_Grid3D(bool bDraw, float3 Center, float4 Rotation, float3 Extents, int3 NumCells, float4 Color)
{
	if ( bDraw && (NDIDebugDrawLineMaxInstances > 0) && (NumCells.x > 0) && (NumCells.y > 0) && (NumCells.z > 0) )
	{
		int NumLines = (NumCells.x + 1) * (NumCells.y + 1) * (NumCells.z + 1) * 4;

		uint InstanceIndex = 0;
		InterlockedAdd(RWNDIDebugDrawArgs[1], NumLines, InstanceIndex);
		if ( InstanceIndex + NumLines <= NDIDebugDrawLineMaxInstances )
		{
			float3x3 RotMatrix = NiagaraQuatTo3x3(Rotation);

			float3 Corner = Center - mul(Extents, RotMatrix);
			float3 XLength = mul(float3(Extents.x * 2.0f, 0.0f, 0.0f), RotMatrix);
			float3 YLength = mul(float3(0.0f, Extents.y * 2.0f, 0.0f), RotMatrix);
			float3 ZLength = mul(float3(0.0f, 0.0f, Extents.z * 2.0f), RotMatrix);
			float3 XDelta = XLength / float(NumCells.x);
			float3 YDelta = YLength / float(NumCells.y);
			float3 ZDelta = ZLength / float(NumCells.z);

			for (int X = 0; X <= NumCells.x; ++X)
			{
				float3 XOffset = XDelta * float(X);
				for (int Y = 0; Y <= NumCells.y; ++Y)
				{
					float3 YOffset = YDelta * float(Y);
					for (int Z = 0; Z <= NumCells.z; ++Z)
					{
						float3 ZOffset = ZDelta * float(Z);

						PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, Corner + ZOffset + XOffset, Corner + ZOffset + XOffset + YLength, Color);		// Z Slice: X -> Y
						PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, Corner + ZOffset + YOffset, Corner + ZOffset + YOffset + XLength, Color);		// Z Slice: Y -> X

						PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, Corner + XOffset + YOffset, Corner + XOffset + YOffset + ZLength, Color);		// X Slice: Y -> Z
						PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, Corner + XOffset + ZOffset, Corner + XOffset + ZOffset + YLength, Color);		// X Slice: Z -> Y
					}
				}
			}
		}
		else
		{
			// Not enough space, reduce count back to original
			InterlockedAdd(RWNDIDebugDrawArgs[1], -NumLines);
		}
	}
}

void NDIDebugDraw_Line(bool bDraw, float3 LineStart, float3 LineEnd, float4 Color)
{
	if ( bDraw && (NDIDebugDrawLineMaxInstances > 0) )
	{
		uint InstanceIndex = 0;
		InterlockedAdd(RWNDIDebugDrawArgs[1], 1, InstanceIndex);
		if ( InstanceIndex < NDIDebugDrawLineMaxInstances )
		{
			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex, LineStart, LineEnd, Color);
		}
		else
		{
			// Not enough space, reduce count back to original
			InterlockedAdd(RWNDIDebugDrawArgs[1], -1);
		}
	}
}

void NDIDebugDraw_Sphere(bool bDraw, float3 Location, float Radius, int Segments, float4 Color)
{
	if ( bDraw && (NDIDebugDrawLineMaxInstances > 0) )
	{
		Segments = clamp(Segments, 4, 16);
		int NumLines = Segments * Segments * 2;

		uint InstanceIndex = 0;
		InterlockedAdd(RWNDIDebugDrawArgs[1], NumLines, InstanceIndex);
		if ( InstanceIndex + NumLines <= NDIDebugDrawLineMaxInstances )
		{
			const float uinc = 2.0f * PI / float(Segments);

			float ux = 0.0f;
			float SinX0 = sin(ux);
			float CosX0 = cos(ux);
			for (int x=0; x < Segments; ++x)
			{
				ux += uinc;
				float SinX1 = sin(ux);
				float CosX1 = cos(ux);

				float uy = 0.0f;
				float SinY0 = sin(uy);
				float CosY0 = cos(uy);
				for (int y=0; y < Segments; ++y)
				{
					uy += uinc;
					float SinY1 = sin(uy);
					float CosY1 = cos(uy);

					float3 Point0 = Location + float3(CosX0 * CosY0, SinY0, SinX0 * CosY0) * Radius;
					float3 Point1 = Location + float3(CosX1 * CosY0, SinY0, SinX1 * CosY0) * Radius;
					float3 Point2 = Location + float3(CosX0 * CosY1, SinY1, SinX0 * CosY1) * Radius;
					PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, Point0, Point1, Color);
					PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, Point0, Point2, Color);

					SinY0 = SinY1;
					CosY0 = CosY1;
				}

				SinX0 = SinX1;
				CosX0 = CosX1;
			}

		}
		else
		{
			// Not enough space, reduce count back to original
			InterlockedAdd(RWNDIDebugDrawArgs[1], -NumLines);
		}
	}
}


void NDIDebugDraw_Cylinder(bool bDraw, float3 Location, float3 Axis, float Height, float Radius, int NumHeightSegments, int NumRadiusSegments, float4 Color)
{
	if ( bDraw && (NDIDebugDrawLineMaxInstances > 0) )
	{
		NumHeightSegments = clamp(NumHeightSegments, 1, 16);
		NumRadiusSegments = clamp(NumRadiusSegments, 3, 16);

		int NumLines = NumRadiusSegments + (NumRadiusSegments * NumHeightSegments + 1);

		uint InstanceIndex = 0;
		InterlockedAdd(RWNDIDebugDrawArgs[1], NumLines, InstanceIndex);
		if ( InstanceIndex + NumLines <= NDIDebugDrawLineMaxInstances )
		{
			float3 AxisNorm = normalize(Axis);

			const float3x3 AxisRotation = NiagaraFindRotationBetweenNormals(float3(0,0,1), AxisNorm);

			float3 TangentX = mul(AxisRotation, float3(1, 0, 0));
			float3 TangentY = mul(AxisRotation, float3(0, 1, 0));

			float3 TangentXScaled = TangentX * Radius;
			float3 TangentYScaled = TangentY * Radius;

			float3 HalfHeightOffset = AxisNorm * Height * 0.5f;
			float3 AxisStep = AxisNorm * Height / NumHeightSegments;

			float CurrentRotation = 0.0f;
			float3 LastPoint = Location + (TangentXScaled * cos(CurrentRotation)) + (TangentYScaled * sin(CurrentRotation));

			for (int Idx = 1; Idx <= NumRadiusSegments; Idx++)
			{
				CurrentRotation = 2.0f * PI / NumRadiusSegments * Idx;
				float3 CurrentPoint = Location + (TangentXScaled * cos(CurrentRotation)) + (TangentYScaled * sin(CurrentRotation));
				
				float3 LastBottom = LastPoint - HalfHeightOffset;
				float3 LastTop = LastPoint + HalfHeightOffset;

				float3 CurrentBottom = CurrentPoint - HalfHeightOffset;
				float3 CurrentTop = CurrentPoint + HalfHeightOffset;
				
				// Add Height Line
				PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, CurrentBottom, CurrentTop, Color);

				// Add Bottom
				PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, LastBottom, CurrentBottom, Color);
				
				// Add all Rings + top
				for (int HeightIdx = 1; HeightIdx <= NumHeightSegments; HeightIdx++)
				{
					PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, LastBottom + AxisStep * HeightIdx, CurrentBottom + AxisStep * HeightIdx, Color);
				}

				LastPoint = CurrentPoint;
			}
		}
		else
		{
			// Not enough space, reduce count back to original
			InterlockedAdd(RWNDIDebugDrawArgs[1], -NumLines);
		}
	}
}

void NDIDebugDraw_Cone(bool bDraw, float3 Location, float3 Axis, float Height, float RadiusTop, float RadiusBottom, int NumHeightSegments, int NumRadiusSegments, float4 Color)
{
    if (!bDraw || (NDIDebugDrawLineMaxInstances <= 0))
    {
        return;
    }

    NumHeightSegments = clamp(NumHeightSegments, 2, 16);
    NumRadiusSegments = clamp(NumRadiusSegments, 3, 16);
    const int NumLines = NumRadiusSegments + (NumRadiusSegments * NumHeightSegments);
 
	uint InstanceIndex = 0;
	InterlockedAdd(RWNDIDebugDrawArgs[1], NumLines, InstanceIndex);
	if (InstanceIndex + NumLines > NDIDebugDrawLineMaxInstances)
	{
		InterlockedAdd(RWNDIDebugDrawArgs[1], -NumLines);
		return;
	}
    
    const float3 AxisNorm = normalize(Axis);
	const float4 Rotation = FindQuatBetweenNormals(float3(0, 0, 1), AxisNorm);
    
	const float HeightOffset = Height * 0.5f;
	const float HeightStep = Height / float(NumHeightSegments - 1);

	const float HeightSegmentStepAlpha = 1.0f / float(NumHeightSegments - 1);

	float CurrentRotation = 0.0f;
	float3 LastPointNormal = float3(sin(CurrentRotation), cos(CurrentRotation), 0.0f);
	
	for (int Idx = 1; Idx <= NumRadiusSegments; Idx++)
	{
		CurrentRotation = 2.0f * PI / float(NumRadiusSegments) * Idx;

		const float3 CurrentPointNormal = float3(sin(CurrentRotation), cos(CurrentRotation), 0.0f);
		const float3 CurrentBottom = RotateVectorByQuat(CurrentPointNormal * RadiusBottom + float3(0.0f, 0.0f, -HeightOffset), Rotation);
		const float3 CurrentTop = RotateVectorByQuat(CurrentPointNormal * RadiusTop + float3(0.0f, 0.0f, Height - HeightOffset), Rotation);

        PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, Location + CurrentBottom, Location + CurrentTop, Color);
        
		for (int HeightIdx=0; HeightIdx < NumHeightSegments; ++HeightIdx)
		{
			const float RingRadius = lerp(RadiusBottom, RadiusTop, HeightSegmentStepAlpha * float(HeightIdx));
			const float3 RingOffset = float3(0.0f, 0.0f, (HeightStep * float(HeightIdx)) - HeightOffset);

			const float3 LastRingPosition = RotateVectorByQuat(LastPointNormal * RingRadius + RingOffset, Rotation);
			const float3 CurrentRingPosition = RotateVectorByQuat(CurrentPointNormal * RingRadius + RingOffset, Rotation);

			PackLine(RWNDIDebugDrawLineVertex, InstanceIndex++, Location + LastRingPosition, Location + CurrentRingPosition, Color);
		}

		LastPointNormal = CurrentPointNormal;
	}
}

void NDIDebugDraw_Torus(bool bDraw, float3 Location, float3 Axis, float MajorRadius, float MinorRadius, float MajorRadiusSegments, float MinorRadiusSegments, float4 Color)
{
	if (!bDraw || (NDIDebugDrawLineMaxInstances <= 0))
	{
		return;
	}

	MinorRadiusSegments = clamp(MinorRadiusSegments, 3, 32);
	MajorRadiusSegments = clamp(MajorRadiusSegments, 3, 32);

	const int NumLines = MajorRadiusSegments * MinorRadiusSegments * 2;
 
	uint InstanceIndex = 0;
	InterlockedAdd(RWNDIDebugDrawArgs[1], NumLines, InstanceIndex);
	if (InstanceIndex + NumLines > NDIDebugDrawLineMaxInstances)
	{
		InterlockedAdd(RWNDIDebugDrawArgs[1], -NumLines);
		return;
	}
    
	const float3 AxisNorm = normalize(Axis);
	const float4 Rotation = FindQuatBetweenNormals(float3(0, 0, 1), AxisNorm);

	const float3 TangentX = RotateVectorByQuat(float3(1, 0, 0), Rotation);
	const float3 TangentY = RotateVectorByQuat(float3(0, 1, 0), Rotation);

	const float MajorRotationDelta = 2.0f * PI / float(MajorRadiusSegments);
	const float MinorRotationDelta = 2.0f * PI / float(MinorRadiusSegments);

	float CurrentMajorRotation = 0.0f;
	float3 PreviousMajorNormal = (TangentX * cos(CurrentMajorRotation)) + (TangentY * sin(CurrentMajorRotation));
	
	for (int MajorIdx = 0; MajorIdx < MajorRadiusSegments; ++MajorIdx)
	{
		CurrentMajorRotation += MajorRotationDelta;
		const float3 CurrentMajorNormal = (TangentX * cos(CurrentMajorRotation)) + (TangentY * sin(CurrentMajorRotation));

		float CurrentMinorRotation = 0.0f;
		float3 PreviousMinorNormal = (CurrentMajorNormal * cos(CurrentMinorRotation)) + (AxisNorm * sin(CurrentMinorRotation));

		for (int MinorIdx = 0; MinorIdx < MinorRadiusSegments; ++MinorIdx)
		{
			CurrentMinorRotation += MinorRotationDelta;
			const float3 CurrentMinorNormal = (CurrentMajorNormal * cos(CurrentMinorRotation)) + (AxisNorm * sin(CurrentMinorRotation));

			PackLine(
				RWNDIDebugDrawLineVertex, InstanceIndex++,
				Location + CurrentMajorNormal * MajorRadius + PreviousMinorNormal * MinorRadius,
				Location + CurrentMajorNormal * MajorRadius + CurrentMinorNormal * MinorRadius,
				Color
			);

			const float3 PreviousMajorMinorNormal = (PreviousMajorNormal * cos(CurrentMinorRotation)) + (AxisNorm * sin(CurrentMinorRotation));
			PackLine(
				RWNDIDebugDrawLineVertex, InstanceIndex++,
				Location + PreviousMajorNormal * MajorRadius + PreviousMajorMinorNormal * MinorRadius,
				Location + CurrentMajorNormal * MajorRadius + CurrentMinorNormal * MinorRadius,
				Color
			);

			PreviousMinorNormal = CurrentMinorNormal;
		}

		PreviousMajorNormal = CurrentMajorNormal;
	}
}


#define MakeDrawBoxPersistent(NAME)					void NAME(float3 Center, int CenterCoordinateSpace, float3 Offset, int OffsetCoordinateSpace, float3 Extents, bool HalfExtents, float3 RotationAxis, float RotationNormalizedAngle, int RotationCoordinateSpace, float4 Color) {}
#define MakeDrawCirclePersistent(NAME)				void NAME(float3 Center, int CenterCoordinateSpace, float3 Offset, int OffsetCoordinateSpace, float Radius, float3 XAxis, float3 YAxis, int AxisCoordinateSpace, int NumSegments, float4 Color) {}
#define MakeDrawConePersistent(NAME)				void NAME(float3 Center, int CenterCoordinateSpace, float3 Offset, int OffsetCoordinateSpace, float Angle, float Length, float3 OrientationAxis, int OrientationAxisCoordinateSpace, float3 NonUniformScale, int NumRadiusSegments, int NumHeightSegments, float4 Color) {}
#define MakeDrawCoordinateSystemPersistent(NAME)	void NAME(float3 Center, int CenterCoordinateSpace, float3 OffsetFromCenter, int OffsetCoordinateSpace, float4 Rotation, int RotationCoordinateSpace, float Scale) {}
#define MakeDrawCylinderPersistent(NAME)			void NAME(float3 Center, int CenterCoordinateSpace, float3 Offset, int OffsetCoordinateSpace, float Radius, float Height, bool IsHalfHeight, bool HemisphereX, bool HemisphereY, int OrientationAxis, int OrientationAxisCoordinateSpace, float3 NonUniformScale, int NumRadiusSegments, int NumHeightSegments, float4 Color) {}
#define MakeDrawGrid2DPersistent(NAME)				void NAME(float3 Center, int CenterCoordinateSpace, float3 OffsetFromCenter, int OffsetCoordinateSpace, float4 Rotation, int RotationCoordinateSpace, float2 Extents, int NumCellsX, int NumCellsY, float4 Color) {}
#define MakeDrawGrid3DPersistent(NAME)				void NAME(float3 Center, int CenterCoordinateSpace, float3 OffsetFromCenter, int OffsetCoordinateSpace, float4 Rotation, int RotationCoordinateSpace, float3 Extents, int NumCellsX, int NumCellsY, int NumCellsZ, float4 Color) {}
#define MakeDrawLinePersistent(NAME)				void NAME(float3 StartLocation, int StartLocationCoordinateSpace, float3 EndLocation, int EndLocationCoordinateSpace, float4 Color) {}
#define MakeDrawRectanglePersistent(NAME)			void NAME(float3 Center, int CenterCoordinateSpace, float3 Offset, int OffsetCoordinateSpace, float2 Extents, bool HalfExtents, float3 XAxis, float3 YAxis, int AxisVectorsCoordinateSpace, int NumXSegments, int NumYSegments, bool UnboundedPlane, float4 Color) {}
#define MakeDrawSpherePersistent(NAME)				void NAME(float3 Center, int CenterCoordinateSpace, float3 OffsetFromCenter, int OffsetCoordinateSpace, float Radius, int RadiusCoordinateSpace, bool HemisphereX, bool HemisphereY, bool HemisphereZ, float3 SphereOrientationAxis, int RotationCoordinateSpace, float4 AdditionalRotation, float3 NonUniformScale, int NumSegments, float4 Color) {}
#define MakeDrawTorusPersistent(NAME)				void NAME(float3 Center, int CenterCoordinateSpace, float3 OffsetFromCenter, int OffsetCoordinateSpace, float MajorRadius, float MinorRadius, float3 OrientationAxis, int OrientationAxisCoordinateSpace, float3 NonUniformScale, int MajorRadiusSegments, int MinorRadiusSegments, float4 Color) {}
