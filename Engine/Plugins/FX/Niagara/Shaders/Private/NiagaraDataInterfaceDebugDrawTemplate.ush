// Copyright Epic Games, Inc. All Rights Reserved.

void DrawLine_{ParameterName}_UEImpureCall(bool bExecute, float3 LineStart, float3 LineEnd, float4 Color)
{
	NDIDebugDraw_Line(bExecute, LineStart, LineEnd, Color);
}

void DrawRectangle_{ParameterName}_UEImpureCall(bool bExecute, float3 Location, float3 XAxis, float3 YAxis, float2 Extents, int NumXSegments, int NumYSegments, float4 Color, bool bUnbounded)
{
	NDIDebugDraw_DrawRectangle(bExecute, Location, XAxis, YAxis, Extents, NumXSegments, NumYSegments, Color, bUnbounded);
}


void DrawCircle_{ParameterName}_UEImpureCall(bool bExecute, float3 Location, float3 XAxis, float3 YAxis, float Scale, int Segments, float4 Color)
{
	NDIDebugDraw_Circle(bExecute, Location, XAxis, YAxis, Scale, Segments, Color);
}

void DrawBox_{ParameterName}_UEImpureCall(bool bExecute, float3 Location, float4 Rotation, float3 Extents, float4 Color)
{
	NDIDebugDraw_DrawBox(bExecute, Location, Rotation, Extents, Color);
}

void DrawSphere_{ParameterName}_UEImpureCall(bool bExecute, float3 LineStart, float Radius, int Segments, float4 Color)
{
	NDIDebugDraw_Sphere(bExecute, LineStart, Radius, Segments, Color);
}

void DrawCylinder_{ParameterName}_UEImpureCall(bool bExecute, float3 Location, float3 Axis, float Height, float Radius, int NumHeightSegments, int NumRadiusSegments, float4 Color)
{
	NDIDebugDraw_Cylinder(bExecute, Location, Axis, Height, Radius, NumHeightSegments, NumRadiusSegments, Color);
}

void DrawCone_{ParameterName}_UEImpureCall(bool bExecute, float3 Location, float3 Axis, float Height, float RadiusTop, float RadiusBottom, int NumHeightSegments, int NumRadiusSegments, float4 Color)
{
	NDIDebugDraw_Cone(bExecute, Location, Axis, Height, RadiusTop, RadiusBottom, NumHeightSegments, NumRadiusSegments, Color);
}

void DrawTorus_{ParameterName}_UEImpureCall(bool bExecute, float3 Location, float3 Axis, float MajorRadius, float MinorRadius, int MajorRadiusSegments, int MinorRadiusSegments, float4 Color)
{
	NDIDebugDraw_Torus(bExecute, Location, Axis, MajorRadius, MinorRadius, MajorRadiusSegments, MinorRadiusSegments, Color);
}

void DrawCoordinateSystem_{ParameterName}_UEImpureCall(bool bExecute, float3 Location, float4 Rotation, float Scale)
{
	NDIDebugDraw_CoordinateSystem(bExecute, Location, Rotation, Scale);
}

void DrawGrid2D_{ParameterName}_UEImpureCall(bool bExecute, float3 Center, float4 Rotation, float2 Extents, int NumCellsX, int NumCellsY, float4 Color)
{
	NDIDebugDraw_Grid2D(bExecute, Center, Rotation, Extents, int2(NumCellsX, NumCellsY), Color);
}

void DrawGrid3D_{ParameterName}_UEImpureCall(bool bExecute, float3 Center, float4 Rotation, float3 Extents, int NumCellsX, int NumCellsY, int NumCellsZ, float4 Color)
{
	NDIDebugDraw_Grid3D(bExecute, Center, Rotation, Extents, int3(NumCellsX, NumCellsY, NumCellsZ), Color);
}
