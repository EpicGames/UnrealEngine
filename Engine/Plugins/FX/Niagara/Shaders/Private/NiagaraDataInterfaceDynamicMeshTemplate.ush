// Copyright Epic Games, Inc. All Rights Reserved.

uint	{ParameterName}_NumSections;
uint	{ParameterName}_NumTriangles;
uint	{ParameterName}_NumVertices;
uint	{ParameterName}_NumTexCoords;

uint	{ParameterName}_PositionOffset;
uint	{ParameterName}_TangentBasisOffset;
uint	{ParameterName}_TexCoordOffset;
uint	{ParameterName}_ColorOffset;

RWBuffer<uint>	{ParameterName}_SectionBuffer;
RWBuffer<uint>	{ParameterName}_IndexBuffer;
RWBuffer<uint>	{ParameterName}_VertexBuffer;

////////////////////////////////////////////////////////
// Helper functions
uint {ParameterName}_GetPositionOffsetUnsafe(int Vertex)				{ return (Vertex * 3) + {ParameterName}_PositionOffset; }
uint {ParameterName}_GetTangentBasisOffsetUnsafe(int Vertex)			{ return (Vertex * 2) + {ParameterName}_TangentBasisOffset; }
uint {ParameterName}_GetTexCoordOffsetUnsafe(int Vertex, int TexCoord)	{ return (Vertex * 2) + (TexCoord * 2) + {ParameterName}_TexCoordOffset; }
uint {ParameterName}_GetColorOffsetUnsafe(int Vertex)					{ return Vertex + {ParameterName}_ColorOffset;}

uint {ParameterName}_GetPositionOffset(int Vertex)
{
	return ({ParameterName}_PositionOffset != -1) && (Vertex >= 0) && (Vertex < {ParameterName}_NumVertices) ? {ParameterName}_GetPositionOffsetUnsafe(Vertex) : -1;
}

uint {ParameterName}_GetTangentBasisOffset(int Vertex)
{
	return ({ParameterName}_TangentBasisOffset != -1) && (Vertex >= 0) && (Vertex < {ParameterName}_NumVertices) ? {ParameterName}_GetTangentBasisOffsetUnsafe(Vertex) : -1;
}

uint {ParameterName}_GetTexCoordOffset(int Vertex, int TexCoord)
{
	return ({ParameterName}_TexCoordOffset != -1) && (Vertex >= 0) && (Vertex < {ParameterName}_NumVertices) && (TexCoord >= 0) && (TexCoord <= {ParameterName}_NumTexCoords) ? {ParameterName}_GetTexCoordOffsetUnsafe(Vertex, TexCoord) : -1;
}

uint {ParameterName}_GetColorOffset(int Vertex)
{
	return ({ParameterName}_ColorOffset != -1) && (Vertex >= 0) && (Vertex < {ParameterName}_NumVertices) ? {ParameterName}_GetColorOffsetUnsafe(Vertex) : -1;
}

float4 {ParameterName}_UnpackTangent(uint Packed)
{
	float4 Unpacked;
	Unpacked.x = (Packed >> 0) & 0xff;
	Unpacked.y = (Packed >> 8) & 0xff;
	Unpacked.z = (Packed >> 16) & 0xff;
	Unpacked.w = (Packed >> 24) & 0xff;
	return Unpacked / 127.5f - 1;
}

uint {ParameterName}_PackTangent(float3 Tangent, uint TangentSign)
{
	uint Packed;
	Packed  = (int(Tangent.x * 127.499f) & 0xff) << 0;
	Packed |= (int(Tangent.y * 127.499f) & 0xff) << 8;
	Packed |= (int(Tangent.z * 127.499f) & 0xff) << 16;
	Packed |= TangentSign << 24;
	return Packed;
}

float4 {ParameterName}_UnpackColor(uint Packed)
{
	float4 Unpacked;
	Unpacked.x = (Packed >> 0) & 0xff;
	Unpacked.y = (Packed >> 8) & 0xff;
	Unpacked.z = (Packed >> 16) & 0xff;
	Unpacked.w = (Packed >> 24) & 0xff;
	Unpacked = Unpacked FMANUALFETCH_COLOR_COMPONENT_SWIZZLE;
	return Unpacked / 255.0f;
}

uint {ParameterName}_PackColor(float4 Color)
{
	Color = Color FMANUALFETCH_COLOR_COMPONENT_SWIZZLE;
	Color = clamp((Color * 255.0f) + 0.5f, 0.0f, 255.0f);

	uint Packed;
	Packed  = uint(Color.x) << 0;
	Packed |= uint(Color.y) << 8;
	Packed |= uint(Color.z) << 16;
	Packed |= uint(Color.w) << 24;
	return Packed;
}

void {ParameterName}_SetVertexPositionUnsafe(int Vertex, float3 Position)
{
	uint Offset = {ParameterName}_GetPositionOffsetUnsafe(Vertex);
	{ParameterName}_VertexBuffer[Offset + 0] = asuint(Position.x);
	{ParameterName}_VertexBuffer[Offset + 1] = asuint(Position.y);
	{ParameterName}_VertexBuffer[Offset + 2] = asuint(Position.z);
}

void {ParameterName}_SetVertexTangentBasisUnsafe(int Vertex, float3 TangentX, float3 TangentY, float3 TangentZ)
{
	uint Offset = {ParameterName}_GetTangentBasisOffsetUnsafe(Vertex);
	uint TangentSign = dot(cross(TangentX, TangentZ), TangentY) < 0 ? 0x80 : 0x7f;
	{ParameterName}_VertexBuffer[Offset + 0] = {ParameterName}_PackTangent(TangentX, TangentSign);
	{ParameterName}_VertexBuffer[Offset + 1] = {ParameterName}_PackTangent(TangentZ, TangentSign);
}

void {ParameterName}_SetVertexTexCoordUnsafe(int Vertex, float2 TexCoord, int TexCoordIndex)
{
	uint Offset = {ParameterName}_GetTexCoordOffsetUnsafe(Vertex, TexCoordIndex);
	{ParameterName}_VertexBuffer[Offset + 0] = asuint(TexCoord.x);
	{ParameterName}_VertexBuffer[Offset + 1] = asuint(TexCoord.y);
}

void {ParameterName}_SetVertexColorUnsafe(int Vertex, float4 Color)
{
	uint Offset = {ParameterName}_GetColorOffsetUnsafe(Vertex);
	{ParameterName}_VertexBuffer[Offset] = {ParameterName}_PackColor(Color);
}

////////////////////////////////////////////////////////
// Mutable Functions
void AllocateSectionTriangles_{ParameterName}_UEImpureCall(bool bExecute, int SectionIndex, int NumTriangles, out int TriangleIndex, out int NumAllocated)
{
	TriangleIndex = -1;
	NumAllocated = 0;
	if (bExecute && (SectionIndex >= 0 && SectionIndex < {ParameterName}_NumSections) && NumTriangles > 0 )
	{
		int MaxTriangles = {ParameterName}_SectionBuffer[SectionIndex * 2 + 0];
		InterlockedAdd({ParameterName}_SectionBuffer[SectionIndex * 2 + 1], NumTriangles, TriangleIndex);

		NumAllocated = min(MaxTriangles - TriangleIndex, NumTriangles);
		if ( NumAllocated < NumTriangles )
		{
			int Correction = min(NumTriangles - NumAllocated, NumTriangles);
			InterlockedAdd({ParameterName}_SectionBuffer[SectionIndex * 2 + 1], -Correction);

			if ( NumAllocated == 0 )
			{
				TriangleIndex = -1;
				NumAllocated = 0;
			}
		}
	}
}

void SetTriangleVertices_{ParameterName}_UEImpureCall(in bool bExecute, in int TriangleIndex, in int Index0, in int Index1, in int Index2)
{
	if ( bExecute && TriangleIndex >= 0 && TriangleIndex < {ParameterName}_NumTriangles )
	{
		//bSuccess = true;
		{ParameterName}_IndexBuffer[TriangleIndex * 3 + 0] = (uint)Index0;
		{ParameterName}_IndexBuffer[TriangleIndex * 3 + 1] = (uint)Index1;
		{ParameterName}_IndexBuffer[TriangleIndex * 3 + 2] = (uint)Index2;
	}
	else
	{
		//bSuccess = false;
	}
}

void SetVertexPosition_{ParameterName}_UEImpureCall(in bool bExecute, in int Vertex, in float3 Position)
{
	uint Offset = {ParameterName}_GetPositionOffset(Vertex);
	if ( bExecute && Offset != -1 )
	{
		//bSuccess = true;
		{ParameterName}_VertexBuffer[Offset + 0] = asuint(Position.x);
		{ParameterName}_VertexBuffer[Offset + 1] = asuint(Position.y);
		{ParameterName}_VertexBuffer[Offset + 2] = asuint(Position.z);
	}
	else
	{
		//bSuccess = false;
	}
}

void SetVertexTangentBasis_{ParameterName}_UEImpureCall(in bool bExecute, in int Vertex, in float3 TangentX, in float3 TangentY, in float3 TangentZ)
{
	uint Offset = {ParameterName}_GetTangentBasisOffset(Vertex);
	if ( bExecute && Offset != -1 )
	{
		//bSuccess = true;
		uint TangentSign = dot(cross(TangentX, TangentZ), TangentY) < 0 ? 0x80 : 0x7f;
		{ParameterName}_VertexBuffer[Offset + 0] = {ParameterName}_PackTangent(TangentX, TangentSign);
		{ParameterName}_VertexBuffer[Offset + 1] = {ParameterName}_PackTangent(TangentZ, TangentSign);
	}
	else
	{
		//bSuccess = false;
	}
}

void SetVertexTexCoord_{ParameterName}_UEImpureCall(in bool bExecute, in int Vertex, in int TexCoordIndex, in float2 TexCoord)
{
	uint Offset = {ParameterName}_GetTexCoordOffset(Vertex, TexCoordIndex);
	if ( bExecute && Offset != -1 )
	{
		//bSuccess = true;
		{ParameterName}_VertexBuffer[Offset + 0] = asuint(TexCoord.x);
		{ParameterName}_VertexBuffer[Offset + 1] = asuint(TexCoord.y);
	}
	else
	{
		//bSuccess = false;
	}
}

void SetVertexColor_{ParameterName}_UEImpureCall(in bool bExecute, in int Vertex, in float4 Color)
{
	uint Offset = {ParameterName}_GetColorOffset(Vertex);
	if ( bExecute && Offset != -1 )
	{
		//bSuccess = true;
		{ParameterName}_VertexBuffer[Offset] = {ParameterName}_PackColor(Color);
	}
	else
	{
		//bSuccess = false;
	}
}

void SetVertexData_{ParameterName}_UEImpureCall(in bool bExecute, in int Vertex, in float3 Position, in float3 TangentX, in float3 TangentY, in float3 TangentZ, in float2 TexCoord, in float4 Color)
{
	if ( bExecute && Vertex >= 0 && Vertex < {ParameterName}_NumVertices )
	{
		//bSuccess = true;
		//bool bDummy;
		SetVertexPosition_{ParameterName}_UEImpureCall(true, Vertex, Position);//, bDummy);
		SetVertexTangentBasis_{ParameterName}_UEImpureCall(true, Vertex, TangentX, TangentY, TangentZ);//, bDummy);
		SetVertexTexCoord_{ParameterName}_UEImpureCall(true, Vertex, 0, TexCoord);//, bDummy);
		SetVertexColor_{ParameterName}_UEImpureCall(true, Vertex, Color);//, bDummy);
	}
	else
	{
		//bSuccess = false;
	}
}

void AppendTriangle_{ParameterName}_UEImpureCall(
	in bool bExecute, in int SectionIndex,
	in float3 Position0, in float3 TangentX0, in float3 TangentY0, in float3 TangentZ0, in float2 TexCoord0, in float4 Color0,
	in float3 Position1, in float3 TangentX1, in float3 TangentY1, in float3 TangentZ1, in float2 TexCoord1, in float4 Color1,
	in float3 Position2, in float3 TangentX2, in float3 TangentY2, in float3 TangentZ2, in float2 TexCoord2, in float4 Color2,
	out int TriangleIndex
)
{
	TriangleIndex = -1;
	if (bExecute && (SectionIndex >= 0 && SectionIndex < {ParameterName}_NumSections))
	{
		int MaxTriangles = {ParameterName}_SectionBuffer[SectionIndex * 2 + 0];
		InterlockedAdd({ParameterName}_SectionBuffer[SectionIndex * 2 + 1], 1, TriangleIndex);
		if (TriangleIndex >= MaxTriangles)
		{
			InterlockedAdd({ParameterName}_SectionBuffer[SectionIndex * 2 + 1], -1);
			TriangleIndex = -1;
		}
		else
		{
			{ParameterName}_IndexBuffer[TriangleIndex * 3 + 0] = (uint)TriangleIndex * 3 + 0;
			{ParameterName}_IndexBuffer[TriangleIndex * 3 + 1] = (uint)TriangleIndex * 3 + 1;
			{ParameterName}_IndexBuffer[TriangleIndex * 3 + 2] = (uint)TriangleIndex * 3 + 2;

			if ( {ParameterName}_PositionOffset != -1 )
			{
				{ParameterName}_SetVertexPositionUnsafe(TriangleIndex * 3 + 0, Position0);
				{ParameterName}_SetVertexPositionUnsafe(TriangleIndex * 3 + 1, Position1);
				{ParameterName}_SetVertexPositionUnsafe(TriangleIndex * 3 + 2, Position2);
			}
			if ( {ParameterName}_TangentBasisOffset != -1 )
			{
				{ParameterName}_SetVertexTangentBasisUnsafe(TriangleIndex * 3 + 0, TangentX0, TangentY0, TangentZ0);
				{ParameterName}_SetVertexTangentBasisUnsafe(TriangleIndex * 3 + 1, TangentX1, TangentY1, TangentZ1);
				{ParameterName}_SetVertexTangentBasisUnsafe(TriangleIndex * 3 + 2, TangentX2, TangentY2, TangentZ2);
			}
			if ( {ParameterName}_TexCoordOffset != -1 )
			{
				{ParameterName}_SetVertexTexCoordUnsafe(TriangleIndex * 3 + 0, TexCoord0, 0);
				{ParameterName}_SetVertexTexCoordUnsafe(TriangleIndex * 3 + 1, TexCoord1, 0);
				{ParameterName}_SetVertexTexCoordUnsafe(TriangleIndex * 3 + 2, TexCoord2, 0);
			}
			if ( {ParameterName}_ColorOffset != -1 )
			{
				{ParameterName}_SetVertexColorUnsafe(TriangleIndex * 3 + 0, Color0);
				{ParameterName}_SetVertexColorUnsafe(TriangleIndex * 3 + 1, Color1);
				{ParameterName}_SetVertexColorUnsafe(TriangleIndex * 3 + 2, Color2);
			}
		}
	}
}

////////////////////////////////////////////////////////
// Imutable Functions

void GetTriangleVertices_{ParameterName}(in int TriangleIndex, out bool bValid, out int Index0, out int Index1, out int Index2)
{
	if ( TriangleIndex >= 0 && TriangleIndex < {ParameterName}_NumTriangles )
	{
		bValid = true;
		Index0 = (int){ParameterName}_IndexBuffer[TriangleIndex * 3 + 0];
		Index1 = (int){ParameterName}_IndexBuffer[TriangleIndex * 3 + 1];
		Index2 = (int){ParameterName}_IndexBuffer[TriangleIndex * 3 + 2];
	}
	else
	{
		bValid = false;
		Index0 = 0;
		Index1 = 0;
		Index2 = 0;
	}
}

void GetVertexPosition_{ParameterName}(in int Vertex, out bool bValid, out float3 Position)
{
	uint Offset = {ParameterName}_GetPositionOffset(Vertex);
	if ( Offset != -1 )
	{
		bValid = true;
		Position.x = asfloat({ParameterName}_VertexBuffer[Offset + 0]);
		Position.y = asfloat({ParameterName}_VertexBuffer[Offset + 1]);
		Position.z = asfloat({ParameterName}_VertexBuffer[Offset + 2]);
	}
	else
	{
		bValid = false;
		Position = 0;
	}
}

void GetVertexTangentBasis_{ParameterName}(in int Vertex, out bool bValid, out float3 OutTangentX, out float3 OutTangentY, out float3 OutTangentZ)
{
	uint Offset = {ParameterName}_GetTangentBasisOffset(Vertex);
	if ( Offset != -1 )
	{
		bValid = true;

		float4 TangentX = {ParameterName}_UnpackTangent({ParameterName}_VertexBuffer[Offset + 0]);
		float4 TangentZ = {ParameterName}_UnpackTangent({ParameterName}_VertexBuffer[Offset + 1]);
		float3 TangentY = cross(TangentZ.xyz, TangentX.xyz) * TangentZ.w;
		OutTangentX = cross(TangentY, TangentZ.xyz) * TangentZ.w;
		OutTangentY = TangentY;
		OutTangentZ = TangentZ.xyz;
	}
	else
	{
		bValid = false;
		OutTangentX = float3(1,0,0);
		OutTangentY = float3(0,1,0);
		OutTangentZ = float3(0,0,1);
	}
}

void GetVertexTexCoord_{ParameterName}(in int Vertex, in int TexCoordIndex, out bool bValid, out float2 TexCoord)
{
	uint Offset = {ParameterName}_GetTexCoordOffset(Vertex, TexCoordIndex);
	if ( Offset != -1 )
	{
		bValid = true;
		TexCoord.x = asfloat({ParameterName}_VertexBuffer[Offset + 0]);
		TexCoord.y = asfloat({ParameterName}_VertexBuffer[Offset + 1]);
	}
	else
	{
		bValid = false;
		TexCoord = 0;
	}
}

void GetVertexColor_{ParameterName}(in int Vertex, out bool bValid, out float4 Color)
{
	uint Offset = {ParameterName}_GetColorOffset(Vertex);
	if ( Offset != -1 )
	{
		bValid = true;
		Color = {ParameterName}_UnpackColor({ParameterName}_VertexBuffer[Offset]);
	}
	else
	{
		bValid = false;
		Color = float4(0,0,0,1);
	}
}

void GetVertexData_{ParameterName}(in int Vertex, out bool bValid, out float3 Position, out float3 TangentX, out float3 TangentY, out float3 TangentZ, out float2 TexCoord, out float4 Color)
{
	if ( Vertex >= 0 && Vertex < {ParameterName}_NumVertices )
	{
		bValid = true;
		bool bDummy;
		GetVertexPosition_{ParameterName}(Vertex, bDummy, Position);
		GetVertexTangentBasis_{ParameterName}(Vertex, bDummy, TangentX, TangentY, TangentZ);
		GetVertexTexCoord_{ParameterName}(Vertex, 0, bDummy, TexCoord);
		GetVertexColor_{ParameterName}(Vertex, bDummy, Color);
	}
	else
	{
		bValid = false;
	}
}
