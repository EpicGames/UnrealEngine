// Copyright Epic Games, Inc. All Rights Reserved.

uint			{ParameterName}_LocalSpace;
uint			{ParameterName}_BoundsValid;
float3			{ParameterName}_BoundsMin;
float3			{ParameterName}_BoundsMax;
uint			{ParameterName}_FixedBoundsValid;
float3			{ParameterName}_FixedBoundsMin;
float3			{ParameterName}_FixedBoundsMax;

void GetLocalSpace_{ParameterName}(out bool bLocalSpace)
{
	bLocalSpace = {ParameterName}_LocalSpace != 0;
}

void GetBounds_{ParameterName}(out bool bValid, out float3 Min, out float3 Max)
{
	// GPU is always fixed no dynamic bounds currently
	bValid = {ParameterName}_BoundsValid != 0;
	Min = {ParameterName}_BoundsMin;
	Max = {ParameterName}_BoundsMax;
}

void GetFixedBounds_{ParameterName}(out bool bValid, out float3 Min, out float3 Max)
{
	bValid = {ParameterName}_FixedBoundsValid != 0;
	Min = {ParameterName}_FixedBoundsMin;
	Max = {ParameterName}_FixedBoundsMax;
}
