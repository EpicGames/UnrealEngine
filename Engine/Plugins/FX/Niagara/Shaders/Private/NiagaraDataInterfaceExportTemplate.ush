// Copyright Epic Games, Inc. All Rights Reserved.

uint			{ParameterName}_WriteBufferSize;
RWBuffer<uint>	{ParameterName}_RWWriteBuffer;

void ExportParticleData_{ParameterName}_UEImpureCall(in bool bStoreData, in float3 Position, in float Size, in float3 Velocity, out bool bSuccess)
{
	bSuccess = false;
	if ( bStoreData && ({ParameterName}_WriteBufferSize > 0) )
	{
		uint StoreIndex;
		InterlockedAdd({ParameterName}_RWWriteBuffer[0], 1u, StoreIndex);
		if ( StoreIndex < {ParameterName}_WriteBufferSize)
		{
			uint StoreOffset = 1 + (StoreIndex * 7);
			{ParameterName}_RWWriteBuffer[StoreOffset + 0] = asuint(Position.x);
			{ParameterName}_RWWriteBuffer[StoreOffset + 1] = asuint(Position.y);
			{ParameterName}_RWWriteBuffer[StoreOffset + 2] = asuint(Position.z);
			{ParameterName}_RWWriteBuffer[StoreOffset + 3] = asuint(Size);
			{ParameterName}_RWWriteBuffer[StoreOffset + 4] = asuint(Velocity.x);
			{ParameterName}_RWWriteBuffer[StoreOffset + 5] = asuint(Velocity.y);
			{ParameterName}_RWWriteBuffer[StoreOffset + 6] = asuint(Velocity.z);
			bSuccess = true;
		}
		else
		{
			InterlockedAdd({ParameterName}_RWWriteBuffer[0], -1u, StoreIndex);
		}
	}
}

void StoreParticleData_{ParameterName}_UEImpureCall(in bool bStoreData, in float3 Position, in float Size, in float3 Velocity, out bool bSuccess)
{
	ExportParticleData_{ParameterName}_UEImpureCall(bStoreData, Position, Size, Velocity, bSuccess);
}
