// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/DeferredShadingCommon.ush"

// Mobile requires multi-pass deferred to access GBuffer
#define NDIGBUFFER_HAS_GBUFFER				(FEATURE_LEVEL >= FEATURE_LEVEL_SM5) || (MOBILE_DEFERRED_SHADING && !ALLOW_FRAMEBUFFER_FETCH)
#define NDIGBUFFER_USE_GBUFFER_VELOCITY		(false && NDIGBUFFER_HAS_GBUFFER)
#define NDIGBUFFER_IS_MOBILE				(FEATURE_LEVEL <= FEATURE_LEVEL_ES3_1)

#define NIAGARA_SAMPLE_SUBSTRATE (SUBSTRATE_GBUFFER_FORMAT==1 && !NDIGBUFFER_IS_MOBILE)

#if NIAGARA_SAMPLE_SUBSTRATE
#define NEEDS_SCENE_TEXTURES 1
#define SCENE_TEXTURES_DISABLED 0
#define SubstratePublicStruct SubstratePublic
#include "/Engine/Public/SubstratePublic.ush"
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

Texture2D {ParameterName}_VelocityTexture;
SamplerState {ParameterName}_VelocityTextureSampler;

#if NIAGARA_SAMPLE_SUBSTRATE
FSubstrateGBuffer GetNiagaraSubstrateGBuffer_{ParameterName}(in float2 ScreenUV)
{
	uint2 ScreenPos = uint2(ScreenUV * View.BufferSizeAndInvSize.xy);
	return SubstratePublic_GetSubstrateGBufferFromFirstBSDF(ScreenPos);
}
#endif

float2 ApplyViewportOffset_{ParameterName}(float2 UV, bool bApplyViewportOffset)
{
	if (bApplyViewportOffset)
	{
		UV = (UV * float2(2.0f, -2.0f) + float2(-1.0f, 1.0f)) * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;
	}
	return UV;
}

void DecodeDiffuseColor_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float3 OutValue)
{
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	IsValid = true;

	#if NIAGARA_SAMPLE_SUBSTRATE
		uint2 ScreenPos = uint2(ScreenUV * View.BufferSizeAndInvSize.xy);
		OutValue = SubstratePublic_GetWorldNormal(ScreenPos);
	#else
		FGBufferData GBufferData = GetGBufferData(ScreenUV);
		OutValue = GBufferData.DiffuseColor;
	#endif
#else
	IsValid = false;
	OutValue = 0;
#endif
}

void DecodeWorldNormal_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float3 OutValue)
{
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	IsValid = true;

	#if NIAGARA_SAMPLE_SUBSTRATE
		uint2 ScreenPos = uint2(ScreenUV * View.BufferSizeAndInvSize.xy);
		OutValue = SubstratePublic_GetWorldNormal(ScreenPos);
	#else
		FGBufferData GBufferData = GetGBufferData(ScreenUV);
		OutValue = GBufferData.WorldNormal;
	#endif
#else
	IsValid = false;
	OutValue = 0;
#endif
}

void DecodeScreenVelocity_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float3 OutValue)
{
	IsValid = false;
	OutValue = 0;
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	float4 EncodedVelocity = 0;
	#if NDIGBUFFER_USE_GBUFFER_VELOCITY
		EncodedVelocity = GetGBufferData(ScreenUV).Velocity;
	#else
		EncodedVelocity = {ParameterName}_VelocityTexture.SampleLevel({ParameterName}_VelocityTextureSampler, ScreenUV, 0);
	#endif
	IsValid = true;
	if ( EncodedVelocity.x > 0 )
	{
	#if VELOCITY_ENCODE_DEPTH
		OutValue = DecodeVelocityFromTexture(EncodedVelocity);
	#else
		OutValue.xy = DecodeVelocityFromTexture(EncodedVelocity).xy;
	#endif
	}
#endif
}

void DecodeWorldVelocity_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float3 OutValue)
{
	IsValid = false;
	OutValue = 0;
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	float4 EncodedVelocity = 0;
	#if NDIGBUFFER_USE_GBUFFER_VELOCITY
		EncodedVelocity = GetGBufferData(ScreenUV).Velocity;
	#else
		EncodedVelocity = {ParameterName}_VelocityTexture.SampleLevel({ParameterName}_VelocityTextureSampler, ScreenUV, 0);
	#endif
	IsValid = true;
	if ( EncodedVelocity.x > 0 )
	{
		float3 Velocity = DecodeVelocityFromTexture(EncodedVelocity);

		float SceneDeviceZ = LookupDeviceZ(ScreenUV);
		float SceneDepth = ConvertFromDeviceZ(SceneDeviceZ);
 		float2 ScreenPosition = (ScreenUV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
		float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
 
		float PrevSceneDepth = 0;
	#if VELOCITY_ENCODE_DEPTH
		PrevSceneDepth = ConvertFromDeviceZ(SceneDeviceZ - Velocity.z);
	#else
 		// Ideally we would use CalcPrevSceneDepth(PrevScreenUV) here to get an accurate location, however we do not store PrevSceneDepth so this is a very rough approximation
		PrevSceneDepth = SceneDepth; // CalcPrevSceneDepth(PrevScreenUV);
	#endif
		float2 PrevScreenPosition = ScreenPosition - Velocity.xy;
		float3 PrevTranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(PrevScreenPosition, PrevSceneDepth), PrevSceneDepth, 1), PrimaryView.PrevScreenToTranslatedWorld).xyz;

		float3 PreViewTranslationOffset = DFFastLocalSubtractDemote(PrimaryView.PreViewTranslation, PrimaryView.PrevPreViewTranslation);
 
		OutValue = TranslatedWorldPosition - PrevTranslatedWorldPosition - PreViewTranslationOffset;
	}
#endif
}

void DecodeBaseColor_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float3 OutValue)
{
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	IsValid = true;

	#if NIAGARA_SAMPLE_SUBSTRATE
		OutValue = GetNiagaraSubstrateGBuffer_{ParameterName}(ScreenUV).BaseColor;
	#else
		FGBufferData GBufferData = GetGBufferData(ScreenUV);
		OutValue = GBufferData.BaseColor;
	#endif

#else
	IsValid = false;
	OutValue = 0;
#endif
}

void DecodeMetallic_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float OutValue)
{
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	IsValid = true;

	#if NIAGARA_SAMPLE_SUBSTRATE
		OutValue = GetNiagaraSubstrateGBuffer_{ParameterName}(ScreenUV).Metallic;
	#else
		FGBufferData GBufferData = GetGBufferData(ScreenUV);
		OutValue = GBufferData.Metallic;
	#endif

#else
	IsValid = false;
	OutValue = 0;
#endif
}

void DecodeSpecular_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float OutValue)
{
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	IsValid = true;

	#if NIAGARA_SAMPLE_SUBSTRATE
		OutValue = GetNiagaraSubstrateGBuffer_{ParameterName}(ScreenUV).Specular;
	#else
		FGBufferData GBufferData = GetGBufferData(ScreenUV);
		OutValue = GBufferData.Specular;
	#endif

#else
	IsValid = false;
	OutValue = 0;
#endif
}

void DecodeRoughness_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float OutValue)
{
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	IsValid = true;

	#if NIAGARA_SAMPLE_SUBSTRATE
		uint2 ScreenPos = uint2(ScreenUV * View.BufferSizeAndInvSize.xy);
		OutValue = SubstratePublic_GetRoughness(ScreenPos);
	#else
		FGBufferData GBufferData = GetGBufferData(ScreenUV);
		OutValue = GBufferData.Roughness;
	#endif

#else
	IsValid = false;
	OutValue = 0;
#endif
}

void DecodeDepth_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float OutValue)
{
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	FGBufferData GBufferData = GetGBufferData(ScreenUV);
	IsValid = true;
	OutValue = GBufferData.Depth;
#elif NDIGBUFFER_IS_MOBILE
	IsValid = true;
	OutValue = CalcSceneDepth(ScreenUV);
#else
	IsValid = false;
	OutValue = 0;
#endif
}

#if 0
// This has not been tested yet
void DecodeStencil_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out int OutValue)
{
#if NDIGBUFFER_HAS_GBUFFER
	IsValid = true;

	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	uint2 IntUV = (uint2)(ScreenUV.xy * (float2)View.BufferSizeAndInvSize.xy);
	uint Stencil = SceneStencilTexture.Load(uint3(IntUV, 0)) STENCIL_COMPONENT_SWIZZLE;
	OutValue = int(Stencil);
#else
	IsValid = false;
	OutValue = 0;
#endif
}
#endif

void DecodePartialDepth_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float OutValue)
{
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	// Make we don't try to access SceneTexturesStruct on a mobile feature level
#if NDIGBUFFER_HAS_GBUFFER && (FEATURE_LEVEL > FEATURE_LEVEL_ES3_1)
	IsValid = true;
	OutValue = ConvertFromDeviceZ(Texture2DSampleLevel(SceneTexturesStruct.ScenePartialDepthTexture, SceneTexturesStruct_ScenePartialDepthTextureSampler, ScreenUV, 0).r);
#elif NDIGBUFFER_IS_MOBILE
	IsValid = true;
	OutValue = ConvertFromDeviceZ(Texture2DSampleLevel(MobileSceneTextures.ScenePartialDepthTexture, MobileSceneTextures.ScenePartialDepthTextureSampler, ScreenUV, 0).r);
#else
	IsValid = false;
	OutValue = 0;
#endif
}


void DecodeCustomDepth_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float OutValue)
{
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	FGBufferData GBufferData = GetGBufferData(ScreenUV);
	IsValid = true;
	OutValue = GBufferData.CustomDepth;
#elif NDIGBUFFER_IS_MOBILE
	IsValid = true;
	OutValue = ConvertFromDeviceZ(Texture2DSampleLevel(MobileSceneTextures.CustomDepthTexture, MobileSceneTextures.CustomDepthTextureSampler, ScreenUV, 0).r);
#else
	IsValid = false;
	OutValue = 0;
#endif
}

void DecodeCustomStencil_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out int OutValue)
{
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	FGBufferData GBufferData = GetGBufferData(ScreenUV);
	IsValid = true;
	OutValue = GBufferData.CustomStencil;
#elif NDIGBUFFER_IS_MOBILE
	IsValid = true;
	OutValue = MobileSceneTextures.CustomStencilTexture.Load(int3(ScreenUV, 0)) STENCIL_COMPONENT_SWIZZLE;
#else
	IsValid = false;
	OutValue = 0;
#endif
}

void DecodeSceneColor_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out float4 OutValue)
{
	// Make we don't try to access SceneTexturesStruct on a mobile feature level
#if NDIGBUFFER_HAS_GBUFFER && (FEATURE_LEVEL > FEATURE_LEVEL_ES3_1)
	IsValid = true;
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	OutValue = Texture2DSampleLevel(SceneTexturesStruct_SceneColorTexture, SceneTexturesStruct_SceneColorTextureSampler, ScreenUV, 0);
	OutValue.rgb *= View.OneOverPreExposure.xxx;
#elif NDIGBUFFER_IS_MOBILE
	IsValid = true;
	OutValue = Texture2DSampleLevel(MobileSceneTextures.SceneColorTexture, MobileSceneTextures.SceneColorTextureSampler, ScreenUV, 0);
	OutValue.rgb *= View.OneOverPreExposure.xxx;
#else
	IsValid = false;
	OutValue = 0;
#endif
}

void DecodeShadingModelID_{ParameterName}(in float2 ScreenUV, in bool bApplyViewportOffset, out bool IsValid, out int OutValue)
{
#if NDIGBUFFER_HAS_GBUFFER
	ScreenUV = ApplyViewportOffset_{ParameterName}(ScreenUV, bApplyViewportOffset);
	IsValid = true;

	#if NIAGARA_SAMPLE_SUBSTRATE
		OutValue = int(GetNiagaraSubstrateGBuffer_{ParameterName}(ScreenUV).ShadingModelID);
	#else
		FGBufferData GBufferData = GetGBufferData(ScreenUV);
		OutValue = GBufferData.ShadingModelID;
	#endif

#else
	IsValid = false;
	OutValue = 0;
#endif
}
