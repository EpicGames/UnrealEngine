// Copyright Epic Games, Inc. All Rights Reserved.

// cubic interpolation between values.
// note f0 = function at t=0
//      f1 = function at t=1
float Monotonic1DCubic_{ParameterName}(float fm1, float f0, float f1, float f2, float t)
{
	float d0 = .5 * (f1 - fm1);
	float d1 = .5 * (f2 - f0);
	float delta0 = f1 - f0;

	if (sign(d0) != sign(d1) || 
		sign(delta0) != sign(d0) ||
		abs(delta0) < 1.1754943508e-38)
	{
		d0 = 0;
		d1 = 0;
	}

	float a0 = f0;
	float a1 = d0;
	float a2 = 3.0f*delta0 - 2.0f * d0 - d1;
	float a3 = d0 + d1 - 2 * delta0;

	return a3 * t*t*t + a2 * t*t + a1 * t + a0;
}

float Basic1DCubic_{ParameterName}(float fm1, float f0, float f1, float f2, float t)
{	
	if (abs(f1 - f0) < 1.1754943508e-38)
	{
		return f0;
	}

	float a0 = f0;
	float a1 = -1./3 * fm1 - 1./2* f0 + f1 - 1./6 * f2;
	float a2 = 1./2 * fm1 - f0 + 1./2 * f1;
	float a3 = -1./6 * fm1 + 1./2 * f0 - 1./2 * f1 + 1./6 * f2;
	return a3 * t*t*t + a2 * t*t + a1 * t + a0;
}

float Basic2DCubic_{ParameterName}(SamplerState Sampler, float3 UVW, int MipLevel)
{
	const int AttributeIndex = round(UVW[2]);

	const float2 GridPos = UVW.xy * {NumCellsName}.xy - .5;
								
	// identify the lower-left-hand corner of the cell
	const int2 GridCell = floor(GridPos);
								
	const int2 MaxCell = {NumCellsName} - int2(3,3);

	if (any((GridCell < int2(2,2)) || (GridCell >= MaxCell)))
	{
		// revert to bilinear hardware sampling at the boundary cells.
		return {GridName}.SampleLevel(Sampler, UVW, MipLevel);
	}
	else 
	{
		const float2 t = frac(GridPos);
		float4 GridVals;
		
		float minv = 3.402823466e+38;
		float maxv = -3.402823466e+38;

		float4 InterpInX;
		for (int j = 0; j < 4; ++j)
		{		
			for (int i = 0; i < 4; ++i)
			{
				const float CurrValue = {GridName}.Load(int4(GridCell.x + i - 1, GridCell.y + j - 1, AttributeIndex, MipLevel));
				GridVals[i] = CurrValue;
				minv = min(CurrValue, minv);
				maxv = max(CurrValue, maxv);
			}
			InterpInX[j] = Basic1DCubic_{ParameterName}(GridVals[0], GridVals[1], GridVals[2], GridVals[3], t[0]);
		}	
		const float BiCubicValue = Basic1DCubic_{ParameterName}(InterpInX[0], InterpInX[1], InterpInX[2], InterpInX[3], t[1]);
		
		if (BiCubicValue < minv || BiCubicValue > maxv)
		{
			return {GridName}.SampleLevel(Sampler, UVW, MipLevel);	
		}	
		return BiCubicValue;
	}
}

float Monotonic2DCubic_{ParameterName}(SamplerState Sampler, float3 UVW, int MipLevel)
{
	const int AttributeIndex = round(UVW[2]);

	const float2 GridPos = UVW.xy * {NumCellsName}.xy - .5;
								
	// identify the lower-left-hand corner of the cell
	const int2 GridCell = floor(GridPos);
								
	const int2 MaxCell = {NumCellsName} - int2(2,2);

	if (any((GridCell < int2(1,1)) || (GridCell >= MaxCell)))
	{
		// revert to bilinear hardware sampling at the boundary cells.
		return {GridName}.SampleLevel(Sampler, UVW, MipLevel);
	}
	else 
	{
		const float2 t = GridPos - GridCell;
		float4 GridVals;
		
		float minv = 3.402823466e+38;
		float maxv = -3.402823466e+38;

		float4 InterpInX;
		for (int j = 0; j < 4; ++j)
		{		
			for (int i = 0; i < 4; ++i)
			{
				const float CurrValue = {GridName}.Load(int4(GridCell.x + i - 1, GridCell.y + j - 1, AttributeIndex, MipLevel));
				GridVals[i] = CurrValue;
				minv = min(CurrValue, minv);
				maxv = max(CurrValue, maxv);
			}
			InterpInX[j] = Monotonic1DCubic_{ParameterName}(GridVals[0], GridVals[1], GridVals[2], GridVals[3], t[0]);
		}	
		const float BiCubicValue = Monotonic1DCubic_{ParameterName}(InterpInX[0], InterpInX[1], InterpInX[2], InterpInX[3], t[1]);
		
		if (BiCubicValue < minv || BiCubicValue > maxv)
		{
			return {GridName}.SampleLevel(Sampler, UVW, MipLevel);	
		}	
		return BiCubicValue;
	}
}

float SampleBiCubic_{ParameterName}(SamplerState Sampler, float3 UVW, int MipLevel)
{		   
	return Basic2DCubic_{ParameterName}(Sampler, UVW, MipLevel);
}