// Copyright Epic Games, Inc. All Rights Reserved.

/*
float SampleExplicitTrilinear_{ParameterName}(SamplerState Sampler, float3 UVW, int MipLevel)
{	
	int3 TotalDim = {NumCellsName} * {NumTiles};
	float3 GridPos = float3(TotalDim[0] * UVW[0] - 0.5,  TotalDim[1] * UVW[1] - 0.5, TotalDim[2] * UVW[2] - 0.5f);
	int3 GridCell = floor(GridPos);
						
	float3 delta = float3( GridPos[0] - GridCell[0], GridPos[1] - GridCell[1], GridPos[2] - GridCell[2]);
	// local values
	float4 F; 
						
	// bottom face
	F[0] = {GridName}.Load( int4( GridCell[0],     GridCell[1], GridCell[2], MipLevel) );
	F[1] = {GridName}.Load( int4( GridCell[0] + 1, GridCell[1], GridCell[2], MipLevel) );
	F[2] = {GridName}.Load( int4( GridCell[0] + 1, GridCell[1] +1 , GridCell[2], MipLevel) );
	F[3] = {GridName}.Load( int4( GridCell[0],     GridCell[1] +1 , GridCell[2], MipLevel) );
	// bottom value
	float BV = (1.-delta.y) * (F[0] *(1.-delta.x) + F[1]*(delta.x)) + delta.y * (F[3]*(1.-delta.x) + F[2]*delta.x);

	// top face
	F[0] = {GridName}.Load( int4( GridCell[0],     GridCell[1],     GridCell[2] + 1, MipLevel) );
	F[1] = {GridName}.Load( int4( GridCell[0] + 1, GridCell[1],     GridCell[2] + 1, MipLevel) );
	F[2] = {GridName}.Load( int4( GridCell[0] + 1, GridCell[1] +1 , GridCell[2] + 1, MipLevel) );
	F[3] = {GridName}.Load( int4( GridCell[0],     GridCell[1] +1 , GridCell[2] + 1, MipLevel) );
	// top value
	float TV = (1.-delta.y) * (F[0] *(1.-delta.x) + F[1]*(delta.x)) + delta.y * (F[3]*(1.-delta.x) + F[2]*delta.x);

	// interp between bottom and top
	return TV * delta.z + BV * (1.-delta.z);	
}
*/

float{NumChannels} SampleTriCubicLagrange_{ParameterName}(SamplerState Sampler, float3 UVW, int MipLevel)
{		    
	int3 TotalDim = {NumCellsName} * {NumTiles};	
	float3 GridPos = float3(TotalDim[0] * UVW[0] - 0.5,  TotalDim[1] * UVW[1] - 0.5, TotalDim[2] * UVW[2] - 0.5f);
								
	// identify the lower-left-hand corner of the cell
	int3 GridCell = floor(GridPos);
								
	int3 LocalCell = GridCell % {NumCellsName};
								
	const int3 MaxCell = {NumCellsName} - int3(2,2,2);

	if (any((LocalCell <= int3(1,1,1)) || (LocalCell >= MaxCell)))
	{
		// revert to trilinear hardware sampling at the boundary cells.
		return {GridName}.SampleLevel(Sampler, UVW, MipLevel);
	}
	else 
	{
		// sample point offset from lower left
		float3 delta = float3( GridPos[0] - GridCell[0], GridPos[1] - GridCell[1], GridPos[2] - GridCell[2]);
							
		
		float t = 1. + delta.x;
		float u = 1. + delta.y;
		float v = 1. + delta.z;
		
		float4x{NumChannels} DataInZ;
		float{NumChannels} minv = 3.402823466e+38;
		float{NumChannels} maxv = -3.402823466e+38;
		for (int zk = 0; zk < 4; zk++)
		{
			int plane = GridCell[2] + zk -1;
		
			float4x{NumChannels}  DataInY;
			for (int yj = 0; yj < 4; yj++)
			{
				float4x{NumChannels} DataInX;
								
				int row = GridCell[1] + yj - 1;
		
				for(int xi = 0; xi < 4; xi++)
				{
					int col = GridCell[0] + xi - 1;
					DataInX[xi] = {GridName}.Load(int4(col, row, plane, MipLevel)); 
					minv = min(DataInX[xi], minv);
					maxv = max(DataInX[xi], maxv);
				}
		
				// coefficients
				float{NumChannels} Cx0 =   6.f * DataInX[0];
				float{NumChannels} Cx1 = -11.f * DataInX[0] + 18.f * DataInX[1] - 9.f * DataInX[2] + 2.f * DataInX[3];
				float{NumChannels} Cx2 =   6.f * DataInX[0] - 15.f * DataInX[1] +12.f * DataInX[2] - 3.f * DataInX[3];
				float{NumChannels} Cx3 =        -DataInX[0] +  3.f * DataInX[1] - 3.f * DataInX[2] +       DataInX[3];
		
				DataInY[yj] = (1.f/6.f) * (Cx0 + t * ( Cx1 +  t * ( Cx2 + t * Cx3 ) ));
		
			}
									
			float{NumChannels} Cy0 =   6.f * DataInY[0];
			float{NumChannels} Cy1 = -11.f * DataInY[0] + 18.f * DataInY[1] - 9.f * DataInY[2] + 2.f * DataInY[3];
			float{NumChannels} Cy2 =   6.f * DataInY[0] - 15.f * DataInY[1] +12.f * DataInY[2] - 3.f * DataInY[3];
			float{NumChannels} Cy3 =        -DataInY[0] +  3.f * DataInY[1] - 3.f * DataInY[2] +       DataInY[3];
		
			DataInZ[zk] = (1.f/6.f) * (Cy0 + u * ( Cy1 +  u * ( Cy2 + u * Cy3 ) ));
									
		}
		
		float{NumChannels} Cz0 =   6.f * DataInZ[0];
		float{NumChannels} Cz1 = -11.f * DataInZ[0] + 18.f * DataInZ[1] - 9.f * DataInZ[2] + 2.f * DataInZ[3];
		float{NumChannels} Cz2 =   6.f * DataInZ[0] - 15.f * DataInZ[1] +12.f * DataInZ[2] - 3.f * DataInZ[3];
		float{NumChannels} Cz3 =        -DataInZ[0] +  3.f * DataInZ[1] - 3.f * DataInZ[2] +       DataInZ[3];
								
		float{NumChannels} TriCubicValue = (1.f/6.f) * (Cz0 + v * ( Cz1 +  v * ( Cz2 + v * Cz3 ) ));
									
		// for the fluid sims, a nice look can be achieved with BlendAmount = 0.25f;
		// float BlendAmount = 0.0f;
		// return lerp(TriCubicValue, TriLinearValue, BlendAmount);		

		return  min(max(TriCubicValue, minv), maxv);
	}
}