// Copyright Epic Games, Inc. All Rights Reserved.

float4				{ParameterName}_TextureSizeAndInvSize;
RWTexture2D<int>	{ParameterName}_RWTextureUAV;

void GetValue_{ParameterName}(int X, int Y, out int CurrValue)
{
	CurrValue = {ParameterName}_RWTextureUAV[int2(X, Y)];
}

void SetValue_{ParameterName}_UEImpureCall(in bool bExecute, int X, int Y, int Value)
{
	if ( bExecute )
	{
		{ParameterName}_RWTextureUAV[int2(X, Y)] = Value;
	}
}

void AtomicAdd_{ParameterName}_UEImpureCall(in bool bExecute, int X, int Y, int Amount, out int CurrValue, out int PrevValue)
{
	if ( bExecute )
	{
		InterlockedAdd({ParameterName}_RWTextureUAV[int2(X, Y)], Amount, PrevValue);
		CurrValue = PrevValue + Amount;
	}
	else
	{
		PrevValue = {ParameterName}_RWTextureUAV[int2(X, Y)];
		CurrValue = PrevValue;
	}
}

void AtomicCompareAndExchange_{ParameterName}_UEImpureCall(in bool bExecute, int X, int Y, int ComparisonValue, int Value, out int OriginalValue)
{
	if ( bExecute )
	{
		InterlockedCompareExchange({ParameterName}_RWTextureUAV[int2(X, Y)], ComparisonValue, Value, OriginalValue);
	}
	else
	{
		OriginalValue = {ParameterName}_RWTextureUAV[int2(X, Y)];
	}
}

void AtomicMax_{ParameterName}_UEImpureCall(in bool bExecute, int X, int Y, int InValue, out int CurrValue, out int PrevValue)
{
	if ( bExecute )
	{
		InterlockedMax({ParameterName}_RWTextureUAV[int2(X, Y)], InValue, PrevValue);
		CurrValue = max(PrevValue, InValue);
	}
	else
	{
		PrevValue = {ParameterName}_RWTextureUAV[int2(X, Y)];
		CurrValue = PrevValue;
	}
}

void AtomicMin_{ParameterName}_UEImpureCall(in bool bExecute, int X, int Y, int InValue, out int CurrValue, out int PrevValue)
{
	if ( bExecute )
	{
		InterlockedMin({ParameterName}_RWTextureUAV[int2(X, Y)], InValue, PrevValue);
		CurrValue = min(PrevValue, InValue);
	}
	else
	{
		PrevValue = {ParameterName}_RWTextureUAV[int2(X, Y)];
		CurrValue = PrevValue;
	}
}

void GetRenderTargetSize_{ParameterName}(out int Width, out int Height)
{
	Width = int({ParameterName}_TextureSizeAndInvSize.x);
	Height = int({ParameterName}_TextureSizeAndInvSize.y);
}

void LinearToIndex_{ParameterName}(int Linear, out int X, out int Y)
{
	X = int(uint(Linear) % uint({ParameterName}_TextureSizeAndInvSize.x));
	Y = int(uint(Linear) / uint({ParameterName}_TextureSizeAndInvSize.x));
}

void LinearToUV_{ParameterName}(int Linear, out float2 UV)
{
	UV.x = (float(uint(Linear) % uint({ParameterName}_TextureSizeAndInvSize.x)) + 0.5f) * {ParameterName}_TextureSizeAndInvSize.z;
	UV.y = (float(uint(Linear) / uint({ParameterName}_TextureSizeAndInvSize.x)) + 0.5f) * {ParameterName}_TextureSizeAndInvSize.w;
}

void ExecToIndex_{ParameterName}(out int X, out int Y)
{
	LinearToIndex_{ParameterName}(ExecIndex(), X, Y);
}

void ExecToUV_{ParameterName}(out float2 UV)
{
	LinearToUV_{ParameterName}(ExecIndex(), UV);
}
