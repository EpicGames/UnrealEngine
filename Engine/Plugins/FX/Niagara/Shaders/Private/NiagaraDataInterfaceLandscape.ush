// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef NUM_VIRTUALTEXTURE_SAMPLES
	#define NUM_VIRTUALTEXTURE_SAMPLES 1
#endif

#include "/Engine/Private/VirtualTextureCommon.ush"
#include "/Engine/Private/Landscape/LandscapeCommon.ush"

// Mirrored from FNiagaraDataInterfaceParametersCS_Landscape
#define VIRTUAL_TEXTURE_UNPACK_NORMAL_NONE 0
#define VIRTUAL_TEXTURE_UNPACK_NORMAL_BC3BC3 1
#define VIRTUAL_TEXTURE_UNPACK_NORMAL_BC5BC1 2

Texture2D			{ParameterName}_BaseColorVirtualTexture;
Texture2D<uint4>	{ParameterName}_BaseColorVirtualTexturePageTable;
SamplerState		{ParameterName}_BaseColorVirtualTextureSampler;
float3				{ParameterName}_BaseColorVirtualTextureLWCTile;
float4x4			{ParameterName}_BaseColorVirtualTextureWorldToUvTransform;
uint				{ParameterName}_BaseColorVirtualTextureUnpackSRGB;
uint				{ParameterName}_BaseColorVirtualTextureUnpackYCoCg;
uint				{ParameterName}_BaseColorVirtualTextureEnabled;
uint4				{ParameterName}_BaseColorVirtualTexturePackedUniform0;
uint4				{ParameterName}_BaseColorVirtualTexturePackedUniform1;
uint4				{ParameterName}_BaseColorVirtualTextureUniforms;
Texture2D			{ParameterName}_HeightVirtualTexture;
Texture2D<uint4>	{ParameterName}_HeightVirtualTexturePageTable;
SamplerState		{ParameterName}_HeightVirtualTextureSampler;
float3				{ParameterName}_HeightVirtualTextureLWCTile;
float4x4			{ParameterName}_HeightVirtualTextureWorldToUvTransform;
uint				{ParameterName}_HeightVirtualTextureEnabled;
uint4				{ParameterName}_HeightVirtualTexturePackedUniform0;
uint4				{ParameterName}_HeightVirtualTexturePackedUniform1;
uint4				{ParameterName}_HeightVirtualTextureUniforms;
Texture2D			{ParameterName}_NormalVirtualTexture0;
Texture2D			{ParameterName}_NormalVirtualTexture1;
Texture2D<uint4>	{ParameterName}_NormalVirtualTexturePageTable;
SamplerState		{ParameterName}_NormalVirtualTexture0Sampler;
SamplerState		{ParameterName}_NormalVirtualTexture1Sampler;
float3				{ParameterName}_NormalVirtualTextureLWCTile;
float4x4			{ParameterName}_NormalVirtualTextureWorldToUvTransform;
uint4				{ParameterName}_NormalVirtualTexturePackedUniform0;
uint4				{ParameterName}_NormalVirtualTexturePackedUniform1;
uint4				{ParameterName}_NormalVirtualTextureUniforms0;
uint4				{ParameterName}_NormalVirtualTextureUniforms1;
int					{ParameterName}_NormalVirtualTextureUnpackMode;
uint				{ParameterName}_NormalVirtualTextureEnabled;
Texture2D			{ParameterName}_CachedHeightTexture;
SamplerState		{ParameterName}_CachedHeightTextureSampler;
uint				{ParameterName}_CachedHeightTextureEnabled;
float3				{ParameterName}_CachedHeightTextureLWCTile;
float4x4			{ParameterName}_CachedHeightTextureWorldToUvTransform;
float4x4			{ParameterName}_CachedHeightTextureUvToWorldTransform;
float4				{ParameterName}_CachedHeightTextureUvScaleBias;
float2				{ParameterName}_CachedHeightTextureWorldGridSize;
int2				{ParameterName}_CachedHeightTextureDimension;
SamplerState		{ParameterName}_PointClampedSampler;
Texture2D<uint>		{ParameterName}_CachedPhysMatTexture;
int2				{ParameterName}_CachedPhysMatTextureDimension;

float2 GetCachedHeightTextureUv_{ParameterName}(float3 InWorldPos)
{
	FLWCVector4 LwcWorldPos = MakeLWCVector4(GetEngineOwnerLWCTile(), float4(InWorldPos, 1.0f));
	FLWCInverseMatrix LwcWorldToUv = MakeLWCInverseMatrix({ParameterName}_CachedHeightTextureLWCTile, {ParameterName}_CachedHeightTextureWorldToUvTransform);

	float2 Uv = LWCMultiply(LwcWorldPos, LwcWorldToUv).xy;
	Uv = Uv * {ParameterName}_CachedHeightTextureUvScaleBias.xy + {ParameterName}_CachedHeightTextureUvScaleBias.zw;

	return Uv;
}

void GetBaseColor_{ParameterName}(float3 InWorldPos, out float3 OutBaseColor, out bool OutIsValid)
{
	if ({ParameterName}_BaseColorVirtualTextureEnabled != 0)
	{
		float3 UnpackBaseColorOrigin = {ParameterName}_BaseColorVirtualTextureWorldToUvTransform[0].xyz;
		float3 UnpackBaseColorU = {ParameterName}_BaseColorVirtualTextureWorldToUvTransform[1].xyz;
		float3 UnpackBaseColorV = {ParameterName}_BaseColorVirtualTextureWorldToUvTransform[2].xyz;

		FDFVector3 LwcWorldPos = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), InWorldPos));
		FDFVector3 LwcBaseColorOrigin = DFFromTileOffset_Hack(MakeLWCVector3({ParameterName}_BaseColorVirtualTextureLWCTile, UnpackBaseColorOrigin));
		float2 SampleUv = VirtualTextureWorldToUV(LwcWorldPos, LwcBaseColorOrigin, UnpackBaseColorU, UnpackBaseColorV);

		VTPageTableResult PageTable = TextureLoadVirtualPageTableLevel(
			{ParameterName}_BaseColorVirtualTexturePageTable,
			VTPageTableUniform_Unpack({ParameterName}_BaseColorVirtualTexturePackedUniform0, {ParameterName}_BaseColorVirtualTexturePackedUniform1),
			SampleUv,
			VTADDRESSMODE_CLAMP,
			VTADDRESSMODE_CLAMP,
			0.0f);

		float4 PackedValue = TextureVirtualSample(
			{ParameterName}_BaseColorVirtualTexture,
			{ParameterName}_BaseColorVirtualTextureSampler,
			PageTable,
			0,
			VTUniform_Unpack({ParameterName}_BaseColorVirtualTextureUniforms));

		if ({ParameterName}_BaseColorVirtualTextureUnpackSRGB)
		{
			OutBaseColor = VirtualTextureUnpackBaseColorSRGB(PackedValue);
		}
		else if ({ParameterName}_BaseColorVirtualTextureUnpackYCoCg)
		{
			OutBaseColor = VirtualTextureUnpackBaseColorYCoCg(PackedValue);
		}
		else
		{
			OutBaseColor = PackedValue.rgb;
		}
		OutIsValid = true;
	}
	else
	{
		OutBaseColor = 0.0f;
		OutIsValid = false;
	}
}

void GetHeight_{ParameterName}(float3 InWorldPos, out float OutHeight, out bool OutIsValid)
{
	if ({ParameterName}_HeightVirtualTextureEnabled != 0)
	{
		float3 UnpackHeightOrigin = {ParameterName}_HeightVirtualTextureWorldToUvTransform[0].xyz;
		float3 UnpackHeightU = {ParameterName}_HeightVirtualTextureWorldToUvTransform[1].xyz;
		float3 UnpackHeightV = {ParameterName}_HeightVirtualTextureWorldToUvTransform[2].xyz;
		float2 UnpackHeightScaleBias = {ParameterName}_HeightVirtualTextureWorldToUvTransform[3].xy;

		FDFVector3 LwcWorldPos = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), InWorldPos));
		FDFVector3 LwcHeightOrigin = DFFromTileOffset_Hack(MakeLWCVector3({ParameterName}_HeightVirtualTextureLWCTile, UnpackHeightOrigin));
		float2 SampleUv = VirtualTextureWorldToUV(LwcWorldPos, LwcHeightOrigin, UnpackHeightU, UnpackHeightV);

		VTPageTableResult PageTable = TextureLoadVirtualPageTableLevel(
			{ParameterName}_HeightVirtualTexturePageTable,
			VTPageTableUniform_Unpack({ParameterName}_HeightVirtualTexturePackedUniform0, {ParameterName}_HeightVirtualTexturePackedUniform1),
			SampleUv,
			VTADDRESSMODE_CLAMP,
			VTADDRESSMODE_CLAMP,
			0.0f);

		float4 PackedValue = TextureVirtualSample(
			{ParameterName}_HeightVirtualTexture,
			{ParameterName}_HeightVirtualTextureSampler,
			PageTable,
			0,
			VTUniform_Unpack({ParameterName}_HeightVirtualTextureUniforms));

		OutHeight = VirtualTextureUnpackHeight(PackedValue, UnpackHeightScaleBias);
		OutIsValid = true;
	}
	else if ({ParameterName}_CachedHeightTextureEnabled != 0)
	{
		float2 Uv = GetCachedHeightTextureUv_{ParameterName}(InWorldPos);

		OutHeight = {ParameterName}_CachedHeightTexture.SampleLevel({ParameterName}_CachedHeightTextureSampler, Uv, 0).x;
		OutIsValid = true;
	}
	else
	{
		OutHeight = InWorldPos.z;
		OutIsValid = false;
	}
}

void GetWorldNormal_{ParameterName}(float3 InWorldPos, out float3 OutNormal, out bool OutIsValid)
{
	if ({ParameterName}_NormalVirtualTextureEnabled != 0)
	{
		float3 UnpackNormalOrigin = {ParameterName}_NormalVirtualTextureWorldToUvTransform[0].xyz;
		float3 UnpackNormalU = {ParameterName}_NormalVirtualTextureWorldToUvTransform[1].xyz;
		float3 UnpackNormalV = {ParameterName}_NormalVirtualTextureWorldToUvTransform[2].xyz;

		FDFVector3 LwcWorldPos = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), InWorldPos));
		FDFVector3 LwcNormalOrigin = DFFromTileOffset_Hack(MakeLWCVector3({ParameterName}_NormalVirtualTextureLWCTile, UnpackNormalOrigin));
		float2 SampleUv = VirtualTextureWorldToUV(LwcWorldPos, LwcNormalOrigin, UnpackNormalU, UnpackNormalV);

		VTPageTableResult PageTable = TextureLoadVirtualPageTableLevel(
			{ParameterName}_NormalVirtualTexturePageTable,
			VTPageTableUniform_Unpack({ParameterName}_NormalVirtualTexturePackedUniform0, {ParameterName}_NormalVirtualTexturePackedUniform1),
			SampleUv,
			VTADDRESSMODE_CLAMP,
			VTADDRESSMODE_CLAMP,
			0.0f);

		float4 PackedValue0 = TextureVirtualSample(
			{ParameterName}_NormalVirtualTexture0,
			{ParameterName}_NormalVirtualTexture0Sampler,
			PageTable,
			0,
			VTUniform_Unpack({ParameterName}_NormalVirtualTextureUniforms0));

		float4 PackedValue1 = TextureVirtualSample(
			{ParameterName}_NormalVirtualTexture1,
			{ParameterName}_NormalVirtualTexture1Sampler,
			PageTable,
			0,
			VTUniform_Unpack({ParameterName}_NormalVirtualTextureUniforms1));

		if ({ParameterName}_NormalVirtualTextureUnpackMode == VIRTUAL_TEXTURE_UNPACK_NORMAL_BC3BC3)
		{
			OutNormal = VirtualTextureUnpackNormalBC3BC3(PackedValue0, PackedValue1);
		}
		else if ({ParameterName}_NormalVirtualTextureUnpackMode == VIRTUAL_TEXTURE_UNPACK_NORMAL_BC5BC1)
		{
			OutNormal = VirtualTextureUnpackNormalBC5BC1(PackedValue0, PackedValue1);
		}
		else
		{
			OutNormal = float3(0.0f, 0.0f, 1.0f);
			OutIsValid = false;
			return;
		}

		OutIsValid = true;
	}
	else if ({ParameterName}_CachedHeightTextureEnabled != 0)
	{
		float2 InTextureCoordinates = GetCachedHeightTextureUv_{ParameterName}(InWorldPos);
		float2 TextureSize = {ParameterName}_CachedHeightTextureDimension.xy;

		// taken from LandscapeLayersPS.usf -> CalculateNormalsFromHeights()
		float2 TexelSize = 1.0 / TextureSize;

		float MaskMinBorderX = (InTextureCoordinates.x <= TexelSize.x) ? 0.0f : 1.0f;			// first pixel X
		float MaskMinBorderY = (InTextureCoordinates.y <= TexelSize.y) ? 0.0f : 1.0f;			// first pixel Y
		float MaskMaxBorderX = (InTextureCoordinates.x >= 1.0f - TexelSize.x) ? 0.0f : 1.0f;	// last pixel X
		float MaskMaxBorderY = (InTextureCoordinates.y >= 1.0f - TexelSize.y) ? 0.0f : 1.0f;	// last pixel Y

		// The triangle topology is the following (where C = center, T = top, B = bottom, L = left, R = right and Nx the normals we need to interpolate):
		// TL ------ TT
		// | \       | \
		// |  \      |  \
		// |   \     |   \
		// | N0 \ N1 | N3 \
		// |     \   |     \
		// |      \  |      \
		// |       \ |       \
		// LL ------ CC ------ RR
		//   \       | \       |
		//    \      |  \      |
		//     \     |   \     |
		//      \ N2 | N4 \ N5 |
		//       \   |     \   |
		//        \  |      \  |
		//         \ |       \ |
		//           BB ------ BR
		// So we can retrieve all 7 samples using 4 gathers : 
		// Make sure we gather the 4 pixels in the bottom right direction by sampling at (0.75, 0.75) of the pixel (gather retrieves the 4 samples that would be used for bilinear interpolation):

		float2 GatherLocation = InTextureCoordinates + 0.25f * TexelSize; 
		float4 Red0 = {ParameterName}_CachedHeightTexture.GatherRed({ParameterName}_CachedHeightTextureSampler, GatherLocation - TexelSize);
		float4 Red1 = {ParameterName}_CachedHeightTexture.GatherRed({ParameterName}_CachedHeightTextureSampler, GatherLocation);

		// [x,y,z] = World Position [w] = valid mask
		float4 TL = float4(float2(-1, -1) * {ParameterName}_CachedHeightTextureWorldGridSize, Red0.w, 1.0f);
		float4 TT = float4(float2(+0, -1) * {ParameterName}_CachedHeightTextureWorldGridSize, Red0.z, 1.0f);
		float4 CC = float4(float2(+0, +0) * {ParameterName}_CachedHeightTextureWorldGridSize, Red0.y, 1.0f);
		float4 LL = float4(float2(-1, +0) * {ParameterName}_CachedHeightTextureWorldGridSize, Red0.x, 1.0f);
		float4 RR = float4(float2(+1, +0) * {ParameterName}_CachedHeightTextureWorldGridSize, Red1.z, 1.0f);
		float4 BR = float4(float2(+1, +1) * {ParameterName}_CachedHeightTextureWorldGridSize, Red1.y, 1.0f);
		float4 BB = float4(float2(+0, +1) * {ParameterName}_CachedHeightTextureWorldGridSize, Red1.x, 1.0f);

		// mask out samples that are off the edge of the texture
		TL.w *= MaskMinBorderX * MaskMinBorderY;
		TT.w *=                  MaskMinBorderY;
		// CC should never be off the edge
		LL.w *= MaskMinBorderX;
		RR.w *= MaskMaxBorderX;
		BR.w *= MaskMaxBorderX * MaskMaxBorderY;
		BB.w *=                  MaskMaxBorderY;

		float3 N0 = ComputeNullableTriangleNormal(CC, LL, TL);
		float3 N1 = ComputeNullableTriangleNormal(TL, TT, CC);
		float3 N2 = ComputeNullableTriangleNormal(LL, CC, BB);
		float3 N3 = ComputeNullableTriangleNormal(RR, CC, TT);
		float3 N4 = ComputeNullableTriangleNormal(BR, BB, CC);
		float3 N5 = ComputeNullableTriangleNormal(CC, RR, BR);

		// average the normals
		float3 FinalNormal = normalize(N0 + N1 + N2 + N3 + N4 + N5);

		OutNormal = normalize(mul(float4(FinalNormal, 0.0f), {ParameterName}_CachedHeightTextureUvToWorldTransform).xyz);
		OutIsValid = true;
	}
	else
	{
		OutNormal = float3(0.0f, 0.0f, 1.0f);
		OutIsValid = false;
	}
}

void GetPhysicalMaterialIndex_{ParameterName}(float3 InWorldPos, out int OutIndex, out bool OutIsValid)
{
	float2 Uv = GetCachedHeightTextureUv_{ParameterName}(InWorldPos);

	int3 SampleIndex = int3(Uv * {ParameterName}_CachedPhysMatTextureDimension, 0);
	OutIndex = (int){ParameterName}_CachedPhysMatTexture.Load(SampleIndex).x;

	if (OutIndex >= 255)
	{
		OutIndex = -1;
		OutIsValid = false;
	}
	else
	{
		OutIsValid = true;
	}
}

#undef VIRTUAL_TEXTURE_UNPACK_NORMAL_NONE
#undef VIRTUAL_TEXTURE_UNPACK_NORMAL_BC3BC3
#undef VIRTUAL_TEXTURE_UNPACK_NORMAL_BC5BC1
