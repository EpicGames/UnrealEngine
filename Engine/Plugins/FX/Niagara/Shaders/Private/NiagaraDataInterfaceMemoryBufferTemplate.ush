// Copyright Epic Games, Inc. All Rights Reserved.

int					{ParameterName}_NumElements;
RWByteAddressBuffer	{ParameterName}_Buffer;

/////////////////////////////////////////////////////////////////////////////
// Helper functions
bool ValidateOffset_{ParameterName}(inout int ElementOffset, int Size)
{
	bool bValid = ElementOffset >= 0 && ElementOffset <= {ParameterName}_NumElements - Size;
	ElementOffset = clamp(ElementOffset, 0, {ParameterName}_NumElements);
	return bValid;
}

/////////////////////////////////////////////////////////////////////////////
// Misc Functions
void GetNumElements_{ParameterName}(out int NumElements)
{
	NumElements = {ParameterName}_NumElements;
}

/////////////////////////////////////////////////////////////////////////////
// Load operations
void LoadInt_{ParameterName}(int ElementOffset, out bool bValid, out int OutValue)
{
	bValid = ValidateOffset_{ParameterName}(ElementOffset, 1);
	const uint BufferValue = {ParameterName}_Buffer.Load(ElementOffset << 2);
	OutValue = asint(BufferValue);
}

void LoadInt2_{ParameterName}(int ElementOffset, out bool bValid, out int OutValue0, out int OutValue1)
{
	bValid = ValidateOffset_{ParameterName}(ElementOffset, 2);
	const uint2 BufferValue = {ParameterName}_Buffer.Load2(ElementOffset << 2);
	OutValue0 = asint(BufferValue.x);
	OutValue1 = asint(BufferValue.y);
}

void LoadInt3_{ParameterName}(int ElementOffset, out bool bValid, out int OutValue0, out int OutValue1, out int OutValue2)
{
	bValid = ValidateOffset_{ParameterName}(ElementOffset, 3);
	const uint3 BufferValue = {ParameterName}_Buffer.Load3(ElementOffset << 2);
	OutValue0 = asint(BufferValue.x);
	OutValue1 = asint(BufferValue.y);
	OutValue2 = asint(BufferValue.z);
}

void LoadInt4_{ParameterName}(int ElementOffset, out bool bValid, out int OutValue0, out int OutValue1, out int OutValue2, out int OutValue3)
{
	bValid = ValidateOffset_{ParameterName}(ElementOffset, 4);
	const uint4 BufferValue = {ParameterName}_Buffer.Load4(ElementOffset << 2);
	OutValue0 = asint(BufferValue.x);
	OutValue1 = asint(BufferValue.y);
	OutValue2 = asint(BufferValue.z);
	OutValue3 = asint(BufferValue.w);
}

void LoadFloat_{ParameterName}(int ElementOffset, out bool bValid, out float OutValue)
{
	bValid = ValidateOffset_{ParameterName}(ElementOffset, 1);
	const uint BufferValue = {ParameterName}_Buffer.Load(ElementOffset << 2);
	OutValue = asfloat(BufferValue);
}

void LoadFloat2_{ParameterName}(int ElementOffset, out bool bValid, out float2 OutValue)
{
	bValid = ValidateOffset_{ParameterName}(ElementOffset, 2);
	const uint2 BufferValue = {ParameterName}_Buffer.Load2(ElementOffset << 2);
	OutValue = asfloat(BufferValue);
}

void LoadFloat3_{ParameterName}(int ElementOffset, out bool bValid, out float3 OutValue)
{
	bValid = ValidateOffset_{ParameterName}(ElementOffset, 3);
	const uint3 BufferValue = {ParameterName}_Buffer.Load3(ElementOffset << 2);
	OutValue = asfloat(BufferValue);
}

void LoadFloat4_{ParameterName}(int ElementOffset, out bool bValid, out float4 OutValue)
{
	bValid = ValidateOffset_{ParameterName}(ElementOffset, 4);
	const uint4 BufferValue = {ParameterName}_Buffer.Load4(ElementOffset << 2);
	OutValue = asfloat(BufferValue);
}

/////////////////////////////////////////////////////////////////////////////
// Store operations
void StoreInt_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, int Value, out bool bSuccess)
{
	bSuccess = bExecute && ValidateOffset_{ParameterName}(ElementOffset, 1);
	if (bExecute)
	{
		{ParameterName}_Buffer.Store(ElementOffset << 2, asuint(Value));
	}
}

void StoreInt2_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, int Value0, int Value1, out bool bSuccess)
{
	bSuccess = bExecute && ValidateOffset_{ParameterName}(ElementOffset, 2);
	if (bExecute)
	{
		{ParameterName}_Buffer.Store(ElementOffset << 2, asuint(int2(Value0, Value1)));
	}
}

void StoreInt3_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, int Value0, int Value1, int Value2, out bool bSuccess)
{
	bSuccess = bExecute && ValidateOffset_{ParameterName}(ElementOffset, 3);
	if (bExecute)
	{
		{ParameterName}_Buffer.Store(ElementOffset << 2, asuint(int3(Value0, Value1, Value2)));
	}
}

void StoreInt4_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, int Value0, int Value1, int Value2, int Value3, out bool bSuccess)
{
	bSuccess = bExecute && ValidateOffset_{ParameterName}(ElementOffset, 4);
	if (bExecute)
	{
		{ParameterName}_Buffer.Store(ElementOffset << 2, asuint(int4(Value0, Value1, Value2, Value3)));
	}
}

void StoreFloat_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, float Value, out bool bSuccess)
{
	bSuccess = bExecute && ValidateOffset_{ParameterName}(ElementOffset, 1);
	if (bExecute)
	{
		{ParameterName}_Buffer.Store(ElementOffset << 2, asuint(Value));
	}
}

void StoreFloat2_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, float2 Value, out bool bSuccess)
{
	bSuccess = bExecute && ValidateOffset_{ParameterName}(ElementOffset, 2);
	if (bExecute)
	{
		{ParameterName}_Buffer.Store(ElementOffset << 2, asuint(Value));
	}
}

void StoreFloat3_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, float3 Value, out bool bSuccess)
{
	bSuccess = bExecute && ValidateOffset_{ParameterName}(ElementOffset, 3);
	if (bExecute)
	{
		{ParameterName}_Buffer.Store(ElementOffset << 2, asuint(Value));
	}
}

void StoreFloat4_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, float4 Value, out bool bSuccess)
{
	bSuccess = bExecute && ValidateOffset_{ParameterName}(ElementOffset, 4);
	if (bExecute)
	{
		{ParameterName}_Buffer.Store(ElementOffset << 2, asuint(Value));
	}
}

/////////////////////////////////////////////////////////////////////////////
// Atomic Ops
void AtomicAdd_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, int Value, out int PreviousValue, out int CurrentValue)
{
	if (bExecute)
	{
		ElementOffset = clamp(ElementOffset, 0, {ParameterName}_NumElements) << 2;
		{ParameterName}_Buffer.InterlockedAdd(ElementOffset, Value, PreviousValue);
		CurrentValue = PreviousValue + Value;
	}
}

void AtomicAnd_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, int Value, out int PreviousValue, out int CurrentValue)
{
	if (bExecute)
	{
		ElementOffset = clamp(ElementOffset, 0, {ParameterName}_NumElements) << 2;
		{ParameterName}_Buffer.InterlockedAnd(ElementOffset, Value, PreviousValue);
		CurrentValue = PreviousValue & Value;
	}
}

void AtomicCompareAndExchange_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, int Value, int Comperand, out int PreviousValue, out int CurrentValue)
{
	if (bExecute)
	{
		ElementOffset = clamp(ElementOffset, 0, {ParameterName}_NumElements) << 2;
		{ParameterName}_Buffer.InterlockedCompareExchange(ElementOffset, Comperand, Value, PreviousValue);
		CurrentValue = PreviousValue == Value ? Value : PreviousValue;
	}
}

void AtomicMax_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, int Value, out int PreviousValue, out int CurrentValue)
{
	if (bExecute)
	{
		ElementOffset = clamp(ElementOffset, 0, {ParameterName}_NumElements) << 2;
		{ParameterName}_Buffer.InterlockedMax(ElementOffset, Value, PreviousValue);
		CurrentValue = max(PreviousValue, Value);
	}
}

void AtomicMin_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, int Value, out int PreviousValue, out int CurrentValue)
{
	if (bExecute)
	{
		ElementOffset = clamp(ElementOffset, 0, {ParameterName}_NumElements) << 2;
		{ParameterName}_Buffer.InterlockedMin(ElementOffset, Value, PreviousValue);
		CurrentValue = min(PreviousValue, Value);
	}
}

void AtomicOr_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, int Value, out int PreviousValue, out int CurrentValue)
{
	if (bExecute)
	{
		ElementOffset = clamp(ElementOffset, 0, {ParameterName}_NumElements) << 2;
		{ParameterName}_Buffer.InterlockedOr(ElementOffset, Value, PreviousValue);
		CurrentValue = PreviousValue | Value;
	}
}

void AtomicXor_{ParameterName}_UEImpureCall(bool bExecute, int ElementOffset, int Value, out int PreviousValue, out int CurrentValue)
{
	if (bExecute)
	{
		ElementOffset = clamp(ElementOffset, 0, {ParameterName}_NumElements) << 2;
		{ParameterName}_Buffer.InterlockedXor(ElementOffset, Value, PreviousValue);
		CurrentValue = PreviousValue ^ Value;
	}
}
