// Copyright Epic Games, Inc. All Rights Reserved.

uint			{ParameterName}_bSubImageBlend;
float2			{ParameterName}_SubImageSize;
uint			{ParameterName}_NumMeshes;
Buffer<float>	{ParameterName}_MeshDataBuffer;

void GetNumMeshes_{ParameterName}(out int OutNumMeshes)
{
	OutNumMeshes = int({ParameterName}_NumMeshes);
}

void GetMeshLocalBounds_{ParameterName}(int MeshIndex, out float3 OutMinBounds, out float3 OutMaxBounds, out float3 OutSize)
{
	OutMinBounds = (float3)0;
	OutMaxBounds = (float3)0;
	OutSize = (float3)0;
	if ( {ParameterName}_NumMeshes > 0 )
	{
		const uint MeshDataNumFloats = 6;
		const uint BufferOffs = clamp(MeshIndex, 0, int({ParameterName}_NumMeshes - 1)) * MeshDataNumFloats;
		OutMinBounds = float3(
			{ParameterName}_MeshDataBuffer[BufferOffs + 0],
			{ParameterName}_MeshDataBuffer[BufferOffs + 1],
			{ParameterName}_MeshDataBuffer[BufferOffs + 2]
		);
		OutMaxBounds = float3(
			{ParameterName}_MeshDataBuffer[BufferOffs + 3],
			{ParameterName}_MeshDataBuffer[BufferOffs + 4],
			{ParameterName}_MeshDataBuffer[BufferOffs + 5]
		);
		OutSize = OutMaxBounds - OutMinBounds;
	}
}

void GetSubUVDetails_{ParameterName}(out bool bBlendEnabled, out float2 SubImageSize)
{
	bBlendEnabled = {ParameterName}_bSubImageBlend != 0;
	SubImageSize = {ParameterName}_SubImageSize;
}
