// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDataInterfaceNeighborQuery.ush
=============================================================================*/

/* -----------------------------------------------------------------
 * Parameter declarations (bound by FShaderParameters / SetShaderParameters)
 * -----------------------------------------------------------------addpar
 */

int				{ParameterName}_MaxCellsPerParticleValue;

Buffer<int>		{ParameterName}_CellCount;
Buffer<int>		{ParameterName}_CellOffset;
Buffer<int>		{ParameterName}_ParticleList;
RWBuffer<int>	{ParameterName}_OutputCellId;
RWBuffer<int>	{ParameterName}_OutputParticleIdIndex;
RWBuffer<int>	{ParameterName}_OutputAcquireTag;
Buffer<int>		{ParameterName}_AcquireTagList;

#ifndef NEIGHBORQUERY_INVALID_CELL
#define NEIGHBORQUERY_INVALID_CELL 0xFFFFFFFF
#endif

// Intra-cell sort key: particle distance to NQ cell center is quantized and packed
// into the low NQ_DIST_BITS bits of the OutputCellId slot. The bitonic scatter pass
// sorts by the combined key, placing particles in ParticleListBuffer ordered
// closest-to-cell-center first within each cell. The SDF gather stage exploits
// this ordering for an inner-loop early-exit.
//
// Combined key layout (32 bits):  [CellId : (32-NQ_DIST_BITS) high bits][QuantizedDist : NQ_DIST_BITS low bits]
// QuantizedDist is capped at NQ_DIST_MAX (= 2^NQ_DIST_BITS - 2); the next value up is never
// written, so no valid combined key can equal NEIGHBORQUERY_INVALID_CELL (0xFFFFFFFF).
// When NQ_DIST_BITS == 0 the cell ID is stored directly with no distance packing.
//
// NQ_DIST_BITS trade-off (step size = 0.866 cell / NQ_DIST_MAX, i.e. half-diagonal / levels):
//
//   Bits | CellId bits | Max cells | Max cubic  | Dist levels | Step size
//   -----+-------------+-----------+------------+-------------+----------
//     0  |     32      |    ~4G    |  ~1626^3   |      —      |    —      (no dist sort)
//     4  |     28      |   ~268M   |  ~644^3    |     14      | ~6.2% cell
//     5  |     27      |   ~134M   |  ~511^3    |     30      | ~2.9% cell
//     6  |     26      |    ~67M   |  ~405^3    |     62      | ~1.4% cell  (default)
//     7  |     25      |    ~33M   |  ~322^3    |    126      | ~0.7% cell
//     8  |     24      |    ~16M   |  ~252^3    |    254      | ~0.3% cell
//
// NQ_DIST_BITS is injected by GetParameterDefinitionHLSL from
// NiagaraNeighborQuerySort::NQDistBits (single source of truth in NiagaraNeighborQuerySort.h).
// Fallback default of 0 disables distance packing (full 32-bit cell ID).

#ifndef NQ_DIST_BITS
#define NQ_DIST_BITS 0
#endif
#define NQ_DIST_MASK ((1 << NQ_DIST_BITS) - 1)
#define NQ_DIST_MAX  ((1 << NQ_DIST_BITS) - 2)  // highest valid quantized value (e.g. 62 when NQ_DIST_BITS=6)

// Quantize squared distance from IndexPos to CellCoord (NQ index-space units) → [0, NQ_DIST_MAX].
// 0.75 = (sqrt(3)/2)^2 = squared half-diagonal of a unit NQ cell (max for round-based assignment).
// Squared distance avoids sqrt; monotonic so sort order is preserved.
// Multi-cell variants may exceed this; they saturate to NQ_DIST_MAX, sorting last — correct.
uint ComputeNQSqDistQuantized_{ParameterName}(float3 IndexPos, int3 CellCoord)
{
	float3 Delta = IndexPos - float3(CellCoord);
	float Dist2 = dot(Delta, Delta);
	return min((uint)NQ_DIST_MAX, (uint)(Dist2 * ((float)NQ_DIST_MAX / 3.0f)));
}

/* -----------------------------------------------------------------
 * Internal helpers (not exposed to users)
 * -----------------------------------------------------------------
 */

int IndexToLinear_{ParameterName}(int3 CellCoord)
{
	return CellCoord.x
		+ CellCoord.y * {ParameterName}_NumCells.x
		+ CellCoord.z * {ParameterName}_NumCells.x * {ParameterName}_NumCells.y;
}

// Write one particle assignment into OutputCellId/ParticleIdIndex/AcquireTag at SlotIndex.
// When NQ_DIST_BITS > 0 the combined key packs the quantized dist-to-cell-center into the
// low bits so the bitonic scatter produces closest-first order within each cell.
// When NQ_DIST_BITS == 0 the cell ID is stored directly (no dist cost, no sort key overhead).
void WriteParticleSlot_{ParameterName}(int SlotIndex, int LinearIndex, float3 IndexPos, int3 CellCoord, int ParticleIndex, int AcquireTag)
{
#if NQ_DIST_BITS == 0
	{ParameterName}_OutputCellId[SlotIndex] = LinearIndex;
#else
	uint CombinedKey = ((uint)LinearIndex << NQ_DIST_BITS) | ComputeNQSqDistQuantized_{ParameterName}(IndexPos, CellCoord);
	{ParameterName}_OutputCellId[SlotIndex] = (int)CombinedKey;
#endif
	{ParameterName}_OutputParticleIdIndex[SlotIndex] = ParticleIndex;
	{ParameterName}_OutputAcquireTag[SlotIndex] = AcquireTag;
}

/* -----------------------------------------------------------------
 * Grid property queries
 * -----------------------------------------------------------------
 */
void MaxCellsPerParticle_{ParameterName}(out int OutMaxCellsPerParticle)
{
	OutMaxCellsPerParticle = {ParameterName}_MaxCellsPerParticleValue;
}

/* -----------------------------------------------------------------
 * Write functions (output stage — pass 1 cell assignment)
 * -----------------------------------------------------------------
 */

void AddParticle_{ParameterName}(bool Enabled, float3 IndexPos, int ParticleIndex, int AcquireTag, out bool Success)
{
	Success = false;
	if (!Enabled) return;

	int3 CellCoord = int3(floor(IndexPos));

	if (all(CellCoord >= 0) && all(CellCoord < {ParameterName}_NumCells))
	{
		// Slot is addressed by ExecIndex() (compact, always in [0, NumParticles)).
		// ParticleIndex (ID.Index) is stored separately for the sort to carry through.
		int SlotBase = ExecIndex() * {ParameterName}_MaxCellsPerParticleValue;
		WriteParticleSlot_{ParameterName}(SlotBase, IndexToLinear_{ParameterName}(CellCoord), IndexPos, CellCoord, ParticleIndex, AcquireTag);

		Success = true;
	}
}

// Exec-pin wrapper — Niagara appends _UEImpureCall to the generated function name for bRequiresExecPin functions
void AddParticle_{ParameterName}_UEImpureCall(bool In_Enabled, float3 In_IndexPos, int In_ParticleIndex, int In_AcquireTag, out bool Out_Success)
{
	AddParticle_{ParameterName}(In_Enabled, In_IndexPos, In_ParticleIndex, In_AcquireTag, Out_Success);
}

void AddParticleWithRadius_{ParameterName}(bool Enabled, float3 IndexPos, float Radius, int ParticleIndex, int AcquireTag, out bool Success)
{
	Success = false;
	if (!Enabled) return;

	int3 MinCell = clamp(int3(floor(IndexPos - Radius)), 0, {ParameterName}_NumCells - 1);
	int3 MaxCell = clamp(int3(floor(IndexPos + Radius)), 0, {ParameterName}_NumCells - 1);

	int SlotBase = ExecIndex() * {ParameterName}_MaxCellsPerParticleValue;
	int Slot = 0;

	for (int z = MinCell.z; z <= MaxCell.z; z++) {
	for (int y = MinCell.y; y <= MaxCell.y; y++) {
	for (int x = MinCell.x; x <= MaxCell.x; x++) {
		if (Slot < {ParameterName}_MaxCellsPerParticleValue)
		{
			int3 CellCoord3 = int3(x, y, z);
			WriteParticleSlot_{ParameterName}(SlotBase + Slot, IndexToLinear_{ParameterName}(CellCoord3), IndexPos, CellCoord3, ParticleIndex, AcquireTag);
			Slot++;
			Success = true;
		}
	}}}
}

// Exec-pin wrapper
void AddParticleWithRadius_{ParameterName}_UEImpureCall(bool In_Enabled, float3 In_IndexPos, float In_Radius, int In_ParticleIndex, int In_AcquireTag, out bool Out_Success)
{
	AddParticleWithRadius_{ParameterName}(In_Enabled, In_IndexPos, In_Radius, In_ParticleIndex, In_AcquireTag, Out_Success);
}

void AddParticleToOverlappedCells_{ParameterName}(bool Enabled, float3 IndexPos, int ParticleIndex, int AcquireTag, out bool Success)
{
	Success = false;
	if (!Enabled) return;

	int3 BaseCell = int3(floor(IndexPos));
	int SlotBase = ExecIndex() * {ParameterName}_MaxCellsPerParticleValue;
	int Slot = 0;

	for (int z = 0; z <= 1; z++) {
	for (int y = 0; y <= 1; y++) {
	for (int x = 0; x <= 1; x++) {
		int3 CellCoord = BaseCell + int3(x, y, z);
		if (Slot < {ParameterName}_MaxCellsPerParticleValue &&
			all(CellCoord >= 0) && all(CellCoord < {ParameterName}_NumCells))
		{
			WriteParticleSlot_{ParameterName}(SlotBase + Slot, IndexToLinear_{ParameterName}(CellCoord), IndexPos, CellCoord, ParticleIndex, AcquireTag);
			Slot++;
			Success = true;
		}
	}}}
}

// Exec-pin wrapper
void AddParticleToOverlappedCells_{ParameterName}_UEImpureCall(bool In_Enabled, float3 In_IndexPos, int In_ParticleIndex, int In_AcquireTag, out bool Out_Success)
{
	AddParticleToOverlappedCells_{ParameterName}(In_Enabled, In_IndexPos, In_ParticleIndex, In_AcquireTag, Out_Success);
}

void AddParticleToNeighborCells_{ParameterName}(bool Enabled, float3 IndexPos, int ParticleIndex, int AcquireTag, out bool Success)
{
	Success = false;
	if (!Enabled) return;

	int3 BaseCell = int3(floor(IndexPos));
	int SlotBase = ExecIndex() * {ParameterName}_MaxCellsPerParticleValue;
	int Slot = 0;

	for (int z = -1; z <= 1; z++) {
	for (int y = -1; y <= 1; y++) {
	for (int x = -1; x <= 1; x++) {
		int3 CellCoord = BaseCell + int3(x, y, z);
		if (Slot < {ParameterName}_MaxCellsPerParticleValue &&
			all(CellCoord >= 0) && all(CellCoord < {ParameterName}_NumCells))
		{
			WriteParticleSlot_{ParameterName}(SlotBase + Slot, IndexToLinear_{ParameterName}(CellCoord), IndexPos, CellCoord, ParticleIndex, AcquireTag);
			Slot++;
			Success = true;
		}
	}}}
}

// Exec-pin wrapper
void AddParticleToNeighborCells_{ParameterName}_UEImpureCall(bool In_Enabled, float3 In_IndexPos, int In_ParticleIndex, int In_AcquireTag, out bool Out_Success)
{
	AddParticleToNeighborCells_{ParameterName}(In_Enabled, In_IndexPos, In_ParticleIndex, In_AcquireTag, Out_Success);
}

/* -----------------------------------------------------------------
 * Read functions (read stage — after counting sort passes 2-4)
 * -----------------------------------------------------------------
 */

void GetParticleNeighborCount_{ParameterName}(int IndexX, int IndexY, int IndexZ, out int OutCount)
{
	int3 CellCoord = int3(IndexX, IndexY, IndexZ);
	if (any(CellCoord < 0) || any(CellCoord >= {ParameterName}_NumCells))
	{
		OutCount = 0;
		return;
	}
	int LinearIndex = IndexToLinear_{ParameterName}(CellCoord);
	OutCount = {ParameterName}_CellCount[LinearIndex];
}

void GetParticleNeighbor_{ParameterName}(int IndexX, int IndexY, int IndexZ, int NeighborIdx, out int OutParticleIdx, out int OutAcquireTag)
{
	int3 CellCoord = int3(IndexX, IndexY, IndexZ);
	bool bValidCell = all(CellCoord >= 0) && all(CellCoord < {ParameterName}_NumCells);
	int LinearIndex = bValidCell ? IndexToLinear_{ParameterName}(CellCoord) : 0;
	int Count = bValidCell ? {ParameterName}_CellCount[LinearIndex] : 0;
	if (!bValidCell || NeighborIdx >= Count || NeighborIdx < 0)
	{
		OutParticleIdx = -1;
		OutAcquireTag = -1;
		return;
	}
	// CellOffset was consumed by the scatter pass (atomicAdd), so the original prefix sum value
	// is CellOffset[i] - CellCount[i].
	int StartOffset = {ParameterName}_CellOffset[LinearIndex] - Count;
	OutParticleIdx = {ParameterName}_ParticleList[StartOffset + NeighborIdx];
	OutAcquireTag = {ParameterName}_AcquireTagList[StartOffset + NeighborIdx];
}
