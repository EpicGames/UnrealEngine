// Copyright Epic Games, Inc. All Rights Reserved.

#define AP_SUPPORT_SAMPLE_ATMOSHPERE 0
#include "/Engine/Private/SkyAtmosphereCommon.ush"

float2				{ParameterName}_CloudVolumetricTextureUVScale;
float2				{ParameterName}_CloudVolumetricTextureUVMax;
Texture2D<float4>	{ParameterName}_CloudVolumetricTexture;
SamplerState		{ParameterName}_CloudVolumetricTextureSampler;

void QueryOcclusionFactorWithRectangle_{ParameterName}(in float3 In_SampleCenterWorldPos, in float In_SampleWindowWidthWorld, in float In_SampleWindowHeightWorld, in float In_SampleSteps, out float Out_VisibilityFraction, out float Out_SampleFraction)
{
	FLWCVector3 LwcSamplePos = MakeLWCVector3(GetEngineOwnerLWCTile(), In_SampleCenterWorldPos);
	float3 ResolvedLwcSamplePos = DFFastToTranslatedWorld(DFFromTileOffset_Hack(LwcSamplePos), PrimaryView.PreViewTranslation);
	const float3 TranslatedWorldViewOrigin = DFFastToTranslatedWorld(PrimaryView.WorldViewOrigin, PrimaryView.PreViewTranslation);
	float CameraDistance = abs(dot(ResolvedLwcSamplePos - TranslatedWorldViewOrigin, View.ViewForward.xyz));

	float4 SamplePosition = float4(ResolvedLwcSamplePos, 1);
	float4 ClipPosition = mul(SamplePosition, PrimaryView.TranslatedWorldToClip);
	float2 ScreenPosition = ClipPosition.xy / ClipPosition.w;
	float2 ScreenUV = ScreenPosition * PrimaryView.ScreenPositionScaleBias.xy + PrimaryView.ScreenPositionScaleBias.wz;

	float Steps = In_SampleSteps <= 1 ? 0 : In_SampleSteps;
	float TotalSamples = 0;
	float OccludedSamples = 0;

	float4 SampleWidthClip = mul(float4(PrimaryView.ViewRight * In_SampleWindowWidthWorld, 0) + SamplePosition, PrimaryView.TranslatedWorldToClip);
	float4 SampleHeightClip = mul(float4(PrimaryView.ViewUp * In_SampleWindowHeightWorld, 0) + SamplePosition, PrimaryView.TranslatedWorldToClip);
				
	float2 SampleWidthUV = SampleWidthClip.xy / SampleWidthClip.w * PrimaryView.ScreenPositionScaleBias.xy + PrimaryView.ScreenPositionScaleBias.wz;
	float2 SampleHeightUV = SampleHeightClip.xy / SampleHeightClip.w * PrimaryView.ScreenPositionScaleBias.xy + PrimaryView.ScreenPositionScaleBias.wz;
				
	float SampleWidth = ScreenUV.x > 1 ? 0 : SampleWidthUV.x - ScreenUV.x;
	float SampleHeight = ScreenUV.y > 1 ? 0 : SampleHeightUV.y - ScreenUV.y;

	if (Steps > 0) 
	{
		const float StepScale = 1.0f / (Steps - 1);
		for (int ys = 0; ys < Steps; ys++)
		{
			float SampleY = ScreenUV.y + SampleHeight * (ys * StepScale - 0.5f);
			if (SampleY <= 1 && SampleY >= 0)
			{
				for (int xs = 0; xs < Steps; xs++)
				{
					float SampleX = ScreenUV.x + SampleWidth * (xs * StepScale - 0.5f);
					if (SampleX <= 1 && SampleX >= 0)
					{
						float Depth = CalcSceneDepth(float2(SampleX, SampleY));
						if (Depth < CameraDistance) 
						{
							OccludedSamples++;
						}
						TotalSamples++;
					}
				}
			} 
		}
	}

	Out_VisibilityFraction = TotalSamples > 0 ? 1 - OccludedSamples / TotalSamples : 0;
	Out_SampleFraction = Steps == 0 ? 0 : (TotalSamples / (Steps * Steps));
}

void QueryOcclusionFactorWithRectangleGPU_{ParameterName}(in float3 In_SampleCenterWorldPos, in float In_SampleWindowWidthWorld, in float In_SampleWindowHeightWorld, in float In_SampleSteps, out float Out_VisibilityFraction, out float Out_SampleFraction)
{
	QueryOcclusionFactorWithRectangle_{ParameterName}(In_SampleCenterWorldPos, In_SampleWindowWidthWorld, In_SampleWindowHeightWorld, In_SampleSteps, Out_VisibilityFraction, Out_SampleFraction);
}

void QueryOcclusionFactorWithCircleGPU_{ParameterName}(in float3 In_SampleCenterWorldPos, in float In_SampleWindowDiameterWorld, in float In_SampleRays, in float In_SampleStepsPerRay, out float Out_VisibilityFraction, out float Out_SampleFraction)
{
	const float PI = 3.14159265;
	const float SPIRAL_TURN = 2 * PI * 0.61803399; // use golden ratio to rotate sample pattern each ring so we get a spiral
	FLWCVector3 LwcSamplePos = MakeLWCVector3(GetEngineOwnerLWCTile(), In_SampleCenterWorldPos);
	float3 ResolvedLwcSamplePos = DFFastToTranslatedWorld(DFFromTileOffset_Hack(LwcSamplePos), PrimaryView.PreViewTranslation);
	const float3 TranslatedWorldViewOrigin = DFFastToTranslatedWorld(PrimaryView.WorldViewOrigin, PrimaryView.PreViewTranslation);
	float CameraDistance = abs(dot(ResolvedLwcSamplePos - TranslatedWorldViewOrigin, View.ViewForward.xyz));

	float4 SamplePosition = float4(ResolvedLwcSamplePos, 1);
	float4 ClipPosition = mul(SamplePosition, PrimaryView.TranslatedWorldToClip);
	float2 ScreenPosition = ClipPosition.xy / ClipPosition.w;
	float2 ScreenUV = ScreenPosition * PrimaryView.ScreenPositionScaleBias.xy + PrimaryView.ScreenPositionScaleBias.wz;

	float Rays = In_SampleRays <= 1 ? 0 : In_SampleRays;
	float Steps = In_SampleStepsPerRay < 1 ? 0 : In_SampleStepsPerRay;
	float TotalSamples = 0;
	float OccludedSamples = 0;

	if (ScreenUV.x <= 1 && ScreenUV.x >= 0 && ScreenUV.y <= 1 && ScreenUV.y >= 0)
	{
		float Depth = CalcSceneDepth(ScreenUV);
		if (Depth < CameraDistance) 
		{
			OccludedSamples++;
		}
		TotalSamples++;
	}
	if (Steps > 0) 
	{
		float Degrees = 0;
		for (int Step = 1; Step <= Steps; Step++)
		{
			float LerpFactor = Step / Steps;
			Degrees += SPIRAL_TURN;
			for (int ray = 0; ray < Rays; ray++)
			{
				// calc ray direction vector
				float3 RayDirection = cos(Degrees) * PrimaryView.ViewUp + sin(Degrees) * PrimaryView.ViewRight;
				float4 RayClip = mul(float4(RayDirection * In_SampleWindowDiameterWorld / 2, 0) + SamplePosition, PrimaryView.TranslatedWorldToClip);
				float2 RayUV = RayClip.xy / RayClip.w * PrimaryView.ScreenPositionScaleBias.xy + PrimaryView.ScreenPositionScaleBias.wz;

				if ((ScreenUV.x > 1 && RayUV.x < 0) || (ScreenUV.y > 1 && RayUV.y < 0) || (ScreenUV.x < 0 && RayUV.x > 1) || (ScreenUV.y < 0 && RayUV.y > 1))
				{
					continue;
				}
						
				float2 SampleUV = lerp(ScreenUV, RayUV, float2(LerpFactor, LerpFactor));
							
				if (SampleUV.x > 1 || SampleUV.x < 0 || SampleUV.y > 1 || SampleUV.y < 0)
				{
					continue;
				}
			
				float Depth = CalcSceneDepth(SampleUV);
				if (Depth < CameraDistance) 
				{
					OccludedSamples++;
				}
				TotalSamples++;
				Degrees += 2 * PI / Rays;
			}						
		}
	}
	Out_VisibilityFraction = TotalSamples > 0 ? 1 - OccludedSamples / TotalSamples : 0;
	Out_SampleFraction = Steps == 0 ? 0 : (TotalSamples / (Rays * Steps + 1));
}

void QueryOcclusionFactorWithCircle_{ParameterName}(float3 WorldPosition, float WorldDiameter, bool bIncludeCenterSample, int NumberOfRings, int SamplesPerRing, out float VisibilityFraction, out float OnScreenFraction, out float DepthPassFraction)
{
	const float PI = 3.14159265;
	const float SPIRAL_TURN = 2.0f * PI * 0.61803399; // use golden ratio to rotate sample pattern each ring so we get a spiral

	const FLWCVector3 LWCWorldPosition = MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition);
	const float3 PreViewPosition = DFFastToTranslatedWorld(DFFromTileOffset_Hack(LWCWorldPosition), PrimaryView.PreViewTranslation);
	const float3 TranslatedWorldViewOrigin = DFFastToTranslatedWorld(PrimaryView.WorldViewOrigin, PrimaryView.PreViewTranslation);
	const float SampleDepth = abs(dot(PreViewPosition - TranslatedWorldViewOrigin, View.ViewForward.xyz));
	
	const float4 ClipPosition = mul(float4(PreViewPosition, 1), PrimaryView.TranslatedWorldToClip);
	const float2 ScreenPosition = ClipPosition.xy / ClipPosition.w;
	const float2 ScreenUV = (ScreenPosition * PrimaryView.ScreenPositionScaleBias.xy) + PrimaryView.ScreenPositionScaleBias.wz;

	int TotalSamples = 0;
	int UnoccludedSamples = 0;
	int OnScreenSamples = 0;

	// Take center tap
	if ( bIncludeCenterSample )
	{
		++TotalSamples;
		if ( all(ScreenUV >= 0.0f) && all(ScreenUV < 1.0f) )
		{
			UnoccludedSamples += CalcSceneDepth(ScreenUV) > SampleDepth;
			++OnScreenSamples;
		}
	}

	// Take ring taps
	if ( NumberOfRings > 0 && SamplesPerRing > 0 )
	{
		const float WorldDepth = dot(PreViewPosition - PrimaryView.TranslatedWorldCameraOrigin, PrimaryView.ViewForward);
		const float2 ScreenRadiusUV = (WorldDiameter * 0.5f) / (GetTanHalfFieldOfView() * 2.0f * WorldDepth);

		const float RadStep = 2.0f * PI / float(SamplesPerRing + 1);
		float Rad = 0.0f;
		for ( int iRing=0; iRing < NumberOfRings; ++iRing )
		{
			const float2 RingRadiusUV = ScreenRadiusUV * (float(iRing + 1) / float(NumberOfRings));
			Rad += SPIRAL_TURN;
			for ( int iSample=0; iSample < SamplesPerRing; ++iSample )
			{
				const float2 SampleUV = ScreenUV + (RingRadiusUV * float2(sin(Rad), cos(Rad)));
				if ( all(SampleUV >= 0.0f) && all(SampleUV < 1.0f) )
				{
					UnoccludedSamples += CalcSceneDepth(SampleUV) > SampleDepth;
					++OnScreenSamples;
				}
				Rad += RadStep;
			}
		}
		TotalSamples += NumberOfRings * SamplesPerRing;
	}

	VisibilityFraction = TotalSamples > 0 ? float(UnoccludedSamples) / float(TotalSamples) : 0.0f;
	OnScreenFraction = TotalSamples > 0 ? float(OnScreenSamples) / float(TotalSamples) : 0.0f;
	DepthPassFraction = OnScreenSamples > 0 ? float(UnoccludedSamples) / float(OnScreenSamples) : 0.0f;
}

void QueryCloudOcclusionWithCircle_{ParameterName}(float3 WorldPosition, float WorldDiameter, bool bIncludeCenterSample, int NumberOfRings, int SamplesPerRing, out float VisibilityFraction, out float SampleFraction, out float3 AtmosphereTransmittance)
{
	const float PI = 3.14159265;
	const float SPIRAL_TURN = 2.0f * PI * 0.61803399; // use golden ratio to rotate sample pattern each ring so we get a spiral

	const FLWCVector3 LWCWorldPosition = MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition);
	const float3 PreViewPosition = DFFastToTranslatedWorld(DFFromTileOffset_Hack(LWCWorldPosition), PrimaryView.PreViewTranslation);
	const float4 ClipPosition = mul(float4(PreViewPosition, 1), PrimaryView.TranslatedWorldToClip);
	const float2 ScreenPosition = ClipPosition.xy / ClipPosition.w;
	const float2 ScreenUV = (ScreenPosition * float2(0.5f, -0.5f)) + 0.5f;

	float TotalSamples = 0.0f;
	float TotalVisibility = 0.0f;

	// Take center tap
	float ExpectedSamples = 0.0f;
	if ( bIncludeCenterSample && (all(ScreenUV >= 0.0f) && all(ScreenUV < 1.0f)) )
	{
		const float4 SampleColor = {ParameterName}_CloudVolumetricTexture.SampleLevel(
			{ParameterName}_CloudVolumetricTextureSampler, 
			min(ScreenUV * {ParameterName}_CloudVolumetricTextureUVScale, {ParameterName}_CloudVolumetricTextureUVMax),
			0);

		TotalVisibility += SampleColor.a;
		TotalSamples += 1.0f;
		ExpectedSamples += 1.0f;
	}

	// Take ring taps
	if ( NumberOfRings > 0 && SamplesPerRing > 0 )
	{
		const float3 TranslatedWorldViewOrigin = DFFastToTranslatedWorld(PrimaryView.WorldViewOrigin, PrimaryView.PreViewTranslation);
		const float WorldDepth = dot(PreViewPosition - TranslatedWorldViewOrigin, View.ViewForward.xyz);
		const float2 ScreenRadiusUV = (WorldDiameter * 0.5f) / (GetTanHalfFieldOfView() * 2.0f * WorldDepth);

		const float RadStep = 2.0f * PI / float(SamplesPerRing + 1);
		float Rad = 0.0f;
		ExpectedSamples += float(NumberOfRings * SamplesPerRing);
		for ( int iRing=0; iRing < NumberOfRings; ++iRing )
		{
			const float2 RingRadiusUV = ScreenRadiusUV * (float(iRing + 1) / float(NumberOfRings));
			Rad += SPIRAL_TURN;
			for ( int iSample=0; iSample < SamplesPerRing; ++iSample )
			{
				const float2 SampleUV = ScreenUV + (RingRadiusUV * float2(sin(Rad), cos(Rad)));
				if ( all(SampleUV >= 0.0f) && all(SampleUV < 1.0f) )
				{
					const float4 SampleColor = {ParameterName}_CloudVolumetricTexture.SampleLevel(
						{ParameterName}_CloudVolumetricTextureSampler, 
						min(SampleUV * {ParameterName}_CloudVolumetricTextureUVScale, {ParameterName}_CloudVolumetricTextureUVMax),
						0);

					TotalVisibility += SampleColor.a;
					TotalSamples += 1.0f;
				}
				Rad += RadStep;
			}
		}
	}

	VisibilityFraction = TotalSamples > 0.0f ? TotalVisibility / TotalSamples : 0.0f;
	SampleFraction = TotalSamples > 0.0f ? TotalSamples / ExpectedSamples : 0.0f;

	// Sample Atmosphere Transmittance
	{
		const float3 RayDirection = normalize(PreViewPosition);

		const float3 PlanetCenterToTranslatedWorldPos = (PreViewPosition - PrimaryView.SkyPlanetTranslatedWorldCenterAndViewHeight.xyz) * CM_TO_SKY_UNIT;
		AtmosphereTransmittance = GetAtmosphereTransmittance(
			PlanetCenterToTranslatedWorldPos,
			RayDirection,
			PrimaryView.SkyAtmosphereBottomRadiusKm, PrimaryView.SkyAtmosphereTopRadiusKm,
			View.TransmittanceLutTexture, View.TransmittanceLutTextureSampler
		);
	}
}
