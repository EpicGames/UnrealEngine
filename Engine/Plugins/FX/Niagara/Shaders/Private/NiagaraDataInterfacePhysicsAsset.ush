// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDataInterfacePhysicsAsset.ush 
=============================================================================*/

/* -----------------------------------------------------------------
 * Physics Asset constants and context
 * -----------------------------------------------------------------
 */

#define PHYSICS_ASSET_MAX_TRANSFORMS 300

#define CURRENT_OFFSET 0
#define PREVIOUS_OFFSET PHYSICS_ASSET_MAX_TRANSFORMS
#define REST_OFFSET 2*PHYSICS_ASSET_MAX_TRANSFORMS

#define BOX_INDEX 0
#define SPHERE_INDEX 1
#define CAPSULE_INDEX 2

uint4				{ParameterName}_ElementOffsets;
Buffer<float4>		{ParameterName}_WorldTransformBuffer;
Buffer<float4>		{ParameterName}_InverseTransformBuffer;
Buffer<float4>		{ParameterName}_ElementExtentBuffer;
Buffer<uint>		{ParameterName}_PhysicsTypeBuffer;
float3				{ParameterName}_BoxOrigin;
float3				{ParameterName}_BoxExtent;

/* -----------------------------------------------------------------
 * Element accessors
 * -----------------------------------------------------------------
 */

void GetNumBoxes_{ParameterName}(out int OutNumBoxes)
{
	OutNumBoxes = {ParameterName}_ElementOffsets[BOX_INDEX+1] - {ParameterName}_ElementOffsets[BOX_INDEX];
}

void GetNumSpheres_{ParameterName}(out int OutNumSpheres)
{
	OutNumSpheres = {ParameterName}_ElementOffsets[SPHERE_INDEX+1] - {ParameterName}_ElementOffsets[SPHERE_INDEX];
}

void GetNumCapsules_{ParameterName}(out int OutNumCapsules)
{
	OutNumCapsules = {ParameterName}_ElementOffsets[CAPSULE_INDEX+1] - {ParameterName}_ElementOffsets[CAPSULE_INDEX];
}


/* -----------------------------------------------------------------
 * Position / Transform utils
 * -----------------------------------------------------------------
 */

float3 GetCurrentPosition_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + CURRENT_OFFSET;
    return float3({ParameterName}_WorldTransformBuffer[BufferOffset].w, {ParameterName}_WorldTransformBuffer[BufferOffset+1].w, {ParameterName}_WorldTransformBuffer[BufferOffset+2].w);
}

float3 GetPreviousPosition_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + PREVIOUS_OFFSET;
    return float3({ParameterName}_WorldTransformBuffer[BufferOffset].w, {ParameterName}_WorldTransformBuffer[BufferOffset+1].w, {ParameterName}_WorldTransformBuffer[BufferOffset+2].w);
}

float4 GetCurrentOrientation_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + CURRENT_OFFSET;
	const float3 OrientationMatrix[3] = {float3({ParameterName}_WorldTransformBuffer[BufferOffset].x, {ParameterName}_WorldTransformBuffer[BufferOffset+1].x, {ParameterName}_WorldTransformBuffer[BufferOffset+2].x),
									     float3({ParameterName}_WorldTransformBuffer[BufferOffset].y, {ParameterName}_WorldTransformBuffer[BufferOffset+1].y, {ParameterName}_WorldTransformBuffer[BufferOffset+2].y),
										 float3({ParameterName}_WorldTransformBuffer[BufferOffset].z, {ParameterName}_WorldTransformBuffer[BufferOffset+1].z, {ParameterName}_WorldTransformBuffer[BufferOffset+2].z)};


	return QuatFromMatrix(OrientationMatrix);
}

float4 GetPreviousOrientation_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + PREVIOUS_OFFSET;
	const float3 OrientationMatrix[3] = {float3({ParameterName}_WorldTransformBuffer[BufferOffset].x, {ParameterName}_WorldTransformBuffer[BufferOffset+1].x, {ParameterName}_WorldTransformBuffer[BufferOffset+2].x),
										 float3({ParameterName}_WorldTransformBuffer[BufferOffset].y, {ParameterName}_WorldTransformBuffer[BufferOffset+1].y, {ParameterName}_WorldTransformBuffer[BufferOffset+2].y),
										 float3({ParameterName}_WorldTransformBuffer[BufferOffset].z, {ParameterName}_WorldTransformBuffer[BufferOffset+1].z, {ParameterName}_WorldTransformBuffer[BufferOffset+2].z)};

	return QuatFromMatrix(OrientationMatrix);
}

float3x4 GetCurrentTransform_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + CURRENT_OFFSET;
    return float3x4({ParameterName}_WorldTransformBuffer[BufferOffset], {ParameterName}_WorldTransformBuffer[BufferOffset+1], {ParameterName}_WorldTransformBuffer[BufferOffset+2]);
}

float3x4 GetInverseTransform_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + CURRENT_OFFSET;
    return float3x4({ParameterName}_InverseTransformBuffer[BufferOffset], {ParameterName}_InverseTransformBuffer[BufferOffset+1], {ParameterName}_InverseTransformBuffer[BufferOffset+2]);
}

float3x4 GetPreviousTransform_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + PREVIOUS_OFFSET;
    return float3x4({ParameterName}_WorldTransformBuffer[BufferOffset], {ParameterName}_WorldTransformBuffer[BufferOffset+1], {ParameterName}_WorldTransformBuffer[BufferOffset+2]);
}

float3x4 GetPreviousInverse_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + PREVIOUS_OFFSET;
    return float3x4({ParameterName}_InverseTransformBuffer[BufferOffset], {ParameterName}_InverseTransformBuffer[BufferOffset+1], {ParameterName}_InverseTransformBuffer[BufferOffset+2]);
}

float3x4 GetRestTransform_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + REST_OFFSET;
    return float3x4({ParameterName}_WorldTransformBuffer[BufferOffset], {ParameterName}_WorldTransformBuffer[BufferOffset+1], {ParameterName}_WorldTransformBuffer[BufferOffset+2]);
}

float3x4 GetRestInverse_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + REST_OFFSET;
    return float3x4({ParameterName}_InverseTransformBuffer[BufferOffset], {ParameterName}_InverseTransformBuffer[BufferOffset+1], {ParameterName}_InverseTransformBuffer[BufferOffset+2]);
}

/* -----------------------------------------------------------------
 * Collision detection utils
 * -----------------------------------------------------------------
 */

// Given a world space position (NodePosition) compute the sphere closest point (position,normal,velocity)
float GetSphereProjection_{ParameterName}(in float3 LocalPosition, in float3 SphereCenter, in float SphereRadius, in int SphereIndex,
			inout float3 OutClosestPosition, inout float3 OutClosestNormal, inout int OutElementIndex, inout float OutMinDistance)
{
	const float3 DeltaPosition = LocalPosition - SphereCenter;
	const float DeltaLength = length(DeltaPosition);
	const float SphereDistance = DeltaLength - SphereRadius;

	if (SphereDistance < OutMinDistance)
	{
		OutMinDistance = SphereDistance;
		OutElementIndex = SphereIndex;

		OutClosestNormal = (DeltaLength > SMALL_NUMBER) ? DeltaPosition / DeltaLength : float3(0,0,0);
		OutClosestPosition = LocalPosition - OutClosestNormal*SphereDistance;
	}
	return SphereDistance;
}

// Given a world space position (NodePosition) compute the sphere closest point (position,normal,velocity)
float GetCapsuleProjection_{ParameterName}(in float3 LocalPosition, in float2 CapsuleSize, in int CapsuleIndex,
			inout float3 OutClosestPosition, inout float3 OutClosestNormal, inout int OutElementIndex, inout float OutMinDistance)
{
	const float HalfLength = 0.5 * CapsuleSize.y;
	const float3 SphereCenter = float3(0,0,clamp(LocalPosition.z, -HalfLength, HalfLength));

	return GetSphereProjection_{ParameterName}(LocalPosition,SphereCenter,CapsuleSize.x,CapsuleIndex,
			OutClosestPosition,OutClosestNormal,OutElementIndex,OutMinDistance);
}

// Given a world space position (NodePosition) compute the sphere closest point (position,normal,velocity)
float GetBoxProjection_{ParameterName}(in float3 LocalPosition, in float3 BoxExtent, in int BoxIndex, 
			inout float3 OutClosestPosition, inout float3 OutClosestNormal, inout int OutElementIndex, inout float OutMinDistance)
{
	const float3 HalfExtent = 0.5 * BoxExtent;
	const float3 DeltaPosition = abs(LocalPosition) - HalfExtent;

	const int ClosestAxis = ((DeltaPosition.x > DeltaPosition.y) && (DeltaPosition.x > DeltaPosition.z)) ? 0 : ( DeltaPosition.y > DeltaPosition.z) ? 1 : 2;
	const float OutsideDistance = length(max(DeltaPosition,0.0));
	const float BoxDistance = OutsideDistance + min(DeltaPosition[ClosestAxis],0.0);

	if (BoxDistance < OutMinDistance)
	{
		OutMinDistance = BoxDistance;
		OutElementIndex = BoxIndex;

		if (BoxDistance <= 0)
		{
			const bool NegativeSide = LocalPosition[ClosestAxis] < 0.0;

			OutClosestPosition = LocalPosition;
			OutClosestNormal = float3(0,0,0);
			if( ClosestAxis == 0) 
			{
				OutClosestPosition.x = NegativeSide ? -HalfExtent.x : HalfExtent.x;
				OutClosestNormal.x = NegativeSide ? -1.0 : 1.0;
			}
			else if( ClosestAxis == 1) 
			{
				OutClosestPosition.y = NegativeSide ? -HalfExtent.y : HalfExtent.y;
				OutClosestNormal.y = NegativeSide ? -1.0 : 1.0;
			}
			else if( ClosestAxis == 2) 
			{
				OutClosestPosition.z = NegativeSide ? -HalfExtent.z : HalfExtent.z;
				OutClosestNormal.z = NegativeSide ? -1.0 : 1.0;
			}
		}
		else
		{
			OutClosestPosition = clamp(LocalPosition,-HalfExtent,HalfExtent);
			OutClosestNormal = (LocalPosition - OutClosestPosition) / OutsideDistance;
		}
	}
	return BoxDistance;
}


/* -----------------------------------------------------------------
 * Get the closest element to the node position
 * -----------------------------------------------------------------
 */

/*float3 GetLocalPosition_{ParameterName}(in float3 NodePosition, in int ElementIndex, in float TimeFraction )
{
	//return mul(GetInverseTransform_{ParameterName}(ElementIndex), float4(NodePosition,1.0)).xyz;

	const float3 CurrentPosition = GetCurrentPosition_{ParameterName}(ElementIndex);
	const float4 CurrentOrientation = GetCurrentOrientation_{ParameterName}(ElementIndex);

	const float3 PreviousPosition = GetPreviousPosition_{ParameterName}(ElementIndex);
	const float4 PreviousOrientation = GetPreviousOrientation_{ParameterName}(ElementIndex);

	const float4 SlerpedOrientation = InverseQuat(SlerpQuat(PreviousOrientation,CurrentOrientation,TimeFraction));
	const float3 SlerpedPosition = PreviousPosition + TimeFraction * (CurrentPosition-PreviousPosition);

	return RotateVectorByQuat( NodePosition - SlerpedPosition, SlerpedOrientation );
}*/

float3 GetLocalPosition_{ParameterName}(in float3 NodePosition, in int ElementIndex, in float TimeFraction )
{
	const float3 CurrentLocal = mul(GetInverseTransform_{ParameterName}(ElementIndex), float4(NodePosition,1.0)).xyz;
	const float3 PreviousLocal = mul(GetPreviousInverse_{ParameterName}(ElementIndex), float4(NodePosition,1.0)).xyz;

	return PreviousLocal + TimeFraction * (CurrentLocal-PreviousLocal);
}

// Given a world space position (NodePosition) compute the sphere closest point (position,normal,velocity)
int GetClosestElement_{ParameterName}(in float3 NodePosition, out float3 OutClosestPosition, 
										  out float3 OutClosestNormal, out float OutMinDistance, in float TimeFraction, in bool IsRestSpace )
{	
	float MinDistance = MAX_DISTANCE;
	int ElementIndex = -1;
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);

	const int SpheresBegin = {ParameterName}_ElementOffsets[SPHERE_INDEX];
	const int SpheresEnd = {ParameterName}_ElementOffsets[SPHERE_INDEX+1];
	for (int SphereIndex = SpheresBegin; SphereIndex < SpheresEnd; ++SphereIndex)
	{
		const float3 LocalPosition = IsRestSpace ? mul(GetRestInverse_{ParameterName}(SphereIndex), float4(NodePosition,1.0)).xyz : 
						GetLocalPosition_{ParameterName}(NodePosition,SphereIndex,TimeFraction);

		GetSphereProjection_{ParameterName}(LocalPosition, float3(0,0,0), {ParameterName}_ElementExtentBuffer[SphereIndex].x, SphereIndex,
				CollisionPosition, CollisionNormal, ElementIndex, MinDistance);
	}

	const int CapsulesBegin = {ParameterName}_ElementOffsets[CAPSULE_INDEX];
	const int CapsulesEnd = {ParameterName}_ElementOffsets[CAPSULE_INDEX+1];
	for (int CapsuleIndex = CapsulesBegin; CapsuleIndex < CapsulesEnd; ++CapsuleIndex)
	{
		const float3 LocalPosition = IsRestSpace ? mul(GetRestInverse_{ParameterName}(CapsuleIndex), float4(NodePosition,1.0)).xyz : 
						GetLocalPosition_{ParameterName}(NodePosition,CapsuleIndex,TimeFraction);

		GetCapsuleProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[CapsuleIndex].xy, CapsuleIndex,
				CollisionPosition, CollisionNormal, ElementIndex, MinDistance);
	}

	const int BoxesBegin = {ParameterName}_ElementOffsets[BOX_INDEX];
	const int BoxesEnd = {ParameterName}_ElementOffsets[BOX_INDEX+1];
	for (int BoxIndex = BoxesBegin; BoxIndex < BoxesEnd; ++BoxIndex)
	{
		const float3 LocalPosition = IsRestSpace ? mul(GetRestInverse_{ParameterName}(BoxIndex), float4(NodePosition,1.0)).xyz : 
						GetLocalPosition_{ParameterName}(NodePosition,BoxIndex,TimeFraction);

		GetBoxProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[BoxIndex].xyz, BoxIndex,
				CollisionPosition, CollisionNormal, ElementIndex, MinDistance);
	}
	OutClosestPosition = CollisionPosition;
	OutClosestNormal = CollisionNormal;
	OutMinDistance = MinDistance;
	return ElementIndex;
}

/* -----------------------------------------------------------------
 * Get the closest point to the physics asset 
 * -----------------------------------------------------------------
 */

// Given a world space position (NodePosition) and an element index compute the physics asset closest point (position,normal,velocity)
void GetElementPoint_{ParameterName}(in float3 NodePosition, in float DeltaTime, in float TimeFraction, in int ElementIndex, out float3 OutClosestPosition, 
										  out float3 OutClosestNormal, out float3 OutClosestVelocity)
{	
	OutClosestNormal = float3(0,0,0);
	OutClosestPosition = float3(0,0,0);
	OutClosestVelocity = float3(0,0,0);

	if (ElementIndex >= 0)
	{
		const uint ElementIndexUint = uint(ElementIndex);
		const float3 LocalPosition = GetLocalPosition_{ParameterName}(NodePosition,ElementIndex,TimeFraction);

		float MinDistance = MAX_DISTANCE;
		int OutputIndex = -1;
		float3 CollisionPosition = float3(0,0,0);
		float3 CollisionNormal = float3(0,0,0);

		if ( ElementIndexUint >= {ParameterName}_ElementOffsets[SPHERE_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[SPHERE_INDEX+1] )
		{
			GetSphereProjection_{ParameterName}(LocalPosition, float3(0,0,0), {ParameterName}_ElementExtentBuffer[ElementIndex].x, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}
		else if ( ElementIndexUint >= {ParameterName}_ElementOffsets[CAPSULE_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[CAPSULE_INDEX+1] )
		{
			GetCapsuleProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[ElementIndex].xy, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}
		else if ( ElementIndexUint >= {ParameterName}_ElementOffsets[BOX_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[BOX_INDEX+1] )
		{
			GetBoxProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[ElementIndex].xyz, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}

		if (ElementIndex != -1)
		{
			const float3 PreviousPosition = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;
			const float3 CurrentPosition = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;

			const float3 PreviousNormal = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionNormal,0.0)).xyz;
			const float3 CurrentNormal = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionNormal,0.0)).xyz;

			OutClosestPosition = PreviousPosition + TimeFraction * (CurrentPosition-PreviousPosition);
			OutClosestNormal = normalize(PreviousNormal + TimeFraction * (CurrentNormal-PreviousNormal));

			OutClosestVelocity = ( CurrentPosition - PreviousPosition ) / DeltaTime;
		}
	}
}

// Given a world space position (NodePosition) and an element index compute the physics asset closest distance
void GetElementDistance_{ParameterName}(in float3 NodePosition, in float TimeFraction, in int ElementIndex, out float OutClosestDistance)
{	
	OutClosestDistance = 0.0;

	if (ElementIndex >= 0)
	{
		const uint ElementIndexUint = uint(ElementIndex);
		const float3 LocalPosition = GetLocalPosition_{ParameterName}(NodePosition,ElementIndex,TimeFraction);

		float MinDistance = MAX_DISTANCE;
		int OutputIndex = -1;
		float3 CollisionPosition = float3(0,0,0);
		float3 CollisionNormal = float3(0,0,0);

		if( ElementIndexUint >= {ParameterName}_ElementOffsets[SPHERE_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[SPHERE_INDEX+1] )
		{
			GetSphereProjection_{ParameterName}(LocalPosition, float3(0,0,0), {ParameterName}_ElementExtentBuffer[ElementIndex].x, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}
		else if ( ElementIndexUint >= {ParameterName}_ElementOffsets[CAPSULE_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[CAPSULE_INDEX+1] )
		{
			GetCapsuleProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[ElementIndex].xy, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}
		else if ( ElementIndexUint >= {ParameterName}_ElementOffsets[BOX_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[BOX_INDEX+1] )
		{
			GetBoxProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[ElementIndex].xyz, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}

		if (ElementIndex != -1)
		{
			OutClosestDistance = ({ParameterName}_PhysicsTypeBuffer[ElementIndex] == 1) ? MinDistance : 0;
		}
	}
}

 // Given a world space position (NodePosition) compute the physics asset closest element
void GetClosestElement_{ParameterName}(in float3 NodePosition, in float TimeFraction, out int OutElementIndex)
{	
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	float MinDistance = 0.0;

	OutElementIndex = GetClosestElement_{ParameterName}(NodePosition,CollisionPosition,CollisionNormal,MinDistance,TimeFraction, false);
}

// Given a world space position (NodePosition) compute the physics asset closest point (position,normal,velocity)
void GetClosestPoint_{ParameterName}(in float3 NodePosition, in float DeltaTime, in float TimeFraction, out float3 OutClosestPosition, 
										  out float3 OutClosestNormal, out float3 OutClosestVelocity)
{	
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	float MinDistance = 0.0;

	const int ElementIndex = GetClosestElement_{ParameterName}(NodePosition,CollisionPosition,CollisionNormal,MinDistance,TimeFraction, false);

	OutClosestNormal = float3(0,0,0);
	OutClosestPosition = float3(0,0,0);
	OutClosestVelocity = float3(0,0,0);

	if (ElementIndex != -1)
	{
		const float3 PreviousPosition = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;
		const float3 CurrentPosition = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;

		const float3 PreviousNormal = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionNormal,0.0)).xyz;
		const float3 CurrentNormal = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionNormal,0.0)).xyz;

		OutClosestPosition = PreviousPosition + TimeFraction * (CurrentPosition-PreviousPosition);
		OutClosestNormal = normalize(PreviousNormal + TimeFraction * (CurrentNormal-PreviousNormal));

		OutClosestVelocity = ( CurrentPosition - PreviousPosition ) / DeltaTime;
	}
}

// Given a world space position (NodePosition) compute the physics asset closest distance
void GetClosestDistance_{ParameterName}(in float3 NodePosition, in float TimeFraction, out float OutClosestDistance)
{	
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	float MinDistance = 0.0;

	const int ElementIndex = GetClosestElement_{ParameterName}(NodePosition,CollisionPosition,CollisionNormal,MinDistance,TimeFraction, false);

	OutClosestDistance = 0.0;

	if (ElementIndex != -1)
	{
		OutClosestDistance = ({ParameterName}_PhysicsTypeBuffer[ElementIndex] == 1) ? MinDistance : 0;
	}
}


/* -----------------------------------------------------------------
 * Get the texture point to the physics asset 
 * -----------------------------------------------------------------
 */

void GetRestDistance_{ParameterName}(in float3 RestPosition, out float OutRestDistance)
{
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	float MinDistance = 0.0;

	const int ElementIndex = GetClosestElement_{ParameterName}(RestPosition,CollisionPosition,CollisionNormal,MinDistance, 1.0, true);

	OutRestDistance = 0.0;

	if (ElementIndex != -1)
	{
		OutRestDistance = MinDistance;
	}
}

void GetTexturePoint_{ParameterName}(in float3 NodePosition, out int OutElementIndex, out float3 OutTexturePosition)
{
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	float MinDistance = 0.0;

	OutTexturePosition = float3(0,0,0);

	OutElementIndex = GetClosestElement_{ParameterName}(NodePosition,CollisionPosition,CollisionNormal,MinDistance, 1.0, false);

	const float AxisExtent = max( max({ParameterName}_BoxExtent.x, {ParameterName}_BoxExtent.y), {ParameterName}_BoxExtent.z );
	const float3 BoxExtent = float3(AxisExtent,AxisExtent,AxisExtent);

	const float3 BoxMinimum = {ParameterName}_BoxOrigin - BoxExtent;
	const float3 BoxSize = 2.0 * BoxExtent;

	if (OutElementIndex != -1)
	{
		const float3 LocalPosition = mul(GetInverseTransform_{ParameterName}(OutElementIndex), float4(NodePosition,1.0)).xyz;
		const float3 RestPosition = mul(GetRestTransform_{ParameterName}(OutElementIndex), float4(LocalPosition,1.0)).xyz;

		OutTexturePosition = clamp( (RestPosition - BoxMinimum)/BoxSize, float3(0,0,0), float3(1,1,1) );
	}
}

void GetProjectionPoint_{ParameterName}(in float3 NodePosition, in float DeltaTime, in int ElementIndex,
			in float TextureValue, in float3 TextureGradient, out float3 OutClosestPosition, out float3 OutClosestNormal, out float3 OutClosestVelocity, out float OutClosestDistance)
{
	OutClosestNormal = float3(0,0,0);
	OutClosestPosition = float3(0,0,0);
	OutClosestVelocity = float3(0,0,0);
	OutClosestDistance = 0.0;

	if (ElementIndex != -1)
	{
		const float AxisExtent = max( max({ParameterName}_BoxExtent.x, {ParameterName}_BoxExtent.y), {ParameterName}_BoxExtent.z );

		const float3 LocalPosition = mul(GetInverseTransform_{ParameterName}(ElementIndex), float4(NodePosition,1.0)).xyz;
		const float3 RestPosition = mul(GetRestTransform_{ParameterName}(ElementIndex), float4(LocalPosition,1.0)).xyz;

		const float3 RestProjection = RestPosition - TextureValue * 2.0 * AxisExtent * TextureGradient;

		const float3 CollisionPosition = mul(GetRestInverse_{ParameterName}(ElementIndex), float4(RestProjection,1.0)).xyz;
		const float3 CollisionNormal = mul(GetRestInverse_{ParameterName}(ElementIndex), float4(TextureGradient,0.0)).xyz;

		const float3 PreviousPosition = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;

		OutClosestPosition = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;
		OutClosestNormal = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionNormal,0.0)).xyz;
		OutClosestVelocity = ( OutClosestPosition - PreviousPosition ) / DeltaTime;
		OutClosestDistance = TextureValue * 2.0 * AxisExtent;
	}
}

#undef PHYSICS_ASSET_MAX_TRANSFORMS

#undef CURRENT_OFFSET
#undef PREVIOUS_OFFSET
#undef REST_OFFSET

#undef BOX_INDEX
#undef SPHERE_INDEX
#undef CAPSULE_INDEX
