// Copyright Epic Games, Inc. All Rights Reserved.

int {ParameterName}_bIsActive;

void IsActive_{ParameterName}(out bool bIsActive)
{
	bIsActive = {ParameterName}_bIsActive != 0;
}
