// Copyright Epic Games, Inc. All Rights Reserved.
// NOTE: Temporary code do not use as it will likely be removed in a future version

ByteAddressBuffer	{ParameterName}_PropertyInterfaceData;
//uint4				{ParameterName}_AttributeReadOffset[];

void ReadValue_{ParameterName}(uint ReadOffset, out bool   Value) {	Value = asint({ParameterName}_PropertyInterfaceData.Load(ReadOffset)) != 0; }
void ReadValue_{ParameterName}(uint ReadOffset, out int    Value) {	Value = asint({ParameterName}_PropertyInterfaceData.Load(ReadOffset)); }
void ReadValue_{ParameterName}(uint ReadOffset, out float  Value) { Value = asfloat({ParameterName}_PropertyInterfaceData.Load(ReadOffset)); }
void ReadValue_{ParameterName}(uint ReadOffset, out float2 Value) { Value = asfloat({ParameterName}_PropertyInterfaceData.Load2(ReadOffset)); }
void ReadValue_{ParameterName}(uint ReadOffset, out float3 Value) { Value = asfloat({ParameterName}_PropertyInterfaceData.Load3(ReadOffset)); }
void ReadValue_{ParameterName}(uint ReadOffset, out float4 Value) { Value = asfloat({ParameterName}_PropertyInterfaceData.Load4(ReadOffset)); }
