// Copyright Epic Games, Inc. All Rights Reserved.

int2					{ParameterName}_TextureSize;
int						{ParameterName}_MipLevels;
RWTexture2D<float4>		{ParameterName}_RWTexture;
Texture2D<float4>		{ParameterName}_Texture;
SamplerState			{ParameterName}_TextureSampler;

void SetRenderTargetValue_{ParameterName}_UEImpureCall(bool bExecute, int IndexX, int IndexY, float4 Value)
{
	if ( bExecute )
	{
		{ParameterName}_RWTexture[int2(IndexX, IndexY)] = Value;
	}
}

void GetRenderTargetValue_{ParameterName}(int IndexX, int IndexY, out float4 Value)
{
	Value = {ParameterName}_Texture.Load(int3(IndexX, IndexY, 0));
}

void LoadRenderTargetValue_{ParameterName}(int IndexX, int IndexY, int MipLevel, out float4 Value)
{
	Value = {ParameterName}_Texture.Load(int3(IndexX, IndexY, MipLevel));
}

void SampleRenderTargetValue_{ParameterName}(float2 UV, float MipLevel, out float4 Value)
{
	Value = {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UV, MipLevel);
}

void GetRenderTargetSize_{ParameterName}(out int Width, out int Height)
{
	Width = {ParameterName}_TextureSize.x;
	Height = {ParameterName}_TextureSize.y;
}

void GetNumMipLevels_{ParameterName}(out int OutMipLevels)
{
	OutMipLevels = {ParameterName}_MipLevels;
}

void LinearToIndex_{ParameterName}(int Linear, out int IndexX, out int IndexY)
{
	IndexX = Linear % {ParameterName}_TextureSize.x;
	IndexY = Linear / {ParameterName}_TextureSize.x;
}

void ExecToIndex_{ParameterName}(out int IndexX, out int IndexY)
{
	LinearToIndex_{ParameterName}(ExecIndex(), IndexX, IndexY);
}

void ExecToUnit_{ParameterName}(out float2 Unit)
{
	int2 Texel;
	LinearToIndex_{ParameterName}(ExecIndex(), Texel.x, Texel.y);
	Unit.x = (float(Texel.x) + 0.5f) / float({ParameterName}_TextureSize.x);
	Unit.y = (float(Texel.y) + 0.5f) / float({ParameterName}_TextureSize.y);
}
