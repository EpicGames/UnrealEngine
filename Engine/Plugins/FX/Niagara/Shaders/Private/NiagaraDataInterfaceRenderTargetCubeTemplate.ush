// Copyright Epic Games, Inc. All Rights Reserved.

int						{ParameterName}_TextureSize;
int						{ParameterName}_MipLevels;
RWTextureCube<float4>	{ParameterName}_RWTexture;
TextureCube<float4>		{ParameterName}_Texture;
SamplerState			{ParameterName}_TextureSampler;

void SetRenderTargetValue_{ParameterName}_UEImpureCall(bool bExecute, int IndexX, int IndexY, int Face, float4 Value)
{
	if ( bExecute )
	{
		{ParameterName}_RWTexture[int3(IndexX, IndexY, Face)] = Value;
	}
}

// Not possible to load from Cube textures
//void GetRenderTargetValue_{ParameterName}(int IndexX, int IndexY, int Face, out float4 Value)
//{
//	Value = {ParameterName}_Texture.Load(int4(IndexX, IndexY, Face, 0));
//}
//
//void LoadRenderTargetValue_{ParameterName}(int IndexX, int IndexY, int Face, int MipLevel, out float4 Value)
//{
//	Value = {ParameterName}_Texture.Load(int4(IndexX, IndexY, Face, MipLevel));
//}

void SampleRenderTargetValue_{ParameterName}(float3 UVW, float MipLevel, out float4 Value)
{
	Value = {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UVW, MipLevel);
}

void GetRenderTargetSize_{ParameterName}(out int Size)
{
	Size = {ParameterName}_TextureSize;
}

void GetNumMipLevels_{ParameterName}(out int OutMipLevels)
{
	OutMipLevels = {ParameterName}_MipLevels;
}

void LinearToIndex_{ParameterName}(int Linear, out int IndexX, out int IndexY, out int Face)
{
	IndexX = Linear % {ParameterName}_TextureSize;
	IndexY = (Linear / {ParameterName}_TextureSize) % {ParameterName}_TextureSize;
	Face = Linear / ({ParameterName}_TextureSize * {ParameterName}_TextureSize);
}

void ExecToIndex_{ParameterName}(out int IndexX, out int IndexY, out int Face)
{
	LinearToIndex_{ParameterName}(ExecIndex(), IndexX, IndexY, Face);
}

void ExecToUnit_{ParameterName}(out float2 Unit, out int Face)
{
	int2 Texel;
	ExecToIndex_{ParameterName}(Texel.x, Texel.y, Face);
	Unit.x = (float(Texel.x) + 0.5f) / float({ParameterName}_TextureSize);
	Unit.y = (float(Texel.y) + 0.5f) / float({ParameterName}_TextureSize);
}
