// Copyright Epic Games, Inc. All Rights Reserved.

int3					{ParameterName}_TextureSize;
int						{ParameterName}_MipLevels;
RWTexture3D<float4>		{ParameterName}_RWTexture;
Texture3D<float4>		{ParameterName}_Texture;
SamplerState			{ParameterName}_TextureSampler;

void SetRenderTargetValue_{ParameterName}_UEImpureCall(bool bExecute, int IndexX, int IndexY, int IndexZ, float4 Value)
{
	if ( bExecute )
	{
		{ParameterName}_RWTexture[int3(IndexX, IndexY, IndexZ)] = Value;
	}
}

void GetRenderTargetValue_{ParameterName}(int IndexX, int IndexY, int IndexZ, out float4 Value)
{
	Value = {ParameterName}_Texture.Load(int4(IndexX, IndexY, IndexZ, 0));
}

void LoadRenderTargetValue_{ParameterName}(int IndexX, int IndexY, int IndexZ, int MipLevel, out float4 Value)
{
	Value = {ParameterName}_Texture.Load(int4(IndexX, IndexY, IndexZ, MipLevel));
}

void SampleRenderTargetValue_{ParameterName}(float3 UVW, float MipLevel, out float4 Value)
{
	Value = {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UVW, MipLevel);
}

void GetRenderTargetSize_{ParameterName}(out int Width, out int Height, out int Slices)
{
	Width = {ParameterName}_TextureSize.x;
	Height = {ParameterName}_TextureSize.y;
	Slices = {ParameterName}_TextureSize.z;
}

void GetNumMipLevels_{ParameterName}(out int OutMipLevels)
{
	OutMipLevels = {ParameterName}_MipLevels;
}

void LinearToIndex_{ParameterName}(int Linear, out int IndexX, out int IndexY, out int IndexZ)
{
	IndexX = Linear % {ParameterName}_TextureSize.x;
	IndexY = (Linear / {ParameterName}_TextureSize.x) % {ParameterName}_TextureSize.y;
	IndexZ = Linear / ({ParameterName}_TextureSize.x * {ParameterName}_TextureSize.y);
}

void ExecToIndex_{ParameterName}(out int IndexX, out int IndexY, out int IndexZ)
{
	LinearToIndex_{ParameterName}(ExecIndex(), IndexX, IndexY, IndexZ);
}

void ExecToUnit_{ParameterName}(out float3 Unit)
{
	int3 Texel;
	ExecToIndex_{ParameterName}(Texel.x, Texel.y, Texel.z);
	Unit.x = (float(Texel.x) + 0.5f) / float({ParameterName}_TextureSize.x);
	Unit.y = (float(Texel.y) + 0.5f) / float({ParameterName}_TextureSize.y);
	Unit.z = (float(Texel.z) + 0.5f) / float({ParameterName}_TextureSize.z);
}
