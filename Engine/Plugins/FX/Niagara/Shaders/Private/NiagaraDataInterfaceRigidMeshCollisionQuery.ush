// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDataInterfaceRigidMeshCollisionQuery.ush 
=============================================================================*/

#include "/Plugin/FX/Niagara/Private/NiagaraDistanceFieldQueries.ush"
#include "/Plugin/FX/Niagara/Private/NiagaraQuaternionUtils.ush"
#include "/Plugin/FX/Niagara/Private/NiagaraTransformUtils.ush"

/* -----------------------------------------------------------------
 * static mesh constants and context
 * -----------------------------------------------------------------
 */

#define BOX_INDEX 0
#define SPHERE_INDEX 1
#define CAPSULE_INDEX 2
#define NUM_ELEMENTS_INDEX 3

uint			{ParameterName}_MaxTransforms;
uint			{ParameterName}_CurrentOffset;
uint			{ParameterName}_PreviousOffset;
uint4			{ParameterName}_ElementOffsets;
Buffer<float4>	{ParameterName}_WorldTransformBuffer;
Buffer<float4>	{ParameterName}_InverseTransformBuffer;
Buffer<float4>	{ParameterName}_MeshScaleBuffer;
Buffer<float4>	{ParameterName}_ElementExtentBuffer;
Buffer<uint>	{ParameterName}_PhysicsTypeBuffer;
Buffer<uint>	{ParameterName}_DFIndexBuffer;
float3			{ParameterName}_CombinedBBoxWorldMin;
float3			{ParameterName}_CombinedBBoxWorldMax;

/* -----------------------------------------------------------------
 * Element accessors
 * -----------------------------------------------------------------
 */

void GetNumBoxes_{ParameterName}(out int OutNumBoxes)
{
	OutNumBoxes = {ParameterName}_ElementOffsets[BOX_INDEX+1] - {ParameterName}_ElementOffsets[BOX_INDEX];
}

void GetNumSpheres_{ParameterName}(out int OutNumSpheres)
{
	OutNumSpheres = {ParameterName}_ElementOffsets[SPHERE_INDEX+1] - {ParameterName}_ElementOffsets[SPHERE_INDEX];
}

void GetNumCapsules_{ParameterName}(out int OutNumCapsules)
{
	OutNumCapsules = {ParameterName}_ElementOffsets[CAPSULE_INDEX+1] - {ParameterName}_ElementOffsets[CAPSULE_INDEX];
}

void GetNumElements_{ParameterName}(out int OutNumElements)
{
	OutNumElements = {ParameterName}_ElementOffsets[NUM_ELEMENTS_INDEX];
}

void GetBoxElementsStartIndex_{ParameterName}(out int OutStartIndex)
{
	OutStartIndex = {ParameterName}_ElementOffsets[BOX_INDEX];
}

void GetSphereElementsStartIndex_{ParameterName}(out int OutStartIndex)
{
	OutStartIndex = {ParameterName}_ElementOffsets[SPHERE_INDEX];
}

void GetCapsuleElementsStartIndex_{ParameterName}(out int OutStartIndex)
{
	OutStartIndex = {ParameterName}_ElementOffsets[CAPSULE_INDEX];
}

void GetSphereRadius_{ParameterName}(in int ElementIndex, out float OutRadius)
{
	OutRadius = {ParameterName}_ElementExtentBuffer[ElementIndex].x;
}

void GetCapsuleSize_{ParameterName}(in int ElementIndex, out float2 OutSize)
{
	OutSize = {ParameterName}_ElementExtentBuffer[ElementIndex].xy;
}

void GetBoxSize_{ParameterName}(in int ElementIndex, out float3 OutSize)
{
	OutSize = {ParameterName}_ElementExtentBuffer[ElementIndex].xyz;
}

/* -----------------------------------------------------------------
 * Position / Transform utils
 * -----------------------------------------------------------------
 */

float3x4 GetCurrentTransform_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + {ParameterName}_CurrentOffset;
    return float3x4({ParameterName}_WorldTransformBuffer[BufferOffset], {ParameterName}_WorldTransformBuffer[BufferOffset+1], {ParameterName}_WorldTransformBuffer[BufferOffset+2]);
}

float3x4 GetInverseTransform_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + {ParameterName}_CurrentOffset;
    return float3x4({ParameterName}_InverseTransformBuffer[BufferOffset], {ParameterName}_InverseTransformBuffer[BufferOffset+1], {ParameterName}_InverseTransformBuffer[BufferOffset+2]);
}

float3x4 GetPreviousTransform_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + {ParameterName}_PreviousOffset;
    return float3x4({ParameterName}_WorldTransformBuffer[BufferOffset], {ParameterName}_WorldTransformBuffer[BufferOffset+1], {ParameterName}_WorldTransformBuffer[BufferOffset+2]);
}

float3x4 GetPreviousInverse_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex + {ParameterName}_PreviousOffset;
    return float3x4({ParameterName}_InverseTransformBuffer[BufferOffset], {ParameterName}_InverseTransformBuffer[BufferOffset+1], {ParameterName}_InverseTransformBuffer[BufferOffset+2]);
}

float3 GetMeshScale_{ParameterName}(in int ElementIndex)
{
	return {ParameterName}_MeshScaleBuffer[ElementIndex].xyz;
}

void IsWorldPositionInsideCombinedBounds_{ParameterName}(in float3 WorldPosition, out bool IsInside)
{
	const FDFVector3 LWCWorldPosition = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition));

	IsInside =  
		WorldPosition.x >= {ParameterName}_CombinedBBoxWorldMin.x && 
		WorldPosition.y >= {ParameterName}_CombinedBBoxWorldMin.y && 
		WorldPosition.z >= {ParameterName}_CombinedBBoxWorldMin.z && 
		WorldPosition.x <= {ParameterName}_CombinedBBoxWorldMax.x && 
		WorldPosition.y <= {ParameterName}_CombinedBBoxWorldMax.y && 
		WorldPosition.z <= {ParameterName}_CombinedBBoxWorldMax.z;
}

/* -----------------------------------------------------------------
 * Collision detection utils
 * -----------------------------------------------------------------
 */

// Given a world space position (WorldPosition) compute the sphere closest point (position,normal,velocity)
float GetSphereProjection_{ParameterName}(in float3 LocalPosition, in float3 SphereCenter, in float SphereRadius, in int SphereIndex,
			inout float3 OutClosestPosition, inout float3 OutClosestNormal, inout int OutElementIndex, inout float OutMinDistance)
{
	const float3 DeltaPosition = LocalPosition - SphereCenter;
	const float DeltaLength = length(DeltaPosition);
	const float SphereDistance = DeltaLength - SphereRadius;

	if (SphereDistance < OutMinDistance)
	{
		OutMinDistance = SphereDistance;
		OutElementIndex = SphereIndex;

		OutClosestNormal = (DeltaLength > SMALL_NUMBER) ? DeltaPosition / DeltaLength : float3(0,0,0);
		OutClosestPosition = LocalPosition - OutClosestNormal*SphereDistance;
	}
	return SphereDistance;
}

// Given a world space position (WorldPosition) compute the sphere closest point (position,normal,velocity)
float GetCapsuleProjection_{ParameterName}(in float3 LocalPosition, in float2 CapsuleSize, in int CapsuleIndex,
			inout float3 OutClosestPosition, inout float3 OutClosestNormal, inout int OutElementIndex, inout float OutMinDistance)
{
	const float HalfLength = 0.5 * CapsuleSize.y;
	const float3 SphereCenter = float3(0,0,clamp(LocalPosition.z, -HalfLength, HalfLength));

	return GetSphereProjection_{ParameterName}(LocalPosition,SphereCenter,CapsuleSize.x,CapsuleIndex,
			OutClosestPosition,OutClosestNormal,OutElementIndex,OutMinDistance);
}

// Given a world space position (WorldPosition) compute the sphere closest point (position,normal,velocity)
float GetBoxProjection_{ParameterName}(in float3 LocalPosition, in float3 BoxExtent, in int BoxIndex, 
			inout float3 OutClosestPosition, inout float3 OutClosestNormal, inout int OutElementIndex, inout float OutMinDistance)
{
	const float3 HalfExtent = 0.5 * BoxExtent;
	const float3 DeltaPosition = abs(LocalPosition) - HalfExtent;

	const int ClosestAxis = ((DeltaPosition.x > DeltaPosition.y) && (DeltaPosition.x > DeltaPosition.z)) ? 0 : ( DeltaPosition.y > DeltaPosition.z) ? 1 : 2;
	const float OutsideDistance = length(max(DeltaPosition,0.0));
	const float BoxDistance = OutsideDistance + min(DeltaPosition[ClosestAxis],0.0);

	if (BoxDistance < OutMinDistance)
	{
		OutMinDistance = BoxDistance;
		OutElementIndex = BoxIndex;

		if (BoxDistance <= 0)
		{
			const bool NegativeSide = LocalPosition[ClosestAxis] < 0.0;

			OutClosestPosition = LocalPosition;
			OutClosestNormal = float3(0,0,0);
			if( ClosestAxis == 0) 
			{
				OutClosestPosition.x = NegativeSide ? -HalfExtent.x : HalfExtent.x;
				OutClosestNormal.x = NegativeSide ? -1.0 : 1.0;
			}
			else if( ClosestAxis == 1) 
			{
				OutClosestPosition.y = NegativeSide ? -HalfExtent.y : HalfExtent.y;
				OutClosestNormal.y = NegativeSide ? -1.0 : 1.0;
			}
			else if( ClosestAxis == 2) 
			{
				OutClosestPosition.z = NegativeSide ? -HalfExtent.z : HalfExtent.z;
				OutClosestNormal.z = NegativeSide ? -1.0 : 1.0;
			}
		}
		else
		{
			OutClosestPosition = clamp(LocalPosition,-HalfExtent,HalfExtent);
			OutClosestNormal = (LocalPosition - OutClosestPosition) / OutsideDistance;
		}
	}
	return BoxDistance;
}


/* -----------------------------------------------------------------
 * Get the closest element to the world position
 * -----------------------------------------------------------------
 */

float3 GetLocalPosition_{ParameterName}(in float3 WorldPosition, in int ElementIndex, in float TimeFraction )
{
	const float3 CurrentLocal = mul(GetInverseTransform_{ParameterName}(ElementIndex), float4(WorldPosition,1.0)).xyz;
	const float3 PreviousLocal = mul(GetPreviousInverse_{ParameterName}(ElementIndex), float4(WorldPosition,1.0)).xyz;

	return PreviousLocal + TimeFraction * (CurrentLocal-PreviousLocal);
}

float3 GetCurrentLocalPosition_{ParameterName}(in float3 WorldPosition, in int ElementIndex )
{
	return mul(GetInverseTransform_{ParameterName}(ElementIndex), float4(WorldPosition,1.0)).xyz;	
}

// Given a world space position (WorldPosition) compute the sphere closest point (position,normal,velocity)
int GetClosestElement_{ParameterName}(in float3 WorldPosition, out float3 OutClosestPosition, out float3 OutClosestNormal, out float OutMinDistance, in float TimeFraction)
{	
	float MinDistance = MAX_DISTANCE;
	int ElementIndex = -1;
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);

	const int SpheresBegin = {ParameterName}_ElementOffsets[SPHERE_INDEX];
	const int SpheresEnd = {ParameterName}_ElementOffsets[SPHERE_INDEX+1];
	for (int SphereIndex = SpheresBegin; SphereIndex < SpheresEnd; ++SphereIndex)
	{
		const float3 LocalPosition = GetLocalPosition_{ParameterName}(WorldPosition,SphereIndex,TimeFraction);

		GetSphereProjection_{ParameterName}(LocalPosition, float3(0,0,0), {ParameterName}_ElementExtentBuffer[SphereIndex].x, SphereIndex,
				CollisionPosition, CollisionNormal, ElementIndex, MinDistance);
	}

	const int CapsulesBegin = {ParameterName}_ElementOffsets[CAPSULE_INDEX];
	const int CapsulesEnd = {ParameterName}_ElementOffsets[CAPSULE_INDEX+1];
	for (int CapsuleIndex = CapsulesBegin; CapsuleIndex < CapsulesEnd; ++CapsuleIndex)
	{
		const float3 LocalPosition = GetLocalPosition_{ParameterName}(WorldPosition,CapsuleIndex,TimeFraction);

		GetCapsuleProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[CapsuleIndex].xy, CapsuleIndex,
				CollisionPosition, CollisionNormal, ElementIndex, MinDistance);
	}

	const int BoxesBegin = {ParameterName}_ElementOffsets[BOX_INDEX];
	const int BoxesEnd = {ParameterName}_ElementOffsets[BOX_INDEX+1];
	for (int BoxIndex = BoxesBegin; BoxIndex < BoxesEnd; ++BoxIndex)
	{
		const float3 LocalPosition = GetLocalPosition_{ParameterName}(WorldPosition,BoxIndex,TimeFraction);

		GetBoxProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[BoxIndex].xyz, BoxIndex,
				CollisionPosition, CollisionNormal, ElementIndex, MinDistance);
	}
	OutClosestPosition = CollisionPosition;
	OutClosestNormal = CollisionNormal;
	OutMinDistance = MinDistance;
	return ElementIndex;
}

// Given a world space position (WorldPosition) compute the sphere closest point (position,normal,velocity)
int GetClosestElementSimple_{ParameterName}(in float3 WorldPosition, out float3 OutClosestPosition, out float OutMinDistance)
{	
	float MinDistance = MAX_DISTANCE;
	int ElementIndex = -1;
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);

	const int SpheresBegin = {ParameterName}_ElementOffsets[SPHERE_INDEX];
	const int SpheresEnd = {ParameterName}_ElementOffsets[SPHERE_INDEX+1];
	for (int SphereIndex = SpheresBegin; SphereIndex < SpheresEnd; ++SphereIndex)
	{
		const float3 LocalPosition = GetCurrentLocalPosition_{ParameterName}(WorldPosition,SphereIndex);

		GetSphereProjection_{ParameterName}(LocalPosition, float3(0,0,0), {ParameterName}_ElementExtentBuffer[SphereIndex].x, SphereIndex,
				CollisionPosition, CollisionNormal, ElementIndex, MinDistance);
	}

	const int CapsulesBegin = {ParameterName}_ElementOffsets[CAPSULE_INDEX];
	const int CapsulesEnd = {ParameterName}_ElementOffsets[CAPSULE_INDEX+1];
	for (int CapsuleIndex = CapsulesBegin; CapsuleIndex < CapsulesEnd; ++CapsuleIndex)
	{
		const float3 LocalPosition = GetCurrentLocalPosition_{ParameterName}(WorldPosition,CapsuleIndex);

		GetCapsuleProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[CapsuleIndex].xy, CapsuleIndex,
				CollisionPosition, CollisionNormal, ElementIndex, MinDistance);
	}

	const int BoxesBegin = {ParameterName}_ElementOffsets[BOX_INDEX];
	const int BoxesEnd = {ParameterName}_ElementOffsets[BOX_INDEX+1];
	for (int BoxIndex = BoxesBegin; BoxIndex < BoxesEnd; ++BoxIndex)
	{
		const float3 LocalPosition = GetCurrentLocalPosition_{ParameterName}(WorldPosition,BoxIndex);

		GetBoxProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[BoxIndex].xyz, BoxIndex,
				CollisionPosition, CollisionNormal, ElementIndex, MinDistance);
	}

	OutClosestPosition = CollisionPosition;	
	OutMinDistance = MinDistance;
	return ElementIndex;
}

/* -----------------------------------------------------------------
 * Get the closest point to the static mesh 
 * -----------------------------------------------------------------
 */

// Given a world space position (WorldPosition) and an element index compute the static mesh closest point (position,normal,velocity)
void GetElementPoint_{ParameterName}(in float3 WorldPosition, in float DeltaTime, in float TimeFraction, in int ElementIndex, out float3 OutClosestPosition, out float3 OutClosestNormal, out float3 OutClosestVelocity)
{	
	OutClosestNormal = float3(0,0,0);
	OutClosestPosition = float3(0,0,0);
	OutClosestVelocity = float3(0,0,0);

	if (ElementIndex >= 0)
	{
		const uint ElementIndexUint = uint(ElementIndex);
		const float3 LocalPosition = GetLocalPosition_{ParameterName}(WorldPosition,ElementIndex,TimeFraction);

		float MinDistance = MAX_DISTANCE;
		int OutputIndex = -1;
		float3 CollisionPosition = float3(0,0,0);
		float3 CollisionNormal = float3(0,0,0);

		if ( ElementIndexUint >= {ParameterName}_ElementOffsets[SPHERE_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[SPHERE_INDEX+1] )
		{
			GetSphereProjection_{ParameterName}(LocalPosition, float3(0,0,0), {ParameterName}_ElementExtentBuffer[ElementIndex].x, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}
		else if ( ElementIndexUint >= {ParameterName}_ElementOffsets[CAPSULE_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[CAPSULE_INDEX+1] )
		{
			GetCapsuleProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[ElementIndex].xy, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}
		else if ( ElementIndexUint >= {ParameterName}_ElementOffsets[BOX_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[BOX_INDEX+1] )
		{
			GetBoxProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[ElementIndex].xyz, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}

		if (ElementIndex != -1)
		{
			const float3 PreviousPosition = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;
			const float3 CurrentPosition = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;

			const float3 PreviousNormal = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionNormal,0.0)).xyz;
			const float3 CurrentNormal = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionNormal,0.0)).xyz;

			OutClosestPosition = PreviousPosition + TimeFraction * (CurrentPosition-PreviousPosition);
			OutClosestNormal = normalize(PreviousNormal + TimeFraction * (CurrentNormal-PreviousNormal));

			OutClosestVelocity = ( CurrentPosition - PreviousPosition ) / DeltaTime;
		}
	}
}

// Given a world space position (WorldPosition) and an element index compute the static mesh closest point using distance fields (position,velocity)
void GetElementPointMeshDistanceFieldNoNormal_{ParameterName}(in float3 WorldPosition, in float DeltaTime, in float TimeFraction, in float MaxDistance, in int ElementIndex, out float OutClosestDistance, out float3 OutClosestVelocity)
{
	OutClosestVelocity = float3(0,0,0);
	OutClosestDistance = MAX_DISTANCE;

	if (ElementIndex >= 0)
	{
		const uint ElementIndexUint = uint(ElementIndex);
		const float3 LocalPosition = GetLocalPosition_{ParameterName}(WorldPosition,ElementIndex,TimeFraction);
		
		int OutputIndex = -1;
		float3 CollisionPosition = float3(0,0,0);
		float3 CollisionNormal = float3(0,0,0);

		if ( ElementIndexUint >= {ParameterName}_ElementOffsets[SPHERE_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[SPHERE_INDEX+1] )
		{
			GetSphereProjection_{ParameterName}(LocalPosition, float3(0,0,0), {ParameterName}_ElementExtentBuffer[ElementIndex].x, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, OutClosestDistance);
		}
		else if ( ElementIndexUint >= {ParameterName}_ElementOffsets[CAPSULE_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[CAPSULE_INDEX+1] )
		{
			GetCapsuleProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[ElementIndex].xy, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, OutClosestDistance);
		}
		else if ( ElementIndexUint >= {ParameterName}_ElementOffsets[BOX_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[BOX_INDEX+1] )
		{
			GetBoxProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[ElementIndex].xyz, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, OutClosestDistance);
		}

		uint DFIndex = {ParameterName}_DFIndexBuffer[ElementIndex];

		if (DFIndex >= 0 && DFIndex < NumSceneObjects)
		{
			MaxDistance = max(.001, MaxDistance);
			const FDFVector3 LWCWorldPosition = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition));
			OutClosestDistance = DistanceToNearestSurfaceForObject(DFIndex, LWCWorldPosition, MaxDistance);			
		}

		const float3 PreviousPosition = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;
		const float3 CurrentPosition = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;
		OutClosestVelocity = ( CurrentPosition - PreviousPosition ) / DeltaTime;		
	}
}

// Given a world space position (WorldPosition) and an element index compute the static mesh closest distance
void GetElementDistance_{ParameterName}(in float3 WorldPosition, in float TimeFraction, in int ElementIndex, out float OutClosestDistance)
{	
	OutClosestDistance = 0.0;

	if (ElementIndex >= 0)
	{
		const uint ElementIndexUint = uint(ElementIndex);
		const float3 LocalPosition = GetLocalPosition_{ParameterName}(WorldPosition,ElementIndex,TimeFraction);

		float MinDistance = MAX_DISTANCE;
		int OutputIndex = -1;
		float3 CollisionPosition = float3(0,0,0);
		float3 CollisionNormal = float3(0,0,0);

		if ( ElementIndexUint >= {ParameterName}_ElementOffsets[SPHERE_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[SPHERE_INDEX+1] )
		{
			GetSphereProjection_{ParameterName}(LocalPosition, float3(0,0,0), {ParameterName}_ElementExtentBuffer[ElementIndex].x, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}
		else if ( ElementIndexUint >= {ParameterName}_ElementOffsets[CAPSULE_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[CAPSULE_INDEX+1] )
		{
			GetCapsuleProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[ElementIndex].xy, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}
		else if ( ElementIndexUint >= {ParameterName}_ElementOffsets[BOX_INDEX] && ElementIndexUint < {ParameterName}_ElementOffsets[BOX_INDEX+1] )
		{
			GetBoxProjection_{ParameterName}(LocalPosition, {ParameterName}_ElementExtentBuffer[ElementIndex].xyz, ElementIndex,
					CollisionPosition, CollisionNormal, OutputIndex, MinDistance);
		}
	
		if (ElementIndex != -1)
		{
			OutClosestDistance = ({ParameterName}_PhysicsTypeBuffer[ElementIndex] == 1) ? MinDistance : 0;
		}
	}
}

 // Given a world space position (WorldPosition) compute the static mesh closest element
void GetClosestElement_{ParameterName}(in float3 WorldPosition, in float TimeFraction, out int OutElementIndex)
{	
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	float MinDistance = 0.0;

	OutElementIndex = GetClosestElement_{ParameterName}(WorldPosition,CollisionPosition,CollisionNormal,MinDistance,TimeFraction);
}

// Given a world space position (WorldPosition) compute the static mesh closest point (position,normal,velocity)
void GetClosestPoint_{ParameterName}(in float3 WorldPosition, in float DeltaTime, in float TimeFraction, out float OutClosestDistance, out float3 OutClosestPosition, out float3 OutClosestNormal, out float3 OutClosestVelocity)
{	
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	OutClosestDistance = 0.0;

	const int ElementIndex = GetClosestElement_{ParameterName}(WorldPosition,CollisionPosition,CollisionNormal,OutClosestDistance,TimeFraction);

	OutClosestNormal = float3(0,0,0);
	OutClosestPosition = float3(0,0,0);
	OutClosestVelocity = float3(0,0,0);

	if (ElementIndex != -1)
	{
		const float3 PreviousPosition = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;
		const float3 CurrentPosition = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;

		const float3 PreviousNormal = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionNormal,0.0)).xyz;
		const float3 CurrentNormal = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionNormal,0.0)).xyz;

		OutClosestPosition = PreviousPosition + TimeFraction * (CurrentPosition-PreviousPosition);
		OutClosestNormal = normalize(PreviousNormal + TimeFraction * (CurrentNormal-PreviousNormal));

		OutClosestVelocity = ( CurrentPosition - PreviousPosition ) / DeltaTime;
	}
}

// Given a world space position (WorldPosition) compute the static mesh closest point (position,normal,velocity)
void GetClosestPointSimple_{ParameterName}(in float3 WorldPosition, in float DeltaTime, out float OutClosestDistance, out float3 OutClosestVelocity)
{	
	float3 CollisionPosition = float3(0,0,0);
	OutClosestDistance = 0.0;

	const int ElementIndex = GetClosestElementSimple_{ParameterName}(WorldPosition,CollisionPosition,OutClosestDistance);

	OutClosestVelocity = float3(0,0,0);

	if (ElementIndex != -1)
	{
		const float3 PreviousPosition = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;
		const float3 CurrentPosition = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;
		OutClosestVelocity = ( CurrentPosition - PreviousPosition ) / DeltaTime;
	}
}

// Given a world space position (WorldPosition) compute the static mesh closest distance
void GetClosestDistance_{ParameterName}(in float3 WorldPosition, in float TimeFraction, out float OutClosestDistance)
{	
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	float MinDistance = 0.0;

	const int ElementIndex = GetClosestElement_{ParameterName}(WorldPosition,CollisionPosition,CollisionNormal,MinDistance,TimeFraction);

	OutClosestDistance = 0.0;

	if (ElementIndex != -1)
	{
		OutClosestDistance = ({ParameterName}_PhysicsTypeBuffer[ElementIndex] == 1) ? MinDistance : 0;
	}
}

void EvaluateWorldPositionOnElement_{ParameterName}(int ElementIndex, float3 WorldPosition, float DeltaTime, float TimeFraction, out float3 OutPosition, out float3 OutVelocity)
{
	const float3 LocalPosition = mul(GetInverseTransform_{ParameterName}(ElementIndex), float4(WorldPosition,1.0)).xyz;			

	const float3 PreviousPosition = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(LocalPosition,1.0)).xyz;
	const float3 CurrentPosition = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(LocalPosition,1.0)).xyz;
	
	OutVelocity = ( CurrentPosition - PreviousPosition ) / DeltaTime;
	OutPosition = PreviousPosition + TimeFraction * (CurrentPosition-PreviousPosition);
}

// Given a world space position (WorldPosition) compute the static mesh closest point (position,normal,velocity)
void GetClosestPointMeshDistanceField_{ParameterName}(in float3 WorldPosition, in float DeltaTime, in float TimeFraction, in float MaxDistance, out float OutClosestDistance, out float3 OutClosestPosition, out float3 OutClosestNormal, out float3 OutClosestVelocity, out bool NormalIsValid, out float MaxEncodedDistance)
{
	const FDFVector3 LWCWorldPosition = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition));

	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	OutClosestDistance = MAX_DISTANCE;	

	OutClosestNormal = float3(0,0,0);
	OutClosestPosition = float3(0,0,0);
	OutClosestVelocity = float3(0,0,0);

	MaxEncodedDistance = 0;

	NormalIsValid = false;

	float OutClosestDistanceTmp;
	const int ElementIndex = GetClosestElement_{ParameterName}(WorldPosition,CollisionPosition,CollisionNormal,OutClosestDistanceTmp,TimeFraction);

	if (ElementIndex != -1)
	{
		uint DFIndex = {ParameterName}_DFIndexBuffer[ElementIndex];

		if (DFIndex >= 0 && DFIndex < NumSceneObjects)
		{
			MaxDistance = max(0.001f, MaxDistance);

			OutClosestDistance = GetDistanceToMeshDistanceField(DFIndex, LWCWorldPosition, MaxDistance);
			
			float3 SampledWorldPosition;
			ComputeClosestPointMeshDistanceField(DFIndex, LWCWorldPosition, OutClosestDistance, SampledWorldPosition, OutClosestNormal, MaxEncodedDistance, NormalIsValid);

			if (NormalIsValid)
			{
				EvaluateWorldPositionOnElement_{ParameterName}(ElementIndex, SampledWorldPosition, DeltaTime, TimeFraction, OutClosestPosition, OutClosestVelocity);
				OutClosestDistance = length(WorldPosition - SampledWorldPosition) * sign(OutClosestDistanceTmp);				
			}
		}
	}
}

// Given a world space position (WorldPosition) compute the static mesh closest point (position,normal,velocity) by querying all overlapping mesh distance fields
void GetClosestPointMeshDistanceFieldAccurate_{ParameterName}(in float3 WorldPosition, in float DeltaTime, in float TimeFraction, in float MaxDistance, out float OutClosestDistance, out float3 OutClosestPosition, out float3 OutClosestNormal, out float3 OutClosestVelocity, out bool NormalIsValid, out float MaxEncodedDistance)
{
	const FDFVector3 LWCWorldPosition = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition));

	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	OutClosestDistance = MAX_DISTANCE;	

	OutClosestNormal = float3(0,0,0);
	OutClosestPosition = float3(0,0,0);
	OutClosestVelocity = float3(0,0,0);
	
	MaxEncodedDistance = 0;

	NormalIsValid = false;

	for (int ElementIndex = 0; ElementIndex < {ParameterName}_ElementOffsets[NUM_ELEMENTS_INDEX]; ++ElementIndex)
	{
		uint DFIndex = {ParameterName}_DFIndexBuffer[ElementIndex];

		// #todo(dmp): this does a dynamic branch based on intersecting the bbox.  Maybe we can factor that out due to the broadphase here?
		// #todo(dmp): Check the distance from the physics asset for a speedup?
		MaxDistance = max(.001, MaxDistance);
		float OutClosestDistanceTmp = GetDistanceToMeshDistanceField(DFIndex, LWCWorldPosition, MaxDistance);

		if (OutClosestDistanceTmp < OutClosestDistance)
		{
			OutClosestDistance = OutClosestDistanceTmp;

			float3 SampledWorldPosition;
			ComputeClosestPointMeshDistanceField(DFIndex, LWCWorldPosition, OutClosestDistance, SampledWorldPosition, OutClosestNormal, MaxEncodedDistance, NormalIsValid);
			
			if (NormalIsValid)
			{
				EvaluateWorldPositionOnElement_{ParameterName}(ElementIndex, SampledWorldPosition, DeltaTime, TimeFraction, OutClosestPosition, OutClosestVelocity);
				OutClosestDistance = length(WorldPosition - SampledWorldPosition) * sign(OutClosestDistanceTmp);				
			}
		}
	}	
}

// Given a world space position (WorldPosition) compute the static mesh closest point (position,normal,velocity)
void GetClosestPointMeshDistanceFieldNoNormal_{ParameterName}(in float3 WorldPosition, in float DeltaTime, in float TimeFraction, in float MaxDistance, out float OutClosestDistance, out float3 OutClosestPosition, out float3 OutClosestVelocity)
{	
	const FDFVector3 LWCWorldPosition = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition));

	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	OutClosestDistance = MAX_DISTANCE;	
	
	OutClosestPosition = float3(0,0,0);
	OutClosestVelocity = float3(0,0,0);
	
	float OutClosestDistanceTmp;
	const int ElementIndex = GetClosestElement_{ParameterName}(WorldPosition,CollisionPosition,CollisionNormal,OutClosestDistanceTmp,TimeFraction);

	if (ElementIndex != -1)
	{
		uint DFIndex = {ParameterName}_DFIndexBuffer[ElementIndex];

		if (DFIndex >= 0 && DFIndex < NumSceneObjects)
		{
			// #todo(dmp): this does a dynamic branch based on intersecting the bbox.  Maybe we can factor that out due to the broadphase here?
			// #todo(dmp): Check the distance from the physics asset for a speedup?
			MaxDistance = max(.001, MaxDistance);

			OutClosestDistance = DistanceToNearestSurfaceForObject(DFIndex, LWCWorldPosition, MaxDistance);			
		}
		else
		{
			OutClosestDistance = OutClosestDistanceTmp;
		}

		const float3 PreviousPosition = mul(GetPreviousTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;
		const float3 CurrentPosition = mul(GetCurrentTransform_{ParameterName}(ElementIndex), float4(CollisionPosition,1.0)).xyz;
		OutClosestVelocity = ( CurrentPosition - PreviousPosition ) / DeltaTime;
		OutClosestPosition = PreviousPosition + TimeFraction * (CurrentPosition-PreviousPosition);
	}
}

void GetMaxEncodedDistanceMeshDistanceField_{ParameterName}(in int ElementIndex, out float MaxEncodedDistance)
{
	MaxEncodedDistance = 0;
	if (ElementIndex != -1)
	{
		uint DFIndex = {ParameterName}_DFIndexBuffer[ElementIndex];

		if (DFIndex >= 0 && DFIndex < NumSceneObjects)
		{
			FDFObjectData DFObjectData = LoadDFObjectData(DFIndex);

			uint NumMips = LoadDFAssetData(DFObjectData.AssetIndex, 0).NumMips;
			FDFAssetData DFAssetData = LoadDFAssetData(DFObjectData.AssetIndex, NumMips - 1);
			MaxEncodedDistance = (DFAssetData.DistanceFieldToVolumeScaleBias.x + DFAssetData.DistanceFieldToVolumeScaleBias.y) * DFObjectData.VolumeScale;
		}
	}
}

void GetElementTransform_{ParameterName}(in int ElementIndex, in float TimeFraction, out float3 OutTranslation, out float4 OutRotation, out float3 OutScale, out bool OutIsValid)
{
	OutIsValid = false;

	int ElementCount = -1;
	GetNumElements_{ParameterName}(ElementCount);
	if (ElementIndex >= 0 && ElementIndex < ElementCount)
	{
		OutIsValid = true;

		FNiagaraTransform PrevTransform = DecomposePackedMatrix(GetPreviousTransform_{ParameterName}(ElementIndex));
		FNiagaraTransform CurrTransform = DecomposePackedMatrix(GetCurrentTransform_{ParameterName}(ElementIndex));

		OutTranslation = lerp(PrevTransform.Translation, CurrTransform.Translation, TimeFraction);
		OutRotation = SlerpQuat(PrevTransform.Rotation, CurrTransform.Rotation, TimeFraction);
		OutScale = lerp(PrevTransform.Scale, CurrTransform.Scale, TimeFraction);
	}
}

void GetElementInverseTransform_{ParameterName}(in int ElementIndex, in float TimeFraction, out float3 OutTranslation, out float4 OutRotation, out float3 OutScale, out bool OutIsValid)
{
	OutIsValid = false;

	int ElementCount = -1;
	GetNumElements_{ParameterName}(ElementCount);
	if (ElementIndex >= 0 && ElementIndex < ElementCount)
	{
		OutIsValid = true;

		FNiagaraTransform PrevTransform = DecomposePackedMatrix(GetPreviousInverse_{ParameterName}(ElementIndex));
		FNiagaraTransform CurrTransform = DecomposePackedMatrix(GetInverseTransform_{ParameterName}(ElementIndex));

		OutTranslation = lerp(PrevTransform.Translation, CurrTransform.Translation, TimeFraction);
		OutRotation = SlerpQuat(PrevTransform.Rotation, CurrTransform.Rotation, TimeFraction);
		OutScale = lerp(PrevTransform.Scale, CurrTransform.Scale, TimeFraction);
	}
}

#undef BOX_INDEX
#undef SPHERE_INDEX
#undef CAPSULE_INDEX