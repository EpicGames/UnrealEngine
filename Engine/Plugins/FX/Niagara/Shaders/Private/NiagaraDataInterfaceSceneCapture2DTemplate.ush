// Copyright Epic Games, Inc. All Rights Reserved.

Texture2D		{ParameterName}_Texture;
SamplerState	{ParameterName}_TextureSampler;
int3			{ParameterName}_TextureSize;
float4x4		{ParameterName}_TextureViewMatrix;
float4x4		{ParameterName}_TextureProjMatrix;
float4x4		{ParameterName}_InvTextureProjMatrix;
uint			{ParameterName}_IsPerspective;

void GetTextureSize_{ParameterName}(out int OutSizeX, out int OutSizeY, out int NumMips)
{
	OutSizeX = {ParameterName}_TextureSize.x;
	OutSizeY = {ParameterName}_TextureSize.y;
	NumMips = {ParameterName}_TextureSize.z;
}

void TextureLoad_{ParameterName}(in int TexelX, in int TexelY, in int MipLevel, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.Load(int3(TexelX, TexelY, MipLevel));
}

void TextureSample_{ParameterName}(in float2 UV, in float MipLevel, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UV, MipLevel);
}

void Project_{ParameterName}(in float3 Position, out float3 UVW)
{
	float4 ProjPosition = mul(float4(Position, 1.0f), {ParameterName}_TextureProjMatrix);
	UVW = ProjPosition.w != 0.0f ? ProjPosition.xyz / ProjPosition.w : ProjPosition.xyz;
	UVW.xy = (UVW.xy * float2(0.5f, -0.5f)) + 0.5f;
}

void Deproject_{ParameterName}(in float2 UV, in float Depth, out float3 Position)
{
	float2 NDC = (UV - 0.5f) * float2(2.0f, -2.0f);
	if ({ParameterName}_IsPerspective != 0)
	{
		NDC *= Depth;
	}
	Position = mul(float4(NDC, Depth, 1.0f), {ParameterName}_InvTextureProjMatrix).xyz;
}

void TextureProject_{ParameterName}(in float3 Position, in float MipLevel, in bool bValidateTextureBounds, in bool bValidateDepthBounds, in float4 DefaultColor, out bool OutInBounds, out float4 OutValue, out float3 UVW)
{
	Project_{ParameterName}(Position, UVW);

	OutInBounds = bValidateTextureBounds ? all(UVW.xy >= 0.0f && UVW.xy < 1.0f) : true;
	OutInBounds &= bValidateDepthBounds ? all(UVW.z >= 0.0f && UVW.z < 1.0f) : true;
	OutValue = OutInBounds ? {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UVW.xy, MipLevel) : DefaultColor;
}

void TextureProjectDepth_{ParameterName}(in float3 Position, in float MipLevel, in bool bPointSample, in bool bValidateTextureBounds, in float DefaultDepth, out bool OutInBounds, out float OutCaptureDepth, out float OutPositionDepth, out float2 OutUV)
{
	float3 UVW;
	Project_{ParameterName}(Position, UVW);

	OutInBounds = bValidateTextureBounds ? all(UVW.xy >= 0.0f && UVW.xy < 1.0f) : true;
	if ( bPointSample )
	{
		const int3 Texel = int3(round(UVW.xy * {ParameterName}_TextureSize.xy), int(round(MipLevel)));
		OutCaptureDepth = {ParameterName}_Texture.Load(Texel).r;
	}
	else
	{
		OutCaptureDepth = OutInBounds ? {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UVW.xy, MipLevel).r : DefaultDepth;
	}
	OutPositionDepth = mul(float4(Position, 1.0f), {ParameterName}_TextureViewMatrix).z;
	OutUV = UVW.xy;
}

void TextureDeprojectDepth_{ParameterName}(in float2 UV, in float MipLevel, in bool bPointSample, out float3 OutPosition, out float OutCaptureDepth)
{
	if ( bPointSample )
	{
		const int3 Texel = int3(round(UV * {ParameterName}_TextureSize.xy), int(round(MipLevel)));
		OutCaptureDepth = {ParameterName}_Texture.Load(Texel).r;
	}
	else
	{
		OutCaptureDepth = {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UV, MipLevel).r;
	}
	Deproject_{ParameterName}(UV, OutCaptureDepth, OutPosition);
}
