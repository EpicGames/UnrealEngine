// Copyright Epic Games, Inc. All Rights Reserved.

int				{ParameterName}_NumFrames;
int				{ParameterName}_NumEmitters;
int				{ParameterName}_EmitterIndex;
Buffer<uint>	{ParameterName}_CacheData;

struct {ParameterName}_EmitterFrameData
{
	int		NumInstances;
	uint	Int32DataOffset;
	uint	FloatDataOffset;
	uint	HalfDataOffset;
};

void GetNumFrames_{ParameterName}(out int NumFrames)
{
	NumFrames = {ParameterName}_NumFrames;
}

void GetNumEmitters_{ParameterName}(out int NumEmitters)
{
	NumEmitters = {ParameterName}_NumEmitters;
}

void GetEmitterIndex_{ParameterName}(out int EmitterIndex)
{
	EmitterIndex = {ParameterName}_EmitterIndex;
}

{ParameterName}_EmitterFrameData GetEmitterFrameData_{ParameterName}(int FrameIndex, int EmitterIndex)
{
	{ParameterName}_EmitterFrameData FrameData;
	FrameData.NumInstances		= 0;
	FrameData.Int32DataOffset	= 0;
	FrameData.FloatDataOffset	= 0;
	FrameData.HalfDataOffset	= 0;

	if ( FrameIndex >= 0 && FrameIndex < {ParameterName}_NumFrames && EmitterIndex >= 0 && EmitterIndex < {ParameterName}_NumEmitters )
	{
		uint FrameDataOffset		= (FrameIndex * 4 * {ParameterName}_NumEmitters) + EmitterIndex;
		FrameData.NumInstances		= int({ParameterName}_CacheData[FrameDataOffset + 0]);
		FrameData.Int32DataOffset	= {ParameterName}_CacheData[FrameDataOffset + 1];
		FrameData.FloatDataOffset	= {ParameterName}_CacheData[FrameDataOffset + 2];
		FrameData.HalfDataOffset	= {ParameterName}_CacheData[FrameDataOffset + 3];
	}
	return FrameData;
}

void GetNumInstances_{ParameterName}(in int FrameIndex, out int NumInstances)
{
	{ParameterName}_EmitterFrameData FrameData = GetEmitterFrameData_{ParameterName}(FrameIndex, {ParameterName}_EmitterIndex);
	NumInstances = FrameData.NumInstances;
}

void GetInstanceInt_{ParameterName}(int FrameIndex, int EmitterIndex, int AttributeIndex, int InstanceIndex, out bool bValid, out int Value)
{
	bValid = false;
	Value = 0;

	{ParameterName}_EmitterFrameData FrameData = GetEmitterFrameData_{ParameterName}(FrameIndex, EmitterIndex);
	if ( InstanceIndex >= 0 && InstanceIndex < FrameData.NumInstances )
	{
		bValid = true;
		Value = asint({ParameterName}_CacheData[FrameData.Int32DataOffset + (AttributeIndex * FrameData.NumInstances) + InstanceIndex]);
	}
}

void GetInstanceFloat_{ParameterName}(int FrameIndex, int EmitterIndex, int AttributeIndex, int InstanceIndex, out bool bValid, out float Value)
{
	bValid = false;
	Value = 0.0f;

	{ParameterName}_EmitterFrameData FrameData = GetEmitterFrameData_{ParameterName}(FrameIndex, EmitterIndex);
	if ( InstanceIndex >= 0 && InstanceIndex < FrameData.NumInstances )
	{
		bValid = true;
		Value = asfloat({ParameterName}_CacheData[FrameData.FloatDataOffset + (AttributeIndex * FrameData.NumInstances) + InstanceIndex]);
	}
}

void GetInstanceFloat2_{ParameterName}(int FrameIndex, int EmitterIndex, int AttributeIndex, int InstanceIndex, out bool bValid, out float2 Value)
{
	bValid = false;
	Value = 0.0f;

	{ParameterName}_EmitterFrameData FrameData = GetEmitterFrameData_{ParameterName}(FrameIndex, EmitterIndex);
	if ( InstanceIndex >= 0 && InstanceIndex < FrameData.NumInstances )
	{
		bValid = true;
		Value.x = asfloat({ParameterName}_CacheData[FrameData.FloatDataOffset + ((AttributeIndex + 0) * FrameData.NumInstances) + InstanceIndex]);
		Value.y = asfloat({ParameterName}_CacheData[FrameData.FloatDataOffset + ((AttributeIndex + 1) * FrameData.NumInstances) + InstanceIndex]);
	}
}

void GetInstanceFloat3_{ParameterName}(int FrameIndex, int EmitterIndex, int AttributeIndex, int InstanceIndex, out bool bValid, out float3 Value)
{
	bValid = false;
	Value = 0.0f;

	{ParameterName}_EmitterFrameData FrameData = GetEmitterFrameData_{ParameterName}(FrameIndex, EmitterIndex);
	if ( InstanceIndex >= 0 && InstanceIndex < FrameData.NumInstances )
	{
		bValid = true;
		Value.x = asfloat({ParameterName}_CacheData[FrameData.FloatDataOffset + ((AttributeIndex + 0) * FrameData.NumInstances) + InstanceIndex]);
		Value.y = asfloat({ParameterName}_CacheData[FrameData.FloatDataOffset + ((AttributeIndex + 1) * FrameData.NumInstances) + InstanceIndex]);
		Value.z = asfloat({ParameterName}_CacheData[FrameData.FloatDataOffset + ((AttributeIndex + 2) * FrameData.NumInstances) + InstanceIndex]);
	}
}

void GetInstanceFloat4_{ParameterName}(int FrameIndex, int EmitterIndex, int AttributeIndex, int InstanceIndex, out bool bValid, out float4 Value)
{
	bValid = false;
	Value = 0.0f;

	{ParameterName}_EmitterFrameData FrameData = GetEmitterFrameData_{ParameterName}(FrameIndex, EmitterIndex);
	if ( InstanceIndex >= 0 && InstanceIndex < FrameData.NumInstances )
	{
		bValid = true;
		Value.x = asfloat({ParameterName}_CacheData[FrameData.FloatDataOffset + ((AttributeIndex + 0) * FrameData.NumInstances) + InstanceIndex]);
		Value.y = asfloat({ParameterName}_CacheData[FrameData.FloatDataOffset + ((AttributeIndex + 1) * FrameData.NumInstances) + InstanceIndex]);
		Value.z = asfloat({ParameterName}_CacheData[FrameData.FloatDataOffset + ((AttributeIndex + 2) * FrameData.NumInstances) + InstanceIndex]);
		Value.w = asfloat({ParameterName}_CacheData[FrameData.FloatDataOffset + ((AttributeIndex + 3) * FrameData.NumInstances) + InstanceIndex]);
	}
}

void GetInstanceID_{ParameterName}(int FrameIndex, int EmitterIndex, int AttributeIndex, int InstanceIndex, out bool bValid, out NiagaraID Value)
{
	bValid = false;
	Value.Index = -1;
	Value.AcquireTag = -1;

	{ParameterName}_EmitterFrameData FrameData = GetEmitterFrameData_{ParameterName}(FrameIndex, EmitterIndex);
	if ( InstanceIndex >= 0 && InstanceIndex < FrameData.NumInstances )
	{
		bValid = true;
		Value.Index = asint({ParameterName}_CacheData[FrameData.Int32DataOffset + ((AttributeIndex + 0) * FrameData.NumInstances) + InstanceIndex]);
		Value.AcquireTag = asint({ParameterName}_CacheData[FrameData.Int32DataOffset + ((AttributeIndex + 1) * FrameData.NumInstances) + InstanceIndex]);
	}
}

void GetInstanceIndexFromID_{ParameterName}(int FrameIndex, int EmitterIndex, int AttributeIndex, NiagaraID ID, out bool bValid, out int Value)
{
	bValid = false;
	Value = -1;

	if ( AttributeIndex != -1 )
	{
		{ParameterName}_EmitterFrameData FrameData = GetEmitterFrameData_{ParameterName}(FrameIndex, EmitterIndex);
		for ( int InstanceIndex=0; InstanceIndex < FrameData.NumInstances; ++InstanceIndex )
		{
			NiagaraID InstanceID;
			InstanceID.Index = asint({ParameterName}_CacheData[FrameData.Int32DataOffset + ((AttributeIndex + 0) * FrameData.NumInstances) + InstanceIndex]);
			InstanceID.AcquireTag = asint({ParameterName}_CacheData[FrameData.Int32DataOffset + ((AttributeIndex + 1) * FrameData.NumInstances) + InstanceIndex]);
			if (InstanceID.Index == ID.Index && InstanceID.AcquireTag == ID.AcquireTag)
			{
				bValid = true;
				Value = InstanceIndex;
				return;
			}
		}
	}
}
