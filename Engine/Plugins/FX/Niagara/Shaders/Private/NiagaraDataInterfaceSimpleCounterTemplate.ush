// Copyright Epic Games, Inc. All Rights Reserved.
// RWInstanceCounts come from NiagaraEmitterInstanceSimulation.ush as we use counts from that buffer

int {ParameterName}_CountOffset;

void Get_{ParameterName}(out int CurrValue)
{
	CurrValue = RWInstanceCounts[{ParameterName}_CountOffset];
}

void Set_{ParameterName}_UEImpureCall(in bool bExecute, in int NewValue)
{
	if ( bExecute )
	{
		int PrevValue;
		InterlockedExchange(RWInstanceCounts[{ParameterName}_CountOffset], NewValue, PrevValue);
	}
}

void Exchange_{ParameterName}_UEImpureCall(in bool bExecute, in int NewValue, out int PrevValue)
{
	if ( bExecute )
	{
		InterlockedExchange(RWInstanceCounts[{ParameterName}_CountOffset], NewValue, PrevValue);
	}
	else
	{
		PrevValue = RWInstanceCounts[{ParameterName}_CountOffset];
	}
}

void Add_{ParameterName}_UEImpureCall(in bool bExecute, in int Amount, out int PrevValue, out int CurrValue)
{
	if ( bExecute )
	{
		InterlockedAdd(RWInstanceCounts[{ParameterName}_CountOffset], Amount, PrevValue);
		CurrValue = PrevValue + Amount;
	}
	else
	{
		PrevValue = RWInstanceCounts[{ParameterName}_CountOffset];
		CurrValue = PrevValue;
	}
}

void Increment_{ParameterName}_UEImpureCall(in bool bExecute, out int PrevValue, out int CurrValue)
{
	Add_{ParameterName}_UEImpureCall(bExecute, 1, PrevValue, CurrValue);
}

void Decrement_{ParameterName}_UEImpureCall(in bool bExecute, out int PrevValue, out int CurrValue)
{
	Add_{ParameterName}_UEImpureCall(bExecute, -1, PrevValue, CurrValue);
}
