// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraBaryCentricUtils.ush"

#define DISKELMESH_BONE_INFLUENCES_MAX4			0
#define DISKELMESH_BONE_INFLUENCES_MAX8			1
#define DISKELMESH_BONE_INFLUENCES_UNLIMITED	2

#ifndef DISKELMESH_BONE_INFLUENCES
#define DISKELMESH_BONE_INFLUENCES	DISKELMESH_BONE_INFLUENCES_UNLIMITED
#endif

#define DISKELMESH_PROBALIAS_FORMAT_64			0		// 64 bits : Float probability	: uint alias
#define DISKELMESH_PROBALIAS_FORMAT_24_8		1		// 32 bits : 8 bits probability	: 24 bits alias
#define DISKELMESH_PROBALIAS_FORMAT_23_9		2		// 32 bits : 9 bits probability	: 23 bits alias

#ifndef DISKELMESH_PROBALIAS_FORMAT
#define DISKELMESH_PROBALIAS_FORMAT	DISKELMESH_PROBALIAS_FORMAT_64
#endif

#define DISKELMESH_ADJ_INDEX_FORMAT_FULL 0				// 32 bits
#define DISKELMESH_ADJ_INDEX_FORMAT_HALF 1				// 16 bits

#ifndef DISKELMESH_ADJ_INDEX_FORMAT
#define DISKELMESH_ADJ_INDEX_FORMAT DISKELMESH_ADJ_INDEX_FORMAT_FULL
#endif

#if FEATURE_LEVEL <= FEATURE_LEVEL_ES3_1
#undef DISKELMESH_ALLOW_DEFORMED
#define DISKELMESH_ALLOW_DEFORMED	0
#endif

//-TODO: We need to consolodate the random interface inside NiagaraEmitterInstanceShader.usf it's a bit messy at the moment but should follow what FNDIRandomHelper does
float DISKelMesh_Random(uint S1, uint S2, uint S3)
{
	return S3 == 0xffffffff ? NiagaraInternalNoise(S1, S2, S3) : rand(1.0f, S1, S2, S3);
}

struct FDISkelMeshSkinnedVertex
{
	float3	Position;
	float3	PrevPosition;
	float3	TangentX;
	float3	TangentY;
	float3	TangentZ;
};

float3 DISKelMesh_RandomBarycentricCoord(uint Seed1, uint Seed2, uint Seed3)
{
	float r0 = DISKelMesh_Random(Seed1, Seed2, Seed3);
	float r1 = DISKelMesh_Random(Seed1, Seed2, Seed3);
	float sqrt0 = sqrt(r0);
	float sqrt1 = sqrt(r1);
	return float3(1.0f - sqrt0, sqrt0 * (1.0 - r1), r1 * sqrt0);
}

int4 DISkelMesh_UnpackIndices4x8(uint Packed)
{
	return int4(Packed & 0xff, Packed >> 8 & 0xff, Packed >> 16 & 0xff, Packed >> 24 & 0xff);
}

int4 DISkelMesh_UnpackIndices4x16(uint PackedA, uint PackedB)
{
	return int4(PackedA & 0xffff, PackedA >> 16 & 0xffff, PackedB & 0xffff, PackedB >> 16 & 0xffff);
}

float4 DISkelMesh_UnpackWeights4x8(uint Packed)
{
	return float4(Packed & 0xff, Packed >> 8 & 0xff, Packed >> 16 & 0xff, Packed >> 24 & 0xff) / 255.0f;
}

float4 DISkelMesh_UnpackWeights4x16(uint PackedA, uint PackedB)
{
	return float4(PackedA & 0xffff, PackedA >> 16 & 0xffff, PackedB & 0xffff, PackedB >> 16 & 0xffff) / 65535.0f;
}

// UvMappingBuffer organized as an array of:
//struct FFrozenNode
//{
//	int32 ChildOffsets[4]
//	int32 ElementCount
//	int32 TriangleIndices[]
//};

#define DISkelMesh_ElementCountOffset 4
#define DISkelMesh_TriangleIndexOffset 5
#define DISkelMesh_MinValidUvMappingBufferSize 24 // minimum size (in bytes) for a quadtree to have valid data

bool DISkelMesh_NormalizedAabbTriangleIntersection(float2 A, float2 B, float2 C)
{
	float2 TriAabbMin = float2(min3(A.x, B.x, C.x), min3(A.y, B.y, C.y));
	float2 TriAabbMax = float2(max3(A.x, B.x, C.x), max3(A.y, B.y, C.y));

	if (any(TriAabbMin > 1.0f) || any(TriAabbMax < 0.0f))
	{
		return false;
	}

	#define AXIS_COUNT 3
	float2 TriangleEdges[AXIS_COUNT] = { C - B, A - C, B - A };

	for (int i = 0; i < AXIS_COUNT; ++i)
	{
		float2 Axis = TriangleEdges[i].yx * (float2(-1.0f, 0.0f));
		float AabbSegmentMin = min(0.0f, min3(Axis.x, Axis.y, Axis.x + Axis.y));
		float AabbSegmentMax = max(0.0f, max3(Axis.x, Axis.y, Axis.x + Axis.y));
		float TriangleSegmentMin = min3(dot(A, Axis), dot(B, Axis), dot(C, Axis));
		float TriangleSegmentMax = max3(dot(A, Axis), dot(B, Axis), dot(C, Axis));
		if (AabbSegmentMin > TriangleSegmentMax || AabbSegmentMax < TriangleSegmentMin)
		{
			return false;
		}
	}

	return true;
}

float4 DISKelMesh_QuatSlerp(float4 Quat1, float4 Quat2, float Slerp)
{
    const float RawCosom = dot(Quat1, Quat2);
    const float Cosom = abs(RawCosom);
	
    float Scale0, Scale1;
    if (Cosom < 0.9999f)
    {
        const float Omega = acos(Cosom);
        const float InvSin = 1.f / sin(Omega);
        Scale0 = sin((1.f - Slerp) * Omega) * InvSin;
        Scale1 = sin(Slerp * Omega) * InvSin;
    }
    else
    {
        Scale0 = 1.0f - Slerp;
        Scale1 = Slerp;
    }

	// In keeping with our flipped Cosom:
    Scale1 = RawCosom >= 0.0f ? Scale1 : -Scale1;

    return (Scale0 * Quat1) + (Scale1 * Quat2);
}
