// Copyright Epic Games, Inc. All Rights Reserved.

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Parameters

Buffer<uint>	{ParameterName}_MeshIndexBuffer;
Buffer<float>	{ParameterName}_MeshVertexBuffer;
Buffer<uint>	{ParameterName}_MeshSkinWeightBuffer;
Buffer<uint>	{ParameterName}_MeshSkinWeightLookupBuffer;
Buffer<float4>	{ParameterName}_MeshCurrBonesBuffer;
Buffer<float4>	{ParameterName}_MeshPrevBonesBuffer;
Buffer<float4>	{ParameterName}_MeshCurrSamplingBonesBuffer;
Buffer<float4>	{ParameterName}_MeshPrevSamplingBonesBuffer;
Buffer<float4>	{ParameterName}_MeshTangentBuffer;
Buffer<float2>	{ParameterName}_MeshTexCoordBuffer;
Buffer<float4>	{ParameterName}_MeshColorBuffer;
Buffer<uint>	{ParameterName}_MeshTriangleSamplerProbAliasBuffer;
uint			{ParameterName}_MeshNumSamplingRegionTriangles;
uint			{ParameterName}_MeshNumSamplingRegionVertices;
Buffer<uint>	{ParameterName}_MeshSamplingRegionsProbAliasBuffer;
Buffer<uint>	{ParameterName}_MeshSampleRegionsTriangleIndices;
Buffer<uint>	{ParameterName}_MeshSampleRegionsVertices;
Buffer<uint>	{ParameterName}_MeshTriangleMatricesOffsetBuffer;
uint			{ParameterName}_MeshTriangleCount;
uint			{ParameterName}_MeshVertexCount;
uint			{ParameterName}_MeshNumTexCoord;
uint			{ParameterName}_MeshNumWeights;
uint			{ParameterName}_MeshBoneWeightStride;
uint			{ParameterName}_MeshBoneIndexSizeBytes;
uint			{ParameterName}_MeshBoneWeightSizeBytes;
int				{ParameterName}_NumBones;
int				{ParameterName}_NumFilteredBones;
int				{ParameterName}_NumUnfilteredBones;
int				{ParameterName}_RandomMaxBone;
int				{ParameterName}_ExcludeBoneIndex;
Buffer<uint>	{ParameterName}_FilteredAndUnfilteredBones;
int				{ParameterName}_NumFilteredSockets;
int				{ParameterName}_FilteredSocketBoneOffset;
Buffer<int>		{ParameterName}_UvMappingBuffer;
uint			{ParameterName}_UvMappingBufferLength;
uint			{ParameterName}_UvMappingSet;
Buffer<uint>	{ParameterName}_ConnectivityBuffer;
uint			{ParameterName}_ConnectivityBufferLength;
uint			{ParameterName}_ConnectivityMaxAdjacentPerVertex;
float4x4		{ParameterName}_InstanceTransform;
float4x4		{ParameterName}_InstancePrevTransform;
float4			{ParameterName}_InstanceRotation;
float4			{ParameterName}_InstancePrevRotation;
float			{ParameterName}_InstanceInvDeltaTime;
float3			{ParameterName}_PreSkinnedLocalBoundsCenter;
float3			{ParameterName}_PreSkinnedLocalBoundsExtents;
uint			{ParameterName}_EnabledFeatures;
#if DISKELMESH_ALLOW_DEFORMED
Buffer<float>	{ParameterName}_DeformedCurrPositionBuffer;
Buffer<float>	{ParameterName}_DeformedPrevPositionBuffer;
Buffer<float4>	{ParameterName}_DeformedTangentBuffer;
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Flags test
bool UniformTriangleSamplingEnable_{ParameterName}()	{ return ({ParameterName}_EnabledFeatures & 0x01) == 0x01; }
bool UniformSamplingRegionEnabled_{ParameterName}()		{ return ({ParameterName}_EnabledFeatures & 0x02) == 0x02; }
bool UnlimitedBoneInfluences_{ParameterName}()			{ return ({ParameterName}_EnabledFeatures & 0x04) == 0x04; }
bool HasMeshColors_{ParameterName}()					{ return ({ParameterName}_EnabledFeatures & 0x08) == 0x08; }

#if DISKELMESH_ALLOW_DEFORMED
bool UseDeformedPositions_{ParameterName}()				{ return ({ParameterName}_EnabledFeatures & 0x10) == 0x10; }
bool UseDeformedTangents_{ParameterName}()				{ return ({ParameterName}_EnabledFeatures & 0x20) == 0x20; }
#endif

void GetIndexWeight_{ParameterName}(int BoneIndexOffset, int BoneWeightOffset, int Layer, out int4 BlendIndices, out float4 BlendWeights)
{
#if DISKELMESH_ALLOW_16BIT
	BlendIndices = {ParameterName}_MeshBoneIndexSizeBytes == 1 ?
		DISkelMesh_UnpackIndices4x8({ParameterName}_MeshSkinWeightBuffer[BoneIndexOffset + Layer]) :
		DISkelMesh_UnpackIndices4x16({ParameterName}_MeshSkinWeightBuffer[BoneIndexOffset + (Layer << 1)], {ParameterName}_MeshSkinWeightBuffer[BoneIndexOffset + (Layer << 1) + 1]);

	BlendWeights = {ParameterName}_MeshBoneWeightSizeBytes == 1 ?
		DISkelMesh_UnpackWeights4x8({ParameterName}_MeshSkinWeightBuffer[BoneWeightOffset + Layer]) :
		DISkelMesh_UnpackWeights4x16({ParameterName}_MeshSkinWeightBuffer[BoneWeightOffset + (Layer << 1)], {ParameterName}_MeshSkinWeightBuffer[BoneWeightOffset + (Layer << 1) + 1]);
#else
	BlendIndices = DISkelMesh_UnpackIndices4x8({ParameterName}_MeshSkinWeightBuffer[BoneIndexOffset + Layer]);
	BlendWeights = DISkelMesh_UnpackWeights4x8({ParameterName}_MeshSkinWeightBuffer[BoneWeightOffset + Layer]);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Misc functions
void GetPreSkinnedLocalBounds_{ParameterName}(out float3 OutCenter, out float3 OutExtentsMin, out float3 OutExtentsMax, out float3 OutExtents, out float3 OutHalfExtents)
{
	OutCenter		= {ParameterName}_PreSkinnedLocalBoundsCenter;
	OutExtents		= {ParameterName}_PreSkinnedLocalBoundsExtents;
	OutHalfExtents	= OutExtents * 0.5f;
	OutExtentsMin	= OutCenter - OutHalfExtents;
	OutExtentsMax	= OutCenter + OutHalfExtents;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Triangle Sampling
float3x4 GetPrevBoneSkinningMatrix_{ParameterName}(uint Bone)
{
    return float3x4({ParameterName}_MeshPrevBonesBuffer[Bone * 3], {ParameterName}_MeshPrevBonesBuffer[Bone * 3 + 1], {ParameterName}_MeshPrevBonesBuffer[Bone * 3 + 2]);
}

float3x4 GetPrevSkinningMatrix_{ParameterName}(uint SectionBoneOffset, int4 BlendIndices, float4 BlendWeights)
{
	float3x4 Result;
    Result = GetPrevBoneSkinningMatrix_{ParameterName}(SectionBoneOffset + BlendIndices.x) * BlendWeights.x;
    Result += GetPrevBoneSkinningMatrix_{ParameterName}(SectionBoneOffset + BlendIndices.y) * BlendWeights.y;
    Result += GetPrevBoneSkinningMatrix_{ParameterName}(SectionBoneOffset + BlendIndices.z) * BlendWeights.z;
    Result += GetPrevBoneSkinningMatrix_{ParameterName}(SectionBoneOffset + BlendIndices.w) * BlendWeights.w;
	return Result;
}

float3x4 GetCurrBoneSkinningMatrix_{ParameterName}(uint Bone)
{
	return float3x4({ParameterName}_MeshCurrBonesBuffer[Bone * 3], {ParameterName}_MeshCurrBonesBuffer[Bone * 3 + 1], {ParameterName}_MeshCurrBonesBuffer[Bone * 3 + 2]);
}

float3x4 GetCurrSkinningMatrix_{ParameterName}(uint SectionBoneOffset, int4 BlendIndices, float4 BlendWeights)
{
	float3x4 Result;
    Result = GetCurrBoneSkinningMatrix_{ParameterName}(SectionBoneOffset + BlendIndices.x) * BlendWeights.x;
    Result += GetCurrBoneSkinningMatrix_{ParameterName}(SectionBoneOffset + BlendIndices.y) * BlendWeights.y;
    Result += GetCurrBoneSkinningMatrix_{ParameterName}(SectionBoneOffset + BlendIndices.z) * BlendWeights.z;
    Result += GetCurrBoneSkinningMatrix_{ParameterName}(SectionBoneOffset + BlendIndices.w) * BlendWeights.w;
	return Result;
}

void IsValidTriCoord_{ParameterName}(in int TriangleIndex, out bool IsValid)
{
	IsValid = TriangleIndex >= 0 && TriangleIndex < {ParameterName}_MeshTriangleCount;
}

void GetVertex_{ParameterName}(uint VertexIndex, out float3 OutPosition, out float3 OutTangentX, out float3 OutTangentY, out float3 OutTangentZ)
{
	// Early out for bad data
	if ( {ParameterName}_MeshVertexCount == 0 )
	{
		OutPosition = float3(0.0f, 0.0f, 0.0f);
		OutTangentX = float3(1.0f, 0.0f, 0.0f);
		OutTangentY = float3(0.0f, 1.0f, 0.0f);
		OutTangentZ = float3(0.0f, 0.0f, 1.0f);
		return;
	}

	VertexIndex = clamp(VertexIndex, 0, {ParameterName}_MeshVertexCount - 1);

	//-TODO: Use a R32G32B32 format
	float3 Position = float3({ParameterName}_MeshVertexBuffer[VertexIndex * 3], {ParameterName}_MeshVertexBuffer[VertexIndex * 3 + 1], {ParameterName}_MeshVertexBuffer[VertexIndex * 3 + 2]);
	float3 TangentX = TangentBias({ParameterName}_MeshTangentBuffer[VertexIndex * 2    ].xyz);
	float4 TangentZ = TangentBias({ParameterName}_MeshTangentBuffer[VertexIndex * 2 + 1].xyzw);	
	float3 TangentY = cross(TangentZ.xyz, TangentX.xyz) * TangentZ.w;

	OutPosition = Position;
	OutTangentX = cross(TangentY, TangentZ.xyz) * TangentZ.w;
	OutTangentY = TangentY;
	OutTangentZ = TangentZ.xyz;
}

void GetVertexData_{ParameterName}(int VertexIndex, out float3 OutPosition, out float3 OutTangentX, out float3 OutTangentY, out float3 OutTangentZ)
{
	GetVertex_{ParameterName}(VertexIndex, OutPosition, OutTangentX, OutTangentY, OutTangentZ);
}

void GetPointOnTriangle_{ParameterName}(uint TriangleIndex, float3 BaryCoord, out float3 OutPosition, out float3 OutTangentX, out float3 OutTangentY, out float3 OutTangentZ)
{
	// Early out for bad data
	if ( {ParameterName}_MeshTriangleCount == 0 )
	{
		OutPosition = float3(0.0f, 0.0f, 0.0f);
		OutTangentX = float3(1.0f, 0.0f, 0.0f);
		OutTangentY = float3(0.0f, 1.0f, 0.0f);
		OutTangentZ = float3(0.0f, 0.0f, 1.0f);
		return;
	}

	TriangleIndex = clamp(TriangleIndex, 0, {ParameterName}_MeshTriangleCount - 1);

	uint IndexBufferOffset = TriangleIndex * 3;
	uint VertexIndex0 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset];
	uint VertexIndex1 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset + 1];
	uint VertexIndex2 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset + 2];

	float3 Positions[3], TangentsX[3], TangentsY[3], TangentsZ[3];
	GetVertex_{ParameterName}(VertexIndex0, Positions[0], TangentsX[0], TangentsY[0], TangentsZ[0]);
	GetVertex_{ParameterName}(VertexIndex1, Positions[1], TangentsX[1], TangentsY[1], TangentsZ[1]);
	GetVertex_{ParameterName}(VertexIndex2, Positions[2], TangentsX[2], TangentsY[2], TangentsZ[2]);
	OutPosition = Positions[0] * BaryCoord.x + Positions[1] * BaryCoord.y  + Positions[2] * BaryCoord.z;
	OutTangentX = TangentsX[0] * BaryCoord.x + TangentsX[1] * BaryCoord.y  + TangentsX[2] * BaryCoord.z;
	OutTangentY = TangentsY[0] * BaryCoord.x + TangentsY[1] * BaryCoord.y  + TangentsY[2] * BaryCoord.z;
	OutTangentZ = TangentsZ[0] * BaryCoord.x + TangentsZ[1] * BaryCoord.y  + TangentsZ[2] * BaryCoord.z;
}

FDISkelMeshSkinnedVertex GetSkinnedVertex_{ParameterName}(uint VertexIndex)
{
	// Early out for bad data
	if ( {ParameterName}_MeshVertexCount == 0 )
	{
		FDISkelMeshSkinnedVertex SkinnedVertex;
		SkinnedVertex.Position = float3(0.0f, 0.0f, 0.0f);
		SkinnedVertex.PrevPosition = float3(0.0f, 0.0f, 0.0f);
		SkinnedVertex.TangentX = float3(1.0f, 0.0f, 0.0f);
		SkinnedVertex.TangentY = float3(0.0f, 1.0f, 0.0f);
		SkinnedVertex.TangentZ = float3(0.0f, 0.0f, 1.0f);
		return SkinnedVertex;
	}

	VertexIndex = clamp(VertexIndex, 0, {ParameterName}_MeshVertexCount - 1);

	// Read Positions For Skinning
	bool bNeedsSkinPositions = true;
	float3 CurrPosition = 0;
	float3 PrevPosition = 0;
#if DISKELMESH_ALLOW_DEFORMED
	if (UseDeformedPositions_{ParameterName}())
	{
		bNeedsSkinPositions = false;
		CurrPosition = float3({ParameterName}_DeformedCurrPositionBuffer[VertexIndex * 3], {ParameterName}_DeformedCurrPositionBuffer[VertexIndex * 3 + 1], {ParameterName}_DeformedCurrPositionBuffer[VertexIndex * 3 + 2]);
		PrevPosition = float3({ParameterName}_DeformedPrevPositionBuffer[VertexIndex * 3], {ParameterName}_DeformedPrevPositionBuffer[VertexIndex * 3 + 1], {ParameterName}_DeformedPrevPositionBuffer[VertexIndex * 3 + 2]);
	}
	else
#endif
	{
		CurrPosition = float3({ParameterName}_MeshVertexBuffer[VertexIndex * 3], {ParameterName}_MeshVertexBuffer[VertexIndex * 3 + 1], {ParameterName}_MeshVertexBuffer[VertexIndex * 3 + 2]);
		PrevPosition = CurrPosition;
	}

	// Read Tangents For Skinning
	bool bNeedsSkinTangents = true;
	float3 TangentX = 0;
	float4 TangentZ = 0;
#if DISKELMESH_ALLOW_DEFORMED
	if (UseDeformedTangents_{ParameterName}())
	{
		bNeedsSkinTangents = false;
		TangentX = TangentBias({ParameterName}_DeformedTangentBuffer[VertexIndex * 2    ].xyz);
		TangentZ = TangentBias({ParameterName}_DeformedTangentBuffer[VertexIndex * 2 + 1].xyzw);
	}
	else
#endif
	{
		TangentX = TangentBias({ParameterName}_MeshTangentBuffer[VertexIndex * 2    ].xyz);
		TangentZ = TangentBias({ParameterName}_MeshTangentBuffer[VertexIndex * 2 + 1].xyzw);
	}

	// Perform skinning if needed
	if ( {ParameterName}_MeshNumWeights > 0 && (bNeedsSkinPositions || bNeedsSkinTangents))
	{
		float3x4 PrevBoneMatrix;
		float3x4 CurrBoneMatrix;

		// Get the matrix offset for each vertex because BlendIndices are stored relatively to each section start vertex.
		uint SectionBoneOffset = {ParameterName}_MeshTriangleMatricesOffsetBuffer[VertexIndex];

#if DISKELMESH_BONE_INFLUENCES == DISKELMESH_BONE_INFLUENCES_UNLIMITED
		BRANCH
		if (UnlimitedBoneInfluences_{ParameterName}())
		{
			const uint BlendOffsetCount = {ParameterName}_MeshSkinWeightLookupBuffer[VertexIndex];
			const int NumBoneInfluences = BlendOffsetCount & 0xff;
			const int StreamOffset = BlendOffsetCount >> 8;
			const int WeightsOffset = StreamOffset + ({ParameterName}_MeshBoneIndexSizeBytes * NumBoneInfluences);
			const float WeightNorm = {ParameterName}_MeshBoneWeightSizeBytes == 1 ? 1.0f / 255.0f : 1.0f / 65535.0f;

			PrevBoneMatrix = float3x4(float4(0,0,0,0), float4(0,0,0,0), float4(0,0,0,0));
			CurrBoneMatrix = float3x4(float4(0,0,0,0), float4(0,0,0,0), float4(0,0,0,0));
			for (int InfluenceIdx = 0; InfluenceIdx < NumBoneInfluences; InfluenceIdx++)
			{
				const int BoneIndexOffset = StreamOffset + ({ParameterName}_MeshBoneIndexSizeBytes * InfluenceIdx);
				int BoneIndex = {ParameterName}_MeshSkinWeightBuffer[BoneIndexOffset];
				if ( {ParameterName}_MeshBoneIndexSizeBytes > 1 )
				{
					BoneIndex |= {ParameterName}_MeshSkinWeightBuffer[BoneIndexOffset + 1] << 8;
				}

				const int BoneWeightOffset = WeightsOffset + ({ParameterName}_MeshBoneWeightSizeBytes * InfluenceIdx);
				int iBoneWeight = {ParameterName}_MeshSkinWeightBuffer[BoneWeightOffset];
				if ( {ParameterName}_MeshBoneWeightSizeBytes > 1 )
				{
					iBoneWeight |= {ParameterName}_MeshSkinWeightBuffer[BoneWeightOffset + 1] << 8;
				}
				float BoneWeight = float(iBoneWeight) * WeightNorm;
				PrevBoneMatrix += GetPrevBoneSkinningMatrix_{ParameterName}(SectionBoneOffset + BoneIndex) * BoneWeight;
				CurrBoneMatrix += GetCurrBoneSkinningMatrix_{ParameterName}(SectionBoneOffset + BoneIndex) * BoneWeight;
			}	
		}
		else
#endif
		{
			const uint BoneIndexOffset = VertexIndex * {ParameterName}_MeshBoneWeightStride;
			const uint BoneWeightOffset = BoneIndexOffset + (({ParameterName}_MeshNumWeights * {ParameterName}_MeshBoneIndexSizeBytes) >> 2);
			int4 BlendIndices;
			float4 BlendWeights;
			{
				GetIndexWeight_{ParameterName}(BoneIndexOffset, BoneWeightOffset, 0, BlendIndices, BlendWeights);
				CurrBoneMatrix = GetCurrSkinningMatrix_{ParameterName}(SectionBoneOffset, BlendIndices, BlendWeights);
				PrevBoneMatrix = GetPrevSkinningMatrix_{ParameterName}(SectionBoneOffset, BlendIndices, BlendWeights);
			}

#if DISKELMESH_BONE_INFLUENCES != DISKELMESH_BONE_INFLUENCES_MAX4
			// We are in 8 or unlimited mode so need to check for the extended weights
			if ({ParameterName}_MeshNumWeights == 8)
			{
				GetIndexWeight_{ParameterName}(BoneIndexOffset, BoneWeightOffset, 1, BlendIndices, BlendWeights);
				CurrBoneMatrix += GetCurrSkinningMatrix_{ParameterName}(SectionBoneOffset, BlendIndices, BlendWeights);
				PrevBoneMatrix += GetPrevSkinningMatrix_{ParameterName}(SectionBoneOffset, BlendIndices, BlendWeights);
			}
#endif
		}

		if ( bNeedsSkinPositions )
		{
			CurrPosition = mul(CurrBoneMatrix, float4(CurrPosition, 1.0f)).xyz;
			PrevPosition = mul(PrevBoneMatrix, float4(PrevPosition, 1.0f)).xyz;
		}

		if ( bNeedsSkinTangents )
		{
			// Not using InverseTranspose of matrices so assuming uniform scaling only (same as SkinCache)
			TangentX.xyz = mul(CurrBoneMatrix, float4(TangentX.xyz, 0.0f)).xyz;
			TangentZ.xyz = mul(CurrBoneMatrix, float4(TangentZ.xyz, 0.0f)).xyz;
		}
	}

	float3 TangentY = cross(TangentZ.xyz, TangentX.xyz) * TangentZ.w;

	FDISkelMeshSkinnedVertex SkinnedVertex;
	SkinnedVertex.Position = CurrPosition;
	SkinnedVertex.PrevPosition = PrevPosition;
	SkinnedVertex.TangentX = cross(TangentY, TangentZ.xyz) * TangentZ.w;
	SkinnedVertex.TangentY = TangentY;
	SkinnedVertex.TangentZ = TangentZ.xyz;
	return SkinnedVertex;
}

FDISkelMeshSkinnedVertex GetSkinnedVertexWS_{ParameterName}(uint VertexIndex)
{
	FDISkelMeshSkinnedVertex SkinnedVertex = GetSkinnedVertex_{ParameterName}(VertexIndex);
    SkinnedVertex.Position		= mul(float4(SkinnedVertex.Position, 1.0f), {ParameterName}_InstanceTransform).xyz;
    SkinnedVertex.PrevPosition	= mul(float4(SkinnedVertex.PrevPosition, 1.0f), {ParameterName}_InstancePrevTransform).xyz;
    SkinnedVertex.TangentX		= normalize(mul(float4(SkinnedVertex.TangentX, 0.0f), {ParameterName}_InstanceTransform).xyz);
    SkinnedVertex.TangentY		= normalize(mul(float4(SkinnedVertex.TangentY, 0.0f), {ParameterName}_InstanceTransform).xyz);
    SkinnedVertex.TangentZ		= normalize(mul(float4(SkinnedVertex.TangentZ, 0.0f), {ParameterName}_InstanceTransform).xyz);
	return SkinnedVertex;
}

FDISkelMeshSkinnedVertex GetSkinnedPointOnTriangle_{ParameterName}(uint TriangleIndex, float3 BaryCoord)
{
	// Early out for bad data
	if ( {ParameterName}_MeshTriangleCount == 0 )
	{
		FDISkelMeshSkinnedVertex SkinnedVertex;
		SkinnedVertex.Position = float3(0.0f, 0.0f, 0.0f);
		SkinnedVertex.PrevPosition = float3(0.0f, 0.0f, 0.0f);
		SkinnedVertex.TangentX = float3(1.0f, 0.0f, 0.0f);
		SkinnedVertex.TangentY = float3(0.0f, 1.0f, 0.0f);
		SkinnedVertex.TangentZ = float3(0.0f, 0.0f, 1.0f);
		return SkinnedVertex;
	}

	TriangleIndex = clamp(TriangleIndex, 0, {ParameterName}_MeshTriangleCount - 1);

	uint IndexBufferOffset = TriangleIndex * 3;
	uint VertexIndex0 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset];
	uint VertexIndex1 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset + 1];
	uint VertexIndex2 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset + 2];

	FDISkelMeshSkinnedVertex SkinnedVertex0 = GetSkinnedVertex_{ParameterName}(VertexIndex0);
	FDISkelMeshSkinnedVertex SkinnedVertex1 = GetSkinnedVertex_{ParameterName}(VertexIndex1);
	FDISkelMeshSkinnedVertex SkinnedVertex2 = GetSkinnedVertex_{ParameterName}(VertexIndex2);

	FDISkelMeshSkinnedVertex FinalVertex;
	FinalVertex.Position		= (SkinnedVertex0.Position * BaryCoord.x) + (SkinnedVertex1.Position * BaryCoord.y) + (SkinnedVertex2.Position * BaryCoord.z);
	FinalVertex.PrevPosition	= (SkinnedVertex0.PrevPosition * BaryCoord.x) + (SkinnedVertex1.PrevPosition * BaryCoord.y) + (SkinnedVertex2.PrevPosition * BaryCoord.z);
	FinalVertex.TangentX		= (SkinnedVertex0.TangentX * BaryCoord.x) + (SkinnedVertex1.TangentX * BaryCoord.y) + (SkinnedVertex2.TangentX * BaryCoord.z);
	FinalVertex.TangentY		= (SkinnedVertex0.TangentY * BaryCoord.x) + (SkinnedVertex1.TangentY * BaryCoord.y) + (SkinnedVertex2.TangentY * BaryCoord.z);
	FinalVertex.TangentZ		= (SkinnedVertex0.TangentZ * BaryCoord.x) + (SkinnedVertex1.TangentZ * BaryCoord.y) + (SkinnedVertex2.TangentZ * BaryCoord.z);

	return FinalVertex;
}

FDISkelMeshSkinnedVertex GetSkinnedPointOnTriangleWS_{ParameterName}(uint TriangleIndex, float3 BaryCoord)
{
	FDISkelMeshSkinnedVertex FinalVertex = GetSkinnedPointOnTriangle_{ParameterName}(TriangleIndex, BaryCoord);
	FinalVertex.Position		= mul(float4(FinalVertex.Position, 1.0f), {ParameterName}_InstanceTransform).xyz;
	FinalVertex.PrevPosition	= mul(float4(FinalVertex.PrevPosition, 1.0f), {ParameterName}_InstancePrevTransform).xyz;
	FinalVertex.TangentX		= mul(float4(FinalVertex.TangentX, 0.0f), {ParameterName}_InstanceTransform).xyz;
	FinalVertex.TangentY		= mul(float4(FinalVertex.TangentY, 0.0f), {ParameterName}_InstanceTransform).xyz;
	FinalVertex.TangentZ		= mul(float4(FinalVertex.TangentZ, 0.0f), {ParameterName}_InstanceTransform).xyz;

	return FinalVertex;
}

void RandomTriCoord_{ParameterName}(uint Seed1, uint Seed2, uint Seed3, out int OutTriangle, out float3 OutBaryCoord)
{
	// Early out for bad data
	if ({ParameterName}_MeshTriangleCount == 0)
	{
		OutTriangle = -1;
		OutBaryCoord = 1.0f / 3.0f;
		return;
	}

	float RandT0 = DISKelMesh_Random(Seed1, Seed2, Seed3);

	[branch]
	if ( !UniformTriangleSamplingEnable_{ParameterName}() )
	{
		// Uniform triangle id selection
		OutTriangle = min(uint(RandT0 * float({ParameterName}_MeshTriangleCount)), {ParameterName}_MeshTriangleCount - 1); // avoid % by using mul/min to Tri = MeshTriangleCountName
	}
	else
	{
		// Uniform area weighted position selection (using alias method from Alias method from FWeightedRandomSampler)
		uint TriangleIndex = min(uint(RandT0*float({ParameterName}_MeshTriangleCount)), {ParameterName}_MeshTriangleCount - 1);

#if DISKELMESH_PROBALIAS_FORMAT == DISKELMESH_PROBALIAS_FORMAT_64
		float TriangleProbability = asfloat({ParameterName}_MeshTriangleSamplerProbAliasBuffer[(TriangleIndex * 2) + 0]);
		uint TriangleAlias = {ParameterName}_MeshTriangleSamplerProbAliasBuffer[(TriangleIndex * 2) + 1];
#elif DISKELMESH_PROBALIAS_FORMAT == DISKELMESH_PROBALIAS_FORMAT_24_8
		uint TriangleProbAlias = {ParameterName}_MeshTriangleSamplerProbAliasBuffer[TriangleIndex];
		float TriangleProbability = float(TriangleProbAlias & 0xff) / 255.0f;
		uint TriangleAlias = TriangleProbAlias >> 8;
#elif DISKELMESH_PROBALIAS_FORMAT == DISKELMESH_PROBALIAS_FORMAT_23_9
		uint TriangleProbAlias = {ParameterName}_MeshTriangleSamplerProbAliasBuffer[TriangleIndex];
		float TriangleProbability = float(TriangleProbAlias & 0x1ff) / 511.0f;
		uint TriangleAlias = TriangleProbAlias >> 9;
#endif

		// Alias check
		float RandT1 = DISKelMesh_Random(Seed1, Seed2, Seed3);
		if( RandT1 > TriangleProbability )
		{
			TriangleIndex = TriangleAlias;
		}
		OutTriangle = int(TriangleIndex);
	}

	OutBaryCoord = DISKelMesh_RandomBarycentricCoord(Seed1, Seed2, Seed3);
}

void GetSkinnedTriangleDataWS_{ParameterName}(in uint TriangleIndex, in float3 BaryCoord, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent)
{
    FDISkelMeshSkinnedVertex SkinnedVertex = GetSkinnedPointOnTriangleWS_{ParameterName}(TriangleIndex, BaryCoord);
	OutPosition	= SkinnedVertex.Position;
	OutVelocity	= (SkinnedVertex.Position - SkinnedVertex.PrevPosition) * {ParameterName}_InstanceInvDeltaTime;
	OutNormal	= normalize(SkinnedVertex.TangentZ);
	OutBinormal	= normalize(SkinnedVertex.TangentY);
	OutTangent	= normalize(SkinnedVertex.TangentX);
}

void GetSkinnedTriangleData_{ParameterName}(in uint TriangleIndex, in float3 BaryCoord, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent)
{
    FDISkelMeshSkinnedVertex SkinnedVertex = GetSkinnedPointOnTriangle_{ParameterName}(TriangleIndex, BaryCoord);
	OutPosition	= SkinnedVertex.Position;
	OutVelocity	= (SkinnedVertex.Position - SkinnedVertex.PrevPosition) * {ParameterName}_InstanceInvDeltaTime;
	OutNormal	= normalize(SkinnedVertex.TangentZ);
	OutBinormal	= normalize(SkinnedVertex.TangentY);
	OutTangent	= normalize(SkinnedVertex.TangentX);
}

void GetSkinnedTriangleDataInterpolatedWS_{ParameterName}(in uint TriangleIndex, in float3 BaryCoord, in float Interp, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent)
{
    FDISkelMeshSkinnedVertex SkinnedVertex = GetSkinnedPointOnTriangleWS_{ParameterName}(TriangleIndex, BaryCoord);
	OutPosition	= lerp(SkinnedVertex.PrevPosition, SkinnedVertex.Position, Interp);
	OutVelocity	= (SkinnedVertex.Position - SkinnedVertex.PrevPosition) * {ParameterName}_InstanceInvDeltaTime;
	OutNormal	= normalize(SkinnedVertex.TangentZ);
	OutBinormal	= normalize(SkinnedVertex.TangentY);
	OutTangent	= normalize(SkinnedVertex.TangentX);
}

void GetSkinnedTriangleDataInterpolated_{ParameterName}(in uint TriangleIndex, in float3 BaryCoord, in float Interp, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent)
{
    FDISkelMeshSkinnedVertex SkinnedVertex = GetSkinnedPointOnTriangle_{ParameterName}(TriangleIndex, BaryCoord);
	OutPosition	= lerp(SkinnedVertex.PrevPosition, SkinnedVertex.Position, Interp);
	OutVelocity	= (SkinnedVertex.Position - SkinnedVertex.PrevPosition) * {ParameterName}_InstanceInvDeltaTime;
	OutNormal	= normalize(SkinnedVertex.TangentZ);
	OutBinormal	= normalize(SkinnedVertex.TangentY);
	OutTangent	= normalize(SkinnedVertex.TangentX);
}

void GetSkinnedTriangleVertexDataHelper_{ParameterName}(
	int TriangleIndex, 
	float Interp,
	bool bWorldSpace,
	out float3 Position0, out float3 Velocity0, out float3 Normal0, out float3 Binormal0, out float3 Tangent0,
	out float3 Position1, out float3 Velocity1, out float3 Normal1, out float3 Binormal1, out float3 Tangent1,
	out float3 Position2, out float3 Velocity2, out float3 Normal2, out float3 Binormal2, out float3 Tangent2
)
{
    int VertexIndex0 = 0;
    int VertexIndex1 = 0;
    int VertexIndex2 = 0;
	if ( {ParameterName}_MeshTriangleCount != 0 )
	{
		TriangleIndex = clamp(TriangleIndex, 0, int({ParameterName}_MeshTriangleCount) - 1);
		const uint IndexBufferOffset = TriangleIndex * 3;
		VertexIndex0 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset];
		VertexIndex1 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset + 1];
		VertexIndex2 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset + 2];
	}

	FDISkelMeshSkinnedVertex SkinnedVertex0;
	FDISkelMeshSkinnedVertex SkinnedVertex1;
	FDISkelMeshSkinnedVertex SkinnedVertex2;

	BRANCH
	if ( bWorldSpace )
	{
		SkinnedVertex0 = GetSkinnedVertexWS_{ParameterName}(VertexIndex0);
		SkinnedVertex1 = GetSkinnedVertexWS_{ParameterName}(VertexIndex1);
		SkinnedVertex2 = GetSkinnedVertexWS_{ParameterName}(VertexIndex2);
	}
	else
	{
		SkinnedVertex0 = GetSkinnedVertex_{ParameterName}(VertexIndex0);
		SkinnedVertex1 = GetSkinnedVertex_{ParameterName}(VertexIndex1);
		SkinnedVertex2 = GetSkinnedVertex_{ParameterName}(VertexIndex2);
	}

	Position0		= lerp(SkinnedVertex0.PrevPosition, SkinnedVertex0.Position, Interp);
	Velocity0		= SkinnedVertex0.Position - SkinnedVertex0.PrevPosition;
	Normal0			= SkinnedVertex0.TangentZ;
	Binormal0		= SkinnedVertex0.TangentY;
	Tangent0		= SkinnedVertex0.TangentX;

	Position1		= lerp(SkinnedVertex1.PrevPosition, SkinnedVertex1.Position, Interp);
	Velocity1		= SkinnedVertex1.Position - SkinnedVertex1.PrevPosition;
	Normal1			= SkinnedVertex1.TangentZ;
	Binormal1		= SkinnedVertex1.TangentY;
	Tangent1		= SkinnedVertex1.TangentX;

	Position2		= lerp(SkinnedVertex2.PrevPosition, SkinnedVertex2.Position, Interp);
	Velocity2		= SkinnedVertex2.Position - SkinnedVertex2.PrevPosition;
	Normal2			= SkinnedVertex2.TangentZ;
	Binormal2		= SkinnedVertex2.TangentY;
	Tangent2		= SkinnedVertex2.TangentX;
}

void GetSkinnedTriangleVertexData_{ParameterName}(
	int TriangleIndex, 
	out float3 Position0, out float3 Velocity0, out float3 Normal0, out float3 Binormal0, out float3 Tangent0,
	out float3 Position1, out float3 Velocity1, out float3 Normal1, out float3 Binormal1, out float3 Tangent1,
	out float3 Position2, out float3 Velocity2, out float3 Normal2, out float3 Binormal2, out float3 Tangent2
)
{
	GetSkinnedTriangleVertexDataHelper_{ParameterName}(
		TriangleIndex, 1.0f, false,
		Position0, Velocity0, Normal0, Binormal0, Tangent0, Position1, Velocity1, Normal1, Binormal1, Tangent1, Position2, Velocity2, Normal2, Binormal2, Tangent2
	);
}

void GetSkinnedTriangleVertexDataWS_{ParameterName}(
	int TriangleIndex, 
	out float3 Position0, out float3 Velocity0, out float3 Normal0, out float3 Binormal0, out float3 Tangent0,
	out float3 Position1, out float3 Velocity1, out float3 Normal1, out float3 Binormal1, out float3 Tangent1,
	out float3 Position2, out float3 Velocity2, out float3 Normal2, out float3 Binormal2, out float3 Tangent2
)
{
	GetSkinnedTriangleVertexDataHelper_{ParameterName}(
		TriangleIndex, 1.0f, true,
		Position0, Velocity0, Normal0, Binormal0, Tangent0, Position1, Velocity1, Normal1, Binormal1, Tangent1, Position2, Velocity2, Normal2, Binormal2, Tangent2
	);
}

void GetSkinnedTriangleVertexDataInterpolated_{ParameterName}(
	int TriangleIndex, 
	float Interp,
	out float3 Position0, out float3 Velocity0, out float3 Normal0, out float3 Binormal0, out float3 Tangent0,
	out float3 Position1, out float3 Velocity1, out float3 Normal1, out float3 Binormal1, out float3 Tangent1,
	out float3 Position2, out float3 Velocity2, out float3 Normal2, out float3 Binormal2, out float3 Tangent2
)
{
	GetSkinnedTriangleVertexDataHelper_{ParameterName}(
		TriangleIndex, Interp, false,
		Position0, Velocity0, Normal0, Binormal0, Tangent0, Position1, Velocity1, Normal1, Binormal1, Tangent1, Position2, Velocity2, Normal2, Binormal2, Tangent2
	);
}

void GetSkinnedTriangleVertexDataWSInterpolated_{ParameterName}(
	int TriangleIndex, 
	float Interp,
	out float3 Position0, out float3 Velocity0, out float3 Normal0, out float3 Binormal0, out float3 Tangent0,
	out float3 Position1, out float3 Velocity1, out float3 Normal1, out float3 Binormal1, out float3 Tangent1,
	out float3 Position2, out float3 Velocity2, out float3 Normal2, out float3 Binormal2, out float3 Tangent2
)
{
	GetSkinnedTriangleVertexDataHelper_{ParameterName}(
		TriangleIndex, Interp, true,
		Position0, Velocity0, Normal0, Binormal0, Tangent0, Position1, Velocity1, Normal1, Binormal1, Tangent1, Position2, Velocity2, Normal2, Binormal2, Tangent2
	);
}

void GetTriUV_{ParameterName}(in uint TriangleIndex, in float3 BaryCoord, in int UVSet, out float2 OutUV)
{
    if ({ParameterName}_MeshNumTexCoord > 0)
	{
        uint IndexBufferOffset = TriangleIndex * 3;
		uint VertexIndex0 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset  ];
		uint VertexIndex1 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset+1];
		uint VertexIndex2 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset+2];

        uint Stride = {ParameterName}_MeshNumTexCoord;
        uint SelectedUVSet = clamp((uint) UVSet, 0, {ParameterName}_MeshNumTexCoord - 1);
		float2 UV0 = {ParameterName}_MeshTexCoordBuffer[VertexIndex0 * Stride + SelectedUVSet];
		float2 UV1 = {ParameterName}_MeshTexCoordBuffer[VertexIndex1 * Stride + SelectedUVSet];
		float2 UV2 = {ParameterName}_MeshTexCoordBuffer[VertexIndex2 * Stride + SelectedUVSet];

		OutUV = UV0 * BaryCoord.x + UV1 * BaryCoord.y + UV2 * BaryCoord.z;
	}
	else	
	{
		OutUV = 0.0f;
	}
}

void RandomTriangle_{ParameterName}(uint Seed1, uint Seed2, uint Seed3, out int OutTriangle, out float3 OutBaryCoord)
{
	float RandT0 = DISKelMesh_Random(Seed1, Seed2, Seed3);
	int TriangleIndex = min(int(RandT0 * float({ParameterName}_MeshTriangleCount)), (int){ParameterName}_MeshTriangleCount - 1); // avoid % by using mul/min to Tri = MeshTriangleCountName

	[branch]
	if ( UniformTriangleSamplingEnable_{ParameterName}() )
	{
		// Uniform area weighted position selection (using alias method from Alias method from FWeightedRandomSampler)
#if DISKELMESH_PROBALIAS_FORMAT == DISKELMESH_PROBALIAS_FORMAT_64
		float TriangleProbability = asfloat({ParameterName}_MeshTriangleSamplerProbAliasBuffer[(TriangleIndex * 2) + 0]);
		uint TriangleAlias = {ParameterName}_MeshTriangleSamplerProbAliasBuffer[(TriangleIndex * 2) + 1];
#elif DISKELMESH_PROBALIAS_FORMAT == DISKELMESH_PROBALIAS_FORMAT_24_8
		uint TriangleProbAlias = {ParameterName}_MeshTriangleSamplerProbAliasBuffer[TriangleIndex];
		float TriangleProbability = float(TriangleProbAlias & 0xff) / 255.0f;
		uint TriangleAlias = TriangleProbAlias >> 8;
#elif DISKELMESH_PROBALIAS_FORMAT == DISKELMESH_PROBALIAS_FORMAT_23_9
		uint TriangleProbAlias = {ParameterName}_MeshTriangleSamplerProbAliasBuffer[TriangleIndex];
		float TriangleProbability = float(TriangleProbAlias & 0x1ff) / 511.0f;
		uint TriangleAlias = TriangleProbAlias >> 9;
#endif

		// Alias check
		float RandT1 = DISKelMesh_Random(Seed1, Seed2, Seed3);
		if( RandT1 > TriangleProbability )
		{
			TriangleIndex = int(TriangleAlias);
		}
	}

	OutTriangle = TriangleIndex;
	OutBaryCoord = DISKelMesh_RandomBarycentricCoord(Seed1, Seed2, Seed3);
}

void GetTriangleCount_{ParameterName}(out int OutCount)
{
	OutCount = {ParameterName}_MeshTriangleCount;
}

void RandomFilteredTriangle_{ParameterName}(uint Seed1, uint Seed2, uint Seed3, out int OutTriangle, out float3 OutBaryCoord)
{
	// Early out for bad data
	if ({ParameterName}_MeshNumSamplingRegionTriangles == 0)
	{
		OutTriangle = -1;
		OutBaryCoord = 1.0f / 3.0f;
		return;
	}

	float RandT0 = DISKelMesh_Random(Seed1, Seed2, Seed3);
	uint RegionTriangle = min(uint(RandT0 * float({ParameterName}_MeshNumSamplingRegionTriangles)), {ParameterName}_MeshNumSamplingRegionTriangles - 1); // avoid % by using mul/min to Tri = MeshTriangleCountName

	[branch]
	if ( UniformSamplingRegionEnabled_{ParameterName}() )
	{
#if DISKELMESH_PROBALIAS_FORMAT == DISKELMESH_PROBALIAS_FORMAT_64
		float TriangleProbability = asfloat({ParameterName}_MeshSamplingRegionsProbAliasBuffer[(RegionTriangle * 2) + 0]);
		uint TriangleAlias = {ParameterName}_MeshSamplingRegionsProbAliasBuffer[(RegionTriangle * 2) + 1];
#elif DISKELMESH_PROBALIAS_FORMAT == DISKELMESH_PROBALIAS_FORMAT_24_8
		uint TriangleProbAlias = {ParameterName}_MeshSamplingRegionsProbAliasBuffer[RegionTriangle];
		float TriangleProbability = float(TriangleProbAlias & 0xff) / 255.0f;
		uint TriangleAlias = TriangleProbAlias >> 8;
#elif DISKELMESH_PROBALIAS_FORMAT == DISKELMESH_PROBALIAS_FORMAT_23_9
		uint TriangleProbAlias = {ParameterName}_MeshSamplingRegionsProbAliasBuffer[RegionTriangle];
		float TriangleProbability = float(TriangleProbAlias & 0x1ff) / 511.0f;
		uint TriangleAlias = TriangleProbAlias >> 9;
#endif
		float RandT1 = DISKelMesh_Random(Seed1, Seed2, Seed3);
		if ( RandT1 > TriangleProbability )
		{
			RegionTriangle = TriangleAlias;
		}
	}
	OutTriangle = (int){ParameterName}_MeshSampleRegionsTriangleIndices[RegionTriangle];
	OutBaryCoord = DISKelMesh_RandomBarycentricCoord(Seed1, Seed2, Seed3);
}

void GetFilteredTriangleCount_{ParameterName}(out int OutCount)
{
	OutCount = {ParameterName}_MeshNumSamplingRegionTriangles;
}

void GetFilteredTriangleAt_{ParameterName}(int FilteredIndex, out int OutTriangle)
{
	if ( {ParameterName}_MeshNumSamplingRegionTriangles > 0 )
	{
		FilteredIndex = clamp(FilteredIndex, 0, (int){ParameterName}_MeshNumSamplingRegionTriangles - 1);
		OutTriangle = (int){ParameterName}_MeshSampleRegionsTriangleIndices[FilteredIndex];
	}
	else
	{
		OutTriangle = 0;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Uv Mapping
void GetTriangleCoordAtUV_{ParameterName}(bool Enabled, float2 InUv, float InTolerance, out int OutTriangle, out float3 OutBaryCoord, out bool OutIsValid)
{
	UvMappingParameters MappingParam;
	MappingParam.NumUVs = {ParameterName}_MeshNumTexCoord;
	MappingParam.IndexBuffer = {ParameterName}_MeshIndexBuffer;
	MappingParam.UVBuffer = {ParameterName}_MeshTexCoordBuffer;
	MappingParam.UvMappingBuffer = {ParameterName}_UvMappingBuffer;
	MappingParam.UvMappingBufferLength = {ParameterName}_UvMappingBufferLength;
	MappingParam.UvMappingSet = {ParameterName}_UvMappingSet;

	UvMapping_GetTriangleCoordAtUV(MappingParam, Enabled, InUv, InTolerance, OutTriangle, OutBaryCoord, OutIsValid);
}

void GetTriangleCoordInAabb_{ParameterName}(bool Enabled, float2 InUvMin, float2 InUvMax, out int OutTriangle, out float3 OutBaryCoord, out bool OutIsValid)
{
	UvMappingParameters MappingParam;
	MappingParam.NumUVs = {ParameterName}_MeshNumTexCoord;
	MappingParam.IndexBuffer = {ParameterName}_MeshIndexBuffer;
	MappingParam.UVBuffer = {ParameterName}_MeshTexCoordBuffer;
	MappingParam.UvMappingBuffer = {ParameterName}_UvMappingBuffer;
	MappingParam.UvMappingBufferLength = {ParameterName}_UvMappingBufferLength;
	MappingParam.UvMappingSet = {ParameterName}_UvMappingSet;

	UvMapping_GetTriangleCoordInAabb(MappingParam, Enabled, InUvMin, InUvMax, OutTriangle, OutBaryCoord, OutIsValid);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Bone Sampling

void GetSkinnedBoneCommon_{ParameterName}(in int Bone, in float Interp, in bool bWorldSpace, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
	// Early out for bad data
	if ( {ParameterName}_NumBones + {ParameterName}_NumFilteredSockets == 0 )
	{
		OutPosition = float3(0.0f, 0.0f, 0.0f);
		OutRotation = float4(0.0f, 0.0f, 0.0f, 1.0f);
		OutScale	= float3(0.0f, 0.0f, 0.0f);
		OutVelocity = float3(0.0f, 0.0f, 0.0f);
		return;
	}

	Bone = clamp(Bone, 0, {ParameterName}_NumBones + {ParameterName}_NumFilteredSockets - 1);

    float3 PrevPosition = {ParameterName}_MeshPrevSamplingBonesBuffer[Bone * 3].xyz;
    float4 PrevRotation = {ParameterName}_MeshPrevSamplingBonesBuffer[Bone * 3 + 1];
	float3 PrevScale	= {ParameterName}_MeshPrevSamplingBonesBuffer[Bone * 3 + 2].xyz;
    float3 CurrPosition = {ParameterName}_MeshCurrSamplingBonesBuffer[Bone * 3].xyz;
    float4 CurrRotation	= {ParameterName}_MeshCurrSamplingBonesBuffer[Bone * 3 + 1];
	float3 CurrScale	= {ParameterName}_MeshCurrSamplingBonesBuffer[Bone * 3 + 2].xyz;

    if (bWorldSpace)
    {
        PrevPosition = mul(float4(PrevPosition, 1), {ParameterName}_InstancePrevTransform).xyz;
        CurrPosition = mul(float4(CurrPosition, 1), {ParameterName}_InstanceTransform).xyz;
		PrevRotation = NiagaraGPU_QuatMul({ParameterName}_InstancePrevRotation, PrevRotation);
		CurrRotation = NiagaraGPU_QuatMul({ParameterName}_InstanceRotation, CurrRotation);
		PrevScale = mul(PrevScale, (float3x3){ParameterName}_InstancePrevTransform);
		CurrScale = mul(CurrScale, (float3x3){ParameterName}_InstanceTransform);
    }
    CurrPosition = lerp(PrevPosition, CurrPosition, Interp);
	CurrScale	 = lerp(PrevScale, CurrScale, Interp);

    OutPosition = CurrPosition;
    OutRotation = DISKelMesh_QuatSlerp(PrevRotation, CurrRotation, Interp);
	OutScale	= CurrScale;
    OutVelocity = (CurrPosition - PrevPosition) * {ParameterName}_InstanceInvDeltaTime;
}

void GetSkinnedBoneData_{ParameterName}(in int Bone, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
    GetSkinnedBoneCommon_{ParameterName}(Bone, 1.0f, false, OutPosition, OutRotation, OutScale, OutVelocity);
}

void GetSkinnedBoneDataInterpolated_{ParameterName}(in int Bone, in float Interp, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
    GetSkinnedBoneCommon_{ParameterName}(Bone, Interp, false, OutPosition, OutRotation, OutScale, OutVelocity);
}

void GetSkinnedBoneDataWS_{ParameterName}(in int Bone, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
    GetSkinnedBoneCommon_{ParameterName}(Bone, 1.0f, true, OutPosition, OutRotation, OutScale, OutVelocity);
}

void GetSkinnedBoneDataWSInterpolated_{ParameterName}(in int Bone, in float Interp, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
    GetSkinnedBoneCommon_{ParameterName}(Bone, Interp, true, OutPosition, OutRotation, OutScale, OutVelocity);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Vertex Sampling

void GetTriVertices_{ParameterName}(in uint TriangleIndex, out int VertexIndex0, out int VertexIndex1, out int VertexIndex2)
{
	// Early out for bad data
	if ( {ParameterName}_MeshTriangleCount == 0 )
	{
		VertexIndex0 = 0;
		VertexIndex1 = 0;
		VertexIndex2 = 0;
		return;
	}

	TriangleIndex = clamp(TriangleIndex, 0, {ParameterName}_MeshTriangleCount - 1);
    uint IndexBufferOffset = TriangleIndex * 3;
    VertexIndex0 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset];
    VertexIndex1 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset + 1];
    VertexIndex2 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset + 2];
}

void GetTriCoordVertices_{ParameterName}(in int TriangleIndex, out int VertexIndex0, out int VertexIndex1, out int VertexIndex2)
{
	GetTriVertices_{ParameterName}(TriangleIndex, VertexIndex0, VertexIndex1, VertexIndex2);
}

void GetTriangleIndices_{ParameterName}(int Tri, out int Index0, out int Index1, out int Index2)
{
	GetTriVertices_{ParameterName}(Tri, Index0, Index1, Index2);
}

void GetTriColor_{ParameterName}(in uint TriangleIndex, in float3 BaryCoord, out float4 OutColor)
{
	// Early out for bad data
	if ( HasMeshColors_{ParameterName}() == false )
	{
		OutColor = float4(1, 1, 1, 1);
		return;
	}
	TriangleIndex = clamp(TriangleIndex, 0, {ParameterName}_MeshTriangleCount - 1);

    uint IndexBufferOffset = TriangleIndex * 3;
    uint VertexIndex0 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset];
    uint VertexIndex1 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset + 1];
    uint VertexIndex2 = {ParameterName}_MeshIndexBuffer[IndexBufferOffset + 2];
    OutColor = ({ParameterName}_MeshColorBuffer[VertexIndex0] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE * BaryCoord.x) + ({ParameterName}_MeshColorBuffer[VertexIndex1] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE * BaryCoord.y) + ({ParameterName}_MeshColorBuffer[VertexIndex2] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE * BaryCoord.z);
}

void GetSkinnedVertexData_{ParameterName}(in int VertexIndex, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent)
{
    FDISkelMeshSkinnedVertex SkinnedVertex = GetSkinnedVertex_{ParameterName}(VertexIndex);
    OutPosition = SkinnedVertex.Position;
    OutVelocity = (SkinnedVertex.Position - SkinnedVertex.PrevPosition) * {ParameterName}_InstanceInvDeltaTime;
	OutNormal = SkinnedVertex.TangentZ;
	OutBinormal = SkinnedVertex.TangentY;
	OutTangent = SkinnedVertex.TangentX;
}

void GetSkinnedVertexDataWS_{ParameterName}(in int VertexIndex, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent)
{
    FDISkelMeshSkinnedVertex SkinnedVertex = GetSkinnedVertex_{ParameterName}(VertexIndex);
    OutPosition = mul(float4(SkinnedVertex.Position, 1.0f), {ParameterName}_InstanceTransform).xyz;
    float3 PrevPosition = mul(float4(SkinnedVertex.PrevPosition, 1.0f), {ParameterName}_InstancePrevTransform).xyz;
    OutVelocity = (SkinnedVertex.Position - SkinnedVertex.PrevPosition) * {ParameterName}_InstanceInvDeltaTime;
    OutNormal   = normalize(mul(float4(SkinnedVertex.TangentZ, 0.0f), {ParameterName}_InstanceTransform).xyz);
    OutBinormal = normalize(mul(float4(SkinnedVertex.TangentY, 0.0f), {ParameterName}_InstanceTransform).xyz);
    OutTangent  = normalize(mul(float4(SkinnedVertex.TangentX, 0.0f), {ParameterName}_InstanceTransform).xyz);
}

void GetSkinnedVertexDataInterpolated_{ParameterName}(in int VertexIndex, in float Interp, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent)
{
    FDISkelMeshSkinnedVertex SkinnedVertex = GetSkinnedVertex_{ParameterName}(VertexIndex);
    OutPosition = lerp(SkinnedVertex.PrevPosition, SkinnedVertex.Position, Interp);
    OutVelocity = (SkinnedVertex.Position - SkinnedVertex.PrevPosition) * {ParameterName}_InstanceInvDeltaTime;
	OutNormal = SkinnedVertex.TangentZ;
	OutBinormal = SkinnedVertex.TangentY;
	OutTangent = SkinnedVertex.TangentX;
}

void GetSkinnedVertexDataInterpolatedWS_{ParameterName}(in int VertexIndex, in float Interp, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent)
{
    FDISkelMeshSkinnedVertex SkinnedVertex = GetSkinnedVertex_{ParameterName}(VertexIndex);
    OutPosition = mul(float4(SkinnedVertex.Position, 1.0f), {ParameterName}_InstanceTransform).xyz;
    const float3 PrevPosition = mul(float4(SkinnedVertex.PrevPosition, 1.0f), {ParameterName}_InstancePrevTransform).xyz;
    OutPosition	= lerp(OutPosition, PrevPosition, Interp);
    OutVelocity = (SkinnedVertex.Position - SkinnedVertex.PrevPosition) * {ParameterName}_InstanceInvDeltaTime;
    OutNormal   = normalize(mul(float4(SkinnedVertex.TangentZ, 0.0f), {ParameterName}_InstanceTransform).xyz);
    OutBinormal = normalize(mul(float4(SkinnedVertex.TangentY, 0.0f), {ParameterName}_InstanceTransform).xyz);
    OutTangent  = normalize(mul(float4(SkinnedVertex.TangentX, 0.0f), {ParameterName}_InstanceTransform).xyz);
}

void GetVertexColor_{ParameterName}(in int VertexIndex, out float4 OutColor)
{
    OutColor = HasMeshColors_{ParameterName}() ? {ParameterName}_MeshColorBuffer[VertexIndex] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE : float4(1, 1, 1, 1);
}

void GetVertexUV_{ParameterName}(in int VertexIndex, in int UVSet, out float2 OutUV)
{
    if ({ParameterName}_MeshNumTexCoord > 0)
    {
        uint Stride = {ParameterName}_MeshNumTexCoord;
        uint SelectedUVSet = clamp((uint) UVSet, 0, {ParameterName}_MeshNumTexCoord - 1);
        OutUV = {ParameterName}_MeshTexCoordBuffer[VertexIndex * Stride + SelectedUVSet];
    }
    else
    {
        OutUV = 0.0f;
    }
}

void IsValidVertex_{ParameterName}(int Vertex, out bool IsValid)
{
    IsValid = Vertex < (int){ParameterName}_MeshVertexCount;
}

void RandomVertex_{ParameterName}(NiagaraRandInfo RandInfo, out int OutVertex)
{
	float RandT0 = NiagaraRandomFloat(RandInfo);
	OutVertex = (int)min(uint(RandT0 * float({ParameterName}_MeshVertexCount)), {ParameterName}_MeshVertexCount - 1); // avoid % by using mul/min to Tri = MeshVertexCount
}

void GetVertexCount_{ParameterName}(out int VertexCount)
{
	VertexCount = (int){ParameterName}_MeshVertexCount;
}

void IsValidFilteredVertex_{ParameterName}(int FilteredIndex, out bool IsValid)
{
	IsValid = FilteredIndex < (int){ParameterName}_MeshNumSamplingRegionVertices;
}

void RandomFilteredVertex_{ParameterName}(NiagaraRandInfo RandInfo, out int OutVertex)
{
	// Early out for bad data
	if ( {ParameterName}_MeshNumSamplingRegionVertices == 0 )
	{
		OutVertex = -1;
		return;
	}

	float RandT0 = NiagaraRandomFloat(RandInfo);
	int FilteredIndex = (int)min(uint(RandT0 * float({ParameterName}_MeshNumSamplingRegionVertices)), {ParameterName}_MeshNumSamplingRegionVertices - 1); // avoid % by using mul/min to Tri = MeshVertexCount
	OutVertex = (int){ParameterName}_MeshSampleRegionsVertices[FilteredIndex];
}

void GetFilteredVertexCount_{ParameterName}(out int VertexCount)
{
	VertexCount = (int){ParameterName}_MeshNumSamplingRegionVertices;
}

void GetFilteredVertex_{ParameterName}(int FilteredIndex, out int VertexIndex)
{
	// Early out for bad data
	if ( {ParameterName}_MeshNumSamplingRegionVertices == 0 )
	{
		VertexIndex = -1;
		return;
	}

	FilteredIndex = clamp(FilteredIndex, 0, (int){ParameterName}_MeshNumSamplingRegionVertices - 1);
	VertexIndex = (int){ParameterName}_MeshSampleRegionsVertices[FilteredIndex];
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Filtered Bones / Sockets
void IsValidBone_{ParameterName}(in int BoneIndex, out bool IsValid)
{
	IsValid = BoneIndex >= 0 && BoneIndex < {ParameterName}_NumBones;
}

void RandomBone_{ParameterName}(NiagaraRandInfo RandInfo, out int Bone)
{
	float RandT0 = NiagaraRandomFloat(RandInfo);
	Bone = min(int(RandT0 * float({ParameterName}_RandomMaxBone)), {ParameterName}_RandomMaxBone);
	Bone = Bone != {ParameterName}_ExcludeBoneIndex ? Bone : Bone + 1;
}

void GetBoneCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_NumBones;
}

void GetParentBone_{ParameterName}(int BoneIndex, out int ParentIndex)
{
	ParentIndex = -1;
	if ( BoneIndex >= 0 && BoneIndex < {ParameterName}_NumBones )
	{
		ParentIndex = asint({ParameterName}_MeshCurrSamplingBonesBuffer[BoneIndex * 3 + 2].w);
		
	}
}

void GetFilteredBoneCount_{ParameterName}(out int Count)
{
    Count = {ParameterName}_NumFilteredBones;
}

void GetFilteredBone_{ParameterName}(in int InBoneIndex, out int Bone)
{
    int BoneIndex = min(InBoneIndex, {ParameterName}_NumFilteredBones - 1);
    Bone = {ParameterName}_NumFilteredBones > 0 ? (int){ParameterName}_FilteredAndUnfilteredBones[BoneIndex] : -1;
}

void RandomFilteredBone_{ParameterName}(NiagaraRandInfo RandInfo, out int Bone)
{
	float RandT0 = NiagaraRandomFloat(RandInfo);
    int FilteredBone = min(int(RandT0 * float({ParameterName}_NumFilteredBones)), {ParameterName}_NumFilteredBones - 1); // avoid % by using mul/min to Tri = MeshVertexCount
    Bone = {ParameterName}_NumFilteredBones > 0 ? (int){ParameterName}_FilteredAndUnfilteredBones[FilteredBone] : -1;
}

void GetUnfilteredBoneCount_{ParameterName}(out int Count)
{
    Count = {ParameterName}_NumUnfilteredBones;
}

void GetUnfilteredBone_{ParameterName}(in int InBoneIndex, out int Bone)
{
    int BoneIndex = min(InBoneIndex, {ParameterName}_NumUnfilteredBones - 1);
    Bone = {ParameterName}_NumFilteredBones > 0 ? (int){ParameterName}_FilteredAndUnfilteredBones[BoneIndex + {ParameterName}_NumFilteredBones] : BoneIndex;
}

void RandomUnfilteredBone_{ParameterName}(NiagaraRandInfo RandInfo, out int Bone)
{
	float RandT0 = NiagaraRandomFloat(RandInfo);
	if ( {ParameterName}_NumFilteredBones == 0 )
	{
		Bone = min(int(RandT0 * float({ParameterName}_RandomMaxBone)), {ParameterName}_RandomMaxBone);
		Bone = Bone != {ParameterName}_ExcludeBoneIndex ? Bone : Bone + 1;
	}
	else
	{
		int UnfilteredBone = min(int(RandT0 * float({ParameterName}_NumUnfilteredBones)), {ParameterName}_NumUnfilteredBones - 1); // avoid % by using mul/min to Tri = MeshVertexCount
		Bone = {ParameterName}_NumFilteredBones > 0 ? (int){ParameterName}_FilteredAndUnfilteredBones[UnfilteredBone + {ParameterName}_NumFilteredBones] : UnfilteredBone;
	}
}

void GetFilteredSocketCount_{ParameterName}(out int Count)
{
    Count = {ParameterName}_NumFilteredSockets;
}

void GetFilteredSocketTransform_{ParameterName}(in int SocketIndex, in bool bApplyTransform, out float3 OutTranslation, out float4 OutRotation, out float3 OutScale)
{
    if ( {ParameterName}_NumFilteredSockets > 0 )
	{
		SocketIndex = clamp(SocketIndex, 0, {ParameterName}_NumFilteredSockets - 1);
		int BufferOffset = ({ParameterName}_FilteredSocketBoneOffset + SocketIndex) * 3;

		OutTranslation = {ParameterName}_MeshCurrSamplingBonesBuffer[BufferOffset + 0].xyz;
		OutRotation = {ParameterName}_MeshCurrSamplingBonesBuffer[BufferOffset + 1];
		OutScale = {ParameterName}_MeshCurrSamplingBonesBuffer[BufferOffset + 2].xyz;

		if (bApplyTransform)
		{
			OutTranslation = mul(float4(OutTranslation, 1), {ParameterName}_InstanceTransform).xyz;
			OutRotation = NiagaraGPU_QuatMul({ParameterName}_InstanceRotation, OutRotation);
			OutScale = mul(OutScale, (float3x3){ParameterName}_InstanceTransform);
		}
	}
	else
	{
		OutTranslation = float3(0.0f, 0.0f, 0.0f);
		OutRotation = float4(0.0f, 0.0f, 0.0f, 1.0f);
		OutScale = float3(0.0f, 0.0f, 0.0f);
	}
}

void GetFilteredSocket_{ParameterName}(in int InSocketIndex, out int Bone)
{
    Bone = {ParameterName}_FilteredSocketBoneOffset + clamp(InSocketIndex, 0, {ParameterName}_NumFilteredSockets - 1);
}

void RandomFilteredSocket_{ParameterName}(NiagaraRandInfo RandInfo, out int SocketBone)
{
	float RandT0 = NiagaraRandomFloat(RandInfo);
    SocketBone = {ParameterName}_FilteredSocketBoneOffset + min(int(RandT0 * float({ParameterName}_NumFilteredSockets)), {ParameterName}_NumFilteredSockets - 1); // avoid % by using mul/min to Tri = MeshVertexCount
}

void RandomFilteredSocketOrBone_{ParameterName}(NiagaraRandInfo RandInfo, out int Bone)
{
	float RandT0 = NiagaraRandomFloat(RandInfo);
	int NumFilteredSocketsAndBones = {ParameterName}_NumFilteredBones + {ParameterName}_NumFilteredSockets;
    int FilteredIndex = min(int(RandT0 * float(NumFilteredSocketsAndBones)), NumFilteredSocketsAndBones - 1);
	if ( FilteredIndex < {ParameterName}_NumFilteredBones )
	{
		Bone = (int){ParameterName}_FilteredAndUnfilteredBones[FilteredIndex];
	}
	else
	{
		Bone = {ParameterName}_FilteredSocketBoneOffset + FilteredIndex - {ParameterName}_NumFilteredBones;
	}
}

void GetFilteredSocketOrBoneCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_NumFilteredBones + {ParameterName}_NumFilteredSockets;
}

void GetFilteredSocketOrBone_{ParameterName}(int FilteredIndex, out int Bone)
{
	int NumFilteredSocketsAndBones = {ParameterName}_NumFilteredBones + {ParameterName}_NumFilteredSockets;
	FilteredIndex = clamp(FilteredIndex, 0, NumFilteredSocketsAndBones - 1);
	if (FilteredIndex < 0)
	{
		Bone = -1;
	}
	else if ( FilteredIndex < {ParameterName}_NumFilteredBones )
	{
		Bone = (int){ParameterName}_FilteredAndUnfilteredBones[FilteredIndex];
	}
	else
	{
		Bone = {ParameterName}_FilteredSocketBoneOffset + FilteredIndex - {ParameterName}_NumFilteredBones;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Connectivity

void GetAdjacentTriangleIndex_{ParameterName}(int VertexId, uint AdjacencyIndex, out int TriangleIndex)
{
	TriangleIndex = -1;

	if (AdjacencyIndex < {ParameterName}_ConnectivityMaxAdjacentPerVertex)
	{
#if DISKELMESH_ADJ_INDEX_FORMAT == DISKELMESH_ADJ_INDEX_FORMAT_FULL
		const uint ReadIndex = (VertexId * {ParameterName}_ConnectivityMaxAdjacentPerVertex + AdjacencyIndex);
		if (ReadIndex < {ParameterName}_ConnectivityBufferLength)
		{
			TriangleIndex = {ParameterName}_ConnectivityBuffer[ReadIndex];
		}
#elif DISKELMESH_ADJ_INDEX_FORMAT == DISKELMESH_ADJ_INDEX_FORMAT_HALF
		const uint ReadIndex = ((VertexId * {ParameterName}_ConnectivityMaxAdjacentPerVertex + AdjacencyIndex) & ~0x1) >> 1;
		if (ReadIndex < {ParameterName}_ConnectivityBufferLength)
		{
			uint PackedIndices = {ParameterName}_ConnectivityBuffer[ReadIndex];
			uint UnpackedIndex = uint2(PackedIndices & 0xFFFF, PackedIndices >> 16)[AdjacencyIndex & 1];
			TriangleIndex = UnpackedIndex == 0xFFFF ? -1 : int(UnpackedIndex);
		}
#else
		#error Unsupported index format for adjacency information
#endif
	}
}

bool IsAdjacent_{ParameterName}(int VertexIndex, int TriangleIndex)
{
	for (uint AdjIt = 0; AdjIt < {ParameterName}_ConnectivityMaxAdjacentPerVertex; ++AdjIt)
	{
		int AdjTriangleIndex = -1;
		GetAdjacentTriangleIndex_{ParameterName}(VertexIndex, AdjIt, AdjTriangleIndex);
		if (AdjTriangleIndex == -1)
		{
			break;
		}
		if (AdjTriangleIndex == TriangleIndex)
		{
			return true;
		}
	}
	return false;
}

void GetTriangleNeighbor_{ParameterName}(int TriangleIndex, int EdgeIndex, out int NeighborTriangleIndex, out int NeighborEdgeIndex, out bool IsValid)
{
	NeighborTriangleIndex = -1;
	NeighborEdgeIndex = -1;

	int3 SourceVertexIndices;

	GetTriVertices_{ParameterName}(TriangleIndex, SourceVertexIndices.x, SourceVertexIndices.y, SourceVertexIndices.z);

	int StartSourceIndex = SourceVertexIndices[(EdgeIndex + 1) % 3];
	int EndSourceIndex = SourceVertexIndices[(EdgeIndex + 2) % 3];

	for (uint SourceAdjIt = 0; SourceAdjIt < {ParameterName}_ConnectivityMaxAdjacentPerVertex; ++SourceAdjIt)
	{
		int SourceAdjTriangleIndex = -1;
		GetAdjacentTriangleIndex_{ParameterName}(StartSourceIndex, SourceAdjIt, SourceAdjTriangleIndex);

		if (SourceAdjTriangleIndex == TriangleIndex) // ignore self
		{
			continue;
		}
		if (SourceAdjTriangleIndex == -1) // stop looking
		{
			break;
		}

		if (IsAdjacent_{ParameterName}(EndSourceIndex, SourceAdjTriangleIndex))
		{
			NeighborTriangleIndex = SourceAdjTriangleIndex;

			// now to find the edge index for the neighbor
			int3 NeighborVertexIndices;
			GetTriVertices_{ParameterName}(NeighborTriangleIndex, NeighborVertexIndices.x, NeighborVertexIndices.y, NeighborVertexIndices.z);

			bool3 NeighborVertMatches = bool3(
				IsAdjacent_{ParameterName}(NeighborVertexIndices.x, TriangleIndex),
				IsAdjacent_{ParameterName}(NeighborVertexIndices.y, TriangleIndex),
				IsAdjacent_{ParameterName}(NeighborVertexIndices.z, TriangleIndex));

			if (all(NeighborVertMatches.xy))
			{
				NeighborEdgeIndex = 2;
			}
			else if (all(NeighborVertMatches.yz))
			{
				NeighborEdgeIndex = 0;
			}
			else if (all(NeighborVertMatches.zx))
			{
				NeighborEdgeIndex = 1;
			}
			break;
		}
	}
	IsValid = NeighborTriangleIndex != -1;
}

void GetAdjacentTriangleIndex_{ParameterName}(in int VertexId, in int AdjacencyIndex, out int TriangleIndex, out bool IsValid)
{
	GetAdjacentTriangleIndex_{ParameterName}(VertexId, max(0, AdjacencyIndex), TriangleIndex);
	IsValid = TriangleIndex != -1;
}
