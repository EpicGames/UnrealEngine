// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/FX/Niagara/Private/NiagaraTransformUtils.ush"

uint				{ParameterName}_IsDataValid;
float				{ParameterName}_InvDeltaSeconds;
int					{ParameterName}_NumSockets;
int					{ParameterName}_NumFilteredSockets;
int					{ParameterName}_NumUnfilteredSockets;
float3				{ParameterName}_ComponentToTranslatedWorld_Translation;
float4				{ParameterName}_ComponentToTranslatedWorld_Rotation;
float3				{ParameterName}_ComponentToTranslatedWorld_Scale;
float3				{ParameterName}_PreviousComponentToTranslatedWorld_Translation;
float4				{ParameterName}_PreviousComponentToTranslatedWorld_Rotation;
float3				{ParameterName}_PreviousComponentToTranslatedWorld_Scale;
uint				{ParameterName}_SocketTransformOffset;
uint				{ParameterName}_PreviousSocketTransformOffset;
ByteAddressBuffer	{ParameterName}_SocketData;

////////////////////////////////////////////////////
// Note that when reading socket / socket index indirection we allocate +1 entries, this is to avoid branching when OOB

int SanitizeSocketIndex_{ParameterName}(int SocketIndex)
{
	return SocketIndex >= 0 && SocketIndex < {ParameterName}_NumSockets ? SocketIndex : {ParameterName}_NumSockets;
}

int SanitizeFilteredSocketIndex_{ParameterName}(int SocketIndex)
{
	const int FilteredSocketIndex = SocketIndex >= 0 && SocketIndex < {ParameterName}_NumFilteredSockets ? (SocketIndex + 1) : 0;
	return {ParameterName}_SocketData.Load(FilteredSocketIndex * 4);
}

int SanitizeUnfilteredSocketIndex_{ParameterName}(int SocketIndex)
{
	const int UnfilteredSocketIndex = SocketIndex >= 0 && SocketIndex < {ParameterName}_NumUnfilteredSockets ? (SocketIndex + 1 + {ParameterName}_NumFilteredSockets) : 0;
	return {ParameterName}_SocketData.Load(UnfilteredSocketIndex * 4);
}

void GetSocketData_{ParameterName}(int SocketIndex, float Interp, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
	uint CurrTransformOffset = {ParameterName}_SocketTransformOffset;
	uint PrevTransformOffset = {ParameterName}_PreviousSocketTransformOffset;
	CurrTransformOffset += SocketIndex * 10 * 4;
	PrevTransformOffset += SocketIndex * 10 * 4;

	FNiagaraTransform ComponentToTranslatedWorld = MakeTransform({ParameterName}_ComponentToTranslatedWorld_Translation, {ParameterName}_ComponentToTranslatedWorld_Rotation, {ParameterName}_ComponentToTranslatedWorld_Scale);
	FNiagaraTransform PreviousComponentToTranslatedWorld = MakeTransform({ParameterName}_PreviousComponentToTranslatedWorld_Translation, {ParameterName}_PreviousComponentToTranslatedWorld_Rotation, {ParameterName}_PreviousComponentToTranslatedWorld_Scale);

	FNiagaraTransform CurrTransform = LoadTransform({ParameterName}_SocketData, CurrTransformOffset);
	FNiagaraTransform PrevTransform = LoadTransform({ParameterName}_SocketData, PrevTransformOffset);

	CurrTransform = MultiplyTransform(CurrTransform, ComponentToTranslatedWorld);
	PrevTransform = MultiplyTransform(PrevTransform, PreviousComponentToTranslatedWorld);

	OutPosition	= lerp(PrevTransform.Translation, CurrTransform.Translation, Interp);
	OutRotation	= SlerpQuat(PrevTransform.Rotation, CurrTransform.Rotation, Interp);
	OutScale	= lerp(PrevTransform.Scale, CurrTransform.Scale, Interp);
	OutVelocity	= (CurrTransform.Translation - PrevTransform.Translation) * {ParameterName}_InvDeltaSeconds;
}

////////////////////////////////////////////////////

void GetComponentToWorld_{ParameterName}(out float3 Translation, out float4 Rotation, out float3 Scale)
{
	Translation = {ParameterName}_ComponentToTranslatedWorld_Translation;
	Rotation	= {ParameterName}_ComponentToTranslatedWorld_Rotation;
	Scale		= {ParameterName}_ComponentToTranslatedWorld_Scale;
}

void IsValid_{ParameterName}(out bool bValid)
{
	bValid = {ParameterName}_IsDataValid != 0;
}

void GetSocketCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_NumSockets;
}

void GetFilteredSocketCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_NumFilteredSockets;
}

void GetUnfilteredSocketCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_NumUnfilteredSockets;
}

void GetSocketTransform_{ParameterName}(int SocketIndex, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
	SocketIndex = SanitizeSocketIndex_{ParameterName}(SocketIndex);
	GetSocketData_{ParameterName}(SocketIndex, 1.0f, OutPosition, OutRotation, OutScale, OutVelocity);
}

void GetFilteredSocketTransform_{ParameterName}(int SocketIndex, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
	SocketIndex = SanitizeFilteredSocketIndex_{ParameterName}(SocketIndex);
	GetSocketData_{ParameterName}(SocketIndex, 1.0f, OutPosition, OutRotation, OutScale, OutVelocity);
}

void GetUnfilteredSocketTransform_{ParameterName}(int SocketIndex, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
	SocketIndex = SanitizeUnfilteredSocketIndex_{ParameterName}(SocketIndex);
	GetSocketData_{ParameterName}(SocketIndex, 1.0f, OutPosition, OutRotation, OutScale, OutVelocity);
}

void GetSocketTransformInterpolated_{ParameterName}(int SocketIndex, float Interpolation, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
	SocketIndex = SanitizeSocketIndex_{ParameterName}(SocketIndex);
	GetSocketData_{ParameterName}(SocketIndex, Interpolation, OutPosition, OutRotation, OutScale, OutVelocity);
}

void GetFilteredSocketTransformInterpolated_{ParameterName}(int SocketIndex, float Interpolation, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
	SocketIndex = SanitizeFilteredSocketIndex_{ParameterName}(SocketIndex);
	GetSocketData_{ParameterName}(SocketIndex, Interpolation, OutPosition, OutRotation, OutScale, OutVelocity);
}

void GetUnfilteredSocketTransformInterpolated_{ParameterName}(int SocketIndex, float Interpolation, out float3 OutPosition, out float4 OutRotation, out float3 OutScale, out float3 OutVelocity)
{
	SocketIndex = SanitizeUnfilteredSocketIndex_{ParameterName}(SocketIndex);
	GetSocketData_{ParameterName}(SocketIndex, Interpolation, OutPosition, OutRotation, OutScale, OutVelocity);
}
