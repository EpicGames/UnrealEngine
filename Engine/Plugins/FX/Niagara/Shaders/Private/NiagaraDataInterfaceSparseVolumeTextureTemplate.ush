// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/SparseVolumeTexture/SparseVolumeTextureCommon.ush"

SamplerState		{ParameterName}_TileDataTextureSampler;
Texture3D<uint>		{ParameterName}_PageTableTexture;
Texture3D			{ParameterName}_PhysicalTileDataATexture;
Texture3D			{ParameterName}_PhysicalTileDataBTexture;
uint4				{ParameterName}_PackedUniforms0;
uint4				{ParameterName}_PackedUniforms1;
int3				{ParameterName}_TextureSize;
int					{ParameterName}_MipLevels;
int					{ParameterName}_NumFrames;


void LoadSparseVolumeTexture_{ParameterName}(in int TexelX, in int TexelY, in int TexelZ, in int MipLevel, out float4 OutAttributesA, out float4 OutAttributesB)
{
	const FSparseVolumeTextureUniforms Uniforms = SparseVolumeTextureUnpackUniforms({ParameterName}_PackedUniforms0, {ParameterName}_PackedUniforms1);
	const int3 VoxelCoord = SparseVolumeTextureLoadPageTable({ParameterName}_PageTableTexture, Uniforms, int3(TexelX, TexelY, TexelZ), MipLevel);
	OutAttributesA = {ParameterName}_PhysicalTileDataATexture.Load(int4(VoxelCoord, 0));
	OutAttributesB = {ParameterName}_PhysicalTileDataBTexture.Load(int4(VoxelCoord, 0));
}

void SampleSparseVolumeTexture_{ParameterName}(in float3 UVW, in int MipLevel, out float4 OutAttributesA, out float4 OutAttributesB)
{
	const FSparseVolumeTextureUniforms Uniforms = SparseVolumeTextureUnpackUniforms({ParameterName}_PackedUniforms0, {ParameterName}_PackedUniforms1);
	const float3 VoxelUVW = SparseVolumeTextureSamplePageTable({ParameterName}_PageTableTexture, Uniforms, UVW, SVTADDRESSMODE_WRAP, SVTADDRESSMODE_WRAP, SVTADDRESSMODE_WRAP, MipLevel);
	OutAttributesA = SparseVolumeTextureSamplePhysicalTileData({ParameterName}_PhysicalTileDataATexture, {ParameterName}_PhysicalTileDataBTexture, {ParameterName}_TileDataTextureSampler, VoxelUVW, 0);
	OutAttributesB = SparseVolumeTextureSamplePhysicalTileData({ParameterName}_PhysicalTileDataATexture, {ParameterName}_PhysicalTileDataBTexture, {ParameterName}_TileDataTextureSampler, VoxelUVW, 1);
}

void GetTextureDimensions_{ParameterName}(int MipLevel, out int SizeX, out int SizeY, out int SizeZ)
{
	SizeX = float(max({ParameterName}_TextureSize.x >> MipLevel, 1));
	SizeY = float(max({ParameterName}_TextureSize.y >> MipLevel, 1));
	SizeZ = float(max({ParameterName}_TextureSize.z >> MipLevel, 1));
}

void GetNumMipLevels_{ParameterName}(out int OutMipLevels)
{
	OutMipLevels = {ParameterName}_MipLevels;
}

void GetNumFrames_{ParameterName}(out int OutNumFrames)
{
	OutNumFrames = {ParameterName}_NumFrames;
}