// Copyright Epic Games, Inc. All Rights Reserved.
	
float4x4 {ParameterName}_SplineTransform;
float4x4 {ParameterName}_SplineTransformRotationMat;
float4x4 {ParameterName}_SplineTransformInverse;
float4x4 {ParameterName}_SplineTransformInverseTranspose;
float4	{ParameterName}_SplineTransformRotation;
float3	{ParameterName}_DefaultUpVector;
float	{ParameterName}_SplineLength;
float	{ParameterName}_SplineDistanceStep;
float	{ParameterName}_InvSplineDistanceStep;
int		{ParameterName}_MaxIndex;

Buffer<float4> {ParameterName}_SplinePositionsLUT;
Buffer<float4> {ParameterName}_SplineScalesLUT;
Buffer<float4> {ParameterName}_SplineRotationsLUT;

void FindNeighborKeys_{ParameterName}(float InDistance, out int PrevKey, out int NextKey, out float Alpha)
{
	const float Key = InDistance * {ParameterName}_InvSplineDistanceStep;
	PrevKey = clamp(int(floor(Key)), 0, {ParameterName}_MaxIndex);
	NextKey = clamp(int(ceil(Key)), 0, {ParameterName}_MaxIndex);
	Alpha = frac(Key);
}

float3 EvaluatePosition_{ParameterName}(float InDistance)
{
	if ({ParameterName}_MaxIndex < 0)
	{
		return float3(0,0,0);
	}
	
	int PrevKey, NextKey;
	float Alpha;
	FindNeighborKeys_{ParameterName}(InDistance, PrevKey, NextKey, Alpha);
	
	if (NextKey == PrevKey)
	{
		return {ParameterName}_SplinePositionsLUT[PrevKey].xyz;
	}
	return lerp({ParameterName}_SplinePositionsLUT[PrevKey].xyz, {ParameterName}_SplinePositionsLUT[NextKey].xyz, Alpha).xyz;
}

float3 EvaluateScale_{ParameterName}(float InDistance)
{
	if ({ParameterName}_MaxIndex < 0)
	{
		return float3(1,1,1);
	}
	
	int PrevKey, NextKey;
	float Alpha;
	FindNeighborKeys_{ParameterName}(InDistance, PrevKey, NextKey, Alpha);
	
	if (NextKey == PrevKey)
	{
		return {ParameterName}_SplineScalesLUT[PrevKey].xyz;
	}
	return lerp({ParameterName}_SplineScalesLUT[PrevKey].xyz, {ParameterName}_SplineScalesLUT[NextKey].xyz, Alpha).xyz;
}
	
float4 EvaluateRotation_{ParameterName}(float InDistance)
{
	if ({ParameterName}_MaxIndex < 0)
	{
		return float4(0,0,0,1);
	}
	
	int PrevKey, NextKey;
	float Alpha;
	FindNeighborKeys_{ParameterName}(InDistance, PrevKey, NextKey, Alpha);

	if (NextKey == PrevKey)
	{
		return {ParameterName}_SplineRotationsLUT[PrevKey];
	}
	return NiagaraQuatSLerp({ParameterName}_SplineRotationsLUT[PrevKey], {ParameterName}_SplineRotationsLUT[NextKey], Alpha);
}
	
float4 GetRotationAtDistanceAlongSpline_{ParameterName}(float InDistance)
{
	float4 Quat = EvaluateRotation_{ParameterName}(InDistance);
	return Quat;
}
	
float3 EvaluateDerivativePosition_{ParameterName}(float InDistance)
{
	if ({ParameterName}_MaxIndex < 0)
	{
		return float3(0,0,0);
	}
	
	int PrevKey, NextKey;
	float Alpha;
	FindNeighborKeys_{ParameterName}(InDistance, PrevKey, NextKey, Alpha);

	if (NextKey == PrevKey)
	{
		if (NextKey < {ParameterName}_MaxIndex)
		{
			NextKey++;
		}
		else if (PrevKey > 0)
		{
			PrevKey--;
		}
		else
		{
			return float3(0,0,0);
		}
	}
	return ({ParameterName}_SplinePositionsLUT[NextKey] - {ParameterName}_SplinePositionsLUT[PrevKey]).xyz;
}

float EvaluateFindNearestPosition_{ParameterName}(float3 InPosition)
{
	if ({ParameterName}_MaxIndex < 0)
	{
		return 0.0f;
	}
	
	float MinDistance = length2({ParameterName}_SplinePositionsLUT[0].xyz - InPosition);
	float KeyToNearest = 0.0f;
	for (int i = 1; i <= {ParameterName}_MaxIndex; i++)
	{
		const float Distance = length2({ParameterName}_SplinePositionsLUT[i].xyz - InPosition);
		if (Distance < MinDistance)
		{
			MinDistance = Distance;
			KeyToNearest = i;
		}
	}
	return KeyToNearest > 0 ? float(KeyToNearest) / float({ParameterName}_MaxIndex) : 0.0f;
}

void SampleSplinePositionByUnitDistance_{ParameterName}(float U, out float3 Position)
{
	Position = EvaluatePosition_{ParameterName}(U * {ParameterName}_SplineLength);
}

void SampleSplinePositionByUnitDistanceWS_{ParameterName}(float U, out float3 Position)
{
	Position = EvaluatePosition_{ParameterName}(U * {ParameterName}_SplineLength);
	Position = mul(float4(Position, 1.0f), {ParameterName}_SplineTransform).xyz;
}

void SampleSplineRotationByUnitDistance_{ParameterName}(float U, out float4 Rotation)
{
	Rotation = GetRotationAtDistanceAlongSpline_{ParameterName}(U * {ParameterName}_SplineLength);
}		

void SampleSplineRotationByUnitDistanceWS_{ParameterName}(float U, out float4 Rotation)
{
	Rotation = GetRotationAtDistanceAlongSpline_{ParameterName}(U * {ParameterName}_SplineLength);
	Rotation = NiagaraQuatMul({ParameterName}_SplineTransformRotation, Rotation);
}

void SampleSplineDirectionByUnitDistance_{ParameterName}(float U, out float3 Direction)
{
	Direction = normalize(EvaluateDerivativePosition_{ParameterName}(U * {ParameterName}_SplineLength));
}

void SampleSplineDirectionByUnitDistanceWS_{ParameterName}(float U, out float3 Direction)
{
	Direction = normalize(EvaluateDerivativePosition_{ParameterName}(U * {ParameterName}_SplineLength));
	Direction = mul(float4(Direction, 1.0f), {ParameterName}_SplineTransformRotationMat).xyz;
}

void SampleSplineUpVectorByUnitDistance_{ParameterName}(float U, out float3 UpVector)
{
	float4 Rotation = GetRotationAtDistanceAlongSpline_{ParameterName}(U * {ParameterName}_SplineLength);
	UpVector = NiagaraQuatRotateVector(Rotation, float3(0,0,1));
}

void SampleSplineUpVectorByUnitDistanceWS_{ParameterName}(float U, out float3 UpVector)
{
	float4 Rotation = GetRotationAtDistanceAlongSpline_{ParameterName}(U * {ParameterName}_SplineLength);
	UpVector = NiagaraQuatRotateVector(Rotation, float3(0,0,1));
	UpVector = mul(float4(UpVector, 0.0f), {ParameterName}_SplineTransformRotationMat).xyz;
}

void SampleSplineRightVectorByUnitDistance_{ParameterName}(float U, out float3 RightVector)
{
	float4 Rotation = GetRotationAtDistanceAlongSpline_{ParameterName}(U * {ParameterName}_SplineLength);
	RightVector = NiagaraQuatRotateVector(Rotation, float3(0,1,0));
}

void SampleSplineRightVectorByUnitDistanceWS_{ParameterName}(float U, out float3 RightVector)
{
	float4 Rotation = GetRotationAtDistanceAlongSpline_{ParameterName}(U * {ParameterName}_SplineLength);
	RightVector = NiagaraQuatRotateVector(Rotation, float3(0,1,0));
	RightVector = mul(float4(RightVector, 0.0f), {ParameterName}_SplineTransformRotationMat).xyz;
}

void SampleSplineTangentByUnitDistance_{ParameterName}(float U, out float3 Tangent)
{
	Tangent = EvaluateDerivativePosition_{ParameterName}(U * {ParameterName}_SplineLength);
}

void SampleSplineTangentByUnitDistanceWS_{ParameterName}(float U, out float3 Tangent)
{
	Tangent = EvaluateDerivativePosition_{ParameterName}(U * {ParameterName}_SplineLength);
	Tangent = mul(float4(Tangent, 0.0f), {ParameterName}_SplineTransformRotationMat).xyz;
}

void GetSplineLocalToWorld_{ParameterName}(out float4x4 Transform)
{
	Transform = {ParameterName}_SplineTransform;
}

void GetSplineLocalToWorldInverseTransposed_{ParameterName}(out float4x4 Transform)
{
	Transform = {ParameterName}_SplineTransformInverseTranspose;
}

void FindClosestUnitDistanceFromPositionWS_{ParameterName}(float3 PositionWS, out float U)
{
	float3 Position = mul(float4(PositionWS, 1.0f), {ParameterName}_SplineTransformInverse).xyz;
	U = EvaluateFindNearestPosition_{ParameterName}(Position);
}

void GetSplineLength_{ParameterName}(out float Length)
{
	Length = {ParameterName}_SplineLength;
}
