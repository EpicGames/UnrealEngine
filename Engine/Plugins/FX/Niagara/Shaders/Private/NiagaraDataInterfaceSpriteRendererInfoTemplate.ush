// Copyright Epic Games, Inc. All Rights Reserved.

uint	{ParameterName}_bIsValid;
int		{ParameterName}_SourceMode;
int		{ParameterName}_Alignment;
int		{ParameterName}_FacingMode;
uint	{ParameterName}_bSubImageBlend;
float2	{ParameterName}_SubImageSize;

void IsValid_{ParameterName}(out bool bIsValid)
{
	bIsValid = {ParameterName}_bIsValid != 0;
}

void GetSourceMode_{ParameterName}(out int SourceMode)
{
	SourceMode = {ParameterName}_SourceMode;
}

void GetAlignment_{ParameterName}(out int Alignment)
{
	Alignment = {ParameterName}_Alignment;
}

void GetFacingMode_{ParameterName}(out int FacingMode)
{
	FacingMode = {ParameterName}_FacingMode;
}

void GetSubUVDetails_{ParameterName}(out bool bBlendEnabled, out float2 SubImageSize)
{
	bBlendEnabled = {ParameterName}_bSubImageBlend != 0;
	SubImageSize = {ParameterName}_SubImageSize;
}

