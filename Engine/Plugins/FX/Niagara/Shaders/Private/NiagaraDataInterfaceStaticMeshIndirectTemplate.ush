// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/FX/Niagara/Private/NiagaraQuaternionUtils.ush"
#include "/Plugin/FX/Niagara/Private/NiagaraDistanceFieldQueries.ush"
#include "/Plugin/FX/Niagara/Private/NiagaraSharedDataGPU_StaticMesh.ush"

//////////////////////////////////////////////////////////////////////////
// Distance Field Functions.

void QueryDistanceField_{ParameterName}(bool bExecute, float3 WorldPosition, NiagaraSharedDataID_StaticMesh StaticMeshID, bool bUseMaxDistance, float MaxDistance, out bool OutIsValid, out float OutDistance)
{
	int DFIndex = ResolveNiagaraManagedReferenceID_StaticMesh(StaticMeshID);
	if ( !bUseMaxDistance )
	{
		MaxDistance = 1e38;
	}

	//const int DFIndex = {ParameterName}_InstanceDistanceFieldIndex;
	OutIsValid = bExecute && DFIndex != -1 && DFIndex < (int)NumSceneObjects;
	OutDistance = MaxDistance;
	if ( OutIsValid )
	{
		// Needs to be a smidge over 0.0 for DistanceToNearestSurfaceForObject to query the SDF
		MaxDistance = max(0.001f, MaxDistance);
		const FDFVector3 LWCWorldPosition = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition));
		OutDistance = DistanceToNearestSurfaceForObject(DFIndex, LWCWorldPosition, MaxDistance);
	}
	else
	{
		OutIsValid = false;
		OutDistance = MaxDistance;
	}
}

void GetClosestPointOnMeshDistanceField_{ParameterName}(bool bExecute, float3 WorldPosition, NiagaraSharedDataID_StaticMesh StaticMeshID, bool bUseMaxDistance, float MaxDistance, out bool OutIsValid, out float OutClosestDistance, out float3 OutClosestPosition, out float3 OutClosestNormal, out float OutMaxEncodedDistance, out bool OutNormalIsValid)
{
	if ( !bUseMaxDistance )
	{
		MaxDistance = MAX_DISTANCE;
	}

	OutIsValid = false;
	OutClosestDistance = MaxDistance;	
	OutClosestNormal = float3(0,0,1);
	OutClosestPosition = WorldPosition;
	OutMaxEncodedDistance = MaxDistance;
	OutNormalIsValid = false;

	// Needs to be a smidge over 0.0 for DistanceToNearestSurfaceForObject to query the SDF
	MaxDistance = max(0.001f, MaxDistance);
	int DFIndex = ResolveNiagaraManagedReferenceID_StaticMesh(StaticMeshID);
	OutIsValid = bExecute && DFIndex != -1 && DFIndex < (int)NumSceneObjects;
	OutClosestDistance = MaxDistance;
	if ( OutIsValid )
	{
		const FDFVector3 LWCWorldPosition = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition));
		OutClosestDistance = GetDistanceToMeshDistanceField(DFIndex, LWCWorldPosition, MaxDistance);
		ComputeClosestPointMeshDistanceField_LWCTiled(DFIndex, WorldPosition, OutClosestDistance, OutClosestPosition, OutClosestNormal, OutMaxEncodedDistance, OutNormalIsValid);				
	}
	else
	{
		OutIsValid = false;
		OutClosestDistance = MaxDistance;
		OutClosestPosition = WorldPosition;
		OutClosestNormal = float3(0,0,1);
		OutMaxEncodedDistance = MaxDistance;
		OutNormalIsValid = false;
	}
}