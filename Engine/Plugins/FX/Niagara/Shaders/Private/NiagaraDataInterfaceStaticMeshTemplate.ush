// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/FX/Niagara/Private/NiagaraQuaternionUtils.ush"

int3			{ParameterName}_NumTriangles;		// x = Num Sections, y = Num Filtered, z = Num Unfiltered
int				{ParameterName}_NumVertices;
int				{ParameterName}_NumUVs;
uint			{ParameterName}_HasColors;
Buffer<uint>	{ParameterName}_IndexBuffer;
Buffer<float>	{ParameterName}_PositionBuffer;
Buffer<float4>	{ParameterName}_TangentBuffer;
Buffer<float2>	{ParameterName}_UVBuffer;
Buffer<float4>	{ParameterName}_ColorBuffer;

uint			{ParameterName}_HasUniformSampling;
Buffer<uint2>	{ParameterName}_UniformSamplingTriangles;

int3			{ParameterName}_SectionCounts;		// x = Num Sections, y = Num Filtered, z = Num Unfiltered
Buffer<uint4>	{ParameterName}_SectionInfos;		// FirstTriangle, NumTriangles, Prob, Alias
Buffer<uint>	{ParameterName}_FilteredAndUnfilteredSections;

int3			{ParameterName}_SocketCounts;		// x = Num Sockets, y = Num Filtered, z = Num Unfiltered
Buffer<float4>	{ParameterName}_SocketTransforms;
Buffer<uint>	{ParameterName}_FilteredAndUnfilteredSockets;

float			{ParameterName}_InvDeltaSeconds;
float4x4		{ParameterName}_InstanceTransform;
float4x4		{ParameterName}_InstanceTransformInverseTransposed;
float4			{ParameterName}_InstanceRotation;
float4x4		{ParameterName}_InstancePreviousTransform;
float4x4		{ParameterName}_InstancePreviousTransformInverseTransposed;
float4			{ParameterName}_InstancePreviousRotation;
float3			{ParameterName}_InstanceWorldVelocity;

float3			{ParameterName}_PreSkinnedLocalBoundsCenter;
float3			{ParameterName}_PreSkinnedLocalBoundsExtents;
float3			{ParameterName}_MeshBoundsWSCenter;
float3			{ParameterName}_MeshBoundsWSExtents;
float3			{ParameterName}_OwnerToMeshVector;

#if DISTATICMESH_ALLOWDISTANCEFIELD
int				{ParameterName}_InstanceDistanceFieldIndex;
#endif

Buffer<int>		{ParameterName}_UvMappingBuffer;
uint			{ParameterName}_UvMappingBufferLength;
uint			{ParameterName}_UvMappingSet;

//////////////////////////////////////////////////////////////////////////
// Fwd declarations
void RandomSection_{ParameterName}(NiagaraRandInfo RandInfo, out int Section);
void RandomFilteredSection_{ParameterName}(NiagaraRandInfo RandInfo, out int Section);
void RandomUnfilteredSection_{ParameterName}(NiagaraRandInfo RandInfo, out int Section);
void RandomSectionTriangle_{ParameterName}(NiagaraRandInfo RandInfo, int Section, out int Tri, out float3 BaryCoord);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Misc functions
int UniformTriangle_{ParameterName}(NiagaraRandInfo RandInfo, int Tri, int SectionOffset)
{
	uint2 ProbAlias = {ParameterName}_UniformSamplingTriangles[Tri];
	float Prob = asfloat(ProbAlias.x);
	int Alias = ProbAlias.y;
	return NiagaraRandomFloat(RandInfo) < Prob ? Tri : Alias + SectionOffset;
}

void GetPreSkinnedLocalBounds_{ParameterName}(out float3 OutCenter, out float3 OutExtentsMin, out float3 OutExtentsMax, out float3 OutExtents, out float3 OutHalfExtents)
{
	OutCenter		= {ParameterName}_PreSkinnedLocalBoundsCenter;
	OutHalfExtents	= {ParameterName}_PreSkinnedLocalBoundsExtents;
	OutExtents		= OutHalfExtents * 2.0f;
	OutExtentsMin	= OutCenter - OutHalfExtents;
	OutExtentsMax	= OutCenter + OutHalfExtents;
}

//////////////////////////////////////////////////////////////////////////
// Vertex Sampling
void IsValidVertex_{ParameterName}(out bool IsValid)
{
	IsValid = {ParameterName}_NumVertices > 0;
}

void RandomVertex_{ParameterName}(NiagaraRandInfo RandInfo, out int Vertex)
{
	Vertex = NiagaraRandomInt(RandInfo, {ParameterName}_NumVertices);
}

void GetVertexCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_NumVertices;
}

void GetVertex_{ParameterName}(int Vertex, out float3 Position, out float3 Velocity, out float3 Normal, out float3 Bitangent, out float3 Tangent)
{
	[branch]
	if ( {ParameterName}_NumVertices > 0 )
	{
		Vertex = clamp(Vertex, 0, {ParameterName}_NumVertices - 1);

		Position.x = {ParameterName}_PositionBuffer[Vertex * 3 + 0];
		Position.y = {ParameterName}_PositionBuffer[Vertex * 3 + 1];
		Position.z = {ParameterName}_PositionBuffer[Vertex * 3 + 2];

		float4 TangentX = TangentBias({ParameterName}_TangentBuffer[Vertex * 2 + 0]);
		float4 TangentZ = TangentBias({ParameterName}_TangentBuffer[Vertex * 2 + 1]);
		float3 TangentY = cross(TangentZ.xyz, TangentX.xyz) * TangentZ.w;

		Velocity	= float3(0, 0, 0);

		// Sometimes we have bogus tangent data in the buffer, i.e. X==Z so ensure we handle this and don't end up with NaNs in the buffer
		if ( length2(TangentY) > 0.001f )
		{
			Normal		= TangentZ.xyz;
			Bitangent	= TangentY;
			Tangent		= TangentX.xyz;
		}
		else
		{
			Normal		= float3(0, 0, 1);
			Bitangent	= float3(0, 1, 0);
			Tangent		= float3(1, 0, 0);
		}
	}
	else
	{
		Position	= float3(0, 0, 0);
		Velocity	= float3(0, 0, 0);
		Normal		= float3(0, 0, 1);
		Bitangent	= float3(0, 1, 0);
		Tangent		= float3(1, 0, 0);
	}
}

void GetVertexWS_{ParameterName}(int Vertex, out float3 Position, out float3 Velocity, out float3 Normal, out float3 Bitangent, out float3 Tangent)
{
	float3 LocalPosition;
	GetVertex_{ParameterName}(Vertex, LocalPosition, Velocity, Normal, Bitangent, Tangent);
	
	Position				= mul(float4(LocalPosition, 1), {ParameterName}_InstanceTransform).xyz;
	float3 PreviousPosition	= mul(float4(LocalPosition, 1), {ParameterName}_InstancePreviousTransform).xyz;
	Velocity				= (Position - PreviousPosition) * {ParameterName}_InvDeltaSeconds;
	Normal					= normalize(mul(float4(Normal, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz);
	Bitangent				= normalize(mul(float4(Bitangent, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz);
	Tangent					= normalize(mul(float4(Tangent, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz);
}

void GetVertexWSInterpolated_{ParameterName}(int Vertex, float Interp, out float3 Position, out float3 Velocity, out float3 Normal, out float3 Bitangent, out float3 Tangent)
{
	float3 LocalPosition, LocalNormal, LocalBitangent, LocalTangent;
	GetVertex_{ParameterName}(Vertex, LocalPosition, Velocity, Normal, Bitangent, Tangent);

	float3 PrevPosition		= mul(float4(LocalPosition, 1), {ParameterName}_InstancePreviousTransform).xyz;
	float3 PrevNormal		= mul(float4(LocalNormal, 0), {ParameterName}_InstancePreviousTransformInverseTransposed).xyz;
	float3 PrevBitangent	= mul(float4(LocalBitangent, 0), {ParameterName}_InstancePreviousTransformInverseTransposed).xyz;
	float3 PrevTangent		= mul(float4(LocalTangent, 0), {ParameterName}_InstancePreviousTransformInverseTransposed).xyz;

	float3 CurrPosition		= mul(float4(LocalPosition, 1), {ParameterName}_InstanceTransform).xyz;
	float3 CurrNormal		= mul(float4(LocalNormal, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz;
	float3 CurrBitangent	= mul(float4(LocalBitangent, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz;
	float3 CurrTangent		= mul(float4(LocalTangent, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz;

	Position				= lerp(PrevPosition, CurrPosition, Interp);
	Velocity				= (CurrPosition - PrevPosition) * {ParameterName}_InvDeltaSeconds;
	Normal					= normalize(lerp(PrevNormal, CurrNormal, Interp));
	Bitangent				= normalize(lerp(PrevBitangent, CurrBitangent, Interp));
	Tangent					= normalize(lerp(PrevTangent, CurrTangent, Interp));
}

void GetVertexColor_{ParameterName}(int Vertex, out float4 Color)
{
	Vertex = clamp(Vertex, 0, {ParameterName}_NumVertices - 1);
	Color = {ParameterName}_HasColors ? {ParameterName}_ColorBuffer[Vertex] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE : float4(1, 1, 1, 1);
}

void GetVertexUV_{ParameterName}(int Vertex, int UVSet, out float2 UV)
{
	Vertex = clamp(Vertex, 0, {ParameterName}_NumVertices - 1);
	UVSet = clamp(UVSet, 0, {ParameterName}_NumUVs - 1);
	UV = UVSet >= 0 && UVSet < {ParameterName}_NumUVs ? {ParameterName}_UVBuffer[Vertex * {ParameterName}_NumUVs + UVSet] : float2(0, 0);
}

//////////////////////////////////////////////////////////////////////////
// VM Triangle Sampling
void IsValidTriangle_{ParameterName}(int Tri, out bool IsValid)
{
	IsValid = Tri >= 0 && Tri < {ParameterName}_NumTriangles.x;
}

void RandomTriangle_{ParameterName}(NiagaraRandInfo RandInfo, out int Tri, out float3 BaryCoord)
{
	int Section;
	RandomSection_{ParameterName}(RandInfo, Section);
	RandomSectionTriangle_{ParameterName}(RandInfo, Section, Tri, BaryCoord);
}

void GetTriangleCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_NumTriangles.x;
}

void IsValidFilteredTriangle_{ParameterName}(int Tri, out bool IsValid)
{
	IsValid = Tri >= 0 && Tri < {ParameterName}_NumTriangles.y;
}

void RandomFilteredTriangle_{ParameterName}(NiagaraRandInfo RandInfo, out int Tri, out float3 BaryCoord)
{
	if ( {ParameterName}_NumTriangles.y > 0 )
	{
		int Section;
		RandomFilteredSection_{ParameterName}(RandInfo, Section);
		RandomSectionTriangle_{ParameterName}(RandInfo, Section, Tri, BaryCoord);
	}
	else
	{
		Tri = 0;
		BaryCoord = NiagaraRandomBaryCoord(RandInfo);
	}
}

void GetFilteredTriangleCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_NumTriangles.y;
}

void GetFilteredTriangleAt_{ParameterName}(int TriangleIndex, out int Tri)
{
	Tri = 0;

	int SectionInfoOffset = {ParameterName}_SectionCounts.x;
	int SectionFirstTriangle = 0;
	for ( int i=0; i < {ParameterName}_SectionCounts.y; ++i )
	{
		int SectionLastTriangle = SectionFirstTriangle + {ParameterName}_SectionInfos[SectionInfoOffset + i].y;
		if ( TriangleIndex >= SectionFirstTriangle && TriangleIndex < SectionLastTriangle )
		{
			Tri = (TriangleIndex - SectionFirstTriangle) + {ParameterName}_SectionInfos[SectionInfoOffset + i].x;
			return;
		}
		SectionFirstTriangle = SectionLastTriangle;
	}
}

void IsValidUnfilteredTriangle_{ParameterName}(int Tri, out bool IsValid)
{
	IsValid = Tri >= 0 && Tri < {ParameterName}_NumTriangles.z;
}

void RandomUnfilteredTriangle_{ParameterName}(NiagaraRandInfo RandInfo, out int Tri, out float3 BaryCoord)
{
	if ( {ParameterName}_NumTriangles.z > 0 )
	{
		int Section;
		RandomUnfilteredSection_{ParameterName}(RandInfo, Section);
		RandomSectionTriangle_{ParameterName}(RandInfo, Section, Tri, BaryCoord);
	}
	else
	{
		Tri = 0;
		BaryCoord = NiagaraRandomBaryCoord(RandInfo);
	}
}

void GetUnfilteredTriangleCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_NumTriangles.z;
}

void GetUnfilteredTriangleAt_{ParameterName}(int TriangleIndex, out int Tri)
{
	Tri = 0;

	int SectionInfoOffset = {ParameterName}_SectionCounts.x + {ParameterName}_SectionCounts.y;
	int SectionFirstTriangle = 0;
	for ( int i=0; i < {ParameterName}_SectionCounts.z; ++i )
	{
		int SectionLastTriangle = SectionFirstTriangle + {ParameterName}_SectionInfos[SectionInfoOffset + i].y;
		if ( TriangleIndex >= SectionFirstTriangle && TriangleIndex < SectionLastTriangle )
		{
			Tri = (TriangleIndex - SectionFirstTriangle) + {ParameterName}_SectionInfos[SectionInfoOffset + i].x;
			return;
		}
		SectionFirstTriangle = SectionLastTriangle;
	}
}

void GetTriangleIndices_{ParameterName}(int Tri, out int Index0, out int Index1, out int Index2)
{
	[branch]
	if ( {ParameterName}_NumTriangles.x > 0 )
	{
		Tri = clamp(Tri, 0, {ParameterName}_NumTriangles.x - 1);
		Index0 = {ParameterName}_IndexBuffer[Tri * 3 + 0];
		Index1 = {ParameterName}_IndexBuffer[Tri * 3 + 1];
		Index2 = {ParameterName}_IndexBuffer[Tri * 3 + 2];
	}
	else
	{
		Index0 = 0;
		Index1 = 0;
		Index2 = 0;
	}
}

void GetTriangle_{ParameterName}(int Tri, float3 BaryCoord, out float3 Position, out float3 Velocity, out float3 Normal, out float3 Bitangent, out float3 Tangent)
{
	int Index[3];
	GetTriangleIndices_{ParameterName}(Tri, Index[0], Index[1], Index[2]);

	float3 Positions[3], Velocities[3], Normals[3], Bitangents[3], Tangents[3];
	GetVertex_{ParameterName}(Index[0], Positions[0], Velocities[0], Normals[0], Bitangents[0], Tangents[0]);
	GetVertex_{ParameterName}(Index[1], Positions[1], Velocities[1], Normals[1], Bitangents[1], Tangents[1]);
	GetVertex_{ParameterName}(Index[2], Positions[2], Velocities[2], Normals[2], Bitangents[2], Tangents[2]);

	Position	= Positions[0] * BaryCoord.x + Positions[1] * BaryCoord.y + Positions[2] * BaryCoord.z;
	Velocity	= float3(0, 0, 0);
	Normal		= Normals[0] * BaryCoord.x + Normals[1] * BaryCoord.y + Normals[2] * BaryCoord.z;
	Bitangent	= Bitangents[0] * BaryCoord.x + Bitangents[1] * BaryCoord.y + Bitangents[2] * BaryCoord.z;
	Tangent		= Tangents[0] * BaryCoord.x + Tangents[1] * BaryCoord.y + Tangents[2] * BaryCoord.z;
}

void GetTriangleWS_{ParameterName}(int Tri, float3 BaryCoord, out float3 Position, out float3 Velocity, out float3 Normal, out float3 Bitangent, out float3 Tangent)
{
	float3 LocalPosition;
	GetTriangle_{ParameterName}(Tri, BaryCoord, LocalPosition, Velocity, Normal, Bitangent, Tangent);

	Position				= mul(float4(LocalPosition, 1), {ParameterName}_InstanceTransform).xyz;
	float3 PreviousPosition	= mul(float4(LocalPosition, 1), {ParameterName}_InstancePreviousTransform).xyz;
	Velocity				= (Position - PreviousPosition) * {ParameterName}_InvDeltaSeconds;
	Normal					= normalize(mul(float4(Normal, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz);
	Bitangent				= normalize(mul(float4(Bitangent, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz);
	Tangent					= normalize(mul(float4(Tangent, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz);
}

void GetTriangleWSInterpolated_{ParameterName}(int Tri, float3 BaryCoord, float Interp, out float3 Position, out float3 Velocity, out float3 Normal, out float3 Bitangent, out float3 Tangent)
{
	float3 LocalPosition, LocalNormal, LocalBitangent, LocalTangent;
	GetTriangle_{ParameterName}(Tri, BaryCoord, LocalPosition, Velocity, LocalNormal, LocalBitangent, LocalTangent);

	float3 PrevPosition		= mul(float4(LocalPosition, 1), {ParameterName}_InstancePreviousTransform).xyz;
	float3 PrevNormal		= mul(float4(LocalNormal, 0), {ParameterName}_InstancePreviousTransformInverseTransposed).xyz;
	float3 PrevBitangent	= mul(float4(LocalBitangent, 0), {ParameterName}_InstancePreviousTransformInverseTransposed).xyz;
	float3 PrevTangent		= mul(float4(LocalTangent, 0), {ParameterName}_InstancePreviousTransformInverseTransposed).xyz;

	float3 CurrPosition		= mul(float4(LocalPosition, 1), {ParameterName}_InstanceTransform).xyz;
	float3 CurrNormal		= mul(float4(LocalNormal, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz;
	float3 CurrBitangent	= mul(float4(LocalBitangent, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz;
	float3 CurrTangent		= mul(float4(LocalTangent, 0), {ParameterName}_InstanceTransformInverseTransposed).xyz;

	Position				= lerp(PrevPosition, CurrPosition, Interp);
	Velocity				= (CurrPosition - PrevPosition) * {ParameterName}_InvDeltaSeconds;
	Normal					= normalize(lerp(PrevNormal, CurrNormal, Interp));
	Bitangent				= normalize(lerp(PrevBitangent, CurrBitangent, Interp));
	Tangent					= normalize(lerp(PrevTangent, CurrTangent, Interp));
}

void GetTriangleColor_{ParameterName}(int Tri, float3 BaryCoord, out float4 Color)
{
	int Index[3];
	GetTriangleIndices_{ParameterName}(Tri, Index[0], Index[1], Index[2]);

	float4 Colors[3];
	GetVertexColor_{ParameterName}(Index[0], Colors[0]);
	GetVertexColor_{ParameterName}(Index[1], Colors[1]);
	GetVertexColor_{ParameterName}(Index[2], Colors[2]);

	Color = Colors[0] * BaryCoord.x + Colors[1] * BaryCoord.y + Colors[2] * BaryCoord.z;
}

void GetTriangleUV_{ParameterName}(int Tri, float3 BaryCoord, int UVSet, out float2 UV)
{
	int Index[3];
	GetTriangleIndices_{ParameterName}(Tri, Index[0], Index[1], Index[2]);

	float2 UVs[3];
	GetVertexUV_{ParameterName}(Index[0], UVSet, UVs[0]);
	GetVertexUV_{ParameterName}(Index[1], UVSet, UVs[1]);
	GetVertexUV_{ParameterName}(Index[2], UVSet, UVs[2]);

	UV = UVs[0] * BaryCoord.x + UVs[1] * BaryCoord.y + UVs[2] * BaryCoord.z;
}

//////////////////////////////////////////////////////////////////////////
// Socket Functions
void GetSocketCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_SocketCounts.x;
}

void GetFilteredSocketCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_SocketCounts.y;
}

void GetUnfilteredSocketCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_SocketCounts.z;
}

void GetFilteredSocket_{ParameterName}(int Index, out int SocketIndex)
{
	SocketIndex = -1;
	if ( {ParameterName}_SocketCounts.y > 0 )
	{
		Index = clamp(Index, 0, {ParameterName}_SocketCounts.y - 1);
		SocketIndex = {ParameterName}_FilteredAndUnfilteredSockets[Index];
	}
}

void GetUnfilteredSocket_{ParameterName}(int Index, out int SocketIndex)
{
	SocketIndex = -1;
	if ( {ParameterName}_SocketCounts.z > 0 )
	{
		Index = {ParameterName}_SocketCounts.y + clamp(Index, 0, {ParameterName}_SocketCounts.z - 1);
		SocketIndex = {ParameterName}_FilteredAndUnfilteredSockets[Index];
	}
}

void RandomSocket_{ParameterName}(NiagaraRandInfo RandInfo, out int Index)
{
	Index = -1;
	if ( {ParameterName}_SocketCounts.x > 0 )
	{
		Index = NiagaraRandomInt(RandInfo, {ParameterName}_SocketCounts.x);
	}
}

void RandomFilteredSocket_{ParameterName}(NiagaraRandInfo RandInfo, out int Index)
{
	Index = -1;
	if ( {ParameterName}_SocketCounts.y > 0 )
	{
		Index = NiagaraRandomInt(RandInfo, {ParameterName}_SocketCounts.y);
		Index = {ParameterName}_FilteredAndUnfilteredSockets[Index];
	}
}

void RandomUnfilteredSocket_{ParameterName}(NiagaraRandInfo RandInfo, out int Index)
{
	Index = -1;
	if ( {ParameterName}_SocketCounts.z > 0 )
	{
		Index = {ParameterName}_SocketCounts.y + NiagaraRandomInt(RandInfo, {ParameterName}_SocketCounts.z);
		Index = {ParameterName}_FilteredAndUnfilteredSockets[Index];
	}
}

void GetSocketTransform_{ParameterName}(int Socket, out float3 Position, out float4 Rotation, out float3 Scale, out float3 Velocity)
{
	[branch]
	if ( {ParameterName}_SocketCounts.x > 0 )
	{
		Socket = clamp(Socket, 0, {ParameterName}_SocketCounts.x - 1);
		Position = {ParameterName}_SocketTransforms[Socket * 3 + 0].xyz;
		Rotation = {ParameterName}_SocketTransforms[Socket * 3 + 1];
		Scale    = {ParameterName}_SocketTransforms[Socket * 3 + 2].xyz;
		Velocity = float3(0, 0, 0);
	}
	else
	{
		Velocity = float3(0, 0, 0);
		Position = float3(0, 0, 0);
		Rotation = float4(0, 0, 0, 1);
		Scale = float3(1, 1, 1);
	}
}

void GetSocketTransformWS_{ParameterName}(int Socket, out float3 Position, out float4 Rotation, out float3 Scale, out float3 Velocity)
{
	float3 LocalPosition;
	GetSocketTransform_{ParameterName}(Socket, LocalPosition, Rotation, Scale, Velocity);

	Position				= mul(float4(LocalPosition, 1.0), {ParameterName}_InstanceTransform).xyz;
	float3 PreviousPosition	= mul(float4(LocalPosition, 1), {ParameterName}_InstancePreviousTransform).xyz;
	Rotation				= NiagaraGPU_QuatMul({ParameterName}_InstanceRotation, Rotation);
	Scale					= mul(float4(Scale, 0.0), {ParameterName}_InstanceTransform).xyz;
	Velocity				= (Position - PreviousPosition) * {ParameterName}_InvDeltaSeconds;
}

void GetSocketTransformWSInterpolated_{ParameterName}(int Socket, float Interp, out float3 Position, out float4 Rotation, out float3 Scale, out float3 Velocity)
{
	float3 LocalPosition, LocalScale, LocalVelocity;
	float4 LocalRotation;
	GetSocketTransform_{ParameterName}(Socket, LocalPosition, LocalRotation, LocalScale, LocalVelocity);

	float3 CurrPosition		= mul(float4(LocalPosition, 1.0), {ParameterName}_InstanceTransform).xyz;
	float3 PrevPosition		= mul(float4(LocalPosition, 1.0), {ParameterName}_InstancePreviousTransform).xyz;
	float4 CurrRotation		= NiagaraGPU_QuatMul({ParameterName}_InstanceRotation, LocalRotation);
	float4 PrevRotation		= NiagaraGPU_QuatMul({ParameterName}_InstancePreviousRotation, LocalRotation);
	float3 CurrScale		= mul(float4(LocalScale, 0.0), {ParameterName}_InstanceTransform).xyz;
	float3 PrevScale		= mul(float4(LocalScale, 0.0), {ParameterName}_InstancePreviousTransform).xyz;

	Position	= lerp(PrevPosition, CurrPosition, Interp);
	Rotation	= SlerpQuat(PrevRotation, CurrRotation, Interp);
	Scale		= lerp(PrevScale, CurrScale, Interp);
	Velocity	= (CurrPosition - PrevPosition) * {ParameterName}_InvDeltaSeconds;
}

void GetFilteredSocketTransform_{ParameterName}(int SocketIndex, out float3 Position, out float4 Rotation, out float3 Scale, out float3 Velocity)
{
	[branch]
	if ( {ParameterName}_SocketCounts.y > 0 )
	{
		SocketIndex = clamp(SocketIndex, 0, {ParameterName}_SocketCounts.y - 1);
		int Socket = {ParameterName}_FilteredAndUnfilteredSockets[SocketIndex];
		GetSocketTransform_{ParameterName}(Socket, Position, Rotation, Scale, Velocity);
		Velocity = float3(0, 0, 0);
	}
	else
	{
		Position = float3(0, 0, 0);
		Rotation = float4(0, 0, 0, 1);
		Scale = float3(1, 1, 1);
		Velocity = float3(0, 0, 0);
	}
}

void GetFilteredSocketTransformWS_{ParameterName}(int SocketIndex, out float3 Position, out float4 Rotation, out float3 Scale, out float3 Velocity)
{
	float3 LocalPosition;
	GetFilteredSocketTransform_{ParameterName}(SocketIndex, LocalPosition, Rotation, Scale, Velocity);

	Position				= mul(float4(LocalPosition, 1.0), {ParameterName}_InstanceTransform).xyz;
	float3 PreviousPosition	= mul(float4(LocalPosition, 1), {ParameterName}_InstancePreviousTransform).xyz;
	Rotation				= NiagaraGPU_QuatMul({ParameterName}_InstanceRotation, Rotation);
	Scale					= mul(float4(Scale, 0.0), {ParameterName}_InstanceTransform).xyz;
	Velocity				= (Position - PreviousPosition) * {ParameterName}_InvDeltaSeconds;
}

void GetFilteredSocketTransformWSInterpolated_{ParameterName}(int Index, float Interp, out float3 Position, out float4 Rotation, out float3 Scale, out float3 Velocity)
{
	int SocketIndex;
	GetFilteredSocket_{ParameterName}(Index, SocketIndex);
	GetSocketTransformWSInterpolated_{ParameterName}(SocketIndex, Interp, Position, Rotation, Scale, Velocity);
}

void GetUnfilteredSocketTransform_{ParameterName}(int SocketIndex, out float3 Position, out float4 Rotation, out float3 Scale, out float3 Velocity)
{
	[branch]
	if ( {ParameterName}_SocketCounts.z > 0 )
	{
		SocketIndex = {ParameterName}_SocketCounts.y + clamp(SocketIndex, 0, {ParameterName}_SocketCounts.z - 1);
		int Socket = {ParameterName}_FilteredAndUnfilteredSockets[SocketIndex];
		GetSocketTransform_{ParameterName}(Socket, Position, Rotation, Scale, Velocity);
		Velocity = float3(0, 0, 0);
	}
	else
	{
		Position = float3(0, 0, 0);
		Rotation = float4(0, 0, 0, 1);
		Scale = float3(1, 1, 1);
		Velocity = float3(0, 0, 0);
	}
}

void GetUnfilteredSocketTransformWS_{ParameterName}(int SocketIndex, out float3 Position, out float4 Rotation, out float3 Scale, out float3 Velocity)
{
	float3 LocalPosition;
	GetUnfilteredSocketTransform_{ParameterName}(SocketIndex, LocalPosition, Rotation, Scale, Velocity);

	Position				= mul(float4(LocalPosition, 1.0), {ParameterName}_InstanceTransform).xyz;
	float3 PreviousPosition	= mul(float4(LocalPosition, 1), {ParameterName}_InstancePreviousTransform).xyz;
	Rotation				= NiagaraGPU_QuatMul({ParameterName}_InstanceRotation, Rotation);
	Scale					= mul(float4(Scale, 0.0), {ParameterName}_InstanceTransform).xyz;
	Velocity				= (Position - PreviousPosition) * {ParameterName}_InvDeltaSeconds;
}

void GetUnfilteredSocketTransformWSInterpolated_{ParameterName}(int Index, float Interp, out float3 Position, out float4 Rotation, out float3 Scale, out float3 Velocity)
{
	int SocketIndex;
	GetUnfilteredSocket_{ParameterName}(Index, SocketIndex);
	GetSocketTransformWSInterpolated_{ParameterName}(SocketIndex, Interp, Position, Rotation, Scale, Velocity);
}

//////////////////////////////////////////////////////////////////////////
// Section functions
void IsValidSection_{ParameterName}(int Section, out bool IsValid)
{
	IsValid = Section >= 0 && Section < {ParameterName}_SectionCounts.x;
}

void GetSectionTriangleCount_{ParameterName}(int Section, out int Count)
{
	Count = Section >= 0 && Section < {ParameterName}_SectionCounts.x ? {ParameterName}_SectionInfos[Section].y : 0;
}

void RandomSectionTriangle_{ParameterName}(NiagaraRandInfo RandInfo, int Section, out int Tri, out float3 BaryCoord)
{
	int SectionTriangleCount;
	GetSectionTriangleCount_{ParameterName}(Section, SectionTriangleCount);

	Tri = 0;
	if (SectionTriangleCount > 0)
	{
		int SectionTriangleOffset = {ParameterName}_SectionInfos[Section].x;
		Tri = NiagaraRandomInt(RandInfo, SectionTriangleCount) + SectionTriangleOffset;

		[branch]
		if ( {ParameterName}_HasUniformSampling )
		{
			Tri = UniformTriangle_{ParameterName}(RandInfo, Tri, SectionTriangleOffset);
		}
	}
	BaryCoord = NiagaraRandomBaryCoord(RandInfo);
}

void GetSectionTriangleAt_{ParameterName}(int Section, int SectionTriangleIndex, out int Tri)
{
	int SectionTriangleCount;
	GetSectionTriangleCount_{ParameterName}(Section, SectionTriangleCount);

	if ( SectionTriangleCount > 0 )
	{
		int SectionTriangleMax = {ParameterName}_SectionInfos[Section].y - 1;
		int SectionTriangleOffset = {ParameterName}_SectionInfos[Section].x;
		Tri = clamp(SectionTriangleIndex, 0, SectionTriangleMax) + SectionTriangleOffset;
	}
	else
	{
		Tri = 0;
	}
}

void GetFilteredSectionAt_{ParameterName}(int SectionIndex, out int Section)
{
	int FilteredSectionCount = {ParameterName}_SectionCounts.y;
	if ( FilteredSectionCount > 0 )
	{
		SectionIndex = clamp(SectionIndex, 0, FilteredSectionCount - 1);
		Section = {ParameterName}_FilteredAndUnfilteredSections[SectionIndex];
	}
	else
	{
		Section = 0;
	}
}

void GetUnfilteredSectionAt_{ParameterName}(int SectionIndex, out int Section)
{
	int FilteredSectionCount = {ParameterName}_SectionCounts.y;
	int UnfilteredSectionCount = {ParameterName}_SectionCounts.z;
	if ( UnfilteredSectionCount > 0 )
	{
		SectionIndex = clamp(SectionIndex, 0, UnfilteredSectionCount - 1);
		Section = {ParameterName}_FilteredAndUnfilteredSections[SectionIndex + FilteredSectionCount];
	}
	else
	{
		Section = 0;
	}
}

void GetSectionCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_SectionCounts.x;
}

void GetFilteredSectionCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_SectionCounts.y;
}

void GetUnfilteredSectionCount_{ParameterName}(out int Count)
{
	Count = {ParameterName}_SectionCounts.z;
}

void RandomSection_{ParameterName}(NiagaraRandInfo RandInfo, out int Section)
{
	int SectionCount = {ParameterName}_SectionCounts.x;
	if (SectionCount > 0)
	{
		Section = NiagaraRandomInt(RandInfo, SectionCount);
		float Prob = asfloat({ParameterName}_SectionInfos[Section].z);
		int Alias = {ParameterName}_SectionInfos[Section].w;
		Section = NiagaraRandomFloat(RandInfo) < Prob ? Section : Alias;
	}
	else
	{
		Section = 0;
	}
}

void RandomFilteredSection_{ParameterName}(NiagaraRandInfo RandInfo, out int Section)
{
	int SectionCount = {ParameterName}_SectionCounts.x;
	int FilteredSectionCount = {ParameterName}_SectionCounts.y;
	if (FilteredSectionCount > 0)
	{
		int SectionIndex = NiagaraRandomInt(RandInfo, FilteredSectionCount);
		int SectionInfoIndex = SectionIndex + SectionCount;
		float Prob = asfloat({ParameterName}_SectionInfos[SectionInfoIndex].z);
		int Alias = {ParameterName}_SectionInfos[SectionInfoIndex].w;
		SectionIndex = NiagaraRandomFloat(RandInfo) < Prob ? SectionIndex : Alias;
		Section = {ParameterName}_FilteredAndUnfilteredSections[SectionIndex];
	}
	else
	{
		Section = 0;
	}
}

void RandomUnfilteredSection_{ParameterName}(NiagaraRandInfo RandInfo, out int Section)
{
	int SectionCount = {ParameterName}_SectionCounts.x;
	int FilteredSectionCount = {ParameterName}_SectionCounts.y;
	int UnfilteredSectionCount = {ParameterName}_SectionCounts.z;
	if (UnfilteredSectionCount > 0)
	{
		int SectionIndex = NiagaraRandomInt(RandInfo, UnfilteredSectionCount);
		int SectionInfoIndex = SectionIndex + SectionCount + FilteredSectionCount;
		float Prob = asfloat({ParameterName}_SectionInfos[SectionInfoIndex].z);
		int Alias = {ParameterName}_SectionInfos[SectionInfoIndex].w;
		SectionIndex = NiagaraRandomFloat(RandInfo) < Prob ? SectionIndex : Alias;
		Section = {ParameterName}_FilteredAndUnfilteredSections[SectionIndex + FilteredSectionCount];
	}
	else
	{
		Section = 0;
	}
}

//////////////////////////////////////////////////////////////////////////
// VM Misc Functions
void IsValid_{ParameterName}(out bool IsValid)
{
	IsValid = {ParameterName}_NumVertices > 0;
}

void GetLocalToWorld_{ParameterName}(out float4x4 OutMatrix)
{
	OutMatrix = {ParameterName}_InstanceTransform;
}

void GetLocalToWorldInverseTransposed_{ParameterName}(out float4x4 OutMatrix)
{
	OutMatrix = {ParameterName}_InstanceTransformInverseTransposed;
}

void GetWorldVelocity_{ParameterName}(out float3 Velocity)
{
	Velocity = {ParameterName}_InstanceWorldVelocity;
}

void GetMeshBounds_{ParameterName}(out float3 Center, out float3 ExtentsMin, out float3 ExtentsMax, out float3 Extents, out float3 HalfExtents, out float3 OwnerToMeshVector)
{
	Center				= {ParameterName}_PreSkinnedLocalBoundsCenter;
	HalfExtents			= {ParameterName}_MeshBoundsWSExtents;
	Extents				= HalfExtents * 2.0f;
	ExtentsMin			= Center - HalfExtents;
	ExtentsMax			= Center + HalfExtents;
	OwnerToMeshVector	= {ParameterName}_OwnerToMeshVector;
}

void GetMeshBoundsWS_{ParameterName}(out float3 Center, out float3 ExtentsMin, out float3 ExtentsMax, out float3 Extents, out float3 HalfExtents, out float3 OwnerToMeshVector)
{
	Center				= {ParameterName}_MeshBoundsWSCenter;
	HalfExtents			= {ParameterName}_MeshBoundsWSExtents;
	Extents				= HalfExtents * 2.0f;
	ExtentsMin			= Center - HalfExtents;
	ExtentsMax			= Center + HalfExtents;
	OwnerToMeshVector	= {ParameterName}_OwnerToMeshVector;
}

//////////////////////////////////////////////////////////////////////////
// Experimental Distance Field Functions
void QueryDistanceField_{ParameterName}(bool bExecute, float3 WorldPosition, bool bUseMaxDistance, float MaxDistance, out bool OutIsValid, out float OutDistance)
{
	if ( !bUseMaxDistance )
	{
		MaxDistance = 1e38;
	}

#if DISTATICMESH_ALLOWDISTANCEFIELD
	const int DFIndex = {ParameterName}_InstanceDistanceFieldIndex;
	OutIsValid = bExecute && DFIndex >= 0 && DFIndex < NumSceneObjects;
	OutDistance = MaxDistance;
	if ( OutIsValid )
	{
		// Needs to be a smidge over 0.0 for DistanceToNearestSurfaceForObject to query the SDF
		MaxDistance = max(0.001f, MaxDistance);
		const FDFVector3 LWCWorldPosition = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition));
		OutDistance = DistanceToNearestSurfaceForObject(DFIndex, LWCWorldPosition, MaxDistance);
	}
#else
	OutIsValid = false;
	OutDistance = MaxDistance;
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Uv Mapping
void GetTriangleCoordAtUV_{ParameterName}(bool Enabled, float2 InUv, float InTolerance, out int OutTriangle, out float3 OutBaryCoord, out bool OutIsValid)
{
	UvMappingParameters MappingParam;
	MappingParam.NumUVs = {ParameterName}_NumUVs;
	MappingParam.IndexBuffer = {ParameterName}_IndexBuffer;
	MappingParam.UVBuffer = {ParameterName}_UVBuffer;
	MappingParam.UvMappingBuffer = {ParameterName}_UvMappingBuffer;
	MappingParam.UvMappingBufferLength = {ParameterName}_UvMappingBufferLength;
	MappingParam.UvMappingSet = {ParameterName}_UvMappingSet;

	UvMapping_GetTriangleCoordAtUV(MappingParam, Enabled, InUv, InTolerance, OutTriangle, OutBaryCoord, OutIsValid);
}

void GetTriangleCoordInAabb_{ParameterName}(bool Enabled, float2 InUvMin, float2 InUvMax, out int OutTriangle, out float3 OutBaryCoord, out bool OutIsValid)
{
	UvMappingParameters MappingParam;
	MappingParam.NumUVs = {ParameterName}_NumUVs;
	MappingParam.IndexBuffer = {ParameterName}_IndexBuffer;
	MappingParam.UVBuffer = {ParameterName}_UVBuffer;
	MappingParam.UvMappingBuffer = {ParameterName}_UvMappingBuffer;
	MappingParam.UvMappingBufferLength = {ParameterName}_UvMappingBufferLength;
	MappingParam.UvMappingSet = {ParameterName}_UvMappingSet;

	UvMapping_GetTriangleCoordInAabb(MappingParam, Enabled, InUvMin, InUvMax, OutTriangle, OutBaryCoord, OutIsValid);
}