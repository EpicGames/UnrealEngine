// Copyright Epic Games, Inc. All Rights Reserved.

float3			{ParameterName}_TextureSize;
Texture2DArray	{ParameterName}_Texture;
SamplerState	{ParameterName}_TextureSampler;

void LoadTexture_{ParameterName}(in int TexelX, in int TexelY, in int TexelZ, in int MipLevel, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.Load(int4(TexelX, TexelY, TexelZ, MipLevel));
}

void GatherRedTexture_{ParameterName}(in float3 UVW, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.Gather({ParameterName}_TextureSampler, UVW);
}

void SampleTexture_{ParameterName}(in float3 UVW, in float MipLevel, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UVW, MipLevel);
}

void TextureDimensions_{ParameterName}(out float3 OutValue)
{
	OutValue = {ParameterName}_TextureSize;
}
