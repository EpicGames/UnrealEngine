// Copyright Epic Games, Inc. All Rights Reserved.

int2			{ParameterName}_TextureSize;
int				{ParameterName}_MipLevels;
Texture2D		{ParameterName}_Texture;
SamplerState	{ParameterName}_TextureSampler;

void LoadTexture2D_{ParameterName}(in int TexelX, in int TexelY, in int MipLevel, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.Load(int3(TexelX, TexelY, MipLevel));
}

void GatherRedTexture2D_{ParameterName}(in float2 UV, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.Gather({ParameterName}_TextureSampler, UV);
}

void SampleTexture2D_{ParameterName}(in float2 UV, in float MipLevel, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UV, MipLevel);
}

void GetTextureDimensions_{ParameterName}(int MipLevel, out float2 OutValue)
{
	OutValue.x = float(max({ParameterName}_TextureSize.x >> MipLevel, 1));
	OutValue.y = float(max({ParameterName}_TextureSize.y >> MipLevel, 1));
}

void GetNumMipLevels_{ParameterName}(out int OutMipLevels)
{
	OutMipLevels = {ParameterName}_MipLevels;
}

void SamplePseudoVolumeTexture_{ParameterName}(in float3 UVW, in float2 XYNumFrames, in float TotalNumFrames, in int MipMode, in float MipLevel, in float2 DDX, in float2 DDY, out float4 OutValue)
{
	OutValue = PseudoVolumeTexture({ParameterName}_Texture, {ParameterName}_TextureSampler, UVW, XYNumFrames, TotalNumFrames, (uint)MipMode, MipLevel, DDX, DDY);
}
