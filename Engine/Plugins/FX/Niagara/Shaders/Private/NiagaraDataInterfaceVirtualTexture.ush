// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef NUM_VIRTUALTEXTURE_SAMPLES
	#define NUM_VIRTUALTEXTURE_SAMPLES 1
#endif

#include "/Engine/Private/VirtualTextureCommon.ush"

// Mirror of ERuntimeVirtualTextureMaterialType
#define ERuntimeVirtualTextureMaterialType_BaseColor								0
#define ERuntimeVirtualTextureMaterialType_Mask4									1
#define ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Roughness				2
#define ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular				3
#define ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular_YCoCg			4
#define ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular_Mask_YCoCg		5
#define ERuntimeVirtualTextureMaterialType_WorldHeight								6
#define ERuntimeVirtualTextureMaterialType_Displacement								7
#define ERuntimeVirtualTextureMaterialType_Count									8

