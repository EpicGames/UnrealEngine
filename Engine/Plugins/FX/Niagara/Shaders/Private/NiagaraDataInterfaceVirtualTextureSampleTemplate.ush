// Copyright Epic Games, Inc. All Rights Reserved.

#define NUM_VIRTUALTEXTURE_SAMPLES 0
// Workaround for unbound View uniform buffer
#define VT_DISABLE_VIEW_UNIFORM_BUFFER 1

#include "/Engine/Private/VirtualTextureCommon.ush"

int2			{ParameterName}_TextureSize;
int				{ParameterName}_MipLevels;

Texture2D<uint4> {ParameterName}_PageTableTexture0;
Texture2D<uint4> {ParameterName}_PageTableTexture1;
Texture2D {ParameterName}_PhysicalTexture;

uint4 {ParameterName}_VTPackedPageTableUniform[2];
uint4 {ParameterName}_VTPackedUniform;

SamplerState {ParameterName}_TextureSampler;

void SampleTexture2D_{ParameterName}(in float2 UV, in float MipLevel, out float4 OutValue)
{
	uint LayerIndex = 0;
	float4 Sample;

	VTPageTableResult PageTableResult = TextureLoadVirtualPageTableLevel({ParameterName}_PageTableTexture0, {ParameterName}_PageTableTexture1, VTPageTableUniform_Unpack({ParameterName}_VTPackedPageTableUniform[0], {ParameterName}_VTPackedPageTableUniform[1]), UV, VTADDRESSMODE_CLAMP, VTADDRESSMODE_CLAMP, MipLevel);
	OutValue = TextureVirtualSample( {ParameterName}_PhysicalTexture, {ParameterName}_TextureSampler, PageTableResult, LayerIndex, VTUniform_Unpack({ParameterName}_VTPackedUniform));	 
}

void GetTextureDimensions_{ParameterName}(int MipLevel, out float2 OutValue)
{
	OutValue.x = float(max({ParameterName}_TextureSize.x >> MipLevel, 1));
	OutValue.y = float(max({ParameterName}_TextureSize.y >> MipLevel, 1));
}

void GetNumMipLevels_{ParameterName}(out int OutMipLevels)
{
	OutMipLevels = {ParameterName}_MipLevels;
}


