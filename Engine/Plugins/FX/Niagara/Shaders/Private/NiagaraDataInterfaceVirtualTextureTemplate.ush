// Copyright Epic Games, Inc. All Rights Reserved.

Texture2D			{ParameterName}_VirtualTexture0;
Texture2D<uint4>	{ParameterName}_VirtualTexture0PageTable;
uint4				{ParameterName}_VirtualTexture0TextureUniforms;

Texture2D			{ParameterName}_VirtualTexture1;
Texture2D<uint4>	{ParameterName}_VirtualTexture1PageTable;
uint4				{ParameterName}_VirtualTexture1TextureUniforms;

Texture2D			{ParameterName}_VirtualTexture2;
Texture2D<uint4>	{ParameterName}_VirtualTexture2PageTable;
uint4				{ParameterName}_VirtualTexture2TextureUniforms;

uint				{ParameterName}_ValidLayersMask;
int					{ParameterName}_MaterialType;
float4				{ParameterName}_UVUniforms[3];
uint4				{ParameterName}_PageTableUniforms[2];
float2				{ParameterName}_WorldHeightUnpack;
SamplerState		{ParameterName}_SharedSampler;

void GetAttributesValid_{ParameterName}(out bool BaseColorValid, out bool SpecularValid, out bool RoughnessValid, out bool NormalValid, out bool WorldHeightValid, out bool MaskValid, out bool DisplacementValid, out bool Mask4Valid)
{
	BaseColorValid = false;
	SpecularValid = false;
	RoughnessValid = false;
	NormalValid = false;
	WorldHeightValid = false;
	MaskValid = false;
	DisplacementValid = false;
	Mask4Valid = false;

	switch ( {ParameterName}_MaterialType )
	{
		case ERuntimeVirtualTextureMaterialType_BaseColor:
			BaseColorValid = true;
			break;

		case ERuntimeVirtualTextureMaterialType_Mask4:
			Mask4Valid = true;
			break;

		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Roughness:
			BaseColorValid = true;
			RoughnessValid = true;
			NormalValid = true;
			break;

		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular:
			BaseColorValid = true;
			SpecularValid = true;
			RoughnessValid = true;
			NormalValid = true;
			break;
		
		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular_YCoCg:
			BaseColorValid = true;
			SpecularValid = true;
			RoughnessValid = true;
			NormalValid = true;
			break;
		
		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular_Mask_YCoCg:
			BaseColorValid = true;
			SpecularValid = true;
			RoughnessValid = true;
			NormalValid = true;
			MaskValid = true;
			break;
		
		case ERuntimeVirtualTextureMaterialType_WorldHeight:
			WorldHeightValid = true;
			break;

		case ERuntimeVirtualTextureMaterialType_Displacement:
			DisplacementValid = true;
			break;
	}
}

float4 SampleRVTLayer_{ParameterName}(float2 SampleUV, Texture2D InTexture, Texture2D<uint4> InPageTable, uint4 InTextureUniforms)
{
	VTPageTableResult PageTable = TextureLoadVirtualPageTableLevel(InPageTable, VTPageTableUniform_Unpack({ParameterName}_PageTableUniforms[0], {ParameterName}_PageTableUniforms[1]), SampleUV, VTADDRESSMODE_CLAMP, VTADDRESSMODE_CLAMP, 0.0f);
	return TextureVirtualSample(InTexture, {ParameterName}_SharedSampler, PageTable, 0, VTUniform_Unpack(InTextureUniforms));
}

void SampleRVT_{ParameterName}(in float3 WorldPosition, out bool bInsideVolume, out float3 BaseColor, out float Specular, out float Roughness, out float3 Normal, out float WorldHeight, out float Mask, out float Displacement, out float4 Mask4)
{
	bInsideVolume = false;
	BaseColor = float3(0.0f, 0.0f, 0.0f);
	Specular = 0.5f;
	Roughness = 0.5f;
	Normal = float3(0.0f, 0.0f, 1.0f);
	WorldHeight = 0.0f;
	Mask = 1.0f;
	Displacement = 0.0f;
	Mask4 = float4(0.0f, 0.0f, 0.0f, 0.0f);

	// Get Sample Location
	FDFVector3 LWCWorldPosition = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition));
	FDFVector3 LWCUVOrigin = DFFromTileOffset_Hack(MakeLWCVector3(GetEngineOwnerLWCTile(), {ParameterName}_UVUniforms[0].xyz));

	float2 SampleUV = VirtualTextureWorldToUV(LWCWorldPosition, LWCUVOrigin, {ParameterName}_UVUniforms[1].xyz, {ParameterName}_UVUniforms[2].xyz);

	// Test to see if we are inside the volume, but still take the samples as it will clamp to the edge
	bInsideVolume = all(SampleUV >- 0.0f) && all(SampleUV < 1.0f);

	// Sample Textures
	float4 LayerSample[3];
	LayerSample[0] = ({ParameterName}_ValidLayersMask & 0x1) != 0 ? SampleRVTLayer_{ParameterName}(SampleUV, {ParameterName}_VirtualTexture0, {ParameterName}_VirtualTexture0PageTable, {ParameterName}_VirtualTexture0TextureUniforms) : 0;
	LayerSample[1] = ({ParameterName}_ValidLayersMask & 0x2) != 0 ? SampleRVTLayer_{ParameterName}(SampleUV, {ParameterName}_VirtualTexture1, {ParameterName}_VirtualTexture1PageTable, {ParameterName}_VirtualTexture1TextureUniforms) : 0;
	LayerSample[2] = ({ParameterName}_ValidLayersMask & 0x4) != 0 ? SampleRVTLayer_{ParameterName}(SampleUV, {ParameterName}_VirtualTexture2, {ParameterName}_VirtualTexture2PageTable, {ParameterName}_VirtualTexture2TextureUniforms) : 0;

	// Sample Available Attributes
	switch ( {ParameterName}_MaterialType )
	{
		case ERuntimeVirtualTextureMaterialType_BaseColor:
		{
			BaseColor = LayerSample[0].rgb;
			break;
		}

		case ERuntimeVirtualTextureMaterialType_Mask4:
		{
			Mask4 = LayerSample[0];
			break;
		}

		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Roughness:
		{
			BaseColor = VirtualTextureUnpackBaseColorSRGB(LayerSample[0]);
			Roughness = LayerSample[1].y;
			Normal = VirtualTextureUnpackNormalBGR565(LayerSample[1]);
			break;
		}

		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular:
		{
			BaseColor = LayerSample[0].rgb;
			Specular = LayerSample[1].x;
			Roughness = LayerSample[1].y;
			Normal = VirtualTextureUnpackNormalBC3BC3(LayerSample[0], LayerSample[1]);
			break;
		}
		
		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular_YCoCg:
		{
			BaseColor = VirtualTextureUnpackBaseColorYCoCg(LayerSample[0]);
			Specular = LayerSample[2].x;
			Roughness = LayerSample[2].y;
			Normal = VirtualTextureUnpackNormalBC5BC1(LayerSample[1], LayerSample[2]);
			break;
		}
		
		case ERuntimeVirtualTextureMaterialType_BaseColor_Normal_Specular_Mask_YCoCg:
		{
			BaseColor = VirtualTextureUnpackBaseColorYCoCg(LayerSample[0]);
			Specular = LayerSample[2].x;
			Roughness = LayerSample[2].y;
			Normal = VirtualTextureUnpackNormalBC5BC1(LayerSample[1], LayerSample[2]);
			Mask = LayerSample[2].w;
			break;
		}
		
		case ERuntimeVirtualTextureMaterialType_WorldHeight:
		{
			WorldHeight = VirtualTextureUnpackHeight(LayerSample[0], {ParameterName}_WorldHeightUnpack);
			break;
		}

		case ERuntimeVirtualTextureMaterialType_Displacement:
		{
			Displacement = LayerSample[0].x;
			break;
		}
	}
}
