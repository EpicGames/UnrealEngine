// Copyright Epic Games, Inc. All Rights Reserved.

float3			{ParameterName}_TextureSize;
Texture3D		{ParameterName}_Texture;
SamplerState	{ParameterName}_TextureSampler;

void SampleCurrentFrameValue_{ParameterName}(in float3 UV, in float MipLevel, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UV, MipLevel);
}

void GetCurrentFrameValue_{ParameterName}(in int x, in int y, in int z, in float MipLevel, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.Load(int4(x,y,z,MipLevel));
}

void GetCurrentFrameNumCells_{ParameterName}(out float3 OutValue)
{
	OutValue = {ParameterName}_TextureSize;
}

void IndexToUnit_{ParameterName}(float3 Index, out float3 Unit)
{
	Unit = (Index + .5) / {ParameterName}_TextureSize;
}
