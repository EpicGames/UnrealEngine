// Copyright Epic Games, Inc. All Rights Reserved.

float3			{ParameterName}_TextureSize;
Texture3D		{ParameterName}_Texture;
SamplerState	{ParameterName}_TextureSampler;

void LoadVolumeTexture_{ParameterName}(in int TexelX, in int TexelY, in int TexelZ, in int MipLevel, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.Load(int4(TexelX, TexelY, TexelZ, MipLevel));
}

void SampleVolumeTexture_{ParameterName}(in float3 UV, in float MipLevel, out float4 OutValue)
{
	OutValue = {ParameterName}_Texture.SampleLevel({ParameterName}_TextureSampler, UV, MipLevel);
}

void TextureDimensions3D_{ParameterName}(out float3 OutValue)
{
	OutValue = {ParameterName}_TextureSize;
}
