// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NiagaraDebugDraw.usf
=============================================================================*/

#include "/Engine/Private/Common.ush"
#include "NiagaraCommon.ush"
#include "NiagaraDebugDraw.ush"

////////////////////////////////////////////////////////////////////////////////////

#if NIAGARA_DEBUGDRAW_CLEARUAV_UINT_CS
RWBuffer<uint>	BufferToClear;
uint4			ClearValue;
uint			ClearSize;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void MainCS(in uint3 ThreadID : SV_DispatchThreadID)
{
	if ( ThreadID.x < ClearSize )
	{
		BufferToClear[ThreadID.x] = ClearValue[ThreadID.x & 3];
	}
}
#endif

////////////////////////////////////////////////////////////////////////////////////

#if NIAGARA_DEBUGDRAW_DRAWLINE_VS
Buffer<uint>	GpuLineBuffer;

void MainVS(
	in uint InstanceID		: SV_InstanceID,
	in uint VertexID		: SV_VertexID,
	out float4 OutPosition	: SV_POSITION,
	out float4 OutColor		: TEXCOORD0
)
{
	float3 Position0;
	float3 Position1;
	float4 Color;

	UnpackLine(GpuLineBuffer, InstanceID, Position0, Position1, Color);

	float3 Position = VertexID == 0 ? Position0 : Position1;
	OutPosition = mul(float4(Position, 1), DFHackToFloat(PrimaryView.WorldToClip));
	OutColor = Color;
}
#endif

#if NIAGARA_DEBUGDRAW_DRAWLINE_PS

Texture2D<float> DepthTexture;
SamplerState DepthSampler;
float2 OutputInvResolution; 
float2 OriginalViewRectMin;
float2 OriginalViewSize;
float2 OriginalBufferInvSize;
float OccludedColorScale;

void MainPS (
	in float4 Position	: SV_POSITION,
	in float4 Color		: TEXCOORD0,
	out float4 OutColor	: SV_Target0
)
{
	uint2 PixelCoord		= Position.xy;
	const float2 ViewportUV = float2(PixelCoord) * OutputInvResolution;
	const float2 DepthUV	= (ViewportUV * OriginalViewSize.xy + OriginalViewRectMin) * OriginalBufferInvSize;
	const float Depth		= DepthTexture.SampleLevel(DepthSampler, DepthUV, 0);
	const bool bIsHidden	= Position.z < Depth; // Reverse-Z

	const float ColorScale = bIsHidden ? OccludedColorScale : 1;
	OutColor.a = Color.a * ColorScale;
	OutColor.rgb = Color.rgb * OutColor.a;
}
#endif
