// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=============================================================================
	NiagaraDebugDraw.ush
	Includes for debug drawing
=============================================================================*/

void PackLine(RWBuffer<uint> OutBuffer, uint InstanceIndex, float3 InPosition0, float3 InPosition1, uint InPackedColor)
{
	uint BufferOffset = InstanceIndex * 7;

	OutBuffer[BufferOffset + 0] = asuint(InPosition0.x);
	OutBuffer[BufferOffset + 1] = asuint(InPosition0.y);
	OutBuffer[BufferOffset + 2] = asuint(InPosition0.z);

	OutBuffer[BufferOffset + 3] = asuint(InPosition1.x);
	OutBuffer[BufferOffset + 4] = asuint(InPosition1.y);
	OutBuffer[BufferOffset + 5] = asuint(InPosition1.z);

	OutBuffer[BufferOffset + 6] = InPackedColor;
}

void PackLine(RWBuffer<uint> OutBuffer, uint InstanceIndex, float3 InPosition0, float3 InPosition1, float4 InColor)
{
	uint PackedColor = 0;
	PackedColor  = uint(saturate(InColor.x) * 255.0f) << 24;
	PackedColor |= uint(saturate(InColor.y) * 255.0f) << 16;
	PackedColor |= uint(saturate(InColor.z) * 255.0f) << 8;
	PackedColor |= uint(saturate(InColor.w) * 255.0f) << 0;
	PackLine(OutBuffer, InstanceIndex, InPosition0, InPosition1, PackedColor);
}

void UnpackLine(in Buffer<uint> InBuffer, uint InstanceIndex, out float3 OutPosition0, out float3 OutPosition1, out float4 OutColor)
{
	uint BufferOffset = InstanceIndex * 7;

	OutPosition0.x = asfloat(InBuffer[BufferOffset + 0]);
	OutPosition0.y = asfloat(InBuffer[BufferOffset + 1]);
	OutPosition0.z = asfloat(InBuffer[BufferOffset + 2]);

	OutPosition1.x = asfloat(InBuffer[BufferOffset + 3]);
	OutPosition1.y = asfloat(InBuffer[BufferOffset + 4]);
	OutPosition1.z = asfloat(InBuffer[BufferOffset + 5]);

	uint PackedColor = InBuffer[BufferOffset + 6];
	OutColor.x = float((PackedColor >> 24) & 0xff) / 255.0f;
	OutColor.y = float((PackedColor >> 16) & 0xff) / 255.0f;
	OutColor.z = float((PackedColor >>  8) & 0xff) / 255.0f;
	OutColor.w = float((PackedColor >>  0) & 0xff) / 255.0f;
}
