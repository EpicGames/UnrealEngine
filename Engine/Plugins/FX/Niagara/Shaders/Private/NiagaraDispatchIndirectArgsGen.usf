// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	NiagaraDrawIndirectArgsGen.usf: Shader to copy sorted index buffers.
==============================================================================*/

#include "/Engine/Private/Common.ush"

#define INDEX_NONE uint(-1)

// Input Format (FNiagaraDispatchIndirectInfoCS)
//	uint3 - Counter Offsets
//	uint3 - Thread Group Size
//	uint  - Indirect Args Offset
Buffer<uint>	DispatchInfos;
uint			NumDispatchInfos;
uint3			MaxGroupsPerDimension;

Buffer<uint>	InstanceCounts;

// Output Format (FNiagaraDispatchIndirectParametersCS)
//	uint4 - Indirect Group Count
//	uint4 - Indirect Thread Count
RWBuffer<uint>	RWDispatchIndirectArgs;

[numthreads(THREAD_COUNT,1,1)]
void IndirectArgsGenCS(uint DispatchIndex : SV_DispatchThreadID)
{
	// Clip dead threads
	if (DispatchIndex >= NumDispatchInfos)
		return;

	uint DispatchOffset = DispatchIndex * 7;

	uint3 NumThreads = uint3(DispatchInfos[DispatchOffset + 0], DispatchInfos[DispatchOffset + 1], DispatchInfos[DispatchOffset + 2]);
	NumThreads.x = NumThreads.x != INDEX_NONE ? InstanceCounts[NumThreads.x] : 0;
	NumThreads.y = NumThreads.y != INDEX_NONE ? InstanceCounts[NumThreads.y] : 1;
	NumThreads.z = NumThreads.z != INDEX_NONE ? InstanceCounts[NumThreads.z] : 1;

	uint3 ThreadGroupSize = uint3(DispatchInfos[DispatchOffset + 3], DispatchInfos[DispatchOffset + 4], DispatchInfos[DispatchOffset + 5]);

	uint3 NumGroups = 0;
	if ( all(NumThreads > 0) && all(ThreadGroupSize > 0) )
	{
		NumGroups = (NumThreads + ThreadGroupSize - 1) / ThreadGroupSize;
		if ( any(NumGroups > MaxGroupsPerDimension) )
		{
			NumGroups = 0;
		}
	}

	uint IndirectArgsOffset = DispatchInfos[DispatchOffset + 6] * 8;
	RWDispatchIndirectArgs[IndirectArgsOffset + 0] = NumGroups.x;
	RWDispatchIndirectArgs[IndirectArgsOffset + 1] = NumGroups.y;
	RWDispatchIndirectArgs[IndirectArgsOffset + 2] = NumGroups.z;
	RWDispatchIndirectArgs[IndirectArgsOffset + 3] = 0;
	RWDispatchIndirectArgs[IndirectArgsOffset + 4] = NumThreads.x;
	RWDispatchIndirectArgs[IndirectArgsOffset + 5] = NumThreads.y;
	RWDispatchIndirectArgs[IndirectArgsOffset + 6] = NumThreads.z;
	RWDispatchIndirectArgs[IndirectArgsOffset + 7] = 0;
}
