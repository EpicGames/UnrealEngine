// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDistanceFieldQueries.ush 
=============================================================================*/

#pragma once

float GetDistanceToMeshDistanceField(int DFIndex, const FDFVector3 LWCWorldPosition, const float MaxDistance)
{
	FDFObjectData DFObjectData = LoadDFObjectData(DFIndex);

	// DistanceToNearestSurfaceForObject()
	const float3 VolumePosition = DFMultiplyAffineDemote(LWCWorldPosition, DFObjectData.WorldToVolume);
	const float3 ClampedVolumePosition = clamp(VolumePosition, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent);

	const float3 ToBox = (abs(ClampedVolumePosition) - DFObjectData.VolumePositionExtent) * DFObjectData.VolumeToWorldScale;
	const float DistanceToBoxOutside = length(max(0.0f, ToBox));
	const float DistanceToBoxInside = min(0.0f, max3(ToBox.x, ToBox.y, ToBox.z));
	const float DistanceToBox = DistanceToBoxOutside + DistanceToBoxInside;

	float DistanceToOccluder = MaxDistance;

	BRANCH
	if (DistanceToBox < MaxDistance)
	{
		uint NumMips = LoadDFAssetData(DFObjectData.AssetIndex, 0).NumMips;
		FDFAssetData DFAssetData = LoadDFAssetData(DFObjectData.AssetIndex, NumMips - 1);

		DistanceToOccluder = DistanceToMeshSurfaceStandalone(ClampedVolumePosition, DFObjectData) * DFObjectData.VolumeScale;
	}

	return DistanceToOccluder;
}

void ComputeClosestPointMeshDistanceField(int DFIndex, const FDFVector3 LWCWorldPosition, const float GridSampledDistance, out float3 OutClosestPosition, out float3 OutClosestNormal, out float MaxEncodedDistance, out bool bNormalIsValid)
{
	FDFObjectData DFObjectData = LoadDFObjectData(DFIndex);
	
	const float3 VolumePosition = DFMultiplyAffineDemote(LWCWorldPosition, DFObjectData.WorldToVolume);
	const float3 ClampedVolumePosition = clamp(VolumePosition, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent);
	
	uint NumMips = LoadDFAssetData(DFObjectData.AssetIndex, 0).NumMips;
	FDFAssetData DFAssetData = LoadDFAssetData(DFObjectData.AssetIndex, NumMips - 1);

	MaxEncodedDistance = (DFAssetData.DistanceFieldToVolumeScaleBias.x + DFAssetData.DistanceFieldToVolumeScaleBias.y) * DFObjectData.VolumeScale;
		
	float3 VolumeGradient = CalculateMeshSDFGradient(ClampedVolumePosition, DFAssetData);
	float VolumeGradientLength = length(VolumeGradient);
	float3 VolumeNormal = VolumeGradientLength > 0.0f ? VolumeGradient / VolumeGradientLength : 0;

	// Transform by transposed inverse to handle non-uniform scaling
	float4x4 WorldToVolume = DFHackToFloat(DFObjectData.WorldToVolume);
	float3 WorldGradient = mul(VolumeNormal, transpose((float3x3)WorldToVolume));
	float WorldGradientLength = length(WorldGradient);

	// @note: we should include checking if we are within the range of max encoded distance, but this breaks backwards compatibility
	bNormalIsValid = WorldGradientLength > 0.0f && (MaxEncodedDistance - abs(GridSampledDistance) > 1e-8);
			
	// output 			
	if (bNormalIsValid)
	{
		OutClosestNormal = WorldGradient / WorldGradientLength;

		const float3 WorldClampedPosition = DFDemote(DFMultiplyAffine(ClampedVolumePosition, DFObjectData.VolumeToWorld));
		OutClosestPosition = WorldClampedPosition - OutClosestNormal * GridSampledDistance;
	}
}
