// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	NiagaraDrawIndirectArgsGen.usf: Shader to copy sorted index buffers.
==============================================================================*/

#include "/Engine/Private/Common.ush"

// Array of task info. Each being a draw indirect arg gen or instance count reset.
// ArgGen = (InstanceCountBufferOffset, NumIndicesPerInstance, StartIndexLocation)
// InstanceCountClear = (InstanceCountBufferOffset)
Buffer<uint> TaskInfos;
Buffer<uint> CulledInstanceCounts;

RWBuffer<uint> RWInstanceCounts;
RWBuffer<uint> RWDrawIndirectArgs;

uint4 TaskCount; // (ArgGenTaskOffset, NumArgGenTasks, NumInstanceCountClearTasks, NumArgGenTasks + NumInstanceCountClearTasks)

#define FLAG_USE_CULLED_COUNTS	(1u << 0u)
#define FLAG_INSTANCED_STEREO	(1u << 1u)

[numthreads(THREAD_COUNT,1,1)]
void MainCS(uint TaskIndex : SV_DispatchThreadID)
{
	const uint ArgGenTaskOffset = TaskCount.x;
	const uint NumArgGenTasks = TaskCount.y;
	const uint NumInstanceCountClearTasks = TaskCount.z;
	const uint NumTotalTasks = TaskCount.w;

	if (TaskIndex < NumArgGenTasks)
	{
		const uint InfoOffset = (ArgGenTaskOffset + TaskIndex) * NIAGARA_DRAW_INDIRECT_TASK_INFO_SIZE;
		const uint ArgOffset = TaskInfos[InfoOffset + 0];
		const uint Flags = TaskInfos[InfoOffset + 4];
		const bool bUseCulledCounts = (Flags & FLAG_USE_CULLED_COUNTS) != 0;
		const bool bInstancedStereo = (Flags & FLAG_INSTANCED_STEREO) != 0;

		uint InstanceCount = 0;
		BRANCH
		if (bUseCulledCounts)
		{
			InstanceCount = CulledInstanceCounts[TaskInfos[InfoOffset + 1]];
		}
		else
		{
			InstanceCount = RWInstanceCounts[TaskInfos[InfoOffset + 1]];
		}

		if (bInstancedStereo)
		{
			InstanceCount *= 2;
		}

		RWDrawIndirectArgs[ArgOffset + 0] = TaskInfos[InfoOffset + 2]; // NumIndicesPerInstance
		RWDrawIndirectArgs[ArgOffset + 1] = InstanceCount;
		RWDrawIndirectArgs[ArgOffset + 2] = TaskInfos[InfoOffset + 3]; // StartIndexLocation
		RWDrawIndirectArgs[ArgOffset + 3] = 0; // BaseVertexLocation
		RWDrawIndirectArgs[ArgOffset + 4] = 0; // StartInstanceLocation
	}

	#if SUPPORTS_TEXTURE_RW == 1
		/*
		Explanation of why we are using two ifs here instead of if/else:
		Mali requires texture indexes to be warp invariant, which means that the if / if code translates to
			if id.x < uniform :
			[to 5 stores]
			[branch to end]
			if <the other condition>
			do 1 image store
			<end>
		and the if / else translates to
			if uniform > ID.x: goto 4 image stores
			if <other condition> :
			setup registers
			goto <last store>
			<4 images stores branch target>
			do 4 stores
			setup registers
			<last store target>
			do 1 store
		Notice how it completely ignores the fact that there are two different texture buffers, which leads the compiler to do an "optimization" and separate the last store.This means that the else condition does indeed evaluate correctly, but we are writing in the wrong texture!
		Note: This is imagestore specific and not compute specific and is a known issue for Arm. This is fixed in driver version R21
		*/

		if (TaskIndex < NumTotalTasks && TaskIndex >= NumArgGenTasks)
		{
			const uint InfoOffset = (ArgGenTaskOffset + NumArgGenTasks) * NIAGARA_DRAW_INDIRECT_TASK_INFO_SIZE + (TaskIndex - NumArgGenTasks);
			RWInstanceCounts[TaskInfos[InfoOffset]] = 0;
		}
#endif
}

[numthreads(THREAD_COUNT,1,1)]
void ResetCountsCS(uint TaskIndex : SV_DispatchThreadID)
{
	const uint ArgGenTaskOffset = TaskCount.x;
	const uint NumArgGenTasks = TaskCount.y;
	const uint NumInstanceCountClearTasks = TaskCount.z;
	if (TaskIndex < NumInstanceCountClearTasks)
	{
		const uint InfoOffset = (ArgGenTaskOffset + NumArgGenTasks) * NIAGARA_DRAW_INDIRECT_TASK_INFO_SIZE + (TaskIndex);
		RWInstanceCounts[TaskInfos[InfoOffset]] = 0;
	}
}

[numthreads(THREAD_COUNT,1,1)]
void InitInstanceCountsCS(uint TaskIndex : SV_DispatchThreadID)
{
	const uint NumTasks = TaskCount.x;
	if (TaskIndex < NumTasks)
	{
		const uint InfoOffset = TaskIndex * NIAGARA_INIT_INSTANCE_COUNT_TASK_INFO_SIZE;
		
		const uint CountOffset = TaskInfos[InfoOffset];
		const uint CountValue = TaskInfos[InfoOffset + 1];
		RWInstanceCounts[CountOffset] = CountValue;
	}
}