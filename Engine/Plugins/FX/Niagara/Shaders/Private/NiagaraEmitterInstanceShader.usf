// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraSimulationShader.usf: 
=============================================================================*/
#pragma warning(disable:4008)
#include "/Engine/Public/Platform.ush"

#include "NiagaraShaderVersion.ush"

#ifndef SMALL_NUMBER
	#define SMALL_NUMBER 1e-8
#endif
#ifndef MAX_DISTANCE
	#define MAX_DISTANCE 1e+38
#endif

#if GPU_SIMULATION
	#include "/Engine/Private/Common.ush"
	#include "/Engine/Private/DistanceField/GlobalDistanceFieldShared.ush"

	#define NEEDS_SCENE_TEXTURES 1
	#define SCENE_TEXTURES_DISABLED 0
	#include "/Engine/Generated/UniformBuffers/SubstratePublic.ush"
#else
	const static float PI = 3.1415926535897932f;
#endif
	const static float TWO_PI = 3.1415926535897932f*2.0f;

#include "/Engine/Private/Definitions.usf"

// Most of the vector implementations work this way. This helps us keep proper precision.
float4 ModuloPrecise(float4 x, float4 y){ return x - y * trunc(x/y); }
float3 ModuloPrecise(float3 x, float3 y){ return x - y * trunc(x/y); }
float2 ModuloPrecise(float2 x, float2 y){ return x - y * trunc(x/y);}
float ModuloPrecise(float x, float y){ return x - y * trunc(x/y); }
int ModuloPrecise(int x, int y){ return x - y * (x/y); }
int Modulo(int x, int y){  return x - y * (x/y);  }

	
// using rcp is only 12 bits of precision, we should usually pay for perf
float4 Reciprocal(float4 x){ return 1.0f/x;}
float3 Reciprocal(float3 x){ return 1.0f/x; }
float2 Reciprocal(float2 x){ return 1.0f/x;}
float Reciprocal(float x){ return 1.0f/x; }

// Annoyingly, all(bool) and any(bool) don't exist, so we'll make Niagara versions which
// work with both scalars and vectors.
bool NiagaraAll(bool b) { return b; }
bool NiagaraAny(bool b) { return b; }
#if GPU_SIMULATION
bool NiagaraAll(bool2 b) { return all(b); }
bool NiagaraAll(bool3 b) { return all(b); }
bool NiagaraAll(bool4 b) { return all(b); }
bool NiagaraAny(bool2 b) { return any(b); }
bool NiagaraAny(bool3 b) { return any(b); }
bool NiagaraAny(bool4 b) { return any(b); }
#else
// No all() and any() opcodes in our VM, emulate them.
bool NiagaraAll(bool2 b) { return b.x && b.y; }
bool NiagaraAll(bool3 b) { return b.x && b.y && b.z; }
bool NiagaraAll(bool4 b) { return b.x && b.y && b.z && b.w; }
bool NiagaraAny(bool2 b) { return b.x || b.y; }
bool NiagaraAny(bool3 b) { return b.x || b.y || b.z; }
bool NiagaraAny(bool4 b) { return b.x || b.y || b.z || b.w; }
#endif

#define GetEngineOwnerLWCTile() Engine_Owner_LWCTile.xyz

/* -----------------------------------------------------------------
 * GPU simulation utility functions
 * -----------------------------------------------------------------
 */
#if GPU_SIMULATION
	#ifndef NIAGARA_PARTICLE_PARTIAL_ENABLED
		#define NIAGARA_PARTICLE_PARTIAL_ENABLED 0
	#endif

	// Type of dispatch mode we are in this allows us to customize some HLSL code to avoid / % operations
	#define NIAGARA_DISPATCH_TYPE_ONE_D		0
	#define NIAGARA_DISPATCH_TYPE_TWO_D		1
	#define NIAGARA_DISPATCH_TYPE_THREE_D	2
	#define NIAGARA_DISPATCH_TYPE_CUSTOM	3
	#ifndef NIAGARA_DISPATCH_TYPE
		#define NIAGARA_DISPATCH_TYPE NIAGARA_DISPATCH_TYPE_ONE_D
	#endif

	#if NIAGARA_DISPATCH_INDIRECT
		static uint3	DispatchThreadIdBounds;		// Used to track thread bounds
		Buffer<uint4>	IndirectDispatchArgs;		// Indirect count buffer we lookup into to get actual thread count
		uint			IndirectDispatchArgsOffset;	// Offset into the indirect count buffer
	#else
		uint3 DispatchThreadIdBounds;			// Used to track thread bounds
		uint3 DispatchThreadIdToLinear;			// Used to convert from DispatchThreadId to a linear thread id
	#endif

	static uint3 GDispatchThreadId;		// SV_DispatchThreadId
	static uint3 GGroupId;				// SV_GroupdId
	static uint3 GGroupThreadId;		// SV_GroupThreadId
	static uint  GGroupIndex;			// SV_GroupIndex
	static uint GLinearThreadId;
	static uint GEmitterTickCounter;
	static uint GRandomSeedOffset = 0;

	// To be removed as well but don't know who is using it
	#include "/Engine/Private/SceneTexturesCommon.ush"

	// Physics common 
	//-TOFIX: This should not be included here
	#include "NiagaraPhysicsCommon.ush"

	uint EmitterTickCounter;

	float4 Modulo(float4 x, float4 y){ return fmod(x,y); }
	float3 Modulo(float3 x, float3 y){ return fmod(x,y); }
	float2 Modulo(float2 x, float2 y){ return fmod(x,y); }
	float Modulo(float x, float y){ return fmod(x,y); }

	// utility function used for scene depth calculations
	FLWCVector3 WorldPositionFromSceneDepth(float2 ScreenPosition, float SceneDepth)
	{
		FLWCVector4 HomogeneousWorldPosition = LWCMultiply(float4(ScreenPosition * SceneDepth, SceneDepth, 1), DFFastToTileOffset(PrimaryView.ScreenToWorld)); //DF_TODO
		FLWCVector3 WorldPosition = MakeLWCVector3(LWCGetTile(HomogeneousWorldPosition).xyz, HomogeneousWorldPosition.Offset.xyz);
		FLWCScalar Scale = MakeLWCScalar(LWCGetTile(HomogeneousWorldPosition).w, HomogeneousWorldPosition.Offset.w);
		return LWCDivide(WorldPosition, LWCToFloat(Scale));
	}

	// MASSIVE HACK - Tracked in JIRA UE-69298
	// Hardcoded random function accessible from inner part of node implementation.
	// It works for now at least and avoids exposing every random needed in the UI. 
	// Temporary solution, it will be replaced when a design is validated.
	float NiagaraInternalNoise(uint u, uint v, uint s)
	{
		uint Seed = (u * 1664525u + v) + s + GRandomSeedOffset;
		GRandomSeedOffset += Seed;
		return float(Rand3DPCG32(int3(u,v,Seed)).x) / 4294967296.0f;
	}

	// NIAGARA_MAX_GPU_SPAWN_INFOS is set from the shader compiler
	#define NIAGARA_MAX_GPU_SPAWN_INFOS_V4	((NIAGARA_MAX_GPU_SPAWN_INFOS + 3) / 4)

	int4	EmitterSpawnInfoOffsets[NIAGARA_MAX_GPU_SPAWN_INFOS_V4];
	float4	EmitterSpawnInfoParams[NIAGARA_MAX_GPU_SPAWN_INFOS];			// Packed data where x = IntervalDt, y = InterpStartDt, z = Group, w = Start Particle Index

	static int GInterpSpawnIndex;
	static float Emitter_SpawnInterval;
	static float Emitter_InterpSpawnStartDt;
	static int Emitter_SpawnGroup;

	static int Engine_ExecutionCount;
	static int GGPUExecIndex;

	static uint GSpawnStartInstance;
	uint NumSpawnedInstances;

	void SetupExecIndexForGPU()
	{
		GGPUExecIndex = GLinearThreadId;
		Engine_ExecutionCount = GSpawnStartInstance;
	}

	void SetupExecIndexAndSpawnInfoForGPU()
	{
		GGPUExecIndex = GLinearThreadId - GSpawnStartInstance;

		int SpawnInfoIndex = 0;
		UNROLL
		for (int i = 0; i < NIAGARA_MAX_GPU_SPAWN_INFOS_V4; ++i)
		{
			// This returns 0xffffffff for each component when the comparison is true, so we'll do a
			// bitwise and with 1 to get increment amounts for each spawn info.
			int4 CompareResults = GGPUExecIndex >= EmitterSpawnInfoOffsets[i];
			CompareResults = CompareResults & int4(1, 1, 1, 1);
			SpawnInfoIndex += CompareResults.x + CompareResults.y + CompareResults.z + CompareResults.w;
		}

		Emitter_SpawnInterval		= EmitterSpawnInfoParams[SpawnInfoIndex].x;
		Emitter_InterpSpawnStartDt	= EmitterSpawnInfoParams[SpawnInfoIndex].y;
		Emitter_SpawnGroup			= asint(EmitterSpawnInfoParams[SpawnInfoIndex].z);

		int GroupSpawnStartIndex    = asint(EmitterSpawnInfoParams[SpawnInfoIndex].w);
		GGPUExecIndex		        = GGPUExecIndex - GroupSpawnStartIndex;
		if ( SpawnInfoIndex == (NIAGARA_MAX_GPU_SPAWN_INFOS - 1) )
		{
			Engine_ExecutionCount   = int(NumSpawnedInstances) - GroupSpawnStartIndex;
		}
		else
		{
			int NextGroupSpawnStartIndex = asint(EmitterSpawnInfoParams[SpawnInfoIndex + 1].w);
			Engine_ExecutionCount       = NextGroupSpawnStartIndex - GroupSpawnStartIndex;
		}
	}

	/* Returns the current instance index relative to the operation (spawn/update)
	 */
	int ExecIndex()
	{
		return GGPUExecIndex;
	}

	float4 NiagaraGPU_QuatMul(float4 Q1, float4 Q2)
	{
		float4 QOut;
		QOut.x = Q1.w*Q2.x + Q1.x*Q2.w + Q1.y*Q2.z - Q1.z*Q2.y;
		QOut.y = Q1.w*Q2.y - Q1.x*Q2.z + Q1.y*Q2.w + Q1.z*Q2.x;
		QOut.z = Q1.w*Q2.z + Q1.x*Q2.y - Q1.y*Q2.x + Q1.z*Q2.w;
		QOut.w = Q1.w*Q2.w - Q1.x*Q2.x - Q1.y*Q2.y - Q1.z*Q2.z;
		return QOut;
	}
#endif

/* ----------------------------------------------------------------------------
 * Seeded/Deterministic random number generation functions
 *
 * This is a variant of NiagaraRand4DPCG32 from Random.ush. 
 *
 * uint is not fully supported in the VM so we simply use ints and drop the 
 * top and bottom bit swap. This should be fine since signed overflow should 
 * produce the same results as unsigned overflow when comparing bit-by-bit on 
 * all relevant architectures.
 *
 * Warning: Only contains 24 bits of randomness, since we produce values in 
 *          the unit interval. Uses the upper 24 bits, as they have the best
 *          quality.
 *
 *          By removing the bit swaps in NiagaraRand4DPCG32 we save a few 
 *          operations, but lose a bit of statistical (but not visual) quality, 
 *          and for our use case this is an acceptable compromise.
 * ----------------------------------------------------------------------------
 */

// Returns 4 random normalized floats based on 4 explicit integer seeds
float4 rand4(int Seed1, int Seed2, int Seed3, int Seed4)
{
	int4 v = int4(Seed4, Seed1, Seed2, Seed3) * 1664525 + 1013904223;

	v.x += v.y*v.w;
	v.y += v.z*v.x;
	v.z += v.x*v.y;
	v.w += v.y*v.z;
	v.x += v.y*v.w;
	v.y += v.z*v.x;
	v.z += v.x*v.y;
	v.w += v.y*v.z;

	// We can use 24 bits of randomness, as all integers in [0, 2^24] 
	// are exactly representable in single precision floats.
	// We use the upper 24 bits as they tend to be higher quality.

	// The divide is often folded with the range scale in the rand functions
	return float4((v >> 8) & 0x00ffffff) / 16777216.0; // 0x01000000 == 16777216
	// return float4((v >> 8) & 0x00ffffff) * (1.0/16777216.0); // bugged, see UE-67738
}

// float3 specialization of the above:
// 
// Returns 3 random normalized floats based on 4 explicit integer seeds.
// 
// All bits of the first and second seeds are used, while only 
// the lower 16 bits of the third and fourth seeds are used.
float3 rand3(int Seed1, int Seed2, int Seed3, int Seed4)
{
	int3 v = int3(Seed1, Seed2, Seed4 | (Seed3 << 16)) * 1664525 + 1013904223;

	v.x += v.y*v.z;
	v.y += v.z*v.x;
	v.z += v.x*v.y;
	v.x += v.y*v.z;
	v.y += v.z*v.x;
	v.z += v.x*v.y;

	return float3((v >> 8) & 0x00ffffff) / 16777216.0; // 0x01000000 == 16777216
}

// Internal counter used to generate a different sequence of random numbers for each call
static int RandomCounterDeterministic = 0;

// Cost using rand4: 6 imad, 1 itof, 1 ishr, 1 add, 2 mul
float rand(float x, int Seed1, int Seed2, int Seed3)
{
	RandomCounterDeterministic += 1;
	return rand3(Seed1, Seed2, Seed3, RandomCounterDeterministic).x * x;
}

// Cost using rand4: 7 imad, 1 itof, 1 ishr, 1 add, 2 mul
float2 rand(float2 x, int Seed1, int Seed2, int Seed3)
{
	RandomCounterDeterministic += 1;
	return rand3(Seed1, Seed2, Seed3, RandomCounterDeterministic).xy * x;
}

// Cost using rand4: 8 imad, 1 itof, 1 ishr, 1 add, 2 mul
float3 rand(float3 x, int Seed1, int Seed2, int Seed3)
{
	RandomCounterDeterministic += 1;
	return rand3(Seed1, Seed2, Seed3, RandomCounterDeterministic).xyz * x;
}

// Cost using rand4: 9 imad, 1 itof, 1 ishr, 1 and, 2 mul
float4 rand(float4 x, int Seed1, int Seed2, int Seed3) 
{
	RandomCounterDeterministic += 1;
	return rand4(Seed1, Seed2, Seed3, RandomCounterDeterministic).xyzw * x;
}

// Cost using rand4: 6 imad, 2 itof, 1 ishr, 1 add, 2 mul, 1 ftoi
int rand(int x, int Seed1, int Seed2, int Seed3)
{
	// Scaling a uniform float range provides better distribution of numbers than using %.
	// Inclusive! So [0, x] instead of [0, x)
	RandomCounterDeterministic += 1;
	return int(rand3(Seed1, Seed2, Seed3, RandomCounterDeterministic).x * (x+1));
}

/* -----------------------------------------------------------------
 * Un-seeded/Non-deterministic random number generation functions
 * -----------------------------------------------------------------
 */
#if GPU_SIMULATION
	// This simply calls the deterministic random number functions from the Seeded RNG section, 
	// but uses non-deterministic seeds as input. 

	// This could perhaps be optimized by using slightly cheaper functions, but the difference is likely negligible. 
	
	// Internal counter used to generate a different sequence of random numbers for each call
	// We need to keep this separate from the Deterministic version so that non-deterministic 
	// calls do not interfere with the deterministic ones. 
	static int RandomCounterNonDeterministic = -1;

	float rand(float x)
	{
		RandomCounterNonDeterministic -= 1;
		return rand4(GLinearThreadId, EmitterTickCounter, GLinearThreadId, RandomCounterNonDeterministic).x * x;
	}

	float2 rand(float2 x)
	{
		RandomCounterNonDeterministic -= 1;
		return rand4(GLinearThreadId, EmitterTickCounter, GLinearThreadId, RandomCounterNonDeterministic).xy * x;
	}

	float3 rand(float3 x)
	{
		RandomCounterNonDeterministic -= 1;
		return rand4(GLinearThreadId, EmitterTickCounter, GLinearThreadId, RandomCounterNonDeterministic).xyz * x;
	}

	float4 rand(float4 x) 
	{
		RandomCounterNonDeterministic -= 1;
		return rand4(GLinearThreadId, EmitterTickCounter, GLinearThreadId, RandomCounterNonDeterministic).xyzw * x;
	}

	// Integer randoms are INCLUSIVE, i.e. includes both the upper and lower limits
	int rand(int x)
	{
		RandomCounterNonDeterministic -= 1;
		return int(rand4(GLinearThreadId, EmitterTickCounter, GLinearThreadId, RandomCounterNonDeterministic).x * (x+1));
	}
#else
	// Old unseeded, passthrough to FRandomStream

	float rand(float x); // Invokes EVectorVMOp::random

	float2 rand(float2 x)
	{
		return float2(rand(x.x), rand(x.y));
	}

	float3 rand(float3 x)
	{
		return float3(rand(x.x), rand(x.y), rand(x.z));
	}

	float4 rand(float4 x) 
	{
		return float4(rand(x.x), rand(x.y), rand(x.z), rand(x.w));
	}
	
	int rand(int x); // Invokes EVectorVMOp::randomi and is semi-open. This is inconsistent with the rest of the functions above. As a result this function and the ones above should be deprecated in favor of the functions below
#endif

// Small changes in the input bits should propagate to a lot of output bits, so the resulting hash is not periodic.
// This is important because the hash inputs are often things like particle ID, but the output should be pseudo-random. 
int hash_single(int a)
{
    int x = (a ^ 61) ^ (a >> 16);
	x += x << 3;
	x ^= x >> 4;
	x *= 0x27d4eb2d;
	x ^= x >> 15;
	return x;
}

int hash(int a, int b)
{
	return hash_single(a) ^ hash_single(b * 31);
}

float hash_float(int a, int b)
{
	return (hash(a, b) & 0x00ffffff) / 16777216.0;
}

// this is used when chaining calls from variable number of inputs, e.g. hash_float(hash_float(a, b), c)
float hash_float(float a, int b)
{
	return (hash(a * 16777216.0, b) & 0x00ffffff) / 16777216.0;
}

// Explicit non-deterministic random overrides used by Random Float/Integer and Seeded Random Float/Integer op nodes
float rand_float(float x)
{
	return rand(x.x);
}

float2 rand_float(float2 x)
{
	return float2(rand_float(x.x), rand_float(x.y));
}

float3 rand_float(float3 x)
{
	return float3(rand_float(x.x), rand_float(x.y), rand_float(x.z));
}

float4 rand_float(float4 x) 
{
	return float4(rand_float(x.x), rand_float(x.y), rand_float(x.z), rand_float(x.w));
}

int rand_int(int x)
{
	// Going through the float function also give us a better distribution than using modulo 
	// to get an integer range.
	// This will not include the upper range as rand_float returns [0, max), not [0, max].
	return (int) rand_float(x.x);
}

// Explicit deterministic random overrides used by Random Float/Integer and Seeded Random Float/Integer op nodes
float rand_float(float x, int Seed1, int Seed2, int Seed3)
{
	return rand(x.x, Seed1, Seed2, Seed3);
}

float2 rand_float(float2 x, int Seed1, int Seed2, int Seed3)
{
	return rand(x, Seed1, Seed2, Seed3);
}

float3 rand_float(float3 x, int Seed1, int Seed2, int Seed3)
{
	return rand(x, Seed1, Seed2, Seed3);
}

float4 rand_float(float4 x, int Seed1, int Seed2, int Seed3) 
{
	return rand(x, Seed1, Seed2, Seed3);
}

int rand_int(int x, int Seed1, int Seed2, int Seed3)
{
	// This will not include the upper range as rand_float returns [0, max), not [0, max]
	// The deterministic rand call will include the upper range, so we subtract a one to compensate
	return rand(x.x-1, Seed1, Seed2, Seed3);
}

// used to interpolate rotations in interpolated spawn scripts
float4 NiagaraQuatSLerp(float4 Quat1, float4 Quat2, float Slerp)
{
	const float RawCosom = dot(Quat1, Quat2);
	const float Cosom = abs(RawCosom);
	
	float Scale0, Scale1;
	if (Cosom < 0.9999f)
	{
		const float Omega = acos(Cosom);
		const float InvSin = 1.f / sin(Omega);
		Scale0 = sin((1.f - Slerp) * Omega) * InvSin;
		Scale1 = sin(Slerp * Omega) * InvSin;
	}
	else
	{
		Scale0 = 1.0f - Slerp;
		Scale1 = Slerp;
	}

	Scale1 = RawCosom >= 0.0f ? Scale1 : -Scale1;

	return (Scale0 * Quat1) + (Scale1 * Quat2);
}

/* -----------------------------------------------------------------
 * VM simulation function declarations
 * -----------------------------------------------------------------
 */
#if VM_SIMULATION
	float noise(float x);
	float noise(float2 x);
	float noise(float3 x);

	//Total hack to get around the cross compiler converting fmod() to "X - (Y * trunc(X/Y))";
	//On gpu just define these as fmod(x,y)
	float4 Modulo(float4 x, float4 y);
	float3 Modulo(float3 x, float3 y);
	float2 Modulo(float2 x, float2 y);
	float Modulo(float x, float y);

	/** Returns the index for this particle in the current execution context. On gpu this'll likely be derived from DispatchThreadId */
	int ExecIndex();

	//Some functions that we use to map to special VM operations for reading in data.
	//TODO: replace with proper buffer reading capability and use standard hlsl.
	int AcquireIndex(int DataSetID, bool DoAcquire);

	void AcquireID(int DataSetID, out int IDIndex, out int IDAcquireTag);
	void UpdateID(int DataSetID, int IDIndex, int InstanceIndex);

	float InputDataFloat(int DataSetIndex, int RegisterIdx);  //DataSetIndex is 0 for main dataset
	int InputDataInt(int DataSetIndex, int RegisterIdx);
	bool InputDataBool(int DataSetIndex, int RegisterIdx);
	float InputDataHalf(int DataSetIndex, int RegisterIdx);

	float InputDataNoadvanceFloat(int DataSetIndex, int RegisterIdx);  //DataSetIndex is 0 for main dataset
	int InputDataNoadvanceInt(int DataSetIndex, int RegisterIdx);
	bool InputDataNoadvanceBool(int DataSetIndex, int RegisterIdx);

	void OutputDataFloat(int DataSetIndex, int RegisterIndex, int InstanceIndex, float Value);
	void OutputDataInt(int DataSetIndex, int RegisterIndex, int InstanceIndex, int Value);
	void OutputDataBool(int DataSetIndex, int RegisterIndex, int InstanceIndex, bool Value);
	void OutputDataHalf(int DataSetIndex, int RegisterIndex, int InstanceIndex, float Value);
#endif




/* -----------------------------------------------------------------
 * GPU simulation code
 * -----------------------------------------------------------------
 */
#if GPU_SIMULATION
	uint ComponentBufferSizeRead;
	uint ComponentBufferSizeWrite;
	uint SimStart;

	/* Buffers for particle data and DrawIndirect calls
	 */
	#if NIAGARA_PARTICLE_PARTIAL_ENABLED == 0
		Buffer<float> InputFloat;
		Buffer<int> InputInt;
		Buffer<half> InputHalf;
	#endif
	RWBuffer<int> RWOutputInt;
	RWBuffer<float> RWOutputFloat;
	RWBuffer<half> RWOutputHalf;

	Buffer<float> StaticInputFloat;

	RWBuffer<uint> RWInstanceCounts;
	uint ReadInstanceCountOffset;
	uint WriteInstanceCountOffset;

	Buffer<int> FreeIDList;
	RWBuffer<int> RWIDToIndexTable;
	
	// X = Count Buffer Instance Count Offset (INDEX_NONE == Use Instance Count)
	// Y = Instance Count
	// Z = Iteration Num | Index
	// W = Loop Num | Index
	uint4 SimulationStageIterationInfo;

	// Note: These are referenced from an asset that passes back the data to the user (see SimulationStageIterationInfo)
	#if NIAGARA_DISPATCH_INDIRECT
		int SimulationStage_GetInstanceCount() { return DispatchThreadIdBounds.x * DispatchThreadIdBounds.y * DispatchThreadIdBounds.z; }
	#else
		int SimulationStage_GetInstanceCount() { return SimulationStageIterationInfo.x == uint(-1) ? int(SimulationStageIterationInfo.y) : int(RWInstanceCounts[SimulationStageIterationInfo.x]); }
	#endif
	int SimulationStage_GetNumIterations()  { return int((SimulationStageIterationInfo.z >> 16) & 0xffff); }
	int SimulationStage_GetIterationIndex() { return int((SimulationStageIterationInfo.z >> 0 ) & 0xffff); }
	int SimulationStage_GetNumLoops()  { return int((SimulationStageIterationInfo.w >> 16) & 0xffff); }
	int SimulationStage_GetLoopIndex() { return int((SimulationStageIterationInfo.w >> 0 ) & 0xffff); }

	float SimulationStage_GetNormalizedIterationIndex() { return float(SimulationStage_GetIterationIndex()) / float(max(SimulationStage_GetNumIterations(), 1) - 1); }
	float SimulationStage_GetNormalizedLoopIndex() { return float(SimulationStage_GetLoopIndex()) / float(max(SimulationStage_GetNumLoops(), 1) - 1); }

	// Where X = Parameter Binding, YZ = Inclusive Range
	uint3 ParticleIterationStateInfo;

	void AcquireID(int DataSetID, out int IDIndex, out int IDAcquireTag)
	{
		// Begin static assert : GPU particles only support DataSetID 0
		int MustBe0[1];
		MustBe0[DataSetID] = 0;
		// End static assert

		// This is the same as ExecIndex() right now, but that function may change in the future to accommodate multiple
		// spawn infos. Revisit this computation if the change affects the meaning of GSpawnStartInstance.
		int SpawnIndex = GLinearThreadId - GSpawnStartInstance;
		IDIndex = FreeIDList[SpawnIndex];
		IDAcquireTag = EmitterTickCounter;
	}

 	void UpdateID(int DataSetID, int IDIndex, int InstanceIndex)
	{
		// Begin static assert : GPU particles only support DataSetID 0
		int MustBe0[1];
		MustBe0[DataSetID] = 0;
		// End static assert

		RWIDToIndexTable[IDIndex] = InstanceIndex;
	}

#define USE_GROUP_SHARED ((THREADGROUP_SIZE == 64 || THREADGROUP_SIZE == 32) && !USE_WAVE_INTRINSICS && !VULKAN_PROFILE_SM5 && !VULKAN_PROFILE_SM6 && !VULKAN_PROFILE)

#if USE_GROUP_SHARED
	#if THREADGROUP_SIZE == 64
		groupshared uint GroupSharedIndex[64];
		groupshared uint GroupSharedIndex4[16];
		groupshared uint GroupSharedIndex16[4];
		groupshared uint GroupSharedIndex64;
	#elif THREADGROUP_SIZE == 32
		groupshared uint GroupSharedIndex[32];
		groupshared uint GroupSharedIndex4[8];
		groupshared uint GroupSharedIndex16[2];
		groupshared uint GroupSharedIndex64;
	#endif
#endif // USE_GROUP_SHARED

	/* Acquire an output index - the default index is the scratch instance; one additional instance is allocated 
	 *	at the end of the buffer, so no branching on -1 is necessary during OutputData operations
	 */
	int AcquireIndex(uniform int DataSetID, bool bDoAcquire)
	{
		// Begin static assert : GPU particles only support DataSetID 0
		int MustBe0[1];
		MustBe0[DataSetID] = 0;
		// End static assert

		int PrevIdx = GSpawnStartInstance + NumSpawnedInstances;	// scratch instance as default; write to that for dead particles

		#if USE_WAVE_INTRINSICS

			uint NumCounters = WaveActiveCountBits(bDoAcquire);
			uint PrefixCounts = WavePrefixCountBits(bDoAcquire);
			if (NumCounters > 0)
			{
				if (WaveIsFirstLane())
				{
					uint RetPrevIdx;
					InterlockedAdd(RWInstanceCounts[WriteInstanceCountOffset], NumCounters, RetPrevIdx);
					PrevIdx = (int)RetPrevIdx;
				}

				if (bDoAcquire)
				{
					PrevIdx = WaveReadLaneFirst(PrevIdx);
					PrevIdx += PrefixCounts;
				}
			}	

		#elif USE_GROUP_SHARED

			GroupSharedIndex[GGroupThreadId.x] = bDoAcquire ? 1 : 0;
			GroupMemoryBarrierWithGroupSync(); 

			// Group by 4
			if ((GGroupThreadId.x & 0x3) == 0)
			{
				const uint Index = GGroupThreadId.x;

				const uint ActiveCount1 = GroupSharedIndex[Index];
				const uint ActiveCount2 = ActiveCount1 + GroupSharedIndex[Index + 1];
				const uint ActiveCount3 = ActiveCount2 + GroupSharedIndex[Index + 2];
				const uint ActiveCount4 = ActiveCount3 + GroupSharedIndex[Index + 3];
				
				GroupSharedIndex[Index] = 0;
				GroupSharedIndex[Index + 1] = ActiveCount1;
				GroupSharedIndex[Index + 2] = ActiveCount2;
				GroupSharedIndex[Index + 3] = ActiveCount3;
				GroupSharedIndex4[Index / 4] = ActiveCount4;
			}
			GroupMemoryBarrierWithGroupSync(); 

			// Group by 16
			if ((GGroupThreadId.x & 0xF) == 0)
			{
				const uint Index = GGroupThreadId.x / 4;

				const uint ActiveCount1 = GroupSharedIndex4[Index];
				const uint ActiveCount2 = ActiveCount1 + GroupSharedIndex4[Index + 1];
				const uint ActiveCount3 = ActiveCount2 + GroupSharedIndex4[Index + 2];
				const uint ActiveCount4 = ActiveCount3 + GroupSharedIndex4[Index + 3];

				GroupSharedIndex4[Index] = 0;
				GroupSharedIndex4[Index + 1] = ActiveCount1;
				GroupSharedIndex4[Index + 2] = ActiveCount2;
				GroupSharedIndex4[Index + 3] = ActiveCount3;
				GroupSharedIndex16[Index / 4] = ActiveCount4;
			}
			GroupMemoryBarrierWithGroupSync(); 

			// Group by 64
			if ((GGroupThreadId.x & 0x3F) == 0)
			{
				const uint Index = GGroupThreadId.x / 16;
				uint RetPrevIdx = 0;
		
				const uint ActiveCount1 = GroupSharedIndex16[Index];
				const uint ActiveCount2 = ActiveCount1 + GroupSharedIndex16[Index + 1];
		#if THREADGROUP_SIZE == 64
				const uint ActiveCount3 = ActiveCount2 + GroupSharedIndex16[Index + 2];
				const uint ActiveCount4 = ActiveCount3 + GroupSharedIndex16[Index + 3];
		#endif

				GroupSharedIndex16[Index] = 0;
				GroupSharedIndex16[Index + 1] = ActiveCount1;
		#if THREADGROUP_SIZE == 64
				GroupSharedIndex16[Index + 2] = ActiveCount2;
				GroupSharedIndex16[Index + 3] = ActiveCount3;

				InterlockedAdd(RWInstanceCounts[WriteInstanceCountOffset], ActiveCount4, RetPrevIdx);
		#elif THREADGROUP_SIZE == 32
				InterlockedAdd(RWInstanceCounts[WriteInstanceCountOffset], ActiveCount2, RetPrevIdx);
		#endif
				GroupSharedIndex64 = RetPrevIdx;
			}
			GroupMemoryBarrierWithGroupSync(); 

			PrevIdx = GroupSharedIndex64 + GroupSharedIndex16[GGroupThreadId.x / 16] + GroupSharedIndex4[GGroupThreadId.x / 4] + GroupSharedIndex[GGroupThreadId.x];

		#else // !USE_WAVE_INTRINSICS && !USE_GROUP_SHARED

			if(bDoAcquire == true)
			{
				// Have to use uint's here to avoid PS4 compiler warnings about InterlockedAdd, cannot propagate uint due to CPU VM limitations...
				uint RetPrevIdx;
				// @TODO : add some TLS logic to avoid thread group for doing atomic for each thread. (gathering the actual required count)
				InterlockedAdd(RWInstanceCounts[WriteInstanceCountOffset], (uint)1U, RetPrevIdx);
				PrevIdx = (int)RetPrevIdx;
			}

		#endif // USE_WAVE_INTRISICS || USE_GROUP_SHARED

		return PrevIdx;
	}

	/* ---------------------------------------------------------------------
	 * InputData operations 
	 * ---------------------------------------------------------------------
	 */
	float InputDataFloat(int DataSetIndex, int RegisterIdx, int InstanceIdx)
	{
		#if NIAGARA_PARTICLE_PARTIAL_ENABLED
			return RWOutputFloat[RegisterIdx*ComponentBufferSizeWrite + InstanceIdx];
		#else
			return InputFloat[RegisterIdx*ComponentBufferSizeRead + InstanceIdx];
		#endif
	}

	int InputDataInt(int DataSetIndex, int RegisterIdx, int InstanceIdx)
	{
		#if NIAGARA_PARTICLE_PARTIAL_ENABLED
			return RWOutputInt[RegisterIdx*ComponentBufferSizeWrite + InstanceIdx];
		#else
			return InputInt[RegisterIdx*ComponentBufferSizeRead + InstanceIdx];
		#endif
	}
	
	bool InputDataBool(int DataSetIndex, int RegisterIdx, int InstanceIdx)
	{
		#if NIAGARA_PARTICLE_PARTIAL_ENABLED
			return RWOutputInt[RegisterIdx*ComponentBufferSizeWrite + InstanceIdx] == -1;
		#else
			return InputInt[RegisterIdx*ComponentBufferSizeRead + InstanceIdx] == -1;
		#endif
	}

	float InputDataHalf(int DataSetIndex, int RegisterIdx, int InstanceIdx)
	{
		#if NIAGARA_PARTICLE_PARTIAL_ENABLED
			return RWOutputHalf[RegisterIdx*ComponentBufferSizeWrite + InstanceIdx];
		#else
			return InputHalf[RegisterIdx*ComponentBufferSizeRead + InstanceIdx];
		#endif
	}

	/* ---------------------------------------------------------------------
	 * OutputData operations 
	 * ---------------------------------------------------------------------
	 */
	void OutputDataFloat(int DataSetIndex, int RegisterIndex, int InstanceIndex, float Value)
	{
		RWOutputFloat[RegisterIndex*ComponentBufferSizeWrite + InstanceIndex] = Value;
	}

	void OutputDataInt(int DataSetIndex, int RegisterIndex, int InstanceIndex, int Value)
	{
		RWOutputInt[RegisterIndex*ComponentBufferSizeWrite + InstanceIndex] = Value;
	}

	void OutputDataBool(int DataSetIndex, int RegisterIndex, int InstanceIndex, bool Value)
	{
		RWOutputInt[RegisterIndex*ComponentBufferSizeWrite + InstanceIndex] = Value ? -1 : 0;
	}

	void OutputDataHalf(int DataSetIndex, int RegisterIndex, int InstanceIndex, float Value)
	{
		RWOutputHalf[RegisterIndex*ComponentBufferSizeWrite + InstanceIndex] = Value;
	}

	void EnterStatScope(int ID)	{}
	void ExitStatScope()	{}
#endif // GPU_SIMULATION

	/* 
	 * Get the index to write onto the output buffer
	 */
	int OutputIndex(const int DataSetID, const bool bStageKillsParticles, const bool bIsValid)
	{
	#if GPU_SIMULATION
		// If this stage cannot kill particles, we can just write them out in the same order as they
		// appear in the input. We must use an if here (as opposed to a ternary operator, or some
		// other branchless construct), because we don't want to call AcquireIndex() at all, since
		// that manipulates the RWInstanceCounts UAV. The generated code will copy the source count
		// at the end of the shader.
		if (!bStageKillsParticles)
		{
			return GLinearThreadId;
		}
	#endif

		return AcquireIndex(DataSetID, bIsValid);
	}

////////////////////////////////////////////////////////////////////////////////////
// Random number functions
struct NiagaraRandInfo
{
	int Seed1;
	int Seed2;
	int Seed3;
};

#if GPU_SIMULATION
NiagaraRandInfo MakeRandInfo()
{
	GRandomSeedOffset += 1664525u;

	NiagaraRandInfo RandInfo;
	RandInfo.Seed1 = GLinearThreadId;
	RandInfo.Seed2 = GRandomSeedOffset;
	RandInfo.Seed3 = GEmitterTickCounter;
	return RandInfo;
}

float NiagaraRandomFloat(NiagaraRandInfo RandInfo)
{
	return uint(RandInfo.Seed3) == 0xffffffff ? NiagaraInternalNoise(uint(RandInfo.Seed1), uint(RandInfo.Seed2), uint(RandInfo.Seed3)) : rand(1.0f, uint(RandInfo.Seed1), uint(RandInfo.Seed2), uint(RandInfo.Seed3));
}

int NiagaraRandomInt(NiagaraRandInfo RandInfo, int Range)
{
	float T = NiagaraRandomFloat(RandInfo);
	return int(floor(float(Range) * T));
}

float3 NiagaraRandomBaryCoord(NiagaraRandInfo RandInfo)
{
	float2 r = float2(NiagaraRandomFloat(RandInfo), NiagaraRandomFloat(RandInfo));
	float sqrt0 = sqrt(r.x);
	float sqrt1 = sqrt(r.y);
	return float3(1.0f - sqrt0, sqrt0 * (1.0 - r.y), r.y * sqrt0);
}
#endif

////////////////////////////////////////////////////////////////////////////////////
//Include the simulation shader code generated by the node graph.
#include "/Engine/Generated/NiagaraEmitterInstance.ush"
