// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

RWBuffer<int>	BufferUAV;
uint			NumElements;
int				FillValue;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void FillIntBuffer(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint Index = DispatchThreadId.x;
	if(Index < NumElements)
	{
		BufferUAV[Index] = FillValue;
	}
}
