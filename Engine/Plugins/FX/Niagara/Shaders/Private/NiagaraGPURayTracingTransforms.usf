// Copyright Epic Games, Inc. All Rights Reserved.

/*=================================================================================================
	NiagaraGPURayTracingTransforms.usf: Shader to pass GPU transforms to the ray tracing world
===================================================================================================*/

#include "/Engine/Private/Common.ush"
#include "NiagaraCommon.ush"

//////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Parameters

uint	ParticleDataFloatStride;
//uint	ParticleDataHalfStride;
uint	ParticleDataIntStride;
uint	CPUNumInstances;
uint	InstanceCountOffset;

float3	SystemLWCTile;

uint	PositionDataOffset;
uint	RotationDataOffset;
uint	ScaleDataOffset;
uint	bLocalSpace;
uint	RenderVisibilityOffset;
uint	MeshIndexOffset;

uint	RenderVisibilityValue;
uint	MeshIndexValue;

float3	DefaultPosition;
float4	DefaultRotation;
float3	DefaultScale;
float3	MeshScale;
float4	MeshRotation;

float4x4	LocalTransform;
float3		LocalTransformTile;

Buffer<float>	ParticleDataFloatBuffer;
//Buffer<half>	ParticleDataHalfBuffer;
Buffer<int>		ParticleDataIntBuffer;
Buffer<uint>	GPUInstanceCountBuffer;

float3 ViewTilePosition;
float3 RelativePreViewTranslation;

// Transforms are float3x4 row-major matrices stored as 3 float4
// because of FXC matrix packing issues when using StructuredBuffer<float3x4>
RWStructuredBuffer<float4> TLASTransforms;

//////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Helper function for buffers

float GetFloat(int RegisterIdx, uint InstanceID)
{
	RegisterIdx &= (~(1u << 31));
	return ParticleDataFloatBuffer[(RegisterIdx * ParticleDataFloatStride + InstanceID)];
}

float GetInt(int RegisterIdx, uint InstanceID)
{
	return ParticleDataIntBuffer[(RegisterIdx * ParticleDataIntStride + InstanceID)];
}

float3 SafeGetVec3(int RegisterIndex, uint InstanceID, float3 DefaultValue)
{
	return RegisterIndex == -1 ? DefaultValue : float3(GetFloat(RegisterIndex, InstanceID), GetFloat(RegisterIndex+1, InstanceID), GetFloat(RegisterIndex+2, InstanceID));
}

float4 SafeGetVec4(int RegisterIndex, uint InstanceID, float4 DefaultValue)
{
	return RegisterIndex == -1 ? DefaultValue : float4(GetFloat(RegisterIndex, InstanceID), GetFloat(RegisterIndex+1, InstanceID), GetFloat(RegisterIndex+2, InstanceID), GetFloat(RegisterIndex+3, InstanceID));
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////

#ifndef THREADGROUP_SIZE
#define THREADGROUP_SIZE 1
#endif

// TODO: This doesn't take a lot of features into account that are needed to solve for a Niagara mesh's transform.
// See NiagaraCalculateMeshParticleTransforms in NiagaraMeshParticleUtils.ush for exactly what all goes into it.
// In an upcoming update, this shader should be deprecated and instead, RayTracing should be getting transforms
// from GPU Scene so this work doesn't have to be duplicated.
[numthreads(THREADGROUP_SIZE,1,1)]
void NiagaraGPURayTracingTransformsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint InstanceIndex = DispatchThreadId.x;
	if ( InstanceIndex >= CPUNumInstances)
	{
		return;
	}

	const uint GPUNumInstances = InstanceCountOffset != -1 ? GPUInstanceCountBuffer[InstanceCountOffset] : 0;

	float4x4 InstanceTransformTransposed = 0;
	if (InstanceIndex < GPUNumInstances)
	{
		float3 InstancePosition	= SafeGetVec3(PositionDataOffset, InstanceIndex, DefaultPosition);
		float4 InstanceRotation	= NiagaraQuatMul(SafeGetVec4(RotationDataOffset, InstanceIndex, DefaultRotation), MeshRotation);
		float3 InstanceScale	= SafeGetVec3(ScaleDataOffset, InstanceIndex, DefaultScale) * MeshScale;

		FLWCMatrix InstanceTransform;
		if (bLocalSpace)
		{
			float4x4 TempTrans = NiagaraComposeTransformMatrix(InstanceScale, NiagaraQuatTo3x3(InstanceRotation), InstancePosition);
			TempTrans = mul(TempTrans, LocalTransform);
			InstanceTransform = MakeLWCMatrix(LocalTransformTile, TempTrans);
		}
		else
		{
			InstanceTransform = NiagaraComposeTransformMatrix(InstanceScale, NiagaraQuatTo3x3(InstanceRotation), MakeLWCVector3(SystemLWCTile, InstancePosition));
		}


		uint bInstanceValid = 1;
		if ( RenderVisibilityOffset != -1 )
		{
			bInstanceValid &= GetInt(RenderVisibilityOffset, InstanceIndex) == RenderVisibilityValue;
		}

		if ( MeshIndexOffset != -1 )
		{
			bInstanceValid &= GetInt(MeshIndexOffset, InstanceIndex) == MeshIndexValue;
		}
		else
		{
			bInstanceValid &= MeshIndexValue == 0;
		}

		if ( bInstanceValid )
		{
			FLWCVector3 PreViewTranslation = MakeLWCVector3(ViewTilePosition, RelativePreViewTranslation);
			float3 MatrixTrans = LWCToFloat(LWCAdd(LWCGetOrigin(InstanceTransform), PreViewTranslation));
			float3x3 MatrixRot = transpose(LWCToFloat3x3(InstanceTransform));

			InstanceTransformTransposed[0].xyz = MatrixRot[0];
			InstanceTransformTransposed[1].xyz = MatrixRot[1];
			InstanceTransformTransposed[2].xyz = MatrixRot[2];
			InstanceTransformTransposed[0].w = MatrixTrans.x;
			InstanceTransformTransposed[1].w = MatrixTrans.y;
			InstanceTransformTransposed[2].w = MatrixTrans.z;
		}
	}
	TLASTransforms[InstanceIndex * 3 + 0] = InstanceTransformTransposed[0];
	TLASTransforms[InstanceIndex * 3 + 1] = InstanceTransformTransposed[1];
	TLASTransforms[InstanceIndex * 3 + 2] = InstanceTransformTransposed[2];
}
