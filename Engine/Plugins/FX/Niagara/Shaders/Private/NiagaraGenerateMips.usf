// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"

float2 SrcTexelSize;
float2 DstTexelSize;

Texture2D MipInSRV;
SamplerState MipInSampler;

int2 MipOutSize;
RWTexture2D<float4> MipOutUAV;

int KernelHWidth;

[numthreads(THREADGROUPSIZE, THREADGROUPSIZE, 1)]
void MainCS(uint3 ThreadID : SV_DispatchThreadID)
{
	if ( any(int2(ThreadID.xy) >= MipOutSize) )
	{
		return;
	}

	float4 OutColor = 0;
	float2 UV = (float2(ThreadID.xy) + 0.5f) * DstTexelSize;
#if GENMIPS_GAUSSIAN
	{
		float TotalWeight = 0.0f;
		float s = KernelHWidth;

		//-OPT: If we still want single pass use groupshared to reduce taps, or go two pass instead (which would add more barriers)
		//-OPT: Consider moving the Offsets / Weights out
		for ( int y=-KernelHWidth; y <= KernelHWidth; ++y )
		{
			for ( int x=-KernelHWidth; x <= KernelHWidth; ++x )
			{
				float2 Offset = float2(x, y);
				float r = sqrt(Offset.x*Offset.x + Offset.y*Offset.y);
				float Weight = exp(-(r*r) / s) / (3.14f * s);
				OutColor += MipInSRV.SampleLevel(MipInSampler, UV + (Offset * SrcTexelSize * 2), 0) * Weight;
				TotalWeight += Weight;
			}
		}
		OutColor /= TotalWeight;
	}
#else
	{
		OutColor = MipInSRV.SampleLevel(MipInSampler, UV, 0);
	}
#endif

#if GENMIPS_SRGB
    OutColor.rgb = LinearToSrgb(OutColor.rgb);
#endif

    MipOutUAV[ThreadID.xy] = OutColor;
}
