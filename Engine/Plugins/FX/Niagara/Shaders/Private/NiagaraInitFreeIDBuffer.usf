// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

RWBuffer<int>	RWNewBuffer;
Buffer<int>		ExistingBuffer;
uint			NumNewElements;
uint			NumExistingElements;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void InitIDBufferCS(uint Index : SV_DispatchThreadID)
{
	if(Index.x < NumNewElements)
	{
		// Place new IDs at the start of the buffer, so we don't have to worry about how many
		// elements are actually in use in the existing buffer.
		RWNewBuffer[Index.x] = NumExistingElements + Index.x;
	}
	else
	{
		// Copy existing IDs after the new IDs.
		RWNewBuffer[Index.x] = ExistingBuffer[Index.x - NumNewElements];
	}
}
