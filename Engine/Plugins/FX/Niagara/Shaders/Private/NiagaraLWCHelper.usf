// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ComputeShaderUtils.ush"

RWBuffer<float>	FloatBuffer;
uint			FloatBufferStride;

Buffer<int>		CountBuffer;
uint			CountBufferOffset;

uint			NumPositions;
DECLARE_SCALAR_ARRAY(uint,	PositionComponentOffsets, MaxPositions);
float3			TileShift;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void TileShiftPositions(uint3 GroupId : SV_GroupID, int GroupThreadIndex : SV_GroupIndex)
{
	const uint LinearThreadId = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, THREADGROUP_SIZE);
	const uint NumInstances = uint(CountBuffer[CountBufferOffset]);	
	if ( LinearThreadId >= NumInstances )
	{
		return;
	}
	
	for ( uint iPosition=0; iPosition < NumPositions; ++iPosition )
	{
		const uint XOffset = LinearThreadId + (GET_SCALAR_ARRAY_ELEMENT(PositionComponentOffsets, iPosition) * FloatBufferStride);
		const uint YOffset = XOffset + FloatBufferStride;
		const uint ZOffset = YOffset + FloatBufferStride;
		FloatBuffer[XOffset] = FloatBuffer[XOffset] + TileShift.x;
		FloatBuffer[YOffset] = FloatBuffer[YOffset] + TileShift.y;
		FloatBuffer[ZOffset] = FloatBuffer[ZOffset] + TileShift.z;
	}
}
