// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// NiagaraMeshParticleUtils
// ------------------------
// This header provides types and methods needed to compute final transformations of particle meshes for the Niagara mesh renderer.
// NOTE: Due to the inclusion of NiagaraParticleAccess.ush below, its requisite methods for accessing particle data buffers are
// expected to be defined before including this file.

#include "/Plugin/FX/Niagara/Private/NiagaraCommon.ush"
#include "/Plugin/FX/Niagara/Private/NiagaraParticleAccess.ush"

// Mesh facing enum values
#define MESH_FACING_DEFAULT 0 			// No facing
#define MESH_FACING_VELOCITY 1 			// Face velocity
#define MESH_FACING_CAMERA_POSITION 2 	// Face camera position
#define MESH_FACING_CAMERA_PLANE 3 		// Face camera plane

// Particle space enum values
#define PARTICLE_SPACE_SIMULATION 0 	// Simulation space (Local when local-space emitter, world otherwise)
#define PARTICLE_SPACE_WORLD 1 			// World space
#define PARTICLE_SPACE_LOCAL 2 			// Local space (to Primitive)

// Decomposed scale, rotation, and translation of a mesh particle
struct NiagaraMeshParticleSRT
{
	float3 Scale;
	float3x3 Rotation;
	FLWCVector3 Translation;
};

// Transform data for a single mesh particle, used for rendering and culling
struct NiagaraMeshParticleTransforms
{
	FLWCMatrix LocalToWorld;
	FLWCInverseMatrix WorldToLocal;
	FLWCMatrix PrevLocalToWorld;
	float3x3 LocalToWorldNoScale;
};

// Parameters needed to compute the NiagaraMeshParticleSRT of a mesh particle
struct NiagaraMeshParticleSRTParams
{
	// Index of particle, used to fetch from the Niagara data buffer
	uint ParticleIndex;
	// The render tile of the system for Large World Coordinates
	float3 SystemLWCTile;
	// Whether or not the Niagara emitter is in local space.
	bool bLocalSpace;
	// The mesh particle facing mode (see MESH_FACING_* values above)
	uint FacingMode;

	// Additional mesh scale and offset to apply to mesh
	float3 MeshScale;
	float4 MeshRotation;
	float3 MeshOffset;
	bool bMeshOffsetIsWorldSpace;

	// Locked axis options
	bool bLockedAxisEnable;
	float3 LockedAxis;
	uint LockedAxisSpace;

	// Offsets for accessing transform information from particle data
	int ScaleDataOffset;
	int RotationDataOffset;
	int PositionDataOffset;
	int CameraOffsetDataOffset;
	
	// Defaults for the above data when no offset provided
	float3 DefaultScale;
	float4 DefaultRotation;
	float3 DefaultPosition;
	float DefaultCameraOffset;

	// Velocity unit direction (XYZ), and velocity magnitude (W) of particle (see NiagaraGetVelocityDirMag)
	float4 VelocityDirMag;

	// Camera basis vectors (in world space)
	FLWCVector3 CameraOrigin;
	float3 CameraForwardDir;
	float3 CameraUpDir;

	// Primitive transform data
	FLWCMatrix PrimitiveLocalToWorld;
	float3 PrimitiveInvNonUniformScale;
};

// Parameters needed to compute the final transforms (current, previous, and inverse) of a mesh particle
struct NiagaraMeshParticleTransformsParams
{
	// Index of particle, used to fetch from the Niagara data buffer
	uint ParticleIndex;
	// The render tile of the system for Large World Coordinates
	float3 SystemLWCTile;
	// If true, specifies that the Niagara emitter is in local space
	bool bLocalSpace;
	// If true, will perform more expensive calculation to determine previous transform
	bool bPreciseMotionVectors;
	// The mesh particle facing mode (see MESH_FACING_* values above)
	uint FacingMode;
	// Elapsed seconds this frame
	float DeltaSeconds;

	// Additional scale and offset to apply to mesh
	float3 MeshScale;
	float4 MeshRotation;
	float3 MeshOffset;
	bool bMeshOffsetIsWorldSpace;

	// Locked axis options
	bool bLockedAxisEnable;
	float3 LockedAxis;
	uint LockedAxisSpace;

	// Offsets for accessing transform information from particle data
	int ScaleDataOffset;
	int RotationDataOffset;
	int PositionDataOffset;
	int CameraOffsetDataOffset;
	int PrevScaleDataOffset;
	int PrevRotationDataOffset;
	int PrevPositionDataOffset;
	int PrevCameraOffsetDataOffset;
	
	// Defaults for the above data when no offset provided
	float3 DefaultScale;
	float4 DefaultRotation;
	float3 DefaultPosition;
	float DefaultCameraOffset;
	float3 DefaultPrevScale;
	float4 DefaultPrevRotation;
	float3 DefaultPrevPosition;
	float DefaultPrevCameraOffset;

	// Velocity unit direction (XYZ), and velocity magnitude (W) of particle (see NiagaraGetVelocityDirMag)
	float4 VelocityDirMag;
	float4 PrevVelocityDirMag;

	// Camera position and basis vectors (in world space)
	FLWCVector3 CameraOrigin;
	float3 CameraForwardDir;
	float3 CameraUpDir;
	FLWCVector3 PrevCameraOrigin;
	float3 PrevCameraForwardDir;
	float3 PrevCameraUpDir;

	// Primitive transform data
	FLWCMatrix PrimitiveLocalToWorld;
	FLWCInverseMatrix PrimitiveWorldToLocal;
	FLWCMatrix PrimitivePrevLocalToWorld;
	float3 PrimitiveInvNonUniformScale;
};

// Transforms a local-space position to world space
FLWCVector3 NiagaraLocalToWorldPos(float3 LocalSpacePos, NiagaraMeshParticleSRTParams Params)
{
	return LWCMultiply(LocalSpacePos, Params.PrimitiveLocalToWorld);
}

// Transforms a simulation-space position to world space
FLWCVector3 NiagaraSimToWorldPos(FLWCVector3 SimSpacePos, NiagaraMeshParticleSRTParams Params)
{
	if (Params.bLocalSpace)
	{
		return NiagaraLocalToWorldPos(LWCToFloat(SimSpacePos), Params);
	}
	
	return SimSpacePos;
}

// Transforms a local-space direction vector to world space (does not apply scale)
float3 NiagaraLocalToWorldVec(float3 LocalSpaceVec, NiagaraMeshParticleSRTParams Params)
{
	return
		Params.PrimitiveInvNonUniformScale.x * Params.PrimitiveLocalToWorld.M[0].xyz * LocalSpaceVec.xxx +
		Params.PrimitiveInvNonUniformScale.y * Params.PrimitiveLocalToWorld.M[1].xyz * LocalSpaceVec.yyy +
		Params.PrimitiveInvNonUniformScale.z * Params.PrimitiveLocalToWorld.M[2].xyz * LocalSpaceVec.zzz;
}

// Transforms a world-space direction vector to local space
float3 NiagaraWorldToLocalVec(float3 WorldSpaceVec, NiagaraMeshParticleSRTParams Params)
{
	float3x3 InvRot =
	{
		Params.PrimitiveInvNonUniformScale.x * Params.PrimitiveLocalToWorld.M[0].xyz,
		Params.PrimitiveInvNonUniformScale.y * Params.PrimitiveLocalToWorld.M[1].xyz,
		Params.PrimitiveInvNonUniformScale.z * Params.PrimitiveLocalToWorld.M[2].xyz
	};
	InvRot = transpose(InvRot);
	return mul(WorldSpaceVec, InvRot);
}

// Transforms a simulation-space direction vector to world space
float3 NiagaraSimToWorldVec(float3 SimSpaceVec, NiagaraMeshParticleSRTParams Params)
{
	if (Params.bLocalSpace)
	{
		return NiagaraLocalToWorldVec(SimSpaceVec, Params);
	}
	
	return SimSpaceVec;
}

// Transforms a world-space direction vector to simulation space
float3 NiagaraWorldToSimVec(float3 InVector, NiagaraMeshParticleSRTParams Params)
{
	if (Params.bLocalSpace)
	{
		return NiagaraWorldToLocalVec(InVector, Params);
	}
	
	return InVector;
}

// Helper to get decomposed velocity direction and magnitude from Niagara particle data
float4 NiagaraGetVelocityDirMag(int VelocityDataOffset, float3 DefaultVelocity, uint ParticleIndex)
{
	float4 DirMag;
	DirMag.xyz = NiagaraSafeGetVec3(VelocityDataOffset, ParticleIndex, DefaultVelocity);
	DirMag.w = length(DirMag.xyz);
	
	DirMag.xyz = DirMag.w > 0.0f ? DirMag.xyz / DirMag.w : float3(0.0f, 0.0f, 0.0f);
	
	return DirMag;
}

// Constructs a rotation matrix that satisfies the mesh particle's FacingMode
float3x3 NiagaraGetMeshFacingMatrix(FLWCVector3 ParticleSimPosition, NiagaraMeshParticleSRTParams Params)
{
	float3 WorldX = float3(1, 0, 0);
	float3 WorldZ = float3(0, 0, 1);
	float3 FacingDir = float3(0, 0, 0);

	// Select simulation-space facing direction
	switch (Params.FacingMode)
	{
		case MESH_FACING_VELOCITY:
		{
			if (Params.VelocityDirMag.w > 0.0f)
			{
				FacingDir = Params.VelocityDirMag.xyz;
			}
			else
			{
				FacingDir = NiagaraWorldToSimVec(WorldZ, Params);
			}
			break;
		}

		case MESH_FACING_CAMERA_POSITION:
		{
			FLWCVector3 WorldPosition = NiagaraSimToWorldPos(ParticleSimPosition, Params);
			float3 CameraDir = LWCNormalize(LWCSubtract(Params.CameraOrigin, WorldPosition));
			FacingDir = NiagaraWorldToSimVec(CameraDir, Params);
			break;
		}

		default: // case MESH_FACING_DEFAULT and MESH_FACING_CAMERA_PLANE
		{
			FacingDir = NiagaraWorldToSimVec(-Params.CameraForwardDir, Params);
			break;
		}
	}

	float3 XAxis = { 1, 0, 0 };
	float3 YAxis = { 0, 1, 0 };
	float3 ZAxis = { 0, 0, 1 };
	if (Params.bLockedAxisEnable)
	{
		// This is a special case where we want to lock the Z-Axis to the locked axis and get the X-Axis as close to facing direction as possible
		const bool bWorldSpaceAxis = (Params.LockedAxisSpace == PARTICLE_SPACE_WORLD) ||
									 (Params.LockedAxisSpace == PARTICLE_SPACE_SIMULATION && !Params.bLocalSpace);
		float3 LockedAxis = Params.LockedAxis;
		if (bWorldSpaceAxis && Params.bLocalSpace)
		{
			// Transform the world-space axis to local space
			LockedAxis = NiagaraWorldToLocalVec(LockedAxis, Params);
		}
		else if (!bWorldSpaceAxis && !Params.bLocalSpace)
		{
			// Transform the local-space axis to world space
			LockedAxis = NiagaraLocalToWorldVec(LockedAxis, Params);
		}

		if (abs(dot(FacingDir, LockedAxis)) > 0.99f)
		{
			// The facing dir and locked axis are too similar, choose a reference direction for the facing dir
			FacingDir = abs(LockedAxis.z) > 0.99f ? float3(1, 0, 0) : float3(0, 0, 1);
		}

		ZAxis = LockedAxis;
		YAxis = normalize(cross(ZAxis, FacingDir));
		XAxis = cross(YAxis, ZAxis);
	}
	else
	{
		// Determine a reference vector to use for up
		float3 RefVector;
		if (Params.FacingMode == MESH_FACING_CAMERA_PLANE)
		{
			// Use the camera upwards direction as a reference vector
			//-TODO: Add ability to remove HMD roll in VR
			RefVector = Params.CameraUpDir;
		}
		else
		{
			// Prefer to use world up as a reference vector, fall back to world X-axis when facing up or down
			const float3 SimFacingDir = NiagaraSimToWorldVec(FacingDir, Params);
			float DotWorldZ = dot(SimFacingDir, WorldZ);
			RefVector = abs(DotWorldZ) > 0.99f ? (-sign(DotWorldZ) * WorldX) : WorldZ;
		}

		// rotate the reference direction to simulation space, if necessary
		RefVector = NiagaraWorldToSimVec(RefVector, Params);

		// Orthonormalize the look-at rotation and generate a matrix
		XAxis = FacingDir;
		YAxis = normalize(cross(RefVector, FacingDir));
		ZAxis = cross(XAxis, YAxis);
	}

	return float3x3(XAxis, YAxis, ZAxis);
}

// Calculates the simulation-space CameraOffset of the mesh particle 
float3 NiagaraCalculateCameraOffset(FLWCVector3 ParticleSimPosition, NiagaraMeshParticleSRTParams Params)
{
	const FLWCVector3 WorldPosition = NiagaraSimToWorldPos(ParticleSimPosition, Params);
	const float Offset = NiagaraSafeGetFloat(Params.CameraOffsetDataOffset, Params.ParticleIndex, Params.DefaultCameraOffset);

	const FLWCVector3 CameraVec = LWCSubtract(Params.CameraOrigin, WorldPosition);
	const float3 CameraDir = NiagaraWorldToSimVec(LWCNormalize(CameraVec), Params);
		
	return CameraDir * Offset;
}

// Calculates scale, rotation, and translation of a mesh particle. NOTE: translation and rotation are in particle simulation space
NiagaraMeshParticleSRT NiagaraCalculateMeshParticleSRT(NiagaraMeshParticleSRTParams Params)
{
	float3 ParticleSimPosition = NiagaraSafeGetVec3(Params.PositionDataOffset, Params.ParticleIndex, Params.DefaultPosition);
	float4 ParticleRotation = NiagaraSafeGetVec4(Params.RotationDataOffset, Params.ParticleIndex, Params.DefaultRotation);
	float3 ParticleScale = NiagaraSafeGetVec3(Params.ScaleDataOffset, Params.ParticleIndex, Params.DefaultScale);
	
	FLWCVector3 ParticlePosition;
	if (Params.bLocalSpace)
	{
		ParticlePosition = MakeLWCVector3((float3)0, ParticleSimPosition);
	}
	else
 	{
		ParticlePosition = MakeLWCVector3(Params.SystemLWCTile, ParticleSimPosition);
	}

	NiagaraMeshParticleSRT SRT;

	////////////////////////////////////////////////////////////////////////////////
	// Scale
	SRT.Scale = ParticleScale * Params.MeshScale;
	
	////////////////////////////////////////////////////////////////////////////////
	// Rotation
	SRT.Rotation = NiagaraQuatTo3x3(NiagaraQuatMul(normalize(ParticleRotation), Params.MeshRotation));
	if (Params.FacingMode != MESH_FACING_DEFAULT)
	{
		// Factor in facing rotation
		float3x3 FacingMat = NiagaraGetMeshFacingMatrix(ParticlePosition, Params);
		SRT.Rotation = mul(SRT.Rotation, FacingMat);
	}
	
	////////////////////////////////////////////////////////////////////////////////
	// Translation
	SRT.Translation = ParticlePosition;

	// Apply CameraOffset
	SRT.Translation = LWCAdd(SRT.Translation, NiagaraCalculateCameraOffset(SRT.Translation, Params));
	
	// Apply MeshOffset
	if (Params.bMeshOffsetIsWorldSpace)
	{
		SRT.Translation = LWCAdd(SRT.Translation, NiagaraWorldToSimVec(Params.MeshOffset, Params));
	}
	else
	{
		// NOTE: MeshOffset here is mesh-local, not Primitive-local
		SRT.Translation = LWCAdd(SRT.Translation, mul(Params.MeshOffset * ParticleScale, SRT.Rotation));
	}

	return SRT;
}

// Calculates final transforms of a mesh particle, used for culling and/or rendering
NiagaraMeshParticleTransforms NiagaraCalculateMeshParticleTransforms(NiagaraMeshParticleTransformsParams Params)
{
	NiagaraMeshParticleSRTParams SRTParams;
	SRTParams.ParticleIndex					= Params.ParticleIndex;
	SRTParams.SystemLWCTile					= Params.SystemLWCTile;
	SRTParams.bLocalSpace 					= Params.bLocalSpace;
	SRTParams.FacingMode					= Params.FacingMode;
	SRTParams.MeshScale 					= Params.MeshScale;
	SRTParams.MeshRotation					= Params.MeshRotation;
	SRTParams.MeshOffset 					= Params.MeshOffset;
	SRTParams.bMeshOffsetIsWorldSpace		= Params.bMeshOffsetIsWorldSpace;
	SRTParams.bLockedAxisEnable				= Params.bLockedAxisEnable;
	SRTParams.LockedAxis					= Params.LockedAxis;
	SRTParams.LockedAxisSpace				= Params.LockedAxisSpace;
	SRTParams.ScaleDataOffset 				= Params.ScaleDataOffset;
	SRTParams.RotationDataOffset 			= Params.RotationDataOffset;
	SRTParams.PositionDataOffset 			= Params.PositionDataOffset;
	SRTParams.CameraOffsetDataOffset		= Params.CameraOffsetDataOffset;
	SRTParams.DefaultScale 					= Params.DefaultScale;
	SRTParams.DefaultRotation 				= Params.DefaultRotation;
	SRTParams.DefaultPosition 				= Params.DefaultPosition;
	SRTParams.DefaultCameraOffset 			= Params.DefaultCameraOffset;
	SRTParams.VelocityDirMag				= Params.VelocityDirMag;
	SRTParams.CameraOrigin 					= Params.CameraOrigin;
	SRTParams.CameraForwardDir				= Params.CameraForwardDir;
	SRTParams.CameraUpDir					= Params.CameraUpDir;
	SRTParams.PrimitiveLocalToWorld			= Params.PrimitiveLocalToWorld;
	SRTParams.PrimitiveInvNonUniformScale	= Params.PrimitiveInvNonUniformScale;

	NiagaraMeshParticleSRT ParticleSRT = NiagaraCalculateMeshParticleSRT(SRTParams);

	NiagaraMeshParticleSRT PrevParticleSRT;
	if (Params.bPreciseMotionVectors)
	{
		// Run through the whole process of generating the components of the transform with previous frame data		
		NiagaraMeshParticleSRTParams PrevSRTParams = SRTParams;
		PrevSRTParams.ScaleDataOffset 				= Params.PrevScaleDataOffset;
		PrevSRTParams.RotationDataOffset 			= Params.PrevRotationDataOffset;
		PrevSRTParams.PositionDataOffset 			= Params.PrevPositionDataOffset;
		PrevSRTParams.CameraOffsetDataOffset		= Params.PrevCameraOffsetDataOffset;
		PrevSRTParams.DefaultScale 					= Params.DefaultPrevScale;
		PrevSRTParams.DefaultRotation 				= Params.DefaultPrevRotation;
		PrevSRTParams.DefaultPosition 				= Params.DefaultPrevPosition;
		PrevSRTParams.DefaultCameraOffset 			= Params.DefaultPrevCameraOffset;
		PrevSRTParams.VelocityDirMag				= Params.PrevVelocityDirMag;
		PrevSRTParams.CameraOrigin 					= Params.PrevCameraOrigin;
		PrevSRTParams.CameraForwardDir				= Params.PrevCameraForwardDir;
		PrevSRTParams.CameraUpDir					= Params.PrevCameraUpDir;
		PrevSRTParams.PrimitiveLocalToWorld			= Params.PrimitivePrevLocalToWorld;

		PrevParticleSRT = NiagaraCalculateMeshParticleSRT(PrevSRTParams);
	}
	else
	{
		// Do a cheaper means of calculating the previous SRT that just extrapolates based on velocity		
		PrevParticleSRT = ParticleSRT;
		
		const float3 Velocity = Params.VelocityDirMag.xyz * Params.VelocityDirMag.w;		
		PrevParticleSRT.Translation = LWCSubtract(PrevParticleSRT.Translation, Velocity * Params.DeltaSeconds);
	}

	NiagaraMeshParticleTransforms Output;
	Output.LocalToWorld = NiagaraComposeTransformMatrix(ParticleSRT.Scale, ParticleSRT.Rotation, ParticleSRT.Translation);
	Output.WorldToLocal = NiagaraComposeInvTransformMatrix(ParticleSRT.Scale, ParticleSRT.Rotation, ParticleSRT.Translation);
	Output.PrevLocalToWorld = NiagaraComposeTransformMatrix(PrevParticleSRT.Scale, PrevParticleSRT.Rotation, PrevParticleSRT.Translation);
	Output.LocalToWorldNoScale = ParticleSRT.Rotation;

	if (Params.bLocalSpace)
	{
		// Transform from primitive to world space
		Output.LocalToWorld = LWCMultiply(LWCToFloat(Output.LocalToWorld), Params.PrimitiveLocalToWorld);
		Output.WorldToLocal = LWCMultiply(Params.PrimitiveWorldToLocal, LWCToFloat(Output.WorldToLocal));
		Output.PrevLocalToWorld = LWCMultiply(LWCToFloat(Output.PrevLocalToWorld), Params.PrimitivePrevLocalToWorld);

		float3x3 PrimLocalToWorldNoScale = LWCToFloat3x3(Params.PrimitiveLocalToWorld);
		PrimLocalToWorldNoScale[0] *= Params.PrimitiveInvNonUniformScale.x;
		PrimLocalToWorldNoScale[1] *= Params.PrimitiveInvNonUniformScale.y;
		PrimLocalToWorldNoScale[2] *= Params.PrimitiveInvNonUniformScale.z;

		Output.LocalToWorldNoScale = mul(Output.LocalToWorldNoScale, PrimLocalToWorldNoScale);
	}

	return Output;
}