// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshParticleVertexFactory.usf: Mesh particle vertex factory shader code.
=============================================================================*/

#include "/Engine/Private/VertexFactoryCommon.ush"
#include "NiagaraCommon.ush"
#include "NiagaraVFCommon.usf"
#include "NiagaraMeshParticleUtils.ush"

#if COMPUTESHADER || RAYHITGROUPSHADER
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/RayTracing/RayTracingHitGroupCommon.ush"
#endif

#define USE_PARTICLE_POSITION (NEEDS_PARTICLE_POSITION)
#define USE_PARTICLE_VELOCITY (NEEDS_PARTICLE_VELOCITY)
#define USE_PARTICLE_TIME (NEEDS_PARTICLE_TIME)
#define USE_PARTICLE_RANDOM (NEEDS_PARTICLE_RANDOM)
#define USE_PARTICLE_TRANSFORM (NEEDS_PARTICLE_TRANSFORM)
#define USE_PARTICLE_LOCAL_TO_WORLD (NEEDS_PARTICLE_LOCAL_TO_WORLD || NEEDS_INSTANCE_LOCAL_TO_WORLD_PS)
#define USE_PARTICLE_WORLD_TO_LOCAL (NEEDS_PARTICLE_WORLD_TO_LOCAL || NEEDS_INSTANCE_WORLD_TO_LOCAL_PS)

#ifndef MANUAL_VERTEX_FETCH
#define MANUAL_VERTEX_FETCH 0
#endif

#define VF_ColorIndexMask_Index 0
#define VF_NumTexcoords_Index 1

struct FVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;

#if !MANUAL_VERTEX_FETCH
	half3	TangentX	: ATTRIBUTE1;
	// TangentZ.w contains sign of tangent basis determinant
	half4	TangentZ	: ATTRIBUTE2;
	half4	VertexColor : ATTRIBUTE3;

	#if NUM_MATERIAL_TEXCOORDS_VERTEX
		#if NUM_MATERIAL_TEXCOORDS_VERTEX == 1
			float2	TexCoords0 : ATTRIBUTE4;
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX >= 2
			float4	TexCoords01 : ATTRIBUTE4;
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX == 3
			float2	TexCoords2 : ATTRIBUTE5;
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX >= 4
			float4	TexCoords23 : ATTRIBUTE5;
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX == 5
			float2	TexCoords4 : ATTRIBUTE6;
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX >= 6
			float4	TexCoords45 : ATTRIBUTE6;
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX == 7
			float2	TexCoords6 : ATTRIBUTE7;
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX >= 8
			float4	TexCoords67 : ATTRIBUTE7;
		#endif
	#elif USE_PARTICLE_SUBUVS
		float2	TexCoords0 : ATTRIBUTE4;
	#endif // NUM_MATERIAL_TEXCOORDS_VERTEX
#endif // !MANUAL_VERTEX_FETCH

#if (VF_USE_PRIMITIVE_SCENE_DATA == 1)
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(13)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
#else
	#undef GetInstanceIdFromVF
	#define GetInstanceIdFromVF(VFInput)				(VFInput.InstanceId)
	
	uint InstanceId	: SV_InstanceID;
#endif
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()

#if PASS_NEEDS_VERTEX_ID || MANUAL_VERTEX_FETCH
	uint VertexId : SV_VertexID;
#endif // #if PASS_NEEDS_VERTEX_ID
};

struct FVertexFactoryInterpolantsVSToPS
{
	TANGENTTOWORLD_INTERPOLATOR_BLOCK

#if USE_PARTICLE_SUBUVS
	float4 SubUV0AndTexCoord0 : TEXCOORD1;
	float4 SubUV1AndLerp : TEXCOORD2;
#else
	#if NUM_TEX_COORD_INTERPOLATORS
		float4	TexCoords[(NUM_TEX_COORD_INTERPOLATORS+1)/2]	: TEXCOORD0;
	#endif
#endif

#if INTERPOLATE_VERTEX_COLOR
	float4	VertexColor : COLOR0;
#endif

#if NEEDS_PARTICLE_COLOR
	nointerpolation float4	ParticleColor : COLOR1;
#endif

#if (DYNAMIC_PARAMETERS_MASK & 1)
	nointerpolation float4 DynamicParameter : COLOR2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	nointerpolation float4 DynamicParameter1 : COLOR3;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	nointerpolation float4 DynamicParameter2 : COLOR4;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	nointerpolation float4 DynamicParameter3 : COLOR5;
#endif

#if USE_PARTICLE_POSITION
	/** Particle position in camera-centered translated world space */
	nointerpolation float3 ParticleTranslatedWorldPosition	: PARTICLE_POSITION;
#endif

#if USE_PARTICLE_VELOCITY
	/** The velocity of the particle, XYZ: direction, W: speed. */
	nointerpolation float4 ParticleVelocity	: PARTICLE_VELOCITY;
#endif

#if USE_PARTICLE_TIME
	/** Relative alive time of the particle */
	nointerpolation float RelativeTime : PARTICLE_TIME;
#endif

#if USE_PARTICLE_RANDOM
	nointerpolation float ParticleRandom : PARTICLE_RANDOM;
#endif

#if USE_PARTICLE_LOCAL_TO_WORLD
	nointerpolation float4 ParticleToWorld[3] : PARTICLE_LOCAL_TO_WORLD;
#endif

#if USE_PARTICLE_WORLD_TO_LOCAL
	nointerpolation float4 WorldToParticle[3] : PARTICLE_WORLD_TO_LOCAL;
#endif

#if (VF_USE_PRIMITIVE_SCENE_DATA == 1)
	nointerpolation uint PrimitiveId : PRIMITIVE_ID;
#endif
};

struct FVertexFactoryIntermediates
{
	/** The index of the particle to use when accessing data buffers */
	uint ParticleIndex;

	/** The color of the vertex. */
	float4 VertexColor;

	/** The color of the particle. */
	float4 ParticleColor;

	/** The texture coordinates for the vertex. */
#if NUM_MATERIAL_TEXCOORDS_VERTEX
	float2	TexCoords[NUM_MATERIAL_TEXCOORDS_VERTEX];
#endif
#if USE_PARTICLE_SUBUVS
	float4 SubUVCoords;
	nointerpolation float SubUVLerp;
#endif

	/** Optional dynamic parameters for the particle. */
#if (DYNAMIC_PARAMETERS_MASK & 1)
	float4 DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	float4 DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	float4 DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	float4 DynamicParameter3;
#endif

	/** The velocity of the particle, XYZ: direction, W: speed. */
	float4 ParticleVelocity;

	/** Particle position in camera-centered translated world space. */
	float3 ParticleTranslatedWorldPosition;

#if NEEDS_PARTICLE_TIME
	/** Relative time. */
	float RelativeTime;
#endif

#if NEEDS_PARTICLE_RANDOM
	/** Particle Random value */
	float ParticleRandom;
#endif

	FLWCMatrix ParticleToWorld;
	FLWCInverseMatrix WorldToParticle;
	FLWCMatrix PrevParticleToWorld;	
	float3x3 ParticleToWorldNoScale;

	float3x3 TangentToLocal;
	float TangentDeterminant;

	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

FDFMatrix GetParticleTransform(FVertexFactoryIntermediates Inters)
{
	return DFFromTileOffset(Inters.ParticleToWorld);
}

FDFInverseMatrix GetParticleInvTransform(FVertexFactoryIntermediates Inters)
{
	return DFFromTileOffset(Inters.WorldToParticle);
}

FDFMatrix GetParticlePrevTransform(FVertexFactoryIntermediates Inters)
{
	return DFFromTileOffset(Inters.PrevParticleToWorld);
}

float3x3 GetParticleRotationNoScale(FVertexFactoryIntermediates Inters)
{
	return Inters.ParticleToWorldNoScale;
}

float2 GetUV(FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex)
{
#if USE_PARTICLE_SUBUVS
	return 0;
#elif NUM_TEX_COORD_INTERPOLATORS
	float4 UVVector = Interpolants.TexCoords[UVIndex / 2];
	return UVIndex % 2 ? UVVector.zw : UVVector.xy;
#else
	return 0;
#endif
}

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if USE_PARTICLE_SUBUVS
	#if  NUM_TEX_COORD_INTERPOLATORS
		UNROLL
		for( int CoordinateIndex = 0; CoordinateIndex <  NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++ )
		{
			Result.TexCoords[CoordinateIndex] = Interpolants.SubUV0AndTexCoord0.zw;
		}
	#endif
	Result.Particle.SubUVCoords[0] = Interpolants.SubUV0AndTexCoord0.xy;
	Result.Particle.SubUVCoords[1] = Interpolants.SubUV1AndLerp.xy;
	Result.Particle.SubUVLerp = Interpolants.SubUV1AndLerp.z;
#elif  NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS / 2; ++CoordinateIndex)
	{
		Result.TexCoords[CoordinateIndex * 2] = Interpolants.TexCoords[CoordinateIndex].xy;
		Result.TexCoords[CoordinateIndex * 2 + 1] = Interpolants.TexCoords[CoordinateIndex].wz;
	}
	#if NUM_TEX_COORD_INTERPOLATORS & 1
		Result.TexCoords[NUM_TEX_COORD_INTERPOLATORS - 1] = Interpolants.TexCoords[NUM_TEX_COORD_INTERPOLATORS / 2].xy;
	#endif // #if NUM_TEX_COORD_INTERPOLATORS & 1
#endif

	half3 TangentToWorld0 = Interpolants.TangentToWorld0.xyz;
	half4 TangentToWorld2 = Interpolants.TangentToWorld2;
	Result.UnMirrored = TangentToWorld2.w;

#if INTERPOLATE_VERTEX_COLOR
	Result.VertexColor = Interpolants.VertexColor;
#else
	Result.VertexColor = 0;
#endif

#if NEEDS_PARTICLE_COLOR
	Result.Particle.Color = Interpolants.ParticleColor;
#endif

#if (DYNAMIC_PARAMETERS_MASK != 0)
	Result.Particle.DynamicParameterValidMask = NiagaraMeshVF.MaterialParamValidMask;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 1)
	Result.Particle.DynamicParameter = Interpolants.DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	Result.Particle.DynamicParameter1 = Interpolants.DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	Result.Particle.DynamicParameter2 = Interpolants.DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	Result.Particle.DynamicParameter3 = Interpolants.DynamicParameter3;
#endif

#if USE_PARTICLE_POSITION
	Result.Particle.TranslatedWorldPositionAndSize.xyz = Interpolants.ParticleTranslatedWorldPosition;
	Result.Particle.TranslatedWorldPositionAndSize.w = 1;
	Result.Particle.PrevTranslatedWorldPositionAndSize = Result.Particle.TranslatedWorldPositionAndSize;
#endif

#if USE_PARTICLE_VELOCITY
	Result.Particle.Velocity = Interpolants.ParticleVelocity;
#endif

#if USE_PARTICLE_TIME
	Result.Particle.RelativeTime = Interpolants.RelativeTime;
#endif

#if USE_PARTICLE_RANDOM
	Result.Particle.Random = Interpolants.ParticleRandom;
#else
	Result.Particle.Random = 0.0f;
#endif

#if USE_PARTICLE_LOCAL_TO_WORLD
	//-TODO: LWC Precision Loss
	FLWCMatrix ParticleToWorld = LWCPromote(transpose(float4x4(Interpolants.ParticleToWorld[0], Interpolants.ParticleToWorld[1], Interpolants.ParticleToWorld[2], float4(0.0f, 0.0f, 0.0f, 1.0f))));
	#if NEEDS_PARTICLE_LOCAL_TO_WORLD
		Result.Particle.ParticleToWorld = DFFromTileOffset(ParticleToWorld);
	#endif
	#if NEEDS_INSTANCE_LOCAL_TO_WORLD_PS
		Result.InstanceLocalToWorld = DFFromTileOffset(ParticleToWorld);
	#endif
#endif

#if USE_PARTICLE_WORLD_TO_LOCAL
	//-TODO: LWC Precision Loss
	FLWCInverseMatrix WorldToParticle = LWCPromoteInverse(transpose(float4x4(Interpolants.WorldToParticle[0], Interpolants.WorldToParticle[1], Interpolants.WorldToParticle[2], float4(0.0f, 0.0f, 0.0f, 1.0f))));
	#if NEEDS_PARTICLE_WORLD_TO_LOCAL
		Result.Particle.WorldToParticle = DFFromTileOffset(WorldToParticle);
	#endif
	#if NEEDS_INSTANCE_WORLD_TO_LOCAL_PS
		Result.InstanceWorldToLocal = DFFromTileOffset(WorldToParticle);
	#endif
#endif

	Result.Particle.MotionBlurFade = 1.0f;
	Result.TangentToWorld = AssembleTangentToWorld( TangentToWorld0, TangentToWorld2 );
	Result.TwoSidedSign = 1;

#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Result.WorldVertexNormal_Center = Interpolants.TangentToWorld2_Center.xyz;
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	Result.PrimitiveId = Interpolants.PrimitiveId;
#endif

	return Result;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	
#if VF_USE_PRIMITIVE_SCENE_DATA
	Result.PrimitiveId = Intermediates.SceneData.PrimitiveId;
#endif

	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.VertexColor = Intermediates.VertexColor;
	Result.PreSkinnedPosition = Input.Position.xyz;
	Result.PreSkinnedNormal = Intermediates.TangentToLocal[2];

#if NEEDS_PARTICLE_POSITION
	Result.Particle.TranslatedWorldPositionAndSize.xyz = Intermediates.ParticleTranslatedWorldPosition;
	Result.Particle.TranslatedWorldPositionAndSize.w = 1;
	Result.Particle.PrevTranslatedWorldPositionAndSize.xyz = LWCToFloat(LWCAdd(LWCGetOrigin(Intermediates.PrevParticleToWorld), ResolvedView.TileOffset.PrevPreViewTranslation));
	Result.Particle.PrevTranslatedWorldPositionAndSize.w = 1;
#endif

#if NEEDS_PARTICLE_VELOCITY
	Result.Particle.Velocity = Intermediates.ParticleVelocity;
#endif

#if NEEDS_PARTICLE_TIME
	Result.Particle.RelativeTime = Intermediates.RelativeTime;
#endif

#if NEEDS_PARTICLE_RANDOM
	Result.Particle.Random = Intermediates.ParticleRandom;
#else
	Result.Particle.Random = 0.0f;
#endif

	Result.Particle.Color = Intermediates.ParticleColor;

#if (DYNAMIC_PARAMETERS_MASK != 0)
	Result.Particle.DynamicParameterValidMask = NiagaraMeshVF.MaterialParamValidMask;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 1)
	Result.Particle.DynamicParameter = Intermediates.DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	Result.Particle.DynamicParameter1 = Intermediates.DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	Result.Particle.DynamicParameter2 = Intermediates.DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	Result.Particle.DynamicParameter3 = Intermediates.DynamicParameter3;
#endif

	Result.InstanceLocalToWorld = GetParticleTransform(Intermediates);
	Result.InstanceWorldToLocal = GetParticleInvTransform(Intermediates);

	Result.Particle.ParticleToWorld = Result.InstanceLocalToWorld;
	Result.Particle.WorldToParticle = Result.InstanceWorldToLocal;

	// Use the particle's unscaled transform for rotating normals to match static mesh
	Result.TangentToWorld = mul(TangentToLocal, GetParticleRotationNoScale(Intermediates));

	Result.PrevFrameLocalToWorld = GetParticlePrevTransform(Intermediates);

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	for(int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = Intermediates.TexCoords[CoordinateIndex];
	}
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

#if ENABLE_NEW_HLSL_GENERATOR
	// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
	EvaluateVertexMaterialAttributes(Result);
#endif

	return Result;
}

float3x3 CalculateTangentToLocal(FVertexFactoryInput Input, out float TangentDeterminant)
{
	float3x3 Result=0;
#if MANUAL_VERTEX_FETCH
	float3 TangentX = NiagaraMeshVF.VertexFetch_PackedTangentsBuffer[2 * Input.VertexId + 0].xyz;
	float4 TangentZ = NiagaraMeshVF.VertexFetch_PackedTangentsBuffer[2 * Input.VertexId + 1].xyzw;
#else
	float3 TangentX = TangentBias(Input.TangentX);
	float4 TangentZ = TangentBias(Input.TangentZ);
#endif // MANUAL_VERTEX_FETCH

	// pass-thru the tangent
	Result[0] = TangentX;
	// pass-thru the normal
	Result[2] = TangentZ.xyz;

	// derive the binormal by getting the cross product of the normal and tangent
	Result[1] = cross(Result[2], Result[0]) * TangentZ.w;
	// Recalculate TangentX off of the other two vectors
	// This corrects quantization error since TangentX was passed in as a quantized vertex input
	// The error shows up most in specular off of a mesh with a smoothed UV seam (normal is smooth, but tangents vary across the seam)
	Result[0] = cross(Result[1], Result[2]) * TangentZ.w;

	TangentDeterminant = TangentZ.w;

	return Result;
}

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates = (FVertexFactoryIntermediates)0;

#if (VF_USE_PRIMITIVE_SCENE_DATA == 1)
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);

	// Pull the particle index from the custom data for the instance
	Intermediates.ParticleIndex = asuint(LoadInstanceCustomDataFloat(Intermediates.SceneData.InstanceData, 0));
#else
	Intermediates.SceneData = GetSceneDataIntermediates();

	const uint InstanceId = GetInstanceId(GetInstanceIdFromVF(Input)); // NOTE: Handles instanced stereo	
	BRANCH
	if (NiagaraMeshVF.SortedIndicesOffset == -1)
	{
		Intermediates.ParticleIndex = InstanceId;
	}
	else
	{
		Intermediates.ParticleIndex = NiagaraMeshVF.SortedIndices[NiagaraMeshVF.SortedIndicesOffset + InstanceId];
	}
#endif // (VF_USE_PRIMITIVE_SCENE_DATA == 1)

	Intermediates.ParticleColor = SafeGetVec4(NiagaraMeshVF.ColorDataOffset, Intermediates.ParticleIndex, NiagaraMeshVF.DefaultColor);
	Intermediates.ParticleVelocity = NiagaraGetVelocityDirMag(NiagaraMeshVF.VelocityDataOffset, NiagaraMeshVF.DefaultVelocity, Intermediates.ParticleIndex);

#if (DYNAMIC_PARAMETERS_MASK & 1)
	Intermediates.DynamicParameter = SafeGetVec4(NiagaraMeshVF.MaterialParamDataOffset, Intermediates.ParticleIndex, NiagaraMeshVF.DefaultDynamicMaterialParameter0);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	Intermediates.DynamicParameter1 = SafeGetVec4(NiagaraMeshVF.MaterialParam1DataOffset, Intermediates.ParticleIndex, NiagaraMeshVF.DefaultDynamicMaterialParameter1);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	Intermediates.DynamicParameter2 = SafeGetVec4(NiagaraMeshVF.MaterialParam2DataOffset, Intermediates.ParticleIndex, NiagaraMeshVF.DefaultDynamicMaterialParameter2);	
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	Intermediates.DynamicParameter3 = SafeGetVec4(NiagaraMeshVF.MaterialParam3DataOffset, Intermediates.ParticleIndex, NiagaraMeshVF.DefaultDynamicMaterialParameter3);	
#endif

#if NEEDS_PARTICLE_TIME
	Intermediates.RelativeTime = SafeGetFloat(NiagaraMeshVF.NormalizedAgeDataOffset, Intermediates.ParticleIndex, NiagaraMeshVF.DefaultNormAge);	
#endif

#if NEEDS_PARTICLE_RANDOM
	Intermediates.ParticleRandom = SafeGetFloat(NiagaraMeshVF.MaterialRandomDataOffset, Intermediates.ParticleIndex, NiagaraMeshVF.DefaultMatRandom);	
#endif

	// Compute transforms
#if (VF_USE_PRIMITIVE_SCENE_DATA == 1)
	Intermediates.ParticleToWorld = DFToTileOffset(Intermediates.SceneData.InstanceData.LocalToWorld);
	Intermediates.WorldToParticle = DFToTileOffset(Intermediates.SceneData.InstanceData.WorldToLocal);
	Intermediates.PrevParticleToWorld = DFToTileOffset(Intermediates.SceneData.InstanceData.PrevLocalToWorld);

	const float3 InstInvScale = Intermediates.SceneData.InstanceData.InvNonUniformScale;
	Intermediates.ParticleToWorldNoScale = LWCToFloat3x3(Intermediates.ParticleToWorld);
	Intermediates.ParticleToWorldNoScale[0] *= InstInvScale.x;
	Intermediates.ParticleToWorldNoScale[1] *= InstInvScale.y;
	Intermediates.ParticleToWorldNoScale[2] *= InstInvScale.z;
#else
	NiagaraMeshParticleTransformsParams Params;
	Params.ParticleIndex				= Intermediates.ParticleIndex;
	Params.SystemLWCTile				= NiagaraMeshVF.SystemLWCTile;
	Params.bLocalSpace					= NiagaraMeshVF.bLocalSpace;
	Params.bPreciseMotionVectors 		= NiagaraMeshVF.AccurateMotionVectors != 0;
	Params.FacingMode					= NiagaraMeshVF.FacingMode;
	Params.DeltaSeconds					= NiagaraMeshVF.DeltaSeconds;
	Params.MeshScale 					= NiagaraMeshVF.MeshScale;
	Params.MeshRotation					= NiagaraMeshVF.MeshRotation;
	Params.MeshOffset 					= NiagaraMeshVF.MeshOffset;
	Params.bMeshOffsetIsWorldSpace		= NiagaraMeshVF.bMeshOffsetIsWorldSpace;
	Params.bLockedAxisEnable			= NiagaraMeshVF.bLockedAxisEnable;
	Params.LockedAxis					= NiagaraMeshVF.LockedAxis;
	Params.LockedAxisSpace				= NiagaraMeshVF.LockedAxisSpace;
	Params.ScaleDataOffset 				= NiagaraMeshVF.ScaleDataOffset;
	Params.RotationDataOffset 			= NiagaraMeshVF.RotationDataOffset;
	Params.PositionDataOffset 			= NiagaraMeshVF.PositionDataOffset;
	Params.CameraOffsetDataOffset		= NiagaraMeshVF.CameraOffsetDataOffset;
	Params.PrevScaleDataOffset 			= NiagaraMeshVF.PrevScaleDataOffset;
	Params.PrevRotationDataOffset 		= NiagaraMeshVF.PrevRotationDataOffset;
	Params.PrevPositionDataOffset 		= NiagaraMeshVF.PrevPositionDataOffset;
	Params.PrevCameraOffsetDataOffset	= NiagaraMeshVF.PrevCameraOffsetDataOffset;
	Params.DefaultScale 				= NiagaraMeshVF.DefaultScale;
	Params.DefaultRotation 				= NiagaraMeshVF.DefaultRotation;
	Params.DefaultPosition 				= NiagaraMeshVF.DefaultPosition;
	Params.DefaultCameraOffset 			= NiagaraMeshVF.DefaultCameraOffset;
	Params.DefaultPrevScale 			= NiagaraMeshVF.DefaultPrevScale;
	Params.DefaultPrevRotation 			= NiagaraMeshVF.DefaultPrevRotation;
	Params.DefaultPrevPosition 			= NiagaraMeshVF.DefaultPrevPosition;
	Params.DefaultPrevCameraOffset 		= NiagaraMeshVF.DefaultPrevCameraOffset;
	Params.VelocityDirMag				= Intermediates.ParticleVelocity;
	Params.PrevVelocityDirMag			= NiagaraGetVelocityDirMag(NiagaraMeshVF.PrevVelocityDataOffset, NiagaraMeshVF.DefaultPrevVelocity, Intermediates.ParticleIndex);
	Params.CameraOrigin 				= ResolvedView.TileOffset.WorldCameraOrigin;
	Params.CameraForwardDir				= ResolvedView.ViewForward;
	Params.CameraUpDir					= ResolvedView.ViewUp;
	Params.PrevCameraOrigin 			= ResolvedView.TileOffset.PrevWorldCameraOrigin;
	Params.PrevCameraForwardDir			= ResolvedView.PrevViewToTranslatedWorld[2].xyz;
	Params.PrevCameraUpDir				= ResolvedView.PrevViewToTranslatedWorld[1].xyz;
	Params.PrimitiveLocalToWorld		= DFToTileOffset(Intermediates.SceneData.Primitive.LocalToWorld);
	Params.PrimitiveWorldToLocal 		= DFToTileOffset(Intermediates.SceneData.Primitive.WorldToLocal);
	Params.PrimitivePrevLocalToWorld 	= DFToTileOffset(Intermediates.SceneData.Primitive.PreviousLocalToWorld);
	Params.PrimitiveInvNonUniformScale	= Intermediates.SceneData.Primitive.InvNonUniformScale;

	NiagaraMeshParticleTransforms Transforms = NiagaraCalculateMeshParticleTransforms(Params);

	Intermediates.ParticleToWorld 			= Transforms.LocalToWorld;
	Intermediates.WorldToParticle 			= Transforms.WorldToLocal;
	Intermediates.PrevParticleToWorld		= Transforms.PrevLocalToWorld;
	Intermediates.ParticleToWorldNoScale	= Transforms.LocalToWorldNoScale;
#endif // (VF_USE_PRIMITIVE_SCENE_DATA == 1)

	// Particle translated world position.
	Intermediates.ParticleTranslatedWorldPosition = LWCToFloat(LWCAdd(LWCGetOrigin(Intermediates.ParticleToWorld), ResolvedView.TileOffset.PreViewTranslation));

	const uint NumFetchTexCoords = NiagaraMeshVF.VertexFetch_Parameters[VF_NumTexcoords_Index];
#if NUM_MATERIAL_TEXCOORDS_VERTEX
	#if MANUAL_VERTEX_FETCH
		UNROLL
		for (uint CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; CoordinateIndex++)
		{
			// Clamp coordinates to mesh's maximum as materials can request more than are available
			uint ClampedCoordinateIndex = min(CoordinateIndex, NumFetchTexCoords-1);
			Intermediates.TexCoords[CoordinateIndex] = NiagaraMeshVF.VertexFetch_TexCoordBuffer[NumFetchTexCoords * Input.VertexId + ClampedCoordinateIndex];
		}
	#else
		#if NUM_MATERIAL_TEXCOORDS_VERTEX == 1
			Intermediates.TexCoords[0] = Input.TexCoords0;
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX >= 2
			Intermediates.TexCoords[0] = Input.TexCoords01.xy;
			Intermediates.TexCoords[1] = NumFetchTexCoords > 1 ? Input.TexCoords01.zw: Intermediates.TexCoords[0];
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX == 3
			Intermediates.TexCoords[2] = NumFetchTexCoords > 2 ? Input.TexCoords2: Intermediates.TexCoords[1];
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX >= 4
			Intermediates.TexCoords[2] = NumFetchTexCoords > 2 ? Input.TexCoords23.xy : Intermediates.TexCoords[1];
			Intermediates.TexCoords[3] = NumFetchTexCoords > 3 ? Input.TexCoords23.zw : Intermediates.TexCoords[2];
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX == 5
			Intermediates.TexCoords[4] = NumFetchTexCoords > 4 ? Input.TexCoords4 : Intermediates.TexCoords[3];
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX >= 6
			Intermediates.TexCoords[4] = NumFetchTexCoords > 4 ? Input.TexCoords45.xy : Intermediates.TexCoords[3];
			Intermediates.TexCoords[5] = NumFetchTexCoords > 5 ? Input.TexCoords45.zw : Intermediates.TexCoords[4];
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX == 7
			Intermediates.TexCoords[6] = NumFetchTexCoords > 6 ? Input.TexCoords6 : Intermediates.TexCoords[5];
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX >= 8
			Intermediates.TexCoords[6] = NumFetchTexCoords > 6 ? Input.TexCoords67.xy : Intermediates.TexCoords[5];
			Intermediates.TexCoords[7] = NumFetchTexCoords > 7 ? Input.TexCoords67.zw : Intermediates.TexCoords[6];
		#endif
	#endif // MANUAL_VERTEX_FETCH
#endif // NUM_MATERIAL_TEXCOORDS_VERTEX

#if USE_PARTICLE_SUBUVS
	// SubUV.
	float SubImageIndex = SafeGetFloat(NiagaraMeshVF.SubImageDataOffset, Intermediates.ParticleIndex, NiagaraMeshVF.DefaultSubImage);
	float SubImageLerp = frac(SubImageIndex);
	float SubImageA = SubImageIndex - SubImageLerp;
	float SubImageB = SubImageA + 1;
	float SubImageAH = fmod(SubImageA, NiagaraMeshVF.SubImageSize.x);
	float SubImageBH = fmod(SubImageB, NiagaraMeshVF.SubImageSize.x);
	float SubImageAV = floor(SubImageA * NiagaraMeshVF.SubImageSize.z);
	float SubImageBV = floor(SubImageB * NiagaraMeshVF.SubImageSize.z);
	Intermediates.SubUVLerp = (NiagaraMeshVF.SubImageBlendMode == 1) ? SubImageLerp : 0.0f;

	#if MANUAL_VERTEX_FETCH
		const uint NumFetchSubUvTexCoords = NiagaraMeshVF.VertexFetch_Parameters[VF_NumTexcoords_Index];
		const float2 SubUvTexCoord = NiagaraMeshVF.VertexFetch_TexCoordBuffer[NumFetchSubUvTexCoords * Input.VertexId];
	#else
		#if NUM_MATERIAL_TEXCOORDS_VERTEX == 1
			const float2 SubUvTexCoord = Input.TexCoords0;
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX >= 2
			const float2 SubUvTexCoord = Input.TexCoords01.xy;
		#else
			const float2 SubUvTexCoord = float2(0.0f, 0.0f);
		#endif
	#endif // MANUAL_VERTEX_FETCH

	Intermediates.SubUVCoords.xy = (float2(SubImageAH, SubImageAV) + SubUvTexCoord) * NiagaraMeshVF.SubImageSize.zw;
	Intermediates.SubUVCoords.zw = (float2(SubImageBH, SubImageBV) + SubUvTexCoord) * NiagaraMeshVF.SubImageSize.zw;
#endif // USE_PARTICLE_SUBUVS

#if MANUAL_VERTEX_FETCH
	Intermediates.VertexColor = NiagaraMeshVF.VertexFetch_ColorComponentsBuffer[Input.VertexId & NiagaraMeshVF.VertexFetch_Parameters[VF_ColorIndexMask_Index]] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE; // Swizzle vertex color.
#else
	// Swizzle vertex color.
	Intermediates.VertexColor = Input.VertexColor FCOLOR_COMPONENT_SWIZZLE;
#endif // MANUAL_VERTEX_FETCH

	Intermediates.TangentToLocal = CalculateTangentToLocal(Input, Intermediates.TangentDeterminant);

	return Intermediates;
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
* this vertex factory will calculate the binormal on-the-fly
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
float3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return Intermediates.TangentToLocal;
}

// @return translated world position
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{	
	FLWCMatrix Transform = Intermediates.ParticleToWorld;
	float3 WorldPosition = LWCToFloat(LWCAdd(LWCMultiply(Input.Position.xyz, Transform), ResolvedView.TileOffset.PreViewTranslation));
	return float4(WorldPosition, Input.Position.w);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Input.Position.xyz; // No support for instancing, so instance == primitive
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return LWCToFloat3x3(Intermediates.ParticleToWorld)[2].xyz;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

void CalcTangentToWorld(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, out float3 TangentToWorld0, out float4 TangentToWorld2)
{
	const float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, Intermediates);

	// Use the particle's unscaled transform for rotating normals to match static mesh
	const float3x3 TransformNoScale = GetParticleRotationNoScale(Intermediates);
	const float3x3 TangentToWorld = mul(TangentToLocal, TransformNoScale);
	const float DetSign = Intermediates.SceneData.InstanceData.DeterminantSign;

	// Note: We do not normalize to match localvertexfactory.ush and the above transform does not contain scale
	//       If for some reason we need to add the normalize back in you will need to consider UE-195371 as some mesh data is 0,0,0
	TangentToWorld0 = TangentToWorld[0];
	TangentToWorld2 = float4(TangentToWorld[2], TangentBias(Intermediates.TangentDeterminant) * DetSign);
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

#if USE_PARTICLE_SUBUVS
	Interpolants.SubUV0AndTexCoord0.xy = Intermediates.SubUVCoords.xy;
	Interpolants.SubUV1AndLerp.xy = Intermediates.SubUVCoords.zw;
	Interpolants.SubUV1AndLerp.zw = Intermediates.SubUVLerp.xx;

	#if  NUM_TEX_COORD_INTERPOLATORS
		float2 CustomizedUVs[ NUM_TEX_COORD_INTERPOLATORS];
		GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
		Interpolants.SubUV0AndTexCoord0.zw = CustomizedUVs[0];
	#else
		Interpolants.SubUV0AndTexCoord0.zw = 0;
	#endif
#elif  NUM_TEX_COORD_INTERPOLATORS
	// Ensure the unused components of the last packed texture coordinate are initialized.
	Interpolants.TexCoords[( NUM_TEX_COORD_INTERPOLATORS + 1) / 2 - 1] = 0;

	float2 CustomizedUVs[ NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);

	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS / 2; ++CoordinateIndex)
	{
		Interpolants.TexCoords[CoordinateIndex].xy = CustomizedUVs[CoordinateIndex * 2];
		Interpolants.TexCoords[CoordinateIndex].wz = CustomizedUVs[CoordinateIndex * 2 + 1];
	}
	#if NUM_TEX_COORD_INTERPOLATORS & 1
		Interpolants.TexCoords[NUM_TEX_COORD_INTERPOLATORS / 2].xy = CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS - 1];
	#endif // #if NUM_TEX_COORD_INTERPOLATORS & 1
#endif


	Interpolants.TangentToWorld0.w = 0;
	CalcTangentToWorld(Input, Intermediates, Interpolants.TangentToWorld0.xyz, Interpolants.TangentToWorld2);
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Interpolants.TangentToWorld2_Center = Interpolants.TangentToWorld2;
#endif

#if INTERPOLATE_VERTEX_COLOR
	Interpolants.VertexColor = Intermediates.VertexColor;
#endif

#if NEEDS_PARTICLE_COLOR
	Interpolants.ParticleColor = Intermediates.ParticleColor;
#endif

#if (DYNAMIC_PARAMETERS_MASK & 1)
	Interpolants.DynamicParameter = Intermediates.DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	Interpolants.DynamicParameter1 = Intermediates.DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	Interpolants.DynamicParameter2 = Intermediates.DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	Interpolants.DynamicParameter3 = Intermediates.DynamicParameter3;
#endif


#if USE_PARTICLE_POSITION
	Interpolants.ParticleTranslatedWorldPosition = Intermediates.ParticleTranslatedWorldPosition;
#endif

#if USE_PARTICLE_VELOCITY
	Interpolants.ParticleVelocity = Intermediates.ParticleVelocity;
#endif

#if USE_PARTICLE_TIME
	Interpolants.RelativeTime = Intermediates.RelativeTime;
#endif

#if USE_PARTICLE_RANDOM
	Interpolants.ParticleRandom = Intermediates.ParticleRandom;
#endif

#if USE_PARTICLE_LOCAL_TO_WORLD
	//-TODO: LWC Precision Loss
	float4x4 ParticleToWorldT = transpose(LWCHackToFloat(Intermediates.ParticleToWorld));
	Interpolants.ParticleToWorld[0] = ParticleToWorldT[0];
	Interpolants.ParticleToWorld[1] = ParticleToWorldT[1];
	Interpolants.ParticleToWorld[2] = ParticleToWorldT[2];
#endif

#if USE_PARTICLE_WORLD_TO_LOCAL
	//-TODO: LWC Precision Loss
	float4x4 WorldToParticleT = transpose(LWCHackToFloat(Intermediates.WorldToParticle));
	Interpolants.WorldToParticle[0] = WorldToParticleT[0];
	Interpolants.WorldToParticle[1] = WorldToParticleT[1];
	Interpolants.WorldToParticle[2] = WorldToParticleT[2];
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	Interpolants.PrimitiveId = Intermediates.SceneData.PrimitiveId;
#endif

	return Interpolants;
}

// @return previous translated world position
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
    FLWCMatrix Transform = Intermediates.PrevParticleToWorld;
    float3 WorldPosition = LWCToFloat(LWCAdd(LWCMultiply(Input.Position.xyz, Transform), ResolvedView.TileOffset.PrevPreViewTranslation));
    return float4(WorldPosition, Input.Position.w);
}

// local position relative to primitive
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Input.Position.xyz; // No support for instancing, so instance == primitive
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
#if VF_USE_PRIMITIVE_SCENE_DATA
	return Interpolants.PrimitiveId;
#else
    return 0;
#endif
}

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return float4(0,0,0,0);
}

#if NEEDS_VERTEX_FACTORY_INTERPOLATION || RAYHITGROUPSHADER
	struct FVertexFactoryRayTracingInterpolants
	{
		FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
	};

	float2 VertexFactoryGetRayTracingTextureCoordinate( FVertexFactoryRayTracingInterpolants Interpolants )
	{
	#if USE_PARTICLE_SUBUVS
		return Interpolants.InterpolantsVSToPS.SubUV0AndTexCoord0.zw;
	#elif NUM_TEX_COORD_INTERPOLATORS
		return Interpolants.InterpolantsVSToPS.TexCoords[0].xy;
	#else
		return float2(0,0);
	#endif
	}

	FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
	{
		return Input.InterpolantsVSToPS;
	}

	FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
	{
		FVertexFactoryRayTracingInterpolants Interpolants;
		Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);
		return Interpolants;
	}

	FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
	{
		FVertexFactoryRayTracingInterpolants O;

		// Do we really need to interpolate TangentToWorld2 here? It should be replaced by the
		// interpolated normal from 'whatever' interpolation scheme we're using

		INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld0.xyz);
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld2);
#if INTERPOLATE_VERTEX_COLOR
		INTERPOLATE_MEMBER(InterpolantsVSToPS.VertexColor);
#endif

#if NEEDS_PARTICLE_COLOR
		INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleColor);
#endif

#if USE_PARTICLE_SUBUVS
		INTERPOLATE_MEMBER(InterpolantsVSToPS.SubUV0AndTexCoord0);
#elif NUM_TEX_COORD_INTERPOLATORS
		for (int i = 0; i < ( NUM_TEX_COORD_INTERPOLATORS + 1) / 2; ++i)
		{
			INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[i]);
		}
#endif

#if (DYNAMIC_PARAMETERS_MASK & 1)
		INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
		INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter1);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
		INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter2);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
		INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter3);
#endif

#if USE_PARTICLE_LOCAL_TO_WORLD
		INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleToWorld[0]);
		INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleToWorld[1]);
		INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleToWorld[2]);
#endif

#if USE_PARTICLE_WORLD_TO_LOCAL
		INTERPOLATE_MEMBER(InterpolantsVSToPS.WorldToParticle[0]);
		INTERPOLATE_MEMBER(InterpolantsVSToPS.WorldToParticle[1]);
		INTERPOLATE_MEMBER(InterpolantsVSToPS.WorldToParticle[2]);
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
		O.InterpolantsVSToPS.PrimitiveId = a.InterpolantsVSToPS.PrimitiveId;
#endif

		return O;
	}
#endif // #if NEEDS_VERTEX_FACTORY_INTERPOLATION

// RHI_RAYTRACING
#if RAYHITGROUPSHADER
	FVertexFactoryInput LoadVertexFactoryInputForHGS(uint TriangleIndex, int VertexIndex)
	{
		FVertexFactoryInput Input = (FVertexFactoryInput) 0;

		FTriangleBaseAttributes Tri = LoadTriangleBaseAttributes(TriangleIndex);
		uint VertexId = Tri.Indices[VertexIndex];

		// Fill FVertexFactoryInput with dummy values, the real ones will be fetched later from NiagaraMeshVF using InstanceId  
		Input.Position = float4(Tri.LocalPositions[VertexIndex], 1.0f);

#if PASS_NEEDS_VERTEX_ID || MANUAL_VERTEX_FETCH
		Input.VertexId = VertexId;
#endif // #if PASS_NEEDS_VERTEX_ID

#if (VF_USE_PRIMITIVE_SCENE_DATA == 1)
		const uint GPUSceneInstanceId = GetInstanceUserData();
		const FInstanceSceneData InstanceSceneData = GetInstanceSceneData(GPUSceneInstanceId);
		VF_GPUSCENE_SET_INPUT_FOR_RT(Input, GPUSceneInstanceId, InstanceSceneData.RelativeId);
#else
	#error "HGS requires GPU Scene support"
#endif

		return Input;
	}
#endif // #if COMPUTESHADER || RAYHITGROUPSHADER

#include "/Engine/Private/VertexFactoryDefaultInterface.ush"
