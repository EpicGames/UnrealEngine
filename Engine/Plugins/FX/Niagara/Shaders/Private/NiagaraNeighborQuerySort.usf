// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

// Counting sort passes for NiagaraDataInterfaceNeighborQuery.
// These run between the output stage (where user modules write cell assignments)
// and the read stage (where user modules query neighbors).
//
// Buffers:
//   CellIdBuffer     — [NumSlots] cell assignment per slot (written by user in output stage)
//   CellCountBuffer  — [NumCells] particles per cell (histogram output)
//   CellOffsetBuffer — [NumCells] exclusive prefix sum write offset per cell
//   ParticleListBuffer — [NumSlots] final sorted particle indices

#define NEIGHBORQUERY_INVALID_CELL 0xFFFFFFFF

// NQ_DIST_BITS is set via ModifyCompilationEnvironment from
// NiagaraNeighborQuerySort::NQDistBits (single source of truth in NiagaraNeighborQuerySort.h).
// CellIdBuffer slots store (CellId << NQ_DIST_BITS) | QuantizedDistToCenter.
// The bitonic scatter sorts by this combined key, producing closest-first order
// within each cell. All cell ID extractions below use >> NQ_DIST_BITS.

//////////////////////////////////////////////////////////////////////////
// Pass 2: Histogram — count particles per cell
//////////////////////////////////////////////////////////////////////////
#if NEIGHBOR_QUERY_HISTOGRAM_CS

// Define USE_BITONIC_HISTOGRAM to enable bitonic sort + coalesced atomics.
// When disabled, uses direct per-slot atomics (faster for sparse grids).
// #define USE_BITONIC_HISTOGRAM

Buffer<int>		CellIdBuffer;
RWBuffer<int>	CellCountBuffer;
uint			NumSlots;
uint			GroupsX;

#ifdef USE_BITONIC_HISTOGRAM
#if NQ_DIST_BITS > 0
#error "USE_BITONIC_HISTOGRAM is incompatible with NQ_DIST_BITS > 0 — run detection needs shifted comparisons"
#endif

groupshared int SortedCellIds[THREADGROUP_SIZE];

// Serial recursive bitonic sort (reference implementation, not used on GPU):
/*
void BitonicSortStep(uint ArrayStart, uint Length, bool IsAscending)
{
	if (Length > 1)
	{
		int k = Length / 2;
		BitonicSortStep(ArrayStart, k, true);
		BitonicSortStep(ArrayStart + k, k, false);
		BitonicMerge(ArrayStart, Length, IsAscending);
	}
}

void BitonicMerge(uint ArrayStart, uint Length, bool IsAscending)
{
	if (Length > 1)
	{
		int k = Length / 2;
		for (int i = ArrayStart; i < ArrayStart + k; ++i)
		{
			if (IsAscending == (SortedCellIds[i] > SortedCellIds[i + k]))
			{
				int Temp = SortedCellIds[i];
				SortedCellIds[i] = SortedCellIds[i + k];
				SortedCellIds[i + k] = Temp;
			}
		}
		BitonicMerge(ArrayStart, k, IsAscending);
		BitonicMerge(ArrayStart + k, k, IsAscending);
	}
}
*/

// Parallel bitonic sort of SortedCellIds in shared memory.
// All threads in the group must participate (even if they hold INVALID_CELL).
//
// Bitonic sort works by building increasingly large "bitonic sequences"
// (sequences that go up then down, or down then up) and merging them.
// The outer loop doubles the block size each iteration; the inner loop
// performs the merge steps within each block.
//
// For THREADGROUP_SIZE=64: 21 barrier steps. For 128: 28 barrier steps.
void BitonicSortShared(uint LocalId)
{
	// Outer loop: k is the size of the bitonic block being formed.
	// k=2: sort pairs. k=4: merge pairs into sorted quads. k=8: merge into octets. etc.
	// This corresponds to BitonicSortStep in the serial version — each doubling of k
	// is one level of the recursion.
	for (uint k = 2; k <= THREADGROUP_SIZE; k <<= 1)
	{
		// Inner loop: j is the distance between elements being compared.
		// Starts at k/2 and halves each step — this is the merge phase.
		// This corresponds to BitonicMerge in the serial version — each halving of j
		// is one level of the merge recursion.
		for (uint j = k >> 1; j > 0; j >>= 1)
		{
			// XOR with j flips the j-th bit of LocalId, giving us the partner
			// to compare with. Partners are always j positions apart.
			// Example: j=4, LocalId=5 (binary 101) → Partner=1 (binary 001)
			uint Partner = LocalId ^ j;

			// Only the lower-indexed thread of each pair does the compare-and-swap.
			// This avoids both threads swapping the same pair (which would undo the swap).
			if (Partner > LocalId)
			{
				// The k-th bit of LocalId determines sort direction for this block.
				// Alternating blocks sort in opposite directions to form the bitonic
				// sequences that the next level of k will merge.
				// Bit clear → ascending, bit set → descending.
				bool Ascending = ((LocalId & k) == 0);

				// Compare as unsigned so INVALID_CELL (0xFFFFFFFF) sorts to the end.
				// Swap if the pair is out of order for the desired direction:
				//   Ascending + [LocalId] > [Partner] → swap (want smaller first)
				//   Descending + [LocalId] <= [Partner] → swap (want larger first)
				if (Ascending == ((uint)SortedCellIds[LocalId] > (uint)SortedCellIds[Partner]))
				{
					int Temp = SortedCellIds[LocalId];
					SortedCellIds[LocalId] = SortedCellIds[Partner];
					SortedCellIds[Partner] = Temp;
				}
			}

			// All threads must complete this step before the next comparison round,
			// since the next step reads values that this step may have written.
			GroupMemoryBarrierWithGroupSync();
		}
	}
}

#endif // USE_BITONIC_HISTOGRAM

[numthreads(THREADGROUP_SIZE, 1, 1)]
void HistogramCS(uint3 GroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID)
{
	uint GroupIndex = GroupId.x + GroupId.y * GroupsX;
	uint LocalId = GroupThreadId.x;
	uint SlotIndex = GroupIndex * THREADGROUP_SIZE + LocalId;

#ifdef USE_BITONIC_HISTOGRAM
	// Bitonic sort path: sort cell IDs in shared memory, then one coalesced
	// atomic per unique cell per group. Wins when cells are dense (many
	// particles per cell), loses when sparse due to sort overhead.
	SortedCellIds[LocalId] = (SlotIndex < NumSlots) ? CellIdBuffer[SlotIndex] : (int)NEIGHBORQUERY_INVALID_CELL;
	GroupMemoryBarrierWithGroupSync();

	BitonicSortShared(LocalId);

	int MyCellId = SortedCellIds[LocalId];

	bool IsRunStart = (LocalId == 0) || (SortedCellIds[LocalId - 1] != MyCellId);
	if (IsRunStart && MyCellId != (int)NEIGHBORQUERY_INVALID_CELL)
	{
		uint RunCount = 1;
		for (uint j = LocalId + 1; j < THREADGROUP_SIZE && SortedCellIds[j] == MyCellId; j++)
		{
			RunCount++;
		}
		InterlockedAdd(CellCountBuffer[MyCellId], RunCount);
	}
#else
	// Direct atomic path: one InterlockedAdd per valid slot.
	// Simpler, faster when contention is low (sparse grids, many unique cells).
	if (SlotIndex >= NumSlots)
	{
		return;
	}

	int CombinedKey = CellIdBuffer[SlotIndex];
	if (CombinedKey != (int)NEIGHBORQUERY_INVALID_CELL)
	{
		uint CellId = (uint)CombinedKey >> NQ_DIST_BITS;
		InterlockedAdd(CellCountBuffer[CellId], 1);
	}
#endif
}

#endif

//////////////////////////////////////////////////////////////////////////
// Pass 3: Parallel exclusive prefix sum (recursive reduce-then-scan)
//
// Two shader sub-passes per recursion level:
//   3a. LocalScan  — each group Blelloch-scans ELEMENTS_PER_GROUP elements,
//                    writes tile total to BlockSumsBuffer
//   3b. Propagate  — each group adds its scanned block prefix to local results
//
// Block sums are scanned by recursively applying the same algorithm.
// Each thread processes ELEMENTS_PER_THREAD elements.
// Tile size = THREADGROUP_SIZE * ELEMENTS_PER_THREAD = ELEMENTS_PER_GROUP.
//////////////////////////////////////////////////////////////////////////

#if NEIGHBOR_QUERY_PREFIX_SUM_LOCAL_SCAN_CS

Buffer<int>		InputBuffer;
RWBuffer<int>	OutputBuffer;
RWBuffer<int>	BlockSumsBuffer;
uint			NumElements;
uint			GroupsX;
uint			NumGroups;

groupshared int SharedPartials[THREADGROUP_SIZE];

[numthreads(THREADGROUP_SIZE, 1, 1)]
void PrefixSumLocalScanCS(uint3 GroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID)
{
	uint GroupIndex = GroupId.x + GroupId.y * GroupsX;
	if (GroupIndex >= NumGroups)
	{
		return;
	}
	uint TileBase = GroupIndex * ELEMENTS_PER_GROUP;
	uint LocalId = GroupThreadId.x;

	// Phase 1: Each thread loads ELEMENTS_PER_THREAD elements and computes a
	// sequential exclusive prefix sum in registers. Store the per-thread total
	// into shared memory for the tree scan.
	int LocalPrefixes[ELEMENTS_PER_THREAD];
	int ThreadSum = 0;
	for (uint e = 0; e < ELEMENTS_PER_THREAD; e++)
	{
		uint GlobalIdx = TileBase + LocalId * ELEMENTS_PER_THREAD + e;
		int Val = (GlobalIdx < NumElements) ? InputBuffer[GlobalIdx] : 0;
		LocalPrefixes[e] = ThreadSum;
		ThreadSum += Val;
	}
	SharedPartials[LocalId] = ThreadSum;
	GroupMemoryBarrierWithGroupSync();

	// Phase 2: Blelloch exclusive scan on THREADGROUP_SIZE per-thread totals.
	// This correctly uses N threads for N elements (1:1 ratio).
	uint N = THREADGROUP_SIZE;

	// Up-sweep (reduce)
	for (uint Stride = 1; Stride < N; Stride <<= 1)
	{
		uint Idx = (LocalId + 1) * (Stride << 1) - 1;
		if (Idx < N)
		{
			SharedPartials[Idx] += SharedPartials[Idx - Stride];
		}
		GroupMemoryBarrierWithGroupSync();
	}

	// Store tile total and clear last element for down-sweep
	if (LocalId == 0)
	{
		BlockSumsBuffer[GroupIndex] = SharedPartials[N - 1];
		SharedPartials[N - 1] = 0;
	}
	GroupMemoryBarrierWithGroupSync();

	// Down-sweep (distribute)
	for (uint Stride2 = N >> 1; Stride2 >= 1; Stride2 >>= 1)
	{
		uint Idx = (LocalId + 1) * (Stride2 << 1) - 1;
		if (Idx < N)
		{
			int Temp = SharedPartials[Idx - Stride2];
			SharedPartials[Idx - Stride2] = SharedPartials[Idx];
			SharedPartials[Idx] += Temp;
		}
		GroupMemoryBarrierWithGroupSync();
	}

	// Phase 3: Combine the per-thread prefix (from the tree scan) with each
	// thread's local sequential prefix sums and write final results.
	int ThreadPrefix = SharedPartials[LocalId];
	for (uint e2 = 0; e2 < ELEMENTS_PER_THREAD; e2++)
	{
		uint GlobalIdx = TileBase + LocalId * ELEMENTS_PER_THREAD + e2;
		if (GlobalIdx < NumElements)
		{
			OutputBuffer[GlobalIdx] = LocalPrefixes[e2] + ThreadPrefix;
		}
	}
}

#endif

//////////////////////////////////////////////////////////////////////////
#if NEIGHBOR_QUERY_PREFIX_SUM_PROPAGATE_CS

Buffer<int>		BlockSumsBuffer;
RWBuffer<int>	OutputBuffer;
uint			NumElements;
uint			GroupsX;
uint			NumGroups;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void PrefixSumPropagateCS(uint3 GroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID)
{
	uint GroupIndex = GroupId.x + GroupId.y * GroupsX;
	// Group 0 has no prefix to add; excess groups from 2D dispatch must also exit
	if (GroupIndex == 0 || GroupIndex >= NumGroups)
	{
		return;
	}

	int BlockPrefix = BlockSumsBuffer[GroupIndex];
	uint TileBase = GroupIndex * ELEMENTS_PER_GROUP;
	uint LocalId = GroupThreadId.x;

	for (uint e = 0; e < ELEMENTS_PER_THREAD; e++)
	{
		uint GlobalIdx = TileBase + LocalId * ELEMENTS_PER_THREAD + e;
		if (GlobalIdx < NumElements)
		{
			OutputBuffer[GlobalIdx] += BlockPrefix;
		}
	}
}

#endif

//////////////////////////////////////////////////////////////////////////
// Pass 4: Scatter — write sorted particle indices
//////////////////////////////////////////////////////////////////////////
#if NEIGHBOR_QUERY_SCATTER_CS

// Define USE_BITONIC_SCATTER to enable bitonic sort + coalesced atomics.
// When disabled, uses direct per-slot atomics (simpler, faster for sparse grids).
#define USE_BITONIC_SCATTER

Buffer<int>		CellIdBuffer;
Buffer<int>		ParticleIdIndexBuffer;
RWBuffer<int>	CellOffsetBuffer;
RWBuffer<int>	ParticleListBuffer;
Buffer<int>		AcquireTagBuffer;
RWBuffer<int>	AcquireTagListBuffer;
uint			NumSlots;
uint			MaxCellsPerParticle;
uint			GroupsX;
uint			bHasPersistentIDBuffers;

#ifdef USE_BITONIC_SCATTER

// Parallel arrays in shared memory: cell IDs and original slot indices.
// After bitonic sort (keyed on cell ID, carrying slot index), each thread
// can derive its local offset within the run and write to a contiguous block
// claimed by one coalesced atomic per unique cell per group.
groupshared int ScatterSortedCellIds[THREADGROUP_SIZE];
groupshared uint ScatterSortedSlotIndices[THREADGROUP_SIZE];
groupshared uint RunWriteBase[THREADGROUP_SIZE];

// Parallel bitonic sort of ScatterSortedCellIds in shared memory, carrying
// ScatterSortedSlotIndices as satellite data (swap when keys swap).
// All threads in the group must participate.
void BitonicSortScatterShared(uint LocalId)
{
	for (uint k = 2; k <= THREADGROUP_SIZE; k <<= 1)
	{
		for (uint j = k >> 1; j > 0; j >>= 1)
		{
			uint Partner = LocalId ^ j;
			if (Partner > LocalId)
			{
				bool Ascending = ((LocalId & k) == 0);
				if (Ascending == ((uint)ScatterSortedCellIds[LocalId] > (uint)ScatterSortedCellIds[Partner]))
				{
					int TempCell = ScatterSortedCellIds[LocalId];
					ScatterSortedCellIds[LocalId] = ScatterSortedCellIds[Partner];
					ScatterSortedCellIds[Partner] = TempCell;

					uint TempSlot = ScatterSortedSlotIndices[LocalId];
					ScatterSortedSlotIndices[LocalId] = ScatterSortedSlotIndices[Partner];
					ScatterSortedSlotIndices[Partner] = TempSlot;
				}
			}
			GroupMemoryBarrierWithGroupSync();
		}
	}
}

#endif // USE_BITONIC_SCATTER

[numthreads(THREADGROUP_SIZE, 1, 1)]
void ScatterCS(uint3 GroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID)
{
	uint GroupIndex = GroupId.x + GroupId.y * GroupsX;
	uint LocalId = GroupThreadId.x;
	uint SlotIndex = GroupIndex * THREADGROUP_SIZE + LocalId;

#ifdef USE_BITONIC_SCATTER
	// Bitonic sort path: sort cell IDs in shared memory, then one coalesced
	// atomic per unique cell per group. Wins when cells are dense (many
	// particles per cell), loses when sparse due to sort overhead.

	// Load cell IDs and slot indices into shared memory
	// Threads past NumSlots load INVALID_CELL so they sort to the end
	int MyCellId = (SlotIndex < NumSlots) ? CellIdBuffer[SlotIndex] : (int)NEIGHBORQUERY_INVALID_CELL;
	ScatterSortedCellIds[LocalId] = MyCellId;
	ScatterSortedSlotIndices[LocalId] = SlotIndex;
	GroupMemoryBarrierWithGroupSync();

	// Sort by cell ID, carrying slot indices as satellite data
	BitonicSortScatterShared(LocalId);

	// After sort, read back this thread's (potentially moved) combined key and slot index.
	// Extract the true cell ID from the high bits; the low NQ_DIST_BITS bits carry the
	// quantized distance used to achieve closest-first order within each cell.
	MyCellId = ScatterSortedCellIds[LocalId];
	uint MySlotIndex = ScatterSortedSlotIndices[LocalId];
	bool IsValid = (MyCellId != (int)NEIGHBORQUERY_INVALID_CELL);
	uint MyExtractedCellId = (uint)MyCellId >> NQ_DIST_BITS;

	// Compute local offset within the run: distance from the run start.
	// Compare extracted cell IDs so all particles in the same NQ cell form one run
	// regardless of their different QuantizedDist values.
	uint LocalOffset = 0;
	if (IsValid)
	{
		for (uint j = LocalId; j > 0 && ((uint)ScatterSortedCellIds[j - 1] >> NQ_DIST_BITS) == MyExtractedCellId; j--)
		{
			LocalOffset++;
		}
	}

	// First thread of each run claims a contiguous block via one coalesced atomic
	// and writes the base position to shared memory for the rest of the run to read.
	bool IsRunStart = IsValid && ((LocalId == 0) || (((uint)ScatterSortedCellIds[LocalId - 1] >> NQ_DIST_BITS) != MyExtractedCellId));
	if (IsRunStart)
	{
		// Count run length forward (same extracted cell ID)
		uint RunCount = 1;
		for (uint k = LocalId + 1; k < THREADGROUP_SIZE && ((uint)ScatterSortedCellIds[k] >> NQ_DIST_BITS) == MyExtractedCellId; k++)
		{
			RunCount++;
		}

		// One atomic per unique cell per group — claim a contiguous block.
		// Particles land in ParticleListBuffer in closest-to-cell-center order
		// within the block (guaranteed by the combined-key sort above).
		int WriteBase;
		InterlockedAdd(CellOffsetBuffer[MyExtractedCellId], (int)RunCount, WriteBase);
		RunWriteBase[LocalId] = (uint)WriteBase;
	}
	GroupMemoryBarrierWithGroupSync();

	if (IsValid)
	{
		// Find the run start to read the base write position
		uint RunStartId = LocalId - LocalOffset;
		uint WritePos = RunWriteBase[RunStartId] + LocalOffset;

		// Write the particle ID index and AcquireTag to the pre-computed position — no atomic needed
		ParticleListBuffer[WritePos] = bHasPersistentIDBuffers
			? ParticleIdIndexBuffer[MySlotIndex]
			: (int)(MySlotIndex / max(MaxCellsPerParticle, 1u));
		if (bHasPersistentIDBuffers)
		{
			AcquireTagListBuffer[WritePos] = AcquireTagBuffer[MySlotIndex];
		}
	}
#else
	// Direct atomic path: one InterlockedAdd per valid slot.
	// Simpler, faster when contention is low (sparse grids, many unique cells).
	// NOTE: this path does NOT preserve closest-first order within cells.
	//       SDF inner-loop early-exit requires USE_BITONIC_SCATTER (enabled above).
	if (SlotIndex >= NumSlots)
	{
		return;
	}

	int CombinedKeyDirect = CellIdBuffer[SlotIndex];
	if (CombinedKeyDirect != (int)NEIGHBORQUERY_INVALID_CELL)
	{
		uint CellId = (uint)CombinedKeyDirect >> NQ_DIST_BITS;
		int WritePos;
		InterlockedAdd(CellOffsetBuffer[CellId], 1, WritePos);

		ParticleListBuffer[WritePos] = bHasPersistentIDBuffers
			? ParticleIdIndexBuffer[SlotIndex]
			: (int)(SlotIndex / max(MaxCellsPerParticle, 1u));
		if (bHasPersistentIDBuffers)
		{
			AcquireTagListBuffer[WritePos] = AcquireTagBuffer[SlotIndex];
		}
	}
#endif
}

#endif
