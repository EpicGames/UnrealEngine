// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NiagaraPhysicsCommon.ush
=============================================================================*/

/* -----------------------------------------------------------------
 * Shared memory for physics datas
 * -----------------------------------------------------------------
 */

groupshared float3 SharedNodePosition[THREADGROUP_SIZE];
groupshared float4 SharedNodeOrientation[THREADGROUP_SIZE];

groupshared float SharedInverseMass[THREADGROUP_SIZE];
groupshared float SharedInverseInertia[THREADGROUP_SIZE];

groupshared float3 SharedPreviousPosition[THREADGROUP_SIZE];
groupshared float4 SharedPreviousOrientation[THREADGROUP_SIZE];
