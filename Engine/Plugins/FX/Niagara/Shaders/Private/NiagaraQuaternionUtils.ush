// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define QUATERNION_IDENTITY float4(0,0,0,1)
#ifndef SMALL_NUMBER
	#define SMALL_NUMBER 1e-8
#endif

float4 NormalizeQuat(in float4 Quat)
{
	float SquaredNorm = dot(Quat,Quat);
	return (SquaredNorm >= SMALL_NUMBER) ? Quat / sqrt(SquaredNorm) : QUATERNION_IDENTITY;
}

float4 InverseQuat(in float4 Quat)
{
	return float4(-Quat.x,-Quat.y,-Quat.z,Quat.w);
}

float4 MultiplyQuat(in float4 QuatA, in float4 QuatB)
{
	return float4(
        QuatB.xyz * QuatA.w + QuatA.xyz * QuatB.w + cross(QuatA.xyz, QuatB.xyz),
        QuatA.w * QuatB.w - dot(QuatA.xyz, QuatB.xyz));
}

float3 RotateVectorByQuat(in float3 Vector, in float4 Quat)
{
	float3 T = 2.0 * cross(Quat.xyz,Vector);
	return Vector + Quat.w * T + cross(Quat.xyz,T);
}

float3 UnrotateVectorByQuat(in float3 Vector, in float4 Quat)
{
	float3 T = 2.0 * cross(Quat.xyz,Vector);
	return Vector - Quat.w * T + cross(Quat.xyz,T);
}

float4 SlerpQuat(in float4 QuatA, in float4 QuatB, float Slerp)
{
	// Get cosine of angle between quats.
	const float RawCosom = 
		    QuatA.x * QuatB.x +
			QuatA.y * QuatB.y +
			QuatA.z * QuatB.z +
			QuatA.w * QuatB.w;
	// Unaligned quats - compensate, results in taking shorter route.
	const float Cosom = (RawCosom >= 0.0) ? RawCosom : -RawCosom;
	
	float Scale0, Scale1;

	if( Cosom < 0.9999f )
	{	
		const float Omega = acos(Cosom);
		const float InvSin = 1.f/sin(Omega);
		Scale0 = sin( (1.f - Slerp) * Omega ) * InvSin;
		Scale1 = sin( Slerp * Omega ) * InvSin;
	}
	else
	{
		// Use linear interpolation.
		Scale0 = 1.0f - Slerp;
		Scale1 = Slerp;	
	}

	// In keeping with our flipped Cosom:
	Scale1 = (RawCosom >= 0.0) ? Scale1 : -Scale1; 

	float4 Result;
		
	Result.x = Scale0 * QuatA.x + Scale1 * QuatB.x;
	Result.y = Scale0 * QuatA.y + Scale1 * QuatB.y;
	Result.z = Scale0 * QuatA.z + Scale1 * QuatB.z;
	Result.w = Scale0 * QuatA.w + Scale1 * QuatB.w;

	return NormalizeQuat(Result);
}

float4 FindQuatBetweenHelper(in float3 A, in float3 B, in float NormAB)
{
	float W = NormAB + dot(A,B);
	float4 Quat = (W>1e-6*NormAB) ? float4(A.y*B.z-A.z*B.y,A.z*B.x-A.x*B.z,A.x*B.y-A.y*B.x,W) :
			(abs(A.x) > abs(A.y)) ? float4(-A.z,0.0,A.x,0.0) : float4(0.0,-A.z,A.y,0.0);

	return NormalizeQuat(Quat);
}

float4 FindQuatBetweenInternal(in float3 An, in float3 Bn)
{
	float3 Cn = normalize(An+Bn);
	return float4(cross(An,Cn),dot(An,Cn));
}

float4 FindQuatBetweenNormals(in float3 NormalA, in float3 NormalB)
{
	float NormAB = 1.0;
	return FindQuatBetweenHelper(NormalA,NormalB,NormAB);
}

float4 FindQuatBetweenVectors(in float3 VectorA, in float3 VectorB)
{
	float NormAB = sqrt(dot(VectorA,VectorA) * dot(VectorB,VectorB));
	return FindQuatBetweenHelper(VectorA,VectorB,NormAB);
}

float4 QuatFromMatrix(in float3 R[3])
{
	float4 Q = 0;
	float s;
	float tr = R[0][0] + R[1][1] + R[2][2];

	if (tr > 0.0f) 
	{
		float InvS = 1.0 / sqrt(tr + 1.f);
		s = 0.5f * InvS;

		Q.w = 0.5f * (1.f / InvS);
		Q.x = (R[1][2] - R[2][1]) * s;
		Q.y = (R[2][0] - R[0][2]) * s;
		Q.z = (R[0][1] - R[1][0]) * s;
	} 
	else 
	{
		uint i = 0;

		if (R[1][1] > R[0][0])
			i = 1;

		if (R[2][2] > R[i][i])
			i = 2;

		const uint j = (i+1)%3;
		const uint k = (j+1)%3;

		tr = R[i][i] - R[j][j] - R[k][k];
		float InvS = 1.0 / sqrt(tr + 1.0);
		s = 0.5 * InvS;

		const float Qi = 0.5 * (1.0 / InvS);
		const float Qj = (R[i][j] + R[j][i]) * s;
		const float Qk = (R[i][k] + R[k][i]) * s;

		Q.w = (R[j][k] - R[k][j]) * s;
		if(i == 0)
		{
			Q.x = Qi;
			Q.y = Qj;
			Q.z = Qk;
		}
		else if(i == 1)
		{
			Q.y = Qi;
			Q.z = Qj;
			Q.x = Qk;
		}
		else if(i == 2)
		{
			Q.z = Qi;
			Q.x = Qj;
			Q.y = Qk;
		}
	}
	return NormalizeQuat(Q);
}

float4 RotatorToQuat(float3 Rotator)
{
	Rotator = frac(Rotator) * 3.1415926535897932f;
	float SP, CP, SY, CY, SR, CR;
	sincos(Rotator.x, SR, CR);
	sincos(Rotator.y, SP, CP);
	sincos(Rotator.z, SY, CY);
	
	return float4(
		 CR*SP*SY - SR*CP*CY,
		-CR*SP*CY - SR*CP*SY,
		 CR*CP*SY - SR*SP*CY,
		 CR*CP*CY + SR*SP*SY
	);
}
