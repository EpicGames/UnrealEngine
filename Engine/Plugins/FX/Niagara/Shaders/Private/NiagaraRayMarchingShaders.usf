// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/DistanceField/GlobalDistanceFieldShared.ush"
#include "NiagaraAsyncGpuTraceCommon.ush"

#if !defined(THREADGROUP_SIZE_X)
	#define THREADGROUP_SIZE_X 32
#endif

#if !defined(MAX_MARCH_COUNT)
	#define MAX_MARCH_COUNT 16
#endif

StructuredBuffer<FNiagaraAsyncGpuTrace> Traces;
Buffer<uint> TraceCounts;
RWStructuredBuffer<FNiagaraAsyncGpuTraceResult> Results;

uint TracesOffset;
uint TraceCountsOffset;
uint ResultsOffset;
float DistanceThreshold;

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void NiagaraRayMarchGlobalSdfCS(uint GroupIndex : SV_GroupIndex)
{
	if (GroupIndex >= TraceCounts[TraceCountsOffset + 0])
	{
		return;
	}

	// read the offset request
	FNiagaraAsyncGpuTrace Trace = Traces[TracesOffset + GroupIndex];

	FNiagaraAsyncGpuTraceResult Result = (FNiagaraAsyncGpuTraceResult) 0;
	Result.WorldPosition = Trace.Origin;
	Result.HitT = 0.0f;

	float3 TranslatedWorldPosition = Result.WorldPosition + DFHackToFloat(PrimaryView.PreViewTranslation); // LWC_TODO

	int MarchCount = 0;
	for (MarchCount = 0; MarchCount < MAX_MARCH_COUNT; ++MarchCount)
	{
		const float CurrentDistance = GetDistanceToNearestSurfaceGlobal(TranslatedWorldPosition);
		Result.HitT += CurrentDistance;
		TranslatedWorldPosition = Trace.Origin + Trace.Direction * min(Result.HitT, Trace.TFar);

		if (CurrentDistance < DistanceThreshold)
		{
			break;
		}
		if (Result.HitT > Trace.TFar)
		{
			// we've marched passed the end of the ray and haven't found anything
			Result.HitT = -1;
			break;
		}
	}

	// if we exceeded our iteration count
	if (MarchCount == MAX_MARCH_COUNT)
	{
		Result.HitT = -1;
	}

	// if we have a hit then generate a normal
	if (Result.HitT >= 0.0f)
	{
		Result.WorldNormal = normalize(GetDistanceFieldGradientGlobal(TranslatedWorldPosition));
	}

	Result.WorldPosition = TranslatedWorldPosition - DFHackToFloat(PrimaryView.PreViewTranslation); // LWC_TODO

	// write the offset result
	Results[ResultsOffset + GroupIndex] = Result;
}