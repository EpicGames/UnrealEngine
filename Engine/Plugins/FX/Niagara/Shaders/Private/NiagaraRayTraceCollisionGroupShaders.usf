// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/HashTable.ush"


RWBuffer<uint> RWHashToCollisionGroups;

Buffer<uint2> NewPrimIdCollisionGroupPairs;
uint NumNewPrims;

[numthreads(THREAD_COUNT, 1, 1)]
void UpdatePrimIdToCollisionGroupMap(uint Index : SV_DispatchThreadID)
{
	if (Index < NumNewPrims)
	{
		uint PrimId = NewPrimIdCollisionGroupPairs[Index].x;
		uint CollisionGroupId = NewPrimIdCollisionGroupPairs[Index].y;
		uint CollisionGroupIdIndex = MurmurMix(PrimId);

		//TODO: Handle failures.
		if (PrimId != -1)
		{
			if(HashTableAdd(PrimId, CollisionGroupIdIndex))
			{
				RWHashToCollisionGroups[CollisionGroupIdIndex] = CollisionGroupId;
			}
		}
	}
}