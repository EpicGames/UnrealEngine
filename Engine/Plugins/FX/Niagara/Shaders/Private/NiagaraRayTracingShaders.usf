// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/SceneData.ush"
#include "/Engine/Private/ShadingCommon.ush"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/RayTracing/RayTracingHitGroupCommon.ush"
#include "NiagaraAsyncGpuTraceCommon.ush"
#include "/Engine/Private/HashTable.ush"
#include "/Engine/Private/RayTracing/VFXTraceRay.ush"

RaytracingAccelerationStructure					TLAS;
StructuredBuffer<FNiagaraAsyncGpuTrace>			Rays;
uint											RaysOffset;
RWStructuredBuffer<FNiagaraAsyncGpuTraceResult>	CollisionOutput;
uint											CollisionOutputOffset;

#if NIAGARA_RAYTRACE_FAKE_INDIRECT || COMPUTESHADER
Buffer<uint> RayTraceCounts;
uint RayTraceCountsOffset;
#endif

Buffer<uint> HashToCollisionGroups;
uint MaxRetraces;

#if NIAGARA_SUPPORTS_COLLISION_GROUPS
bool FindCollisionGroup(int GPUSceneIndexId, out int CollisionGroup)
{
	uint Index = MurmurMix(GPUSceneIndexId);
	if (HashTableFind(GPUSceneIndexId, Index))
	{
		CollisionGroup = HashToCollisionGroups[Index];
		return true;
	}
	CollisionGroup = 0;
	return false;
}
#endif

#if COMPUTESHADER
[numthreads(RAY_TRACING_THREAD_GROUP_SIZE_X, 1, 1)]
void NiagaraCollisionRayTraceCS(uint3 DispatchThreadIndex : SV_DispatchThreadID, uint3 DispatchGroupId : SV_GroupID, uint DispatchGroupIndex : SV_GroupIndex)
{
	const uint RayIndex = DispatchThreadIndex.x;
#else
RAY_TRACING_ENTRY_RAYGEN(NiagaraCollisionRayTraceRG)
{
	const uint RayIndex = DispatchRaysIndex().x;
#endif

#if NIAGARA_RAYTRACE_FAKE_INDIRECT || COMPUTESHADER
	if (RayIndex >= RayTraceCounts[RayTraceCountsOffset]) 
	{
		return;
	}
#endif

	FNiagaraAsyncGpuTrace InputRay = Rays[RayIndex + RaysOffset];

	FRayDesc Ray;
	Ray.Origin = InputRay.Origin + DFHackToFloat(PrimaryView.PreViewTranslation); // LWC_TODO
	Ray.Direction = InputRay.Direction;
	Ray.TMin = 0.0f;
	Ray.TMax = InputRay.TFar;

	uint RayFlags = RAY_FLAG_FORCE_OPAQUE | RAY_FLAG_CULL_BACK_FACING_TRIANGLES; // don't run anyhit shader

	FVFXTracePayload Payload = (FVFXTracePayload) 0;
	uint RayCollisionGroup = InputRay.CollisionGroup;

#if NIAGARA_SUPPORTS_COLLISION_GROUPS
	if (RayCollisionGroup != -1)
	{
		uint NumCasts = 0;
		LOOP
		while (NumCasts++ <= MaxRetraces)
		{
			Payload = (FVFXTracePayload)0;
			#if COMPUTESHADER
				VfxTraceRayInline(
					TLAS,   // AccelerationStructure
					RayFlags,
					RAY_TRACING_MASK_OPAQUE,          // InstanceInclusionMask
					Ray.GetNativeDesc(),    // FRayDesc
					Payload // Payload
				);
			#else
				VfxTraceRay(
					TLAS,   // AccelerationStructure
					RayFlags,
					RAY_TRACING_MASK_OPAQUE,          // InstanceInclusionMask
					RAY_TRACING_SHADER_SLOT_MATERIAL, // RayContributionToHitGroupIndex
					RAY_TRACING_NUM_SHADER_SLOTS,     // MultiplierForGeometryContributionToShaderIndex
					0,      // MissShaderIndex
					Ray.GetNativeDesc(),    // RayDesc
					Payload // Payload
				); 
			#endif

			uint PrimCollisionGroup; 
			if (FindCollisionGroup(Payload.GPUSceneInstanceId, PrimCollisionGroup))
			{
				//If we collide with something of a different group to the Ray then break and allow the collision. Otherwise loop and attempt to recast just infront of the hit we're discarding.
				if (PrimCollisionGroup == RayCollisionGroup)
				{
					Ray.TMin = asfloat(asuint(Payload.HitT) + 1);
					//Clear the hit state on the payload in case we don't attempt a retrace
					Payload.HitT = -1;
				}
				else
				{
					break;
				}
			}
		}
	}
	else
#endif
	{
		#if COMPUTESHADER
			VfxTraceRayInline(
				TLAS,   // AccelerationStructure
				RayFlags,
				RAY_TRACING_MASK_OPAQUE,          // InstanceInclusionMask
				Ray.GetNativeDesc(),    // FRayDesc
				Payload // Payload
			);
		#else
			VfxTraceRay(
				TLAS,   // AccelerationStructure
				RayFlags,
				RAY_TRACING_MASK_OPAQUE,          // InstanceInclusionMask
				RAY_TRACING_SHADER_SLOT_MATERIAL, // RayContributionToHitGroupIndex
				RAY_TRACING_NUM_SHADER_SLOTS,     // MultiplierForGeometryContributionToShaderIndex
				0,      // MissShaderIndex
				Ray.GetNativeDesc(),    // FRayDesc
				Payload // Payload
			);
		#endif
	}

	uint OutputIndex = RayIndex + CollisionOutputOffset;
	CollisionOutput[OutputIndex] = (FNiagaraAsyncGpuTraceResult) 0;
	CollisionOutput[OutputIndex].HitT = Payload.HitT;
	if (Payload.IsHit())
	{
		CollisionOutput[OutputIndex].WorldPosition = Payload.TranslatedWorldPosition - DFHackToFloat(PrimaryView.PreViewTranslation); // LWC_TODO
		CollisionOutput[OutputIndex].WorldNormal = Payload.WorldNormal;
	}
}

RAY_TRACING_ENTRY_CLOSEST_HIT(NiagaraCollisionRayTraceCH,
	FVFXTracePayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	InitVFXTracePayloadFromHitShader(Attributes, Payload);
}

RAY_TRACING_ENTRY_MISS(NiagaraCollisionRayTraceMiss, FVFXTracePayload, Payload)
{
	Payload.SetMiss();
}
