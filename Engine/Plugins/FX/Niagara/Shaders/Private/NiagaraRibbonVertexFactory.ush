// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleBeamTrailVertexFactory.hlsl: Particle vertex factory shader code.
=============================================================================*/

#include "/Engine/Private/VertexFactoryCommon.ush"
#include "NiagaraVFCommon.usf"

#define USE_PARTICLE_POSITION (NEEDS_PARTICLE_POSITION)
#define USE_PARTICLE_VELOCITY (NEEDS_PARTICLE_VELOCITY)
#define USE_PARTICLE_TIME (NEEDS_PARTICLE_TIME)
#define USE_PARTICLE_SIZE (NEEDS_PARTICLE_SIZE)
#define USE_PARTICLE_RANDOM (NEEDS_PARTICLE_RANDOM)
#define USE_PARTICLE_INTERPOLATION 1

#define U_DISTRIBUTION_SCALED_UNIFORMLY 0
#define U_DISTRIBUTION_SCALED_USING_RIBBON_LENGTH 1
#define U_DISTRIBUTION_TILED_OVER_RIBBON_LENGTH 2
#define U_DISTRIBUTION_TILED_FROM_START_OVER_RIBBON_LENGTH 3

#define SLICEVERTEXDATA_POSITION_OFFSET 0
#define SLICEVERTEXDATA_NORMAL_OFFSET 2
#define SLICEVERTEXDATA_TEXTUREV_OFFSET 4
#define SLICEVERTEXDATA_STRIDE 5

#define INDEX_GEN_DRAW_INDIRECT_OFFSET				3		// Duplicated in NiagaraRibbonCommon.ush

#define INDEX_GEN_NUM_SUB_SEGMENTS_OFFSET			15		// Duplicated in NiagaraRibbonCommon.ush
#define INDEX_GEN_ONE_OVER_NUM_SUB_SEGMENTS_OFFSET	16		// Duplicated in NiagaraRibbonCommon.ush

uint UnpackParticleId(uint InterpIdx)
{
	return (InterpIdx >> NiagaraRibbonVF.ParticleIdShift) & NiagaraRibbonVF.ParticleIdMask;
}

uint UnpackInterpSlice(uint InterpIdx)
{
	return (InterpIdx >> NiagaraRibbonVF.InterpIdShift) & NiagaraRibbonVF.InterpIdMask;	
}

uint UnpackParticleVertexId(uint InterpIdx)
{
	return InterpIdx & NiagaraRibbonVF.SliceVertexIdMask;
}

float2 UnpackParticleVertexPosition(uint InterpIdx)
{
	uint SliceVertexPositionIndex = UnpackParticleVertexId(InterpIdx) * SLICEVERTEXDATA_STRIDE + SLICEVERTEXDATA_POSITION_OFFSET;
		
	return float2(NiagaraRibbonVFLooseParameters.SliceVertexData[SliceVertexPositionIndex],	NiagaraRibbonVFLooseParameters.SliceVertexData[SliceVertexPositionIndex + 1]);
}

float2 UnpackParticleVertexNormal(uint InterpIdx)
{
	uint SliceVertexNormalIndex = UnpackParticleVertexId(InterpIdx) * SLICEVERTEXDATA_STRIDE + SLICEVERTEXDATA_NORMAL_OFFSET;
		
	return float2(NiagaraRibbonVFLooseParameters.SliceVertexData[SliceVertexNormalIndex], NiagaraRibbonVFLooseParameters.SliceVertexData[SliceVertexNormalIndex + 1]);
}

float UnpackParticleVertexTextureV(uint InterpIdx)
{
	return NiagaraRibbonVFLooseParameters.SliceVertexData[UnpackParticleVertexId(InterpIdx) * SLICEVERTEXDATA_STRIDE + SLICEVERTEXDATA_TEXTUREV_OFFSET];
}

FLWCVector3 TransformPosition(FLWCMatrix LocalToWorld, float3 InPosition)
{
	if (NiagaraRibbonVF.bLocalSpace)
	{
		return LWCMultiply(InPosition, LocalToWorld);
	}
	else
	{
		return MakeLWCVector3(NiagaraRibbonVF.SystemLWCTile, InPosition);
	}
}

float3 TransformVector(FLWCMatrix LocalToWorld, float3 InvScale, float3 InVector)
{
	if (NiagaraRibbonVF.bLocalSpace)
	{
		return RotateLocalToWorld(InVector, LWCToFloat3x3(LocalToWorld), InvScale);
	}
	else
	{
		return InVector;
	}
}

float3 CubicInterp(float3 P0, float3 T0, float3 P1, float3 T1, float A, float TS)
{
	const float A2 = A  * A;
	const float A3 = A2 * A;
	return (((2*A3)-(3*A2)+1) * P0) + (TS * (A3-(2*A2)+A) * T0) + (TS * (A3-A2) * T1) + (((-2*A3)+(3*A2)) * P1);
}

/*********************************************************************************
 *							Particle specific
 *********************************************************************************/
 
// Naming convention :
// RawParticleId : Used to access per particle data stored in the VF. Actual particle data needs to be indirected through SortedIndices.
// ParticleId : Used to access particle data in NiagaraVFParticleAccess.
// VertexId : Used to represent a vertex on both side of the particles. Has some specific data like UV.
// InterpVtxId : A virtual index that represent a vertex of or between 2 VertexId.

struct FVertexFactoryInput
{
	uint InterpVtxId					: SV_VertexID;

	// Optional instance ID for vertex layered rendering
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && SUPPORT_SHADOW_DEPTH_POINT && USING_VERTEX_SHADER_LAYER
	#undef GetInstanceIdFromVF
	#define GetInstanceIdFromVF(VFInput)				(VFInput.InstanceId)
	uint InstanceId					: SV_InstanceID;
#else
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
#endif
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()	

};

// The interpolation data required for an interp vertex.
struct FInterpVtxData
{
	uint RawParticleId0;
	uint RawParticleId1;
	uint ParticleId0;
	uint ParticleId1;
	float Alpha;
};

struct FSegmentData
{
	float2 UV0Scale;
	float2 UV0Offset;
	float2 UV1Scale;
	float2 UV1Offset;
	float U0DistributionScaler;
	float U1DistributionScaler;
	uint StartParticleId;
	uint LastParticleId;
};

struct FRibbonPositionIntermediates
{
	FLWCVector3 Position;
	float3 LocalPosition;
	float3 Direction;
	float DistanceOnSegment;
	float RibbonWidth;
	
	float3 GeoUp;
	float3 GeoRight;
	
	float3 TangentX;
	float3 TangentY;
	float3 TangentZ;
};

struct FVertexFactoryIntermediates
{
	FRibbonPositionIntermediates Current;
	FRibbonPositionIntermediates Previous;
	float4 Color;
	
	float4 TexCoord;
	
#if USE_PARTICLE_POSITION
	float4 TranslatedWorldPositionAndSize;
#endif

	float4 ParticleVelocity;

#if USE_PARTICLE_TIME
	float RelativeTime;
#endif

#if USE_PARTICLE_SIZE
	float2 ParticleSize;
#endif

#if USE_PARTICLE_RANDOM
	float ParticleRandom;
#endif

#if (DYNAMIC_PARAMETERS_MASK & 1) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	float4 DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	float4 DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	float4 DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	float4 DynamicParameter3;
#endif
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

struct FVertexFactoryInterpolantsVSToPS
{
	TANGENTTOWORLD_INTERPOLATOR_BLOCK

#if (DYNAMIC_PARAMETERS_MASK & 1)
	float4 DynamicParameter	: PARTICLE_DYNAMIC_PARAM0;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	float4 DynamicParameter1	: PARTICLE_DYNAMIC_PARAM1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	float4 DynamicParameter2	: PARTICLE_DYNAMIC_PARAM2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	float4 DynamicParameter3	: PARTICLE_DYNAMIC_PARAM3;
#endif

#if NEEDS_PARTICLE_COLOR
	float4  Color : TEXCOORD0;
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	float4	TexCoords[(NUM_TEX_COORD_INTERPOLATORS + 1) / 2] : TEXCOORD1;
#endif

#if USE_PARTICLE_POSITION
	/** Cam-relative (translated) particle center and radius */
	float4 TranslatedWorldPositionAndSize : PARTICLE_POSITION;
#endif

#if USE_PARTICLE_VELOCITY
	float4 ParticleVelocity : PARTICLE_VELOCITY;
#endif

#if USE_PARTICLE_TIME
	float RelativeTime : PARTICLE_TIME;
#endif

#if USE_PARTICLE_SIZE
	float2 ParticleSize : PARTICLE_SIZE;
#endif

#if USE_PARTICLE_RANDOM
	float ParticleRandom : PARTICLE_RANDOM;
#endif
};

#if NUM_TEX_COORD_INTERPOLATORS
bool UVIndexUseZW(int UVIndex)
{
	return (UVIndex % 2) != 0;
}

float2 GetUV(FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex)
{
	float4 UVVector = Interpolants.TexCoords[UVIndex / 2];
	return UVIndexUseZW(UVIndex) ? UVVector.zw : UVVector.xy;
}

void SetUV(inout FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex, float2 InValue)
{
	FLATTEN
	if (UVIndexUseZW(UVIndex))
	{
		Interpolants.TexCoords[UVIndex / 2].zw = InValue;
	}
	else
	{
		Interpolants.TexCoords[UVIndex / 2].xy = InValue;
	}
}

#endif

uint GetInterpCount()
{
	BRANCH
	if (NiagaraRibbonVFLooseParameters.IndirectDrawOutputOffset != -1)
	{
		return NiagaraRibbonVFLooseParameters.IndirectDrawOutput[NiagaraRibbonVFLooseParameters.IndirectDrawOutputOffset + INDEX_GEN_NUM_SUB_SEGMENTS_OFFSET];
	}
	else
	{
		return NiagaraRibbonVF.InterpCount;
	}
}

float GetOneOverInterpCount()
{
	BRANCH
	if (NiagaraRibbonVFLooseParameters.IndirectDrawOutputOffset != -1)
	{
		return asfloat(NiagaraRibbonVFLooseParameters.IndirectDrawOutput[NiagaraRibbonVFLooseParameters.IndirectDrawOutputOffset + INDEX_GEN_ONE_OVER_NUM_SUB_SEGMENTS_OFFSET]);
	}
	else
	{
		return NiagaraRibbonVF.OneOverInterpCount;
	}
}

void GetInterpVtxAndRibbonDataData(FVertexFactoryInput Input, out FInterpVtxData InterpVtxData, out FSegmentData SegmentData)
{
	// Get First Particle Data
	InterpVtxData.RawParticleId0 = UnpackParticleId(Input.InterpVtxId);
	InterpVtxData.ParticleId0 = NiagaraRibbonVFLooseParameters.SortedIndices[InterpVtxData.RawParticleId0];

	// Unpack Ribbon Data
	const uint RibbonIndex = NiagaraRibbonVF.ShouldUseMultiRibbon == 1 ? NiagaraRibbonVFLooseParameters.MultiRibbonIndices[InterpVtxData.RawParticleId0] : 0;
	const uint RibbonBufferIndex = RibbonIndex * 8;

	SegmentData.UV0Scale = float2(asfloat(NiagaraRibbonVFLooseParameters.PackedPerRibbonDataByIndex[RibbonBufferIndex + 0]), NiagaraRibbonVF.PackedVData.x);
	SegmentData.UV0Offset = float2(asfloat(NiagaraRibbonVFLooseParameters.PackedPerRibbonDataByIndex[RibbonBufferIndex + 1]), NiagaraRibbonVF.PackedVData.y);
	SegmentData.U0DistributionScaler = asfloat(NiagaraRibbonVFLooseParameters.PackedPerRibbonDataByIndex[RibbonBufferIndex + 2]);
	
	SegmentData.UV1Scale = float2(asfloat(NiagaraRibbonVFLooseParameters.PackedPerRibbonDataByIndex[RibbonBufferIndex + 3]), NiagaraRibbonVF.PackedVData.z);
	SegmentData.UV1Offset = float2(asfloat(NiagaraRibbonVFLooseParameters.PackedPerRibbonDataByIndex[RibbonBufferIndex + 4]), NiagaraRibbonVF.PackedVData.w);
	SegmentData.U1DistributionScaler = asfloat(NiagaraRibbonVFLooseParameters.PackedPerRibbonDataByIndex[RibbonBufferIndex + 5]);

	SegmentData.StartParticleId = NiagaraRibbonVFLooseParameters.PackedPerRibbonDataByIndex[RibbonBufferIndex + 6];
	SegmentData.LastParticleId = NiagaraRibbonVFLooseParameters.PackedPerRibbonDataByIndex[RibbonBufferIndex + 7];

	// Get Remaining Interp Data
	InterpVtxData.RawParticleId1 = min(InterpVtxData.RawParticleId0 + 1, SegmentData.LastParticleId);
	InterpVtxData.ParticleId1 = NiagaraRibbonVFLooseParameters.SortedIndices[InterpVtxData.RawParticleId1];

#if USE_PARTICLE_INTERPOLATION
	{
		InterpVtxData.Alpha = float(UnpackInterpSlice(Input.InterpVtxId)) * GetOneOverInterpCount();
	}
#else
	InterpVtxData.Alpha = 0;
#endif
}

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = GetUV(Interpolants, CoordinateIndex);
	}
#endif	// NUM_MATERIAL_TEXCOORDS

	Result.VertexColor = 1;

#if NEEDS_PARTICLE_COLOR
	Result.Particle.Color = Interpolants.Color;
#endif

	half3 TangentToWorld0 = Interpolants.TangentToWorld0.xyz;
	half4 TangentToWorld2 = Interpolants.TangentToWorld2;
	Result.TangentToWorld = AssembleTangentToWorld( TangentToWorld0, TangentToWorld2 );
	Result.UnMirrored = 1;

	Result.TwoSidedSign = 1;

#if (DYNAMIC_PARAMETERS_MASK != 0)
	Result.Particle.DynamicParameterValidMask = NiagaraRibbonVF.MaterialParamValidMask;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 1)
	Result.Particle.DynamicParameter = Interpolants.DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	Result.Particle.DynamicParameter1 = Interpolants.DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	Result.Particle.DynamicParameter2 = Interpolants.DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	Result.Particle.DynamicParameter3 = Interpolants.DynamicParameter3;
#endif

#if USE_PARTICLE_POSITION
	Result.Particle.TranslatedWorldPositionAndSize = Interpolants.TranslatedWorldPositionAndSize;
	Result.Particle.PrevTranslatedWorldPositionAndSize = Result.Particle.TranslatedWorldPositionAndSize;
#endif

#if USE_PARTICLE_VELOCITY
	Result.Particle.Velocity = Interpolants.ParticleVelocity;
#endif

#if USE_PARTICLE_TIME
	Result.Particle.RelativeTime = Interpolants.RelativeTime;
#endif

#if USE_PARTICLE_SIZE
	Result.Particle.Size = Interpolants.ParticleSize;
#endif

#if USE_PARTICLE_RANDOM
	Result.Particle.Random = Interpolants.ParticleRandom;
#else
	Result.Particle.Random = 0.0f;
#endif

	return Result;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input,
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.VertexColor = Intermediates.Color;
	Result.TangentToWorld = mul(TangentToLocal, GetLocalToWorld3x3()); 
	Result.Particle.Color = Intermediates.Color;
	Result.PreSkinnedPosition = LWCToFloat(Intermediates.Current.Position);
	Result.PreSkinnedNormal = TangentToLocal[2].xyz;

#if (DYNAMIC_PARAMETERS_MASK != 0)
	Result.Particle.DynamicParameterValidMask = NiagaraRibbonVF.MaterialParamValidMask;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 1)
	Result.Particle.DynamicParameter = Intermediates.DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	Result.Particle.DynamicParameter1 = Intermediates.DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	Result.Particle.DynamicParameter2 = Intermediates.DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	Result.Particle.DynamicParameter3 = Intermediates.DynamicParameter3;
#endif

#if USE_PARTICLE_POSITION
	Result.Particle.TranslatedWorldPositionAndSize = Intermediates.TranslatedWorldPositionAndSize;
	Result.Particle.PrevTranslatedWorldPositionAndSize = Result.Particle.TranslatedWorldPositionAndSize;
#endif

#if USE_PARTICLE_VELOCITY
	Result.Particle.Velocity = Intermediates.ParticleVelocity;
#endif

#if USE_PARTICLE_TIME
	Result.Particle.RelativeTime = Intermediates.RelativeTime;
#endif

#if USE_PARTICLE_SIZE
	Result.Particle.Size = Intermediates.ParticleSize;
#endif

#if USE_PARTICLE_RANDOM
	Result.Particle.Random = Intermediates.ParticleRandom;
#endif
	Result.PrevFrameLocalToWorld = GetPrimitiveDataFromUniformBuffer().PreviousLocalToWorld;

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	Result.TexCoords[0] = Intermediates.TexCoord.xy;
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
		Result.TexCoords[1] = Intermediates.TexCoord.zw;
	#endif
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

	#if ENABLE_NEW_HLSL_GENERATOR
	// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
	EvaluateVertexMaterialAttributes(Result);
#endif

	return Result;
}

float3 SafeNormalize(float3 V, float3 Default)
{
	const float len2 = dot(V, V);
	if ( len2 > 0.0f )
	{
		return V * rsqrt(len2);
	}
	return Default;
}

float4 GetDirectionAndMagnitude(float3 V)
{
	const float MagSquared = dot(V, V);
	if (MagSquared > 0)
	{
		const float MagScale = rsqrt(MagSquared);
		return float4(MagScale * V, rcp(MagScale));
	}

	return float4(0.0f, 0.0f, 0.0f, 0.0f);
}

/** derive basis vectors */
float3x3 CalcTangentBasis(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{	
	return float3x3(Intermediates.Current.TangentX, Intermediates.Current.TangentY, Intermediates.Current.TangentZ);
}

float GetTangentAndDistanceComponent(int RawParticleID, int ElementIndex)
{
	return NiagaraRibbonVFLooseParameters.TangentsAndDistances[RawParticleID * 4 + ElementIndex];
}

// known limitations:
// -TangentsAndDistances is from the current frame and if we're generating intermediates for the previous frame for velocity
// rendering we won't get the exact same results.  Issues around trying to supply a previous version of the TangentsAndDistances
// buffer include the fact that the active particles can differ between frames and so we'd also need to supply particle indices
// for the previous frames as well
FRibbonPositionIntermediates GetRibbonPositionIntermediates(
	FInterpVtxData InterpVtxData,
	int PositionDataOffset,
	int FacingDataOffset,
	int TwistDataOffset,
	int WidthDataOffset,
	float2 SliceVertexNormals,
	FLWCMatrix LocalToWorld,
	float3 InvScale)
{
	FRibbonPositionIntermediates Intermediates = (FRibbonPositionIntermediates) 0;

	BRANCH // Also acts as ISOLATE
	if (PositionDataOffset != -1)
	{
		const float3 Pos0 = GetVec3(PositionDataOffset, InterpVtxData.ParticleId0);
		const float3 Pos1 = GetVec3(PositionDataOffset, InterpVtxData.ParticleId1);
		const float4 TangentAndDist0 = float4(
			GetTangentAndDistanceComponent(InterpVtxData.RawParticleId0, 0),
			GetTangentAndDistanceComponent(InterpVtxData.RawParticleId0, 1), 
			GetTangentAndDistanceComponent(InterpVtxData.RawParticleId0, 2),
			GetTangentAndDistanceComponent(InterpVtxData.RawParticleId0, 3));
		const float4 TangentAndDist1 = float4(
			GetTangentAndDistanceComponent(InterpVtxData.RawParticleId1, 0),
			GetTangentAndDistanceComponent(InterpVtxData.RawParticleId1, 1), 
			GetTangentAndDistanceComponent(InterpVtxData.RawParticleId1, 2),
			GetTangentAndDistanceComponent(InterpVtxData.RawParticleId1, 3));
		
		const float3 LocalPosition = CubicInterp(Pos0, TangentAndDist0.xyz, Pos1, TangentAndDist1.xyz, InterpVtxData.Alpha, TangentAndDist1.w - TangentAndDist0.w);
		Intermediates.LocalPosition = LocalPosition;
		Intermediates.Position = TransformPosition(LocalToWorld, LocalPosition);
		Intermediates.Direction = SafeNormalize(TransformVector(LocalToWorld, InvScale, lerp(TangentAndDist0.xyz, TangentAndDist1.xyz, InterpVtxData.Alpha)), float3(0, 0, 0));
		Intermediates.DistanceOnSegment = lerp(TangentAndDist0.w, TangentAndDist1.w, InterpVtxData.Alpha);
	}

	float3 Facing;
	
	BRANCH
	if(FacingDataOffset != -1)
	{
		const float3 Facing0 = GetVec3(FacingDataOffset, InterpVtxData.ParticleId0);
		const float3 Facing1 = GetVec3(FacingDataOffset, InterpVtxData.ParticleId1);
		Facing = TransformVector(LocalToWorld, InvScale, lerp(Facing0, Facing1, InterpVtxData.Alpha));
	}
	else
	{
		Facing = LWCToFloat(LWCSubtract(ResolvedView.TileOffset.WorldCameraOrigin, Intermediates.Position));
	}
	Facing = SafeNormalize(Facing, float3(0, 0, 0));

	float3 Right = cross(Facing, Intermediates.Direction);
	Right = SafeNormalize(Right, float3(0, 0, 0));
	
	if (NiagaraRibbonVFLooseParameters.FacingMode == 2 /* CustomSide */)
	{
		// Swap axis
		const float3 Temp = Right;
		Right = -Facing;
		Facing = Temp;
	}

	BRANCH
	if (TwistDataOffset != -1)
	{
		const float Twist0 = GetFloat(TwistDataOffset, InterpVtxData.ParticleId0);
		const float Twist1 = GetFloat(TwistDataOffset, InterpVtxData.ParticleId1);
		float SinTwist, CosTwist;
		sincos(lerp(Twist0, Twist1, InterpVtxData.Alpha) /* + ResolvedView.RealTime * .5 */, SinTwist, CosTwist);

		// Apply a rotation matrix to up and right.
		float3 OriginalRight = Right; // Back it up for the transform.
		Right = CosTwist * OriginalRight + SinTwist * Facing;
		Facing = -SinTwist * OriginalRight + CosTwist * Facing;
	}

	Intermediates.TangentY = Intermediates.Direction;
	Intermediates.TangentX = Right;
	
	Intermediates.GeoUp = cross(Intermediates.TangentY, Intermediates.TangentX);
	Intermediates.GeoRight = Intermediates.TangentX;

	Intermediates.TangentZ = NiagaraRibbonVFLooseParameters.UseGeometryNormals ? Intermediates.GeoUp : Facing;

	Intermediates.TangentZ = SafeNormalize(Intermediates.TangentZ * SliceVertexNormals.y + Intermediates.TangentX * SliceVertexNormals.x, float3(0, 0, 1));
	
	// Flip the normal only for multiplane shape if the normal is looking away from the camera.
	// This lets us still render as two sided, but at least gets the normals corrected so the plane faces the camera.
	float3 ViewNormal = SafeNormalize(LWCToFloat(LWCSubtract(ResolvedView.TileOffset.WorldCameraOrigin, Intermediates.Position)), float3(0,0,0));
	Intermediates.TangentZ *= (NiagaraRibbonVF.ShouldFlipNormalToView > 0 && dot(ViewNormal, Intermediates.TangentZ) < 0) ? -1 : 1;
	
	Intermediates.TangentX = cross(Intermediates.TangentZ, Intermediates.TangentY);

	// Tangent X/Y are incorrectly swapped above detangling that results in 2 paths so less messy to swap here
	float3 t = Intermediates.TangentX;
	Intermediates.TangentX = Intermediates.TangentY;
	Intermediates.TangentY = t;


	// Overrides (FVertexFactoryIntermediates)0; 
	Intermediates.RibbonWidth = 1.0f;

	FLATTEN
	if (WidthDataOffset != -1)
	{
		Intermediates.RibbonWidth = lerp(GetFloat(WidthDataOffset, InterpVtxData.ParticleId0), GetFloat(WidthDataOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha);
	}

	return Intermediates;
}

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates = (FVertexFactoryIntermediates)0;
	FInterpVtxData InterpVtxData = (FInterpVtxData)0;
	FSegmentData SegmentData = (FSegmentData)0;
	GetInterpVtxAndRibbonDataData(Input, InterpVtxData, SegmentData);
	
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	
	const float2 SliceVertexNormals = UnpackParticleVertexNormal(Input.InterpVtxId);

	FPrimitiveSceneData PrimitiveData = GetPrimitiveDataFromUniformBuffer();
	FLWCMatrix LocalToWorld = DFFastToTileOffset(GetPrimitiveDataFromUniformBuffer().LocalToWorld);
	float3 InvScale = PrimitiveData.InvNonUniformScale;

	Intermediates.Current = GetRibbonPositionIntermediates(
		InterpVtxData,
		NiagaraRibbonVF.PositionDataOffset,
		NiagaraRibbonVF.FacingDataOffset,
		NiagaraRibbonVF.TwistDataOffset,
		NiagaraRibbonVF.WidthDataOffset,
		SliceVertexNormals,
		LocalToWorld,
		InvScale);

	BRANCH
	if(NiagaraRibbonVF.VelocityDataOffset != -1)
	{
		const float3 Velocity0 = GetVec3(NiagaraRibbonVF.VelocityDataOffset, InterpVtxData.ParticleId0);
		const float3 Velocity1 = GetVec3(NiagaraRibbonVF.VelocityDataOffset, InterpVtxData.ParticleId1);
		const float3 Velocity = TransformVector(LocalToWorld, InvScale, lerp(Velocity0, Velocity1, InterpVtxData.Alpha));

		Intermediates.ParticleVelocity = GetDirectionAndMagnitude(Velocity);
	}

	BRANCH
	if (NiagaraRibbonVFLooseParameters.NeedsPreciseMotionVectors)
	{
		FLWCMatrix PreviousLocalToWorld = DFFastToTileOffset(PrimitiveData.PreviousLocalToWorld);

		// we don't cache the InvScale so we generate it here
		float3x3 PreviousTransform = LWCToFloat3x3(PreviousLocalToWorld);
		const float3 PreviousXAxis = float3(PreviousTransform[0][0], PreviousTransform[0][1], PreviousTransform[0][2]);
		const float3 PreviousYAxis = float3(PreviousTransform[1][0], PreviousTransform[1][1], PreviousTransform[1][2]);
		const float3 PreviousZAxis = float3(PreviousTransform[2][0], PreviousTransform[2][1], PreviousTransform[2][2]);

		float3 PreviousInvScale = float3(
			rsqrt(max(0.0001f, dot(PreviousXAxis, PreviousXAxis))),
			rsqrt(max(0.0001f, dot(PreviousYAxis, PreviousYAxis))),
			rsqrt(max(0.0001f, dot(PreviousZAxis, PreviousZAxis)))
		);

		Intermediates.Previous = GetRibbonPositionIntermediates(
			InterpVtxData,
			NiagaraRibbonVF.PrevPositionDataOffset,
			NiagaraRibbonVF.PrevFacingDataOffset,
			NiagaraRibbonVF.PrevTwistDataOffset,
			NiagaraRibbonVF.PrevWidthDataOffset,
			SliceVertexNormals,
			PreviousLocalToWorld,
			PreviousInvScale);
	}
	else
	{
		const float3 VelocityOffset = TransformVector(LocalToWorld, InvScale, -Intermediates.ParticleVelocity.xyz * (Intermediates.ParticleVelocity.w * NiagaraRibbonVF.DeltaSeconds));

		Intermediates.Previous = Intermediates.Current;
		Intermediates.Previous.Position = LWCAdd(Intermediates.Current.Position, LWCPromote(VelocityOffset));
	}

	// Overrides (FVertexFactoryIntermediates)0; 
	Intermediates.Color = float4(1.0f, 1.0f, 1.0f, 1.0f); 

	if (NiagaraRibbonVF.ColorDataOffset != -1)
	{
		const float4 Color0 = GetVec4(NiagaraRibbonVF.ColorDataOffset, InterpVtxData.ParticleId0);
		const float4 Color1 = GetVec4(NiagaraRibbonVF.ColorDataOffset, InterpVtxData.ParticleId1);
		Intermediates.Color = lerp(Color0, Color1, InterpVtxData.Alpha);
	}

#if (DYNAMIC_PARAMETERS_MASK & 1) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	// Overrides (FVertexFactoryIntermediates)0; 
	Intermediates.DynamicParameter = float4(1.0f, 1.0f, 1.0f, 1.0f);

	FLATTEN
	if (NiagaraRibbonVF.MaterialParamDataOffset != -1)
	{
		Intermediates.DynamicParameter = lerp(GetVec4(NiagaraRibbonVF.MaterialParamDataOffset, InterpVtxData.ParticleId0), GetVec4(NiagaraRibbonVF.MaterialParamDataOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha);
	}
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	// Overrides (FVertexFactoryIntermediates)0; 
	Intermediates.DynamicParameter1 = float4(1.0f, 1.0f, 1.0f, 1.0f);

	FLATTEN
	if (NiagaraRibbonVF.MaterialParam1DataOffset != -1)
	{
		Intermediates.DynamicParameter1 = lerp(GetVec4(NiagaraRibbonVF.MaterialParam1DataOffset, InterpVtxData.ParticleId0), GetVec4(NiagaraRibbonVF.MaterialParam1DataOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha);
	}
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	// Overrides (FVertexFactoryIntermediates)0; 
	Intermediates.DynamicParameter2 = float4(1.0f, 1.0f, 1.0f, 1.0f);

	FLATTEN
	if (NiagaraRibbonVF.MaterialParam2DataOffset != -1)
	{
		Intermediates.DynamicParameter2 = lerp(GetVec4(NiagaraRibbonVF.MaterialParam2DataOffset, InterpVtxData.ParticleId0), GetVec4(NiagaraRibbonVF.MaterialParam2DataOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha);
	}
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	// Overrides (FVertexFactoryIntermediates)0; 
	Intermediates.DynamicParameter3 = float4(1.0f, 1.0f, 1.0f, 1.0f);

	FLATTEN
	if (NiagaraRibbonVF.MaterialParam3DataOffset != -1)
	{
		Intermediates.DynamicParameter3 = lerp(GetVec4(NiagaraRibbonVF.MaterialParam3DataOffset, InterpVtxData.ParticleId0), GetVec4(NiagaraRibbonVF.MaterialParam3DataOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha);
	}
#endif

	float U0ForSegment = 0;
	BRANCH
	if (NiagaraRibbonVF.U0OverrideDataOffset != -1)
	{
		U0ForSegment = lerp(GetFloat(NiagaraRibbonVF.U0OverrideDataOffset, InterpVtxData.ParticleId0), GetFloat(NiagaraRibbonVF.U0OverrideDataOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha);
	}
	else if (NiagaraRibbonVF.U0DistributionMode == U_DISTRIBUTION_SCALED_UNIFORMLY)
	{
		U0ForSegment = ((float) (InterpVtxData.RawParticleId0 - SegmentData.StartParticleId) + InterpVtxData.Alpha) * SegmentData.U0DistributionScaler;
	}
	else if (NiagaraRibbonVF.U0DistributionMode == U_DISTRIBUTION_SCALED_USING_RIBBON_LENGTH || NiagaraRibbonVF.U0DistributionMode == U_DISTRIBUTION_TILED_OVER_RIBBON_LENGTH)
	{
		U0ForSegment = Intermediates.Current.DistanceOnSegment * SegmentData.U0DistributionScaler;
	}
	else if (NiagaraRibbonVF.DistanceFromStartOffset != -1 && NiagaraRibbonVF.U0DistributionMode == U_DISTRIBUTION_TILED_FROM_START_OVER_RIBBON_LENGTH)
	{
		U0ForSegment = lerp(GetFloat(NiagaraRibbonVF.DistanceFromStartOffset, InterpVtxData.ParticleId0), GetFloat(NiagaraRibbonVF.DistanceFromStartOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha) * SegmentData.U0DistributionScaler;
	}

	float U1ForSegment = 0;
	BRANCH
	if(NiagaraRibbonVF.U1OverrideDataOffset != -1)
	{
		U1ForSegment = lerp(GetFloat(NiagaraRibbonVF.U1OverrideDataOffset, InterpVtxData.ParticleId0), GetFloat(NiagaraRibbonVF.U1OverrideDataOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha);
	}
	else if (NiagaraRibbonVF.U1DistributionMode == U_DISTRIBUTION_SCALED_UNIFORMLY)
	{
		U1ForSegment = ((float)(InterpVtxData.RawParticleId0 - SegmentData.StartParticleId) + InterpVtxData.Alpha) * SegmentData.U1DistributionScaler;
	}
	else if (NiagaraRibbonVF.U1DistributionMode == U_DISTRIBUTION_SCALED_USING_RIBBON_LENGTH || NiagaraRibbonVF.U1DistributionMode == U_DISTRIBUTION_TILED_OVER_RIBBON_LENGTH)
	{
		U1ForSegment = Intermediates.Current.DistanceOnSegment * SegmentData.U1DistributionScaler;
	}
	else if (NiagaraRibbonVF.DistanceFromStartOffset != -1 && NiagaraRibbonVF.U1DistributionMode == U_DISTRIBUTION_TILED_FROM_START_OVER_RIBBON_LENGTH)
	{
		U1ForSegment = lerp(GetFloat(NiagaraRibbonVF.DistanceFromStartOffset, InterpVtxData.ParticleId0), GetFloat(NiagaraRibbonVF.DistanceFromStartOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha) * SegmentData.U1DistributionScaler;
	}

	float V0ForSegment;
	BRANCH
	if (NiagaraRibbonVF.V0RangeOverrideDataOffset != -1)
	{
		float2 V0RangeForSegment = lerp(GetVec2(NiagaraRibbonVF.V0RangeOverrideDataOffset, InterpVtxData.ParticleId0), GetVec2(NiagaraRibbonVF.V0RangeOverrideDataOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha);
		V0ForSegment = (1 - UnpackParticleVertexTextureV(Input.InterpVtxId) * V0RangeForSegment.x) + (UnpackParticleVertexTextureV(Input.InterpVtxId) * V0RangeForSegment.y);
	}
	else
	{
		V0ForSegment = UnpackParticleVertexTextureV(Input.InterpVtxId);
	}
	
	float V1ForSegment;
	BRANCH
	if (NiagaraRibbonVF.V1RangeOverrideDataOffset != -1)
	{
		float2 V1RangeForSegment = lerp(GetVec2(NiagaraRibbonVF.V1RangeOverrideDataOffset, InterpVtxData.ParticleId0), GetVec2(NiagaraRibbonVF.V1RangeOverrideDataOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha);
		V1ForSegment = (1 - UnpackParticleVertexTextureV(Input.InterpVtxId) * V1RangeForSegment.x) + (UnpackParticleVertexTextureV(Input.InterpVtxId) * V1RangeForSegment.y);
	}
	else
	{
		V1ForSegment = UnpackParticleVertexTextureV(Input.InterpVtxId);
	}
	
	float2 UV0ForSegment = float2(U0ForSegment, V0ForSegment);
	float2 UV1ForSegment = float2(U1ForSegment, V1ForSegment);
	
	Intermediates.TexCoord.xy = UV0ForSegment * SegmentData.UV0Scale + SegmentData.UV0Offset;
	Intermediates.TexCoord.zw = UV1ForSegment * SegmentData.UV1Scale + SegmentData.UV1Offset;

#if USE_PARTICLE_POSITION
	Intermediates.TranslatedWorldPositionAndSize = float4(LWCToFloat(LWCAdd(Intermediates.Current.Position, ResolvedView.TileOffset.PreViewTranslation)),  .5 * Intermediates.Current.RibbonWidth);
#endif

#if USE_PARTICLE_TIME
	FLATTEN
	if (NiagaraRibbonVF.NormalizedAgeDataOffset != -1)
	{
		Intermediates.RelativeTime = lerp(GetFloat(NiagaraRibbonVF.NormalizedAgeDataOffset, InterpVtxData.ParticleId0), GetFloat(NiagaraRibbonVF.NormalizedAgeDataOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha);
	}
#endif

#if USE_PARTICLE_SIZE
	Intermediates.ParticleSize = Intermediates.Current.RibbonWidth.xx;
#endif

#if USE_PARTICLE_RANDOM
	FLATTEN
	if (NiagaraRibbonVF.MaterialRandomDataOffset != -1)
	{
		Intermediates.ParticleRandom = lerp(GetFloat(NiagaraRibbonVF.MaterialRandomDataOffset, InterpVtxData.ParticleId0), GetFloat(NiagaraRibbonVF.MaterialRandomDataOffset, InterpVtxData.ParticleId1), InterpVtxData.Alpha);
	}
#endif

	return Intermediates;
}

FLWCVector3 ConstructWorldPositionLWC(FVertexFactoryInput Input, FRibbonPositionIntermediates Intermediates)
{
	const float2 SliceVertexPositions = UnpackParticleVertexPosition(Input.InterpVtxId);
	const float3 Offset = ((Intermediates.GeoRight * SliceVertexPositions.xxx) + (Intermediates.GeoUp * SliceVertexPositions.yyy)) * Intermediates.RibbonWidth.xxx;
	return LWCAdd(Intermediates.Position, LWCPromote(Offset));
}

float3 ConstructWorldPosition(FVertexFactoryInput Input, FRibbonPositionIntermediates Intermediates, FLWCVector3 PreViewTranslation)
{
	const FLWCVector3 WorldPosition = ConstructWorldPositionLWC(Input, Intermediates);
	const FLWCVector3 TranslatedPosition = LWCAdd(WorldPosition, PreViewTranslation);
	return LWCToFloat(TranslatedPosition);
}

float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return float4(ConstructWorldPosition(Input, Intermediates.Current, ResolvedView.TileOffset.PreViewTranslation), 1);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.Current.LocalPosition;
}

float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return float4(ConstructWorldPosition(Input, Intermediates.Previous, ResolvedView.TileOffset.PrevPreViewTranslation), 1);
}

// local position relative to instance
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.Previous.LocalPosition;
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.Current.TangentZ;
}


float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;

#if NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);

	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		SetUV(Interpolants, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
	}
#endif

#if NEEDS_PARTICLE_COLOR
	Interpolants.Color		= Intermediates.Color;
#endif

#if (DYNAMIC_PARAMETERS_MASK & 1)
	Interpolants.DynamicParameter = Intermediates.DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	Interpolants.DynamicParameter1 = Intermediates.DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	Interpolants.DynamicParameter2 = Intermediates.DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	Interpolants.DynamicParameter3 = Intermediates.DynamicParameter3;
#endif

#if USE_PARTICLE_POSITION
	Interpolants.TranslatedWorldPositionAndSize = Intermediates.TranslatedWorldPositionAndSize;
#endif

#if USE_PARTICLE_VELOCITY
	Interpolants.ParticleVelocity = Intermediates.ParticleVelocity;
#endif

#if USE_PARTICLE_TIME
	Interpolants.RelativeTime = Intermediates.RelativeTime;
#endif

#if USE_PARTICLE_SIZE
	Interpolants.ParticleSize = Intermediates.ParticleSize;
#endif

#if USE_PARTICLE_RANDOM
	Interpolants.ParticleRandom = Intermediates.ParticleRandom;
#endif

	float3x3 TangentToWorld = CalcTangentBasis(Input, Intermediates);
	Interpolants.TangentToWorld0.xyz = TangentToWorld[0];
	Interpolants.TangentToWorld0.w = 0;
	// GetNiagaraParticleTangents() technically makes the determinant to be always 1.
	Interpolants.TangentToWorld2 = float4(TangentToWorld[2], 1 /*sign(determinant(TangentToWorld))*/);

	return Interpolants;
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
float3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return CalcTangentBasis(Input, Intermediates);
}

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

// RHI_RAYTRACING
#if COMPUTESHADER
FVertexFactoryInput LoadVertexFactoryInputForDynamicUpdate(uint TriangleIndex, int VertexIndex, uint PrimitiveId, uint DrawInstanceId)
{
	FVertexFactoryInput Input;

	Input.InterpVtxId = TriangleIndex * 3 + VertexIndex;
	if ( NiagaraRibbonVFLooseParameters.UseIndexBufferForRayTracing != 0 )
	{
		Input.InterpVtxId = NiagaraRibbonVFLooseParameters.IndexBuffer[Input.InterpVtxId];
	}

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && SUPPORT_SHADOW_DEPTH_POINT && USING_VERTEX_SHADER_LAYER
	Input.InstanceId = 0;
#endif
	return Input;
}

uint GetNumRayTracingDynamicMeshVerticesIndirect()
{
	return NiagaraRibbonVFLooseParameters.IndirectDrawOutput[NiagaraRibbonVFLooseParameters.IndirectDrawOutputOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 0];
}
#endif

#if RAYHITGROUPSHADER
FVertexFactoryInput LoadVertexFactoryInputForHGS(uint TriangleIndex, int VertexIndex)
{
	FVertexFactoryInput Input;

	FTriangleBaseAttributes Tri = LoadTriangleBaseAttributes(TriangleIndex);
	uint VertexId = Tri.Indices[VertexIndex];
	Input.InterpVtxId = VertexId;
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && SUPPORT_SHADOW_DEPTH_POINT && USING_VERTEX_SHADER_LAYER
	Input.InstanceId = 0;
#endif
	return Input;
}

uint GetNumRayTracingDynamicMeshVerticesIndirect()
{
	return NiagaraRibbonVFLooseParameters.IndirectDrawOutput[NiagaraRibbonVFLooseParameters.IndirectDrawOutputOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 0];
}
#endif



#if RAYHITGROUPSHADER

struct FVertexFactoryRayTracingInterpolants
{
	FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
};

FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryRayTracingInterpolants Interpolants;
	
	Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);
	
	return Interpolants;
}

float2 VertexFactoryGetRayTracingTextureCoordinate( FVertexFactoryRayTracingInterpolants Interpolants )
{
#if NUM_TEX_COORD_INTERPOLATORS
	return Interpolants.InterpolantsVSToPS.TexCoords[0].xy;
#else // #if NUM_MATERIAL_TEXCOORDS
	return float2(0,0);
#endif // #if NUM_MATERIAL_TEXCOORDS
}

FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
{
	FVertexFactoryRayTracingInterpolants O;

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for(int tc = 0; tc < (NUM_TEX_COORD_INTERPOLATORS+1)/2; ++tc)
	{
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[tc]);
	}
#endif

	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld0);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld2);

#if NEEDS_PARTICLE_COLOR
	INTERPOLATE_MEMBER(InterpolantsVSToPS.Color);
#endif

#if (DYNAMIC_PARAMETERS_MASK & 1)
	INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter1);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter2);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter3);
#endif

#if USE_PARTICLE_POSITION
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TranslatedWorldPositionAndSize);
#endif

#if USE_PARTICLE_VELOCITY
	INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleVelocity);
#endif

#if USE_PARTICLE_TIME
	INTERPOLATE_MEMBER(InterpolantsVSToPS.RelativeTime);
#endif

#if USE_PARTICLE_SIZE
	INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleSize);
#endif

#if USE_PARTICLE_RANDOM
	INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleRandom);
#endif

	return O;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
{
	return Input.InterpolantsVSToPS;
}

#endif

#if WITH_NIAGARA_VERTEX_FACTORY_EXPORT
FVertexFactoryInput NiagaraVertexFactoryExport_InitVertexFactoryInput(uint TriangleIndex, int VertexIndex, uint PrimitiveId, uint DrawInstanceId)
{
	FVertexFactoryInput Input;

	Input.InterpVtxId = TriangleIndex * 3 + VertexIndex;
	Input.InterpVtxId = NiagaraRibbonVFLooseParameters.IndexBuffer[Input.InterpVtxId];

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && SUPPORT_SHADOW_DEPTH_POINT && USING_VERTEX_SHADER_LAYER
	Input.InstanceId = 0;
#endif
	return Input;
}

uint NiagaraVertexFactoryExport_GetIndirectVertexCount()
{
	return NiagaraRibbonVFLooseParameters.IndirectDrawOutput[NiagaraRibbonVFLooseParameters.IndirectDrawOutputOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 0];
}

FLWCVector3 NiagaraVertexFactoryExport_GetPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return ConstructWorldPositionLWC(Input, Intermediates.Current);
}

float4 NiagaraVertexFactoryExport_GetColor(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.Color;
}

float3x3 NiagaraVertexFactoryExport_GetTangentBasis(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return float3x3(Intermediates.Current.TangentX, Intermediates.Current.TangentY, Intermediates.Current.TangentZ);
}

float2 NiagaraVertexFactoryExport_GetTexCoord(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, uint CoordIndex)
{
	switch (CoordIndex)
	{
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 0
		case 0:	return Intermediates.TexCoord.xy;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
		case 1:	return Intermediates.TexCoord.zw;
	#endif
		default: return float2(0,0);
	}
}

float4 NiagaraVertexFactoryExport_GetDynamicParameter(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, uint ParameterIndex)
{
	float4 Value = 1.0f;
	if ( ParameterIndex == 0 )
	{
		Value = Intermediates.DynamicParameter;
	}
	else if ( ParameterIndex == 1 )
	{
		Value = Intermediates.DynamicParameter1;
	}
	else if ( ParameterIndex == 2 )
	{
		Value = Intermediates.DynamicParameter2;
	}
	else if ( ParameterIndex == 3 )
	{
		Value = Intermediates.DynamicParameter3;
	}
	else
	{
		return 1.0f;
	}

	uint MaskBit = ParameterIndex * 4u;
	for ( int i=0; i < 4; ++i )
	{
		Value[i] = ((NiagaraRibbonVF.MaterialParamValidMask & (1u << MaskBit)) != 0u) ? Value[i] : 1.0f;
		++MaskBit;
	}
	return Value;
}

#endif //WITH_NIAGARA_VERTEX_FACTORY_EXPORT

#include "/Engine/Private/VertexFactoryDefaultInterface.ush"
