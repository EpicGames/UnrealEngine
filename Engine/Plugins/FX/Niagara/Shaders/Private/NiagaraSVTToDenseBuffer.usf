// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/SparseVolumeTexture/SparseVolumeTextureCommon.ush"
RWTexture3D<float4>	DestinationBuffer;

SamplerState		TileDataTextureSampler;
Texture3D<uint>		SparseVolumeTexturePageTable;
Texture3D			SparseVolumeTextureA;
uint4				PackedSVTUniforms0;
uint4				PackedSVTUniforms1;
int3				TextureSize;
int					MipLevels;

[numthreads(4, 4, 4)]
void PerformCopyCS(uint3 Index : SV_DispatchThreadID)
{
	const FSparseVolumeTextureUniforms Uniforms = SparseVolumeTextureUnpackUniforms(PackedSVTUniforms0, PackedSVTUniforms1);
	const int MipLevel = 0; // We want to sample level 0 but the page table can actually point to lower resolution mip level voxels
	const int3 VirtualCoord = Index >> MipLevel;
	const int3 VoxelCoord = SparseVolumeTextureLoadPageTable(SparseVolumeTexturePageTable, Uniforms, VirtualCoord, MipLevel);

	DestinationBuffer[Index] = SparseVolumeTextureA.Load(int4(VoxelCoord, 0));
}
