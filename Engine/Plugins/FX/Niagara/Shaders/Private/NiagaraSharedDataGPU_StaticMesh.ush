// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraManagedReference_StaticMesh.ush 

Utility/Support code for using NiagaraSharedDataID_StaticMesh values to indirectly reference static mesh data from the managed static mesh buffers.

=============================================================================*/

#pragma once

/** Counterpart to struct in NiagaraManagedReference_StaticMesh.h*/
struct FNiagaraSharedDataGPU_StaticMesh
{
	int IDTag;
	int DistanceFieldIndex;
	int PrimitiveIndex;
	int Padding;
};

StructuredBuffer<FNiagaraSharedDataGPU_StaticMesh> NiagaraSharedData_StaticMesh_Buffer;
int NiagaraSharedData_StaticMesh_NumSlots;

/** Resolves a mesh DF Reference into a DFIndex that can be used to access distance field data. */ 
int ResolveNiagaraManagedReferenceID_StaticMesh(NiagaraSharedDataID_StaticMesh StaticMeshID)
{
	int Slot = StaticMeshID.Index;
	int Tag = StaticMeshID.AcquireTag;
 	
	int DFIndex = -1;
	if (Slot >= 0 && Slot < NiagaraSharedData_StaticMesh_NumSlots)
	{
		FNiagaraSharedDataGPU_StaticMesh StaticMeshInfo = NiagaraSharedData_StaticMesh_Buffer[Slot];
		if (StaticMeshInfo.IDTag == Tag)
		{
			DFIndex = StaticMeshInfo.DistanceFieldIndex;
		}
	}
	return DFIndex;
}

/** 
Returns closest point and normal on a mesh distance field to a given world space point.
Duplicate implementation to NiagaraDistanceFieldQueries.ush but this handles LWC internally and takes in a tiled point directly from a Niagara Sim and outputs the correctly tiled point back ready for the Niagara Sim.
*/
void ComputeClosestPointMeshDistanceField_LWCTiled(int DFIndex, const float3 WorldPosition, const float GridSampledDistance, out float3 OutClosestPosition, out float3 OutClosestNormal, out float MaxEncodedDistance, out bool bNormalIsValid)
{
	FDFObjectData DFObjectData = LoadDFObjectData(DFIndex);
	
	const FLWCVector3 TiledLWCPos = MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition);
	const FDFVector3 LWCWorldPosition = DFFromTileOffset_Hack(TiledLWCPos);
	
	const float3 VolumePosition = DFMultiplyAffineDemote(LWCWorldPosition, DFObjectData.WorldToVolume);
	const float3 ClampedVolumePosition = clamp(VolumePosition, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent);
	
	uint NumMips = LoadDFAssetData(DFObjectData.AssetIndex, 0).NumMips;
	FDFAssetData DFAssetData = LoadDFAssetData(DFObjectData.AssetIndex, NumMips - 1);

	MaxEncodedDistance = (DFAssetData.DistanceFieldToVolumeScaleBias.x + DFAssetData.DistanceFieldToVolumeScaleBias.y) * DFObjectData.VolumeScale;
		
	float3 VolumeGradient = CalculateMeshSDFGradient(ClampedVolumePosition, DFAssetData);
	float VolumeGradientLength = length(VolumeGradient);
	float3 VolumeNormal = VolumeGradientLength > 0.0f ? VolumeGradient / VolumeGradientLength : 0;

	// Transform by transposed inverse to handle non-uniform scaling
	float4x4 WorldToVolume = DFHackToFloat(DFObjectData.WorldToVolume);
	float3 WorldGradient = mul(VolumeNormal, transpose((float3x3) WorldToVolume));
	float WorldGradientLength = length(WorldGradient);

	// @note: we should include checking if we are within the range of max encoded distance, but this breaks backwards compatibility
	bNormalIsValid = WorldGradientLength > 0.0f && (MaxEncodedDistance - abs(GridSampledDistance) > 1e-8);
			
	// output
	if (bNormalIsValid)
	{
		OutClosestNormal = WorldGradient / WorldGradientLength;
		
		const FDFVector3 WorldClampedPosition = DFMultiplyAffine(ClampedVolumePosition, DFObjectData.VolumeToWorld);
		const float3 TiledWorldClampedPosition = DFSubtractDemote(WorldClampedPosition, LWCGetTileOffset(TiledLWCPos));
		OutClosestPosition = TiledWorldClampedPosition - OutClosestNormal * GridSampledDistance;
	}
	else
	{
		OutClosestPosition = WorldPosition;
		OutClosestNormal = float3(0, 0, 1);
	}
}