// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	NiagaraSortKeyGen.usf: Shader to generate particle sorting keys.
==============================================================================*/

/*------------------------------------------------------------------------------
	Compile time parameters:
		THREAD_COUNT - The number of threads to launch per workgroup.
		SORT_NONE
		SORT_VIEW_DEPTH
		SORT_VIEW_DISTANCE
		SORT_CUSTOM_ASCENDING
		SORT_CUSTOM_DESCENDING
------------------------------------------------------------------------------*/

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ComputeShaderUtils.ush"

Buffer<float> NiagaraParticleDataFloat;
Buffer<half> NiagaraParticleDataHalf;
Buffer<int> NiagaraParticleDataInt;
Buffer<uint> GPUParticleCountBuffer;

uint FloatDataStride;
uint HalfDataStride;
uint IntDataStride;
uint ParticleCount;
uint GPUParticleCountOffset;
uint CulledGPUParticleCountOffset;
uint EmitterKey;
uint OutputOffset;
float3 CameraPosition;
float3 CameraDirection;
uint SortMode;
uint SortAttributeOffset;
#if SORT_MAX_PRECISION
	uint SortKeyMask;
	uint SortKeyShift;
	uint SortKeySignBit;
#endif
uint CullPositionAttributeOffset;
uint CullOrientationAttributeOffset;
uint CullScaleAttributeOffset;
int NumCullPlanes;
float4 CullPlanes[NIAGARA_KEY_GEN_MAX_CULL_PLANES];
int RendererVisibility;
uint RendererVisTagAttributeOffset;
int MeshIndex;
uint MeshIndexAttributeOffset;
float2 CullDistanceRangeSquared;
float4 LODScreenSize;
float4 LocalBoundingSphere;
float3 CullingWorldSpaceOffset;

RWBuffer<uint> OutKeys;
RWBuffer<int> OutParticleIndices;
RWBuffer<uint> OutCulledParticleCounts;

#if ENABLE_CULLING && !USE_WAVE_INTRINSICS
groupshared uint CullGroupCount;
#endif

float GetFloat(uint RegisterIdx, uint InstanceID)
{
	BRANCH
	if (RegisterIdx & (1u << 31U))
	{
		RegisterIdx &= ~(1u << 31U);
		return NiagaraParticleDataHalf[RegisterIdx * HalfDataStride + InstanceID];
	}
	else
	{
		return NiagaraParticleDataFloat[RegisterIdx * FloatDataStride + InstanceID];
	}
}

int GetInt(uint RegisterIdx, uint InstanceID)
{
	return NiagaraParticleDataInt[RegisterIdx * IntDataStride + InstanceID];
}

float2 GetVec2(uint RegisterIndex, uint InstanceID)
{
	return float2(GetFloat(RegisterIndex, InstanceID), GetFloat(RegisterIndex + 1, InstanceID));
}

float3 GetVec3(uint RegisterIndex, uint InstanceID)
{
	return float3(GetFloat(RegisterIndex, InstanceID), GetFloat(RegisterIndex + 1, InstanceID), GetFloat(RegisterIndex + 2, InstanceID));
}

float4 GetVec4(uint RegisterIndex, uint InstanceID)
{
	return float4(GetFloat(RegisterIndex, InstanceID), GetFloat(RegisterIndex + 1, InstanceID), GetFloat(RegisterIndex + 2, InstanceID), GetFloat(RegisterIndex + 3, InstanceID));
}

float3 RotateVectorByQuat(in float3 Vector, in float4 Quat)
{
	float3 T = 2.0 * cross(Quat.xyz, Vector);
	return Vector + Quat.w * T + cross(Quat.xyz, T);
}

void CullingPrologue(in uint GroupTID)
{
#if ENABLE_CULLING && !USE_WAVE_INTRINSICS
	if (GroupTID == 0)
	{
		CullGroupCount = 0;
	}

	GroupMemoryBarrierWithGroupSync();
#endif
}

void CullingEpilogue(in bool bCulled, in uint GroupTID)
{
#if ENABLE_CULLING
	uint CountOffset = CulledGPUParticleCountOffset;

#if USE_WAVE_INTRINSICS
	uint CullWaveCount = WaveActiveCountBits(!bCulled);
	if (WaveIsFirstLane())
	{
		InterlockedAdd(OutCulledParticleCounts[CountOffset], CullWaveCount);
	}
#else
	InterlockedAdd(CullGroupCount, bCulled ? 0 : 1);
	GroupMemoryBarrierWithGroupSync();

	if (GroupTID == 0)
	{
		InterlockedAdd(OutCulledParticleCounts[CountOffset], CullGroupCount);
	}
#endif // USE_WAVE_INTRINSICS
#endif // ENABLE_CULLING
}

bool CullParticle(in uint ParticleIndex)
{
#if ENABLE_CULLING
	float3 Pos = 0;
	if (CullPositionAttributeOffset != -1)
	{
		Pos = GetVec3(CullPositionAttributeOffset, ParticleIndex);
	}

	float4 Quat = { 0, 0, 0, 1 };
	if (CullOrientationAttributeOffset != -1)
	{
		Quat = GetVec4(CullOrientationAttributeOffset, ParticleIndex);
	}

	float3 Scale = 1;
	if (CullScaleAttributeOffset != -1)
	{
		Scale = GetVec3(CullScaleAttributeOffset, ParticleIndex);
	}

	// Transform the bounding sphere
	float4 BSphere = LocalBoundingSphere;
	BSphere.xyz = RotateVectorByQuat(BSphere.xyz * Scale, Quat) + Pos + CullingWorldSpaceOffset;
	BSphere.w *= max(abs(Scale.x), max(abs(Scale.y), abs(Scale.z)));

	// Cull particles using distance ranges
	float3 CamVec = BSphere.xyz - CameraPosition;
	float CamDistSq = dot(CamVec, CamVec);
	if (CamDistSq < CullDistanceRangeSquared.x  || CamDistSq > CullDistanceRangeSquared.y)
	{
		// We are outside the distance ranges required
		return true;
	}

	// Cull based on LOD level
	const float ScreenRadius = saturate(LODScreenSize.z / max(1.0f, CamDistSq * LODScreenSize.w));
	if ( ScreenRadius < LODScreenSize.x || ScreenRadius >= LODScreenSize.y )
	{
		return true;
	}
	
	// Cull based on renderer visibility
	if (RendererVisTagAttributeOffset != -1)
	{
		int Category = GetInt(RendererVisTagAttributeOffset, ParticleIndex);
		if (Category != RendererVisibility)
		{
			// This particle is not visible in this renderer.
			return true;
		}
	}
	
	// Cull based on mesh index (when applicable)
	if (MeshIndexAttributeOffset != -1)
	{
		int Index = GetInt(MeshIndexAttributeOffset, ParticleIndex);
		if (Index != MeshIndex)
		{
			// This particle is not visible in this renderer.
			return true;
		}
	}

	// Cull particles using cull planes
	int NumCullPlanesToCheck = min(NumCullPlanes, NIAGARA_KEY_GEN_MAX_CULL_PLANES);
	bool Found = false;
	int i = 0;
	while (!Found && (i < NumCullPlanesToCheck))
	{
		float4 Plane = CullPlanes[i];
		Found = ((dot(Plane.xyz, BSphere.xyz) - Plane.w) > BSphere.w);
		i = i + 1;
	}
	return Found;
#else
	return false;
#endif
}

[numthreads(THREAD_COUNT,1,1)]
void GenerateParticleSortKeys(uint3 GroupId : SV_GroupID, int GroupThreadIndex : SV_GroupIndex)
{
	const uint ParticleIndex = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, THREAD_COUNT);
	
	CullingPrologue(GroupThreadIndex);

	const uint GPUParticleCount = GPUParticleCountOffset != -1 ? GPUParticleCountBuffer[GPUParticleCountOffset] : ParticleCount;
	
#if !SORT_MAX_PRECISION
	const uint SortKeyMask = 0xffff;
	const uint SortKeySignBit = 0x8000;
#endif

	bool bCulled = true;
	if (ParticleIndex < ParticleCount)
	{
		float SortValue = 0;
		uint SortKey = SortKeyMask;

		if (ParticleIndex < GPUParticleCount)
		{
			if (SortMode == SORT_NONE)
			{
				// This happens if we're only culling particles, set live particles to a constant valid key
				SortKey = 0;
			}
			else if (SortMode == SORT_VIEW_DEPTH || SortMode == SORT_VIEW_DISTANCE)
			{
				float3 ParticlePosition = GetVec3(SortAttributeOffset, ParticleIndex); 
				float3 CameraVector = ParticlePosition - CameraPosition;
				SortValue = SortMode == SORT_VIEW_DEPTH ? dot(CameraVector, CameraDirection) : length(CameraVector);

#if SORT_MAX_PRECISION
				SortKey = (asuint(SortValue)) >> SortKeyShift;
#else
				SortKey = f32tof16(SortValue);
#endif
				// Render far away first, knowing that SortValue > 0
				SortKey =  (~SortKey) & SortKeyMask;
			}
			else
			{
				SortValue = GetFloat(SortAttributeOffset, ParticleIndex);

#if SORT_MAX_PRECISION
				SortKey = (asuint(SortValue)) >> SortKeyShift;
#else
				SortKey = f32tof16(SortValue);
#endif
				// Special handling for negative values.
				SortKey = SortValue >= 0 ? (SortKey | SortKeySignBit) : ~SortKey;

				SortKey = (SortMode == SORT_CUSTOM_ASCENDING ? SortKey : ~SortKey) & SortKeyMask;
			}

			// Clamp to max-1 to make sure that out-of-bound indices are at the end.
			SortKey = min(SortKey , SortKeyMask - 1);
			
			bCulled = CullParticle(ParticleIndex);
		}

		if (bCulled)
		{
			SortKey = SortKeyMask;
		}

		SortKey |= EmitterKey;

		
		OutKeys[OutputOffset + ParticleIndex] = SortKey;
		OutParticleIndices[OutputOffset + ParticleIndex] = ParticleIndex;
	}

	CullingEpilogue(bCulled, GroupThreadIndex);
}

