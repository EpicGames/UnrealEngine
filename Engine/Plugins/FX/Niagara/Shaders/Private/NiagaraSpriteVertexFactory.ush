// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleSpriteVertexFactory.hlsl: Particle vertex factory shader code.
	Shared by standard sprite particles and SubUV sprite particles.
=============================================================================*/

#include "/Engine/Private/VertexFactoryCommon.ush"
#include "/Engine/Private/ParticleVertexFactoryCommon.ush"
#include "NiagaraVFCommon.usf"

#define USE_PARTICLE_LIGHTING_OFFSET (FEATURE_LEVEL >= FEATURE_LEVEL_SM5 && !MATERIAL_SHADINGMODEL_UNLIT)
#define USE_PARTICLE_POSITION (NEEDS_PARTICLE_POSITION || PASS_NEEDS_PRIMITIVE_VOLUME_BOUNDS)
#define USE_PARTICLE_VELOCITY (NEEDS_PARTICLE_VELOCITY)
#define USE_PARTICLE_TIME (NEEDS_PARTICLE_TIME)
#define USE_PARTICLE_SIZE (NEEDS_PARTICLE_SIZE)
#define USE_PARTICLE_RANDOM (NEEDS_PARTICLE_RANDOM)
#define USE_PARTICLE_SPRITE_ROTATION (NEEDS_PARTICLE_SPRITE_ROTATION)

// Must keep these in sync with ENiagaraSpriteAlignment in NiagaraSpriteRendererProperties.h
/** Only Particles.SpriteRotation and FacingMode impact the alignment of the particle.*/
#define ALIGNMENT_MODE_UNALIGNED 0
/** Imagine the particle texture having an arrow pointing up, this mode makes the arrow point in the direction of the Particles.Velocity attribute. Particles.SpriteRotation rotation is ignored.*/
#define ALIGNMENT_MODE_VELOCITYALIGNED 1
/** Imagine the particle texture having an arrow pointing up, this mode makes the arrow point towards the axis defined by the Particles.SpriteAlignment attribute, after taking Particles.SpriteRotation and the FacingMode into account.*/
#define ALIGNMENT_MODE_CUSTOMALIGNMENT 2

/** The sprite billboard origin is always "looking at" the camera origin, trying to keep its up axis aligned to the camera's up axis. */
#define FACING_MODE_FACECAMERA 0
/** The sprite billboard plane is completely parallel to the camera plane. Particle always looks "flat" */
#define FACING_MODE_FACECAMERAPLANE 1
/** The sprite billboard faces toward the "Particles.SpriteFacing" vector attribute. */
#define FACING_MODE_CUSTOMFACINGVECTOR 2
/** Faces the camera position, but is not dependent on the camera rotation.  This method produces more stable particles under camera rotation. Uses the up axis of (0,0,1).*/
#define FACING_MODE_FACECAMERAPOSITION 3 
/** Blends between FaceCamera and FaceCameraPosition.*/
#define FACING_MODE_FACECAMERADISTANCEBLEND 4

#undef GetInstanceIdFromVF
#define GetInstanceIdFromVF(VFInput)				(VFInput.InstanceId)

struct FVertexFactoryInput
{
	float2	TexCoord				: ATTRIBUTE0;
	uint VertexId					: SV_VertexID;
	uint InstanceId					: SV_InstanceID;
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()
};

uint GetCutoutNumVertices()
{
	return NiagaraSpriteVFLooseParameters.CutoutParameters & ~0x1;
}

bool GetCutoutUseSubImage()
{
	return (NiagaraSpriteVFLooseParameters.CutoutParameters & 0x1) != 0;
}

float2 NoneZeroSign(float2 v)
{
	return float2(v.x >= 0.0f ? 1.0f : -1.0f, v.y >= 0.0f ? 1.0f : -1.0f);
}

// RHI_RAYTRACING
#if COMPUTESHADER || RAYHITGROUPSHADER
FVertexFactoryInput LoadVertexFactoryInputForHGS(uint TriangleIndex, int VertexIndex)
{
	// Different numbers of cutout vertices correspond to different index buffers
	// For 8 verts, use GSixTriangleParticleIndexBuffer
	if (GetCutoutNumVertices() == 8)
	{		
		FVertexFactoryInput Input;
		
		// Hard coded GSixTriangleParticleIndexBuffer
		uint IndexBuffer[18] = { 0, 1, 2, 0, 2, 3, 0, 3, 4, 0, 4, 5, 0, 5, 6, 0, 6, 7 };
		uint VertexId = IndexBuffer[(TriangleIndex * 3 + VertexIndex) % 18];
		Input.TexCoord = NiagaraSpriteVFLooseParameters.CutoutGeometry[VertexId];
		Input.VertexId = VertexId;
		Input.InstanceId = TriangleIndex / 6;
	
		return Input;
	}
	else
	{
		// For 4 verts cutout geometry and normal particle geometry, use the typical 6 indices
		FVertexFactoryInput Input;
		
		uint IndexBuffer[6] = { 0, 2, 3, 0, 1, 2 };
		uint VertexId = IndexBuffer[(TriangleIndex * 3 + VertexIndex) % 6];
		if (GetCutoutNumVertices() > 0)
		{
			Input.TexCoord = NiagaraSpriteVFLooseParameters.CutoutGeometry[VertexId];
		}
		else
		{
			float2 TexCoords[4] = { float2(0.0f, 0.0f), float2(0.0f, 1.0f), float2(1.0f, 1.0f), float2(1.0f, 0.0f) };
			Input.TexCoord = TexCoords[VertexId];
		}
		Input.VertexId = VertexId;
		Input.InstanceId = TriangleIndex / 2;
	
		return Input;
	}
}

FVertexFactoryInput LoadVertexFactoryInputForDynamicUpdate(uint TriangleIndex, int VertexIndex, uint PrimitiveId, uint DrawInstanceId)
{
	return LoadVertexFactoryInputForHGS(TriangleIndex, VertexIndex);
}
uint GetNumRayTracingDynamicMeshVerticesIndirect()
{
	const uint NumVertices = NiagaraSpriteVFLooseParameters.IndirectArgsBuffer[NiagaraSpriteVFLooseParameters.IndirectArgsOffset + 0];
	const uint NumInstances = NiagaraSpriteVFLooseParameters.IndirectArgsBuffer[NiagaraSpriteVFLooseParameters.IndirectArgsOffset + 1];
	return NumVertices * NumInstances;
}
#endif

struct FVertexFactoryInterpolantsVSToPS
{
	// First row of the tangent to world matrix, Interp_Sizer used by SUBUV_PARTICLES in w
	float4	TangentToWorld0AndInterp_Sizer : TANGENTTOWORLD0;
	// Last row of the tangent to world matrix in xyz
	float4	TangentToWorld2	: TANGENTTOWORLD2;
	
#if (DYNAMIC_PARAMETERS_MASK & 1)
	nointerpolation float4 DynamicParameter	: PARTICLE_DYNAMIC_PARAM0;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	nointerpolation float4 DynamicParameter1 : PARTICLE_DYNAMIC_PARAM1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	nointerpolation float4 DynamicParameter2 : PARTICLE_DYNAMIC_PARAM2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	nointerpolation float4 DynamicParameter3 : PARTICLE_DYNAMIC_PARAM3;
#endif

#if NEEDS_PARTICLE_COLOR
	float4	Color				: TEXCOORD0;
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	float4	TexCoords[(NUM_TEX_COORD_INTERPOLATORS + 1) / 2]	: TEXCOORD1;
#endif

	//Not sure this is actually being used now and it's awkward to slot in now we're supporting custom UVs so I'm just giving this its own interpolant.
#if LIGHTMAP_UV_ACCESS
	float2 LightMapUVs : LIGHTMAP_UVS;
#endif

#if USE_PARTICLE_SUBUVS
	float4 ParticleSubUVs : PARTICLE_SUBUVS;
#endif

#if USE_PARTICLE_POSITION
	/** Cam-relative (translated) particle center and radius */
	nointerpolation float4 ParticleTranslatedWorldPositionAndSize	: PARTICLE_POSITION;
	#if GENERATE_SPHERICAL_PARTICLE_NORMALS
		float3 DisplacedParticleCenterForNormal : PARTICLE_POSITION_FSN;
	#endif
#endif

#if USE_PARTICLE_VELOCITY
	nointerpolation float4 ParticleVelocity : PARTICLE_VELOCITY;
#endif

#if USE_PARTICLE_TIME
	nointerpolation float RelativeTime : PARTICLE_TIME;
#endif

#if USE_PARTICLE_LIGHTING_OFFSET
	float3 LightingPositionOffset : PARTICLE_LIGHTING_OFFSET;
#endif

#if USE_PARTICLE_SIZE
	nointerpolation float2 ParticleSize : PARTICLE_SIZE;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	nointerpolation float ParticleSpriteRotation : PARTICLE_SPRITE_ROTATION;
#endif

#if USE_PARTICLE_RANDOM
	nointerpolation float ParticleRandom : PARTICLE_RANDOM;
#endif
};

struct FVertexFactoryIntermediates
{
	/** The position of the vertex in translated world space. */
	FLWCVector3 VertexWorldPosition;
	/** Last frame's position of the vertex in translated world space. */
	FLWCVector3 PreviousVertexWorldPosition;
	/** Particle translated world space position and size. */
	float4 TranslatedWorldPositionAndSize;
	float4 PrevTranslatedWorldPositionAndSize;
	/** Particle translated world space position without offsets. */
	float3 TranslatedWorldPosition;
	
#if USE_PARTICLE_LIGHTING_OFFSET
	float3 LightingPositionOffset;
#endif

	/** The texture coordinate at this vertex. */
	float4 TexCoord;
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3 //Unflipped UVs are placed in UVs 2-3
	/** A second UV set. Always non-UV flipped. Allows use of UV flipping with Normal maps etc. */
	float4 TexCoord_Unflipped;
#endif
	/** The sprite tangent in world space (+V). */
	float3 TangentUp;
	/** The sprite tangent in world space (+U). */
	float3 TangentRight;
	/** The color of the sprite. */
	float4 Color;
	/** The velocity of the particle, XYZ: direction, W: speed. */
	float4 ParticleVelocity;
	/** The uv scale of the sprite. */
	float2 UVScale;
	/** The particle worldspace position */
	FLWCVector3 Position;
	
	float2 Size;
	
	float3 CustomFacingVector;
	float3 CustomAlignmentVector;

#if (DYNAMIC_PARAMETERS_MASK & 1) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	float4 DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	float4 DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	float4 DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	float4 DynamicParameter3;
#endif

	/** The sub-image lerp. */
	float SubImageLerp;

#if USE_PARTICLE_TIME
	/** Relative time. */
	float RelativeTime;
#endif

	/** Transform from tangent space to local space. */
	float3x3 TangentToLocal;
	/** Size of the particle */
	float2 ParticleSize;


#if USE_PARTICLE_SPRITE_ROTATION
	float ParticleSpriteRotation;
#endif

#if USE_PARTICLE_RANDOM
	/** Particle Random value */
	float ParticleRandom;
#endif

	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

float2 GetUV(FVertexFactoryIntermediates Intermediates, uint CoordIndex)
{
	switch (CoordIndex)
	{
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 4
		case 3:		return Intermediates.TexCoord_Unflipped.zw;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3
		case 2:		return Intermediates.TexCoord_Unflipped.xy;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 2
		case 1:		return Intermediates.TexCoord.zw;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 1
		case 0:		return Intermediates.TexCoord.xy;
	#endif
		default:	return float2(0,0);
	}
}

#if NUM_TEX_COORD_INTERPOLATORS
bool UVIndexUseZW(int UVIndex)
{
	return (UVIndex % 2) != 0;
}

float2 GetUV(FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex)
{
	float4 UVVector = Interpolants.TexCoords[UVIndex / 2];
	return UVIndexUseZW(UVIndex) ? UVVector.zw : UVVector.xy;
}

void SetUV(inout FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex, float2 InValue)
{
	FLATTEN
	if (UVIndexUseZW(UVIndex))
	{
		Interpolants.TexCoords[UVIndex / 2].zw = InValue;
	}
	else
	{
		Interpolants.TexCoords[UVIndex / 2].xy = InValue;
	}
}
#endif

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = GetUV(Interpolants, CoordinateIndex);
	}
#endif	// NUM_TEX_COORD_INTERPOLATORS

	Result.VertexColor = 1;

	half4 TangentToWorld0 = Interpolants.TangentToWorld0AndInterp_Sizer;
	half4 TangentToWorld2 = Interpolants.TangentToWorld2;
	float SubImageLerp = Interpolants.TangentToWorld0AndInterp_Sizer.w;

	#if GENERATE_SPHERICAL_PARTICLE_NORMALS && USE_PARTICLE_POSITION
		{	
			// Use displaced position (SvPosition already includes WPO)
			float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(SvPosition);
			float3 SphereCenter = Interpolants.DisplacedParticleCenterForNormal;
			Result.TangentToWorld = GetSphericalParticleNormal(TranslatedWorldPosition, SphereCenter, Interpolants.ParticleTranslatedWorldPositionAndSize.w);
		}
	#else
		Result.TangentToWorld = AssembleTangentToWorld(TangentToWorld0.xyz, TangentToWorld2);
	#endif

	Result.UnMirrored = 1;
	Result.Particle.MacroUV = NiagaraSpriteVF.MacroUVParameters;

#if NEEDS_PARTICLE_COLOR
	Result.Particle.Color = Interpolants.Color;
#endif

#if (DYNAMIC_PARAMETERS_MASK != 0)
	Result.Particle.DynamicParameterValidMask = NiagaraSpriteVF.MaterialParamValidMask;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 1)
	Result.Particle.DynamicParameter = Interpolants.DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	Result.Particle.DynamicParameter1 = Interpolants.DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	Result.Particle.DynamicParameter2 = Interpolants.DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	Result.Particle.DynamicParameter3 = Interpolants.DynamicParameter3;
#endif

#if USE_PARTICLE_POSITION
	Result.Particle.TranslatedWorldPositionAndSize = Interpolants.ParticleTranslatedWorldPositionAndSize;
	Result.Particle.PrevTranslatedWorldPositionAndSize = Result.Particle.TranslatedWorldPositionAndSize;
#endif

#if USE_PARTICLE_VELOCITY
	Result.Particle.Velocity = Interpolants.ParticleVelocity;
#endif

#if USE_PARTICLE_TIME
	Result.Particle.RelativeTime = Interpolants.RelativeTime;
#endif

	Result.Particle.MotionBlurFade = 1.0f;


#if USE_PARTICLE_LIGHTING_OFFSET
	Result.LightingPositionOffset = Interpolants.LightingPositionOffset;
#endif

#if USE_PARTICLE_SUBUVS
	Result.Particle.SubUVCoords[0] = Interpolants.ParticleSubUVs.xy;
	Result.Particle.SubUVCoords[1] = Interpolants.ParticleSubUVs.zw;
	Result.Particle.SubUVLerp = SubImageLerp;
#endif

#if LIGHTMAP_UV_ACCESS
	Result.LightmapUVs = Interpolants.LightMapUVs;
#endif

#if USE_PARTICLE_SIZE
	Result.Particle.Size = Interpolants.ParticleSize;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	Result.Particle.SpriteRotation = Interpolants.ParticleSpriteRotation;
#else
	Result.Particle.SpriteRotation = 0.0f;
#endif

#if USE_PARTICLE_RANDOM
	Result.Particle.Random = Interpolants.ParticleRandom;
#else
	Result.Particle.Random = 0.0f;
#endif

	Result.TwoSidedSign = 1;
	return Result;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.VertexColor = Intermediates.Color;
	Result.TangentToWorld = mul(TangentToLocal, GetLocalToWorld3x3()); 
	Result.Particle.MacroUV = NiagaraSpriteVF.MacroUVParameters;
	Result.Particle.Color = Intermediates.Color;
	Result.Particle.MotionBlurFade = 1.0f;
	Result.PreSkinnedPosition = LWCToFloat(Intermediates.Position);
	Result.PreSkinnedNormal = TangentToLocal[2].xyz;

	// Previous frame not handled deliberately. Lacks necessary information and
	// primitives using this VF are usually transparent and hence don't output velocity
	Result.PrevFrameLocalToWorld = DFPromote(float4x4(float4(1.0f, 0.0f, 0.0f, 0.0f), float4(0.0f, 1.0f, 0.0f, 0.0f), float4(0.0f, 0.0f, 1.0f, 0.0f), float4(0.0f, 0.0f, 0.0f, 1.0f)));

#if (DYNAMIC_PARAMETERS_MASK != 0)
	Result.Particle.DynamicParameterValidMask = NiagaraSpriteVF.MaterialParamValidMask;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 1)
	Result.Particle.DynamicParameter = Intermediates.DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	Result.Particle.DynamicParameter1 = Intermediates.DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	Result.Particle.DynamicParameter2 = Intermediates.DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	Result.Particle.DynamicParameter3 = Intermediates.DynamicParameter3;
#endif

#if USE_PARTICLE_POSITION
	Result.Particle.TranslatedWorldPositionAndSize = Intermediates.TranslatedWorldPositionAndSize;
	Result.Particle.PrevTranslatedWorldPositionAndSize = Result.Particle.TranslatedWorldPositionAndSize;
#endif

#if USE_PARTICLE_VELOCITY
	Result.Particle.Velocity = Intermediates.ParticleVelocity;
#endif

#if USE_PARTICLE_TIME
	Result.Particle.RelativeTime = Intermediates.RelativeTime;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	Result.Particle.SpriteRotation = Intermediates.ParticleSpriteRotation;
#else
	Result.Particle.SpriteRotation = 0.0f;
#endif

#if USE_PARTICLE_RANDOM
	Result.Particle.Random = Intermediates.ParticleRandom;
#else
	Result.Particle.Random = 0.0f;
#endif

#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 1
	Result.TexCoords[0] = GetUV(Intermediates, 0);
#endif
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 2
	Result.TexCoords[1] = GetUV(Intermediates, 1);
#endif
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3
	Result.TexCoords[2] = GetUV(Intermediates, 2);
#endif
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 4
	Result.TexCoords[3] = GetUV(Intermediates, 3);
#endif

#if USE_PARTICLE_SUBUVS
	Result.Particle.SubUVCoords[0] = Intermediates.TexCoord.xy;
	Result.Particle.SubUVCoords[1] = Intermediates.TexCoord.zw;
#endif

#if USE_PARTICLE_SIZE
	Result.Particle.Size = Intermediates.ParticleSize;
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

#if ENABLE_NEW_HLSL_GENERATOR
	// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
	EvaluateVertexMaterialAttributes(Result);
#endif

	return Result;
}

float3 SafeNormalize(float3 V)
{
	// NOTE: Use a very low threshold, close to machine-epsilon when used on near-unit-length vectors. 
	//       This is primarily to avoid division-by-zero, so we can use a small constant and let things blow up slightly
	//       on very small vectors, which won't work very well anyway. 
	float L = length(V);
	return V = V / max(L, 0.000001); 
}

float3 SafeNormalizeWithFallback(float3 V, float3 Fallback)
{
	float L = length(V);
	return L > 0.000001 ? V / L : Fallback; 
}

void GetTangents(FVertexFactoryInput Input, float3 ParticleTranslatedPosition, float Rotation, float3 CustomFacing, float3 CustomAlignment, float3 ParticleDirection, out float3 OutRight, out float3 OutUp)
{
	// Select camera up/right vectors.
	float3 ResolvedViewRight = lerp(ResolvedView.ViewRight, ResolvedView.HMDViewNoRollRight, NiagaraSpriteVF.RemoveHMDRoll);
	float3 ResolvedViewUp = lerp(ResolvedView.ViewUp, ResolvedView.HMDViewNoRollUp, NiagaraSpriteVF.RemoveHMDRoll);

	// The standard camera facing vectors
	float3 CameraRight = ResolvedViewRight;
	float3 CameraUp = -ResolvedViewUp;
	float3 CameraFacingVector = -ResolvedView.ViewForward;

	// Determine the vector from the particle to the camera and the particle's movement direction.
	const float3 CameraVector = ResolvedView.TranslatedWorldCameraOrigin - ParticleTranslatedPosition;
	const float3 CameraDirection = SafeNormalize(CameraVector);

	float3 CustomFacingVector = NiagaraSpriteVFLooseParameters.ParticleFacingMode == FACING_MODE_CUSTOMFACINGVECTOR ? CustomFacing : CameraDirection;
	
	// the particle's movement direction.
	float3 DirectionVector	= lerp(ParticleDirection, CustomAlignment, NiagaraSpriteVFLooseParameters.ParticleAlignmentMode == ALIGNMENT_MODE_CUSTOMALIGNMENT);
	
	// Tangent vectors for any non-default alignment mode.
	float3 CameraRightAligned	= SafeNormalize(cross(CustomFacingVector, DirectionVector));
	float3 CameraUpAligned = lerp(-DirectionVector, cross(CustomFacingVector, CameraRightAligned), NiagaraSpriteVFLooseParameters.ParticleFacingMode == FACING_MODE_CUSTOMFACINGVECTOR);

	// Tangent vectors for camera plane facing mode
	float3 CameraRightPlaneFacing = SafeNormalize(cross(CameraFacingVector,ResolvedViewUp));
	float3 CameraUpPlaneFacing = cross(CameraFacingVector, CameraRightPlaneFacing);

	// Tangent vectors for camera custom facing mode
	float3 CameraRightCustomFacing = SafeNormalize(cross(CustomFacingVector,ResolvedViewUp));
	float3 CameraUpCustomFacing = cross(CustomFacingVector, CameraRightCustomFacing);

	// Tangent vectors for camera facing position.
	float3 CameraRightPosition = SafeNormalize(cross(CameraDirection,float3(0,0,1)));
	float3 CameraUpPosition = cross(CameraDirection, CameraRightPosition);

	float3 Right;
	float3 Up;
	
	/**This could well be better as a branch*/
	BRANCH
	if(NiagaraSpriteVFLooseParameters.ParticleAlignmentMode == ALIGNMENT_MODE_UNALIGNED)
	{
		FLATTEN
		if (NiagaraSpriteVFLooseParameters.ParticleFacingMode == FACING_MODE_FACECAMERADISTANCEBLEND)
		{
			float CameraDistanceSq = dot(CameraVector, CameraVector);
			float Interp = saturate(CameraDistanceSq * NiagaraSpriteVF.CameraFacingBlend.y - NiagaraSpriteVF.CameraFacingBlend.z);
			Right = cross(CameraDirection,float3(0,0,1));
			Up = cross(CameraDirection, Right);
			Right = lerp(Right, CameraRight.xyz, Interp);
			Up = lerp(Up, CameraUp.xyz, Interp);
		} 
		else if(NiagaraSpriteVFLooseParameters.ParticleFacingMode == FACING_MODE_FACECAMERA)
		{
			Right = CameraRight.xyz;
			Up = CameraUp.xyz;
		}		 
		else if(NiagaraSpriteVFLooseParameters.ParticleFacingMode == FACING_MODE_FACECAMERAPLANE)
		{
			Right = CameraRightPlaneFacing;
			Up = CameraUpPlaneFacing;
		}
		else if(NiagaraSpriteVFLooseParameters.ParticleFacingMode == FACING_MODE_CUSTOMFACINGVECTOR)
		{
			Right = CameraRightCustomFacing;
			Up = CameraUpCustomFacing;
		}
		else//(NiagaraSpriteVFLooseParameters.ParticleFacingMode == FACING_MODE_FACECAMERAPOSITION)
		{
			Right = CameraRightPosition;
			Up = CameraUpPosition;
		}
	}
	else 
	{	
		Right = CameraRightAligned;
		Up = CameraUpAligned;
	}


	// Determine the angle of rotation.
	float SinRotation;
	float CosRotation;
	sincos(Rotation, SinRotation, CosRotation);

	float3 SR = Right * SinRotation;
	float3 SU = Up * SinRotation;
	float3 CR = Right * CosRotation;
	float3 CU = Up * CosRotation;
	OutRight = SU + CR;
	OutUp = CU - SR;
}

/** derive basis vectors */
float3x3 CalcTangentBasis(FVertexFactoryIntermediates Intermediates)
{
	// Using camera facing TangentX and TangentY.  The resulting tangent basis is not orthonormal with anything other than ENM_CameraFacing, 
	// So there are artifacts with tangent space calculations like the TransformVector node,
	// But this allows lighting based on a world space shape via the normal while still having normal maps camera aligned.
	float3x3 Result;
	const float2 UVSign = NoneZeroSign(Intermediates.UVScale);
	Result[0] = Intermediates.TangentRight * UVSign.x;
	Result[1] = Intermediates.TangentUp * UVSign.y;

	// ENM_CameraFacing
	//@todo - use static branching
	if (NiagaraSpriteVF.NormalsType < .5f)
	{
		Result[2] = normalize(cross(Result[0],Result[1]));
		
		if (UVSign.x != UVSign.y)
		{
			Result[2] *= -1.0f;
		}
		
	}
	// ENM_Spherical
	else if (NiagaraSpriteVF.NormalsType < 1.5f)
	{
		float3 TangentZ = normalize(LWCToFloat(LWCSubtract(Intermediates.Position, NiagaraSpriteVF.NormalsSphereCenter.xyz)));
		Result[2] = TangentZ;	
	}
	// ENM_Cylindrical
	else
	{
		float3 ClosestPointOnCylinder = NiagaraSpriteVF.NormalsSphereCenter.xyz + dot(NiagaraSpriteVF.NormalsCylinderUnitDirection.xyz, LWCToFloat(LWCSubtract(Intermediates.Position, NiagaraSpriteVF.NormalsSphereCenter.xyz))) * NiagaraSpriteVF.NormalsCylinderUnitDirection.xyz;
		float3 TangentZ = normalize(LWCToFloat(LWCSubtract(Intermediates.Position, ClosestPointOnCylinder)));
		Result[2] = TangentZ;	
	}

	return Result;
}

void ComputeBillboardUVs(FVertexFactoryInput Input, float2 ParticleSize, float SubImageIndex, float2 UVScale, out float2 UVForPosition, out float2 UVForTexturing, out float2 UVForTexturingUnflipped)
{
#if FEATURE_LEVEL >= FEATURE_LEVEL_ES3_1
	const uint NumCutOutVertices = GetCutoutNumVertices();

	BRANCH
	if (NumCutOutVertices > 0)
	{
		// Avoid uint divide which is extremely slow on GCN
		const uint VertexIndexMask = NumCutOutVertices == 8 ? 7 : 3; // BVC_FourVertices || BVC_EightVertices
		uint CutoutVertexIndex = Input.VertexId & VertexIndexMask;
		const float NumFrames = NiagaraSpriteVF.SubImageSize.x * NiagaraSpriteVF.SubImageSize.y;
		const uint SubImageIndexInt = (uint)fmod(SubImageIndex, NumFrames);

		const float2 UVFlip = NoneZeroSign(ParticleSize) * NoneZeroSign(UVScale);
		
		// Reverse the winding order of the polygon when only flipping in one direction to counteract UV flipping
		FLATTEN
		if (UVFlip.x * UVFlip.y < 0.0f)
		{
			CutoutVertexIndex = NumCutOutVertices - 1 - CutoutVertexIndex;
		}

		// Fetch the billboard space positions from the appropriate frame of the preprocessed cutout geometry
		UVForPosition = NiagaraSpriteVFLooseParameters.CutoutGeometry[SubImageIndexInt * NumCutOutVertices + CutoutVertexIndex];
		UVForTexturingUnflipped = UVForPosition;
		UVForTexturing = UVForPosition;
		
		// Invert positions on the billboard so that the cutout geometry still contains the visible portion
		// This changes the winding order when only one direction is flipped
		UVForPosition.x = UVFlip.x < 0.0f ? 1.0f - UVForTexturing.x : UVForTexturing.x;
		UVForPosition.y = UVFlip.y < 0.0f ? 1.0f - UVForTexturing.y : UVForTexturing.y;

		UVForTexturing = float2(0.5f, 0.5f) + ((UVForPosition - float2(0.5f, 0.5f)) * UVScale * NoneZeroSign(ParticleSize));
	}
	else
#endif
	{
		// Note: not inverting positions, as that would change the winding order
		UVForPosition = Input.TexCoord.xy;
		UVForTexturingUnflipped = UVForPosition;
		UVForTexturing = float2(0.5f, 0.5f) + ((UVForTexturingUnflipped - float2(0.5f, 0.5f)) * UVScale * NoneZeroSign(ParticleSize));
	}
}

FLWCVector3 SimToWorldPos(float3 SimSpacePos, FLWCMatrix LocalToWorld)
{
	if (NiagaraSpriteVF.bLocalSpace)
	{
		return LWCMultiply(SimSpacePos, LocalToWorld);
	}
	
	return MakeLWCVector3(NiagaraSpriteVF.SystemLWCTile, SimSpacePos);
}

float3 LocalToWorldVec(float3 LocalSpaceVec, float3x3 LocalToWorld)
{
	const float3 InvScale = GetPrimitiveDataFromUniformBuffer().InvNonUniformScale;
	return
		InvScale.x * LocalToWorld[0] * LocalSpaceVec.xxx +
		InvScale.y * LocalToWorld[1] * LocalSpaceVec.yyy +
		InvScale.z * LocalToWorld[2] * LocalSpaceVec.zzz;
}

float3 SimToWorldVec(float3 SimSpaceVec, float3x3 LocalToWorld)
{
	if (NiagaraSpriteVF.bLocalSpace)
	{
		return LocalToWorldVec(SimSpaceVec, LocalToWorld);
	}
	
	return SimSpaceVec;
}

/*********************************************************************************
 *							Particle specific
 *********************************************************************************/

float3 GetNiagaraParticleSimPosition(uint InstanceID)
{
	return SafeGetVec3(NiagaraSpriteVF.PositionDataOffset, InstanceID, NiagaraSpriteVF.DefaultPos.xyz);
}

FLWCVector3 GetNiagaraParticlePosition(uint InstanceID)
{
	float3 ParticlePos = GetNiagaraParticleSimPosition(InstanceID);
	return SimToWorldPos(ParticlePos, DFFastToTileOffset(GetPrimitiveDataFromUniformBuffer().LocalToWorld));
}

float3 GetNiagaraParticleTranslatedWorldPosition(uint InstanceID)
{
	FLWCVector3 WorldPos = GetNiagaraParticlePosition(InstanceID);
	return LWCToFloat(LWCAdd(WorldPos, ResolvedView.TileOffset.PreViewTranslation));
}

FLWCVector3 GetNiagaraPreviousParticlePosition(uint InstanceID)
{
	float3 PrevParticlePos = SafeGetVec3(NiagaraSpriteVF.PrevPositionDataOffset, InstanceID, NiagaraSpriteVF.DefaultPrevPos.xyz);
	return SimToWorldPos(PrevParticlePos, DFFastToTileOffset(GetPrimitiveDataFromUniformBuffer().PreviousLocalToWorld));
}

float3 GetNiagaraPreviousParticleTranslatedWorldPosition(uint InstanceID)
{
	FLWCVector3 WorldPos = GetNiagaraPreviousParticlePosition(InstanceID);
	return LWCToFloat(LWCAdd(WorldPos, ResolvedView.TileOffset.PrevPreViewTranslation));
}

float3 GetNiagaraParticleVelocity(uint InstanceID)
{
	return SimToWorldVec(SafeGetVec3(NiagaraSpriteVF.VelocityDataOffset, InstanceID, NiagaraSpriteVF.DefaultVelocity.xyz), DFToFloat3x3(GetPrimitiveDataFromUniformBuffer().LocalToWorld));
}

float3 GetNiagaraPreviousParticleVelocity(uint InstanceID)
{
	return SimToWorldVec(SafeGetVec3(NiagaraSpriteVF.PrevVelocityDataOffset, InstanceID, NiagaraSpriteVF.DefaultPrevVelocity.xyz), DFToFloat3x3(GetPrimitiveDataFromUniformBuffer().PreviousLocalToWorld));
}

float2 GetNiagaraParticleSize(uint InstanceID)
{
	return SafeGetVec2(NiagaraSpriteVF.SizeDataOffset, InstanceID, NiagaraSpriteVF.DefaultSize.xy);
}

float2 GetNiagaraPreviousParticleSize(uint InstanceID)
{
	return SafeGetVec2(NiagaraSpriteVF.PrevSizeDataOffset, InstanceID, NiagaraSpriteVF.DefaultPrevSize.xy);
}

float GetNiagaraParticleRotation(uint InstanceID)
{
	// See FNiagaraConstants for a synchronized value...
	const float Rotation = (SafeGetFloat(NiagaraSpriteVF.RotationDataOffset, InstanceID, NiagaraSpriteVF.DefaultRotation) / 180.0f) * PI;
	return Rotation * NiagaraSpriteVF.RotationScale + NiagaraSpriteVF.RotationBias;
}

float GetNiagaraPreviousParticleRotation(uint InstanceID)
{
	// See FNiagaraConstants for a synchronized value...
	const float PrevRotation = (SafeGetFloat(NiagaraSpriteVF.PrevRotationDataOffset, InstanceID, NiagaraSpriteVF.DefaultPrevRotation) / 180.0f) * PI;
	return PrevRotation * NiagaraSpriteVF.RotationScale + NiagaraSpriteVF.RotationBias;
}

float4 GetNiagaraParticleColor(uint InstanceID)
{
	return SafeGetVec4(NiagaraSpriteVF.ColorDataOffset, InstanceID, NiagaraSpriteVF.DefaultColor);
}

float2 GetNiagaraUVScale(uint InstanceID)
{
	return SafeGetVec2(NiagaraSpriteVF.UVScaleDataOffset, InstanceID, NiagaraSpriteVF.DefaultUVScale.xy);
}

float2 GetNiagaraPivotOffset(uint InstanceID)
{
	return SafeGetVec2(NiagaraSpriteVF.PivotOffsetDataOffset, InstanceID, NiagaraSpriteVF.DefaultPivotOffset);
}

float2 GetNiagaraPreviousPivotOffset(uint InstanceID)
{
	return SafeGetVec2(NiagaraSpriteVF.PrevPivotOffsetDataOffset, InstanceID, NiagaraSpriteVF.DefaultPrevPivotOffset);
}

float GetNiagaraParticleRandom(uint InstanceID)
{
	return SafeGetFloat(NiagaraSpriteVF.MaterialRandomDataOffset, InstanceID, NiagaraSpriteVF.DefaultMatRandom);
}

float GetNiagaraCameraOffset(uint InstanceID)
{
	return SafeGetFloat(NiagaraSpriteVF.CameraOffsetDataOffset, InstanceID, NiagaraSpriteVF.DefaultCamOffset);
}

float GetNiagaraPreviousCameraOffset(uint InstanceID)
{
	return SafeGetFloat(NiagaraSpriteVF.PrevCameraOffsetDataOffset, InstanceID, NiagaraSpriteVF.DefaultPrevCamOffset);
}

#if USE_PARTICLE_TIME
float GetNiagaraNormalizedAge(uint InstanceID)
{
	return SafeGetFloat(NiagaraSpriteVF.NormalizedAgeDataOffset, InstanceID, NiagaraSpriteVF.DefaultNormAge);
}
#endif

#if (DYNAMIC_PARAMETERS_MASK & 1) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
float4 GetNiagaraParticleDynamicParameters(uint InstanceID)
{
	return SafeGetVec4(NiagaraSpriteVF.MaterialParamDataOffset, InstanceID, NiagaraSpriteVF.DefaultDynamicMaterialParameter0);
}
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
float4 GetNiagaraParticleDynamicParameters1(uint InstanceID)
{
	return SafeGetVec4(NiagaraSpriteVF.MaterialParam1DataOffset, InstanceID, NiagaraSpriteVF.DefaultDynamicMaterialParameter1);
}
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
float4 GetNiagaraParticleDynamicParameters2(uint InstanceID)
{
	return SafeGetVec4(NiagaraSpriteVF.MaterialParam2DataOffset, InstanceID, NiagaraSpriteVF.DefaultDynamicMaterialParameter2);
}
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
float4 GetNiagaraParticleDynamicParameters3(uint InstanceID)
{
	return SafeGetVec4(NiagaraSpriteVF.MaterialParam3DataOffset, InstanceID, NiagaraSpriteVF.DefaultDynamicMaterialParameter3);
}
#endif

float GetNiagaraParticleSubimage(uint InstanceID)
{
	// See FNiagaraConstants for a synchronized value...
	return SafeGetFloat(NiagaraSpriteVF.SubimageDataOffset, InstanceID, NiagaraSpriteVF.DefaultSubImage);
}

float3 GetNiagaraParticleFacingVector(uint InstanceID)
{
	// See FNiagaraConstants for a synchronized value...
	return SimToWorldVec(SafeGetVec3(NiagaraSpriteVF.FacingDataOffset, InstanceID, NiagaraSpriteVF.DefaultFacing.xyz), DFToFloat3x3(GetPrimitiveDataFromUniformBuffer().LocalToWorld));
}

float3 GetNiagaraPreviousParticleFacingVector(uint InstanceID)
{
	// See FNiagaraConstants for a synchronized value...
	return SimToWorldVec(SafeGetVec3(NiagaraSpriteVF.PrevFacingDataOffset, InstanceID, NiagaraSpriteVF.DefaultPrevFacing.xyz), DFToFloat3x3(GetPrimitiveDataFromUniformBuffer().PreviousLocalToWorld));
}

float3 GetNiagaraParticleAlignmentVector(uint InstanceID)
{
	return SimToWorldVec(SafeGetVec3(NiagaraSpriteVF.AlignmentDataOffset, InstanceID, NiagaraSpriteVF.DefaultAlignment.xyz), DFToFloat3x3(GetPrimitiveDataFromUniformBuffer().LocalToWorld));
}

float3 GetNiagaraPreviousParticleAlignmentVector(uint InstanceID)
{
	return SimToWorldVec(SafeGetVec3(NiagaraSpriteVF.PrevAlignmentDataOffset, InstanceID, NiagaraSpriteVF.DefaultPrevAlignment.xyz), DFToFloat3x3(GetPrimitiveDataFromUniformBuffer().PreviousLocalToWorld));
}

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates = (FVertexFactoryIntermediates)0;
	uint ParticleID = GetInstanceId(GetInstanceIdFromVF(Input)); // NOTE: Handles instanced stereo

	if(NiagaraSpriteVFLooseParameters.SortedIndicesOffset != 0xFFFFFFFF)
	{
		ParticleID = NiagaraSpriteVFLooseParameters.SortedIndices[NiagaraSpriteVFLooseParameters.SortedIndicesOffset + ParticleID];
	}

	const FLWCVector3 ParticlePosition = GetNiagaraParticlePosition(ParticleID);
	const float3 ParticleTranslatedWorldPosition = LWCToFloat(LWCAdd(ParticlePosition, ResolvedView.TileOffset.PreViewTranslation));

	const float ParticleRotation = GetNiagaraParticleRotation(ParticleID);
	const float2 ParticleSize = GetNiagaraParticleSize(ParticleID);
	const float3 ParticleVelocity = GetNiagaraParticleVelocity(ParticleID);
	const float SubImageIndex = GetNiagaraParticleSubimage(ParticleID);
	const float3 CustomFacing = SafeNormalize(GetNiagaraParticleFacingVector(ParticleID));
	const float3 CustomAlignment = SafeNormalize(GetNiagaraParticleAlignmentVector(ParticleID));	
	const float2 PivotOffset = GetNiagaraPivotOffset(ParticleID);
	const float2 UVScale = GetNiagaraUVScale(ParticleID);
	const float3 CameraOffset = SafeNormalize(ResolvedView.TranslatedWorldCameraOrigin - ParticleTranslatedWorldPosition) * GetNiagaraCameraOffset(ParticleID);
	
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	Intermediates.Position = ParticlePosition;
	Intermediates.TranslatedWorldPosition = ParticleTranslatedWorldPosition;
	Intermediates.Size = ParticleSize;
	Intermediates.Color	= GetNiagaraParticleColor(ParticleID);
	Intermediates.CustomFacingVector = CustomFacing;
	Intermediates.CustomAlignmentVector = SafeNormalize(GetNiagaraParticleAlignmentVector(ParticleID));
	Intermediates.UVScale = UVScale;

#if (DYNAMIC_PARAMETERS_MASK & 1) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	Intermediates.DynamicParameter = GetNiagaraParticleDynamicParameters(ParticleID);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	Intermediates.DynamicParameter1 = GetNiagaraParticleDynamicParameters1(ParticleID);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	Intermediates.DynamicParameter2 = GetNiagaraParticleDynamicParameters2(ParticleID);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8) || WITH_NIAGARA_VERTEX_FACTORY_EXPORT
	Intermediates.DynamicParameter3 = GetNiagaraParticleDynamicParameters3(ParticleID);
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	Intermediates.ParticleSpriteRotation = ParticleRotation;
#endif

	// Tangents.
	{
		const float3 ParticleDirection = SafeNormalizeWithFallback(ParticleVelocity, float3(0,0,1));
		float3 Right,Up;

		GetTangents(Input, ParticleTranslatedWorldPosition, ParticleRotation, CustomFacing, CustomAlignment, ParticleDirection, Right, Up);
		Intermediates.TangentUp = Up;
		Intermediates.TangentRight = Right;
	}

	float2 UVForPosition;
	float2 UVForTexturing;
	float2 UVForTexturingUnflipped;
	ComputeBillboardUVs(Input, ParticleSize, SubImageIndex, UVScale, UVForPosition, UVForTexturing, UVForTexturingUnflipped);

	const float2 Size = abs(Intermediates.Size.xy);

	// When enabled try to main a pixel sized sprite and optionally fade particles based on coverage
	// In reality this is more like 2x2 coverage to ensure we always hit 1 pixel, the alternative would be to snap
	// both the size and the location to pixel centers.
	// This also does not work correctly with sprites that are not camera facing, future work may be to
	// calculate the size based on Tangents and the camera axis.
	float PixelSizeRatio = 1.0f;
	if ( NiagaraSpriteVF.PixelCoverageEnabled != 0 )
	{
		const float Depth = dot(ParticleTranslatedWorldPosition - ResolvedView.TranslatedWorldCameraOrigin, ResolvedView.ViewForward);
		const float HalfSize = max(Size.x, Size.y) * 0.5f;
		const float PixelSize = max(View.ViewSizeAndInvSize.z * GetTanHalfFieldOfView().x * Depth * 2.0f, HalfSize);
		PixelSizeRatio = PixelSize / HalfSize;

		float Coverage = saturate(HalfSize * 2.0f / PixelSize);
		Coverage = Coverage * Coverage;
		Intermediates.Color = (Intermediates.Color * Coverage * NiagaraSpriteVF.PixelCoverageColorBlend) + (Intermediates.Color * (1 - NiagaraSpriteVF.PixelCoverageColorBlend));
	}

	// Vertex position
	const float2x3 Tangents = float2x3(Intermediates.TangentRight, Intermediates.TangentUp);
	const float3 VertexOffset = CameraOffset + mul(Size * PixelSizeRatio * (UVForPosition - PivotOffset), Tangents);

	Intermediates.VertexWorldPosition = LWCAdd(ParticlePosition, VertexOffset);
		
	// Prev vertex position (approximated from current velocity unless enabled)
	FLWCVector3 PrevParticlePosition;
	float3 PrevCameraOffset = CameraOffset;
	if ( NiagaraSpriteVF.AccurateMotionVectors != 0 )
	{
		PrevParticlePosition = GetNiagaraPreviousParticlePosition(ParticleID);
		const float3 PrevParticleTranslatedWorldPosition = LWCToFloat(LWCAdd(PrevParticlePosition, ResolvedView.TileOffset.PrevPreViewTranslation));
		const float PrevParticleRotation = GetNiagaraPreviousParticleRotation(ParticleID);
		const float2 PrevParticleSize = GetNiagaraPreviousParticleSize(ParticleID);
		const float3 PrevParticleVelocity = GetNiagaraPreviousParticleVelocity(ParticleID);		
		const float3 PrevCustomFacing = SafeNormalize(GetNiagaraPreviousParticleFacingVector(ParticleID));
		const float3 PrevCustomAlignment = SafeNormalize(GetNiagaraPreviousParticleAlignmentVector(ParticleID));
		const float2 PrevPivotOffset = GetNiagaraPreviousPivotOffset(ParticleID);
		PrevCameraOffset = SafeNormalize(LWCToFloat(LWCSubtract(ResolvedView.TileOffset.PrevWorldCameraOrigin, PrevParticlePosition))) * GetNiagaraPreviousCameraOffset(ParticleID);
		const float3 PrevParticleDirection = SafeNormalizeWithFallback(PrevParticleVelocity, float3(0,0,1));
			
		float3 PrevRight, PrevUp;
		GetTangents(Input, PrevParticleTranslatedWorldPosition, PrevParticleRotation, PrevCustomFacing, PrevCustomAlignment, PrevParticleDirection, PrevRight, PrevUp);
		
		// NOTE: Don't use previous SubImageIndex here because cutout verts don't correlate between sub images
		float2 PrevUVForPosition;
		float2 PrevUVForTexturing;
		float2 PrevUVForTexturingUnflipped;
		ComputeBillboardUVs(Input, PrevParticleSize, SubImageIndex, UVScale, PrevUVForPosition, PrevUVForTexturing, PrevUVForTexturingUnflipped);
			
		const float2x3 PrevTangents = float2x3(PrevRight, PrevUp);
		const float3 PrevVertexOffset = PrevCameraOffset + mul(abs(PrevParticleSize) * (PrevUVForPosition - PivotOffset), PrevTangents);
		
		Intermediates.PreviousVertexWorldPosition = LWCAdd(PrevParticlePosition, PrevVertexOffset);
	}
	else
	{
		Intermediates.PreviousVertexWorldPosition = LWCSubtract(Intermediates.VertexWorldPosition, LWCPromote(ParticleVelocity * NiagaraSpriteVF.DeltaSeconds));
		PrevParticlePosition = LWCSubtract(Intermediates.Position, LWCPromote(ParticleVelocity * NiagaraSpriteVF.DeltaSeconds));
	}

#if USE_PARTICLE_POSITION
	// Do we really want the camera offset here ? What about when GENERATE_SPHERICAL_PARTICLE_NORMALS
	Intermediates.TranslatedWorldPositionAndSize.xyz = LWCToFloat(LWCAdd(Intermediates.Position, ResolvedView.TileOffset.PreViewTranslation)) + CameraOffset;
	Intermediates.TranslatedWorldPositionAndSize.w = .5f * min(Size.x, Size.y);
	
	Intermediates.PrevTranslatedWorldPositionAndSize.xyz = LWCToFloat(LWCAdd(PrevParticlePosition, ResolvedView.TileOffset.PrevPreViewTranslation)) + PrevCameraOffset;
	Intermediates.PrevTranslatedWorldPositionAndSize.w = .5f * min(Size.x, Size.y);
#if GENERATE_SPHERICAL_PARTICLE_NORMALS
	// Recenter the particle position for GENERATE_SPHERICAL_PARTICLE_NORMALS to use the right center.
	Intermediates.TranslatedWorldPositionAndSize.xyz += mul(Size * (float2(.5, .5) - PivotOffset), Tangents);
#endif // GENERATE_SPHERICAL_PARTICLE_NORMALS
#endif // USE_PARTICLE_POSITION
	
	// SubUV.
	float SubImageLerp = frac(SubImageIndex);

	// add a bias to SubImageA to avoid numerical precision issues around SubImageSize.x vs SubImageSize.z
	float SubImageA = SubImageIndex - SubImageLerp + 0.5f;
	float SubImageB = SubImageA + 1;
	float SubImageAH = floor( fmod( SubImageA, NiagaraSpriteVF.SubImageSize.x ) );
	float SubImageBH = floor( fmod( SubImageB, NiagaraSpriteVF.SubImageSize.x ) );
	float SubImageAV = floor( SubImageA * NiagaraSpriteVF.SubImageSize.z );
	float SubImageBV = floor( SubImageB * NiagaraSpriteVF.SubImageSize.z );
	Intermediates.TexCoord.xy = (float2( SubImageAH, SubImageAV ) + UVForTexturing) * NiagaraSpriteVF.SubImageSize.zw;
	Intermediates.TexCoord.zw = (float2(SubImageBH, SubImageBV) + UVForTexturing) * NiagaraSpriteVF.SubImageSize.zw;
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3
	Intermediates.TexCoord_Unflipped.xy = (float2(SubImageAH, SubImageAV) + UVForTexturingUnflipped) * NiagaraSpriteVF.SubImageSize.zw;
	Intermediates.TexCoord_Unflipped.zw = (float2(SubImageBH, SubImageBV) + UVForTexturingUnflipped) * NiagaraSpriteVF.SubImageSize.zw;
#endif
	Intermediates.SubImageLerp = (NiagaraSpriteVF.SubImageBlendMode == 1) ? SubImageLerp : 0.0f;

#if USE_PARTICLE_VELOCITY
	{
		const float3 DirectionVector = NiagaraSpriteVFLooseParameters.ParticleAlignmentMode == ALIGNMENT_MODE_CUSTOMALIGNMENT ? Intermediates.CustomAlignmentVector : ParticleVelocity;
		Intermediates.ParticleVelocity.xyz = normalize(DirectionVector);
		Intermediates.ParticleVelocity.w = length(DirectionVector);
	}
#endif

#if USE_PARTICLE_TIME
	Intermediates.RelativeTime = GetNiagaraNormalizedAge(ParticleID);
#endif

#if USE_PARTICLE_RANDOM
	Intermediates.ParticleRandom = GetNiagaraParticleRandom(ParticleID);
#endif

	Intermediates.TangentToLocal = CalcTangentBasis(Intermediates);

#if USE_PARTICLE_LIGHTING_OFFSET && USE_PARTICLE_RANDOM
	// Hash function based on the particle ID to generate a uniformly distributed 3d offset
	float3 RandomParticleOffset = frac(Square(Intermediates.ParticleRandom * 47 + 10) * float3(1361.456345, 2333.578, 3623.983)) * 2 - 1;
	Intermediates.LightingPositionOffset = .5f * RandomParticleOffset;
#endif

#if USE_PARTICLE_SIZE
	Intermediates.ParticleSize = Size;
#endif

	return Intermediates;
}

float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return float4(LWCToFloat(LWCAdd(Intermediates.VertexWorldPosition, ResolvedView.TileOffset.PreViewTranslation)), 1.0f);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	// No support for instancing, so instance == primitive
	return LWCToFloat(LWCMultiply(Intermediates.VertexWorldPosition, DFToTileOffset(GetPrimitiveDataFromUniformBuffer().WorldToLocal)));
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return CalcTangentBasis(Intermediates)[2].xyz;
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 TranslatedWorldPosition)
{
#if SPHERICAL_PARTICLE_OPACITY
	// For particles using spherical opacity, move the quad toward the viewer so that it lies in front of the sphere defined by the particle
	// This avoids opaque objects intersecting the particle from causing clipping artifacts due to depth testing
	// The downside is that the particle will clip the near plane sooner

	float Radius = .5f * min(Intermediates.Size.x, Intermediates.Size.y);
	return ReprojectPosition(TranslatedWorldPosition, Radius);
#else
	return TranslatedWorldPosition;
#endif
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
#if (TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL)
	// Do per-vertex lighting with particle position instead of vertex position when we're using SubUV cutouts, since the vertex positions vary per-frame which would cause popping
	return GetCutoutUseSubImage() ? Intermediates.TranslatedWorldPosition : TranslatedWorldPosition;
#else
	return TranslatedWorldPosition;
#endif
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

	// Initialize the whole struct to 0
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;

#if NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);

	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		SetUV(Interpolants, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
	}
#endif

#if LIGHTMAP_UV_ACCESS
	Interpolants.LightMapUVs = Intermediates.TexCoord.xy;
#endif

#if USE_PARTICLE_SUBUVS
	Interpolants.ParticleSubUVs.xy = VertexParameters.Particle.SubUVCoords[0];
	Interpolants.ParticleSubUVs.zw = VertexParameters.Particle.SubUVCoords[1];
#endif

	// Calculate the transform from tangent to world space.
	// Note that "local" space for particles is actually oriented in world space! Therefore no rotation is needed.
	float3x3 TangentToWorld = Intermediates.TangentToLocal;

	Interpolants.TangentToWorld0AndInterp_Sizer.xyz = TangentToWorld[0];
	Interpolants.TangentToWorld0AndInterp_Sizer.w = Intermediates.SubImageLerp;
	Interpolants.TangentToWorld2 = float4(TangentToWorld[2], sign(determinant(Intermediates.TangentToLocal)));

#if NEEDS_PARTICLE_COLOR
	Interpolants.Color = Intermediates.Color;
#endif

#if (DYNAMIC_PARAMETERS_MASK & 1)
	Interpolants.DynamicParameter = Intermediates.DynamicParameter;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	Interpolants.DynamicParameter1 = Intermediates.DynamicParameter1;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	Interpolants.DynamicParameter2 = Intermediates.DynamicParameter2;
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	Interpolants.DynamicParameter3 = Intermediates.DynamicParameter3;
#endif

#if USE_PARTICLE_POSITION
	Interpolants.ParticleTranslatedWorldPositionAndSize = Intermediates.TranslatedWorldPositionAndSize;
	#if GENERATE_SPHERICAL_PARTICLE_NORMALS
		// Center for spherical normal must include WPO so shading follows displaced vertices
		Interpolants.DisplacedParticleCenterForNormal = Intermediates.TranslatedWorldPositionAndSize.xyz + GetMaterialWorldPositionOffset(VertexParameters);
	#endif
#endif

#if USE_PARTICLE_VELOCITY
	Interpolants.ParticleVelocity = Intermediates.ParticleVelocity;
#endif

#if USE_PARTICLE_TIME
	Interpolants.RelativeTime = Intermediates.RelativeTime;
#endif

#if USE_PARTICLE_LIGHTING_OFFSET
	Interpolants.LightingPositionOffset = Intermediates.LightingPositionOffset;
#endif

#if USE_PARTICLE_SIZE
	Interpolants.ParticleSize = Intermediates.ParticleSize;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	Interpolants.ParticleSpriteRotation = Intermediates.ParticleSpriteRotation;
#endif

#if USE_PARTICLE_RANDOM
	Interpolants.ParticleRandom = Intermediates.ParticleRandom;
#endif

	return Interpolants;
}

#if RAYHITGROUPSHADER
struct FVertexFactoryRayTracingInterpolants
{
	FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
};

FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryRayTracingInterpolants Interpolants;
	
	Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);
	
	return Interpolants;
}

float2 VertexFactoryGetRayTracingTextureCoordinate(FVertexFactoryRayTracingInterpolants Interpolants)
{
#if NUM_MATERIAL_TEXCOORDS
	return Interpolants.InterpolantsVSToPS.TexCoords[0].xy;
#else // #if NUM_MATERIAL_TEXCOORDS
	return float2(0, 0);
#endif // #if NUM_MATERIAL_TEXCOORDS
}

FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
{
	FVertexFactoryRayTracingInterpolants O;

#if LIGHTMAP_UV_ACCESS
	INTERPOLATE_MEMBER(InterpolantsVSToPS.LightMapUVs);
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for(int tc = 0; tc < (NUM_TEX_COORD_INTERPOLATORS+1)/2; ++tc)
	{
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[tc]);
	}
#endif

#if USE_PARTICLE_SUBUVS
	INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleSubUVs);
#endif

	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld0AndInterp_Sizer);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld2);

#if NEEDS_PARTICLE_COLOR
	INTERPOLATE_MEMBER(InterpolantsVSToPS.Color);
#endif

#if (DYNAMIC_PARAMETERS_MASK & 1)
	INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 2)
	INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter1);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 4)
	INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter2);
#endif
#if (DYNAMIC_PARAMETERS_MASK & 8)
	INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter3);
#endif

#if USE_PARTICLE_POSITION
	INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleTranslatedWorldPositionAndSize);
#endif

#if USE_PARTICLE_VELOCITY
	INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleVelocity);
#endif

#if USE_PARTICLE_TIME
	INTERPOLATE_MEMBER(InterpolantsVSToPS.RelativeTime);
#endif

#if USE_PARTICLE_LIGHTING_OFFSET
	INTERPOLATE_MEMBER(InterpolantsVSToPS.LightingPositionOffset);
#endif

#if USE_PARTICLE_SIZE
	INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleSize);
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleSpriteRotation);
#endif

#if USE_PARTICLE_RANDOM
	INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleRandom);
#endif

	return O;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
{
	return Input.InterpolantsVSToPS;
}
#endif

float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return float4(LWCToFloat(LWCAdd(Intermediates.PreviousVertexWorldPosition, ResolvedView.TileOffset.PrevPreViewTranslation)), 1.0f);
}

// local position relative to primitive
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	// No support for instancing, so instance == primitive
	return LWCToFloat(LWCMultiply(Intermediates.PreviousVertexWorldPosition, DFToTileOffset(GetPrimitiveDataFromUniformBuffer().PreviousWorldToLocal)));
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
float3x3 VertexFactoryGetTangentToLocal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.TangentToLocal;
}

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
#if USE_PARTICLE_POSITION
	return Interpolants.ParticleTranslatedWorldPositionAndSize;
#else
	return 0;
#endif
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

#if WITH_NIAGARA_VERTEX_FACTORY_EXPORT
FVertexFactoryInput NiagaraVertexFactoryExport_InitVertexFactoryInput(uint TriangleIndex, int VertexIndex, uint PrimitiveId, uint DrawInstanceId)
{
	const uint VertsPerInstance = GetCutoutNumVertices() == 0 ? 6 : ((GetCutoutNumVertices() - 2) * 3);
	const uint TrisPerInstance = VertsPerInstance / 3;
	TriangleIndex += TrisPerInstance * DrawInstanceId;
	VertexIndex += VertsPerInstance * DrawInstanceId;
	return LoadVertexFactoryInputForHGS(TriangleIndex, VertexIndex);
}

uint NiagaraVertexFactoryExport_GetIndirectVertexCount()
{
	const uint NumVertices = NiagaraSpriteVFLooseParameters.IndirectArgsBuffer[NiagaraSpriteVFLooseParameters.IndirectArgsOffset + 0];
	const uint NumInstances = NiagaraSpriteVFLooseParameters.IndirectArgsBuffer[NiagaraSpriteVFLooseParameters.IndirectArgsOffset + 1];
	return NumVertices * NumInstances;
}

FLWCVector3 NiagaraVertexFactoryExport_GetPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.VertexWorldPosition;
}

float4 NiagaraVertexFactoryExport_GetColor(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.Color;
}

float3x3 NiagaraVertexFactoryExport_GetTangentBasis(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return CalcTangentBasis(Intermediates);
}

float2 NiagaraVertexFactoryExport_GetTexCoord(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, uint CoordIndex)
{
	return GetUV(Intermediates, CoordIndex);
}

float4 NiagaraVertexFactoryExport_GetDynamicParameter(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, uint ParameterIndex)
{
	float4 Value = 1.0f;
	if ( ParameterIndex == 0 )
	{
		Value = Intermediates.DynamicParameter;
	}
	else if ( ParameterIndex == 1 )
	{
		Value = Intermediates.DynamicParameter1;
	}
	else if ( ParameterIndex == 2 )
	{
		Value = Intermediates.DynamicParameter2;
	}
	else if ( ParameterIndex == 3 )
	{
		Value = Intermediates.DynamicParameter3;
	}
	else
	{
		return 1.0f;
	}

	uint MaskBit = ParameterIndex * 4u;
	for ( int i=0; i < 4; ++i )
	{
		Value[i] = ((NiagaraSpriteVF.MaterialParamValidMask & (1u << MaskBit)) != 0u) ? Value[i] : 1.0f;
		++MaskBit;
	}
	return Value;
}
#endif //WITH_NIAGARA_VERTEX_FACTORY_EXPORT

#include "/Engine/Private/VertexFactoryDefaultInterface.ush"
