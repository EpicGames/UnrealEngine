// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Plugin/FX/Niagara/Private/NiagaraCommon.ush"
#include "/Plugin/FX/Niagara/Private/NiagaraQuaternionUtils.ush"

struct FNiagaraTransform
{
	float3	Translation;
	float4	Rotation;
	float3	Scale;
};

FNiagaraTransform LoadTransform(ByteAddressBuffer RawBufferData, uint Offset)
{
	FNiagaraTransform R;
	R.Translation	= asfloat(RawBufferData.Load3(Offset + 0));
	R.Rotation		= asfloat(RawBufferData.Load4(Offset + 12));
	R.Scale			= asfloat(RawBufferData.Load3(Offset + 28));
	return R;
}

FNiagaraTransform MakeTransform(float3 Translation, float4 Rotation, float3 Scale)
{
	FNiagaraTransform R;
	R.Translation	= Translation;
	R.Rotation		= Rotation;
	R.Scale			= Scale;
	return R;
}

FNiagaraTransform MultiplyTransform(in FNiagaraTransform A, in FNiagaraTransform B)
{
	FNiagaraTransform R;
	R.Translation	= RotateVectorByQuat(A.Translation * B.Scale, B.Rotation) + B.Translation;
	R.Rotation		= MultiplyQuat(B.Rotation, A.Rotation);
	R.Scale			= A.Scale * B.Scale;
	return R;
}

float4x4 TransformToMatrix44(FNiagaraTransform T)
{
	const float3x3 Rotation = NiagaraQuatTo3x3(T.Rotation);
	
	return float4x4(  
		float4(Rotation[0] * T.Scale.x, 0.0f),
		float4(Rotation[1] * T.Scale.y, 0.0f),
		float4(Rotation[2] * T.Scale.z, 0.0f),
		float4(T.Translation, 1.0f)
	);
}

float3x3 TransformToMatrix33(FNiagaraTransform T)
{
	const float3x3 Rotation = NiagaraQuatTo3x3(T.Rotation);
	return float3x3(Rotation[0] * T.Scale.x,  Rotation[1] * T.Scale.y, Rotation[2] * T.Scale.z);
}

FNiagaraTransform DecomposeMatrix(float4x4 InMatrix)
{
	FNiagaraTransform R;
	R.Translation = InMatrix._m30_m31_m32;

	float3 Col0 = InMatrix._m00_m10_m20;
	float3 Col1 = InMatrix._m01_m11_m21;
	float3 Col2 = InMatrix._m02_m12_m23;

	R.Scale = float3(length(Col0), length(Col1), length(Col2));
	float3x3 NormalizedRotationMatrix = transpose(float3x3(Col0 / R.Scale.x, Col1 / R.Scale.y, Col2 / R.Scale.z));
	float3 RotationVectors[3] =
	{
		NormalizedRotationMatrix[0],
		NormalizedRotationMatrix[1],
		NormalizedRotationMatrix[2]
	};

	R.Rotation = QuatFromMatrix(RotationVectors);

	return R;
}

// PackedMatrix here is a linear matrix packed for serialization.  Effectively a float4x3 transposed into vec4s
FNiagaraTransform DecomposePackedMatrix(float3x4 InPackedMatrix)
{
	float4x4 Matrix = transpose(float4x4(InPackedMatrix, float4(0.0f, 0.0f, 0.0f, 1.0f)));

	return DecomposeMatrix(Matrix);
}