// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ComputeShaderUtils.ush"
#include "/Engine/Private/GPUScene/GPUSceneWriter.ush"
#include "NiagaraMeshParticleUtils.ush"

Buffer<uint> SortedIndices;
int SortedIndicesOffset;

float3 SystemLWCTile;
int bLocalSpace;
float DeltaSeconds;
uint FacingMode;

int ScaleDataOffset;
int RotationDataOffset;
int PositionDataOffset;
int VelocityDataOffset;
int CameraOffsetDataOffset;
int PrevScaleDataOffset;
int PrevRotationDataOffset;
int PrevPositionDataOffset;
int PrevVelocityDataOffset;
int PrevCameraOffsetDataOffset;

float3 DefaultScale;
float4 DefaultRotation;
float3 DefaultPosition;
float3 DefaultVelocity;
float DefaultCameraOffset;
float3 DefaultPrevScale;
float4 DefaultPrevRotation;
float3 DefaultPrevPosition;
float3 DefaultPrevVelocity;
float DefaultPrevCameraOffset;

float3 MeshScale;
float4 MeshRotation;
int bMeshOffsetIsWorldSpace;
float3 MeshOffset;

uint bLockedAxisEnable;
float3 LockedAxis;
uint LockedAxisSpace;

uint PrimitiveId;

uint ParticleCount;
Buffer<uint> GPUParticleCountBuffer;
uint GPUParticleCountOffset;

int MeshIndex;
int MeshIndexDataOffset;
int RendererVisibility;
int VisibilityTagDataOffset;

float3 LocalBoundingCenter;
float2 DistanceCullRangeSquared;
float4 LODScreenSize;

int bNeedsPrevTransform;

uint GetPayloadDataFlags()
{
	uint Flags = INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA;
	if (bNeedsPrevTransform)
	{
		Flags |= INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA;
	}
	return Flags;
}

void CullInstance(uint InstanceId)
{
	// This will minimally initialize the instance data such that it will be ignored and culled
	WriteInstancePrimitiveIdAndFlags(InstanceId, PrimitiveId, INSTANCE_SCENE_DATA_FLAG_HIDDEN);
}

[numthreads(THREAD_GROUP_SIZE, 1, 1)]
void UpdateMeshInstancesCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint DispatchId = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, THREAD_GROUP_SIZE);

	// Retrieve the primitive data
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveId);
	const uint LocalInstanceId = DispatchId.x;
	const uint InstanceId = PrimitiveData.InstanceSceneDataOffset + LocalInstanceId;
	uint NumParticles = ParticleCount;
	if (LocalInstanceId >= NumParticles)
	{
		// Past the CPU particle count, bail and do nothing
		return;
	}

	if (GPUParticleCountOffset != (uint)-1)
	{
		NumParticles = GPUParticleCountBuffer[GPUParticleCountOffset];
		if (LocalInstanceId >= NumParticles)
		{
			// Past the GPU particle count, flag the instance as culled
			CullInstance(InstanceId);
			return;
		}
	}

	uint ParticleIndex = LocalInstanceId;
	if (SortedIndicesOffset != -1)
	{
		ParticleIndex = SortedIndices[SortedIndicesOffset + LocalInstanceId];
	}

	if (NiagaraSafeGetInt(MeshIndexDataOffset, ParticleIndex, 0) != MeshIndex ||
		NiagaraSafeGetInt(VisibilityTagDataOffset, ParticleIndex, RendererVisibility) != RendererVisibility)
	{
		// Culled from either mesh index or visibility tag mismatch
		CullInstance(InstanceId);
		return;
	}

	NiagaraMeshParticleTransformsParams Params;
	Params.ParticleIndex				= ParticleIndex;
	Params.SystemLWCTile				= SystemLWCTile;
	Params.bLocalSpace					= bLocalSpace;
#if ENABLE_PRECISE_MOTION_VECTORS
	Params.bPreciseMotionVectors 		= true;
#else
	Params.bPreciseMotionVectors 		= false;
#endif
	Params.FacingMode					= FacingMode;
	Params.DeltaSeconds					= DeltaSeconds;
	Params.MeshScale 					= MeshScale;
	Params.MeshOffset 					= MeshOffset;
	Params.MeshRotation					= MeshRotation;
	Params.bMeshOffsetIsWorldSpace		= bMeshOffsetIsWorldSpace;
	Params.bLockedAxisEnable			= bLockedAxisEnable;
	Params.LockedAxis					= LockedAxis;
	Params.LockedAxisSpace				= LockedAxisSpace;
	Params.ScaleDataOffset 				= ScaleDataOffset;
	Params.RotationDataOffset 			= RotationDataOffset;
	Params.PositionDataOffset 			= PositionDataOffset;
	Params.CameraOffsetDataOffset		= CameraOffsetDataOffset;
	Params.PrevScaleDataOffset 			= PrevScaleDataOffset;
	Params.PrevRotationDataOffset 		= PrevRotationDataOffset;
	Params.PrevPositionDataOffset 		= PrevPositionDataOffset;
	Params.PrevCameraOffsetDataOffset	= PrevCameraOffsetDataOffset;
	Params.DefaultScale 				= DefaultScale;
	Params.DefaultRotation 				= DefaultRotation;
	Params.DefaultPosition 				= DefaultPosition;
	Params.DefaultCameraOffset 			= DefaultCameraOffset;
	Params.DefaultPrevScale 			= DefaultPrevScale;
	Params.DefaultPrevRotation 			= DefaultPrevRotation;
	Params.DefaultPrevPosition 			= DefaultPrevPosition;
	Params.DefaultPrevCameraOffset 		= DefaultPrevCameraOffset;
	Params.VelocityDirMag				= NiagaraGetVelocityDirMag(VelocityDataOffset, DefaultVelocity, ParticleIndex);
	Params.PrevVelocityDirMag			= NiagaraGetVelocityDirMag(PrevVelocityDataOffset, DefaultPrevVelocity, ParticleIndex);
	Params.CameraOrigin 				= PrimaryView.TileOffset.WorldCameraOrigin;
	Params.CameraForwardDir				= View.ViewForward;
	Params.CameraUpDir					= View.ViewUp;
	Params.PrevCameraOrigin 			= PrimaryView.TileOffset.PrevWorldCameraOrigin;
	Params.PrevCameraForwardDir			= View.PrevViewToTranslatedWorld[2].xyz;
	Params.PrevCameraUpDir				= View.PrevViewToTranslatedWorld[1].xyz;
	Params.PrimitiveLocalToWorld		= DFToTileOffset(PrimitiveData.LocalToWorld);
	Params.PrimitiveWorldToLocal 		= DFToTileOffset(PrimitiveData.WorldToLocal);
	Params.PrimitivePrevLocalToWorld 	= DFToTileOffset(PrimitiveData.PreviousLocalToWorld);
	Params.PrimitiveInvNonUniformScale	= PrimitiveData.InvNonUniformScale;

	NiagaraMeshParticleTransforms Transforms = NiagaraCalculateMeshParticleTransforms(Params);

	// Transform the bounding center to world-space and distance cull
	{
		const FLWCVector3 BCenter = LWCMultiply(LocalBoundingCenter, Transforms.LocalToWorld);
		const float3 CamVec = LWCToFloat(LWCSubtract(BCenter, PrimaryView.TileOffset.WorldCameraOrigin));
		const float CamDistSq = length2(CamVec);
		if (CamDistSq < DistanceCullRangeSquared.x  || CamDistSq > DistanceCullRangeSquared.y)
		{
			// We are outside the distance ranges required
			CullInstance(InstanceId);
			return;
		}

		const float3 ParticleScale = NiagaraSafeGetVec3(Params.ScaleDataOffset, Params.ParticleIndex, Params.DefaultScale);
		const float3 FinalScale = ParticleScale * MeshScale * PrimitiveData.NonUniformScale.xyz;
		const float MaxScaleAxis = max3(abs(FinalScale.x), abs(FinalScale.y), abs(FinalScale.z));
		const float ScaledCamDstSq = MaxScaleAxis > 0.0f ? CamDistSq / (MaxScaleAxis*MaxScaleAxis) : CamDistSq;
		
		const float ScreenRadius = saturate(LODScreenSize.z / max(1.0f, ScaledCamDstSq * LODScreenSize.w));
		if ( ScreenRadius < LODScreenSize.x || ScreenRadius >= LODScreenSize.y )
		{
			CullInstance(InstanceId);
			return;
		}
	}

	const uint PayloadDataFlags = GetPayloadDataFlags();
	InitializeInstanceSceneDataWS(
		InstanceId,
		PrimitiveId,
		LocalInstanceId,
 		PayloadDataFlags,
 		1,
 		0.0f,
		DFFromTileOffset(Transforms.LocalToWorld)
	);
	
	FInstancePayloadDataOffsets PayloadOffsets = GetInstancePayloadDataOffsets(PrimitiveId, PayloadDataFlags, LocalInstanceId);
	WriteInstanceDynamicDataWS(PayloadOffsets, PrimitiveId, DFFromTileOffset(Transforms.PrevLocalToWorld));
	WriteInstanceCustomData(PayloadOffsets, 0, float4(asfloat(ParticleIndex), 0.0f, 0.0f, 0.0f));
}