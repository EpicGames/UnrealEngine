// Copyright Epic Games, Inc. All Rights Reserved.

/*********************************************************************************
 *							Niagara Vertex Factory data access
 *********************************************************************************/

// These lines set up access to Niagara particle data buffers for Vertex Factories
// NOTE: the buffer accessors have to be macros to appease certain shader platforms
#define NiagaraGetFloatBuffer() (NiagaraVFLooseParameters.NiagaraParticleDataFloat)
#define NiagaraGetHalfBuffer() (NiagaraVFLooseParameters.NiagaraParticleDataHalf)
uint NiagaraGetFloatDataStride() { return NiagaraVFLooseParameters.NiagaraFloatDataStride; }
#define USE_GLOBAL_NIAGARA_DATA_BUFFERS 0 // We provided the buffer accessors above
#define ENABLE_NIAGARA_INT_DATA_ACCESS 0 // Currently, ints not supported in VFs (but could be)
#include "/Plugin/FX/Niagara/Private/NiagaraParticleAccess.ush"

// Define some shorter name functions for legacy use in existing VFs
float GetFloat(int RegisterIdx, uint InstanceID) { return NiagaraGetFloat(RegisterIdx, InstanceID); }	
float2 GetVec2(int RegisterIdx, uint InstanceID) { return NiagaraGetVec2(RegisterIdx, InstanceID); }
float3 GetVec3(int RegisterIdx, uint InstanceID) { return NiagaraGetVec3(RegisterIdx, InstanceID); }
float4 GetVec4(int RegisterIdx, uint InstanceID) { return NiagaraGetVec4(RegisterIdx, InstanceID); }
float SafeGetFloat(int RegisterIdx, uint InstanceID, float Default) { return NiagaraSafeGetFloat(RegisterIdx, InstanceID, Default); }
float2 SafeGetVec2(int RegisterIdx, uint InstanceID, float2 Default) { return NiagaraSafeGetVec2(RegisterIdx, InstanceID, Default); }
float3 SafeGetVec3(int RegisterIdx, uint InstanceID, float3 Default) { return NiagaraSafeGetVec3(RegisterIdx, InstanceID, Default); }
float4 SafeGetVec4(int RegisterIdx, uint InstanceID, float4 Default) { return NiagaraSafeGetVec4(RegisterIdx, InstanceID, Default); }
