// Copyright Epic Games, Inc. All Rights Reserved.

// Change to force shader compilation of this shader

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ComputeShaderUtils.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"
 
#ifndef THREAD_GROUP_SIZE
	#define THREAD_GROUP_SIZE 64
#endif

uint IsIndirectDraw;
uint NumInstances;
uint NumVerticesPerInstance;
uint bApplyWPO;
uint bWorldSpace;
uint SectionInfoOutputOffset;

uint VertexStride;
uint VertexPositionOffset;
uint VertexColorOffset;
uint VertexTangentBasisOffset;
uint VertexTexCoordOffset;
uint VertexTexCoordNum;
uint4 VertexTexCoordBindings[(8 * 2) / 4];		// NUM_TEXCOORDS * Channels
uint VertexOutputOffset;
RWByteAddressBuffer	RWVertexData;

static const uint INVALID_COMPONENT = 0xffffffff;

void OutputPosition(uint VertexIndex, float3 Value)
{
	if ( VertexPositionOffset != INVALID_COMPONENT )
	{
		const uint OutputVertexIndex = VertexOutputOffset + VertexIndex;
		const uint Offset = (OutputVertexIndex * VertexStride) + VertexPositionOffset;
		RWVertexData.Store3(Offset, asuint(Value));
	}
}

void OutputColor(uint VertexIndex, float4 Value)
{
	if ( VertexColorOffset != INVALID_COMPONENT )
	{
		const uint OutputVertexIndex = VertexOutputOffset + VertexIndex;
		const uint Offset = (OutputVertexIndex * VertexStride) + VertexColorOffset;
		RWVertexData.Store4(Offset, asuint(Value));
	}
}

void OutputTangentBasis(uint VertexIndex, float3x3 Value)
{
	if ( VertexTangentBasisOffset != INVALID_COMPONENT )
	{
		const uint OutputVertexIndex = VertexOutputOffset + VertexIndex;
		const uint Offset = (OutputVertexIndex * VertexStride) + VertexTangentBasisOffset;
		RWVertexData.Store3(Offset, asuint(Value[0]));
		RWVertexData.Store3(Offset + 12, asuint(Value[1]));
		RWVertexData.Store3(Offset + 24, asuint(Value[2]));
	}
}

void OutputTexCoord(uint VertexIndex, uint TexCoord, float2 Value)
{
	if ( VertexTexCoordOffset != INVALID_COMPONENT && TexCoord < VertexTexCoordNum )
	{
		const uint OutputVertexIndex = VertexOutputOffset + VertexIndex;
		const uint Offset = (OutputVertexIndex * VertexStride) + VertexTexCoordOffset + (8 * TexCoord);
		RWVertexData.Store2(Offset, asuint(Value));
	}
}

[numthreads(THREAD_GROUP_SIZE, 1, 1)]
void VertexFactoryExportCS(uint3 GroupId : SV_GroupID, int GroupThreadIndex : SV_GroupIndex)
{
	const uint VertexIndex = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, THREAD_GROUP_SIZE);
	const uint MaxOrNumVertices = NumInstances * NumVerticesPerInstance;
	const uint NumVertices = IsIndirectDraw == 0 ? MaxOrNumVertices : NiagaraVertexFactoryExport_GetIndirectVertexCount();
	if (VertexIndex == 0)
	{
		RWVertexData.Store(SectionInfoOutputOffset, NumVertices);
	}

	if (VertexIndex >= NumVertices)
	{
		// Helpful for debugging as it is obvious the position is NaN
		if (VertexIndex < MaxOrNumVertices)
		{
			uint4 NaN = uint4(0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF);
			OutputPosition(VertexIndex, asfloat(NaN.xyz));
		}
		return;
	}

	// Genreate the vertex data
	ResolvedView = ResolveView();

	const uint	PrimitiveId = 0;
	const int	InstanceId = VertexIndex / NumVerticesPerInstance;
	const uint	InstanceVertexIndex = VertexIndex % NumVerticesPerInstance;
	
	FVertexFactoryInput Input = NiagaraVertexFactoryExport_InitVertexFactoryInput(InstanceVertexIndex / 3, InstanceVertexIndex % 3, PrimitiveId, InstanceId);

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);

	float4 TranslatedWorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, TranslatedWorldPosition.xyz, TangentToLocal);

	float3 MaterialWPO = 0;
	if (bApplyWPO)
	{
		MaterialWPO = MakeFinite(GetMaterialWorldPositionOffset(VertexParameters));
	}

	const FLWCVector3 VertexPosition = NiagaraVertexFactoryExport_GetPosition(Input, VFIntermediates);
	const float4 VertexColor = NiagaraVertexFactoryExport_GetColor(Input, VFIntermediates);
	const float3x3 VertexTangentBasis = NiagaraVertexFactoryExport_GetTangentBasis(Input, VFIntermediates);
	if (bWorldSpace)
	{
		OutputPosition(VertexIndex, LWCToFloat(VertexPosition) + MaterialWPO);
		OutputTangentBasis(VertexIndex, VertexTangentBasis);
	}
	else
	{
		const FLWCInverseMatrix WorldToLocal = DFToTileOffset(GetPrimitiveDataFromUniformBuffer().WorldToLocal);
		OutputPosition(VertexIndex, LWCMultiply(VertexPosition, WorldToLocal) + MaterialWPO);
		OutputTangentBasis(VertexIndex, mul(VertexTangentBasis, (float3x3)WorldToLocal.M));
	}
	OutputColor(VertexIndex, VertexColor);
	
	if ( VertexTexCoordNum > 0 )
	{
		float4 BindingValues[8];
		for ( int i=0; i < 4; ++ i)
		{
			BindingValues[i] = NiagaraVertexFactoryExport_GetDynamicParameter(Input, VFIntermediates, i);
		}
		for ( int i=0; i < 4; ++ i)
		{
			BindingValues[4 + i].xy = NiagaraVertexFactoryExport_GetTexCoord(Input, VFIntermediates, (i * 2) + 0);
			BindingValues[4 + i].zw = NiagaraVertexFactoryExport_GetTexCoord(Input, VFIntermediates, (i * 2) + 1);
		}

		for ( uint i=0; i < VertexTexCoordNum; ++i )
		{
			const uint UInt4Index = i / 2;
			const uint UInt4Offset = (i % 2) * 2;
			
			const uint XBinding = VertexTexCoordBindings[UInt4Index][UInt4Offset + 0];
			const uint YBinding = VertexTexCoordBindings[UInt4Index][UInt4Offset + 1];
			const float2 Value = float2(BindingValues[XBinding / 4][XBinding % 4], BindingValues[YBinding / 4][YBinding % 4]);
			
			OutputTexCoord(VertexIndex, i, Value);
		}
	}
}
