// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NiagaraVisualizeTexture.usf
=============================================================================*/

#include "/Engine/Private/Common.ush"

#if TEXTURE_INTEGER
	Texture2D<int4>			Texture2DObject;
	Texture2DArray<int4>	Texture2DArrayObject;
	Texture3D<int4>			Texture3DObject;
	TextureCube<int4>		TextureCubeObject;
#else
	Texture2D<float4>		Texture2DObject;
	Texture2DArray<float4>	Texture2DArrayObject;
	Texture3D<float4>		Texture3DObject;
	TextureCube<float4>		TextureCubeObject;
#endif
SamplerState	TextureSampler;

int4		NumTextureAttributes;		// Number of attributes the texture holds in each dimension
int			NumAttributesToVisualize;	// Number of valid attributes to visualize, up to 4
int4		AttributesToVisualize;		// Indicies of each attribute to visualize
uint3		TextureDimensions;			// Dimensions of the texture
float4		PerChannelScale;
float4		PerChannelBias;
uint		DebugFlags;
uint		TickCounter;
uint		TextureSlice;

float4 ContainsNaNInf(float4 Color)
{
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	if ( DebugFlags & 0x1 )
	{
		if (isfinite(Color.rgb).x == false)
		{
			Color = ((TickCounter >> 3) & 1) == 0 ? float4(1,0,0,1) : float4(1,1,1,1);
		}
	}
#endif
	return Color;
}

float2 GetAttributeUV(float2 UV, int AttributeIndex)
{
	AttributeIndex = min(AttributeIndex, (NumTextureAttributes.x * NumTextureAttributes.y) - 1);

	UV = UV / float2(NumTextureAttributes.xy);

	int UIndex = AttributeIndex % NumTextureAttributes.x;
	int VIndex = AttributeIndex / NumTextureAttributes.x;

	return UV + float2(UIndex, VIndex) * 1.0f / NumTextureAttributes.xy;
}

float ReadAttribute2D(float2 UV, int AttributeIndex)
{
#if TEXTURE_INTEGER
	int3 LoadLocation = int3(GetAttributeUV(UV, AttributeIndex) * TextureDimensions.xy, 0);
	return Texture2DObject.Load(LoadLocation).r;
#else
	return Texture2DObject.Sample(TextureSampler, GetAttributeUV(UV, AttributeIndex)).r;
#endif
}

float3 GetAttributeUVW(float2 UV, int AttributeIndex)
{
	int TileIndexX = AttributeIndex % NumTextureAttributes.x;
	int TileIndexY = (AttributeIndex / NumTextureAttributes.x) % NumTextureAttributes.y;
	int TileIndexZ = AttributeIndex / (NumTextureAttributes.x * NumTextureAttributes.y);	
	float W = 0.0f;
		
	float3 UVW =
	{
		UV.x / NumTextureAttributes.x + 1.0*TileIndexX/NumTextureAttributes.x,
		UV.y / NumTextureAttributes.y + 1.0*TileIndexY/NumTextureAttributes.y,
		W / NumTextureAttributes.z + 1.0*TileIndexZ/NumTextureAttributes.y
	};
	
		
	return UVW;
}

float ReadAttribute3D(float2 UV, int AttributeIndex)
{
#if TEXTURE_INTEGER
	int4 LoadLocation = int4(GetAttributeUVW(UV, AttributeIndex) * TextureDimensions.xyz, 0);
	return Texture3DObject.Load(LoadLocation).r;
#else
	return Texture3DObject.SampleLevel(TextureSampler, GetAttributeUVW(UV, AttributeIndex), 0).r;
#endif
}

//////////////////////////////////////////////////////////////////////////

void Main(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
)
{
// Texture 2D
#if TEXTURE_TYPE == 0
	float4 Color = float4(0,0,0,1);
	if ( NumAttributesToVisualize > 0 )
	{
		float2 AttributeUV = Input.UV / float2(NumTextureAttributes.xy);

		switch ( NumAttributesToVisualize )
		{
			case 1:
				Color.rgb = ReadAttribute2D(Input.UV, AttributesToVisualize.x);
				break;
			case 2:
				Color.r = ReadAttribute2D(Input.UV, AttributesToVisualize.x).r;
				Color.g = ReadAttribute2D(Input.UV, AttributesToVisualize.y).r;
				break;
			case 3:
				Color.r = ReadAttribute2D(Input.UV, AttributesToVisualize.x).r;
				Color.g = ReadAttribute2D(Input.UV, AttributesToVisualize.y).r;
				Color.b = ReadAttribute2D(Input.UV, AttributesToVisualize.z).r;
				break;
			case 4:
				Color.r = ReadAttribute2D(Input.UV, AttributesToVisualize.x).r;
				Color.g = ReadAttribute2D(Input.UV, AttributesToVisualize.y).r;
				Color.b = ReadAttribute2D(Input.UV, AttributesToVisualize.z).r;
				Color.a = ReadAttribute2D(Input.UV, AttributesToVisualize.w).r;
				break;
			default:
				Color = float4(1,0,0,1);
				break;
		}
	}
	else
	{
	#if TEXTURE_INTEGER
		int3 LoadLocation = int3(Input.UV * TextureDimensions.xy, 0);
		Color = Texture2DObject.Load(LoadLocation);
	#else
		Color = Texture2DObject.Sample(TextureSampler, Input.UV);
	#endif
	}
// Texture 2D Array
#elif TEXTURE_TYPE == 1
	float4 Color = float4(0,0,0,1);
	{
	#if TEXTURE_INTEGER
		int4 LoadLocation = int4(Input.UV * TextureDimensions.xy, TextureSlice, 0);
		Color = Texture2DArrayObject.Load(LoadLocation);
	#else
		Color = Texture2DArrayObject.Sample(TextureSampler, float3(Input.UV, TextureSlice));
	#endif
	}
// Texture 3D
#elif TEXTURE_TYPE == 2
	float4 Color = float4(0,0,0,1);
	if ( NumAttributesToVisualize > 0 )
	{
		float2 AttributeUV = Input.UV ;/// float2(NumTextureAttributes.xy);

		switch ( NumAttributesToVisualize )
		{
			case 1:
				Color.rgb = ReadAttribute3D(Input.UV, AttributesToVisualize.x);
				break;
			case 2:
				Color.r = ReadAttribute3D(Input.UV, AttributesToVisualize.x).r;
				Color.g = ReadAttribute3D(Input.UV, AttributesToVisualize.y).r;
				break;
			case 3:
				Color.r = ReadAttribute3D(Input.UV, AttributesToVisualize.x).r;
				Color.g = ReadAttribute3D(Input.UV, AttributesToVisualize.y).r;
				Color.b = ReadAttribute3D(Input.UV, AttributesToVisualize.z).r;
				break;
			case 4:
				Color.r = ReadAttribute3D(Input.UV, AttributesToVisualize.x).r;
				Color.g = ReadAttribute3D(Input.UV, AttributesToVisualize.y).r;
				Color.b = ReadAttribute3D(Input.UV, AttributesToVisualize.z).r;
				Color.a = ReadAttribute3D(Input.UV, AttributesToVisualize.w).r;
				break;
			default:
				Color = float4(1,0,0,1);
				break;
		}
	}
	else
	{
		float W = (float(TextureSlice) + 0.5f) / float(TextureDimensions.z);
	#if TEXTURE_INTEGER
		int4 LoadLocation = int4(float3(Input.UV, W) * TextureDimensions.xyz, 0);
		Color = Texture3DObject.Load(LoadLocation);
	#else
		Color = Texture3DObject.Sample(TextureSampler, float3(Input.UV, W));
	#endif
	}
// TextureCube
#elif TEXTURE_TYPE == 3
	float4 Color = float4(0,0,0,1);
	{
		const float PI = 3.14159265;
		float p = Input.UV.x * PI * 2.0f;
		float t = (-Input.UV.y + 0.5f) * PI;

		float3 CubeDir = float3(cos(p)*cos(t), sin(t), sin(p) * cos(t));

	#if !TEXTURE_INTEGER
		Color = TextureCubeObject.Sample(TextureSampler, CubeDir);
	#endif
	}
#endif
	Color = (Color * PerChannelScale) - PerChannelBias;
	OutColor = ContainsNaNInf(Color);
}
