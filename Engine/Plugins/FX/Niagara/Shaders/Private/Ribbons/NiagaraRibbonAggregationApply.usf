// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "NiagaraRibbonCommon.ush"

Buffer<uint> SortedIndices;

StructuredBuffer<FRibbonAccumulationValues> InputAccumulation;

RWBuffer<float> OutputTangentsAndDistances;

#if HAS_RIBBON_ID
RWBuffer<uint> OutputMultiRibbonIndices;
#endif

RWBuffer<float> OutputTessellationStats;

RWBuffer<uint> OutputSegments;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void RibbonAggregationApply(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint TotalNumParticles = GetTotalNumParticles();
	const uint RawParticleID = DispatchThreadId.x;

	if (RawParticleID < TotalNumParticles)
	{
		OutputTangentsAndDistances[4 * RawParticleID + 3] = InputAccumulation[RawParticleID].RibbonDistance;

		// deal with the masking of invalid segments that has been inserted into the OutputSegments during initialization
		if (OutputSegments[RawParticleID] != INDEX_NONE)
		{
			OutputSegments[RawParticleID] = InputAccumulation[RawParticleID].SegmentCount - 1;
		}

#if HAS_RIBBON_ID
		OutputMultiRibbonIndices[RawParticleID] = InputAccumulation[RawParticleID].MultiRibbonCount - 1;
#endif

		if (RibbonWantsAutomaticTessellation())
		{
			const uint TessellationBaseOffset = RawParticleID * GetTessellationDataStride();

			OutputTessellationStats[TessellationBaseOffset + 0] = InputAccumulation[RawParticleID].TessTotalLength;
			OutputTessellationStats[TessellationBaseOffset + 1] = InputAccumulation[RawParticleID].TessAvgSegmentLength;
			OutputTessellationStats[TessellationBaseOffset + 2] = InputAccumulation[RawParticleID].TessAvgSegmentAngle;
			if (RibbonHasTwist())
			{
				OutputTessellationStats[TessellationBaseOffset + 3] = InputAccumulation[RawParticleID].TessTwistAvgAngle;
				OutputTessellationStats[TessellationBaseOffset + 4] = InputAccumulation[RawParticleID].TessTwistAvgWidth;
			}
		}
	}
}
