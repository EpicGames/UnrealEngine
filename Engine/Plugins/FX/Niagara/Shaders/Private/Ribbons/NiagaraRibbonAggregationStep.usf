// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "NiagaraRibbonCommon.ush"

Buffer<uint> SortedIndices;

StructuredBuffer<FRibbonAccumulationValues> InputAccumulation;
RWStructuredBuffer<FRibbonAccumulationValues> OutputAccumulation;

uint PrefixScanStride;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void RibbonAggregationStep(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint TotalNumParticles = GetTotalNumParticles();
	const uint RawParticleID = DispatchThreadId.x;
	
	if (RawParticleID < TotalNumParticles)
	{
		if (RawParticleID >= PrefixScanStride)
		{
			const uint RawSourceParticleID = RawParticleID - PrefixScanStride;

			// ribbon distance is treated a bit differently because we have to evaluate if the ribbon id is consistent between the two points
#if HAS_RIBBON_ID
			const uint SourceParticleID = SortedIndices[RawSourceParticleID];
			const uint ParticleID = SortedIndices[RawParticleID];
			
			const FRibbonID SourceRibbonID = GetRibbonID(SourceParticleID);
			const FRibbonID RibbonID = GetRibbonID(ParticleID);

			if (AreRibbonIDsEqual(SourceRibbonID, RibbonID))
#endif
			{
				OutputAccumulation[RawParticleID].RibbonDistance = InputAccumulation[RawParticleID].RibbonDistance + InputAccumulation[RawSourceParticleID].RibbonDistance;
			}
#if HAS_RIBBON_ID
			else
			{
				OutputAccumulation[RawParticleID].RibbonDistance = InputAccumulation[RawParticleID].RibbonDistance;
			}
#endif

			OutputAccumulation[RawParticleID].SegmentCount = InputAccumulation[RawParticleID].SegmentCount + InputAccumulation[RawSourceParticleID].SegmentCount;
#if HAS_RIBBON_ID
			OutputAccumulation[RawParticleID].MultiRibbonCount = InputAccumulation[RawParticleID].MultiRibbonCount + InputAccumulation[RawSourceParticleID].MultiRibbonCount;
#endif // HAS_RIBBON_ID

			if (RibbonWantsAutomaticTessellation())
			{
				OutputAccumulation[RawParticleID].TessTotalLength = InputAccumulation[RawParticleID].TessTotalLength + InputAccumulation[RawSourceParticleID].TessTotalLength;
				OutputAccumulation[RawParticleID].TessAvgSegmentLength = InputAccumulation[RawParticleID].TessAvgSegmentLength + InputAccumulation[RawSourceParticleID].TessAvgSegmentLength;
				OutputAccumulation[RawParticleID].TessAvgSegmentAngle = InputAccumulation[RawParticleID].TessAvgSegmentAngle + InputAccumulation[RawSourceParticleID].TessAvgSegmentAngle;
				if (RibbonHasTwist())
				{
					OutputAccumulation[RawParticleID].TessTwistAvgAngle = InputAccumulation[RawParticleID].TessTwistAvgAngle + InputAccumulation[RawSourceParticleID].TessTwistAvgAngle;
					OutputAccumulation[RawParticleID].TessTwistAvgWidth = InputAccumulation[RawParticleID].TessTwistAvgWidth + InputAccumulation[RawSourceParticleID].TessTwistAvgWidth;
				}
			}
		}
		else
		{
			OutputAccumulation[RawParticleID] = InputAccumulation[RawParticleID];
		}
	}
}
