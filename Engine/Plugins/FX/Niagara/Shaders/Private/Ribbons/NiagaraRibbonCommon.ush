// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Plugin/FX/Niagara/Shared/NiagaraRibbonShared.h"

#define INDEX_NONE (-1)
#define INT_MIN         (-2147483647 - 1)
#define UE_SMALL_NUMBER			(1.e-8f)

#define PACKED_PER_RIBBON_DATA_STRIDE 8
#define PACKED_PER_RIBBON_UVDATAOFFSET 0
#define PACKED_PER_RIBBON_STARTPARTICLEOFFSET 6
#define PACKED_PER_RIBBON_ENDPARTICLEOFFSET 7

#define VERTEX_GEN_OUTPUT_DATA_FINALIZATION_INDIRECT_ARGS_OFFSET 0
#define VERTEX_GEN_OUTPUT_DATA_TOTAL_NUM_SEGMENTS_OFFSET 3
#define VERTEX_GEN_OUTPUT_DATA_TOTAL_NUM_RIBBONS_OFFSET 4

// VERTEX_GEN_OUTPUT_DATA_TESSELLATION is only valid when RibbonTessellationMode is none zero
#define VERTEX_GEN_OUTPUT_DATA_TESSELLATION_ANGLE 5
#define VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURVATURE 6
#define VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TWIST_ANGLE 7
#define VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TWIST_CURVATURE 8
#define VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TOTAL_SEGMENT_LENGTH 9

#define VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_TOTAL_SEGMENT_LENGTH 10
#define VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_SEGMENT_LENGTH 11
#define VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_SEGMENT_ANGLE 12
#define VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_TWIST_ANGLE 13
#define VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_WIDTH 14

// This layout must match FNiagaraRibbonIndirectDrawBufferLayout
#define INDEX_GEN_EXECUTE_INDIRECT_OFFSET			0
#define INDEX_GEN_DRAW_INDIRECT_OFFSET				3		// Duplicated in NiagaraRibbonVertexFactory.ush
#define INDEX_GEN_STEREO_DRAW_INDIRECT_OFFSET		8

#define INDEX_GEN_TESSELLATION_FACTOR_OFFSET		13
#define INDEX_GEN_NUM_SEGMENTS_OFFSET				14
#define INDEX_GEN_NUM_SUB_SEGMENTS_OFFSET			15		// Duplicated in NiagaraRibbonVertexFactory.ush
#define INDEX_GEN_ONE_OVER_NUM_SUB_SEGMENTS_OFFSET	16		// Duplicated in NiagaraRibbonVertexFactory.ush

#define INDEX_GEN_INDIRECT_ARGS_STRIDE				17


// Total particle Count
uint TotalNumParticlesDirect;
Buffer<uint> EmitterParticleCountsBuffer;
uint EmitterParticleCountsBufferOffset;

// Niagara sim data
Buffer<float> NiagaraParticleDataFloat;
Buffer<float> NiagaraParticleDataHalf;
Buffer<int> NiagaraParticleDataInt;
int NiagaraFloatDataStride;
int NiagaraIntDataStride;

// Int bindings
int RibbonIdDataOffset;

// Float bindings
int PositionDataOffset;
int PrevPositionDataOffset;
int VelocityDataOffset;
int WidthDataOffset;
int PrevWidthDataOffset;
int TwistDataOffset;
int PrevTwistDataOffset;
int ColorDataOffset;
int FacingDataOffset;
int PrevFacingDataOffset;
int NormalizedAgeDataOffset;
int MaterialRandomDataOffset;
uint MaterialParamValidMask;
int MaterialParamDataOffset;
int MaterialParam1DataOffset;
int MaterialParam2DataOffset;
int MaterialParam3DataOffset;
int DistanceFromStartOffset;
int U0OverrideDataOffset;
int V0RangeOverrideDataOffset;
int U1OverrideDataOffset;
int V1RangeOverrideDataOffset;
int U0DistributionMode;
int U1DistributionMode;

int RibbonLinkOrderIsFloat;
int RibbonLinkOrderDataOffset;

int RibbonTessellationMode;

// Gets the num particles from the indirect buffer if available, or if not from the fallback direct input
int GetTotalNumParticles()
{
	if (EmitterParticleCountsBufferOffset != INDEX_NONE)
	{
		return EmitterParticleCountsBuffer[EmitterParticleCountsBufferOffset];
	}
	return TotalNumParticlesDirect;
}


#define USE_GLOBAL_NIAGARA_DATA_BUFFERS 0 // We provided the buffer accessors above
#define ENABLE_NIAGARA_INT_DATA_ACCESS 1 // Enable int support as we need switch

#define NiagaraGetFloatBuffer() (NiagaraParticleDataFloat)
#define NiagaraGetHalfBuffer() (NiagaraParticleDataHalf)
#define NiagaraGetIntBuffer() (NiagaraParticleDataInt)
uint NiagaraGetFloatDataStride() { return NiagaraFloatDataStride; }
uint NiagaraGetIntDataStride() { return NiagaraIntDataStride; }

#include "/Plugin/FX/Niagara/Private/NiagaraParticleAccess.ush"

// Define some shorter name functions for legacy use in existing VFs
float GetFloat(int RegisterIdx, uint InstanceID) { return NiagaraGetFloat(RegisterIdx, InstanceID); }	
float2 GetVec2(int RegisterIdx, uint InstanceID) { return NiagaraGetVec2(RegisterIdx, InstanceID); }
float3 GetVec3(int RegisterIdx, uint InstanceID) { return NiagaraGetVec3(RegisterIdx, InstanceID); }
float4 GetVec4(int RegisterIdx, uint InstanceID) { return NiagaraGetVec4(RegisterIdx, InstanceID); }
float SafeGetFloat(int RegisterIdx, uint InstanceID, float Default) { return NiagaraSafeGetFloat(RegisterIdx, InstanceID, Default); }
float2 SafeGetVec2(int RegisterIdx, uint InstanceID, float2 Default) { return NiagaraSafeGetVec2(RegisterIdx, InstanceID, Default); }
float3 SafeGetVec3(int RegisterIdx, uint InstanceID, float3 Default) { return NiagaraSafeGetVec3(RegisterIdx, InstanceID, Default); }
float4 SafeGetVec4(int RegisterIdx, uint InstanceID, float4 Default) { return NiagaraSafeGetVec4(RegisterIdx, InstanceID, Default); }

int GetInt(int RegisterIdx, uint InstanceID) { return NiagaraGetInt(RegisterIdx, InstanceID); }
int2 GetInt2(int RegisterIdx, uint InstanceID) { return NiagaraGetInt2(RegisterIdx, InstanceID); }
int SafeGetInt(int RegisterIdx, uint InstanceID, int Default) { return NiagaraSafeGetInt(RegisterIdx, InstanceID, Default); }
int2 SafeGetInt2(int RegisterIdx, uint InstanceID, int2 Default) { return NiagaraSafeGetInt2(RegisterIdx, InstanceID, Default); }

#define HAS_RIBBON_ID (RIBBONID_IS_NIAGARAID || RIBBONID_IS_INT)

#define FNiagaraID int2
static const FNiagaraID InvalidID = { INT_MIN, INT_MIN };

#if RIBBONID_IS_NIAGARAID

#define FRibbonID int2
	static const FNiagaraID InvalidRibbonID = { INT_MIN, INT_MIN };

	FRibbonID GetRibbonID(int InstanceID)
	{
		return GetInt2(RibbonIdDataOffset, InstanceID);
	}

	FNiagaraID ConvertRibbonIDToNiagaraID(FRibbonID RibbonID)
	{
		return RibbonID;
	}

	FNiagaraID GetRibbonIDAsNiagaraID(int InstanceID)
	{
		return GetRibbonID(InstanceID);
	}

	bool AreRibbonIDsEqual(FRibbonID Left, FRibbonID Right)
	{
		return Left.x == Right.x && Left.y == Right.y;
	}

	int CompareRibbonIDs(FRibbonID Left, FRibbonID Right)
	{
		if (Left.x > Right.x)
		{
			return 1;
		}

		if (Left.x == Right.x)
		{
			if (Left.y > Right.y)
			{
				return 1;
			}

			if (Left.y == Right.y)
			{
				return 0;
			}			
		}

		return -1;		
	}

#elif RIBBONID_IS_INT

#define FRibbonID int
	static const FRibbonID InvalidRibbonID = INT_MIN;

	FRibbonID GetRibbonID(int InstanceID)
	{
		return GetInt(RibbonIdDataOffset, InstanceID);
	}

	FNiagaraID ConvertRibbonIDToNiagaraID(FRibbonID RibbonID)
	{
		return int2(RibbonID.x, INT_MIN);
	}

	FNiagaraID GetRibbonIDAsNiagaraID(int InstanceID)
	{
		return ConvertRibbonIDToNiagaraID(GetRibbonID(InstanceID));
	}

	bool AreRibbonIDsEqual(FRibbonID Left, FRibbonID Right)
	{
		return Left == Right;
	}

	int CompareRibbonIDs(FRibbonID Left, FRibbonID Right)
	{
		if (Left > Right)
		{
			return 1;
		}

		if (Left == Right)
		{
			return 0;		
		}

		return -1;		
	}

#endif

float GetRibbonLinkID(int InstanceID)
{
	if ( RibbonLinkOrderIsFloat != 0 )
	{
		return GetFloat(RibbonLinkOrderDataOffset, InstanceID);
	}
	else
	{
		return GetInt(RibbonLinkOrderDataOffset, InstanceID);
	}
}

bool RibbonHasTwist()
{
	return TwistDataOffset != -1;
}

bool RibbonWantsAutomaticTessellation()
{
	return RibbonTessellationMode == ENiagaraGpuTessellationMode::Automatic;	
}

bool RibbonWantsConstantTessellation()
{
	return RibbonTessellationMode == ENiagaraGpuTessellationMode::Constant;
}

uint GetTessellationDataStride()
{
	return RibbonHasTwist() ? NIAGARA_RIBBON_TESSELLATION_FULL_STATS_NUM_ELEMENTS : NIAGARA_RIBBON_TESSELLATION_NO_TWIST_STATS_NUM_ELEMENTS;
}

uint GetVertexGenOutputDataStride()
{
	// See FNiagaraRibbonCommandBufferLayout
	return RibbonTessellationMode == ENiagaraGpuTessellationMode::Automatic ? VERTEX_GEN_OUTPUT_DATA_AUTO_TESSELLATION_NUM_ELEMENTS : VERTEX_GEN_OUTPUT_DATA_NUM_ELEMENTS;
}

// max absolute error 9.0x10^-3
// Eberly's polynomial degree 1 - respect bounds
// input [-1, 1] and output [0, PI]
float AcosFast(float InX)
{
	float X = abs(InX);
	float Res = -0.156583f * X + (0.5 * PI);
	Res *= sqrt(max(0.f, 1.0f - X));
	return (InX >= 0) ? Res : PI - Res;
}

