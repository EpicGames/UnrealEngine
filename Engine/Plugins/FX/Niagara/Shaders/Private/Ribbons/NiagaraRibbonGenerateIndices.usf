// Copyright Epic Games, Inc. All Rights Reserved.


#include "/Engine/Private/Common.ush"
#include "NiagaraRibbonCommon.ush"

// Output generated vertices buffer
RWBuffer<uint> GeneratedIndicesBuffer;

Buffer<uint> SortedIndices;
// Index of the ribbon for this particle
// created through prefix sum across all the starting points of the ribbons
#if HAS_RIBBON_ID
Buffer<uint> MultiRibbonIndices;
#endif
// Index of the final segment this particle starts
Buffer<uint> Segments;


// Buffer containing the indirect dispatch args we're using as well as the
// tessellation factor from the previous stage
Buffer<uint> IndirectDrawInfo;

// Buffer containing the triangle to vertex ids for the current shape
Buffer<uint> TriangleToVertexIds;

uint IndexBufferOffset;
uint IndirectDrawInfoIndex;
uint TriangleToVertexIdsCount;

uint TrianglesPerSegment;
uint NumVerticesInSlice;
uint BitsNeededForShape;
uint BitMaskForShape;
uint SegmentBitShift;
uint SegmentBitMask;
uint SubSegmentBitShift;
uint SubSegmentBitMask;

uint PackIndex(uint SortedIndex, uint SubSegmentID, uint SliceVertex)
{
	return	((SortedIndex	& SegmentBitMask)		<< SegmentBitShift) |
			((SubSegmentID	& SubSegmentBitMask)	<< SubSegmentBitShift) |
			((SliceVertex	& BitMaskForShape));
}

[numthreads(THREADGROUP_SIZE, 1, 1)]
void GenerateIndices(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint TotalNumParticles = GetTotalNumParticles();
	const uint MaxPossibleSegments = TotalNumParticles > 0 ? TotalNumParticles - 1 : 0;
	
	const uint IndirectDrawOffset = IndirectDrawInfoIndex * INDEX_GEN_INDIRECT_ARGS_STRIDE;
	
	const uint SubSegmentCount = IndirectDrawInfo[IndirectDrawOffset + INDEX_GEN_TESSELLATION_FACTOR_OFFSET];
	
	const uint RawParticleID = DispatchThreadId.x / SubSegmentCount;
	const uint SubSegmentIndex = DispatchThreadId.x % SubSegmentCount;

	const uint IndicesPerSubSegment = TrianglesPerSegment * 3;	
	const uint SegmentIndex = Segments[RawParticleID];
	
	// make sure to ignore those segments that have been marked as invalid (INDEX_NONE)
	if (RawParticleID < MaxPossibleSegments && SegmentIndex != INDEX_NONE)
	{		
#if HAS_RIBBON_ID
		const int ThisRibbonID = MultiRibbonIndices[RawParticleID];
		const int NextRibbonID = MultiRibbonIndices[RawParticleID + 1];
		
		if (ThisRibbonID == NextRibbonID)
#endif		
		{			
			uint IndicesOffset = IndexBufferOffset + (SegmentIndex * SubSegmentCount * IndicesPerSubSegment) + (SubSegmentIndex * IndicesPerSubSegment);
			
			
			const uint FlipGeometryIndex = TriangleToVertexIdsCount / 2;

			const bool bIsFinalInterp = SubSegmentIndex == SubSegmentCount - 1;

			const uint ThisSegmentOffset = RawParticleID << SegmentBitShift;
			const uint NextRawParticleID = RawParticleID + (bIsFinalInterp ? 1 : 0);
			const uint NextSegmentOffset = NextRawParticleID << SegmentBitShift;

			const uint ThisSubSegmentOffset = SubSegmentIndex << SubSegmentBitShift;
			const uint NextSubSegmentIndex = (bIsFinalInterp ? 0 : SubSegmentIndex + 1);
			const uint NextSubSegmentOffset = NextSubSegmentIndex << SubSegmentBitShift;

			const uint CurrSegment = ThisSegmentOffset | ThisSubSegmentOffset;
			const uint NextSegment = NextSegmentOffset | NextSubSegmentOffset;
	    
			uint TriangleId = 0;

			for (; TriangleId < FlipGeometryIndex; TriangleId += 2)
			{
				const uint FirstIndex = TriangleToVertexIds[TriangleId];
				const uint SecondIndex = TriangleToVertexIds[TriangleId + 1];

				GeneratedIndicesBuffer[IndicesOffset + 0] = CurrSegment | FirstIndex;
				GeneratedIndicesBuffer[IndicesOffset + 1] = CurrSegment | SecondIndex;				
				GeneratedIndicesBuffer[IndicesOffset + 2] = NextSegment | FirstIndex;				
				GeneratedIndicesBuffer[IndicesOffset + 3] = CurrSegment | SecondIndex;	//GeneratedIndicesBuffer[IndicesOffset + 1];				
				GeneratedIndicesBuffer[IndicesOffset + 4] = NextSegment | SecondIndex;				
				GeneratedIndicesBuffer[IndicesOffset + 5] = NextSegment | FirstIndex;	//GeneratedIndicesBuffer[IndicesOffset + 2];

				IndicesOffset += 6;
			}
			
			for (; TriangleId < TriangleToVertexIdsCount; TriangleId += 2)
			{
				const uint FirstIndex = TriangleToVertexIds[TriangleId];
				const uint SecondIndex = TriangleToVertexIds[TriangleId + 1];

				GeneratedIndicesBuffer[IndicesOffset + 0] = CurrSegment | FirstIndex;				
				GeneratedIndicesBuffer[IndicesOffset + 1] = CurrSegment | SecondIndex;				
				GeneratedIndicesBuffer[IndicesOffset + 2] = NextSegment | SecondIndex;				
				GeneratedIndicesBuffer[IndicesOffset + 3] = CurrSegment | FirstIndex;	//GeneratedIndicesBuffer[IndicesOffset + 0];				
				GeneratedIndicesBuffer[IndicesOffset + 4] = NextSegment | SecondIndex;	//GeneratedIndicesBuffer[IndicesOffset + 2];				
				GeneratedIndicesBuffer[IndicesOffset + 5] = NextSegment | FirstIndex;
				
				IndicesOffset += 6;
			}		
		}
	}
}
