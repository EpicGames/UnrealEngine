// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "NiagaraRibbonCommon.ush"

RWBuffer<uint> IndirectDrawOutput;

// Stats of the vertex gen, including tessellation stats
Buffer<uint> VertexGenerationResults;

uint IndirectDrawOutputIndex;
uint VertexGenerationResultsIndex;

uint IndexGenThreadSize;
uint TrianglesPerSegment;

float ViewDistance;
float LODDistanceFactor;
uint TessellationMode;
uint bCustomUseConstantFactor;
uint CustomTessellationFactor;
float CustomTessellationMinAngle;
uint bCustomUseScreenSpace;
uint GNiagaraRibbonMaxTessellation;
float GNiagaraRibbonTessellationAngle;
float GNiagaraRibbonTessellationScreenPercentage;
uint GNiagaraRibbonTessellationEnabled;
float GNiagaraRibbonTessellationMinDisplacementError;

#define TESSELLATIONMODE_AUTOMATIC 0
#define TESSELLATIONMODE_CUSTOM 1
#define TESSELLATIONMODE_DISABLED 2

[numthreads(1, 1, 1)]
void InitializeIndices(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint TotalNumParticles = GetTotalNumParticles();
	const uint IndirectDrawOffset = IndirectDrawOutputIndex * INDEX_GEN_INDIRECT_ARGS_STRIDE;

	// Early out for no particles
	if (TotalNumParticles == 0)
	{
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_EXECUTE_INDIRECT_OFFSET + 0] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_EXECUTE_INDIRECT_OFFSET + 1] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_EXECUTE_INDIRECT_OFFSET + 2] = 0;
			
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 0] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 1] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 2] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 3] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 4] = 0;
	
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_STEREO_DRAW_INDIRECT_OFFSET + 0] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_STEREO_DRAW_INDIRECT_OFFSET + 1] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_STEREO_DRAW_INDIRECT_OFFSET + 2] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_STEREO_DRAW_INDIRECT_OFFSET + 3] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_STEREO_DRAW_INDIRECT_OFFSET + 4] = 0;
	
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_TESSELLATION_FACTOR_OFFSET] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_NUM_SEGMENTS_OFFSET] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_NUM_SUB_SEGMENTS_OFFSET] = 0;
		IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_ONE_OVER_NUM_SUB_SEGMENTS_OFFSET] = asuint(0.0f);
		return;
	}

	const uint VertexGenerationResultsOffset = VertexGenerationResultsIndex * GetVertexGenOutputDataStride();
	uint SegmentTessellation = 1;
	if ( RibbonWantsAutomaticTessellation() )
	{
		bool bUseConstantFactor = false;
		uint TessellationFactor = GNiagaraRibbonMaxTessellation;
		float TessellationMinAngle = GNiagaraRibbonTessellationAngle;
		float ScreenPercentage = GNiagaraRibbonTessellationScreenPercentage;

		switch (TessellationMode)
		{
			case TESSELLATIONMODE_AUTOMATIC:
				break;
			case TESSELLATIONMODE_CUSTOM:
				TessellationFactor = min(TessellationFactor, CustomTessellationFactor); // Don't allow factors bigger than the platform limit.
				bUseConstantFactor = bCustomUseConstantFactor == 1;
				TessellationMinAngle = CustomTessellationMinAngle;
				ScreenPercentage = bCustomUseScreenSpace == 1 && !bUseConstantFactor ? GNiagaraRibbonTessellationScreenPercentage : 0.f;
				break;
			case TESSELLATIONMODE_DISABLED:
				TessellationFactor = 1;
				break;
			default:
				break;
		}

		float TessellationAngle = asfloat(VertexGenerationResults[VertexGenerationResultsOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_ANGLE]);
		float TessellationCurvature = asfloat(VertexGenerationResults[VertexGenerationResultsOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURVATURE]);
		float TessellationTwistAngle = asfloat(VertexGenerationResults[VertexGenerationResultsOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TWIST_ANGLE]);
		float TessellationTwistCurvature = asfloat(VertexGenerationResults[VertexGenerationResultsOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TWIST_CURVATURE]);
		float TessellationTotalSegmentLength = asfloat(VertexGenerationResults[VertexGenerationResultsOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TOTAL_SEGMENT_LENGTH]);

		if (GNiagaraRibbonTessellationEnabled == 1 && TessellationFactor > 1 && TessellationCurvature > UE_SMALL_NUMBER)
		{	
			const float MinTesselation = (TessellationMinAngle == 0.f || bUseConstantFactor)?
											 float(TessellationFactor)	:
											 max(1.f, max(TessellationTwistAngle, TessellationAngle) / max(UE_SMALL_NUMBER, TessellationMinAngle));

			const float MAX_CURVATURE_FACTOR = 0.002f; // This will clamp the curvature to around 2.5 km and avoid numerical issues.
		
			const float MaxDisplacementError = max(GNiagaraRibbonTessellationMinDisplacementError, ScreenPercentage * sqrt(ViewDistance) / LODDistanceFactor);
			float Tess = TessellationAngle / max(MAX_CURVATURE_FACTOR, acos(TessellationCurvature / (TessellationCurvature + MaxDisplacementError)));
			// FMath::RoundUpToPowerOfTwo ? This could avoid vertices moving around as tesselation increases

			if (TessellationTwistAngle > 0 && TessellationTwistCurvature > 0)
			{
				const float TwistTess = TessellationTwistAngle / max(MAX_CURVATURE_FACTOR, acos(TessellationTwistCurvature / (TessellationTwistCurvature + MaxDisplacementError)));
				Tess = max(TwistTess, Tess);
			}
			SegmentTessellation = clamp(uint(round(Tess)), uint(round(MinTesselation)), TessellationFactor);

			// Ensure we don't blow the max tessellation allowed, this shouldn't happen but let's be safe
			SegmentTessellation = min(SegmentTessellation, GNiagaraRibbonMaxTessellation);
		}
	}
	// Has a constant factor so just use it
	else if ( RibbonWantsConstantTessellation() )
	{
		SegmentTessellation = min(GNiagaraRibbonMaxTessellation, CustomTessellationFactor);
	}

	// We need to run for every possible segment * tessellation so one less than total particles * tessellation
	const uint TotalNumInvocations = (TotalNumParticles - 1) * SegmentTessellation;	
	const uint NumThreadGroups = DivideAndRoundUp(TotalNumInvocations, IndexGenThreadSize);

	const uint SegmentCount = VertexGenerationResults[VertexGenerationResultsOffset + VERTEX_GEN_OUTPUT_DATA_TOTAL_NUM_SEGMENTS_OFFSET];

	const float OneOverSubSegmentTessellation = 1.0f / float(SegmentTessellation);
	
	// Output the dispatch indirect args followed by the segmentation amount
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_EXECUTE_INDIRECT_OFFSET + 0] = NumThreadGroups;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_EXECUTE_INDIRECT_OFFSET + 1] = 1;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_EXECUTE_INDIRECT_OFFSET + 2] = 1;

	// DrawIndirect Params: IndexCount, NumInstances, FirstIndexOffset, FirstVertexOffset, FirstInstanceOffset
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 0] = SegmentCount * SegmentTessellation * TrianglesPerSegment * 3;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 1] = 1;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 2] = 0;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 3] = 0;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_DRAW_INDIRECT_OFFSET + 4] = 0;

	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_STEREO_DRAW_INDIRECT_OFFSET + 0] = SegmentCount * SegmentTessellation * TrianglesPerSegment * 3;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_STEREO_DRAW_INDIRECT_OFFSET + 1] = 2;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_STEREO_DRAW_INDIRECT_OFFSET + 2] = 0;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_STEREO_DRAW_INDIRECT_OFFSET + 3] = 0;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_STEREO_DRAW_INDIRECT_OFFSET + 4] = 0;

	// Misc data for generation
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_TESSELLATION_FACTOR_OFFSET] = SegmentTessellation;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_NUM_SEGMENTS_OFFSET] = SegmentCount;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_NUM_SUB_SEGMENTS_OFFSET] = SegmentTessellation;
	IndirectDrawOutput[IndirectDrawOffset + INDEX_GEN_ONE_OVER_NUM_SUB_SEGMENTS_OFFSET] = asuint(OneOverSubSegmentTessellation);
}
