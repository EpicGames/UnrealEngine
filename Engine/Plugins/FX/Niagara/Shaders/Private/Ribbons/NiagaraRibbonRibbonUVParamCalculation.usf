// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "NiagaraRibbonCommon.ush"

Buffer<uint> SortedIndices;
// Tangent of the particle, and the distance along the ribbon if we're computing it
RWBuffer<float> TangentsAndDistances;

// Buffer containing basic stats about each ribbon
// including the first/last indices, as well as UV offsets/scales/distributions for both channels
RWBuffer<uint> PackedPerRibbonData;

// Output of the previous stage, allows us access to things like total num segments, and total num ribbons
Buffer<uint> CommandBuffer;
int CommandBufferOffset;

// Settings for UV Channel 0
float2 UV0Settings_Offset;
float2 UV0Settings_Scale;
float UV0Settings_TilingLength;
int UV0Settings_DistributionMode;
int UV0Settings_LeadingEdgeMode;
int UV0Settings_TrailingEdgeMode;
int UV0Settings_bEnablePerParticleUOverride;
int UV0Settings_bEnablePerParticleVRangeOverride;

// Settings for UV Channel 1
float2 UV1Settings_Offset;
float2 UV1Settings_Scale;
float UV1Settings_TilingLength;
int UV1Settings_DistributionMode;
int UV1Settings_LeadingEdgeMode;
int UV1Settings_TrailingEdgeMode;
int UV1Settings_bEnablePerParticleUOverride;
int UV1Settings_bEnablePerParticleVRangeOverride;

struct FNiagaraRibbonUVSettings
{
	float2 Offset;
	float2 Scale;
	float TilingLength;
	int DistributionMode;
	int LeadingEdgeMode;
	int TrailingEdgeMode;
	bool bEnablePerParticleUOverride;
	bool bEnablePerParticleVRangeOverride;
};

FNiagaraRibbonUVSettings GetUV0Params()
{
	FNiagaraRibbonUVSettings NewParams;
	NewParams.Offset = UV0Settings_Offset;
	NewParams.Scale = UV0Settings_Scale;
	NewParams.TilingLength = UV0Settings_TilingLength;
	NewParams.DistributionMode = UV0Settings_DistributionMode;
	NewParams.LeadingEdgeMode = UV0Settings_LeadingEdgeMode;
	NewParams.TrailingEdgeMode = UV0Settings_TrailingEdgeMode;
	NewParams.bEnablePerParticleUOverride = UV0Settings_bEnablePerParticleUOverride;
	NewParams.bEnablePerParticleVRangeOverride = UV0Settings_bEnablePerParticleVRangeOverride;
	return NewParams;
}
FNiagaraRibbonUVSettings GetUV1Params()
{
	FNiagaraRibbonUVSettings NewParams;
	NewParams.Offset = UV1Settings_Offset;
	NewParams.Scale = UV1Settings_Scale;
	NewParams.TilingLength = UV1Settings_TilingLength;
	NewParams.DistributionMode = UV1Settings_DistributionMode;
	NewParams.LeadingEdgeMode = UV1Settings_LeadingEdgeMode;
	NewParams.TrailingEdgeMode = UV1Settings_TrailingEdgeMode;
	NewParams.bEnablePerParticleUOverride = UV1Settings_bEnablePerParticleUOverride;
	NewParams.bEnablePerParticleVRangeOverride = UV1Settings_bEnablePerParticleVRangeOverride;
	return NewParams;
}

struct FRibbonUVScaleAndOffsets
{
	float UScale;
	float UOffset;
	float UDistributionScalar;
};

#define UV_EDGE_MODE_SMOOTH_TRANSITION 0
#define UV_EDGE_MODE_LOCKED 1

#define UV_DISTRIBUTION_MODE_SCALED_UNIFORMLY 0
#define UV_DISTRIBUTION_MODE_SCALED_USING_RIBBON_SEGMENT_LENGTH 1
#define UV_DISTRIBUTION_MODE_TILED_OVER_RIBBON_LENGTH 2
#define UV_DISTRIBUTION_MODE_TILED_FROM_START_OVER_RIBBON_LENGTH 3

float3 LoadTangent(int Index)
{
	return float3(
		TangentsAndDistances[Index * 4 + 0],
		TangentsAndDistances[Index * 4 + 1],
		TangentsAndDistances[Index * 4 + 2]);
}

void StoreTangent(int Index, float3 NewTangent)
{
	TangentsAndDistances[Index * 4 + 0] = NewTangent.x;
	TangentsAndDistances[Index * 4 + 1] = NewTangent.y;
	TangentsAndDistances[Index * 4 + 2] = NewTangent.z;
}

float GetSegmentDistance(uint ParticleID)
{
	return TangentsAndDistances[ParticleID * 4 + 3];
}

FRibbonUVScaleAndOffsets CalculateUVScaleAndOffsets(const FNiagaraRibbonUVSettings UVSettings, uint FirstIndex, uint LastIndex, float TotalLength)
{
	FRibbonUVScaleAndOffsets Output;

	const uint NumSegments = (LastIndex - FirstIndex) + 1;
	
	float NormalizedLeadingSegmentOffset;
	if (UVSettings.LeadingEdgeMode == UV_EDGE_MODE_SMOOTH_TRANSITION)
	{
		const float FirstAge = GetFloat(NormalizedAgeDataOffset, SortedIndices[FirstIndex]);
		const float SecondAge = GetFloat(NormalizedAgeDataOffset, SortedIndices[FirstIndex + 1]);

		const float StartTimeStep = SecondAge - FirstAge;
		const float StartTimeOffset = FirstAge < StartTimeStep ? StartTimeStep - FirstAge : 0;

		NormalizedLeadingSegmentOffset = StartTimeStep > 0 ? StartTimeOffset / StartTimeStep : 0.0f;
	}
	else
	{
		NormalizedLeadingSegmentOffset = 0;
	}


	float NormalizedTrailingSegmentOffset;
	if (UVSettings.TrailingEdgeMode == UV_EDGE_MODE_SMOOTH_TRANSITION)
	{
		const float SecondToLastAge = GetFloat(NormalizedAgeDataOffset, SortedIndices[LastIndex - 1]);
		const float LastAge = GetFloat(NormalizedAgeDataOffset, SortedIndices[LastIndex]);

		const float EndTimeStep = LastAge - SecondToLastAge;
		const float EndTimeOffset = 1 - LastAge < EndTimeStep ? EndTimeStep - (1 - LastAge) : 0;

		NormalizedTrailingSegmentOffset = EndTimeStep > 0 ? EndTimeOffset / EndTimeStep : 0.0f;
	}
	else
	{
		NormalizedTrailingSegmentOffset = 0;
	}

	float CalculatedUOffset;
	float CalculatedUScale;
	if (UVSettings.DistributionMode == UV_DISTRIBUTION_MODE_SCALED_UNIFORMLY)
	{
		const float AvailableSegments = NumSegments - (NormalizedLeadingSegmentOffset + NormalizedTrailingSegmentOffset);
		CalculatedUScale = NumSegments / AvailableSegments;
		CalculatedUOffset = -((NormalizedLeadingSegmentOffset / NumSegments) * CalculatedUScale);
		Output.UDistributionScalar = 1.0f / NumSegments;
	}
	else if (UVSettings.DistributionMode == UV_DISTRIBUTION_MODE_SCALED_USING_RIBBON_SEGMENT_LENGTH)
	{
		const float SecondDistance = GetSegmentDistance(FirstIndex + 1);
		const float LeadingDistanceOffset = SecondDistance * NormalizedLeadingSegmentOffset;

		const float SecondToLastDistance = GetSegmentDistance(LastIndex - 1);
		const float LastDistance = GetSegmentDistance(LastIndex);
		const float TrailingDistanceOffset = (LastDistance - SecondToLastDistance) * NormalizedTrailingSegmentOffset;

		const float AvailableLength = TotalLength - (LeadingDistanceOffset + TrailingDistanceOffset);

		CalculatedUScale = TotalLength / AvailableLength;
		CalculatedUOffset = -((LeadingDistanceOffset / TotalLength) * CalculatedUScale);
		Output.UDistributionScalar = 1.0f / TotalLength;
	}
	else if (UVSettings.DistributionMode == UV_DISTRIBUTION_MODE_TILED_OVER_RIBBON_LENGTH)
	{
		const float SecondDistance = GetSegmentDistance(FirstIndex + 1);
		const float LeadingDistanceOffset = SecondDistance * NormalizedLeadingSegmentOffset;

		CalculatedUScale = TotalLength / UVSettings.TilingLength;
		CalculatedUOffset = -(LeadingDistanceOffset / UVSettings.TilingLength);
		Output.UDistributionScalar = 1.0f / TotalLength;
	}
	else if (UVSettings.DistributionMode == UV_DISTRIBUTION_MODE_TILED_FROM_START_OVER_RIBBON_LENGTH)
	{
		CalculatedUScale = TotalLength / UVSettings.TilingLength;
		CalculatedUOffset = 0;
		Output.UDistributionScalar = 1.0f / TotalLength;
	}
	else
	{
		CalculatedUScale = 1;
		CalculatedUOffset = 0;
		Output.UDistributionScalar = 1.0f;
	}

	Output.UScale = CalculatedUScale * UVSettings.Scale.x;
	Output.UOffset = (CalculatedUOffset * UVSettings.Scale.x) + UVSettings.Offset.x;

	return Output;
}

// We only need a single thread if we're not accounting for ribbon ids
#if HAS_RIBBON_ID
[numthreads(THREADGROUP_SIZE, 1, 1)]
#else
[numthreads(1,1,1)]
#endif
void GenerateRibbonUVParams(uint3 DispatchThreadId : SV_DispatchThreadID)
{
#if HAS_RIBBON_ID
	const int RibbonIndex = DispatchThreadId.x;
	const int TotalNumRibbons = CommandBuffer[CommandBufferOffset * GetVertexGenOutputDataStride() + VERTEX_GEN_OUTPUT_DATA_TOTAL_NUM_RIBBONS_OFFSET];
	
	if (RibbonIndex < TotalNumRibbons)
	{
		const int PackedPerRibbonDataOffset = RibbonIndex * PACKED_PER_RIBBON_DATA_STRIDE;
#else
		const int PackedPerRibbonDataOffset = 0;
#endif
		
		const int FirstParticleID = PackedPerRibbonData[PackedPerRibbonDataOffset + PACKED_PER_RIBBON_STARTPARTICLEOFFSET];
		const int LastParticleID = PackedPerRibbonData[PackedPerRibbonDataOffset + PACKED_PER_RIBBON_ENDPARTICLEOFFSET];

		// Grab the total length from the last particle
		const float TotalLength = GetSegmentDistance(LastParticleID);
		const uint NumSegments = LastParticleID - FirstParticleID;
	
		const FNiagaraRibbonUVSettings UV0Settings = GetUV0Params();
		const FNiagaraRibbonUVSettings UV1Settings = GetUV1Params();

		const int PackedPerRibbonDataOffsetUVs = PackedPerRibbonDataOffset + PACKED_PER_RIBBON_UVDATAOFFSET;
		if ( NumSegments > 0 )
		{
			// UV0
			BRANCH
			if (UV0Settings.bEnablePerParticleUOverride && U0OverrideDataOffset != INDEX_NONE)
			{
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 0] = asuint(1.0f);
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 1] = asuint(0.0f);
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 2] = asuint(1.0f);
			}
			else
			{
				const FRibbonUVScaleAndOffsets UVData = CalculateUVScaleAndOffsets(UV0Settings, FirstParticleID, LastParticleID, TotalLength);
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 0] = asuint(UVData.UScale);
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 1] = asuint(UVData.UOffset);
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 2] = asuint(UVData.UDistributionScalar);
			}
		
			// UV1
			BRANCH
			if (UV1Settings.bEnablePerParticleUOverride && U1OverrideDataOffset != INDEX_NONE)
			{
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 3] = asuint(1.0f);
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 4] = asuint(0.0f);
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 5] = asuint(1.0f);
			}
			else
			{
				const FRibbonUVScaleAndOffsets UVData = CalculateUVScaleAndOffsets(UV1Settings, FirstParticleID, LastParticleID, TotalLength);
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 3] = asuint(UVData.UScale);
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 4] = asuint(UVData.UOffset);
				PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 5] = asuint(UVData.UDistributionScalar);
			}

			// Update the tangents for the first and last vertex, apply a reflect vector logic so that the initial and final curvature is continuous.
			const float3 FirstTangent = LoadTangent(FirstParticleID);
			const float3 NextToFirstTangent = LoadTangent(FirstParticleID + 1);
			StoreTangent(FirstParticleID, (2.0f * dot(FirstTangent, NextToFirstTangent)) * FirstTangent - NextToFirstTangent);
			
			const float3 LastTangent = LoadTangent(LastParticleID);
			const float3 PrevToLastTangent = LoadTangent(LastParticleID - 1);
			StoreTangent(LastParticleID, (2.0f * dot(LastTangent, PrevToLastTangent)) * LastTangent - PrevToLastTangent);
		}
		else
		{
			PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 0] = asuint(1.0f);
			PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 1] = asuint(0.0f);
			PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 2] = asuint(1.0f);
			PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 3] = asuint(1.0f);
			PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 4] = asuint(0.0f);
			PackedPerRibbonData[PackedPerRibbonDataOffsetUVs + 5] = asuint(1.0f);
		}
#if HAS_RIBBON_ID
	}
#endif
}