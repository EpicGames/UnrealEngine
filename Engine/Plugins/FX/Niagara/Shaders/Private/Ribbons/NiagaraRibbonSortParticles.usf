// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "NiagaraRibbonCommon.ush"

Buffer<uint> SortedIndices;
RWBuffer<uint> DestinationSortedIndices;
Buffer<uint> EmitterCounts;
uint CountOffset;

int MergeSortSourceBlockSize;
int MergeSortDestinationBlockSize;

#if RIBBONLINK_IS_FLOAT
	#define FRibbonLink	float
	bool CompareRibbonLink(float Lhs, float Rhs) { return Lhs > Rhs; }
#else
	#define FRibbonLink	int
	bool CompareRibbonLink(int Lhs, int Rhs) { return Lhs < Rhs; }
#endif

groupshared uint			SortParticleIDs[THREADGROUP_SIZE + 1];
groupshared FRibbonLink		SortLinkIDs[THREADGROUP_SIZE + 1];
#if HAS_RIBBON_ID
groupshared FRibbonID		SortRibbonIDs[THREADGROUP_SIZE + 1];
#endif

[numthreads(THREADGROUP_SIZE, 1, 1)]
void InitialSortList(uint3 DispatchThreadId : SV_DispatchThreadID, uint InGroupIndex : SV_GroupIndex, uint3 InGroupID : SV_GroupID)
{
	const uint TotalNumParticles = GetTotalNumParticles();
	const int GroupStartOffset = InGroupID.x * THREADGROUP_SIZE;
	const int ParticlesRemaining = TotalNumParticles - GroupStartOffset;
	const int MaxParticlesInSet = THREADGROUP_SIZE;
	const int TotalNumParticlesInSet = min(MaxParticlesInSet, ParticlesRemaining);
	
	const int GlobalIndexOffset = DispatchThreadId.x;
	const int LocalIndexOffset = InGroupIndex;

	// Make local sort information
	{
		const uint LocalParticleID			= min(GlobalIndexOffset, int(TotalNumParticles - 1));
		SortParticleIDs[LocalIndexOffset]	= LocalParticleID;
		SortLinkIDs[LocalIndexOffset]		= GetRibbonLinkID(LocalParticleID);
	#if HAS_RIBBON_ID
		SortRibbonIDs[LocalIndexOffset]		= GetRibbonID(LocalParticleID);
	#endif
	}
	GroupMemoryBarrierWithGroupSync();
	
	for (int SortGroup=0; SortGroup < THREADGROUP_SIZE; SortGroup++)
	{
		bool bShouldSwap = false;
		if ( ((LocalIndexOffset & 1) == (SortGroup & 1)) && (LocalIndexOffset + 1 < TotalNumParticlesInSet) )
		{
		#if HAS_RIBBON_ID			
			const int RibbonIDComparisonResult = CompareRibbonIDs(SortRibbonIDs[LocalIndexOffset + 0], SortRibbonIDs[LocalIndexOffset + 1]);
			bShouldSwap = RibbonIDComparisonResult > 0 || (RibbonIDComparisonResult == 0 && CompareRibbonLink(SortLinkIDs[LocalIndexOffset + 0], SortLinkIDs[LocalIndexOffset + 1]));
		#else
			bShouldSwap = CompareRibbonLink(SortLinkIDs[LocalIndexOffset + 0], SortLinkIDs[LocalIndexOffset + 1]);
		#endif
		}

		GroupMemoryBarrierWithGroupSync();
		if (bShouldSwap)
		{
			uint TempParticleID = SortParticleIDs[LocalIndexOffset + 0];
			SortParticleIDs[LocalIndexOffset + 0] = SortParticleIDs[LocalIndexOffset + 1];
			SortParticleIDs[LocalIndexOffset + 1] = TempParticleID;

			FRibbonLink TempLinkID = SortLinkIDs[LocalIndexOffset + 0];
			SortLinkIDs[LocalIndexOffset + 0] = SortLinkIDs[LocalIndexOffset + 1];
			SortLinkIDs[LocalIndexOffset + 1] = TempLinkID;

		#if HAS_RIBBON_ID
			FRibbonID TempRibbonID = SortRibbonIDs[LocalIndexOffset + 0];
			SortRibbonIDs[LocalIndexOffset + 0] = SortRibbonIDs[LocalIndexOffset + 1];
			SortRibbonIDs[LocalIndexOffset + 1] = TempRibbonID;
		#endif
		}
		GroupMemoryBarrierWithGroupSync();
	}

	if (LocalIndexOffset < TotalNumParticlesInSet)
	{
		DestinationSortedIndices[GlobalIndexOffset] = SortParticleIDs[LocalIndexOffset];
	}
}


int MergeSortCompare(FRibbonLink SearchLinkOrder, FRibbonLink TestLinkOrder)
{
	if (CompareRibbonLink(SearchLinkOrder, TestLinkOrder))
	{
		return 1;
	}

	if (SearchLinkOrder == TestLinkOrder)
	{
		return 0;
	}
	
	return -1;
}

#if HAS_RIBBON_ID
int MergeSortCompare(FRibbonID Left, FRibbonLink LeftLinkOrder, FRibbonID Right, FRibbonLink RightLinkOrder)
{
	const int RibbonIDComparisonResult = CompareRibbonIDs(Left, Right);

	if (RibbonIDComparisonResult > 0)
	{
		return 1;
	}

	if (RibbonIDComparisonResult == 0)
	{
		return MergeSortCompare(LeftLinkOrder, RightLinkOrder);
	}

	return -1;
}
#endif

// This is a scatter based mergesort kernel
[numthreads(THREADGROUP_SIZE, 1, 1)]
void MergeSort(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint TotalNumParticles = GetTotalNumParticles();
	const uint Index = DispatchThreadId.x;
	
	if (Index < TotalNumParticles)
	{		
		const uint DestinationBlockStart = (Index / MergeSortDestinationBlockSize) * MergeSortDestinationBlockSize;
		const uint SourceBlockStart = (Index / MergeSortSourceBlockSize) * MergeSortSourceBlockSize;
		const bool bIsInPrimaryBlock = Index < (DestinationBlockStart + MergeSortSourceBlockSize);
		const uint OtherSourceBlockStart = DestinationBlockStart + (bIsInPrimaryBlock? MergeSortSourceBlockSize : 0);
		
		const uint IndexWithinSourceBlock = Index - SourceBlockStart;
		
		const uint ParticleID = SortedIndices[Index];
		const FRibbonLink LinkID = GetRibbonLinkID(ParticleID);
#if HAS_RIBBON_ID
		const FRibbonID RibbonID = GetRibbonID(ParticleID);
#endif
			
		// we find the index of the last particle before this in the other half,
		// add that index to ours and we have the number of particles preceeding
		// so we can scatter into the destination
		// Handle equal values by placing all of those in the primary block before those in the secondary block
		
		int SearchStart = 0;
		int SearchStop = min(int(MergeSortSourceBlockSize), max(int(TotalNumParticles) - int(OtherSourceBlockStart), 0));
		
		while (SearchStart != SearchStop)
		{
			const int Mid = (SearchStart + SearchStop) / 2;
			const int SearchIndex = OtherSourceBlockStart + Mid;
			const uint OtherParticleID = SortedIndices[SearchIndex];
			const FRibbonLink OtherLinkID = GetRibbonLinkID(OtherParticleID);
#if HAS_RIBBON_ID
			const FRibbonID OtherRibbonID = GetRibbonID(OtherParticleID);
			const int Result = MergeSortCompare(RibbonID, LinkID, OtherRibbonID, OtherLinkID);
#else
			const int Result = MergeSortCompare(LinkID, OtherLinkID);			
#endif
			
			if (Result > 0 || (!bIsInPrimaryBlock && Result == 0))
			{
				SearchStart = min(Mid + 1, SearchStop);							
			}
			else
			{
				SearchStop = max(Mid, SearchStart);				
			}
		}
		
		const int FinalIndex = DestinationBlockStart + IndexWithinSourceBlock + SearchStop;
		DestinationSortedIndices[FinalIndex] = ParticleID;
	}
}
