// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "NiagaraRibbonCommon.ush"


Buffer<uint> SortedIndices;
// Index of the ribbon for this particle
// created through prefix sum across all the starting points of the ribbons
#if HAS_RIBBON_ID
Buffer<uint> MultiRibbonIndices;
#endif

// Index of the final segment this particle starts
Buffer<uint> Segments;

// Tessellation reduction stats if we're computing them
Buffer<float> TessellationStats;

StructuredBuffer<FRibbonAccumulationValues> AccumulationBuffer;

// Buffer containing basic stats about each ribbon
// including the first/last indices, as well as UV offsets/scales/distributions for both channels
RWBuffer<uint> PackedPerRibbonData;

// Buffer to output the results of vertex gen to
// This is shared across all running ribbon renderers as it's readback to the CPU in one shot
RWBuffer<uint> OutputCommandBuffer;

// Index of the entry we should write too
int OutputCommandBufferIndex;

// ThreadBlock size of the finalization shader that comes next.
// We compute the draw indirect args while we're here
int FinalizationThreadBlockSize;

// We only need a single thread if we're not accounting for ribbon ids
#if HAS_RIBBON_ID
[numthreads(THREADGROUP_SIZE, 1, 1)]
#else
[numthreads(1,1,1)]
#endif
void OutputRibbonStats(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint TotalNumParticles = GetTotalNumParticles();
	const uint OutputCommandBufferOffset = OutputCommandBufferIndex * GetVertexGenOutputDataStride();

	// Early out if we have no particles
	if ( TotalNumParticles == 0 )
	{
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_FINALIZATION_INDIRECT_ARGS_OFFSET + 0] = 0;
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_FINALIZATION_INDIRECT_ARGS_OFFSET + 1] = 0;
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_FINALIZATION_INDIRECT_ARGS_OFFSET + 2] = 0;
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TOTAL_NUM_SEGMENTS_OFFSET] = 0;
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TOTAL_NUM_RIBBONS_OFFSET] = 0;
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TOTAL_NUM_SEGMENTS_OFFSET] = 0;
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TOTAL_NUM_RIBBONS_OFFSET] = 0;
		if (RibbonWantsAutomaticTessellation())
		{
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_ANGLE] = asuint(0.0f);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURVATURE] = asuint(0.0f);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TWIST_ANGLE] = asuint(0.0f);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TWIST_CURVATURE] = asuint(0.0f);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TOTAL_SEGMENT_LENGTH] = asuint(0.0f);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_TOTAL_SEGMENT_LENGTH] = asuint(0.0f);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_SEGMENT_LENGTH] = asuint(0.0f);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_SEGMENT_ANGLE] = asuint(0.0f);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_TWIST_ANGLE] = asuint(0.0f);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_WIDTH] = asuint(0.0f);
		}
		return;
	}

	const uint RawParticleID = DispatchThreadId.x;
	const uint LastParticleIndex = TotalNumParticles - 1;
	
	// First we aggregate all the per ribbon stats by finding the first/last particle of each ribbon
	// We skip this step if we're not supporting multi ribbon and we'll just fill in the s
#if HAS_RIBBON_ID
	if (RawParticleID < TotalNumParticles)
	{
		// First we Gen the ribbon lookup table if we have multiribbon enabled
		const uint CurrRibbonIndex = MultiRibbonIndices[RawParticleID];
		const uint PrevRibbonIndex = RawParticleID > 0 ? MultiRibbonIndices[RawParticleID - 1] : INDEX_NONE;
		const uint NextRibbonIndex = RawParticleID < LastParticleIndex ? MultiRibbonIndices[RawParticleID + 1] : INDEX_NONE;
		
		// Track the starting index of each ribbon segment
		BRANCH
		if (CurrRibbonIndex != PrevRibbonIndex)
		{			
			PackedPerRibbonData[CurrRibbonIndex * PACKED_PER_RIBBON_DATA_STRIDE + PACKED_PER_RIBBON_STARTPARTICLEOFFSET] = uint(RawParticleID);		
		}

		// Track the ending index of each ribbon segment
		BRANCH
		if (CurrRibbonIndex != NextRibbonIndex)
		{		
			PackedPerRibbonData[CurrRibbonIndex * PACKED_PER_RIBBON_DATA_STRIDE + PACKED_PER_RIBBON_ENDPARTICLEOFFSET] = uint(RawParticleID);
		}
	}
#else
	// Write the first ribbon data to the entirety of the sim since we're not supporting multi ribbon
	PackedPerRibbonData[PACKED_PER_RIBBON_STARTPARTICLEOFFSET] = uint(0);
	PackedPerRibbonData[PACKED_PER_RIBBON_ENDPARTICLEOFFSET] = LastParticleIndex;
#endif

	// Very first compute instance, we grab all the stats into the output buffer
	// This is so if we don't need multiribbon, then we'll only run one compute instance
#if HAS_RIBBON_ID	
	BRANCH
	if (RawParticleID == 0)
#endif
	{			
#if !HAS_RIBBON_ID
		const int TotalNumRibbons = 1;
#else
		const int TotalNumRibbons = AccumulationBuffer[LastParticleIndex].MultiRibbonCount;
#endif		

		// Calculate the indirect args for the finalization shader
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_FINALIZATION_INDIRECT_ARGS_OFFSET + 0] = DivideAndRoundUp(TotalNumRibbons, FinalizationThreadBlockSize);
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_FINALIZATION_INDIRECT_ARGS_OFFSET + 1] = 1;
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_FINALIZATION_INDIRECT_ARGS_OFFSET + 2] = 1;
		
		// Total num segments
		const int TotalSegments = AccumulationBuffer[LastParticleIndex].SegmentCount;
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TOTAL_NUM_SEGMENTS_OFFSET] = TotalSegments;

		// Total num ribbons
		OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TOTAL_NUM_RIBBONS_OFFSET] = TotalNumRibbons;

		if (RibbonWantsAutomaticTessellation())
		{
			const uint TessellationDataStride = GetTessellationDataStride();
			const uint TessellationBaseOffset = LastParticleIndex * TessellationDataStride;
			float TotalSegmentLength = TessellationStats[TessellationBaseOffset + 0];
			float AverageSegmentLength = TessellationStats[TessellationBaseOffset + 1];
			float AverageSegmentAngle = TessellationStats[TessellationBaseOffset + 2];
			float AverageTwistAngle = 0;
			float AverageWidth = 0;
			if (RibbonHasTwist())
			{
				AverageTwistAngle = TessellationStats[TessellationBaseOffset + 3];
				AverageWidth = TessellationStats[TessellationBaseOffset + 4];
			}
		
			// Generate smoothed tessellation information
			float TessellationAngle = 0.0f;
			float TessellationCurvature = 0.0f;
			float TessellationTwistAngle = 0.0f;
			float TessellationTwistCurvature = 0.0f;
			float TessellationTotalSegmentLength = 0.0f;
		
			if (TotalSegmentLength > 0.0)
			{
				// Read Tessellation Smmothing Data (accumulated between frames)
				TessellationAngle = asfloat(OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_ANGLE]);
				TessellationCurvature = asfloat(OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURVATURE]);
				TessellationTwistAngle = asfloat(OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TWIST_ANGLE]);
				TessellationTwistCurvature = asfloat(OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TWIST_CURVATURE]);
				TessellationTotalSegmentLength = asfloat(OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TOTAL_SEGMENT_LENGTH]);

				// Blend the result between the last frame tessellation factors and the current frame base on the total length of all segments.
				// This is only used to increase the tessellation value of the current frame data to prevent glitches where tessellation is significantly changin between frames.
				const float OneOverTotalSegmentLength = 1.f / max(1.0f, TotalSegmentLength);
				const float AveragingFactor = TessellationTotalSegmentLength / (TotalSegmentLength + TessellationTotalSegmentLength);
				TessellationTotalSegmentLength = TotalSegmentLength;
			
				AverageSegmentAngle *= OneOverTotalSegmentLength;
				AverageSegmentLength *= OneOverTotalSegmentLength;
				const float AverageSegmentCurvature = AverageSegmentLength / max(UE_SMALL_NUMBER, abs(sin(AverageSegmentAngle)));

				TessellationAngle = lerp(AverageSegmentAngle, max(TessellationAngle, AverageSegmentAngle), AveragingFactor);
				TessellationCurvature = lerp(AverageSegmentCurvature, max(TessellationCurvature, AverageSegmentCurvature), AveragingFactor);
			
				if (RibbonHasTwist())
				{
					AverageTwistAngle *= OneOverTotalSegmentLength;
					AverageWidth *= OneOverTotalSegmentLength;

					TessellationTwistAngle = lerp(AverageTwistAngle, max(TessellationTwistAngle, AverageTwistAngle), AveragingFactor);
					TessellationTwistCurvature = lerp(AverageWidth, max(TessellationTwistCurvature, AverageWidth), AveragingFactor);
				}
			}

			// Output Tessellation Smmothing Data (accumulated between frames)
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_ANGLE] = asuint(TessellationAngle);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURVATURE] = asuint(TessellationCurvature);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TWIST_ANGLE] = asuint(TessellationTwistAngle);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TWIST_CURVATURE] = asuint(TessellationTwistCurvature);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_TOTAL_SEGMENT_LENGTH] = asuint(TessellationTotalSegmentLength);

			// Write out current frame tessellation stats
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_TOTAL_SEGMENT_LENGTH] = asuint(TotalSegmentLength);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_SEGMENT_LENGTH] = asuint(AverageSegmentLength);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_SEGMENT_ANGLE] = asuint(AverageSegmentAngle);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_TWIST_ANGLE] = asuint(AverageTwistAngle);
			OutputCommandBuffer[OutputCommandBufferOffset + VERTEX_GEN_OUTPUT_DATA_TESSELLATION_CURRENT_FRAME_AVERAGE_WIDTH] = asuint(AverageWidth);
		}
	}
}
