// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "NiagaraRibbonCommon.ush"


Buffer<uint> SortedIndices;

// In some cases TangentsAndDistances will be the same buffer as OutputDistances in the case where the number of
// prefix sum passes will be even so it takes starting with the final buffer to end in the final buffer

// Tangent of the particle, and the distance along the ribbon if we're computing it
RWBuffer<float> OutputTangentsAndDistances;

// Index of the ribbon for this particle
// created through prefix sum across all the starting points of the ribbons
#if HAS_RIBBON_ID
RWBuffer<uint> OutputMultiRibbonIndices;
#endif

// Index of the final segment this particle starts
RWBuffer<uint> OutputSegments;

RWStructuredBuffer<FRibbonAccumulationValues> OutputAccumulationBuffer;

float CurveTension;

// Super simple shader just to initialize the ribbon reduction list sizes to zero
[numthreads(THREADGROUP_SIZE, 1, 1)]
void VertexGenInitialize(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint TotalNumParticles = GetTotalNumParticles();
	const uint RawParticleID = DispatchThreadId.x;	

	BRANCH
	if (RawParticleID < TotalNumParticles)
	{
		OutputAccumulationBuffer[RawParticleID] = (FRibbonAccumulationValues)0;

		const int ParticleID = SortedIndices[RawParticleID];
#if HAS_RIBBON_ID
		const FRibbonID RibbonID = GetRibbonID(ParticleID);
#endif

		bool bHasPrevParticle = RawParticleID > 0;
		bool bHasNextParticle = (RawParticleID + 1) < TotalNumParticles;

		int PrevParticleID = INDEX_NONE;
		int NextParticleID = INDEX_NONE;
		
		if (bHasPrevParticle)
		{
			PrevParticleID = SortedIndices[RawParticleID - 1];
#if HAS_RIBBON_ID
			FRibbonID PrevRibbonID = GetRibbonID(PrevParticleID);
			bHasPrevParticle = AreRibbonIDsEqual(RibbonID, PrevRibbonID);
#endif
		}

		if (bHasNextParticle)
		{
			NextParticleID = SortedIndices[RawParticleID + 1];
#if HAS_RIBBON_ID
			FRibbonID NextRibbonID = GetRibbonID(NextParticleID);
			bHasNextParticle = AreRibbonIDsEqual(RibbonID, NextRibbonID);
#endif
		}
		

#if HAS_RIBBON_ID
		// We start a new ribbon index each time it breaks from the previous
		const uint MultiRibbonCountIncr = bHasPrevParticle ? 0 : 1;
		OutputMultiRibbonIndices[RawParticleID] = MultiRibbonCountIncr;
		OutputAccumulationBuffer[RawParticleID].MultiRibbonCount =  MultiRibbonCountIncr;
#endif

		// Track the segment offset
		OutputSegments[RawParticleID] = bHasNextParticle ? 1 : INDEX_NONE;
		OutputAccumulationBuffer[RawParticleID].SegmentCount = bHasNextParticle ? 1 : 0;

		float4 TangentAndDistance = float4(0,0,0,0);

		BRANCH
		if (bHasNextParticle && bHasPrevParticle)
		{
			const float3 ThisPosition = GetVec3(PositionDataOffset, ParticleID);
			const float3 PreviousPosition = GetVec3(PositionDataOffset, PrevParticleID);
			const float3 NextPosition = GetVec3(PositionDataOffset, NextParticleID);

			float3 CurrToNextVec = NextPosition - ThisPosition;
			float3 LastToCurrVec = ThisPosition - PreviousPosition;

			const float CurrToNextSize = length(CurrToNextVec);
			const float LastToCurrSize = length(LastToCurrVec);

			if (CurrToNextSize > 0)
			{
				CurrToNextVec *= 1.f / CurrToNextSize;
			}
			
			if (LastToCurrSize > 0)
			{
				LastToCurrVec *= 1.f / LastToCurrSize;				
			}

			const float3 AverageNormal = normalize(LastToCurrVec + CurrToNextVec);
			const float3 Tangent = (1.0f - CurveTension) * AverageNormal;

			
			if (CurrToNextSize + LastToCurrSize > 0)
			{
				TangentAndDistance = float4(Tangent, LastToCurrSize);
			}
			else
			{
				TangentAndDistance = float4(0,0,0,0);
			}

			if (RibbonWantsAutomaticTessellation())
			{
				// TotalLength
				OutputAccumulationBuffer[RawParticleID].TessTotalLength = CurrToNextSize;
				// AverageSegmentLength
				OutputAccumulationBuffer[RawParticleID].TessAvgSegmentLength = CurrToNextSize * CurrToNextSize;
				// AverageSegmentAngle
				OutputAccumulationBuffer[RawParticleID].TessAvgSegmentAngle = CurrToNextSize * AcosFast(dot(LastToCurrVec, CurrToNextVec));

				if (RibbonHasTwist())
				{
					const float LastTwist = SafeGetFloat(TwistDataOffset, PrevParticleID, 0);
					const float ThisTwist = SafeGetFloat(TwistDataOffset, ParticleID, 0);

					//const float LastWidth = SafeGetFloat(WidthDataOffset, PrevParticleID, 1);
					const float ThisWidth = SafeGetFloat(WidthDataOffset, ParticleID, 1);
					// AverageTwistAngle
					OutputAccumulationBuffer[RawParticleID].TessTwistAvgAngle = CurrToNextSize * abs(ThisTwist - LastTwist);
					// AverageWidth
					OutputAccumulationBuffer[RawParticleID].TessTwistAvgWidth = CurrToNextSize * ThisWidth;
				}
			}
		}
		else if (bHasNextParticle)
		{
			// Special case to find the first particle of ribbon and set it up correctly

			const float3 ThisPosition = GetVec3(PositionDataOffset, ParticleID);
			const float3 NextPosition = GetVec3(PositionDataOffset, NextParticleID);

			float3 CurrToNextVec = NextPosition - ThisPosition;
			const float CurrToNextSize = length(CurrToNextVec);

			if (CurrToNextSize > 0)
			{
				CurrToNextVec *= 1.f / CurrToNextSize;
			}
			
			TangentAndDistance = float4(CurrToNextVec, 0);
		}
		else if (bHasPrevParticle)
		{
			// Special case to find the first particle of ribbon and set it up correctly

			const float3 ThisPosition = GetVec3(PositionDataOffset, ParticleID);
			const float3 PreviousPosition = GetVec3(PositionDataOffset, PrevParticleID);

			float3 LastToCurrVec = ThisPosition - PreviousPosition;

			const float LastToCurrSize = length(LastToCurrVec);

			if (LastToCurrSize > 0)
			{
				LastToCurrVec *= 1.f / LastToCurrSize;				
			}
			
			TangentAndDistance = float4(LastToCurrVec, LastToCurrSize);			
		}

		
		OutputTangentsAndDistances[RawParticleID * 4 + 0] = TangentAndDistance.x;
		OutputTangentsAndDistances[RawParticleID * 4 + 1] = TangentAndDistance.y;
		OutputTangentsAndDistances[RawParticleID * 4 + 2] = TangentAndDistance.z;
			
		// This is the distance up to this point, we'll prefix sum this in the next stage
		OutputAccumulationBuffer[RawParticleID].RibbonDistance = TangentAndDistance.w;
	}
}






