// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "NiagaraRibbonCommon.ush"

Buffer<uint> SortedIndices;

// Tangent of the particle, and the distance along the ribbon if we're computing it
Buffer<float> InputTangentsAndDistances;
RWBuffer<float> OutputTangentsAndDistances;

// Index of the ribbon for this particle
// created through prefix sum across all the starting points of the ribbons

#if HAS_RIBBON_ID
Buffer<uint> InputMultiRibbonIndices;
RWBuffer<uint> OutputMultiRibbonIndices;
#endif

// Index of the final segment this particle starts
Buffer<uint> InputSegments;
RWBuffer<uint> OutputSegments;

// Tessellation reduction stats if we're computing them
Buffer<float> InputTessellationStats;
RWBuffer<float> OutputTessellationStats;

int PrefixScanStride;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void VertexGenPrefixSumPropagation(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint TotalNumParticles = GetTotalNumParticles();
	const int RawParticleID = DispatchThreadId.x;
	if (RawParticleID < TotalNumParticles)
	{
		// We direct copy the tangent part of tangentanddistances
		OutputTangentsAndDistances[RawParticleID * 4 + 0] = InputTangentsAndDistances[RawParticleID * 4 + 0];
		OutputTangentsAndDistances[RawParticleID * 4 + 1] = InputTangentsAndDistances[RawParticleID * 4 + 1];
		OutputTangentsAndDistances[RawParticleID * 4 + 2] = InputTangentsAndDistances[RawParticleID * 4 + 2];
		
		// Are we copying or adding
		BRANCH
		if (RawParticleID >= PrefixScanStride)
		{
			const int RawSourceParticleID = RawParticleID - PrefixScanStride;
			
			// Copy segments
			OutputSegments[RawParticleID] = InputSegments[RawParticleID] + InputSegments[RawSourceParticleID];
			
#if HAS_RIBBON_ID
			// Copy multi Ribbon
			OutputMultiRibbonIndices[RawParticleID] = InputMultiRibbonIndices[RawParticleID] + InputMultiRibbonIndices[RawSourceParticleID];

			const int SourceParticleID = SortedIndices[RawSourceParticleID];
			const int ParticleID = SortedIndices[RawParticleID];
			
			const FRibbonID SourceRibbonID = GetRibbonID(SourceParticleID);
			const FRibbonID RibbonID = GetRibbonID(ParticleID);
					
			
			if (AreRibbonIDsEqual(SourceRibbonID, RibbonID))
#endif
			{
				// Copy tangents
				OutputTangentsAndDistances[RawParticleID * 4 + 3] = InputTangentsAndDistances[RawParticleID * 4 + 3] + InputTangentsAndDistances[RawSourceParticleID * 4 + 3];
			}
#if HAS_RIBBON_ID
			else
			{			
				// Copy tangents	
				OutputTangentsAndDistances[RawParticleID * 4 + 3] = InputTangentsAndDistances[RawParticleID * 4 + 3];
			}
#endif

			if (RibbonWantsAutomaticTessellation())
			{
				const uint TessellationDataStride = GetTessellationDataStride();
				const uint TessellationBaseOffset = RawParticleID * TessellationDataStride;
				const uint TessellationSourceBaseOffset = RawSourceParticleID * TessellationDataStride;
				// TotalLength
				OutputTessellationStats[TessellationBaseOffset + 0] = InputTessellationStats[TessellationBaseOffset + 0] + InputTessellationStats[TessellationSourceBaseOffset + 0];
				// AverageSegmentLength
				OutputTessellationStats[TessellationBaseOffset + 1] = InputTessellationStats[TessellationBaseOffset + 1] + InputTessellationStats[TessellationSourceBaseOffset + 1];
				// AverageSegmentAngle
				OutputTessellationStats[TessellationBaseOffset + 2] = InputTessellationStats[TessellationBaseOffset + 2] + InputTessellationStats[TessellationSourceBaseOffset + 2];

				if (RibbonHasTwist())
				{
					// AverageTwistAngle
					OutputTessellationStats[TessellationBaseOffset + 3] = InputTessellationStats[TessellationBaseOffset + 3] + InputTessellationStats[TessellationSourceBaseOffset + 3];
					// AverageWidth
					OutputTessellationStats[TessellationBaseOffset + 4] = InputTessellationStats[TessellationBaseOffset + 4] + InputTessellationStats[TessellationSourceBaseOffset + 4];
				}
			}
		}
		else
		{			
			// Copy segments
			OutputSegments[RawParticleID] = InputSegments[RawParticleID];
			
#if HAS_RIBBON_ID
			// Copy multi Ribbon
			OutputMultiRibbonIndices[RawParticleID] = InputMultiRibbonIndices[RawParticleID];
#endif
			
			// Copy tangents
			OutputTangentsAndDistances[RawParticleID * 4 + 3] = InputTangentsAndDistances[RawParticleID * 4 + 3];


			if (RibbonWantsAutomaticTessellation())
			{
				const uint TessellationDataStride = GetTessellationDataStride();
				const uint TessellationBaseOffset = RawParticleID * TessellationDataStride;

				// TotalLength
				OutputTessellationStats[TessellationBaseOffset + 0] = InputTessellationStats[TessellationBaseOffset + 0];
				// AverageSegmentLength
				OutputTessellationStats[TessellationBaseOffset + 1] = InputTessellationStats[TessellationBaseOffset + 1];
				// AverageSegmentAngle
				OutputTessellationStats[TessellationBaseOffset + 2] = InputTessellationStats[TessellationBaseOffset + 2];
			
				if (RibbonHasTwist())
				{
					// AverageTwistAngle
					OutputTessellationStats[TessellationBaseOffset + 3] = InputTessellationStats[TessellationBaseOffset + 3];
					// AverageWidth
					OutputTessellationStats[TessellationBaseOffset + 4] = InputTessellationStats[TessellationBaseOffset + 4];
				}
			}
		}
	}
}
