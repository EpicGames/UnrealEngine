// Copyright Epic Games, Inc. All Rights Reserved.

void CalculateAccurateVelocity_Simulate(inout FStatelessParticle Particle)
{  
	Particle.Velocity			= (Particle.Position - Particle.PreviousPosition) * Particle.InvDeltaTime;
	Particle.PreviousVelocity	= Particle.Velocity;
}
	