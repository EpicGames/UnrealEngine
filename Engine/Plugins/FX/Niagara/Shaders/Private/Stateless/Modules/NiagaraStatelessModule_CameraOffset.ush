// Copyright Epic Games, Inc. All Rights Reserved.

FNiagaraStatelessBuiltDistributionType CameraOffset_Distribution;
float CameraOffset_Scale;

void CameraOffset_Simulate(inout FStatelessParticle Particle)
{
	if (FNiagaraStatelessBuiltDistribution::IsValid(CameraOffset_Distribution))
	{
		const float2 CameraOffsets = SampleDistributionValuesFloat(CameraOffset_Distribution, 0, Particle.NormalizedAge, Particle.PreviousNormalizedAge);
		Particle.CameraOffset			= CameraOffsets[0] * CameraOffset_Scale;
		Particle.PreviousCameraOffset	= CameraOffsets[1] * CameraOffset_Scale;
	}
}
