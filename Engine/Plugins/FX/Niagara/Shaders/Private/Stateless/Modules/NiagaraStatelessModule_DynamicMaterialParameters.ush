// Copyright Epic Games, Inc. All Rights Reserved.

uint	DynamicMaterialParameters_ChannelMask;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter0X;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter0Y;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter0Z;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter0W;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter1X;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter1Y;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter1Z;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter1W;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter2X;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter2Y;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter2Z;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter2W;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter3X;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter3Y;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter3Z;
FNiagaraStatelessBuiltDistributionType	DynamicMaterialParameters_Parameter3W;
	
void DynamicMaterialParameters_Simulate(inout FStatelessParticle Particle)
{
	Particle.DynamicMaterialParameter = 0.0f;
	Particle.DynamicMaterialParameter1 = 0.0f;
	Particle.DynamicMaterialParameter2 = 0.0f;
	Particle.DynamicMaterialParameter3 = 0.0f;

	if ((DynamicMaterialParameters_ChannelMask & 0x000f) != 0)
	{
		Particle.DynamicMaterialParameter.x = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter0X, 0, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter.y = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter0Y, 1, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter.z = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter0Z, 2, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter.w = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter0W, 3, Particle.NormalizedAge);
	}

	if ((DynamicMaterialParameters_ChannelMask & 0x00f0) != 0)
	{
		Particle.DynamicMaterialParameter1.x = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter1X, 4, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter1.y = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter1Y, 5, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter1.z = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter1Z, 6, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter1.w = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter1W, 7, Particle.NormalizedAge);
	}

	if ((DynamicMaterialParameters_ChannelMask & 0x0f00) != 0)
	{
		Particle.DynamicMaterialParameter2.x = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter2X,  8, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter2.y = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter2Y,  9, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter2.z = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter2Z, 10, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter2.w = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter2W, 11, Particle.NormalizedAge);
	}

	if ((DynamicMaterialParameters_ChannelMask & 0xf000) != 0)
	{
		Particle.DynamicMaterialParameter3.x = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter3X, 12, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter3.y = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter3Y, 13, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter3.z = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter3Z, 14, Particle.NormalizedAge);
		Particle.DynamicMaterialParameter3.w = SampleDistributionValueFloat(DynamicMaterialParameters_Parameter3W, 15, Particle.NormalizedAge);
	}
}
