// Copyright Epic Games, Inc. All Rights Reserved.

float3	InitialMeshOrientation_RotationScale;
float3	InitialMeshOrientation_RotationBias;

void InitialMeshOrientation_Simulate(inout FStatelessParticle Particle)
{
	const float3 InitialRotation		= RandomScaleBiasFloat(0, InitialMeshOrientation_RotationScale, InitialMeshOrientation_RotationBias);
	Particle.MeshOrientation			= RotatorToQuat(InitialRotation);
	Particle.PreviousMeshOrientation	= Particle.MeshOrientation;
}
