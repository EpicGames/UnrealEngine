// Copyright Epic Games, Inc. All Rights Reserved.

static const uint EInitializeParticleModuleFlag_UniformSpriteSize	= 1 << 0;
static const uint EInitializeParticleModuleFlag_UniformMeshScale	= 1 << 1;

uint	InitializeParticle_ModuleFlags;
FNiagaraStatelessBuiltDistributionType	InitializeParticle_InitialPosition;
FNiagaraStatelessBuiltDistributionType	InitializeParticle_InitialColor;
float2	InitializeParticle_SpriteSizeScale;
float2	InitializeParticle_SpriteSizeBias;
float	InitializeParticle_SpriteRotationScale;
float	InitializeParticle_SpriteRotationBias;

float3	InitializeParticle_MeshScaleScale;
float3	InitializeParticle_MeshScaleBias;

float	InitializeParticle_RibbonWidthScale;
float	InitializeParticle_RibbonWidthBias;
	
void InitializeParticle_Simulate(inout FStatelessParticle Particle)
{  
	const bool bUniformSpriteSize = (InitializeParticle_ModuleFlags & EInitializeParticleModuleFlag_UniformSpriteSize) != 0;
	const bool bUniformMeshScale  = (InitializeParticle_ModuleFlags & EInitializeParticleModuleFlag_UniformMeshScale) != 0;

	Particle.Position		= SampleDistributionValueFloat3(InitializeParticle_InitialPosition, 0, float(Particle.UniqueID));
	Particle.Color			= SampleDistributionValueFloat4(InitializeParticle_InitialColor, 1, float(Particle.UniqueID));
	Particle.RibbonWidth	= RandomScaleBiasFloat(2, InitializeParticle_RibbonWidthScale, InitializeParticle_RibbonWidthBias);
	Particle.SpriteSize		= RandomScaleBiasFloat(3, InitializeParticle_SpriteSizeScale, InitializeParticle_SpriteSizeBias, bUniformSpriteSize);
	Particle.SpriteRotation	= RandomScaleBiasFloat(4, InitializeParticle_SpriteRotationScale, InitializeParticle_SpriteRotationBias);
	Particle.Scale			= RandomScaleBiasFloat(5, InitializeParticle_MeshScaleScale, InitializeParticle_MeshScaleBias, bUniformMeshScale);

	Particle.PreviousPosition		= Particle.Position;
	Particle.PreviousRibbonWidth	= Particle.RibbonWidth;
	Particle.PreviousSpriteSize		= Particle.SpriteSize;
	Particle.PreviousScale			= Particle.Scale;
	Particle.PreviousSpriteRotation	= Particle.SpriteRotation;
}
