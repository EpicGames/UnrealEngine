// Copyright Epic Games, Inc. All Rights Reserved.

int MeshIndex_Index;
int MeshIndex_TableOffset;
int MeshIndex_TableNumElements;	// -1 to avoid added instructions

void MeshIndex_Simulate(inout FStatelessParticle Particle)
{
	const bool bIsParameter = (MeshIndex_Index & 0x80000000) != 0;
	int MeshIndex = MeshIndex_Index & ~0x80000000u;
	
	if (bIsParameter)
	{
		MeshIndex = GetParameterBufferInt(MeshIndex, 0);
	}
	else
	{
		if ( MeshIndex_TableNumElements > 0 )
		{
			const float2 Rand = RandomFloat2(0);
			MeshIndex = round(Rand.x * float(MeshIndex_TableNumElements));
			const float Probability = GetStaticFloat(MeshIndex_TableOffset, (MeshIndex * 2) + 0);
			if (Rand.y > Probability)
			{
				MeshIndex = int(GetStaticFloat(MeshIndex_TableOffset, (MeshIndex * 2) + 1));
			}
		}
	}
	Particle.MeshIndex = MeshIndex;
}

