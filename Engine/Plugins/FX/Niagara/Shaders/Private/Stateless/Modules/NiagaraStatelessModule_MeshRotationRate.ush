// Copyright Epic Games, Inc. All Rights Reserved.

int										MeshRotationRate_ModuleEnabled;
float3									MeshRotationRate_Scale;
float3									MeshRotationRate_Bias;
FNiagaraStatelessBuiltDistributionType	MeshRotationRate_RateScaleParameters;

void MeshRotationRate_Simulate(inout FStatelessParticle Particle)
{
	if ( MeshRotationRate_ModuleEnabled == 0 )
	{
		return;
	}
	const float3 RotationRate = RandomScaleBiasFloat(0, MeshRotationRate_Scale, MeshRotationRate_Bias);
	const float3 MultiplyRate = SampleCurveFloat3(MeshRotationRate_RateScaleParameters, Particle.NormalizedAge);
	const float3 PreviousMultiplyRate = SampleCurveFloat3(MeshRotationRate_RateScaleParameters, Particle.PreviousNormalizedAge);

	Particle.MeshOrientation			= MultiplyQuat(Particle.MeshOrientation,			RotatorToQuat(Particle.Lifetime * RotationRate * MultiplyRate));
	Particle.PreviousMeshOrientation	= MultiplyQuat(Particle.PreviousMeshOrientation,	RotatorToQuat(Particle.Lifetime * RotationRate * PreviousMultiplyRate));
}

