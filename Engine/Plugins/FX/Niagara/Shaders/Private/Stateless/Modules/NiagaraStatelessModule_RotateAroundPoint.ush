// Copyright Epic Games, Inc. All Rights Reserved.

float	RotateAroundPoint_RateScale;
float	RotateAroundPoint_RateBias;
float	RotateAroundPoint_RadiusScale;
float	RotateAroundPoint_RadiusBias;
float	RotateAroundPoint_InitialPhaseScale;
float	RotateAroundPoint_InitialPhaseBias;
float3	RotateAroundPoint_CenterScale;
float3	RotateAroundPoint_CenterBias;
float3	RotateAroundPoint_RotationAxisScale;
float3	RotateAroundPoint_RotationAxisBias;

struct FStatelessModule_RotateAroundPoint
{
	float	Rate;
	float	Radius;
	float	InitialPhase;
	float3	Center;
	float4	RotationQuat;
};

void RotateAroundPoint_IntegratePosition(in FStatelessModule_RotateAroundPoint ModuleData, float Age, inout float3 Position)
{
	const float  Time	= ModuleData.InitialPhase + Age * ModuleData.Rate;
	const float3 Offset = float3(cos(Time), sin(Time), 0.0f) * ModuleData.Radius;

	Position += ModuleData.Center + RotateVectorByQuat(Offset, ModuleData.RotationQuat);
}

void RotateAroundPoint_Simulate(inout FStatelessParticle Particle)
{
	FStatelessModule_RotateAroundPoint ModuleData;
	ModuleData.Rate			= RandomScaleBiasFloat(0, RotateAroundPoint_RateScale, RotateAroundPoint_RateBias);
	ModuleData.Radius		= RandomScaleBiasFloat(1, RotateAroundPoint_RadiusScale, RotateAroundPoint_RadiusBias);
	ModuleData.InitialPhase	= RandomScaleBiasFloat(2, RotateAroundPoint_InitialPhaseScale, RotateAroundPoint_InitialPhaseBias);
	ModuleData.Center		= RandomScaleBiasFloat(3, RotateAroundPoint_CenterScale, RotateAroundPoint_CenterBias);
	ModuleData.RotationQuat	= RotatorToQuat(RandomScaleBiasFloat(4, RotateAroundPoint_RotationAxisScale, RotateAroundPoint_RotationAxisBias));
		
	RotateAroundPoint_IntegratePosition(ModuleData, Particle.NormalizedAge, Particle.Position);
	RotateAroundPoint_IntegratePosition(ModuleData, Particle.PreviousNormalizedAge, Particle.PreviousPosition);
}
