// Copyright Epic Games, Inc. All Rights Reserved.

FNiagaraStatelessBuiltDistributionType	ScaleColor_Distribution;

void ScaleColor_Simulate(inout FStatelessParticle Particle)
{
	if (FNiagaraStatelessBuiltDistribution::IsValid(ScaleColor_Distribution))
	{
		Particle.Color *= SampleDistributionValueFloat4(ScaleColor_Distribution, 0, Particle.NormalizedAge);
	}
}
