// Copyright Epic Games, Inc. All Rights Reserved.

FNiagaraStatelessBuiltDistributionType	ScaleMeshSize_Distribution;
float3	ScaleMeshSize_CurveScale;
int		ScaleMeshSize_CurveScaleOffset;
	
void ScaleMeshSize_Simulate(inout FStatelessParticle Particle)
{
	if (FNiagaraStatelessBuiltDistribution::IsValid(ScaleMeshSize_Distribution))
	{
		const float3 ScaleFactor = GetParameterBufferFloat3(ScaleMeshSize_CurveScaleOffset, 0, ScaleMeshSize_CurveScale);
		const float2x3 Scales = SampleDistributionValuesFloat3(ScaleMeshSize_Distribution, 0, Particle.NormalizedAge, Particle.PreviousNormalizedAge);

		Particle.Scale			*= Scales[0] * ScaleFactor;
		Particle.PreviousScale	*= Scales[1] * ScaleFactor;
	}
}
