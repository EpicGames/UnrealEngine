// Copyright Epic Games, Inc. All Rights Reserved.

float3	ScaleMeshSizeBySpeed_ScaleFactorBias;
float3	ScaleMeshSizeBySpeed_ScaleFactorScale;
FNiagaraStatelessBuiltDistributionType	ScaleMeshSizeBySpeed_ScaleDistribution;
float	ScaleMeshSizeBySpeed_VelocityNorm;

void ScaleMeshSizeBySpeed_Simulate(inout FStatelessParticle Particle)
{
	if (FNiagaraStatelessBuiltDistribution::IsValid(ScaleMeshSizeBySpeed_ScaleDistribution))
	{
		const float3 Velocity	= (Particle.Position - Particle.PreviousPosition) * Particle.InvDeltaTime;
		const float Speed		= length2(Velocity);
		const float NormSpeed	= saturate(Speed * ScaleMeshSizeBySpeed_VelocityNorm);
		const float Interp		= saturate(SampleDistributionValueFloat(ScaleMeshSizeBySpeed_ScaleDistribution, 0, NormSpeed));
		const float3 Scale		= ScaleMeshSizeBySpeed_ScaleFactorBias + (ScaleMeshSizeBySpeed_ScaleFactorScale * Interp);
			
		Particle.Scale			*= Scale;
		Particle.PreviousScale	*= Scale;
	}
}
