// Copyright Epic Games, Inc. All Rights Reserved.

FNiagaraStatelessBuiltDistributionType	ScaleRibbonWidth_Distribution;
float	ScaleRibbonWidth_CurveScale;

void ScaleRibbonWidth_Simulate(inout FStatelessParticle Particle)
{
	if (FNiagaraStatelessBuiltDistribution::IsValid(ScaleRibbonWidth_Distribution))
	{
		const float ScaleFactor = ScaleRibbonWidth_CurveScale;
		const float2 Scales = SampleDistributionValuesFloat(ScaleRibbonWidth_Distribution, 0, Particle.NormalizedAge, Particle.PreviousNormalizedAge);
	
		Particle.RibbonWidth			*= Scales[0] * ScaleFactor;
		Particle.PreviousRibbonWidth	*= Scales[1] * ScaleFactor;
	}
}
