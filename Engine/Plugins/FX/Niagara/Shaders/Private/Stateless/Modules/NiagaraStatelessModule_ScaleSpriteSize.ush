// Copyright Epic Games, Inc. All Rights Reserved.

FNiagaraStatelessBuiltDistributionType	ScaleSpriteSize_Distribution;
float2	ScaleSpriteSize_CurveScale;

void ScaleSpriteSize_Simulate(inout FStatelessParticle Particle)
{
	if (FNiagaraStatelessBuiltDistribution::IsValid(ScaleSpriteSize_Distribution))
	{
		const float2 ScaleFactor = ScaleSpriteSize_CurveScale;
		const float2x2 Scales = SampleDistributionValuesFloat2(ScaleSpriteSize_Distribution, 0, Particle.NormalizedAge, Particle.PreviousNormalizedAge);

		Particle.SpriteSize			*= Scales[0] * ScaleFactor;
		Particle.PreviousSpriteSize	*= Scales[1] * ScaleFactor;
	}
}
