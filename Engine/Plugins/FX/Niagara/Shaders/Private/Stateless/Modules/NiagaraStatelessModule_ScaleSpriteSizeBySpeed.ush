// Copyright Epic Games, Inc. All Rights Reserved.

float2	ScaleSpriteSizeBySpeed_ScaleFactorBias;
float2	ScaleSpriteSizeBySpeed_ScaleFactorScale;
FNiagaraStatelessBuiltDistributionType	ScaleSpriteSizeBySpeed_ScaleDistribution;
float	ScaleSpriteSizeBySpeed_VelocityNorm;

void ScaleSpriteSizeBySpeed_Simulate(inout FStatelessParticle Particle)
{
	if (FNiagaraStatelessBuiltDistribution::IsValid(ScaleSpriteSizeBySpeed_ScaleDistribution))
	{
		const float3 Velocity	= (Particle.Position - Particle.PreviousPosition) * Particle.InvDeltaTime;
		const float Speed		= length2(Velocity);
		const float NormSpeed	= saturate(Speed * ScaleSpriteSizeBySpeed_VelocityNorm);
		const float Interp		= saturate(SampleDistributionValueFloat(ScaleSpriteSizeBySpeed_ScaleDistribution, 0, NormSpeed));
		const float2 Scale		= ScaleSpriteSizeBySpeed_ScaleFactorBias + (ScaleSpriteSizeBySpeed_ScaleFactorScale * Interp);
			
		Particle.SpriteSize			*= Scale;
		Particle.PreviousSpriteSize	*= Scale;
	}
}
