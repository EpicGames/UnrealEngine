// Copyright Epic Games, Inc. All Rights Reserved.

uint4	ShapeLocation_Mode;
float4	ShapeLocation_Parameters0;
float4	ShapeLocation_Parameters1;
float4	ShapeLocation_Parameters2;
FNiagaraStatelessBuiltDistributionType	ShapeLocation_TransformOffset;
float3	ShapeLocation_TransformOffsetScale;

float3 ShapeLocation_GetLocation()
{
	// ENSM_ShapePrimitive::Box | ENSM_ShapePrimitive::Plane
	BRANCH
	if ( ShapeLocation_Mode.x == 0 )
	{
		const float3 BoxSize = ShapeLocation_Parameters0.xyz;
		const float4 Rotation = ShapeLocation_Parameters2;
		const bool bOnSurface = ShapeLocation_Mode.y == 1;

		const float3 P0 = RandomFloat3(0) - 0.5f;
		if (bOnSurface)
		{
			const float3 FaceDirA[] = { float3(0.0f, 1.0f, 0.0f), float3( 0.0f, 1.0f, 0.0f), float3(0.0f, 0.0f, 1.0f), float3(0.0f, 0.0f, 1.0f), float3(1.0f, 0.0f, 0.0f), float3(1.0f, 0.0f, 0.0f) };
			const float3 FaceDirB[] = { float3(0.0f, 0.0f, 1.0f), float3( 0.0f, 0.0f, 1.0f), float3(1.0f, 0.0f, 0.0f), float3(1.0f, 0.0f, 0.0f), float3(0.0f, 1.0f, 0.0f), float3(0.0f, 1.0f, 0.0f) };
			const float3 FaceDirC[] = { float3(1.0f, 0.0f, 0.0f), float3(-1.0f, 0.0f, 0.0f), float3(0.0f, 1.0f, 0.0f), float3(0.0f,-1.0f, 0.0f), float3(0.0f, 0.0f, 1.0f), float3(0.0f, 0.0f,-1.0f) };
			const uint NumFaces = ShapeLocation_Mode.z;

			const float3 Thickness = ShapeLocation_Parameters1.xyz;
			const float3 LocalBoxSize = BoxSize + (Thickness * P0.z);
				
			const uint iAxis = RandomUInt(1) % NumFaces;
			float3 Location;
			Location  = LocalBoxSize * FaceDirA[iAxis] * P0.x;
			Location += LocalBoxSize * FaceDirB[iAxis] * P0.y;
			Location += LocalBoxSize * FaceDirC[iAxis] * 0.5f;
				
			return RotateVectorByQuat(Location, Rotation);
		}
		return RotateVectorByQuat(P0 * BoxSize, Rotation);
	}

	// ENSM_ShapePrimitive::Cylinder:
	BRANCH
	if ( ShapeLocation_Mode.x == 1 )
	{
		const float HeightScale	= ShapeLocation_Parameters0.w;
		const float HeightBias	= ShapeLocation_Parameters1.w;
		const float Radius		= ShapeLocation_Parameters2.w;

		const float3 AxisX		= ShapeLocation_Parameters0.xyz;
		const float3 AxisY		= ShapeLocation_Parameters1.xyz;
		const float3 AxisZ		= ShapeLocation_Parameters2.xyz;

		const bool bOnSurface	= (ShapeLocation_Mode.y & 0x1) != 0;
		const float4 Random = RandomFloat4(0);

		float XYOffset = Radius * Random.z;
		float ZOffset = Random.w * HeightScale + HeightBias;
		if (bOnSurface)
		{
			const bool bOnSurfaceIncludeEndCaps = (ShapeLocation_Mode.y & 0x2) != 0;
			const float ThicknessScale = asfloat(ShapeLocation_Mode.z);
			const float ThicknessBias = asfloat(ShapeLocation_Mode.w);

			const uint Edge = (RandomUInt(1) & 0x3) + (bOnSurfaceIncludeEndCaps ? 0 : 2);
			ZOffset  = Edge == 0 ? HeightBias - ThicknessBias - (ThicknessScale * Random.w) : ZOffset;
			ZOffset  = Edge == 1 ? HeightBias + HeightScale + ThicknessBias + (ThicknessScale * Random.w) : ZOffset;
			XYOffset = Edge >= 2 ? Radius + ThicknessBias + (ThicknessScale * Random.z) : (Radius + ThicknessBias + ThicknessScale) * Random.z;
		}

		const float2 XYLocation = SafeNormalize(Random.xy - 0.5f) * XYOffset;

		float3 Location;
		Location  = AxisX * XYLocation.x;
		Location += AxisY * XYLocation.y;
		Location += AxisZ * ZOffset;
		return Location;
	}

	// ENSM_ShapePrimitive::Ring:
	BRANCH
	if ( ShapeLocation_Mode.x == 2 )
	{
		const float RadiusScale			= ShapeLocation_Parameters0.x;
		const float RadiusBias			= ShapeLocation_Parameters0.y;
		const float UDistributionScale	= ShapeLocation_Parameters0.z;
		const float UDistributionBias	= ShapeLocation_Parameters0.w;
		const float3 AxisX				= ShapeLocation_Parameters1.xyz;
		const float3 AxisY				= ShapeLocation_Parameters2.xyz;

		const float Radius = RandomScaleBiasFloat(0, RadiusScale, RadiusBias);
		const float U = RandomScaleBiasFloat(1, UDistributionScale, UDistributionBias);

		return (AxisX * cos(U) * Radius) + (AxisY * sin(U) * Radius);
	}

	// ENSM_ShapePrimitive::Sphere:
	{
		const float SphereScale	= ShapeLocation_Parameters0.w;
		const float SphereBias	= ShapeLocation_Parameters1.w;
		const float3 AxisX		= ShapeLocation_Parameters0.xyz;
		const float3 AxisY		= ShapeLocation_Parameters1.xyz;
		const float3 AxisZ		= ShapeLocation_Parameters2.xyz;

		const float3 Vector = RandomUnitFloat3(0);
		const float3 Location = Vector * RandomScaleBiasFloat(1, SphereScale, SphereBias);
		return (Location.x * AxisX) + (Location.y * AxisY) + (Location.z * AxisZ);
	}
}

void ShapeLocation_Simulate(inout FStatelessParticle Particle)
{
	const float3 Location = ShapeLocation_GetLocation();
	const float2x3 PositionOffsets = SampleDistributionValuesFloat3(ShapeLocation_TransformOffset, 1, Particle.NormalizedAge, Particle.PreviousNormalizedAge);
	Particle.Position += (PositionOffsets[0] * ShapeLocation_TransformOffsetScale) + Location;
	Particle.PreviousPosition += (PositionOffsets[1] * ShapeLocation_TransformOffsetScale) + Location;
}
