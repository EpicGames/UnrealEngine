// Copyright Epic Games, Inc. All Rights Reserved.

float3	SolveVelocitiesAndForces_OwnerAccelerationScale;
//float	SolveVelocitiesAndForces_OwnerDragScale;
float3	SolveVelocitiesAndForces_OwnerVelocityScale;
float	SolveVelocitiesAndForces_MassScale;
float	SolveVelocitiesAndForces_MassBias;
float	SolveVelocitiesAndForces_DragScale;
float	SolveVelocitiesAndForces_DragBias;
float3	SolveVelocitiesAndForces_VelocityScale;
float3	SolveVelocitiesAndForces_VelocityBias;
float3	SolveVelocitiesAndForces_WindScale;
float3	SolveVelocitiesAndForces_WindBias;
float3	SolveVelocitiesAndForces_AccelerationScale;
float3	SolveVelocitiesAndForces_AccelerationBias;
float3	SolveVelocitiesAndForces_GravityScale;
float3	SolveVelocitiesAndForces_GravityBias;

uint	SolveVelocitiesAndForces_ConeVelocityEnabled;
float4	SolveVelocitiesAndForces_ConeQuat;
float	SolveVelocitiesAndForces_ConeVelocityScale;
float	SolveVelocitiesAndForces_ConeVelocityBias;
float	SolveVelocitiesAndForces_ConeAngleScale;
float	SolveVelocitiesAndForces_ConeAngleBias;
float	SolveVelocitiesAndForces_ConeVelocityFalloff;

uint	SolveVelocitiesAndForces_PointVelocityEnabled;
float	SolveVelocitiesAndForces_PointVelocityScale;
float	SolveVelocitiesAndForces_PointVelocityBias;
float3	SolveVelocitiesAndForces_PointOrigin;

uint				SolveVelocitiesAndForces_NoiseEnabled;
float				SolveVelocitiesAndForces_NoiseStrength;
float				SolveVelocitiesAndForces_NoiseFrequency;
float3				SolveVelocitiesAndForces_NoiseFieldOffsetScale;
float3				SolveVelocitiesAndForces_NoiseFieldOffsetBias;
float				SolveVelocitiesAndForces_NoiseLUTNormalize;
uint				SolveVelocitiesAndForces_NoiseLUTRowMask;
uint				SolveVelocitiesAndForces_NoiseLUTRowWidth;
ByteAddressBuffer	SolveVelocitiesAndForces_NoiseLUT;

struct FStatelessModule_SolveVelocitiesAndForces
{
	float  Mass;
	float  Drag;
	float3 Velocity;
	float3 Wind;
	float3 Acceleration;
};

void SolveVelocitiesAndForces_EvaluateNoise(in FStatelessModule_SolveVelocitiesAndForces ModuleData, float Lifetime, float NormalizedAge, float Age, float3 NoiseFieldOffset, inout float3 Position)
{
	const float3 SamplePosition	= (Position + NoiseFieldOffset) * SolveVelocitiesAndForces_NoiseFrequency;
	
	const float fRow	= dot(abs(SamplePosition), float3(1.0f / 8.0f, 2.0f / 8.0f, 5.0f / 8.0f));
	const uint RowA		= uint(floor(abs(fRow))) & SolveVelocitiesAndForces_NoiseLUTRowMask;
	const uint RowB		= (RowA + 1) & SolveVelocitiesAndForces_NoiseLUTRowMask;

	const float TotalDistance	= Lifetime * SolveVelocitiesAndForces_NoiseStrength;
	const float EndColumn		= clamp(TotalDistance * SolveVelocitiesAndForces_NoiseLUTNormalize, 1.0f, float(SolveVelocitiesAndForces_NoiseLUTRowWidth - 1));
    
	const float fColumn = NormalizedAge * EndColumn;
	const uint iColumnA = min(floor(fColumn), SolveVelocitiesAndForces_NoiseLUTRowWidth - 1);
	const uint iColumnB = min(iColumnA + 1, SolveVelocitiesAndForces_NoiseLUTRowWidth - 1);

	const float3 ValueA0 = asfloat(SolveVelocitiesAndForces_NoiseLUT.Load3(((SolveVelocitiesAndForces_NoiseLUTRowWidth * RowA) + iColumnA) * 12));
	const float3 ValueA1 = asfloat(SolveVelocitiesAndForces_NoiseLUT.Load3(((SolveVelocitiesAndForces_NoiseLUTRowWidth * RowB) + iColumnA) * 12));
	const float3 ValueB0 = asfloat(SolveVelocitiesAndForces_NoiseLUT.Load3(((SolveVelocitiesAndForces_NoiseLUTRowWidth * RowA) + iColumnB) * 12));
	const float3 ValueB1 = asfloat(SolveVelocitiesAndForces_NoiseLUT.Load3(((SolveVelocitiesAndForces_NoiseLUTRowWidth * RowB) + iColumnB) * 12));
	const float3 ValueA  = lerp(ValueA0, ValueA1, frac(fRow));
	const float3 ValueB  = lerp(ValueB0, ValueB1, frac(fRow));
	
	Position += lerp(ValueA, ValueB, frac(fColumn)) * TotalDistance / EndColumn;
}

void SolveVelocitiesAndForces_IntegratePosition(in FStatelessModule_SolveVelocitiesAndForces ModuleData, float Age, inout float3 Position)
{
	if (ModuleData.Drag > 0.0001f)
	{
		const float3 TerminalVelocity = ModuleData.Acceleration * rcp(ModuleData.Drag) + ModuleData.Wind;
		const float3 IntVelocity = ModuleData.Velocity - TerminalVelocity;
		const float LambdaAge = (1.0f - exp(-(Age * ModuleData.Drag))) * rcp(ModuleData.Drag);
		Position += IntVelocity * LambdaAge + TerminalVelocity * Age;
	}
	else
	{
		// without drag, we can use the simpler formula for Newtonian motion v*t + 1/2*a*t²
		Position += Age * (ModuleData.Velocity + ModuleData.Wind) + 0.5f * ModuleData.Acceleration * Age * Age;
	}
}

void SolveVelocitiesAndForces_Simulate(inout FStatelessParticle Particle)
{
	// Initialize shared parameters
	FStatelessModule_SolveVelocitiesAndForces ModuleData;
	ModuleData.Mass = RandomScaleBiasFloat(0, SolveVelocitiesAndForces_MassScale, SolveVelocitiesAndForces_MassBias);
	ModuleData.Drag = RandomScaleBiasFloat(1, SolveVelocitiesAndForces_DragScale, SolveVelocitiesAndForces_DragBias);
	ModuleData.Velocity = RandomScaleBiasFloat(2, SolveVelocitiesAndForces_VelocityScale, SolveVelocitiesAndForces_VelocityBias);
	ModuleData.Wind = RandomScaleBiasFloat(3, SolveVelocitiesAndForces_WindScale, SolveVelocitiesAndForces_WindBias);
	const float MassFactor = rcp(max(ModuleData.Mass, UE_KINDA_SMALL_NUMBER)); // factor in mass for acceleration force
	ModuleData.Acceleration = MassFactor * RandomScaleBiasFloat(4, SolveVelocitiesAndForces_AccelerationScale, SolveVelocitiesAndForces_AccelerationBias); 
	ModuleData.Acceleration += RandomScaleBiasFloat(41, SolveVelocitiesAndForces_GravityScale, SolveVelocitiesAndForces_GravityBias);

	if ( SolveVelocitiesAndForces_ConeVelocityEnabled != 0 )
	{
		const float ConeAngle = RandomScaleBiasFloat(5, SolveVelocitiesAndForces_ConeAngleScale, SolveVelocitiesAndForces_ConeAngleBias);
		const float ConeRotation = RandomFloat(6) * UE_TWO_PI;
			
		float2 scAng = SinCos(ConeAngle);
		float2 scRot = SinCos(ConeRotation);
		const float3 Direction = float3(scRot.x * scAng.x, scRot.y * scAng.x, scAng.y);
		
		float VelocityScale = RandomScaleBiasFloat(7, SolveVelocitiesAndForces_ConeVelocityScale, SolveVelocitiesAndForces_ConeVelocityBias);
		if ( SolveVelocitiesAndForces_ConeVelocityFalloff > 0.0f )
		{
			const float pf = pow(saturate(scAng.y), SolveVelocitiesAndForces_ConeVelocityFalloff * 10.0f);
			VelocityScale *= lerp(1.0f, pf, SolveVelocitiesAndForces_ConeVelocityFalloff);
		}
			
		ModuleData.Velocity += RotateVectorByQuat(Direction, SolveVelocitiesAndForces_ConeQuat) * VelocityScale;
	}

	if ( SolveVelocitiesAndForces_PointVelocityEnabled != 0 )
	{
		const float3 FallbackDir = RandomUnitFloat3(8);
		const float3 Delta = Particle.Position - SolveVelocitiesAndForces_PointOrigin;
		const float3 Dir = SafeNormalize(Delta, FallbackDir);
			
		const float VelocityScale = RandomScaleBiasFloat(9, SolveVelocitiesAndForces_PointVelocityScale, SolveVelocitiesAndForces_PointVelocityBias);
			
		ModuleData.Velocity += Dir * VelocityScale;
	}

	if ( SolveVelocitiesAndForces_NoiseEnabled  != 0 )
	{
		const float3 NosieFieldBias = RandomScaleBiasFloat(10, SolveVelocitiesAndForces_NoiseFieldOffsetScale, SolveVelocitiesAndForces_NoiseFieldOffsetBias);
		SolveVelocitiesAndForces_EvaluateNoise(ModuleData, Particle.Lifetime, Particle.NormalizedAge, Particle.Age, NosieFieldBias, Particle.Position);
		SolveVelocitiesAndForces_EvaluateNoise(ModuleData, Particle.Lifetime, Particle.PreviousNormalizedAge, Particle.PreviousAge, NosieFieldBias, Particle.PreviousPosition);
	}

	// Apply Owner Scaling to data
	//ModuleData.Drag			*= SolveVelocitiesAndForces_OwnerDragScale;
	ModuleData.Velocity		*= SolveVelocitiesAndForces_OwnerVelocityScale;
	ModuleData.Wind			*= SolveVelocitiesAndForces_OwnerAccelerationScale;
	ModuleData.Acceleration *= SolveVelocitiesAndForces_OwnerAccelerationScale;

	// Simulate
	// Note we need to ensure velocity aligned sprites work we need to make sure Age & PreviousAge != 0.0
	SolveVelocitiesAndForces_IntegratePosition(ModuleData, max(Particle.Age, UE_KINDA_SMALL_NUMBER), Particle.Position);
	SolveVelocitiesAndForces_IntegratePosition(ModuleData, Particle.PreviousAge, Particle.PreviousPosition);
		
	Particle.Velocity = (Particle.Position - Particle.PreviousPosition) * Particle.InvDeltaTime;
	Particle.PreviousVelocity = Particle.Velocity;
}
