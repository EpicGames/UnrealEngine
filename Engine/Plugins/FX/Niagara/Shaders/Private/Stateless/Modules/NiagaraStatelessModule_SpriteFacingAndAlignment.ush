// Copyright Epic Games, Inc. All Rights Reserved.

float3	SpriteFacingAndAlignment_SpriteFacing;
float3	SpriteFacingAndAlignment_SpriteAlignment;

void SpriteFacingAndAlignment_Simulate(inout FStatelessParticle Particle)
{
	Particle.SpriteFacing = SpriteFacingAndAlignment_SpriteFacing;
	Particle.SpriteAlignment = SpriteFacingAndAlignment_SpriteAlignment;

	Particle.PreviousSpriteFacing = SpriteFacingAndAlignment_SpriteFacing;
	Particle.PreviousSpriteAlignment = SpriteFacingAndAlignment_SpriteAlignment;
}
