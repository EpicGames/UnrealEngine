// Copyright Epic Games, Inc. All Rights Reserved.

int										SpriteRotationRate_Enabled;
float									SpriteRotationRate_Scale;
float									SpriteRotationRate_Bias;
FNiagaraStatelessBuiltDistributionType	SpriteRotationRate_RateScaleParameters;

void SpriteRotationRate_Simulate(inout FStatelessParticle Particle)
{
	if (SpriteRotationRate_Enabled == 0)
	{
		 return;
	}
		
	const float RotationRate = RandomScaleBiasFloat(0, SpriteRotationRate_Scale, SpriteRotationRate_Bias);
	const float MultiplyRate = SampleCurveFloat(SpriteRotationRate_RateScaleParameters, Particle.NormalizedAge);
	const float PreviousMultiplyRate = SampleCurveFloat(SpriteRotationRate_RateScaleParameters, Particle.PreviousNormalizedAge);
		
	Particle.SpriteRotation += Particle.Lifetime * RotationRate * MultiplyRate;
	Particle.PreviousSpriteRotation += Particle.Lifetime * RotationRate * PreviousMultiplyRate;
}
