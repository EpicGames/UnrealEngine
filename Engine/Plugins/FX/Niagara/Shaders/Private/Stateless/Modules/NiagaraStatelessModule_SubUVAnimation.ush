// Copyright Epic Games, Inc. All Rights Reserved.

int		SubUVAnimation_Mode;
float	SubUVAnimation_NumFrames;
float	SubUVAnimation_InitialFrameScale;
float	SubUVAnimation_InitialFrameBias;
float	SubUVAnimation_InitialFrameRateChange;
float	SubUVAnimation_AnimFrameStart;
float	SubUVAnimation_AnimFrameRange;
float	SubUVAnimation_RateScale;

void SubUVAnimation_Simulate(inout FStatelessParticle Particle)
{
	const uint SeedOffset = uint(Particle.Age * SubUVAnimation_InitialFrameRateChange);
	float Frame = RandomFloat(SeedOffset) * SubUVAnimation_InitialFrameScale + SubUVAnimation_InitialFrameBias;

	if ( SubUVAnimation_Mode == 1 )
	{
		const float Interp = Particle.Age * SubUVAnimation_RateScale;
		Frame = frac(Frame + SubUVAnimation_AnimFrameStart + (Interp * SubUVAnimation_AnimFrameRange));
	}
	else if ( SubUVAnimation_Mode == 2 )
	{
		const float Interp = Particle.NormalizedAge * SubUVAnimation_RateScale;
		Frame = saturate(Frame + SubUVAnimation_AnimFrameStart + (Interp * SubUVAnimation_AnimFrameRange));
	}
	Particle.SubImageIndex = Frame * SubUVAnimation_NumFrames;
}
