// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/Random.ush"

#include "/Plugin/FX/Niagara/Shared/NiagaraStatelessDefinitions.h"
#include "/Plugin/FX/Niagara/Shared/NiagaraStatelessBuiltDistribution.h"
#include "/Plugin/FX/Niagara/Private/NiagaraQuaternionUtils.ush"

////////////////////////////////////////////////////////////////////////////////////

#define INVALID_COMPONENT		-1
#define UE_SMALL_NUMBER			(1.e-8f)
#define UE_KINDA_SMALL_NUMBER	(1.e-4f)
#define UE_PI 					(3.1415926535897932f)
#define UE_INV_PI				(0.31830988618f)
#define UE_HALF_PI				(1.57079632679f)
#define UE_TWO_PI				(6.28318530717f)
#define UE_PI_SQUARED			(9.86960440108f)
#define UE_DEG_TO_RAD			(UE_PI / 180.0f)

bool IsValidComponent(int ComponentOffset)
{
	return ComponentOffset != INVALID_COMPONENT;
}

float4				Common_ToSimulationRotations[3];
uint				Common_OutputBufferStride;
RWBuffer<float>		Common_FloatOutputBuffer;
RWBuffer<int>		Common_IntOutputBuffer;
ByteAddressBuffer	Common_StaticDataBuffer;
ByteAddressBuffer	Common_DynamicDataBuffer;

////////////////////////////////////////////////////////////////////////////////////

float4 GetToSimulationRotation(int SourceSpace)
{
	return Common_ToSimulationRotations[SourceSpace];
}

////////////////////////////////////////////////////////////////////////////////////

float2 SinCos(float v)
{
	float2 sc;
	sincos(v, sc.x, sc.y);
	return sc;
}

float2 SafeNormalize(float2 v, float2 Fallback)
{
	const float l2 = length2(v);
	return l2 < UE_KINDA_SMALL_NUMBER ? Fallback : v * rsqrt(l2);
}

float2 SafeNormalize(float2 v)
{
	return SafeNormalize(v, float2(1.0f, 0.0f));
}

float3 SafeNormalize(float3 v, float3 Fallback)
{
	const float l2 = length2(v);
	return l2 < UE_KINDA_SMALL_NUMBER ? Fallback : v * rsqrt(l2);
}

float3 SafeNormalize(float3 v)
{
	return SafeNormalize(v, float3(1.0f, 0.0f, 0.0f));
}

static uint4 GRandomSeedInternal;

uint4 RandomUInt4(uint RandomSeedOffset)
{
	uint4 RandomSeed = FNiagaraStatelessDefinitions::OffsetRandomSeedForCall(GRandomSeedInternal, RandomSeedOffset);
	return Rand4DPCG32(RandomSeed);
}
uint   RandomUInt(uint RandomSeedOffset) { return RandomUInt4(RandomSeedOffset).x; }
uint2  RandomUInt2(uint RandomSeedOffset) { return RandomUInt4(RandomSeedOffset).xy; }
uint3  RandomUInt3(uint RandomSeedOffset) { return RandomUInt4(RandomSeedOffset).xyz; }

float4 RandomFloat4(uint RandomSeedOffset)
{
	uint4 v = RandomUInt4(RandomSeedOffset);
	return float4((v >> 8) & 0x00ffffff) / 16777216.0; // 0x01000000 == 16777216
}
float  RandomFloat(uint RandomSeedOffset)  { return RandomFloat4(RandomSeedOffset).x; }
float2 RandomFloat2(uint RandomSeedOffset) { return RandomFloat4(RandomSeedOffset).xy; }
float3 RandomFloat3(uint RandomSeedOffset) { return RandomFloat4(RandomSeedOffset).xyz; }
	
float  RandomRangeFloat(uint RandomSeedOffset, float  Min, float  Max) { return (RandomFloat(RandomSeedOffset)  * (Max - Min)) + Min; }
float2 RandomRangeFloat(uint RandomSeedOffset, float2 Min, float2 Max) { return (RandomFloat2(RandomSeedOffset) * (Max - Min)) + Min; }
float3 RandomRangeFloat(uint RandomSeedOffset, float3 Min, float3 Max) { return (RandomFloat3(RandomSeedOffset) * (Max - Min)) + Min; }
float4 RandomRangeFloat(uint RandomSeedOffset, float4 Min, float4 Max) { return (RandomFloat4(RandomSeedOffset) * (Max - Min)) + Min; }

float  RandomScaleBiasFloat(uint RandomSeedOffset, float  Scale, float  Bias) { return (RandomFloat(RandomSeedOffset)  * Scale) + Bias; }
float2 RandomScaleBiasFloat(uint RandomSeedOffset, float2 Scale, float2 Bias) { return (RandomFloat2(RandomSeedOffset) * Scale) + Bias; }
float3 RandomScaleBiasFloat(uint RandomSeedOffset, float3 Scale, float3 Bias) { return (RandomFloat3(RandomSeedOffset) * Scale) + Bias; }
float4 RandomScaleBiasFloat(uint RandomSeedOffset, float4 Scale, float4 Bias) { return (RandomFloat4(RandomSeedOffset) * Scale) + Bias; }

float  RandomScaleBiasFloat(uint RandomSeedOffset, float  Scale, float  Bias, bool bUniform) { return (RandomFloat(RandomSeedOffset)  * Scale) + Bias; }
float2 RandomScaleBiasFloat(uint RandomSeedOffset, float2 Scale, float2 Bias, bool bUniform) { float2 Random = (RandomFloat2(RandomSeedOffset) * Scale) + Bias; return bUniform ? Random.xx : Random; }
float3 RandomScaleBiasFloat(uint RandomSeedOffset, float3 Scale, float3 Bias, bool bUniform) { float3 Random = (RandomFloat3(RandomSeedOffset) * Scale) + Bias; return bUniform ? Random.xxx : Random; }
float4 RandomScaleBiasFloat(uint RandomSeedOffset, float4 Scale, float4 Bias, bool bUniform) { float4 Random = (RandomFloat4(RandomSeedOffset) * Scale) + Bias; return bUniform ? Random.xxxx : Random; }

float2 RandomUnitFloat2(uint RandomSeedOffset) { return SafeNormalize(RandomFloat2(RandomSeedOffset) - 0.5f); }
float3 RandomUnitFloat3(uint RandomSeedOffset) { return SafeNormalize(RandomFloat3(RandomSeedOffset) - 0.5f); }

uint   GetStaticUInt  (int BaseOffset, int Element)	{ return asuint (Common_StaticDataBuffer.Load ((BaseOffset + Element) * 4)); }
int    GetStaticInt   (int BaseOffset, int Element)	{ return asint  (Common_StaticDataBuffer.Load ((BaseOffset + Element) * 4)); }
float  GetStaticFloat (int BaseOffset, int Element)	{ return asfloat(Common_StaticDataBuffer.Load ((BaseOffset + Element) * 4)); }
float2 GetStaticFloat2(int BaseOffset, int Element)	{ return asfloat(Common_StaticDataBuffer.Load2((BaseOffset + Element*2) * 4)); }
float3 GetStaticFloat3(int BaseOffset, int Element)	{ return asfloat(Common_StaticDataBuffer.Load3((BaseOffset + Element*3) * 4)); }
float4 GetStaticFloat4(int BaseOffset, int Element)	{ return asfloat(Common_StaticDataBuffer.Load4((BaseOffset + Element*4) * 4)); }

uint   GetParameterBufferUint  (int BaseOffset, int Element) { return asuint (Common_DynamicDataBuffer.Load ((BaseOffset + Element) * 4)); }
int    GetParameterBufferInt   (int BaseOffset, int Element) { return asint  (Common_DynamicDataBuffer.Load ((BaseOffset + Element) * 4)); }
float  GetParameterBufferFloat (int BaseOffset, int Element) { return asfloat(Common_DynamicDataBuffer.Load ((BaseOffset + Element) * 4)); }
float2 GetParameterBufferFloat2(int BaseOffset, int Element) { return asfloat(Common_DynamicDataBuffer.Load2((BaseOffset + Element*2) * 4)); }
float3 GetParameterBufferFloat3(int BaseOffset, int Element) { return asfloat(Common_DynamicDataBuffer.Load3((BaseOffset + Element*3) * 4)); }
float4 GetParameterBufferFloat4(int BaseOffset, int Element) { return asfloat(Common_DynamicDataBuffer.Load4((BaseOffset + Element*4) * 4)); }

uint   GetParameterBufferUint  (int BaseOffset, int Element, uint  DefaultValue)  { if ( BaseOffset >= 0 ) { return asuint (Common_DynamicDataBuffer.Load ((BaseOffset + Element) * 4)); }   return DefaultValue; }
int    GetParameterBufferInt   (int BaseOffset, int Element, int    DefaultValue) { if ( BaseOffset >= 0 ) { return asint  (Common_DynamicDataBuffer.Load ((BaseOffset + Element) * 4)); }   return DefaultValue; }
float  GetParameterBufferFloat (int BaseOffset, int Element, float  DefaultValue) { if ( BaseOffset >= 0 ) { return asfloat(Common_DynamicDataBuffer.Load ((BaseOffset + Element) * 4)); }   return DefaultValue; }
float2 GetParameterBufferFloat2(int BaseOffset, int Element, float2 DefaultValue) { if ( BaseOffset >= 0 ) { return asfloat(Common_DynamicDataBuffer.Load2((BaseOffset + Element*2) * 4)); } return DefaultValue; }
float3 GetParameterBufferFloat3(int BaseOffset, int Element, float3 DefaultValue) { if ( BaseOffset >= 0 ) { return asfloat(Common_DynamicDataBuffer.Load3((BaseOffset + Element*3) * 4)); } return DefaultValue; }
float4 GetParameterBufferFloat4(int BaseOffset, int Element, float4 DefaultValue) { if ( BaseOffset >= 0 ) { return asfloat(Common_DynamicDataBuffer.Load4((BaseOffset + Element*4) * 4)); } return DefaultValue; }
	
float  StatelessLerp(float  lhs, float  rhs, float  u, bool bUniform) { return lerp(lhs, rhs, u); }
float2 StatelessLerp(float2 lhs, float2 rhs, float2 u, bool bUniform) { return lerp(lhs, rhs, bUniform ? u.xx : u); }
float3 StatelessLerp(float3 lhs, float3 rhs, float3 u, bool bUniform) { return lerp(lhs, rhs, bUniform ? u.xxx : u); }
float4 StatelessLerp(float4 lhs, float4 rhs, float4 u, bool bUniform) { return lerp(lhs, rhs, bUniform ? u.xxxx : u); }

uint GetComponentOffset(uint OutputIndex, int ComponentOffset)
{
	return ComponentOffset * Common_OutputBufferStride + OutputIndex;
}

void OutputComponentData(uint OutputIndex, int ComponentOffset, float Value)
{
	if (IsValidComponent(ComponentOffset))
	{
		Common_FloatOutputBuffer[GetComponentOffset(OutputIndex, ComponentOffset)] = Value;
	}
}

void OutputComponentData(uint OutputIndex, int ComponentOffset, float2 Value)
{
	if (IsValidComponent(ComponentOffset))
	{
		Common_FloatOutputBuffer[GetComponentOffset(OutputIndex, ComponentOffset + 0)] = Value.x;
		Common_FloatOutputBuffer[GetComponentOffset(OutputIndex, ComponentOffset + 1)] = Value.y;
	}
}

void OutputComponentData(uint OutputIndex, int ComponentOffset, float3 Value)
{
	if (IsValidComponent(ComponentOffset))
	{
		Common_FloatOutputBuffer[GetComponentOffset(OutputIndex, ComponentOffset + 0)] = Value.x;
		Common_FloatOutputBuffer[GetComponentOffset(OutputIndex, ComponentOffset + 1)] = Value.y;
		Common_FloatOutputBuffer[GetComponentOffset(OutputIndex, ComponentOffset + 2)] = Value.z;
	}
}

void OutputComponentData(uint OutputIndex, int ComponentOffset, float4 Value)
{
	if (IsValidComponent(ComponentOffset))
	{
		Common_FloatOutputBuffer[GetComponentOffset(OutputIndex, ComponentOffset + 0)] = Value.x;
		Common_FloatOutputBuffer[GetComponentOffset(OutputIndex, ComponentOffset + 1)] = Value.y;
		Common_FloatOutputBuffer[GetComponentOffset(OutputIndex, ComponentOffset + 2)] = Value.z;
		Common_FloatOutputBuffer[GetComponentOffset(OutputIndex, ComponentOffset + 3)] = Value.w;
	}
}

void OutputComponentData(uint OutputIndex, int ComponentOffset, int Value)
{
	if (IsValidComponent(ComponentOffset))
	{
		Common_IntOutputBuffer[GetComponentOffset(OutputIndex, ComponentOffset)] = Value;
	}
}

#include "/Plugin/FX/Niagara/Private/Stateless/NiagaraStatelessDistribution.ush"
