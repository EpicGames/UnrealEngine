// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#define SAMPLEFUNC(HLSLTYPE, HLSLPAIRTYPE, NAMETYPE) \
	HLSLPAIRTYPE Private_GetDistributionTableValues##NAMETYPE##(FNiagaraStatelessBuiltDistributionType BuiltData, uint IndexA, uint IndexB) \
	{ \
		const uint BaseOffset = FNiagaraStatelessBuiltDistribution::GetBufferOffset(BuiltData); \
		if (FNiagaraStatelessBuiltDistribution::IsBufferReadModeStatic(BuiltData)) \
		{ \
			return HLSLPAIRTYPE(GetStatic##NAMETYPE##(BaseOffset, IndexA), GetStatic##NAMETYPE##(BaseOffset, IndexB)); \
		} \
		else \
		{ \
			return HLSLPAIRTYPE(GetParameterBuffer##NAMETYPE##(BaseOffset, IndexA), GetParameterBuffer##NAMETYPE##(BaseOffset, IndexB)); \
		} \
	} \
	\
	HLSLTYPE Private_GetDistributionRandomValue##NAMETYPE##(FNiagaraStatelessBuiltDistributionType BuiltData, uint RandomSeedOffset) \
	{ \
		const float Random = RandomFloat(RandomSeedOffset); \
		uint Indicies[2]; \
		float Interp; \
		FNiagaraStatelessBuiltDistribution::ConvertRandomToLookup(BuiltData, Random, Indicies[0], Indicies[1], Interp); \
		HLSLPAIRTYPE Values = Private_GetDistributionTableValues##NAMETYPE##(BuiltData, Indicies[0], Indicies[1]); \
		if (FNiagaraStatelessBuiltDistribution::IsInterpolationModeLinear(BuiltData)) \
		{ \
			const bool bUniformRandom = FNiagaraStatelessBuiltDistribution::IsUniformRandomEnabled(BuiltData); \
			return StatelessLerp(Values[0], Values[1], Random##NAMETYPE##(RandomSeedOffset), bUniformRandom); \
		} \
		else \
		{ \
			return Values[0]; \
		} \
	} \
	HLSLTYPE Private_GetDistributionTimeValue##NAMETYPE##(FNiagaraStatelessBuiltDistributionType BuiltData, float Time) \
	{ \
		uint Indicies[2]; \
		float Interp; \
		FNiagaraStatelessBuiltDistribution::ConvertTimeToLookup(BuiltData, Time, Indicies[0], Indicies[1], Interp); \
		HLSLPAIRTYPE Values = Private_GetDistributionTableValues##NAMETYPE##(BuiltData, Indicies[0], Indicies[1]); \
		return lerp(Values[0], Values[1], Interp); \
	} \
	\
	HLSLTYPE SampleDistributionValue##NAMETYPE##(FNiagaraStatelessBuiltDistributionType BuiltData, uint RandomSeedOffset, float Time) \
	{ \
		if (FNiagaraStatelessBuiltDistribution::IsLookupValueModeRandom(BuiltData)) \
		{ \
			return Private_GetDistributionRandomValue##NAMETYPE##(BuiltData, RandomSeedOffset); \
		} \
		else \
		{ \
			return Private_GetDistributionTimeValue##NAMETYPE##(BuiltData, Time); \
		} \
	} \
	HLSLPAIRTYPE SampleDistributionValues##NAMETYPE##(FNiagaraStatelessBuiltDistributionType BuiltData, uint RandomSeedOffset, float TimeA, float TimeB) \
	{ \
		if (FNiagaraStatelessBuiltDistribution::IsLookupValueModeRandom(BuiltData)) \
		{ \
			HLSLTYPE Value = Private_GetDistributionRandomValue##NAMETYPE##(BuiltData, RandomSeedOffset); \
			return HLSLPAIRTYPE(Value, Value); \
		} \
		else \
		{ \
			return HLSLPAIRTYPE(Private_GetDistributionTimeValue##NAMETYPE##(BuiltData, TimeA), Private_GetDistributionTimeValue##NAMETYPE##(BuiltData, TimeB)); \
		} \
	}\
    HLSLTYPE SampleCurve##NAMETYPE##(FNiagaraStatelessBuiltDistributionType Distribution, float Time) \
    { \
	    uint Indicies[2]; \
	    float Interp; \
	    FNiagaraStatelessBuiltDistribution::ConvertTimeToLookup(Distribution, Time, Indicies[0], Indicies[1], Interp); \
	    const uint DataOffset = FNiagaraStatelessBuiltDistribution::GetBufferOffset(Distribution); \
	    const HLSLTYPE Value0 = GetStatic##NAMETYPE##(DataOffset, Indicies[0]); \
	    const HLSLTYPE Value1 = GetStatic##NAMETYPE##(DataOffset, Indicies[1]); \
	    return lerp(Value0, Value1, Interp); \
    }

	SAMPLEFUNC(float,  float2,   Float)
	SAMPLEFUNC(float2, float2x2, Float2)
	SAMPLEFUNC(float3, float2x3, Float3)
	SAMPLEFUNC(float4, float2x4, Float4)

#undef READFUNCS
