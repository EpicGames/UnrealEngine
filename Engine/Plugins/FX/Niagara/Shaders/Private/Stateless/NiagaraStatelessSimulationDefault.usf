// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/FX/Niagara/Private/NiagaraQuaternionUtils.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/NiagaraStatelessCommon.ush"

// Defines all paricles attributes
// - PARTICLE_ATTRIBUTE_OUTPUT - Defines an output variable
// - PARTICLE_ATTRIBUTE_OUTPUT_IMPLICIT - Defines an output variable which is already part of the main particle payload (i.e. always present)
// - PARTICLE_ATTRIBUTE_TRANSIENT - Defines an output that is temporary and not output
#define PARTICLE_ATTRIBUTES \
	PARTICLE_ATTRIBUTE_OUTPUT_IMPLICIT(int,		UniqueID) \
	PARTICLE_ATTRIBUTE_OUTPUT_IMPLICIT(float,	MaterialRandom) \
	PARTICLE_ATTRIBUTE_OUTPUT(float3,	Position) \
	PARTICLE_ATTRIBUTE_OUTPUT(float,	CameraOffset) \
	PARTICLE_ATTRIBUTE_OUTPUT(float4,	Color) \
	PARTICLE_ATTRIBUTE_OUTPUT(float4,	DynamicMaterialParameter) \
	PARTICLE_ATTRIBUTE_OUTPUT(float4,	DynamicMaterialParameter1) \
	PARTICLE_ATTRIBUTE_OUTPUT(float4,	DynamicMaterialParameter2) \
	PARTICLE_ATTRIBUTE_OUTPUT(float4,	DynamicMaterialParameter3) \
	PARTICLE_ATTRIBUTE_OUTPUT(int,		MeshIndex) \
	PARTICLE_ATTRIBUTE_OUTPUT(float4,	MeshOrientation) \
	PARTICLE_ATTRIBUTE_OUTPUT(float,	RibbonWidth) \
	PARTICLE_ATTRIBUTE_OUTPUT(float3,	Scale) \
	PARTICLE_ATTRIBUTE_OUTPUT(float2,	SpriteSize) \
	PARTICLE_ATTRIBUTE_OUTPUT(float3,	SpriteFacing) \
	PARTICLE_ATTRIBUTE_OUTPUT(float3,	SpriteAlignment) \
	PARTICLE_ATTRIBUTE_OUTPUT(float,	SpriteRotation) \
	PARTICLE_ATTRIBUTE_OUTPUT(float,	SubImageIndex) \
	PARTICLE_ATTRIBUTE_OUTPUT(float3,	Velocity) \
	PARTICLE_ATTRIBUTE_OUTPUT(float,	PreviousCameraOffset) \
	PARTICLE_ATTRIBUTE_OUTPUT(float3,	PreviousPosition) \
	PARTICLE_ATTRIBUTE_OUTPUT(float4,	PreviousMeshOrientation) \
	PARTICLE_ATTRIBUTE_OUTPUT(float,	PreviousRibbonWidth) \
	PARTICLE_ATTRIBUTE_OUTPUT(float3,	PreviousScale) \
	PARTICLE_ATTRIBUTE_OUTPUT(float2,	PreviousSpriteSize) \
	PARTICLE_ATTRIBUTE_OUTPUT(float3,	PreviousSpriteFacing) \
	PARTICLE_ATTRIBUTE_OUTPUT(float3,	PreviousSpriteAlignment) \
	PARTICLE_ATTRIBUTE_OUTPUT(float,	PreviousSpriteRotation) \
	PARTICLE_ATTRIBUTE_OUTPUT(float3,	PreviousVelocity) \

// Defines modules and order of execution
#define PARTICLE_MODULES \
	/* Initializer Modules */ \
	PARTICLE_MODULE(InitializeParticle) \
	PARTICLE_MODULE(InitialMeshOrientation) \
	PARTICLE_MODULE(ShapeLocation) \
	/* Before Solve */ \
	PARTICLE_MODULE(Null) /* AddVelocity */ \
	PARTICLE_MODULE(Null) /* AccelerationForce */ \
	PARTICLE_MODULE(Null) /* CurlNoiseForce */ \
	PARTICLE_MODULE(Null) /* Drag */ \
	PARTICLE_MODULE(Null) /* GravityForce */ \
	PARTICLE_MODULE(Null) /* ApplyOwnerScaleToAttributes */ \
	PARTICLE_MODULE(SolveVelocitiesAndForces) \
	/* Post Solve */ \
	PARTICLE_MODULE(CameraOffset) \
	PARTICLE_MODULE(Null) /* DecalAttributes */ \
	PARTICLE_MODULE(DynamicMaterialParameters) \
	PARTICLE_MODULE(Null) /* LightAttributes */ \
	PARTICLE_MODULE(MeshIndex) \
	PARTICLE_MODULE(MeshRotationRate) \
	PARTICLE_MODULE(ScaleColor) \
	PARTICLE_MODULE(ScaleRibbonWidth) \
	PARTICLE_MODULE(ScaleSpriteSize) \
	PARTICLE_MODULE(ScaleSpriteSizeBySpeed) \
	PARTICLE_MODULE(ScaleMeshSize) \
	PARTICLE_MODULE(ScaleMeshSizeBySpeed) \
	PARTICLE_MODULE(SpriteFacingAndAlignment) \
	PARTICLE_MODULE(SpriteRotationRate) \
	PARTICLE_MODULE(SubUVAnimation) \

#include "/Plugin/FX/Niagara/Private/Stateless/Template/GenerateCS_PreModules.ush"

//#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_CalculateAccurateVelocity.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_CameraOffset.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_DynamicMaterialParameters.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_InitializeParticle.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_InitialMeshOrientation.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_MeshIndex.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_MeshRotationRate.ush"
//#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_RotateAroundPoint.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ShapeLocation.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleColor.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleRibbonWidth.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleSpriteSize.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleSpriteSizeBySpeed.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleMeshSize.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleMeshSizeBySpeed.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_SolveVelocitiesAndForces.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_SpriteFacingAndAlignment.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_SpriteRotationRate.ush"
#include "/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_SubUVAnimation.ush"

#include "/Plugin/FX/Niagara/Private/Stateless/Template/GenerateCS_PostModules.ush"
