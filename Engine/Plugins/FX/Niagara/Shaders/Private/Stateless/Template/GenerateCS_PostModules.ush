// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/ComputeShaderUtils.ush"

uint	Common_RandomSeed;
float	Common_SimulationTime;
float	Common_SimulationDeltaTime;
float	Common_SimulationInvDeltaTime;

uint				Common_GPUCountBufferOffset;
RWBuffer<uint>		Common_GPUCountBuffer;

DECLARE_SCALAR_ARRAY(uint,	SpawnInfo_NumActive, NIAGARA_MAX_GPU_SPAWN_INFOS);
DECLARE_SCALAR_ARRAY(uint,	SpawnInfo_ParticleOffset, NIAGARA_MAX_GPU_SPAWN_INFOS);
DECLARE_SCALAR_ARRAY(uint,	SpawnInfo_UniqueOffset, NIAGARA_MAX_GPU_SPAWN_INFOS);
DECLARE_SCALAR_ARRAY(float,	SpawnInfo_Time, NIAGARA_MAX_GPU_SPAWN_INFOS);
DECLARE_SCALAR_ARRAY(float,	SpawnInfo_Rate, NIAGARA_MAX_GPU_SPAWN_INFOS);
DECLARE_SCALAR_ARRAY(float,	SpawnInfo_Probability, NIAGARA_MAX_GPU_SPAWN_INFOS);
DECLARE_SCALAR_ARRAY(float,	SpawnInfo_LifetimeScale, NIAGARA_MAX_GPU_SPAWN_INFOS);
DECLARE_SCALAR_ARRAY(float,	SpawnInfo_LifetimeBias, NIAGARA_MAX_GPU_SPAWN_INFOS);

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

[numthreads(THREADGROUP_SIZE, 1, 1)]
void StatelessMain(uint3 GroupId : SV_GroupID, int GroupThreadIndex : SV_GroupIndex)
{
	// Convert thread ID into linear thread
	uint LinearThreadId = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, THREADGROUP_SIZE);

	// Initialize our particle data
	FStatelessParticle Particle = (FStatelessParticle)0;
	Particle.Age = -1.0f;	

	// Find which spawn info we belong to and fill out details
	uint ModuleSeedOffset = 0;
	{
		uint SpawnInfoIndex = LinearThreadId;
		for ( int i=0; i < NIAGARA_MAX_GPU_SPAWN_INFOS; ++i )
		{
			const uint SpawnInfoNumActive		= GET_SCALAR_ARRAY_ELEMENT(SpawnInfo_NumActive, i);
			const uint SpawnInfoParticleOffset	= GET_SCALAR_ARRAY_ELEMENT(SpawnInfo_ParticleOffset, i);
			const uint SpawnInfoUniqueOffset	= GET_SCALAR_ARRAY_ELEMENT(SpawnInfo_UniqueOffset, i);
			const float SpawnInfoTime			= GET_SCALAR_ARRAY_ELEMENT(SpawnInfo_Time, i);
			const float SpawnInfoRate			= GET_SCALAR_ARRAY_ELEMENT(SpawnInfo_Rate, i);
			const float SpawnInfoProbability	= GET_SCALAR_ARRAY_ELEMENT(SpawnInfo_Probability, i);
			const float SpawnInfoLifetimeScale	= GET_SCALAR_ARRAY_ELEMENT(SpawnInfo_LifetimeScale, i);
			const float SpawnInfoLifetimeBias	= GET_SCALAR_ARRAY_ELEMENT(SpawnInfo_LifetimeBias, i);
			if ( SpawnInfoIndex < SpawnInfoNumActive )
			{
				const uint SpawnParticleIndex = SpawnInfoIndex + SpawnInfoParticleOffset;
				Particle.UniqueID		= SpawnInfoIndex + SpawnInfoUniqueOffset + SpawnInfoParticleOffset;
				Particle.Age			= Common_SimulationTime - (SpawnInfoTime + float(SpawnParticleIndex) * SpawnInfoRate);
				GRandomSeedInternal		= FNiagaraStatelessDefinitions::MakeRandomSeed(Common_RandomSeed, Particle.UniqueID, ModuleSeedOffset++, 0);
				Particle.Lifetime		= RandomScaleBiasFloat(0, SpawnInfoLifetimeScale, SpawnInfoLifetimeBias);
				Particle.Lifetime		= RandomFloat(2) <= SpawnInfoProbability ? Particle.Lifetime : -1.0f;
				break;
			}

			SpawnInfoIndex -= SpawnInfoNumActive;
		}
		if ( Particle.Age < 0.0f )
		{
			return;
		}
	}
	
	// Initialize variables / determine lifetime / etc
	if ( Particle.Lifetime <= 0.0f || Particle.Age >= Particle.Lifetime )
	{
		return;
	}

	Particle.NormalizedAge	= Particle.Age / Particle.Lifetime;

	Particle.PreviousAge			= max(Particle.Age - Common_SimulationDeltaTime, 0.0f);
	Particle.PreviousNormalizedAge	= Particle.PreviousAge / Particle.Lifetime;

	Particle.DeltaTime		= Common_SimulationDeltaTime;
	Particle.InvDeltaTime	= Common_SimulationInvDeltaTime;
	
	Particle.bAlive = true;

	Particle.MaterialRandom	= RandomFloat(1);
	
	// Run Particle Simulate
	#define PARTICLE_MODULE(NAME) \
		GRandomSeedInternal = FNiagaraStatelessDefinitions::MakeRandomSeed(Common_RandomSeed, Particle.UniqueID, ModuleSeedOffset++, 0); \
		##NAME##_Simulate(Particle);

		PARTICLE_MODULES

	#undef PARTICLE_MODULE

	if ( !Particle.bAlive )
	{
		return;
	}
	
	// Output Particles that are still alive
	uint OutputIndex;
	
	//WaveInterlockedAddScalar_(Common_GPUCountBuffer[Common_GPUCountBufferOffset], 1, Particle.OutputIndex);
	InterlockedAdd(Common_GPUCountBuffer[Common_GPUCountBufferOffset], 1, OutputIndex);

	#define PARTICLE_ATTRIBUTE_OUTPUT(TYPE, NAME) if ( IsValidComponent(Permutation_##NAME##Component) ) { OutputComponentData(OutputIndex, Permutation_##NAME##Component, Particle.##NAME); }
	#define PARTICLE_ATTRIBUTE_OUTPUT_IMPLICIT(TYPE, NAME) if ( IsValidComponent(Permutation_##NAME##Component) ) { OutputComponentData(OutputIndex, Permutation_##NAME##Component, Particle.##NAME); }
	#define PARTICLE_ATTRIBUTE_TRANSIENT(TYPE, NAME)
		PARTICLE_ATTRIBUTES
	#undef PARTICLE_ATTRIBUTE
	#undef PARTICLE_ATTRIBUTE_OUTPUT_IMPLICIT
	#undef PARTICLE_ATTRIBUTE_TRANSIENT
}
