// Copyright Epic Games, Inc. All Rights Reserved.

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FStatelessParticle
{
	bool bAlive;					// If the particle should remain alive or not (only evaluated as specific points)
	uint UniqueID;					// Unique particle index, sequential based on previous spawn info accumulation
	float MaterialRandom;			// Random float passed to materials
	float Lifetime;					// Overall lifetime for the particle
	float Age;						// Current age for the particle
	float NormalizedAge;			// Current normalized age for the particle
	float PreviousAge;				// Previous age for the particle
	float PreviousNormalizedAge;	// Previous normalized age for the particle

	float DeltaTime;				// Simulation DT
	float InvDeltaTime;				// Simulation Inverse DT
	
	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// TBD this might produce sub optimal code, needs investigate and is temporary
	#define PARTICLE_ATTRIBUTE_OUTPUT(TYPE, NAME) TYPE NAME##;
	#define PARTICLE_ATTRIBUTE_OUTPUT_IMPLICIT(TYPE, NAME)
	#define PARTICLE_ATTRIBUTE_TRANSIENT(TYPE, NAME) TYPE NAME##;
		PARTICLE_ATTRIBUTES
	#undef PARTICLE_ATTRIBUTE
	#undef PARTICLE_ATTRIBUTE_OUTPUT_IMPLICIT
	#undef PARTICLE_ATTRIBUTE_TRANSIENT
	/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
};

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#define PARTICLE_ATTRIBUTE_OUTPUT(TYPE, NAME) int Permutation_##NAME##Component;
#define PARTICLE_ATTRIBUTE_OUTPUT_IMPLICIT(TYPE, NAME) int Permutation_##NAME##Component;
#define PARTICLE_ATTRIBUTE_TRANSIENT(TYPE, NAME)
	PARTICLE_ATTRIBUTES
#undef PARTICLE_ATTRIBUTE
#undef PARTICLE_ATTRIBUTE_OUTPUT_IMPLICIT
#undef PARTICLE_ATTRIBUTE_TRANSIENT

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void Null_Simulate(inout FStatelessParticle Particle)
{
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
