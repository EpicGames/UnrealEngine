// Copyright Epic Games, Inc. All Rights Reserved.
// This file is shared between C++ & HLSL

#pragma once

#ifndef GPU_SIMULATION
	#define uint	uint32
#endif

struct FRibbonAccumulationValues
{
	float RibbonDistance;
	uint SegmentCount;
	uint MultiRibbonCount;			// Only valid when HAS_RIBBON_ID
	float TessTotalLength;			// Only valid when RibbonWantsAutomaticTessellation()
	float TessAvgSegmentLength;		// Only valid when RibbonWantsAutomaticTessellation()
	float TessAvgSegmentAngle;		// Only valid when RibbonWantsAutomaticTessellation()
	float TessTwistAvgAngle;		// Only valid when RibbonWantsAutomaticTessellation() && RibbonHasTwist()
	float TessTwistAvgWidth;		// Only valid when RibbonWantsAutomaticTessellation() && RibbonHasTwist()
};

struct ENiagaraGpuTessellationMode
{
	enum Type
	{
		Disabled,
		Automatic,
		Constant,
	};
};

#define NIAGARA_RIBBON_TESSELLATION_FULL_STATS_NUM_ELEMENTS		5
#define NIAGARA_RIBBON_TESSELLATION_NO_TWIST_STATS_NUM_ELEMENTS	3

#define VERTEX_GEN_OUTPUT_DATA_AUTO_TESSELLATION_NUM_ELEMENTS	15
#define VERTEX_GEN_OUTPUT_DATA_NUM_ELEMENTS						5

#ifndef GPU_SIMULATION
	#undef uint
#endif
