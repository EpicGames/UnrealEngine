// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifndef GPU_SIMULATION
	#define uint				uint32
	typedef FUintVector3		FNiagaraStatelessBuiltDistributionType;
	typedef const FUintVector3&	FNiagaraStatelessBuiltDistributionTypeIn;

	#define UEAsFloat(X) reinterpret_cast<const float&>(X);
	#define UEMin(X, Y) FMath::Min(X, Y)
	#define UEMax(X, Y) FMath::Max(X, Y)
	#define UEFrac(X) FMath::Fractional(X)
	#define UERoundToInt(X) FMath::RoundToInt(X)
	#define UEOutParam(X) X&
#else
	//typedef uint3			FNiagaraStatelessBuiltDistributionType;
	#define FNiagaraStatelessBuiltDistributionType		uint3
	#define FNiagaraStatelessBuiltDistributionTypeIn	uint3

	#define UEAsFloat(X) asfloat(X)
	#define UEMin(X, Y) min(X, Y)
	#define UEMax(X, Y) max(X, Y)
	#define UEFrac(X) frac(X)
	#define UERoundToInt(X) uint(round(X))
	#define UEOutParam(X) out X
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FNiagaraStatelessBuiltDistribution
{
	struct EFlags
	{
		enum
		{
			BufferOffsetShift		= 0u,				// Offset in the buffer where the data is stored
			BufferOffsetBits		= 17u,				// 128k max offset
			BufferOffsetMask		= (1u << BufferOffsetBits) - 1u,

			TableLengthShift		= 17u,				// Length of the table data minus 1
			TableLengthBits			= 9u,				// 512 entries max
			TableLengthMask			= (1u << TableLengthBits) - 1u,

			LookupValueModeShift	= 26u,				// Lookup Value Mode where all bits set is assumed to be random internally
			LookupValueModeBits		= 2u,				// The caller must decide how to interpret this data
			LookupValueModeMask		= (1u << LookupValueModeBits) - 1u,

			UniformRandom			= 1u << 28u,		// When true random will be uniform when interpolating vs non-uniform
			AddressModeWrap			= 1u << 29u,		// The address mode, when 0 clamp, when 1 wrap
			InterpolationModeLinear	= 1u << 30u,		// The interpolation mode, when 0 none, when 1 linear
			BufferReadModeStatic	= 1u << 31u,		// The buffer read mode, when 0 dynamic, when 1 static
		};
	};

	static bool IsValid(FNiagaraStatelessBuiltDistributionTypeIn BuiltData) { return BuiltData[2] != 0; }

	static bool IsUniformRandomEnabled(FNiagaraStatelessBuiltDistributionTypeIn BuiltData) { return (BuiltData[0] & EFlags::UniformRandom) != 0; }
	static bool IsAddressModeWrap(FNiagaraStatelessBuiltDistributionTypeIn BuiltData) { return (BuiltData[0] & EFlags::AddressModeWrap) != 0; }
	static bool IsInterpolationModeLinear(FNiagaraStatelessBuiltDistributionTypeIn BuiltData) { return (BuiltData[0] & EFlags::InterpolationModeLinear) != 0; }
	static bool IsBufferReadModeStatic(FNiagaraStatelessBuiltDistributionTypeIn BuiltData) { return (BuiltData[0] & EFlags::BufferReadModeStatic) != 0; }
	static bool IsLookupValueModeRandom(FNiagaraStatelessBuiltDistributionTypeIn BuiltData) { const uint Mask = EFlags::LookupValueModeMask << EFlags::LookupValueModeShift; return (BuiltData[0] & Mask) == Mask; }

	static uint GetBufferOffset(FNiagaraStatelessBuiltDistributionTypeIn BuiltData) { return (BuiltData[0] >> EFlags::BufferOffsetShift) & EFlags::BufferOffsetMask; }
	static uint GetTableLength(FNiagaraStatelessBuiltDistributionTypeIn BuiltData) { return (BuiltData[0] >> EFlags::TableLengthShift) & EFlags::TableLengthMask; }
	static uint GetLookupValueMode(FNiagaraStatelessBuiltDistributionTypeIn BuiltData) { return (BuiltData[0] >> EFlags::LookupValueModeShift) & EFlags::LookupValueModeMask; }

	static void ConvertRandomToLookup(FNiagaraStatelessBuiltDistributionTypeIn BuiltData, float Random, UEOutParam(uint) OutIndexA, UEOutParam(uint) OutIndexB, UEOutParam(float) OutInterp)
	{
		const uint TableLength = GetTableLength(BuiltData);
		const bool bInterpolate = IsInterpolationModeLinear(BuiltData);
		const float Index = Random * float(TableLength);
		OutIndexA = bInterpolate ? uint(Index) : UERoundToInt(Index);
		OutIndexB = OutIndexA + 1;
		OutInterp = bInterpolate ? UEFrac(Index) : 0.0f;
		if (IsAddressModeWrap(BuiltData))
		{
			OutIndexB = OutIndexB > TableLength ? 0 : OutIndexB;
		}
		else
		{
			OutIndexB = UEMin(OutIndexB, TableLength);
		}
	}

	static void ConvertTimeToLookup(FNiagaraStatelessBuiltDistributionTypeIn BuiltData, float Time, UEOutParam(uint) OutIndexA, UEOutParam(uint) OutIndexB, UEOutParam(float) OutInterp)
	{
		const float TimeBias	= UEAsFloat(BuiltData[1]);
		const float TimeScale	= UEAsFloat(BuiltData[2]);
		const uint  TableLength	= GetTableLength(BuiltData);
		const bool bInterpolate = IsInterpolationModeLinear(BuiltData);
		Time = UEMax((Time - TimeBias) * TimeScale, 0.0f);

		OutIndexA = bInterpolate ? uint(Time) : UERoundToInt(Time);
		OutIndexB = OutIndexA + 1;
		if (IsAddressModeWrap(BuiltData))
		{
			OutIndexA = OutIndexA % (TableLength + 1);
			OutIndexB = OutIndexB > TableLength ? 0 : OutIndexB;
		}
		else
		{
			OutIndexA = UEMin(OutIndexA, TableLength);
			OutIndexB = UEMin(OutIndexB, TableLength);
		}
		OutInterp = bInterpolate ? UEFrac(Time) : 0.0f;
	}

#ifndef GPU_SIMULATION
	static FNiagaraStatelessBuiltDistributionType GetDefault() { return FNiagaraStatelessBuiltDistributionType::ZeroValue; }

	static void SetInterpolationMode(FNiagaraStatelessBuiltDistributionType& BuiltData, bool bLinearInterpolation)
	{
		BuiltData[0] &= ~EFlags::InterpolationModeLinear;
		BuiltData[0] |= bLinearInterpolation ? EFlags::InterpolationModeLinear : 0;
	}

	static void SetAddressMode(FNiagaraStatelessBuiltDistributionType& BuiltData, bool bClampAddress)
	{
		BuiltData[0] &= ~EFlags::AddressModeWrap;
		BuiltData[0] |= bClampAddress ? 0 : EFlags::AddressModeWrap;
	}

	static void SetBufferReadMode(FNiagaraStatelessBuiltDistributionType& BuiltData, bool bStaticBuffer)
	{
		BuiltData[0] &= ~EFlags::BufferReadModeStatic;
		BuiltData[0] |= bStaticBuffer ? EFlags::BufferReadModeStatic : 0;
	}

	static void SetLookupValueMode(FNiagaraStatelessBuiltDistributionType& BuiltData, uint8 ValueMode)
	{
		BuiltData[0] &= ~(EFlags::LookupValueModeMask << EFlags::LookupValueModeShift);
		if (ValueMode == 0xff)
		{
			BuiltData[0] |= EFlags::LookupValueModeMask << EFlags::LookupValueModeShift;
		}
		else
		{
			checkf(ValueMode <= EFlags::LookupValueModeMask - 1, TEXT("ValueMode (%d) is out of range (0 - %d)"), ValueMode, EFlags::LookupValueModeMask - 1);
			BuiltData[0] |= ValueMode << EFlags::LookupValueModeShift;
		}
	}

	static void SetUniformRandom(FNiagaraStatelessBuiltDistributionType& BuiltData, bool bEnabled)
	{
		BuiltData[0] = (BuiltData[0] & ~EFlags::UniformRandom) | (bEnabled ? EFlags::UniformRandom : 0);
	}

	static void SetTimeScaleBias(FNiagaraStatelessBuiltDistributionType& BuiltData, float TimeScale, float TimeBias)
	{
		reinterpret_cast<float&>(BuiltData[1]) = TimeBias;
		reinterpret_cast<float&>(BuiltData[2]) = FMath::Max(TimeScale, UE_KINDA_SMALL_NUMBER);
	}

	static void SetLookupParameters(FNiagaraStatelessBuiltDistributionType& BuiltData, uint BufferOffset, uint TableLength)
	{
		check(BufferOffset <= EFlags::BufferOffsetMask);
		check(TableLength > 0 && TableLength <= EFlags::TableLengthMask);

		BuiltData[0] &= ~(EFlags::BufferOffsetMask << EFlags::BufferOffsetShift);
		BuiltData[0] |= BufferOffset << EFlags::BufferOffsetShift;

		BuiltData[0] &= ~(EFlags::TableLengthMask << EFlags::TableLengthShift);
		BuiltData[0] |= (TableLength - 1) << EFlags::TableLengthShift;

		SetTimeScaleBias(BuiltData, float(TableLength - 1), 0.0f);
	}

	static void SetLookupParameters(FNiagaraStatelessBuiltDistributionType& BuiltData, uint BufferOffset, uint TableLength, float MinTime, float MaxTime)
	{
		check(TableLength > 0);

		SetLookupParameters(BuiltData, BufferOffset, TableLength);

		const uint TableLengthMinusOne = TableLength - 1;
		const float TimeDuration = MaxTime - MinTime;
		const float TimeScale = TableLengthMinusOne > 0 && TimeDuration > 0.0f ? float(TableLengthMinusOne) / TimeDuration : 0.0f;
		SetTimeScaleBias(BuiltData, TimeScale, MinTime);
	}
#endif
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#ifndef GPU_SIMULATION
	#undef uint
#endif
#undef UEAsFloat
#undef UEMin
#undef UEMax
#undef UEFrac
#undef UERoundToInt
#undef UEOutParam
