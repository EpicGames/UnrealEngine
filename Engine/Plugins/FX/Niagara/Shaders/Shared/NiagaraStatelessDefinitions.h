// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifndef GPU_SIMULATION
	#define uint	uint32
	#define uint4	FUintVector4
#endif

// Core random functionality
struct FNiagaraStatelessDefinitions
{
	static uint4 MakeRandomSeed(uint EmitterSeed, uint UniqueID, uint ModuleOffset, uint CallOffset) { return uint4(7123u + CallOffset, EmitterSeed, 3581u + UniqueID, 6405u + ModuleOffset); }
	static uint4 OffsetRandomSeedForCall(uint4 Seed, uint CallOffset) { Seed[0] += CallOffset; return Seed; }
};

#ifndef GPU_SIMULATION
	#undef uint
	#undef uint4
#endif
