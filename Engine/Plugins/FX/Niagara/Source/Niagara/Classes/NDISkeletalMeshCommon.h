// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/SkinnedAssetCommon.h"
#include "NiagaraDataInterfaceSkeletalMesh.h"

// These are to help readability of template specializations
using TNDISkelMesh_FilterModeNone = TIntegralConstant<ENDISkeletalMesh_FilterMode, ENDISkeletalMesh_FilterMode::None>;
using TNDISkelMesh_FilterModeSingle = TIntegralConstant<ENDISkeletalMesh_FilterMode, ENDISkeletalMesh_FilterMode::SingleRegion>;
using TNDISkelMesh_FilterModeMulti = TIntegralConstant<ENDISkeletalMesh_FilterMode, ENDISkeletalMesh_FilterMode::MultiRegion>;

using TNDISkelMesh_AreaWeightingOff = TIntegralConstant<ENDISkelMesh_AreaWeightingMode, ENDISkelMesh_AreaWeightingMode::None>;
using TNDISkelMesh_AreaWeightingOn = TIntegralConstant<ENDISkelMesh_AreaWeightingMode, ENDISkelMesh_AreaWeightingMode::AreaWeighted>;

using TNDISkelMesh_SkinningModeInvalid = TIntegralConstant<ENDISkeletalMesh_SkinningMode, ENDISkeletalMesh_SkinningMode::Invalid>;
using TNDISkelMesh_SkinningModeNone = TIntegralConstant<ENDISkeletalMesh_SkinningMode, ENDISkeletalMesh_SkinningMode::None>;
using TNDISkelMesh_SkinningModeOnTheFly = TIntegralConstant<ENDISkeletalMesh_SkinningMode, ENDISkeletalMesh_SkinningMode::SkinOnTheFly>;
using TNDISkelMesh_SkinningModePreSkin = TIntegralConstant<ENDISkeletalMesh_SkinningMode, ENDISkeletalMesh_SkinningMode::PreSkin>;

//////////////////////////////////////////////////////////////////////////
// Helper classes for reducing duplicate code when accessing vertex positions. 

struct FSkeletalMeshAccessorHelper
{
	FSkeletalMeshAccessorHelper() = default;
	~FSkeletalMeshAccessorHelper()
	{
		if (SkinningData != nullptr)
		{
			SkinningData->ExitRead();
		}
	}

	UE_NONCOPYABLE(FSkeletalMeshAccessorHelper);

	template<typename FilterMode, typename AreaWeightingMode>
	inline void Init(const FNDISkeletalMesh_InstanceData* InstData)
	{
		Comp = Cast<USkeletalMeshComponent>(InstData->WeakSceneComponent.Get());
		Mesh = Cast<USkeletalMesh>(InstData->WeakSkinnedMesh.Get());
		LODData = InstData->CachedLODData;
		SkinWeightBuffer = InstData->GetSkinWeights();
		IndexBuffer = LODData ? LODData->MultiSizeIndexContainer.GetIndexBuffer() : nullptr;
		SkinningData = InstData->SkinningData.SkinningData.Get();
		Usage = InstData->SkinningData.Usage;

		if (SkinningData != nullptr)
		{
			SkinningData->EnterRead();
		}
	}

	inline bool AreBonesAccessible() const
	{
		return Mesh != nullptr;
	}

	inline bool IsLODAccessible() const
	{
		return LODData != nullptr;
	}

	USkeletalMeshComponent* Comp = nullptr;
	USkeletalMesh* Mesh = nullptr;
	TWeakObjectPtr<USkeletalMesh> MeshSafe;
	const FSkeletalMeshLODRenderData* LODData = nullptr;
	const FSkinWeightVertexBuffer* SkinWeightBuffer = nullptr;
	const FRawStaticIndexBuffer16or32Interface* IndexBuffer = nullptr;
	const FSkeletalMeshSamplingRegion* SamplingRegion = nullptr;
	const FSkeletalMeshSamplingRegionBuiltData* SamplingRegionBuiltData = nullptr;
	FSkeletalMeshSkinningData* SkinningData = nullptr;
	FSkeletalMeshSkinningDataUsage Usage;
};

template<>
void FSkeletalMeshAccessorHelper::Init<TNDISkelMesh_FilterModeSingle, TNDISkelMesh_AreaWeightingOff>(const FNDISkeletalMesh_InstanceData* InstData);

template<>
void FSkeletalMeshAccessorHelper::Init<TNDISkelMesh_FilterModeSingle, TNDISkelMesh_AreaWeightingOn>(const FNDISkeletalMesh_InstanceData* InstData);

//////////////////////////////////////////////////////////////////////////

template<typename SkinningMode>
struct FSkinnedPositionAccessorHelper
{
	inline int32 GetBoneCount(FSkeletalMeshAccessorHelper& Accessor, bool RequiresPrevious) const = delete;
	inline void GetTriangleIndices(int32 Tri, int32& Idx0, int32& Idx1, int32& Idx2) = delete;
	inline void GetSkinnedTrianglePositions(FSkeletalMeshAccessorHelper& Accessor, int32 Tri, FVector3f& OutPos0, FVector3f& OutPos1, FVector3f& OutPos2, FVector3f& OutPrev0, FVector3f& OutPrev1, FVector3f& OutPrev2, int32& Idx0, int32& Idx1, int32& Idx2) = delete;
	inline void GetSkinnedTrianglePositions(FSkeletalMeshAccessorHelper& Accessor, int32 Tri, FVector3f& OutPos0, FVector3f& OutPos1, FVector3f& OutPos2, int32& Idx0, int32& Idx1, int32& Idx2) = delete;
	inline FVector3f GetSkinnedVertexPosition(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex) = delete;
	inline FVector3f GetSkinnedVertexPreviousPosition(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex) = delete;
	inline void GetSkinnedTangentBasis(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ) = delete;
	inline void GetSkinnedPreviousTangentBasis(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ) = delete;
	inline FVector3f GetSkinnedBonePosition(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex) = delete;
	inline FVector3f GetSkinnedBonePreviousPosition(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex) = delete;
	inline FQuat4f GetSkinnedBoneRotation(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex) = delete;
	inline FQuat4f GetSkinnedBonePreviousRotation(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex) = delete;
	inline FVector3f GetSkinnedBoneScale(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex) = delete;
	inline FVector3f GetSkinnedBonePreviousScale(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex) = delete;
};

template<>
struct FSkinnedPositionAccessorHelper<TNDISkelMesh_SkinningModeInvalid>
{
	inline int32 GetBoneCount(FSkeletalMeshAccessorHelper& Accessor, bool RequiresPrevious) const
	{
		if (const USkeletalMesh* Mesh = Accessor.Mesh)
		{
			return Mesh->GetRefSkeleton().GetNum();
		}

		return 0;
	}

	inline void GetTriangleIndices(FSkeletalMeshAccessorHelper& Accessor, int32 Tri, int32& Idx0, int32& Idx1, int32& Idx2)
	{
		Idx0 = Idx1 = Idx2 = -1;
	}

	inline void GetSkinnedTrianglePositions(FSkeletalMeshAccessorHelper& Accessor, int32 Idx0, int32 Idx1, int32 Idx2, FVector3f& OutPos0, FVector3f& OutPos1, FVector3f& OutPos2)
	{
		OutPos0 = OutPos1 = OutPos2 = FVector3f::ZeroVector;
	}

	inline void GetSkinnedTrianglePreviousPositions(FSkeletalMeshAccessorHelper& Accessor, int32 Idx0, int32 Idx1, int32 Idx2, FVector3f& OutPos0, FVector3f& OutPos1, FVector3f& OutPos2)
	{
		OutPos0 = OutPos1 = OutPos2 = FVector3f::ZeroVector;
	}

	inline FVector3f GetSkinnedVertexPosition(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex)
	{
		return FVector3f::ZeroVector;
	}

	inline FVector3f GetSkinnedVertexPreviousPosition(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex)
	{
		return FVector3f::ZeroVector;
	}

	inline void GetSkinnedTangentBasis(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
	{
		OutTangentX = FVector3f(1.0f, 0.0f, 0.0f);
		OutTangentY = FVector3f(0.0f, 1.0f, 0.0f);
		OutTangentZ = FVector3f(0.0f, 0.0f, 1.0f);
	}

	inline void GetSkinnedPreviousTangentBasis(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
	{
		OutTangentX = FVector3f(1.0f, 0.0f, 0.0f);
		OutTangentY = FVector3f(0.0f, 1.0f, 0.0f);
		OutTangentZ = FVector3f(0.0f, 0.0f, 1.0f);
	}

	// The bone accessor functions are valid if a mesh is present, so don't stub them entirely
	inline FVector3f GetSkinnedBonePosition(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		const int32 NumRealBones = Accessor.Mesh->GetRefSkeleton().GetRawBoneNum();
		if (BoneIndex < NumRealBones)
		{
			return (FVector3f)Accessor.Mesh->GetComposedRefPoseMatrix(BoneIndex).GetOrigin();
		}

		const FTransform& RefTransform = Accessor.Mesh->GetRefSkeleton().GetRefBonePose()[BoneIndex];
		return (FVector3f)RefTransform.GetLocation();
	}

	inline FVector3f GetSkinnedBonePreviousPosition(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return GetSkinnedBonePosition(Accessor, BoneIndex);
	}

	inline FQuat4f GetSkinnedBoneRotation(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		const int32 NumRealBones = Accessor.Mesh->GetRefSkeleton().GetRawBoneNum();
		if (BoneIndex < NumRealBones)
		{
			return (FQuat4f)Accessor.Mesh->GetComposedRefPoseMatrix(BoneIndex).GetMatrixWithoutScale().ToQuat();
		}

		const FTransform& RefTransform = Accessor.Mesh->GetRefSkeleton().GetRefBonePose()[BoneIndex];
		return (FQuat4f)RefTransform.GetRotation();
	}

	inline FQuat4f GetSkinnedBonePreviousRotation(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return GetSkinnedBoneRotation(Accessor, BoneIndex);
	}

	inline FVector3f GetSkinnedBoneScale(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		const int32 NumRealBones = Accessor.Mesh->GetRefSkeleton().GetRawBoneNum();
		if (BoneIndex < NumRealBones)
		{
			return (FVector3f)Accessor.Mesh->GetComposedRefPoseMatrix(BoneIndex).ExtractScaling();
		}

		const FTransform& RefTransform = Accessor.Mesh->GetRefSkeleton().GetRefBonePose()[BoneIndex];
		return (FVector3f)RefTransform.GetScale3D();
	}

	inline FVector3f GetSkinnedBonePreviousScale(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return GetSkinnedBoneScale(Accessor, BoneIndex);
	}
};

template<>
struct FSkinnedPositionAccessorHelper<TNDISkelMesh_SkinningModeNone>
{
	inline int32 GetBoneCount(FSkeletalMeshAccessorHelper& Accessor, bool RequiresPrevious) const
	{
		if (const USkeletalMesh* Mesh = Accessor.Mesh)
		{
			return Mesh->GetRefSkeleton().GetNum();
		}

		return 0;
	}

	inline void GetTriangleIndices(FSkeletalMeshAccessorHelper& Accessor, int32 Tri, int32& Idx0, int32& Idx1, int32& Idx2)
	{
		const int32 BaseIndex = Tri * 3;
		check(BaseIndex + 2 < Accessor.IndexBuffer->Num());
		Idx0 = Accessor.IndexBuffer->Get(BaseIndex);
		Idx1 = Accessor.IndexBuffer->Get(BaseIndex + 1);
		Idx2 = Accessor.IndexBuffer->Get(BaseIndex + 2);
	}

	inline void GetSkinnedTrianglePositions(FSkeletalMeshAccessorHelper& Accessor, int32 Idx0, int32 Idx1, int32 Idx2, FVector3f& OutPos0, FVector3f& OutPos1, FVector3f& OutPos2)
	{
		OutPos0 = (FVector3f)GetSkeletalMeshRefVertLocation(Accessor.Mesh, *Accessor.LODData, *Accessor.SkinWeightBuffer, Idx0);
		OutPos1 = (FVector3f)GetSkeletalMeshRefVertLocation(Accessor.Mesh, *Accessor.LODData, *Accessor.SkinWeightBuffer, Idx1);
		OutPos2 = (FVector3f)GetSkeletalMeshRefVertLocation(Accessor.Mesh, *Accessor.LODData, *Accessor.SkinWeightBuffer, Idx2);
	}

	inline void GetSkinnedTrianglePreviousPositions(FSkeletalMeshAccessorHelper& Accessor, int32 Idx0, int32 Idx1, int32 Idx2, FVector3f& OutPos0, FVector3f& OutPos1, FVector3f& OutPos2)
	{
		OutPos0 = (FVector3f)GetSkeletalMeshRefVertLocation(Accessor.Mesh, *Accessor.LODData, *Accessor.SkinWeightBuffer, Idx0);
		OutPos1 = (FVector3f)GetSkeletalMeshRefVertLocation(Accessor.Mesh, *Accessor.LODData, *Accessor.SkinWeightBuffer, Idx1);
		OutPos2 = (FVector3f)GetSkeletalMeshRefVertLocation(Accessor.Mesh, *Accessor.LODData, *Accessor.SkinWeightBuffer, Idx2);
	}

	inline FVector3f GetSkinnedVertexPosition(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex)
	{
		return (FVector3f)GetSkeletalMeshRefVertLocation(Accessor.Mesh, *Accessor.LODData, *Accessor.SkinWeightBuffer, VertexIndex);
	}

	inline FVector3f GetSkinnedVertexPreviousPosition(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex)
	{
		return (FVector3f)GetSkeletalMeshRefVertLocation(Accessor.Mesh, *Accessor.LODData, *Accessor.SkinWeightBuffer, VertexIndex);
	}

	inline void GetSkinnedTangentBasis(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
	{
		GetSkeletalMeshRefTangentBasis(Accessor.Mesh, *Accessor.LODData, *Accessor.SkinWeightBuffer, VertexIndex, OutTangentX, OutTangentY, OutTangentZ);
	}

	inline void GetSkinnedPreviousTangentBasis(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
	{
		GetSkeletalMeshRefTangentBasis(Accessor.Mesh, *Accessor.LODData, *Accessor.SkinWeightBuffer, VertexIndex, OutTangentX, OutTangentY, OutTangentZ);
	}

	inline FVector3f GetSkinnedBonePosition(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		const int32 NumRealBones = Accessor.Mesh->GetRefSkeleton().GetRawBoneNum();
		if (BoneIndex < NumRealBones)
		{
			return (FVector3f)Accessor.Mesh->GetComposedRefPoseMatrix(BoneIndex).GetOrigin();
		}

		const FTransform& RefTransform = Accessor.Mesh->GetRefSkeleton().GetRefBonePose()[BoneIndex];
		return (FVector3f)RefTransform.GetLocation();
	}

	inline FVector3f GetSkinnedBonePreviousPosition(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return GetSkinnedBonePosition(Accessor, BoneIndex);
	}

	inline FQuat4f GetSkinnedBoneRotation(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		const int32 NumRealBones = Accessor.Mesh->GetRefSkeleton().GetRawBoneNum();
		if (BoneIndex < NumRealBones)
		{
			return (FQuat4f)Accessor.Mesh->GetComposedRefPoseMatrix(BoneIndex).GetMatrixWithoutScale().ToQuat();
		}

		const FTransform& RefTransform = Accessor.Mesh->GetRefSkeleton().GetRefBonePose()[BoneIndex];
		return (FQuat4f)RefTransform.GetRotation();
	}

	inline FQuat4f GetSkinnedBonePreviousRotation(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return GetSkinnedBoneRotation(Accessor, BoneIndex);
	}

	inline FVector3f GetSkinnedBoneScale(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		const int32 NumRealBones = Accessor.Mesh->GetRefSkeleton().GetRawBoneNum();
		if (BoneIndex < NumRealBones)
		{
			return (FVector3f)Accessor.Mesh->GetComposedRefPoseMatrix(BoneIndex).ExtractScaling();
		}

		const FTransform& RefTransform = Accessor.Mesh->GetRefSkeleton().GetRefBonePose()[BoneIndex];
		return (FVector3f)RefTransform.GetScale3D();
	}

	inline FVector3f GetSkinnedBonePreviousScale(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return GetSkinnedBoneScale(Accessor, BoneIndex);
	}
};

template<>
struct FSkinnedPositionAccessorHelper<TNDISkelMesh_SkinningModeOnTheFly>
{
	inline int32 GetBoneCount(FSkeletalMeshAccessorHelper& Accessor, bool RequiresPrevious) const
	{
		if (const FSkeletalMeshSkinningData* SkinningData = Accessor.SkinningData)
		{
			return SkinningData->GetBoneCount(RequiresPrevious);
		}

		return 0;
	}

	inline void GetTriangleIndices(FSkeletalMeshAccessorHelper& Accessor, int32 Tri, int32& Idx0, int32& Idx1, int32& Idx2)
	{
		const int32 BaseIndex = Tri * 3;
		check(BaseIndex + 2 < Accessor.IndexBuffer->Num());
		Idx0 = Accessor.IndexBuffer->Get(BaseIndex);
		Idx1 = Accessor.IndexBuffer->Get(BaseIndex + 1);
		Idx2 = Accessor.IndexBuffer->Get(BaseIndex + 2);
	}

	inline void GetSkinnedTrianglePositions(FSkeletalMeshAccessorHelper& Accessor, int32 Idx0, int32 Idx1, int32 Idx2, FVector3f& OutPos0, FVector3f& OutPos1, FVector3f& OutPos2)
	{
		OutPos0 = USkeletalMeshComponent::GetSkinnedVertexPosition(Accessor.Comp, Idx0, *Accessor.LODData, *Accessor.SkinWeightBuffer, Accessor.SkinningData->CurrBoneRefToLocals());
		OutPos1 = USkeletalMeshComponent::GetSkinnedVertexPosition(Accessor.Comp, Idx1, *Accessor.LODData, *Accessor.SkinWeightBuffer, Accessor.SkinningData->CurrBoneRefToLocals());
		OutPos2 = USkeletalMeshComponent::GetSkinnedVertexPosition(Accessor.Comp, Idx2, *Accessor.LODData, *Accessor.SkinWeightBuffer, Accessor.SkinningData->CurrBoneRefToLocals());
	}

	inline void GetSkinnedTrianglePreviousPositions(FSkeletalMeshAccessorHelper& Accessor, int32 Idx0, int32 Idx1, int32 Idx2, FVector3f& OutPos0, FVector3f& OutPos1, FVector3f& OutPos2)
	{
		OutPos0 = USkeletalMeshComponent::GetSkinnedVertexPosition(Accessor.Comp, Idx0, *Accessor.LODData, *Accessor.SkinWeightBuffer, Accessor.SkinningData->PrevBoneRefToLocals());
		OutPos1 = USkeletalMeshComponent::GetSkinnedVertexPosition(Accessor.Comp, Idx1, *Accessor.LODData, *Accessor.SkinWeightBuffer, Accessor.SkinningData->PrevBoneRefToLocals());
		OutPos2 = USkeletalMeshComponent::GetSkinnedVertexPosition(Accessor.Comp, Idx2, *Accessor.LODData, *Accessor.SkinWeightBuffer, Accessor.SkinningData->PrevBoneRefToLocals());
	}

	inline FVector3f GetSkinnedVertexPosition(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex)
	{
		return USkeletalMeshComponent::GetSkinnedVertexPosition(Accessor.Comp, VertexIndex, *Accessor.LODData, *Accessor.SkinWeightBuffer, Accessor.SkinningData->CurrBoneRefToLocals());
	}

	inline FVector3f GetSkinnedVertexPreviousPosition(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex)
	{
		return USkeletalMeshComponent::GetSkinnedVertexPosition(Accessor.Comp, VertexIndex, *Accessor.LODData, *Accessor.SkinWeightBuffer, Accessor.SkinningData->PrevBoneRefToLocals());
	}

	inline void GetSkinnedTangentBasis(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
	{
		USkeletalMeshComponent::GetSkinnedTangentBasis(Accessor.Comp, VertexIndex, *Accessor.LODData, *Accessor.SkinWeightBuffer, Accessor.SkinningData->CurrBoneRefToLocals(), OutTangentX, OutTangentY, OutTangentZ);
	}

	inline void GetSkinnedPreviousTangentBasis(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
	{
		USkeletalMeshComponent::GetSkinnedTangentBasis(Accessor.Comp, VertexIndex, *Accessor.LODData, *Accessor.SkinWeightBuffer, Accessor.SkinningData->PrevBoneRefToLocals(), OutTangentX, OutTangentY, OutTangentZ);
	}

	inline FVector3f GetSkinnedBonePosition(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return (FVector3f)Accessor.SkinningData->CurrComponentTransforms()[BoneIndex].GetLocation();
	}

	inline FVector3f GetSkinnedBonePreviousPosition(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return (FVector3f)Accessor.SkinningData->PrevComponentTransforms()[BoneIndex].GetLocation();
	}

	inline FQuat4f GetSkinnedBoneRotation(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return (FQuat4f)Accessor.SkinningData->CurrComponentTransforms()[BoneIndex].GetRotation();
	}

	inline FQuat4f GetSkinnedBonePreviousRotation(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return (FQuat4f)Accessor.SkinningData->PrevComponentTransforms()[BoneIndex].GetRotation();
	}

	inline FVector3f GetSkinnedBoneScale(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return (FVector3f)Accessor.SkinningData->CurrComponentTransforms()[BoneIndex].GetScale3D();
	}

	inline FVector3f GetSkinnedBonePreviousScale(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return (FVector3f)Accessor.SkinningData->PrevComponentTransforms()[BoneIndex].GetScale3D();
	}
};

template<>
struct FSkinnedPositionAccessorHelper<TNDISkelMesh_SkinningModePreSkin>
{
	inline int32 GetBoneCount(FSkeletalMeshAccessorHelper& Accessor, bool RequiresPrevious) const
	{
		if (const FSkeletalMeshSkinningData* SkinningData = Accessor.SkinningData)
		{
			return SkinningData->GetBoneCount(RequiresPrevious);
		}

		return 0;
	}

	inline void GetTriangleIndices(FSkeletalMeshAccessorHelper& Accessor, int32 Tri, int32& Idx0, int32& Idx1, int32& Idx2)
	{
		const int32 BaseIndex = Tri * 3;
		check(BaseIndex + 2 < Accessor.IndexBuffer->Num());
		Idx0 = Accessor.IndexBuffer->Get(BaseIndex);
		Idx1 = Accessor.IndexBuffer->Get(BaseIndex + 1);
		Idx2 = Accessor.IndexBuffer->Get(BaseIndex + 2);
	}

	inline void GetSkinnedTrianglePositions(FSkeletalMeshAccessorHelper& Accessor, int32 Idx0, int32 Idx1, int32 Idx2, FVector3f& OutPos0, FVector3f& OutPos1, FVector3f& OutPos2)
	{
		OutPos0 = Accessor.SkinningData->GetPosition(Accessor.Usage.GetLODIndex(), Idx0);
		OutPos1 = Accessor.SkinningData->GetPosition(Accessor.Usage.GetLODIndex(), Idx1);
		OutPos2 = Accessor.SkinningData->GetPosition(Accessor.Usage.GetLODIndex(), Idx2);
	}

	inline void GetSkinnedTrianglePreviousPositions(FSkeletalMeshAccessorHelper& Accessor, int32 Idx0, int32 Idx1, int32 Idx2, FVector3f& OutPos0, FVector3f& OutPos1, FVector3f& OutPos2)
	{
		OutPos0 = Accessor.SkinningData->GetPreviousPosition(Accessor.Usage.GetLODIndex(), Idx0);
		OutPos1 = Accessor.SkinningData->GetPreviousPosition(Accessor.Usage.GetLODIndex(), Idx1);
		OutPos2 = Accessor.SkinningData->GetPreviousPosition(Accessor.Usage.GetLODIndex(), Idx2);
	}

	inline FVector3f GetSkinnedVertexPosition(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex)
	{
		return Accessor.SkinningData->GetPosition(Accessor.Usage.GetLODIndex(), VertexIndex);
	}

	inline FVector3f GetSkinnedVertexPreviousPosition(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex)
	{
		return Accessor.SkinningData->GetPreviousPosition(Accessor.Usage.GetLODIndex(), VertexIndex);
	}

	inline void GetSkinnedTangentBasis(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
	{
		Accessor.SkinningData->GetTangentBasis(Accessor.Usage.GetLODIndex(), VertexIndex, OutTangentX, OutTangentY, OutTangentZ);
	}

	inline void GetSkinnedPreviousTangentBasis(FSkeletalMeshAccessorHelper& Accessor, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
	{
		Accessor.SkinningData->GetPreviousTangentBasis(Accessor.Usage.GetLODIndex(), VertexIndex, OutTangentX, OutTangentY, OutTangentZ);
	}

	inline FVector3f GetSkinnedBonePosition(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return Accessor.SkinningData->CurrComponentTransforms()[BoneIndex].GetLocation();
	}

	inline FVector3f GetSkinnedBonePreviousPosition(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return Accessor.SkinningData->PrevComponentTransforms()[BoneIndex].GetLocation();
	}

	inline FQuat4f GetSkinnedBoneRotation(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return (FQuat4f)Accessor.SkinningData->CurrComponentTransforms()[BoneIndex].GetRotation();
	}

	inline FQuat4f GetSkinnedBonePreviousRotation(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return (FQuat4f)Accessor.SkinningData->PrevComponentTransforms()[BoneIndex].GetRotation();
	}

	inline FVector3f GetSkinnedBoneScale(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return (FVector3f)Accessor.SkinningData->CurrComponentTransforms()[BoneIndex].GetScale3D();
	}

	inline FVector3f GetSkinnedBonePreviousScale(FSkeletalMeshAccessorHelper& Accessor, int32 BoneIndex)
	{
		return (FVector3f)Accessor.SkinningData->PrevComponentTransforms()[BoneIndex].GetScale3D();
	}
};

//////////////////////////////////////////////////////////////////////////
// Helpers for accessing misc vertex data
template<bool bUseFullPrecisionUVs>
struct FSkelMeshVertexAccessor
{
	inline FVector2D GetVertexUV(const FSkeletalMeshLODRenderData* LODData, int32 VertexIdx, int32 UVChannel)const
	{
		check(LODData);
		if (bUseFullPrecisionUVs)
		{
			return FVector2D(LODData->StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV_Typed<EStaticMeshVertexUVType::HighPrecision>(VertexIdx, UVChannel));
		}
		else
		{
			return FVector2D(LODData->StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV_Typed<EStaticMeshVertexUVType::Default>(VertexIdx, UVChannel));
		}
	}

	inline FLinearColor GetVertexColor(FSkeletalMeshLODRenderData* LODData, int32 VertexIdx)const
	{
		check(LODData);
		return LODData->StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIdx);
	}
};

struct FSkelMeshVertexAccessorNoop
{
	inline FVector2D GetVertexUV(const FSkeletalMeshLODRenderData* LODData, int32 VertexIdx, int32 UVChannel)const
	{
		return FVector2D(0.0f, 0.0f);
	}

	inline FLinearColor GetVertexColor(const FSkeletalMeshLODRenderData* LODData, int32 VertexIdx)const
	{
		return FLinearColor::White;
	}
};

//////////////////////////////////////////////////////////////////////////
//Function Binders.

//External function binder choosing between template specializations based on if we're area weighting or not.
template<typename NextBinder>
struct TAreaWeightingModeBinder
{
	template<typename... ParamTypes>
	static void Bind(UNiagaraDataInterface* Interface, const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
	{
		FNDISkeletalMesh_InstanceData* InstData = (FNDISkeletalMesh_InstanceData*)InstanceData;
		check(InstData);

		UNiagaraDataInterfaceSkeletalMesh* MeshInterface = CastChecked<UNiagaraDataInterfaceSkeletalMesh>(Interface);

		bool bAreaWeighting = false;
		if (InstData->bAllowCPUMeshDataAccess)
		{
			if (const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(InstData->WeakSkinnedMesh.Get()))
			{
				if (InstData->SamplingRegionIndices.Num() > 1)
				{
					if (InstData->SamplingRegionAreaWeightedSampler.IsValid())
					{
						bAreaWeighting = true;

						// check that all the regions have sampling data
						const FSkeletalMeshSamplingInfo& SamplingInfo = SkeletalMesh->GetSamplingInfo();

						for (int32 RegionIndex : InstData->SamplingRegionIndices)
						{
							const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(RegionIndex);
							const FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = SamplingInfo.GetRegionBuiltData(RegionIndex);

							if (RegionBuiltData.AreaWeightedSampler.GetNumEntries() == 0)
							{
								bAreaWeighting = false;
							}
						}
					}
				}
				else if (InstData->SamplingRegionIndices.Num() == 1)
				{
					const int32 RegionIndex = InstData->SamplingRegionIndices[0];

					const FSkeletalMeshSamplingInfo& SamplingInfo = SkeletalMesh->GetSamplingInfo();
					const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(RegionIndex);

					bAreaWeighting = Region.bSupportUniformlyDistributedSampling
						&& (SamplingInfo.GetRegionBuiltData(RegionIndex).AreaWeightedSampler.GetNumEntries() > 0);
				}
				else
				{
					const int32 LODIndex = InstData->GetLODIndex();

					const FSkeletalMeshSamplingInfo& SamplingInfo = SkeletalMesh->GetSamplingInfo();
					const FSkeletalMeshSamplingBuiltData& SamplingBuildData = SamplingInfo.GetBuiltData();

					bAreaWeighting =
						SamplingBuildData.WholeMeshBuiltData.IsValidIndex(LODIndex) &&
						(SamplingBuildData.WholeMeshBuiltData[LODIndex].AreaWeightedTriangleSampler.GetNumEntries() > 0) &&
						SkeletalMesh->GetLODInfo(LODIndex)->bSupportUniformlyDistributedSampling;
				}
			}
		}

		if (bAreaWeighting)
		{
			NextBinder::template Bind<ParamTypes..., TNDISkelMesh_AreaWeightingOn>(Interface, BindingInfo, InstanceData, OutFunc);
		}
		else
		{
			NextBinder::template Bind<ParamTypes..., TNDISkelMesh_AreaWeightingOff>(Interface, BindingInfo, InstanceData, OutFunc);
		}
	}
};

//External function binder choosing between template specializations based on filtering methods
template<typename NextBinder>
struct TFilterModeBinder
{
	template<typename... ParamTypes>
	static void Bind(UNiagaraDataInterface* Interface, const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
	{
		FNDISkeletalMesh_InstanceData* InstData = (FNDISkeletalMesh_InstanceData*)InstanceData;
		check(InstData);

		if (InstData->SamplingRegionIndices.Num() == 1)
		{
			NextBinder::template Bind<ParamTypes..., TNDISkelMesh_FilterModeSingle>(Interface, BindingInfo, InstanceData, OutFunc);
		}
		else if (InstData->SamplingRegionIndices.Num() > 1)
		{
			NextBinder::template Bind<ParamTypes..., TNDISkelMesh_FilterModeMulti>(Interface, BindingInfo, InstanceData, OutFunc);
		}
		else
		{
			NextBinder::template Bind<ParamTypes..., TNDISkelMesh_FilterModeNone>(Interface, BindingInfo, InstanceData, OutFunc);
		}
	}
};

//External function binder choosing between template specializations based vertex data format
template<typename NextBinder>
struct TVertexAccessorBinder
{
	template<typename... ParamTypes>
	static void Bind(UNiagaraDataInterface* Interface, const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
	{
		FNDISkeletalMesh_InstanceData* InstData = (FNDISkeletalMesh_InstanceData*)InstanceData;
		check(InstData);

		if (InstData->bAllowCPUMeshDataAccess)
		{
			const FSkeletalMeshLODRenderData* LODData = InstData->CachedLODData;
			if (LODData->StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs())
			{
				NextBinder::template Bind<ParamTypes..., FSkelMeshVertexAccessor<true>>(Interface, BindingInfo, InstanceData, OutFunc);
			}
			else
			{
				NextBinder::template Bind<ParamTypes..., FSkelMeshVertexAccessor<false>>(Interface, BindingInfo, InstanceData, OutFunc);
			}
		}
		else
		{
			NextBinder::template Bind<ParamTypes..., FSkelMeshVertexAccessorNoop>(Interface, BindingInfo, InstanceData, OutFunc);
		}
	}
};

//External function binder choosing between template specializations based on skinning mode
template<typename NextBinder>
struct TSkinningModeBinder
{
	template<typename... ParamTypes>
	static void BindIgnoreCPUAccess(UNiagaraDataInterface* Interface, const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
	{
		FNDISkeletalMesh_InstanceData* InstData = (FNDISkeletalMesh_InstanceData*)InstanceData;
		UNiagaraDataInterfaceSkeletalMesh* MeshInterface = CastChecked<UNiagaraDataInterfaceSkeletalMesh>(Interface);
		USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(InstData->WeakSceneComponent.Get());
		if (MeshInterface->SkinningMode == ENDISkeletalMesh_SkinningMode::None || !Component) // Can't skin if we have no component.
		{
			NextBinder::template Bind<ParamTypes..., FSkinnedPositionAccessorHelper<TNDISkelMesh_SkinningModeNone>>(Interface, BindingInfo, InstanceData, OutFunc);
		}
		else if (MeshInterface->SkinningMode == ENDISkeletalMesh_SkinningMode::SkinOnTheFly)
		{
			check(Component);
			NextBinder::template Bind<ParamTypes..., FSkinnedPositionAccessorHelper<TNDISkelMesh_SkinningModeOnTheFly>>(Interface, BindingInfo, InstanceData, OutFunc);
		}
		else if (MeshInterface->SkinningMode == ENDISkeletalMesh_SkinningMode::PreSkin)
		{
			check(Component);
			NextBinder::template Bind<ParamTypes..., FSkinnedPositionAccessorHelper<TNDISkelMesh_SkinningModePreSkin>>(Interface, BindingInfo, InstanceData, OutFunc);
		}
		else
		{
			checkf(false, TEXT("Invalid skinning mode in %s"), *Interface->GetPathName());
		}
	}

	template<typename... ParamTypes>
	static void BindCheckCPUAccess(UNiagaraDataInterface* Interface, const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
	{
		FNDISkeletalMesh_InstanceData* InstData = (FNDISkeletalMesh_InstanceData*)InstanceData;
		if (!InstData->bAllowCPUMeshDataAccess) // No-op when we can't access the mesh on CPU
		{
			NextBinder::template Bind<ParamTypes..., FSkinnedPositionAccessorHelper<TNDISkelMesh_SkinningModeInvalid>>(Interface, BindingInfo, InstanceData, OutFunc);
		}
		else
		{
			BindIgnoreCPUAccess(Interface, BindingInfo, InstanceData, OutFunc);
		}
	}
};
