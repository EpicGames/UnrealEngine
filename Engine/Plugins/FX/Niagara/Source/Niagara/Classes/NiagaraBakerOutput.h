// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/PathViews.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"

#include "NiagaraBakerOutput.generated.h"

USTRUCT()
struct FNiagaraBakerTextureSource
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Source")
	FString DisplayString;

	UPROPERTY(EditAnywhere, Category = "Source")
	FName SourceName;
};

UCLASS(Abstract, MinimalAPI)
class UNiagaraBakerOutput : public UObject
{
	GENERATED_BODY()
public:
	/** Optional output name, useful when you have multiple outputs */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = "0"))
	FString OutputName;

	/** Used to determine if the bake is out of date or not. */
	virtual bool Equals(const UNiagaraBakerOutput& Other) const { return OutputName == Other.OutputName; }

	/** Make an OutputName used when creating a new output */
	NIAGARA_API virtual FString MakeOutputName() const;

	static NIAGARA_API FString SanitizeOutputName(FString Name);

#if WITH_EDITOR
	/** Find any warnings about the output. */
	NIAGARA_API virtual void FindWarnings(TArray<FText>& OutWarnings) const;

	NIAGARA_API FString GetAssetPath(FString PathFormat, int32 FrameIndex) const;
	NIAGARA_API FString GetAssetFolder(FString PathFormat, int32 FrameIndex) const;
	NIAGARA_API FString GetExportPath(FString PathFormat, int32 FrameIndex) const;
	NIAGARA_API FString GetExportFolder(FString PathFormat, int32 FrameIndex) const;

	template<class TObjectClass>
	TObjectClass* GetAsset(FString PathFormat, int32 FrameIndex)
	{
		const FString AssetFullName = GetAssetPath(PathFormat, FrameIndex);
		const FString AssetName = FString(FPathViews::GetCleanFilename(AssetFullName));
		const FString AssetPath = AssetFullName + "." + AssetName;
		return TSoftObjectPtr<TObjectClass>(FSoftObjectPath(AssetPath)).LoadSynchronous();
	}

	static NIAGARA_API UObject* GetOrCreateAsset(const FString& PackagePath, UClass* ObjectClass, UClass* FactoryClass);

	template<typename TObjectClass, typename TFactoryClass>
	static TObjectClass* GetOrCreateAsset(const FString& PackagePath)
	{
		UObject* FoundObject = GetOrCreateAsset(PackagePath, TObjectClass::StaticClass(), TFactoryClass::StaticClass());
		return FoundObject ? CastChecked<TObjectClass>(FoundObject) : nullptr;
	}
#endif

	NIAGARA_API virtual void PostInitProperties() override;
};
