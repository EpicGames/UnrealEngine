// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraBakerOutput.h"
#include "NiagaraSimCache.h"
#include "NiagaraBakerOutputSimCache.generated.h"

UCLASS(meta = (DisplayName = "Simulation Cache Output"), MinimalAPI)
class UNiagaraBakerOutputSimCache : public UNiagaraBakerOutput
{
	GENERATED_BODY()

public:
	UNiagaraBakerOutputSimCache(const FObjectInitializer& Init)
	{
	}

	UPROPERTY(EditAnywhere, Category = "Settings")
	FString SimCacheAssetPathFormat = TEXT("{AssetFolder}/{AssetName}_SimCache_{OutputName}");

	UPROPERTY(EditAnywhere, Category = "Settings")
	FNiagaraSimCacheCreateParameters CreateParameters;

	NIAGARA_API virtual bool Equals(const UNiagaraBakerOutput& Other) const override;

#if WITH_EDITOR
	NIAGARA_API FString MakeOutputName() const override;
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
