// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraBakerOutput.h"
#include "NiagaraParameterBinding.h"
#include "NiagaraBakerOutputSparseVolumeTexture.generated.h"

UCLASS(meta = (DisplayName = "SparseVolume Texture Output"), MinimalAPI)
class UNiagaraBakerOutputSparseVolumeTexture : public UNiagaraBakerOutput
{
	GENERATED_BODY()

public:
	UNiagaraBakerOutputSparseVolumeTexture(const FObjectInitializer& Init)	
	{
#if WITH_EDITORONLY_DATA
		VolumeWorldSpaceSizeBinding.SetUsage(ENiagaraParameterBindingUsage::User);
		VolumeWorldSpaceSizeBinding.SetAllowedTypeDefinitions( { FNiagaraTypeDefinition::GetVec3Def() } );
#endif
	}

	UPROPERTY(EditAnywhere, Category = "Settings")
	FNiagaraBakerTextureSource SourceBinding;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FNiagaraParameterBinding VolumeWorldSpaceSizeBinding;

	/**
	When enabled a volume atlas is created, the atlas is along X & Y not Z based on baker settings.
	*/
	UPROPERTY(EditAnywhere, Category = "Settings")
	FString SparseVolumeTextureAssetPathFormat = TEXT("{AssetFolder}/{AssetName}_BakedSVT_{OutputName}");

	/**
	Enable outputting a seamlessly blended looped SVT sequence
	*/
	UPROPERTY(EditAnywhere, Category = "Looping")
	bool bEnableLoopedOutput = false;
	
	/**
	Path for the looped SVT
	*/
	UPROPERTY(EditAnywhere, Category = "Looping", meta = (EditCondition = "bEnableLoopedOutput", EditConditionHides))
	FString LoopedSparseVolumeTextureAssetPathFormat = TEXT("{AssetFolder}/{AssetName}_Looped_BakedSVT_{OutputName}");

	/**
	Time in seconds to start the looped output from
	*/
	UPROPERTY(EditAnywhere, Category = "Looping", meta = (EditCondition = "bEnableLoopedOutput", EditConditionHides))
	float StartTime = 4.0f;

	/**
	Number of seconds to blend the output for
	*/
	UPROPERTY(EditAnywhere, Category = "Looping", meta = (EditCondition = "bEnableLoopedOutput", EditConditionHides))
	float BlendDuration = 2.0f;

	NIAGARA_API virtual bool Equals(const UNiagaraBakerOutput& Other) const override;

#if WITH_EDITOR
	NIAGARA_API FString MakeOutputName() const override;
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
