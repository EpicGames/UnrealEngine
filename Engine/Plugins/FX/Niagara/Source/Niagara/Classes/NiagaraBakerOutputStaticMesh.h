// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraBakerOutput.h"
#include "NiagaraRendererReadback.h"
#include "NiagaraBakerOutputStaticMesh.generated.h"

UCLASS(meta = (DisplayName = "Static Mesh Output"), MinimalAPI)
class UNiagaraBakerOutputStaticMesh : public UNiagaraBakerOutput
{
	GENERATED_BODY()

public:
	UNiagaraBakerOutputStaticMesh(const FObjectInitializer& Init)
	{
	}

	UPROPERTY(EditAnywhere, Category = "Settings")
	FString FramesAssetPathFormat = TEXT("{AssetFolder}/{AssetName}_BakedFrame_{OutputName}_{FrameIndex}");

	UPROPERTY(EditAnywhere, Category = "Settings")
	FNiagaraRendererReadbackParameters ExportParameters;

	NIAGARA_API virtual bool Equals(const UNiagaraBakerOutput& Other) const override;

#if WITH_EDITOR
	NIAGARA_API FString MakeOutputName() const override;
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
