// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture.h"
#include "NiagaraBakerOutput.h"
#include "NiagaraBakerOutputTexture2D.generated.h"

UCLASS(meta = (DisplayName = "Texture 2D Output"), MinimalAPI)
class UNiagaraBakerOutputTexture2D : public UNiagaraBakerOutput
{
	GENERATED_BODY()

public:
	UNiagaraBakerOutputTexture2D(const FObjectInitializer& Init)
		: bGenerateAtlas(true)
		, bSetTextureAddressX(true)
		, bSetTextureAddressY(true)
	{
	}

	/** Source visualization we should capture, i.e. Scene Color, World Normal, etc */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FNiagaraBakerTextureSource SourceBinding;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bGenerateAtlas : 1;	

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bGenerateFrames : 1;	

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bExportFrames : 1;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bSetTextureAddressX : 1;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bSetTextureAddressY : 1;

	/** Size of each frame we generate. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FIntPoint FrameSize = FIntPoint(128, 128);

	/** Size of the atlas texture when generating an atlas. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition="bGenerateAtlas"))
	FIntPoint AtlasTextureSize = FIntPoint(128 * 8, 128 * 8);

	/** When either exporting or generating individual frames this is the size of the texture. */
	FIntPoint SequenceFrameSize = FIntPoint(128, 128);

	/** After baking sets the texture address mode to use on the X axis. */
	UPROPERTY(EditAnywhere, Category="Settings", meta = (EditCondition = "bSetTextureAddressX"))
	TEnumAsByte<enum TextureAddress> TextureAddressX = TextureAddress::TA_Wrap;

	/** After baking sets the texture address mode to use on the Y axis. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bSetTextureAddressY"))
	TEnumAsByte<enum TextureAddress> TextureAddressY = TextureAddress::TA_Wrap;

	/**
	When enabled a texture atlas is created
	*/
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bGenerateAtlas"))
	FString AtlasAssetPathFormat = TEXT("{AssetFolder}/{AssetName}_BakedAtlas_{OutputName}");

	/**
	When enabled each frame will create an asset.
	*/
	UPROPERTY(EditAnywhere, Category="Settings", meta=(EditCondition="bGenerateFrames"))
	FString FramesAssetPathFormat = TEXT("{AssetFolder}/{AssetName}_BakedFrame_{OutputName}_{FrameIndex}");

	/**
	When enabled each frame will be exported to the output path using the format extension.
	*/
	UPROPERTY(EditAnywhere, Category="Settings", meta=(EditCondition="bExportFrames"))
	FString FramesExportPathFormat = TEXT("{SavedDir}/NiagaraBaker/{AssetName}_{OutputName}/Frame_{FrameIndex}.png");

	NIAGARA_API virtual bool Equals(const UNiagaraBakerOutput& Other) const override;

	NIAGARA_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	NIAGARA_API virtual FString MakeOutputName() const override;
	NIAGARA_API virtual void FindWarnings(TArray<FText>& OutWarnings) const override;
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
