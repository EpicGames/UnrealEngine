// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraBakerOutput.h"
#include "NiagaraBakerOutputVolumeTexture.generated.h"

UCLASS(meta = (DisplayName = "Volume Texture Output"), MinimalAPI)
class UNiagaraBakerOutputVolumeTexture : public UNiagaraBakerOutput
{
	GENERATED_BODY()

public:
	UNiagaraBakerOutputVolumeTexture(const FObjectInitializer& Init)
		: bGenerateAtlas(true)
	{
	}

	UPROPERTY(EditAnywhere, Category = "Settings")
	FNiagaraBakerTextureSource SourceBinding;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bGenerateAtlas : 1;	

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bGenerateFrames : 1;	

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bExportFrames : 1;

	/**
	When enabled a volume atlas is created, the atlas is along X & Y not Z based on baker settings.
	*/
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bGenerateAtlas"))
	FString AtlasAssetPathFormat = TEXT("{AssetFolder}/{AssetName}_BakedAtlas_{OutputName}");

	/**
	When enabled each frame will create an asset.
	*/
	UPROPERTY(EditAnywhere, Category="Settings", meta=(EditCondition="bGenerateFrames"))
	FString FramesAssetPathFormat = TEXT("{AssetFolder}/{AssetName}_BakedFrame_{OutputName}_{FrameIndex}");

	/**
	When enabled each frame will be exported to the output path using the format extension.
	*/
	UPROPERTY(EditAnywhere, Category="Settings", meta=(EditCondition="bExportFrames"))
	FString FramesExportPathFormat = TEXT("{SavedDir}/NiagaraBaker/{AssetName}_{OutputName}/Frame_{FrameIndex}.png");

	NIAGARA_API virtual bool Equals(const UNiagaraBakerOutput& Other) const override;

#if WITH_EDITOR
	NIAGARA_API FString MakeOutputName() const override;
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
