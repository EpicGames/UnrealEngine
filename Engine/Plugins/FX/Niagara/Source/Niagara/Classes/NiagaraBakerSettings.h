// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Camera/CameraTypes.h"
#include "Misc/PathViews.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"

#include "NiagaraBakerOutput.h"
#include "NiagaraBakerSettings.generated.h"

UENUM()
enum class ENiagaraBakerViewMode
{
	Perspective,
	OrthoFront,
	OrthoBack,
	OrthoLeft,
	OrthoRight,
	OrthoTop,
	OrthoBottom,
	Num UMETA(Hidden)
};

USTRUCT()
struct FNiagaraBakerCameraSettings
{
	GENERATED_BODY()

	UPROPERTY()
	ENiagaraBakerViewMode ViewMode = ENiagaraBakerViewMode::Perspective;

	UPROPERTY()
	FVector ViewportLocation = FVector::ZeroVector;

	UPROPERTY()
	FRotator ViewportRotation = FRotator::ZeroRotator;

	UPROPERTY()
	float OrbitDistance = 200.f;

	UPROPERTY()
	float FOV = 90.0f;

	UPROPERTY()
	float OrthoWidth = 512.0f;

	UPROPERTY()
	bool bUseAspectRatio = false;

	UPROPERTY()
	float AspectRatio = 1.0f;

	bool IsOrthographic() const { return ViewMode != ENiagaraBakerViewMode::Perspective; }
	bool IsPerspective() const { return ViewMode == ENiagaraBakerViewMode::Perspective; }

	void ResetToDefault()
	{
		if (IsPerspective())
		{
			ViewportLocation = FVector(0.0f, -200.0f, 0.0f);
			ViewportRotation = FRotator(180.0f, 0.0f, 90.0f);
		}
		else
		{
			ViewportLocation = FVector::ZeroVector;
			ViewportRotation = FRotator::ZeroRotator;
		}
		OrbitDistance = 200.0f;
		FOV = 90.0f;
		OrthoWidth = 512.0f;
		bUseAspectRatio = false;
		AspectRatio = 1.0f;
	}

	bool operator==(const FNiagaraBakerCameraSettings& Other) const
	{
		return
			ViewMode == Other.ViewMode &&
			ViewportLocation.Equals(Other.ViewportLocation) &&
			ViewportRotation.Equals(Other.ViewportRotation) &&
			OrbitDistance == Other.OrbitDistance &&
			FOV == Other.FOV &&
			OrthoWidth == Other.OrthoWidth &&
			bUseAspectRatio == Other.bUseAspectRatio &&
			AspectRatio == Other.AspectRatio;
	}
};

USTRUCT()
struct FNiagaraBakerTextureSettings
{
	GENERATED_BODY()

	FNiagaraBakerTextureSettings()
		: bUseFrameSize(false)
	{
	}

	/** Optional output name, if left empty a name will be auto generated using the index of the texture/ */
	UPROPERTY(EditAnywhere, Category = "Texture")
	FName OutputName;
	
	/** Source visualization we should capture, i.e. Scene Color, World Normal, etc */
	UPROPERTY(EditAnywhere, Category = "Texture")
	FNiagaraBakerTextureSource SourceBinding;

	UPROPERTY(EditAnywhere, Category = "Texture", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bUseFrameSize : 1;

	/** Size of each frame generated. */
	UPROPERTY(EditAnywhere, Category = "Texture", meta = (EditCondition="bUseFrameSize"))
	FIntPoint FrameSize = FIntPoint(128, 128);

	/** Overall texture size that will be generated. */
	UPROPERTY(EditAnywhere, Category = "Texture", meta = (EditCondition="!bUseFrameSize"))
	FIntPoint TextureSize = FIntPoint(128 * 8, 128 * 8);

	//-TODO: Add property to control generated texture compression format
	//UPROPERTY(EditAnywhere, Category = "Texture")
	//TEnumAsByte<ETextureRenderTargetFormat> Format = RTF_RGBA16f;

	/** Final texture generated, an existing entry will be updated with new capture data. */
	UPROPERTY(EditAnywhere, Category = "Texture")
	TObjectPtr<UTexture2D> GeneratedTexture = nullptr;
};

struct FNiagaraBakerOutputFrameIndices
{
	int		NumFrames = 1;
	float	NormalizedTime = 0.0f;
	int		FrameIndexA = 0;
	int		FrameIndexB = 0;
	float	Interp = 0.0f;
};

UCLASS(config=Niagara, defaultconfig, MinimalAPI)
class UNiagaraBakerSettings : public UObject
{
	GENERATED_BODY()

public:
	NIAGARA_API UNiagaraBakerSettings(const FObjectInitializer& Init);

	/**
	This is the start time of the simulation where we begin the capture.
	I.e. 2.0 would mean the simulation warms up by 2 seconds before we begin capturing.
	*/
	UPROPERTY(EditAnywhere, Category="Settings")
	float StartSeconds = 0.0f;

	/** Duration in seconds to take the capture over. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float DurationSeconds = 4.0f;

	/**
	The frame rate to run the simulation at during capturing.
	This is only used for the preview view and calculating the number of ticks to execute
	as we capture the generated texture.
	*/
	UPROPERTY(EditAnywhere, Category = "Settings", AdvancedDisplay, meta = (ClampMin=1, ClampMax=480))
	int FramesPerSecond = 60;

	/** Locks the playback to the simulation frame rate, i.e. no multi-tick. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	uint8 bLockToSimulationFrameRate : 1;

	/** Should the preview playback as looping or not. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	uint8 bPreviewLooping : 1;

	/** Number of frames in each dimension. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FIntPoint FramesPerDimension = FIntPoint(8, 8);	//-TODO: Remove me...

	/** Array of outputs for the baker to generate. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<TObjectPtr<UNiagaraBakerOutput>> Outputs;

	/** Camera Settings, will always be at least ENiagaraBakerViewMode::Num elements and those are fixed cameras. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FNiagaraBakerCameraSettings> CameraSettings;

	/** Active camera that we were saved with */
	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 CurrentCameraIndex = 0;

	/** What quality level to use when baking the simulation, where None means use the current quality level. */
	UPROPERTY(EditAnywhere, Category = "Settings", config)
	FName BakeQualityLevel;

	/** Should we render just the component or the whole scene. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	uint8 bRenderComponentOnly : 1;

	/** Should we preview the baked looped simulation if it has been generated, or the full baked sim */
	UPROPERTY(EditAnywhere, Category = "Settings")
	uint8 bPreviewLoopedOutput : 1;
	
	NIAGARA_API bool Equals(const UNiagaraBakerSettings& Other) const;

	float GetSeekDelta() const { return 1.0f / float(FramesPerSecond); }

	NIAGARA_API FVector GetCameraLocation() const;
	NIAGARA_API FRotator GetCameraRotation() const;
	NIAGARA_API FMatrix GetViewportMatrix() const;
	NIAGARA_API FMatrix GetViewMatrix() const;
	NIAGARA_API FMatrix GetProjectionMatrix() const;

	FNiagaraBakerCameraSettings& GetCurrentCamera() { return CameraSettings[CurrentCameraIndex]; }
	const FNiagaraBakerCameraSettings& GetCurrentCamera() const { return CameraSettings[CurrentCameraIndex]; }

	NIAGARA_API int GetOutputNumFrames(UNiagaraBakerOutput* BakerOutput) const;
	NIAGARA_API FNiagaraBakerOutputFrameIndices GetOutputFrameIndices(UNiagaraBakerOutput* BakerOutput, float RelativeTime) const;

	NIAGARA_API int GetOutputNumFrames(int OutputIndex) const;
	NIAGARA_API FNiagaraBakerOutputFrameIndices GetOutputFrameIndices(int OutputIndex, float RelativeTime) const;

	NIAGARA_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static NIAGARA_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	// Deprecated properties
	UPROPERTY()
	TArray<FNiagaraBakerTextureSettings> OutputTextures_DEPRECATED;

	UPROPERTY()
	ENiagaraBakerViewMode CameraViewportMode_DEPRECATED = ENiagaraBakerViewMode::Perspective;

	UPROPERTY()
	FVector CameraViewportLocation_DEPRECATED[(int)ENiagaraBakerViewMode::Num];

	UPROPERTY()
	FRotator CameraViewportRotation_DEPRECATED[(int)ENiagaraBakerViewMode::Num];

	UPROPERTY()
	float CameraOrbitDistance_DEPRECATED = 200.f;

	UPROPERTY()
	float CameraFOV_DEPRECATED = 90.0f;

	UPROPERTY()
	float CameraOrthoWidth_DEPRECATED = 512.0f;

	UPROPERTY()
	uint8 bUseCameraAspectRatio_DEPRECATED : 1;

	UPROPERTY()
	float CameraAspectRatio_DEPRECATED = 1.0f;
};
