// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEvents.h"
#include "WorldCollision.h"

#include "NiagaraCollision.generated.h"

class FNiagaraDataSet;

UENUM()
enum class ENiagaraCollisionMode : uint8
{
	None = 0,
	SceneGeometry,
	DepthBuffer,
	DistanceField
};

struct FNiagaraCollisionTrace
{
	FNiagaraCollisionTrace(const FVector& InStartPos, const FVector& InEndPos, ECollisionChannel InChannel, const FCollisionQueryParams& InQueryParams)
		: HitIndex(INDEX_NONE)
		, CollisionQueryParams(InQueryParams)
		, StartPos(InStartPos)
		, EndPos(InEndPos)
		, Channel(InChannel)
	{}

	FTraceHandle CollisionTraceHandle;
	int32 HitIndex;
	const FCollisionQueryParams CollisionQueryParams;
	const FVector StartPos;
	const FVector EndPos;
	const ECollisionChannel Channel;
};

struct FNiagaraDICollsionQueryResult
{
	FVector CollisionPos;
	FVector CollisionNormal;
	FVector CollisionVelocity;
	int32 PhysicalMaterialIdx;
	float Friction;
	float Restitution;
	bool IsInsideMesh;
};

class FNiagaraDICollisionQueryBatch
{
public:
	FNiagaraDICollisionQueryBatch()
		: BatchID(0)
		, CurrBuffer(0)
	{
	}

	~FNiagaraDICollisionQueryBatch() = default;

	int32 GetWriteBufferIdx()
	{
		return CurrBuffer;
	}

	int32 GetReadBufferIdx()
	{
		return CurrBuffer ^ 0x1;
	}

	void DispatchQueries();
	void CollectResults();

	void ClearWrite()
	{
		uint32 PrevNum = CollisionTraces[CurrBuffer].Num();
		CollisionTraces[CurrBuffer].Empty(PrevNum);
	}


	void Init(FNiagaraSystemInstanceID InBatchID, UWorld *InCollisionWorld, TStatId InStatId)
	{
		BatchID = InBatchID;
		CollisionWorld = InCollisionWorld;
		StatId = InStatId;
		CollisionTraces[0].Empty();
		CollisionTraces[1].Empty();
		CurrBuffer = 0;
	}

	int32 SubmitQuery(FVector StartPos, FVector EndPos, ECollisionChannel TraceChannel, bool bTraceComplex = false);
	bool PerformQuery(FVector StartPos, FVector EndPos, FNiagaraDICollsionQueryResult &Result, ECollisionChannel TraceChannel, bool bTraceComplex = false);
	bool GetQueryResult(uint32 TraceID, FNiagaraDICollsionQueryResult &Result);

private:
	void FlipBuffers()
	{
		CurrBuffer = CurrBuffer ^ 0x1;
	}

	//TArray<FTraceHandle> CollisionTraceHandles;
	const static FName CollisionTagName;
	const static FName TraceTagName;

	FRWLock CollisionTraceLock;
	TArray<FNiagaraCollisionEventPayload> CollisionEvents;
	FNiagaraDataSet *CollisionEventDataSet = nullptr;

	FNiagaraSystemInstanceID BatchID;
	TArray<FNiagaraCollisionTrace> CollisionTraces[2];
	TArray<FNiagaraDICollsionQueryResult> CollisionResults;
	uint32 CurrBuffer;
	UWorld *CollisionWorld = nullptr;
	TStatId StatId;
};
