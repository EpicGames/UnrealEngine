// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraScriptExecutionContext.h"
#include "NiagaraSimStageData.h"
#include "NiagaraSimStageExecutionData.h"
#include "RHIGPUReadback.h"

class FNiagaraGPUInstanceCountManager;
class FNiagaraGpuComputeDispatchInterface;
class FNiagaraGPUSystemTick;
struct FNiagaraComputeInstanceData;
struct FNiagaraComputeExecutionContext;
class FNiagaraScriptComputeRuntimeData;

class FNiagaraRHIUniformBufferLayout : public FRHIUniformBufferLayout
{
public:
	explicit FNiagaraRHIUniformBufferLayout(const TCHAR* LayoutName, uint32 ConstantBufferSize)
		: FRHIUniformBufferLayout(FRHIUniformBufferLayoutInitializer(LayoutName, ConstantBufferSize))
	{
	}
};

struct FNiagaraGpuSpawnInfoParams
{
	float IntervalDt;
	float InterpStartDt;
	int32 SpawnGroup;
	int32 GroupSpawnStartIndex;
};

struct FNiagaraGpuSpawnInfo
{
	uint32 EventSpawnTotal = 0;
	uint32 SpawnRateInstances = 0;
	uint32 MaxParticleCount = 0;
	int32 SpawnInfoStartOffsets[NIAGARA_MAX_GPU_SPAWN_INFOS];
	FNiagaraGpuSpawnInfoParams SpawnInfoParams[NIAGARA_MAX_GPU_SPAWN_INFOS];

	void Reset()
	{
		EventSpawnTotal = 0;
		SpawnRateInstances = 0;
		MaxParticleCount = 0;
		for (int32 i = 0; i < NIAGARA_MAX_GPU_SPAWN_INFOS; ++i)
		{
			SpawnInfoStartOffsets[i] = 0;

			SpawnInfoParams[i].IntervalDt = 0;
			SpawnInfoParams[i].InterpStartDt = 0;
			SpawnInfoParams[i].SpawnGroup = 0;
			SpawnInfoParams[i].GroupSpawnStartIndex = 0;
		}
	}
};

struct INiagaraComputeDataBufferInterface
{
	virtual ~INiagaraComputeDataBufferInterface() = default;
	virtual bool HasTranslucentDataToRender() const = 0;
	virtual FNiagaraDataBuffer* GetDataToRender(FRHICommandListBase& RHICmdList, bool bIsLowLatencyTranslucent) const = 0;
};

struct FNiagaraComputeExecutionContext : public INiagaraComputeDataBufferInterface
{
	FNiagaraComputeExecutionContext();
	virtual ~FNiagaraComputeExecutionContext();

	// Begin: INiagaraComputeDataBufferInterface
	virtual bool HasTranslucentDataToRender() const override { return TranslucentDataToRender != nullptr; }
	virtual FNiagaraDataBuffer* GetDataToRender(FRHICommandListBase& RHICmdList, bool bIsLowLatencyTranslucent) const override
	{
		if (bIsLowLatencyTranslucent)
		{
			// Translucent rendering uses current frame (newest) data, which is TranslucentDataToRender if present
			return TranslucentDataToRender ? TranslucentDataToRender : DataToRender;
		}
		else
		{
		#if WITH_MGPU
			// Non-translucent rendering uses previous frame (older) data, which is MultiViewPreviousDataToRender if present
			return MultiViewPreviousDataToRender ? MultiViewPreviousDataToRender : DataToRender;
		#else //WITH_MGPU
			return DataToRender;
		#endif //WITH_MGPU
		}
	}
	// End: INiagaraComputeDataBufferInterface

	void Reset(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface);

	void InitParams(UNiagaraScript* InGPUComputeScript, TSharedPtr<const FNiagaraScriptRuntimeData> InScriptRuntimeData, const FNiagaraSimStageExecutionDataPtr& InSimStageExecData);
	void DirtyDataInterfaces();
	bool Tick(FNiagaraSystemInstance* ParentSystemInstance);

	bool OptionalContexInit(FNiagaraSystemInstance* ParentSystemInstance);

	void PostTick();

	void SetDataToRender(FNiagaraDataBuffer* InDataToRender);
	void SetTranslucentDataToRender(FNiagaraDataBuffer* InTranslucentDataToRender);
#if WITH_MGPU
	void SetMultiViewPreviousDataToRender(FNiagaraDataBuffer* InMultiViewPreviousDataToRender);
#endif //WITH_MGPU

	int32 GetConstantBufferSize() const;
	uint8* WriteConstantBufferInstanceData(uint8* InTargetBuffer, FNiagaraComputeInstanceData& InstanceData) const;

	const FNiagaraScriptRuntimeData* GetScriptRuntimeData() const { return ScriptRuntimeData.Get(); }

	struct 
	{
		// The offset at which the GPU instance count (see FNiagaraGPUInstanceCountManager()).
		uint32 GPUCountOffset = INDEX_NONE;
		// The CPU instance count at the time the GPU count readback was issued. Always bigger or equal to the GPU count.
		uint32 CPUCount = 0;
	}  EmitterInstanceReadback;
	
#if WITH_NIAGARA_DEBUG_EMITTER_NAME
	FName GetDebugSimFName() const { return DebugSimFName; }
	const TCHAR* GetDebugSimName() const { return *DebugSimName; }
	void SetDebugSimName(const TCHAR* InDebugSimName, FName InDebugSimFName)
	{
		DebugSimName = InDebugSimName;
		DebugSimFName = InDebugSimFName;
	}
#else
	FName GetDebugSimFName() const { return NAME_None; }
	const TCHAR* GetDebugSimName() const { return TEXT(""); }
	void SetDebugSimName(const TCHAR*, FName) { }
#endif

#if STATS
	TStatId SystemStatID;
	TStatId EmitterStatID;
#endif

//-TOOD:private:
	void ResetInternal(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface);

public:
	static uint32 TickCounter;

#if WITH_NIAGARA_DEBUG_EMITTER_NAME
	FName DebugSimFName;
	FString DebugSimName;
#endif
	TWeakObjectPtr<class USceneComponent>	ProfilingComponentPtr;
	FVersionedNiagaraEmitterWeakPtr			ProfilingEmitterPtr;

	const TArray<UNiagaraDataInterface*>& GetDataInterfaces()const { return CombinedParamStore.GetDataInterfaces(); }

	class FNiagaraDataSet *MainDataSet;
	UNiagaraScript* GPUScript;
	class FNiagaraShaderScript*  GPUScript_RT;

private:
	TSharedPtr<const FNiagaraScriptRuntimeData> ScriptRuntimeData;

public:
	// persistent layouts used to create the constant buffers for the compute sim shader
	uint32 ExternalCBufferLayoutSize = 0;
	TRefCountPtr<FNiagaraRHIUniformBufferLayout> ExternalCBufferLayout;

	//Dynamic state updated either from GT via RT commands or from the RT side sim code itself.
	//TArray<uint8, TAlignedHeapAllocator<16>> ParamData_RT;		// RT side copy of the parameter data
	FNiagaraScriptInstanceParameterStore CombinedParamStore;
#if DO_CHECK
	TArray<FName>  DIClassNames;
#endif

	TArray<FNiagaraDataInterfaceProxy*> DataInterfaceProxies;

	// Most current buffer that can be used for rendering.
	FNiagaraDataBufferRef DataToRender = nullptr;

	// Optional buffer which can be used to render translucent data with no latency (i.e. this frames data)
	FNiagaraDataBufferRef TranslucentDataToRender = nullptr;

#if WITH_MGPU
	// For MultiView, we need to track the previous frame's data for rendering systems on other view families
	FNiagaraDataBufferRef MultiViewPreviousDataToRender = nullptr;
#endif //WITH_MGPU

	// Game thread spawn info will be sent to the render thread inside FNiagaraComputeInstanceData
	FNiagaraGpuSpawnInfo GpuSpawnInfo_GT;

	bool HasInterpolationParameters = false;

	// Do we have a reset pending, controlled by the game thread and passed to instance data
	bool bResetPending_GT = true;

	// Particle count read / write fence used when to avoid racing with the render thread
	// Read count is used on the game thread / game concurrent threads and allows us to read the particle count when <= the write fence
	// When the fence has not been passed we fallback to the potentially spawned particles
	std::atomic<uint32> ParticleCountReadFence = 1;
	std::atomic<uint32> ParticleCountWriteFence = 0;

	// Render thread data
	FNiagaraDataBuffer* GetPrevDataBuffer() { check(IsInRenderingThread() && (BufferSwapsThisFrame_RT > 0)); return DataBuffers_RT[(BufferSwapsThisFrame_RT & 1) ^ 1]; }
	FNiagaraDataBuffer* GetNextDataBuffer() { check(IsInRenderingThread()); return DataBuffers_RT[(BufferSwapsThisFrame_RT & 1)]; }
	void AdvanceDataBuffer() { ++BufferSwapsThisFrame_RT; }

	FNiagaraDataBuffer* DataBuffers_RT[2] = { nullptr, nullptr };
	uint32 BufferSwapsThisFrame_RT = 0;
	uint32 CountOffset_RT = INDEX_NONE;
	uint32 FinalDispatchGroup_RT = INDEX_NONE;
	uint32 FinalDispatchGroupInstance_RT = INDEX_NONE;

	// Used only when we multi-tick and need to keep track of pointing back to the correct FNiagaraDataBuffer
	FNiagaraDataBuffer* DataSetOriginalBuffer_RT = nullptr;

	// Used to track if we have processed any ticks for this context this frame
	bool bHasTickedThisFrame_RT = false;

	// The current number of instances on the RT
	// Before ticks are processed on the RT this will be CurrentData's NumInstances
	// As ticks are processed (or we generated the tick batches) this will change and won't be accurate until dispatches are executed
	UE_DEPRECATED(5.8, "Direct access of CurrentNumInstances_RT is deprecated please use the get / set methods")
	std::atomic<uint32> CurrentNumInstances_RT = 0;
	// The current maximum of instances on the RT
	uint32 CurrentMaxInstances_RT = 0;
	// The current maximum instances we should allocate on the RT
	uint32 CurrentMaxAllocateInstances_RT = 0;

	FNiagaraSimStageExecutionDataPtr SimStageExecData;

	void SetCurrentNumInstances(uint32 NumInstances)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		CurrentNumInstances_RT.store(NumInstances, std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	uint32 GetCurrentNumInstances() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CurrentNumInstances_RT.load(std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool IsOutputStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 SimulationStageIndex) const;
	bool IsInputStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 SimulationStageIndex) const;
	bool IsIterationStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 SimulationStageIndex) const;
	FNiagaraDataInterfaceProxyRW* FindIterationInterface(const TArray<FNiagaraDataInterfaceProxyRW*>& InProxies, uint32 SimulationStageIndex) const;
};
