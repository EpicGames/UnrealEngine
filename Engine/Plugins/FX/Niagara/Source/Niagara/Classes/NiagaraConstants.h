// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NiagaraTypes.h"
#include "NiagaraCommon.h"

#define PARAM_MAP_NPC_STR TEXT("NPC.")
#define PARAM_MAP_ENGINE_STR TEXT("Engine.")
#define PARAM_MAP_ENGINE_OWNER_STR TEXT("Engine.Owner.")
#define PARAM_MAP_ENGINE_SYSTEM_STR TEXT("Engine.System.")
#define PARAM_MAP_ENGINE_EMITTER_STR TEXT("Engine.Emitter.")
#define PARAM_MAP_LOCAL_MODULE_STR TEXT("Local.Module.")
#define PARAM_MAP_OUTPUT_MODULE_STR TEXT("Output.Module.")
#define PARAM_MAP_USER_STR TEXT("User.")
#define PARAM_MAP_SYSTEM_STR TEXT("System.")
#define PARAM_MAP_EMITTER_STR TEXT("Emitter.")
#define PARAM_MAP_MODULE_STR TEXT("Module.")
#define PARAM_MAP_ATTRIBUTE_STR TEXT("Particles.")
#define PARAM_MAP_INITIAL_STR TEXT("Initial.")
#define PARAM_MAP_INITIAL_BASE_STR TEXT("Initial")
#define PARAM_MAP_PREVIOUS_BASE_STR TEXT("Previous")
#define PARAM_MAP_RAPID_ITERATION_STR TEXT("Constants.")
#define PARAM_MAP_INDICES_STR TEXT("Array.")
#define PARAM_MAP_RAPID_ITERATION_BASE_STR TEXT("Constants")
#define PARAM_MAP_SCRIPT_PERSISTENT_STR TEXT("ScriptPersistent.")
#define PARAM_MAP_SCRIPT_TRANSIENT_STR TEXT("ScriptTransient.")
#define PARAM_MAP_INTERMEDIATE_STR TEXT("Intermedate.")
#define PARAM_MAP_LOCAL_STR TEXT("Local.")
#define PARAM_MAP_OUTPUT_STR TEXT("Output.")
#define PARAM_MAP_TRANSIENT_STR TEXT("Transient.")

#define TRANSLATOR_SET_VARIABLES_STR TEXT("SetVariables")
#define TRANSLATOR_SET_VARIABLES_UNDERSCORE_STR TEXT("SetVariables_")

#define SYS_PARAM_ENGINE_WORLD_DELTA_TIME                INiagaraModule::GetVar_Engine_WorldDeltaTime()
#define SYS_PARAM_ENGINE_DELTA_TIME                      INiagaraModule::GetVar_Engine_DeltaTime()
#define SYS_PARAM_ENGINE_INV_DELTA_TIME                  INiagaraModule::GetVar_Engine_InvDeltaTime()
#define SYS_PARAM_ENGINE_TIME                            INiagaraModule::GetVar_Engine_Time()
#define SYS_PARAM_ENGINE_REAL_TIME                       INiagaraModule::GetVar_Engine_RealTime()
#define SYS_PARAM_ENGINE_QUALITY_LEVEL                   INiagaraModule::GetVar_Engine_QualityLevel()
#define SYS_PARAM_ENGINE_POSITION                        INiagaraModule::GetVar_Engine_Owner_Position()
#define SYS_PARAM_ENGINE_VELOCITY                        INiagaraModule::GetVar_Engine_Owner_Velocity()
#define SYS_PARAM_ENGINE_X_AXIS                          INiagaraModule::GetVar_Engine_Owner_XAxis()
#define SYS_PARAM_ENGINE_Y_AXIS                          INiagaraModule::GetVar_Engine_Owner_YAxis()
#define SYS_PARAM_ENGINE_Z_AXIS                          INiagaraModule::GetVar_Engine_Owner_ZAxis()
#define SYS_PARAM_ENGINE_ROTATION                        INiagaraModule::GetVar_Engine_Owner_Rotation()
#define SYS_PARAM_ENGINE_SCALE                           INiagaraModule::GetVar_Engine_Owner_Scale()
#define SYS_PARAM_ENGINE_LWC_TILE                        INiagaraModule::GetVar_Engine_Owner_LWC_Tile()
#define SYS_PARAM_ENGINE_EXEC_INDEX                      INiagaraModule::GetVar_Engine_ExecIndex()

#define SYS_PARAM_ENGINE_LOCAL_TO_WORLD                  INiagaraModule::GetVar_Engine_Owner_SystemLocalToWorld()
#define SYS_PARAM_ENGINE_WORLD_TO_LOCAL                  INiagaraModule::GetVar_Engine_Owner_SystemWorldToLocal()
#define SYS_PARAM_ENGINE_LOCAL_TO_WORLD_TRANSPOSED       INiagaraModule::GetVar_Engine_Owner_SystemLocalToWorldTransposed()
#define SYS_PARAM_ENGINE_WORLD_TO_LOCAL_TRANSPOSED       INiagaraModule::GetVar_Engine_Owner_SystemWorldToLocalTransposed()
#define SYS_PARAM_ENGINE_LOCAL_TO_WORLD_NO_SCALE         INiagaraModule::GetVar_Engine_Owner_SystemLocalToWorldNoScale()
#define SYS_PARAM_ENGINE_WORLD_TO_LOCAL_NO_SCALE         INiagaraModule::GetVar_Engine_Owner_SystemWorldToLocalNoScale()


#define SYS_PARAM_ENGINE_TIME_SINCE_RENDERED			 INiagaraModule::GetVar_Engine_Owner_TimeSinceRendered()
#define SYS_PARAM_ENGINE_LOD_DISTANCE					 INiagaraModule::GetVar_Engine_Owner_LODDistance()
#define SYS_PARAM_ENGINE_LOD_DISTANCE_FRACTION			 INiagaraModule::GetVar_Engine_Owner_LODDistanceFraction()


#define SYS_PARAM_ENGINE_EXECUTION_STATE                 INiagaraModule::GetVar_Engine_Owner_ExecutionState()

#define SYS_PARAM_ENGINE_EXEC_COUNT                      INiagaraModule::GetVar_Engine_ExecutionCount()
#define SYS_PARAM_ENGINE_EMITTER_NUM_PARTICLES           INiagaraModule::GetVar_Engine_Emitter_NumParticles()
#define SYS_PARAM_ENGINE_EMITTER_SIMULATION_POSITION	 INiagaraModule::GetVar_Engine_Emitter_SimulationPosition()
#define SYS_PARAM_ENGINE_EMITTER_TOTAL_SPAWNED_PARTICLES INiagaraModule::GetVar_Engine_Emitter_TotalSpawnedParticles()
#define SYS_PARAM_ENGINE_EMITTER_SPAWN_COUNT_SCALE       INiagaraModule::GetVar_Engine_Emitter_SpawnCountScale()
#define SYS_PARAM_ENGINE_EMITTER_INSTANCE_SEED           INiagaraModule::GetVar_Engine_Emitter_InstanceSeed()
#define SYS_PARAM_ENGINE_EMITTER_ID		                 INiagaraModule::GetVar_Engine_Emitter_ID()
#define SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS_ALIVE       INiagaraModule::GetVar_Engine_System_NumEmittersAlive()
#define SYS_PARAM_ENGINE_SYSTEM_SIGNIFICANCE_INDEX		 INiagaraModule::GetVar_Engine_System_SignificanceIndex()
#define SYS_PARAM_ENGINE_SYSTEM_RANDOM_SEED				INiagaraModule::GetVar_Engine_System_RandomSeed()

#define SYS_PARAM_ENGINE_SYSTEM_CURRENT_TIME_STEP		 INiagaraModule::GetVar_Engine_System_CurrentTimeStep()
#define SYS_PARAM_ENGINE_SYSTEM_NUM_TIME_STEPS			 INiagaraModule::GetVar_Engine_System_NumTimeSteps()
#define SYS_PARAM_ENGINE_SYSTEM_TIME_STEP_FRACTION			 INiagaraModule::GetVar_Engine_System_TimeStepFraction()
#define SYS_PARAM_ENGINE_SYSTEM_NUM_PARTICLES			INiagaraModule::GetVar_Engine_System_NumParticles()


#define SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS             INiagaraModule::GetVar_Engine_System_NumEmitters()
#define SYS_PARAM_ENGINE_NUM_SYSTEM_INSTANCES            INiagaraModule::GetVar_Engine_NumSystemInstances()

#define SYS_PARAM_ENGINE_GLOBAL_SPAWN_COUNT_SCALE        INiagaraModule::GetVar_Engine_GlobalSpawnCountScale()
#define SYS_PARAM_ENGINE_GLOBAL_SYSTEM_COUNT_SCALE       INiagaraModule::GetVar_Engine_GlobalSystemScale()

#define SYS_PARAM_ENGINE_SYSTEM_AGE                      INiagaraModule::GetVar_Engine_System_Age()
#define SYS_PARAM_ENGINE_SYSTEM_TICK_COUNT               INiagaraModule::GetVar_Engine_System_TickCount()

#define SYS_PARAM_EMITTER_AGE                            INiagaraModule::GetVar_Emitter_Age()
#define SYS_PARAM_EMITTER_LOCALSPACE                     INiagaraModule::GetVar_Emitter_LocalSpace()
#define SYS_PARAM_EMITTER_DETERMINISM                    INiagaraModule::GetVar_Emitter_Determinism()
#define SYS_PARAM_EMITTER_INTERP_SPAWN                   INiagaraModule::GetVar_Emitter_InterpolatedSpawn()
#define SYS_PARAM_EMITTER_OVERRIDE_GLOBAL_SPAWN_COUNT_SCALE       INiagaraModule::GetVar_Emitter_OverrideGlobalSpawnCountScale()
#define SYS_PARAM_EMITTER_RANDOM_SEED                    INiagaraModule::GetVar_Emitter_RandomSeed()
#define SYS_PARAM_EMITTER_SPAWNRATE                      INiagaraModule::GetVar_Emitter_SpawnRate()
#define SYS_PARAM_EMITTER_SPAWN_INTERVAL                 INiagaraModule::GetVar_Emitter_SpawnInterval()
#define SYS_PARAM_EMITTER_SIMULATION_TARGET              INiagaraModule::GetVar_Emitter_SimulationTarget()
#define SYS_PARAM_EMITTER_INTERP_SPAWN_START_DT          INiagaraModule::GetVar_Emitter_InterpSpawnStartDt()
#define SYS_PARAM_EMITTER_SPAWN_GROUP                    INiagaraModule::GetVar_Emitter_SpawnGroup()

#define SYS_PARAM_PARTICLES_UNIQUE_ID                    INiagaraModule::GetVar_Particles_UniqueID()
#define SYS_PARAM_PARTICLES_ID                           INiagaraModule::GetVar_Particles_ID()
#define SYS_PARAM_PARTICLES_POSITION                     INiagaraModule::GetVar_Particles_Position()
#define SYS_PARAM_PARTICLES_VELOCITY                     INiagaraModule::GetVar_Particles_Velocity()
#define SYS_PARAM_PARTICLES_COLOR                        INiagaraModule::GetVar_Particles_Color()
#define SYS_PARAM_PARTICLES_SPRITE_ROTATION              INiagaraModule::GetVar_Particles_SpriteRotation()
#define SYS_PARAM_PARTICLES_AGE                          INiagaraModule::GetVar_Particles_Age()
#define SYS_PARAM_PARTICLES_NORMALIZED_AGE               INiagaraModule::GetVar_Particles_NormalizedAge()
#define SYS_PARAM_PARTICLES_SPRITE_SIZE                  INiagaraModule::GetVar_Particles_SpriteSize()
#define SYS_PARAM_PARTICLES_SPRITE_FACING                INiagaraModule::GetVar_Particles_SpriteFacing()
#define SYS_PARAM_PARTICLES_SPRITE_ALIGNMENT             INiagaraModule::GetVar_Particles_SpriteAlignment()
#define SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX              INiagaraModule::GetVar_Particles_SubImageIndex()
#define SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM       INiagaraModule::GetVar_Particles_DynamicMaterialParameter()
#define SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1     INiagaraModule::GetVar_Particles_DynamicMaterialParameter1()
#define SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2     INiagaraModule::GetVar_Particles_DynamicMaterialParameter2()
#define SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3     INiagaraModule::GetVar_Particles_DynamicMaterialParameter3()
#define SYS_PARAM_PARTICLES_SCALE                        INiagaraModule::GetVar_Particles_Scale()
#define SYS_PARAM_PARTICLES_LIFETIME                     INiagaraModule::GetVar_Particles_Lifetime()
#define SYS_PARAM_PARTICLES_MESH_ORIENTATION             INiagaraModule::GetVar_Particles_MeshOrientation()
#define SYS_PARAM_PARTICLES_UV_SCALE                     INiagaraModule::GetVar_Particles_UVScale()
#define SYS_PARAM_PARTICLES_PIVOT_OFFSET				 INiagaraModule::GetVar_Particles_PivotOffset()
#define SYS_PARAM_PARTICLES_CAMERA_OFFSET                INiagaraModule::GetVar_Particles_CameraOffset()
#define SYS_PARAM_PARTICLES_MATERIAL_RANDOM              INiagaraModule::GetVar_Particles_MaterialRandom()
#define SYS_PARAM_PARTICLES_LIGHT_RADIUS                 INiagaraModule::GetVar_Particles_LightRadius()
#define SYS_PARAM_PARTICLES_LIGHT_EXPONENT               INiagaraModule::GetVar_Particles_LightExponent()
#define SYS_PARAM_PARTICLES_LIGHT_ENABLED                INiagaraModule::GetVar_Particles_LightEnabled()
#define SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING  INiagaraModule::GetVar_Particles_LightVolumetricScattering()
#define SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE         INiagaraModule::GetVar_Particles_LightSpecularScale()
#define SYS_PARAM_PARTICLES_LIGHT_DIFFUSE_SCALE          INiagaraModule::GetVar_Particles_LightDiffuseScale()
#define SYS_PARAM_PARTICLES_VISIBILITY_TAG				 INiagaraModule::GetVar_Particles_VisibilityTag()
#define SYS_PARAM_PARTICLES_MESH_INDEX					 INiagaraModule::GetVar_Particles_MeshIndex()
#define SYS_PARAM_PARTICLES_COMPONENTS_ENABLED           INiagaraModule::GetVar_Particles_ComponentsEnabled()

#define SYS_PARAM_PARTICLES_RIBBONID                     INiagaraModule::GetVar_Particles_RibbonID()
#define SYS_PARAM_PARTICLES_RIBBONWIDTH                  INiagaraModule::GetVar_Particles_RibbonWidth()
#define SYS_PARAM_PARTICLES_RIBBONTWIST                  INiagaraModule::GetVar_Particles_RibbonTwist()
#define SYS_PARAM_PARTICLES_RIBBONFACING                 INiagaraModule::GetVar_Particles_RibbonFacing()
#define SYS_PARAM_PARTICLES_RIBBONLINKORDER              INiagaraModule::GetVar_Particles_RibbonLinkOrder()
#define RIBBONUVDISTANCE 								 INiagaraModule::GetVar_Particles_RibbonUVDistance()
#define SYS_PARAM_PARTICLES_RIBBONU0OVERRIDE             INiagaraModule::GetVar_Particles_RibbonU0Override()
#define SYS_PARAM_PARTICLES_RIBBONV0RANGEOVERRIDE        INiagaraModule::GetVar_Particles_RibbonV0RangeOverride()
#define SYS_PARAM_PARTICLES_RIBBONU1OVERRIDE             INiagaraModule::GetVar_Particles_RibbonU1Override()
#define SYS_PARAM_PARTICLES_RIBBONV1RANGEOVERRIDE        INiagaraModule::GetVar_Particles_RibbonV1RangeOverride()

#define SYS_PARAM_INSTANCE_ALIVE                         INiagaraModule::GetVar_DataInstance_Alive()
#define SYS_PARAM_SCRIPT_USAGE                           INiagaraModule::GetVar_ScriptUsage()
#define SYS_PARAM_SCRIPT_CONTEXT                         INiagaraModule::GetVar_ScriptContext()
#define SYS_PARAM_FUNCTION_DEBUG_STATE					 INiagaraModule::GetVar_FunctionDebugState()
#define TRANSLATOR_PARAM_BEGIN_DEFAULTS                  INiagaraModule::GetVar_BeginDefaults()
#define TRANSLATOR_PARAM_CALL_ID                  		 INiagaraModule::GetVar_CallID()

enum class ENiagaraKnownConstantType : uint8
{
	Attribute,
	Other,
};

struct FNiagaraKnownConstantInfo
{
	FNiagaraKnownConstantInfo()
		: ConstantVar(nullptr)
		, ConstantType(ENiagaraKnownConstantType::Other)
	{};

	FNiagaraKnownConstantInfo(const FNiagaraVariable* InConstantVar, const ENiagaraKnownConstantType InConstantType)
		: ConstantVar(InConstantVar)
		, ConstantType(InConstantType)
	{};

	const FNiagaraVariable* ConstantVar = nullptr;
	const ENiagaraKnownConstantType ConstantType = ENiagaraKnownConstantType::Other;
};

struct FNiagaraConstants
{
	static NIAGARA_API void Init();
	static NIAGARA_API const TArray<FNiagaraVariable>& GetEngineConstants();
	static NIAGARA_API const TArray<FNiagaraVariable>& GetTranslatorConstants();
	static NIAGARA_API const TArray<FNiagaraVariable>& GetStaticSwitchConstants();
	static NIAGARA_API FNiagaraVariable UpdateEngineConstant(const FNiagaraVariable& InVar);
	static NIAGARA_API const FNiagaraVariable *FindEngineConstant(const FNiagaraVariable& InVar);
	static NIAGARA_API FText GetEngineConstantDescription(const FNiagaraVariable& InVar);
	static NIAGARA_API const TArray<FNiagaraVariable>& GetOldPositionTypeVariables();

	static NIAGARA_API const TArray<FNiagaraVariable>& GetCommonParticleAttributes();
	static NIAGARA_API FText GetAttributeDescription(const FNiagaraVariableBase& InVar);
	static NIAGARA_API FString GetAttributeDefaultValue(const FNiagaraVariableBase& InVar);
	static NIAGARA_API FNiagaraVariable GetAttributeWithDefaultValue(const FNiagaraVariableBase& InAttribute);
	static NIAGARA_API FNiagaraVariableBase GetAttributeAsParticleDataSetKey(const FNiagaraVariableBase& InAttribute);
	static NIAGARA_API FNiagaraVariableBase GetAttributeAsEmitterDataSetKey(const FNiagaraVariableBase& InAttribute);
	static NIAGARA_API FNiagaraVariableAttributeBinding GetAttributeDefaultBinding(const FNiagaraVariableBase& InAttribute);

	static NIAGARA_API bool IsNiagaraConstant(const FNiagaraVariableBase& InVar);
	static NIAGARA_API const FNiagaraVariableMetaData* GetConstantMetaData(const FNiagaraVariableBase& InVar);

	static NIAGARA_API const FNiagaraVariable* GetKnownConstant(const FName& InName, bool bAllowPartialNameMatch);
	static NIAGARA_API const FNiagaraKnownConstantInfo GetKnownConstantInfo(const FName& InName, bool bAllowPartialNameMatch);
	static NIAGARA_API const FNiagaraVariable *FindStaticSwitchConstant(const FName& InName);

	static NIAGARA_API bool IsEngineManagedAttribute(const FNiagaraVariableBase& Var);

	static NIAGARA_API const FName InputPinName;
	static NIAGARA_API const FName OutputPinName;


	/** Reserved Namespace Names and Scope Names */
	static NIAGARA_API const FName UserNamespace;
	static NIAGARA_API const FName EngineNamespace;
	static NIAGARA_API const FName SystemNamespace;
	static NIAGARA_API const FName EmitterNamespace;
	static NIAGARA_API const FName ParticleAttributeNamespace;
	static NIAGARA_API const FName ModuleNamespace;
	static NIAGARA_API const FName OutputNamespace;
	static NIAGARA_API const FName TransientNamespace;
	static NIAGARA_API const FName StackContextNamespace;
	static NIAGARA_API const FName DataInstanceNamespace;
	static NIAGARA_API const FName StaticSwitchNamespace;
	static NIAGARA_API const FName ArrayNamespace;
	static NIAGARA_API const FName ParameterCollectionNamespace;
	static NIAGARA_API const FString InitialPrefix;
	static NIAGARA_API const FName LocalNamespace;
	static NIAGARA_API const FName InitialNamespace;
	static NIAGARA_API const FName PreviousNamespace;
	static NIAGARA_API const FName OwnerNamespace;
	static NIAGARA_API const FName SharedNamespace;

	static NIAGARA_API const FName EngineOwnerScopeName;
	static NIAGARA_API const FName EngineSystemScopeName;
	static NIAGARA_API const FName EngineEmitterScopeName;

	static NIAGARA_API const FName ScriptTransientScopeName;
	static NIAGARA_API const FName ScriptPersistentScopeName;
	static NIAGARA_API const FName InputScopeName;
	static NIAGARA_API const FName OutputScopeName;
	static NIAGARA_API const FName UniqueOutputScopeName;
	static NIAGARA_API const FName CustomScopeName;

	/** String version of namespaces */
	static NIAGARA_API const FString AssignmentNodePrefixString;
	static NIAGARA_API const FString EngineNamespaceString;
	static NIAGARA_API const FString EmitterNamespaceString;
	static NIAGARA_API const FString ModuleNamespaceString;
	static NIAGARA_API const FString OutputNamespaceString;
	static NIAGARA_API const FString ParameterCollectionNamespaceString;
	static NIAGARA_API const FString ParticleAttributeNamespaceString;
	static NIAGARA_API const FString RapidIterationParametersNamespaceString;
	static NIAGARA_API const FString StackContextNamespaceString;
	static NIAGARA_API const FString SystemNamespaceString;
	static NIAGARA_API const FString UserNamespaceString;
	static NIAGARA_API const FString InternalNamespaceString;
	static NIAGARA_API const FString PreviousNamespaceString;

	static NIAGARA_API const int32 MaxCategoryNameLength;
	static NIAGARA_API const int32 MaxParameterLength;
	static NIAGARA_API const int32 MaxScriptNameLength;
private:
	static TArray<FNiagaraVariable> SystemParameters;
	static TArray<FNiagaraVariable> TranslatorParameters;
	static TArray<FNiagaraVariable> SwitchParameters;
	static TArray<FNiagaraVariable> OldPositionTypes;
	static TMap<FName, FNiagaraVariable> UpdatedSystemParameters;
	static TMap<FNiagaraVariable, FText> SystemStrMap;
	static TArray<FNiagaraVariable> Attributes;
	static TMap<FNiagaraVariableBase, FString> AttrDefaultsStrMap;
	static TMap<FNiagaraVariableBase, FNiagaraVariable> AttrDefaultsValueMap;
	static TMap<FNiagaraVariableBase, FNiagaraVariable> AttrDataSetKeyMap;
	static TMap<FNiagaraVariableBase, FText> AttrDescStrMap;
	static TMap<FNiagaraVariableBase, FNiagaraVariableMetaData> AttrMetaData;
	
	static TArray<FNiagaraVariable> EngineManagedAttributes;

};
