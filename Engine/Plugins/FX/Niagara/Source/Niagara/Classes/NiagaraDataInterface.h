// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraDataInterfaceBase.h"
#include "NiagaraScriptBase.h"
#include "NiagaraShared.h"
#include "VectorVM.h"
#include "NiagaraDataInterface.generated.h"

class FRDGBuilder;
class FRDGExternalAccessQueue;

class INiagaraCompiler;
class UCanvas;
class UCurveVector;
class UCurveLinearColor;
class UCurveFloat;
struct FNiagaraDataInterfaceProxy;
struct FNiagaraDataInterfaceProxyRW;
struct FNiagaraComputeInstanceData;
class FNiagaraEmitterInstance;
class FNiagaraGpuComputeDispatchInterface;
class FNiagaraGPUSystemTick;
struct FNiagaraSimStageData;
class FNiagaraSystemInstance;
struct FNiagaraDataInterfaceHlslGenerationContext;
class FNiagaraGPUInstanceCountManager;
class FNiagaraGpuReadbackManager;
class FNiagaraScriptRuntimeData;
struct FNiagaraScriptResolvedDataInterfaceInfo;

namespace ERHIFeatureLevel { enum Type : int; }

struct FNDITransformHandlerNoop
{
	inline void TransformPosition(FVector3f& V, const FMatrix44f& M) const { }
	inline void TransformPosition(FVector3d& V, const FMatrix44d& M) const { }
	inline void TransformUnitVector(FVector3f& V, const FMatrix44f& M) const { }
	inline void TransformUnitVector(FVector3d& V, const FMatrix44d& M) const { }
	inline void TransformNotUnitVector(FVector3f& V, const FMatrix44f& M) const { }
	inline void TransformNotUnitVector(FVector3d& V, const FMatrix44d& M) const { }
	inline void TransformRotation(FQuat4f& Q1, const FQuat4f& Q2) const { }
	inline void TransformRotation(FQuat4d& Q1, const FQuat4d& Q2) const { }

	UE_DEPRECATED(5.4, "Please update your code to use TransformUnitVector")
	inline void TransformVector(FVector3f& V, const FMatrix44f& M) const { }
	UE_DEPRECATED(5.4, "Please update your code to use TransformUnitVector")
	inline void TransformVector(FVector3d& V, const FMatrix44d& M) const { }
};

struct FNDITransformHandler
{
	inline void TransformPosition(FVector3f& P, const FMatrix44f& M) const { P = M.TransformPosition(P); }
	inline void TransformPosition(FVector3d& P, const FMatrix44d& M) const { P = M.TransformPosition(P); }
	inline void TransformUnitVector(FVector3f& V, const FMatrix44f& M) const { V = M.TransformVector(V).GetUnsafeNormal3(); }
	inline void TransformUnitVector(FVector3d& V, const FMatrix44d& M) const { V = M.TransformVector(V).GetUnsafeNormal3(); }
	inline void TransformNotUnitVector(FVector3f& V, const FMatrix44f& M) const { V = M.TransformVector(V); }
	inline void TransformNotUnitVector(FVector3d& V, const FMatrix44d& M) const { V = M.TransformVector(V); }
	inline void TransformRotation(FQuat4f& Q1, const FQuat4f& Q2) const { Q1 = Q2 * Q1; }
	inline void TransformRotation(FQuat4d& Q1, const FQuat4d& Q2) const { Q1 = Q2 * Q1; }

	UE_DEPRECATED(5.4, "Please update your code to use TransformUnitVector")
	inline void TransformVector(FVector3f& V, const FMatrix44f& M) const { V = M.TransformVector(V).GetUnsafeNormal3(); }
	UE_DEPRECATED(5.4, "Please update your code to use TransformUnitVector")
	inline void TransformVector(FVector3d& V, const FMatrix44d& M) const { V = M.TransformVector(V).GetUnsafeNormal3(); }
};

// FNiagaraDataInterfaceProxy should always outlive any ticks, etc, that are on the rendering thread.
// Enabling this will incur cost but will validate that this contract is not broken
#define NIAGARA_VALIDATE_NDIPROXY_REFS	0//!UE_BUILD_SHIPPING

//////////////////////////////////////////////////////////////////////////
// Some helper classes allowing neat, init time binding of templated vm external functions.

struct TNDINoopBinder {};

// Adds a known type to the parameters
template<typename DirectType, typename NextBinder>
struct TNDIExplicitBinder
{
	template<typename... ParamTypes>
	static void Bind(UNiagaraDataInterface* Interface, const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
	{
		NextBinder::template Bind<ParamTypes..., DirectType>(Interface, BindingInfo, InstanceData, OutFunc);
	}
};

// Binder that tests the location of an operand and adds the correct handler type to the Binding parameters.
template<int32 ParamIdx, typename DataType, typename NextBinder>
struct TNDIParamBinder
{
	template<typename... ParamTypes>
	static void Bind(UNiagaraDataInterface* Interface, const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
	{
		if (BindingInfo.InputParamLocations[ParamIdx])
		{
			NextBinder::template Bind<ParamTypes..., VectorVM::FExternalFuncConstHandler<DataType>>(Interface, BindingInfo, InstanceData, OutFunc);
		}
		else
		{
			NextBinder::template Bind<ParamTypes..., VectorVM::FExternalFuncRegisterHandler<DataType>>(Interface, BindingInfo, InstanceData, OutFunc);
		}
	}
};

template<int32 ParamIdx, typename DataType>
struct TNDIParamBinder<ParamIdx, DataType, TNDINoopBinder>
{
	template<typename... ParamTypes>
	static void Bind(UNiagaraDataInterface* Interface, const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
	{
	}
};

#define NDI_FUNC_BINDER(ClassName, FuncName) T##ClassName##_##FuncName##Binder

#define DEFINE_NDI_FUNC_BINDER(ClassName, FuncName)\
struct NDI_FUNC_BINDER(ClassName, FuncName)\
{\
	template<typename ... ParamTypes>\
	static void Bind(UNiagaraDataInterface* Interface, const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)\
	{\
		OutFunc.BindUObject(static_cast<ClassName*>(Interface), &ClassName::FuncName<ParamTypes...>);\
	}\
};

#define DEFINE_NDI_DIRECT_FUNC_BINDER(ClassName, FuncName)\
struct NDI_FUNC_BINDER(ClassName, FuncName)\
{\
	static void Bind(UNiagaraDataInterface* Interface, FVMExternalFunction &OutFunc)\
	{\
		OutFunc.BindUObject(static_cast<ClassName*>(Interface), &ClassName::FuncName);\
	}\
};

#define DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(ClassName, FuncName)\
struct NDI_FUNC_BINDER(ClassName, FuncName)\
{\
	template <typename ... VarTypes>\
	static void Bind(UNiagaraDataInterface* Interface, FVMExternalFunction &OutFunc, VarTypes... Var)\
	{\
		auto Lambda = [Interface, Var...](FVectorVMExternalFunctionContext& Context) { static_cast<ClassName*>(Interface)->FuncName(Context, Var...); };\
		OutFunc.BindLambda(Lambda);\
	}\
};

#if WITH_EDITOR
// Helper class for GUI error handling
DECLARE_DELEGATE_RetVal(bool, FNiagaraDataInterfaceFix);
class FNiagaraDataInterfaceError
{
public:
	FNiagaraDataInterfaceError(FText InErrorText,
		FText InErrorSummaryText,
		FNiagaraDataInterfaceFix InFix)
		: ErrorText(InErrorText)
		, ErrorSummaryText(InErrorSummaryText)
		, Fix(InFix)

	{}

	FNiagaraDataInterfaceError()
	{}

	/** Returns true if the error can be fixed automatically. */
	bool GetErrorFixable() const
	{
		return Fix.IsBound();
	}

	/** Applies the fix if a delegate is bound for it.*/
	bool TryFixError()
	{
		return Fix.IsBound() ? Fix.Execute() : false;
	}

	/** Full error description text */
	FText GetErrorText() const
	{
		return ErrorText;
	}

	/** Shortened error description text*/
	FText GetErrorSummaryText() const
	{
		return ErrorSummaryText;
	}

	inline bool operator !=(const FNiagaraDataInterfaceError& Other) const
	{
		return !(*this == Other);
	}

	inline bool operator == (const FNiagaraDataInterfaceError& Other) const
	{
		return ErrorText.EqualTo(Other.ErrorText) && ErrorSummaryText.EqualTo(Other.ErrorSummaryText);
	}

private:
	FText ErrorText;
	FText ErrorSummaryText;
	FNiagaraDataInterfaceFix Fix;
};

// Helper class for GUI feedback handling
DECLARE_DELEGATE_RetVal(bool, FNiagaraDataInterfaceFix);
class FNiagaraDataInterfaceFeedback
{
public:
	FNiagaraDataInterfaceFeedback(FText InFeedbackText,
		FText InFeedbackSummaryText,
		FNiagaraDataInterfaceFix InFix)
		: FeedbackText(InFeedbackText)
		, FeedbackSummaryText(InFeedbackSummaryText)
		, Fix(InFix)

	{}

	FNiagaraDataInterfaceFeedback()
	{}

	/** Returns true if the feedback can be fixed automatically. */
	bool GetFeedbackFixable() const
	{
		return Fix.IsBound();
	}

	/** Applies the fix if a delegate is bound for it.*/
	bool TryFixFeedback()
	{
		return Fix.IsBound() ? Fix.Execute() : false;
	}

	/** Full feedback description text */
	FText GetFeedbackText() const
	{
		return FeedbackText;
	}

	/** Shortened feedback description text*/
	FText GetFeedbackSummaryText() const
	{
		return FeedbackSummaryText;
	}

	inline bool operator !=(const FNiagaraDataInterfaceFeedback& Other) const
	{
		return !(*this == Other);
	}

	inline bool operator == (const FNiagaraDataInterfaceFeedback& Other) const
	{
		return FeedbackText.EqualTo(Other.FeedbackText) && FeedbackSummaryText.EqualTo(Other.FeedbackSummaryText);
	}

private:
	FText FeedbackText;
	FText FeedbackSummaryText;
	FNiagaraDataInterfaceFix Fix;
};
#endif

//////////////////////////////////////////////////////////////////////////

#if WITH_NIAGARA_DEBUGGER
struct FNDIDrawDebugHudContext
{
	FNDIDrawDebugHudContext(bool bInVerbose, UWorld* InWorld, UCanvas* InCanvas, FNiagaraSystemInstance* InSystemInstance)
		: bVerbose(bInVerbose)
		, World(InWorld)
		, Canvas(InCanvas)
		, SystemInstance(InSystemInstance)
	{
	}

	bool IsVerbose() const { return bVerbose; }
	UWorld* GetWorld() const { return World; }
	UCanvas* GetCanvas() const { return Canvas; }
	const FNiagaraSystemInstance* GetSystemInstance() const { return SystemInstance; }
	FString& GetOutputString() { return OutputString; }

protected:
	bool					bVerbose;
	UWorld*					World;
	UCanvas*				Canvas;
	FNiagaraSystemInstance*	SystemInstance;
	FString					OutputString;
};
#endif //WITH_NIAGARA_DEBUGGER

//////////////////////////////////////////////////////////////////////////

struct FNDIGpuComputeContext
{
	FNDIGpuComputeContext(FRDGBuilder& InGraphBuilder, FNiagaraGpuComputeDispatchInterface& InComputeDispatchInterface)
		: GraphBuilder(InGraphBuilder)
		, ComputeDispatchInterface(InComputeDispatchInterface)
	{
	}

	FRDGBuilder& GetGraphBuilder() const { return GraphBuilder; }
	NIAGARA_API FRDGExternalAccessQueue& GetRDGExternalAccessQueue() const;

	// Expose only const direct access to the DispatchInterface.
	// For any non-const operations this and child context structs should expose only necessary and safe non const operations for that context.
	const FNiagaraGpuComputeDispatchInterface& GetComputeDispatchInterface() const { return ComputeDispatchInterface; }

protected:
	FRDGBuilder& GraphBuilder;
	FNiagaraGpuComputeDispatchInterface& ComputeDispatchInterface;
};

struct FNDIGpuComputeResetContext : public FNDIGpuComputeContext
{
	FNDIGpuComputeResetContext(FRDGBuilder& InGraphBuilder, FNiagaraGpuComputeDispatchInterface& InComputeDispatchInterface, FNiagaraSystemInstanceID InSystemInstanceID)
		: FNDIGpuComputeContext(InGraphBuilder, InComputeDispatchInterface)
		, SystemInstanceID(InSystemInstanceID)
	{
	}

	FNiagaraSystemInstanceID GetSystemInstanceID() const { return SystemInstanceID; }

protected:
	FNiagaraSystemInstanceID SystemInstanceID = FNiagaraSystemInstanceID();
};

struct FNDIGpuComputePrePostStageContext : public FNDIGpuComputeContext
{
	FNDIGpuComputePrePostStageContext(FRDGBuilder& InGraphBuilder, FNiagaraGpuComputeDispatchInterface& InComputeDispatchInterface, const FNiagaraGPUSystemTick& InSystemTick, const FNiagaraComputeInstanceData& InComputeInstanceData, const FNiagaraSimStageData& InSimStageData)
		: FNDIGpuComputeContext(InGraphBuilder, InComputeDispatchInterface)
		, SystemTick(InSystemTick)
		, ComputeInstanceData(InComputeInstanceData)
		, SimStageData(InSimStageData)
	{
	}

	const FNiagaraComputeInstanceData& GetComputeInstanceData() const { return ComputeInstanceData; }
	const FNiagaraSimStageData& GetSimStageData() const { return SimStageData; }

	NIAGARA_API FNiagaraSystemInstanceID GetSystemInstanceID() const;
	NIAGARA_API FVector3f GetSystemLWCTile() const;
	NIAGARA_API bool IsOutputStage() const;
	NIAGARA_API bool IsInputStage() const;
	NIAGARA_API bool IsIterationStage() const;

	void SetDataInterfaceProxy(FNiagaraDataInterfaceProxy* InDataInterfaceProxy) { DataInterfaceProxy = InDataInterfaceProxy; }

	FNiagaraGPUInstanceCountManager& GetInstanceCountManager()const;

protected:
	const FNiagaraGPUSystemTick& SystemTick;
	const FNiagaraComputeInstanceData& ComputeInstanceData;
	const FNiagaraSimStageData& SimStageData;
	FNiagaraDataInterfaceProxy* DataInterfaceProxy = nullptr;
};

struct FNDIGpuComputePreStageContext : public FNDIGpuComputePrePostStageContext
{
	FNDIGpuComputePreStageContext(FRDGBuilder& InGraphBuilder, FNiagaraGpuComputeDispatchInterface& InComputeDispatchInterface, const FNiagaraGPUSystemTick& InSystemTick, const FNiagaraComputeInstanceData& InComputeInstanceData, const FNiagaraSimStageData& InSimStageData)
		: FNDIGpuComputePrePostStageContext(InGraphBuilder, InComputeDispatchInterface, InSystemTick, InComputeInstanceData, InSimStageData)
	{
	}
};

struct FNDIGpuComputePostStageContext : public FNDIGpuComputePrePostStageContext
{
	FNDIGpuComputePostStageContext(FRDGBuilder& InGraphBuilder, FNiagaraGpuComputeDispatchInterface& InComputeDispatchInterface, const FNiagaraGPUSystemTick& InSystemTick, const FNiagaraComputeInstanceData& InComputeInstanceData, const FNiagaraSimStageData& InSimStageData)
		: FNDIGpuComputePrePostStageContext(InGraphBuilder, InComputeDispatchInterface, InSystemTick, InComputeInstanceData, InSimStageData)
	{
	}
};

struct FNDIGpuComputePostSimulateContext : public FNDIGpuComputeContext
{
	FNDIGpuComputePostSimulateContext(FRDGBuilder& InGraphBuilder, FNiagaraGpuComputeDispatchInterface& InComputeDispatchInterface, FNiagaraSystemInstanceID InSystemInstanceID, bool InFinalPostSimulate)
		: FNDIGpuComputeContext(InGraphBuilder, InComputeDispatchInterface)
		, SystemInstanceID(InSystemInstanceID)
		, bFinalPostSimulate(InFinalPostSimulate)
	{
	}

	FNiagaraSystemInstanceID GetSystemInstanceID() const { return SystemInstanceID; }
	bool IsFinalPostSimulate() const { return bFinalPostSimulate; }

	FNiagaraGPUInstanceCountManager& GetInstanceCountManager()const;

protected:
	FNiagaraSystemInstanceID SystemInstanceID = FNiagaraSystemInstanceID();
	bool bFinalPostSimulate = false;
};

//////////////////////////////////////////////////////////////////////////

struct FNiagaraDataInterfaceProxy
{
	FNiagaraDataInterfaceProxy() {}
	virtual ~FNiagaraDataInterfaceProxy()
	{
#if NIAGARA_VALIDATE_NDIPROXY_REFS
		checkf(ProxyTickRefs == 0, TEXT("NiagaraDataInterfaceProxy(%p) is still referenced but being destroyed."));
#endif
	}

	virtual int32 PerInstanceDataPassedToRenderThreadSize() const = 0;
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) { check(false); }

	// New data interface path
	virtual void ResetData(const FNDIGpuComputeResetContext& Context) { }
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) {}
	virtual void PostStage(const FNDIGpuComputePostStageContext& Context) {}
	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) {}

	virtual bool RequiresPreStageFinalize() const { return false; }
	virtual void FinalizePreStage(FRDGBuilder& GraphBuilder, const FNiagaraGpuComputeDispatchInterface& ComputeDispatchInterface) {}

	virtual bool RequiresPostStageFinalize() const { return false; }
	virtual void FinalizePostStage(FRDGBuilder& GraphBuilder, const FNiagaraGpuComputeDispatchInterface& ComputeDispatchInterface) {}

	virtual FNiagaraDataInterfaceProxyRW* AsIterationProxy() { return nullptr; }

	inline FName GetSourceDIName() const
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return IsInGameThread() || IsInParallelGameThread() ? SourceDIName : SourceDIName_RT;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetSourceDIName(FName InName);

#if NIAGARA_VALIDATE_NDIPROXY_REFS
	std::atomic<int32> ProxyTickRefs;
#endif

	// #todo(dmp): move all of this stuff to the RW interface to keep it out of here?
	UE_DEPRECATED(5.8, "Please update your API to use the get API to read SourceDIName")
	FName SourceDIName;

private:
	FName SourceDIName_RT;
};

//////////////////////////////////////////////////////////////////////////

struct FNiagaraDataInterfaceSetShaderParametersContext
{
	FNiagaraDataInterfaceSetShaderParametersContext(FRDGBuilder& InGraphBuilder, const FNiagaraGpuComputeDispatchInterface& InComputeDispatchInterface, const FNiagaraGPUSystemTick& InSystemTick, const FNiagaraComputeInstanceData& InComputeInstanceData, const FNiagaraSimStageData& InSimStageData, const FNiagaraShaderRef& InShaderRef, const FNiagaraShaderScriptParametersMetadata& InShaderParametersMetadata, uint8* InBaseParameters)
		: GraphBuilder(InGraphBuilder)
		, ComputeDispatchInterface(InComputeDispatchInterface)
		, SystemTick(InSystemTick)
		, ComputeInstanceData(InComputeInstanceData)
		, SimStageData(InSimStageData)
		, ShaderRef(InShaderRef)
		, ShaderParametersMetadata(InShaderParametersMetadata)
		, BaseParameters(InBaseParameters)
	{
	}

	FRDGBuilder& GetGraphBuilder() const { return GraphBuilder; }
	NIAGARA_API FRDGExternalAccessQueue& GetRDGExternalAccessQueue() const;

	template<typename T> T& GetProxy() const { check(DataInterfaceProxy); return static_cast<T&>(*DataInterfaceProxy); }
	const FNiagaraGpuComputeDispatchInterface& GetComputeDispatchInterface() const { return ComputeDispatchInterface; }
	const FNiagaraGPUSystemTick& GetSystemTick() const { return SystemTick; }
	const FNiagaraComputeInstanceData& GetComputeInstanceData() const { return ComputeInstanceData; }
	const FNiagaraSimStageData& GetSimStageData() const { return SimStageData; }

	NIAGARA_API FNiagaraSystemInstanceID GetSystemInstanceID() const;
	NIAGARA_API FVector3f GetSystemLWCTile() const;
	NIAGARA_API bool IsResourceBound(const void* ResourceAddress) const;
	NIAGARA_API bool IsParameterBound(const void* ParameterAddress) const;
	NIAGARA_API bool IsOutputStage() const;
	NIAGARA_API bool IsIterationStage() const;

	template<typename T> bool IsStructBound(const T* StructAddress) const
	{
		return IsStructBoundInternal(StructAddress, sizeof(T));
	}

	bool IsStructBound(const uint8* StructAddress, const FShaderParametersMetadata* StructMetadata) const
	{
		return IsStructBoundInternal(StructAddress, StructMetadata->GetSize());
	}

	template<typename T> T* GetParameterNestedStruct() const
	{
		const uint32 StructOffset = Align(ParametersOffset, TShaderParameterStructTypeInfo<T>::Alignment);
		ParametersOffset = StructOffset + TShaderParameterStructTypeInfo<T>::GetStructMetadata()->GetSize();

		return reinterpret_cast<T*>(BaseParameters + StructOffset);
	}
	inline uint8* GetParameterIncludedStruct(const FShaderParametersMetadata* StructMetadata) const
	{
		const uint16 StructOffset = GetParameterIncludedStructInternal(StructMetadata);
		return BaseParameters + StructOffset;
	}
	template<typename T> T* GetParameterIncludedStruct() const
	{
		return reinterpret_cast<T*>(GetParameterIncludedStruct(TShaderParameterStructTypeInfo<T>::GetStructMetadata()));
	}
	template<typename T> TArrayView<T> GetParameterLooseArray(int32 NumElements) const
	{
		const uint32 ArrayOffset = Align(ParametersOffset, SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT);
		ParametersOffset = ArrayOffset + (sizeof(T) * NumElements);
		return TArrayView<T>(reinterpret_cast<T*>(BaseParameters + ArrayOffset), NumElements);
	}

	template<typename T> const T& GetShaderStorage() const
	{
		check(ShaderStorage != nullptr);
		return static_cast<const T&>(*ShaderStorage);
	}

	void SetDataInterface(FNiagaraDataInterfaceProxy* InDataInterfaceProxy, uint32 InParametersOffset, const FNiagaraDataInterfaceParametersCS* InShaderStorage)
	{
		DataInterfaceProxy = InDataInterfaceProxy;
		ParametersOffset = InParametersOffset;
		ShaderStorage = InShaderStorage;
	}

#if WITH_NIAGARA_DEBUG_EMITTER_NAME
	/** Formats a string with the simulation & stage name */
	NIAGARA_API FString GetDebugString() const;
#endif

private:
	NIAGARA_API bool IsStructBoundInternal(const void* StructAddress, uint32 StructSize) const;
	NIAGARA_API uint16 GetParameterIncludedStructInternal(const FShaderParametersMetadata* StructMetadata) const;

private:
	FRDGBuilder& GraphBuilder;
	const FNiagaraGpuComputeDispatchInterface& ComputeDispatchInterface;
	const FNiagaraGPUSystemTick& SystemTick;
	const FNiagaraComputeInstanceData& ComputeInstanceData;
	const FNiagaraSimStageData& SimStageData;
	const FNiagaraShaderRef& ShaderRef;
	const FNiagaraShaderScriptParametersMetadata& ShaderParametersMetadata;
	uint8* BaseParameters = nullptr;
	const FNiagaraDataInterfaceParametersCS* ShaderStorage = nullptr;

	FNiagaraDataInterfaceProxy* DataInterfaceProxy = nullptr;
	mutable uint32 ParametersOffset = 0;
};

//////////////////////////////////////////////////////////////////////////

/** Context for pre and post stage ticks for DIs on the CPU. A similar tick flow also occurs for stages on the GPU via the proxy and dispatcher. */
struct FNDICpuPrePostStageContext
{
	FNiagaraSystemInstance* SystemInstance = nullptr;
	void* PerInstanceData = nullptr;
	float DeltaSeconds = 0.0f;
	ENiagaraScriptUsage Usage;

	template<typename T>
	T* GetPerInstanceData(){ return reinterpret_cast<T*>(PerInstanceData); }
};

using FNDICpuPreStageContext = FNDICpuPrePostStageContext;
using FNDICpuPostStageContext = FNDICpuPrePostStageContext;

/** A utility class to store a tick list for DIs in a system instance. Allows easy pre & post stage ticking for DIs at the various script executions. */
struct FNDIStageTickHandler
{
	/** Indices into the DIInstanceData info pair array the require this tick. */
	TArray<int32> PreStageTickList;
	TArray<int32> PostStageTickList;
	ENiagaraScriptUsage Usage;

	void Init(ENiagaraScriptUsage ScriptUsage, const FNiagaraScriptRuntimeData* ScriptRuntimeData, FNiagaraSystemInstance* Instance);
	void PreStageTick(FNiagaraSystemInstance* Instance, float DeltaSeconds);
	void PostStageTick(FNiagaraSystemInstance* Instance, float DeltaSeconds);
	void Empty()
	{
		PreStageTickList.Empty();
		PostStageTickList.Empty();
	}
};

//////////////////////////////////////////////////////////////////////////

/** Base class for all Niagara data interfaces. */
UCLASS(abstract, EditInlineNew, MinimalAPI)
class UNiagaraDataInterface : public UNiagaraDataInterfaceBase
{
	GENERATED_UCLASS_BODY()

public:
	NIAGARA_API virtual ~UNiagaraDataInterface() override;

	// UObject Interface
#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	NIAGARA_API virtual void PostLoad() override;
	// UObject Interface END

#if WITH_EDITOR
	/** Does this data interface need setup and teardown for each stage when working a sim stage sim source? */
	virtual bool SupportsSetupAndTeardownHLSL() const { return false; }
	/** Generate the necessary HLSL to set up data when being added as a sim stage sim source. */
	virtual bool GenerateSetupHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, TConstArrayView<FNiagaraVariable> InArguments, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const { return false;}
	/** Generate the necessary HLSL to tear down data when being added as a sim stage sim source. */
	virtual bool GenerateTeardownHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, TConstArrayView<FNiagaraVariable> InArguments, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const { return false; }
	/** Can this data interface be used as a StackContext parameter map replacement when being used as a sim stage iteration source? */
	virtual bool SupportsIterationSourceNamespaceAttributesHLSL() const { return false; }
	/** Generate the necessary plumbing HLSL at the beginning of the stage where this is used as a sim stage iteration source. Note that this should inject other internal calls using the CustomHLSL node syntax. See GridCollection2D for an example.*/
	virtual bool GenerateIterationSourceNamespaceReadAttributesHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, const FNiagaraVariable& InIterationSourceVariable, TConstArrayView<FNiagaraVariable> InArguments, TConstArrayView<FNiagaraVariable> InAttributes, TConstArrayView<FString> InAttributeHLSLNames, bool bInSetToDefaults, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const { return false; };
	/** Generate the necessary plumbing HLSL at the end of the stage where this is used as a sim stage iteration source. Note that this should inject other internal calls using the CustomHLSL node syntax. See GridCollection2D for an example.*/
	virtual bool GenerateIterationSourceNamespaceWriteAttributesHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, const FNiagaraVariable& InIterationSourceVariable, TConstArrayView<FNiagaraVariable> InArguments, TConstArrayView<FNiagaraVariable> InAttributes, TConstArrayView<FString> InAttributeHLSLNames, TConstArrayView<FNiagaraVariable> InAllAttributes, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const { return false; };
	/** Used by the translator when dealing with signatures that turn into compiler tags to figure out the precise compiler tag. */
	virtual bool GenerateCompilerTagPrefix(const FNiagaraFunctionSignature& InSignature, FString& OutPrefix) const  { return false; }

	virtual ENiagaraGpuDispatchType GetGpuDispatchType() const { return ENiagaraGpuDispatchType::OneD; }
	virtual FIntVector GetGpuDispatchNumThreads() const { return FIntVector(64, 1, 1); }
	virtual bool GetGpuUseIndirectDispatch() const { return false; }
#endif

	/** Allows data interfaces to cache any static data that may be shared between instances */
	virtual void CacheStaticBuffers(struct FNiagaraSystemStaticBuffers& StaticBuffers, bool bIsExternalDataInterface, bool bUsedByCPU, bool bUsedByGPU) {}

	virtual bool NeedsGPUContextInit() const { return false; }
	virtual bool GPUContextInit(const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDataInterfaceData, const TArray<FVMExternalFunctionBindingInfo>& CalledFunctions, void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) const { return false; }

	/** Initializes the per instance data for this interface. Returns false if there was some error and the simulation should be disabled. */
	virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) { return true; }

	/** Destroys the per instance data for this interface. */
	virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) {}

	/** Ticks the per instance data for this interface, if it has any. */
	virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) { return false; }
	virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) { return false; }

	/** Called per instance for each DI before each execution state in which it is used. */
	virtual void PreStageTick(FNDICpuPreStageContext& Context){}

	/** Called per instance for each DI after each execution state in which it is used. */
	virtual void PostStageTick(FNDICpuPostStageContext& Context){}
	
#if WITH_EDITORONLY_DATA
	/** Allows the generic class defaults version of this class to specify any dependencies/version/etc that might invalidate the compile. It should never depend on the value of specific properties.*/
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const;
#endif

#if WITH_EDITOR
	/** Allows data interfaces to influence the compilation of GPU shaders and is only called on the CDO object not the instance. */
	NIAGARA_API virtual void ModifyCompilationEnvironment(EShaderPlatform ShaderPlatform, struct FShaderCompilerEnvironment& OutEnvironment) const;

	/** Allows data interfaces to prevent compilation of GPU shaders and is only called on the CDO object not the instance. */
	virtual bool ShouldCompile(EShaderPlatform ShaderPlatform) const { return true; };
#endif

	/** 
		Subclasses that wish to work with GPU systems/emitters must implement this.
		Those interfaces must fill DataForRenderThread with the data needed to upload to the GPU. It will be the last thing called on this
		data interface for a specific tick.
		This will be consumed by the associated FNiagaraDataInterfaceProxy.
		Note: This class does not own the memory pointed to by DataForRenderThread. It will be recycled automatically. 
			However, if you allocate memory yourself to pass via this buffer you ARE responsible for freeing it when it is consumed by the proxy (Which is what ChaosDestruction does).
			Likewise, the class also does not own the memory in PerInstanceData. That pointer is the pointer passed to PerInstanceTick/PerInstanceTickPostSimulate.
			
		This will not be called if PerInstanceDataPassedToRenderThreadSize is 0.
	*/
	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
	{
		check(false);
	}

	/**
	 * The size of the data this class will provide to ProvidePerInstanceDataForRenderThread.
	 * MUST be 16 byte aligned!
	 */
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const 
	{ 
		if (!Proxy)
		{
			return 0;
		}
		check(Proxy);
		return Proxy->PerInstanceDataPassedToRenderThreadSize();
	}

	/** 
	Returns the size of the per instance data for this interface. 0 if this interface has no per instance data. 
	Must depend solely on the class of the interface and not on any particular member data of a individual interface.
	*/
	virtual int32 PerInstanceDataSize()const { return 0; }

#if WITH_EDITORONLY_DATA
	/**
	Gets all the available functions for this data interface.
	*/
	NIAGARA_API void GetFunctionSignatures(TArray<FNiagaraFunctionSignature>& OutFunctions) const;
#endif

	/** Returns the delegate for the passed function signature. */
	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) { };
	
	/** Copies the contents of this DataInterface to another.*/
	NIAGARA_API bool CopyTo(UNiagaraDataInterface* Destination) const;

	/** Determines if this DataInterface is the same as another.*/
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const;

	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target)const { return false; }

	virtual bool HasPreSimulateTick() const { return false; }
	virtual bool HasPostSimulateTick() const { return false; }
	virtual bool HasPreStageTick(ENiagaraScriptUsage Usage) const { return false; }
	virtual bool HasPostStageTick(ENiagaraScriptUsage Usage) const { return false; }

	UE_DEPRECATED(5.7, "Please update to the PostCompile that takes the owning system as this api will be removed in a future version.")
	virtual void PostCompile() { }

	/** Called after system compilation completes. Useful for caching any constant, compile dependent data. */
	virtual void PostCompile(const UNiagaraSystem& OwningSystem)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PostCompile();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	When set to true the simulation may not complete in the same frame it started, allowing maximum overlap with the GameThread.
	You must override and set to false if you require the data interface to complete before PostActorTick completes.
	*/
	virtual bool PostSimulateCanOverlapFrames() const { return true; }

	/**
	When set to true simulation may not complete in the same tick group it started.
	You must override and set to false if you require the data interface post stage tick to complete before the tick group ends.
	*/
	virtual bool PostStageCanOverlapTickGroups() const { return true; }

	UE_DEPRECATED(5.4, "RequiresDistanceFieldData was renamed to RequiresGlobalDistanceField.")
	virtual bool RequiresDistanceFieldData() const { return false; }

	virtual bool RequiresGlobalDistanceField() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return RequiresDistanceFieldData();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	virtual bool RequiresDepthBuffer() const { return false; }
	virtual bool RequiresEarlyViewData() const { return false; }
	virtual bool RequiresRayTracingScene() const { return false; }
	virtual bool RequiresCurrentFrameNDC() const { return false; }

	//Returns an estimate allocation count for the GPU instance count manager. This is experimental and will likely be modified in the future.
	virtual uint32 GetGpuCountBufferEstimate() const { return 0; }

	virtual bool HasTickGroupPrereqs() const { return false; }
	virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const { return NiagaraFirstTickGroup; }
	virtual bool HasTickGroupPostreqs() const { return false; }
	virtual ETickingGroup CalculateFinalTickGroup(const void* PerInstanceData) const { return NiagaraLastTickGroup; }

	/** Determines if this type definition matches to a known data interface type.*/
	static NIAGARA_API bool IsDataInterfaceType(const FNiagaraTypeDefinition& TypeDef);

#if WITH_EDITORONLY_DATA
	/** Allows data interfaces to provide common functionality that will be shared across interfaces on that type. */
	virtual void GetCommonHLSL(FString& OutHLSL)
	{
	}
	
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL);

	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL);

	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
	{
	}

	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
	{
		return false;
	}

	/**
	Allows data interfaces the opportunity to rename / change the function signature and perform an upgrade.
	Return true if the signature was modified and we need to refresh the pins / name, etc.
	*/
	virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
	{
		return false;
	}

	/** Formats and appends a template file onto the output HLSL */
	NIAGARA_API void AppendTemplateHLSL(FString& OutHLSL, const TCHAR* TemplateShaderFile, const TMap<FString, FStringFormatArg>& TemplateArgs) const;
#endif

#if WITH_NIAGARA_DEBUGGER
	/**
	Override this function to provide additional context to the debug HUD.
	Fill VariableDataString with the data to display on the variable information in the HUD, this information
	should be kept light to avoid polluting the display.
	You can also use the Canvas to draw additional information based on verbosity
	*/
	virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DrawDebugHud(DebugHudContext.GetCanvas(), const_cast<FNiagaraSystemInstance*>(DebugHudContext.GetSystemInstance()), DebugHudContext.GetOutputString(), DebugHudContext.IsVerbose());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// Deprecated method will be removed in a future version
	UE_DEPRECATED(5.3, "Please update to the context based DrawDebugHud as this will be removed in a future version.")
	virtual void DrawDebugHud(UCanvas* Canvas, FNiagaraSystemInstance* SystemInstance, FString& VariableDataString, bool bVerbose) const {}
#endif

#if WITH_EDITOR	
	/** Refreshes and returns the errors detected with the corresponding data, if any.*/
	virtual TArray<FNiagaraDataInterfaceError> GetErrors() { return TArray<FNiagaraDataInterfaceError>(); }
	
	/**
		Query the data interface to give feedback to the end user. 
		Note that the default implementation, just calls GetErrors on the DataInterface, but derived classes can do much more.
		Also, InAsset or InComponent may be null values, as the UI for DataInterfaces is displayed in a variety of locations. 
		In these cases, only provide information that is relevant to that context.
	*/
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo);

	static NIAGARA_API void GetFeedback(UNiagaraDataInterface* DataInterface, TArray<FNiagaraDataInterfaceError>& Errors, TArray<FNiagaraDataInterfaceFeedback>& Warnings,
		TArray<FNiagaraDataInterfaceFeedback>& Info);

	/** Validates a function being compiled and allows interface classes to post custom compile errors when their API changes. */
	NIAGARA_API virtual void ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors);

	NIAGARA_API void RefreshErrors();

	NIAGARA_API FSimpleMulticastDelegate& OnErrorsRefreshed();

#endif

    /** Method to add asset tags that are specific to this data interface. By default we add in how many instances of this class exist in the list.*/
	NIAGARA_API virtual void GetAssetTagsForContext(const UObject* InAsset, FGuid AssetVersion, const TArray<const UNiagaraDataInterface*>& InProperties, TMap<FName, uint32>& NumericKeys, TMap<FName, FString>& StringKeys) const;
	virtual bool CanExposeVariables() const { return false; }
	virtual void GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const {}
	virtual bool GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const { return false; }

	virtual bool CanRenderVariablesToCanvas() const { return false; }
	virtual void GetCanvasVariables(TArray<FNiagaraVariableBase>& OutVariables) const { }
	virtual bool RenderVariableToCanvas(FNiagaraSystemInstanceID SystemInstanceID, FName VariableName, class FCanvas* Canvas, const FIntRect& DrawRect) const { return false; }

	FNiagaraDataInterfaceProxy* GetProxy()
	{
		return Proxy.Get();
	}

	/**
	* Allows a DI to specify data dependencies between emitters, so the system can ensure that the emitter instances are executed in the correct order.
	* The Dependencies array may already contain items, and this method should only append to it.
	*/
	virtual void GetEmitterDependencies(UNiagaraSystem* Asset, TArray<FVersionedNiagaraEmitter>& Dependencies) const {}

	UE_DEPRECATED(5.4, "Implement GetEmitterReferencesByName() instead as this will be removed in a future version.")
	virtual bool ReadsEmitterParticleData(const FString& EmitterName) const
	{
		TArray<FString> EmitterReferences;
		GetEmitterReferencesByName(EmitterReferences);

		return EmitterReferences.Contains(EmitterName);
	}

	virtual void GetEmitterReferencesByName(TArray<FString>& EmitterReferences) const {};

	/**
	Set the shader parameters will only be called if the data interface provided shader parameters.
	You must write the parameters in the order you added them to the structure.
	*/
	virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const {}

	static NIAGARA_API EObjectFlags BuildObjectFlagsForOwner(UObject* OwnerObject, EObjectFlags DefaultFlags = RF_NoFlags);

protected:
	virtual void PushToRenderThreadImpl() {}

	// deprecated function for backwards compatibility.  Callers should be using GetFunctionSignatures
	// and sub classes should be implementing GetFunctionsInternal().
	UE_DEPRECATED(5.4, "GetFunctions() should be renamed GetFunctionsInternal() and guarded with WITH_EDITORONLY_DATA.")
	virtual void GetFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions)
	{
#if WITH_EDITORONLY_DATA
		GetFunctionsInternal(OutFunctions);
#endif
	}

#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const {};
#endif

public:
	void PushToRenderThread()
	{
		if (bUsedWithGPUScript && bRenderDataDirty)
		{
			PushToRenderThreadImpl();
			bRenderDataDirty = false;
		}
	}

	void MarkRenderDataDirty()
	{
		bRenderDataDirty = true;
		PushToRenderThread();
	}

	void SetUsedWithCPUScript(bool bUsed = true)
	{
		check(IsInGameThread());
		bUsedWithCPUScript = bUsed;
	}

	void SetUsedWithGPUScript(bool bUsed = true)
	{
		check(IsInGameThread());
		bUsedWithGPUScript = bUsed;
		PushToRenderThread();
	}

	UE_DEPRECATED(5.3, "Please update your code to use IsUsedWithCPUScript")
	bool IsUsedByCPUEmitter() const { return bUsedWithCPUScript; }
	UE_DEPRECATED(5.3, "Please update your code to use IsUsedWithGPUScript")
	bool IsUsedByGPUEmitter() const { return bUsedWithGPUScript; }
	UE_DEPRECATED(5.3, "Please update your code to use IsUsedWithCPUScript")
	bool IsUsedWithCPUEmitter() const { return bUsedWithCPUScript; }
	UE_DEPRECATED(5.3, "Please update your code to use IsUsedWithGPUScript")
	bool IsUsedWithGPUEmitter() const { return bUsedWithGPUScript; }

	/** Used to determine if we need to create CPU resources for the emitter. */
	bool IsUsedWithCPUScript() const { return bUsedWithCPUScript; }
	/** Used to determine if we need to create GPU resources for the emitter. */
	bool IsUsedWithGPUScript() const { return bUsedWithGPUScript; }

protected:
	template<typename T>
	T* GetProxyAs()
	{
		T* TypedProxy = static_cast<T*>(Proxy.Get());
		check(TypedProxy != nullptr);
		return TypedProxy;
	}

	template<typename T>
	const T* GetProxyAs() const
	{
		const T* TypedProxy = static_cast<const T*>(Proxy.Get());
		check(TypedProxy != nullptr);
		return TypedProxy;
	}

	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const;

	TUniquePtr<FNiagaraDataInterfaceProxy> Proxy;

	uint32 bRenderDataDirty : 1;
	uint32 bUsedWithCPUScript : 1;
	uint32 bUsedWithGPUScript : 1;

private:
#if WITH_EDITOR
	FSimpleMulticastDelegate OnErrorsRefreshedDelegate;
#endif

#if WITH_EDITORONLY_DATA
	mutable FRWLock CachedFunctionSignatureLock;
	mutable TArray<FNiagaraFunctionSignature> CachedFunctionSignatures;
#endif
};

/** Helper class for decoding NDI parameters into a usable struct type. */
template<typename T>
struct FNDIParameter
{
	FNDIParameter(FVectorVMExternalFunctionContext& Context) = delete;
	inline void GetAndAdvance(T& OutValue) = delete;
	inline bool IsConstant() const = delete;
};

template<>
struct FNDIParameter<FNiagaraRandInfo>
{
	VectorVM::FExternalFuncInputHandler<int32> Seed1Param;
	VectorVM::FExternalFuncInputHandler<int32> Seed2Param;
	VectorVM::FExternalFuncInputHandler<int32> Seed3Param;
	
	FVectorVMExternalFunctionContext& Context;

	FNDIParameter(FVectorVMExternalFunctionContext& InContext)
		: Context(InContext)
	{
		Seed1Param.Init(Context);
		Seed2Param.Init(Context);
		Seed3Param.Init(Context);
	}

	inline void GetAndAdvance(FNiagaraRandInfo& OutValue)
	{
		OutValue.Seed1 = Seed1Param.GetAndAdvance();
		OutValue.Seed2 = Seed2Param.GetAndAdvance();
		OutValue.Seed3 = Seed3Param.GetAndAdvance();
	}


	inline bool IsConstant()const
	{
		return Seed1Param.IsConstant() && Seed2Param.IsConstant() && Seed3Param.IsConstant();
	}
};

// Completely random policy which will pull from the contexts random stream
struct FNDIRandomStreamPolicy
{
	FNDIRandomStreamPolicy(FVectorVMExternalFunctionContext& InContext) : Context(InContext) {}

	inline void GetAndAdvance() {}
	inline bool IsDeterministic() const { return false; }
	inline FVector4 Rand4(int32 InstanceIndex) const { return FVector4(Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction()); }
	inline FVector Rand3(int32 InstanceIndex) const { return FVector(Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction()); }
	inline FVector2D Rand2(int32 InstanceIndex) const { return FVector2D(Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction()); }
	inline float Rand(int32 InstanceIndex) const { return Context.GetRandStream().GetFraction(); }

	FVectorVMExternalFunctionContext& Context;
};

// Random policy which can be optionally deterministic depending on the info
struct FNDIRandomInfoPolicy
{
	FNDIRandomInfoPolicy(FVectorVMExternalFunctionContext& InContext)
		: Context(InContext)
		, RandParam(InContext)
	{}

	inline void GetAndAdvance() { RandParam.GetAndAdvance(RandInfo); }
	inline bool IsDeterministic() const { return RandInfo.Seed3 != INDEX_NONE; }

	//////////////////////////////////////////////////////////////////////////
	inline FVector4f Rand4(int32 InstanceIndex) const
	{
		if (IsDeterministic())
		{
			int32 RandomCounter = Context.GetRandCounters()[InstanceIndex]++;

			FIntVector4 v = FIntVector4(RandomCounter, RandInfo.Seed1, RandInfo.Seed2, RandInfo.Seed3) * 1664525 + FIntVector4(1013904223);

			v.X += v.Y * v.W;
			v.Y += v.Z * v.X;
			v.Z += v.X * v.Y;
			v.W += v.Y * v.Z;
			v.X += v.Y * v.W;
			v.Y += v.Z * v.X;
			v.Z += v.X * v.Y;
			v.W += v.Y * v.Z;

			// NOTE(mv): We can use 24 bits of randomness, as all integers in [0, 2^24] 
			//           are exactly representable in single precision floats.
			//           We use the upper 24 bits as they tend to be higher quality.

			// NOTE(mv): The divide can often be folded with the range scale in the rand functions
			FVector4f OutValue;
			OutValue.X = float((v.X >> 8) & 0x00ffffff) / 16777216.0f; // 0x01000000 == 16777216
			OutValue.Y = float((v.Y >> 8) & 0x00ffffff) / 16777216.0f;
			OutValue.Z = float((v.Z >> 8) & 0x00ffffff) / 16777216.0f;
			OutValue.W = float((v.W >> 8) & 0x00ffffff) / 16777216.0f;
			return OutValue;
			// return float4((v >> 8) & 0x00ffffff) * (1.0/16777216.0); // bugged, see UE-67738
		}
		else
		{
			return FVector4f(Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction());
		}
	}

	inline FVector3f Rand3(int32 InstanceIndex) const
	{
		if (IsDeterministic())
		{
			int32 RandomCounter = Context.GetRandCounters()[InstanceIndex]++;

			FIntVector v = FIntVector(RandInfo.Seed1, RandInfo.Seed2, RandomCounter | (RandInfo.Seed3 << 16)) * 1664525 + FIntVector(1013904223);

			v.X += v.Y * v.Z;
			v.Y += v.Z * v.X;
			v.Z += v.X * v.Y;
			v.X += v.Y * v.Z;
			v.Y += v.Z * v.X;
			v.Z += v.X * v.Y;

			FVector3f OutValue;
			OutValue.X = float((v.X >> 8) & 0x00ffffff) / 16777216.0f; // 0x01000000 == 16777216
			OutValue.Y = float((v.Y >> 8) & 0x00ffffff) / 16777216.0f;
			OutValue.Z = float((v.Z >> 8) & 0x00ffffff) / 16777216.0f;
			return OutValue;
		}
		else
		{
			return FVector3f(Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction());
		}
	}

	inline FVector2f Rand2(int32 InstanceIndex) const
	{
		if (IsDeterministic())
		{
			FVector3f Rand3D = Rand3(InstanceIndex);
			return FVector2f(Rand3D.X, Rand3D.Y);
		}
		else
		{
			return FVector2f(Context.GetRandStream().GetFraction(), Context.GetRandStream().GetFraction());
		}
	}

	inline float Rand(int32 InstanceIndex) const
	{
		if (IsDeterministic())
		{
			return Rand3(InstanceIndex).X;
		}
		else
		{
			return Context.GetRandStream().GetFraction();
		}
	}

	FVectorVMExternalFunctionContext& Context;
	FNDIParameter<FNiagaraRandInfo> RandParam;
	FNiagaraRandInfo RandInfo;
};

template<typename TRandomPolicy>
struct TNDIRandomHelper : public TRandomPolicy
{
	TNDIRandomHelper(FVectorVMExternalFunctionContext& InContext)
		: TRandomPolicy(InContext)
	{
	}

	inline FVector4 RandRange(int32 InstanceIndex, FVector4 Min, FVector4 Max) const
	{
		FVector4 Range = Max - Min;
		return Min + (TRandomPolicy::Rand(InstanceIndex) * Range);
	}

	inline FVector RandRange(int32 InstanceIndex, FVector Min, FVector Max) const
	{
		FVector Range = Max - Min;
		return Min + (TRandomPolicy::Rand(InstanceIndex) * Range);
	}

	inline FVector2D RandRange(int32 InstanceIndex, FVector2D Min, FVector2D Max) const
	{
		FVector2D Range = Max - Min;
		return Min + (TRandomPolicy::Rand(InstanceIndex) * Range);
	}

	inline float RandRange(int32 InstanceIndex, float Min, float Max) const
	{
		float Range = Max - Min;
		return Min + (TRandomPolicy::Rand(InstanceIndex) * Range);
	}

	inline int32 RandRange(int32 InstanceIndex, int32 Min, int32 Max) const
	{
		// NOTE: Scaling a uniform float range provides better distribution of 
		//       numbers than using %.
		// NOTE: Inclusive! So [0, x] instead of [0, x)
		int32 Range = Max - Min;
		return Min + (int(TRandomPolicy::Rand(InstanceIndex) * (Range + 1)));
	}

	inline FVector3f RandomBarycentricCoord(int32 InstanceIndex) const
	{
		//TODO: This is gonna be slooooow. Move to an LUT possibly or find faster method.
		//Can probably handle lower quality randoms / uniformity for a decent speed win.
		FVector2f r = FVector2f(TRandomPolicy::Rand2(InstanceIndex));
		float sqrt0 = FMath::Sqrt(r.X);
		return FVector3f(1.0f - sqrt0, sqrt0 * (1.0f - r.Y), r.Y * sqrt0);
	}
};

using FNDIRandomHelper = TNDIRandomHelper<FNDIRandomInfoPolicy>;
using FNDIRandomHelperFromStream = TNDIRandomHelper<FNDIRandomStreamPolicy>;

//Helper to deal with types with potentially several input registers.
template<typename T>
struct FNDIInputParam
{
	static_assert(sizeof(T) == sizeof(float), "Generic template assumes 4 bytes per element");
	VectorVM::FExternalFuncInputHandler<T> Data;	
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : Data(Context) {}
	inline FNDIInputParam(){ }
	inline void Init(FVectorVMExternalFunctionContext& Context){ new(this)FNDIInputParam<T>(Context); }
	inline T GetAndAdvance() { return Data.GetAndAdvance(); }
	inline T Get() { return Data.Get(); }
	inline void Advance(int32 Count=1) { return Data.Advance(Count); }
	inline bool IsConstant() const { return Data.IsConstant(); }
	inline void Reset() { Data.Reset(); }
};

template<>
struct FNDIInputParam<FFloat16>
{
	VectorVM::FExternalFuncInputHandler<FFloat16> Data;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : Data(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<FFloat16>(Context); }
	inline FFloat16 GetAndAdvance() { return Data.GetAndAdvance(); }
	inline FFloat16 Get() { return Data.Get(); }
	inline void Advance(int32 Count=1) { return Data.Advance(Count); }
	inline bool IsConstant() const { return Data.IsConstant(); }
	inline void Reset() { Data.Reset(); }
};

template<>
struct FNDIInputParam<FNiagaraBool>
{
	VectorVM::FExternalFuncInputHandler<FNiagaraBool> Data;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : Data(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<FNiagaraBool>(Context); }
	inline bool GetAndAdvance() { return Data.GetAndAdvance().GetValue(); }
	inline bool Get() { return Data.Get().GetValue(); }
	inline void Advance(int32 Count=1) { return Data.Advance(Count); }
	inline bool IsConstant() const { return Data.IsConstant(); }
	inline void Reset() { Data.Reset(); }
};

template<>
struct FNDIInputParam<bool>
{
	VectorVM::FExternalFuncInputHandler<FNiagaraBool> Data;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : Data(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<bool>(Context); }
	inline bool GetAndAdvance() { return Data.GetAndAdvance().GetValue(); }
	inline bool Get() { return Data.Get().GetValue(); }
	inline void Advance(int32 Count=1) { return Data.Advance(Count); }
	inline bool IsConstant() const { return Data.IsConstant(); }
	inline void Reset() { Data.Reset(); }
};

template<>
struct FNDIInputParam<FVector2f>
{
	VectorVM::FExternalFuncInputHandler<float> X;
	VectorVM::FExternalFuncInputHandler<float> Y;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<FVector2f>(Context); }
	inline FVector2f GetAndAdvance() { return FVector2f(X.GetAndAdvance(), Y.GetAndAdvance()); }
	inline FVector2f Get() { return FVector2f(X.Get(), Y.Get()); }
	inline void Advance(int32 Count=1) { X.Advance(Count), Y.Advance(Count); }
	inline bool IsConstant() const { return X.IsConstant() && Y.IsConstant(); }
	inline void Reset() { X.Reset(); Y.Reset(); }
};

template<>
struct FNDIInputParam<FVector3f>
{
	VectorVM::FExternalFuncInputHandler<float> X;
	VectorVM::FExternalFuncInputHandler<float> Y;
	VectorVM::FExternalFuncInputHandler<float> Z;
	FNDIInputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<FVector3f>(Context); }
	inline FVector3f GetAndAdvance() { return FVector3f(X.GetAndAdvance(), Y.GetAndAdvance(), Z.GetAndAdvance()); }
	inline FVector3f Get() { return FVector3f(X.Get(), Y.Get(), Z.Get()); }
	inline void Advance(int32 Count=1) { X.Advance(Count); Y.Advance(Count); Z.Advance(Count); }
	inline bool IsConstant() const { return X.IsConstant() && Y.IsConstant() && Z.IsConstant(); }
	inline void Reset() { X.Reset(); Y.Reset(); Z.Reset(); }
};

template<>
struct FNDIInputParam<FNiagaraPosition>
{
	VectorVM::FExternalFuncInputHandler<float> X;
	VectorVM::FExternalFuncInputHandler<float> Y;
	VectorVM::FExternalFuncInputHandler<float> Z;
	FNDIInputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<FNiagaraPosition>(Context); }
	inline FNiagaraPosition GetAndAdvance() { return FNiagaraPosition(X.GetAndAdvance(), Y.GetAndAdvance(), Z.GetAndAdvance()); }
	inline FNiagaraPosition Get() { return FNiagaraPosition(X.Get(), Y.Get(), Z.Get()); }
	inline void Advance(int32 Count=1) { X.Advance(Count); Y.Advance(Count); Z.Advance(Count); }
	inline void Reset() { X.Reset(); Y.Reset(); Z.Reset(); }
};
template<>
struct FNDIInputParam<FVector4f>
{
	VectorVM::FExternalFuncInputHandler<float> X;
	VectorVM::FExternalFuncInputHandler<float> Y;
	VectorVM::FExternalFuncInputHandler<float> Z;
	VectorVM::FExternalFuncInputHandler<float> W;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context), W(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<FVector4f>(Context); }
	inline FVector4f GetAndAdvance() { return FVector4f(X.GetAndAdvance(), Y.GetAndAdvance(), Z.GetAndAdvance(), W.GetAndAdvance()); }
	inline FVector4f Get() { return FVector4f(X.Get(), Y.Get(), Z.Get(), W.Get()); }
	inline void Advance(int32 Count=1) { return X.Advance(Count); Y.Advance(Count); Z.Advance(Count); W.Advance(Count); }
	inline bool IsConstant() const { return X.IsConstant() && Y.IsConstant() && Z.IsConstant() && W.IsConstant(); }
	inline void Reset() { X.Reset(); Y.Reset(); Z.Reset(); W.Reset(); }
};

template<>
struct FNDIInputParam<FQuat4f>
{
	VectorVM::FExternalFuncInputHandler<float> X;
	VectorVM::FExternalFuncInputHandler<float> Y;
	VectorVM::FExternalFuncInputHandler<float> Z;
	VectorVM::FExternalFuncInputHandler<float> W;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context), W(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<FQuat4f>(Context); }
	inline FQuat4f GetAndAdvance() { return FQuat4f(X.GetAndAdvance(), Y.GetAndAdvance(), Z.GetAndAdvance(), W.GetAndAdvance()); }
	inline FQuat4f Get() { return FQuat4f(X.Get(), Y.Get(), Z.Get(), W.Get()); }
	inline void Advance(int32 Count=1) { X.Advance(Count); Y.Advance(Count); Z.Advance(Count); W.Advance(Count); }
	inline bool IsConstant() const { return X.IsConstant() && Y.IsConstant() && Z.IsConstant() && W.IsConstant(); }
	inline void Reset() { X.Reset(); Y.Reset(); Z.Reset(); W.Reset(); }
};

template<>
struct FNDIInputParam<FMatrix44f>
{
	VectorVM::FExternalFuncInputHandler<float> M[16];

	inline FNDIInputParam() { }
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) { Init(Context); }
	inline void Init(FVectorVMExternalFunctionContext& Context)
	{
		for (int32 i=0; i < 16; ++i)
		{
			M[i].Init(Context);
		}
	}
	inline FMatrix44f GetAndAdvance()
	{
		FMatrix44f Matrix;
		for (int32 i=0; i < 16; ++i)
		{
			Matrix.M[i/4][i%4] = M[i].GetAndAdvance();
		}
		return Matrix;
	}
	inline FMatrix44f Get()
	{
		FMatrix44f Matrix;
		for (int32 i = 0; i < 16; ++i)
		{
			Matrix.M[i / 4][i % 4] = M[i].Get();
		}
		return Matrix;
	}
	inline void Advance(int32 Count = 1)
	{
		for (int32 i = 0; i < 16; ++i)
		{
			M[i].Advance(Count);
		}
	}
	inline bool IsConstant() const
	{
		bool bConstant = true;
		for (int32 i = 0; i < 16; ++i)
		{
			bConstant &= M[i].IsConstant();
		}
		return bConstant;
	}
	inline void Reset()
	{
		for (int32 i = 0; i < 16; ++i)
		{
			M[i].Reset();
		}
	}
};

template<>
struct FNDIInputParam<FLinearColor>
{
	VectorVM::FExternalFuncInputHandler<float> R;
	VectorVM::FExternalFuncInputHandler<float> G;
	VectorVM::FExternalFuncInputHandler<float> B;
	VectorVM::FExternalFuncInputHandler<float> A;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : R(Context), G(Context), B(Context), A(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<FLinearColor>(Context); }
	inline FLinearColor GetAndAdvance() { return FLinearColor(R.GetAndAdvance(), G.GetAndAdvance(), B.GetAndAdvance(), A.GetAndAdvance()); }
	inline FLinearColor Get() { return FLinearColor(R.Get(), G.Get(), B.Get(), A.Get()); }
	inline void Advance(int32 Count=1) { R.Advance(Count); G.Advance(Count); B.Advance(Count); A.Advance(Count); }
	inline bool IsConstant() const { return R.IsConstant() && G.IsConstant() && B.IsConstant() && A.IsConstant(); }
	inline void Reset() { R.Reset(); G.Reset(); B.Reset(); A.Reset(); }
};

template<>
struct FNDIInputParam<FNiagaraID>
{
	VectorVM::FExternalFuncInputHandler<int32> Index;
	VectorVM::FExternalFuncInputHandler<int32> AcquireTag;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : Index(Context), AcquireTag(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<FNiagaraID>(Context); }
	inline FNiagaraID GetAndAdvance() { return FNiagaraID(Index.GetAndAdvance(), AcquireTag.GetAndAdvance()); }
	inline FNiagaraID Get() { return FNiagaraID(Index.Get(), AcquireTag.Get()); }
	inline void Advance(int32 Count=1) { return Index.Advance(Count); AcquireTag.Advance(Count); }
	inline bool IsConstant() const { return Index.IsConstant() && AcquireTag.IsConstant(); }
	inline void Reset() { Index.Reset(); AcquireTag.Reset(); }
};

template<>
struct FNDIInputParam<FNiagaraSpawnInfo>
{
	VectorVM::FExternalFuncInputHandler<int32> Count;
	VectorVM::FExternalFuncInputHandler<float> InterpStartDt;
	VectorVM::FExternalFuncInputHandler<float> IntervalDt;
	VectorVM::FExternalFuncInputHandler<int32> SpawnGroup;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : Count(Context), InterpStartDt(Context), IntervalDt(Context), SpawnGroup(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<FNiagaraSpawnInfo>(Context); }
	inline FNiagaraSpawnInfo GetAndAdvance() { return FNiagaraSpawnInfo(Count.GetAndAdvance(), InterpStartDt.GetAndAdvance(),IntervalDt.GetAndAdvance(),SpawnGroup.GetAndAdvance()); }
	inline FNiagaraSpawnInfo Get() { return FNiagaraSpawnInfo(Count.Get(), InterpStartDt.Get(),IntervalDt.Get(),SpawnGroup.Get()); }
	inline void Advance(int32 InCount=1) { return Count.Advance(InCount); InterpStartDt.Advance(InCount);  IntervalDt.Advance(InCount);  SpawnGroup.Advance(InCount); }
	inline bool IsConstant() const { return Count.IsConstant() && InterpStartDt.IsConstant() && IntervalDt.IsConstant() && SpawnGroup.IsConstant(); }
	inline void Reset() { Count.Reset(); InterpStartDt.Reset();  IntervalDt.Reset(); SpawnGroup.Reset();}
};

template<>
struct FNDIInputParam<FNiagaraEmitterID>
{
	VectorVM::FExternalFuncInputHandler<int32> Index;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : Index(Context) {}
	inline FNDIInputParam() { }
	inline void Init(FVectorVMExternalFunctionContext& Context) { new(this)FNDIInputParam<FNiagaraEmitterID>(Context); }
	inline FNiagaraEmitterID GetAndAdvance() { return FNiagaraEmitterID(Index.GetAndAdvance()); }
	inline FNiagaraEmitterID Get() { return FNiagaraEmitterID(Index.Get()); }
	inline void Advance(int32 Count = 1) { return Index.Advance(Count); }
	inline bool IsConstant() const { return Index.IsConstant(); }
	inline void Reset() { Index.Reset(); }
};

template<>
struct FNDIInputParam<FIntVector2>
{
	VectorVM::FExternalFuncInputHandler<int32> X;
	VectorVM::FExternalFuncInputHandler<int32> Y;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context) {}
	inline FIntVector2 GetAndAdvance() { return FIntVector2(X.GetAndAdvance(), Y.GetAndAdvance()); }
	inline bool IsConstant() const { return X.IsConstant() && Y.IsConstant(); }
};

template<>
struct FNDIInputParam<FIntVector3>
{
	VectorVM::FExternalFuncInputHandler<int32> X;
	VectorVM::FExternalFuncInputHandler<int32> Y;
	VectorVM::FExternalFuncInputHandler<int32> Z;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context) {}
	inline FIntVector3 GetAndAdvance() { return FIntVector3(X.GetAndAdvance(), Y.GetAndAdvance(), Z.GetAndAdvance()); }
	inline bool IsConstant() const { return X.IsConstant() && Y.IsConstant() && Z.IsConstant(); }
};

template<>
struct FNDIInputParam<FIntVector4>
{
	VectorVM::FExternalFuncInputHandler<int32> X;
	VectorVM::FExternalFuncInputHandler<int32> Y;
	VectorVM::FExternalFuncInputHandler<int32> Z;
	VectorVM::FExternalFuncInputHandler<int32> W;
	inline FNDIInputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context), W(Context) {}
	inline FIntVector4 GetAndAdvance() { return FIntVector4(X.GetAndAdvance(), Y.GetAndAdvance(), Z.GetAndAdvance(), W.GetAndAdvance()); }
	inline bool IsConstant() const { return X.IsConstant() && Y.IsConstant() && Z.IsConstant() && W.IsConstant(); }
};

//Helper to deal with types with potentially several output registers.
template<typename T>
struct FNDIOutputParam
{
	static_assert(sizeof(T) == sizeof(float), "Generic template assumes 4 bytes per element");
	VectorVM::FExternalFuncRegisterHandler<T> Data;
	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : Data(Context) {}
	inline bool IsValid() const { return Data.IsValid();  }
	inline void SetAndAdvance(T Val) { *Data.GetDestAndAdvance() = Val; }
};

template<>
struct FNDIOutputParam<FNiagaraBool>
{
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> Data;
	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : Data(Context) {}
	inline bool IsValid() const { return Data.IsValid(); }
	inline void SetAndAdvance(bool Val) { Data.GetDestAndAdvance()->SetValue(Val); }
	inline void SetAndAdvance(FNiagaraBool Val) { *Data.GetDestAndAdvance() = Val; }
};

template<>
struct FNDIOutputParam<bool>
{
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> Data;
	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : Data(Context) {}
	inline bool IsValid() const { return Data.IsValid(); }
	inline void SetAndAdvance(bool Val) { Data.GetDestAndAdvance()->SetValue(Val); }
};

template<>
struct FNDIOutputParam<FVector2f>
{
	VectorVM::FExternalFuncRegisterHandler<float> X;
	VectorVM::FExternalFuncRegisterHandler<float> Y;
	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context) {}
	inline bool IsValid() const { return X.IsValid() || Y.IsValid(); }
	inline void SetAndAdvance(FVector2f Val)
	{
		*X.GetDestAndAdvance() = Val.X;
		*Y.GetDestAndAdvance() = Val.Y;
	}
};

template<>
struct FNDIOutputParam<FVector3f>
{
	VectorVM::FExternalFuncRegisterHandler<float> X;
	VectorVM::FExternalFuncRegisterHandler<float> Y;
	VectorVM::FExternalFuncRegisterHandler<float> Z;
	FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context) {}
	inline bool IsValid() const { return X.IsValid() || Y.IsValid() || Z.IsValid(); }
	inline void SetAndAdvance(FVector3f Val)
	{
		*X.GetDestAndAdvance() = Val.X;
		*Y.GetDestAndAdvance() = Val.Y;
		*Z.GetDestAndAdvance() = Val.Z;
	}
};

template<>
struct FNDIOutputParam<FNiagaraPosition>
{
	VectorVM::FExternalFuncRegisterHandler<float> X;
	VectorVM::FExternalFuncRegisterHandler<float> Y;
	VectorVM::FExternalFuncRegisterHandler<float> Z;
	FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context) {}
	inline bool IsValid() const { return X.IsValid() || Y.IsValid() || Z.IsValid(); }
	inline void SetAndAdvance(FNiagaraPosition Val)
	{
		*X.GetDestAndAdvance() = Val.X;
		*Y.GetDestAndAdvance() = Val.Y;
		*Z.GetDestAndAdvance() = Val.Z;
	}
};

template<>
struct FNDIOutputParam<FVector4f>
{
	VectorVM::FExternalFuncRegisterHandler<float> X;
	VectorVM::FExternalFuncRegisterHandler<float> Y;
	VectorVM::FExternalFuncRegisterHandler<float> Z;
	VectorVM::FExternalFuncRegisterHandler<float> W;
	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context), W(Context) {}
	inline bool IsValid() const { return X.IsValid() || Y.IsValid() || Z.IsValid() || W.IsValid(); }
	inline void SetAndAdvance(FVector4f Val)
	{
		*X.GetDestAndAdvance() = Val.X;
		*Y.GetDestAndAdvance() = Val.Y;
		*Z.GetDestAndAdvance() = Val.Z;
		*W.GetDestAndAdvance() = Val.W;
	}
};

template<>
struct FNDIOutputParam<FQuat4f>
{
	VectorVM::FExternalFuncRegisterHandler<float> X;
	VectorVM::FExternalFuncRegisterHandler<float> Y;
	VectorVM::FExternalFuncRegisterHandler<float> Z;
	VectorVM::FExternalFuncRegisterHandler<float> W;
	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context), W(Context) {}
	inline bool IsValid() const { return X.IsValid() || Y.IsValid() || Z.IsValid() || W.IsValid(); }
	inline void SetAndAdvance(FQuat4f Val)
	{
		*X.GetDestAndAdvance() = Val.X;
		*Y.GetDestAndAdvance() = Val.Y;
		*Z.GetDestAndAdvance() = Val.Z;
		*W.GetDestAndAdvance() = Val.W;
	}
};

template<>
struct FNDIOutputParam<FMatrix44f>
{
	VectorVM::FExternalFuncRegisterHandler<float> Out00;
	VectorVM::FExternalFuncRegisterHandler<float> Out01;
	VectorVM::FExternalFuncRegisterHandler<float> Out02;
	VectorVM::FExternalFuncRegisterHandler<float> Out03;
	VectorVM::FExternalFuncRegisterHandler<float> Out04;
	VectorVM::FExternalFuncRegisterHandler<float> Out05;
	VectorVM::FExternalFuncRegisterHandler<float> Out06;
	VectorVM::FExternalFuncRegisterHandler<float> Out07;
	VectorVM::FExternalFuncRegisterHandler<float> Out08;
	VectorVM::FExternalFuncRegisterHandler<float> Out09;
	VectorVM::FExternalFuncRegisterHandler<float> Out10;
	VectorVM::FExternalFuncRegisterHandler<float> Out11;
	VectorVM::FExternalFuncRegisterHandler<float> Out12;
	VectorVM::FExternalFuncRegisterHandler<float> Out13;
	VectorVM::FExternalFuncRegisterHandler<float> Out14;
	VectorVM::FExternalFuncRegisterHandler<float> Out15;

	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : Out00(Context), Out01(Context), Out02(Context), Out03(Context), Out04(Context), Out05(Context),
		Out06(Context), Out07(Context), Out08(Context), Out09(Context), Out10(Context), Out11(Context), Out12(Context), Out13(Context), Out14(Context), Out15(Context)	{}
	inline bool IsValid() const { return Out00.IsValid(); }
	inline void SetAndAdvance(const FMatrix44f& Val)
	{
		*Out00.GetDestAndAdvance() = Val.M[0][0];
		*Out01.GetDestAndAdvance() = Val.M[0][1];
		*Out02.GetDestAndAdvance() = Val.M[0][2];
		*Out03.GetDestAndAdvance() = Val.M[0][3];
		*Out04.GetDestAndAdvance() = Val.M[1][0];
		*Out05.GetDestAndAdvance() = Val.M[1][1];
		*Out06.GetDestAndAdvance() = Val.M[1][2];
		*Out07.GetDestAndAdvance() = Val.M[1][3];
		*Out08.GetDestAndAdvance() = Val.M[2][0];
		*Out09.GetDestAndAdvance() = Val.M[2][1];
		*Out10.GetDestAndAdvance() = Val.M[2][2];
		*Out11.GetDestAndAdvance() = Val.M[2][3];
		*Out12.GetDestAndAdvance() = Val.M[3][0];
		*Out13.GetDestAndAdvance() = Val.M[3][1];
		*Out14.GetDestAndAdvance() = Val.M[3][2];
		*Out15.GetDestAndAdvance() = Val.M[3][3];
	}
};

template<>
struct FNDIOutputParam<FLinearColor>
{
	VectorVM::FExternalFuncRegisterHandler<float> R;
	VectorVM::FExternalFuncRegisterHandler<float> G;
	VectorVM::FExternalFuncRegisterHandler<float> B;
	VectorVM::FExternalFuncRegisterHandler<float> A;
	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : R(Context), G(Context), B(Context), A(Context) {}
	inline bool IsValid() const { return R.IsValid() || G.IsValid() || B.IsValid() || A.IsValid(); }
	inline void SetAndAdvance(FLinearColor Val)
	{
		*R.GetDestAndAdvance() = Val.R;
		*G.GetDestAndAdvance() = Val.G;
		*B.GetDestAndAdvance() = Val.B;
		*A.GetDestAndAdvance() = Val.A;
	}
};

template<>
struct FNDIOutputParam<FNiagaraID>
{
	VectorVM::FExternalFuncRegisterHandler<int32> Index;
	VectorVM::FExternalFuncRegisterHandler<int32> AcquireTag;
	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : Index(Context), AcquireTag(Context) {}
	inline bool IsValid() const { return Index.IsValid() || AcquireTag.IsValid(); }
	inline void SetAndAdvance(FNiagaraID Val)
	{
		*Index.GetDestAndAdvance() = Val.Index;
		*AcquireTag.GetDestAndAdvance() = Val.AcquireTag;
	}
};

template<>
struct FNDIOutputParam<FIntVector2>
{
	VectorVM::FExternalFuncRegisterHandler<int32> X;
	VectorVM::FExternalFuncRegisterHandler<int32> Y;
	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context) {}
	inline bool IsValid() const { return X.IsValid() || Y.IsValid(); }
	inline void SetAndAdvance(FIntVector2 Val)
	{
		*X.GetDestAndAdvance() = Val.X;
		*Y.GetDestAndAdvance() = Val.Y;
	}
};

template<>
struct FNDIOutputParam<FIntVector3>
{
	VectorVM::FExternalFuncRegisterHandler<int32> X;
	VectorVM::FExternalFuncRegisterHandler<int32> Y;
	VectorVM::FExternalFuncRegisterHandler<int32> Z;
	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context) {}
	inline bool IsValid() const { return X.IsValid() || Y.IsValid() || Z.IsValid(); }
	inline void SetAndAdvance(FIntVector3 Val)
	{
		*X.GetDestAndAdvance() = Val.X;
		*Y.GetDestAndAdvance() = Val.Y;
		*Z.GetDestAndAdvance() = Val.Z;
	}
};

template<>
struct FNDIOutputParam<FIntVector4>
{
	VectorVM::FExternalFuncRegisterHandler<int32> X;
	VectorVM::FExternalFuncRegisterHandler<int32> Y;
	VectorVM::FExternalFuncRegisterHandler<int32> Z;
	VectorVM::FExternalFuncRegisterHandler<int32> W;
	inline FNDIOutputParam(FVectorVMExternalFunctionContext& Context) : X(Context), Y(Context), Z(Context), W(Context) {}
	inline bool IsValid() const { return X.IsValid() || Y.IsValid() || Z.IsValid() || W.IsValid(); }
	inline void SetAndAdvance(FIntVector4 Val)
	{
		*X.GetDestAndAdvance() = Val.X;
		*Y.GetDestAndAdvance() = Val.Y;
		*Z.GetDestAndAdvance() = Val.Z;
		*W.GetDestAndAdvance() = Val.W;
	}
};

class FNDI_GeneratedData
{
public:
	virtual ~FNDI_GeneratedData() = default;

	typedef uint32 TypeHash;

	virtual void Tick(ETickingGroup TickGroup, float DeltaSeconds) = 0;
};

class FNDI_SharedResourceUsage
{
public:
	FNDI_SharedResourceUsage() = default;
	FNDI_SharedResourceUsage(bool InRequiresCpuAccess, bool InRequiresGpuAccess)
		: RequiresCpuAccess(InRequiresCpuAccess)
		, RequiresGpuAccess(InRequiresGpuAccess)
	{}

	bool IsValid() const { return RequiresCpuAccess || RequiresGpuAccess; }

	bool RequiresCpuAccess = false;
	bool RequiresGpuAccess = false;
};

template<typename ResourceType, typename UsageType>
class FNDI_SharedResourceHandle
{
	using HandleType = FNDI_SharedResourceHandle<ResourceType, UsageType>;

public:
	FNDI_SharedResourceHandle()
		: Resource(nullptr)
	{}

	FNDI_SharedResourceHandle(UsageType InUsage, const TSharedPtr<ResourceType>& InResource, bool bNeedsDataImmediately)
		: Usage(InUsage)
		, Resource(InResource)
	{
		if (ResourceType* ResourceData = Resource.Get())
		{
			ResourceData->RegisterUser(Usage, bNeedsDataImmediately);
		}
	}

	FNDI_SharedResourceHandle(const HandleType& Other) = delete;
	FNDI_SharedResourceHandle(HandleType&& Other)
		: Usage(Other.Usage)
		, Resource(Other.Resource)
	{
		Other.Resource = nullptr;
	}

	~FNDI_SharedResourceHandle()
	{
		if (ResourceType* ResourceData = Resource.Get())
		{
			ResourceData->UnregisterUser(Usage);
		}
	}

	FNDI_SharedResourceHandle& operator=(const HandleType& Other) = delete;
	FNDI_SharedResourceHandle& operator=(HandleType&& Other)
	{
		if (this != &Other)
		{
			if (ResourceType* ResourceData = Resource.Get())
			{
				ResourceData->UnregisterUser(Usage);
			}

			Usage = Other.Usage;
			Resource = Other.Resource;
			Other.Resource = nullptr;
		}

		return *this;
	}

	explicit operator bool() const
	{
		return Resource.IsValid();
	}

	const ResourceType& ReadResource() const
	{
		return *Resource;
	}

	UsageType Usage;

private:
	TSharedPtr<ResourceType> Resource;
};
