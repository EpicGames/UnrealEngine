// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "NiagaraDataInterface2DArrayTexture.generated.h"

class UTexture2DArray;

/** Data Interface allowing sampling of a texture */
UCLASS(EditInlineNew, Category = "Texture", CollapseCategories, meta = (DisplayName = "2D Array Texture Sample"), MinimalAPI)
class UNiagaraDataInterface2DArrayTexture : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FVector3f,					TextureSize)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray,	Texture)
		SHADER_PARAMETER_SAMPLER(SamplerState,		TextureSampler)
	END_SHADER_PARAMETER_STRUCT()

public:
	UPROPERTY(EditAnywhere, Category = "Texture", meta = (AllowedClasses = "/Script/Engine.Texture2DArray,/Script/Engine.TextureRenderTarget2DArray"))
	TObjectPtr<UTexture> Texture;

	UPROPERTY(EditAnywhere, Category = "Texture", meta = (ToolTip = "When valid the user parameter is used as the texture rather than the one on the data interface"))
	FNiagaraUserParameterBinding TextureUserParameter;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties()override;
	virtual bool CanBeInCluster() const override { return false; }	// Note: Due to BP functionality we can change a UObject property on this DI we can not put into a cluster
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target)const override { return Target == ENiagaraSimTarget::GPUComputeSim; }

	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;

	virtual bool HasPreSimulateTick() const override { return true; }
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	//UNiagaraDataInterface Interface End

	NIAGARA_API void GetTextureDimensions(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API void SetTexture(UTexture* InTexture);

protected:
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

protected:
	static NIAGARA_API const TCHAR* TemplateShaderFilePath;
	static NIAGARA_API const FName LoadTextureName;
	static NIAGARA_API const FName GatherRedTextureName;
	static NIAGARA_API const FName SampleTextureName;
	static NIAGARA_API const FName TextureDimsName;
};
