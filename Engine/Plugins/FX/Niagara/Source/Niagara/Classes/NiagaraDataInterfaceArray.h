// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TransactionallySafeRWLock.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraShader.h"
#include "NiagaraSimCacheCustomStorageInterface.h"
#include "ShaderParameterUtils.h"
#include "RHIUtilities.h"

#include "NiagaraDataInterfaceArray.generated.h"

template<typename TArrayType, class TOwnerType>
struct FNDIArrayProxyImpl;
class UNDIArraySimCacheData;
struct FNDIArraySimCacheDataFrame;

#define NDIARRAY_GENERATE_BODY(CLASSNAME, TYPENAME, MEMBERNAME) \
	using FProxyType = FNDIArrayProxyImpl<TYPENAME, CLASSNAME>; \
	virtual void PostInitProperties() override; \
	template<typename TFromArrayType> \
	void SetVariantArrayData(TConstArrayView<TFromArrayType> InArrayData); \
	template<typename TFromArrayType> \
	void SetVariantArrayValue(int Index, const TFromArrayType& Value, bool bSizeToFit); \
	const TArray<TYPENAME>& GetArrayReference() const { return MEMBERNAME; } \
	TArray<TYPENAME>& GetArrayReference() { return MEMBERNAME; }

#if WITH_EDITORONLY_DATA
	#define NDIARRAY_GENERATE_BODY_LWC(CLASSNAME, TYPENAME, MEMBERNAME) \
		using FProxyType = FNDIArrayProxyImpl<TYPENAME, CLASSNAME>; \
		virtual void PostInitProperties() override; \
		virtual void PostLoad() override; \
		virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override; \
		virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override; \
		virtual bool Equals(const UNiagaraDataInterface* Other) const override; \
		template<typename TFromArrayType> \
		void SetVariantArrayData(TConstArrayView<TFromArrayType> InArrayData); \
		template<typename TFromArrayType> \
		void SetVariantArrayValue(int Index, const TFromArrayType& Value, bool bSizeToFit); \
		const TArray<TYPENAME>& GetArrayReference() const { return Internal##MEMBERNAME; } \
		TArray<TYPENAME>& GetArrayReference() { return Internal##MEMBERNAME; }
#else
	#define NDIARRAY_GENERATE_BODY_LWC(CLASSNAME, TYPENAME, MEMBERNAME) NDIARRAY_GENERATE_BODY(CLASSNAME, TYPENAME, Internal##MEMBERNAME)
#endif

struct INDIArrayProxyBase : public FNiagaraDataInterfaceProxyRW
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
		SHADER_PARAMETER(FIntPoint,		ArrayBufferParams)
		SHADER_PARAMETER_SRV(Buffer,	ArrayReadBuffer)
		SHADER_PARAMETER_UAV(RWBuffer,	ArrayRWBuffer)
	END_SHADER_PARAMETER_STRUCT()

	virtual ~INDIArrayProxyBase() = default;
	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) = 0;
#if WITH_EDITORONLY_DATA
	virtual void GetFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const = 0;

	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) const = 0;
	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) const = 0;
	virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const = 0;
	virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) const = 0;
#endif
#if WITH_NIAGARA_DEBUGGER
	virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const = 0;
#endif
	virtual bool SimCacheWriteFrame(UNDIArraySimCacheData* CacheData, int FrameIndex, FNiagaraSystemInstance* SystemInstance) const = 0;
	virtual bool SimCacheReadFrame(UNDIArraySimCacheData* CacheData, int FrameIndex, FNiagaraSystemInstance* SystemInstance) = 0;
	virtual bool SimCacheCompareElement(const uint8* LhsData, const uint8* RhsData, int32 Element, float Tolerance) const = 0;
	virtual FString SimCacheVisualizerRead(const UNDIArraySimCacheData* CacheData, const FNDIArraySimCacheDataFrame& FrameData, int Element) const = 0;

	virtual bool CopyToInternal(INDIArrayProxyBase* Destination) const = 0;
	virtual bool Equals(const INDIArrayProxyBase* Other) const = 0;
	virtual int32 PerInstanceDataSize() const = 0;
	virtual bool InitPerInstanceData(UNiagaraDataInterface* DataInterface, void* InPerInstanceData, FNiagaraSystemInstance* SystemInstance) = 0;
	virtual void DestroyPerInstanceData(void* InPerInstanceData, FNiagaraSystemInstance* SystemInstance) = 0;
	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) = 0;
	virtual void SetShaderParameters(FShaderParameters* ShaderParameters, FNiagaraSystemInstanceID SystemInstanceID) const = 0;
};

UCLASS(abstract, EditInlineNew, MinimalAPI)
class UNiagaraDataInterfaceArray : public UNiagaraDataInterfaceRWBase, public INiagaraSimCacheCustomStorageInterface
{
	GENERATED_UCLASS_BODY()

public:
	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//UObject Interface End

	//UNiagaraDataInterface Interface
	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override { GetProxyAs<INDIArrayProxyBase>()->GetVMExternalFunction(BindingInfo, InstanceData, OutFunc); }
#if WITH_EDITORONLY_DATA
	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override { GetProxyAs<INDIArrayProxyBase>()->GetParameterDefinitionHLSL(ParamInfo, OutHLSL); }
	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override { return GetProxyAs<INDIArrayProxyBase>()->GetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, OutHLSL); }
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override { return GetProxyAs<INDIArrayProxyBase>()->UpgradeFunctionCall(FunctionSignature); }
#endif
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

#if WITH_NIAGARA_DEBUGGER
	virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const { GetProxyAs<INDIArrayProxyBase>()->DrawDebugHud(DebugHudContext); }
#endif

	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const;
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const;

	virtual int32 PerInstanceDataSize() const override { return GetProxyAs<INDIArrayProxyBase>()->PerInstanceDataSize(); }
	virtual bool InitPerInstanceData(void* InPerInstanceData, FNiagaraSystemInstance* SystemInstance) override { return GetProxyAs<INDIArrayProxyBase>()->InitPerInstanceData(this, InPerInstanceData, SystemInstance); }
	virtual void DestroyPerInstanceData(void* InPerInstanceData, FNiagaraSystemInstance * SystemInstance) override { GetProxyAs<INDIArrayProxyBase>()->DestroyPerInstanceData(InPerInstanceData, SystemInstance); }

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override { GetProxyAs<INDIArrayProxyBase>()->ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, SystemInstance); }

	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	//UNiagaraDataInterface Interface

	//~ INiagaraSimCacheCustomStorageInterface interface BEGIN
	NIAGARA_API virtual UObject* SimCacheBeginWrite(UObject* SimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const override;
	NIAGARA_API virtual bool SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const override;
	//virtual bool SimCacheEndWrite(UObject* StorageObject) const override;
	NIAGARA_API virtual bool SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext) override;
	NIAGARA_API virtual bool SimCacheCompareFrame(const UObject* LhsStorageObject, const UObject* RhsStorageObject, int FrameIndex, TOptional<float> Tolerance, FString& OutErrors) const override;
	//~ INiagaraSimCacheCustomStorageInterface interface END
	NIAGARA_API FString SimCacheVisualizerRead(const UNDIArraySimCacheData* CacheData, const FNDIArraySimCacheDataFrame& FrameData, int Element) const;

#if WITH_EDITOR
	NIAGARA_API void CreateAndSetVariant(UNiagaraComponent* NiagaraComponent, FName OverrideName, TFunction<void(UNiagaraDataInterfaceArray*)> SetDataDelegate);
#endif

	/** ReadWrite lock to ensure safe access to the underlying array. */
	FTransactionallySafeRWLock ArrayRWGuard;

	/** How to do we want to synchronize modifications to the array data. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category="Array")
	ENiagaraGpuSyncMode GpuSyncMode = ENiagaraGpuSyncMode::SyncCpuToGpu;

	/** When greater than 0 sets the maximum number of elements the array can hold, only relevant when using operations that modify the array size. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category="Array", meta=(ClampMin="0"))
	int32 MaxElements;

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override { GetProxyAs<INDIArrayProxyBase>()->GetFunctions(OutFunctions); }
#endif
};

USTRUCT()
struct FNDIArraySimCacheDataFrame
{
	GENERATED_BODY()
		
	UPROPERTY()
	int32 NumElements = 0;

	UPROPERTY()
	int32 DataOffset = INDEX_NONE;
};

UCLASS(MinimalAPI)
class UNDIArraySimCacheData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<FNDIArraySimCacheDataFrame> CpuFrameData;

	UPROPERTY()
	TArray<FNDIArraySimCacheDataFrame> GpuFrameData;

	UPROPERTY()
	TArray<uint8> BufferData;

	// Internal use only
	NIAGARA_API int32 FindOrAddData(TConstArrayView<uint8> ArrayData);

protected:
	UE::FMutex FindOrAddGuard;
};
