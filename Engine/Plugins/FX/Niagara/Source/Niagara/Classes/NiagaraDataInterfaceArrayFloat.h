// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceArray.h"
#include "NiagaraDataInterfaceArrayFloat.generated.h"

UCLASS(EditInlineNew, Category = "Array", CollapseCategories, meta = (DisplayName = "Float Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayFloat : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<float> FloatData;

	NDIARRAY_GENERATE_BODY(UNiagaraDataInterfaceArrayFloat, float, FloatData)
};

UCLASS(EditInlineNew, Category = "Array", CollapseCategories, meta = (DisplayName = "Vector 2D Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayFloat2 : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<FVector2D> FloatData;
#endif

	UPROPERTY()
	TArray<FVector2f> InternalFloatData;

	NDIARRAY_GENERATE_BODY_LWC(UNiagaraDataInterfaceArrayFloat2, FVector2f, FloatData)
};

UCLASS(EditInlineNew, Category = "Array", CollapseCategories, meta = (DisplayName = "Vector Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayFloat3 : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<FVector> FloatData;
#endif

	UPROPERTY()
	TArray<FVector3f> InternalFloatData;

	NDIARRAY_GENERATE_BODY_LWC(UNiagaraDataInterfaceArrayFloat3, FVector3f, FloatData)
};

UCLASS(EditInlineNew, Category = "Array", CollapseCategories, meta = (DisplayName = "Position Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayPosition : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

#if WITH_NIAGARA_DEBUGGER
	virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<FNiagaraPosition> PositionData;

	NDIARRAY_GENERATE_BODY(UNiagaraDataInterfaceArrayPosition, FNiagaraPosition, PositionData)
};

UCLASS(EditInlineNew, Category = "Array", CollapseCategories, meta = (DisplayName = "Vector 4 Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayFloat4 : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<FVector4> FloatData;
#endif

	UPROPERTY()
	TArray<FVector4f> InternalFloatData;

	NDIARRAY_GENERATE_BODY_LWC(UNiagaraDataInterfaceArrayFloat4, FVector4f, FloatData)
};

UCLASS(EditInlineNew, Category = "Array", CollapseCategories, meta = (DisplayName = "Color Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayColor : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<FLinearColor> ColorData;

	NDIARRAY_GENERATE_BODY(UNiagaraDataInterfaceArrayColor, FLinearColor, ColorData)
};

UCLASS(EditInlineNew, Category = "Array", CollapseCategories, meta = (DisplayName = "Quaternion Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayQuat : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<FQuat> QuatData;
#endif

	UPROPERTY()
	TArray<FQuat4f> InternalQuatData;

	NDIARRAY_GENERATE_BODY_LWC(UNiagaraDataInterfaceArrayQuat, FQuat4f, QuatData)
};

UCLASS(EditInlineNew, Category = "Array", CollapseCategories, meta = (DisplayName = "Matrix Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayMatrix : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<FMatrix> MatrixData;
#endif

	UPROPERTY()
	TArray<FMatrix44f> InternalMatrixData;

	NDIARRAY_GENERATE_BODY_LWC(UNiagaraDataInterfaceArrayMatrix, FMatrix44f, MatrixData)
};
