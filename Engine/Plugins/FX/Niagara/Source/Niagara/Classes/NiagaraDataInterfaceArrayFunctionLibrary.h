// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "NiagaraDataInterfaceArrayFunctionLibrary.generated.h"

class UNiagaraComponent;

/**
* C++ and Blueprint library for accessing array types
*/
UCLASS(MinimalAPI)
class UNiagaraDataInterfaceArrayFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Sets Niagara Array Float Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta=(DisplayName = "Niagara Set Float Array"))
	static NIAGARA_API void SetNiagaraArrayFloat(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<float>& ArrayData);
	/** Sets Niagara Array FVector2D Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Vector 2D Array"))
	static NIAGARA_API void SetNiagaraArrayVector2D(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<FVector2D>& ArrayData);
	/** Sets Niagara Array FVector Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Vector Array"))
	static NIAGARA_API void SetNiagaraArrayVector(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<FVector>& ArrayData);
	/** Sets Niagara Array FVector Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Position Array"))
	static NIAGARA_API void SetNiagaraArrayPosition(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<FVector>& ArrayData);
	/** Sets Niagara Array FVector4 Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Vector 4 Array"))
	static NIAGARA_API void SetNiagaraArrayVector4(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<FVector4>& ArrayData);
	/** Sets Niagara Array FLinearColor Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Color Array"))
	static NIAGARA_API void SetNiagaraArrayColor(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<FLinearColor>& ArrayData);
	/** Sets Niagara Array FQuat Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Quaternion Array"))
	static NIAGARA_API void SetNiagaraArrayQuat(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<FQuat>& ArrayData);
	/**
	 * Sets Niagara Array FMatrix Data.
	 * @param bApplyLWCRebase When enabled the matrix translation will have the simulation tile offset subtracted from it
	 */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Matrix Array"))
	static NIAGARA_API void SetNiagaraArrayMatrix(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<FMatrix>& ArrayData, bool bApplyLWCRebase = true);
	/** Sets Niagara Array Int32 Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Int32 Array"))
	static NIAGARA_API void SetNiagaraArrayInt32(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<int32>& ArrayData);
	/** Sets Niagara Array UInt8 Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set UInt8 Array"))
	static NIAGARA_API void SetNiagaraArrayUInt8(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<int32>& ArrayData);
	/** Sets Niagara Array Bool Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Bool Array"))
	static NIAGARA_API void SetNiagaraArrayBool(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<bool>& ArrayData);

	/** Gets a copy of Niagara Float Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Float Array"))
	static NIAGARA_API TArray<float> GetNiagaraArrayFloat(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName);
	/** Gets a copy of Niagara FVector2D Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Vector 2D Array"))
	static NIAGARA_API TArray<FVector2D> GetNiagaraArrayVector2D(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName);
	/** Gets a copy of Niagara FVector Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Vector Array"))
	static NIAGARA_API TArray<FVector> GetNiagaraArrayVector(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName);
	/** Gets a copy of Niagara Position Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Position Array"))
	static NIAGARA_API TArray<FVector> GetNiagaraArrayPosition(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName);
	/** Gets a copy of Niagara FVector4 Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Vector 4 Array"))
	static NIAGARA_API TArray<FVector4> GetNiagaraArrayVector4(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName);
	/** Gets a copy of Niagara FLinearColor Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Color Array"))
	static NIAGARA_API TArray<FLinearColor> GetNiagaraArrayColor(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName);
	/** Gets a copy of Niagara FQuat Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Quaternion Array"))
	static NIAGARA_API TArray<FQuat> GetNiagaraArrayQuat(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName);
	/**
	 * Gets a copy of Niagara FMatrix Data.
	 * @param bApplyLWCRebase When enabled the matrix translation will have the simulation tile offset added to it
	 */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Matrix Array"))
	static NIAGARA_API TArray<FMatrix> GetNiagaraArrayMatrix(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, bool bApplyLWCRebase = true);
	/** Gets a copy of Niagara Int32 Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Int32 Array"))
	static NIAGARA_API TArray<int32> GetNiagaraArrayInt32(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName);
	/** Gets a copy of Niagara UInt8 Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get UInt8 Array"))
	static NIAGARA_API TArray<int32> GetNiagaraArrayUInt8(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName);
	/** Gets a copy of Niagara Bool Data. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Bool Array"))
	static NIAGARA_API TArray<bool> GetNiagaraArrayBool(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName);

	/** Sets a single value within a Niagara Array Float. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Float Array Value"))
	static NIAGARA_API void SetNiagaraArrayFloatValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, float Value, bool bSizeToFit);
	/** Sets a single value within a Niagara Array FVector2D. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Vector 2D Array Value"))
	static NIAGARA_API void SetNiagaraArrayVector2DValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, const FVector2D& Value, bool bSizeToFit);
	/** Sets a single value within a Niagara Array FVector. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Vector Array Value"))
	static NIAGARA_API void SetNiagaraArrayVectorValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, const FVector& Value, bool bSizeToFit);
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Position Array Value"))
	static NIAGARA_API void SetNiagaraArrayPositionValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, const FVector& Value, bool bSizeToFit);
	/** Sets a single value within a Niagara Array FVector4. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Vector 4 Array Value"))
	static NIAGARA_API void SetNiagaraArrayVector4Value(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, const FVector4& Value, bool bSizeToFit);
	/** Sets a single value within a Niagara Array FLinearColor. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Color Array Value"))
	static NIAGARA_API void SetNiagaraArrayColorValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, const FLinearColor& Value, bool bSizeToFit);
	/** Sets a single value within a Niagara Array FQuat. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Quaternion Array Value"))
	static NIAGARA_API void SetNiagaraArrayQuatValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, const FQuat& Value, bool bSizeToFit);
	/**
	 * Sets a single value within a Niagara Array FMatrix.
	 * @param bApplyLWCRebase When enabled the matrix translation will have the simulation tile offset subtracted from it
	 */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Matrix Array Value"))
	static NIAGARA_API void SetNiagaraArrayMatrixValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, const FMatrix& Value, bool bSizeToFit, bool bApplyLWCRebase = true);
	/** Sets a single value within a Niagara Array Int32. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Int32 Array Value"))
	static NIAGARA_API void SetNiagaraArrayInt32Value(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, int32 Value, bool bSizeToFit);
	/** Sets a single value within a Niagara Array UInt8. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set UInt8 Array Value"))
	static NIAGARA_API void SetNiagaraArrayUInt8Value(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, int32 Value, bool bSizeToFit);
	/** Sets a single value within a Niagara Array Bool. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Bool Array Value"))
	static NIAGARA_API void SetNiagaraArrayBoolValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, const bool& Value, bool bSizeToFit);

	/** Gets a single value within a Niagara Array Float. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Float Array Value"))
	static NIAGARA_API float GetNiagaraArrayFloatValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index);
	/** Gets a single value within a Niagara Array FVector2D. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Vector 2D Array Value"))
	static NIAGARA_API FVector2D GetNiagaraArrayVector2DValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index);
	/** Gets a single value within a Niagara Array FVector. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Vector Array Value"))
	static NIAGARA_API FVector GetNiagaraArrayVectorValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index);
	/** Gets a single value within a Niagara Array Position. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Position Array Value"))
	static NIAGARA_API FVector GetNiagaraArrayPositionValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index);
	/** Gets a single value within a Niagara Array FVector4. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Vector 4 Array Value"))
	static NIAGARA_API FVector4 GetNiagaraArrayVector4Value(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index);
	/** Gets a single value within a Niagara Array FLinearColor. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Color Array Value"))
	static NIAGARA_API FLinearColor GetNiagaraArrayColorValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index);
	/** Gets a single value within a Niagara Array FQuat. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Quaternion Array Value"))
	static NIAGARA_API FQuat GetNiagaraArrayQuatValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index);
	/**
	 * Gets a single value within a Niagara Array FMatrix.
	 * @param bApplyLWCRebase When enabled the matrix translation will have the simulation tile offset added to it
	 */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Matrix Array Value"))
	static NIAGARA_API FMatrix GetNiagaraArrayMatrixValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index, bool bApplyLWCRebase = true);
	/** Gets a single value within a Niagara Array Int32. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Int32 Array Value"))
	static NIAGARA_API int32 GetNiagaraArrayInt32Value(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index);
	/** Gets a single value within a Niagara Array UInt8. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get UInt8 Array Value"))
	static NIAGARA_API int32 GetNiagaraArrayUInt8Value(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index);
	/** Gets a single value within a Niagara Array Bool. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Get Bool Array Value"))
	static NIAGARA_API bool GetNiagaraArrayBoolValue(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, int Index);

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// None BP compatable set functions
	static NIAGARA_API void SetNiagaraArrayInt32(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<int64> ArrayData);
	static NIAGARA_API void SetNiagaraArrayFloat(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<double> ArrayData);
	static NIAGARA_API void SetNiagaraArrayVector2D(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<FVector2f> ArrayData);
	static NIAGARA_API void SetNiagaraArrayVector(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<FVector3f> ArrayData);
	static NIAGARA_API void SetNiagaraArrayVector4(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<FVector4f> ArrayData);
	static NIAGARA_API void SetNiagaraArrayQuat(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<FQuat4f> ArrayData);
	static NIAGARA_API void SetNiagaraArrayMatrix(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<FMatrix44f> ArrayData);
	static NIAGARA_API void SetNiagaraArrayUInt8(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<uint8> ArrayData);
};
