// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraClearCounts.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraDataInterfaceArray.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGpuReadbackManager.h"
#include "NiagaraSystemInstance.h"

#include "Async/Async.h"
#include "Containers/ArrayView.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"
#include "ShaderCompilerCore.h"

//////////////////////////////////////////////////////////////////////////
// Internal array data interface implementation
// WARNING: Do not use this is external code as the API is subject to change and is not guaranteed to support backwards compatability

//////////////////////////////////////////////////////////////////////////
// Helpers

#define NDIARRAY_GENERATE_IMPL(CLASSNAME, TYPENAME, MEMBERNAME) \
	void CLASSNAME::PostInitProperties() \
	{ \
		Proxy.Reset(new FProxyType(this)); \
		Super::PostInitProperties(); \
	} \
	template<typename TFromArrayType> \
	void CLASSNAME::SetVariantArrayData(TConstArrayView<TFromArrayType> InArrayData) \
	{ \
		MEMBERNAME = InArrayData; \
	} \
	template<typename TFromArrayType> \
	void CLASSNAME::SetVariantArrayValue(int Index, const TFromArrayType& Value, bool bSizeToFit) \
	{ \
		const int NumRequired = Index + 1 - MEMBERNAME.Num(); \
		if ( NumRequired > 0 && !bSizeToFit ) return; \
		MEMBERNAME.AddDefaulted(FMath::Max(NumRequired, 0)); \
		MEMBERNAME[Index] = Value; \
	}

#if WITH_EDITORONLY_DATA
	#define NDIARRAY_GENERATE_IMPL_LWC(CLASSNAME, TYPENAME, MEMBERNAME) \
		void CLASSNAME::PostInitProperties() \
		{ \
			Super::PostInitProperties(); \
			Proxy.Reset(new FProxyType(this)); \
		} \
		void CLASSNAME::PostLoad() \
		{ \
			Super::PostLoad(); \
			GetProxyAs<FProxyType>()->template SetArrayData<decltype(MEMBERNAME)::ElementType>(MakeArrayView(MEMBERNAME)); \
		} \
		void CLASSNAME::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) \
		{ \
			Super::PostEditChangeProperty(PropertyChangedEvent); \
			GetProxyAs<FProxyType>()->template SetArrayData<decltype(MEMBERNAME)::ElementType>(MakeArrayView(MEMBERNAME)); \
		} \
		bool CLASSNAME::CopyToInternal(UNiagaraDataInterface* Destination) const \
		{ \
			if ( Super::CopyToInternal(Destination) == false ) \
			{ \
				return false; \
			} \
			CLASSNAME* TypedDestination = Cast<CLASSNAME>(Destination); \
			if ( TypedDestination != nullptr ) \
			{ \
				TypedDestination->MEMBERNAME = MEMBERNAME; \
				TypedDestination->GetProxyAs<FProxyType>()->template SetArrayData<decltype(MEMBERNAME)::ElementType>(MakeArrayView(TypedDestination->MEMBERNAME)); \
			} \
			return TypedDestination != nullptr;  \
		} \
		bool CLASSNAME::Equals(const UNiagaraDataInterface* Other) const \
		{ \
			const CLASSNAME* TypedOther = Cast<const CLASSNAME>(Other); \
			return \
				Super::Equals(Other) && \
				TypedOther != nullptr && \
				TypedOther->MEMBERNAME == MEMBERNAME; \
		} \
		template<typename TFromArrayType> \
		void CLASSNAME::SetVariantArrayData(TConstArrayView<TFromArrayType> InArrayData) \
		{ \
			if constexpr (std::is_same_v<TFromArrayType, decltype(MEMBERNAME)::ElementType>) \
			{ \
				MEMBERNAME = InArrayData; \
				GetProxyAs<FProxyType>()->template SetArrayData<decltype(MEMBERNAME)::ElementType>(InArrayData); \
			} \
			else \
			{ \
				MEMBERNAME.SetNumUninitialized(InArrayData.Num()); \
				FNDIArrayImplHelper<TYPENAME>::CopyCpuToCpuMemory(MEMBERNAME.GetData(), InArrayData.GetData(), InArrayData.Num()); \
				GetProxyAs<FProxyType>()->template SetArrayData<decltype(Internal##MEMBERNAME)::ElementType>(InArrayData); \
			} \
		} \
		template<typename TFromArrayType> \
		void CLASSNAME::SetVariantArrayValue(int Index, const TFromArrayType& Value, bool bSizeToFit) \
		{ \
			const int NumRequired = Index + 1 - MEMBERNAME.Num(); \
			if ( NumRequired > 0 && !bSizeToFit ) return; \
			MEMBERNAME.AddDefaulted(FMath::Max(NumRequired, 0)); \
			MEMBERNAME[Index] = Value; \
			GetProxyAs<FProxyType>()->template SetArrayData<decltype(MEMBERNAME)::ElementType>(MEMBERNAME); \
		}
#else
	#define NDIARRAY_GENERATE_IMPL_LWC(CLASSNAME, TYPENAME, MEMBERNAME) NDIARRAY_GENERATE_IMPL(CLASSNAME, TYPENAME, Internal##MEMBERNAME)
#endif

template<typename TArrayType>
struct FNDIArrayImplHelperBase
{
	static constexpr bool bSupportsCPU = true;
	static constexpr bool bSupportsGPU = true;
	static constexpr bool bSupportsAtomicOps = false;

	//-OPT: We can reduce the differences between read and RW if we have typed UAV loads
	//static constexpr TCHAR const* HLSLVariableType		= TEXT("float");
	//static constexpr EPixelFormat ReadPixelFormat		= PF_R32_FLOAT;
	//static constexpr TCHAR const* ReadHLSLBufferType	= TEXT("float");
	//static constexpr TCHAR const* ReadHLSLBufferRead	= TEXT("Value = BUFFER_NAME[Index]");
	//static constexpr EPixelFormat RWPixelFormat			= PF_R32_FLOAT;
	//static constexpr TCHAR const* RWHLSLBufferType		= TEXT("float");
	//static constexpr TCHAR const* RWHLSLBufferRead		= TEXT("Value = BUFFER_NAME[Index]");
	//static constexpr TCHAR const* RWHLSLBufferWrite		= TEXT("BUFFER_NAME[Index] = Value");

	//static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetFloatDef(); }
	//static const TArrayType GetDefaultValue();

	static void CopyCpuToCpuMemory(TArrayType* Dest, const TArrayType* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(TArrayType));
	}

	static void CopyCpuToGpuMemory(void* Dest, const TArrayType* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(TArrayType));
	}

	static void CopyGpuToCpuMemory(void* Dest, const void* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(TArrayType));
	}

	static bool IsNearlyEqual(const TArrayType& Lhs, const TArrayType& Rhs, float Tolerance)
	{
		return FMath::IsNearlyEqual(Lhs, Rhs, Tolerance);
	}

	//static TArrayType AtomicAdd(TArrayType* Dest, TArrayType Value) { check(false); }
	//static TArrayType AtomicMin(TArrayType* Dest, TArrayType Value) { check(false); }
	//static TArrayType AtomicMax(TArrayType* Dest, TArrayType Value) { check(false); }
};

template<typename TArrayType>
struct FNDIArrayImplHelper : public FNDIArrayImplHelperBase<TArrayType>
{
};

struct FNiagaraDataInterfaceArrayImplInternal
{
	struct FFunctionVersion
	{
		enum Type
		{
			InitialVersion = 0,
			AddOptionalExecuteToSet = 1,

			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};
	};

	NIAGARA_API static const TCHAR* HLSLReadTemplateFile;
	NIAGARA_API static const TCHAR* HLSLReadWriteTemplateFile;

	NIAGARA_API static const FName Function_LengthName;
	NIAGARA_API static const FName Function_IsValidIndexName;
	NIAGARA_API static const FName Function_LastIndexName;
	NIAGARA_API static const FName Function_GetName;

	NIAGARA_API static const FName Function_ClearName;
	NIAGARA_API static const FName Function_ResizeName;
	NIAGARA_API static const FName Function_ResizeAndFillName;
	NIAGARA_API static const FName Function_SetArrayElemName;
	NIAGARA_API static const FName Function_AddName;
	NIAGARA_API static const FName Function_RemoveLastElemName;

	NIAGARA_API static const FName Function_AtomicAddName;
	NIAGARA_API static const FName Function_AtomicMinName;
	NIAGARA_API static const FName Function_AtomicMaxName;

#if WITH_EDITORONLY_DATA
	NIAGARA_API static void GetFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, UClass* DIClass, FNiagaraTypeDefinition ValueTypeDef, bool bSupportsCPU, bool bSupportsGPU, bool bSupportsAtomicOps);
	static void GetImmutableFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& DefaultImmutableSig, FNiagaraTypeDefinition ValueTypeDef);
	static void GetMutableFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& DefaultMutableSig, FNiagaraTypeDefinition ValueTypeDef);
	static void GetAtomicOpFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& DefaultMutableSig, FNiagaraTypeDefinition ValueTypeDef);
	NIAGARA_API static bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature);
#endif
	static const TCHAR* GetHLSLTemplateFile(bool bIsRWArray)
	{
		return bIsRWArray ? HLSLReadWriteTemplateFile : HLSLReadTemplateFile;
	}
	NIAGARA_API static bool IsRWFunction(const FName FunctionName);

	NIAGARA_API static ERHIAccess GetCountBufferRHIAccess(const FNiagaraGpuComputeDispatchInterface& ComputeInterface);
};

//////////////////////////////////////////////////////////////////////////
// Instance Data, Proxy with Impl

template<typename TArrayType>
struct FNDIArrayInstanceData_GameThread
{
	FNiagaraSystemInstance*				OwnerInstance = nullptr;
	bool								bIsModified = false;		// True if the array has ever been modified and we are reading instance data
	bool								bIsRenderDirty = true;		// True if we have made modifications that could be pushed to the render thread
	mutable FTransactionallySafeRWLock	ArrayRWGuard;
	TArray<TArrayType>					ArrayData;					// Modified array data
};

struct FNDIArrayInstanceData_RenderThreadBase
{
	NIAGARA_API ~FNDIArrayInstanceData_RenderThreadBase();
	NIAGARA_API void Initialize(FRHICommandListImmediate& RHICmdList, FNiagaraGpuComputeDispatchInterface* InComputeInterface, int32 InDefaultElements, bool bRWGpuArray);
	NIAGARA_API void UpdateDataInternal(FRHICommandList& RHICmdList, int32 ArrayNum, int32 NewNumElements, uint32 ElementSize, EPixelFormat PixelFormat);
	NIAGARA_API void ReleaseData();

	NIAGARA_API void SimCacheWriteFrame(FRHICommandListImmediate& RHICmdList, UNDIArraySimCacheData* CacheData, int32 FrameIndex, int32 ArrayTypeSize, void(*CopyGpuToCpuMemory)(void*, const void*, int32)) const;

	bool IsReadOnly() const { return CountOffset == INDEX_NONE; }

	FNiagaraGpuComputeDispatchInterface* ComputeInterface = nullptr;

	FBufferRHIRef				ArrayBuffer;
	FUnorderedAccessViewRHIRef	ArrayUAV;
	FShaderResourceViewRHIRef	ArraySRV;
	uint32						ArrayNumBytes = 0;

	int32						DefaultElements = 0;		// The default number of elements in the buffer, can be used to reduce allocations / required for RW buffers
	int32						NumElements = INDEX_NONE;	// Number of elements in the buffer, for RW buffers this is the buffer size since the actual size is in the counter
	uint32						CountOffset = INDEX_NONE;	// Counter offset for RW buffers
};

template<typename TArrayType>
struct FNDIArrayInstanceData_RenderThread : public FNDIArrayInstanceData_RenderThreadBase
{
	using TVMArrayType = typename FNDIArrayImplHelper<TArrayType>::TVMArrayType;

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<T::bSupportsGPU>::Type UpdateDataImpl(FRHICommandList& RHICmdList, TConstArrayView<TArrayType> InArrayData)
	{
		const int32 NewNumElements = FMath::Max(DefaultElements, InArrayData.Num());

		// Resize if required and update the count value
		const EPixelFormat PixelFormat = IsReadOnly() ? FNDIArrayImplHelper<TArrayType>::ReadPixelFormat : FNDIArrayImplHelper<TArrayType>::RWPixelFormat;
		UpdateDataInternal(RHICmdList, InArrayData.Num(), NewNumElements, sizeof(TVMArrayType), PixelFormat);

		// Copy data in new data over
		{
			uint8* GPUMemory = reinterpret_cast<uint8*>(RHICmdList.LockBuffer(ArrayBuffer, 0, ArrayNumBytes, RLM_WriteOnly));
			if (InArrayData.Num() > 0)
			{
				T::CopyCpuToGpuMemory(GPUMemory, InArrayData.GetData(), InArrayData.Num());
			}

			const TArrayType DefaultValue = TArrayType(FNDIArrayImplHelper<TArrayType>::GetDefaultValue());
			T::CopyCpuToGpuMemory(GPUMemory + (sizeof(TVMArrayType) * NumElements), &DefaultValue, 1);

			RHICmdList.UnlockBuffer(ArrayBuffer);
		}
	}

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<!T::bSupportsGPU>::Type UpdateDataImpl(FRHICommandList& RHICmdList, TConstArrayView<TArrayType> InArrayData)
	{
	}

	void UpdateData(FRHICommandList& RHICmdList, TArray<TArrayType>& InArrayData)
	{
		UpdateDataImpl(RHICmdList, InArrayData);
	}
};

template<typename TArrayType, class TOwnerType>
struct FNDIArrayProxyImpl : public INDIArrayProxyBase
{
	static constexpr int32 kSafeMaxElements = TNumericLimits<int32>::Max();
	using TVMArrayType = typename FNDIArrayImplHelper<TArrayType>::TVMArrayType;

	struct FReadArrayRef
	{
		FReadArrayRef(const TOwnerType* Owner, const FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData)
		{
			if ( InstanceData )
			{
				LockObject = &InstanceData->ArrayRWGuard;
				LockObject->ReadLock();
				ArrayData = InstanceData->bIsModified ? &InstanceData->ArrayData : &Owner->GetArrayReference();
			}
			else
			{
				ArrayData = &Owner->GetArrayReference();
			}
		}

		~FReadArrayRef()
		{
			if ( LockObject )
			{
				LockObject->ReadUnlock();
			}
		}

		const TArray<TArrayType>& GetArray() { return *ArrayData; }

	private:
		FTransactionallySafeRWLock* LockObject = nullptr;
		const TArray<TArrayType>*	ArrayData = nullptr;
		UE_NONCOPYABLE(FReadArrayRef);
	};

	struct FWriteArrayRef
	{
		FWriteArrayRef(TOwnerType* Owner, FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData)
		{
			if (InstanceData)
			{
				LockObject = &InstanceData->ArrayRWGuard;
				LockObject->WriteLock();

 				if (InstanceData->bIsModified == false)
 				{
 					InstanceData->bIsModified = true;
 					InstanceData->ArrayData = Owner->GetArrayReference();
 				}
				ArrayData = &InstanceData->ArrayData;
			}
			else
			{
				ArrayData = &Owner->GetArrayReference();
			}
		}

		~FWriteArrayRef()
		{
			if (LockObject)
			{
				LockObject->WriteUnlock();
			}
		}

		TArray<TArrayType>& GetArray() { return *ArrayData; }

	private:
		FTransactionallySafeRWLock*	LockObject = nullptr;
		TArray<TArrayType>*	        ArrayData = nullptr;
		UE_NONCOPYABLE(FWriteArrayRef);
	};

	struct FGameToRenderInstanceData
	{
		bool				bUpdateData = false;
		TArray<TArrayType>	ArrayData;
	};

	FNDIArrayProxyImpl(TOwnerType* InOwner)
		: Owner(InOwner)
	{
		CachePropertiesFromOwner();
	}

	void CachePropertiesFromOwner()
	{
		bShouldSyncToGpu = FNiagaraUtilities::ShouldSyncCpuToGpu(Owner->GpuSyncMode);
		bShouldSyncToCpu = FNiagaraUtilities::ShouldSyncGpuToCpu(Owner->GpuSyncMode) && Owner->IsUsedWithCPUScript();
	}

	//////////////////////////////////////////////////////////////////////////
	// FNiagaraDataInterfaceProxyRW
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return sizeof(FGameToRenderInstanceData);
	}
	
	virtual void ProvidePerInstanceDataForRenderThread(void* InDataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID)
	{
		FGameToRenderInstanceData* GameToRenderInstanceData = new(InDataForRenderThread) FGameToRenderInstanceData();
		FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData_GT = reinterpret_cast<FNDIArrayInstanceData_GameThread<TArrayType>*>(PerInstanceData);
		if (InstanceData_GT->bIsRenderDirty)
		{
			FReadArrayRef ArrayData(Owner, InstanceData_GT);

			GameToRenderInstanceData->bUpdateData	= true;
			GameToRenderInstanceData->ArrayData		= ArrayData.GetArray();

			InstanceData_GT->bIsRenderDirty = false;
		}
	}

	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override
	{
		FGameToRenderInstanceData* GameToRenderInstanceData = reinterpret_cast<FGameToRenderInstanceData*>(PerInstanceData);
		if ( GameToRenderInstanceData->bUpdateData )
		{
			if ( FNDIArrayInstanceData_RenderThread<TArrayType>* InstanceData_RT = PerInstanceData_RenderThread.Find(InstanceID) )
			{
				FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
				InstanceData_RT->UpdateData(RHICmdList, GameToRenderInstanceData->ArrayData);
			}
		}
		GameToRenderInstanceData->~FGameToRenderInstanceData();
	}

	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override
	{
		if (const FNDIArrayInstanceData_RenderThread<TArrayType>* InstanceData_RT = PerInstanceData_RenderThread.Find(Context.GetSystemInstanceID()))
		{
			Context.SetDirect(InstanceData_RT->NumElements, InstanceData_RT->CountOffset);
		}
	}

	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override
	{
		if (bShouldSyncToCpu == false)
		{
			return;
		}
		
		const FNDIArrayInstanceData_RenderThread<TArrayType>* InstanceData_RT = PerInstanceData_RenderThread.Find(Context.GetSystemInstanceID());
		if ( !InstanceData_RT || InstanceData_RT->IsReadOnly() || (InstanceData_RT->ArrayNumBytes == 0) )
		{
			return;
		}

		const FNiagaraGPUInstanceCountManager& CountManager = Context.GetComputeDispatchInterface().GetGPUInstanceCounterManager();
		FNiagaraGpuReadbackManager* ReadbackManager = Context.GetComputeDispatchInterface().GetGpuReadbackManager();

		const FNiagaraGpuReadbackManager::FBufferRequest BufferRequests[] =
		{
			{CountManager.GetInstanceCountBuffer().Buffer, uint32(InstanceData_RT->CountOffset * sizeof(uint32)), sizeof(uint32)},
			{InstanceData_RT->ArrayBuffer, 0, InstanceData_RT->ArrayNumBytes},		//-TODO: Technically last element is default for RW buffers
		};

		const ERHIAccess CountRHIAccess = FNiagaraDataInterfaceArrayImplInternal::GetCountBufferRHIAccess(Context.GetComputeDispatchInterface());
		const FRHITransitionInfo TransitionsBefore[] =
		{
			FRHITransitionInfo(CountManager.GetInstanceCountBuffer().UAV, CountRHIAccess, ERHIAccess::CopySrc),
			FRHITransitionInfo(InstanceData_RT->ArrayBuffer, ERHIAccess::UAVCompute, ERHIAccess::CopySrc),
		};
		const FRHITransitionInfo TransitionsAfter[] =
		{
			FRHITransitionInfo(CountManager.GetInstanceCountBuffer().UAV, ERHIAccess::CopySrc, CountRHIAccess),
			FRHITransitionInfo(InstanceData_RT->ArrayBuffer, ERHIAccess::CopySrc, ERHIAccess::UAVCompute),
		};

		AddPass(
			Context.GetGraphBuilder(),
			RDG_EVENT_NAME("NDIArrayReadback"),
			[BufferRequests, TransitionsBefore, TransitionsAfter, SystemInstanceID=Context.GetSystemInstanceID(), WeakOwner=TWeakObjectPtr(Owner), Proxy=this, ReadbackManager](FRHICommandListImmediate& RHICmdList)
			{
				RHICmdList.Transition(TransitionsBefore);
				ReadbackManager->EnqueueReadbacks(
					RHICmdList,
					BufferRequests,
					[SystemInstanceID, WeakOwner, Proxy](TConstArrayView<TPair<void*, uint32>> ReadbackData)
					{
						const int32 NumElements = *reinterpret_cast<const uint32*>(ReadbackData[0].Key);
						TArray<TArrayType> ArrayData;
						if ( NumElements > 0 )
						{
							ArrayData.AddUninitialized(NumElements);
							FNDIArrayImplHelper<TArrayType>::CopyGpuToCpuMemory(ArrayData.GetData(), reinterpret_cast<const TVMArrayType*>(ReadbackData[1].Key), NumElements);
						}

						AsyncTask(
							ENamedThreads::GameThread,
							[SystemInstanceID, ArrayData=MoveTemp(ArrayData), WeakOwner, Proxy]()
							{
								// If this is nullptr the proxy is no longer valid so discard
								if ( WeakOwner.Get() == nullptr )
								{
									return;
								}

								Proxy->SetInstanceArrayData(SystemInstanceID, ArrayData);
							}
						);
					}
				);
				RHICmdList.Transition(TransitionsAfter);
			}
		);
	}

	//////////////////////////////////////////////////////////////////////////
	// BP user parameter accessors, should remove if we every start to share the object between instances
	void BeginSetArrayFromBP(bool bCopyFromInstanceData)
	{
		for (auto It = PerInstanceData_GameThread.CreateConstIterator(); It; ++It)
		{
			FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData = It.Value();
			InstanceData->ArrayRWGuard.WriteLock();
			if (InstanceData->bIsModified && bCopyFromInstanceData)
			{
				Owner->GetArrayReference() = InstanceData->ArrayData;
			}
			bCopyFromInstanceData = false;
			InstanceData->bIsModified = false;
			InstanceData->bIsRenderDirty |= bShouldSyncToGpu;
			InstanceData->ArrayData.Empty();
		}
	}

	void EndSetArrayFromBP()
	{
		for (auto It = PerInstanceData_GameThread.CreateConstIterator(); It; ++It)
		{
			FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData = It.Value();
			InstanceData->ArrayRWGuard.WriteUnlock();
		}
	}

	template<typename TFromArrayType>
	void SetArrayData(TConstArrayView<TFromArrayType> InArrayData)
	{
		if (PerInstanceData_GameThread.IsEmpty())
		{
			Owner->GetArrayReference().SetNumUninitialized(InArrayData.Num());
			FNDIArrayImplHelper<TArrayType>::CopyCpuToCpuMemory(Owner->GetArrayReference().GetData(), InArrayData.GetData(), InArrayData.Num());
		}
		else
		{
			BeginSetArrayFromBP(false);

			Owner->GetArrayReference().SetNum(InArrayData.Num());
			FNDIArrayImplHelper<TArrayType>::CopyCpuToCpuMemory(Owner->GetArrayReference().GetData(), InArrayData.GetData(), InArrayData.Num());

			EndSetArrayFromBP();
		}
	}

	template<typename TFromArrayType>
	void SetArrayDataAndRecreateRenderState(TConstArrayView<TFromArrayType> InArrayData)
	{
		SetArrayData(InArrayData);
		RecreateRenderState();
	}

	void RecreateRenderState()
	{
		for (auto It = PerInstanceData_GameThread.CreateConstIterator(); It; ++It)
		{
			FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData = It.Value();
			FNiagaraSystemInstance* SystemInstance = InstanceData->OwnerInstance;
			USceneComponent* SceneComponent = SystemInstance ? SystemInstance->GetAttachComponent() : nullptr;
			if (SceneComponent && SceneComponent->IsRenderStateCreated() && !SceneComponent->IsRenderStateRecreating())
			{
				SceneComponent->RecreateRenderState_Concurrent();
			}
			// Ideally we would replace this with a mark for recreate but things like renderers would need know how to handle changes
			//SceneComponent->MarkForNeededEndOfFrameRecreate();
		}
	}

	template<typename TToArrayType>
	TArray<TToArrayType> GetArrayDataCopy()
	{
		ensure(PerInstanceData_GameThread.Num() <= 1);
		FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData = PerInstanceData_GameThread.IsEmpty() ? nullptr : PerInstanceData_GameThread.CreateConstIterator().Value();
		FReadArrayRef ArrayRef(Owner, InstanceData);
		TArray<TToArrayType> OutArray;
		OutArray.SetNum(ArrayRef.GetArray().Num());
		FNDIArrayImplHelper<TArrayType>::CopyCpuToCpuMemory(OutArray.GetData(), ArrayRef.GetArray().GetData(), ArrayRef.GetArray().Num());
		return OutArray;
	}

	template<typename TFromArrayType>
	void SetArrayValue(int Index, const TFromArrayType& Value, bool bSizeToFit)
	{
		ensure(PerInstanceData_GameThread.Num() <= 1);

		BeginSetArrayFromBP(true);
		ON_SCOPE_EXIT { EndSetArrayFromBP(); };

		TArray<TArrayType>& ArrayRef = Owner->GetArrayReference();
		if (!ArrayRef.IsValidIndex(Index))
		{
			if (!bSizeToFit)
			{
				return;
			}
			ArrayRef.AddDefaulted(Index + 1 - ArrayRef.Num());
		}
		FNDIArrayImplHelper<TArrayType>::CopyCpuToCpuMemory(ArrayRef.GetData() + Index, &Value, 1);
	}

	template<typename TToArrayType>
	TToArrayType GetArrayValue(int Index)
	{
		TArrayType ValueOut = TArrayType(FNDIArrayImplHelper<TArrayType>::GetDefaultValue());

		ensure(PerInstanceData_GameThread.Num() <= 1);
		FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData = PerInstanceData_GameThread.IsEmpty() ? nullptr : PerInstanceData_GameThread.CreateConstIterator().Value();
		FReadArrayRef ArrayRef(Owner, InstanceData);

		if (ArrayRef.GetArray().IsValidIndex(Index))
		{
			ValueOut = ArrayRef.GetArray()[Index];
		}

		TToArrayType ToValueOut;
		FNDIArrayImplHelper<TArrayType>::CopyCpuToCpuMemory(&ToValueOut, &ValueOut, 1);
		return ToValueOut;
	}

	void SetInstanceArrayData(FNiagaraSystemInstanceID InstanceID, const TArray<TArrayType>& InArrayData)
	{
		if ( FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData = PerInstanceData_GameThread.FindRef(InstanceID) )
		{
			FWriteArrayRef ArrayData(Owner, InstanceData);
			ArrayData.GetArray() = InArrayData;
			InstanceData->bIsRenderDirty |= bShouldSyncToGpu;
		}
	}

#if WITH_EDITORONLY_DATA
	//////////////////////////////////////////////////////////////////////////
	// INDIArrayProxyBase
	virtual void GetFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const override
	{
		FNiagaraDataInterfaceArrayImplInternal::GetFunctions(
			OutFunctions,
			TOwnerType::StaticClass(),
			FNDIArrayImplHelper<TArrayType>::GetTypeDefinition(),
			FNDIArrayImplHelper<TArrayType>::bSupportsCPU,
			FNDIArrayImplHelper<TArrayType>::bSupportsGPU,
			FNDIArrayImplHelper<TArrayType>::bSupportsAtomicOps
		);
	}
#endif

	void GetVMExternalFunction_Internal(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
	{
		if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_LengthName)
		{
			check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMGetLength);
		}
		else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_IsValidIndexName)
		{
			check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMIsValidIndex);
		}
		else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_LastIndexName)
		{
			check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMGetLastIndex);
		}
	}

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<T::bSupportsCPU>::Type GetVMExternalFunction_CPUAccessInternal(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
	{
		// Immutable functions
		if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_GetName)
		{
			// Note: Outputs is variable based upon type
			//check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMGetValue);
		}
		// Mutable functions
		else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_ClearName)
		{
			check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 0);
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMClear);
		}
		else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_ResizeName)
		{
			check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 0);
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMResize);
		}
		else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_ResizeAndFillName)
		{
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMResizeAndFill);
		}
		else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_SetArrayElemName)
		{
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMSetValue);
		}
		else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_AddName)
		{
			// Note: Inputs is variable based upon type
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMPushValue);
		}
		else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_RemoveLastElemName)
		{
			// Note: Outputs is variable based upon type
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMPopValue);
		}
	}

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<!T::bSupportsCPU>::Type GetVMExternalFunction_CPUAccessInternal(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
	{
	}

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<T::bSupportsAtomicOps>::Type GetVMExternalFunction_AtomicInternal(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
	{
		if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_AtomicAddName)
		{
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMAtomicAdd);
		}
		else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_AtomicMinName)
		{
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMAtomicMin);
		}
		else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_AtomicMaxName)
		{
			OutFunc.BindRaw(this, &FNDIArrayProxyImpl::VMAtomicMax);
		}
	}

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<!T::bSupportsAtomicOps>::Type GetVMExternalFunction_AtomicInternal(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
	{
	}

	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override
	{
		GetVMExternalFunction_Internal(BindingInfo, InstanceData, OutFunc);
		if (OutFunc.IsBound() == false)
		{
			GetVMExternalFunction_CPUAccessInternal(BindingInfo, InstanceData, OutFunc);
		}
		if (OutFunc.IsBound() == false)
		{
			GetVMExternalFunction_AtomicInternal(BindingInfo, InstanceData, OutFunc);
		}
	}

#if WITH_EDITORONLY_DATA
	bool IsRWGpuArray(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo) const
	{
		return ParamInfo.GeneratedFunctions.ContainsByPredicate(
			[&](const FNiagaraDataInterfaceGeneratedFunction& Function)
			{
				return FNiagaraDataInterfaceArrayImplInternal::IsRWFunction(Function.DefinitionName);
			}
		);
	}

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<T::bSupportsGPU>::Type GetParameterDefinitionHLSL_Internal(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) const
	{
		TMap<FString, FStringFormatArg> TemplateArgs =
		{
			{TEXT("ParameterName"),		ParamInfo.DataInterfaceHLSLSymbol},
			{TEXT("VariableType"),		FNDIArrayImplHelper<TArrayType>::HLSLVariableType},
			{TEXT("ReadBufferType"),	FNDIArrayImplHelper<TArrayType>::ReadHLSLBufferType},
			{TEXT("ReadBufferRead"),	FNDIArrayImplHelper<TArrayType>::ReadHLSLBufferRead},
			{TEXT("RWBufferType"),		FNDIArrayImplHelper<TArrayType>::RWHLSLBufferType},
			{TEXT("RWBufferRead"),		FNDIArrayImplHelper<TArrayType>::RWHLSLBufferRead},
			{TEXT("RWBufferWrite"),		FNDIArrayImplHelper<TArrayType>::RWHLSLBufferWrite},
			{TEXT("bSupportsAtomicOps"),	FNDIArrayImplHelper<TArrayType>::bSupportsAtomicOps ? 1 : 0},
		};

		FString TemplateFile;
		LoadShaderSourceFile(FNiagaraDataInterfaceArrayImplInternal::GetHLSLTemplateFile(IsRWGpuArray(ParamInfo)), EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
		OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
	}

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<!T::bSupportsGPU>::Type GetParameterDefinitionHLSL_Internal(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) const
	{
	}

	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) const override
	{
		GetParameterDefinitionHLSL_Internal(ParamInfo, OutHLSL);
	}

	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) const override
	{
		if (FNDIArrayImplHelper<TArrayType>::bSupportsGPU)
		{
			if ((FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_LengthName) ||
				(FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_IsValidIndexName) ||
				(FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_LastIndexName) ||
				(FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_GetName))
			{
				return true;
			}

			if ((FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_ClearName) ||
				(FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_ResizeName) ||
				(FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_ResizeAndFillName) ||
				(FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_SetArrayElemName) ||
				(FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_AddName) ||
				(FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_RemoveLastElemName))
			{
				return true;
			}

			if ( FNDIArrayImplHelper<TArrayType>::bSupportsAtomicOps )
			{
				if ((FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_AtomicAddName) ||
					(FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_AtomicMinName) ||
					(FunctionInfo.DefinitionName == FNiagaraDataInterfaceArrayImplInternal::Function_AtomicMaxName) )
				{
					return true;
				}
			}
		}
		return false;
	}

	virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override
	{
		if (FNDIArrayImplHelper<TArrayType>::bSupportsGPU)
		{
			InVisitor->UpdateShaderFile(FNiagaraDataInterfaceArrayImplInternal::GetHLSLTemplateFile(false));
			InVisitor->UpdateShaderFile(FNiagaraDataInterfaceArrayImplInternal::GetHLSLTemplateFile(true));
		}
		return true;
	}

	virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) const override
	{
		return FNiagaraDataInterfaceArrayImplInternal::UpgradeFunctionCall(FunctionSignature);
	}
#endif
#if WITH_NIAGARA_DEBUGGER
	virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const override
	{
		FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData = PerInstanceData_GameThread.FindRef(DebugHudContext.GetSystemInstance()->GetId());
		if (InstanceData == nullptr )
		{
			return;
		}

		FReadArrayRef ArrayData(Owner, InstanceData);
		FString CpuValuesString;

		const int32 MaxStringElements = 8;
		const int32 NumElements = FMath::Min(MaxStringElements, ArrayData.GetArray().Num());
		for (int32 i=0; i < NumElements; ++i)
		{
			CpuValuesString.Append(i > 0 ? TEXT(", [") : TEXT("["));
			FNDIArrayImplHelper<TArrayType>::AppendValueToString(ArrayData.GetArray()[i], CpuValuesString);
			CpuValuesString.Append(TEXT("]"));
		}
		if (MaxStringElements < ArrayData.GetArray().Num())
		{
			CpuValuesString.Append(TEXT(", ..."));
		}

		DebugHudContext.GetOutputString().Appendf(
			TEXT("Type(%s) CpuLength(%d) CpuValues(%s)"),
			*FNDIArrayImplHelper<TArrayType>::GetTypeDefinition().GetName(),
			ArrayData.GetArray().Num(),
			*CpuValuesString
		);
	}
#endif

	virtual bool SimCacheWriteFrame(UNDIArraySimCacheData* CacheData, int FrameIndex, FNiagaraSystemInstance* SystemInstance) const override
	{
		const FNiagaraSystemInstanceID InstanceID = SystemInstance->GetId();
		FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData = PerInstanceData_GameThread.FindRef(InstanceID);
		if (InstanceData == nullptr)
		{
			return false;
		}

		// Write CPU Data
		{
			FReadArrayRef ArrayData(Owner, InstanceData);
			if (CacheData->CpuFrameData.Num() <= FrameIndex)
			{
				CacheData->CpuFrameData.AddDefaulted(FrameIndex + 1 - CacheData->CpuFrameData.Num());
			}
			FNDIArraySimCacheDataFrame& FrameData = CacheData->CpuFrameData[FrameIndex];
			FrameData.NumElements = ArrayData.GetArray().Num();
			FrameData.DataOffset = CacheData->FindOrAddData(
				MakeArrayView(
					reinterpret_cast<const uint8*>(ArrayData.GetArray().GetData()),
					ArrayData.GetArray().Num() * ArrayData.GetArray().GetTypeSize()
				)
			);
		}

		// Write GPU Data
		if (FNDIArrayImplHelper<TArrayType>::bSupportsGPU && Owner->IsUsedWithGPUScript())
		{
			FNiagaraGpuComputeDispatchInterface* ComputeInterface = FNiagaraGpuComputeDispatchInterface::Get(SystemInstance->GetWorld());
			ENQUEUE_RENDER_COMMAND(NDIArray_SimCacheWrite)(
				[Proxy_RT=this, InstanceID, CacheData, FrameIndex, ComputeInterface](FRHICommandListImmediate& RHICmdList)
				{
					const FNDIArrayInstanceData_RenderThread<TArrayType>* InstanceData_RT = Proxy_RT->PerInstanceData_RenderThread.Find(InstanceID);
					if (InstanceData_RT)
					{
						InstanceData_RT->SimCacheWriteFrame(
							RHICmdList,
							CacheData,
							FrameIndex,
							sizeof(TVMArrayType),
							&FNDIArrayImplHelper<TArrayType>::CopyGpuToCpuMemory
						);
					}
				}
			);

			FlushRenderingCommands();
		}
		return true;
	}

	virtual bool SimCacheReadFrame(UNDIArraySimCacheData* CacheData, int FrameIndex, FNiagaraSystemInstance* SystemInstance) override
	{
		const FNiagaraSystemInstanceID InstanceID = SystemInstance->GetId();
		FNDIArrayInstanceData_GameThread<TArrayType>* InstanceData = PerInstanceData_GameThread.FindRef(InstanceID);
		if (InstanceData == nullptr)
		{
			return false;
		}

		// Read CPU Data
		if (CacheData->CpuFrameData.IsValidIndex(FrameIndex))
		{
			FWriteArrayRef ArrayData(Owner, InstanceData);
			const FNDIArraySimCacheDataFrame& FrameData = CacheData->CpuFrameData[FrameIndex];
			ArrayData.GetArray().SetNumUninitialized(FrameData.NumElements);
			if (FrameData.NumElements > 0 )
			{
				check(FrameData.DataOffset != INDEX_NONE);
				FMemory::Memcpy(ArrayData.GetArray().GetData(), CacheData->BufferData.GetData() + FrameData.DataOffset, FrameData.NumElements * sizeof(TArrayType));
			}
		}

		// Read GPU Data
		if constexpr (FNDIArrayImplHelper<TArrayType>::bSupportsGPU)
		{
			if (Owner->IsUsedWithGPUScript() && CacheData->GpuFrameData.IsValidIndex(FrameIndex))
			{
				FNiagaraGpuComputeDispatchInterface* ComputeInterface = FNiagaraGpuComputeDispatchInterface::Get(SystemInstance->GetWorld());
				ENQUEUE_RENDER_COMMAND(NDIArray_SimCacheWrite)(
					[Proxy_RT=this, InstanceID, CacheData, FrameIndex, ComputeInterface](FRHICommandListImmediate& RHICmdList)
					{
						if ( FNDIArrayInstanceData_RenderThread<TArrayType>* InstanceData_RT = Proxy_RT->PerInstanceData_RenderThread.Find(InstanceID) )
						{
							const FNDIArraySimCacheDataFrame& FrameData = CacheData->GpuFrameData[FrameIndex];
							TConstArrayView<TArrayType> ArrayData(reinterpret_cast<const TArrayType*>(CacheData->BufferData.GetData() + FrameData.DataOffset), FrameData.NumElements);
							InstanceData_RT->UpdateDataImpl(RHICmdList, ArrayData);
						}
					}
				);
			}
		}
		return true;
	}

	virtual bool SimCacheCompareElement(const uint8* LhsData, const uint8* RhsData, int32 Element, float Tolerance) const override
	{
		const TArrayType* LhsArrayData = reinterpret_cast<const TArrayType*>(LhsData);
		const TArrayType* RhsArrayData = reinterpret_cast<const TArrayType*>(RhsData);
		return FNDIArrayImplHelper<TArrayType>::IsNearlyEqual(LhsArrayData[Element], RhsArrayData[Element], Tolerance);
	}

	virtual FString SimCacheVisualizerRead(const UNDIArraySimCacheData* CacheData, const FNDIArraySimCacheDataFrame& FrameData, int Element) const override
	{
		FString OutValue;
		if (Element < FrameData.NumElements)
		{
			TArrayType Value;
			FMemory::Memcpy(&Value, CacheData->BufferData.GetData() + FrameData.DataOffset + (sizeof(TArrayType) * Element), sizeof(TArrayType));
			FNDIArrayImplHelper<TArrayType>::AppendValueToString(Value, OutValue);
		}
		return OutValue;
	}

	virtual bool CopyToInternal(INDIArrayProxyBase* InDestination) const override
	{
		auto Destination = static_cast<FNDIArrayProxyImpl<TArrayType, TOwnerType>*>(InDestination);
		Destination->Owner->GetArrayReference() = Owner->GetArrayReference();
		return true;
	}

	virtual bool Equals(const INDIArrayProxyBase* InOther) const override
	{
		auto Other = static_cast<const FNDIArrayProxyImpl<TArrayType, TOwnerType>*>(InOther);
		return Other->Owner->GetArrayReference() == Owner->GetArrayReference();
	}

	virtual int32 PerInstanceDataSize() const override
	{
		return sizeof(FNDIArrayInstanceData_GameThread<TArrayType>);
	}

	virtual bool InitPerInstanceData(UNiagaraDataInterface* DataInterface, void* InPerInstanceData, FNiagaraSystemInstance* SystemInstance) override
	{
		// Ensure we have the latest sync mode settings
		CachePropertiesFromOwner();

		auto* InstanceData_GT = new(InPerInstanceData) FNDIArrayInstanceData_GameThread<TArrayType>();
		InstanceData_GT->OwnerInstance = SystemInstance;
		InstanceData_GT->bIsRenderDirty = true;

		PerInstanceData_GameThread.Emplace(SystemInstance->GetId(), InstanceData_GT);

		if ( FNDIArrayImplHelper<TArrayType>::bSupportsGPU && Owner->IsUsedWithGPUScript() )
		{
			bool bRWGpuArray = false;
			FNiagaraDataInterfaceUtilities::ForEachGpuFunction(
				DataInterface, SystemInstance,
				[&](const UNiagaraScript* Script, const FNiagaraDataInterfaceGeneratedFunction& Function) -> bool
				{
					bRWGpuArray = FNiagaraDataInterfaceArrayImplInternal::IsRWFunction(Function.DefinitionName);
					return !bRWGpuArray;
				}
			);

			ENQUEUE_RENDER_COMMAND(FNDIArrayProxyImpl_AddProxy)
			(
				[Proxy_RT=this, InstanceID_RT=SystemInstance->GetId(), ComputeInterface_RT=SystemInstance->GetComputeDispatchInterface(), MaxElements_RT=Owner->MaxElements, bRWGpuArray_RT = bRWGpuArray](FRHICommandListImmediate& RHICmdList)
				{
					FNDIArrayInstanceData_RenderThread<TArrayType>* InstanceData_RT = &Proxy_RT->PerInstanceData_RenderThread.Add(InstanceID_RT);
					InstanceData_RT->Initialize(RHICmdList, ComputeInterface_RT, MaxElements_RT, bRWGpuArray_RT);
				}
			);
		}

		return true;
	}

	virtual void DestroyPerInstanceData(void* InPerInstanceData, FNiagaraSystemInstance* SystemInstance) override
	{
		auto* InstanceData_GT = reinterpret_cast<FNDIArrayInstanceData_GameThread<TArrayType>*>(InPerInstanceData);

		if ( FNDIArrayImplHelper<TArrayType>::bSupportsGPU && Owner->IsUsedWithGPUScript() )
		{
			ENQUEUE_RENDER_COMMAND(FNDIArrayProxyImpl_RemoveProxy)
			(
				[Proxy_RT=this, InstanceID_RT=SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
				{
					Proxy_RT->PerInstanceData_RenderThread.Remove(InstanceID_RT);
				}
			);
		}
		PerInstanceData_GameThread.Remove(SystemInstance->GetId());
		InstanceData_GT->~FNDIArrayInstanceData_GameThread<TArrayType>();
	}

	virtual void SetShaderParameters(FShaderParameters* ShaderParameters, FNiagaraSystemInstanceID SystemInstanceID) const override
	{
		const auto* InstanceData_RT = &PerInstanceData_RenderThread.FindChecked(SystemInstanceID);
		if ( InstanceData_RT->IsReadOnly() )
		{
			ShaderParameters->ArrayBufferParams.X	= InstanceData_RT->NumElements;
			ShaderParameters->ArrayBufferParams.Y	= FMath::Max(0, InstanceData_RT->NumElements - 1);
			ShaderParameters->ArrayReadBuffer		= InstanceData_RT->ArraySRV;
		}
		else
		{
			ShaderParameters->ArrayBufferParams.X	= (int32)InstanceData_RT->CountOffset;
			ShaderParameters->ArrayBufferParams.Y	= InstanceData_RT->NumElements;
			ShaderParameters->ArrayRWBuffer			= InstanceData_RT->ArrayUAV;
		}
	}

	//////////////////////////////////////////////////////////////////////////
	void VMGetLength(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIOutputParam<int32> OutValue(Context);

		FReadArrayRef ArrayData(Owner, InstanceData);
		const int32 Num = ArrayData.GetArray().Num();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutValue.SetAndAdvance(Num);
		}
	}

	void VMIsValidIndex(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIInputParam<int32> IndexParam(Context);
		FNDIOutputParam<FNiagaraBool> OutValue(Context);

		FReadArrayRef ArrayData(Owner, InstanceData);
		const int32 Num = ArrayData.GetArray().Num();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Index = IndexParam.GetAndAdvance();
			OutValue.SetAndAdvance((Index >= 0) && (Index < Num));
		}
	}

	void VMGetLastIndex(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIOutputParam<int32> OutValue(Context);

		FReadArrayRef ArrayData(Owner, InstanceData);
		const int32 Num = ArrayData.GetArray().Num() - 1;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutValue.SetAndAdvance(Num);
		}
	}

	void VMGetValue(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIInputParam<int32> IndexParam(Context);
		FNDIOutputParam<TVMArrayType> OutValue(Context);

		FReadArrayRef ArrayData(Owner, InstanceData);
		const int32 Num = ArrayData.GetArray().Num() - 1;
		if (Num >= 0)
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				const int32 Index = FMath::Clamp(IndexParam.GetAndAdvance(), 0, Num);
				OutValue.SetAndAdvance(TVMArrayType(ArrayData.GetArray()[Index]));
			}
		}
		else
		{
			const TVMArrayType DefaultValue = FNDIArrayImplHelper<TArrayType>::GetDefaultValue();
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				OutValue.SetAndAdvance(DefaultValue);
			}
		}
	}

	void VMClear(FVectorVMExternalFunctionContext& Context)
	{
		ensureMsgf(Context.GetNumInstances() == 1, TEXT("Setting the number of values in an array with more than one instance, which doesn't make sense"));
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);

		FWriteArrayRef ArrayData(Owner, InstanceData);
		ArrayData.GetArray().Reset();

		InstanceData->bIsRenderDirty |= bShouldSyncToGpu;
	}

	void VMResize(FVectorVMExternalFunctionContext& Context)
	{
		ensureMsgf(Context.GetNumInstances() == 1, TEXT("Setting the number of values in an array with more than one instance, which doesn't make sense"));
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIInputParam<int32> NewNumParam(Context);

		FWriteArrayRef ArrayData(Owner, InstanceData);

		const int32 OldNum = ArrayData.GetArray().Num();
		const int32 NewNum = FMath::Min(NewNumParam.GetAndAdvance(), kSafeMaxElements);
		ArrayData.GetArray().SetNumUninitialized(NewNum);

		if (NewNum > OldNum)
		{
			const TArrayType DefaultValue = TArrayType(FNDIArrayImplHelper<TArrayType>::GetDefaultValue());
			for (int32 i = OldNum; i < NewNum; ++i)
			{
				ArrayData.GetArray()[i] = DefaultValue;
			}
		}

		InstanceData->bIsRenderDirty |= bShouldSyncToGpu;
	}

	void VMResizeAndFill(FVectorVMExternalFunctionContext& Context)
	{
		ensureMsgf(Context.GetNumInstances() == 1, TEXT("Setting the number of values in an array with more than one instance, which doesn't make sense"));
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIInputParam<int32>			InNewNum(Context);
		FNDIInputParam<bool>			InFillNewOnly(Context);
		FNDIInputParam<TVMArrayType>	InDefaultValue(Context);

		FWriteArrayRef ArrayData(Owner, InstanceData);

		const int32 OldNum = ArrayData.GetArray().Num();
		const int32 NewNum = FMath::Min(InNewNum.GetAndAdvance(), kSafeMaxElements);
		ArrayData.GetArray().SetNumUninitialized(NewNum);

		const int32 FirstToFill	= InFillNewOnly.GetAndAdvance() ? OldNum : 0;
		if (NewNum > FirstToFill)
		{
			const TArrayType DefaultValue = InDefaultValue.GetAndAdvance();
			for (int32 i = FirstToFill; i < NewNum; ++i)
			{
				ArrayData.GetArray()[i] = DefaultValue;
			}
		}

		InstanceData->bIsRenderDirty |= bShouldSyncToGpu;
	}

	void VMSetValue(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIInputParam<FNiagaraBool> InSkipSet(Context);
		FNDIInputParam<int32> IndexParam(Context);
		FNDIInputParam<TVMArrayType> InValue(Context);

		FWriteArrayRef ArrayData(Owner, InstanceData);
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Index = IndexParam.GetAndAdvance();
			const TArrayType Value = (TArrayType)InValue.GetAndAdvance();
			const bool bSkipSet = InSkipSet.GetAndAdvance();

			if (!bSkipSet && ArrayData.GetArray().IsValidIndex(Index))
			{
				ArrayData.GetArray()[Index] = Value;
			}
		}

		InstanceData->bIsRenderDirty |= bShouldSyncToGpu;
	}

	void VMPushValue(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIInputParam<FNiagaraBool> InSkipExecute(Context);
		FNDIInputParam<TVMArrayType> InValue(Context);

		const int32 MaxElements = Owner->MaxElements > 0 ? Owner->MaxElements : kSafeMaxElements;

		FWriteArrayRef ArrayData(Owner, InstanceData);
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool bSkipExecute = InSkipExecute.GetAndAdvance();
			const TArrayType Value = (TArrayType)InValue.GetAndAdvance();
			if (!bSkipExecute && (ArrayData.GetArray().Num() < MaxElements))
			{
				ArrayData.GetArray().Emplace(Value);
			}
		}

		InstanceData->bIsRenderDirty |= bShouldSyncToGpu;
	}

	void VMPopValue(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIInputParam<FNiagaraBool> InSkipExecute(Context);
		FNDIOutputParam<TVMArrayType> OutValue(Context);
		FNDIOutputParam<FNiagaraBool> OutIsValid(Context);
		const TVMArrayType DefaultValue = FNDIArrayImplHelper<TArrayType>::GetDefaultValue();

		FWriteArrayRef ArrayData(Owner, InstanceData);
		for (int32 i=0; i < Context.GetNumInstances(); ++i)
		{
			const bool bSkipExecute = InSkipExecute.GetAndAdvance();
			if (bSkipExecute || (ArrayData.GetArray().Num() == 0))
			{
				OutValue.SetAndAdvance(DefaultValue);
				OutIsValid.SetAndAdvance(false);
			}
			else
			{
				OutValue.SetAndAdvance(TVMArrayType(ArrayData.GetArray().Pop()));
				OutIsValid.SetAndAdvance(true);
			}
		}

		InstanceData->bIsRenderDirty |= bShouldSyncToGpu;
	}

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<!T::bSupportsAtomicOps>::Type VMAtomicAdd(FVectorVMExternalFunctionContext& Context) { check(false); }
	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<!T::bSupportsAtomicOps>::Type VMAtomicMin(FVectorVMExternalFunctionContext& Context) { check(false); }
	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<!T::bSupportsAtomicOps>::Type VMAtomicMax(FVectorVMExternalFunctionContext& Context) { check(false); }

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<T::bSupportsAtomicOps>::Type VMAtomicAdd(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIInputParam<FNiagaraBool>	InSkipOp(Context);
		FNDIInputParam<int32>			InIndex(Context);
		FNDIInputParam<TVMArrayType>	InValue(Context);
		FNDIOutputParam<TVMArrayType>	OutPrevValue(Context);
		FNDIOutputParam<TVMArrayType>	OutCurrValue(Context);
		
		const TVMArrayType DefaultValue = FNDIArrayImplHelper<TArrayType>::GetDefaultValue();

		//-OPT: Can do a in batches or single atomic op rather than one per instance
		FWriteArrayRef ArrayData(Owner, InstanceData);
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool bSkipExecute = InSkipOp.GetAndAdvance();
			const int32 Index = InIndex.GetAndAdvance();
			const TVMArrayType Value = InValue.GetAndAdvance();
			if (!bSkipExecute && ArrayData.GetArray().IsValidIndex(Index))
			{
				TVMArrayType PreviousValue = FNDIArrayImplHelper<TArrayType>::AtomicAdd(&ArrayData.GetArray().GetData()[Index], Value);
				OutPrevValue.SetAndAdvance(PreviousValue);
				OutCurrValue.SetAndAdvance(PreviousValue + Value);
			}
			else
			{
				OutPrevValue.SetAndAdvance(DefaultValue);
				OutCurrValue.SetAndAdvance(DefaultValue);
			}
		}

		InstanceData->bIsRenderDirty |= bShouldSyncToGpu;
	}

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<T::bSupportsAtomicOps>::Type VMAtomicMin(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIInputParam<FNiagaraBool>	InSkipOp(Context);
		FNDIInputParam<int32>			InIndex(Context);
		FNDIInputParam<TVMArrayType>	InValue(Context);
		FNDIOutputParam<TVMArrayType>	OutPrevValue(Context);
		FNDIOutputParam<TVMArrayType>	OutCurrValue(Context);

		const TVMArrayType DefaultValue = FNDIArrayImplHelper<TArrayType>::GetDefaultValue();

		//-OPT: Can do a in batches or single atomic op rather than one per instance
		FWriteArrayRef ArrayData(Owner, InstanceData);
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool bSkipExecute = InSkipOp.GetAndAdvance();
			const int32 Index = InIndex.GetAndAdvance();
			const TVMArrayType Value = InValue.GetAndAdvance();
			if (!bSkipExecute && ArrayData.GetArray().IsValidIndex(Index))
			{
				TVMArrayType PreviousValue = FNDIArrayImplHelper<TArrayType>::AtomicMin(&ArrayData.GetArray().GetData()[Index], Value);
				OutPrevValue.SetAndAdvance(PreviousValue);
				OutCurrValue.SetAndAdvance(FMath::Min(PreviousValue, Value));
			}
			else
			{
				OutPrevValue.SetAndAdvance(DefaultValue);
				OutCurrValue.SetAndAdvance(DefaultValue);
			}
		}

		InstanceData->bIsRenderDirty |= bShouldSyncToGpu;
	}

	template<typename T = FNDIArrayImplHelper<TArrayType>>
	typename TEnableIf<T::bSupportsAtomicOps>::Type VMAtomicMax(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIArrayInstanceData_GameThread<TArrayType>> InstanceData(Context);
		FNDIInputParam<FNiagaraBool>	InSkipOp(Context);
		FNDIInputParam<int32>			InIndex(Context);
		FNDIInputParam<TVMArrayType>	InValue(Context);
		FNDIOutputParam<TVMArrayType>	OutPrevValue(Context);
		FNDIOutputParam<TVMArrayType>	OutCurrValue(Context);

		const TVMArrayType DefaultValue = FNDIArrayImplHelper<TArrayType>::GetDefaultValue();

		//-OPT: Can do a in batches or single atomic op rather than one per instance
		FWriteArrayRef ArrayData(Owner, InstanceData);
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool bSkipExecute = InSkipOp.GetAndAdvance();
			const int32 Index = InIndex.GetAndAdvance();
			const TVMArrayType Value = InValue.GetAndAdvance();
			if (!bSkipExecute && ArrayData.GetArray().IsValidIndex(Index))
			{
				TVMArrayType PreviousValue = FNDIArrayImplHelper<TArrayType>::AtomicMax(&ArrayData.GetArray().GetData()[Index], Value);
				OutPrevValue.SetAndAdvance(PreviousValue);
				OutCurrValue.SetAndAdvance(FMath::Max(PreviousValue, Value));
			}
			else
			{
				OutPrevValue.SetAndAdvance(DefaultValue);
				OutCurrValue.SetAndAdvance(DefaultValue);
			}
		}

		InstanceData->bIsRenderDirty |= bShouldSyncToGpu;
	}

	const FNDIArrayInstanceData_GameThread<TArrayType>* GetPerInstanceData_GameThread(FNiagaraSystemInstanceID SystemInstanceID) const
	{
		return PerInstanceData_GameThread.FindRef(SystemInstanceID);
	}

private:
	TOwnerType*	Owner = nullptr;
	bool bShouldSyncToGpu = false;
	bool bShouldSyncToCpu = false;

	TMap<FNiagaraSystemInstanceID, FNDIArrayInstanceData_GameThread<TArrayType>*>	PerInstanceData_GameThread;
	TMap<FNiagaraSystemInstanceID, FNDIArrayInstanceData_RenderThread<TArrayType>>	PerInstanceData_RenderThread;
};
