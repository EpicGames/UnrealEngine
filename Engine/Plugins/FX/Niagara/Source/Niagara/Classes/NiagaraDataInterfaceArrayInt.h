// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceArray.h"
#include "NiagaraDataInterfaceArrayInt.generated.h"

UCLASS(EditInlineNew, Category = "Array", CollapseCategories, meta = (DisplayName = "Int32 Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayInt32 : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<int32> IntData;

	NDIARRAY_GENERATE_BODY(UNiagaraDataInterfaceArrayInt32, int32, IntData)
};

UCLASS(EditInlineNew, Category = "Array", CollapseCategories, meta = (DisplayName = "UInt8 Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayUInt8 : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<int32> IntData;
#endif

	UPROPERTY()
	TArray<uint8> InternalIntData;

	NDIARRAY_GENERATE_BODY_LWC(UNiagaraDataInterfaceArrayUInt8, uint8, IntData)
};

UCLASS(EditInlineNew, Category = "Array", CollapseCategories, meta = (DisplayName = "Bool Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayBool : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<bool> BoolData;

	NDIARRAY_GENERATE_BODY(UNiagaraDataInterfaceArrayBool, bool, BoolData)
};
