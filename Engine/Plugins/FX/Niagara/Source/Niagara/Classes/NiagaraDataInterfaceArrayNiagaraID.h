// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceArray.h"
#include "NiagaraDataInterfaceArrayNiagaraID.generated.h"

UCLASS(EditInlineNew, Category = "Array", meta = (DisplayName = "NiagaraID Array", Experimental), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayNiagaraID : public UNiagaraDataInterfaceArray
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Array")
	TArray<FNiagaraID> IntData;

	NDIARRAY_GENERATE_BODY(UNiagaraDataInterfaceArrayNiagaraID, FNiagaraID, IntData)
};
