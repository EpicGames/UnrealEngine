// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraDataInterface.h"
#include "NiagaraSettings.h"
#include "NiagaraShared.h"

#include "NiagaraDataInterfaceAsyncGpuTrace.generated.h"

class FNiagaraSystemInstance;

/** Data interface for handling latent (delayed 1 frame) traces against the scene for GPU simulations. */
UCLASS(EditInlineNew, Category = "Collision", CollapseCategories, meta = (DisplayName = "Async Gpu Trace"), MinimalAPI)
class UNiagaraDataInterfaceAsyncGpuTrace : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	/** The maximum number of traces (per particle) that can be created per frame.  Defines the size of the preallocated 
		buffer that is used to contain the traces. */
	UPROPERTY(EditAnywhere, Category = "Async GPU Trace")
	int32 MaxTracesPerParticle = 1;

	/** If a collision is rejected, how many times do we attempt to retrace from that collision point forward to find a new, valid collision.*/
	UPROPERTY(EditAnywhere, Category = "Async GPU Trace")
	int32 MaxRetraces = 0;

	/** Defines which technique is used to resolve the trace - see 'AsyncGpuTraceDI/Provider Priority' in the Niagara project settings. */
	UPROPERTY(EditAnywhere, Category = "Async GPU Trace")
	TEnumAsByte<ENDICollisionQuery_AsyncGpuTraceProvider::Type> TraceProvider = ENDICollisionQuery_AsyncGpuTraceProvider::Default;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;

#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo) override;
#endif
	//UObject Interface End

	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance) override;

	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target == ENiagaraSimTarget::GPUComputeSim; }
	NIAGARA_API virtual bool RequiresGlobalDistanceField() const override;
	NIAGARA_API virtual bool RequiresRayTracingScene() const override;

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

private:
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	NIAGARA_API virtual void PushToRenderThreadImpl() override;
};
