// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "Sound/SoundSubmix.h"

#include "NiagaraDataInterfaceAudioOscilloscope.generated.h"

/** Data Interface allowing sampling of recent audio data. */
UCLASS(EditInlineNew, Category = "Audio", CollapseCategories, meta = (DisplayName = "Audio Oscilloscope"), MinimalAPI)
class UNiagaraDataInterfaceAudioOscilloscope : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Oscilloscope")
	TObjectPtr<USoundSubmix> Submix;

	// The number of samples of audio to pass to the GPU. Audio will be resampled to fit this resolution.
	// Increasing this number will increase the resolution of the waveform, but will also increase usage of the GPU memory bus,
	// potentially causing issues across the application.
	UPROPERTY(EditAnywhere, Category = "Oscilloscope", AdvancedDisplay, meta = (ClampMin = "64", ClampMax = "8192"))
	int32 Resolution = 512;

	// The number of milliseconds of audio to show.
	UPROPERTY(EditAnywhere, Category = "Oscilloscope", meta = (ClampMin = "5.0", ClampMax = "400.0"))
	float ScopeInMilliseconds = 20.0f;

	//VM function overrides:
	NIAGARA_API void SampleAudio(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetNumChannels(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;

	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override
	{
		return true;
	}

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual void PostInitProperties() override;
	
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasTickGroupPrereqs() const override { return true; }
	NIAGARA_API virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const override;

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
};

