// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraDataInterface.h"
#include "NiagaraParameterStore.h"
#include "Sound/SoundAttenuation.h"

#include "NiagaraDataInterfaceAudioPlayer.generated.h"

class FNiagaraSystemInstance;
class UAudioComponent;
class USoundConcurrency;
struct FAudioPlayerInterface_InstanceData;

struct FAudioParticleData
{
	FVector Position;
	FRotator Rotation;
	float Volume = 1;
	float Pitch = 1;
	float StartTime = 1;
	int32 ParticleID = -1;
};

struct FPersistentAudioParticleData
{
	int32 AudioHandle = 0;

	/** The update callback is executed in PerInstanceTickPostSimulate, which runs on the game thread */
	TFunction<void(struct FAudioPlayerInterface_InstanceData*,UAudioComponent*,FNiagaraSystemInstance*)> UpdateCallback;
};

struct FAudioInitialParamData
{
	int32 ParticleID = -1;
	FNiagaraVariable Value;
};

UCLASS(EditInlineNew, Category = "Audio", meta = (DisplayName = "Niagara Audio Player Settings"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceAudioPlayerSettings : public UObject
{
public:
	GENERATED_BODY()

	NIAGARA_API UNiagaraDataInterfaceAudioPlayerSettings();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio", meta=(InlineEditConditionToggle))
	bool bOverrideConcurrency = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio", meta=(EditCondition="bOverrideConcurrency"))
	TObjectPtr<USoundConcurrency> Concurrency;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio")
	bool bOverrideAttenuationSettings = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio", meta=(EditCondition="bOverrideAttenuationSettings"))
	FSoundAttenuationSettings AttenuationSettings;
};

struct FAudioPlayerInterface_InstanceData
{
	/** We use a lock-free queue here because multiple threads might try to push data to it at the same time. */
	TQueue<FAudioParticleData, EQueueMode::Mpsc> PlayAudioQueue;
	TQueue<FAudioInitialParamData, EQueueMode::Mpsc> InitialParamDataQueue;
	TQueue<FPersistentAudioParticleData, EQueueMode::Mpsc> PersistentAudioActionQueue;
	FThreadSafeCounter HandleCount;
	int32 ParamCountEstimate = 0;

	TSortedMap<int32, TWeakObjectPtr<UAudioComponent>> PersistentAudioMapping;

	TWeakObjectPtr<USoundBase> SoundToPlay;
	TWeakObjectPtr<USoundAttenuation> Attenuation;
	TWeakObjectPtr<USoundConcurrency> Concurrency;
	TWeakObjectPtr<UNiagaraDataInterfaceAudioPlayerSettings> CachedUserParam;
	TArray<FName> ParameterNames;
	TSet<FNiagaraVariable> GlobalInitialParameterValues;

	FNiagaraLWCConverter LWCConverter;
	int32 MaxPlaysPerTick = 0;
	bool bStopWhenComponentIsDestroyed = true;
	bool bValidOneShotSound = false;
	bool bSoundToPlayIsLooping = false;
#if WITH_EDITORONLY_DATA
	bool bOnlyActiveDuringGameplay = false;
#endif

	/** A binding to the user ptr we're reading from (if we are). */
	FNiagaraParameterDirectBinding<UObject*> UserParamBinding;
};

/** This Data Interface can be used to play one-shot audio effects driven by particle data. */
UCLASS(EditInlineNew, Category = "Audio", meta = (DisplayName = "Audio Player"), MinimalAPI)
class UNiagaraDataInterfaceAudioPlayer : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	/** Reference to the audio asset to play */
	UPROPERTY(EditAnywhere, Category = "Audio")
    TObjectPtr<USoundBase> SoundToPlay;

	/** Optional sound attenuation setting to use */
	UPROPERTY(EditAnywhere, Category = "Audio")
	TObjectPtr<USoundAttenuation> Attenuation;

	/** Optional sound concurrency setting to use */
	UPROPERTY(EditAnywhere, Category = "Audio")
	TObjectPtr<USoundConcurrency> Concurrency;
	
	/** A set of parameter names that can be referenced via index when setting sound cue parameters on persistent audio */
	UPROPERTY(EditAnywhere, Category = "Parameters")
	TArray<FName> ParameterNames;

	/** If bound to a valid user parameter object of type UNiagaraDataInterfaceAudioPlayerSettings, then configured settings like sound attenuation are set via the user parameter. This allows the sound settings to be dynamically changed via blueprint or C++.
	 *  Only used by persistent audio, one-shot audio ignores this option. */
	UPROPERTY(EditAnywhere, Category = "Parameters")
	FNiagaraUserParameterBinding ConfigurationUserParameter;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Audio", meta = (InlineEditConditionToggle))
    bool bLimitPlaysPerTick;

	/** This sets the max number of sounds played each tick.
	 *  If more particles try to play a sound in a given tick, then it will play sounds until the limit is reached and discard the rest.
	 *  The particles to discard when over the limit are *not* chosen in a deterministic way. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Audio", meta=(EditCondition="bLimitPlaysPerTick", ClampMin="0", UIMin="0"))
    int32 MaxPlaysPerTick;

	/** If false then it the audio component keeps playing after the niagara component was destroyed. Looping sounds are always stopped when the component is destroyed. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Audio")
	bool bStopWhenComponentIsDestroyed = true;

	/** Playing looping sounds as persistent audio is not a problem, as the sound is stopped when a particle dies, but one-shot audio outlives the niagara system and can never be stopped. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Audio")
	bool bAllowLoopingOneShotSounds = false;

#if WITH_EDITORONLY_DATA
	/** If true then this data interface only processes sounds during active gameplay. This is useful when you are working in the preview window and the sounds annoy you. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Audio")
	bool bOnlyActiveDuringGameplay = false;

	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	
	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual int32 PerInstanceDataSize() const override { return sizeof(FAudioPlayerInterface_InstanceData); }
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target == ENiagaraSimTarget::CPUSim; }

	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }
	virtual bool PostSimulateCanOverlapFrames() const override { return false; }
	//UNiagaraDataInterface Interface

	NIAGARA_API virtual void PlayOneShotAudio(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void PlayPersistentAudio(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void SetParameterBool(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void SetParameterInteger(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void SetParameterFloat(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void SetInitialParameterBool(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void SetInitialParameterInteger(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void SetInitialParameterFloat(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void UpdateVolume(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void UpdatePitch(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void UpdateLocation(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void UpdateRotation(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void SetPausedState(FVectorVMExternalFunctionContext& Context);

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	
private:
	static NIAGARA_API const FName PlayAudioName;
	static NIAGARA_API const FName PlayPersistentAudioName;
	static NIAGARA_API const FName SetPersistentAudioVolumeName;
	static NIAGARA_API const FName SetPersistentAudioPitchName;
	static NIAGARA_API const FName SetPersistentAudioLocationName;
	static NIAGARA_API const FName SetPersistentAudioRotationName;
	static NIAGARA_API const FName SetPersistentAudioBoolParamName;
	static NIAGARA_API const FName SetPersistentAudioIntegerParamName;
	static NIAGARA_API const FName SetPersistentAudioFloatParamName;
	static NIAGARA_API const FName SetInitialAudioBoolParamName;
	static NIAGARA_API const FName SetInitialAudioIntegerParamName;
	static NIAGARA_API const FName SetInitialAudioFloatParamName;
	static NIAGARA_API const FName PausePersistentAudioName;
};
