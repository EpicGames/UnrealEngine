// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "Sound/SoundSubmix.h"

#include "NiagaraDataInterfaceAudioSpectrum.generated.h"

class FNDIAudio_SharedResource;

/** Data Interface allowing sampling of recent audio spectrum. */
UCLASS(EditInlineNew, Category = "Audio", meta = (DisplayName = "Audio Spectrum"), MinimalAPI)
class UNiagaraDataInterfaceAudioSpectrum : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	/** The audio submix where analyzed. */
	UPROPERTY(EditAnywhere, Category = "Audio")
	TObjectPtr<USoundSubmix> Submix;

	/** The number of spectrum samples to pass to the GPU */
	UPROPERTY(EditAnywhere, Category = "Spectrum", meta = (ClampMin = "16", ClampMax = "1024") )
	int32 Resolution = 512;

	/** The minimum frequency represented in the spectrum. */
	UPROPERTY(EditAnywhere, Category = "Spectrum", meta = (ClampMin = "20.0", ClampMax = "20000.0"))
	float MinimumFrequency = 55.0f;

	/** The maximum frequency represented in the spectrum. */
	UPROPERTY(EditAnywhere, Category = "Spectrum", meta = (ClampMin = "20.0", ClampMax = "20000.0"))
	float MaximumFrequency = 10000.f;

	/** The decibel level considered as silence. This is used to scale the output of the spectrum. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Spectrum", meta = (ClampMin = "-120.0", ClampMax = "0.0"))
	float NoiseFloorDb = -60.0f;

	//VM function overrides:
	NIAGARA_API void GetSpectrumValue(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetNumChannels(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;

	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override 
	{
		return true;
	}

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual void PostInitProperties() override;

	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasTickGroupPrereqs() const override { return true; }
	NIAGARA_API virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const override;

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
private:

};

