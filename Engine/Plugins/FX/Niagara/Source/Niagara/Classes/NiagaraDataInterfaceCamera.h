// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"
#include "Engine/ViewportSplitScreen.h"
#include "NiagaraDataInterfaceCamera.generated.h"

struct FDistanceData
{
	FNiagaraID ParticleID;
	double DistanceSquared;
};

struct FCameraDataInterface_InstanceData
{
	FNiagaraPosition CameraLocation = FVector::ZeroVector;
	FRotator CameraRotation = FRotator::ZeroRotator;
	float CameraFOV = 0.0f;
	float ViewportSizeX = 0.0f;
	float ViewportSizeY = 0.0f;
	float NearClipPlane = 0.0f;
	ESplitScreenType::Type SplitscreenMode = ESplitScreenType::Type::None;
	FNiagaraLWCConverter LWCConverter;

	TQueue<FDistanceData, EQueueMode::Mpsc> DistanceSortQueue;
	TArray<FDistanceData> ParticlesSortedByDistance;	
};

UCLASS(EditInlineNew, Category = "Camera", CollapseCategories, meta = (DisplayName = "Camera Query"), MinimalAPI)
class UNiagaraDataInterfaceCamera : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(int32, SplitscreenMode)
	END_SHADER_PARAMETER_STRUCT();

public:
	/** This is used to determine which camera position to query for cpu emitters. If no valid index is supplied, the first controller is used as camera reference. */
	UPROPERTY(EditAnywhere, Category = "Camera")
	int32 PlayerControllerIndex = 0;


	/** When this option is disabled, we use the previous frame's data for the camera and issue the simulation early. This greatly
	reduces overhead and allows the game thread to run faster, but comes at a tradeoff if the dependencies might leave gaps or other visual artifacts.*/
	UPROPERTY(EditAnywhere, Category = "Performance")
	bool bRequireCurrentFrameData = true;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual int32 PerInstanceDataSize() const override { return sizeof(FCameraDataInterface_InstanceData); }
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
	virtual bool HasTickGroupPrereqs() const override { return true; }
	NIAGARA_API virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const override;
	virtual bool RequiresEarlyViewData() const override { return true; }
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool HasPreSimulateTick() const override { return true; }
#if WITH_EDITOR	
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors,
        TArray<FNiagaraDataInterfaceFeedback>& Warnings, TArray<FNiagaraDataInterfaceFeedback>& Info) override;
#endif
	//UNiagaraDataInterface Interface

	NIAGARA_API void CalculateParticleDistances(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetClosestParticles(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetCameraFOV(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetCameraProperties(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetGetSplitScreenInfo(FVectorVMExternalFunctionContext& Context);

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
private:
	static NIAGARA_API const TCHAR* TemplateShaderFilePath;
	static NIAGARA_API const FName CalculateDistancesName;
	static NIAGARA_API const FName QueryClosestName;
	static NIAGARA_API const FName GetViewPropertiesName;
	static NIAGARA_API const FName GetClipSpaceTransformsName;
	static NIAGARA_API const FName GetViewSpaceTransformsName;
	static NIAGARA_API const FName GetCameraPropertiesName;
	static NIAGARA_API const FName GetFieldOfViewName;
	static NIAGARA_API const FName GetTAAJitterName;
	static NIAGARA_API const FName GetSplitScreenName;
	static NIAGARA_API const FName ApplyPreViewTranslationToPosition;
	static NIAGARA_API const FName RemovePreViewTranslationFromPosition;
};
