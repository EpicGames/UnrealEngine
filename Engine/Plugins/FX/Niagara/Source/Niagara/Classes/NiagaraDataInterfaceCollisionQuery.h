// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraCollision.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraShared.h"
#include "ShaderParameterMacros.h"
#include "VectorVM.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceCollisionQuery.generated.h"

class INiagaraCompiler;
class FNiagaraSystemInstance;


struct CQDIPerInstanceData
{
	FNiagaraSystemInstance *SystemInstance;
	FNiagaraDICollisionQueryBatch CollisionBatch;
};


/** Data Interface that can be used to query collision related data, like geometry traces or sampling the depth buffer. */
UCLASS(EditInlineNew, Category = "Collision", meta = (DisplayName = "Collision Query"), MinimalAPI)
class UNiagaraDataInterfaceCollisionQuery : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	//UObject Interface End

	/** Initializes the per instance data for this interface. Returns false if there was some error and the simulation should be disabled. */
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance) override;
	/** Destroys the per instence data for this interface. */
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance) override;

	/** Ticks the per instance data for this interface, if it has any. */
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual int32 PerInstanceDataSize() const override { return sizeof(CQDIPerInstanceData); }
	
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	NIAGARA_API virtual void GetAssetTagsForContext(const UObject* InAsset, FGuid AssetVersion, const TArray<const UNiagaraDataInterface*>& InProperties, TMap<FName, uint32>& NumericKeys, TMap<FName, FString>& StringKeys) const override;

	// VM functions
	NIAGARA_API void PerformQuerySyncCPU(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void PerformQueryAsyncCPU(FVectorVMExternalFunctionContext& Context);

	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
	virtual bool RequiresGlobalDistanceField() const override { return true; }
	virtual bool RequiresDepthBuffer() const override { return true; }

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

#if WITH_EDITOR	
	NIAGARA_API virtual void ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors) override;
#endif
	
	virtual bool HasPreSimulateTick() const override{ return true; }
	virtual bool HasPostSimulateTick() const override { return true; }
	virtual bool PostSimulateCanOverlapFrames() const { return false; }

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

private:
	static NIAGARA_API FCriticalSection CriticalSection;
	UEnum* TraceChannelEnum;
};
