// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "NiagaraDataInterfaceCurlNoise.generated.h"

/** Data Interface allowing sampling of curl noise LUT. */
UCLASS(EditInlineNew, Category = "Curl Noise LUT", CollapseCategories, meta = (DisplayName = "Curl Noise"), MinimalAPI)
class UNiagaraDataInterfaceCurlNoise : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FVector3f, OffsetFromSeed)
	END_SHADER_PARAMETER_STRUCT();

public:
	UPROPERTY(EditAnywhere, Category = "Curl Noise")
	uint32 Seed;

	// Precalculated when Seed changes. 
	FVector3f OffsetFromSeed;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties()override;
	NIAGARA_API virtual void PostLoad() override;
#if WITH_EDITOR
	NIAGARA_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target)const override { return true; }
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	//UNiagaraDataInterface Interface End

	NIAGARA_API void SampleNoiseField(FVectorVMExternalFunctionContext& Context);

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	NIAGARA_API virtual void PushToRenderThreadImpl() override;
};

struct FNiagaraDataInterfaceProxyCurlNoise : public FNiagaraDataInterfaceProxy
{
	FNiagaraDataInterfaceProxyCurlNoise(const FVector3f& InOffset)
	{
		OffsetFromSeed = InOffset;
	}

	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { check(false); }
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return 0;
	}

	FVector3f OffsetFromSeed;
};
