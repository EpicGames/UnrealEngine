// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "StaticMeshResources.h"
#include "Curves/RichCurve.h"
#include "Curves/CurveBase.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceCurveBase.generated.h"

/** Base class for curve data interfaces which facilitates handling the curve data in a standardized way. */
UCLASS(EditInlineNew, abstract, MinimalAPI)
class UNiagaraDataInterfaceCurveBase : public UNiagaraDataInterface
{
public:
	static constexpr float DefaultOptimizeThreshold = 0.01f;

protected:
	GENERATED_BODY()

	UPROPERTY()
	TArray<float> ShaderLUT;

	UPROPERTY()
	float LUTMinTime;

	UPROPERTY()
	float LUTMaxTime;

	UPROPERTY()
	float LUTInvTimeRange;

	UPROPERTY()
	float LUTNumSamplesMinusOne;

	uint32 LUTOffset = INDEX_NONE;

	/** Remap a sample time for this curve to 0 to 1 between first and last keys for LUT access.*/
	inline float NormalizeTime(float T) const
	{
		return (T - LUTMinTime) * LUTInvTimeRange;
	}

	/** Remap a 0 to 1 value between the first and last keys to a real sample time for this curve. */
	inline float UnnormalizeTime(float T) const
	{
		return (T / LUTInvTimeRange) + LUTMinTime;
	}

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Curve")
	TObjectPtr<UCurveBase> CurveAsset;
#endif
	
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Curve")
	uint32 bUseLUT : 1;

	/** Generates a texture for the curve which can be exposed to material bindings. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Curve", meta = (DisplayName = "Expose Curve to Material"))
	uint32 bExposeCurve : 1;

#if WITH_EDITORONLY_DATA
	/** Do we optimize the LUT, this saves memory but may introduce errors.  Errors can be reduced modifying the threshold. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Curve")
	uint32 bOptimizeLUT : 1;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Curve")
	uint32 bOverrideOptimizeThreshold : 1;

	UPROPERTY(Transient)
	uint32 HasEditorData : 1;

	/** Threshold used to optimize the LUT. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Curve", meta = (EditCondition = "bOverrideOptimizeThreshold"))
	float OptimizeThreshold;
#endif

	/** Sets a custom name for the binding to make it easier to identify. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Curve", meta = (DisplayName = "Exposed Curve Name"))
	FName ExposedName;

	/** The texture generated and exposed to materials, will be nullptr if we do not expose to the renderers. */
	UPROPERTY()
	TObjectPtr<class UTexture2D> ExposedTexture;

#if WITH_EDITOR	
	/** Refreshes and returns the errors detected with the corresponding data, if any.*/
	NIAGARA_API virtual TArray<FNiagaraDataInterfaceError> GetErrors() override;
#endif

public:
	NIAGARA_API UNiagaraDataInterfaceCurveBase();
	NIAGARA_API UNiagaraDataInterfaceCurveBase(FObjectInitializer const& ObjectInitializer);

	enum
	{
		CurveLUTDefaultWidth = 128,
	};

	/** Structure to facilitate getting standardized curve information from a curve data interface. */
	struct FCurveData
	{
		FCurveData(FRichCurve* InCurve, FName InName, FLinearColor InColor)
			: Curve(InCurve)
			, Name(InName)
			, Color(InColor)
		{
		}
		/** A pointer to the curve. */
		FRichCurve* Curve;
		/** The name of the curve, unique within the data interface, which identifies the curve in the UI. */
		FName Name;
		/** The color to use when displaying this curve in the UI. */
		FLinearColor Color;
	};

	//UObject Interface
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//UObject Interface End

	/** Gets information for all of the curves owned by this curve data interface. */
	virtual void GetCurveData(TArray<FCurveData>& OutCurveData) { }

	NIAGARA_API virtual void CacheStaticBuffers(struct FNiagaraSystemStaticBuffers& StaticBuffers, bool bIsExternalDataInterface, bool bUsedByCPU, bool bUsedByGPU) override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	NIAGARA_API virtual void PostCompile(const UNiagaraSystem& OwningSystem) override;

	NIAGARA_API void SetDefaultLUT();
#if WITH_EDITORONLY_DATA
	NIAGARA_API void UpdateLUT(bool bFromSerialize = false);
	NIAGARA_API void OptimizeLUT();
	NIAGARA_API void UpdateExposedTexture();
	virtual void SyncCurvesToAsset() {};
#endif

	//UNiagaraDataInterface interface
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	virtual bool CanExposeVariables() const override { return true; }
	NIAGARA_API virtual void GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const override;
	NIAGARA_API virtual bool GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const override;

	virtual int32 GetCurveNumElems() const PURE_VIRTUAL(UNiagaraDataInterfaceCurveBase::GetCurveNumElems(), return 0;)
#if WITH_EDITORONLY_DATA
	virtual FName GetCurveSampleFunctionName() const PURE_VIRTUAL(UNiagaraDataInterfaceCurveBase::GetCurveSampleFunctionName(), return NAME_None;)
#endif

	virtual void UpdateTimeRanges() PURE_VIRTUAL(UNiagaraDataInterfaceCurveBase::UpdateTimeRanges(), )
	virtual TArray<float> BuildLUT(int32 NumEntries) const PURE_VIRTUAL(UNiagaraDataInterfaceCurveBase::BuildLUT(), return TArray<float>(); )

	inline float GetMinTime()const { return LUTMinTime; }
	inline float GetMaxTime()const { return LUTMaxTime; }
	inline float GetInvTimeRange()const { return LUTInvTimeRange; }

protected:
	NIAGARA_API virtual void PushToRenderThreadImpl() override;
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	NIAGARA_API virtual bool CompareLUTS(const TArray<float>& OtherLUT) const;
	//UNiagaraDataInterface interface END
};

//External function binder choosing between template specializations based on if a curve should use the LUT over full evaluation.
template<typename NextBinder>
struct TCurveUseLUTBinder
{
	template<typename... ParamTypes>
	static void Bind(UNiagaraDataInterface* Interface, const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
	{
		UNiagaraDataInterfaceCurveBase* CurveInterface = CastChecked<UNiagaraDataInterfaceCurveBase>(Interface);
		if (CurveInterface->bUseLUT)
		{
			NextBinder::template Bind<ParamTypes..., TIntegralConstant<bool, true>>(Interface, BindingInfo, InstanceData, OutFunc);
		}
		else
		{
			NextBinder::template Bind<ParamTypes..., TIntegralConstant<bool, false>>(Interface, BindingInfo, InstanceData, OutFunc);
		}
	}
};
