// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"
#include "UObject/Interface.h"
#include "NiagaraDataInterfaceExport.generated.h"

USTRUCT(BlueprintType, Blueprintable)
struct FBasicParticleData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "Particle Data")
	FVector Position = FVector(ForceInitToZero);

	UPROPERTY(BlueprintReadOnly, Category = "Particle Data")
	float Size = 0.0f;

	UPROPERTY(BlueprintReadOnly, Category = "Particle Data")
	FVector Velocity = FVector(ForceInitToZero);
};

UINTERFACE(BlueprintType)
class UNiagaraParticleCallbackHandler : public UInterface
{
	GENERATED_BODY()
};

class INiagaraParticleCallbackHandler
{
	GENERATED_BODY()

public:
	/** This function is called once per tick with the gathered particle data. It will not be called if there is no particle data to call it with. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Niagara")
	void ReceiveParticleData(const TArray<FBasicParticleData>& Data, UNiagaraSystem* NiagaraSystem, const FVector& SimulationPositionOffset);
};

UENUM()
enum class ENDIExport_GPUAllocationMode : uint8
{
	/** Use a fixed number of elements to write into per frame. */
	FixedSize,
	/** Use a variable number of elements to write into per frame based on number of emitter property particle count multiplied by the modifier. */
	PerParticle,
};

/** This Data Interface can be used to gather particles at execution time and call either a 
C++ or blueprint object with the gathered particle data each tick. */
UCLASS(EditInlineNew, Category = "Export", meta = (DisplayName = "Export particle data"), MinimalAPI)
class UNiagaraDataInterfaceExport : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	/** Reference to a user parameter that should receive the particle data after the simulation tick. The supplied parameter object needs to implement the INiagaraParticleCallbackHandler interface. */
	UPROPERTY(EditAnywhere, Category = "Export")
	FNiagaraUserParameterBinding CallbackHandlerParameter;

	UPROPERTY(EditAnywhere, Category = "Export GPU", meta = (Tooltip = "Sets the allocation scheme for the number of elements we reserve for the GPU.  The number of elements reserved should be as low as possible to improve performance."))
	ENDIExport_GPUAllocationMode GPUAllocationMode = ENDIExport_GPUAllocationMode::FixedSize;

	/** Reserve a fixed number of elements we can write per frame.  Once the limit is reached we ignore further writes. */
	UPROPERTY(EditAnywhere, Category = "Export GPU", meta = (EditCondition = "GPUAllocationMode == ENDIExport_GPUAllocationMode::FixedSize"))
	int GPUAllocationFixedSize = 64;

	/** Uses the emitter property particle count * this to determine the number of elements we reserve for write per frame.  The console variable fx.Niagara.NDIExport.GPUMaxReadbackCount is used to cap the maximum.  Once the limit is reached we ignore further writes. */
	UPROPERTY(EditAnywhere, Category = "Export GPU", meta = (EditCondition = "GPUAllocationMode == ENDIExport_GPUAllocationMode::PerParticle", UIMin="0.0", ClampMin="0.0"))
	float GPUAllocationPerParticleSize = 1.0f;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }
	virtual bool PostSimulateCanOverlapFrames() const { return false; }
	//UNiagaraDataInterface Interface

	virtual bool HasInternalAttributeReads(const UNiagaraEmitter* OwnerEmitter, const UNiagaraEmitter* Provider) const override { return OwnerEmitter == Provider; };

	NIAGARA_API virtual void StoreData(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API virtual void ExportData(FVectorVMExternalFunctionContext& Context);

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
};
