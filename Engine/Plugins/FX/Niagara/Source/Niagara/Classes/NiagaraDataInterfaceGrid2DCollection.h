// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceRW.h"
#include "ClearQuad.h"
#include "NiagaraComponent.h"
#include "NiagaraRenderGraphUtils.h"
#include "NiagaraDataInterfaceRWUtils.h"

#include "NiagaraDataInterfaceGrid2DCollection.generated.h"

class FNiagaraSystemInstance;
class UTextureRenderTarget;
class UTextureRenderTarget2DArray;

using FGrid2DBuffer = FNiagaraPooledRWTexture;

struct FGrid2DCollectionRWInstanceData_GameThread
{
	bool ClearBeforeNonIterationStage = true;

	FIntPoint NumCells = FIntPoint(EForceInit::ForceInitToZero);
	int32 NumAttributes = 0;
	FVector2D CellSize = FVector2D::ZeroVector;
	FVector2D WorldBBoxSize = FVector2D::ZeroVector;
	TOptional<EPixelFormat> PixelFormat;
#if WITH_EDITORONLY_DATA
	bool bPreviewGrid = false;
	FIntVector4 PreviewAttribute = FIntVector4(INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE);
#endif

	/** A binding to the user ptr we're reading the RT from (if we are). */
	FNiagaraParameterDirectBinding<UObject*> RTUserParamBinding;

	UTextureRenderTarget* TargetTexture = nullptr;	
	TArray<FNiagaraVariableBase> Vars;
	TArray<uint32> Offsets;

	bool NeedsRealloc = false;

	// We need to essentially make this a linked list to avoid more refactoring for now
	// eventually we can clean this logic up, but this allows us to have a subclass that
	// overrides the render thread data, which in this case is for a grid reader
	UNiagaraDataInterface* OtherDI = nullptr;
	FGrid2DCollectionRWInstanceData_GameThread* OtherInstanceData = nullptr;

	int32 FindAttributeIndexByName(const FName& InName, int32 NumChannels);

	bool UpdateTargetTexture(ENiagaraGpuBufferFormat BufferFormat, UObject* Owner);
};

struct FGrid2DCollectionRWInstanceData_RenderThread
{
	FName SourceDIName;
	bool ClearBeforeNonIterationStage = true;

	FIntPoint NumCells = FIntPoint(EForceInit::ForceInitToZero);
	int32 NumAttributes = 0;
	FVector2D CellSize = FVector2D::ZeroVector;
	FVector2D WorldBBoxSize = FVector2D::ZeroVector;
	TOptional<EPixelFormat> PixelFormat;

	TArray<TUniquePtr<FGrid2DBuffer>, TInlineAllocator<2>> Buffers;
	FGrid2DBuffer* CurrentData = nullptr;
	FGrid2DBuffer* DestinationData = nullptr;

	FTextureRHIRef RenderTargetToCopyTo; 
	TArray<int32> AttributeIndices;
	TArray<FName> Vars;
	TArray<int32> VarComponents;
	TArray<uint32> Offsets;

#if WITH_EDITORONLY_DATA
	bool bPreviewGrid = false;
	FIntVector4 PreviewAttribute = FIntVector4(INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE);
#endif

	// We need to essentially make this a linked list to avoid more refactoring for now
	// eventually we can clean this logic up, but this allows us to have a subclass that
	// overrides the render thread data, which in this case is for a grid reader
	FNiagaraDataInterfaceProxy* OtherProxy = nullptr;

	void BeginSimulate(FRDGBuilder& GraphBuilder, bool RequiresBuffering);
	void EndSimulate();
};

struct FNiagaraDataInterfaceProxyGrid2DCollectionProxy : public FNiagaraDataInterfaceProxyRW
{
	FNiagaraDataInterfaceProxyGrid2DCollectionProxy() {}
	
	virtual void ResetData(const FNDIGpuComputeResetContext& Context) override;
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override;
	virtual void PostStage(const FNDIGpuComputePostStageContext& Context) override;
	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override;

	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override;

	/* List of proxy data for each system instances*/
	// #todo(dmp): this should all be refactored to avoid duplicate code
	TMap<FNiagaraSystemInstanceID, FGrid2DCollectionRWInstanceData_RenderThread> SystemInstancesToProxyData_RT;

};

struct FNiagaraDataInterfaceParametersCS_Grid2DCollection : public FNiagaraDataInterfaceParametersCS
{
	DECLARE_TYPE_LAYOUT(FNiagaraDataInterfaceParametersCS_Grid2DCollection, NonVirtual);

	LAYOUT_FIELD(TMemoryImageArray<FMemoryImageName>, AttributeNames);
	LAYOUT_FIELD(TMemoryImageArray<uint32>, AttributeChannelCount);
};

UCLASS(EditInlineNew, Category = "Grid", CollapseCategories, meta = (DisplayName = "Grid2D Collection"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceGrid2DCollection : public UNiagaraDataInterfaceGrid2D
{
	GENERATED_UCLASS_BODY()

public:
	/** Reference to a user parameter if we're reading one. */
	UPROPERTY(EditAnywhere, Category = "Grid")
	FNiagaraUserParameterBinding RenderTargetUserParameter;

	/** When enabled overrides the format used to store data inside the grid, otherwise uses the project default setting.  Lower bit depth formats will save memory and performance at the cost of precision. */
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "bOverrideFormat"))
	ENiagaraGpuBufferFormat OverrideBufferFormat;

	UPROPERTY(EditAnywhere, Category = "Grid", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverrideFormat : 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bPreviewGrid : 1;

	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "bPreviewGrid", ToolTip = "When enabled allows you to preview the grid in a debug display") )
	FName PreviewAttribute = NAME_None;
#endif

	NIAGARA_API virtual void PostInitProperties() override;
	
	//~ UNiagaraDataInterface interface
	// VM functionality
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;

#if WITH_EDITOR	
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors,
		TArray<FNiagaraDataInterfaceFeedback>& Warnings, TArray<FNiagaraDataInterfaceFeedback>& Info) override;
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	NIAGARA_API virtual FNiagaraDataInterfaceParametersCS* CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const override;
	NIAGARA_API virtual const FTypeLayoutDesc* GetShaderStorageType() const override;

#if WITH_EDITOR
	virtual ENiagaraGpuDispatchType GetGpuDispatchType() const override { return ENiagaraGpuDispatchType::TwoD; }
	virtual FIntVector GetGpuDispatchNumThreads() const { return FIntVector(8, 8, 1); }
#endif

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override {}
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual int32 PerInstanceDataSize()const override { return sizeof(FGrid2DCollectionRWInstanceData_GameThread); }
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }

	virtual bool CanExposeVariables() const override { return true;}
	NIAGARA_API virtual void GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const override;
	NIAGARA_API virtual bool GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const override;

	virtual bool CanRenderVariablesToCanvas() const { return true; }
	NIAGARA_API virtual void GetCanvasVariables(TArray<FNiagaraVariableBase>& OutVariables) const override;
	NIAGARA_API virtual bool RenderVariableToCanvas(FNiagaraSystemInstanceID SystemInstanceID, FName VariableName, class FCanvas* Canvas, const FIntRect& DrawRect) const override;
	//~ UNiagaraDataInterface interface END

private:
	static NIAGARA_API void CollectAttributesForScript(UNiagaraScript* Script, const FNiagaraScriptRuntimeData& ScriptRuntimeData, const UNiagaraDataInterface* DataInterFace, TArray<FNiagaraVariableBase>& OutVariables, TArray<uint32>& OutVariableOffsets, int32& TotalAttributes, TArray<FText>* OutWarnings = nullptr);
public:
	/** Finds all attributes by locating the data interface amongst the parameter stores. */
	NIAGARA_API void FindAttributes(TArray<FNiagaraVariableBase>& OutVariables, TArray<uint32>& OutVariableOffsets, int32& OutNumAttribChannelsFound, TArray<FText>* OutWarnings = nullptr) const;

	// Fills a texture render target 2d with the current data from the simulation
	// #todo(dmp): this will eventually go away when we formalize how data makes it out of Niagara
	UFUNCTION(BlueprintCallable, Category = Niagara, meta=(DeprecatedFunction, DeprecationMessage = "This function has been replaced by object user variables on the emitter to specify render targets to fill with data."))
	NIAGARA_API virtual bool FillTexture2D(const UNiagaraComponent *Component, UTextureRenderTarget2D *dest, int AttributeIndex);
	
	UFUNCTION(BlueprintCallable, Category = Niagara, meta=(DeprecatedFunction, DeprecationMessage = "This function has been replaced by object user variables on the emitter to specify render targets to fill with data."))
	NIAGARA_API virtual bool FillRawTexture2D(const UNiagaraComponent *Component, UTextureRenderTarget2D *Dest, int &TilesX, int &TilesY);
	
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DeprecatedFunction, DeprecationMessage = "This function has been replaced by object user variables on the emitter to specify render targets to fill with data."))
	NIAGARA_API virtual void GetRawTextureSize(const UNiagaraComponent *Component, int &SizeX, int &SizeY);

	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API virtual void GetTextureSize(const UNiagaraComponent *Component, int &SizeX, int &SizeY);

	NIAGARA_API void VMGetWorldBBoxSize(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetCellSize(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetNumCells(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMSetNumCells(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetAttributeIndex(FVectorVMExternalFunctionContext& Context, const FName& InName, int32 NumChannels);

	static NIAGARA_API const FString GridName;
	static NIAGARA_API const FString OutputGridName;
	static NIAGARA_API const FString SamplerName;

	static NIAGARA_API const FName ClearCellFunctionName;
	static NIAGARA_API const FName CopyPreviousToCurrentForCellFunctionName;

	static NIAGARA_API const FName SetValueFunctionName;
	static NIAGARA_API const FName GetValueFunctionName;
	static NIAGARA_API const FName SampleGridFunctionName;
	static NIAGARA_API const FName CubicSampleGridFunctionName;

	static NIAGARA_API const FName SetValueAtIndexFunctionName;
	static NIAGARA_API const FName GetPreviousValueAtIndexFunctionName;
	static NIAGARA_API const FName SamplePreviousGridAtIndexFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridAtIndexFunctionName;

	static NIAGARA_API const FName SetVector4ValueFunctionName;
	static NIAGARA_API const FName GetVector4ValueFunctionName;
	static NIAGARA_API const FName SampleGridVector4FunctionName;

	static NIAGARA_API const FName SetVector3ValueFunctionName;
	static NIAGARA_API const FName GetVector3ValueFunctionName;
	static NIAGARA_API const FName SampleGridVector3FunctionName;

	static NIAGARA_API const FName SetVector2ValueFunctionName;
	static NIAGARA_API const FName GetVector2ValueFunctionName;
	static NIAGARA_API const FName SampleGridVector2FunctionName;

	static NIAGARA_API const FName SetFloatValueFunctionName;
	static NIAGARA_API const FName GetFloatValueFunctionName;
	static NIAGARA_API const FName SampleGridFloatFunctionName;

	static NIAGARA_API const FName GetPreviousVector4ValueFunctionName;
	static NIAGARA_API const FName SamplePreviousGridVector4FunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridVector4FunctionName;

	static NIAGARA_API const FName SetVectorValueFunctionName;
	static NIAGARA_API const FName GetPreviousVectorValueFunctionName;
	static NIAGARA_API const FName SamplePreviousGridVectorFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridVectorFunctionName;

	static NIAGARA_API const FName SetVector2DValueFunctionName;
	static NIAGARA_API const FName GetPreviousVector2DValueFunctionName;
	static NIAGARA_API const FName SamplePreviousGridVector2DFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridVector2DFunctionName;

	static NIAGARA_API const FName GetPreviousFloatValueFunctionName;
	static NIAGARA_API const FName SamplePreviousGridFloatFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridFloatFunctionName;
	
	static NIAGARA_API const FString AttributeIndicesBaseName;
	static NIAGARA_API const TCHAR* VectorComponentNames[];

	static NIAGARA_API const FName SetNumCellsFunctionName;

	static NIAGARA_API const FName GetVector4AttributeIndexFunctionName;
	static NIAGARA_API const FName GetVectorAttributeIndexFunctionName;
	static NIAGARA_API const FName GetVector2DAttributeIndexFunctionName;
	static NIAGARA_API const FName GetFloatAttributeIndexFunctionName;

	static NIAGARA_API const FString AnonymousAttributeString;

	static NIAGARA_API const TCHAR* TemplateShaderFilePath;

#if WITH_EDITOR
	virtual bool SupportsSetupAndTeardownHLSL() const { return true; }
	NIAGARA_API virtual bool GenerateSetupHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, TConstArrayView<FNiagaraVariable> InArguments, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const;
	NIAGARA_API virtual bool GenerateTeardownHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, TConstArrayView<FNiagaraVariable> InArguments, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const;
	virtual bool SupportsIterationSourceNamespaceAttributesHLSL() const override { return true; }
	NIAGARA_API virtual bool GenerateIterationSourceNamespaceReadAttributesHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, const FNiagaraVariable& IterationSourceVar, TConstArrayView<FNiagaraVariable> InArguments, TConstArrayView<FNiagaraVariable> InAttributes, TConstArrayView<FString> InAttributeHLSLNames, bool bInSetToDefaults, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const override;
	NIAGARA_API virtual bool GenerateIterationSourceNamespaceWriteAttributesHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, const FNiagaraVariable& IterationSourceVar, TConstArrayView<FNiagaraVariable> InArguments, TConstArrayView<FNiagaraVariable> InAttributes, TConstArrayView<FString> InAttributeHLSLNames, TConstArrayView<FNiagaraVariable> InAllAttributes, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const override;
#endif

	static NIAGARA_API int32 GetComponentCountFromFuncName(const FName& FuncName);
	static NIAGARA_API FNiagaraTypeDefinition GetValueTypeFromFuncName(const FName& FuncName);
	static NIAGARA_API bool CanCreateVarFromFuncName(const FName& FuncName);

	TMap<FNiagaraSystemInstanceID, FGrid2DCollectionRWInstanceData_GameThread*>& GetSystemInstancesToProxyData_GT() { return SystemInstancesToProxyData_GT; }
protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API void WriteSetHLSL(const FNiagaraDataInterfaceRWAttributeHelper &AttributeHelper, const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL);
	NIAGARA_API void WriteGetHLSL(const FNiagaraDataInterfaceRWAttributeHelper& AttributeHelper, const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL);
	NIAGARA_API void WriteSampleHLSL(const FNiagaraDataInterfaceRWAttributeHelper& AttributeHelper, const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, bool IsCubic, FString& OutHLSL);
	NIAGARA_API void WriteAttributeGetIndexHLSL(const FNiagaraDataInterfaceRWAttributeHelper& AttributeHelper, const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL);

	NIAGARA_API FString GenerateAttributeIndexHLSL(const FNiagaraDataInterfaceRWAttributeHelper& AttributeHelper, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo);

	NIAGARA_API const TCHAR* TypeDefinitionToHLSLTypeString(const FNiagaraTypeDefinition& InDef) const;
	NIAGARA_API FName TypeDefinitionToGetFunctionName(const FNiagaraTypeDefinition& InDef) const;
	NIAGARA_API FName TypeDefinitionToSetFunctionName(const FNiagaraTypeDefinition& InDef) const;
#endif

	// Only used with CDO for hlsl generation
	bool bUseIndirection;

	//~ UNiagaraDataInterface interface
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	//~ UNiagaraDataInterface interface END

	static NIAGARA_API FNiagaraVariableBase ExposedRTVar;

	TMap<FNiagaraSystemInstanceID, FGrid2DCollectionRWInstanceData_GameThread*> SystemInstancesToProxyData_GT;

	UPROPERTY(Transient)
	TMap< uint64, TObjectPtr<UTextureRenderTarget2DArray>> ManagedRenderTargets;
	
};
