// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceRW.h"
#include "ClearQuad.h"
#include "NiagaraComponent.h"
#include "NiagaraRenderGraphUtils.h"
#include "RHIUtilities.h"

#include "NiagaraDataInterfaceGrid3DCollection.generated.h"

class FNiagaraSystemInstance;
class UTextureRenderTarget;
class UTextureRenderTargetVolume;

using FGrid3DBuffer = FNiagaraPooledRWTexture;

struct FGrid3DCollectionRWInstanceData_GameThread
{
	bool ClearBeforeNonIterationStage = true;

	FIntVector NumCells = FIntVector::ZeroValue;
	FIntVector NumTiles = FIntVector::ZeroValue;
	int32 TotalNumAttributes = 0;
	int32 TotalNumNamedAttributes = 0;
	FVector CellSize = FVector::ZeroVector;
	FVector WorldBBoxSize = FVector::ZeroVector;
	TOptional<EPixelFormat> PixelFormat;
	bool UseRGBATexture = false;
#if WITH_EDITOR
	bool bPreviewGrid = false;
	FIntVector4 PreviewAttribute = FIntVector4(INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE);
#endif

	bool NeedsRealloc = false;

	/** A binding to the user ptr we're reading the RT from (if we are). */
	FNiagaraParameterDirectBinding<UObject*> RTUserParamBinding;

	UTextureRenderTargetVolume* TargetTexture = nullptr;
	TArray<FNiagaraVariableBase> Vars;
	TArray<uint32> Offsets;

	// We need to essentially make this a linked list to avoid more refactoring for now
	// eventually we can clean this logic up, but this allows us to have a subclass that
	// overrides the render thread data, which in this case is for a grid reader
	UNiagaraDataInterface* OtherDI = nullptr;
	FGrid3DCollectionRWInstanceData_GameThread* OtherInstanceData = nullptr;

	int32 FindAttributeIndexByName(const FName& InName, int32 NumChannels);
	bool UpdateTargetTexture(ENiagaraGpuBufferFormat BufferFormat, UObject* Owner);
};

struct FGrid3DCollectionRWInstanceData_RenderThread
{
	FName SourceDIName;

	bool ClearBeforeNonIterationStage = true;

	FIntVector NumCells = FIntVector(1, 1, 1);
	FIntVector NumTiles = FIntVector(1, 1, 1);
	int32 TotalNumAttributes = 0;
	int32 TotalNumNamedAttributes = 0;
	FVector CellSize = FVector::ZeroVector;
	FVector WorldBBoxSize = FVector::ZeroVector;
	TOptional<EPixelFormat> PixelFormat;
	bool UseRGBATexture = false;

	TArray<int32> AttributeIndices;

	TArray<FName> Vars;
	TArray<int32> VarComponents;
	TArray<uint32> Offsets;
#if WITH_EDITOR
	bool bPreviewGrid = false;
	FIntVector4 PreviewAttribute = FIntVector4(INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE);
#endif

	TArray<TUniquePtr<FGrid3DBuffer>, TInlineAllocator<2>> Buffers;
	FGrid3DBuffer* CurrentData = nullptr;
	FGrid3DBuffer* DestinationData = nullptr;

	FReadBuffer PerAttributeData;

	FTextureRHIRef RenderTargetToCopyTo;

	TArray<FName> AttributeNames;

	// We need to essentially make this a linked list to avoid more refactoring for now
	// eventually we can clean this logic up, but this allows us to have a subclass that
	// overrides the render thread data, which in this case is for a grid reader
	FNiagaraDataInterfaceProxy* OtherProxy = nullptr;

	void BeginSimulate(FRDGBuilder& GraphBuilder, bool RequiresBuffering);
	void EndSimulate();
};

struct FNiagaraDataInterfaceProxyGrid3DCollectionProxy : public FNiagaraDataInterfaceProxyRW
{
	FNiagaraDataInterfaceProxyGrid3DCollectionProxy() {}

	virtual void ResetData(const FNDIGpuComputeResetContext& Context) override;
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override;
	virtual void PostStage(const FNDIGpuComputePostStageContext& Context) override;
	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override;

	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override;

	/* List of proxy data for each system instances*/
	// #todo(dmp): this should all be refactored to avoid duplicate code
	TMap<FNiagaraSystemInstanceID, FGrid3DCollectionRWInstanceData_RenderThread> SystemInstancesToProxyData_RT;
};

struct FNiagaraDataInterfaceParametersCS_Grid3DCollection : public FNiagaraDataInterfaceParametersCS
{
	DECLARE_TYPE_LAYOUT(FNiagaraDataInterfaceParametersCS_Grid3DCollection, NonVirtual);

	LAYOUT_FIELD(TMemoryImageArray<FMemoryImageName>, AttributeNames);
	LAYOUT_FIELD(TMemoryImageArray<uint32>, AttributeChannelCount);
};

UCLASS(EditInlineNew, Category = "Grid", CollapseCategories, meta = (DisplayName = "Grid3D Collection"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceGrid3DCollection : public UNiagaraDataInterfaceGrid3D
{
	GENERATED_UCLASS_BODY()

public:
	// Number of attributes stored on the grid
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Grid")
	int32 NumAttributes;

	/** Reference to a user parameter if we're reading one. */
	UPROPERTY(EditAnywhere, Category = "Grid")
	FNiagaraUserParameterBinding RenderTargetUserParameter;

	/** When enabled overrides the format used to store data inside the grid, otherwise uses the project default setting.  Lower bit depth formats will save memory and performance at the cost of precision. */
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "bOverrideFormat"))
	ENiagaraGpuBufferFormat OverrideBufferFormat;

	UPROPERTY(EditAnywhere, Category = "Grid", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverrideFormat : 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bPreviewGrid : 1;

	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "bPreviewGrid", ToolTip = "When enabled allows you to preview the grid in a debug display"))
	FName PreviewAttribute = NAME_None;
#endif

	NIAGARA_API virtual void PostInitProperties() override;

	//~ UNiagaraDataInterface interface
	// VM functionality
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;

#if WITH_EDITOR	
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors,
		TArray<FNiagaraDataInterfaceFeedback>& Warnings, TArray<FNiagaraDataInterfaceFeedback>& Info) override;
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	// GPU sim functionality
#if WITH_EDITOR
	NIAGARA_API virtual bool ShouldCompile(EShaderPlatform ShaderPlatform) const override;
#endif
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	NIAGARA_API virtual FNiagaraDataInterfaceParametersCS* CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const override;
	NIAGARA_API virtual const FTypeLayoutDesc* GetShaderStorageType() const override;

#if WITH_EDITOR
	virtual ENiagaraGpuDispatchType GetGpuDispatchType() const override { return ENiagaraGpuDispatchType::ThreeD; }
#endif

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override {}
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual int32 PerInstanceDataSize()const override { return sizeof(FGrid3DCollectionRWInstanceData_GameThread); }
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }

	virtual bool CanExposeVariables() const override { return true; }
	NIAGARA_API virtual void GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const override;
	NIAGARA_API virtual bool GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const override;
	//~ UNiagaraDataInterface interface END

public:
	// Fills a texture render target 2d with the current data from the simulation
	// #todo(dmp): this will eventually go away when we formalize how data makes it out of Niagara
	// #todo(dmp): reimplement for 3d

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DeprecatedFunction, DeprecationMessage = "This function has been replaced by object user variables on the emitter to specify render targets to fill with data."))
	NIAGARA_API virtual bool FillVolumeTexture(const UNiagaraComponent* Component, UVolumeTexture* dest, int AttributeIndex);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DeprecatedFunction, DeprecationMessage = "This function has been replaced by object user variables on the emitter to specify render targets to fill with data."))
	NIAGARA_API virtual bool FillRawVolumeTexture(const UNiagaraComponent* Component, UVolumeTexture* Dest, int& TilesX, int& TilesY, int& TileZ);

	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API virtual void GetRawTextureSize(const UNiagaraComponent* Component, int& SizeX, int& SizeY, int& SizeZ);

	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API virtual void GetTextureSize(const UNiagaraComponent* Component, int& SizeX, int& SizeY, int& SizeZ);

	NIAGARA_API void VMGetWorldBBoxSize(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetCellSize(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void VMGetNumCells(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMSetNumCells(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMUnitToFloatIndex(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void VMGetAttributeIndex(FVectorVMExternalFunctionContext& Context, const FName& InName, int32 NumChannels);

	static NIAGARA_API const FString NumTilesName;
	static NIAGARA_API const FString OneOverNumTilesName;
	static NIAGARA_API const FString UnitClampMinName;
	static NIAGARA_API const FString UnitClampMaxName;

	static NIAGARA_API const FString GridName;
	static NIAGARA_API const FString OutputGridName;
	static NIAGARA_API const FString SamplerName;

	static NIAGARA_API const FName ClearCellFunctionName;
	static NIAGARA_API const FName CopyPreviousToCurrentForCellFunctionName;
	static NIAGARA_API const FName CopyMaskedPreviousToCurrentForCellFunctionName;

	static NIAGARA_API const FName SetValueFunctionName;
	static NIAGARA_API const FName GetValueFunctionName;
	static NIAGARA_API const FName SampleGridFunctionName;
	static NIAGARA_API const FName CubicSampleGridFunctionName;

	static NIAGARA_API const FName SetFullGridValueFunctionName;
	static NIAGARA_API const FName GetFullGridPreviousValueFunctionName;
	static NIAGARA_API const FName SamplePreviousFullGridFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousFullGridFunctionName;

	static NIAGARA_API const FName SetVector4ValueFunctionName;
	static NIAGARA_API const FName SetVector3ValueFunctionName;
	static NIAGARA_API const FName SetVector2ValueFunctionName;
	static NIAGARA_API const FName GetVector2ValueFunctionName;
	static NIAGARA_API const FName SetFloatValueFunctionName;
	static NIAGARA_API const FName GetPreviousValueAtIndexFunctionName;
	static NIAGARA_API const FName SamplePreviousGridAtIndexFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridAtIndexFunctionName;

	static NIAGARA_API const FName SetVector4ValueAtIndexFunctionName;
	static NIAGARA_API const FName SetVector3ValueAtIndexFunctionName;
	static NIAGARA_API const FName SetVector2ValueAtIndexFunctionName;
	static NIAGARA_API const FName SetFloatValueAtIndexFunctionName;

	static NIAGARA_API const FName GetPreviousVector4ValueAtIndexFunctionName;
	static NIAGARA_API const FName GetPreviousVector3ValueAtIndexFunctionName;
	static NIAGARA_API const FName GetPreviousVector2ValueAtIndexFunctionName;
	static NIAGARA_API const FName GetPreviousFloatValueAtIndexFunctionName;

	static NIAGARA_API const FName SamplePreviousGridVector4AtIndexFunctionName;
	static NIAGARA_API const FName SamplePreviousGridVector3AtIndexFunctionName;
	static NIAGARA_API const FName SamplePreviousGridVector2AtIndexFunctionName;
	static NIAGARA_API const FName SamplePreviousGridFloatAtIndexFunctionName;

	static NIAGARA_API const FName CubicSamplePreviousGridVector4AtIndexFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridVector3AtIndexFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridVector2AtIndexFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridFloatAtIndexFunctionName;

	static NIAGARA_API const FName GetPreviousVector4ValueFunctionName;
	static NIAGARA_API const FName SamplePreviousGridVector4FunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridVector4FunctionName;
	static NIAGARA_API const FName SetVectorValueFunctionName;
	static NIAGARA_API const FName GetPreviousVectorValueFunctionName;
	static NIAGARA_API const FName SamplePreviousGridVectorFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridVectorFunctionName;
	static NIAGARA_API const FName SetVector2DValueFunctionName;
	static NIAGARA_API const FName GetPreviousVector2DValueFunctionName;
	static NIAGARA_API const FName SamplePreviousGridVector2DFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridVector2DFunctionName;
	static NIAGARA_API const FName GetPreviousFloatValueFunctionName;
	static NIAGARA_API const FName SamplePreviousGridFloatFunctionName;
	static NIAGARA_API const FName CubicSamplePreviousGridFloatFunctionName;

	static NIAGARA_API const FName GetOutputGridFloatValueFunctionName;
	static NIAGARA_API const FName GetOutputGridVector2ValueFunctionName;
	static NIAGARA_API const FName GetOutputGridVectorValueFunctionName;
	static NIAGARA_API const FName GetOutputGridVector4ValueFunctionName;
	static NIAGARA_API const FString AttributeIndicesBaseName;
	static NIAGARA_API const FString PerAttributeDataName;
	static NIAGARA_API const TCHAR* VectorComponentNames[];

	static NIAGARA_API const FName SetNumCellsFunctionName;

	static NIAGARA_API const FName GetVector4AttributeIndexFunctionName;
	static NIAGARA_API const FName GetVectorAttributeIndexFunctionName;
	static NIAGARA_API const FName GetVector2DAttributeIndexFunctionName;
	static NIAGARA_API const FName GetFloatAttributeIndexFunctionName;

	static NIAGARA_API const FString AnonymousAttributeString;

	static NIAGARA_API const TCHAR* TemplateShaderFilePath;

#if WITH_EDITOR
	virtual bool SupportsSetupAndTeardownHLSL() const { return true; }
	NIAGARA_API virtual bool GenerateSetupHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, TConstArrayView<FNiagaraVariable> InArguments, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const;
	NIAGARA_API virtual bool GenerateTeardownHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, TConstArrayView<FNiagaraVariable> InArguments, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const;
	virtual bool SupportsIterationSourceNamespaceAttributesHLSL() const override { return true; }
	NIAGARA_API virtual bool GenerateIterationSourceNamespaceReadAttributesHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, const FNiagaraVariable& IterationSourceVar, TConstArrayView<FNiagaraVariable> InArguments, TConstArrayView<FNiagaraVariable> InAttributes, TConstArrayView<FString> InAttributeHLSLNames, bool bInSetToDefaults, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const override;
	NIAGARA_API virtual bool GenerateIterationSourceNamespaceWriteAttributesHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, const FNiagaraVariable& IterationSourceVar, TConstArrayView<FNiagaraVariable> InArguments, TConstArrayView<FNiagaraVariable> InAttributes, TConstArrayView<FString> InAttributeHLSLNames, TConstArrayView<FNiagaraVariable> InAllAttributes, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const override;
#endif

	static NIAGARA_API int32 GetComponentCountFromFuncName(const FName& FuncName);
	static NIAGARA_API FNiagaraTypeDefinition GetValueTypeFromFuncName(const FName& FuncName);
	static NIAGARA_API bool CanCreateVarFromFuncName(const FName& FuncName);

	TMap<FNiagaraSystemInstanceID, FGrid3DCollectionRWInstanceData_GameThread*>& GetSystemInstancesToProxyData_GT() { return SystemInstancesToProxyData_GT; }
protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;

	NIAGARA_API void WriteSetHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL);
	NIAGARA_API void WriteSetAtIndexHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int32 InNumChannels, FString& OutHLSL);
	NIAGARA_API void WriteGetHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL);
	NIAGARA_API void WriteGetOutputGridHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL);
	NIAGARA_API void WriteGetAtIndexHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int32 InNumChannels, FString& OutHLSL);
	NIAGARA_API void WriteSampleHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString SampleFunction, FString& OutHLSL);
	NIAGARA_API void WriteSampleAtIndexHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int32 InNumChannels, FString SampleFunction, FString& OutHLSL);
	NIAGARA_API void WriteAttributeGetIndexHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL);

	NIAGARA_API const TCHAR* TypeDefinitionToHLSLTypeString(const FNiagaraTypeDefinition& InDef) const;
	NIAGARA_API FName TypeDefinitionToGetFunctionName(const FNiagaraTypeDefinition& InDef) const;
	NIAGARA_API FName TypeDefinitionToSetFunctionName(const FNiagaraTypeDefinition& InDef) const;
	NIAGARA_API FName TypeDefinitionToAttributeIndexFunctionName(const FNiagaraTypeDefinition& InDef) const;
#endif

	//~ UNiagaraDataInterface interface
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	//~ UNiagaraDataInterface interface END

	TMap<FNiagaraSystemInstanceID, FGrid3DCollectionRWInstanceData_GameThread*> SystemInstancesToProxyData_GT;

	static NIAGARA_API FNiagaraVariableBase ExposedRTVar;

public:
	/** Finds all attributes by locating the data interface amongst the parameter stores. */
	NIAGARA_API void FindAttributes(TArray<FNiagaraVariableBase>& OutVariables, TArray<uint32>& OutVariableOffsets, int32& OutNumAttribChannelsFound, TArray<FText>* OutWarnings = nullptr) const;
protected:
	static NIAGARA_API void CollectAttributesForScript(UNiagaraScript* Script, const FNiagaraScriptRuntimeData& ScriptRuntimeData, const UNiagaraDataInterface* DataInterface, TArray<FNiagaraVariableBase>& OutVariables, TArray<uint32>& OutVariableOffsets, int32& TotalAttributes, TArray<FText>* OutWarnings = nullptr);
};
