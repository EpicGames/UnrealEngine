// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceRW.h"
#include "ClearQuad.h"
#include "NiagaraComponent.h"
#include "NiagaraDataInterfaceGrid3DCollection.h"

#include "NiagaraDataInterfaceGrid3DCollectionReader.generated.h"

class FNiagaraSystemInstance;
class UTextureRenderTarget;
class UTextureRenderTargetVolume;

UCLASS(EditInlineNew, Category = "Grid", CollapseCategories, hidecategories=(Grid3DCollection,Grid, Deprecated), meta = (DisplayName = "Grid3D Collection Reader"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceGrid3DCollectionReader : public UNiagaraDataInterfaceGrid3DCollection
{
	GENERATED_UCLASS_BODY()

public:
	// Name of the emitter to read from
	UPROPERTY(EditAnywhere, Category = "Reader")
	FString EmitterName;

	// Name of the Grid2DCollection Data Interface on the emitter
	UPROPERTY(EditAnywhere, Category = "Reader")
	FString DIName;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void GetEmitterDependencies(UNiagaraSystem* Asset, TArray<FVersionedNiagaraEmitter>& Dependencies) const override;
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;

protected:
	//~ UNiagaraDataInterface interface
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	//~ UNiagaraDataInterface interface END
};
