// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceRW.h"
#include "ClearQuad.h"
#include "NiagaraComponent.h"
#include "NiagaraDataInterfaceIntRenderTarget2D.generated.h"

class UTextureRenderTarget2D;

UCLASS(EditInlineNew, Category = "Render Target", CollapseCategories, meta = (DisplayName = "Integer Render Target 2D"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceIntRenderTarget2D : public UNiagaraDataInterfaceRWBase
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FVector4f,								TextureSizeAndInvSize)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>,	RWTextureUAV)
	END_SHADER_PARAMETER_STRUCT()

public:
	NIAGARA_API virtual void PostInitProperties() override;
	virtual bool CanBeInCluster() const override { return false; }

	//~ UNiagaraDataInterface interface
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target)const override { return true; }
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override {}
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance, float DeltaSeconds) override;
	virtual bool HasPostSimulateTick() const override { return true; }

	virtual bool CanExposeVariables() const override { return true;}
	NIAGARA_API virtual void GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const override;
	NIAGARA_API virtual bool GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const override;
	//~ UNiagaraDataInterface interface END

	NIAGARA_API bool UpdateInstanceTexture(FNiagaraSystemInstance* SystemInstance, struct FNDIIntRenderTarget2DInstanceData_GameThread* InstanceData);

	NIAGARA_API void VMGetSize(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMSetSize(FVectorVMExternalFunctionContext& Context);

	UPROPERTY(EditAnywhere, Category = "Render Target", meta = (DisplayPriority = 1))
	FIntPoint Size = FIntPoint::ZeroValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Render Target", meta = (DisplayPriority = 2))
	uint8 bPreviewRenderTarget : 1;

	/* The range to normaliez the preview display to. */
	UPROPERTY(EditAnywhere, Category = "Render Target", meta = (EditCondition = "bPreviewRenderTarget", EditConditionHides, DisplayPriority = 3))
	FVector2D PreviewDisplayRange = FVector2D(0.0f, 255.0f);
#endif

	UPROPERTY(EditAnywhere, Category = "Render Target", meta = (DisplayPriority = 0, ToolTip = "When valid the user parameter is used as the render target rather than creating one internal, note that the input render target will be adjusted by the Niagara simulation"))
	FNiagaraUserParameterBinding RenderTargetUserParameter;

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	static NIAGARA_API FNiagaraVariableBase ExposedRTVar;
};
