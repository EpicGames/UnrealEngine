// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "NiagaraSimCacheCustomStorageInterface.h"
#include "VectorVM.h"

#include "NiagaraDataInterfaceLandscape.generated.h"

class ALandscape;
struct FNDILandscapeData_GameThread;
class UPhysicalMaterial;

UENUM()
enum class ENDILandscape_SourceMode : uint8
{
	/**
	Default behavior.
	- Use "Source" when explicitly specified.
	- When no source is specified, fall back on attached actor or component or world.
	*/
	Default,

	/**
	Only use "Source".
	*/
	Source,

	/**
	Only use the parent actor or component the system is attached to.
	*/
	AttachParent
};

/** Data Interface allowing sampling of a Landscape */
UCLASS(EditInlineNew, Category = "Landscape", CollapseCategories, meta = (DisplayName = "Landscape Sample"), MinimalAPI)
class UNiagaraDataInterfaceLandscape : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Landscape")
	TObjectPtr<AActor> SourceLandscape;

	UPROPERTY(EditAnywhere, Category = "Landscape")
	ENDILandscape_SourceMode SourceMode = ENDILandscape_SourceMode::Default;

	UPROPERTY(EditAnywhere, Category = "Landscape")
	TArray<TObjectPtr<UPhysicalMaterial>> PhysicalMaterials;

	/** Can be used to ignore virtual textures even if they are defined for the landscape. */
	UPROPERTY(EditAnywhere, Category = "Landscape")
	bool bVirtualTexturesSupported = true;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;	
	//UObject Interface End

	//UNiagaraDataInterface Interface
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target == ENiagaraSimTarget::GPUComputeSim; }
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	//UNiagaraDataInterface Interface End

#if WITH_NIAGARA_DEBUGGER
	virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const override;
#endif

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual bool HasPreSimulateTick() const override { return true; }

	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* PerInstanceData, FVMExternalFunction& OutFunc) override;

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	NIAGARA_API void ApplyLandscape(const FNiagaraSystemInstance& SystemInstance, FNDILandscapeData_GameThread& InstanceData) const;
	NIAGARA_API ALandscape* GetLandscape(const FNiagaraSystemInstance& SystemInstance, FNDILandscapeData_GameThread& Hint) const;

	void VMGenerateRvtFeedback(FVectorVMExternalFunctionContext& Context);
};


UCLASS(MinimalAPI)
class UNDILandscapeSimCacheData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<TObjectPtr<UTexture2D>> HeightFieldTextures;
};
