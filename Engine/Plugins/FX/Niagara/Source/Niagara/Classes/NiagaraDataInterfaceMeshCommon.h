// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "NiagaraDataInterfaceMeshCommon.generated.h"

//A coordinate on a mesh usable in Niagara.
//Do not alter this struct without updating the data interfaces that use it!
USTRUCT()
struct FMeshTriCoordinate
{
	GENERATED_USTRUCT_BODY();
	
	UPROPERTY(EditAnywhere, Category="Coordinate")
	int32 Tri;

	UPROPERTY(EditAnywhere, Category="Coordinate")
	FVector3f BaryCoord;

	FMeshTriCoordinate()
	: Tri(INDEX_NONE)
	, BaryCoord(FVector3f::ZeroVector)
	{}

	FMeshTriCoordinate(int32 InTri, FVector3f InBaryCoord)
		: Tri(InTri)
		, BaryCoord(InBaryCoord)
	{}
};

inline FVector3f RandomBarycentricCoord(FRandomStream& RandStream)
{
	//TODO: This is gonna be slooooow. Move to an LUT possibly or find faster method.
	//Can probably handle lower quality randoms / uniformity for a decent speed win.
	float r0 = RandStream.GetFraction();
	float r1 = RandStream.GetFraction();
	float sqrt0 = FMath::Sqrt(r0);
	return FVector3f(1.0f - sqrt0, sqrt0 * (1.0f - r1), r1 * sqrt0);
}

template<typename T>
inline T BarycentricInterpolate(float BaryX, float BaryY, float BaryZ, T V0, T V1, T V2)
{
	return V0 * BaryX + V1 * BaryY + V2 * BaryZ;
}

// Overload for FVector4 to work around C2719: (formal parameter with requested alignment of 16 won't be aligned)
inline FVector4 BarycentricInterpolate(float BaryX, float BaryY, float BaryZ, const FVector4& V0, const FVector4& V1, const FVector4& V2)
{
	return V0 * BaryX + V1 * BaryY + V2 * BaryZ;
}

template<typename T>
inline T BarycentricInterpolate(FVector3f BaryCoord, T V0, T V1, T V2)
{
	return V0 * BaryCoord.X + V1 * BaryCoord.Y + V2 * BaryCoord.Z;
}

// Overload for FVector4 to work around C2719: (formal parameter with requested alignment of 16 won't be aligned)
inline FVector4 BarycentricInterpolate(FVector3f BaryCoord, const FVector4& V0, const FVector4& V1, const FVector4& V2)
{
	return V0 * BaryCoord.X + V1 * BaryCoord.Y + V2 * BaryCoord.Z;
}
