// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"
#include "NiagaraParameterStore.h"
#include "NiagaraDataInterfaceMeshRendererInfo.generated.h"

class UNiagaraMeshRendererProperties;
class FNDIMeshRendererInfo;

/** This Data Interface can be used to query information about the mesh renderers of an emitter */
UCLASS(EditInlineNew, Category = "Mesh Particles", CollapseCategories, meta = (DisplayName = "Mesh Renderer Info"), MinimalAPI)
class UNiagaraDataInterfaceMeshRendererInfo : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	struct FMeshData
	{
		FVector3f MinLocalBounds = FVector3f(ForceInitToZero);
		FVector3f MaxLocalBounds = FVector3f(ForceInitToZero);
	};

public:
	UNiagaraMeshRendererProperties* GetMeshRenderer() const { return MeshRenderer; }

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API virtual void BeginDestroy() override;
#if WITH_EDITOR	
	NIAGARA_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif 
	//UObject Interface End

	NIAGARA_API void OnMeshRendererChanged(UNiagaraMeshRendererProperties* NewMeshRenderer);

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
#if WITH_EDITOR
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo) override;
#endif
	//UNiagaraDataInterface Interface	

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	NIAGARA_API virtual void PushToRenderThreadImpl() override;
	NIAGARA_API void UpdateCachedData();

	NIAGARA_API void VMGetNumMeshes(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetMeshLocalBounds(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetSubUVDetails(FVectorVMExternalFunctionContext& Context);

	/** The name of the mesh renderer */
	UPROPERTY(EditAnywhere, Category = "Source")
	TObjectPtr<UNiagaraMeshRendererProperties> MeshRenderer;

	TArray<FMeshData> CachedMeshData;

#if WITH_EDITOR
	FDelegateHandle OnMeshRendererChangedHandle;
#endif
};
