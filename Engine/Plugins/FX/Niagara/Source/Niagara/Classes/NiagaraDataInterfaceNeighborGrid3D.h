// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClearQuad.h"
#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraRenderGraphUtils.h"

#include "NiagaraDataInterfaceNeighborGrid3D.generated.h"

class FNiagaraSystemInstance;

// store all data in in a class
// move all data management to use per instance data
// remove references to push data to render thread

struct FNDINeighborGrid3DInstanceData_GT
{
	FIntVector	NumCells = FIntVector::ZeroValue;
	float		CellSize = 0.0f;
	uint32		MaxNeighborsPerCell = 0;
	FVector		WorldBBoxSize = FVector::ZeroVector;
	bool		bNeedsRealloc = false;
};

struct FNDINeighborGrid3DInstanceData_RT
{
	void ResizeBuffers(FRDGBuilder& GraphBuilder);

	bool ClearBeforeNonIterationStage = true;

	FIntVector	NumCells = FIntVector::ZeroValue;
	float		CellSize = 0.0f;
	uint32		MaxNeighborsPerCell = 0;
	FVector		WorldBBoxSize = FVector::ZeroVector;
	bool		bNeedsRealloc = false;

	FNiagaraPooledRWBuffer NeighborhoodBuffer;
	FNiagaraPooledRWBuffer NeighborhoodCountBuffer;
};

struct FNiagaraDataInterfaceProxyNeighborGrid3D : public FNiagaraDataInterfaceProxyRW
{	
	virtual void ResetData(const FNDIGpuComputeResetContext& Context) override;
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override;
	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override;

	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override {}
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }	

	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override;

	/* List of proxy data for each system instances*/
	// #todo(dmp): this should all be refactored to avoid duplicate code
	TMap<FNiagaraSystemInstanceID, FNDINeighborGrid3DInstanceData_RT> SystemInstancesToProxyData_RT;
};

UCLASS(EditInlineNew, Category = "Grid", CollapseCategories, meta = (DisplayName = "Neighbor Grid3D"), MinimalAPI)
class UNiagaraDataInterfaceNeighborGrid3D : public UNiagaraDataInterfaceGrid3D
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FIntVector,		NumCells)
		SHADER_PARAMETER(FVector3f,			UnitToUV)
		SHADER_PARAMETER(FVector3f,			CellSize)
		SHADER_PARAMETER(FVector3f,			WorldBBoxSize)

		SHADER_PARAMETER(int32,							MaxNeighborsPerCellValue)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	ParticleNeighbors)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	ParticleNeighborCount)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	OutputParticleNeighbors)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	OutputParticleNeighborCount)
	END_SHADER_PARAMETER_STRUCT()

public:	
	UPROPERTY(EditAnywhere, Category = "Grid")
	uint32 MaxNeighborsPerCell;

	virtual void PostInitProperties() override
	{
		Super::PostInitProperties();

		//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
		if (HasAnyFlags(RF_ClassDefaultObject))
		{
			ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
			FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
		}
	}

	//~ UNiagaraDataInterface interface
	// VM functionality
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override {}
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override { return false;  }
	virtual int32 PerInstanceDataSize()const override { return sizeof(FNDINeighborGrid3DInstanceData_GT); }
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual bool HasPostSimulateTick() const override { return true; }
	virtual bool HasPreSimulateTick() const override { return true; }
	//~ UNiagaraDataInterface interface END

	NIAGARA_API void GetWorldBBoxSize(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetNumCells(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetMaxNeighborsPerCell(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void SetNumCells(FVectorVMExternalFunctionContext& Context);

	static NIAGARA_API const FName SetNumCellsFunctionName;


protected:
	//~ UNiagaraDataInterface interface
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	//~ UNiagaraDataInterface interface END	
};


