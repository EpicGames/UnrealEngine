// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraRenderGraphUtils.h"

#include "NiagaraDataInterfaceNeighborQuery.generated.h"

class FNiagaraSystemInstance;

struct FNDINeighborQueryInstanceData_GT
{
	FIntVector	NumCells = FIntVector::ZeroValue;
	uint32		MaxCellsPerParticle = 1;
	bool		bCountOnly = false;
	bool		bUsePersistentIDs = true;
	bool		bNeedsRealloc = false;
};

struct FNDINeighborQueryInstanceData_RT
{
	void ResizeBuffers(FRDGBuilder& GraphBuilder);

	bool ClearBeforeNonIterationStage = true;

	FIntVector	NumCells = FIntVector::ZeroValue;
	uint32		MaxCellsPerParticle = 1;
	uint32		NumParticles = 0;
	uint32		AllocatedNumSlots = 0;
	uint32		AllocatedNumCells = 0;
	bool		bCountOnly = false;
	bool		bUsePersistentIDs = true;
	bool		bNeedsRealloc = false;

	// Counting sort buffers
	FNiagaraPooledRWBuffer CellIdBuffer;			// numSlots (numParticles * MaxCellsPerParticle) - cell assignment per slot
	FNiagaraPooledRWBuffer ParticleIdIndexBuffer;	// numSlots - ID.Index per slot, written by AddParticle, read by Scatter
	FNiagaraPooledRWBuffer CellCountBuffer;			// numCells - particles per cell (histogram output)
	FNiagaraPooledRWBuffer CellOffsetBuffer;		// numCells - prefix sum write offset per cell
	FNiagaraPooledRWBuffer ParticleListBuffer;		// numSlots - final sorted particle indices (ID.Index values)
	FNiagaraPooledRWBuffer AcquireTagBuffer;		// numSlots - AcquireTag per slot, written by AddParticle
	FNiagaraPooledRWBuffer AcquireTagListBuffer;	// numSlots - sorted AcquireTag, written by Scatter, read by GetParticleNeighbor
};

struct FNiagaraDataInterfaceProxyNeighborQuery : public FNiagaraDataInterfaceProxyRW
{
	virtual void ResetData(const FNDIGpuComputeResetContext& Context) override;
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override;
	virtual void PostStage(const FNDIGpuComputePostStageContext& Context) override;
	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override;

	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override {}
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override;

	TMap<FNiagaraSystemInstanceID, FNDINeighborQueryInstanceData_RT> SystemInstancesToProxyData_RT;
};

UCLASS(EditInlineNew, Category = "Grid", CollapseCategories, meta = (DisplayName = "Neighbor Query"), MinimalAPI)
class UNiagaraDataInterfaceNeighborQuery : public UNiagaraDataInterfaceGrid3D
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FIntVector,		NumCells)
		SHADER_PARAMETER(FVector3f,			UnitToUV)
		SHADER_PARAMETER(FVector3f,			CellSize)
		SHADER_PARAMETER(FVector3f,			WorldBBoxSize)
		SHADER_PARAMETER(int32,				MaxCellsPerParticleValue)

		// Read path
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	CellCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	CellOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	ParticleList)

		// Write path AddParticle calls write to this
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	OutputCellId)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	OutputParticleIdIndex)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	OutputAcquireTag)

		// Read path for sorted AcquireTag (alongside ParticleList)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	AcquireTagList)
	END_SHADER_PARAMETER_STRUCT()

public:
	UPROPERTY(EditAnywhere, Category = "Grid", AdvancedDisplay)
	uint32 MaxCellsPerParticle;

	/** Skip prefix sum and scatter passes — only the histogram is computed.
	  * GetParticleNeighborCount works; GetParticleNeighbor does not.
	  * Saves 2 GPU dispatches and ~2x buffer memory. */
	UPROPERTY(EditAnywhere, Category = "Grid")
	bool bCountOnly = false;

	/** When enabled (default), AcquireTag is tracked through the counting sort for compaction-safe
	  * neighbor lookups via GetPositionByID. The emitter must have Persistent IDs enabled.
	  * When disabled, AcquireTag buffers are not allocated and GetParticleNeighbor returns 0 for
	  * AcquireTag. Only disable if you know compaction will not reorder particles. */
	UPROPERTY(EditAnywhere, Category = "Grid", AdvancedDisplay)
	bool bUsePersistentIDs = true;

	virtual void PostInitProperties() override
	{
		Super::PostInitProperties();

		if (HasAnyFlags(RF_ClassDefaultObject))
		{
			ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
			FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
		}
	}

	//~ UNiagaraDataInterface interface
	// VM functionality
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

#if WITH_EDITOR
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo) override;
#endif

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override {}
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override { return false; }
	virtual int32 PerInstanceDataSize() const override { return sizeof(FNDINeighborQueryInstanceData_GT); }
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual bool HasPostSimulateTick() const override { return true; }
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target == ENiagaraSimTarget::GPUComputeSim; }
	
	//~ UNiagaraDataInterface interface END

	NIAGARA_API void GetNumCells(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetMaxCellsPerParticle(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void SetNumCells(FVectorVMExternalFunctionContext& Context);


protected:
	//~ UNiagaraDataInterface interface
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	//~ UNiagaraDataInterface interface END
};
