// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceOcclusion.generated.h"

UCLASS(EditInlineNew, Category = "Camera", meta = (DisplayName = "Occlusion Query"), MinimalAPI)
class UNiagaraDataInterfaceOcclusion : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FVector2f,				CloudVolumetricTextureUVScale)
		SHADER_PARAMETER(FVector2f,				CloudVolumetricTextureUVMax)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CloudVolumetricTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,	CloudVolumetricTextureSampler)
	END_SHADER_PARAMETER_STRUCT();

public:
	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target == ENiagaraSimTarget::GPUComputeSim; }
	virtual bool RequiresDepthBuffer() const override { return true; }
	//UNiagaraDataInterface Interface

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

};

struct FNiagaraDataIntefaceProxyOcclusionQuery : public FNiagaraDataInterfaceProxy
{
	// There's nothing in this proxy. It just reads from scene textures.

	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return 0;
	}
};
