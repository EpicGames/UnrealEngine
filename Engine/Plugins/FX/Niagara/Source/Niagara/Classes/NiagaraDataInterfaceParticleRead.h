// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceRW.h"
#include "NiagaraCommon.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraDataInterfaceEmitterBinding.h"
#include "NiagaraDataInterfaceParticleRead.generated.h"

UCLASS(EditInlineNew, Category = "ParticleRead", CollapseCategories, meta = (DisplayName = "Particle Attribute Reader"), MinimalAPI)
class UNiagaraDataInterfaceParticleRead : public UNiagaraDataInterfaceRWBase
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(uint32,			IsLocalSpace)
		SHADER_PARAMETER(int,				NumSpawnedParticles)
		SHADER_PARAMETER(int,				SpawnedParticlesAcquireTag)
		SHADER_PARAMETER(uint32,			InstanceCountOffset)
		SHADER_PARAMETER(uint32,			ParticleStrideFloat)
		SHADER_PARAMETER(uint32,			ParticleStrideInt)
		SHADER_PARAMETER(uint32,			ParticleStrideHalf)
		SHADER_PARAMETER(int,				AcquireTagRegisterIndex)
		SHADER_PARAMETER_SRV(Buffer<int>,	SpawnedIDsBuffer)
		SHADER_PARAMETER_SRV(Buffer<int>,	IDToIndexTable)
		SHADER_PARAMETER_SRV(Buffer<float>,	InputFloatBuffer)
		SHADER_PARAMETER_SRV(Buffer<int>,	InputIntBuffer)
		SHADER_PARAMETER_SRV(Buffer<half>,	InputHalfBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	/** Selects which emitter the data interface will bind to, i.e the emitter we are contained within or a named emitter. */
	UPROPERTY(EditAnywhere, Category = "Emitter")
	FNiagaraDataInterfaceEmitterBinding EmitterBinding;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
#endif
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	NIAGARA_API virtual FNiagaraDataInterfaceParametersCS* CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const override;
	NIAGARA_API virtual const FTypeLayoutDesc* GetShaderStorageType() const override;

	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;
#if WITH_EDITOR	
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& Warnings, TArray<FNiagaraDataInterfaceFeedback>& Info) override;
#endif
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	NIAGARA_API virtual void GetEmitterDependencies(UNiagaraSystem* Asset, TArray<FVersionedNiagaraEmitter>& Dependencies) const override;
	NIAGARA_API virtual void GetEmitterReferencesByName(TArray<FString>& EmitterReferences) const override;

	NIAGARA_API virtual bool HasInternalAttributeReads(const UNiagaraEmitter* OwnerEmitter, const UNiagaraEmitter* Provider) const override;
	//UNiagaraDataInterface Interface End

	NIAGARA_API void VMGetLocalSpace(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetNumSpawnedParticles(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetSpawnedIDAtIndex(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetNumParticles(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetParticleIndex(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetParticleIndexFromIDTable(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void ReadInt(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadBool(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadFloat(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadVector2(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadVector3(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadVector4(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadColor(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadPosition(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadQuat(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadID(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadIntByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadBoolByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadFloatByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadVector2ByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadVector3ByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadVector4ByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadPositionByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadColorByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadQuatByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);
	NIAGARA_API void ReadIDByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead);

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;

	NIAGARA_API void GetPersistentIDFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const;
	NIAGARA_API void GetIndexFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString EmitterName_DEPRECATED;
#endif
};
