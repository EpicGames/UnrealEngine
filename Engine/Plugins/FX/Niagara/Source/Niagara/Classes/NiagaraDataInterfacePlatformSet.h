// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraPlatformSet.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfacePlatformSet.generated.h"


/** Data Interface allowing querying of the current platform set. */
UCLASS(EditInlineNew, Category = "Scalability", CollapseCategories, meta = (DisplayName = "Platform Set"), MinimalAPI)
class UNiagaraDataInterfacePlatformSet : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Platform Set")
	FNiagaraPlatformSet Platforms;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	virtual void PostLoad()override;
	
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
//UObject Interface End

	// UNiagaraDataInterface interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target)const override { return true; }

#if WITH_EDITORONLY_DATA
	virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
#endif
	virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

protected:
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	//~ UNiagaraDataInterface interface

public:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API void IsActive(FVectorVMExternalFunctionContext& Context);
};

