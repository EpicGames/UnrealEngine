// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraDispatchIndirect.h"
#include "NiagaraDataInterfaceRW.generated.h"

struct FNiagaraSimStageData;
struct FNiagaraSimStageDispatchArgs;
struct FNiagaraDispatchIndirectInfoCS;
class FNiagaraSystemInstance;

BEGIN_SHADER_PARAMETER_STRUCT(FNDIGpuComputeDispatchArgsGenParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FNiagaraDispatchIndirectArgsGenCS::FParameters,	IndirectArgsGenParameters)
	
	RDG_BUFFER_ACCESS_ARRAY(BufferAccessArray)
	RDG_TEXTURE_ACCESS_ARRAY(TextureAccessArray)
END_SHADER_PARAMETER_STRUCT()

struct FNDIGpuComputeDispatchArgsGenContext : public FNDIGpuComputeContext
{
	using FIndirectArgs = TPair<FRDGBuffer*, uint32>;
	using FCreateIndirectCallback = TFunction<void(FRHICommandList&, FRDGBuffer*, uint32)>;

	explicit FNDIGpuComputeDispatchArgsGenContext(FRDGBuilder& InGraphBuilder, FNiagaraGpuComputeDispatchInterface& InComputeDispatchInterface)
		: FNDIGpuComputeContext(InGraphBuilder, InComputeDispatchInterface)
	{
	}

	~FNDIGpuComputeDispatchArgsGenContext() { FlushPass(); }

	void SetInstanceData(FNiagaraSystemInstanceID InSystemInstanceID, FNiagaraSimStageData& InSimStageData)
	{
		SystemInstanceID = InSystemInstanceID;
		SimStageData = &InSimStageData;
	}

	const FNiagaraSimStageData& GetSimStageData() const { return *SimStageData; }
	FNiagaraSystemInstanceID GetSystemInstanceID() const { return SystemInstanceID; }

	// Set Direct Dispatch Element Count / Gpu Element Count Offset
	void SetDirect(uint32 InElementCount, uint32 GpuCountOffset = INDEX_NONE) const { SetDirect(FIntVector3(InElementCount, 1, 1), GpuCountOffset); }
	void SetDirect(const FIntPoint& InElementCount, uint32 GpuCountOffset = INDEX_NONE) const { SetDirect(FIntVector3(InElementCount.X, InElementCount.Y, 1), GpuCountOffset); }
	void SetDirect(const FIntVector2& InElementCountXY, uint32 GpuCountOffset = INDEX_NONE) const { SetDirect(FIntVector3(InElementCountXY.X, InElementCountXY.Y, 1), GpuCountOffset); }
	NIAGARA_API void SetDirect(const FIntVector3& InElementCountXYZ, uint32 GpuCountOffset = INDEX_NONE) const;

	// Set Indirect buffer to use
	NIAGARA_API void SetIndirect(FRDGBuffer* InBuffer, uint32 BufferByteOffset) const;
	// Set indirect buffer to use, the callback will be inside the RDG pass
	NIAGARA_API void SetIndirect(FRDGBuffer* InBuffer, uint32 BufferByteOffset, FCreateIndirectCallback&& Callback) const;
	// Create indirect buffer from count values that are held in the GPU count buffer
	// The returned indirect argument information can be kept around for the current frame
	FIndirectArgs CreateIndirect(uint32 InCounterOffset) const { return CreateIndirect(FUintVector3(InCounterOffset, INDEX_NONE, INDEX_NONE)); }
	FIndirectArgs CreateIndirect(const FUintVector2& InCounterOffsets) const { return CreateIndirect(FUintVector3(InCounterOffsets.X, InCounterOffsets.Y, INDEX_NONE)); }
	NIAGARA_API FIndirectArgs CreateIndirect(const FUintVector3& InCounterOffsets) const;

	// Add buffer / texture access used for batching transitions together
	NIAGARA_API void AddBufferAccess(FRDGBuffer* InBuffer, ERHIAccess InAccess) const;
	NIAGARA_API void AddTextureAccess(FRDGTexture* InTexture, ERHIAccess InAccess) const;

protected:
	NIAGARA_API FNDIGpuComputeDispatchArgsGenParameters* GetPassParameters() const;
	NIAGARA_API void FlushPass() const;

protected:
	FNiagaraSystemInstanceID SystemInstanceID;
	FNiagaraSimStageData* SimStageData = nullptr;

	mutable FRDGBuffer* IndirectBuffer = nullptr;
	mutable FNiagaraDispatchIndirectInfoCS* IndirectCounterGenArgs = nullptr;
	mutable int32 NumIndirectCounterGenArgs = 0;
	mutable TArray<TPair<FCreateIndirectCallback, FIndirectArgs>> IndirectCallbacks;
	mutable FNDIGpuComputeDispatchArgsGenParameters* PassParameters = nullptr;
};

UENUM()
enum class ESetResolutionMethod
{
	Independent,
	MaxAxis,
	CellSize
};

// #todo(dmp): some of the stuff we'd expect to see here is on FNiagaraDataInterfaceProxy - refactor?
struct FNiagaraDataInterfaceProxyRW : public FNiagaraDataInterfaceProxy
{
public:
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { check(false); }
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }	

	// Called before PreStage to get the dispatch arguments
	// The default implementation is temporary and will call GetElementCount & GetGPUInstanceCountOffset to set direct arguments.
	// This method will become a pure virtual in 5.3 to enforce its usage over the older methods.
	NIAGARA_API virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context);

	// Get the element count for this instance
	UE_DEPRECATED(5.2, "This function will be removed in 5.3.  You must implement GetDispatchArgs instead.")
	virtual FIntVector GetElementCount(FNiagaraSystemInstanceID SystemInstanceID) const { return FIntVector::ZeroValue; }

	// For data interfaces that support iteration on the GPU we need to be able to get the 'real' element count as known only by the GPU
	// The dispatch will use the CPU count, which is potentially an over-estimation, and the value inside the buffer will be used to clip instances that are not valid
	UE_DEPRECATED(5.2, "This function will be removed in 5.3.  You must implement GetDispatchArgs instead.")
	virtual uint32 GetGPUInstanceCountOffset(FNiagaraSystemInstanceID SystemInstanceID) const { return INDEX_NONE; }

	virtual FNiagaraDataInterfaceProxyRW* AsIterationProxy() override { return this; }
};

UCLASS(abstract, EditInlineNew, MinimalAPI)
class UNiagaraDataInterfaceRWBase : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()
public:
	// Global HLSL variable base names, used by HLSL.
	static NIAGARA_API const FString NumAttributesName;
	static NIAGARA_API const FString NumNamedAttributesName;
	static NIAGARA_API const FString NumCellsName;
	static NIAGARA_API const FString UnitToUVName;
	static NIAGARA_API const FString CellSizeName;
	static NIAGARA_API const FString WorldBBoxSizeName;

	// Attribute names
	static NIAGARA_API const FName NAME_Attribute;

	// Global VM function names, also used by the shaders code generation methods.
	static NIAGARA_API const FName NumCellsFunctionName;
	static NIAGARA_API const FName CellSizeFunctionName;

	static NIAGARA_API const FName WorldBBoxSizeFunctionName;

	static NIAGARA_API const FName SimulationToUnitFunctionName;
	static NIAGARA_API const FName UnitToSimulationFunctionName;
	static NIAGARA_API const FName UnitToIndexFunctionName;
	static NIAGARA_API const FName UnitToFloatIndexFunctionName;
	static NIAGARA_API const FName IndexToUnitFunctionName;

	static NIAGARA_API const FName IndexToUnitStaggeredXFunctionName;
	static NIAGARA_API const FName IndexToUnitStaggeredYFunctionName;

	static NIAGARA_API const FName IndexToLinearFunctionName;
	static NIAGARA_API const FName LinearToIndexFunctionName;

	static NIAGARA_API const FName ExecutionIndexToGridIndexFunctionName;
	static NIAGARA_API const FName ExecutionIndexToUnitFunctionName;
	//~ UObject interface

	virtual void PostLoad() override
	{
		Super::PostLoad();
	}

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);		
	}	
	
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override
	{
		Super::PreEditChange(PropertyAboutToChange);

		// Flush the rendering thread before making any changes to make sure the 
		// data read by the compute shader isn't subject to a race condition.
		// TODO(mv): Solve properly using something like a RT Proxy.
		//FlushRenderingCommands();
	}

#endif
//~ UObject interface END

	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override
	{
		return true;
	}

#if WITH_EDITOR	
	// Editor functionality
	virtual TArray<FNiagaraDataInterfaceError> GetErrors() override
	{
		// TODO(mv): Improve error messages?
		TArray<FNiagaraDataInterfaceError> Errors;

		return Errors;
	}
#endif
};


UCLASS(abstract, EditInlineNew, MinimalAPI)
class UNiagaraDataInterfaceGrid3D : public UNiagaraDataInterfaceRWBase
{
	GENERATED_UCLASS_BODY()

public:

	// Option to clear the buffer prior to a stage where the iteration count does not match the grid resolution.  Useful for stages where one wants to do sparse writes
	// and accumulate values.
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Grid")
	bool ClearBeforeNonIterationStage;

	// Number of cells
	UPROPERTY(EditAnywhere, Category = "Grid", meta=(EditCondition = "false", EditConditionHides))
	FIntVector NumCells;

	// World space size of a cell
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "false", EditConditionHides))
	float CellSize;

	// Number of cells on the longest axis
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "false", EditConditionHides))
	int32 NumCellsMaxAxis;

	// Method for setting the grid resolution
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "false", EditConditionHides))
	ESetResolutionMethod SetResolutionMethod;
	
	// World size of the grid
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "false", EditConditionHides))
	FVector WorldBBoxSize;

public:

	//~ UNiagaraDataInterface interface
	// VM functionality
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;	


	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	//~ UNiagaraDataInterface interface END

protected:
	//~ UNiagaraDataInterface interface
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	//~ UNiagaraDataInterface interface END	
};


UCLASS(abstract, EditInlineNew, MinimalAPI)
class UNiagaraDataInterfaceGrid2D : public UNiagaraDataInterfaceRWBase
{
	GENERATED_UCLASS_BODY()

public:

	// Option to clear the buffer prior to a stage where the iteration count does not match the grid resolution.  Useful for stages where one wants to do sparse writes
	// and accumulate values.
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Grid")
	bool ClearBeforeNonIterationStage;

	// Number of cells in X
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "false", EditConditionHides))
	int32 NumCellsX;

	// Number of cells in Y
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "false", EditConditionHides))
	int32 NumCellsY;
	
	// Number of cells on the longest axis
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "false", EditConditionHides))
	int32 NumCellsMaxAxis;

	// Number of Attributes
	UPROPERTY(EditAnywhere, Category = "Grid", AdvancedDisplay)
	int32 NumAttributes = 0;

	// Set grid resolution according to longest axis
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "false", EditConditionHides))
	bool SetGridFromMaxAxis;	

	// World size of the grid
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (EditCondition = "false", EditConditionHides))
	FVector2D WorldBBoxSize;


public:

	NIAGARA_API virtual void Serialize(FArchive& Ar) override;

	//~ UNiagaraDataInterface interface
	// VM functionality
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif

#if WITH_EDITOR		
	NIAGARA_API virtual void ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors) override;
#endif
	//~ UNiagaraDataInterface interface END


protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	//~ UNiagaraDataInterface interface
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	//~ UNiagaraDataInterface interface END	
};
