// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"

struct FNiagaraDataInterfaceRWAttributeHelper	
{
	static TArray<FString> Channels;

	struct FAttributeInfo
	{
		FName					Name;
		FNiagaraTypeDefinition	TypeDef;
		int32					NumChannels = 0;
		int32					ChannelOffset = 0;
		int32					AttributeIndex = 0;
	};

	explicit FNiagaraDataInterfaceRWAttributeHelper(const FNiagaraDataInterfaceGPUParamInfo& InParamInfo);
	
	template<typename T> void Init(TArray<FText>* OutWarnings = nullptr);
	const FAttributeInfo* FindAttributeInfo(FName AttributeName) const;

	int GetNumAttributes() const
	{
		return AttributeInfos.Num();
	}

	int GetTotalChannels() const
	{
		return TotalChannels;
	}

#if WITH_EDITORONLY_DATA
	// Translates named attribute into actual attribute index	
	FString GetAttributeIndex(const FAttributeInfo* AttributeInfo, int Channel = 0) const;
	FString GetGridChannelString()  const;
	void GetChannelStrings(int AttributeIndex, int AttributeNumChannels, FString& NumChannelsString, FString& AttrGridChannels)  const;
	void GetChannelStrings(const FAttributeInfo* AttributeInfo, FString& NumChannelsString, FString& AttrGridChannels)  const;
#endif //WITH_EDITORONLY_DATA

	const FNiagaraDataInterfaceGPUParamInfo& ParamInfo;
	TArray<FAttributeInfo>						AttributeInfos;
	int32										TotalChannels = 0;
};