// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraRenderGraphUtils.h"
#include "ClearQuad.h"
#include "RHIUtilities.h"

#include "NiagaraDataInterfaceRasterizationGrid3D.generated.h"

class FNiagaraSystemInstance;

// store all data in in a class
// move all data management to use per instance data
// remove references to push data to render thread

class RasterizationGrid3DRWInstanceData
{
public:
	void ResizeBuffers();
	
	bool ClearBeforeNonIterationStage = true;

	int32 TotalNumAttributes = 0;
	FIntVector NumCells = FIntVector::ZeroValue;
	FIntVector NumTiles = FIntVector::ZeroValue;
	float Precision = 0;
	int ResetValue = 0;

	bool NeedsRealloc = false;
	
	FNiagaraPooledRWTexture RasterizationTexture;

	FReadBuffer PerAttributeData;
};

struct FNiagaraDataInterfaceProxyRasterizationGrid3D : public FNiagaraDataInterfaceProxyRW
{	
	virtual void ResetData(const FNDIGpuComputeResetContext& Context) override;
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override;
	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override;

	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override;

	/* List of proxy data for each system instances*/
	// #todo(dmp): this should all be refactored to avoid duplicate code
	TMap<FNiagaraSystemInstanceID, RasterizationGrid3DRWInstanceData> SystemInstancesToProxyData;
};

UCLASS(EditInlineNew, Category = "Grid", CollapseCategories, meta = (DisplayName = "Rasterization Grid3D"), MinimalAPI)
class UNiagaraDataInterfaceRasterizationGrid3D : public UNiagaraDataInterfaceGrid3D
{
	GENERATED_UCLASS_BODY()

public:	

	// Number of attributes stored on the grid
	UPROPERTY(EditAnywhere, Category = "Grid")
	int32 NumAttributes;

	UPROPERTY(EditAnywhere, Category = "Grid")
	float Precision;

	UPROPERTY(EditAnywhere, Category = "Grid")
	int ResetValue;

public:
	virtual void PostInitProperties() override
	{
		Super::PostInitProperties();

		//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
		if (HasAnyFlags(RF_ClassDefaultObject))
		{
			ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
			FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
		}
	}

	//~ UNiagaraDataInterface interface
	// VM functionality
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
#if WITH_EDITOR
	NIAGARA_API virtual bool ShouldCompile(EShaderPlatform ShaderPlatform) const override;
#endif

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
#if WITH_EDITOR
	virtual ENiagaraGpuDispatchType GetGpuDispatchType() const override { return ENiagaraGpuDispatchType::ThreeD; }
#endif

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override {}
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override { return false;  }
	virtual int32 PerInstanceDataSize()const override { return sizeof(RasterizationGrid3DRWInstanceData); }
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual bool HasPostSimulateTick() const override { return true; }
	virtual bool HasPreSimulateTick() const override { return true; }
	//~ UNiagaraDataInterface interface END

	NIAGARA_API void VMGetNumCells(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMSetNumCells(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMSetNumAttributes(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMSetFloatResetValue(FVectorVMExternalFunctionContext& Context);

protected:
	//~ UNiagaraDataInterface interface
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	//~ UNiagaraDataInterface interface END	
};


