// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/TextureDefines.h"
#include "RenderGraphDefinitions.h"

#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraComponent.h"
#include "NiagaraDataInterfaceRenderTarget2DArray.generated.h"

class FNiagaraSystemInstance;
class UTextureRenderTarget2DArray;


struct FRenderTarget2DArrayRWInstanceData_GameThread
{
	FIntVector Size = FIntVector(EForceInit::ForceInitToZero);
	EPixelFormat Format = EPixelFormat::PF_A16B16G16R16;
	TextureFilter Filter = TextureFilter::TF_Default;

	bool bManagedTexture = false;
	UTextureRenderTarget2DArray* TargetTexture = nullptr;
#if WITH_EDITORONLY_DATA
	bool bPreviewTexture = false;
#endif
	FNiagaraParameterDirectBinding<UObject*> RTUserParamBinding;
};

struct FRenderTarget2DArrayRWInstanceData_RenderThread
{
	FIntVector Size = FIntVector(EForceInit::ForceInitToZero);
	int MipLevels = 0;
	bool bWroteThisFrame = false;
	bool bReadThisFrame = false;

	FSamplerStateRHIRef SamplerStateRHI;
	TRefCountPtr<IPooledRenderTarget> RenderTarget;

	FRDGTextureRef		TransientRDGTexture = nullptr;
	FRDGTextureSRVRef	TransientRDGSRV = nullptr;
	FRDGTextureUAVRef	TransientRDGUAV = nullptr;

#if WITH_EDITORONLY_DATA
	bool bPreviewTexture = false;
#endif
#if STATS
	void UpdateMemoryStats();
	uint64 MemorySize = 0;
#endif
};

struct FNiagaraDataInterfaceProxyRenderTarget2DArrayProxy : public FNiagaraDataInterfaceProxyRW
{
	FNiagaraDataInterfaceProxyRenderTarget2DArrayProxy() {}
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override {}
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return 0;
	}

	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override;

	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override;

	/* List of proxy data for each system instances*/
	// #todo(dmp): this should all be refactored to avoid duplicate code
	TMap<FNiagaraSystemInstanceID, FRenderTarget2DArrayRWInstanceData_RenderThread> SystemInstancesToProxyData_RT;
};

UCLASS(EditInlineNew, Category = "Rendering", CollapseCategories, meta = (DisplayName = "Render Target 2D Array"), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceRenderTarget2DArray : public UNiagaraDataInterfaceRWBase
{
	GENERATED_UCLASS_BODY()

public:
	NIAGARA_API virtual void PostInitProperties() override;

	//~ UNiagaraDataInterface interface
	// VM functionality
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target)const override { return true; }
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override {}
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual int32 PerInstanceDataSize()const override { return sizeof(FRenderTarget2DArrayRWInstanceData_GameThread); }
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance, float DeltaSeconds) override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }

	virtual bool CanExposeVariables() const override { return true;}
	NIAGARA_API virtual void GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const override;
	NIAGARA_API virtual bool GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const override;

	//~ UNiagaraDataInterface interface END
	NIAGARA_API void VMGetSize(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMSetSize(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetNumMipLevels(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMSetFormat(FVectorVMExternalFunctionContext& Context);

	UPROPERTY(EditAnywhere, Category = "Render Target", meta = (DisplayPriority = 2, EditCondition = "!bInheritUserParameterSettings", EditConditionHides))
	FIntVector Size;

	/** When enabled overrides the format of the render target, otherwise uses the project default setting. */
	UPROPERTY(EditAnywhere, Category = "Render Target", meta = (DisplayPriority = 11, EditCondition = "!bInheritUserParameterSettings && bOverrideFormat", EditConditionHides))
	TEnumAsByte<ETextureRenderTargetFormat> OverrideRenderTargetFormat;

	/** When enabled overrides the filter of the render target, otherwise uses the project default setting. */
	UPROPERTY(EditAnywhere, Category = "Render Target", meta = (DisplayPriority = 12, EditCondition = "!bInheritUserParameterSettings", EditConditionHides))
	TEnumAsByte<TextureFilter> OverrideRenderTargetFilter = TextureFilter::TF_Default;

	/**
	When enabled texture parameters (size / etc) are taken from the user provided render target.
	If no valid user parameter is set the system will be invalid.
	Note: The resource will be recreated if UAV access is not available, which will reset the contents.
	*/
	UPROPERTY(EditAnywhere, Category = "Render Target", meta = (DisplayPriority = 0))
	uint8 bInheritUserParameterSettings : 1;

	UPROPERTY(EditAnywhere, Category = "Render Target", meta = (DisplayPriority = 10, EditCondition = "!bInheritUserParameterSettings", EditConditionHides))
	uint8 bOverrideFormat : 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Render Target", meta = (DisplayPriority = 20))
	uint8 bPreviewRenderTarget : 1;
#endif

	UPROPERTY(EditAnywhere, Category = "Render Target", meta = (DisplayPriority = 1, ToolTip = "When valid the user parameter is used as the render target rather than creating one internal, note that the input render target will be adjusted by the Niagara simulation"))
	FNiagaraUserParameterBinding RenderTargetUserParameter;

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	static NIAGARA_API FNiagaraVariableBase ExposedRTVar;
};
