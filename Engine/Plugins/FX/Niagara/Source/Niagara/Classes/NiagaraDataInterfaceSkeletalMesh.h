// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/SkeletalMeshComponent.h"
#include "Containers/Array.h"
#include "Engine/SkeletalMesh.h"
#include "Experimental/NiagaraMeshUvMappingHandle.h"
#include "NiagaraDataInterfaceMeshCommon.h"
#include "NiagaraParameterStore.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "WeightedRandomSampler.h"

#include "NiagaraDataInterfaceSkeletalMesh.generated.h"

class UNiagaraDataInterfaceSkeletalMesh;
class USkinnedAsset;
class USkeletalMesh;
struct FSkeletalMeshSkinningData;
struct FSkeletalMeshConnectivity;
class FSkeletalMeshConnectivityProxy;
struct FSkeletalMeshUvMapping;
class FMeshUvMappingBufferProxy;
struct FNDISkeletalMesh_InstanceData;
class FSkinWeightVertexBuffer;
struct FSkeletalMeshSamplingRegion;
struct FSkeletalMeshSamplingRegionLODBuiltData;
struct FSkeletalMeshAccessorHelper;

//////////////////////////////////////////////////////////////////////////

struct FSkeletalMeshSkinningDataUsage
{
	FSkeletalMeshSkinningDataUsage()
		: LODIndex(INDEX_NONE)
		, bUsesBoneMatrices(false)
		, bUsesPreSkinnedVerts(false)
	{}

	FSkeletalMeshSkinningDataUsage(int32 InLODIndex, bool bInUsesBoneMatrices, bool bInUsesPreSkinnedVerts)
		: LODIndex(InLODIndex)
		, bUsesBoneMatrices(bInUsesBoneMatrices)
		, bUsesPreSkinnedVerts(bInUsesPreSkinnedVerts)
	{}

	inline bool NeedBoneMatrices()const { return bUsesBoneMatrices || bUsesPreSkinnedVerts; }
	inline bool NeedPreSkinnedVerts()const { return bUsesPreSkinnedVerts; }
	inline int32 GetLODIndex()const { return LODIndex; }
private:
	int32 LODIndex;
	uint32 bUsesBoneMatrices : 1;
	uint32 bUsesPreSkinnedVerts : 1;
};

struct FSkeletalMeshSkinningDataHandle
{
	FSkeletalMeshSkinningDataHandle();
	FSkeletalMeshSkinningDataHandle(FSkeletalMeshSkinningDataUsage InUsage, const TSharedPtr<struct FSkeletalMeshSkinningData>& InSkinningData, bool bNeedsDataImmediately);
	FSkeletalMeshSkinningDataHandle(const FSkeletalMeshSkinningDataHandle& Other) = delete;
	FSkeletalMeshSkinningDataHandle(FSkeletalMeshSkinningDataHandle&& Other) noexcept;
	~FSkeletalMeshSkinningDataHandle();

	FSkeletalMeshSkinningDataHandle& operator=(const FSkeletalMeshSkinningDataHandle& Other) = delete;
	FSkeletalMeshSkinningDataHandle& operator=(FSkeletalMeshSkinningDataHandle&& Other) noexcept;

	FSkeletalMeshSkinningDataUsage Usage;
	TSharedPtr<FSkeletalMeshSkinningData> SkinningData;
};

struct FSkeletalMeshSkinningData
{
	explicit FSkeletalMeshSkinningData(TWeakObjectPtr<USkinnedMeshComponent> InWeakMeshComponent)
		: WeakMeshComponent(InWeakMeshComponent)
		, DeltaSeconds(.0333f)
		, CurrIndex(0)
		, BoneMatrixUsers(0)
		, TotalPreSkinnedVertsUsers(0)
		, bForceDataRefresh(false)
	{}

	void RegisterUser(FSkeletalMeshSkinningDataUsage Usage, bool bNeedsDataImmediately);
	void UnregisterUser(FSkeletalMeshSkinningDataUsage Usage);
	bool IsUsed()const;
	void ForceDataRefresh();

	TWeakObjectPtr<USkinnedMeshComponent> GetWeakMeshComponent() const { return WeakMeshComponent; }

	bool Tick(float InDeltaSeconds, bool bRequirePreskin = true);

	inline void EnterRead()
	{
		RWGuard.ReadLock();
	}

	inline void ExitRead()
	{
		RWGuard.ReadUnlock();
	}

	inline int32 GetBoneCount(bool RequiresPrevious) const
	{
		int32 BoneCount = CurrComponentTransforms().Num();
		if (RequiresPrevious)
		{
			BoneCount = FMath::Min(BoneCount, PrevComponentTransforms().Num());
		}

		return BoneCount;
	}

	inline FVector3f GetPosition(int32 LODIndex, int32 VertexIndex) const
	{
		return LODData.IsValidIndex(LODIndex) ? LODData[LODIndex].SkinnedCPUPositions[CurrIndex][VertexIndex] : FVector3f::ZeroVector;
	}

	inline FVector3f GetPreviousPosition(int32 LODIndex, int32 VertexIndex) const
	{
		return LODData.IsValidIndex(LODIndex) ? LODData[LODIndex].SkinnedCPUPositions[CurrIndex ^ 1][VertexIndex] : FVector3f::ZeroVector;
	}

	inline void GetTangentBasis(int32 LODIndex, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
	{
		const bool bValidLOD = LODData.IsValidIndex(LODIndex);
		OutTangentX = bValidLOD ? LODData[LODIndex].SkinnedTangentBasis[CurrIndex][(VertexIndex * 3) + 0] : FVector3f(1.0f, 0.0f, 0.0f);
		OutTangentY = bValidLOD ? LODData[LODIndex].SkinnedTangentBasis[CurrIndex][(VertexIndex * 3) + 1] : FVector3f(0.0f, 1.0f, 0.0f);
		OutTangentZ = bValidLOD ? LODData[LODIndex].SkinnedTangentBasis[CurrIndex][(VertexIndex * 3) + 2] : FVector3f(0.0f, 0.0f, 1.0f);
	}

	inline void GetPreviousTangentBasis(int32 LODIndex, int32 VertexIndex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
	{
		const bool bValidLOD = LODData.IsValidIndex(LODIndex);
		OutTangentX = bValidLOD ? LODData[LODIndex].SkinnedTangentBasis[CurrIndex ^ 1][(VertexIndex * 3) + 0] : FVector3f(1.0f, 0.0f, 0.0f);
		OutTangentY = bValidLOD ? LODData[LODIndex].SkinnedTangentBasis[CurrIndex ^ 1][(VertexIndex * 3) + 1] : FVector3f(0.0f, 1.0f, 0.0f);
		OutTangentZ = bValidLOD ? LODData[LODIndex].SkinnedTangentBasis[CurrIndex ^ 1][(VertexIndex * 3) + 2] : FVector3f(0.0f, 0.0f, 1.0f);
	}

private:
	inline TArray<FVector3f>& CurrSkinnedPositions(int32 LODIndex)
	{
		return LODData[LODIndex].SkinnedCPUPositions[CurrIndex];
	}

	inline TArray<FVector3f>& PrevSkinnedPositions(int32 LODIndex)
	{
		return LODData[LODIndex].SkinnedCPUPositions[CurrIndex ^ 1];
	}

	inline TArray<FVector3f>& CurrSkinnedTangentBasis(int32 LODIndex)
	{
		return LODData[LODIndex].SkinnedTangentBasis[CurrIndex];
	}

	inline TArray<FVector3f>& PrevSkinnedTangentBasis(int32 LODIndex)
	{
		return LODData[LODIndex].SkinnedTangentBasis[CurrIndex ^ 1];
	}

public:
	inline TArray<FMatrix44f>& CurrBoneRefToLocals()
	{
		return BoneRefToLocals[CurrIndex];
	}

	inline const TArray<FMatrix44f>& CurrBoneRefToLocals() const
	{
		return BoneRefToLocals[CurrIndex];
	}

	inline TArray<FMatrix44f>& PrevBoneRefToLocals()
	{
		return BoneRefToLocals[CurrIndex ^ 1];
	}

	inline const TArray<FMatrix44f>& PrevBoneRefToLocals() const
	{
		return BoneRefToLocals[CurrIndex ^ 1];
	}

	inline TArray<FTransform3f>& CurrComponentTransforms()
	{
		return ComponentTransforms[CurrIndex];
	}

	inline const TArray<FTransform3f>& CurrComponentTransforms() const
	{
		return ComponentTransforms[CurrIndex];
	}

	inline TArray<FTransform3f>& PrevComponentTransforms()
	{
		return ComponentTransforms[CurrIndex ^ 1];
	}

	inline const TArray<FTransform3f>& PrevComponentTransforms() const
	{
		return ComponentTransforms[CurrIndex ^ 1];
	}

	inline bool NeedPreSkinnedVerts() const
	{
		return TotalPreSkinnedVertsUsers > 0;
	}

	/** Whether this has been ticked this frame.*/
	mutable bool bHasTicked = false;

private:

	void UpdateBoneTransforms();

	FRWLock RWGuard;

	TWeakObjectPtr<USkinnedMeshComponent> WeakMeshComponent;

	/** Delta seconds between calculations of the previous and current skinned positions. */
	float DeltaSeconds;

	/** Index of the current frames skinned positions and bone matrices. */
	int32 CurrIndex;

	/** Number of users for cached bone matrices. */
	int32 BoneMatrixUsers;
	/** Total number of users for pre skinned verts.  (From LODData) */
	int32 TotalPreSkinnedVertsUsers;

	/** Cached bone matrices. */
	TArray<FMatrix44f> BoneRefToLocals[2];

	/** Component space transforms */
	TArray<FTransform3f> ComponentTransforms[2];

	struct FLODData
	{

		/** Number of users for pre skinned verts. */
		int32 PreSkinnedVertsUsers = 0;

		/** CPU Skinned vertex positions. Double buffered to allow accurate velocity calculation. */
		TArray<FVector3f> SkinnedCPUPositions[2];

		/** CPU Skinned tangent basis, where each vertex will map to TangentX + TangentZ */
		TArray<FVector3f> SkinnedTangentBasis[2];
	};
	TArray<FLODData> LODData;

	bool bForceDataRefresh;
};

struct FSkeletalMeshConnectivityUsage
{
	FSkeletalMeshConnectivityUsage() = default;
	FSkeletalMeshConnectivityUsage(bool InRequiresCpuAccess, bool InRequiresGpuAccess)
		: RequiresCpuAccess(InRequiresCpuAccess)
		, RequiresGpuAccess(InRequiresGpuAccess)
	{}

	bool IsValid() const { return RequiresCpuAccess || RequiresGpuAccess; }

	bool RequiresCpuAccess = false;
	bool RequiresGpuAccess = false;
};

struct FSkeletalMeshConnectivityHandle
{
	NIAGARA_API FSkeletalMeshConnectivityHandle();
	NIAGARA_API FSkeletalMeshConnectivityHandle(FSkeletalMeshConnectivityUsage InUsage, const TSharedPtr<struct FSkeletalMeshConnectivity>& InConnectivityData, bool bNeedsDataImmediately);
	FSkeletalMeshConnectivityHandle(const FSkeletalMeshConnectivityHandle& Other) = delete;
	NIAGARA_API FSkeletalMeshConnectivityHandle(FSkeletalMeshConnectivityHandle&& Other) noexcept;
	NIAGARA_API ~FSkeletalMeshConnectivityHandle();

	FSkeletalMeshConnectivityHandle& operator=(const FSkeletalMeshConnectivityHandle& Other) = delete;
	NIAGARA_API FSkeletalMeshConnectivityHandle& operator=(FSkeletalMeshConnectivityHandle&& Other) noexcept;
	NIAGARA_API explicit operator bool() const;

	FSkeletalMeshConnectivityUsage Usage;

	NIAGARA_API int32 GetAdjacentTriangleIndex(int32 VertexIndex, int32 AdjacencyIndex) const;

	NIAGARA_API const FSkeletalMeshConnectivityProxy* GetProxy() const;

	NIAGARA_API void PinAndInvalidateHandle();

private:
	TSharedPtr<FSkeletalMeshConnectivity> ConnectivityData;
};

class FNDI_SkeletalMesh_GeneratedData : public FNDI_GeneratedData
{
	FRWLock CachedSkinningDataGuard;
	TMap<TWeakObjectPtr<USkinnedMeshComponent>, TSharedPtr<FSkeletalMeshSkinningData> > CachedSkinningData;

	FRWLock CachedUvMappingGuard;
	TArray<TSharedPtr<FSkeletalMeshUvMapping>> CachedUvMapping;

	FRWLock CachedConnectivityGuard;
	TArray<TSharedPtr<FSkeletalMeshConnectivity>> CachedConnectivity;

public:
	NIAGARA_API FSkeletalMeshSkinningDataHandle GetCachedSkinningData(TWeakObjectPtr<USkinnedMeshComponent> InComponent, FSkeletalMeshSkinningDataUsage Usage, bool bNeedsDataImmediately);
	NIAGARA_API FSkeletalMeshUvMappingHandle GetCachedUvMapping(TWeakObjectPtr<USkinnedAsset> InMeshObject, int32 InLodIndex, int32 InUvSetIndex, FMeshUvMappingUsage Usage, bool bNeedsDataImmediately);
	NIAGARA_API FSkeletalMeshConnectivityHandle GetCachedConnectivity(TWeakObjectPtr<USkinnedAsset> InMeshObject, int32 InLodIndex, FSkeletalMeshConnectivityUsage Usage, bool bNeedsDataImmediately);

	NIAGARA_API virtual void Tick(ETickingGroup TickGroup, float DeltaSeconds) override;

	static TypeHash GetTypeHash()
	{
		static const TypeHash Hash = FCrc::Strihash_DEPRECATED(TEXT("FNDI_SkeletalMesh_GeneratedData"));
		return Hash;
	}
};

//////////////////////////////////////////////////////////////////////////

UENUM()
enum class ENDISkeletalMesh_SourceMode : uint8
{
	/**
	Default behavior.
	- Use "Source" when specified (either set explicitly or via blueprint with Set Niagara Skeletal Mesh Component).
	- When no source is specified, fall back on attached actor or component.
	- If no attach actor look at the default mesh
	*/
	Default,

	/**	Only use "Source" (either set explicitly or via blueprint with Set Niagara Skeletal Mesh Component). */
	Source,

	/**	Only use the parent actor or component the system is attached to. */
	AttachParent,

	/** Only use the "Default Mesh" specified. */
	DefaultMeshOnly,

	/**
	Search for the component as a child of the Niagara component.
	It does not recurse the hierarchy, only looking for direct children.
	*/
	ChildrenOnly,
};

UENUM()
enum class ENDISkeletalMesh_SkinningMode : uint8
{
	Invalid = (uint8)-1 UMETA(Hidden),

	/**
	No skinning, use for reference pose only.
	- Bone and socket sampling will be calculated on demand.
	- Triangle and vertex sampling will be calculated on demand.
	*/
	None = 0,
	/**
	Skin as required, use for bone or socket sampling or when reading a subset of triangles or vertices.
	- Bone and socket sampling will be calculated up front.
	- Triangle and vertex sampling will be calculated on demand (Note: CPU Access required).
	*/
	SkinOnTheFly,
	/**
	Pre-skin the whole mesh, can be more optimal when reading a lot of triangle or vertex data.
	- Bone and socket sampling will be calculated up front.
	- Triangle and vertex sampling will be calculated up front (Note: CPU Access required).
	*/
	PreSkin,
};

enum class ENDISkeletalMesh_FilterMode : uint8
{
	/** No filtering, use all triangles. */
	None,
	/** Filtered to a single region. */
	SingleRegion,
	/** Filtered to multiple regions. */
	MultiRegion,
};

enum class ENDISkelMesh_AreaWeightingMode : uint8
{
	None,
	AreaWeighted,
};

/** Allows perfect area weighted sampling between different skeletal mesh Sampling regions. */
struct FSkeletalMeshSamplingRegionAreaWeightedSampler : FWeightedRandomSampler
{
	FSkeletalMeshSamplingRegionAreaWeightedSampler();
	void Init(FNDISkeletalMesh_InstanceData* InOwner);
	virtual float GetWeights(TArray<float>& OutWeights)override;

	inline bool IsValid() { return TotalWeight > 0.0f; }

	int32 GetEntries() const { return Alias.Num(); }
protected:
	FNDISkeletalMesh_InstanceData* Owner;
};

/**
 * This contains static data created once from the DI.
 * This should be in a proxy create by GT and accessible on RT.
 * Right now we cannot follow a real Proxy pattern since Niagara does not prevent unloading of UI while RT data is still in use.
 * See https://jira.it.epicgames.net/browse/UE-69336
 */
class FSkeletalMeshGpuSpawnStaticBuffers : public FRenderResource
{
public:
	virtual ~FSkeletalMeshGpuSpawnStaticBuffers() override;

	inline void Initialise(struct FNDISkeletalMesh_InstanceData* InstData, int32 InLODIndex, const FSkeletalMeshLODRenderData& SkeletalMeshLODRenderData, const FSkeletalMeshSamplingLODBuiltData* SkeletalMeshSamplingLODBuiltData, FNiagaraSystemInstance* SystemInstance);

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	virtual FString GetFriendlyName() const override { return TEXT("FSkeletalMeshGpuSpawnStaticBuffers"); }

	USceneComponent* GetSceneComponent() const { return WeakSceneComponent.Get(); }

	int32 GetLODIndex() const { return LODIndex; };

	FRHIShaderResourceView* GetBufferTriangleUniformSamplerProbAliasSRV() const { return BufferTriangleUniformSamplerProbAliasSRV; }
	FRHIShaderResourceView* GetBufferTriangleMatricesOffsetSRV() const { return BufferTriangleMatricesOffsetSRV; }
	uint32 GetTriangleCount() const { return TriangleCount; }
	uint32 GetVertexCount() const { return VertexCount; }

	bool IsSamplingRegionsAllAreaWeighted() const { return bSamplingRegionsAllAreaWeighted; }
	bool IsUseGpuUniformlyDistributedSampling() const { return bUseGpuUniformlyDistributedSampling; }
	int32 GetNumSamplingRegionTriangles() const { return NumSamplingRegionTriangles; }
	int32 GetNumSamplingRegionVertices() const { return NumSamplingRegionVertices; }
	FRHIShaderResourceView* GetSampleRegionsProbAliasSRV() const { return SampleRegionsProbAliasSRV; }
	FRHIShaderResourceView* GetSampleRegionsTriangleIndicesSRV() const { return SampleRegionsTriangleIndicesSRV; }
	FRHIShaderResourceView* GetSampleRegionsVerticesSRV() const { return SampleRegionsVerticesSRV; }

	bool IsMeshValid() const { return bMeshValid; }
	bool HasMeshColors() const { return bHasMeshColors; }

	FRHIShaderResourceView* GetBufferPositionSRV() const { return MeshVertexBufferSRV; }
	FRHIShaderResourceView* GetBufferIndexSRV() const { return MeshIndexBufferSRV; }
	FRHIShaderResourceView* GetBufferTangentSRV() const { return MeshTangentBufferSRV; }
	FRHIShaderResourceView* GetBufferTexCoordSRV() const { return MeshTexCoordBufferSRV; }
	FRHIShaderResourceView* GetBufferColorSRV() const { return MeshColorBufferSRV; }

	uint32 GetNumTexCoord() const { return NumTexCoord; }
	uint32 GetNumWeights() const { return NumWeights; }

	int32 GetNumFilteredBones() const { return NumFilteredBones; }
	int32 GetNumUnfilteredBones() const { return NumUnfilteredBones;  }
	int32 GetExcludedBoneIndex() const { return ExcludedBoneIndex; }
	FRHIShaderResourceView* GetFilteredAndUnfilteredBonesSRV() const { return FilteredAndUnfilteredBonesSRV; }

	int32 GetNumFilteredSockets() const { return NumFilteredSockets; }
	int32 GetFilteredSocketBoneOffset() const { return FilteredSocketBoneOffset; }

protected:
	TWeakObjectPtr<USceneComponent> WeakSceneComponent = nullptr;

	FBufferRHIRef BufferTriangleUniformSamplerProbAliasRHI = nullptr;
	FShaderResourceViewRHIRef BufferTriangleUniformSamplerProbAliasSRV = nullptr;
	FBufferRHIRef BufferTriangleMatricesOffsetRHI = nullptr;
	FShaderResourceViewRHIRef BufferTriangleMatricesOffsetSRV = nullptr;

	bool bSamplingRegionsAllAreaWeighted = false;
	int32 NumSamplingRegionTriangles = 0;
	int32 NumSamplingRegionVertices = 0;
	TResourceArray<uint32> SampleRegionsProbAlias;
	TResourceArray<int32> SampleRegionsTriangleIndicies;
	TResourceArray<int32> SampleRegionsVertices;

	FBufferRHIRef SampleRegionsProbAliasBuffer;
	FShaderResourceViewRHIRef SampleRegionsProbAliasSRV;
	FBufferRHIRef SampleRegionsTriangleIndicesBuffer;
	FShaderResourceViewRHIRef SampleRegionsTriangleIndicesSRV;
	FBufferRHIRef SampleRegionsVerticesBuffer;
	FShaderResourceViewRHIRef SampleRegionsVerticesSRV;

	int32 NumFilteredBones = 0;
	int32 NumUnfilteredBones = 0;
	int32 ExcludedBoneIndex = INDEX_NONE;
	TResourceArray<uint16> FilteredAndUnfilteredBonesArray;
	FBufferRHIRef FilteredAndUnfilteredBonesBuffer;
	FShaderResourceViewRHIRef FilteredAndUnfilteredBonesSRV;

	int32 NumFilteredSockets = 0;
	int32 FilteredSocketBoneOffset = 0;

	/** Cached SRV to gpu buffers of the mesh we spawn from */
	bool bMeshValid = false;
	bool bHasMeshColors = false;
	FShaderResourceViewRHIRef MeshVertexBufferSRV;
	FShaderResourceViewRHIRef MeshIndexBufferSRV;
	FShaderResourceViewRHIRef MeshTangentBufferSRV;
	FShaderResourceViewRHIRef MeshTexCoordBufferSRV;
	FShaderResourceViewRHIRef MeshColorBufferSRV;

	uint32 NumTexCoord = 0;
	uint32 NumWeights = 0;

	// Cached data for resource creation on RenderThread
	int32 LODIndex = INDEX_NONE;
	const FSkeletalMeshLODRenderData* LODRenderData = nullptr;
	const FSkeletalMeshSamplingLODBuiltData* SkeletalMeshSamplingLODBuiltData = nullptr;
	uint32 TriangleCount = 0;
	uint32 VertexCount = 0;
	uint32 InputWeightStride = 0;
	bool bUseGpuUniformlyDistributedSampling = false;

#if STATS
	int64 GPUMemoryUsage = 0;
#endif
};

/**
 * This contains dynamic data created per frame from the DI.
 * This should be in a proxy create by GT and accessible on RT. Right now we cannot follow a real Proxy pattern since Niagara does not prevent unloading of UI while RT data is still in use.
 * See https://jira.it.epicgames.net/browse/UE-69336
 */
class FSkeletalMeshGpuDynamicBufferProxy : public FRenderResource
{
public:

	FSkeletalMeshGpuDynamicBufferProxy() = default;
	virtual ~FSkeletalMeshGpuDynamicBufferProxy() override = default;

	void Initialise(const FReferenceSkeleton& RefSkel, const FSkeletalMeshLODRenderData& SkeletalMeshLODRenderData, uint32 InSamplingSocketCount);

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	void NewFrame(const FNDISkeletalMesh_InstanceData* InstanceData, int32 LODIndex);

	bool DoesBoneDataExist() const { return bBoneGpuBufferValid;}

	int32 GetNumBones() const { return (int32)SamplingBoneCount;  }

	/** Encapsulates a GPU read / CPU write buffer for bone data */
	struct FSkeletalBuffer
	{
		FBufferRHIRef SectionBuffer;
		FShaderResourceViewRHIRef SectionSRV;

		FBufferRHIRef SamplingBuffer;
		FShaderResourceViewRHIRef SamplingSRV;
	};

	FSkeletalBuffer& GetRWBufferBone() { return RWBufferBones[CurrentBoneBufferId % 2]; }
	FSkeletalBuffer& GetRWBufferPrevBone() { return bPrevBoneGpuBufferValid ? RWBufferBones[(CurrentBoneBufferId + 1) % 2] : GetRWBufferBone(); }

	uint32 GetSectionBufferNumElements() const { return SectionBoneCount; }
	uint32 GetSamplingBufferNumElements() const { return SamplingBoneCount + SamplingSocketCount; }

private:
	uint32 SamplingBoneCount = 0;
	uint32 SamplingSocketCount = 0;
	uint32 SectionBoneCount = 0;

	enum { BufferBoneCount = 2 };
	FSkeletalBuffer RWBufferBones[BufferBoneCount];
	uint8 CurrentBoneBufferId = 0;

	bool bBoneGpuBufferValid = false;
	bool bPrevBoneGpuBufferValid = false;

#if STATS
	int64 GPUMemoryUsage = 0;
#endif
};

struct FNDISkeletalMesh_InstanceData
{
	/** Cached ptr to Scene Component we sample from, when found. Otherwise, the scene component to use to transform the PreviewMesh */
	TWeakObjectPtr<USceneComponent> WeakSceneComponent;

	/** A binding to the user ptr we're reading the mesh from (if we are). */
	FNiagaraParameterDirectBinding<UObject*> UserParamBinding;

	/** Always reset the DI when the attach parent changes. */
	TWeakObjectPtr<USceneComponent> WeakCachedAttachParent;

	UObject* CachedUserParam;

	TWeakObjectPtr<USkinnedAsset> WeakSkinnedMesh;

	/** Handle to our skinning data. */
	FSkeletalMeshSkinningDataHandle SkinningData;

	/** Handle to our uv mapping data. */
	FSkeletalMeshUvMappingHandle UvMapping;

	/** Handle to connectivity data. */
	FSkeletalMeshConnectivityHandle Connectivity;

	/** Indices of all valid Sampling regions on the mesh to sample from. */
	TArray<int32> SamplingRegionIndices;

	/** Additional sampler for if we need to do area weighting sampling across multiple area weighted regions. */
	FSkeletalMeshSamplingRegionAreaWeightedSampler SamplingRegionAreaWeightedSampler;

	/** Cached ComponentToWorld of the mesh (falls back to WorldTransform of the system instance). */
	FMatrix Transform;
	/** InverseTranspose of above for transforming normals/tangents. */
	FMatrix TransformInverseTransposed;

	/** Cached ComponentToWorld from previous tick. */
	FMatrix PrevTransform;

	/** Time separating Transform and PrevTransform. */
	float DeltaSeconds;

	/** Preskinned local bounds pulled from attached skeletal mesh */
	FVector3f PreSkinnedLocalBoundsCenter = FVector3f::ZeroVector;
	FVector3f PreSkinnedLocalBoundsExtents = FVector3f::ZeroVector;

	/* Excluded bone for some specific functions, generally the root bone which you don't want to include when picking a random bone. */
	int32 ExcludedBoneIndex = INDEX_NONE;

	/** Number of filtered bones in the array. */
	int32 NumFilteredBones = 0;
	/** Number of unfiltered bones in the array. */
	int32 NumUnfilteredBones = 0;
	/** Indices of the bones filtered by the user followed by the unfiltered bones, if this array is empty no filtering is in effect. */
	TArray<uint16> FilteredAndUnfilteredBones;

	/** Name of all the sockets we use. */
	struct FCachedSocketInfo
	{
		FCachedSocketInfo() : BoneIdx(INDEX_NONE){}
		FTransform3f Transform;
		int32 BoneIdx;
	};
	TArray<FCachedSocketInfo> FilteredSocketInfo;

	/** Bone index of the first socket, sockets are appended to the end of the bone array */
	int32 FilteredSocketBoneOffset = 0;

	/** Index into which socket transforms to use.  */
	uint32 FilteredSocketTransformsIndex = 0;
	/** Transforms for sockets. */
	TStaticArray<TArray<FTransform3f>, 2> FilteredSocketTransforms;

	uint32 ChangeId;

	/** True if SceneComponent was valid on initialization (used to track invalidation of the component on tick) */
	uint32 bComponentValid : 1;

	/** True if StaticMesh was valid on initialization (used to track invalidation of the mesh on tick) */
	uint32 bMeshValid : 1;

	/** True if the mesh we're using allows area weighted sampling on GPU. */
	uint32 bIsGpuUniformlyDistributedSampling : 1;

	uint32 bReadDeformedGeometry : 1 = true;

	/** True if the mesh we're using is to be rendered in unlimited bone influences mode. */
	uint32 bUnlimitedBoneInfluences : 1;
	const FSkinWeightDataVertexBuffer* MeshSkinWeightBuffer;
	const FSkinWeightLookupVertexBuffer* MeshSkinWeightLookupBuffer;
	uint32 MeshBoneWeightStrideBytes;
	uint32 MeshBoneIndexSizeBytes;
	uint32 MeshBoneWeightSizeBytes;

	/** Extra mesh data upload to GPU.*/
	FSkeletalMeshGpuSpawnStaticBuffers* MeshGpuSpawnStaticBuffers;
	FSkeletalMeshGpuDynamicBufferProxy* MeshGpuSpawnDynamicBuffers;

	/** Flag to stub VM functions that rely on mesh data being accessible on the CPU */
	bool bAllowCPUMeshDataAccess;

	/** Whether to reset the emitter if any LOD get streamed in. Used when the required LOD was not initially available. */
	bool bResetOnLODStreamedIn = false;
	/** The cached LODIdx used to initialize the FNDIStaticMesh_InstanceData.*/
	int32 CachedLODIdx = 0;
	/** The referenced LOD data, used to prevent streaming out LODs while they are being referenced*/
	TRefCountPtr<const FSkeletalMeshLODRenderData> CachedLODData;

	bool ResetRequired(UNiagaraDataInterfaceSkeletalMesh* Interface, FNiagaraSystemInstance* SystemInstance) const;

	bool Init(UNiagaraDataInterfaceSkeletalMesh* Interface, FNiagaraSystemInstance* SystemInstance);
	bool Tick(UNiagaraDataInterfaceSkeletalMesh* Interface, FNiagaraSystemInstance* SystemInstance, float InDeltaSeconds);
	void Release();

	inline int32 GetLODIndex()const { return CachedLODIdx; }

	inline const FSkinWeightVertexBuffer* GetSkinWeights() const
	{
		USkinnedMeshComponent* SkinnedComponent = Cast<USkinnedMeshComponent>(WeakSceneComponent.Get());
		if (SkinnedComponent != nullptr && SkinnedComponent->GetSkinnedAsset() != nullptr)
		{
			return SkinnedComponent->GetSkinWeightBuffer(CachedLODIdx);
		}
		return CachedLODData ? &CachedLODData->SkinWeightVertexBuffer : nullptr;
	}

	FTransform CalculateComponentTransform(FNiagaraSystemInstance* SystemInstance) const;
	void UpdateFilteredSocketTransforms();
	TArray<FTransform3f>& GetFilteredSocketsWriteBuffer() { return FilteredSocketTransforms[FilteredSocketTransformsIndex]; }
	const TArray<FTransform3f>& GetFilteredSocketsCurrBuffer() const { return FilteredSocketTransforms[FilteredSocketTransformsIndex]; }
	const TArray<FTransform3f>& GetFilteredSocketsPrevBuffer() const { return FilteredSocketTransforms[(FilteredSocketTransformsIndex + 1) % FilteredSocketTransforms.Num()]; }

	bool HasColorData();
};

/** Data Interface allowing sampling of skeletal meshes. */
UCLASS(EditInlineNew, Category = "Meshes", meta = (DisplayName = "Skeletal Mesh Reader"), MinimalAPI)
class UNiagaraDataInterfaceSkeletalMesh : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	/** Controls how to retrieve the Skeletal Mesh Component to attach to. */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	ENDISkeletalMesh_SourceMode SourceMode;

#if WITH_EDITORONLY_DATA
	/** Mesh used to sample from when not overridden by a source actor from the scene. Only available in editor for previewing. This is removed in cooked builds. */
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (DisallowedClasses = "/Script/ApexDestruction.DestructibleMesh"))
	TSoftObjectPtr<USkeletalMesh> PreviewMesh;
#endif

	/** Mesh used to sample from when not overridden by a source actor from the scene. This mesh is NOT removed from cooked builds. */
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (DisallowedClasses = "/Script/ApexDestruction.DestructibleMesh"))
	TObjectPtr<USkeletalMesh> DefaultMesh;

protected:
	/** The source actor from which to sample. Takes precedence over the direct mesh. Note that this can only be set when used as a user variable on a component in the world.*/
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (DisplayName = "Source Actor"))
	TSoftObjectPtr<AActor> SoftSourceActor;

	/** If defined, the supplied tags will be used to identify a valid component */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	TArray<FName> ComponentTags;

	/** Class type we should look for when attaching defaults to USkeletalMeshComponent if unset */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	TSubclassOf<USkinnedMeshComponent> OptionalComponentClass;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<AActor> Source_DEPRECATED;
#endif

	/** The source component from which to sample. Takes precedence over the direct mesh. Not exposed to the user, only indirectly accessible from blueprints. */
	UPROPERTY(Transient)
	TObjectPtr<USkinnedMeshComponent> SourceComponent;

public:
	/** Reference to a user parameter if we're reading one. */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	FNiagaraUserParameterBinding MeshUserParameter;

	/** Selects which skinning mode to use, for most cases Skin On The Fly will cover your requirements, see individual tooltips for more information. */
	UPROPERTY(EditAnywhere, Category="Mesh")
	ENDISkeletalMesh_SkinningMode SkinningMode;

	/** Sampling regions on the mesh from which to sample. Leave this empty to sample from the whole mesh. */
	UPROPERTY(EditAnywhere, Category="Mesh")
	TArray<FName> SamplingRegions;

	/**
	If no regions are specified, we'll sample the whole mesh at this LODIndex.
	-1 will use the lowest quality LOD available, i.e. Number of LODs - 1.
	*/
	UPROPERTY(EditAnywhere, Category="Mesh")
	int32 WholeMeshLOD;

	/** Set of filtered bones that can be used for sampling. Select from these with GetFilteredBoneAt and RandomFilteredBone. */
	UPROPERTY(EditAnywhere, Category = "Skeleton")
	TArray<FName> FilteredBones;

	/** Set of filtered sockets that can be used for sampling. Select from these with GetFilteredSocketAt and RandomFilteredSocket. */
	UPROPERTY(EditAnywhere, Category = "Skeleton")
	TArray<FName> FilteredSockets;

	/**
	Optionally remove a single bone from Random / Random Unfiltered access.
	You can still include this bone in filtered list and access using the direct index functionality.
	*/
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta=(EditCondition="bExcludeBone"))
	FName ExcludeBoneName;

	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (InlineEditConditionToggle))
	uint8 bExcludeBone : 1;

	UPROPERTY(EditAnywhere, Category = "Experimental - UV Mapping")
	int32 UvSetIndex = 0;

	/** When this option is disabled, we use the previous frame's data for the skeletal mesh and can often issue the simulation early. This greatly
	reduces overhead and allows the game thread to run faster, but comes at a tradeoff if the dependencies might leave gaps or other visual artifacts.*/
	UPROPERTY(EditAnywhere, Category = "Performance")
	bool bRequireCurrentFrameData = true;

	/**
	Overrides the project setting and allows you to opt out of reading from deformed geometry.
	These is not performance gain from doing this, the branches will still exist in the generated code.
	*/
	UPROPERTY(EditAnywhere, Category = "Performance")
	bool bReadDeformedGeometry = true;

	/** Cached change id off of the data interface.*/
	uint32 ChangeId;

	//~ UObject interface
	NIAGARA_API virtual void PostInitProperties()override;
	NIAGARA_API virtual void PostLoad()override;
#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	NIAGARA_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif
	virtual bool CanBeInCluster() const override { return false; }	// Note: Due to BP functionality we can change a UObject property on this DI we can not put into a cluster
	//~ UObject interface END


	//~ UNiagaraDataInterface interface
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual int32 PerInstanceDataSize()const override { return sizeof(FNDISkeletalMesh_InstanceData); }
	virtual bool HasPreSimulateTick() const override { return true; }

	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)override;
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target)const override { return true; }
#if WITH_NIAGARA_DEBUGGER
	NIAGARA_API virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const override;
#endif
#if WITH_EDITOR
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors,
		TArray<FNiagaraDataInterfaceFeedback>& Warnings, TArray<FNiagaraDataInterfaceFeedback>& Info) override;
	NIAGARA_API virtual void ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors) override;
#endif
	virtual bool HasTickGroupPrereqs() const override { return true; }
	NIAGARA_API virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const override;

#if WITH_EDITOR
	NIAGARA_API virtual void ModifyCompilationEnvironment(EShaderPlatform ShaderPlatform, struct FShaderCompilerEnvironment& OutEnvironment) const override;
#endif
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	//~ UNiagaraDataInterface interface END

	/** This overload is for use when initializing per-instance data. It possibly uses the SystemInstance and instance data to initialize a user binding */
	NIAGARA_API USkinnedAsset* GetSkinnedMesh(FNiagaraSystemInstance* SystemInstance, USceneComponent* AttachComponent, TWeakObjectPtr<USceneComponent>& SceneComponent, USkinnedMeshComponent*& FoundSkinnedComponent, FNDISkeletalMesh_InstanceData* InstData);
	/** Finds the skeletal mesh based on settings of the DI and the hierarchy of the object provided */
	NIAGARA_API USkinnedAsset* GetSkinnedMesh(UNiagaraComponent* Component);

	USkinnedMeshComponent* GetSourceComponent() const { return SourceComponent; }
	AActor* GetSourceActor() const { return SoftSourceActor.Get(); }

	NIAGARA_API int32 CalculateLODIndexAndSamplingRegions(USkinnedAsset* SkinnedMesh, TArray<int32>& OutSamplingRegionIndices, bool& OutAllRegionsAreAreaWeighting) const;

	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	// Bind/unbind delegates to release references to the source actor & component.
	NIAGARA_API void UnbindSourceDelegates();
	NIAGARA_API void BindSourceDelegates();

	UFUNCTION()
	NIAGARA_API void OnSourceEndPlay(AActor* InSource, EEndPlayReason::Type Reason);

	//////////////////////////////////////////////////////////////////////////
	// Misc Functions
	NIAGARA_API void VMGetPreSkinnedLocalBounds(FVectorVMExternalFunctionContext& Context);

	//////////////////////////////////////////////////////////////////////////
	//Triangle sampling
	//Triangles are sampled a using MeshTriangleCoordinates which are composed of Triangle index and a bary centric coordinate on that triangle.
public:
#if WITH_EDITORONLY_DATA
	NIAGARA_API void GetTriangleSamplingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const;
#endif
	NIAGARA_API void BindTriangleSamplingFunction(const FVMExternalFunctionBindingInfo& BindingInfo, FNDISkeletalMesh_InstanceData* InstData, FVMExternalFunction &OutFunc);

	template<typename FilterMode, typename AreaWeightingMode>
	void GetFilteredTriangleCount(FVectorVMExternalFunctionContext& Context);

	template<typename FilterMode, typename AreaWeightingMode>
	void GetFilteredTriangleAt(FVectorVMExternalFunctionContext& Context);

	template<typename FilterMode, typename AreaWeightingMode>
	void RandomTriCoord(FVectorVMExternalFunctionContext& Context);

	template<typename FilterMode, typename AreaWeightingMode>
	void IsValidTriCoord(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void GetTriangleData(FVectorVMExternalFunctionContext& Context);
	void GetTriangleIndices(FVectorVMExternalFunctionContext& Context);

	template<typename SkinningHandlerType, typename TransformHandlerType, typename VertexAccessorType, typename bInterpolated>
	void GetTriCoordSkinnedData(FVectorVMExternalFunctionContext& Context);

	template<typename TransformHandlerType, typename bInterpolated>
	void GetTriCoordSkinnedDataFallback(FVectorVMExternalFunctionContext& Context);

	template<typename SkinningHandlerType, typename TransformHandlerType, typename VertexAccessorType, typename bInterpolated>
	void VMGetSkinnedTriangleVertexData(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void GetTriCoordColor(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void GetTriCoordColorFallback(FVectorVMExternalFunctionContext& Context);

	template<typename VertexAccessorType>
	void GetTriCoordUV(FVectorVMExternalFunctionContext& Context);

	template<typename VertexAccessorType>
	void GetTriangleCoordAtUV(FVectorVMExternalFunctionContext& Context);

	template<typename VertexAccessorType>
	void GetTriangleCoordInAabb(FVectorVMExternalFunctionContext& Context);

private:
	template<typename FilterMode, typename AreaWeightingMode>
	inline int32 RandomTriIndex(FNDIRandomHelper& RandHelper, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 InstanceIndex);

	NIAGARA_API void RandomTriangle(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetTriangleCount(FVectorVMExternalFunctionContext& Context);

	template<typename FilterMode, typename AreaWeightingMode>
	inline int32 GetFilteredTriangleCount(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData);

	template<typename FilterMode, typename AreaWeightingMode>
	inline int32 GetFilteredTriangleAt(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIdx);
	//End of Mesh Sampling
	//////////////////////////////////////////////////////////////////////////

	//////////////////////////////////////////////////////////////////////////
	//Vertex Sampling
	//Vertex sampling done with direct vertex indices.
public:

#if WITH_EDITORONLY_DATA
	NIAGARA_API void GetVertexSamplingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const;
#endif
	NIAGARA_API void BindVertexSamplingFunction(const FVMExternalFunctionBindingInfo& BindingInfo, FNDISkeletalMesh_InstanceData* InstData, FVMExternalFunction &OutFunc);

	NIAGARA_API void IsValidVertex(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void RandomVertex(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetVertexCount(FVectorVMExternalFunctionContext& Context);

	template<typename FilterMode>
	void IsValidFilteredVertex(FVectorVMExternalFunctionContext& Context);
	template<typename FilterMode>
	void RandomFilteredVertex(FVectorVMExternalFunctionContext& Context);
	template<typename FilterMode>
	void GetFilteredVertexCount(FVectorVMExternalFunctionContext& Context);
	template<typename FilterMode>
	void GetFilteredVertexAt(FVectorVMExternalFunctionContext& Context);

	template<typename VertexAccessorType>
	void GetVertexData(FVectorVMExternalFunctionContext& Context);

	template<typename SkinningHandlerType, typename TransformHandlerType, typename Interpolated, typename VertexAccessorType>
	void GetVertexSkinnedData(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void GetVertexColor(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void GetVertexColorFallback(FVectorVMExternalFunctionContext& Context);

	template<typename VertexAccessorType>
	void GetVertexUV(FVectorVMExternalFunctionContext& Context);

private:
	template<typename FilterMode>
	inline int32 RandomFilteredVertIndex(FNDIRandomHelper& RandHelper, int32 Instance, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData);

	template<typename FilterMode>
	inline int32 GetFilteredVertexCount(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData);

	template<typename FilterMode>
	inline int32 GetFilteredVertexAt(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIdx);

	//End of Vertex Sampling
	//////////////////////////////////////////////////////////////////////////

	//////////////////////////////////////////////////////////////////////////
	// Direct Bone + Socket Sampling

public:
#if WITH_EDITORONLY_DATA
	NIAGARA_API void GetSkeletonSamplingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const;
#endif
	NIAGARA_API void BindSkeletonSamplingFunction(const FVMExternalFunctionBindingInfo& BindingInfo, FNDISkeletalMesh_InstanceData* InstData, FVMExternalFunction &OutFunc);

	template<typename SkinningHandlerType, typename TransformHandlerType, typename bInterpolated>
	void GetSkinnedBoneData(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void IsValidBone(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void RandomBone(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetBoneCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetParentBone(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void GetFilteredBoneCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetFilteredBoneAt(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void RandomFilteredBone(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void GetUnfilteredBoneCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetUnfilteredBoneAt(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void RandomUnfilteredBone(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void GetFilteredSocketCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetFilteredSocketBoneAt(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetFilteredSocketTransform(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void RandomFilteredSocket(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void RandomFilteredSocketOrBone(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetFilteredSocketOrBoneCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetFilteredSocketOrBoneBoneAt(FVectorVMExternalFunctionContext& Context);
	// End of Direct Bone + Socket Sampling
	//////////////////////////////////////////////////////////////////////////

	NIAGARA_API void SetSourceComponentFromBlueprints(USkinnedMeshComponent* ComponentToUse);
	NIAGARA_API void SetSamplingRegionsFromBlueprints(const TArray<FName>& InSamplingRegions);
	NIAGARA_API void SetFilteredBonesFromBlueprints(const TArray<FName>& InFilteredBones);
	NIAGARA_API void SetFilteredSocketsFromBlueprints(const TArray<FName>& InFilteredSockets);
	NIAGARA_API void SetWholeMeshLODFromBlueprints(int32 MeshLODLevel);
};


class FSkeletalMeshInterfaceHelper
{
public:
	// Triangle Sampling
	static const FName RandomTriCoordName;
	static const FName IsValidTriCoordName;
	static const FName GetTriangleDataName;
	static const FName GetTriangleIndicesName;
	static const FName GetSkinnedTriangleDataName;
	static const FName GetSkinnedTriangleDataWSName;
	static const FName GetSkinnedTriangleDataInterpName;
	static const FName GetSkinnedTriangleDataWSInterpName;
	static const FName GetSkinnedTriangleVertexDataName;
	static const FName GetSkinnedTriangleVertexDataWSName;
	static const FName GetSkinnedTriangleVertexDataInterpName;
	static const FName GetSkinnedTriangleVertexDataWSInterpName;
	static const FName GetTriColorName;
	static const FName GetTriUVName;
	static const FName GetTriCoordVerticesName;
	static const FName RandomTriangleName;
	static const FName GetTriangleCountName;
	static const FName RandomFilteredTriangleName;
	static const FName GetFilteredTriangleCountName;
	static const FName GetFilteredTriangleAtName;

	// Bone Sampling
	static const FName GetSkinnedBoneDataName;
	static const FName GetSkinnedBoneDataWSName;
	static const FName GetSkinnedBoneDataInterpolatedName;
	static const FName GetSkinnedBoneDataWSInterpolatedName;
	static const FName IsValidBoneName;
	static const FName RandomBoneName;
	static const FName GetBoneCountName;
	static const FName GetParentBoneName;

	static const FName RandomFilteredBoneName;
	static const FName GetFilteredBoneCountName;
	static const FName GetFilteredBoneAtName;

	static const FName RandomUnfilteredBoneName;
	static const FName GetUnfilteredBoneCountName;
	static const FName GetUnfilteredBoneAtName;

	static const FName RandomFilteredSocketName;
	static const FName GetFilteredSocketCountName;
	static const FName GetFilteredSocketBoneAtName;
	static const FName GetFilteredSocketTransformName;

	static const FName RandomFilteredSocketOrBoneName;
	static const FName GetFilteredSocketOrBoneCountName;
	static const FName GetFilteredSocketOrBoneAtName;

	// Vertex Sampling
	static const FName GetVertexDataName;
	static const FName GetSkinnedVertexDataName;
	static const FName GetSkinnedVertexDataWSName;
	static const FName GetSkinnedVertexDataInterpolatedName;
	static const FName GetSkinnedVertexDataInterpolatedWSName;
	static const FName GetVertexColorName;
	static const FName GetVertexUVName;

	static const FName IsValidVertexName;
	static const FName RandomVertexName;
	static const FName GetVertexCountName;

	static const FName IsValidFilteredVertexName;
	static const FName RandomFilteredVertexName;
	static const FName GetFilteredVertexCountName;
	static const FName GetFilteredVertexAtName;

	// Uv Mapping
	static const FName GetTriangleCoordAtUVName;
	static const FName GetTriangleCoordInAabbName;

	// Adjacency
	static const FName GetAdjacentTriangleIndexName;
	static const FName GetTriangleNeighborName;
};

struct FNiagaraDISkeletalMeshPassedDataToRT
{
	FSkeletalMeshGpuSpawnStaticBuffers* StaticBuffers = nullptr;
	FSkeletalMeshGpuDynamicBufferProxy* DynamicBuffer = nullptr;
	const FSkinWeightDataVertexBuffer* MeshSkinWeightBuffer = nullptr;
	const FSkinWeightLookupVertexBuffer* MeshSkinWeightLookupBuffer = nullptr;
	const FMeshUvMappingBufferProxy* UvMappingBuffer = nullptr;
	const FSkeletalMeshConnectivityProxy* ConnectivityBuffer = nullptr;

	bool bIsGpuUniformlyDistributedSampling = false;
	bool bReadDeformedGeometry = false;
	bool bUnlimitedBoneInfluences = false;
	uint32 MeshBoneWeightStrideBytes = 0;
	uint32 MeshBoneIndexSizeBytes = 0;
	uint32 MeshBoneWeightSizeBytes = 0;
	FMatrix44f Transform = FMatrix44f::Identity;
	FMatrix44f PrevTransform = FMatrix44f::Identity;
	float DeltaSeconds = 0.0f;
	uint32 UvMappingSet = 0;
	FVector3f PreSkinnedLocalBoundsCenter = FVector3f::ZeroVector;
	FVector3f PreSkinnedLocalBoundsExtents = FVector3f::ZeroVector;
};

typedef FNiagaraDISkeletalMeshPassedDataToRT FNiagaraDataInterfaceProxySkeletalMeshData;

struct FNiagaraDataInterfaceProxySkeletalMesh : public FNiagaraDataInterfaceProxy
{
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return sizeof(FNiagaraDISkeletalMeshPassedDataToRT);
	}

	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override;

	TMap<FNiagaraSystemInstanceID, FNiagaraDataInterfaceProxySkeletalMeshData> SystemInstancesToData;
};
