// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "NiagaraDataInterfaceSparseVolumeTexture.generated.h"

class USparseVolumeTexture;

/** Data Interface allowing sampling of a sparse volume texture */
UCLASS(EditInlineNew, Category = "Texture", CollapseCategories, meta = (DisplayName = "Sparse Volume Texture Sample"), MinimalAPI)
class UNiagaraDataInterfaceSparseVolumeTexture : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_SAMPLER(SamplerState, TileDataTextureSampler)
		SHADER_PARAMETER_TEXTURE(Texture3D<uint>, PageTableTexture)
		SHADER_PARAMETER_TEXTURE(Texture3D, PhysicalTileDataATexture)
		SHADER_PARAMETER_TEXTURE(Texture3D, PhysicalTileDataBTexture)
		SHADER_PARAMETER(FUintVector4, PackedUniforms0)
		SHADER_PARAMETER(FUintVector4, PackedUniforms1)
		SHADER_PARAMETER(FIntVector3, TextureSize)
		SHADER_PARAMETER(int32, MipLevels)
		SHADER_PARAMETER(int32, NumFrames)
	END_SHADER_PARAMETER_STRUCT()

public:
	UPROPERTY(EditAnywhere, Category = "SparseVolumeTexture")
	TObjectPtr<USparseVolumeTexture> SparseVolumeTexture;

	UPROPERTY(EditAnywhere, Category = "SparseVolumeTexture", meta = (ToolTip = "When valid the user parameter is used as the texture rather than the one on the data interface"))
	FNiagaraUserParameterBinding SparseVolumeTextureUserParameter;
	
	UPROPERTY(EditAnywhere, Category = "SparseVolumeTexture")
	bool BlockingStreamingRequests;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API virtual void Serialize(FArchive& Ar) override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target == ENiagaraSimTarget::GPUComputeSim; }

	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;

	virtual bool HasPreSimulateTick() const override { return true; }
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	//UNiagaraDataInterface Interface End

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API void VMGetTextureDimensions(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetNumMipLevels(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMRequestSparseVolumeTextureFrame(FVectorVMExternalFunctionContext& Context);	
	NIAGARA_API void VMGetNumFrames(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void SetTexture(USparseVolumeTexture* InSparseVolumeTexture);

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

protected:
	static NIAGARA_API const TCHAR* TemplateShaderFilePath;
	static NIAGARA_API const FName LoadSparseVolumeTextureName;
	static NIAGARA_API const FName SampleSparseVolumeTextureName;
	static NIAGARA_API const FName GetTextureDimensionsName;
	static NIAGARA_API const FName GetNumMipLevelsName;
	static NIAGARA_API const FName RequestSparseVolumeTextureFrameName;
	static NIAGARA_API const FName GetNumFramesName;
};
