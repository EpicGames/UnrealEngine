// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/SplineComponent.h"
#include "NiagaraCommon.h"
#include "NiagaraDataInterface.h"
#include "NiagaraParameterStore.h"
#include "RHIUtilities.h"
#include "VectorVM.h"
#include "NiagaraDataInterfaceSpline.generated.h"

class UNiagaraDataInterfaceSpline;

UENUM()
enum class ENDISpline_SourceMode : uint8
{
	// Default will use all possible sources in the following order
	// SplineUserParameter - if change will reset the system
	// SoftSourceActor - looks for the first spline component on the root actor
	// AttachParent - looks for the first spline component on the attached parent
	Default,

	// AttachParent - looks for the first spline component on the attached parent
	// Will check every frame to see if it has changed
	AttachParentOnly,

	// SplineUserParameter - if change will reset the system
	// Will check every frame to see if it has changed
	ParameterBindingOnly,
};

/** Proxy data for splines */
struct FNDISpline_InstanceData_RenderThread
{
	~FNDISpline_InstanceData_RenderThread()
	{
		Reset();
	}

	FMatrix44f SplineTransform;
	FMatrix44f SplineTransformRotationMat;
	FMatrix44f SplineTransformInverse;
	FMatrix44f SplineTransformInverseTranspose;
	FQuat4f SplineTransformRotation;

	FVector3f DefaultUpVector;

	float SplineLength;
	float SplineDistanceStep;
	float InvSplineDistanceStep;
	int32 MaxIndex;

	FReadBuffer SplinePositionsLUT;
	FReadBuffer SplineScalesLUT;
	FReadBuffer SplineRotationsLUT;

	void Reset();
};


struct FNiagaraDataInterfaceProxySpline : public FNiagaraDataInterfaceProxy
{
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override {}
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

	// List of proxy data for each system instance
	TMap<FNiagaraSystemInstanceID, FNDISpline_InstanceData_RenderThread> SystemInstancesToProxyData_RT;
};



USTRUCT()
struct FNiagaraDataInterfaceSplineLUT
{
	GENERATED_BODY()
public:
	UPROPERTY()
	TArray<FVector> Positions;
	UPROPERTY()
	TArray<FVector> Scales;
	UPROPERTY()
	TArray<FQuat> Rotations;
	UPROPERTY()
	float SplineLength = 0;
	UPROPERTY()
	float SplineDistanceStep = 0;
	UPROPERTY()
	float InvSplineDistanceStep = 0;
	UPROPERTY()
	int32 MaxIndex = INDEX_NONE;

	void BuildLUT(const FSplineCurves& SplineCurves, int32 NumSteps);
	void Reset();

	void FindNeighborKeys(float InDistance, int32& PrevKey, int32& NextKey, float& Alpha) const;
};

struct FNDISpline_InstanceData
{
	//Cached ptr to component we sample from. 
	TWeakObjectPtr<USplineComponent> Component;

	/** A binding to the user ptr we're reading the mesh from (if we are). */
	FNiagaraParameterDirectBinding<UObject*> UserParamBinding;
	UObject* CachedUserParam = nullptr;

	//Cached ComponentToWorld Rotation
	FQuat TransformQuat = FQuat::Identity;
	//Cached ComponentToWorld.
	FMatrix Transform = FMatrix::Identity;
	//InverseTranspose of above for transforming normals/tangents.
	FMatrix TransformInverseTransposed = FMatrix::Identity;
	FTransform ComponentTransform = FTransform::Identity;
	FNiagaraLWCConverter LwcConverter;

	FVector DefaultUpVector = FVector::UpVector;
	FSplineCurves SplineCurves;
	FNiagaraDataInterfaceSplineLUT SplineLUT;

	bool bSyncedGPUCopy = false;

	// We cache the version of the current spline curves so that we can reset the curves structure if we're using the LUT
	TOptional<uint32> SplineCurvesVersion;

	template<typename UseLUT>
	float GetSplineLength() const;

	bool IsValid() const;




	template<typename UseLUT>
	FVector GetLocationAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;
	template<typename UseLUT>
	FQuat GetQuaternionAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;
	template<typename UseLUT>
	FVector GetDirectionAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;
	template<typename UseLUT>
	FVector GetTangentAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;
	template<typename UseLUT>
	FVector GetUpVectorAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;
	template<typename UseLUT>
	FVector GetRightVectorAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const;

	template<typename UseLUT>
	float GetFinalKeyTime() const;

	template<typename UseLUT>
	float FindInputKeyClosestToWorldLocation(const FVector& WorldLocation) const;

private:
	template<typename UseLUT>
	float ConvertDistanceToKey(float InDistance) const;

	template<typename UseLUT>
	FVector EvaluatePosition(float InKey) const;

	template<typename UseLUT>
	FVector EvaluateScale(float InKey) const;

	template<typename UseLUT>
	FQuat EvaluateRotation(float InKey) const;

	template<typename UseLUT>
	float EvaluateFindNearestPosition(FVector InPosition) const;

	template<typename UseLUT>
	FVector EvaluateDerivativePosition(float InKey) const;
};


/** Data Interface allowing sampling of in-world spline components. Note that this data interface is very experimental. */
UCLASS(EditInlineNew, Category = "Splines", CollapseCategories, meta = (DisplayName = "Spline"), MinimalAPI)
class UNiagaraDataInterfaceSpline : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	/** Controls how the spline source is located each frame. */
	UPROPERTY(EditAnywhere, Category = "Spline")
	ENDISpline_SourceMode SourceMode = ENDISpline_SourceMode::Default;

	/** The source actor from which to sample. Note that this can only be set when used as a user variable on a component in the world. */
	UPROPERTY(EditAnywhere, Category = "Spline", meta = (DisplayName = "Source Actor", EditCondition = "SourceMode == ENDISpline_SourceMode::Default"))
	TSoftObjectPtr<AActor> SoftSourceActor;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<AActor> Source_DEPRECATED;
#endif

	/** Reference to a user parameter if we're reading one. This should be an Object user parameter that is either a USplineComponent or an AActor containing a USplineComponent. */
	UPROPERTY(EditAnywhere, Category = "Spline", meta = (EditCondition = "SourceMode == ENDISpline_SourceMode::Default || SourceMode == ENDISpline_SourceMode::ParameterBindingOnly"))
	FNiagaraUserParameterBinding SplineUserParameter;

	UPROPERTY(EditAnywhere, Category = "Spline")
	bool bUseLUT;

	UPROPERTY(EditAnywhere, Category = "Spline", Meta = (EditCondition = "bUseLuT"))
	int32 NumLUTSteps;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual int32 PerInstanceDataSize()const override;
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target)const override { return true; }
	virtual bool HasPreSimulateTick() const override { return true; }
#if WITH_NIAGARA_DEBUGGER
	NIAGARA_API virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const override;
#endif
	//UNiagaraDataInterface Interface End

	USplineComponent* ResolveSplineComponent(FNiagaraSystemInstance* SystemInstance, FNDISpline_InstanceData* InstanceData) const;

	template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
	void SampleSplinePositionByUnitDistance(FVectorVMExternalFunctionContext& Context);
	template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
	void SampleSplineRotationByUnitDistance(FVectorVMExternalFunctionContext& Context);
	template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
	void SampleSplineUpVectorByUnitDistance(FVectorVMExternalFunctionContext& Context);
	template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
	void SampleSplineRightVectorByUnitDistance(FVectorVMExternalFunctionContext& Context);
	template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
	void SampleSplineDirectionByUnitDistance(FVectorVMExternalFunctionContext& Context);
	template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
	void SampleSplineTangentByUnitDistance(FVectorVMExternalFunctionContext& Context);
	template<typename UseLUT>
	void FindClosestUnitDistanceFromPositionWS(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void GetLocalToWorld(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetLocalToWorldInverseTransposed(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void VMGetSplineLength(FVectorVMExternalFunctionContext& Context);

	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return 0;
	}

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;

	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;

	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

private:

	NIAGARA_API void WriteTransform(const FMatrix44f& ToWrite, FVectorVMExternalFunctionContext& Context);

	TMap<FNiagaraSystemInstanceID, FNDISpline_InstanceData*> SystemInstancesToProxyData_GT;
};
