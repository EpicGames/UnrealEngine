// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"
#include "NiagaraParameterStore.h"
#include "NiagaraDataInterfaceSpriteRendererInfo.generated.h"

class UNiagaraSpriteRendererProperties;

/** This Data Interface can be used to query information about the sprite renderers of an emitter */
UCLASS(EditInlineNew, Category = "Sprite Particles", CollapseCategories, meta = (DisplayName = "Sprite Renderer Info"), MinimalAPI)
class UNiagaraDataInterfaceSpriteRendererInfo : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	UNiagaraSpriteRendererProperties* GetSpriteRenderer() const { return SpriteRenderer; }

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API virtual void BeginDestroy() override;
#if WITH_EDITOR	
	NIAGARA_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif 
	//UObject Interface End

	NIAGARA_API void OnSpriteRendererChanged(UNiagaraSpriteRendererProperties* NewSpriteRenderer);

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
#if WITH_EDITOR
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo) override;
#endif
	//UNiagaraDataInterface Interface	

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	NIAGARA_API virtual void PushToRenderThreadImpl() override;

	NIAGARA_API void VMIsValid(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetSourceMode(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetAlignment(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetFacingMode(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetSubUVDetails(FVectorVMExternalFunctionContext& Context);

	/** The sprite renderer to use. */
	UPROPERTY(EditAnywhere, Category = "Source")
	TObjectPtr<UNiagaraSpriteRendererProperties> SpriteRenderer;

#if WITH_EDITOR
	FDelegateHandle OnSpriteRendererChangedHandle;
#endif
};
