// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "VectorVM.h"
#include "StaticMeshResources.h"
#include "Curves/RichCurve.h"
#include "NiagaraDataInterfaceCurveBase.h"
#include "NiagaraDataInterfaceVector4Curve.generated.h"

class INiagaraCompiler;
class UCurveVector;
class UCurveLinearColor;
class UCurveFloat;
class FNiagaraSystemInstance;

/** Data Interface allowing sampling of vector curves. */
UCLASS(EditInlineNew, Category = "Curves", CollapseCategories, meta = (DisplayName = "Curve for Vector4's"), MinimalAPI)
class UNiagaraDataInterfaceVector4Curve : public UNiagaraDataInterfaceCurveBase
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Curve")
	FRichCurve XCurve;

	UPROPERTY(EditAnywhere, Category = "Curve")
	FRichCurve YCurve;

	UPROPERTY(EditAnywhere, Category = "Curve")
	FRichCurve ZCurve;

	UPROPERTY(EditAnywhere, Category = "Curve")
	FRichCurve WCurve;

#if WITH_EDITORONLY_DATA
private:
	UPROPERTY()
	FRichCurve XCurveCookedEditorCache;
	UPROPERTY()
	FRichCurve YCurveCookedEditorCache;
	UPROPERTY()
	FRichCurve ZCurveCookedEditorCache;
	UPROPERTY()
	FRichCurve WCurveCookedEditorCache;
public:
#endif
	
	enum
	{
		CurveLUTNumElems = 4,
	};

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void Serialize(FArchive& Ar) override;
	//UObject Interface End

	NIAGARA_API virtual void UpdateTimeRanges() override;
	NIAGARA_API virtual TArray<float> BuildLUT(int32 NumEntries) const override;

	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;

	template<typename UseLUT>
	void SampleCurve(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	//~ UNiagaraDataInterfaceCurveBase interface
	NIAGARA_API virtual void GetCurveData(TArray<FCurveData>& OutCurveData) override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif

	virtual int32 GetCurveNumElems() const override { return CurveLUTNumElems; }
#if WITH_EDITORONLY_DATA
	virtual FName GetCurveSampleFunctionName() const override { return SampleCurveName; }
#endif

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

private: 
	template<typename UseLUT>
	inline FVector4f SampleCurveInternal(float X);

	static NIAGARA_API const FName SampleCurveName;
};
