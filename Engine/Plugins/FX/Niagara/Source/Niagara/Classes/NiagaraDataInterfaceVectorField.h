// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "VectorField/VectorField.h"
#include "NiagaraDataInterfaceVectorField.generated.h"

class FNiagaraSystemInstance;

UCLASS(EditInlineNew, Category = "Vector Field", CollapseCategories, meta = (DisplayName = "Vector Field"), MinimalAPI)
class UNiagaraDataInterfaceVectorField : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FVector3f,				TilingAxes)
		SHADER_PARAMETER(FVector3f,				Dimensions)
		SHADER_PARAMETER(FVector3f,				MinBounds)
		SHADER_PARAMETER(FVector3f,				MaxBounds)
		SHADER_PARAMETER_TEXTURE(Texture3D,		Texture)
		SHADER_PARAMETER_SAMPLER(SamplerState,	Sampler)
	END_SHADER_PARAMETER_STRUCT();

public:
	/** Vector field to sample from. */
	UPROPERTY(EditAnywhere, Category = VectorField)
	TObjectPtr<UVectorField> Field;

	UPROPERTY(EditAnywhere, Category = VectorField)
	bool bTileX;
	UPROPERTY(EditAnywhere, Category = VectorField)
	bool bTileY;
	UPROPERTY(EditAnywhere, Category = VectorField)
	bool bTileZ;

public:
	//~ UObject interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override; 
#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	NIAGARA_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
#endif
	//~ UObject interface END

	//~ UNiagaraDataInterface interface
	// VM functionality
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override;

#if WITH_EDITOR	
	// Editor functionality
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo) override;

#endif

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	//~ UNiagaraDataInterface interface END

	// VM functions
	NIAGARA_API void GetFieldDimensions(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void GetFieldBounds(FVectorVMExternalFunctionContext& Context); 
	NIAGARA_API void GetFieldTilingAxes(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void SampleVectorField(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void LoadVectorField(FVectorVMExternalFunctionContext& Context);
	
	//	
	NIAGARA_API FVector GetTilingAxes() const;
	NIAGARA_API FVector GetDimensions() const;
	NIAGARA_API FVector GetMinBounds() const;
	NIAGARA_API FVector GetMaxBounds() const;
protected:
	//~ UNiagaraDataInterface interface
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	//~ UNiagaraDataInterface interface END

	NIAGARA_API virtual void PushToRenderThreadImpl() override;
};

struct FNiagaraDataInterfaceProxyVectorField : public FNiagaraDataInterfaceProxy
{
	FVector Dimensions;
	FVector MinBounds;
	FVector MaxBounds;
	bool bTileX;
	bool bTileY;
	bool bTileZ;
	FTextureRHIRef TextureRHI;

	FVector GetTilingAxes() const
	{
		return FVector(float(bTileX), float(bTileY), float(bTileZ));
	}

	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { check(false); }
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return 0;
	}
};

#if !defined(NIAGARA_VECTOR_FIELD_ISPC_ENABLED_DEFAULT)
#define NIAGARA_VECTOR_FIELD_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool GNiagaraVectorFieldUseIspc = INTEL_ISPC && NIAGARA_VECTOR_FIELD_ISPC_ENABLED_DEFAULT;
#else
extern bool GNiagaraVectorFieldUseIspc;
#endif
