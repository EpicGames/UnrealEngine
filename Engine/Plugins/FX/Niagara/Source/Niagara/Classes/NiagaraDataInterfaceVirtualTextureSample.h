// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "NiagaraDataInterfaceVirtualTextureSample.generated.h"

/** Data Interface allowing sampling of a texture */
UCLASS(EditInlineNew, Category = "Texture", CollapseCategories, meta = (DisplayName = "Virtual Texture Sample", Experimental), MinimalAPI)
class UNiagaraDataInterfaceVirtualTextureSample : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FIntPoint,				TextureSize)
		SHADER_PARAMETER(int32,					MipLevels)				
		SHADER_PARAMETER_TEXTURE(Texture2D,		PageTableTexture0)
		SHADER_PARAMETER_TEXTURE(Texture2D,		PageTableTexture1)
		SHADER_PARAMETER_SRV(Texture2D,			PhysicalTexture)
		SHADER_PARAMETER_ARRAY(FUintVector4,	VTPackedPageTableUniform, [2])
		SHADER_PARAMETER(FUintVector4,			VTPackedUniform)
		SHADER_PARAMETER_SAMPLER(SamplerState,	TextureSampler)
	END_SHADER_PARAMETER_STRUCT()

public:
	UPROPERTY(EditAnywhere, Category = "Texture")
	TObjectPtr<UTexture> Texture;

	UPROPERTY(EditAnywhere, Category = "Texture", meta = (ToolTip = "When valid the user parameter is used as the texture rather than the one on the data interface"))
	FNiagaraUserParameterBinding TextureUserParameter;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties()override;
	virtual bool CanBeInCluster() const override { return false; }	// Note: Due to BP functionality we can change a UObject property on this DI we can not put into a cluster
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target)const override { return Target==ENiagaraSimTarget::GPUComputeSim; }

	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;

	virtual bool HasPreSimulateTick() const override { return true; }
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	//UNiagaraDataInterface Interface End

	NIAGARA_API void VMGetTextureDimensions(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetNumMipLevels(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;	
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

protected:
	static NIAGARA_API const TCHAR* TemplateShaderFilePath;
	static NIAGARA_API const FName LoadTexture2DName;	
	static NIAGARA_API const FName SampleTexture2DName;	
	static NIAGARA_API const FName GetTextureDimensionsName;
	static NIAGARA_API const FName GetNumMipLevelsName;
};
