// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "VectorVM.h"
#include "Engine/VolumeTexture.h"
#include "VolumeCache.h"

#include "NiagaraDataInterfaceVolumeCache.generated.h"

struct FVolumeCacheInstanceData_GameThread;

UCLASS(EditInlineNew, Category = "Grid", CollapseCategories, meta = (DisplayName = "Volume Cache"), MinimalAPI)
class UNiagaraDataInterfaceVolumeCache : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FVector3f, TextureSize)
		SHADER_PARAMETER_TEXTURE(Texture3D, Texture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
	END_SHADER_PARAMETER_STRUCT()

public:
		
	UPROPERTY(EditAnywhere, Category = "File")
	TObjectPtr<UVolumeCache> VolumeCache;

	NIAGARA_API virtual void PostInitProperties() override;

	// UNiagaraDataInterface Interface Begin
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual int32 PerInstanceDataSize()const override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;	
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual bool HasPreSimulateTick() const override { return true; }
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	
	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* InDataFromGT, void* InInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif

	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API void SetFrame(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void ReadFile(FVectorVMExternalFunctionContext& Context);	
	NIAGARA_API void GetNumCells(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API FString GetAssetPath(FString PathFormat, int32 FrameIndex) const;

protected:
	//~ UNiagaraDataInterface interface
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	//~ UNiagaraDataInterface interface END

	TMap<FNiagaraSystemInstanceID, FVolumeCacheInstanceData_GameThread*> SystemInstancesToProxyData_GT;	

	static NIAGARA_API const FName SetFrameName;
	static NIAGARA_API const FName ReadFileName;
	static NIAGARA_API const FName GetNumCellsName;
	static NIAGARA_API const FName IndexToUnitName;
	static NIAGARA_API const FName SampleCurrentFrameValueName;
	static NIAGARA_API const FName GetCurrentFrameValue;
	static NIAGARA_API const FName GetCurrentFrameNumCells;
	static NIAGARA_API const TCHAR* TemplateShaderFilePath;
};
