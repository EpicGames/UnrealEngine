// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSetCompiledData.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "VectorVM.h"
#include "RenderingThread.h"

class FNiagaraDataSet;
class FNiagaraShader;
class FNiagaraGPUInstanceCountManager;
class FNiagaraGpuComputeDispatchInterface;
struct FNiagaraComputeExecutionContext;

//Base class for objects in Niagara that are owned by one object but are then passed for reading to other objects, potentially on other threads.
//This class allows us to know if the object is being used so we do not overwrite it and to ensure it's lifetime so we do not access freed data.
class FNiagaraSharedObject
{
public:
	FNiagaraSharedObject()
		: ReadRefCount(0)
	{}

	/** The owner of this object is now done with it but it may still be in use by others, possibly on other threads. Add to the deletion queue so it can be safely freed when it's no longer in use. */
	void Destroy();
	static void FlushDeletionList();

	inline bool IsInUse()const { return ReadRefCount.Load() != 0; }
	inline bool IsBeingRead()const { return ReadRefCount.Load() > 0; }
	inline bool IsBeingWritten()const { return ReadRefCount.Load() == INDEX_NONE; }

	inline void AddRef()
	{
		check(!IsBeingWritten());
		ReadRefCount++;
	}

	inline void Release()
	{
		check(IsBeingRead());
		ReadRefCount--;
	}

	inline bool TryLock()
	{
		//Only lock if we have no readers.
		//Using INDEX_NONE as a special case value for write locks.
		int32 Expected = 0;
		return ReadRefCount.CompareExchange(Expected, INDEX_NONE);
	}

	//Remove the write lock
	inline void Unlock()
	{
		int32 Expected = INDEX_NONE;
		ensureAlwaysMsgf(ReadRefCount.CompareExchange(Expected, 0), TEXT("Trying to release a write lock on a Niagara shared object that is not locked for write."));
	}
	
	//Removes the write lock and moves directly into a read state.
	inline TRefCountPtr<FNiagaraSharedObject> UnlockForRead()
	{
		int32 Expected = INDEX_NONE;
		ensureAlwaysMsgf(ReadRefCount.CompareExchange(Expected, 1), TEXT("Trying to release a write lock on a Niagara shared object that is not locked for write."));
		return TRefCountPtr<FNiagaraSharedObject>(this, false);
	}

protected:

	/**
	Count of other object currently reading this data. Keeps us from writing to or deleting this data while it's in use. These reads can be on any thread so atomic is used.
	INDEX_NONE used as special case marking this object as locked for write.
	*/
	TAtomic<int32> ReadRefCount;

	static FCriticalSection CritSec;
	static TArray<FNiagaraSharedObject*> DeferredDeletionList;

	virtual ~FNiagaraSharedObject() = default;
};

/** Buffer containing one frame of Niagara simulation data. */
class FNiagaraDataBuffer : public FNiagaraSharedObject
{
	friend class FScopedNiagaraDataSetGPUReadback;
	//friend class FNiagaraGpuComputeDispatchInterface;
	
protected:
	NIAGARA_API virtual ~FNiagaraDataBuffer();

public:
	inline TRefCountPtr<FNiagaraDataBuffer> UnlockForRead()
	{
		int32 Expected = INDEX_NONE;
		ensureAlwaysMsgf(ReadRefCount.CompareExchange(Expected, 1), TEXT("Trying to release a write lock on a Niagara shared object that is not locked for write."));
		return TRefCountPtr<FNiagaraDataBuffer>(this, false);
	}

	NIAGARA_API FNiagaraDataBuffer(FNiagaraDataSet* InOwner);
	NIAGARA_API void Allocate(uint32 NumInstances, bool bMaintainExisting = false);
	NIAGARA_API void ReleaseCPU();

	NIAGARA_API void AllocateGPU(FRHICommandListBase& RHICmdList, uint32 InNumInstances, ERHIFeatureLevel::Type FeatureLevel, const TCHAR* DebugSimName);
	NIAGARA_API void SwapGPU(FNiagaraDataBuffer* BufferToSwap);
	NIAGARA_API void ReleaseGPU();

	NIAGARA_API void SwapInstances(uint32 OldIndex, uint32 NewIndex);
	NIAGARA_API void KillInstance(uint32 InstanceIdx);
	NIAGARA_API void CopyTo(FNiagaraDataBuffer& DestBuffer, int32 SrcStartIdx, int32 DestStartIdx, int32 NumInstances) const;
	NIAGARA_API void CopyToUnrelated(FNiagaraDataBuffer& DestBuffer, int32 SrcStartIdx, int32 DestStartIdx, int32 NumInstances) const;
	NIAGARA_API void GPUCopyFrom(const float* GPUReadBackFloat, const int* GPUReadBackInt, const FFloat16* GPUReadBackHalf, int32 StartIdx, int32 NumInstances, uint32 InSrcFloatStride, uint32 InSrcIntStride, uint32 InSrcHalfStride);
	NIAGARA_API void PushCPUBuffersToGPU(const TArray<FNiagaraDataBufferRef>& SourceBuffers, bool bReleaseRef, FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, const TCHAR* DebugSimName, bool bAllocate=true);
	NIAGARA_API void TransferGPUToCPUImmediate(FRHICommandListImmediate& RHICmdList, FNiagaraGpuComputeDispatchInterface* ComputeInterface, FNiagaraDataBuffer* CPUBuffer) const;
	NIAGARA_API void Dump(int32 StartIndex, int32 NumInstances, const FString& Label, const FName& SortParameterKey = FName()) const;

	inline TArrayView<uint8 const* RESTRICT const> ReadRegisterTable() const { return TArrayView<uint8 const* RESTRICT const>(RegisterTable); }
	inline TArrayView<uint8* RESTRICT const> EditRegisterTable() const { return TArrayView<uint8* RESTRICT const>(RegisterTable); }
	
	typedef uint32 RegisterTypeOffsetType[3];
	inline const RegisterTypeOffsetType& GetRegisterTypeOffsets() { return RegisterTypeOffsets; }

	inline const TArray<uint8>& GetFloatBuffer() const { return FloatData; }
	inline const TArray<uint8>& GetInt32Buffer() const { return Int32Data; }
	inline const TArray<uint8>& GetHalfBuffer() const { return HalfData; }

	inline const uint8* GetComponentPtrFloat(uint32 ComponentIdx) const { return FloatData.GetData() + FloatStride * ComponentIdx; }
	inline const uint8* GetComponentPtrInt32(uint32 ComponentIdx) const { return Int32Data.GetData() + Int32Stride * ComponentIdx; }
	inline const uint8* GetComponentPtrHalf(uint32 ComponentIdx) const { return HalfData.GetData() + HalfStride * ComponentIdx; }
	inline uint8* GetComponentPtrFloat(uint32 ComponentIdx) { return FloatData.GetData() + FloatStride * ComponentIdx; }
	inline uint8* GetComponentPtrInt32(uint32 ComponentIdx) { return Int32Data.GetData() + Int32Stride * ComponentIdx;	}
	inline uint8* GetComponentPtrHalf(uint32 ComponentIdx) { return HalfData.GetData() + HalfStride * ComponentIdx; }

	inline float* GetInstancePtrFloat(uint32 ComponentIdx, uint32 InstanceIdx)	{ return (float*)(GetComponentPtrFloat(ComponentIdx)) + InstanceIdx; }
	inline int32* GetInstancePtrInt32(uint32 ComponentIdx, uint32 InstanceIdx)	{ return (int32*)(GetComponentPtrInt32(ComponentIdx)) + InstanceIdx; }
	inline FFloat16* GetInstancePtrHalf(uint32 ComponentIdx, uint32 InstanceIdx) { return (FFloat16*)(GetComponentPtrHalf(ComponentIdx)) + InstanceIdx; }

	inline const float* GetInstancePtrFloat(uint32 ComponentIdx, uint32 InstanceIdx) const { return (float*)(GetComponentPtrFloat(ComponentIdx)) + InstanceIdx; }
	inline const int32* GetInstancePtrInt32(uint32 ComponentIdx, uint32 InstanceIdx) const { return (int32*)(GetComponentPtrInt32(ComponentIdx)) + InstanceIdx; }
	inline const FFloat16* GetInstancePtrHalf(uint32 ComponentIdx, uint32 InstanceIdx) const { return (FFloat16*)(GetComponentPtrHalf(ComponentIdx)) + InstanceIdx; }

	inline uint8* GetComponentPtrFloat(float* BasePtr, uint32 ComponentIdx) const { return (uint8*)BasePtr + FloatStride * ComponentIdx; }
	inline uint8* GetComponentPtrInt32(int* BasePtr, uint32 ComponentIdx) const { return (uint8*)BasePtr + Int32Stride * ComponentIdx; }
	inline uint8* GetComponentPtrHalf(FFloat16* BasePtr, uint32 ComponentIdx) const { return (uint8*)(BasePtr) + HalfStride * ComponentIdx; }

	inline float* GetInstancePtrFloat(float* BasePtr, uint32 ComponentIdx, uint32 InstanceIdx)const { return (float*)GetComponentPtrFloat(BasePtr, ComponentIdx) + InstanceIdx; }
	inline int32* GetInstancePtrInt32(int* BasePtr, uint32 ComponentIdx, uint32 InstanceIdx)const { return (int32*)GetComponentPtrInt32(BasePtr, ComponentIdx) + InstanceIdx; }
	inline FFloat16* GetInstancePtrHalf(FFloat16 *BasePtr, uint32 ComponentIdx, uint32 InstanceIdx)const { return (FFloat16*)GetComponentPtrHalf(BasePtr, ComponentIdx) + InstanceIdx; }

	inline uint32 GetNumInstances()const { return NumInstances; }
	inline uint32 GetNumInstancesAllocated()const { return NumInstancesAllocated; }
	inline uint32 GetNumSpawnedInstances() const { return NumSpawnedInstances; }

	inline void SetNumInstances(uint32 InNumInstances) { check(InNumInstances <= NumInstancesAllocated); NumInstances = InNumInstances; }
	inline void SetNumSpawnedInstances(uint32 InNumSpawnedInstances) { NumSpawnedInstances = InNumSpawnedInstances; }

	inline void SetGPUDataReadyStage(ENiagaraGpuComputeTickStage::Type InReadyStage) { GPUDataReadyStage = InReadyStage; }
	inline ENiagaraGpuComputeTickStage::Type GetGPUDataReadyStage() const { return GPUDataReadyStage; }
	inline FRWBuffer& GetGPUBufferFloat() { return GPUBufferFloat; }
	inline FRWBuffer& GetGPUBufferInt() { return GPUBufferInt; }
	inline FRWBuffer& GetGPUBufferHalf() { return GPUBufferHalf; }
	inline uint32 GetGPUInstanceCountBufferOffset() const { return GPUInstanceCountBufferOffset; }
	inline FRWBuffer& GetGPUIDToIndexTable() { return GPUIDToIndexTable; }

	inline void SetGPUInstanceCountBufferOffset(uint32 Offset) { GPUInstanceCountBufferOffset = Offset; }

	inline int32 GetSafeComponentBufferSize() const { return GetSafeComponentBufferSize(GetNumInstancesAllocated()); }
	inline uint32 GetFloatStride() const { return FloatStride; }
	inline uint32 GetInt32Stride() const { return Int32Stride; }
	inline uint32 GetHalfStride() const { return HalfStride; }

	inline uint32 GetIDAcquireTag() const { return IDAcquireTag; }
	inline void SetIDAcquireTag(uint32 InTag) { IDAcquireTag = InTag; }

	inline FNiagaraDataSet* GetOwner()const { return Owner; }

	NIAGARA_API int32 TransferInstance(FNiagaraDataBuffer& SourceBuffer, int32 InstanceIndex, bool bRemoveFromSource=true);

	NIAGARA_API bool CheckForNaNs()const;

	inline TArray<int32>& GetIDTable() { return IDToIndexTable; }
	inline const TArray<int32>& GetIDTable() const { return IDToIndexTable; }

	inline void ClearGPUInstanceCount() { GPUInstanceCountBufferOffset = INDEX_NONE; }

	NIAGARA_API void BuildRegisterTable();
	
	void ZeroCPUBuffers();

#if NIAGARA_MEMORY_TRACKING
	int32 GetAllocationSizeBytes() const { return AllocationSizeBytes; }
#endif


private:
	inline void CheckUsage(bool bReadOnly)const;

	inline int32 GetSafeComponentBufferSize(int32 RequiredSize) const
	{
		//Round up to VECTOR_WIDTH_BYTES.
		//Both aligns the component buffers to the vector width but also ensures their ops cannot stomp over one another.		
		return Align(RequiredSize, VECTOR_WIDTH_BYTES) + VECTOR_WIDTH_BYTES;
	}

	/** Back ptr to our owning data set. Used to access layout info for the buffer. */
	FNiagaraDataSet* Owner;

	//////////////////////////////////////////////////////////////////////////
	//CPU Data
	/** Float components of simulation data. */
	TArray<uint8> FloatData;
	/** Int32 components of simulation data. */
	TArray<uint8> Int32Data;
	/** Half components of simulation data. */
	TArray<uint8> HalfData;

	/** Table of IDs to real buffer indices. */
	TArray<int32> IDToIndexTable;
	//////////////////////////////////////////////////////////////////////////

	//////////////////////////////////////////////////////////////////////////
	// GPU Data
	/** Location in the frame where GPU data will be ready, for CPU this is always the first group, for GPU is depends on the features used as to which phase. */
	ENiagaraGpuComputeTickStage::Type GPUDataReadyStage = ENiagaraGpuComputeTickStage::First;
	/** The buffer offset where the instance count is accumulated. */
	uint32 GPUInstanceCountBufferOffset;
	/** GPU Buffer containing floating point values for GPU simulations. */
	FRWBuffer GPUBufferFloat;
	/** GPU Buffer containing integer values for GPU simulations. */
	FRWBuffer GPUBufferInt;
	/** GPU table which maps particle ID to index. */
	FRWBuffer GPUIDToIndexTable;
	/** GPU Buffer containing half values for GPU simulations. */
	FRWBuffer GPUBufferHalf;
#if NIAGARA_MEMORY_TRACKING
	int32 AllocationSizeBytes = 0;
#endif
	//////////////////////////////////////////////////////////////////////////

	/** Number of instances in data. */
	uint32 NumInstances;
	/** Number of instances the buffer has been allocated for. */
	uint32 NumInstancesAllocated;
	/** Stride between components in the float buffer. */
	uint32 FloatStride;
	/** Stride between components in the int32 buffer. */
	uint32 Int32Stride;
	/** Stride between components in the half buffer. */
	uint32 HalfStride;
	/** Number of instances spawned in the last tick. */
	uint32 NumSpawnedInstances;
	/** ID acquire tag used in the last tick. */
	uint32 IDAcquireTag;

	/** Table containing current base locations for all registers in this dataset. */
	TArray<uint8*> RegisterTable;//TODO: Should make inline? Feels like a useful size to keep local would be too big.
	RegisterTypeOffsetType RegisterTypeOffsets;
};

//////////////////////////////////////////////////////////////////////////

/**
General storage class for all per instance simulation data in Niagara.
*/
class FNiagaraDataSet
{
	friend FNiagaraDataBuffer;
	friend class FNiagaraGpuComputeDispatch;

public:

	NIAGARA_API FNiagaraDataSet();
	NIAGARA_API ~FNiagaraDataSet();
	FNiagaraDataSet& operator=(const FNiagaraDataSet&) = delete;

	/** Initialize the data set with the compiled data */
	NIAGARA_API void Init(const FNiagaraDataSetCompiledData* InDataSetCompiledData, int32 DefaultNumBuffers=0);

	/** Resets current data but leaves variable/layout information etc intact. */
	NIAGARA_API void ResetBuffers();

	/** Allocates a new buffer from this data set, or reuses an unused one. */
	NIAGARA_API FNiagaraDataBuffer& AllocateBuffer();

	/** Begins a new simulation pass and grabs a destination buffer. Returns the new destination data buffer. */
	NIAGARA_API FNiagaraDataBuffer& BeginSimulate(bool bResetDestinationData = true);

	/** Ends a simulation pass and sets the current simulation state. */
	NIAGARA_API void EndSimulate(bool SetCurrentData = true);

	/** Set current data directly, you can not call this while inside a BeginSimulate block */
	NIAGARA_API void SetCurrentData(FNiagaraDataBuffer* CurrentData);

	/** Allocates space for NumInstances in the current destination buffer. */
	NIAGARA_API void Allocate(int32 NumInstances, bool bMaintainExisting = false);

	/** Returns size in bytes for all data buffers currently allocated by this dataset. */
	NIAGARA_API int64 GetSizeBytes() const;

	inline bool IsInitialized() const { return bInitialized; }
	inline ENiagaraSimTarget GetSimTarget() const { return CompiledData->SimTarget; }
	inline FNiagaraDataSetID GetID() const { return CompiledData->ID; }	
	inline bool RequiresPersistentIDs() const { return CompiledData->bRequiresPersistentIDs; }

	inline TArray<int32>& GetFreeIDTable() { return FreeIDsTable; }
	inline TArray<int32>& GetSpawnedIDsTable() { return SpawnedIDsTable; }
	inline const TArray<int32>& GetFreeIDTable() const { return FreeIDsTable; }
	inline const TArray<int32>& GetSpawnedIDsTable() const { return SpawnedIDsTable; }
	inline int32* GetNumFreeIDsPtr() { return &NumFreeIDs; }
	inline int32* GetMaxUsedIDPtr() { return &MaxUsedID; }
	inline int32 GetIDAcquireTag() const { return IDAcquireTag; }
	inline void SetIDAcquireTag(int32 InTag) { IDAcquireTag = InTag; }
	inline FRWBuffer& GetGPUFreeIDs() { return GPUFreeIDs; }
	inline uint32 GetGPUNumAllocatedIDs() const { return GPUNumAllocatedIDs; }

	inline const TArray<FNiagaraVariableBase>& GetVariables() const { return CompiledData->Variables; }
	inline uint32 GetNumVariables() const { return CompiledData->Variables.Num(); }
	inline bool HasVariable(const FNiagaraVariableBase& Var) const { return CompiledData->Variables.Contains(Var); }
	inline bool HasVariable(const FName& Name) const { return CompiledData->Variables.FindByPredicate([&](const FNiagaraVariableBase& VarInfo) { return VarInfo.GetName() == Name; }) != nullptr; }
	inline uint32 GetNumFloatComponents() const { return CompiledData->TotalFloatComponents; }
	inline uint32 GetNumInt32Components() const { return CompiledData->TotalInt32Components; }
	inline uint32 GetNumHalfComponents() const { return CompiledData->TotalHalfComponents; }

	const TArray<FNiagaraVariableLayoutInfo>& GetVariableLayouts() const { return CompiledData->VariableLayouts; }
	NIAGARA_API const FNiagaraVariableLayoutInfo* GetVariableLayout(const FNiagaraVariableBase& Var) const;
	NIAGARA_API bool GetVariableComponentOffsets(const FNiagaraVariableBase& Var, int32 &FloatStart, int32 &IntStart, int32& HalfStart) const;

	NIAGARA_API void CopyTo(FNiagaraDataSet& Other, int32 StartIdx = 0, int32 NumInstances = INDEX_NONE, bool bResetOther=true)const;

	NIAGARA_API void CopyFromGPUReadback(const float* GPUReadBackFloat, const int* GPUReadBackInt, const FFloat16* GPUReadBackHalf, int32 StartIdx = 0, int32 NumInstances = INDEX_NONE, uint32 FloatStride = 0, uint32 IntStride = 0, uint32 HalfStride = 0);

	NIAGARA_API void CheckForNaNs() const;

	NIAGARA_API void Dump(int32 StartIndex, int32 NumInstances, const FString& Label, const FName& SortParameterKey = FName()) const;

	inline bool IsCurrentDataValid()const { return CurrentData != nullptr; }
	inline FNiagaraDataBuffer* GetCurrentData()const {	return CurrentData; }
	inline FNiagaraDataBuffer* GetDestinationData()const { return DestinationData; }

	inline FNiagaraDataBuffer& GetCurrentDataChecked() const
	{
		check(CurrentData);
		return *CurrentData;
	}

	inline FNiagaraDataBuffer& GetDestinationDataChecked() const
	{
		check(DestinationData);
		return *DestinationData;
	}

	NIAGARA_API void AllocateGPUFreeIDs(uint32 InNumInstances, FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, const TCHAR* DebugSimName);

	void SetMaxInstanceCount(uint32 InMaxInstanceCount) { MaxInstanceCount = InMaxInstanceCount; }
	void SetMaxAllocationCount(uint32 InMaxAllocationCount) { MaxAllocationCount = InMaxAllocationCount; }
	uint32 GetMaxInstanceCount() const { return MaxInstanceCount; }
	uint32 GetMaxAllocationCount() const { return MaxAllocationCount; }

	const FNiagaraDataSetCompiledData& GetCompiledData() const { check(CompiledData.Get() != nullptr); return *CompiledData.Get(); }

	int NumSpawnedIDs;

private:
	NIAGARA_API void Reset();

	NIAGARA_API void ResetBuffersInternal();

	inline void CheckCorrectThread()const
	{
		// In some rare occasions, the render thread might be null, like when offloading work to Lightmass 
		// The final GIsThreadedRendering check keeps us from inadvertently failing when that happens.
#if DO_GUARD_SLOW
		ENiagaraSimTarget SimTarget = GetSimTarget();
		bool CPUSimOK = (SimTarget == ENiagaraSimTarget::CPUSim && !IsInRenderingThread());
		bool GPUSimOK = (SimTarget == ENiagaraSimTarget::GPUComputeSim && IsInRenderingThread());
		checkfSlow(!GIsThreadedRendering || CPUSimOK || GPUSimOK, TEXT("NiagaraDataSet function being called on incorrect thread."));
#endif
	}

	FNiagaraCompiledDataReference<FNiagaraDataSetCompiledData> CompiledData;

	/** Table of free IDs available to allocate next tick. */
	TArray<int32> FreeIDsTable;

	/** Number of free IDs in FreeIDTable. */
	int32 NumFreeIDs;

	/** Max ID seen in last execution. Allows us to shrink the IDTable. */
	int32 MaxUsedID;

	/** Tag to use when new IDs are acquired. Should be unique per tick. */
	int32 IDAcquireTag;

	/** Table of IDs spawned in the last tick (just the index part, the acquire tag is IDAcquireTag for all of them). */
	TArray<int32> SpawnedIDsTable;

	/** GPU buffer of free IDs available on the next tick. */
	FRWBuffer GPUFreeIDs;

	/** NUmber of IDs allocated for the GPU simulation. */
	uint32 GPUNumAllocatedIDs;

	/** Buffer containing the current simulation state. */
	FNiagaraDataBuffer* CurrentData;

	/** Buffer we're currently simulating into. Only valid while we're simulating i.e between PrepareForSimulate and EndSimulate calls.*/
	FNiagaraDataBuffer* DestinationData;

#if NIAGARA_MEMORY_TRACKING
	/** Tracked memory allocations */
	std::atomic<int64> BufferSizeBytes;
#endif

	/**
	Actual data storage. These are passed to and read directly by the RT.
	This is effectively a pool of buffers for this simulation.
	Typically this should only be two or three entries and we search for a free buffer to write into on BeginSimulate();
	We keep track of the Current and Previous buffers which move with each simulate.
	Additional buffers may be in here if they are currently being used by the render thread.
	*/
	TArray<FNiagaraDataBuffer*, TInlineAllocator<2>> Data;

	/* Max instance count is the maximum number of instances we allow. */
	uint32 MaxInstanceCount;
	/* Max allocation couns it eh maximum number of instances we can allocate which can be > MaxInstanceCount due to rounding. */
	uint32 MaxAllocationCount;

	bool bInitialized;
};

/**
Iterator that will pull or push data between a NiagaraDataBuffer and some FNiagaraVariables it contains.
Super slow. Don't use at runtime.
*/
struct FNiagaraDataVariableIterator
{
	FNiagaraDataVariableIterator(const FNiagaraDataBuffer* InData, uint32 StartIdx = 0)
		: Data(InData)
		, CurrIdx(StartIdx)
	{
		FNiagaraVariable::ConvertFromBaseArray(Data->GetOwner()->GetVariables(), Variables);
	}

	void Get()
	{
		const TArray<FNiagaraVariableLayoutInfo>& VarLayouts = Data->GetOwner()->GetVariableLayouts();
		for (int32 VarIdx = 0; VarIdx < Variables.Num(); ++VarIdx)
		{
			FNiagaraVariable& Var = Variables[VarIdx];
			const FNiagaraVariableLayoutInfo& Layout = VarLayouts[VarIdx];
			Var.AllocateData();
			const uint8* ValuePtr = Var.GetData();

			for (uint32 CompIdx = 0; CompIdx < Layout.GetNumFloatComponents(); ++CompIdx)
			{
				uint32 CompBufferOffset = Layout.GetFloatComponentStart() + CompIdx;
				const float* Src = Data->GetInstancePtrFloat(CompBufferOffset, CurrIdx);
				float* Dst = (float*)(ValuePtr + Layout.LayoutInfo.GetFloatComponentByteOffset(CompIdx));
				*Dst = *Src;
			}

			for (uint32 CompIdx = 0; CompIdx < Layout.GetNumInt32Components(); ++CompIdx)
			{
				uint32 CompBufferOffset = Layout.GetInt32ComponentStart() + CompIdx;
				const int32* Src = Data->GetInstancePtrInt32(CompBufferOffset, CurrIdx);
				int32* Dst = (int32*)(ValuePtr + Layout.LayoutInfo.GetInt32ComponentByteOffset(CompIdx));
				*Dst = *Src;
			}

			for (uint32 CompIdx = 0; CompIdx < Layout.GetNumHalfComponents(); ++CompIdx)
			{
				uint32 CompBufferOffset = Layout.GetHalfComponentStart() + CompIdx;
				const FFloat16* Src = Data->GetInstancePtrHalf(CompBufferOffset, CurrIdx);
				FFloat16* Dst = (FFloat16*)(ValuePtr + Layout.LayoutInfo.GetHalfComponentByteOffset(CompIdx));
				*Dst = *Src;
			}
		}
	}

	void Advance() { ++CurrIdx; }
	bool IsValid()const { return Data && CurrIdx < Data->GetNumInstances(); }
	uint32 GetCurrIndex()const { return CurrIdx; }
	const TArray<FNiagaraVariable>& GetVariables()const { return Variables; }
private:

	const FNiagaraDataBuffer* Data;
	TArray<FNiagaraVariable> Variables;

	uint32 CurrIdx;
};

#if WITH_EDITOR
/**
Allows immediate access to GPU data on the CPU, you can then use FNiagaraDataSetAccessor to access the data.
This will make a copy of the GPU data and will stall the CPU until the data is ready from the GPU,
therefore it should only be used for tools / debugging.  For async readback see FNiagaraSystemInstance::RequestCapture.
*/
class FScopedNiagaraDataSetGPUReadback
{
public:
	FScopedNiagaraDataSetGPUReadback() {}
	inline ~FScopedNiagaraDataSetGPUReadback()
	{
		if (DataBuffer != nullptr)
		{
			DataBuffer->FloatData.Empty();
			DataBuffer->Int32Data.Empty();
		}
	}

	NIAGARA_API void ReadbackData(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, FNiagaraDataSet* InDataSet);
	uint32 GetNumInstances() const { check(DataSet != nullptr); return NumInstances; }

private:
	FNiagaraDataSet*	DataSet = nullptr;
	FNiagaraDataBuffer* DataBuffer = nullptr;
	FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = nullptr;
	uint32				NumInstances = 0;
};
#endif

//////////////////////////////////////////////////////////////////////////

inline void FNiagaraDataBuffer::CheckUsage(bool bReadOnly)const
{
	checkSlow(Owner);

	//We can read on the RT but any modifications must be GT (or GT Task).
	//For GPU sims we must be on the RT.
	checkSlow(
		(Owner->GetSimTarget() == ENiagaraSimTarget::CPUSim && (IsInGameThread() || bReadOnly || !GIsThreadedRendering || !IsInRenderingThread())) ||
		(Owner->GetSimTarget() == ENiagaraSimTarget::GPUComputeSim && IsInParallelRenderingThread())
	);
}

namespace NiagaraDataSetPrivate
{
	inline const FNiagaraDataSetCompiledData& GetCompiledData(const FNiagaraDataSet& DataSet)
	{
		return DataSet.GetCompiledData();
	}
	
	inline const FNiagaraDataSetCompiledData& GetCompiledData(const FNiagaraDataBuffer* DataBuffer)
	{
		return DataBuffer->GetOwner()->GetCompiledData();
	}

	inline FNiagaraDataBuffer* GetCurrentData(const FNiagaraDataSet& DataSet)
	{
		return DataSet.GetCurrentData();
	}

	inline FNiagaraDataBuffer* GetDestinationData(const FNiagaraDataSet& DataSet)
	{
		return DataSet.GetDestinationData();
	}

	inline uint8* GetComponentPtrFloat(FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx)
	{
		return DataBuffer->GetComponentPtrFloat(ComponentIdx);
	}

	inline uint8* GetComponentPtrHalf(FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx)
	{
		return DataBuffer->GetComponentPtrHalf(ComponentIdx);
	}

	inline uint8* GetComponentPtrInt32(FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx)
	{
		return DataBuffer->GetComponentPtrInt32(ComponentIdx);
	}

	inline const uint8* GetComponentPtrFloat(const FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx)
	{
		return DataBuffer->GetComponentPtrFloat(ComponentIdx);
	}

	inline const uint8* GetComponentPtrHalf(const FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx)
	{
		return DataBuffer->GetComponentPtrHalf(ComponentIdx);
	}

	inline const uint8* GetComponentPtrInt32(const FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx)
	{
		return DataBuffer->GetComponentPtrInt32(ComponentIdx);
	}

	inline uint32 GetNumInstances(const FNiagaraDataBuffer* DataBuffer)
	{
		return DataBuffer->GetNumInstances();
	}

	inline uint32 GetComponentStride(const FNiagaraDataBuffer* DataBuffer)
	{
		return DataBuffer->GetFloatStride() / sizeof(float);
	}
}

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "NiagaraCompileHashVisitor.h"
#endif
