// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraTypes.h"
#include "NiagaraDataSetCompiledData.h"

class FNiagaraDataBuffer;
class FNiagaraDataSet;
template<typename TType>
struct FNiagaraDataSetAccessorTypeInfo;

namespace NiagaraDataSetPrivate
{
	inline const FNiagaraDataSetCompiledData& GetCompiledData(const FNiagaraDataSet& DataSet);
	inline const FNiagaraDataSetCompiledData& GetCompiledData(const FNiagaraDataBuffer* DataBuffer);
	inline FNiagaraDataBuffer* GetCurrentData(const FNiagaraDataSet& DataSet);
	inline FNiagaraDataBuffer* GetDestinationData(const FNiagaraDataSet& DataSet);
	inline uint8* GetComponentPtrFloat(FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx);
	inline uint8* GetComponentPtrHalf(FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx);
	inline uint8* GetComponentPtrInt32(FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx);
	inline const uint8* GetComponentPtrFloat(const FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx);
	inline const uint8* GetComponentPtrHalf(const FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx);
	inline const uint8* GetComponentPtrInt32(const FNiagaraDataBuffer* DataBuffer, uint32 ComponentIdx);
	inline uint32 GetNumInstances(const FNiagaraDataBuffer* DataBuffer);
	inline uint32 GetComponentStride(const FNiagaraDataBuffer* DataBuffer);
}

template<typename TType>
struct FNiagaraDataSetAccessor : public FNiagaraDataSetAccessorTypeInfo<TType>::TAccessorBaseClass
{
	static_assert(!TIsUECoreVariant<TType, double>::Value, "Double core variant. Must be float type!");
	inline FNiagaraDataSetAccessor<TType>() {}
	inline explicit FNiagaraDataSetAccessor<TType>(const FNiagaraDataSet& DataSet, const FName VariableName) { FNiagaraDataSetAccessorTypeInfo<TType>::TAccessorBaseClass::Init(NiagaraDataSetPrivate::GetCompiledData(DataSet), VariableName); }
	inline explicit FNiagaraDataSetAccessor<TType>(const FNiagaraDataBuffer* DataBuffer, const FName VariableName) { FNiagaraDataSetAccessorTypeInfo<TType>::TAccessorBaseClass::Init(NiagaraDataSetPrivate::GetCompiledData(DataBuffer), VariableName); }
	inline explicit FNiagaraDataSetAccessor<TType>(const FNiagaraDataSetCompiledData& DataSetCompiledData, const FName VariableName) { FNiagaraDataSetAccessorTypeInfo<TType>::TAccessorBaseClass::Init(DataSetCompiledData, VariableName); }
};

//////////////////////////////////////////////////////////////////////////
// Float / Half readers

template<typename TType>
struct FNiagaraDataSetReaderFloat
{
	static_assert(!TIsUECoreVariant<TType, double>::Value, "Double core variant. Must be float type!");
	static constexpr bool bSupportsHalf = FNiagaraDataSetAccessorTypeInfo<TType>::bSupportsHalf;

	inline FNiagaraDataSetReaderFloat() {}

	explicit FNiagaraDataSetReaderFloat(const FNiagaraDataBuffer* DataBuffer, bool bInIsFloat, int32 ComponentIndex)
		: bIsFloat(bInIsFloat)
	{
		using namespace NiagaraDataSetPrivate;
		if (DataBuffer != nullptr && ComponentIndex != INDEX_NONE)
		{
			checkf(bInIsFloat || bSupportsHalf, TEXT("Must be float if we do not support halfs, how has this happened?"));
			bIsValid = true;
			for (int i = 0; i < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements; ++i)
			{
				ComponentData[i] = bIsFloat ? GetComponentPtrFloat(DataBuffer, ComponentIndex + i) : GetComponentPtrHalf(DataBuffer, ComponentIndex + i);
			}
			NumInstances = GetNumInstances(DataBuffer);
		}
	}

	inline bool IsValid() const { return bIsValid; }

	inline TType operator[](int32 Index) const
	{
		return Get(Index);
	}

	inline TType Get(int Index) const
	{
		check(bIsValid && Index <= NumInstances);
		TType Value;
		if (!bSupportsHalf || bIsFloat)
		{
			for (int i = 0; i < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements; ++i)
			{
				reinterpret_cast<float*>(&Value)[i] = reinterpret_cast<const float*>(ComponentData[i])[Index];
			}
		}
		else
		{
			for (int i = 0; i < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements; ++i)
			{
				reinterpret_cast<float*>(&Value)[i] = FPlatformMath::LoadHalf(&reinterpret_cast<const uint16*>(ComponentData[i])[Index]);
			}
		}
		return Value;
	}

	inline TType GetSafe(int Index, const TType& Default) const
	{
		if (bIsValid && (Index < NumInstances))
		{
			return Get(Index);
		}
		return Default;
	}

	inline float GetMaxElement(int32 ElementIndex) const
	{
		check(IsValid());
		check(ElementIndex < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements);
		if (!bSupportsHalf || bIsFloat)
		{
			float MaxValue = -FLT_MAX;
			for (int i = 0; i < NumInstances; ++i)
			{
				MaxValue = FMath::Max(MaxValue, reinterpret_cast<const float*>(ComponentData[ElementIndex])[i]);
			}
			return MaxValue;
		}
		else
		{
			int16 MaxValue = static_cast<int16>(0xfc00u);
			for (int i = 0; i < NumInstances; ++i)
			{
				MaxValue = FMath::Max(MaxValue, reinterpret_cast<const int16*>(ComponentData[ElementIndex])[i]);
			}
			return FPlatformMath::LoadHalf((uint16*)&MaxValue);
		}
	}

	inline void GetMinMaxElement(int32 ElementIndex, float& MinValueOut, float& MaxValueOut) const
	{
		check(IsValid());
		check(ElementIndex < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements);
		if (!bSupportsHalf || bIsFloat)
		{
			float MinValue = FLT_MAX;
			float MaxValue = -FLT_MAX;
			for (int i = 0; i < NumInstances; ++i)
			{
				MinValue = FMath::Min(MinValue, reinterpret_cast<const float*>(ComponentData[ElementIndex])[i]);
				MaxValue = FMath::Max(MaxValue, reinterpret_cast<const float*>(ComponentData[ElementIndex])[i]);
			}
			MinValueOut = MinValue;
			MaxValueOut = MaxValue;
		}
		else
		{
			int16 MinValue = static_cast<int16>(0x7c00u);
			int16 MaxValue = static_cast<int16>(0xfc00u);
			for (int i = 0; i < NumInstances; ++i)
			{
				MinValue = FMath::Min(MinValue, reinterpret_cast<const int16*>(ComponentData[ElementIndex])[i]);
				MaxValue = FMath::Max(MaxValue, reinterpret_cast<const int16*>(ComponentData[ElementIndex])[i]);
			}
			MinValueOut = FPlatformMath::LoadHalf((uint16*)&MinValue);
			MaxValueOut = FPlatformMath::LoadHalf((uint16*)&MaxValue);
		}
	}

	inline TType GetMax() const
	{
		TType Value;
		for (int i = 0; i < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements; ++i)
		{
			reinterpret_cast<float*>(&Value)[i] = GetMaxElement(i);
		}
		return Value;
	}

	inline void GetMinMax(TType& Min, TType& Max) const
	{
		for (int i = 0; i < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements; ++i)
		{
			GetMinMaxElement(i, reinterpret_cast<float*>(&Min)[i], reinterpret_cast<float*>(&Max)[i]);
		}
	}

private:
	bool bIsValid = false;
	bool bIsFloat = true;
	int32 NumInstances = 0;
	const uint8* ComponentData[FNiagaraDataSetAccessorTypeInfo<TType>::NumElements] = {};
};

//template<typename TType>
//struct FNiagaraDataSetWriterFloat
//{
//};

template<typename TType>
struct FNiagaraDataSetAccessorFloat
{
	static_assert(!TIsUECoreVariant<TType, double>::Value, "Double core variant. Must be float type!");
	inline FNiagaraDataSetAccessorFloat() {}
	inline explicit FNiagaraDataSetAccessorFloat(const FNiagaraDataSet& DataSet, const FName VariableName) { Init(NiagaraDataSetPrivate::GetCompiledData(DataSet), VariableName); }
	inline explicit FNiagaraDataSetAccessorFloat(const FNiagaraDataBuffer* DataBuffer, const FName VariableName) { Init(NiagaraDataSetPrivate::GetCompiledData(DataBuffer), VariableName); }
	inline explicit FNiagaraDataSetAccessorFloat(const FNiagaraDataSetCompiledData& DataSetCompiledData, const FName VariableName) { Init(DataSetCompiledData, VariableName); }
	inline explicit FNiagaraDataSetAccessorFloat(const FNiagaraDataSetCompiledData* DataSetCompiledData, const FName VariableName) { Init(DataSetCompiledData, VariableName); }
	inline void Init(const FNiagaraDataSet& DataSet, const FName VariableName) { Init(NiagaraDataSetPrivate::GetCompiledData(DataSet), VariableName); }

	void Init(const FNiagaraDataSetCompiledData* DataSetCompiledData, const FName VariableName)
	{
		if (DataSetCompiledData != nullptr)
		{
			Init(*DataSetCompiledData, VariableName);
		}
		else
		{
			bIsFloat = true;
			ComponentIndex = INDEX_NONE;
		}
	}

	void Init(const FNiagaraDataSetCompiledData& DataSetCompiledData, const FName VariableName)
	{
		ComponentIndex = INDEX_NONE;

		const int32 FloatVariableIndex = DataSetCompiledData.Variables.IndexOfByPredicate(FNiagaraVariableMatch(FNiagaraDataSetAccessorTypeInfo<TType>::GetFloatType(), VariableName));
		if (FloatVariableIndex != INDEX_NONE)
		{
			bIsFloat = true;
			ComponentIndex = DataSetCompiledData.VariableLayouts[FloatVariableIndex].GetFloatComponentStart();
			return;
		}

		if (FNiagaraDataSetAccessorTypeInfo<TType>::bSupportsAlternateType)
		{
			const int32 AlternateVariableIndex = DataSetCompiledData.Variables.IndexOfByPredicate(FNiagaraVariableMatch(FNiagaraDataSetAccessorTypeInfo<TType>::GetAlternateType(), VariableName));
			if (AlternateVariableIndex != INDEX_NONE)
			{
				bIsFloat = true;
				ComponentIndex = DataSetCompiledData.VariableLayouts[AlternateVariableIndex].GetFloatComponentStart();
				return;
			}
		}
		
		if (FNiagaraDataSetAccessorTypeInfo<TType>::bSupportsHalf)
		{
			const int32 HalfVariableIndex = DataSetCompiledData.Variables.IndexOfByPredicate(FNiagaraVariableMatch(FNiagaraDataSetAccessorTypeInfo<TType>::GetHalfType(), VariableName));
			if (HalfVariableIndex != INDEX_NONE)
			{
				bIsFloat = false;
				ComponentIndex = DataSetCompiledData.VariableLayouts[HalfVariableIndex].GetHalfComponentStart();
			}
		}
	}

	inline bool IsValid() const { return ComponentIndex != INDEX_NONE; }

	inline static FNiagaraDataSetReaderFloat<TType> CreateReader(const FNiagaraDataSet& DataSet, const FName VariableName) { return FNiagaraDataSetAccessorFloat<TType>(DataSet, VariableName).GetReader(DataSet); }
	inline static FNiagaraDataSetReaderFloat<TType> CreateReader(const FNiagaraDataBuffer* DataBuffer, const FName VariableName) { return FNiagaraDataSetAccessorFloat<TType>(DataBuffer, VariableName).GetReader(DataBuffer); }
	//FORCEINLINE static FNiagaraDataSetWriterFloat<TType> CreateWriter(const FNiagaraDataSet& DataSet, const FName VariableName) { return FNiagaraDataSetAccessorFloat<TType>(DataSet, VariableName).GetWriter(DataSet); }

	inline FNiagaraDataSetReaderFloat<TType> GetReader(const FNiagaraDataSet& DataSet) const { return FNiagaraDataSetReaderFloat<TType>(NiagaraDataSetPrivate::GetCurrentData(DataSet), bIsFloat, ComponentIndex); }
	inline FNiagaraDataSetReaderFloat<TType> GetReader(const FNiagaraDataBuffer* Data) const { return FNiagaraDataSetReaderFloat<TType>(Data, bIsFloat, ComponentIndex); }
	//FORCEINLINE FNiagaraDataSetWriterFloat<TType> GetWriter(const FNiagaraDataSet& DataSet) const { return FNiagaraDataSetWriterFloat<TType>(NiagaraDataSetPrivate::GetDestinationData(DataSet), bIsFloat, ComponentIndex); }

private:
	bool bIsFloat = true;
	int32 ComponentIndex = INDEX_NONE;
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<float>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorFloat<float>;

	static constexpr bool bSupportsHalf = true;
	static constexpr bool bSupportsAlternateType = false;
	static constexpr int32 NumElements = 1;
	static const FNiagaraTypeDefinition& GetFloatType() { return FNiagaraTypeDefinition::GetFloatDef(); }
	static const FNiagaraTypeDefinition& GetHalfType() { return FNiagaraTypeDefinition::GetHalfDef(); }
	static const FNiagaraTypeDefinition& GetAlternateType() { return FNiagaraTypeDefinition::GetFloatDef(); }
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<FVector2f>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorFloat<FVector2f>;

	static constexpr bool bSupportsHalf = true;
	static constexpr bool bSupportsAlternateType = false;
	static constexpr int32 NumElements = 2;
	static const FNiagaraTypeDefinition& GetFloatType() { return FNiagaraTypeDefinition::GetVec2Def(); }
	static const FNiagaraTypeDefinition& GetHalfType() { return FNiagaraTypeDefinition::GetHalfVec2Def(); }
	static const FNiagaraTypeDefinition& GetAlternateType() { return FNiagaraTypeDefinition::GetVec2Def(); }
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<FVector3f>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorFloat<FVector3f>;

	static constexpr bool bSupportsHalf = true;
	static constexpr bool bSupportsAlternateType = false;
	static constexpr int32 NumElements = 3;
	static const FNiagaraTypeDefinition& GetFloatType() { return FNiagaraTypeDefinition::GetVec3Def(); }
	static const FNiagaraTypeDefinition& GetHalfType() { return FNiagaraTypeDefinition::GetHalfVec3Def(); }
	static const FNiagaraTypeDefinition& GetAlternateType() { return FNiagaraTypeDefinition::GetVec3Def(); }
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<FNiagaraPosition>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorFloat<FNiagaraPosition>;

	static constexpr bool bSupportsHalf = true;
	static constexpr bool bSupportsAlternateType = true;
	static constexpr int32 NumElements = 3;
	static const FNiagaraTypeDefinition& GetFloatType() { return FNiagaraTypeDefinition::GetPositionDef(); }
	static const FNiagaraTypeDefinition& GetHalfType() { return FNiagaraTypeDefinition::GetHalfVec3Def(); }
	static const FNiagaraTypeDefinition& GetAlternateType() { return FNiagaraTypeDefinition::GetVec3Def(); }
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<FVector4f>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorFloat<FVector4f>;

	static constexpr bool bSupportsHalf = true;
	static constexpr bool bSupportsAlternateType = false;
	static constexpr int32 NumElements = 4;
	static const FNiagaraTypeDefinition& GetFloatType() { return FNiagaraTypeDefinition::GetVec4Def(); }
	static const FNiagaraTypeDefinition& GetHalfType() { return FNiagaraTypeDefinition::GetHalfVec4Def(); }
	static const FNiagaraTypeDefinition& GetAlternateType() { return FNiagaraTypeDefinition::GetVec4Def(); }
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<FLinearColor>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorFloat<FLinearColor>;

	static constexpr bool bSupportsHalf = true;
	static constexpr bool bSupportsAlternateType = false;
	static constexpr int32 NumElements = 4;
	static const FNiagaraTypeDefinition& GetFloatType() { return FNiagaraTypeDefinition::GetColorDef(); }
	static const FNiagaraTypeDefinition& GetHalfType() { return FNiagaraTypeDefinition::GetHalfVec4Def(); }
	static const FNiagaraTypeDefinition& GetAlternateType() { return FNiagaraTypeDefinition::GetColorDef(); }
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<FQuat4f>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorFloat<FQuat4f>;

	static constexpr bool bSupportsHalf = true;
	static constexpr bool bSupportsAlternateType = false;
	static constexpr int32 NumElements = 4;
	static const FNiagaraTypeDefinition& GetFloatType() { return FNiagaraTypeDefinition::GetQuatDef(); }
	static const FNiagaraTypeDefinition& GetHalfType() { return FNiagaraTypeDefinition::GetHalfVec4Def(); }
	static const FNiagaraTypeDefinition& GetAlternateType() { return FNiagaraTypeDefinition::GetQuatDef(); }
};

//////////////////////////////////////////////////////////////////////////
// Integer readers

template<typename TType>
struct FNiagaraDataSetReaderInt32
{
	inline FNiagaraDataSetReaderInt32() { }

	explicit FNiagaraDataSetReaderInt32(const FNiagaraDataBuffer* DataBuffer, int32 ComponentIndex)
	{
		using namespace NiagaraDataSetPrivate;
		if (DataBuffer != nullptr && ComponentIndex != INDEX_NONE)
		{
			bIsValid = true;
			for (int i = 0; i < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements; ++i)
			{
				ComponentData[i] = GetComponentPtrInt32(DataBuffer, ComponentIndex + i);
			}
			NumInstances = GetNumInstances(DataBuffer);
		}
	}

	inline bool IsValid() const { return bIsValid; }

	inline TType operator[](int32 Index) const
	{
		return Get(Index);
	}

	inline TType Get(int Index) const
	{
		check(bIsValid && Index <= NumInstances);
		TType Value;
		for (int i = 0; i < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements; ++i)
		{
			reinterpret_cast<int32*>(&Value)[i] = reinterpret_cast<const int32*>(ComponentData[i])[Index];
		}
		return Value;
	}

	inline TType GetSafe(int Index, const TType& Default) const
	{
		if (bIsValid && (Index < NumInstances))
		{
			return Get(Index);
		}
		return Default;
	}

private:
	bool bIsValid = false;
	int32 NumInstances = 0;
	const uint8* ComponentData[FNiagaraDataSetAccessorTypeInfo<TType>::NumElements] = {};
};

template<typename TType>
struct FNiagaraDataSetWriterInt32
{
	inline FNiagaraDataSetWriterInt32() { }

	explicit FNiagaraDataSetWriterInt32(FNiagaraDataBuffer* DataBuffer, int32 ComponentIndex)
	{
		using namespace NiagaraDataSetPrivate;
		if (DataBuffer != nullptr && ComponentIndex != INDEX_NONE)
		{
			bIsValid = true;
			for (int i = 0; i < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements; ++i)
			{
				ComponentData[i] = GetComponentPtrInt32(DataBuffer, ComponentIndex + i);
			}
			NumInstances = GetNumInstances(DataBuffer);
		}
	}

	inline bool IsValid() const { return bIsValid; }

	inline void Set(int Index, const TType& Value)
	{
		check(bIsValid && Index <= NumInstances);
		for (int i = 0; i < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements; ++i)
		{
			reinterpret_cast<int32*>(ComponentData[i])[Index] = reinterpret_cast<const int32*>(&Value)[i];
		}
	}

	inline void SetSafe(int Index, const TType& Value)
	{
		if (bIsValid && Index <= NumInstances)
		{
			for (int i = 0; i < FNiagaraDataSetAccessorTypeInfo<TType>::NumElements; ++i)
			{
				reinterpret_cast<int32*>(ComponentData[i])[Index] = reinterpret_cast<const int32*>(&Value)[i];
			}
		}
	}

private:
	bool bIsValid = false;
	int32 NumInstances = 0;
	uint8* ComponentData[FNiagaraDataSetAccessorTypeInfo<TType>::NumElements] = {};
};

//template<typename TType>
//struct FNiagaraDataSetWriterInt
//{
//};

template<typename TType>
struct FNiagaraDataSetAccessorInt32
{
	inline FNiagaraDataSetAccessorInt32() {}
	inline explicit FNiagaraDataSetAccessorInt32(const FNiagaraDataSet& DataSet, const FName VariableName) { Init(NiagaraDataSetPrivate::GetCompiledData(DataSet), VariableName); }
	inline explicit FNiagaraDataSetAccessorInt32(const FNiagaraDataBuffer* DataBuffer, const FName VariableName) { Init(NiagaraDataSetPrivate::GetCompiledData(DataBuffer), VariableName); }
	inline explicit FNiagaraDataSetAccessorInt32(const FNiagaraDataSetCompiledData& DataSetCompiledData, const FName VariableName) { Init(DataSetCompiledData, VariableName); }
	inline explicit FNiagaraDataSetAccessorInt32(const FNiagaraDataSetCompiledData* DataSetCompiledData, const FName VariableName) { Init(DataSetCompiledData, VariableName); }
	inline void Init(const FNiagaraDataSet& DataSet, const FName VariableName) { Init(NiagaraDataSetPrivate::GetCompiledData(DataSet), VariableName); }

	inline void Init(const FNiagaraDataSetCompiledData* DataSetCompiledData, const FName VariableName)
	{
		if (DataSetCompiledData != nullptr)
		{
			Init(*DataSetCompiledData, VariableName);
		}
		else
		{
			ComponentIndex = INDEX_NONE;
		}
	}

	void Init(const FNiagaraDataSetCompiledData& DataSetCompiledData, const FName VariableName)
	{
		ComponentIndex = INDEX_NONE;

		const int32 VariableIndex = DataSetCompiledData.Variables.IndexOfByPredicate(FNiagaraVariableMatch(FNiagaraDataSetAccessorTypeInfo<TType>::GetIntType(), VariableName));
		if (VariableIndex != INDEX_NONE)
		{
			ComponentIndex = DataSetCompiledData.VariableLayouts[VariableIndex].GetInt32ComponentStart();
		}
	}

	inline bool IsValid() const { return ComponentIndex != INDEX_NONE; }

	inline static FNiagaraDataSetReaderInt32<TType> CreateReader(const FNiagaraDataSet& DataSet, const FName VariableName) { return FNiagaraDataSetAccessorInt32<TType>(DataSet, VariableName).GetReader(DataSet); }
	inline static FNiagaraDataSetReaderInt32<TType> CreateReader(const FNiagaraDataBuffer* Data, const FName VariableName) { return FNiagaraDataSetAccessorInt32<TType>(Data, VariableName).GetReader(Data); }
	inline static FNiagaraDataSetWriterInt32<TType> CreateWriter(const FNiagaraDataSet& DataSet, const FName VariableName) { return FNiagaraDataSetAccessorInt32<TType>(DataSet, VariableName).GetWriter(DataSet); }

	inline FNiagaraDataSetReaderInt32<TType> GetReader(const FNiagaraDataSet& DataSet) const { return FNiagaraDataSetReaderInt32<TType>(NiagaraDataSetPrivate::GetCurrentData(DataSet), ComponentIndex); }
	inline FNiagaraDataSetReaderInt32<TType> GetReader(const FNiagaraDataBuffer* Data) const { return FNiagaraDataSetReaderInt32<TType>(Data, ComponentIndex); }
	inline FNiagaraDataSetWriterInt32<TType> GetWriter(const FNiagaraDataSet& DataSet) const { return FNiagaraDataSetWriterInt32<TType>(NiagaraDataSetPrivate::GetDestinationData(DataSet), ComponentIndex); }

private:
	int32 ComponentIndex = INDEX_NONE;
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<int32>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorInt32<int32>;

	static constexpr int32 NumElements = 1;
	static const FNiagaraTypeDefinition& GetIntType() { return FNiagaraTypeDefinition::GetIntDef(); }
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<FNiagaraBool>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorInt32<FNiagaraBool>;

	static constexpr int32 NumElements = 1;
	static const FNiagaraTypeDefinition& GetIntType() { return FNiagaraTypeDefinition::GetBoolDef(); }
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<ENiagaraExecutionState>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorInt32<ENiagaraExecutionState>;

	static constexpr int32 NumElements = 1;
	static FNiagaraTypeDefinition GetIntType() { return FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetExecutionStateEnum()); }
};

//////////////////////////////////////////////////////////////////////////
// Structure readers


template<typename TType>
struct FNiagaraDataSetReaderStruct
{
	inline FNiagaraDataSetReaderStruct() {}

	explicit FNiagaraDataSetReaderStruct(const FNiagaraDataBuffer* DataBuffer, int32 FloatComponentIndex, int32 Int32ComponentIndex)
	{
		bIsValid  = DataBuffer != nullptr;
		bIsValid &= (FNiagaraDataSetAccessorTypeInfo<TType>::NumFloatComponents == 0) || (FloatComponentIndex != INDEX_NONE);
		bIsValid &= (FNiagaraDataSetAccessorTypeInfo<TType>::NumInt32Components == 0) || (Int32ComponentIndex != INDEX_NONE);
		if (bIsValid)
		{
			FloatComponents = FNiagaraDataSetAccessorTypeInfo<TType>::NumFloatComponents > 0 ? reinterpret_cast<const float*>(NiagaraDataSetPrivate::GetComponentPtrFloat(DataBuffer, FloatComponentIndex)) : nullptr;
			Int32Components = FNiagaraDataSetAccessorTypeInfo<TType>::NumInt32Components > 0 ? reinterpret_cast<const int32*>(NiagaraDataSetPrivate::GetComponentPtrInt32(DataBuffer, Int32ComponentIndex)) : nullptr;
			NumInstances = NiagaraDataSetPrivate::GetNumInstances(DataBuffer);
			ComponentStride = NiagaraDataSetPrivate::GetComponentStride(DataBuffer);
		}
	}

	inline bool IsValid() const { return bIsValid; }

	inline TType operator[](int32 Index) const
	{
		return Get(Index);
	}

	inline TType Get(int Index) const
	{
		check(bIsValid && Index <= NumInstances);
		return FNiagaraDataSetAccessorTypeInfo<TType>::ReadStruct(Index, FloatComponents, Int32Components, ComponentStride);
	}

	inline TType GetSafe(int Index, const TType& Default) const
	{
		if (bIsValid && (Index < NumInstances))
		{
			return Get(Index);
		}
		return Default;
	}

	inline int32 GetNumInstances()const { return NumInstances; }

private:
	bool bIsValid = false;
	int32 NumInstances = 0;
	int32 ComponentStride = 0;
	const float* FloatComponents = nullptr;
	const int32* Int32Components = nullptr;
};

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(default:6294)
#endif

template<typename TType>
struct FNiagaraDataSetAccessorStruct
{
	inline FNiagaraDataSetAccessorStruct() {}
	inline explicit FNiagaraDataSetAccessorStruct(const FNiagaraDataSet& DataSet, const FName VariableName) { Init(NiagaraDataSetPrivate::GetCompiledData(DataSet), VariableName); }
	inline explicit FNiagaraDataSetAccessorStruct(const FNiagaraDataSetCompiledData& DataSetCompiledData, const FName VariableName) { Init(DataSetCompiledData, VariableName); }
	inline explicit FNiagaraDataSetAccessorStruct(const FNiagaraDataSetCompiledData* DataSetCompiledData, const FName VariableName) { Init(DataSetCompiledData, VariableName); }
	inline void Init(const FNiagaraDataSet& DataSet, const FName VariableName) { Init(NiagaraDataSetPrivate::GetCompiledData(DataSet), VariableName); }

	void Init(const FNiagaraDataSetCompiledData* DataSetCompiledData, const FName VariableName)
	{
		if (DataSetCompiledData != nullptr)
		{
			Init(*DataSetCompiledData, VariableName);
		}
		else
		{
			bIsValid = false;
			FloatComponentIndex = INDEX_NONE;
			Int32ComponentIndex = INDEX_NONE;
		}
	}

	void Init(const FNiagaraDataSetCompiledData& DataSetCompiledData, const FName VariableName)
	{
		bIsValid = false;

		const int32 VariableIndex = DataSetCompiledData.Variables.IndexOfByPredicate(FNiagaraVariableMatch(FNiagaraDataSetAccessorTypeInfo<TType>::GetStructType(), VariableName));
		if (VariableIndex != INDEX_NONE)
		{
			const FNiagaraVariableLayoutInfo& VariableLayout = DataSetCompiledData.VariableLayouts[VariableIndex];
			ensure(VariableLayout.GetNumHalfComponents() == 0);	// We don't support reading halfs for structs
			bIsValid = VariableLayout.GetNumHalfComponents() == 0;
			FloatComponentIndex = VariableLayout.GetFloatComponentStart();
			Int32ComponentIndex = VariableLayout.GetInt32ComponentStart();
		}
	}

	inline bool IsValid() const { return bIsValid; }

	inline static FNiagaraDataSetReaderStruct<TType> CreateReader(const FNiagaraDataSet& DataSet, const FName VariableName) { return FNiagaraDataSetAccessorStruct<TType>(DataSet, VariableName).GetReader(DataSet); }
	//FORCEINLINE static FNiagaraDataSetWriterStruct<TType> CreateWriter(const FNiagaraDataSet& DataSet, const FName VariableName) { return FNiagaraDataSetAccessorStruct<TType>(DataSet, VariableName).GetWriter(DataSet); }

	inline FNiagaraDataSetReaderStruct<TType> GetReader(const FNiagaraDataSet& DataSet) const { return FNiagaraDataSetReaderStruct<TType>(NiagaraDataSetPrivate::GetCurrentData(DataSet), FloatComponentIndex, Int32ComponentIndex); }
	//FNiagaraDataSetReaderStruct<TType> GetWriter(const FNiagaraDataSet& DataSet) { return FNiagaraDataSetReaderStruct<TType>(NiagaraDataSetPrivate::GetDestinationData(DataSet), FloatComponentIndex, Int32ComponentIndex); }

	inline FNiagaraDataSetReaderStruct<TType> GetReader(const FNiagaraDataBuffer* DataBuffer) const { return FNiagaraDataSetReaderStruct<TType>(DataBuffer, FloatComponentIndex, Int32ComponentIndex); }
private:
	bool bIsValid = false;
	int32 FloatComponentIndex = INDEX_NONE;
	int32 Int32ComponentIndex = INDEX_NONE;
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<FNiagaraSpawnInfo>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorStruct<FNiagaraSpawnInfo>;

	static constexpr int32 NumFloatComponents = 2;
	static constexpr int32 NumInt32Components = 2;
	static FNiagaraTypeDefinition GetStructType() { return FNiagaraTypeDefinition(FNiagaraSpawnInfo::StaticStruct()); }

	static FNiagaraSpawnInfo ReadStruct(int32 InstanceIndex, const float* FloatComponents, const int32* Int32Components, int32 ComponentStride)
	{
		FNiagaraSpawnInfo SpawnInfo;
		SpawnInfo.Count			= Int32Components[(ComponentStride * 0) + InstanceIndex];
		SpawnInfo.InterpStartDt	= FloatComponents[(ComponentStride * 0) + InstanceIndex];
		SpawnInfo.IntervalDt	= FloatComponents[(ComponentStride * 1) + InstanceIndex];
		SpawnInfo.SpawnGroup	= Int32Components[(ComponentStride * 1) + InstanceIndex];
		return SpawnInfo;
	}
};

template<>
struct FNiagaraDataSetAccessorTypeInfo<FNiagaraID>
{
	using TAccessorBaseClass = FNiagaraDataSetAccessorStruct<FNiagaraID>;

	static constexpr int32 NumFloatComponents = 0;
	static constexpr int32 NumInt32Components = 2;
	static FNiagaraTypeDefinition GetStructType() { return FNiagaraTypeDefinition(FNiagaraID::StaticStruct()); }

	static FNiagaraID ReadStruct(int32 InstanceIndex, const float* FloatComponents, const int32* Int32Components, int32 ComponentStride)
	{
		FNiagaraID NiagaraID;
		NiagaraID.Index			= Int32Components[(ComponentStride * 0) + InstanceIndex];
		NiagaraID.AcquireTag	= Int32Components[(ComponentStride * 1) + InstanceIndex];
		return NiagaraID;
	}
};
