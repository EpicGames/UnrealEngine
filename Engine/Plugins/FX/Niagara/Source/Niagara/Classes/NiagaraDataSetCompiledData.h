// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraDataSetCompiledData.generated.h"

/** Helper class defining the layout and location of an FNiagaraVariable in an FNiagaraDataBuffer-> */
USTRUCT()
struct FNiagaraVariableLayoutInfo
{
	GENERATED_BODY()

private:
	/** Start index for the float components in the main buffer. */
	UPROPERTY()
	uint16 FloatComponentStart = 0;

	/** Start index for the int32 components in the main buffer. */
	UPROPERTY()
	uint16 Int32ComponentStart = 0;

	/** Start index for the half components in the main buffer. */
	UPROPERTY()
	uint16 HalfComponentStart = 0;

public:
	uint32 GetFloatComponentStart() const { return FloatComponentStart; }
	uint32 GetInt32ComponentStart() const { return Int32ComponentStart; }
	uint32 GetHalfComponentStart() const { return HalfComponentStart; }

	void SetFloatComponentStart(uint32 Offset) { check(Offset <= TNumericLimits<uint16>::Max()); FloatComponentStart = static_cast<uint16>(Offset); }
	void SetInt32ComponentStart(uint32 Offset) { check(Offset <= TNumericLimits<uint16>::Max()); Int32ComponentStart = static_cast<uint16>(Offset); }
	void SetHalfComponentStart(uint32 Offset) { check(Offset <= TNumericLimits<uint16>::Max()); HalfComponentStart = static_cast<uint16>(Offset); }

	uint32 GetNumFloatComponents() const { return LayoutInfo.GetNumFloatComponents(); }
	uint32 GetNumInt32Components() const { return LayoutInfo.GetNumInt32Components(); }
	uint32 GetNumHalfComponents() const { return LayoutInfo.GetNumHalfComponents(); }

	/** This variable's type layout info. */
	UPROPERTY()
	FNiagaraTypeLayoutInfo LayoutInfo;
};

USTRUCT()
struct FNiagaraDataSetCompiledData
{
	GENERATED_BODY()

	/** Variables in the data set. */
	UPROPERTY()
	TArray<FNiagaraVariableBase> Variables;

	/** Data describing the layout of variable data. */
	UPROPERTY()
	TArray<FNiagaraVariableLayoutInfo> VariableLayouts;

	/** Unique ID for this DataSet. Used to allow referencing from other emitters and Systems. */
	UPROPERTY()
	FNiagaraDataSetID ID;

	/** Total number of components of each type in the data set. */
	UPROPERTY()
	uint32 TotalFloatComponents;

	UPROPERTY()
	uint32 TotalInt32Components;

	UPROPERTY()
	uint32 TotalHalfComponents;

	/** Whether or not this dataset require persistent IDs. */
	UPROPERTY()
	uint32 bRequiresPersistentIDs : 1;

	/** Sim target this DataSet is targeting (CPU/GPU). */
	UPROPERTY()
	ENiagaraSimTarget SimTarget;

	NIAGARA_API FNiagaraDataSetCompiledData();
	NIAGARA_API const FNiagaraVariableLayoutInfo* FindVariableLayoutInfo(const FNiagaraVariableBase& VariableDef) const;
	NIAGARA_API void BuildLayout();
	NIAGARA_API void Empty();

	static NIAGARA_API FNiagaraDataSetCompiledData DummyCompiledData;

	uint32 GetLayoutHash()const
	{
		if(CachedLayoutHash == INDEX_NONE)
		{
			CachedLayoutHash = 0;
			for(const FNiagaraVariableBase& Var : Variables)
			{
				CachedLayoutHash = HashCombine(CachedLayoutHash, GetTypeHash(Var));
			}
		}
		return CachedLayoutHash;
	}

	bool CheckHashConflict(const FNiagaraDataSetCompiledData& Other)const
	{
		//If we have the same hash ensure we have the same data.
		if(GetLayoutHash() == Other.GetLayoutHash())
		{
			return Variables != Other.Variables;			
		}
		return false;
	}

private:
	/** A hash of all variables in this layout. NOT persistent. Will differ from run to run. */
	mutable uint32 CachedLayoutHash = INDEX_NONE;

};

//////////////////////////////////////////////////////////////////////////

