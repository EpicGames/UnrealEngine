// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraTypes.h"
#include "NiagaraDataSet.h"

struct FNiagaraDataSetDebugAccessor
{
	bool Init(const FNiagaraDataSetCompiledData& CompiledData, FName InVariableName);

	float ReadFloat(const FNiagaraDataBuffer* DataBuffer, uint32 Instance, uint32 Component) const;
	float ReadHalf(const FNiagaraDataBuffer* DataBuffer, uint32 Instance, uint32 Component) const;
	int32 ReadInt(const FNiagaraDataBuffer* DataBuffer, uint32 Instance, uint32 Component) const;
	

	template<typename TString>
	void StringAppend(TString& StringType, const FNiagaraDataBuffer* DataBuffer, uint32 Instance) const
	{
		if (bIsEnum)
		{
			const int32 Value = ReadInt(DataBuffer, Instance, 0);
			StringType.Appendf(TEXT("%d (%s)"), Value, *NiagaraType.GetEnum()->GetDisplayNameTextByValue(Value).ToString());
		}
		else if (bIsBool)
		{
			const TCHAR* TrueText = TEXT("true");
			const TCHAR* FalseText = TEXT("false");
			StringType.Append(ReadInt(DataBuffer, Instance, 0) == FNiagaraBool::True ? TrueText : FalseText);
		}
		else
		{
			bool bNeedsComma = false;
			for (uint32 iComponent=0; iComponent < NumComponentsInt32; ++iComponent)
			{
				if (bNeedsComma)
				{
					StringType.Append(TEXT(", "));
				}
				bNeedsComma = true;
				StringType.Appendf(TEXT("%d"), ReadInt(DataBuffer, Instance, iComponent));
			}

			for (uint32 iComponent=0; iComponent < NumComponentsFloat; ++iComponent)
			{
				if (bNeedsComma)
				{
					StringType.Append(TEXT(", "));
				}
				bNeedsComma = true;
				StringType.Appendf(TEXT("%.2f"), ReadFloat(DataBuffer, Instance, iComponent));
			}

			for (uint32 iComponent=0; iComponent < NumComponentsHalf; ++iComponent)
			{
				if (bNeedsComma)
				{
					StringType.Append(TEXT(", "));
				}
				bNeedsComma = true;
				StringType.Appendf(TEXT("%.2f"), ReadHalf(DataBuffer, Instance, iComponent));
			}
		}
	}

	FName GetName() const { return VariableName; }
	bool IsEnum() const { return bIsEnum; }
	bool IsBool() const { return bIsBool; }

	static bool ValidateDataBuffer(const FNiagaraDataSetCompiledData& CompiledData, const FNiagaraDataBuffer* DataBuffer, uint32 iInstance, TFunction<void(const FNiagaraVariableBase&, int32)> ErrorCallback);
	static bool ValidateDataBuffer(const FNiagaraDataSetCompiledData& CompiledData, const FNiagaraDataBuffer* DataBuffer, TFunction<void(const FNiagaraVariableBase&, uint32, int32)> ErrorCallback);

private:
	FName					VariableName;
	FNiagaraTypeDefinition	NiagaraType;
	bool					bIsEnum = false;
	bool					bIsBool = false;
	uint32					NumComponentsInt32 = 0;
	uint32					NumComponentsFloat = 0;
	uint32					NumComponentsHalf = 0;
	uint32					ComponentIndexInt32 = INDEX_NONE;
	uint32					ComponentIndexFloat = INDEX_NONE;
	uint32					ComponentIndexHalf = INDEX_NONE;
};
