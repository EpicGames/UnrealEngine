// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InGamePerformanceTracker.h"
#include "RenderCommandFence.h"
#include "NiagaraPlatformSet.h"
#include "NiagaraPerfBaseline.h"
#include "NiagaraValidationRule.h"
#include "NiagaraValidationRuleSet.h"
#include "Particles/ParticlePerfStats.h"

#include "NiagaraScalabilitySettings.h"

#include "NiagaraEffectType.generated.h"

#define DEBUG_SCALABILITY_STATE (!UE_BUILD_SHIPPING)

#define WITH_PER_FXTYPE_PARTICLE_PERF_STATS (WITH_PER_SYSTEM_PARTICLE_PERF_STATS && WITH_PARTICLE_PERF_CSV_STATS && !UE_BUILD_SHIPPING)

/** Controls what action is taken by a Niagara system that fails it's cull checks. */
UENUM()
enum class ENiagaraCullReaction
{
	/** The system instance will be deactivated. Particles will be allowed to die naturally. It will not be reactivated automatically by the scalability system. */
	Deactivate UMETA(DisplayName = "Kill"),
	/** The system instance will be deactivated and particles killed immediately. It will not be reactivated automatically by the scalability system. */
	DeactivateImmediate UMETA(DisplayName = "Kill and Clear"),
	/** The system instance will be deactivated. Particles will be allowed to die naturally. Will reactivate when it passes cull tests again. */
	DeactivateResume UMETA(DisplayName = "Asleep"),
	/** The system instance will be deactivated and particles killed immediately. Will reactivate when it passes cull tests again. */
	DeactivateImmediateResume UMETA(DisplayName = "Asleep and Clear"),
	/** The system instance will be paused, maintaining it's current state, but will resume ticking when it passes cull tests again. */
	PauseResume UMETA(DisplayName = "Pause"),
};

/** Controls how often should we update scalability states for these effects. */
UENUM()
enum class ENiagaraScalabilityUpdateFrequency
{
	/** Scalability will be checked only on spawn. */
	SpawnOnly,
	/** Scalability will be checked every 1.0s (default for fx.NiagaraScalabilityUpdateTime_Low). */
	Low,
	/** Scalability will be checked every 0.5s (default for fx.NiagaraScalabilityUpdateTime_Medium). */
	Medium,
	/** Scalability will be checked every 0.25s (default for fx.NiagaraScalabilityUpdateTime_High). */
	High,
	/** Scalability will be checked every frame. */
	Continuous,
};

/** Controls how cull proxies should be handled for a system. */
UENUM()
enum class ENiagaraCullProxyMode : uint32
{
	/** No cull proxy replaces culled systems. */
	None,
	/** A single simulation is used but rendered in place of all culled systems. This saves on simulation cost but can still incur significant render thread cost. */
	Instanced_Rendered,
	
	/** TODO: Some method of instancing that bypasses the need to fully re-render the system in each location. */
	//Instanced_NoRender,
	/** TODO: When we have baked simulations we could us these as cull proxies. */
	//BakedSimulation,
	/** TODO: Possible to get very cheap proxies using an impostor billboard using a pre-generated flipbook of the system using the Baker tool. */
	//FlipbookImpostor,
};

/** 
Simple linear ramp to drive scaling values. 
TODO: Replace with Curve that can baked down for fast eval and has good inline UI widgets.
*/
USTRUCT()
struct FNiagaraLinearRamp
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Ramp")
	float StartX = 0.0f;
	
	UPROPERTY(EditAnywhere, Category = "Ramp")
	float StartY = 0.0f;
	
	UPROPERTY(EditAnywhere, Category = "Ramp")
	float EndX = 1.0f;
	
	UPROPERTY(EditAnywhere, Category = "Ramp")
	float EndY = 1.0f;

	inline float Evaluate(float X)const 
	{
		if (X <= StartX)
		{
			return StartY;
		}
		else if (X >= EndX)
		{
			return EndY;
		}

		float Alpha = (X - StartX) / (EndX - StartX);
		return FMath::Lerp(StartY, EndY, Alpha);
	}
	
	FNiagaraLinearRamp(float InStartX = 0.0f, float InStartY = 0.0f, float InEndX = 1.0f, float InEndY = 1.0f)
	: StartX(InStartX)
	, StartY(InStartY)
	, EndX(InEndX)
	, EndY(InEndY)
	{}
};

USTRUCT()
struct FNiagaraGlobalBudgetScaling
{
	GENERATED_USTRUCT_BODY()	
	
	FNiagaraGlobalBudgetScaling();

	/** Controls whether global budget based culling is enabled. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (InlineEditConditionToggle))
	uint32 bCullByGlobalBudget : 1;

	/** Controls whether we scale down the MaxDistance based on the global budget use. Allows us to increase aggression of culling when performance is poor. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (InlineEditConditionToggle))
	uint32 bScaleMaxDistanceByGlobalBudgetUse : 1;

	/** Controls whether we scale down the system instance counts by global budget usage. Allows us to increase aggression of culling when performance is poor. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (InlineEditConditionToggle))
	uint32 bScaleMaxInstanceCountByGlobalBudgetUse : 1;

	/** Controls whether we scale down the effect type instance counts by global budget usage. Allows us to increase aggression of culling when performance is poor. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (InlineEditConditionToggle))
	uint32 bScaleSystemInstanceCountByGlobalBudgetUse : 1;

	/** Effects will be culled if the global budget usage exceeds this fraction. A global budget usage of 1.0 means current global FX workload has reached it's max budget. Budgets are set by CVars under FX.Budget... */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition = "bCullByGlobalBudget", DisplayInScalabilityValuesBar))
	float MaxGlobalBudgetUsage;

	/** When enabled, MaxDistance is scaled down by the global budget use based on this curve. Allows us to cull more aggressively when performance is poor.	*/
	UPROPERTY(EditAnywhere, Category = "Budget Scaling", meta = (EditCondition = "bScaleMaxDistanceByGlobalBudgetUse", DisplayInScalabilityValuesBar))
	FNiagaraLinearRamp MaxDistanceScaleByGlobalBudgetUse;
	
	/** When enabled, Max Effect Type Instances is scaled down by the global budget use based on this curve. Allows us to cull more aggressively when performance is poor.	*/
	UPROPERTY(EditAnywhere, Category = "Budget Scaling", meta = (EditCondition = "bScaleMaxInstanceCountByGlobalBudgetUse", DisplayInScalabilityValuesBar))
	FNiagaraLinearRamp MaxInstanceCountScaleByGlobalBudgetUse;
	
	/** When enabled, Max System Instances is scaled down by the global budget use based on this curve. Allows us to cull more aggressively when performance is poor.	*/
	UPROPERTY(EditAnywhere, Category = "Budget Scaling", meta = (EditCondition = "bScaleSystemInstanceCountByGlobalBudgetUse", DisplayInScalabilityValuesBar))
	FNiagaraLinearRamp MaxSystemInstanceCountScaleByGlobalBudgetUse;
};

/** Scalability settings for Niagara Systems for a particular platform set (unless overridden). */
USTRUCT()
struct FNiagaraSystemVisibilityCullingSettings
{
	GENERATED_USTRUCT_BODY()

	FNiagaraSystemVisibilityCullingSettings();

	/** Controls whether we cull when not rendered. This includes all reasons for being not rendered such as view frustum, occlusion and hidden flags etc. As this requires feedback form the rendering system, it cannot be used for pre-culling. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (InlineEditConditionToggle))
	uint32 bCullWhenNotRendered : 1;

	/**
	Controls whether view frustum culling is enabled. Niagara can do it's own separate view frustum culling to give coarse visibility culling that does not depend directly on rendering like CullWhenNotRendered.
	This requires fixed bounds to be used in pre-culling.
	*/
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (InlineEditConditionToggle))
	uint32 bCullByViewFrustum : 1;

	/** If true and bCullByViewFrustum is enabled, we allow view frustum checks in pre-culling. Meaning we can cull off screen systems before they even spawn. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition = "bCullByViewFrustum"))
	uint32 bAllowPreCullingByViewFrustum : 1;

	/** Effects will be culled if they go longer than this time outside the view frustum. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition = "bCullByViewFrustum", DisplayInScalabilityValuesBar))
	float MaxTimeOutsideViewFrustum;

	/** Effects will be culled if they go longer than this time without being rendered. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition = "bCullWhenNotRendered", DisplayInScalabilityValuesBar))
	float MaxTimeWithoutRender;
};

//////////////////////////////////////////////////////////////////////////

/** Scalability settings for Niagara Systems for a particular platform set (unless overridden). */
USTRUCT()
struct FNiagaraSystemScalabilitySettings
{
	GENERATED_USTRUCT_BODY()

	/** The platforms on which these settings are active (unless overridden). */
	UPROPERTY(EditAnywhere, Category = "Scalability")
	FNiagaraPlatformSet Platforms;

	/** Controls whether distance culling is enabled. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (InlineEditConditionToggle))
	uint32 bCullByDistance : 1;
	/** Controls whether we should cull systems based on how many instances with the same Effect Type are active. */
	UPROPERTY(EditAnywhere, Category = "Scalability", DisplayName = "Cull By Effect Type Instance Count", meta = (InlineEditConditionToggle))
	uint32 bCullMaxInstanceCount : 1;
	/** Controls whether we should cull systems based on how many instances of the system are active. */
	UPROPERTY(EditAnywhere, Category = "Scalability", DisplayName = "Cull By System Instance Count", meta = (InlineEditConditionToggle))
	uint32 bCullPerSystemMaxInstanceCount : 1;

	/** Effects of this type are culled beyond this distance. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition = "bCullByDistance", DisplayInScalabilityValuesBar))
	float MaxDistance;

	/** Deprecated visibility culling property. Moved into FNiagaraSystemVisibilityCullingSettings. */
	UPROPERTY()
	uint32 bCullByMaxTimeWithoutRender_DEPRECATED : 1;

	/** 
	Effects of this type will be culled when total active instances using this same EffectType exceeds this number. 
	If the effect type has a significance handler, instances are sorted by their significance and only the N most significant will be kept. The rest are culled.
	If it does not have a significance handler, instance count culling will be applied at spawn time only. New FX that would exceed the counts are not spawned/activated.
	*/
	UPROPERTY(EditAnywhere, Category = "Scalability", DisplayName = "Max Effect Type Instances", meta = (EditCondition = "bCullMaxInstanceCount", DisplayInScalabilityValuesBar))
	int32 MaxInstances;

	/**
	Effects of this type will be culled when total active instances of the same NiagaraSystem exceeds this number. 
	If the effect type has a significance handler, instances are sorted by their significance and only the N most significant will be kept. The rest are culled.
	If it does not have a significance handler, instance count culling will be applied at spawn time only. New FX that would exceed the counts are not spawned/activated.
	*/
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition = "bCullPerSystemMaxInstanceCount", DisplayInScalabilityValuesBar))
	int32 MaxSystemInstances;

	/** Depreceated visibiltiy culling property. Moved into FNiagaraSystemVisibilityCullingSettings. */
	UPROPERTY()
	float MaxTimeWithoutRender_DEPRECATED;

	/** Controls what, if any, proxy will be used in place of culled systems.  */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayInScalabilityValuesBar))
	ENiagaraCullProxyMode CullProxyMode = ENiagaraCullProxyMode::None;

	/** 
	Limit on the number of proxies that can be used at once per system.
	While much cheaper than full FX instances, proxies still incur some cost so must have a limit.
	When significance information is available using a significance handler, the most significance proxies will be kept up to this limit.
	*/
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition = "CullProxyMode != ENiagaraCullProxyMode::None", DisplayInScalabilityValuesBar))
	int32 MaxSystemProxies = 32;
	
	/** Settings controlling how systems are culled by visibility. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayInScalabilityValuesBar))
	FNiagaraSystemVisibilityCullingSettings VisibilityCulling;

	/** Settings related to scaling down FX based on the current budget usage. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayInScalabilityValuesBar))
	FNiagaraGlobalBudgetScaling BudgetScaling;
	
	NIAGARA_API FNiagaraSystemScalabilitySettings();

	NIAGARA_API void Clear();
	NIAGARA_API void PostLoad(int32 Version);
};

/** Container struct for an array of system scalability settings. Enables details customization and data validation. */
USTRUCT()
struct FNiagaraSystemScalabilitySettingsArray
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Scalability")
	TArray<FNiagaraSystemScalabilitySettings> Settings;
};

USTRUCT()
struct FNiagaraSystemScalabilityOverride : public FNiagaraSystemScalabilitySettings
{
	GENERATED_USTRUCT_BODY()

	NIAGARA_API FNiagaraSystemScalabilityOverride();

	/** Controls whether we override the distance culling settings. */
	UPROPERTY(EditAnywhere, Category = "Override")
	uint32 bOverrideDistanceSettings : 1;
	/** Controls whether we override the instance count culling settings. */
	UPROPERTY(EditAnywhere, Category = "Override", DisplayName = "Override Effect Type Instance Count Settings")
	uint32 bOverrideInstanceCountSettings : 1;
	/** Controls whether we override the per system instance count culling settings. */
	UPROPERTY(EditAnywhere, Category = "Override", DisplayName = "Override System Instance Count Settings")
	uint32 bOverridePerSystemInstanceCountSettings : 1;
	/** Controls whether we override the visibility culling settings. */
	UPROPERTY(EditAnywhere, Category = "Override")
	uint32 bOverrideVisibilitySettings : 1;
	/** Controls whether we override the global budget scaling settings. */
	UPROPERTY(EditAnywhere, Category = "Override")
	uint32 bOverrideGlobalBudgetScalingSettings : 1;
	/** Controls whether we override the cull proxy settings. */
	UPROPERTY(EditAnywhere, Category = "Override")
	uint32 bOverrideCullProxySettings : 1;
};

/** Scalability settings for Niagara Emitters on a particular platform set. */
USTRUCT()
struct FNiagaraEmitterScalabilitySettings
{
	GENERATED_USTRUCT_BODY()

	/** The platforms on which these settings are active (unless overridden). */
	UPROPERTY(EditAnywhere, Category = "Scalability")
	FNiagaraPlatformSet Platforms;

	/** Enable spawn count scaling */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (InlineEditConditionToggle))
	uint32 bScaleSpawnCount : 1;

	/** Scale factor applied to spawn counts for this emitter. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition = "bScaleSpawnCount", DisplayInScalabilityValuesBar))
	float SpawnCountScale;

	NIAGARA_API FNiagaraEmitterScalabilitySettings();
	NIAGARA_API void Clear();
};

/** Container struct for an array of emitter scalability settings. Enables details customization and data validation. */
USTRUCT()
struct FNiagaraEmitterScalabilitySettingsArray
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Scalability")
	TArray<FNiagaraEmitterScalabilitySettings> Settings;
};

USTRUCT()
struct FNiagaraEmitterScalabilityOverride : public FNiagaraEmitterScalabilitySettings
{
	GENERATED_USTRUCT_BODY()

	NIAGARA_API FNiagaraEmitterScalabilityOverride();

	//Controls whether spawn count scale should be overridden.
	UPROPERTY(EditAnywhere, Category = "Override")
	uint32 bOverrideSpawnCountScale : 1;
};

/** Container struct for an array of emitter scalability overrides. Enables details customization and data validation. */
USTRUCT()
struct FNiagaraEmitterScalabilityOverrides
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Override")
	TArray<FNiagaraEmitterScalabilityOverride> Overrides;
};

//////////////////////////////////////////////////////////////////////////

class UNiagaraComponent;
struct FNiagaraScalabilityState;
struct FNiagaraScalabilitySystemData;

/**
Base class for significance handlers. 
These allow Niagara's scalability system to determine the relative significance of different FX in the scene.
Some basic ones are provided but projects are free to implement their own more complex determinations of significance.
For example, FX attached to the player character could be given higher priority.
*/
UCLASS(abstract, EditInlineNew, MinimalAPI)
class UNiagaraSignificanceHandler : public UObject
{
	GENERATED_BODY()

public:
	virtual void CalculateSignificance(TConstArrayView<UNiagaraComponent*> Components, TArrayView<FNiagaraScalabilityState> OutState, TConstArrayView<FNiagaraScalabilitySystemData> SystemData, TArray<int32>& OutIndices)PURE_VIRTUAL(CalculateSignificance, );
};

/** Significance is determined by the system's distance to the nearest camera. Closer systems are more significant. */
UCLASS(EditInlineNew, meta = (DisplayName = "Distance"), MinimalAPI)
class UNiagaraSignificanceHandlerDistance : public UNiagaraSignificanceHandler
{
	GENERATED_BODY()

public:
	NIAGARA_API virtual void CalculateSignificance(TConstArrayView<UNiagaraComponent*> Components, TArrayView<FNiagaraScalabilityState> OutState, TConstArrayView<FNiagaraScalabilitySystemData> SystemData, TArray<int32>& OutIndices) override;
};

/** Significance is determined by the system's age. Newer systems are more significant. */
UCLASS(EditInlineNew, meta = (DisplayName = "Age"), MinimalAPI)
class UNiagaraSignificanceHandlerAge : public UNiagaraSignificanceHandler
{
	GENERATED_BODY()

public:
	NIAGARA_API virtual void CalculateSignificance(TConstArrayView<UNiagaraComponent*> Components, TArrayView<FNiagaraScalabilityState> OutState, TConstArrayView<FNiagaraScalabilitySystemData> SystemData, TArray<int32>& OutIndices) override;
};

//////////////////////////////////////////////////////////////////////////

/** Contains settings and working data shared among many Niagara systems that share some commonality of type, for example ImpactFX vs EnvironmentalFX.
 *  Main usage of effect types is to control scalability settings for a group of effects, setting visibility and cull reactions on a per-platform basis.
 *
 *  Effect types can also be used for validation, checking that the content passes all the configured validation rule sets.
 *
 *  The effect type is set in Niagara systems in the system properties.
 */
UCLASS(config = Niagara, perObjectConfig, MinimalAPI)
class UNiagaraEffectType : public UObject
{
	GENERATED_UCLASS_BODY()

	//UObject Interface
	NIAGARA_API virtual void BeginDestroy()override;
	NIAGARA_API virtual bool IsReadyForFinishDestroy()override;
	NIAGARA_API virtual void Serialize(FArchive& Ar)override;
	NIAGARA_API virtual void PostLoad()override;
#if WITH_EDITORONLY_DATA
	static NIAGARA_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
#endif
	//UObject Interface END

	/** Controls whether or not culling is allowed for FX that are owned by, attached to or instigated by a locally controlled pawn. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta=(DisplayInSystemScalability, ScalabilityBarDisplayName="Local Player Culling"))
	bool bAllowCullingForLocalPlayers = false;

	/** How regularly effects of this type are checked for scalability. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta=(DisplayInSystemScalability, ScalabilityBarDisplayName="Frequency"))
	ENiagaraScalabilityUpdateFrequency UpdateFrequency;

	/**
	How effects of this type react when they fail the cull checks.
	Applied to all effects using this effect type and can not be overridden per effect.
	*/
	UPROPERTY(EditAnywhere, Category = "Scalability", meta=(DisplayInSystemScalability, ScalabilityBarDisplayName="Reaction"))
	ENiagaraCullReaction CullReaction;

	/** Used to determine the relative significance of FX in the scene which is used in other scalability systems such as instance count culling. */
	UPROPERTY(EditAnywhere, Instanced, Category = "Scalability", meta=(DisplayInSystemScalability, DisplayClassDisplayName, ScalabilityBarDisplayName="Significance"))
	TObjectPtr<UNiagaraSignificanceHandler> SignificanceHandler;

	/** Cull settings to use at each detail level. */
	UPROPERTY()
	TArray<FNiagaraSystemScalabilitySettings> DetailLevelScalabilitySettings_DEPRECATED;

	UPROPERTY(EditAnywhere, Category = "Scalability")
	FNiagaraSystemScalabilitySettingsArray SystemScalabilitySettings;
	
	UPROPERTY(EditAnywhere, Category = "Scalability")
	FNiagaraEmitterScalabilitySettingsArray EmitterScalabilitySettings;
	
	/** Readiness Settings used by systems using this Effect Type. If unset, the defaults in the project settings will be used. */
	UPROPERTY(EditAnywhere, Category = "Scalability")
	TObjectPtr<UNiagaraSystemReadinessSettings> ReadinessSettings;

	/** How urgent it is that System's using this Effect Type are ready in time for activation and rendering. Settings controlling readiness and possible delays in rendering can be found in ReadinessSettings. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayInScalabilityValuesBar, Categories = "Niagara.ScalabilityProfile"))
	FGameplayTag ReadinessProfile = TAG_Niagara_ScalabilityProfile_Medium;

#if WITH_EDITORONLY_DATA
	/** A set of rules to apply when checking content. To create your own rules, write a custom class that extends UNiagaraValidationRule. */
	UPROPERTY(EditAnywhere, Category = "Validation", Instanced)
	TArray<TObjectPtr<UNiagaraValidationRule>> ValidationRules;

	/** Array of reusable rule sets to apply when checking content. To create your own rules, write a custom class that extends UNiagaraValidationRule. */
	UPROPERTY(EditAnywhere, Category = "Validation")
	TArray<TObjectPtr<UNiagaraValidationRuleSet>> ValidationRuleSets;
#endif

	inline const FNiagaraSystemScalabilitySettingsArray& GetSystemScalabilitySettings()const { return SystemScalabilitySettings; }
	inline const FNiagaraEmitterScalabilitySettingsArray& GetEmitterScalabilitySettings()const { return EmitterScalabilitySettings; }

	NIAGARA_API const FNiagaraSystemScalabilitySettings& GetActiveSystemScalabilitySettings()const;
	NIAGARA_API const FNiagaraEmitterScalabilitySettings& GetActiveEmitterScalabilitySettings()const;

	UNiagaraSignificanceHandler* GetSignificanceHandler()const { return SignificanceHandler; }

	/** Total number of instances across all systems for this effect type. */
	int32 NumInstances;

	/** Marks that there have been new systems added for this effect type since it's last scalability manager update. Will force a manager update. */
	uint32 bNewSystemsSinceLastScalabilityUpdate : 1;

#if NIAGARA_PERF_BASELINES
	UNiagaraBaselineController* GetPerfBaselineController() { return PerformanceBaselineController; }
	FNiagaraPerfBaselineStats& GetPerfBaselineStats()const { return PerfBaselineStats; }
	inline bool IsPerfBaselineValid()const { return PerfBaselineVersion == CurrentPerfBaselineVersion; }
	NIAGARA_API void UpdatePerfBaselineStats(FNiagaraPerfBaselineStats& NewBaselineStats);

	NIAGARA_API void InvalidatePerfBaseline();

	NIAGARA_API void SpawnBaselineActor(UWorld* World);
#endif
	/** Performs the passed action for all FNiagaraPlatformSets in this system. */
	template<typename TAction>
	void ForEachPlatformSet(TAction Func);

	const FNiagaraReadinessSettingsProfile* FindReadinessSettingsProfile(FGameplayTag Profile = FGameplayTag()) const;
private:

	/** Controls generation of performance baseline data for this effect type. */
	UPROPERTY(EditAnywhere, Category="Performance", Instanced)
	TObjectPtr<UNiagaraBaselineController> PerformanceBaselineController;

	/**
	Performance data gathered from the Baseline System. 
	These give artists a good idea of the perf to aim for in their own FX.
	*/
	UPROPERTY(config)
	mutable FNiagaraPerfBaselineStats PerfBaselineStats;

	//Version guid at the time these baseline stats were generated.
	//Allows us to invalidate perf baseline results if there are significant performance optimizations
	UPROPERTY(config)
	FGuid PerfBaselineVersion;

#if NIAGARA_PERF_BASELINES
	/** The current version for perf baselines. Regenerate this if there are significant performance improvements that would invalidate existing baseline data. */
	static NIAGARA_API const FGuid CurrentPerfBaselineVersion;

	DECLARE_DELEGATE_OneParam(FGeneratePerfBaselines, TArray<UNiagaraEffectType*>&/**BaselinesToGenerate*/);

	/** Delegate allowing us to call into editor code to generate performance baselines. */
	static NIAGARA_API FGeneratePerfBaselines GeneratePerfBaselinesDelegate;
public:
	static FGeneratePerfBaselines& OnGeneratePerfBaselines(){ return GeneratePerfBaselinesDelegate; }
	static NIAGARA_API void GeneratePerfBaselines();
#endif

#if WITH_PER_FXTYPE_PARTICLE_PERF_STATS
	void PopulatePerfStatNames();

	//Cached CSV Stat names for this system.
	FName CSVStat_Count = NAME_None;
	FName CSVStat_Total = NAME_None;
	FName CSVStat_GTOnly = NAME_None;
	FName CSVStat_RT = NAME_None;
	FName CSVStat_GPU = NAME_None;
#endif
};

/** Performs the passed action for all FNiagaraPlatformSets in this FXType. */
template<typename TAction>
void UNiagaraEffectType::ForEachPlatformSet(TAction Func)
{
	for (FNiagaraSystemScalabilitySettings& Setting : SystemScalabilitySettings.Settings)
	{
		Func(Setting.Platforms);
	}

	for (FNiagaraEmitterScalabilitySettings& Setting : EmitterScalabilitySettings.Settings)
	{
		Func(Setting.Platforms);
	}
}

//////////////////////////////////////////////////////////////////////////

#if WITH_PER_FXTYPE_PARTICLE_PERF_STATS
/** Listener to gather accumulated stats for all systems in a particular Effect Type. */
class FParticlePerfStatsListener_EffectType : public FParticlePerfStatsListener
{
public:
	FParticlePerfStatsListener_EffectType()
	{}

	virtual ~FParticlePerfStatsListener_EffectType() = default;

	virtual bool Tick() override;
	virtual void TickRT() override;


	virtual bool NeedsWorldStats() const { return false; }
	virtual bool NeedsSystemStats() const { return true; }
	virtual bool NeedsComponentStats() const { return false; }
};
#endif//WITH_PER_FXTYPE_PARTICLE_PERF_STATS
