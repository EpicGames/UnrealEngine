// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "NiagaraScript.h"
#include "NiagaraMessageDataBase.h"
#include "NiagaraMessageStore.h"
#include "INiagaraMergeManager.h"
#include "NiagaraEffectType.h"
#include "NiagaraEmitterBase.h"
#include "NiagaraDataInterfaceEmitterBinding.h"
#include "NiagaraDataSetAccessor.h"
#include "NiagaraBoundsCalculator.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraParameterDefinitionsBase.h"
#include "NiagaraParameterDefinitionsSubscriber.h"
#include "NiagaraScratchPadContainer.h"
#include "NiagaraSimStageExecutionData.h"
#include "NiagaraDataInterfacePlatformSet.h"

#include "NiagaraEmitter.generated.h"

struct FNiagaraAssetTagDefinitionReference;
class UNiagaraScratchPadContainer;
class UMaterial;
class UNiagaraEmitter;
class UNiagaraEventReceiverEmitterAction;
class UNiagaraSimulationStageBase;
class UNiagaraEditorDataBase;
class UNiagaraSystem;
struct FNiagaraScriptRuntimeCompiledData;

//TODO: Event action that spawns other whole Systems?
//One that calls a BP exposed delegate?

USTRUCT()
struct FNiagaraEventReceiverProperties
{
	GENERATED_BODY()

	FNiagaraEventReceiverProperties()
	: Name(NAME_None)
	, SourceEventGenerator(NAME_None)
	, SourceEmitter(NAME_None)
	{

	}

	FNiagaraEventReceiverProperties(FName InName, FName InEventGenerator, FName InSourceEmitter)
		: Name(InName)
		, SourceEventGenerator(InEventGenerator)
		, SourceEmitter(InSourceEmitter)
	{

	}

	/** The name of this receiver. */
	UPROPERTY(EditAnywhere, Category = "Event Receiver")
	FName Name;

	/** The name of the EventGenerator to bind to. */
	UPROPERTY(EditAnywhere, Category = "Event Receiver")
	FName SourceEventGenerator;

	/** The name of the emitter from which the Event Generator is taken. */
	UPROPERTY(EditAnywhere, Category = "Event Receiver")
	FName SourceEmitter;

	//UPROPERTY(EditAnywhere, Category = "Event Receiver")
	//TArray<UNiagaraEventReceiverEmitterAction*> EmitterActions;
};

USTRUCT()
struct FNiagaraEventGeneratorProperties
{
	GENERATED_BODY()

	FNiagaraEventGeneratorProperties() = default;

	FNiagaraEventGeneratorProperties(FNiagaraDataSetProperties &Props, FName InEventGenerator)
		: ID(Props.ID.Name)
	{
		DataSetCompiledData.Variables = Props.Variables;
		DataSetCompiledData.ID = Props.ID;
		DataSetCompiledData.SimTarget = ENiagaraSimTarget::CPUSim;
		DataSetCompiledData.BuildLayout();
	}

	/** Max Number of Events that can be generated per frame. */
	UPROPERTY(EditAnywhere, Category = "Event Receiver")
	int32 MaxEventsPerFrame = 64; //TODO - More complex allocation so that we can grow dynamically if more space is needed ?

	UPROPERTY()
	FName ID;

	UPROPERTY()
	FNiagaraDataSetCompiledData DataSetCompiledData;
};


UENUM()
enum class EScriptExecutionMode : uint8
{
	/** The event script is run on every existing particle in the emitter.*/
	EveryParticle = 0,
	/** The event script is run only on the particles that were spawned in response to the current event in the emitter.*/
	SpawnedParticles,
	/** The event script is run only on the particle whose int32 ParticleIndex is specified in the event payload.*/
	SingleParticle UMETA(Hidden)
};

UENUM()
enum class EParticleAllocationMode : uint8
{
	/** This mode tries to estimate the max particle count at runtime by using previous simulations as reference.*/
	AutomaticEstimate = 0,
	/** This mode is useful if the particle count can vary wildly at runtime (e.g. due to user parameters) and a lot of reallocations happen.*/
	ManualEstimate,
	/** This mode defines an upper limit on the number of particles that will be simulated.  Useful for rejection sampling where we expect many spawned particles to get killed. */
	FixedCount
};

UENUM()
enum class ENiagaraEmitterCalculateBoundMode : uint8
{
	/** Bounds are calculated per frame (Only available for CPU emitters). */
	Dynamic,
	/** Bounds are set from the emitter's fixed bounds. */
	Fixed,
	/** Bounds will be set from the script using the Emitter Properties Data Interface, or blueprint.  If not set from either source the emitter has no bounds. */
	Programmable
};

USTRUCT()
struct FNiagaraEmitterScriptProperties
{
	FNiagaraEmitterScriptProperties() : Script(nullptr)
	{

	}

	GENERATED_BODY()
	
	UPROPERTY()
	TObjectPtr<UNiagaraScript> Script;

	UPROPERTY()
	TArray<FNiagaraEventReceiverProperties> EventReceivers;

	UPROPERTY()
	TArray<FNiagaraEventGeneratorProperties> EventGenerators;

	NIAGARA_API void InitDataSetAccess();
};

USTRUCT()
struct FNiagaraEventScriptProperties : public FNiagaraEmitterScriptProperties
{
	GENERATED_BODY()
			
	FNiagaraEventScriptProperties() : FNiagaraEmitterScriptProperties()
	{
		ExecutionMode = EScriptExecutionMode::EveryParticle;
		SpawnNumber = 0;
		MaxEventsPerFrame = 0;
		bRandomSpawnNumber = false;
		MinSpawnNumber = 0;
		UpdateAttributeInitialValues = true;
	}
	
	/** Controls which particles have the event script run on them.*/
	UPROPERTY(EditAnywhere, Category="Event Handler Options")
	EScriptExecutionMode ExecutionMode;

	/** Controls whether or not particles are spawned as a result of handling the event. Only valid for EScriptExecutionMode::SpawnedParticles. If Random Spawn Number is used, this will act as the maximum spawn range. */
	UPROPERTY(EditAnywhere, Category="Event Handler Options")
	uint32 SpawnNumber;

	/** Controls how many events are consumed by this event handler. If there are more events generated than this value, they will be ignored.*/
	UPROPERTY(EditAnywhere, Category="Event Handler Options")
	uint32 MaxEventsPerFrame;

	/** Id of the Emitter Handle that generated the event. If all zeroes, the event generator is assumed to be this emitter.*/
	UPROPERTY(EditAnywhere, Category="Event Handler Options")
	FGuid SourceEmitterID;

	/** The name of the event generated. This will be "Collision" for collision events and the Event Name field on the DataSetWrite node in the module graph for others.*/
	UPROPERTY(EditAnywhere, Category="Event Handler Options")
	FName SourceEventName;

	/** Whether using a random spawn number. */
	UPROPERTY(EditAnywhere, Category = "Event Handler Options")
	bool bRandomSpawnNumber;

	/** The minimum spawn number when random spawn is used. Spawn Number is used as the maximum range. */
	UPROPERTY(EditAnywhere, Category = "Event Handler Options")
	uint32 MinSpawnNumber;
	
	/** Should Event Spawn Scripts modify the Initial values for particle attributes they modify. */
	UPROPERTY(EditAnywhere, Category = "Event Handler Options", meta = (EditCondition="ExecutionMode==EScriptExecutionMode::SpawnedParticles"))
	bool UpdateAttributeInitialValues;
};

/** Legacy struct for spawn count scale overrides. This is now done in FNiagaraEmitterScalabilityOverrides*/
USTRUCT()
struct FNiagaraDetailsLevelScaleOverrides 
{
	GENERATED_BODY()

	FNiagaraDetailsLevelScaleOverrides();
	UPROPERTY()
	float Low;
	UPROPERTY()
	float Medium;
	UPROPERTY()
	float High;
	UPROPERTY()
	float Epic;
	UPROPERTY()
	float Cine;
};

struct FMemoryRuntimeEstimation
{
	TMap<uint64, int32> RuntimeAllocations;
	UE_DEPRECATED(5.8, "Direct use of IsEstimationDirty is deprecated please use the get / set methods")
	std::atomic<bool> IsEstimationDirty = false;
	int32 AllocationEstimate = 0;

	FMemoryRuntimeEstimation();
	FMemoryRuntimeEstimation(const FMemoryRuntimeEstimation& Other) { *this = Other; }

	FMemoryRuntimeEstimation& operator=(const FMemoryRuntimeEstimation& Other)
	{
		EstimationCriticalSection = Other.EstimationCriticalSection;
		RuntimeAllocations = Other.RuntimeAllocations;
		SetIsEstimationDirty(Other.GetIsEstimationDirty());
		AllocationEstimate = Other.AllocationEstimate;
		return *this;
	}

	void SetIsEstimationDirty(bool bIsDirty)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		IsEstimationDirty.store(bIsDirty, std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool GetIsEstimationDirty() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return IsEstimationDirty.load(std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	NIAGARA_API FCriticalSection* GetCriticalSection();

private:
	TSharedPtr<FCriticalSection> EstimationCriticalSection;
};

/** Struct containing all of the data that can be different between different emitter versions.*/
USTRUCT(BlueprintInternalUseOnly)
struct FVersionedNiagaraEmitterData
{
	GENERATED_BODY()

	NIAGARA_API FVersionedNiagaraEmitterData();
	
	UPROPERTY()
	FNiagaraAssetVersion Version;

#if WITH_EDITORONLY_DATA
	/** What changed in this version compared to the last? Displayed to the user when upgrading to a new script version. */
	UPROPERTY()
	FText VersionChangeDescription;

	/** Reference to a python script that is executed when the user updates from a previous version to this version. */
	UPROPERTY()
	ENiagaraPythonUpdateScriptReference UpdateScriptExecution = ENiagaraPythonUpdateScriptReference::None;

	/** Python script to run when updating to this script version. */
	UPROPERTY()
	FString PythonUpdateScript;

	/** Asset reference to a python script to run when updating to this script version. */
	UPROPERTY()
	FFilePath ScriptAsset;

	UPROPERTY()
	TSubclassOf<UNiagaraVersionUpgradeBase> UpdateUtilityClass;
#endif //WITH_EDITORONLY_DATA

	/* If this emitter version is no longer meant to be used, this option should be set.*/
	UPROPERTY()
	bool bDeprecated = false;

	/* Message to display when the script is deprecated. */
	UPROPERTY()
	FText DeprecationMessage;

	/** Toggles whether or not the particles within this emitter are relative to the emitter origin or in global space. */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Emitter")
	bool bLocalSpace = false;

	/** Toggles whether to globally make the random number generator be deterministic or non-deterministic. Any random calculation that is set to the emitter defaults will inherit this value. It is still possible to tweak individual random to be deterministic or not. In this case deterministic means that it will return the same results for the same configuration of the emitter as long as delta time is not variable. Any changes to the emitter's individual scripts will adjust the results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Emitter")
	bool bDeterminism = false;

	/** An emitter-based seed for the deterministic random number generator. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Emitter", meta = (EditCondition = "bDeterminism", EditConditionHides="true"))
	int32 RandomSeed = 0;

	/** This defines if newly spawned particles run only the spawn script on the first frame or both spawn and update with optional parameter interpolation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Emitter", meta = (SegmentedDisplay))
	ENiagaraInterpolatedSpawnMode InterpolatedSpawnMode;

#if WITH_EDITORONLY_DATA

	UPROPERTY(meta = (DeprecatedProperty))
	uint32 bInterpolatedSpawning_DEPRECATED : 1;
	
	/**
	GPU scripts were incorrectly running both particle spawn & update, CPU only runs spawn when interpolated spawning is disabled.
	This flag allows backwards compatability so content does not change post the fix.
	*/
	UPROPERTY()
	uint32 bGpuAlwaysRunParticleUpdateScript : 1;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Emitter", meta = (SegmentedDisplay))
	ENiagaraSimTarget SimTarget = ENiagaraSimTarget::CPUSim;
	
	// Determines how important this emitter is for the runtime and rendering of the system.
	// An emitter with low priority does not block system execution or other emitters from rendering if its own materials are not yet done compiling.
	// A critical emitter can block both activation and rendering of the system until it's ready.
	// See also: readiness settings in the effect type.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Emitter")
	ENiagaraEmitterImportance Importance = ENiagaraEmitterImportance::Normal;

	/**
	How should we calculate bounds for the emitter.
	Note: If this is greyed out it means fixed bounds are enabled in the System Properties and these bounds are therefore ignored.
	*/
	UPROPERTY(EditAnywhere, Category = "Emitter", meta = (SegmentedDisplay))
	ENiagaraEmitterCalculateBoundMode CalculateBoundsMode = ENiagaraEmitterCalculateBoundMode::Dynamic;
	
	/**
	The fixed bounding box value. CalculateBoundsMode is the condition whether the fixed bounds can be edited.
	Note: If this is greyed out it means fixed bounds are enabled in the System Properties and these bounds are therefore ignored.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Emitter", meta = (EditConditionHides, EditCondition = "CalculateBoundsMode == ENiagaraEmitterCalculateBoundMode::Fixed"))
	FBox FixedBounds;

	/** Creates a stable Identifier (Particles.ID) which does not vary from frame to frame. This comes at a small memory and performance cost. This allows external objects to track the same particle over multiple frames. Particle arrays are tightly packed and a particle's actual index in the array may change from frame to frame. This optionally lets you use a lookup table to track a particle by index in the lookup table. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Emitter")
	uint32 bRequiresPersistentIDs : 1;

	UPROPERTY(meta=(NiagaraNoMerge))
	TArray<FNiagaraEventScriptProperties> EventHandlerScriptProps;
	
	UPROPERTY(EditAnywhere, Category = "Scalability", meta=(DisplayInScalabilityContext))
	FNiagaraPlatformSet Platforms;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta=(DisplayInScalabilityContext))
	FNiagaraEmitterScalabilityOverrides ScalabilityOverrides;

	/** An override on the max number of GPU particles we expect to spawn in a single frame. A value of 0 means it'll use fx.MaxNiagaraGPUParticlesSpawnPerFrame.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Emitter", meta = (EditCondition = "SimTarget == ENiagaraSimTarget::GPUComputeSim", DisplayName = "Max GPU Particles Spawn per Frame", EditConditionHides))
	int32 MaxGPUParticlesSpawnPerFrame;

	/**
	The emitter needs to allocate memory for the particles each tick.
	To prevent reallocations, the emitter should allocate as much memory as is needed for the max particle count.
	This setting controls if the allocation size should be automatically determined or manually entered.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Emitter", meta = (SegmentedDisplay))
	EParticleAllocationMode AllocationMode = EParticleAllocationMode::AutomaticEstimate;
	
	/** 
	The emitter will allocate at least this many particles on it's first tick.
	This can aid performance by avoiding many allocations as an emitter ramps up to it's max size.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Emitter", meta = (EditCondition = "AllocationMode != EParticleAllocationMode::AutomaticEstimate", EditConditionHides))
	int32 PreAllocationCount = 0;

	/**
	List of emitter dependencies to use when calculating the execution order for emitter particle scripts.
	This is generally only required when you are using advanced features, such as reading / writing to a data interface in different emitters
	and need to ensure the emitters can not run concurrently with one another, either on the CPU or the GPU.
	**/
	UPROPERTY(EditAnywhere, Category = "Emitter")
	TArray<FNiagaraDataInterfaceEmitterBinding> EmitterDependencies;

	UPROPERTY()
	FNiagaraEmitterScriptProperties UpdateScriptProps;

	UPROPERTY()
	FNiagaraEmitterScriptProperties SpawnScriptProps;

	UPROPERTY()
	FNiagaraParameterStore RendererBindings;

	UPROPERTY()
	TArray<FNiagaraExternalUObjectInfo> RendererBindingsExternalObjects;

	UPROPERTY()
	TMap<FNiagaraVariableBase, FNiagaraVariableBase> ResolvedDIBindings;

	NIAGARA_API void CopyFrom(const FVersionedNiagaraEmitterData& Source);
	NIAGARA_API void PostLoad(UNiagaraEmitter& Emitter, int32 NiagaraVer);

	NIAGARA_API void PostInitProperties(UNiagaraEmitter* Outer);
	NIAGARA_API bool UsesCollection(const UNiagaraParameterCollection* Collection) const;
	NIAGARA_API bool UsesScript(const UNiagaraScript* Script) const;
	NIAGARA_API bool IsReadyToRun() const;
	const TArray<UNiagaraRendererProperties*>& GetRenderers() const { return RendererProperties; }
	UNiagaraRendererProperties* GetRenderer(int32 RendererIndex)const { return RendererProperties.IsValidIndex(RendererIndex) ? RendererProperties[RendererIndex] : nullptr; }
	NIAGARA_API void GetScripts(TArray<UNiagaraScript*>& OutScripts, bool bCompilableOnly = true, bool bEnabledOnly = false) const;
	NIAGARA_API UNiagaraScript* GetScript(ENiagaraScriptUsage Usage, FGuid UsageId) const;
	UNiagaraScript* GetGPUComputeScript() { return GPUComputeScript; }
	const UNiagaraScript* GetGPUComputeScript() const { return GPUComputeScript; }
	inline const TArray<FNiagaraEventScriptProperties>& GetEventHandlers() const { return EventHandlerScriptProps; }
	NIAGARA_API void CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData, const UNiagaraEmitter& Emitter, const FGuid& EmitterHandleId);
	NIAGARA_API void CacheFromShaderCompiled();
	bool IsAllowedToExecute() const { return bIsAllowedToExecute; }
	NIAGARA_API bool UsesInterpolatedSpawning() const;

	NIAGARA_API FGraphEventArray PrecacheComputePSOs(const UNiagaraEmitter& NiagaraEmitter);
	bool DidPSOPrecacheFail() const { return PSOPrecacheResult == EPSOPrecacheResult::NotSupported; }

	bool RequiresViewUniformBuffer() const { return bRequiresViewUniformBuffer; }
	bool NeedsPartialDepthTexture() const { return bNeedsPartialDepthTexture; }
	uint32 GetMaxInstanceCount() const { return MaxInstanceCount; }
	uint32 GetMaxAllocationCount() const { return MaxAllocationCount; }
	TConstArrayView<TSharedPtr<FNiagaraBoundsCalculator>> GetBoundsCalculators() const { return MakeArrayView(BoundsCalculators); }
	NIAGARA_API bool RequiresPersistentIDs() const;
	const TArray<UNiagaraSimulationStageBase*>& GetSimulationStages() const { return SimulationStages; }
	FNiagaraSimStageExecutionDataPtr GetSimStageExcecutionData() const { return SimStageExecutionData; }
	inline const FNiagaraEmitterScalabilitySettings& GetScalabilitySettings()const { return CurrentScalabilitySettings; }
	NIAGARA_API const FNiagaraEmitterScalabilityOverride& GetCurrentOverrideSettings() const;
	NIAGARA_API UNiagaraSimulationStageBase* GetSimulationStageById(FGuid ScriptUsageId) const;
	NIAGARA_API bool BuildParameterStoreRendererBindings(FNiagaraParameterStore& ParameterStore) const;
	NIAGARA_API void RebuildRendererBindings(const UNiagaraEmitter& Emitter);
	inline static FBox GetDefaultFixedBounds() { return FBox(FVector(-100), FVector(100)); }
	
	NIAGARA_API FVersionedNiagaraEmitter GetParent() const;
	NIAGARA_API FVersionedNiagaraEmitter GetParentAtLastMerge() const;
	NIAGARA_API void RemoveParent();
	NIAGARA_API void Reparent(const FVersionedNiagaraEmitter& InParent);
	
	NIAGARA_API bool IsValid() const;
	NIAGARA_API void UpdateDebugName(const UNiagaraEmitter& Emitter, const FNiagaraDataSetCompiledData* CompiledData);
	NIAGARA_API void SyncEmitterAlias(const FString& InOldName, const UNiagaraEmitter& InEmitter);

	template<typename TAction>
	void ForEachRenderer(TAction Func) const;

	template<typename TAction>
	void ForEachEnabledRenderer(TAction Func) const;

	template<typename TAction>
	void ForEachEnabledRendererWithIndex(TAction Func) const;

	template<typename TAction>
	void ForEachScript(TAction Func) const;

	/** Performs the passed action for all platform sets in this emitter.
	Include NiagaraEmitterImpl.h to be able to call this function */
	template<typename TAction>
	inline void ForEachPlatformSet(const UNiagaraSystem& OwningSystem, const FGuid& EmitterHandleId, TAction Func);

	/** Returns true if this emitter's platform filter allows it on this platform and quality level. */
	NIAGARA_API bool IsAllowedByScalability() const;

	/* Returns the number of max expected particles for memory allocations. */
	NIAGARA_API int32 GetMaxParticleCountEstimate();

	/* Gets whether or not the supplied event generator id matches an event generator which is shared between the particle spawn and update scrips. */
	NIAGARA_API bool IsEventGeneratorShared(FName EventGeneratorId) const;

	/* This is used by the emitter instances to report runtime allocations to reduce reallocation in future simulation runs. */
	NIAGARA_API int32 AddRuntimeAllocation(uint64 ReporterHandle, int32 AllocationCount);
	NIAGARA_API void ClearRuntimeAllocationEstimate(uint64 ReportHandle = INDEX_NONE);

	/* Gets a pointer to an event handler by script usage id.  This method is potentially unsafe because modifications to
	the event handler array can make this pointer become invalid without warning. */
	NIAGARA_API FNiagaraEventScriptProperties* GetEventHandlerByIdUnsafe(FGuid ScriptUsageId);
 
#if WITH_EDITORONLY_DATA
	/** An allow list of Particle attributes (e.g. "Particle.Position" or "Particle.Age") that will not be removed from the DataSet  even if they aren't read by the VM.
	    Used in conjunction with UNiagaraSystem::bTrimAttributes */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Emitter")
	TArray<FString> AttributesToPreserve;

	/** This determines how emitters will be added to a system by default. If summary view is setup, consider setting this to 'Summary'. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Asset Options", meta = (SegmentedDisplay))
	ENiagaraEmitterDefaultSummaryState AddEmitterDefaultViewState = ENiagaraEmitterDefaultSummaryState::Default;
	
	UPROPERTY()
	FNiagaraEmitterScriptProperties EmitterSpawnScriptProps;

	UPROPERTY()
	FNiagaraEmitterScriptProperties EmitterUpdateScriptProps;

	/** 'Source' data/graphs for the scripts used by this emitter. */
	UPROPERTY()
	TObjectPtr<UNiagaraScriptSourceBase> GraphSource = nullptr;

	UPROPERTY()
	TObjectPtr<UNiagaraScratchPadContainer> ScratchPads = nullptr;

	UPROPERTY()
	TObjectPtr<UNiagaraScratchPadContainer> ParentScratchPads = nullptr;

	UPROPERTY()
	FVersionedNiagaraEmitter VersionedParent;

	UPROPERTY()
	FVersionedNiagaraEmitter VersionedParentAtLastMerge;

	NIAGARA_API bool AreAllScriptAndSourcesSynchronized() const;
	NIAGARA_API void InvalidateCompileResults();
	NIAGARA_API bool IsSynchronizedWithParent() const;
	NIAGARA_API bool UsesEmitter(const UNiagaraEmitter& InEmitter) const;
	NIAGARA_API void GatherStaticVariables(TArray<FNiagaraVariable>& OutVars) const;
	
	NIAGARA_API UNiagaraEditorDataBase* GetEditorData() const;
	NIAGARA_API UNiagaraEditorParametersAdapterBase* GetEditorParameters();
#endif

#if STATS
	FNiagaraStatDatabase& GetStatData() { return StatDatabase; }
#endif

#if WITH_NIAGARA_DEBUG_EMITTER_NAME
	const TCHAR* GetDebugSimName() const { return *DebugSimName; }
#else
	const TCHAR* GetDebugSimName() const { return TEXT(""); }
#endif

	NIAGARA_API void GatherCompiledParticleAttributes(TArray<FNiagaraVariableBase>& OutVariables) const;

private:
	UPROPERTY()
	TArray<TObjectPtr<UNiagaraRendererProperties>> RendererProperties;

	UPROPERTY(meta = (NiagaraNoMerge))
	TArray<TObjectPtr<UNiagaraSimulationStageBase>> SimulationStages;

	UPROPERTY()
	TArray<FNiagaraSimStageExecutionLoopData> SimStageExecutionLoops;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Advanceddisplay, Category = "Emitter", meta = (DisplayName = "Sim Stage Loops"))
	TArray<FNiagaraSimStageExecutionLoopEditorData> SimStageExecutionLoopEditorData;
#endif

	FNiagaraSimStageExecutionDataPtr SimStageExecutionData;

	UPROPERTY()
	TObjectPtr<UNiagaraScript> GPUComputeScript = nullptr;
	
	UPROPERTY()
	TArray<FName> SharedEventGeneratorIds;

	UPROPERTY(Transient)
	FNiagaraEmitterScalabilitySettings CurrentScalabilitySettings;

	/** Can this emitter run with the current scalability settings, etc */
	uint32 bIsAllowedToExecute : 1;

	/** Indicates that the GPU script requires the view uniform buffer. */
	uint32 bRequiresViewUniformBuffer : 1;

	/** Indicates we use the partial depth textures. */
	uint32 bNeedsPartialDepthTexture : 1;

	/** Maximum number of instances we can create for this emitter. */
	uint32 MaxInstanceCount = 0;

	/** Maximum instance allocations size for the emitter, can be larger than MaxInstanceCount */
	uint32 MaxAllocationCount = 0;

	/** Optional list of bounds calculators. */
	TArray<TSharedPtr<FNiagaraBoundsCalculator>, TInlineAllocator<1>> BoundsCalculators;
	
	FMemoryRuntimeEstimation RuntimeEstimation;
#if STATS
	FNiagaraStatDatabase StatDatabase;
#endif
	
#if WITH_NIAGARA_DEBUG_EMITTER_NAME
	FString DebugSimName;
#endif

#if WITH_EDITORONLY_DATA
	/** Data used by the editor to maintain UI state etc.. */
	UPROPERTY()
	TObjectPtr<UNiagaraEditorDataBase> EditorData;

	/** Wrapper for editor only parameters. */
	UPROPERTY()
	TObjectPtr<UNiagaraEditorParametersAdapterBase> EditorParameters;

	mutable TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalData;
#endif

	friend UNiagaraEmitter;

	NIAGARA_API void EnsureScriptsPostLoaded();
	NIAGARA_API void OnPostCompile(const UNiagaraEmitter& InEmitter);

	EPSOPrecacheResult PSOPrecacheResult = EPSOPrecacheResult::Unknown;

	bool IsValidInternal() const;
	bool IsReadyToRunInternal() const;
#if !WITH_EDITORONLY_DATA
	mutable TOptional<bool> IsValidCached;
	mutable TOptional<bool> IsReadyToRunCached;
#endif
};

/** 
 *	Niagara Emitters are particle spawners that can be reused for different effects by putting them into Niagara systems.
 *	Emitters render their particles using different renderers, such as Sprite Renderers or Mesh Renderers to produce different effects.
 *
 *	Emitter assets cannot be spawned or used in a level directly, but need to be placed in a Niagara system. Emitters support inheritance, so that
 *	changes to the base asset are automatically picked up by child emitter assets and emitters in system assets.
 */
UCLASS(MinimalAPI)
class UNiagaraEmitter : public UNiagaraEmitterBase, public INiagaraParameterDefinitionsSubscriber, public FNiagaraVersionedObject
{
	GENERATED_UCLASS_BODY()

	friend struct FNiagaraEmitterHandle;
public:
#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnPropertiesChanged);
	DECLARE_MULTICAST_DELEGATE(FOnSimStagesChanged);
	DECLARE_MULTICAST_DELEGATE(FOnEventHandlersChanged);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEmitterCompiled, FVersionedNiagaraEmitter);

	struct PrivateMemberNames
	{
		static NIAGARA_API const FName EventHandlerScriptProps;
	};
#endif

public:
#if WITH_EDITOR
	/** Creates a new emitter with the supplied emitter as a parent emitter and the supplied system as its owner. */
	NIAGARA_API static UNiagaraEmitter* CreateWithParentAndOwner(FVersionedNiagaraEmitter InParentEmitter, UObject* InOwner, FName InName, EObjectFlags FlagMask);

	/** Creates a new emitter by duplicating an existing emitter. The new emitter will reference the same parent emitter if one is available. */
	static UNiagaraEmitter* CreateAsDuplicate(const UNiagaraEmitter& InEmitterToDuplicate, FName InDuplicateName, UNiagaraSystem& InDuplicateOwnerSystem);

	//Begin UObject Interface
	virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS // Suppress compiler warning on override of deprecated function
	UE_DEPRECATED(5.0, "Use version that takes FObjectPreSaveContext instead.")
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void PostEditChangeVersionedProperty(FPropertyChangedEvent& PropertyChangedEvent, const FGuid& Version);
	NIAGARA_API FOnPropertiesChanged& OnPropertiesChanged();
	NIAGARA_API FOnSimStagesChanged& OnSimStagesChanged();
	NIAGARA_API FOnSimStagesChanged& OnEventHandlersChanged();

	/** Helper method for binding the notifications needed for proper editor integration. */
	NIAGARA_API void RebindNotifications();
#endif
	virtual bool NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const override;
	virtual void Serialize(FArchive& Ar)override;
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	virtual bool IsEditorOnly() const override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	//End UObject Interface

	// Begin UNiagaraEmitterBase Interface
	virtual FNiagaraVariableBase GetResolvedDIBinding(const FNiagaraVariableBase& InVariable) const override;
	virtual bool CanObtainParticleAttribute(const FNiagaraVariableBase& InVar, const FGuid& EmitterVersion, FNiagaraTypeDefinition& OutBoundType) const override;
	virtual bool CanObtainEmitterAttribute(const FNiagaraVariableBase& InVarWithUniqueNameNamespace, FNiagaraTypeDefinition& OutBoundType) const override;
#if WITH_EDITORONLY_DATA
protected:
	virtual void OnUniqueEmitterNameChanged(const FString& OldName) override;
public:
#endif //WITH_EDITORONLY_DATA
#if WITH_EDITOR
	virtual void HandleVariableRenamed(const FNiagaraVariable& InOldVariable, const FNiagaraVariable& InNewVariable, bool bUpdateContexts, FGuid EmitterVersion) override;
	virtual void HandleVariableRemoved(const FNiagaraVariable& InOldVariable, bool bUpdateContexts, FGuid EmitterVersion) override;
#endif
	// End UNiagaraEmitterBase Interface

#if WITH_EDITORONLY_DATA
	//~ Begin INiagaraParameterDefinitionsSubscriber Interface
	virtual const TArray<FParameterDefinitionsSubscription>& GetParameterDefinitionsSubscriptions() const override { return ParameterDefinitionsSubscriptions; }
	virtual TArray<FParameterDefinitionsSubscription>& GetParameterDefinitionsSubscriptions() override { return ParameterDefinitionsSubscriptions; };

	/** Get all UNiagaraScriptSourceBase of this subscriber. */
	virtual TArray<UNiagaraScriptSourceBase*> GetAllSourceScripts() override;

	/** Get the path to the UObject of this subscriber. */
	virtual FString GetSourceObjectPathName() const override;

	/** Get All adapters to editor only script vars owned directly by this subscriber. */
	virtual TArray<UNiagaraEditorParametersAdapterBase*> GetEditorOnlyParametersAdapters() override;
	//~ End INiagaraParameterDefinitionsSubscriber Interface

	/** Get the cached parameter map traversal for this emitter.  */
	NIAGARA_API const TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe>& GetCachedTraversalData(const FGuid& EmitterVersion) const;
	NIAGARA_API void InvalidateCachedTraversalData(const FGuid& EmitterVersion);
#endif

	bool IsEnabledOnPlatform(const FString& PlatformName) const;

	/** Returns the emitter data for latest exposed version. */
	NIAGARA_API FVersionedNiagaraEmitterData* GetLatestEmitterData();
	NIAGARA_API const FVersionedNiagaraEmitterData* GetLatestEmitterData() const;

	/** Returns the emitter data for a specific version or nullptr if no such version is found. For the null-Guid it returns the exposed version.  */
	NIAGARA_API FVersionedNiagaraEmitterData* GetEmitterData(const FGuid& VersionGuid);
	NIAGARA_API const FVersionedNiagaraEmitterData* GetEmitterData(const FGuid& VersionGuid) const;

	/** Returns all available versions for this emitter. */
	NIAGARA_API virtual TArray<FNiagaraAssetVersion> GetAllAvailableVersions() const override;

	template <typename TAction>
	void ForEachVersionData(TAction Func) const;

	template <typename TAction>
	void ForEachVersionData(TAction Func);
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual TSharedPtr<FNiagaraVersionDataAccessor> GetVersionDataAccessor(const FGuid& Version) override; 

	/** If true then this script asset uses active version control to track changes. */
	virtual bool IsVersioningEnabled() const override { return bVersioningEnabled; }

	/** Returns the version of the exposed version data (i.e. the version used when adding an emitter to a system) */
	NIAGARA_API virtual FNiagaraAssetVersion GetExposedVersion() const override;

	/** Returns the version data for the given guid, if it exists. Otherwise returns nullptr. */
	NIAGARA_API virtual FNiagaraAssetVersion const* FindVersionData(const FGuid& VersionGuid) const override;
	
	/** Creates a new data entry for the given version number. The version must be > 1.0 and must not collide with an already existing version. The data will be a copy of the previous minor version. */
	NIAGARA_API virtual FGuid AddNewVersion(int32 MajorVersion, int32 MinorVersion) override;

	/** Deletes the version data for an existing version. The exposed version cannot be deleted and will result in an error. Does nothing if the guid does not exist in the version data. */
	NIAGARA_API virtual void DeleteVersion(const FGuid& VersionGuid) override;

	/** Changes the exposed version. Does nothing if the guid does not exist in the version data. */
	NIAGARA_API virtual void ExposeVersion(const FGuid& VersionGuid) override;

	/** Enables versioning for this emitter asset. */
	NIAGARA_API virtual void EnableVersioning() override;

	/** Disables versioning and keeps only the data from the given version guid. Note that this breaks ALL references from existing assets and should only be used when creating a copy of an emitter, as the effect is very destructive.  */
	NIAGARA_API virtual void DisableVersioning(const FGuid& VersionGuidToUse) override;

	/** Makes sure that the default version data is available and fixes old emitter assets. */
	NIAGARA_API void CheckVersionDataAvailable();
#endif

private:
	/** The exposed version is the version that is used by default when a user adds this emitter somewhere. It is basically the published version and allows a user to create and test newer versions. */
	UPROPERTY()
	FGuid ExposedVersion;

	/** If true then this emitter asset uses active version control to track changes. */
	UPROPERTY()
	bool bVersioningEnabled = false;

	/** Contains all of the versioned emitter data. */
	UPROPERTY()
	TArray<FVersionedNiagaraEmitterData> VersionData;

public:
	
#if WITH_EDITORONLY_DATA
	/** If an emitter is inheritable, new emitters based on an inheritable emitter, or Niagara Systems using an inheritable emitter, will automatically inherit changes made to the original emitter. */
	UPROPERTY(EditAnywhere, Category = "Asset Options", AssetRegistrySearchable)
	bool bIsInheritable = true;
	
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Asset Options", AssetRegistrySearchable, DisplayName="Asset Description")
	FText TemplateAssetDescription;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Asset Options", AssetRegistrySearchable, meta = (SkipSystemResetOnChange = "true", AllowedClasses="/Script/MediaAssets.FileMediaSource"))
	FSoftObjectPath PreviewMoviePath;
	
	/** Category to collate this emitter into for "add new emitter" dialogs.*/
	UPROPERTY(AssetRegistrySearchable)
	FText Category;

	///** The thumbnail image used for the asset. This is always the latest recorded thumbnail. This can be different from the thumbnails that are saved per emitter version in collapsed view. */
	UPROPERTY()
	TObjectPtr<UTexture2D> ThumbnailImage;
	
	/** If this emitter is exposed to the library, or should be explicitly hidden. */
    UPROPERTY(AssetRegistrySearchable)
    ENiagaraScriptLibraryVisibility LibraryVisibility = ENiagaraScriptLibraryVisibility::Unexposed;

	/** This is used as a transient value to open a specific version in the editor */
	UPROPERTY(Transient)
	FGuid VersionToOpenInEditor;

	const static FGuid EmitterMergeMessageId;
#endif

	// ----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
	// ------------------ Most properties below this point are deprecated and stored in the versioned emitter data instead!  ------------------------------------------------------------------------------------------------
	// ----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
	
#if WITH_EDITORONLY_DATA
	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	bool bLocalSpace_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	bool bDeterminism_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	int32 RandomSeed_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	EParticleAllocationMode AllocationMode_DEPRECATED;
	
	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	int32 PreAllocationCount_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	FNiagaraEmitterScriptProperties UpdateScriptProps_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	FNiagaraEmitterScriptProperties SpawnScriptProps_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty))
	ENiagaraScriptTemplateSpecification TemplateSpecification_DEPRECATED;
	
	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	FNiagaraEmitterScriptProperties EmitterSpawnScriptProps_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	FNiagaraEmitterScriptProperties EmitterUpdateScriptProps_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<FString> AttributesToPreserve_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TObjectPtr<UNiagaraScript>> ParentScratchPadScripts_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	ENiagaraSimTarget SimTarget_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty, ScriptNoExport))
	FBox FixedBounds_DEPRECATED;
	
	UPROPERTY()
	int32 MinDetailLevel_DEPRECATED;
	UPROPERTY()
	int32 MaxDetailLevel_DEPRECATED;
	UPROPERTY()
	FNiagaraDetailsLevelScaleOverrides GlobalSpawnCountScaleOverrides_DEPRECATED;
	
	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	FNiagaraPlatformSet Platforms_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	FNiagaraEmitterScalabilityOverrides ScalabilityOverrides_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	uint32 bInterpolatedSpawning_DEPRECATED : 1;

	UPROPERTY(meta = (DeprecatedProperty))
	FNiagaraParameterStore RendererBindings_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty, ScriptNoExport))
	uint32 bFixedBounds_DEPRECATED : 1;

	/** Whether to use the min detail or not. */
	UPROPERTY()
	uint32 bUseMinDetailLevel_DEPRECATED : 1;
	
	/** Whether to use the min detail or not. */
	UPROPERTY()
	uint32 bUseMaxDetailLevel_DEPRECATED : 1;

	/** Legacy bool to control overriding the global spawn count scales. */
	UPROPERTY()
	uint32 bOverrideGlobalSpawnCountScale_DEPRECATED : 1;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	uint32 bRequiresPersistentIDs_DEPRECATED : 1;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	uint32 MaxGPUParticlesSpawnPerFrame_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty))
	TArray<FNiagaraAssetTagDefinitionReference> AssetTags_DEPRECATED;
#endif
	
	NIAGARA_API void UpdateEmitterAfterLoad();

#if WITH_EDITORONLY_DATA
	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<UNiagaraScriptSourceBase> GraphSource_DEPRECATED;

	NIAGARA_API void OnPostCompile(const FGuid& EmitterVersion);

	/* Gets a Guid which is updated any time data in this emitter is changed. */
	NIAGARA_API FGuid GetChangeId() const;

	/** Deprecated library exposure bool. Use the LibraryVisibility enum instead. FNiagaraEditorUtilities has accessor functions that takes deprecation into account */
	UPROPERTY()
	bool bExposeToLibrary_DEPRECATED;
	
	/** Deprecated template asset bool. Use the TemplateSpecification enum instead. */
	UPROPERTY()
	bool bIsTemplateAsset_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TObjectPtr<UNiagaraScript>> ScratchPadScripts_DEPRECATED;

	/** Callback issued whenever a VM compilation successfully happened (even if the results are a script that cannot be executed due to errors)*/
	NIAGARA_API FOnEmitterCompiled& OnEmitterVMCompiled();

	/** Callback issued whenever a VM compilation successfully happened (even if the results are a script that cannot be executed due to errors)*/
	NIAGARA_API FOnEmitterCompiled& OnEmitterGPUCompiled();

	/** Callback issued whenever a GPU compilation successfully happened (even if the results are a script that cannot be executed due to errors)*/
	FOnEmitterCompiled& OnGPUCompilationComplete()
	{
		return OnGPUScriptCompiledDelegate;
	}
	
	NIAGARA_API static bool GetForceCompileOnLoad();

	/** Whether or not this emitter is synchronized with its parent emitter. */
	NIAGARA_API bool IsSynchronizedWithParent() const;

	/** Merges in any changes from the parent emitter into this emitter. */
	NIAGARA_API TArray<INiagaraMergeManager::FMergeEmitterResults> MergeChangesFromParent();	

	/**
	Duplicates this emitter, but prevents the duplicate from merging in changes from the parent emitter.
	The resulting duplicate will have no parent information and will clear RF_Standalone | RF_Public flags.
	*/
	NIAGARA_API UNiagaraEmitter* DuplicateWithoutMerging(UObject* InOuter);

	NIAGARA_API void SetEditorData(UNiagaraEditorDataBase* InEditorData, const FGuid& VersionGuid);
#endif

	void NIAGARA_API AddRenderer(UNiagaraRendererProperties* Renderer, FGuid EmitterVersion);
	void NIAGARA_API RemoveRenderer(UNiagaraRendererProperties* Renderer, FGuid EmitterVersion);
	void NIAGARA_API MoveRenderer(UNiagaraRendererProperties* Renderer, int32 NewIndex, FGuid EmitterVersion);
	void NIAGARA_API AddEventHandler(FNiagaraEventScriptProperties EventHandler, FGuid EmitterVersion);
	void NIAGARA_API RemoveEventHandlerByUsageId(FGuid EventHandlerUsageId, FGuid EmitterVersion);
	void NIAGARA_API AddSimulationStage(UNiagaraSimulationStageBase* SimulationStage, FGuid EmitterVersion);
	void NIAGARA_API RemoveSimulationStage(UNiagaraSimulationStageBase* SimulationStage, FGuid EmitterVersion);
	void NIAGARA_API MoveSimulationStageToIndex(UNiagaraSimulationStageBase* SimulationStage, int32 TargetIndex, FGuid EmitterVersion);

	TStatId GetStatID(bool bGameThread, bool bConcurrent) const;

	void UpdateScalability();

#if WITH_EDITORONLY_DATA
	NIAGARA_API void SetParent(const FVersionedNiagaraEmitter& InParent);
	NIAGARA_API void ChangeParentVersion(const FGuid& NewParentVersion, const FGuid& EmitterVersion);

	NIAGARA_API void NotifyScratchPadScriptsChanged();
	FNiagaraMessageStore& GetMessageStore() { return MessageStore; }
#endif

protected:
	virtual void BeginDestroy() override;

#if WITH_EDITORONLY_DATA
private:
	void UpdateFromMergedCopy(const INiagaraMergeManager& MergeManager, UNiagaraEmitter* MergedEmitter, FVersionedNiagaraEmitterData* EmitterData);
	void UpdateChangeId(const FString& Reason);
	void ScriptRapidIterationParameterChanged();
	void SimulationStageChanged();
	void RendererChanged();
	void GraphSourceChanged();
	void PersistentEditorDataChanged();

private:
	/** Adjusted every time that we compile this emitter. Lets us know that we might differ from any cached versions.*/
	UPROPERTY()
	FGuid ChangeId;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<UNiagaraEditorDataBase> EditorData_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<UNiagaraEditorParametersAdapterBase> EditorParameters_DEPRECATED;

	/** A multicast delegate which is called whenever all the scripts for this emitter have been compiled (successfully or not). */
	FOnEmitterCompiled OnVMScriptCompiledDelegate;

	/** A multicast delegate which is called whenever all the scripts for this emitter have been compiled (successfully or not). */
	FOnEmitterCompiled OnGPUScriptCompiledDelegate;

	void RaiseOnEmitterGPUCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion);

	/** Use property in struct returned from GetEmitterData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TObjectPtr<UNiagaraRendererProperties>> RendererProperties_DEPRECATED;
	
	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<FNiagaraEventScriptProperties> EventHandlerScriptProps_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TObjectPtr<UNiagaraSimulationStageBase>> SimulationStages_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<UNiagaraScript> GPUComputeScript_DEPRECATED;

	/** Use property in struct returned from GetEmitterData() instead */ 
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<FName> SharedEventGeneratorIds_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<UNiagaraEmitter> Parent_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<UNiagaraEmitter> ParentAtLastMerge_DEPRECATED;

	/** Subscriptions to definitions of parameters. */
	UPROPERTY()
	TArray<FParameterDefinitionsSubscription> ParameterDefinitionsSubscriptions;
#endif

	bool bFullyLoaded = false;

#if WITH_EDITOR
	FOnPropertiesChanged OnPropertiesChangedDelegate;
	FOnSimStagesChanged OnSimStagesChangedDelegate;
	FOnEventHandlersChanged OnEventHandlersChangedDelegate;
#endif


public:
	void UpdateStatID() const;
private:
	void GenerateStatID() const;
#if STATS
	mutable TStatId StatID_GT;
	mutable TStatId StatID_GT_CNC;
	mutable TStatId StatID_RT;
	mutable TStatId StatID_RT_CNC;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TMap<FGuid, TObjectPtr<UNiagaraMessageDataBase>> MessageKeyToMessageMap_DEPRECATED;

	UPROPERTY()
	FNiagaraMessageStore MessageStore;
#endif

	void ResolveScalabilitySettings();
};

template <typename TAction>
void UNiagaraEmitter::ForEachVersionData(TAction Func) const
{
	for (const FVersionedNiagaraEmitterData& Data : VersionData)
	{
		Func(Data);
	}
}

template <typename TAction>
void UNiagaraEmitter::ForEachVersionData(TAction Func)
{
	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
		Func(Data);
	}
}

template <typename TAction>
void FVersionedNiagaraEmitterData::ForEachRenderer(TAction Func) const
{
	for (UNiagaraRendererProperties* Renderer : RendererProperties)
	{
		if (Renderer)
		{
			Func(Renderer);
		}
	}
}

template<typename TAction>
void FVersionedNiagaraEmitterData::ForEachEnabledRenderer(TAction Func) const
{
	for (UNiagaraRendererProperties* Renderer : RendererProperties)
	{
		if (Renderer && Renderer->GetIsEnabled() && Renderer->IsSimTargetSupported(this->SimTarget))
		{
			Func(Renderer);
		}
	}
}

template<typename TAction>
void FVersionedNiagaraEmitterData::ForEachEnabledRendererWithIndex(TAction Func) const
{
	for (int32 i=0; i < RendererProperties.Num(); ++i)
	{
		UNiagaraRendererProperties* Renderer = RendererProperties[i];
		if (Renderer && Renderer->GetIsEnabled() && Renderer->IsSimTargetSupported(this->SimTarget))
		{
			Func(Renderer, i);
		}
	}
}

template<typename TAction>
void FVersionedNiagaraEmitterData::ForEachScript(TAction Func) const
{
	Func(SpawnScriptProps.Script);
	Func(UpdateScriptProps.Script);

	if (GPUComputeScript)
	{
		Func(GPUComputeScript);
	}

	for (auto& EventScriptProps : EventHandlerScriptProps)
	{
		Func(EventScriptProps.Script);
	}
}
