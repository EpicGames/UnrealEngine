// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"

#include "NiagaraEmitterBase.generated.h"

class UNiagaraEmitterBase;
struct FVersionedNiagaraEmitterBaseWeakPtr;

// Reference to a versioned emitter base
// As the Emitter pointer is not a TStrongObjectPtr you must be careful with GC if holding onto one of these
struct FVersionedNiagaraEmitterBase
{
	FVersionedNiagaraEmitterBase() = default;
	[[nodiscard]] explicit FVersionedNiagaraEmitterBase(UNiagaraEmitterBase* InEmitter, const FGuid& InVersion) : Emitter(InEmitter), Version(InVersion) {}
	[[nodiscard]] NIAGARA_API explicit FVersionedNiagaraEmitterBase(const FVersionedNiagaraEmitterBaseWeakPtr& Weak);

	[[nodiscard]] bool IsValid() const { return Emitter != nullptr; }
	[[nodiscard]] NIAGARA_API FVersionedNiagaraEmitterBaseWeakPtr ToWeakPtr() const;

	UNiagaraEmitterBase* Emitter = nullptr;
	FGuid Version;
};

// Weak reference to a versioned emitter base
struct FVersionedNiagaraEmitterBaseWeakPtr
{
	FVersionedNiagaraEmitterBaseWeakPtr() = default;
	[[nodiscard]] explicit FVersionedNiagaraEmitterBaseWeakPtr(UNiagaraEmitterBase* InEmitter, const FGuid& InVersion) : Emitter(InEmitter), Version(InVersion) {}
	[[nodiscard]] explicit FVersionedNiagaraEmitterBaseWeakPtr(TWeakObjectPtr<UNiagaraEmitterBase> InEmitter, const FGuid& InVersion) : Emitter(InEmitter), Version(InVersion) {}
	[[nodiscard]] NIAGARA_API explicit FVersionedNiagaraEmitterBaseWeakPtr(const FVersionedNiagaraEmitterBase& VersionedEmitter);

	[[nodiscard]] bool IsValid() const { return Emitter.IsValid(); }
	[[nodiscard]] FVersionedNiagaraEmitterBase ResolveWeakPtr() const { return FVersionedNiagaraEmitterBase(*this); }

	TWeakObjectPtr<UNiagaraEmitterBase> Emitter;
	FGuid Version;
};

// Base class for all derived Niagara emitter types (i.e. Stateful and Stateless)
UCLASS(MinimalAPI)
class UNiagaraEmitterBase : public UObject
{
	GENERATED_BODY()

public:
	NIAGARA_API const FString& GetUniqueEmitterName() const;
	NIAGARA_API bool SetUniqueEmitterName(const FString& InName);

	virtual FNiagaraVariableBase GetResolvedDIBinding(const FNiagaraVariableBase& InVariable) const { return InVariable; }

	virtual bool CanObtainParticleAttribute(const FNiagaraVariableBase& InVariable, const FGuid& EmitterVersion, FNiagaraTypeDefinition& OutBoundType) const { return false; }
	virtual bool CanObtainEmitterAttribute(const FNiagaraVariableBase& InVariableWithUniqueNameNamespace, FNiagaraTypeDefinition& OutBoundType) const { return false; }
	bool CanObtainSystemAttribute(const FNiagaraVariableBase& InVariable, FNiagaraTypeDefinition& OutBoundType) const;
	bool CanObtainUserVariable(const FNiagaraVariableBase& InVariable) const;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnRenderersChanged);

	FOnRenderersChanged& OnRenderersChanged() { return OnRenderersChangedDelegate; }

	virtual void HandleVariableRenamed(const FNiagaraVariable& InOldVariable, const FNiagaraVariable& InNewVariable, bool bUpdateContexts, FGuid EmitterVersion) {}
	virtual void HandleVariableRemoved(const FNiagaraVariable& InOldVariable, bool bUpdateContexts, FGuid EmitterVersion) {}
#endif

protected:
#if WITH_EDITORONLY_DATA
	// Called if the emitter name was successfully changed
	virtual void OnUniqueEmitterNameChanged(const FString& OldName) {}
#endif

protected:
	UPROPERTY()
	FString UniqueEmitterName;

#if WITH_EDITOR
	FOnRenderersChanged OnRenderersChangedDelegate;
#endif
};
