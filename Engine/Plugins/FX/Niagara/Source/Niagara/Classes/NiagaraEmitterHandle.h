// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "NiagaraTypes.h"
#include "NiagaraEmitterHandle.generated.h"

struct FNiagaraPlatformSet;
class UNiagaraEmitterBase;
class UNiagaraSystem;
class UNiagaraStatelessEmitter;

UENUM()
enum class ENiagaraEmitterMode : uint8
{
	Standard,
	Stateless
};

/** 
 * Stores emitter information within the context of a System.
 */
USTRUCT()
struct FNiagaraEmitterHandle
{
	GENERATED_USTRUCT_BODY()
public:
	/** Creates a new invalid emitter handle. */
	NIAGARA_API FNiagaraEmitterHandle();

#if WITH_EDITORONLY_DATA
	/** Creates a new emitter handle from an emitter. */
	NIAGARA_API FNiagaraEmitterHandle(UNiagaraEmitter& InEmitter, const FGuid& Version);
	
	/** Creates a new emitter handle from an emitter. */
	NIAGARA_API FNiagaraEmitterHandle(const FVersionedNiagaraEmitter& InEmitter);

	//-TODO:Stateless
	NIAGARA_API FNiagaraEmitterHandle(UNiagaraStatelessEmitter& InEmitter);
	//-TODO:Stateless
#endif

	/** Whether or not this is a valid emitter handle. */
	NIAGARA_API bool IsValid() const;

	/** Gets the unique id for this handle. */
	NIAGARA_API FGuid GetId() const;

	// HACK!  Data sets used to use the emitter name, but this isn't guaranteed to be unique.  This is a temporary hack
	// to allow the data sets to continue work with using names, but that code needs to be refactored to use the id defined here.
	NIAGARA_API FName GetIdName() const;

	/** Gets the display name for this emitter in the System. */
	NIAGARA_API FName GetName() const;

	/** Sets the display name for this emitter in the System. The system is needed here in order to ensure uniqueness of the name. */
	NIAGARA_API void SetName(FName InName, UNiagaraSystem& InOwnerSystem);

	/** Gets whether or not this emitter is enabled within the System.  Disabled emitters aren't simulated. */
	NIAGARA_API bool GetIsEnabled() const;

	/** Sets whether or not this emitter is enabled within the System.  Disabled emitters aren't simulated. Returns whether or not the enabled state changed. */
	NIAGARA_API bool SetIsEnabled(bool bInIsEnabled, UNiagaraSystem& InOwnerSystem, bool bRecompileIfChanged);

	/** Test to see if the emitter is allowed by scalability or not */
	NIAGARA_API bool IsAllowedByScalability() const;

#if WITH_EDITORONLY_DATA
	/** Test to see if the emitter is enabled on the provided quality level or not. */
	NIAGARA_API bool IsEnabledOnEffectQualityLevel(int32 QualityLevel) const;

	/** Get the PlatformSet for this emitter. */
	NIAGARA_API const FNiagaraPlatformSet* GetPlatformSet() const;

	bool IsIsolated() const {	return bIsolated; }
	void SetIsolated(bool bInIsolated) { bIsolated = bInIsolated; }
#endif

	/** Gets the copied instance of the emitter this handle references. */
	[[nodiscard]] NIAGARA_API const FVersionedNiagaraEmitter GetInstance() const;
	NIAGARA_API void SetInstance(const FVersionedNiagaraEmitter& VersionedData);
	NIAGARA_API void SetInstanceVersion(const FGuid& InVersion);

	NIAGARA_API FVersionedNiagaraEmitterData* GetEmitterData() const;

	NIAGARA_API UNiagaraEmitterBase* GetEmitterBase() const;

	//-TODO:Stateless: Should we return a bass class here / have a factory method to generate the runtime instance?
	UNiagaraStatelessEmitter* GetStatelessEmitter() const { return StatelessEmitter; }
	void SetStatelessEmitter(UNiagaraStatelessEmitter* InEmitter) { StatelessEmitter = InEmitter; }
	//-TODO:Stateless: Should we return a bass class here / have a factory method to generate the runtime instance?

	ENiagaraEmitterMode GetEmitterMode() const { return EmitterMode; }
#if WITH_EDITORONLY_DATA
	NIAGARA_API void SetEmitterMode(UNiagaraSystem& InOwningSystem, ENiagaraEmitterMode InEmitterMode);
	FSimpleMulticastDelegate& OnEmitterModeChanged() { return OnEmitterModeChangedDelegate; }
#endif

	/** Iterate over all the enabled renderers on the emitter */
	NIAGARA_API void ForEachEnabledRendererWithIndex(TFunction<void(const class UNiagaraRendererProperties*, int32)> Delegate) const;

	/** Gets a unique name for this emitter instance for use in scripts and parameter stores etc.*/
	NIAGARA_API FString GetUniqueInstanceName()const;

#if WITH_EDITORONLY_DATA
	/** Determine whether or not the Instance script is in synch with its graph.*/
	NIAGARA_API bool NeedsRecompile() const;

	/** Calls conditional post load on all sub-objects this handle references. */
	NIAGARA_API void ConditionalPostLoad(int32 NiagaraCustomVersion);

	/** Whether or not this handle uses the supplied emitter. */
	NIAGARA_API bool UsesEmitter(const FVersionedNiagaraEmitter& InEmitter) const;
	NIAGARA_API bool UsesEmitter(const FVersionedNiagaraEmitterBase& InEmitter) const;
	NIAGARA_API bool UsesEmitter(const UNiagaraEmitterBase& InEmitter) const;

	NIAGARA_API void ClearEmitter();

	bool GetDebugShowBounds() const { return bDebugShowBounds; }
	void SetDebugShowBounds(bool bShowBounds) { bDebugShowBounds = bShowBounds; }
#endif
public:
	/** A static const invalid handle. */
	static NIAGARA_API const FNiagaraEmitterHandle InvalidHandle;

private:
	/** The display name for this emitter in the System. */
	UPROPERTY()
	FName Name;
	
	/** The id of this emitter handle. */
	UPROPERTY(VisibleAnywhere, Category="Emitter ID")
	FGuid Id;

	// HACK!  Data sets used to use the emitter name, but this isn't guaranteed to be unique.  This is a temporary hack
	// to allow the data sets to continue work with using names, but that code needs to be refactored to use the id defined here.
	UPROPERTY(VisibleAnywhere, Category = "Emitter ID")
	FName IdName;

	/** Whether or not this emitter is enabled within the System.  Disabled emitters aren't simulated. */
	UPROPERTY()
	bool bIsEnabled;

	UPROPERTY()
	ENiagaraEmitterMode EmitterMode = ENiagaraEmitterMode::Standard;

#if WITH_EDITORONLY_DATA
	/** The source emitter this emitter handle was built from. */
	UPROPERTY()
	TObjectPtr<UNiagaraEmitter> Source_DEPRECATED;

	/** An unmodified copy of the emitter this handle references for use when merging change from the source emitter. */
	UPROPERTY()
	TObjectPtr<UNiagaraEmitter> LastMergedSource_DEPRECATED;

	UPROPERTY(Transient)
	bool bIsolated;

	/** The copied instance of the emitter this handle references. */
	UPROPERTY()
	TObjectPtr<UNiagaraEmitter> Instance_DEPRECATED;

	bool bDebugShowBounds = false;
#endif

	/** The copied instance of the emitter this handle references. */
	UPROPERTY()
	FVersionedNiagaraEmitter VersionedInstance;

	//-TODO:Stateless: Should we return a bass class here / have a factory method to generate the runtime instance?
	UPROPERTY()
	TObjectPtr<UNiagaraStatelessEmitter> StatelessEmitter = nullptr;
	//-TODO:Stateless: Should we return a bass class here / have a factory method to generate the runtime instance?

#if WITH_EDITORONLY_DATA
	FSimpleMulticastDelegate OnEmitterModeChangedDelegate;
#endif
	
};
