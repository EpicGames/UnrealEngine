// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEmitter.h"
#include "NiagaraScriptRuntimeData.h"
#include "NiagaraSystem.h"

template<typename TAction>
void FVersionedNiagaraEmitterData::ForEachPlatformSet(const UNiagaraSystem& OwningSystem, const FGuid& EmitterHandleId, TAction Func)
{
	Func(Platforms);

	for (FNiagaraEmitterScalabilityOverride& Override : ScalabilityOverrides.Overrides)
	{
		Func(Override.Platforms);
	}

	for (UNiagaraRendererProperties* Renderer : RendererProperties)
	{
		if (Renderer)
		{
			Renderer->ForEachPlatformSet(Func);
		}
	}

	auto HandleScript = [&OwningSystem, &EmitterHandleId, Func](UNiagaraScript* Script)
		{
			if (Script == nullptr)
			{
				return;
			}

			TSharedPtr<const FNiagaraScriptRuntimeData> ScriptRuntimeData = OwningSystem.GetScriptRuntimeData(EmitterHandleId, Script->GetUsage(), Script->GetUsageId());
			if (ScriptRuntimeData.IsValid() == false)
			{
				return;
			}

			for (const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDataInterfaceInfo : ScriptRuntimeData->GetResolvedDataInterfaces())
			{
				if (UNiagaraDataInterfacePlatformSet* PlatformSetDI = Cast<UNiagaraDataInterfacePlatformSet>(ResolvedDataInterfaceInfo.ResolvedDataInterface))
				{
					Func(PlatformSetDI->Platforms);
				}
			}
		};

	ForEachScript(HandleScript);
}