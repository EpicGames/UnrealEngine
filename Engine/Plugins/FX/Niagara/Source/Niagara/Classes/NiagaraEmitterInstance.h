// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSet.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraEmitter.h"
#include "NiagaraScriptExecutionContext.h"

struct INiagaraComputeDataBufferInterface;
struct FNiagaraComputeExecutionContext;
struct FNiagaraScriptExecutionContext;
class UNiagaraDataInterface;
class UNiagaraParameterCollectionInstance;
class FNiagaraSystemInstance;

class FNiagaraEmitterInstanceImpl;
class FNiagaraStatelessEmitterInstance;

/**
* Base class for different emitter instances
*/
class FNiagaraEmitterInstance
{
	friend class UNiagaraSimCache;

public:
	explicit FNiagaraEmitterInstance(FNiagaraSystemInstance* InParentSystemInstance);
	virtual ~FNiagaraEmitterInstance() = default;

	//~Begin: Define Emitter Interface
	virtual void Init(int32 InEmitterIdx);
	virtual void ResetSimulation(bool bKillExisting = true) = 0;
	virtual void SetEmitterEnable(bool bNewEnableState) = 0;
	virtual void OnPooledReuse() = 0;
	virtual bool HandleCompletion(bool bForce = false) = 0;

	virtual void BindParameters(bool bExternalOnly) = 0;
	virtual void UnbindParameters(bool bExternalOnly) = 0;
	virtual void RebindParameterCollection(UNiagaraParameterCollectionInstance* OldCollection, UNiagaraParameterCollectionInstance* NewCollection) = 0;

	virtual bool ShouldTick() const = 0;
	virtual void PreTick() {}
	virtual void Tick(float DeltaSeconds) = 0;
	//~End: Define Emitter Interface

	FNiagaraSystemInstance* GetParentSystemInstance() const { return ParentSystemInstance; }
	bool IsLocalSpace() const { return bLocalSpace; }
	bool IsDeterministic() const { return bDeterministic; }
	bool NeedsPartialDepthTexture() const { return bNeedsPartialDepthTexture; }
	bool NeedsEarlyViewUniformBuffer() const { return bNeedsEarlyViewUniformBuffer; }
	ENiagaraSimTarget GetSimTarget() const { return SimTarget; }
#if STATS
	TStatId GetEmitterStatID(bool bGameThread, bool bConcurrent) const;
	TStatId GetSystemStatID(bool bGameThread, bool bConcurrent) const;
#endif
	FNiagaraDataSet& GetParticleData() { check(ParticleDataSet); return *ParticleDataSet; }
	const FNiagaraDataSet& GetParticleData() const { check(ParticleDataSet); return *ParticleDataSet; }

	FNiagaraComputeExecutionContext* GetGPUContext() const { return GPUExecContext; }
	INiagaraComputeDataBufferInterface* GetComputeDataBufferInterface() const { return GPUDataBufferInterfaces; }

	ENiagaraExecutionState GetExecutionState() const { return ExecutionState; }
	bool IsActive() const { return ExecutionState == ENiagaraExecutionState::Active; }
	bool IsDisabled() const { return ExecutionState == ENiagaraExecutionState::Disabled; }
	bool IsInactive() const { return ExecutionState == ENiagaraExecutionState::Inactive; }
	bool IsComplete() const { return ExecutionState == ENiagaraExecutionState::Complete || ExecutionState == ENiagaraExecutionState::Disabled; }

	//-TODO:Stateless: Does this need to be virtual?  Can we cache the value after ticking / allocating the data / have a cache value somewhere instead?
	NIAGARA_API virtual int32 GetNumParticles() const;
	int32 GetTotalSpawnedParticles() const { return TotalSpawnedParticles; }

	virtual uint32 GetGpuCountBufferEstimate() const { return 0; }

	bool AreBoundsDynamic() const { return bCachedBoundsDynamic; }
	[[nodiscard]] FBox GetBounds() const { return CachedBounds; }

	void SetSystemFixedBoundsOverride(FBox SystemFixedBounds);

	void SetFixedBounds(const FBox& InLocalBounds);
	[[nodiscard]] FBox GetFixedBounds() const;
	
	[[nodiscard]] bool IsWaitingForPSOCaching() const { return bWaitingForPSO; }
	
#if WITH_EDITORONLY_DATA
	uint32 GetTickTimeCycles() const { return TickTimeCycles; }
	NIAGARA_API bool IsDisabledFromIsolation() const;
#endif

	[[nodiscard]] NIAGARA_API int64 GetTotalBytesUsed() const;

	[[nodiscard]] NIAGARA_API const FNiagaraEmitterHandle& GetEmitterHandle() const;

	FNiagaraParameterStore& GetRendererBoundVariables() { return RendererBindings; }
	const FNiagaraParameterStore& GetRendererBoundVariables() const { return RendererBindings; }

	NIAGARA_API bool GetBoundRendererValue_GT(const FNiagaraVariableBase& InBaseVar, const FNiagaraVariableBase& InSubVar, void* OutValueData) const;

	[[nodiscard]] NIAGARA_API TObjectPtr<UObject> FindBinding(const FNiagaraVariable& InVariable) const;
	[[nodiscard]] NIAGARA_API UNiagaraDataInterface* FindDataInterface(const FNiagaraVariable& InVariable) const;

	[[nodiscard]] FNiagaraEmitterID GetEmitterID() const{ return FNiagaraEmitterID(EmitterIndex); }
	
	virtual void UpdatePSOStatus();

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	////////////////////////////////////////////////////////////////////////////////////////////////////
	//-TODO:Stateless: Consider removing these virtual functions
	// Can we remove these virtual functions
	virtual FNiagaraEmitterInstanceImpl* AsStateful() { return nullptr; }
	virtual FNiagaraStatelessEmitterInstance* AsStateless() { return nullptr; }

	const FNiagaraEmitterInstanceImpl* AsStateful() const { return const_cast<FNiagaraEmitterInstance*>(this)->AsStateful(); }
	const FNiagaraStatelessEmitterInstance* AsStateless() const { return const_cast<FNiagaraEmitterInstance*>(this)->AsStateless(); }

	UNiagaraEmitter* GetEmitter() const { return VersionedEmitter.Emitter; }
	const FVersionedNiagaraEmitter& GetVersionedEmitter() const { return VersionedEmitter; }
	//FVersionedNiagaraEmitterData* GetVersionedEmitterData() const { return VersionedEmitter.GetEmitterData(); }

	template<typename TAction>
	void ForEachEnabledRenderer(TAction Func) const
	{
		for (const UNiagaraRendererProperties* Renderer : GetRenderers())
		{
			if (Renderer && Renderer->GetIsEnabled() && Renderer->IsSimTargetSupported(SimTarget))
			{
				Func(Renderer);
			}
		}
	}

	template<typename TAction>
	void ForEachEnabledRenderer(TAction Func)
	{
		for (UNiagaraRendererProperties* Renderer : GetRenderers())
		{
			if (Renderer && Renderer->GetIsEnabled() && Renderer->IsSimTargetSupported(SimTarget))
			{
				Func(Renderer);
			}
		}
	}

	[[nodiscard]] const UNiagaraRendererProperties* GetRenderer(int32 i) const
	{
		TConstArrayView<UNiagaraRendererProperties*> LocalRenderers = GetRenderers();
		return LocalRenderers.IsValidIndex(i) ? LocalRenderers[i] : nullptr;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
	//-TODO:Stateless: Should we cache this data? clean up const casting
protected:
	virtual TConstArrayView<UNiagaraRendererProperties*> GetRenderers() const = 0;
	TArrayView<UNiagaraRendererProperties*> GetRenderers()
	{
		TConstArrayView<UNiagaraRendererProperties*> Renderers = static_cast<const FNiagaraEmitterInstance*>(this)->GetRenderers();
		return MakeArrayView(const_cast<UNiagaraRendererProperties**>(Renderers.GetData()), Renderers.Num());
	}

public:
	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Deprecated functionality code needs to be updated
	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	NIAGARA_API FNiagaraScriptExecutionContext& GetSpawnExecutionContext();

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	NIAGARA_API FNiagaraScriptExecutionContext& GetUpdateExecutionContext();

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	NIAGARA_API TArrayView<FNiagaraScriptExecutionContext> GetEventExecutionContexts();

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	inline  FVersionedNiagaraEmitter GetCachedEmitter() const { return VersionedEmitter; }

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	inline FVersionedNiagaraEmitterData* GetCachedEmitterData()const { return VersionedEmitter.GetEmitterData(); }

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	NIAGARA_API TArray<FNiagaraSpawnInfo>& GetSpawnInfo();

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	void SetParticleComponentActive(FObjectKey ComponentKey, int32 ParticleID) const;

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	bool IsParticleComponentActive(FObjectKey ComponentKey, int32 ParticleID) const;

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	NIAGARA_API bool IsReadyToRun() const;

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	NIAGARA_API bool HasTicked() const;

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	NIAGARA_API void SetExecutionState(ENiagaraExecutionState InState);

#if WITH_EDITOR
	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	NIAGARA_API void CalculateFixedBounds(const FTransform& ToWorldSpace);
#endif

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	NIAGARA_API float GetTotalCPUTimeMS() const;

	UE_DEPRECATED(5.4, "Please update your code to handle execution path for different emitter types")
	NIAGARA_API FName GetCachedIDName() const;

	UE_DEPRECATED(5.4, "Please update your code to use GetParticleData")
	FNiagaraDataSet& GetData() { check(ParticleDataSet); return *ParticleDataSet; }
	UE_DEPRECATED(5.4, "Please update your code to use GetParticleData")
	const FNiagaraDataSet& GetData() const { check(ParticleDataSet); return *ParticleDataSet; }
	////////////////////////////////////////////////////////////////////////////////////////////////////

protected:
	FNiagaraSystemInstance*				ParentSystemInstance = nullptr;
	FNiagaraDataSet*					ParticleDataSet = nullptr;
	FNiagaraComputeExecutionContext*	GPUExecContext = nullptr;
	INiagaraComputeDataBufferInterface* GPUDataBufferInterfaces = nullptr;
	FVersionedNiagaraEmitter			VersionedEmitter = {};

	FNiagaraParameterStore				RendererBindings;

	int32								EmitterIndex = INDEX_NONE;
	int32								TotalSpawnedParticles = 0;

	uint8								bLocalSpace : 1 = true;
	uint8								bDeterministic : 1 = true;
	uint8								bNeedsPartialDepthTexture : 1 = false;
	uint8								bNeedsEarlyViewUniformBuffer : 1 = false;
	ENiagaraSimTarget					SimTarget = ENiagaraSimTarget::CPUSim;
	ENiagaraExecutionState				ExecutionState = ENiagaraExecutionState::Active;

	bool								bCachedBoundsDynamic = false;
	bool								bWaitingForPSO = false;
	FBox								CachedBounds = FBox(ForceInit);
	FBox								CachedSystemFixedBounds = FBox(ForceInit);

	mutable FRWLock						FixedBoundsGuard;
	FBox								FixedBounds = FBox(ForceInit);

#if WITH_EDITORONLY_DATA
	uint32								TickTimeCycles = 0;
#endif
};

using FNiagaraEmitterInstanceRef = TSharedRef<FNiagaraEmitterInstance, ESPMode::ThreadSafe>;
using FNiagaraEmitterInstancePtr = TSharedPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe>;
