// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraEmitterInstance.h: Niagara emitter simulation class
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "NiagaraCommon.h"
#include "NiagaraEvents.generated.h"

struct FNiagaraEventReceiverProperties;

#define NIAGARA_BUILTIN_EVENTNAME_COLLISION FName("NiagaraSystem_Collision")
#define NIAGARA_BUILTIN_EVENTNAME_SPAWN FName("Spawn")
#define NIAGARA_BUILTIN_EVENTNAME_DEATH FName("Death")

/**
 *	Type struct for collision event payloads; collision event data set is based on this
 *  TODO: figure out how we can pipe attributes from the colliding particle in here
 */
USTRUCT()
struct FNiagaraCollisionEventPayload
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FVector CollisionPos = FVector(ForceInitToZero);
	UPROPERTY()
	FVector CollisionNormal = FVector(ForceInitToZero);
	UPROPERTY()
	FVector CollisionVelocity = FVector(ForceInitToZero);
	UPROPERTY()
	int32 ParticleIndex = 0;
	UPROPERTY()
	int32 PhysicalMaterialIndex = 0;
};


class FNiagaraEmitterInstance;

/**
Base class for actions that an event receiver will perform at the emitter level.
*/
UCLASS(abstract)
class UNiagaraEventReceiverEmitterAction : public UObject
{
	GENERATED_BODY()
public:
	virtual void PerformAction(FNiagaraEmitterInstance& OwningSim, const FNiagaraEventReceiverProperties& OwningEventReceiver){}
};

UCLASS(EditInlineNew)
class UNiagaraEventReceiverEmitterAction_SpawnParticles : public UNiagaraEventReceiverEmitterAction
{
	GENERATED_BODY()

public:

	/** Number of particles to spawn per event received. */
	UPROPERTY(EditAnywhere, Category = "Spawn")
	uint32 NumParticles;

	virtual void PerformAction(FNiagaraEmitterInstance& OwningSim, const FNiagaraEventReceiverProperties& OwningEventReceiver)override;
};
