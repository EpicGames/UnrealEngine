// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraGPUInstanceCountManager.h: GPU particle count handling
==============================================================================*/
#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "RHI.h"
#include "NiagaraDrawIndirect.h"
#include "RHIGPUReadback.h"

class FRHIGPUBufferReadback;
class FNiagaraGpuComputeDispatchInterface;

// The number of GPU renderers registered in the instance count manager.
// Shared between the manager and the renderers.
class FNiagaraGPURendererCount : public FRefCountedObject
{
public:
	int32 Value = 0;
};

inline uint32 GetTypeHash(const FNiagaraDrawIndirectArgGenTaskInfo& Info)
{
	return HashCombine(Info.InstanceCountBufferOffset, HashCombine(Info.NumIndicesPerInstance, HashCombine(Info.StartIndexLocation, Info.Flags)));
}

namespace ENiagaraGPUCountUpdatePhase
{
	enum Type
	{
		PreOpaque,
		PostOpaque,
		Max
	};
}

/**
 * A manager that handles the buffer containing the GPU particle count.
 * Also provides related functionalities like the generation of the draw indirect buffer.
 */
class FNiagaraGPUInstanceCountManager
{
public:
	struct FIndirectArgSlot
	{
		FBufferRHIRef Buffer;
		FShaderResourceViewRHIRef SRV;
		uint32 Offset = INDEX_NONE;

		FIndirectArgSlot() {}
		FIndirectArgSlot(FBufferRHIRef InBuffer, FShaderResourceViewRHIRef InSRV, uint32 InOffset) : Buffer(InBuffer), SRV(InSRV), Offset(InOffset) {}
		inline bool IsValid() const { return Offset != INDEX_NONE; }
	};

	FNiagaraGPUInstanceCountManager(ERHIFeatureLevel::Type FeatureLevel);
	~FNiagaraGPUInstanceCountManager();

	// Init resource for the first time.
	void InitRHI(FRHICommandListBase& RHICmdList);
	// Free resources.
	void ReleaseRHI();

	const FRWBuffer& GetInstanceCountBuffer() const
	{
		check(UsedInstanceCounts <= AllocatedInstanceCounts); // Can't resize after after the buffer gets bound.
		return CountBuffer;
	}
	
	/** Acquire an entry from the free list, assumes this comes from being presized. */
	uint32 AcquireEntry();
	/** Acquire an entry, this will either come from the free list or reallocate the buffer. */
	uint32 AcquireOrAllocateEntry(FRHICommandListImmediate& RHICmdList);

	/** Free the entry and reset it to INDEX_NONE if valid. */
	void FreeEntry(uint32& BufferOffset);
	/** Free and array of entries, you are expected to reset or change to INDEX_NONE. */
	void FreeEntryArray(TConstArrayView<uint32> EntryArray);

	bool CanAcquireCulledEntry() const
	{
		return !bAcquiredCulledCounts;
	}

	uint32 AcquireCulledEntry()
	{
		check(!bAcquiredCulledCounts);
		return RequiredCulledCounts++;
	}

	FRWBuffer* AcquireCulledCountsBuffer(FRHICommandListImmediate& RHICmdList);

	const uint32* GetGPUReadback();
	void ReleaseGPUReadback();
	void EnqueueGPUReadback(FRHICommandListImmediate& RHICmdList);
	bool HasPendingGPUReadback() const;

	/** Add a draw indirect task to generate the draw indirect args. Returns the draw indirect arg buffer offset. */
	NIAGARA_API FIndirectArgSlot AddDrawIndirect(FRHICommandListBase& RHICmdList, uint32 InstanceCountBufferOffset, uint32 NumIndicesPerInstance, uint32 StartIndexLocation, bool bIsInstancedStereoEnabled, bool bCulled, ENiagaraGpuComputeTickStage::Type ReadyTickStage);

	// Resize instance count and draw indirect buffers to ensure it is big enough to hold all draw indirect args.
	void ResizeBuffers(FRHICommandListImmediate& RHICmdList, int32 ReservedInstanceCounts);

	void FlushIndirectArgsPool(FRHICommandListBase& RHICmdList);

	// Generate the draw indirect buffers, and reset all release counts.
	void UpdateDrawIndirectBuffers(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, FRHICommandList& RHICmdList, ENiagaraGPUCountUpdatePhase::Type CountPhase);

	static const ERHIAccess kCountBufferDefaultState;
	static const ERHIAccess kIndirectArgsDefaultState;

	bool HasEntriesPendingFree() const { return InstanceCountClearTasks.Num() > 0; }

	void CopyToMultiViewCountBuffer(FRHICommandListImmediate& RHICmdList);

	void ProcessInitInstanceCountTasks(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, FRHICommandList& RHICmdList);
	void AddInstanceCountInitTask(uint32 Offset, uint32 Value);

protected:
	struct FIndirectArgsPoolEntry
	{
		FRWBuffer	Buffer;

		uint32		AllocatedEntries = 0;
		uint32		UsedEntriesTotal = 0;
		uint32		UsedEntries[ENiagaraGPUCountUpdatePhase::Max] = {};
	};

	using FIndirectArgsPoolEntryPtr = TUniquePtr<FIndirectArgsPoolEntry>;

	void ReleaseCounts();

	UE::FMutex AddDrawIndirectGuard;

	ERHIFeatureLevel::Type FeatureLevel;

	/** The current used instance counts allocated from FNiagaraDataBuffer::AllocateGPU() */
	int32 UsedInstanceCounts = 0;
	/** The allocated instance counts in CountBuffer */
	int32 AllocatedInstanceCounts = 0;

	/** The number of culled instance counts needed from view culling */
	std::atomic<int32> RequiredCulledCounts = 0;
	/** The allocated instance counts in the culled count buffer*/
	int32 AllocatedCulledCounts = 0;
	/** Whether or not the culled counts were acquired this frame */
	bool bAcquiredCulledCounts = false;
	/** Used to track RHI access for the culled counts buffer. */
	ERHIAccess CulledCountsRHIAccess = ERHIAccess::Unknown;

	/** A buffer holding the each emitter particle count after a simulation tick. */
	FRWBuffer CountBuffer;
	FRWBuffer CulledCountBuffer;
	TArray<uint32> FreeEntries;
	FRHIGPUBufferReadback* CountReadback = nullptr;
	int32 CountReadbackSize = 0;

	/** Mirrored copy of CountBuffer for multi-view rendering. */
	int32 MultiViewAllocatedInstanceCounts = 0;
	FRWBuffer MultiViewCountBuffer;

	/** The list of all draw indirected tasks that are to be run in UpdateDrawIndirectBuffers() */
	TArray<FNiagaraDrawIndirectArgGenTaskInfo> DrawIndirectArgGenTasks[ENiagaraGPUCountUpdatePhase::Max];

	/** Set to allow de-duplication of FNiagaraDrawIndirectArgGenTaskInfo. */
	struct FNiagaraDrawIndirectArgGenSlotInfo
	{
		uint32 PoolIndex = INDEX_NONE;
		uint32 BufferOffset = INDEX_NONE;
	};
	TMap<FNiagaraDrawIndirectArgGenTaskInfo, FNiagaraDrawIndirectArgGenSlotInfo> DrawIndirectArgMap;

	/** The list of all instance count clear tasks that are to be run in UpdateDrawIndirectBuffers() */
	TArray<uint32> InstanceCountClearTasks;

	struct FInstanceCountInitTask
	{
		uint32 Offset = INDEX_NONE;
		uint32 Value = INDEX_NONE;
	};
	/** The list of initialization tasks we process in ProcessInitializationTasks. These initialize instance counts with data from the CPU. */
	TArray<uint32> InstanceCountInitTasks;

	/** Buffers holding drawindirect data to render GPU emitter renderers. */
	TArray<FIndirectArgsPoolEntryPtr> DrawIndirectPool;
	uint32 DrawIndirectLowWaterFrames = 0;

	UE::FMutex	AcquireEntryGuard;
};
