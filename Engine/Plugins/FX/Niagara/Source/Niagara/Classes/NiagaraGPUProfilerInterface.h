// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraEmitter.h"
#include "NiagaraSystemGpuComputeProxy.h"

struct FNiagaraComputeInstanceData;
struct FNiagaraSimStageData;

//////////////////////////////////////////////////////////////////////////
// Public API for tracking GPU time when the profiler is enabled
struct FNiagaraGpuProfileEvent
{
#if WITH_NIAGARA_GPU_PROFILER
	friend class FNiagaraGPUProfiler;

	NIAGARA_API explicit FNiagaraGpuProfileEvent(const FNiagaraComputeInstanceData& InstanceData, const FNiagaraSimStageData& SimStageData, const bool bFirstInstanceData);
	NIAGARA_API explicit FNiagaraGpuProfileEvent(const FNiagaraComputeInstanceData& InstanceData, FName CustomStageName);

private:
	uint32									bUniqueInstance : 1;
	TWeakObjectPtr<class USceneComponent>	OwnerComponent;
	FVersionedNiagaraEmitterWeakPtr			OwnerEmitter;
	FName									StageName;
#else
	explicit FNiagaraGpuProfileEvent(const FNiagaraComputeInstanceData& InstanceData, const FNiagaraSimStageData& SimStageData, const bool bFirstInstanceData) {}
	explicit FNiagaraGpuProfileEvent(const FNiagaraComputeInstanceData& InstanceData, FName CustomStageName) {}
#endif
};

struct FNiagaraGpuProfileScope
{
#if WITH_NIAGARA_GPU_PROFILER
	NIAGARA_API explicit FNiagaraGpuProfileScope(FRHICommandList& RHICmdList, const class FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, const FNiagaraGpuProfileEvent& Event);
	NIAGARA_API ~FNiagaraGpuProfileScope();

private:
	FRHICommandList& RHICmdList;
	class FNiagaraGPUProfiler* GPUProfiler = nullptr;
#else
	explicit FNiagaraGpuProfileScope(FRHICommandList& RHICmdList, const class FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, const FNiagaraGpuProfileEvent& Event) {}
	~FNiagaraGpuProfileScope() = default;
#endif
};

#if WITH_NIAGARA_GPU_PROFILER
//////////////////////////////////////////////////////////////////////////
/** Results generated when the frame is ready and sent to that game thread */
struct FNiagaraGpuFrameResults : public TSharedFromThis<FNiagaraGpuFrameResults, ESPMode::ThreadSafe>
{
	struct FDispatchResults
	{
		uint32									bUniqueInstance : 1;		// Set only once for all dispatches from an instance across all ticks
		TWeakObjectPtr<class USceneComponent>	OwnerComponent;				// Optional pointer back to owning Component
		FVersionedNiagaraEmitterWeakPtr			OwnerEmitter;				// Optional pointer back to owning Emitter
		FName									StageName;					// Generally the simulation stage but may be a DataInterface name
		uint64									DurationMicroseconds;		// Duration in microseconds of the dispatch
	};

	uintptr_t					OwnerContext = 0;
	int32						TotalDispatches = 0;
	int32						TotalDispatchGroups = 0;
	uint64						TotalDurationMicroseconds = 0;
	TArray<FDispatchResults>	DispatchResults;
};

using FNiagaraGpuFrameResultsPtr = TSharedPtr<FNiagaraGpuFrameResults, ESPMode::ThreadSafe>;

//////////////////////////////////////////////////////////////////////////
// Allows various systems to listen to profiler results
struct FNiagaraGpuProfilerListener
{
	NIAGARA_API FNiagaraGpuProfilerListener();
	NIAGARA_API ~FNiagaraGpuProfilerListener();

	NIAGARA_API void SetEnabled(bool bEnabled);
	NIAGARA_API void SetHandler(TFunction<void(const FNiagaraGpuFrameResultsPtr&)> Function);
	bool IsEnabled() const { return bEnabled; }

private:
	bool			bEnabled = false;
	FDelegateHandle	GameThreadHandler;
};

//////////////////////////////////////////////////////////////////////////
/** Public API to Niagara GPU Profiling. */
class FNiagaraGPUProfilerInterface
{
	friend FNiagaraGpuProfilerListener;
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFrameResults, const FNiagaraGpuFrameResultsPtr&);

public:
	static FOnFrameResults& GetOnFrameResults_GameThread() { check(IsInGameThread()); return OnFrameResults_GameThread; }
	static FOnFrameResults& GetOnFrameResults_RenderThread() { check(IsInRenderingThread()); return OnFrameResults_RenderThread; }

protected:
	void PostResults(const FNiagaraGpuFrameResultsPtr& FrameResults);

protected:
	static std::atomic<int>	NumReaders;
	static FOnFrameResults	OnFrameResults_GameThread;
	static FOnFrameResults	OnFrameResults_RenderThread;
};

#endif //WITH_NIAGARA_GPU_PROFILER
