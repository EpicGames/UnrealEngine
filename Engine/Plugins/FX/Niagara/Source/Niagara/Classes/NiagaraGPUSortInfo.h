// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraGPUSortInfo.h: GPU particle sorting helper
==============================================================================*/
#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHIResources.h"
#include "GPUSortManager.h"
#include "NiagaraCommon.h"
#include "NiagaraGPUSortInfo.generated.h"

UENUM()
enum class ENiagaraSortMode : uint8
{
	/** Perform no additional sorting prior to rendering.*/
	None,
	/** Sort by depth to the camera's near plane.*/
	ViewDepth,
	/** Sort by distance to the camera's origin.*/
	ViewDistance,
	/**
	Sort according to particles CustomSortingBinding (defaults to Particles.NormalizedAge).
	Lower values will be sorted before higher values, i.e. 1 would draw on top of 0.
	*/
	CustomAscending,
	/**
	Sort according to particles CustomSortingBinding (defaults to Particles.NormalizedAge).
	Higher values will be sorted before lower values, i.e. 0 would draw on top of 1.
	*/
	CustomDecending,
};

struct FNiagaraGPUSortInfo
{
	static constexpr uint32 MaxCullPlanes = 12;

#if STATS
	// Owner name, used to find who instigated sorting
	FName OwnerName;
#endif
	// The number of particles in the system.
	int32 ParticleCount = 0;
	// How the particles should be sorted.
	ENiagaraSortMode SortMode = ENiagaraSortMode::None;
	// On which attribute to base the sorting
	int32 SortAttributeOffset = INDEX_NONE;	
	// The data buffers that hold the particle attributes and their strides
	FShaderResourceViewRHIRef ParticleDataFloatSRV;
	FShaderResourceViewRHIRef ParticleDataHalfSRV;
	FShaderResourceViewRHIRef ParticleDataIntSRV;
	uint32 FloatDataStride = 0;
	uint32 HalfDataStride = 0;
	uint32 IntDataStride = 0;
	// The actual GPU sim particle count. Needed to get an exact match on the index list.
	FShaderResourceViewRHIRef GPUParticleCountSRV;
	uint32 GPUParticleCountOffset = INDEX_NONE;
	uint32 CulledGPUParticleCountOffset = INDEX_NONE;
	// View data.
	FVector ViewOrigin = FVector(0, 0, 0);
	FVector ViewDirection = FVector(0, 0, 1);
	// Culling/Visibility data
	bool bEnableCulling = false;
	int32 CullPositionAttributeOffset = INDEX_NONE;
	int32 CullOrientationAttributeOffset = INDEX_NONE;
	int32 CullScaleAttributeOffset = INDEX_NONE;
	int32 RendererVisTagAttributeOffset = INDEX_NONE;
	int32 RendererVisibility = 0;
	int32 MeshIndexAttributeOffset = INDEX_NONE;
	int32 MeshIndex = 0;
	FSphere LocalBSphere = FSphere(0);
	FVector CullingWorldSpaceOffset = FVector(0, 0, 0);
	FVector2f DistanceCullRange { 0.0f, FLT_MAX };
	FVector4f LODScreenSize = FVector4f(0.0f, 2.0f, 1.0f, 0.0f);
	TArray<FPlane, TFixedAllocator<MaxCullPlanes>> CullPlanes;

	// The GPUSortManager bindings for this sort task.
	FGPUSortManager::FAllocationInfo AllocationInfo;
	// The sort constraints for the task in GPUSortManager.
	EGPUSortFlags SortFlags = EGPUSortFlags::None;

	// Set the SortFlags based on the emitter and material constraints.
	inline void SetSortFlags(bool bHighPrecisionKeys, ENiagaraGpuComputeTickStage::Type ReadyTickStage)
	{
		SortFlags =
			EGPUSortFlags::ValuesAsInt32 | 
			(bHighPrecisionKeys ? EGPUSortFlags::HighPrecisionKeys : EGPUSortFlags::LowPrecisionKeys) |
			(ReadyTickStage == ENiagaraGpuComputeTickStage::PostOpaqueRender ? EGPUSortFlags::SortAfterPostRenderOpaque | EGPUSortFlags::KeyGenAfterPostRenderOpaque : EGPUSortFlags::SortAfterPreRender | EGPUSortFlags::KeyGenAfterPreRender);
	}
};