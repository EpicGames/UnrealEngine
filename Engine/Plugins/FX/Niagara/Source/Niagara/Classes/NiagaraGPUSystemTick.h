// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraComputeExecutionContext.h"
#include "RHIGPUReadback.h"

struct FNiagaraComputeDataInterfaceInstanceData
{
	UE_NONCOPYABLE(FNiagaraComputeDataInterfaceInstanceData);
	FNiagaraComputeDataInterfaceInstanceData() {}
#if NIAGARA_VALIDATE_NDIPROXY_REFS
	~FNiagaraComputeDataInterfaceInstanceData();
#endif

	void* PerInstanceDataForRT = nullptr;
	TMap<FNiagaraDataInterfaceProxy*, int32> InterfaceProxiesToOffsets;
	uint32 PerInstanceDataSize = 0;
	uint32 Instances = 0;
};

struct FNiagaraComputeInstanceData
{
	UE_NONCOPYABLE(FNiagaraComputeInstanceData);

	struct FPerStageInfo
	{
		uint16		SimStageIndex = 0;
		uint16		NumIterations = 0;
		uint16		LoopIndex = 0;
		uint16		NumLoops = 0;
		FIntVector	ElementCountXYZ = FIntVector::ZeroValue;
	};

	FNiagaraGpuSpawnInfo SpawnInfo;
	uint8* EmitterParamData = nullptr;
	uint8* ExternalParamData = nullptr;
	uint32 ExternalParamDataSize = 0;
	FNiagaraComputeExecutionContext* Context = nullptr;
	TArray<FNiagaraDataInterfaceProxy*> DataInterfaceProxies;
	TArray<FNiagaraDataInterfaceProxyRW*> IterationDataInterfaceProxies;
	TArray<FPerStageInfo, TInlineAllocator<1>> PerStageInfo;
	uint32 ParticleCountFence = INDEX_NONE;
	uint32 TotalDispatches = 0;
	uint32 bResetData : 1 = false;
	uint32 bStartNewOverlapGroup : 1 = false;
	uint32 bHasMultipleStages : 1 = false;

	FNiagaraComputeInstanceData() = default;
	bool IsOutputStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 CurrentStage) const;
	bool IsInputStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 CurrentStage) const;
	bool IsIterationStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 CurrentStage) const;
	FNiagaraDataInterfaceProxyRW* FindIterationInterface(uint32 SimulationStageIndex) const;
};

/*
	Represents all the information needed to dispatch a single tick of a FNiagaraSystemInstance.
	This object will be created on the game thread and passed to the renderthread.

	It contains the PerInstance data buffer for every DataInterface referenced by the system as well
	as the Data required to dispatch updates for each Emitter in the system.

	DataInterface data is packed tightly. It includes a TMap that associates the data interface with
	the offset into the packed buffer. At that offset is the Per-Instance data for this System.

	InstanceData_ParamData_Packed packs FNiagaraComputeInstanceData and ParamData into one buffer.
	There is padding after the array of FNiagaraComputeInstanceData so we can upload ParamData directly into a UniformBuffer
	(it is 16 byte aligned).

*/
class FNiagaraGPUSystemTick
{
public:
	void Init(FNiagaraSystemInstance* InSystemInstance);
	void Destroy();

	inline TArrayView<FNiagaraComputeInstanceData> GetInstances() const
	{
		return MakeArrayView(reinterpret_cast<FNiagaraComputeInstanceData*>(InstanceData_ParamData_Packed), InstanceCount);
	};

	inline void GetGlobalParameters(const FNiagaraComputeInstanceData& InstanceData, void* OutputParameters) const
	{
		const uint32 ParamSize = sizeof(FNiagaraGlobalParameters) * (InstanceData.Context->HasInterpolationParameters ? 2 : 1);
		FMemory::Memcpy(OutputParameters, GlobalParamData, ParamSize);
	}

	inline void GetSystemParameters(const FNiagaraComputeInstanceData& InstanceData, void* OutputParameters) const
	{
		const uint32 ParamSize = sizeof(FNiagaraSystemParameters) * (InstanceData.Context->HasInterpolationParameters ? 2 : 1);
		FMemory::Memcpy(OutputParameters, SystemParamData, ParamSize);
	}

	inline void GetOwnerParameters(const FNiagaraComputeInstanceData& InstanceData, void* OutputParameters) const
	{
		const uint32 ParamSize = sizeof(FNiagaraOwnerParameters) * (InstanceData.Context->HasInterpolationParameters ? 2 : 1);
		FMemory::Memcpy(OutputParameters, OwnerParamData, ParamSize);
	}

	inline void GetEmitterParameters(const FNiagaraComputeInstanceData& InstanceData, void* OutputParameters) const
	{
		const uint32 ParamSize = sizeof(FNiagaraEmitterParameters) * (InstanceData.Context->HasInterpolationParameters ? 2 : 1);
		FMemory::Memcpy(OutputParameters, InstanceData.EmitterParamData, ParamSize);
	}

	void BuildUniformBuffers();

	inline FRHIUniformBuffer* GetExternalUniformBuffer(const FNiagaraComputeInstanceData& InstanceData, bool bPrevious) const
	{
		const int32 InstanceIndex = UE_PTRDIFF_TO_INT32(&InstanceData - GetInstances().GetData());
		const int32 BufferIndex = InstanceIndex + (bPrevious ? InstanceCount : 0);
		return ExternalUnformBuffers_RT[BufferIndex];
	}

	bool IsValid()const{ return !bHasErrors; }

public:
	// Transient data used by the RT
	TArray<FUniformBufferRHIRef> ExternalUnformBuffers_RT;

	// data assigned by GT
	FNiagaraSystemInstanceID SystemInstanceID = 0LL;						//-TODO: Remove?
	class FNiagaraSystemGpuComputeProxy* SystemGpuComputeProxy = nullptr;	//-TODO: Can we remove this?
	FNiagaraComputeDataInterfaceInstanceData* DIInstanceData = nullptr;
	uint8* InstanceData_ParamData_Packed = nullptr;
	uint8* GlobalParamData = nullptr;
	uint8* SystemParamData = nullptr;
	uint8* OwnerParamData = nullptr;
	uint32 InstanceCount = 0;
	uint32 TotalDispatches = 0;
	bool bIsFinalTick = false;
	bool bHasInterpolatedParameters = false;
	
	bool bHasErrors = false;

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	// Debugging only
	TConstArrayView<FNiagaraComputeInstanceData> InstanceDataDebuggingOnly;
#endif
};
