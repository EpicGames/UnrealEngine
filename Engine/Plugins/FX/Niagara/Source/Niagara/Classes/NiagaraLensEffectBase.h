// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "NiagaraActor.h"
#include "Camera/CameraLensEffectInterface.h"

#include "NiagaraLensEffectBase.generated.h"

class APlayerCameraManager;

/**
 * Niagara equivalent of AEmitterCameraLensEffectBase.
 * This class is supported by APlayerCameraManager (via ICameraLensEffectInterface) as a camera lens effect.
 */
UCLASS(abstract, Blueprintable, hideCategories=(NiagaraActorActivation, "Components|Activation", Input, Collision, "Game|Damage"), MinimalAPI)
class ANiagaraLensEffectBase : public ANiagaraActor, public ICameraLensEffectInterface
{
	GENERATED_BODY()


protected:
	NIAGARA_API ANiagaraLensEffectBase(const FObjectInitializer& ObjectInitializer);

	/**
	 * Relative offset from the camera (where X is out from the camera)
	 * Might be changed slightly when the FOV is different than expected.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Positioning And Scale")
	FTransform DesiredRelativeTransform;

	/**
	 * FOVs that differ from this may cause adjustments in positioning.
	 * This is the FOV which the effect was tested with.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Positioning And Scale")
	float BaseAuthoredFOV;

	/**
	 * Are multiple instances of the effect allowed? 
	 * When disabled, a single instance of the effect may be retriggered!
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Effect Activation")
	uint8 bAllowMultipleInstances:1;

	/**
	 * When an instance of this effect is retriggered (because more than one instance is not allowed)
	 * should the particle system be explicitly reset? Activate(bReset = true);
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Effect Activation", meta = (EditCondition = "!bAllowMultipleInstances"))
	uint8 bResetWhenRetriggered:1;
	
	/** 
	 *  If an effect class in this array is currently playing, do not play this effect.
	 *  Useful for preventing multiple similar or expensive camera effects from playing simultaneously.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Effect Activation", meta = (MustImplement = "/Script/Engine.CameraLensEffectInterface"))
	TArray<TSubclassOf<AActor>> EmittersToTreatAsSame;

	UPROPERTY()
	TObjectPtr<APlayerCameraManager> OwningCameraManager;

protected:

	//~ Begin ICameraLensEffectInterface interface
	NIAGARA_API virtual const FTransform& GetRelativeTransform() const override;
	NIAGARA_API virtual float GetBaseFOV() const override;
	NIAGARA_API virtual bool ShouldAllowMultipleInstances() const override;
	NIAGARA_API virtual bool ResetWhenTriggered() const override;
	NIAGARA_API virtual bool ShouldTreatEmitterAsSame(TSubclassOf<AActor> OtherEmitter) const override;
	NIAGARA_API virtual void NotifyWillBePooled() override;
	NIAGARA_API virtual void AdjustBaseFOV(float NewFOV);
	NIAGARA_API virtual void RegisterCamera(APlayerCameraManager* CameraManager) override;
	NIAGARA_API virtual void NotifyRetriggered() override;
	NIAGARA_API virtual void ActivateLensEffect() override;
	NIAGARA_API virtual void DeactivateLensEffect() override;
	NIAGARA_API virtual bool IsLooping() const override;
	//~ End ICameraLensEffectInterface interface

	NIAGARA_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
};
