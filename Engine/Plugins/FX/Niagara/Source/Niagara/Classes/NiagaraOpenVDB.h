// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "NiagaraUseOpenVDB.h"

#if UE_USE_OPENVDB

THIRD_PARTY_INCLUDES_START

UE_PUSH_MACRO("check")
#undef check

#ifdef _MSC_VER
#pragma warning (push)
#pragma warning (disable : 4146) // unary minus operator applied to unsigned type, result still unsigned
#pragma warning (disable : 6297) // Arithmetic overflow:  32-bit value is shifted, then cast to 64-bit value.  Results might not be an expected value.
#endif

#ifndef M_PI
#define M_PI    3.14159265358979323846
#define LOCAL_M_PI 1
#endif

#ifndef M_PI_2
#define M_PI_2  1.57079632679489661923 // pi/2
#define LOCAL_M_PI_2 1
#endif

#define BOOST_ALLOW_DEPRECATED_HEADERS

#include <openvdb/openvdb.h>
#include <openvdb/io/Stream.h>
#include <openvdb/io/File.h>
#include <openvdb/tools/Prune.h>
#include <openvdb/tools/Dense.h>
#include <openvdb/math/Vec4.h>
#include <openvdb/math/Half.h>

using Vec4s = openvdb::Vec4s;
using Vec4STree = openvdb::tree::Tree4<Vec4s, 5, 4, 3>::Type;
using Vec4SGrid = openvdb::Grid<Vec4STree>;
using Vec4SDense = openvdb::tools::Dense<Vec4s, openvdb::tools::MemoryLayout::LayoutXYZ>;

using Vec4h = openvdb::v8_1::math::Vec4<openvdb::math::internal::half>;
OPENVDB_IS_POD(Vec4h)

using Vec4HTree = openvdb::tree::Tree4<Vec4h, 5, 4, 3>::Type;
using Vec4HGrid = openvdb::Grid<Vec4HTree>;
using Vec4HDense = openvdb::tools::Dense<Vec4h, openvdb::tools::MemoryLayout::LayoutXYZ>;

using Vec4 = Vec4h;
using Vec4Tree = openvdb::tree::Tree4<Vec4, 5, 4, 3>::Type;
using Vec4Grid = openvdb::Grid<Vec4Tree>;
using Vec4Dense = openvdb::tools::Dense<Vec4, openvdb::tools::MemoryLayout::LayoutXYZ>;
using Vec4GridPtr = Vec4Grid::Ptr;
using Vec4DensePtr = Vec4Dense::Ptr;


#if LOCAL_M_PI
#undef M_PI
#endif

#if LOCAL_M_PI_2
#undef M_PI_2
#endif

#ifdef _MSC_VER
#pragma warning (pop)
#endif

UE_POP_MACRO("check")

THIRD_PARTY_INCLUDES_END

#endif