// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraParameterStore.h"
#include "NiagaraCommon.h"
#include "NiagaraCompileHash.h"
#include "NiagaraParameterCollection.generated.h"

class UMaterialParameterCollection;
class UMaterialParameterCollectionInstance;
class UNiagaraParameterCollection;

/**
 * Can be used to override selected parameters from a Niagara parameter collection with another value.
 * The values in the parameter collection instance can be set from Blueprint or C++, same as the regular parameter collection. 
 */
UCLASS(BlueprintType, MinimalAPI)
class UNiagaraParameterCollectionInstance : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	NIAGARA_API virtual ~UNiagaraParameterCollectionInstance();

	//~UObject interface
	NIAGARA_API virtual void PostLoad()override;
	//~UObject interface
	
	NIAGARA_API bool IsDefaultInstance()const;
	
	NIAGARA_API void SetParent(UNiagaraParameterCollection* InParent);
	UNiagaraParameterCollection* GetParent()const { return Collection; }

	FNiagaraParameterStore& GetParameterStore() { return ParameterStorage; }
	const FNiagaraParameterStore& GetParameterStore() const { return ParameterStorage; }

	NIAGARA_API bool AddParameter(const FNiagaraVariable& Parameter);
	NIAGARA_API bool RemoveParameter(const FNiagaraVariable& Parameter);
	NIAGARA_API void RenameParameter(const FNiagaraVariable& Parameter, FName NewName);
	NIAGARA_API void Empty();
	NIAGARA_API void GetParameters(TArray<FNiagaraVariable>& OutParameters);
	NIAGARA_API bool IsExternallyDriven(const FNiagaraVariable& Parameter) const;

	NIAGARA_API void Tick(UWorld* World);

	//TODO: Abstract to some interface to allow a hierarchy like UMaterialInstance?
	UPROPERTY(EditAnywhere, Category=Instance)
	TObjectPtr<UNiagaraParameterCollection> Collection;

	/**
	When editing instances, we must track which parameters are overridden so we can pull in any changes to the default.
	*/
	UPROPERTY()
	TArray<FNiagaraVariable> OverridenParameters;
	NIAGARA_API bool OverridesParameter(const FNiagaraVariable& Parameter)const;
	NIAGARA_API void SetOverridesParameter(const FNiagaraVariable& Parameter, bool bOverrides);

	/** Synchronizes this instance with any changes with it's parent collection. */
	NIAGARA_API void SyncWithCollection();
	NIAGARA_API void SyncWithCollectionParameters(TConstArrayView<FName> ParametersToRefresh);

	NIAGARA_API void Bind(UWorld* World);
	NIAGARA_API void Unbind(UWorld* World);

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnParameterValuesChanged, TConstArrayView<FName>)
	FOnParameterValuesChanged& ParameterValuesChanged() { return OnParameterValuesChanged; }
#endif

private:
	NIAGARA_API void RefreshSourceParameters(UWorld* World, const TArray<TPair<FName, float>>& ScalarParameters, const TArray<TPair<FName, FLinearColor>>& VectorParameters);
	NIAGARA_API bool EnsureNotBoundToMaterialParameterCollection(FName InVariableName, FString CallingFunction) const;
	
	void InitDelegates(UNiagaraParameterCollection* ParentCollection);
	void ResetDelegates(UNiagaraParameterCollection* ParentCollection);

	UPROPERTY()
	FNiagaraParameterStore ParameterStorage;

	UPROPERTY(Transient)
	TObjectPtr<UMaterialParameterCollectionInstance> SourceMaterialCollectionInstance;

	FRWLock DirtyParameterLock;
	TArray<TPair<FName, float>> DirtyScalarParameters;
	TArray<TPair<FName, FLinearColor>> DirtyVectorParameters;

	FDelegateHandle ScalarParamUpdateDelegate;
	FDelegateHandle VectorParamUpdateDelegate;
#if WITH_EDITOR
	FDelegateHandle ParentCollectionChangedDelegate;
	FDelegateHandle MaterialCollectionChangedDelegate;
	FDelegateHandle MaterialParametersChangedHandle;
	FDelegateHandle ParameterValuesEditedHandle;
	std::atomic<bool> ParametersNeedReset;

	void DirtyParameters() { ParametersNeedReset = true; }

	FOnParameterValuesChanged OnParameterValuesChanged;
#endif

	//TODO: These overrides should be settable per platform.
	//UPROPERTY()
	//TMap<FString, FNiagaraParameterStore>

	//~UObject interface
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~UObject interface
public:
	//Accessors from Blueprint. For now just exposing common types but ideally we can expose any somehow in future.
	
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Bool Parameter"))
	NIAGARA_API bool GetBoolParameter(const FString& InVariableName);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Float Parameter"))
	NIAGARA_API float GetFloatParameter(const FString& InVariableName);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Int Parameter"))
	NIAGARA_API int32 GetIntParameter(const FString& InVariableName);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Vector2D Parameter"))
	NIAGARA_API FVector2D GetVector2DParameter(const FString& InVariableName);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Vector Parameter"))
	NIAGARA_API FVector GetVectorParameter(const FString& InVariableName);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Vector4 Parameter"))
	NIAGARA_API FVector4 GetVector4Parameter(const FString& InVariableName);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Quaternion Parameter"))
	NIAGARA_API FQuat GetQuatParameter(const FString& InVariableName);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Color Parameter"))
	NIAGARA_API FLinearColor GetColorParameter(const FString& InVariableName);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Bool Parameter"))
	NIAGARA_API void SetBoolParameter(const FString& InVariableName, bool InValue);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Float Parameter"))
	NIAGARA_API void SetFloatParameter(const FString& InVariableName, float InValue);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Int Parameter"))
	NIAGARA_API void SetIntParameter(const FString& InVariableName, int32 InValue);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Vector2D Parameter"))
	NIAGARA_API void SetVector2DParameter(const FString& InVariableName, FVector2D InValue);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Vector Parameter"))
	NIAGARA_API void SetVectorParameter(const FString& InVariableName, FVector InValue); // TODO[mg]: add position setter for LWC

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Vector4 Parameter"))
	NIAGARA_API void SetVector4Parameter(const FString& InVariableName, const FVector4& InValue);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Color Parameter"))
	NIAGARA_API void SetColorParameter(const FString& InVariableName, FLinearColor InValue);
	
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Quaternion Parameter"))
	NIAGARA_API void SetQuatParameter(const FString& InVariableName, const FQuat& InValue);
};

/** Asset containing a collection of global parameters usable by Niagara. Similar to Material parameter collections,
 *  any number of Niagara assets may reference attributes from this parameter collection and will get new values when they are changed.
 *
 *  A Niagara parameter collection can reference a Material parameter collection, so it is in sync with the values provided to a Material.
 *  To use a value from a parameter collection in a Niagara system or emitter, add a reference to it from the Parameters panel (in the Niagara Parameter Collection section).
 */
UCLASS(MinimalAPI)
class UNiagaraParameterCollection : public UObject
{
	GENERATED_UCLASS_BODY()
public:

	//~UObject interface
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	NIAGARA_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
#endif

#if WITH_EDITOR
	NIAGARA_API virtual void PreEditUndo() override;
	NIAGARA_API virtual void PostEditUndo() override;
#endif
	//~UObject interface

	NIAGARA_API int32 IndexOfParameter(const FNiagaraVariable& Var);

	NIAGARA_API int32 AddParameter(const FNiagaraVariable& Parameter);
	NIAGARA_API int32 AddParameter(FName Name, FNiagaraTypeDefinition Type);
	NIAGARA_API void RemoveParameter(const FNiagaraVariable& Parameter);
	NIAGARA_API void RenameParameter(FNiagaraVariable& Parameter, FName NewName);

	TArray<FNiagaraVariable>& GetParameters() { return Parameters; }
	const TArray<FNiagaraVariable>& GetParameters() const { return Parameters; }

	const UMaterialParameterCollection* GetSourceCollection() const { return SourceMaterialCollection; }

	//TODO: Optional per platform overrides of the above.
	//TMap<FString, UNiagaraParameterCollectionOverride> PerPlatformOverrides;

	inline UNiagaraParameterCollectionInstance* GetDefaultInstance() { return DefaultInstance; }
	inline const UNiagaraParameterCollectionInstance* GetDefaultInstance() const { return DefaultInstance; }
	
	/**
	Takes the friendly name presented to the UI and converts to the real parameter name used under the hood.
	Converts from "ParameterName" to "CollectionUniqueName_ParameterName".
	*/
	UE_DEPRECATED(5.5, "Use ParameterNameFromFriendlyString(FString) or ConditionalAddFullNamespace(FName) instead")
	NIAGARA_API FString ParameterNameFromFriendlyName(const FString& FriendlyName) const;

	NIAGARA_API FName ConditionalAddFullNamespace(FName FriendlyParameterName) const;
	NIAGARA_API FName ParameterNameFromFriendlyString(const FString& FriendlyName) const;

	/**
	Takes the real parameter name used under the hood and converts to the friendly name for use in the UI.
	Converts from "CollectionUniqueName_ParameterName" to "ParameterName".
	*/

	NIAGARA_API FNiagaraVariable CollectionParameterFromFriendlyParameter(const FNiagaraVariable& FriendlyParameter)const;
	NIAGARA_API FNiagaraVariable FriendlyParameterFromCollectionParameter(const FNiagaraVariable& CollectionParameter)const;

	UE_DEPRECATED(5.5, "Use FriendlyNameFromParameterName(FName) instead")
	NIAGARA_API FString FriendlyNameFromParameterName(FString ParameterName) const;
	NIAGARA_API FName FriendlyNameFromParameterName(FName ParameterName) const;

	UE_DEPRECATED(5.5, "Use GetFullNamespaceName() instead")
	NIAGARA_API FString GetFullNamespace() const;

	FName GetFullNamespaceName() const { return FullNamespace; };

	/** The compile Id is an indicator to any compiled scripts that reference this collection that contents may have changed and a recompile is recommended to be safe.*/
	NIAGARA_API FNiagaraCompileHash GetCompileHash() const;

	/** If for any reason this data has changed externally and needs recompilation and it isn't autodetected, use this method. */
	NIAGARA_API void RefreshCompileId();

	//~UObject interface
	NIAGARA_API virtual void PostLoad()override;
	//~UObject interface

	FName GetNamespace()const { return Namespace; }

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnCollectionChanged);
	FOnCollectionChanged OnCollectionChangedDelegate;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnParameterValuesChanged, TConstArrayView<FName>)

	// invoked when the collection's material source is updated with new parameter values
	FOnParameterValuesChanged OnMaterialParametersChanged;

	// invoked when the collection's default values are updated from the NPC editor
	FOnParameterValuesChanged OnParameterValuesEdited;
#endif

protected:
	
#if WITH_EDITORONLY_DATA
	NIAGARA_API void MakeNamespaceNameUnique();
	NIAGARA_API void NamespaceChanged();
	NIAGARA_API void RefreshSourceMaterialCollection();
#endif

	/** Namespace for this parameter collection. Is enforced to be unique across all parameter collections. */
	UPROPERTY(EditAnywhere, Category = "Parameter Collection", AssetRegistrySearchable)
	FName Namespace;

	UPROPERTY()
	TArray<FNiagaraVariable> Parameters;

	/** Optional set of MPC that can drive scalar and vector parameters */
	UPROPERTY(EditAnywhere, Category = "Parameter Collection")
	TObjectPtr<UMaterialParameterCollection> SourceMaterialCollection;
	
	UPROPERTY()
	TObjectPtr<UNiagaraParameterCollectionInstance> DefaultInstance;

	/** Used to track whenever something of note changes in this parameter collection that might invalidate a compilation downstream of a script/emitter/system.*/
	UPROPERTY()
	FGuid CompileId;

	void BuildFullNamespace();

	// transient variable holding the full namespace (NPC.<namespace>.)
	FName FullNamespace;

#if WITH_EDITOR
	FDelegateHandle OnMaterialCollectionChanged;

	// in order to manage updating the NPC if a source MPC deletes a parameter while in the editor we need to keep track of which
	// parameters are actually 'owned' by the MPC.  This is only relevant for the UI.
	TSet<FName> ParametersOwnedBySourceCollection;
#endif

};