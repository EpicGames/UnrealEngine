// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"

#include "NiagaraParameterDefinitionsBase.generated.h"

/** Stub class. Collection of UNiagaraScriptVariables to synchronize between UNiagaraScripts. */
UCLASS(MinimalAPI, abstract)
class UNiagaraParameterDefinitionsBase : public UObject
{
public:
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	DECLARE_MULTICAST_DELEGATE(FOnParameterDefinitionsChanged);

	//~ UObject Interface
	virtual bool IsEditorOnly() const override { return true; }
	//~ End UObject Interface

	const FGuid& GetDefinitionsUniqueId() const { return UniqueId; };
	virtual int32 GetChangeIdHash() const { return 0; };
	virtual TSet<FGuid> GetParameterIds() const { return TSet<FGuid>(); };

	FOnParameterDefinitionsChanged& GetOnParameterDefinitionsChanged() { return OnParameterDefinitionsChangedDelegate; };

protected:
	UPROPERTY()
	FGuid UniqueId;

	FOnParameterDefinitionsChanged OnParameterDefinitionsChangedDelegate;
#endif
};
