// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "NiagaraCommon.h"
#include "NiagaraParameterDefinitionsDelegates.h"

#include "NiagaraParameterDefinitionsSubscriber.generated.h"

class UNiagaraEditorParametersAdapterBase;
class UNiagaraParameterDefinitionsBase;
class UNiagaraScriptSourceBase;


USTRUCT()
struct FParameterDefinitionsSubscription
{
public:
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	FParameterDefinitionsSubscription() 
		: Definitions(nullptr)
		, DefinitionsId_DEPRECATED()
		, CachedChangeIdHash(0)
	{};

	UPROPERTY()
	TObjectPtr<UNiagaraParameterDefinitionsBase> Definitions;

	UPROPERTY(meta = (DeprecatedProperty))
	FGuid DefinitionsId_DEPRECATED;

	UPROPERTY()
	int32 CachedChangeIdHash;
#endif
};

/** Interface for classes which subscribe to parameter definitions. */
class INiagaraParameterDefinitionsSubscriber
{
#if WITH_EDITORONLY_DATA
public:
	using FDeferredSyncFunction = TFunction<TArray<INiagaraParameterDefinitionsSubscriber*>(UObject*)>;

	virtual ~INiagaraParameterDefinitionsSubscriber() = default;

	// Post load definitions
	// If the asset registry has not finished loading files the DeferredSync will be called when it has done so with the first argument being resolved Owner
	// The callback is expected to return all of the subsribers that WeakOwner contains
	NIAGARA_API void PostLoadDefinitionsSubscriptions(TWeakObjectPtr<UObject> WeakOwner, FDeferredSyncFunction DeferredSync);

	//~ Begin Pure Virtual Methods
	virtual const TArray<FParameterDefinitionsSubscription>& GetParameterDefinitionsSubscriptions() const = 0;
	virtual TArray<FParameterDefinitionsSubscription>& GetParameterDefinitionsSubscriptions() = 0;

	/** Get all UNiagaraScriptSourceBase of this subscriber. */
	virtual TArray<UNiagaraScriptSourceBase*> GetAllSourceScripts() = 0;

	/** Get the path to the UObject of this subscriber. */
	virtual FString GetSourceObjectPathName() const = 0;
	//~ End Pure Virtual Methods

	/** Get All adapters to editor only script vars owned directly by this subscriber. */
	virtual TArray<UNiagaraEditorParametersAdapterBase*> GetEditorOnlyParametersAdapters() { return TArray<UNiagaraEditorParametersAdapterBase*>(); };

	/** Get all subscribers that are owned by this subscriber.
	 *  Note: Implemented for synchronizing UNiagaraSystem. UNiagaraSystem returns all UNiagaraEmitters it owns to call SynchronizeWithParameterDefinitions for each.
	 */
	virtual TArray<INiagaraParameterDefinitionsSubscriber*> GetOwnedParameterDefinitionsSubscribers() { return TArray<INiagaraParameterDefinitionsSubscriber*>(); };

	NIAGARA_API TArray<UNiagaraParameterDefinitionsBase*> GetSubscribedParameterDefinitions() const;
	NIAGARA_API bool GetIsSubscribedToParameterDefinitions(const UNiagaraParameterDefinitionsBase* Definition) const;
	NIAGARA_API UNiagaraParameterDefinitionsBase* FindSubscribedParameterDefinitionsById(const FGuid& DefinitionsId) const;

	NIAGARA_API void SubscribeToParameterDefinitions(UNiagaraParameterDefinitionsBase* NewParameterDefinitions, bool bDoNotAssertIfAlreadySubscribed = false);
	NIAGARA_API void UnsubscribeFromParameterDefinitions(const FGuid& ParameterDefinitionsToRemoveId);
	NIAGARA_API void SynchronizeWithParameterDefinitions(const FSynchronizeWithParameterDefinitionsArgs Args = FSynchronizeWithParameterDefinitionsArgs());

	FOnSubscribedParameterDefinitionsChanged& GetOnSubscribedParameterDefinitionsChangedDelegate() { return OnSubscribedParameterDefinitionsChangedDelegate; };

private:
	FOnSubscribedParameterDefinitionsChanged OnSubscribedParameterDefinitionsChangedDelegate;

private:
	/** Update the cached change Id of specified Synchronized Subscribed Parameter Definition subscriptions so that they are not marked pending sync. */
	NIAGARA_API void MarkParameterDefinitionSubscriptionsSynchronized(TArray<FGuid> SynchronizedParameterDefinitionsIds /*= TArray<FGuid>()*/);
#endif
};
