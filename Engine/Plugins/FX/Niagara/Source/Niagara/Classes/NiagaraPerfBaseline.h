// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Particles/ParticlePerfStatsManager.h"
#include "GameFramework/Actor.h"

#include "NiagaraPerfBaseline.generated.h"

#define NIAGARA_PERF_BASELINES ((!UE_BUILD_SHIPPING || WITH_EDITOR) && WITH_PARTICLE_PERF_STATS)

class UNiagaraComponent;
class UNiagaraSystem;
class UNiagaraEffectType;
class UTextRenderComponent;
class FCommonViewportClient;

USTRUCT(BlueprintType)
struct FNiagaraPerfBaselineStats
{
	GENERATED_BODY()

	/** Per instance average time spent on the GameThread (µs). */
	UPROPERTY(EditAnywhere, Category="Baseline", config, BlueprintReadWrite)
	float PerInstanceAvg_GT = 0.0f;

	/** Per instance average time spent on the RenderThread (µs). */
	UPROPERTY(EditAnywhere, Category = "Baseline", config, BlueprintReadWrite)
	float PerInstanceAvg_RT = 0.0f;

	/** Per instance max time spent on the GameThread (µs). */
	UPROPERTY(EditAnywhere, Category = "Baseline", config, BlueprintReadWrite)
	float PerInstanceMax_GT = 0.0f;
	
	/** Per instance max time spent on the RenderThread (µs). */
	UPROPERTY(EditAnywhere, Category = "Baseline", config, BlueprintReadWrite)
	float PerInstanceMax_RT = 0.0f;

	enum class EComparisonResult
	{
		Good,/** The system has good performance relative to it's baseline. */
		Poor,/** The system has poor performance relative to it's baseline. Ideally this should be improved. */
		Bad,/** The system has bad performance relative to it's baseline. This should be improved before shipping this system. */
		Unknown,/** The system stats or the baseline stats are not valid so it's relative performance is unknown. */
	};

#if NIAGARA_PERF_BASELINES
	FNiagaraPerfBaselineStats() {}
	/**
	Construct from accumulated perf stats. 
	@param Stats	Accumulated stats we take values from.
	@param bSyncRT	Whether to sync with the RT. If true we flush the RT and can get the current stat values. If false we get the current GT stats but have to just use the RT stats from the last completed frame.
	*/
	NIAGARA_API FNiagaraPerfBaselineStats(FAccumulatedParticlePerfStats& Stats, bool bSyncRT);

	inline bool IsValid()const { return PerInstanceAvg_GT > 0.0f && PerInstanceAvg_RT > 0.0f && PerInstanceMax_GT > 0.0f && PerInstanceMax_RT > 0.0f; }
	/**
	Compares a system's performance stats with that of a baseline.
	*/
	static NIAGARA_API EComparisonResult Compare(const FNiagaraPerfBaselineStats& Stat, const FNiagaraPerfBaselineStats& Baseline, FNiagaraPerfBaselineStats& OutRatio);
	static NIAGARA_API EComparisonResult Compare(const FNiagaraPerfBaselineStats& Stat, const FNiagaraPerfBaselineStats& Baseline, FNiagaraPerfBaselineStats& OutRatio, EComparisonResult& OutGTAvgResult, EComparisonResult& OutGTMaxResult, EComparisonResult& OutRTAvgResult, EComparisonResult& OutRTMaxResult);
	static NIAGARA_API EComparisonResult Compare(float Stat, float Baseline, float& OutRatio);	
	static NIAGARA_API FLinearColor GetComparisonResultColor(EComparisonResult Result);
	static NIAGARA_API FText GetComparisonResultText(EComparisonResult Result);
#endif
};

/**
* Base class for baseline controllers. These can are responsible for spawning and manipulating the FX needed for the baseline perf tests.
*/
UCLASS(abstract, EditInlineNew, BlueprintType, Blueprintable, MinimalAPI)
class UNiagaraBaselineController : public UObject
{
	GENERATED_BODY()
	
public:

	/** Called from the stats system when we begin gathering stats for the given System asset.*/
	UFUNCTION(BlueprintNativeEvent)
	NIAGARA_API void OnBeginTest();

	/** Returns whether the baseline test is complete. */
	UFUNCTION(BlueprintNativeEvent)
	NIAGARA_API bool OnTickTest();

	/** Called from the stats system on completion of the test with the final stats for the given system asset. */
	UFUNCTION(BlueprintNativeEvent)
	NIAGARA_API void OnEndTest(FNiagaraPerfBaselineStats Stats);

	/** Called when the owning actor is ticked. */
	UFUNCTION(BlueprintNativeEvent)
	NIAGARA_API void OnOwnerTick(float DeltaTime);

	/** Returns the System for this baseline. Will synchronously load the system if needed. */
	UFUNCTION(BlueprintCallable, Category="Baseline")
	NIAGARA_API UNiagaraSystem* GetSystem();
	
	/** Duration to gather performance stats for the given system. */
	UPROPERTY(EditAnywhere, Category = "Baseline", BlueprintReadWrite)
	float TestDuration = 5.0f;

	/** The effect type this controller is in use by. */
	UPROPERTY(Category = "Baseline", BlueprintReadOnly)
	TObjectPtr<UNiagaraEffectType> EffectType = nullptr;

	/** The owning actor for this baseline controller. */
	UPROPERTY(Category = "Baseline", BlueprintReadOnly)
	TObjectPtr<ANiagaraPerfBaselineActor> Owner;

private:
	/** The baseline system to spawn. */
	UPROPERTY(EditAnywhere, Category="Baseline")
	TSoftObjectPtr<UNiagaraSystem> System = nullptr;
};

/**
Simple controller that will just spawn the given system N times. If any instance completes, it will spawn a new one to replace it.
Can handle simple burst or looping systems.
*/
UCLASS(EditInlineNew, MinimalAPI)
class UNiagaraBaselineController_Basic : public UNiagaraBaselineController
{
	GENERATED_BODY()
	
	NIAGARA_API virtual void OnBeginTest_Implementation();
	NIAGARA_API virtual bool OnTickTest_Implementation();
	NIAGARA_API virtual void OnEndTest_Implementation(FNiagaraPerfBaselineStats Stats);
	NIAGARA_API virtual void OnOwnerTick_Implementation(float DeltaTime);

	UPROPERTY(EditAnywhere, Category = "Baseline")
	int32 NumInstances = 1;

	UPROPERTY()
	TArray<TObjectPtr<UNiagaraComponent>> SpawnedComponents;
};

/** Actor that controls how the baseline system behaves and also controls stats gathering for. */
UCLASS(MinimalAPI)
class ANiagaraPerfBaselineActor : public AActor
{
	GENERATED_UCLASS_BODY()

	DECLARE_DELEGATE_OneParam(FFocusOnComponent, UNiagaraComponent*);
public:

	UPROPERTY(EditAnywhere, Category="Baseline")
	TObjectPtr<UNiagaraBaselineController> Controller;

	UPROPERTY(EditAnywhere, Category="Baseline")
	TObjectPtr<UTextRenderComponent> Label;

#if NIAGARA_PERF_BASELINES

	//AActor Interface
	NIAGARA_API virtual void BeginPlay();
	NIAGARA_API virtual void TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction);
	//AActor Interface END
#endif
};

#if NIAGARA_PERF_BASELINES

/** 
Listener that accumulates a short run of stats for a particular baseline system.
These are then stored for comparison against WIP systems in the editor to give a baseline for performance.
*/
class FNiagaraPerfBaselineStatsListener : public FParticlePerfStatsListener
{
public:
	NIAGARA_API FNiagaraPerfBaselineStatsListener(UNiagaraBaselineController* OwnerBaseline);
	NIAGARA_API virtual void Begin()override;
	NIAGARA_API virtual void End()override;
	NIAGARA_API virtual bool Tick()override;
	NIAGARA_API virtual void TickRT()override;

	virtual bool NeedsWorldStats()const override { return false; }
	virtual bool NeedsSystemStats()const override { return true; }
	virtual bool NeedsComponentStats()const override { return false; }
	
	virtual bool AllowOrphaned()const { return true; }

private:
	/** The baseline actor controlling the test conditions and which we'll send the completed stats to. */
	TWeakObjectPtr<UNiagaraBaselineController> Baseline;
	/** The stats we accumulate for the baseline system. */
	FAccumulatedParticlePerfStats AccumulatedStats;
};

/**
This listener gathers perf stats for all systems and will render them to the editor viewport with a comparison to their perf baselines.
*/
class FParticlePerfStatsListener_NiagaraBaselineComparisonRender : public FParticlePerfStatsListener_GatherAll
{
public:
	NIAGARA_API FParticlePerfStatsListener_NiagaraBaselineComparisonRender();
	NIAGARA_API virtual bool Tick()override;
	NIAGARA_API virtual void TickRT()override;
	virtual bool NeedsWorldStats()const { return false; }
	virtual bool NeedsSystemStats()const { return true; }

	NIAGARA_API int32 RenderStats(UWorld* World, class FViewport* Viewport, class FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation);

	int32 NumFrames = 0;
	int32 NumFramesRT = 0;

	TMap<TWeakObjectPtr<const UFXSystemAsset>, FNiagaraPerfBaselineStats> CurrentStats;
};

//////////////////////////////////////////////////////////////////////////

/**
This listener gathers perf stats for all systems and will give less intrusive reports on systems that exceed their baseline cost.
*/
class FParticlePerfStatsListener_NiagaraPerformanceReporter : public FParticlePerfStatsListener_GatherAll
{
public:
	NIAGARA_API FParticlePerfStatsListener_NiagaraPerformanceReporter(UWorld* InWorld);

	NIAGARA_API virtual bool Tick()override;
	NIAGARA_API virtual void TickRT()override;
	virtual bool NeedsWorldStats()const override { return false; }
	virtual bool NeedsSystemStats()const override { return true; }

	NIAGARA_API void HandleTestResults();
	NIAGARA_API void ReportToScreen();
	NIAGARA_API void ReportToLog();

	TWeakObjectPtr<UWorld> World;

	int32 NumFrames = 0;
	int32 NumFramesRT = 0;

	//How many test have been done in total so far.
	uint32 TotalTests = 0;

	/** We store the current target world time so we can give an indication of the time range for poor tests in the report. */
	double CurrentWorldTime = 0.0;
	uint32 CurrentFrameNumber = 0;
	FString TestNameString;

	/** Info on a particular test run. */
	struct FStatTestInfo
	{
		uint32 TestIndex = 0;
		double StartTime = 0.0;
		double EndTime = 0.0;
		FAccumulatedParticlePerfStats AccumulatedStats;
	};

	struct FStoredStatsInfo
	{
		/** Stats seen at last snapshot. */
		FAccumulatedParticlePerfStats Current;
				
		/** A history of all tests in which this system performed badly. */
		TArray<FStatTestInfo> BadTestHistory;
	};
	/** Track the worst recorded stats for each system. */
	TMap<TWeakObjectPtr<const UFXSystemAsset>, FStoredStatsInfo> StoredStats;

	/** We use an RT Command Fence to know when the RT data is valid and we can report the current stats to the screen. */
	FRenderCommandFence ResultsFence;
	bool bResultsTrigger = false;

	static NIAGARA_API const int32 TestDebugMessageID;
};

// Helper class for managing the generation and tracking of Niagara Performance Baselines.
struct FNiagaraPerfBaselineHandler
{
	/** Listener to render detailed baseline comparison stats UI to the viewport. */
	TSharedPtr<FParticlePerfStatsListener_NiagaraBaselineComparisonRender, ESPMode::ThreadSafe> DebugRenderListener;

	/** More minimal listener that will report poor and badly performing systems to the screen and the logs.*/
	TSharedPtr<FParticlePerfStatsListener_NiagaraPerformanceReporter, ESPMode::ThreadSafe> PerfBaselineListener;

	FNiagaraPerfBaselineHandler();
	~FNiagaraPerfBaselineHandler();

	void Tick(UWorld* World, float DeltaSeconds);

	void GenerateBaselines(TArray<UNiagaraEffectType*>& BaselinesToGenerate);

	void OnWorldBeginTearDown(UWorld* World);

private:
	bool ToggleStatPerfBaselines(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream);
	int32 RenderStatPerfBaselines(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation);

	enum class EBaselineGenState
	{
		Complete,
		Begin,
		WaitingToGenerate,
		Generating,
	};
	TArray<TWeakObjectPtr<UNiagaraEffectType>> BaselinesToGenerate;
	EBaselineGenState BaselineGenerationState = EBaselineGenState::Complete;
	double WorldTimeToGenerate = 0.0;
	TWeakObjectPtr<UWorld> BaselineGenWorld;
};

#endif
