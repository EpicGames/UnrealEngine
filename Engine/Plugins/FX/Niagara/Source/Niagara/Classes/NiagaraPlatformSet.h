// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "NiagaraPlatformSet.generated.h"

class UDeviceProfile;

UENUM()
enum class ENiagaraPlatformSelectionState : uint8
{
	Default,/** Neither explicitly enabled or disabled, this platform is enabled or not based on other settings in the platform set. */
	Enabled,/** This platform is explicitly disabled. */
	Disabled,/** This platform is explicitly disabled. */
};

USTRUCT()
struct FNiagaraDeviceProfileStateEntry
{
	GENERATED_BODY();

	FNiagaraDeviceProfileStateEntry()
		: QualityLevelMask(0)
		, SetQualityLevelMask(0)
	{}

	UPROPERTY(EditAnywhere, Category = Profile)
	FName ProfileName;

	/** The state of each set quality level. */
	UPROPERTY(EditAnywhere, Category = Profile)
	uint32 QualityLevelMask;

	/** Which Effects Qualities are set. */
	UPROPERTY(EditAnywhere, Category = Profile)
	uint32 SetQualityLevelMask;

	inline ENiagaraPlatformSelectionState GetState(int32 QualityLevel)const;

	inline void SetState(int32 QualityLevel, ENiagaraPlatformSelectionState State);

	inline bool AllDefaulted()const { return SetQualityLevelMask == 0; }

	inline bool operator==(const FNiagaraDeviceProfileStateEntry& Other)const
	{
		return ProfileName == Other.ProfileName && QualityLevelMask == Other.QualityLevelMask && SetQualityLevelMask == Other.SetQualityLevelMask;
	}
};

UENUM()
enum class ENiagaraPlatformSetState : uint8
{
	Disabled, //This platform set is disabled.
	Enabled, //This device profile is enabled but not active.
	Active, //This device profile is enabled and active now.
	Unknown UMETA(Hidden),
};

struct FNiagaraPlatformSet;

USTRUCT()
struct FNiagaraPlatformSetConflictEntry
{
	GENERATED_BODY()

	FNiagaraPlatformSetConflictEntry()
		: ProfileName(NAME_None)
		, QualityLevelMask(0)
	{}

	UPROPERTY()
	FName ProfileName;

	/** Mask of conflicting effects qualities for this profile. */
	UPROPERTY()
	int32 QualityLevelMask;
};

USTRUCT()
struct FNiagaraPlatformSetConflictInfo
{
	GENERATED_BODY()

	FNiagaraPlatformSetConflictInfo()
		: SetAIndex(INDEX_NONE)
		, SetBIndex(INDEX_NONE)
	{}
	/** Index of the first conflicting set in the checked array. */
	UPROPERTY()
	int32 SetAIndex;

	/** Index of the second conflicting set in the checked array. */
	UPROPERTY()
	int32 SetBIndex;

	/** Array of all conflicts between these sets. */
	UPROPERTY()
	TArray<FNiagaraPlatformSetConflictEntry> Conflicts;
};

struct FNiagaraPlatformSetEnabledState
{
	bool bIsActive = true;
	bool bCanBeActive = true;
};

struct FNiagaraPlatformSetEnabledStateDetails
{
#if WITH_EDITOR
	int32 DefaultQualityMask = INDEX_NONE;
	int32 AvailableQualityMask = INDEX_NONE;

	/** Provide reasons why this profile is not currently active. */
	TArray<FText> ReasonsForInActive;
	/** Provide reasons why this profile could never be active. */
	TArray<FText> ReasonsForDisabled;
#endif
};

#if WITH_EDITOR

struct FNiagaraCVarValue
{
	union 
	{
		bool bValue;
		int32 IntValue = 0;
		float FloatValue;
	}Data;
	
	template<typename T> inline T Get()const;	
	template<typename T> inline void SetTyped(const FString& Str);
	inline void Set(const FString& Str, IConsoleVariable* CVar);
};

template<> inline bool FNiagaraCVarValue::Get<bool>()const{ return Data.bValue; }
template<> inline void FNiagaraCVarValue::SetTyped<bool>(const FString& Str){ LexFromString(Data.bValue, *Str); }

template<> inline int32 FNiagaraCVarValue::Get<int32>()const { return Data.IntValue; }
template<> inline void FNiagaraCVarValue::SetTyped<int32>(const FString& Str) { LexFromString(Data.IntValue, *Str); }

template<> inline float FNiagaraCVarValue::Get<float>()const { return Data.FloatValue; }
template<> inline void FNiagaraCVarValue::SetTyped<float>(const FString& Str) { LexFromString(Data.FloatValue, *Str); }

inline void FNiagaraCVarValue::Set(const FString& Str, IConsoleVariable* CVar)
{
	if (CVar->IsVariableBool()) { SetTyped<bool>(Str); }
	else if (CVar->IsVariableInt()) { SetTyped<int32>(Str); }
	else if (CVar->IsVariableFloat()) { SetTyped<float>(Str); }
}

struct FNiagaraCVarValues
{
	FNiagaraCVarValue Default;

	TArray<FNiagaraCVarValue> PerQualityLevelValues;
};

//Helper class for accessing and caching the value of CVars for device profiles.
struct FDeviceProfileValueCache
{
	static void Empty();

	static const FNiagaraCVarValues& GetValues(const UDeviceProfile* DeviceProfile, IConsoleVariable* CVar, FName CVarName);

private:

	typedef TMap<IConsoleVariable*, FNiagaraCVarValues> FCVarValueMap;
	/** Cached values for any polled CVar on particular device profiles. */
	static TMap<const UDeviceProfile*, FCVarValueMap> CachedDeviceProfileValues;
	/** Cached values for any polled CVar from the platform ini files. */
	static TMap<FName, FCVarValueMap> CachedPlatformValues;
};
#endif

UENUM()
enum class ENiagaraCVarConditionResponse : uint8
{
	None,
	Enable,
	Disable,
};

/** Imposes a condition that a CVar must contain a set value or range of values for a platform set to be enabled. */
USTRUCT()
struct FNiagaraPlatformSetCVarCondition
{
	GENERATED_BODY();
	
	NIAGARA_API FNiagaraPlatformSetCVarCondition();

	/** Return true if this condition is met for the given device profile. */
	NIAGARA_API bool Evaluate(const UDeviceProfile* DeviceProfile, int32 QualityLevel, bool bCheckCurrentStateOnly, bool& bOutCanEverPass, bool& bOutCanEverFail)const;

	/** Returns the CVar for this condition. Can be null if the name give is not a valid CVar or the CVar is removed. */
	NIAGARA_API IConsoleVariable* GetCVar()const;

	NIAGARA_API void SetCVar(FName InCVarName);

	/** The name of the CVar we're testing the value of. */
	UPROPERTY(EditAnywhere, Category = "CVar")
	FName CVarName = NAME_None;

	/** If this CVar condition passes, how should this affect the state of the Platform Set? */
	UPROPERTY(EditAnywhere, Category = "CVar", AdvancedDisplay)
	ENiagaraCVarConditionResponse PassResponse = ENiagaraCVarConditionResponse::None;
	
	/** If this CVar condition fails, how should this affect the state of the Platform Set? */
	UPROPERTY(EditAnywhere, Category = "CVar", AdvancedDisplay)
	ENiagaraCVarConditionResponse FailResponse = ENiagaraCVarConditionResponse::Disable;

	/** The value this CVar must contain for this platform set to be enabled. */
	UPROPERTY(EditAnywhere, Category = "CVar", meta=(DisplayName="Required Value"))
	bool Value = true;

	/** If the value of the CVar is less than this minimum then the PlatformSet will not be enabled. */
	UPROPERTY(EditAnywhere, Category = "CVar", meta = (EditCondition="bUseMinInt", DisplayName = "Minumum Int For Enabled"))
	int32 MinInt = 1;

	/** If the value of the CVar is greater than this maximum then the PlatformSet will not be enabled. */
	UPROPERTY(EditAnywhere, Category = "CVar", meta = (EditCondition = "bUseMaxInt", DisplayName = "Maximum Int For Enabled"))
	int32 MaxInt = 1;

	/** If the value of the CVar is less than this minimum then the PlatformSet will not be enabled. */
	UPROPERTY(EditAnywhere, Category = "CVar", meta = (EditCondition = "bUseMinFloat", DisplayName = "Minumum Float For Enabled"))
	float MinFloat = 1.0f;

	/** If the value of the CVar is greater than this maximum then the PlatformSet will not be enabled. */
	UPROPERTY(EditAnywhere, Category = "CVar", meta = (EditCondition = "bUseMaxFloat", DisplayName = "Maximum Float For Enabled"))
	float MaxFloat = 1.0f;
	
	/** True if we should apply the minimum restriction for int CVars. */
	UPROPERTY(EditAnywhere, Category = "CVar", meta = (InlineEditConditionToggle))
	uint32 bUseMinInt : 1;
	
	/** True if we should apply the maximum restriction for int CVars. */
	UPROPERTY(EditAnywhere, Category = "CVar", meta = (InlineEditConditionToggle))
	uint32 bUseMaxInt : 1;

	/** True if we should apply the minimum restriction for float CVars. */
	UPROPERTY(EditAnywhere, Category = "CVar", meta = (InlineEditConditionToggle))
	uint32 bUseMinFloat : 1;

	/** True if we should apply the maximum restriction for float CVars. */
	UPROPERTY(EditAnywhere, Category = "CVar", meta = (InlineEditConditionToggle))
	uint32 bUseMaxFloat : 1;

	template<typename T>
	bool Evaluate_Internal(const UDeviceProfile* DeviceProfile, int32 QualityLevel, bool bCheckCurrentStateOnly, bool& bOutCanEverPass, bool& bOutCanEverFail)const;

	inline bool CheckValue(bool CVarValue)const { return CVarValue == Value; }
	inline bool CheckValue(int32 CVarValue)const { return (!bUseMinInt || CVarValue >= MinInt) && (!bUseMaxInt || CVarValue <= MaxInt); }
	inline bool CheckValue(float CVarValue)const { return (!bUseMinFloat || CVarValue >= MinFloat) && (!bUseMaxFloat || CVarValue <= MaxFloat); }

	template<typename T>
	inline T GetCVarValue(IConsoleVariable* CVar)const;

private:
	mutable IConsoleVariable* CachedCVar = nullptr;
	mutable uint32 LastCachedFrame = INDEX_NONE;
};

void NIAGARA_API SetGNiagaraDeviceProfile(UDeviceProfile* Profile);

UENUM()
enum class ENiagaraDeviceProfileRedirectMode : uint8
{
	/** Replace Device Profile Reference with a CVar Condition. */
	CVar,
	/** Replace Device Profile Reference with a different Device Profile. */
	DeviceProfile,
};

/**
Allows us to replace a specific device profile usage condition in all NiagaraPlatformSets.
Helpful when dealing with changes to device profile structure.
*/
USTRUCT()
struct FNiagaraPlatformSetRedirect
{
	GENERATED_BODY()

	FNiagaraPlatformSetRedirect();

	/** The names of any device profile entry that will apply this redirect. */
	UPROPERTY(EditAnywhere, Category = Redirect)
	TArray<FName> ProfileNames;

	UPROPERTY(EditAnywhere, Category = Redirect)
	ENiagaraDeviceProfileRedirectMode Mode = ENiagaraDeviceProfileRedirectMode::CVar;

	/** When in Device Profile mode, the name of the device profile to redirect to. */
	UPROPERTY(EditAnywhere, Category = Redirect, meta = (EditCondition = "Mode == ENiagaraDeviceProfileRedirectMode::DeviceProfile"))
	FName RedirectProfileName;

	/** When in CVar mode, the CVar condition to replace this device profile entry with when the profile entry is in the Enabled state. */
	UPROPERTY(EditAnywhere, Category = Redirect, meta = (EditCondition = "Mode == ENiagaraDeviceProfileRedirectMode::CVar"))
	FNiagaraPlatformSetCVarCondition CVarConditionEnabled;
	
	/** When in CVar mode, the CVar condition to replace this device profile entry with when the profile entry is in the Disabled state. */
	UPROPERTY(EditAnywhere, Category = Redirect, meta = (EditCondition = "Mode == ENiagaraDeviceProfileRedirectMode::CVar"))
	FNiagaraPlatformSetCVarCondition CVarConditionDisabled;
};

USTRUCT()
struct FNiagaraPlatformSet
{
	GENERATED_BODY();

	static inline int32 QualityLevelFromMask(int32 QLMask);
	static inline int32 CreateQualityLevelMask(int32 QL);
	static NIAGARA_API FText GetQualityLevelText(int32 QualityLevel);
	static NIAGARA_API FText GetQualityLevelMaskText(int32 QualityLevelMask);

	static NIAGARA_API int32 GetQualityLevel();
	static NIAGARA_API int32 GetMinQualityLevel();
	static NIAGARA_API int32 GetMaxQualityLevel();
	static NIAGARA_API int32 GetAvailableQualityLevelMask();
	static NIAGARA_API int32 GetFullQualityLevelMask(int32 NumQualityLevels);
	
public:

	//Runtime public API

	NIAGARA_API FNiagaraPlatformSet(int32 QLMask);
	NIAGARA_API FNiagaraPlatformSet();

	NIAGARA_API bool operator==(const FNiagaraPlatformSet& Other)const;

	/** Is this set active right now. i.e. enabled for the current device profile and quality level. */
	NIAGARA_API bool IsActive()const;
	
	/** Is this platform set enabled for the give device profile. */
	NIAGARA_API int32 IsEnabledForDeviceProfile(const UDeviceProfile* DeviceProfile)const;

	/** Is this platform set enabled on any quality level for the passed device profile. Returns the QualityLevelMask for all enabled effects qualities for this profile. */
	NIAGARA_API int32 GetEnabledMaskForDeviceProfile(const UDeviceProfile* DeviceProfile)const;

	/** Is this platform set enabled at this quality level on any device profile. */
	NIAGARA_API bool IsEnabledForQualityLevel(int32 QualityLevel)const;

	/** Fill OutProfiles with all device profiles that have been overridden at the passed QualityLevel. */
	NIAGARA_API void GetOverridenDeviceProfiles(int32 QualityLevel, TArray<UDeviceProfile*>& OutEnabledProfiles, TArray<UDeviceProfile*>& OutDisabledProfiles)const;

	/** Returns true if this set is enabled for any profiles on the specified platform. */
	NIAGARA_API bool IsEnabledForPlatform(const FString& PlatformName)const;

	/** Returns true if the current platform can modify it's Niagara scalability settings at runtime. */
	static NIAGARA_API bool CanChangeScalabilityAtRuntime();

	/**Will force all platform sets to regenerate their cached data next time they are used.*/
	static NIAGARA_API void InvalidateCachedData();

	/** Returns the currenlty(or default) active quality mask for a profile. */
	static NIAGARA_API int32 GetActiveQualityMaskForDeviceProfile(const UDeviceProfile* Profile);

	/** Returns the mask of all available quality levels for a device profile. */
	static NIAGARA_API int32 GetAvailableQualityMaskForDeviceProfile(const UDeviceProfile* Profile);

	/** Returns true if the passed platform should prune emitters on cook. */
	static NIAGARA_API bool ShouldPruneEmittersOnCook(const FString& PlatformName);

	/** Returns true if the passed platform can modify it's niagara scalability settings at runtime. */
	static NIAGARA_API bool CanChangeScalabilityAtRuntime(const UDeviceProfile* DeviceProfile);

	NIAGARA_API FNiagaraPlatformSetEnabledState IsEnabled(const UDeviceProfile* Profile, int32 QualityLevel, bool bCheckCurrentStateOnly, FNiagaraPlatformSetEnabledStateDetails* OutDetails=nullptr)const;

	NIAGARA_API bool CanConsiderDeviceProfile(const UDeviceProfile* Profile)const;

	NIAGARA_API bool ApplyRedirects();

	static NIAGARA_API void RefreshScalability();
	static NIAGARA_API void OnCVarChanged(IConsoleVariable* CVar);
	static NIAGARA_API void OnCVarUnregistered(IConsoleVariable* CVar);
	static NIAGARA_API IConsoleVariable* GetCVar(FName CVarName);

	//Editor only public API
#if WITH_EDITOR
	/**
	Does the enabled set of profiles in this set conflict with those in another set.
	In some cases this is not allowed.
	*/
	//bool Conflicts(const FNiagaraPlatformSet& Other, TArray<const UDeviceProfile*>& OutConflictingProfiles, bool bIncludeProfilesWithVariableQualityLevel=false)const;
	
	/** Direct state accessors for the UI. */
	NIAGARA_API bool IsEffectQualityEnabled(int32 EffectQuality)const;
	NIAGARA_API void SetEnabledForEffectQuality(int32 EffectQuality, bool bEnabled);
	NIAGARA_API void SetDeviceProfileState(UDeviceProfile* Profile, int32 QualityLevel, ENiagaraPlatformSelectionState NewState);
	NIAGARA_API ENiagaraPlatformSelectionState GetDeviceProfileState(UDeviceProfile* Profile, int32 QualityLevel)const;
	
	/** Invalidates any cached data on this platform set when something has changed. */
	NIAGARA_API void OnChanged();

	/** Inspects the passed sets and generates an array of all conflicts between these sets. Used to keep arrays of platform sets orthogonal. */
	static NIAGARA_API bool GatherConflicts(TConstArrayView<const FNiagaraPlatformSet*> PlatformSets, TArray<FNiagaraPlatformSetConflictInfo>& OutConflicts);
	
	DECLARE_DELEGATE_RetVal(int32, FOnOverrideQualityLevel);
	FOnOverrideQualityLevel OnOverrideQualityLevelDelegate;
	DECLARE_DELEGATE_RetVal(TOptional<TObjectPtr<UDeviceProfile>>, FOnOverrideActiveDeviceProfile);
	FOnOverrideActiveDeviceProfile OnOverrideActiveDeviceProfileDelegate;
#endif

	/** States of specific device profiles we've set. */
	UPROPERTY(EditAnywhere, Category = Platforms)
	TArray<FNiagaraDeviceProfileStateEntry> DeviceProfileStates;

	/** Set of CVars values we require for this platform set to be enabled. If any of the linked CVars don't have the required values then this platform set will not be enabled. */
	UPROPERTY(EditAnywhere, Category = Platforms)
	TArray<FNiagaraPlatformSetCVarCondition> CVarConditions;

	/** Mask defining which effects qualities this set matches. */
	UPROPERTY(EditAnywhere, Category = Platforms)
	int32 QualityLevelMask;

#if WITH_EDITOR
	//Data we pull from platform ini files.
	struct FPlatformIniSettings
	{
		FPlatformIniSettings();
		FPlatformIniSettings(int32 InbCanChangeScalabilitySettingsAtRuntime, int32 InbPruneEmittersOnCook, int32 InEffectsQuality, int32 InMinQualityLevel, int32 InMaxQualityLevel);
		int32 bCanChangeScalabilitySettingsAtRuntime;
		int32 bPruneEmittersOnCook;
		int32 EffectsQuality;
		int32 MinQualityLevel;
		int32 MaxQualityLevel;
		int32 QualityLevelMask;

		TArray<int32> QualityLevelsPerEffectsQuality;
	};
	static NIAGARA_API FPlatformIniSettings& GetPlatformIniSettings(const FString& PlatformName);

	static NIAGARA_API int32 GetEffectQualityMaskForPlatform(const FString& PlatformName);
#endif

	static uint32 GetLastDirtiedFrame(){ return LastDirtiedFrame; }

	static NIAGARA_API void SetNiagaraQualityLevelOverride(int32 Override);
	static NIAGARA_API void ClearNiagaraQualityLevelOverride();

private:

	mutable uint32 bEnabledForCurrentProfileAndEffectQuality : 1;

	//Last frame we built our cached data. 
	mutable uint32 LastBuiltFrame;

	//Set from outside when we need to force all cached values to be regenerated. For example on CVar changes.
	static NIAGARA_API uint32 LastDirtiedFrame;

	/** Cached value of the CVar fx.Niagara.QualityLevel. Allows us to detect when it has changed so we can properly apply scalability changes. */
	static NIAGARA_API int32 CachedQualityLevel;
	static NIAGARA_API int32 CachedAvailableQualityLevelMask;

	/** An override for Niagara quality level, driven from the Niagara UI. */
	static NIAGARA_API int32 QualityLevelOverride;

#if WITH_EDITOR
	static NIAGARA_API FCriticalSection CachedPlatformIniCritSec;

	//Cached data read from platform ini files.
	static NIAGARA_API TMap<FName, FPlatformIniSettings> CachedPlatformIniSettings;

	//Cached final QualityLevel setting for each device profile.
	static NIAGARA_API TMap<const UDeviceProfile*, int32> CachedQLMasksPerDeviceProfile;
#endif

	/**
	Callbacks for any CVars Niagara has looked at during this run.
	*/
	struct FCachedCVarInfo
	{
		IConsoleVariable* CVar = nullptr;
		FDelegateHandle ChangedHandle;
	};
	static NIAGARA_API TMap<FName, FCachedCVarInfo> CachedCVarInfo;
	static NIAGARA_API FCriticalSection CachedCVarInfoCritSec;

	static NIAGARA_API TArray<FName> ChangedCVars;
};


inline int32 FNiagaraPlatformSet::QualityLevelFromMask(int32 QLMask)
{
	//Iterate through bits in mask to find the first set quality level.
	int32 QualityLevel = 0;
	while (QLMask != 0)
	{
		if (QLMask & 0x1)
		{
			return QualityLevel;
		}
		++QualityLevel;
		QLMask >>= 1;
	}

	return INDEX_NONE;
}

inline int32 FNiagaraPlatformSet::CreateQualityLevelMask(int32 QL)
{
	return QL == INDEX_NONE ? INDEX_NONE : (1 << QL);
}


inline ENiagaraPlatformSelectionState FNiagaraDeviceProfileStateEntry::GetState(int32 QualityLevel)const
{
	int32 QLMask = FNiagaraPlatformSet::CreateQualityLevelMask(QualityLevel);
	return (QLMask & SetQualityLevelMask) == 0 ? ENiagaraPlatformSelectionState::Default : ((QLMask & QualityLevelMask) != 0 ? ENiagaraPlatformSelectionState::Enabled : ENiagaraPlatformSelectionState::Disabled);
}

inline void FNiagaraDeviceProfileStateEntry::SetState(int32 QualityLevel, ENiagaraPlatformSelectionState State)
{
	int32 QLMask = FNiagaraPlatformSet::CreateQualityLevelMask(QualityLevel);
	if (State == ENiagaraPlatformSelectionState::Default)
	{
		SetQualityLevelMask &= ~QLMask;
		QualityLevelMask &= ~QLMask;
	}
	else if (State == ENiagaraPlatformSelectionState::Enabled)
	{
		SetQualityLevelMask |= QLMask;
		QualityLevelMask |= QLMask;
	}
	else //(State == ENiagaraPlatformSelectionState::Disabled)
	{
		SetQualityLevelMask |= QLMask;
		QualityLevelMask &= ~QLMask;
	}
}
