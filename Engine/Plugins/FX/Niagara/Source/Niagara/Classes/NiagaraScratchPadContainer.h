// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "NiagaraScript.h"
#include "NiagaraScratchPadContainer.generated.h"

/**
 * Wrapper class for scratch pad scripts in an emitter. This is needed because each emitter version can have it's own copy of a scratch pad with the same name and they can't all be outered to the emitter that way.
 */
UCLASS(MinimalAPI)
class UNiagaraScratchPadContainer : public UObject
{
	GENERATED_BODY()
	
public:
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API void CheckConsistency();
	NIAGARA_API void SetScripts(const TArray<TObjectPtr<UNiagaraScript>>& InScripts);
	NIAGARA_API void AppendScripts(const TArray<TObjectPtr<UNiagaraScript>>& InScripts);
	NIAGARA_API void AppendScripts(TObjectPtr<UNiagaraScratchPadContainer> InScripts);
	
	// The scripts get added in specific ordering through inheritance. These accessors just allow us to deal with checks 
	// for known scripts 
	NIAGARA_API int32 FindIndexForScript(UNiagaraScript* InScript) const;
	NIAGARA_API UNiagaraScript* FindScriptAtIndex(int32 Index) const;

#if WITH_EDITORONLY_DATA

	UPROPERTY()
	TArray<TObjectPtr<UNiagaraScript>> Scripts;

#endif
};
