// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeBool.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "NiagaraCommon.h"
#include "NiagaraScriptBase.h"
#include "NiagaraShared.h"
#include "NiagaraParameters.h"
#include "NiagaraDataSet.h"
#include "NiagaraStackSection.h"
#include "NiagaraParameterDefinitionsSubscriber.h"
#include "NiagaraValidationRule.h"
#include "NiagaraVersionedObject.h"
#include "HAL/CriticalSection.h"
#include "VectorVM.h"

#include "NiagaraParameterStore.h"

#include "NiagaraScript.generated.h"

class UNiagaraVersionUpgradeBase;
struct FAppendToClassSchemaContext;
class UNiagaraDataInterface;
class FNiagaraCompileRequestDataBase;
class FNiagaraCompileRequestDuplicateDataBase;
class UNiagaraConvertInPlaceUtilityBase;
class UHierarchyRoot;

#define NIAGARA_INVALID_MEMORY (0xBA)

#define NIAGARA_SCRIPT_COMPILE_LOGGING_MEDIUM

DECLARE_STATS_GROUP(TEXT("Niagara Detailed"), STATGROUP_NiagaraDetailed, STATCAT_Niagara);

/** Defines what will happen to unused attributes when a script is run. */
UENUM()
enum class EUnusedAttributeBehaviour : uint8
{
	/** The previous value of the attribute is copied across. */
	Copy,
	/** The attribute is set to zero. */
	Zero,
	/** The attribute is untouched. */
	None,
	/** The memory for the attribute is set to NIAGARA_INVALID_MEMORY. */
	MarkInvalid, 
	/** The attribute is passed through without double buffering */
	PassThrough,
};

UENUM()
enum class ENiagaraModuleDependencyType : uint8
{
	/** The dependency belongs before the module. */
	PreDependency,
	/** The dependency belongs after the module. */
	PostDependency
};

UENUM()
enum class ENiagaraModuleDependencyScriptConstraint : uint8
{
	/** The module providing the dependency must be in the same script e.g. if the module requiring the dependency is in "Particle Spawn" the module providing the dependency must also be in "Particle Spawn". */
	SameScript,
	/** The module providing the dependency can be in any script as long as it satisfies the dependency type, e.g. if the module requiring the dependency is in "Particle Spawn" the module providing the dependency could be in "Emitter Spawn". */
	AllScripts
};

UENUM()
enum class ENiagaraScriptLibraryVisibility : uint8
{
	Invalid = 0 UMETA(Hidden),
	
	/** The script is not visible by default to the user, but can be made visible by disabling the "Library only" filter option. */
	Unexposed UMETA(DisplayName = "Unexposed"),

	/** The script is exposed to the asset library and always visible to the user. */
	Library UMETA(DisplayName = "Exposed"),

	/** The script is never visible to the user. This is useful to "soft deprecate" assets that should not be shown to a user, but should also not generate errors for existing usages. */
	Hidden UMETA(DisplayName = "Hidden")
};

UENUM()
enum class ENiagaraScriptTemplateSpecification : uint8
{
	None,
	Template,
	Behavior UMETA(DisplayName = "Learning Example")
};

UENUM()
enum class ENiagaraEmitterDefaultSummaryState : uint8
{
	/** The emitter will be added in default, full view by default. */
	Default,
	/** The emitter will be added in summary state by default. */
	Summary
};


/** Defines different usages for a niagara script's module dependecies. */
UENUM()
enum class ENiagaraModuleDependencyUsage : uint8
{
	/** Default entry to catch invalid usages */
	None UMETA(Hidden),
	/** Evaluate when the script is called during the spawn phase. */
	Spawn,
	/** Evaluate when the script is called during the update phase. */
	Update,
	/** Evaluate when the script is called in an event context. */
	Event,
	/** Evaluate when the script is called in a simulation stage. */
	SimulationStage
};

USTRUCT()
struct FNiagaraModuleDependency
{
	GENERATED_USTRUCT_BODY()
public:
	/** Specifies the provided id of the required dependent module (e.g. 'ProvidesNormalizedAge') */
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = Script)
	FName Id;

	/** Whether the dependency belongs before or after this module */
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = Script)
	ENiagaraModuleDependencyType Type; // e.g. PreDependency

	/** Specifies constraints related to the source script a modules provides as dependency. */
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = Script)
	ENiagaraModuleDependencyScriptConstraint ScriptConstraint;

	// Specifies the version constraint that module providing the dependency must fulfill.
	// Example usages:
	// '1.2' requires the exact version 1.2 of the source script
	// '1.2+' requires at least version 1.2, but any higher version is also ok
	// '1.2-2.0' requires any version between 1.2 and 2.0
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = Script)
	FString RequiredVersion;

	/** This property can limit where the dependency is evaluated. By default, the dependency is enforced in all script usages */
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = Script, meta = (Bitmask, BitmaskEnum = "/Script/Niagara.ENiagaraModuleDependencyUsage"))
	int32 OnlyEvaluateInScriptUsage;
	
	/** Detailed description of the dependency */
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = Script, meta = (MultiLine = true))
	FText Description;

	FNiagaraModuleDependency()
	{
		Type = ENiagaraModuleDependencyType::PreDependency;
		ScriptConstraint = ENiagaraModuleDependencyScriptConstraint::SameScript;
		OnlyEvaluateInScriptUsage = 1 << static_cast<int32>(ENiagaraModuleDependencyUsage::Spawn) |
									1 << static_cast<int32>(ENiagaraModuleDependencyUsage::Update) |
									1 << static_cast<int32>(ENiagaraModuleDependencyUsage::Event) |
									1 << static_cast<int32>(ENiagaraModuleDependencyUsage::SimulationStage);
	}

#if WITH_EDITOR
	bool HasValidVersionDependency() const;
	NIAGARA_API bool IsVersionAllowed(const FNiagaraAssetVersion& Version) const;
	
	void CheckVersionCache() const;
#endif
private:
	
	struct FResolvedVersions
	{
		bool bValid = false;
		FString SourceProperty;
		int32 MinMajorVersion = 0;
		int32 MinMinorVersion = 0;
		int32 MaxMajorVersion = 0;
		int32 MaxMinorVersion = 0;
	};
	mutable FResolvedVersions VersionDependencyCache;
};

USTRUCT()
struct FNiagaraCompilerTag
{
	GENERATED_USTRUCT_BODY()
public:
	FNiagaraCompilerTag() {}
	FNiagaraCompilerTag(const FNiagaraVariable& InVariable, const FString& InStringValue) : Variable(InVariable), StringValue(InStringValue) {}
	UPROPERTY()
	FNiagaraVariable Variable;

	UPROPERTY()
	FString StringValue;

	static NIAGARA_API FNiagaraCompilerTag* FindTag(TArray< FNiagaraCompilerTag>& InTags,  const FNiagaraVariableBase& InSearchVar);
	static NIAGARA_API const FNiagaraCompilerTag* FindTag(const TArray< FNiagaraCompilerTag>& InTags, const FNiagaraVariableBase& InSearchVar);

};


struct FNiagaraScriptDebuggerInfo
{
	FNiagaraScriptDebuggerInfo();
	FNiagaraScriptDebuggerInfo(FName InName, ENiagaraScriptUsage InUsage, const FGuid& InUsageId);

	bool bWaitForGPU;

	FName HandleName;

	ENiagaraScriptUsage Usage;

	FGuid UsageId;

	int32 FrameLastWriteId;

	FNiagaraDataSet Frame;

	FNiagaraParameterStore Parameters;

	TAtomic<bool> bWritten;
};

#if WITH_EDITORONLY_DATA
struct FNiagaraScriptHashCollector
{
	FNiagaraScriptHashCollector(bool bCollectHashSources);

	NIAGARA_API void AddHash(const FNiagaraCompileHash& CompileHash, FStringView CompileHashSource);
	FString BuildCompileHashSourceString(const FNiagaraCompileHash& CompileHash) const;

	TArray<FNiagaraCompileHash> ReferencedHashes;
	TMap<FNiagaraCompileHash, TArray<FString>> ReferencedHashSources;

	const bool bCollectSources = false;
};
#endif

/** Struct containing all of the data necessary to look up a NiagaraScript's VM executable results from the Derived Data Cache.*/
USTRUCT()
struct FNiagaraVMExecutableDataId
{
	GENERATED_USTRUCT_BODY()
public:
	/** The version of the compiler that this needs to be built against.*/
	UPROPERTY()
	FGuid CompilerVersionID;

	/** Do we require interpolated spawning */
	UPROPERTY()
	ENiagaraInterpolatedSpawnMode InterpolatedSpawnMode;

#if WITH_EDITORONLY_DATA
	/** The instance id of this script usage type.*/
	UPROPERTY()
	FGuid ScriptUsageTypeID;
	
	/** The type of script this was used for.*/
	UPROPERTY()
	ENiagaraScriptUsage ScriptUsageType;
	
	/** Configuration options*/
	UPROPERTY()
	TArray<FString> AdditionalDefines;

	UPROPERTY()
	TArray<FNiagaraVariableBase> AdditionalVariables;

	NIAGARA_API TArray<FString> GetAdditionalVariableStrings() const;

	/** Whether or not we allow debug switches to be used. */
	UPROPERTY()
	uint32 bDisableDebugSwitches : 1;
	
	/** Do we require persistent IDs */
	UPROPERTY()
	uint32 bRequiresPersistentIDs : 1;

	/** Whether or not we need to bake Rapid Iteration params. True to keep params, false to bake.*/
	UPROPERTY()
	uint32 bUsesRapidIterationParams : 1;
#endif
	
	/**
	* The hash of the subgraph this shader primarily represents.
	*/
	UPROPERTY()
	FNiagaraCompileHash BaseScriptCompileHash;

#if WITH_EDITORONLY_DATA
	/** Compile hashes of any top level scripts the script was dependent on that might trigger a recompile if they change. */
	UPROPERTY()
	TArray<FNiagaraCompileHash> ReferencedCompileHashes;
	
	/** The version of the script that was compiled. If empty then just the latest version. */
	UPROPERTY()
	FGuid ScriptVersionID;
#endif

	FNiagaraVMExecutableDataId()
		: CompilerVersionID()
		, InterpolatedSpawnMode(ENiagaraInterpolatedSpawnMode::NoInterpolation)
#if WITH_EDITORONLY_DATA
		, ScriptUsageType(ENiagaraScriptUsage::Function)
		, bDisableDebugSwitches(false)
		, bRequiresPersistentIDs(false)
		, bUsesRapidIterationParams(true)
#endif
	{ }


	~FNiagaraVMExecutableDataId() = default;

	NIAGARA_API bool IsValid() const;
	NIAGARA_API void Invalidate();
	
	friend uint32 GetTypeHash(const FNiagaraVMExecutableDataId& Ref)
	{
		return Ref.BaseScriptCompileHash.GetTypeHash();
	}

	SIZE_T GetSizeBytes() const
	{
		return sizeof(*this);
	}

	NIAGARA_API bool HasInterpolatedParameters() const;
#if 0
	/** Hashes the script-specific part of this shader map Id. */
	NIAGARA_API void GetScriptHash(FSHAHash& OutHash) const;
#endif

	/**
	* Tests this set against another for equality, disregarding override settings.
	*
	* @param ReferenceSet	The set to compare against
	* @return				true if the sets are equal
	*/
	NIAGARA_API bool operator==(const FNiagaraVMExecutableDataId& ReferenceSet) const;

	bool operator!=(const FNiagaraVMExecutableDataId& ReferenceSet) const
	{
		return !(*this == ReferenceSet);
	}

#if WITH_EDITORONLY_DATA
	/** Appends string representations of this Id to a key string. */
	NIAGARA_API void AppendKeyString(FString& KeyString, const FString& Delimiter = TEXT("_"), bool bAppendObjectForDebugging = false, const FNiagaraScriptHashCollector* HashCollector = nullptr) const;
#endif
};

USTRUCT()
struct FNiagaraVMExecutableByteCode
{
	GENERATED_USTRUCT_BODY()
private:
	UPROPERTY()
	TArray<uint8> Data;

	UPROPERTY()
	int32 UncompressedSize = INDEX_NONE;

public:
	NIAGARA_API bool HasByteCode() const;
	NIAGARA_API bool IsCompressed() const;
	NIAGARA_API bool Compress();
	NIAGARA_API bool Uncompress();

	NIAGARA_API void Reset();

	NIAGARA_API void SetData(const TArray<uint8>& InData);
	NIAGARA_API void SetData(TArray<uint8>&& InData);

	const TArray<uint8>& GetData() const { return Data; }
	TArray<uint8>& GetData() { return Data; }
	const uint8* GetDataPtr() const { return Data.GetData(); }
	int32 GetLength() const { return Data.Num(); }

	void Reserve(int32 Number) { Data.Reserve(Number); }
	void Shrink() { Data.Shrink(); }

	/** Used to upgrade a serialized property to our own struct */
	NIAGARA_API bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

template<>
struct TStructOpsTypeTraits<FNiagaraVMExecutableByteCode> : public TStructOpsTypeTraitsBase2<FNiagaraVMExecutableByteCode>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

/** Struct containing all of the data needed to run a Niagara VM executable script.*/
USTRUCT()
struct FNiagaraVMExecutableData
{
	GENERATED_USTRUCT_BODY()
public:
	NIAGARA_API FNiagaraVMExecutableData();

	/** Byte code to execute for this system. */
	UPROPERTY()
	FNiagaraVMExecutableByteCode ByteCode;

	/** Number of temp registers used by this script. */
	UPROPERTY()
	int32 NumTempRegisters;

	/** Number of user pointers we must pass to the VM. */
	UPROPERTY()
	int32 NumUserPtrs;

#if WITH_EDITORONLY_DATA
	/** All the data for using external constants in the script, laid out in the order they are expected in the uniform table.*/
	UPROPERTY()
	FNiagaraParameters Parameters;

	/** All the data for using external constants in the script, laid out in the order they are expected in the uniform table.*/
	UPROPERTY()
	FNiagaraParameters InternalParameters;

	/** List of all external dependencies of this script. If not met, linking should result in an error.*/
	UPROPERTY()
	TArray<FNiagaraCompileDependency> ExternalDependencies;

	/** The default set of rapid iteration parameters defined by the script that this data represents.  In the case of baked
	    in RI parameters this will be the values that are baked in, otherwise it will be the set of defaults based on the graphs. */
	UPROPERTY()
	TArray<FNiagaraVariable> BakedRapidIterationParameters;

	UPROPERTY()
	TArray<FNiagaraCompilerTag> CompileTagsEditorOnly;
#endif

	UPROPERTY()
	TArray<FNiagaraCompilerTag> CompileTags;

	UPROPERTY()
	TArray<uint8> ScriptLiterals;

	/** Attributes used by this script. */
	UPROPERTY()
	TArray<FNiagaraVariableBase> Attributes;

	/** Contains various usage information for this script. */
	UPROPERTY()
	FNiagaraScriptDataUsageInfo DataUsage;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TMap<FName, FNiagaraParameters> DataSetToParameters;

	UPROPERTY()
	TArray<FNiagaraFunctionSignature> AdditionalExternalFunctions;
#endif

	/** Information about all the UObjects used by this script. */
	UPROPERTY()
	TArray<FNiagaraScriptUObjectCompileInfo> UObjectInfos;

#if WITH_EDITORONLY_DATA
	/** Information about all data interfaces used by this script. */
	UPROPERTY()
	TArray<FNiagaraScriptDataInterfaceCompileInfo> DataInterfaceInfo;
#endif

	/** Array of ordered vm external functions to place in the function table. */
	UPROPERTY()
	TArray<FVMExternalFunctionBindingInfo> CalledVMExternalFunctions;

	UPROPERTY()
	TArray<FNiagaraDataSetID> ReadDataSets;

	UPROPERTY()
	TArray<FNiagaraDataSetProperties> WriteDataSets;

	/** Scopes we'll track with stats.*/
	UPROPERTY()
	TArray<FNiagaraStatScope> StatScopes;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	FString LastHlslTranslation;

	// Note that this is currently needed to be non-transient because of how we kick off compilation of GPUComputeScripts
	UPROPERTY()
	FString LastHlslTranslationGPU;

	UPROPERTY(Transient)
	FString LastAssemblyTranslation;

	UPROPERTY(Transient)
	uint32 LastOpCount;
#endif

	UPROPERTY()
	FNiagaraShaderScriptParametersMetadata ShaderScriptParametersMetadata; //TODO: GPU Param info should not be in the "VM executable data"

#if WITH_EDITORONLY_DATA
	/** The parameter collections used by this script. */
	UPROPERTY()
	TArray<FString> ParameterCollectionPaths;
#endif

	/** Last known compile status. Lets us determine the latest state of the script byte buffer.*/
	UPROPERTY()
	ENiagaraScriptCompileStatus LastCompileStatus;

	UPROPERTY()
	TArray<FSimulationStageMetaData> SimulationStageMetaData;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bReadsAttributeData;

	/** List of all attributes explicitly written by this VM script graph. Used to verify external dependencies.*/
	UPROPERTY()
	TArray<FNiagaraVariableBase> AttributesWritten;

	/** List of all attributes explicitly written by this VM script graph. Used to verify external dependencies.*/
	UPROPERTY()
	TArray<FNiagaraVariable> StaticVariablesWritten;

	UPROPERTY()
	FString ErrorMsg;

	/** Array of all compile events generated last time the script was compiled.*/
	UPROPERTY()
	TArray<FNiagaraCompileEvent> LastCompileEvents;
#endif

	UPROPERTY()
	TArray<uint8> ExperimentalContextData;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString LastExperimentalAssemblyScript;
#endif

	UPROPERTY()
	uint32 bReadsSignificanceIndex : 1;

	UPROPERTY()
	uint32 bNeedsGPUContextInit : 1;

	NIAGARA_API void SerializeData(FArchive& Ar, bool bDDCData);
	
	NIAGARA_API bool IsValid() const;

	NIAGARA_API void Reset();

#if WITH_EDITORONLY_DATA
	NIAGARA_API void BakeScriptLiterals(TArray<uint8>& OutLiterals) const;
#endif

	NIAGARA_API bool HasByteCode() const;
};

struct FNiagaraGraphCachedDataBase
{
	virtual ~FNiagaraGraphCachedDataBase() = default;
	virtual void GetStaticVariables(TArray<FNiagaraVariable>& OutVars) {}

	virtual bool IsValidForSystem(const UNiagaraSystem* InSystem) const { return false; };
	virtual void SetSourceSystem(const UNiagaraSystem* InSystem) {};

	virtual bool IsValidForEmitter(const FVersionedNiagaraEmitterData* InEmitterData) const { return false; };
	virtual void SetSourceEmitter(const FVersionedNiagaraEmitterData* InEmitterData) {};
};

/** Defines usages for inline dynamic input format tokens. */
UENUM()
enum class ENiagaraInlineDynamicInputFormatTokenUsage
{
	/** The format token refers to a dynamic input function input. */
	Input,
	/** The format token represents text which is displayed along with inputs. */
	Decorator,
	/** The format token represents a line break which will force the start a new row when sorting format tokens into groups. */
	LineBreak
};

/** Defines a single token in a format for displaying dynamic input trees inline in the UI.  These tokens can represent the
	inputs to the dynamic input or additional text and layout options. */
USTRUCT()
struct FNiagaraInlineDynamicInputFormatToken
{
	GENERATED_BODY();
#if WITH_EDITORONLY_DATA

	FNiagaraInlineDynamicInputFormatToken()
		: Usage(ENiagaraInlineDynamicInputFormatTokenUsage::Decorator)
	{
	}

	FNiagaraInlineDynamicInputFormatToken(ENiagaraInlineDynamicInputFormatTokenUsage InUsage, const FString& InValue)
		: Usage(InUsage)
		, Value(InValue)
	{
	}

	/** Defines how the value of this token should be used when formatting the dynamic input tree. */
	UPROPERTY(EditAnywhere, Category="Format")
	ENiagaraInlineDynamicInputFormatTokenUsage Usage;

	/** The value of this token which is used for formatting an inline dynamic input tree.  The purpose of this value is different
		depending on the value of the Usage property. */
	UPROPERTY(EditAnywhere, Category ="Format")
	FString Value;
#endif
};

/** Struct containing all of the data that can be different between different script versions.*/
USTRUCT()
struct FVersionedNiagaraScriptData
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
public:
	NIAGARA_API FVersionedNiagaraScriptData();

	UPROPERTY()
	FNiagaraAssetVersion Version;
	
	/** What changed in this version compared to the last? Displayed to the user when upgrading to a new script version. */
	UPROPERTY()
	FText VersionChangeDescription;

	/** When used as a module, what are the appropriate script types for referencing this module?*/
	UPROPERTY(EditAnywhere, Category = Script, meta = (Bitmask, BitmaskEnum = "/Script/Niagara.ENiagaraScriptUsage"))
	int32 ModuleUsageBitmask;

	/** Used to break up scripts of the same Usage type in UI display.*/
	UPROPERTY(EditAnywhere, Category = Script)
	FText Category;

	/** If this script is displayed inline in the overview (dynamic input in the overview node for example), use this display name.
	 *  Should be short and concise. */
	UPROPERTY(EditAnywhere, Category = Script)
	FText InlineOverviewDisplayName;

	/** If true, this script will be added to a 'Suggested' category at the top of menus during searches */
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = Script)
	bool bSuggested = false;
	
	/** Array of Ids of dependencies provided by this module to other modules on the stack (e.g. 'ProvidesNormalizedAge') */
	UPROPERTY(EditAnywhere, Category = Script)
	TArray<FName> ProvidedDependencies;

	/** Dependencies required by this module from other modules on the stack */
	UPROPERTY(EditAnywhere, Category = Script)
	TArray<FNiagaraModuleDependency> RequiredDependencies;
	
	/* If this script is no longer meant to be used, this option should be set.*/
	UPROPERTY()
	bool bDeprecated;
	
	/* Message to display when the script is deprecated. */
	UPROPERTY()
	FText DeprecationMessage;

	/* Which script to use if this is deprecated.*/
	UPROPERTY(EditAnywhere, Category = Script)
	TObjectPtr<UNiagaraScript> DeprecationRecommendation;

	/* If true then a python script will be executed when changing from this script to the selected deprecation recommendation. This allows the current script to transfer its inputs to the new script. */
	UPROPERTY(EditAnywhere, Category = Script)
	bool bUsePythonScriptConversion = false;

	/** Reference to a python script that is executed when the user updates from a previous version to this version. */
	UPROPERTY(EditAnywhere, Category = Script, meta=(EditCondition="bUsePythonScriptConversion", EditConditionHides, SegmentedDisplay))
	ENiagaraPythonUpdateScriptReference ConversionScriptExecution = ENiagaraPythonUpdateScriptReference::ScriptAsset;

	/** Python script to run when converting this script to the recommended deprecation update script. */
	UPROPERTY(EditAnywhere, Category = Script, meta=(MultiLine = true, EditCondition="bUsePythonScriptConversion && ConversionScriptExecution == ENiagaraPythonUpdateScriptReference::DirectTextEntry", EditConditionHides))
	FString PythonConversionScript;

	/** Asset reference to a python script to run when converting this script to the recommended deprecation update script. */
	UPROPERTY(EditAnywhere, Category = Script, meta=(EditCondition="bUsePythonScriptConversion && ConversionScriptExecution == ENiagaraPythonUpdateScriptReference::ScriptAsset", EditConditionHides))
	FFilePath ConversionScriptAsset;

	/* Custom logic to convert the contents of an existing script assignment to this script. Will also be used as a fallback if a python script is configured but the environment doesn't allow executing python scripts. */
	UPROPERTY(EditAnywhere, Category = Script)
	TSubclassOf<UNiagaraConvertInPlaceUtilityBase> ConversionUtility;

	/** Is this script experimental and less supported? */
	UPROPERTY(EditAnywhere, Category = Script)
	uint32 bExperimental : 1;

	/** The message to display when a function is marked experimental. */
	UPROPERTY(EditAnywhere, Category = Script, meta = (EditCondition = "bExperimental", MultiLine = true))
	FText ExperimentalMessage;

	/** A message to display when adding the module to the stack. This is useful to highlight pitfalls or weird behavior of the module. */
	UPROPERTY(EditAnywhere, Category = Script, meta = (MultiLine = true))
	FText NoteMessage;

	/** A message to display on UI actions handling debug draw state. */
	UPROPERTY(EditAnywhere, Category = Script, meta = (MultiLine = true))
	FText DebugDrawMessage;

	/* Defines if this script is visible to the user when searching for modules to add to an emitter.  */
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = Script, meta = (SegmentedDisplay))
	ENiagaraScriptLibraryVisibility LibraryVisibility;

	/** The mode to use when deducing the type of numeric output pins from the types of the input pins. */
	UPROPERTY(EditAnywhere, Category=Script)
	ENiagaraNumericOutputTypeSelectionMode NumericOutputTypeSelectionMode;

	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = Script, meta = (MultiLine = true))
	FText Description;

	/** A list of space separated keywords which can be used to find this script in editor menus. */
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = Script)
	FText Keywords;

	/** The format for the text to display in the stack if the value is collapsed.
	*  This supports formatting placeholders for the function inputs, for example "myfunc({0}, {1})" will be converted to "myfunc(1.23, Particles.Position)". */
	UPROPERTY(EditAnywhere, Category = Script)
	FText CollapsedViewFormat;

	UPROPERTY(EditAnywhere, Category = "Script")
	TArray<FNiagaraInlineDynamicInputFormatToken> InlineExpressionFormat;

	UPROPERTY(EditAnywhere, Category = "Script")
	TArray<FNiagaraInlineDynamicInputFormatToken> InlineGraphFormat;

	/* If used as a dynamic input with exactly one input and output of different types, setting this to true will auto-insert this script to convert from one type to another when dragging and dropping parameters in the stack.
	   For example, a script with a bool as input and a float as output will be automatically inserted in the stack when dropping a bool parameter into the float input of a module in the stack. */
	UPROPERTY(EditAnywhere, Category = Script)
	uint32 bCanBeUsedForTypeConversions : 1;

	UPROPERTY(EditAnywhere, Category = Script, DisplayName = "Script Metadata", meta = (ToolTip = "Script Metadata"))
	TMap<FName, FString> ScriptMetaData;

	/** Adjusted every time ComputeVMCompilationId is called.*/
	UPROPERTY()
	mutable FNiagaraVMExecutableDataId LastGeneratedVMId;

	/** Reference to a python script that is executed when the user updates from a previous version to this version. */
	UPROPERTY()
	ENiagaraPythonUpdateScriptReference UpdateScriptExecution = ENiagaraPythonUpdateScriptReference::None;

	/** Python script to run when updating to this script version. */
	UPROPERTY()
	FString PythonUpdateScript;

	/** Asset reference to a python script to run when updating to this script version. */
	UPROPERTY()
	FFilePath ScriptAsset;

	/* Custom logic to convert the contents of an existing script assignment to this script.*/
	UPROPERTY()
	TSubclassOf<UNiagaraVersionUpgradeBase> UpdateUtilityClass;

	/** Subscriptions to parameter definitions for this script version */
	UPROPERTY()
	TArray<FParameterDefinitionsSubscription> ParameterDefinitionsSubscriptions;
	NIAGARA_API TArray<ENiagaraScriptUsage> GetSupportedUsageContexts() const;
	
	class UNiagaraScriptSourceBase* GetSource() { return Source; }

	UPROPERTY()
	TArray<FNiagaraStackSection> InputSections_DEPRECATED;
private:
	friend class UNiagaraScript;

	/** 'Source' data/graphs for this script */
	UPROPERTY()
	TObjectPtr<class UNiagaraScriptSourceBase>	Source = nullptr;
#endif	
};

/** Scripts are function graphs that define the runtime execution for a Niagara system (similar to a Blueprint).
 *
 * There are three types of scripts:
 * 1) Module: can be added as a standalone part to the emitter stack and encapsulates a single behavior, for example "Add Velocity"
 * 2) Dynamic input: has a single output value and can be added to any input in the stack to compute such a value, for example "Random Vector" 
 * 3) Function: usually reserved for helper functions; can only be called from within modules or dynamic inputs 
 */
UCLASS(MinimalAPI)
class UNiagaraScript : public UNiagaraScriptBase, public FNiagaraVersionedObject
{
	GENERATED_UCLASS_BODY()
public:
	UNiagaraScript();

#if WITH_EDITORONLY_DATA
	/** If true then this script asset uses active version control to track changes. */
	virtual bool IsVersioningEnabled() const override { return bVersioningEnabled; }

	/** Returns the script data for latest exposed version. */
	NIAGARA_API FVersionedNiagaraScriptData* GetLatestScriptData();
	NIAGARA_API const FVersionedNiagaraScriptData* GetLatestScriptData() const;

	/** Returns the script data for a specific version or nullptr if no such version is found. For the null-Guid it returns the exposed version.  */
	NIAGARA_API FVersionedNiagaraScriptData* GetScriptData(const FGuid& VersionGuid);
	NIAGARA_API const FVersionedNiagaraScriptData* GetScriptData(const FGuid& VersionGuid) const;

	/** Returns all available versions for this script. */
	NIAGARA_API virtual TArray<FNiagaraAssetVersion> GetAllAvailableVersions() const override;

	/** Returns the version of the exposed version data (i.e. the version used when adding a module to the stack) */
	NIAGARA_API virtual FNiagaraAssetVersion GetExposedVersion() const override;

	/** Returns the version data for the given guid, if it exists. Otherwise returns nullptr. */
	NIAGARA_API virtual FNiagaraAssetVersion const* FindVersionData(const FGuid& VersionGuid) const override;
	
	/** Creates a new data entry for the given version number. The version must be > 1.0 and must not collide with an already existing version. The data will be a copy of the previous minor version. */
	NIAGARA_API virtual FGuid AddNewVersion(int32 MajorVersion, int32 MinorVersion) override;

	/** Deletes the version data for an existing version. The exposed version cannot be deleted and will result in an error. Does nothing if the guid does not exist in the script's version data. */
	NIAGARA_API virtual void DeleteVersion(const FGuid& VersionGuid) override;

	/** Changes the exposed version. Does nothing if the guid does not exist in the script's version data. */
	NIAGARA_API virtual void ExposeVersion(const FGuid& VersionGuid) override;

	/** Enables versioning for this script asset. */
	NIAGARA_API virtual void EnableVersioning() override;

	/** Disables versioning and keeps only the data from the given version guid. Note that this breaks ALL references from existing assets and should only be used when creating a copy of a script, as the effect is very destructive.  */
	NIAGARA_API virtual void DisableVersioning(const FGuid& VersionGuidToUse) override;

	NIAGARA_API virtual TSharedPtr<FNiagaraVersionDataAccessor> GetVersionDataAccessor(const FGuid& Version) override; 

	/** Makes sure that the default version data is available and fixes old script assets. */
	NIAGARA_API void CheckVersionDataAvailable();

	/** Creates a shallow transient copy of this script for compilation purposes. */
	NIAGARA_API UNiagaraScript* CreateCompilationCopy();

	/** A set of rules to apply when this script is used in the stack. To create your own rules, write a custom class that extends UNiagaraValidationRule. */
	UPROPERTY(EditAnywhere, Category = "Validation", Instanced)
	TArray<TObjectPtr<UNiagaraValidationRule>> ValidationRules;
#endif

	/** Workaround for emitter versioning because we used a lot of Script->GetOuter() previously. */
	NIAGARA_API FVersionedNiagaraEmitter GetOuterEmitter() const;

	// how this script is to be used. cannot be private due to use of GET_MEMBER_NAME_CHECKED
	UPROPERTY(AssetRegistrySearchable)
	ENiagaraScriptUsage Usage;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnScriptCompiled, UNiagaraScript*, const FGuid&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPropertyChanged, FPropertyChangedEvent& /* PropertyChangedEvent */)
#endif

private:
	/** Specifies a unique id for use when there are multiple scripts with the same usage, e.g. events. */
	UPROPERTY()
	FGuid UsageId;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category="Preview", AssetRegistrySearchable, meta = (AllowedClasses="/Script/MediaAssets.FileMediaSource"))
	FSoftObjectPath PreviewMoviePath;
	
	/** The exposed version is the version that is used by default when a user adds this script somewhere. It is basically the published version and allows a script maintainer to create and test newer versions that are not used by normal users. */
	UPROPERTY()
	FGuid ExposedVersion;

	/** If true then this script asset uses active version control to track changes. */
	UPROPERTY()
	bool bVersioningEnabled = false;

	/** Contains all of the versioned script data. */
	UPROPERTY()
	TArray<FVersionedNiagaraScriptData> VersionData;

	/** Editor time adapters to a specific VersionData and this Script ptr to handle synchronizing changes made by parameter definitions. */
	TArray<struct FVersionedNiagaraScript> VersionedScriptAdapters;
#endif

public:

	/** Contains all of the top-level values that are iterated on in the UI. These are usually "Module" variables in the graph. They don't necessarily have to be in the order that they are expected in the uniform table.*/
	UPROPERTY()
	FNiagaraParameterStore RapidIterationParameters;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FNiagaraParameterStore RapidIterationParametersCookedEditorCache;

	/** This is used as a transient value to open a specific version in the graph editor */
	UPROPERTY(Transient)
	FGuid VersionToOpenInEditor;

	/** Which instance of the usage in the graph to use.  This is now deprecated and is handled by UsageId. */
	UPROPERTY()
	int32 UsageIndex_DEPRECATED;
	
	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	int32 ModuleUsageBitmask_DEPRECATED;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	FText Category_DEPRECATED;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<FName> ProvidedDependencies_DEPRECATED;
	
	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<FNiagaraModuleDependency> RequiredDependencies_DEPRECATED;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	uint32 bDeprecated_DEPRECATED : 1;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	FText DeprecationMessage_DEPRECATED;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<UNiagaraScript> DeprecationRecommendation_DEPRECATED;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	TSubclassOf<UNiagaraConvertInPlaceUtilityBase> ConversionUtility_DEPRECATED;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	uint32 bExperimental_DEPRECATED : 1;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	FText ExperimentalMessage_DEPRECATED;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	FText NoteMessage_DEPRECATED;

	/* Deprecated, use LibraryVisibility instead. */
	UPROPERTY(meta = (DeprecatedProperty))
	uint32 bExposeToLibrary_DEPRECATED : 1;
	
	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	ENiagaraScriptLibraryVisibility LibraryVisibility_DEPRECATED;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	ENiagaraNumericOutputTypeSelectionMode NumericOutputTypeSelectionMode_DEPRECATED;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	FText Description_DEPRECATED;

	/** Use property in struct returned from GetScriptData() instead */
	UPROPERTY(meta = (DeprecatedProperty))
	FText Keywords_DEPRECATED;

	/* Deprecated, use LibraryVisibility instead. */
	UPROPERTY(meta = (DeprecatedProperty))
	FText CollapsedViewFormat_DEPRECATED;

	/* Deprecated, use LibraryVisibility instead. */
	UPROPERTY(meta = (DeprecatedProperty))
	TMap<FName, FString> ScriptMetaData_DEPRECATED;

	/** 'Source' data/graphs for this script */
	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<class UNiagaraScriptSourceBase>	Source_DEPRECATED;

	NIAGARA_API static const FName NiagaraCustomVersionTagName;

	NIAGARA_API void ComputeVMCompilationId(FNiagaraVMExecutableDataId& Id, const FGuid& VersionGuid, FNiagaraScriptHashCollector* HashCollector = nullptr) const;
#endif

	const FNiagaraVMExecutableDataId& GetComputedVMCompilationId() const
	{
#if WITH_EDITORONLY_DATA
		if (!IsScriptCooked())
		{
			return GetLastGeneratedVMId();
		}
#endif
		return CachedScriptVMId;
	}

	void SetUsage(ENiagaraScriptUsage InUsage) { Usage = InUsage; }
	ENiagaraScriptUsage GetUsage() const { return Usage; }

	void SetUsageId(const FGuid& InUsageId) { UsageId = InUsageId; }
	FGuid GetUsageId() const { return UsageId; }

	NIAGARA_API bool ContainsUsage(ENiagaraScriptUsage InUsage) const;
	bool IsEquivalentUsage(ENiagaraScriptUsage InUsage) const {return (InUsage == Usage) || (Usage == ENiagaraScriptUsage::ParticleSpawnScript && InUsage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated) || (Usage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated && InUsage == ENiagaraScriptUsage::ParticleSpawnScript);}
	static bool IsEquivalentUsage(ENiagaraScriptUsage InUsageA, ENiagaraScriptUsage InUsageB) { return (InUsageA == InUsageB) || (InUsageB == ENiagaraScriptUsage::ParticleSpawnScript && InUsageA == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated) || (InUsageB == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated && InUsageA == ENiagaraScriptUsage::ParticleSpawnScript); }
	/** Is usage A dependent on Usage B?*/
	NIAGARA_API static bool IsUsageDependentOn(ENiagaraScriptUsage InUsageA, ENiagaraScriptUsage InUsageB);

	bool IsParticleSpawnScript() const { return IsParticleSpawnScript(Usage); }
	bool IsInterpolatedParticleSpawnScript() const { return IsInterpolatedParticleSpawnScript(Usage); }
	bool IsParticleUpdateScript() const { return IsParticleUpdateScript(Usage); }
	bool IsModuleScript() const { return IsModuleScript(Usage); }
	bool IsFunctionScript()	const { return IsFunctionScript(Usage); }
	bool IsDynamicInputScript() const { return IsDynamicInputScript(Usage); }
	bool IsParticleEventScript() const { return IsParticleEventScript(Usage); }
	bool IsParticleScript() const {	return IsParticleScript(Usage);}
	bool IsGPUScript() const { return IsGPUScript(Usage); }

	bool IsNonParticleScript() const { return IsNonParticleScript(Usage); }
	
	bool IsSystemSpawnScript() const { return IsSystemSpawnScript(Usage); }
	bool IsSystemUpdateScript() const { return IsSystemUpdateScript(Usage); }
	bool IsEmitterSpawnScript() const { return IsEmitterSpawnScript(Usage); }
	bool IsEmitterUpdateScript() const { return IsEmitterUpdateScript(Usage); }
	bool IsStandaloneScript() const { return IsStandaloneScript(Usage); }

	bool IsSpawnScript() const { return IsParticleSpawnScript() || IsEmitterSpawnScript() || IsSystemSpawnScript(); }

	bool IsCompilable() const { return IsCompilable(Usage); }


	static bool IsGPUScript(ENiagaraScriptUsage Usage) { return Usage == ENiagaraScriptUsage::ParticleGPUComputeScript; }
	static bool IsParticleSpawnScript(ENiagaraScriptUsage Usage)  { return Usage == ENiagaraScriptUsage::ParticleSpawnScript || Usage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated; }
	static bool IsInterpolatedParticleSpawnScript(ENiagaraScriptUsage Usage) { return Usage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated; }
	static bool IsParticleUpdateScript(ENiagaraScriptUsage Usage) { return Usage == ENiagaraScriptUsage::ParticleUpdateScript; }
	static bool IsParticleStageScript(ENiagaraScriptUsage Usage) { return Usage == ENiagaraScriptUsage::ParticleSimulationStageScript; }
	static bool IsModuleScript(ENiagaraScriptUsage Usage) { return Usage == ENiagaraScriptUsage::Module; }
	static bool IsFunctionScript(ENiagaraScriptUsage Usage)	 { return Usage == ENiagaraScriptUsage::Function; }
	static bool IsDynamicInputScript(ENiagaraScriptUsage Usage)	 { return Usage == ENiagaraScriptUsage::DynamicInput; }
	static bool IsParticleEventScript(ENiagaraScriptUsage Usage) { return Usage == ENiagaraScriptUsage::ParticleEventScript; }
	static bool IsParticleScript(ENiagaraScriptUsage Usage)  { return Usage >= ENiagaraScriptUsage::ParticleSpawnScript && Usage <= ENiagaraScriptUsage::ParticleGPUComputeScript; }

	static bool IsNonParticleScript(ENiagaraScriptUsage Usage) { return Usage >= ENiagaraScriptUsage::EmitterSpawnScript; }

	static bool IsSystemSpawnScript(ENiagaraScriptUsage Usage) { return Usage == ENiagaraScriptUsage::SystemSpawnScript; }
	static bool IsSystemUpdateScript(ENiagaraScriptUsage Usage) { return Usage == ENiagaraScriptUsage::SystemUpdateScript; }
	static bool IsSystemScript(ENiagaraScriptUsage Usage) { return IsSystemSpawnScript(Usage) || IsSystemUpdateScript(Usage);}

	static bool IsEmitterScript(ENiagaraScriptUsage Usage) { return IsEmitterSpawnScript(Usage) || IsEmitterUpdateScript(Usage); }
	static bool IsEmitterSpawnScript(ENiagaraScriptUsage Usage) { return Usage == ENiagaraScriptUsage::EmitterSpawnScript; }
	static bool IsEmitterUpdateScript(ENiagaraScriptUsage Usage) { return Usage == ENiagaraScriptUsage::EmitterUpdateScript; }
	static bool IsStandaloneScript(ENiagaraScriptUsage Usage)  { return IsDynamicInputScript(Usage) || IsFunctionScript(Usage) || IsModuleScript(Usage); }

	static bool IsSpawnScript(ENiagaraScriptUsage Usage) { return IsParticleSpawnScript(Usage) || IsEmitterSpawnScript(Usage) || IsSystemSpawnScript(Usage); }

	static bool IsCompilable(ENiagaraScriptUsage Usage)  { return !IsEmitterSpawnScript(Usage) && !IsEmitterUpdateScript(Usage); }

	static bool NIAGARA_API ConvertUsageToGroup(ENiagaraScriptUsage InUsage, ENiagaraScriptGroup& OutGroup);

#if WITH_EDITORONLY_DATA
	static NIAGARA_API TArray<ENiagaraScriptUsage> GetSupportedUsageContextsForBitmask(int32 InModuleUsageBitmask, bool bIncludeHiddenUsages = false);
	static NIAGARA_API bool IsSupportedUsageContextForBitmask(int32 InModuleUsageBitmask, ENiagaraScriptUsage InUsageContext, bool bIncludeHiddenUsages = false);
	static NIAGARA_API bool ContainsEquivilentUsage(const TArray<ENiagaraScriptUsage>& Usages, ENiagaraScriptUsage InUsage);
#endif

	NIAGARA_API bool CanBeRunOnGpu() const;
	NIAGARA_API bool IsReadyToRun(ENiagaraSimTarget SimTarget) const;
#if WITH_EDITORONLY_DATA
	NIAGARA_API static void SetPreviewFeatureLevel(ERHIFeatureLevel::Type PreviewFeatureLevel);
	NIAGARA_API static ERHIFeatureLevel::Type GetPreviewFeatureLevel();
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API class UNiagaraScriptSourceBase* GetLatestSource();
	NIAGARA_API const class UNiagaraScriptSourceBase* GetLatestSource() const;
	NIAGARA_API class UNiagaraScriptSourceBase* GetSource(const FGuid& VersionGuid);
	NIAGARA_API const class UNiagaraScriptSourceBase* GetSource(const FGuid& VersionGuid) const;
	NIAGARA_API void SetLatestSource(class UNiagaraScriptSourceBase* InSource);
	NIAGARA_API void SetSource(class UNiagaraScriptSourceBase* InSource, const FGuid& VersionGuid);

	NIAGARA_API FGuid GetBaseChangeID(const FGuid& VersionGuid = FGuid()) const;
	NIAGARA_API ENiagaraScriptCompileStatus GetLastCompileStatus() const;

	NIAGARA_API bool HandleVariableRenames(const TMap<FNiagaraVariable, FNiagaraVariable>& OldToNewVars, const FString& UniqueEmitterName);
#endif

	//~ Begin UObject interface
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void Serialize(FArchive& Ar)override;
	virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeVersionedProperty(FPropertyChangedEvent& PropertyChangedEvent, const FGuid& Version);
	virtual void PostEditUndo();
#endif
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

	virtual bool IsEditorOnly() const override;

	virtual NIAGARA_API void BeginDestroy() override;
	virtual NIAGARA_API bool IsReadyForFinishDestroy() override;
	//~ End UObject interface

	//~ Begin UNiagaraScriptBase interface
	virtual bool ShouldCompile(EShaderPlatform Platform) const override;
	virtual void ModifyCompilationEnvironment(EShaderPlatform Platform, struct FShaderCompilerEnvironment& OutEnvironment) const override;
	virtual TConstArrayView<FSimulationStageMetaData> GetSimulationStageMetaData() const override { return MakeArrayView(CachedScriptVM.SimulationStageMetaData); }
	//~ End UNiagaraScriptBase interface

	// Infrastructure for GPU compute Shaders
#if WITH_EDITOR
	// utility function answering whether it's the NiagaraSystem or the NiagaraScript that will be responsible
	// for the generation of the FNiagaraShaderMap and general compilation for GPU scripts
	NIAGARA_API static bool AreGpuScriptsCompiledBySystem();

	NIAGARA_API void CacheResourceShadersForCooking(EShaderPlatform ShaderPlatform, TArray<TUniquePtr<FNiagaraShaderScript>>& InOutCachedResources, const ITargetPlatform* TargetPlatform = nullptr);

	NIAGARA_API void CacheResourceShadersForRendering(bool bRegenerateId, bool bForceRecompile=false);
	void BeginCacheForCookedPlatformData(const ITargetPlatform *TargetPlatform);
	virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	void CacheShadersForResources(FNiagaraShaderScript* ResourceToCache, bool bApplyCompletedShaderMapForRendering, bool bForceRecompile = false, bool bCooking=false, const ITargetPlatform* TargetPlatform = nullptr);
	void SaveShaderStableKeys(const class ITargetPlatform* TP);

	TArray<FName> FindShaderFormatsForCooking(const ITargetPlatform* TargetPlatform) const;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	/**
	 * Called from UNiagaraSystem::AppendToClassSchema() to collect the Script specific details for iterative cooking
	 */
	static void BuildClassSchema(FAppendToClassSchemaContext& Context);
#endif

	FNiagaraShaderScript* GetRenderThreadScript()
	{
		return ScriptResource.Get();
	}

	const FNiagaraShaderScript* GetRenderThreadScript() const
	{
		return ScriptResource.Get();
	}

	NIAGARA_API void GenerateStatIDs();

	NIAGARA_API bool IsScriptCompilationPending(bool bGPUScript) const;
	NIAGARA_API bool DidScriptCompilationSucceed(bool bGPUScript) const;

#if WITH_EDITORONLY_DATA
	template<typename T>
	TOptional<T> GetCompiledStaticVariableValue(const FNiagaraVariableBase& InVar) const
	{
		for (const FNiagaraVariable& StaticVariable : CachedScriptVM.StaticVariablesWritten)
		{
			if (static_cast<const FNiagaraVariableBase&>(StaticVariable) == InVar)
			{
				if (StaticVariable.IsDataAllocated())
				{
					return TOptional<T>(StaticVariable.GetValue<T>());
				}
			}
		}

		return TOptional<T>();
	}

	// use of this function differs from the above in that the data is being pulled from the RapidIterationParameters rather than
	// the compiled data
	void GatherScriptStaticVariables(TArray<FNiagaraVariable>& StaticVariables) const;
	bool HasScriptStaticVariables() const;
#endif

	template<typename T>
	TOptional<T> GetCompilerTag(const FNiagaraVariableBase& InVar, const FNiagaraParameterStore* FallbackParameterStore = nullptr) const
	{
		for (const FNiagaraCompilerTag& Tag : CachedScriptVM.CompileTags)
		{
			if (static_cast<const FNiagaraVariableBase&>(Tag.Variable) == InVar)
			{
				if (Tag.Variable.IsDataAllocated())
				{
					return TOptional<T>(Tag.Variable.GetValue<T>());
				}
				else if (const int32* Offset = RapidIterationParameters.FindParameterOffset(FNiagaraVariableBase(Tag.Variable.GetType(), *Tag.StringValue)))
				{
					return TOptional<T>(*(T*)RapidIterationParameters.GetParameterData(*Offset, Tag.Variable.GetType()));
				}
				else if (FallbackParameterStore)
				{
					if (const int32* OffsetAlternate = FallbackParameterStore->FindParameterOffset(FNiagaraVariableBase(Tag.Variable.GetType(), *Tag.StringValue)))
						return TOptional<T>(*(T*)FallbackParameterStore->GetParameterData(*OffsetAlternate, Tag.Variable.GetType()));
				}
			}
		}

		return TOptional<T>();
	}

	TOptional<FVector3d> GetCompilerTag(const FNiagaraVariableBase& InVar, const FNiagaraParameterStore* FallbackParameterStore) const
	{
		TOptional<FVector3f> Optional = GetCompilerTag<FVector3f>(InVar, FallbackParameterStore);
		if (Optional.IsSet())
		{
			return TOptional(FVector3d(Optional.GetValue()));
		}
		return TOptional<FVector3d>();
	}

#if WITH_EDITORONLY_DATA
	NIAGARA_API void InvalidateCompileResults(const FString& Reason);
	NIAGARA_API FText GetDescription(const FGuid& VersionGuid);

	NIAGARA_API static bool ShouldUseDDC();

	/** Helper to convert the struct from its binary data out of the DDC to it's actual in-memory version.
		Do not call this on anything other than the game thread as it depends on the FObjectAndNameAsStringProxyArchive,
		which calls FindStaticObject which can fail when used in any other thread!*/
	static bool BinaryToExecData(const UNiagaraScript* Script, const TArray<uint8>& InBinaryData, FNiagaraVMExecutableData& OutExecData);

	/** Reverse of the BinaryToExecData() function */
	static bool ExecToBinaryData(const UNiagaraScript* Script, TArray<uint8>& OutBinaryData, FNiagaraVMExecutableData& InExecData);

	/** Determine if the Script and its source graph are in sync.*/
	NIAGARA_API bool AreScriptAndSourceSynchronized(const FGuid& VersionGuid = FGuid()) const;

	/** Ensure that the Script and its source graph are marked out of sync.*/
	NIAGARA_API void MarkScriptAndSourceDesynchronized(FString Reason, const FGuid& VersionGuid);

	NIAGARA_API bool IsScriptShaderSynchronized(const FGuid& VersionGuid = FGuid()) const;
	
	/** Request a synchronous compile for the script, possibly forcing it to compile.*/
	NIAGARA_API void RequestCompile(const FGuid& ScriptVersion, bool bForceCompile = false);

	/** Request an asynchronous compile for the script, possibly forcing it to compile. The output values are the compilation id of the data as well as the async handle to 
		gather up the results with. The function returns whether or not any compiles were actually issued. */
	NIAGARA_API bool RequestExternallyManagedAsyncCompile(const TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe>& RequestData, const TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe>& RequestDuplicateData, FNiagaraVMExecutableDataId& OutCompileId, uint32& OutAsyncHandle);

	/** Builds the DDC string for the derived data cache using the supplied CompiledId and ScriptPath */
	static FString BuildNiagaraDDCKeyString(const FNiagaraVMExecutableDataId& CompileId, const FString& ScriptPath);

	/** Creates a string key for the derived data cache */
	FString GetNiagaraDDCKeyString(const FGuid& ScriptVersion, const FString& ScriptPath);

	/** Callback issued whenever a VM script compilation successfully happened (even if the results are a script that cannot be executed due to errors)*/
	NIAGARA_API FOnScriptCompiled& OnVMScriptCompiled();

	/** Callback issued whenever a GPU script compilation successfully happened (even if the results are a script that cannot be executed due to errors)*/
	NIAGARA_API FOnScriptCompiled& OnGPUScriptCompiled();

	/** Callback issues whenever post edit changed is called on this script. */
	NIAGARA_API FOnPropertyChanged& OnPropertyChanged();

	/** External call used to identify the values for a successful VM script compilation. OnVMScriptCompiled will be issued in this case.*/
	void SetVMCompilationResults(const FNiagaraVMExecutableDataId& InCompileId, FNiagaraVMExecutableData& InScriptVM, const FString& EmitterUniqueName, const TMap<FName, UNiagaraDataInterface*>& ObjectNameMap, bool ApplyRapidIterationParameters);

	void SetComputeCompilationResults(
		const ITargetPlatform* TargetPlatform,
		EShaderPlatform ShaderPlatform,
		ERHIFeatureLevel::Type FeatureLevel,
		const FNiagaraShaderScriptParametersMetadata& SharedParameters,
		const FNiagaraShaderMapRef& ShaderMap,
		TConstArrayView<FShaderCompilerError> CompilationErrors);

	NIAGARA_API bool IsShaderMapCached(const ITargetPlatform* TargetPlatform, const FNiagaraShaderMapId& ShaderMapId) const;

	NIAGARA_API void AssignRapidIterationParameters(const FNiagaraVMExecutableData& InScriptVM);

	/** Updates the RI parameter store based on the provided variables (missing entries will be
	    added and stale entries will be removed.  Returns true if the parameter store was modified. */
	NIAGARA_API bool ApplyRapidIterationParameters(TConstArrayView<FNiagaraVariable> InParameters, bool bAllowRemoval);

	/** In the event where we "merge" we duplicate the changes of the source script onto the newly cloned copy. This function will synchronize the compiled script 
		results assuming that the scripts themselves are bound to the same key. This saves looking things up in the DDC. It returns true if successfully synchronized and 
		false if not.*/
	NIAGARA_API bool SynchronizeExecutablesWithCompilation(const UNiagaraScript* Script, const TMap<FString, FString>& RenameMap);

	NIAGARA_API FString GetFriendlyName() const;

	NIAGARA_API void SyncAliases(const FNiagaraAliasContext& ResolveAliasesContext);
#endif
	
	UFUNCTION()
	void RaiseOnGPUCompilationComplete();


	inline FNiagaraVMExecutableData& GetVMExecutableData() { return CachedScriptVM; }
	inline const FNiagaraVMExecutableData& GetVMExecutableData() const { return CachedScriptVM; }
	inline const FNiagaraVMExecutableDataId& GetVMExecutableDataCompilationId() const { return CachedScriptVMId; }

	TConstArrayView<FNiagaraDataInterfaceGPUParamInfo> GetDataInterfaceGPUParamInfos() const { return CachedScriptVM.ShaderScriptParametersMetadata.DataInterfaceParamInfo; }

	NIAGARA_API TArray<TObjectPtr<UNiagaraParameterCollection>>& GetCachedParameterCollectionReferences();

#if WITH_EDITORONLY_DATA
	TArray<FNiagaraScriptDataInterfaceInfo>& GetCachedDefaultDataInterfaces() { return CachedDefaultDataInterfaces; }
	TConstArrayView<FNiagaraScriptDataInterfaceInfo> GetCachedDefaultDataInterfaces() const { return MakeArrayView(CachedDefaultDataInterfaces); }

	TArrayView<FNiagaraScriptUObjectCompileInfo> GetCachedDefaultUObjects() { return CachedDefaultUObjects; }
	TConstArrayView<FNiagaraScriptUObjectCompileInfo> GetCachedDefaultUObjects() const { return CachedDefaultUObjects; }
#endif

#if STATS
	TArrayView<const TStatId> GetStatScopeIDs() const { return MakeArrayView(StatScopesIDs); }
#elif ENABLE_STATNAMEDEVENTS
	TArrayView<const FString> GetStatNamedEvents() const { return MakeArrayView(StatNamedEvents); }
#endif

	bool UsesCollection(const class UNiagaraParameterCollection* Collection)const;

	bool IsScriptCooked() const
	{
	#if WITH_EDITORONLY_DATA
		return IsCooked;
	#else
		return true;
	#endif
	}

private:
	bool OwnerCanBeRunOnGpu() const;
	bool LegacyCanBeRunOnGpu()const;

	void ProcessSerializedShaderMaps();
	void SerializeNiagaraShaderMaps(FArchive& Ar, int32 NiagaraVer, bool IsValidShaderScript);

	/** Return the expected SimTarget for this script. Only returns a valid target if there is valid data to run with. */
	TOptional<ENiagaraSimTarget> GetSimTarget() const;

	void ProcessCompilationErrors(bool bCompileSucceeded, TConstArrayView<FShaderCompilerError> CompilationErrors, TArray<FString>& ProcessedErrors);

public:
	static int32 NIAGARA_API LogCompileStaticVars;

#if WITH_EDITORONLY_DATA
	/* Safely resolves soft object parameter collection references into hard references. */
	void ResolveParameterCollectionReferences();

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
	class UNiagaraSystem* FindRootSystem();

	bool HasIdsRequiredForShaderCaching() const;

	/** A multicast delegate which is called whenever the script has been compiled (successfully or not). */
	FOnScriptCompiled OnVMScriptCompiledDelegate;
	FOnScriptCompiled OnGPUScriptCompiledDelegate;
	FOnPropertyChanged OnPropertyChangedDelegate;

	mutable FNiagaraVMExecutableDataId LastReportedVMId;

	mutable TOptional<TMap<FName, FString>> CustomAssetRegistryTagCache;
#endif

	/** Adjusted every time that we compile this script. Lets us know that we might differ from any cached versions.*/
	UPROPERTY()
	FNiagaraVMExecutableDataId CachedScriptVMId;

#if WITH_EDITORONLY_DATA
	FNiagaraVMExecutableDataId& GetLastGeneratedVMId(const FGuid& VersionGuid = FGuid()) const;
#endif

	TSharedPtr<FNiagaraShaderScript> ScriptResource;

#if WITH_EDITORONLY_DATA
	TArray<FNiagaraShaderScript> LoadedScriptResources;
	TSharedPtr<FNiagaraShaderScript> ScriptResourcesByFeatureLevel[ERHIFeatureLevel::Num];
#endif

	/** Runtime stat IDs generated from StatScopes. */
#if STATS
	TArray<TStatId> StatScopesIDs;
#elif ENABLE_STATNAMEDEVENTS
	TArray<FString> StatNamedEvents;
#endif

#if WITH_EDITORONLY_DATA
	/* script resources being cached for cooking. */
	TMap<const class ITargetPlatform*, TArray<TUniquePtr<FNiagaraShaderScript>>> CachedScriptResourcesForCooking;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UObject>> ActiveCompileRoots;

	/* Flag set on load based on whether we're loading from a cooked package. */
	bool IsCooked = false;

	/* Transient flag marking that the script will need to rebuild the hierarchy root based on the custom version encountered during serialize.
	*  Note that this is required because GetLinkerCustomVersion() is not reliable when assets are being duplicated, something that happens
	*  often to our scripts during emitter merging, for example */
	bool bMigrateParameterDataToHierarchyRoot = false;
#endif

	/** Compiled VM bytecode and data necessary to run this script.*/
	UPROPERTY()
	FNiagaraVMExecutableData CachedScriptVM;
	
	UPROPERTY()
	TArray<TObjectPtr<UNiagaraParameterCollection>> CachedParameterCollectionReferences;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FNiagaraScriptDataInterfaceInfo> CachedDefaultDataInterfaces;
#endif

#if WITH_EDITORONLY_DATA
	// Populated from compilation process, used to generate resolved list
	UPROPERTY()
	TArray<FNiagaraScriptUObjectCompileInfo> CachedDefaultUObjects;
#endif

	static UNiagaraDataInterface* CopyDataInterface(UNiagaraDataInterface* Src, UObject* Owner);

	/** Flag used to guarantee that the RT isn't accessing the FNiagaraScriptResource before cleanup. */
	FThreadSafeBool ReleasedByRT;

	private :

#if WITH_EDITORONLY_DATA
	void ComputeVMCompilationId_EmitterShared(FNiagaraVMExecutableDataId& Id, const FVersionedNiagaraEmitter& Emitter, ENiagaraRendererSourceDataMode InSourceMode) const;

	// These data interfaces are populated via InvalidateAndEmptyDefaultDataInterfaces().  This helps avoid naming conflicts of the DataInterfaces stored within
	// CachedDefaultDataInterfaces.  When we are updating the compiled data we end up creating new DataInterfaces but we want to ensure that they are deterministically
	// named, and simply having the original DI become unreferenced leaves us in a state where a non-deterministic GC during the cook process could free the DI, but
	// is not guaranteed.  We resolve this by renaming our invalidated DI, thus creating room for the newly created DI to be named properly.  The last caveat is that
	// we have to be careful about when we rename the DI as the package containing the DI may get reloaded, which would recreate the DI (with the original name) leaving
	// us, once again, in a state where the named slot is taken.
	UPROPERTY(Transient, DuplicateTransient)
	TArray<TObjectPtr<UNiagaraDataInterface>> DataInterfacesPendingInvalidation;

	enum EDataInterfaceInvalidationMode
	{
		QueueInvaliation,	// data interfaces will be queued up to invalidate in the future
		ProcessInvaliation, // immediately invalidates any new or previously queued data interfaces
	};

	// Empties CachedDefaultDataInterfaces and moves the referenced DI to DataInterfacesPendingInvalidation.
	void InvalidateAndEmptyDefaultDataInterfaces(int32 SlackCount, EDataInterfaceInvalidationMode InvalidationMode);
#endif
};

// Forward decl FVersionedNiagaraScriptWeakPtr to suport FVersionedNiagaraScript::ToWeakPtr().
struct FVersionedNiagaraScriptWeakPtr;

/** Struct combining a script with a specific version.*/
struct FVersionedNiagaraScript : public INiagaraParameterDefinitionsSubscriber
{
#if WITH_EDITORONLY_DATA
public:
	FVersionedNiagaraScript() //@todo(ng) refactor to never allow constructing with null script
		: Script(nullptr)
		, Version(FGuid())
	{};

	FVersionedNiagaraScript(UNiagaraScript* InScript)
		: Script(InScript)
		, Version(FGuid())
	{
	};

	FVersionedNiagaraScript(UNiagaraScript* InScript, const FGuid& InVersion)
		: Script(InScript)
		, Version(InVersion)
	{
	};

	//~ Begin INiagaraParameterDefinitionsSubscriber Interface
	virtual const TArray<FParameterDefinitionsSubscription>& GetParameterDefinitionsSubscriptions() const override { return GetScriptData()->ParameterDefinitionsSubscriptions; };
	virtual TArray<FParameterDefinitionsSubscription>& GetParameterDefinitionsSubscriptions() override { return GetScriptData()->ParameterDefinitionsSubscriptions; };

	/** Get all UNiagaraScriptSourceBase of this subscriber. */
	NIAGARA_API virtual TArray<UNiagaraScriptSourceBase*> GetAllSourceScripts() override;

	/** Get the path to the UObject of this subscriber. */
	NIAGARA_API virtual FString GetSourceObjectPathName() const override;
	//~ End INiagaraParameterDefinitionsSubscriber Interface

	NIAGARA_API FVersionedNiagaraScriptWeakPtr ToWeakPtr();
	NIAGARA_API FVersionedNiagaraScriptData* GetScriptData() const;

public:
	TObjectPtr<UNiagaraScript> Script = nullptr;

	FGuid Version;
#endif
};

/** Struct combining a script with a specific version.*/
struct FVersionedNiagaraScriptWeakPtr : public INiagaraParameterDefinitionsSubscriber
{
#if WITH_EDITORONLY_DATA
public:
	FVersionedNiagaraScriptWeakPtr(UNiagaraScript* InScript, const FGuid& InVersion)
		: Script(InScript)
		, Version(InVersion)
	{
	};

	//~ Begin INiagaraParameterDefinitionsSubscriber Interface
	virtual const TArray<FParameterDefinitionsSubscription>& GetParameterDefinitionsSubscriptions() const override { return Pin().GetScriptData()->ParameterDefinitionsSubscriptions; };
	virtual TArray<FParameterDefinitionsSubscription>& GetParameterDefinitionsSubscriptions() override { return Pin().GetScriptData()->ParameterDefinitionsSubscriptions; };

	/** Get all UNiagaraScriptSourceBase of this subscriber. */
	NIAGARA_API virtual TArray<UNiagaraScriptSourceBase*> GetAllSourceScripts() override;

	/** Get the path to the UObject of this subscriber. */
	NIAGARA_API virtual FString GetSourceObjectPathName() const override;
	//~ End INiagaraParameterDefinitionsSubscriber Interface

	NIAGARA_API const FVersionedNiagaraScript Pin() const;
	NIAGARA_API FVersionedNiagaraScript Pin();

public:
	TWeakObjectPtr<UNiagaraScript> Script;

	FGuid Version;
#endif
};
