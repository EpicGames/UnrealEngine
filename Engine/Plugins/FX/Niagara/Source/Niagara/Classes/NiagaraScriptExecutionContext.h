// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraEmitterInstance.h: Niagara emitter simulation class
==============================================================================*/
#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSet.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraEmitter.h"
#include "NiagaraScriptExecutionParameterStore.h"
#include "NiagaraTypes.h"
#include "VectorVM.h"

struct FNiagaraDataInterfaceProxy;
class FNiagaraSystemInstance;
class FNiagaraScriptRuntimeData;

/** All scripts that will use the system script execution context. */
enum class ENiagaraSystemSimulationScript : uint8
{
	Spawn,
	Update,
	Num,
	//TODO: Maybe add emitter spawn and update here if we split those scripts out.
};

/** Container for data needed to process event data. */
struct FNiagaraEventHandlingInfo
{
	FNiagaraEventHandlingInfo()
		: TotalSpawnCount(0)
		, EventData(nullptr)
		, SourceEmitterName(NAME_None)
	{
	}

	~FNiagaraEventHandlingInfo()
	{
		SetEventData(nullptr);
	}

	void SetEventData(FNiagaraDataBuffer* InEventData)
	{
		EventData = InEventData;
	}

	TArray<int32, TInlineAllocator<16>> SpawnCounts;
	int32 TotalSpawnCount;
	FNiagaraDataBufferRef EventData;
	FGuid SourceEmitterGuid;
	FName SourceEmitterName;
};


struct FNiagaraDataSetExecutionInfo
{
	FNiagaraDataSetExecutionInfo()
		: DataSet(nullptr)
		, Input(nullptr)
		, StartInstance(0)
		, bUpdateInstanceCount(false)
	{
		Reset();
	}


	inline void Init(FNiagaraDataSet* InDataSet, FNiagaraDataBuffer* InInput, int32 InStartInstance, bool bInUpdateInstanceCount)
	{
		DataSet = InDataSet;
		Input = InInput;
		StartInstance = InStartInstance;
		bUpdateInstanceCount = bInUpdateInstanceCount;

		check(DataSet);
		check(Input == nullptr || DataSet == Input->GetOwner());
	}
	
	~FNiagaraDataSetExecutionInfo()
	{
		Reset();
	}

	inline void Reset()
	{
		DataSet = nullptr;
		Input = nullptr;
		StartInstance = INDEX_NONE;
		bUpdateInstanceCount = false;
	}

	FNiagaraDataBuffer* GetOutput()const { return DataSet->GetDestinationData(); }

	FNiagaraDataSet* DataSet;
	FNiagaraDataBufferRef Input;
	int32 StartInstance;
	bool bUpdateInstanceCount;
};

struct FScriptExecutionConstantBufferTable
{
	static constexpr uint32 MaxBufferCount = 12;

	TArray<const uint8*, TInlineAllocator<MaxBufferCount>> Buffers;
	TArray<int32, TInlineAllocator<MaxBufferCount>> BufferSizes;

	void Reset(int32 ResetSize)
	{
		Buffers.Reset(ResetSize);
		BufferSizes.Reset(ResetSize);
	}

	template<typename T>
	void AddTypedBuffer(const T& Buffer)
	{
		Buffers.Add(reinterpret_cast<const uint8*>(&Buffer));
		BufferSizes.Add(sizeof(T));
	}

	void AddRawBuffer(const uint8* BufferData, int32 BufferSize)
	{
		Buffers.Add(BufferData);
		BufferSizes.Add(BufferSize);
	}
};

struct FNiagaraScriptExecutionContextBase
{
	UNiagaraScript* Script;

protected:
	TSharedPtr<const FNiagaraScriptRuntimeData> ScriptRuntimeData;
public:

	VectorVM::Runtime::FVectorVMState* VectorVMState = nullptr;
	
	/** Table of external function delegate handles called from the VM. */
	TArray<const FVMExternalFunction*> FunctionTable;

	/**
	Table of user ptrs to pass to the VM.
	*/
	TArray<void*> UserPtrTable;

	/** Parameter store. Contains all data interfaces and a parameter buffer that can be used directly by the VM or GPU. */
	FNiagaraScriptInstanceParameterStore Parameters;

	TArray<FDataSetMeta, TInlineAllocator<2>> DataSetMetaTable;

	TArray<FNiagaraDataSetExecutionInfo, TInlineAllocator<2>> DataSetInfo;

	/** The script type this context is for. Allows us to access the correct per instance function table on the system instance. */
	ENiagaraSystemSimulationScript ScriptType;

	int32 HasInterpolationParameters : 1;
	int32 bAllowParallel : 1;
	int32 bHasDIsWithPreStageTick : 1;
	int32 bHasDIsWithPostStageTick : 1;
#if STATS
	TArray<VectorVM::FStatScopeData> StatScopeData;
	TMap<TStatIdData const*, float> ExecutionTimings;
	void CreateStatScopeData();
	TMap<TStatIdData const*, float> ReportStats();
#endif

	FNDIStageTickHandler DIStageTickHandler;

	FNiagaraScriptExecutionContextBase();
	virtual ~FNiagaraScriptExecutionContextBase();

	virtual bool Init(FNiagaraSystemInstance* Instance, UNiagaraScript* InScript, TSharedPtr<const FNiagaraScriptRuntimeData> InScriptRuntimeData);
	virtual void InitDITickLists(class FNiagaraSystemInstance* Instance);
	virtual bool Tick(class FNiagaraSystemInstance* Instance, ENiagaraSimTarget SimTarget) = 0;
	virtual bool Execute(class FNiagaraSystemInstance* Instance, float DeltaSeconds, uint32 NumInstances, const FScriptExecutionConstantBufferTable& ConstantBufferTable);

	void BindData(int32 Index, FNiagaraDataSet& DataSet, int32 StartInstance, bool bUpdateInstanceCounts);
	void BindData(int32 Index, FNiagaraDataBuffer* Input, int32 StartInstance, bool bUpdateInstanceCounts);

	const TArray<UNiagaraDataInterface*>& GetDataInterfaces()const { return Parameters.GetDataInterfaces(); }

	TArrayView<const uint8> GetScriptLiterals() const;

	FNDIStageTickHandler& GetNDIStageTickHander(){ return DIStageTickHandler; }

	void DirtyDataInterfaces();
	void PostTick();

	//Unused. These are only useful in the new SystemScript context.
	virtual void BindSystemInstances(TArray<FNiagaraSystemInstance*>& InSystemInstances) {}
	virtual bool GeneratePerInstanceDIFunctionTable(FNiagaraSystemInstance* Inst, TArray<struct FNiagaraPerInstanceDIFuncInfo>& OutFunctions) {return true;}

	const FNiagaraScriptRuntimeData* GetScriptRuntimeData() const { return ScriptRuntimeData.Get(); }

private:
	bool ExecuteInternal(uint32 NumInstances, const FScriptExecutionConstantBufferTable& ConstantBufferTable);
};

struct FNiagaraScriptExecutionContext : public FNiagaraScriptExecutionContextBase
{
protected:
	/**
	Table of external function delegates unique to the instance.
	*/
	TArray<FVMExternalFunction> LocalFunctionTable;

public:
	virtual bool Tick(class FNiagaraSystemInstance* Instance, ENiagaraSimTarget SimTarget)override;
};

/**
For function calls from system scripts on User DIs or those with per instance data, we build a per instance binding table that is called from a helper function in the exec context.
TODO: We can embed the instance data in the lambda capture for reduced complexity here. No need for the user ptr table.
We have to rebind if the instance data is recreated anyway.
*/
struct FNiagaraPerInstanceDIFuncInfo
{
	FVMExternalFunction Function;
	void* InstData;
};

/** Specialized exec context for system scripts. Allows us to better handle the added complication of Data Interfaces across different system instances. */
struct FNiagaraSystemScriptExecutionContext : public FNiagaraScriptExecutionContextBase
{
protected:

	struct FExternalFuncInfo
	{
		FVMExternalFunction Function;
	};

	TArray<FExternalFuncInfo> ExtFunctionInfo;

	/**
	Array of system instances the context is currently operating on.
	We need this to allow us to call into per instance DI functions.
	*/
	TArray<FNiagaraSystemInstance*>* SystemInstances;

public:
	FNiagaraSystemScriptExecutionContext(ENiagaraSystemSimulationScript InScriptType) : SystemInstances(nullptr) { ScriptType = InScriptType; }
	
	virtual bool Init(FNiagaraSystemInstance* Instance, UNiagaraScript* InScript, TSharedPtr<const FNiagaraScriptRuntimeData> InScriptRuntimeData) override;
	virtual bool Tick(FNiagaraSystemInstance* Instance, ENiagaraSimTarget SimTarget)override;
	virtual bool Execute(FNiagaraSystemInstance* Instance, float DeltaSeconds, uint32 NumInstances, const FScriptExecutionConstantBufferTable& ConstantBufferTable)override;

	void BindSystemInstances(TArray<FNiagaraSystemInstance*>& InSystemInstances) { SystemInstances = &InSystemInstances; }

	/** Generates a table of DI calls unique to the passed system instance. These are then accesss inside the PerInstanceFunctionHook. */
	virtual bool GeneratePerInstanceDIFunctionTable(FNiagaraSystemInstance* Inst, TArray<FNiagaraPerInstanceDIFuncInfo>& OutFunctions);
};
