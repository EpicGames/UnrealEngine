// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraScriptHighlight.generated.h"

USTRUCT()
struct FNiagaraScriptHighlight
{
	GENERATED_BODY()

	NIAGARA_API FNiagaraScriptHighlight();

	NIAGARA_API bool IsValid() const;

	UPROPERTY(EditAnywhere, Category = Highlight)
	FLinearColor Color;

	UPROPERTY(EditAnywhere, Category = Highlight)
	FText DisplayName;

	NIAGARA_API bool operator==(const FNiagaraScriptHighlight& Other) const;

#if WITH_EDITORONLY_DATA
	static NIAGARA_API void ArrayToJson(const TArray<FNiagaraScriptHighlight>& InHighlights, FString& OutJson);
	static NIAGARA_API void JsonToArray(const FString& InJson, TArray<FNiagaraScriptHighlight>& OutHighlights);
#endif
};
