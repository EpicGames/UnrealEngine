// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "NiagaraCommon.h"
#include "NiagaraTypes.h"
#include "NiagaraCompileHash.h"
#include "Misc/Guid.h"
#include "NiagaraScriptSourceBase.generated.h"

class INiagaraParameterDefinitionsSubscriber;
class UNiagaraParameterDefinitionsBase;
struct FNiagaraParameterStore;
struct FNiagaraScriptHashCollector;
class UNiagaraScriptSourceBase;
struct FNiagaraVMExecutableDataId;
struct FVersionedNiagaraScriptData;

struct EditorExposedVectorConstant
{
	FName ConstName;
	FVector4 Value;
};

struct EditorExposedVectorCurveConstant
{
	FName ConstName;
	class UCurveVector *Value;
};

struct FNiagaraScriptSourceAnalytics
{
	int32 ActiveModules = 0;
	int32 DisabledModules = 0;
	TSet<FString> UsedNiagaraModules;
};

/** External reference to the compile request data generated.*/
class FNiagaraCompileRequestDataBase
{
public:
	virtual ~FNiagaraCompileRequestDataBase() = default;
	virtual void GatherPreCompiledVariables(const FString& InNamespaceFilter, TArray<FNiagaraVariable>& OutVars) const = 0;
	virtual int32 GetDependentRequestCount() const = 0;
	virtual TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> GetDependentRequest(int32 Index) = 0;
	virtual const FNiagaraCompileRequestDataBase* GetDependentRequest(int32 Index) const = 0;
	virtual FName ResolveEmitterAlias(FName VariableName) const = 0;
	virtual bool GetUseRapidIterationParams() const = 0;
	virtual bool GetDisableDebugSwitches() const = 0;
};

/** External reference to the compile request data generated.*/
class FNiagaraCompileRequestDuplicateDataBase
{
public:
	virtual ~FNiagaraCompileRequestDuplicateDataBase() = default;
	virtual bool IsDuplicateDataFor(UNiagaraSystem* InSystem, UNiagaraEmitter* InEmitter, UNiagaraScript* InScript) const = 0;
	virtual void GetDuplicatedObjects(TArray<UObject*>& Objects) = 0;
	virtual const TMap<FName, UNiagaraDataInterface*>& GetObjectNameMap() = 0;
	virtual const UNiagaraScriptSourceBase* GetScriptSource() const =  0;
	virtual int32 GetDependentRequestCount() const = 0;
	virtual TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> GetDependentRequest(int32 Index) = 0;
	virtual void ReleaseCompilationCopies() = 0;
};

class FNiagaraCompileOptions
{
public:
	NIAGARA_API static const FString CpuScriptDefine;
	NIAGARA_API static const FString GpuScriptDefine;
	NIAGARA_API static const FString EventSpawnDefine;
	NIAGARA_API static const FString EventSpawnInitialAttribWritesDefine;
	NIAGARA_API static const FString AccurateQuatInterpolation;

	FNiagaraCompileOptions() : TargetUsage(ENiagaraScriptUsage::Function), TargetUsageBitmask(0)
	{
	}

	FNiagaraCompileOptions(ENiagaraScriptUsage InTargetUsage, FGuid InTargetUsageId, int32 InTargetUsageBitmask,  const FString& InPathName, const FString& InFullName, const FString& InName)
		: TargetUsage(InTargetUsage), TargetUsageId(InTargetUsageId), PathName(InPathName), FullName(InFullName), Name(InName), TargetUsageBitmask(InTargetUsageBitmask)
	{
	}

	const FString& GetFullName() const { return FullName; }
	const FString& GetName() const { return Name; }
	const FString& GetPathName() const { return PathName; }
	int32 GetTargetUsageBitmask() const { return TargetUsageBitmask; }

	void SetCpuScript() { AdditionalDefines.AddUnique(CpuScriptDefine); }
	void SetGpuScript() { AdditionalDefines.AddUnique(GpuScriptDefine); }

	bool IsCpuScript() const { return AdditionalDefines.Contains(CpuScriptDefine); }
	bool IsGpuScript() const { return AdditionalDefines.Contains(GpuScriptDefine); }

	ENiagaraScriptUsage TargetUsage;
	FGuid TargetUsageId;
	FString PathName;
	FString FullName;
	FString Name;
	int32 TargetUsageBitmask;
	TArray<FString> AdditionalDefines;
	TArray<FNiagaraVariableBase> AdditionalVariables;
};

/** Runtime data for a Niagara system */
UCLASS(MinimalAPI)
class UNiagaraScriptSourceBase : public UObject
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnChanged);
#endif

	TArray<TSharedPtr<EditorExposedVectorConstant> > ExposedVectorConstants;
	TArray<TSharedPtr<EditorExposedVectorCurveConstant> > ExposedVectorCurveConstants;

	virtual bool IsEditorOnly()const override{ return true; }

	/** Determines if the input change id is equal to the current source graph's change id.*/
	virtual bool IsSynchronized(const FGuid& InChangeId) { return true; }

	/** Enforce that the source graph is now out of sync with the script.*/
	virtual void MarkNotSynchronized(FString Reason) {}

	virtual FGuid GetChangeID() { return FGuid(); }

	virtual void RegisterVMCompilationIdDependencies(FNiagaraScriptHashCollector& Collector, ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const {}
	virtual void ComputeVMCompilationId(FNiagaraVMExecutableDataId& Id, FNiagaraScriptHashCollector& HashCollector, ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const {};

#if WITH_EDITORONLY_DATA
	// Will conditionally refresh the graph's CompileId
	virtual void RefreshGraphCompileId() {};
	virtual void ReportAnalyticsData(FNiagaraScriptSourceAnalytics& InData) const {};
#endif

	virtual TMap<FName, UNiagaraDataInterface*> ComputeObjectNameMap(UNiagaraSystem& System, ENiagaraScriptUsage Usage, FGuid UsageId, FString EmitterUniqueName) const { return TMap<FName, UNiagaraDataInterface*>(); }

	virtual FGuid GetCompileBaseId(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const { return FGuid(); }

	virtual FNiagaraCompileHash GetCompileHash(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const { return FNiagaraCompileHash(); }
	
	/** Cause the source to build up any internal variables that will be useful in the compilation process.*/
	virtual TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> PreCompile(UNiagaraEmitter* Emitter, const TArray<FNiagaraVariable>& EncounterableVariables, TArray<TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe>>& ReferencedCompileRequests, bool bClearErrors = true) { return nullptr; }

	/** 
	 * Allows the derived editor only script source to handle a post load requested by an owning emitter. 
	 * @param OwningEmitter The emitter requesting the post load.
	 */
	virtual void PostLoadFromEmitter(FVersionedNiagaraEmitter OwningEmitter) { }

	/** Adds a module if it isn't already in the graph. If the module isn't found bOutFoundModule will be false. If it is found and it did need to be added, the function returns true. If it already exists, it returns false. */
	virtual bool AddModuleIfMissing(FString ModulePath, ENiagaraScriptUsage Usage, bool& bOutFoundModule) { bOutFoundModule = false; return false; }

	virtual void MigrateParameterDataToHierarchyRoot(FVersionedNiagaraScriptData& OwnerData) {}
#if WITH_EDITOR
	virtual void CleanUpOldAndInitializeNewRapidIterationParameters(const FVersionedNiagaraEmitter& Emitter, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FNiagaraParameterStore& RapidIterationParameters) const { checkf(false, TEXT("Not implemented")); }

	FOnChanged& OnChanged() { return OnChangedDelegate; }

	virtual void ForceGraphToRecompileOnNextCheck() {}

	virtual void RefreshFromExternalChanges() {}

	virtual void CollectDataInterfaces(TArray<const UNiagaraDataInterfaceBase*>& DataInterfaces) const {};

	/** Synchronize all source script variables that have been changed or removed from the parameter definitions to all eligible destination script variables owned by the graph.
	 *
	 *  @param TargetDefinitions			The set of parameter definitions that will be synchronized with the graph parameters.
	 *	@param AllDefinitions				All parameter definitions in the project. Used to add new subscriptions to definitions if specified in Args.
	 *  @param AllDefinitionsParameterIds	All unique Ids of all parameter definitions.
	 *	@param Subscriber					The INiagaraParameterDefinitionsSubscriber that owns the graph. Used to add new subscriptions to definitions if specified in Args.
	 *	@param Args							Additional arguments that specify how to perform the synchronization.
	 */
	virtual void SynchronizeGraphParametersWithParameterDefinitions(
		const TArray<UNiagaraParameterDefinitionsBase*> TargetDefinitions,
		const TArray<UNiagaraParameterDefinitionsBase*> AllDefinitions,
		const TSet<FGuid>& AllDefinitionsParameterIds,
		INiagaraParameterDefinitionsSubscriber* Subscriber,
		FSynchronizeWithParameterDefinitionsArgs Args
	) {};

	/** Rename all graph assignment and map set node pins.
	 *  Used when synchronizing definitions with source scripts of systems and emitters.
	 */
	virtual void RenameGraphAssignmentAndSetNodePins(const FName OldName, const FName NewName) {};

	/** Checks if any of the provided variables are linked to function inputs of position type data
	 */
	virtual void GetLinkedPositionTypeInputs(const TArray<FNiagaraVariable>& ParametersToCheck, TSet<FNiagaraVariable>& OutLinkedParameters) {}
	virtual void ChangedLinkedInputTypes(const FNiagaraVariable& ParametersToCheck, const FNiagaraTypeDefinition& NewType) {}
	virtual void ReplaceScriptReferences(UNiagaraScript* OldScript, UNiagaraScript* NewScript) {}

protected:
	FOnChanged OnChangedDelegate;
#endif
};
