// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraScript.h"

#include "Serialization/BulkData.h"
#include "Serialization/BulkDataBuffer.h"

#include "NiagaraSimCache.generated.h"

class UNiagaraComponent;
class UNiagaraSimCacheDebugData;
struct FNiagaraSimCacheDataBuffersLayout;

UENUM(BlueprintType)
enum class ENiagaraSimCacheAttributeCaptureMode : uint8
{
	/**
	Captures all attributes available.
	This kind of cache will be useful for restarting a simulation from or debugging.
	*/
	All UMETA(DisplayName = "Capture All Attributes"),

	/**
	Captures attributes that are required to render the system only.
	This kind of cache is useful for rendering only and should have a much smaller
	size than capturing all attributes.
	*/
	RenderingOnly UMETA(DisplayName = "Capture Attributes Needed For Rendering"),

	/**
	Captures only attributes that match the 'ExplicitCaptureAttributes' list provided by the user.
	This kind of cache is useful to keep the size down if you need to debug a very
	specific attribute, or you want to do some additional process on the attributes
	i.e. capture MyEmitter.Particles.Position and place static meshes in those locations.
	*/
	ExplicitAttributes UMETA(DisplayName = "Capture Explicit Attributes Only"),
};

USTRUCT(BlueprintType)
struct FNiagaraSimCacheCreateParameters
{
	GENERATED_BODY()

	/**
	How do we want to capture attributes for the simulation cache.
	The mode selected depends on what situations the cache can be used in.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache")
	ENiagaraSimCacheAttributeCaptureMode AttributeCaptureMode = ENiagaraSimCacheAttributeCaptureMode::All;

	/**
	When enabled allows the SimCache to be re-based.
	i.e. World space emitters can be moved to the new component's location
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SimCache")
	uint32 bAllowRebasing : 1 = true;

	/**
	When enabled Data Interface data will be stored in the SimCache.
	This can result in a large increase to the cache size, depending on what Data Interfaces are used
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SimCache")
	uint32 bAllowDataInterfaceCaching : 1 = true;

	/**
	When enabled we allow the cache to be generated for interpolation.
	This will increase the memory usage for the cache slightly but can allow you to reduce the capture rate.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SimCache")
	uint32 bAllowInterpolation : 1 = false;

	/**
	When enabled all support types for interpolation will be included, this set may increase in future releases.
	When disabled we only interpolate Position / Quat attributes.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache", meta = (EditCondition = "bAllowInterpolation"))
	uint32 bInterpolateAllTypes : 1 = true;
	
	/**
	When enabled we allow the cache to be generated for extrapolation.
	This will force the velocity attribute to be maintained.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="SimCache")
	uint32 bAllowVelocityExtrapolation : 1 = false;

	/**
	When enabled the cache will support serializing large amounts of cache data.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache")
	uint32 bAllowSerializeLargeCache : 1 = true;

	/**
	When enabled additional information is stored that can be useful for debugging a simulation
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache")
	uint32 bIncludeDebugData : 1 = false;

	/**
	List of Attributes to force include in the SimCache rebase, they should be the full path to the attribute
	For example, MyEmitter.Particles.MyQuat would force the particle attribute MyQuat to be included for MyEmitter
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache")
	TArray<FName> RebaseIncludeAttributes;

	/**
	List of Attributes to force exclude from the SimCache rebase, they should be the full path to the attribute
	For example, MyEmitter.Particles.MyQuat would force the particle attribute MyQuat to be included for MyEmitter
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache")
	TArray<FName> RebaseExcludeAttributes;

	/**
	List of specific Attributes to include when using interpolation.  They must be types that are supported for interpolation.
	For example, MyEmitter.Particles.MyPosition would force MyPosition to be interpolated.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache")
	TArray<FName> InterpolationIncludeAttributes;

	/**
	List of specific Attributes to exclude interpolation for.  They must be types that are supported for interpolation.
	For example, MyEmitter.Particles.MyPosition would force MyPosition to be interpolated.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache")
	TArray<FName> InterpolationExcludeAttributes;

	/**
	List of attributes to capture when the capture attribute capture mode is set to explicit.
	For example, adding MyEmitter.Particles.Position will only gather that attribute inside the cache.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache", meta=(EditCondition="AttributeCaptureMode == ENiagaraSimCacheAttributeCaptureMode::ExplicitAttributes"))
	TArray<FName> ExplicitCaptureAttributes;

	// Helper function to setup the parameters for capturing a cache for debugging
	NIAGARA_API static FNiagaraSimCacheCreateParameters CreateForDebugging();
};

USTRUCT()
struct FNiagaraSimCacheDataBuffers
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 NumInstances = 0;

	UPROPERTY()
	uint32 IDAcquireTag = 0;

	UPROPERTY()
	uint32 IDToIndexTableElements = 0;

	TArrayView<uint8> FloatData;
	TArrayView<uint8> HalfData;
	TArrayView<uint8> Int32Data;
	TArrayView<int32> IDToIndexTable;
	TArrayView<uint32> InterpMapping;

	FBulkDataBuffer<uint8> DataBuffer;
	FByteBulkData BulkData;

	void SetupForWrite(const FNiagaraSimCacheDataBuffersLayout& CacheLayout);
	bool SetupForRead(const FNiagaraSimCacheDataBuffersLayout& CacheLayout);

	bool SerializeAsArray(FArchive& Ar, UObject* OwnerObject, const FNiagaraSimCacheDataBuffersLayout& CacheLayout);
	bool SerializeAsBulkData(FArchive& Ar, UObject* OwnerObject, const FNiagaraSimCacheDataBuffersLayout& CacheLayout);

	bool operator==(const FNiagaraSimCacheDataBuffers& Other) const;
	bool operator!=(const FNiagaraSimCacheDataBuffers& Other) const;
};

USTRUCT()
struct FNiagaraSimCacheEmitterFrame
{
	GENERATED_BODY()

	//-TODO: We may not require these
	UPROPERTY()
	FBox LocalBounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	int32 TotalSpawnedParticles = 0;

	UPROPERTY()
	FNiagaraSimCacheDataBuffers ParticleDataBuffers;
};

USTRUCT()
struct FNiagaraSimCacheSystemFrame
{
	GENERATED_BODY()
		
	UPROPERTY()
	FBox LocalBounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	FNiagaraSimCacheDataBuffers SystemDataBuffers;
};

USTRUCT()
struct FNiagaraSimCacheFrame
{
	GENERATED_BODY()

	UPROPERTY()
	FTransform LocalToWorld = FTransform::Identity;

	UPROPERTY()
	FVector3f LWCTile = FVector3f::ZeroVector;

	UPROPERTY()
	float SimulationAge = 0.0f;

	UPROPERTY()
	int32 SimulationTickCount = 0;

	UPROPERTY()
	FNiagaraSimCacheSystemFrame SystemData;

	UPROPERTY()
	TArray<FNiagaraSimCacheEmitterFrame> EmitterData;
};

USTRUCT()
struct FNiagaraSimCacheVariable
{
	GENERATED_BODY()

	UPROPERTY()
	FNiagaraVariableBase Variable;

	UPROPERTY()
	uint16 FloatOffset = INDEX_NONE;

	UPROPERTY()
	uint16 FloatCount = 0;

	UPROPERTY()
	uint16 HalfOffset = INDEX_NONE;

	UPROPERTY()
	uint16 HalfCount = 0;

	UPROPERTY()
	uint16 Int32Offset = INDEX_NONE;

	UPROPERTY()
	uint16 Int32Count = 0;

	NIAGARA_API bool operator==(const FNiagaraSimCacheVariable& Other) const;
};

USTRUCT()
struct FNiagaraSimCacheDataBuffersLayout
{
	GENERATED_BODY()

	struct FVariableCopyContext
	{
		float					FrameFraction		= 0.0f;
		float					FrameDeltaSeconds	= 0.0f;
		float					SimDeltaSeconds		= 0.0f;
		float					PrevFrameFraction	= 0.0f;
		uint32					NumInstances		= 0;
		uint8*					DestCurr			= nullptr;
		uint8*					DestPrev			= nullptr;
		uint32					DestStride			= 0;
		const uint8*			SourceACurr			= nullptr;
		const uint8*			SourceAPrev			= nullptr;
		uint32					SourceAStride		= 0;
		const uint8*			SourceBCurr			= nullptr;
		uint32					SourceBStride		= 0;
		const uint8*			Velocity			= nullptr;
		FTransform				RebaseATransform;
		FTransform				RebaseBTransform;
		TConstArrayView<uint32>	InterpMappings;
	};

	typedef void (*FVariableCopyFunction)(const FVariableCopyContext& CopyData);

	struct FVariableCopyMapping
	{
		FVariableCopyMapping() = default;
		explicit FVariableCopyMapping(uint16 InComponentFrom, uint16 InComponentTo, FVariableCopyFunction InCopyFunc)
			: CurrComponentFrom(InComponentFrom)
			, PrevComponentFrom(InComponentFrom)
			, CurrComponentTo(InComponentTo)
			, PrevComponentTo(InComponentTo)
			, CopyFunc(InCopyFunc)
		{
		}
		explicit FVariableCopyMapping(uint16 InCurrComponentFrom, uint16 InPrevComponentFrom, uint16 InCurrComponentTo, uint16 InPrevComponentTo, FVariableCopyFunction InCopyFunc)
			: CurrComponentFrom(InCurrComponentFrom)
			, PrevComponentFrom(InPrevComponentFrom)
			, CurrComponentTo(InCurrComponentTo)
			, PrevComponentTo(InPrevComponentTo)
			, CopyFunc(InCopyFunc)
		{
		}

		uint16					CurrComponentFrom = 0;
		uint16					PrevComponentFrom = 0;
		uint16					CurrComponentTo = 0;
		uint16					PrevComponentTo = 0;
		FVariableCopyFunction	CopyFunc;
	};

	UPROPERTY()
	FName LayoutName;

	UPROPERTY()
	ENiagaraSimTarget SimTarget = ENiagaraSimTarget::CPUSim;

	UPROPERTY()
	TArray<FNiagaraSimCacheVariable> Variables;

	UPROPERTY()
	uint16 FloatCount = 0;

	UPROPERTY()
	uint16 HalfCount = 0;

	UPROPERTY()
	uint16 Int32Count = 0;

	UPROPERTY()
	bool bLocalSpace = false;

	UPROPERTY()
	bool bAllowInterpolation = false;

	UPROPERTY()
	bool bAllowVelocityExtrapolation = false;

	UPROPERTY()
	TArray<FName> RebaseVariableNames;

	UPROPERTY()
	TArray<FName> InterpVariableNames;

	UPROPERTY()
	uint16 ComponentVelocity = INDEX_NONE;

	struct FCacheBufferWriteInfo
	{
		uint16			ComponentUniqueID = INDEX_NONE;		// Used during building for interpolation to track particles
		TArray<uint16>	ComponentMappingsFromDataBuffer;	// Used to map individual components from a Niagara Data Buffer -> Cache Buffer
	};
	FCacheBufferWriteInfo CacheBufferWriteInfo;

	struct FCacheBufferReadInfo
	{
		TArray<uint16>					ComponentMappingsToDataBuffer;		// Used to map individual components from Cache Buffer -> Niagara Data Buffer
		TArray<FVariableCopyMapping>	VariableCopyMappingsToDataBuffer;	// Used for more complex operations when we need to rebase / interpolate / etc
	};
	FCacheBufferReadInfo CacheBufferReadInfo;
	mutable bool bNeedsCacheBufferReadInfoUpdateForRT = false;
	mutable FCacheBufferReadInfo CacheBufferReadInfo_RT;

	bool IsLayoutValid() const { return !LayoutName.IsNone(); }
	int32 IndexOfCacheVariable(const FNiagaraVariableBase& InVariable) const;
	const FNiagaraSimCacheVariable* FindCacheVariable(const FNiagaraVariableBase& InVariable) const;
};

USTRUCT()
struct FNiagaraSimCacheLayout
{
	GENERATED_BODY()

	UPROPERTY()
	FNiagaraSimCacheDataBuffersLayout SystemLayout;

	UPROPERTY()
	TArray<FNiagaraSimCacheDataBuffersLayout> EmitterLayouts;
};

struct FNiagaraSimCacheFeedbackContext
{
	FNiagaraSimCacheFeedbackContext() = default;
	explicit FNiagaraSimCacheFeedbackContext(bool bInAutoLogIssues) : bAutoLogIssues(bInAutoLogIssues) {}
	NIAGARA_API ~FNiagaraSimCacheFeedbackContext();

	bool bAutoLogIssues = true;
	TArray<FString> Errors;
	TArray<FString> Warnings;
};

/**
 * Recording of multiple frames of simulation data from a running Niagara system.
 * Can be used to play back the captured recording or to inspect the captured data for debug purposes.
 * Depending on the capture settings, not all attributes from the simulation are present in the cache.
 *
 * To capture a cache, either
 * (1) use the baker tool in the system editor,
 * (2) use the Niagara component cache track in sequencer or
 * (3) manually capture a running system with the "CaptureNiagaraSimCache" Blueprint functions
 */
UCLASS(BlueprintType, MinimalAPI)
class UNiagaraSimCache : public UObject
{
	friend struct FNiagaraSimCacheAttributeReaderHelper;
	friend struct FNiagaraSimCacheExport;
	friend struct FNiagaraSimCacheCompare;
	friend struct FNiagaraSimCacheHelper;
	friend struct FNiagaraSimCacheGpuResource;

	GENERATED_UCLASS_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCacheBeginWrite, UNiagaraSimCache*)
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCacheEndWrite, UNiagaraSimCache*)

	// Used to reduce rounding issues with age
	static constexpr float CacheAgeResolution = 10000.0f;

	// UObject Interface
	NIAGARA_API virtual void Serialize(FArchive& Ar) override;
	NIAGARA_API virtual bool IsReadyForFinishDestroy() override;
	// UObject Interface

	/** Get the caches assigned GUID, this can be set from a user or will be auto created on first write of the cache. */
	const FGuid& GetCacheGuid() const { return CacheGuid; }
	/** Set the caches GUID to use. */
	void SetCacheGuid(const FGuid& InGuid) { CacheGuid = InGuid; }

	/** Get the system asset this cache was created from. */
	TSoftObjectPtr<UNiagaraSystem> GetSystemAsset() const { return SoftNiagaraSystem; }

	/** A valid cache is one that contains at least 1 frames worth of data. */
	UFUNCTION(BlueprintCallable, Category=NiagaraSimCache, meta=(DisplayName="IsValid"))
	bool IsCacheValid() const { return SoftNiagaraSystem.IsNull() == false; }

	/** An empty cache contains no frame data and can not be used */
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	bool IsEmpty() const { return !IsCacheValid(); }

	/**	How were the attributes captured for this sim cache. */
	UFUNCTION(BlueprintCallable, Category=NiagaraSimCache)
	ENiagaraSimCacheAttributeCaptureMode GetAttributeCaptureMode() const { return CreateParameters.AttributeCaptureMode; }

	/** Returns true if we have any rebasing enabled. */
	bool IsRebaseEnabled() const { return CreateParameters.bAllowRebasing; }

	NIAGARA_API bool BeginWrite(FNiagaraSimCacheCreateParameters InCreateParameters, UNiagaraComponent* NiagaraComponent);
	NIAGARA_API bool BeginWrite(FNiagaraSimCacheCreateParameters InCreateParameters, UNiagaraComponent* NiagaraComponent, FNiagaraSimCacheFeedbackContext& FeedbackContext);

	NIAGARA_API bool BeginAppend(FNiagaraSimCacheCreateParameters InCreateParameters, UNiagaraComponent* NiagaraComponent);
	NIAGARA_API bool BeginAppend(FNiagaraSimCacheCreateParameters InCreateParameters, UNiagaraComponent* NiagaraComponent, FNiagaraSimCacheFeedbackContext& FeedbackContext);

	NIAGARA_API bool WriteFrame(UNiagaraComponent* NiagaraComponent);
	NIAGARA_API bool WriteFrame(UNiagaraComponent* NiagaraComponent, FNiagaraSimCacheFeedbackContext& FeedbackContext);

	NIAGARA_API bool EndWrite(bool bAllowAnalytics = false);

	NIAGARA_API bool CanRead(UNiagaraSystem* NiagaraSystem);
	NIAGARA_API bool Read(float TimeSeconds, FNiagaraSystemInstance* SystemInstance) const;
	NIAGARA_API bool ReadFrame(int32 FrameIndex, float FrameFraction, FNiagaraSystemInstance* SystemInstance) const;

	bool CanRunAsync() const { return bCanRunAsync; }

	/** Get the time the simulation was at when recorded. */
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	float GetStartSeconds() const { return StartSeconds; }

	/** Get the duration the cache was recorded over. */
	float GetDurationSeconds() const { return DurationSeconds; }

	/** Get number of frames stored in the cache. */
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	int GetNumFrames() const { return CacheFrames.Num(); }

	/** Get number of emitters stored inside the cache. */
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	int GetNumEmitters() const { return CacheLayout.EmitterLayouts.Num(); }

	/** Get the emitter index from a name */
	NIAGARA_API int GetEmitterIndex(FName EmitterName) const;

	/** Get the system this cache is based on */
	NIAGARA_API UNiagaraSystem* GetSystem (bool bLoadSynchronous = false);
	
	/** Get the emitter name at the provided index. */
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	FName GetEmitterName(int32 EmitterIndex) const { return CacheLayout.EmitterLayouts.IsValidIndex(EmitterIndex) ? CacheLayout.EmitterLayouts[EmitterIndex].LayoutName : NAME_None; }

	/** Returns a list of emitters we have captured in the SimCache. */
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	NIAGARA_API TArray<FName> GetEmitterNames() const;

	/** Returns true if we have stored data interfaces, otherwise false. */
	bool HasStoredDataInterfaces() const { return !DataInterfaceStorage.IsEmpty(); }

	/** Returns a list of data interfaces we have captured in the SimCache. */
	NIAGARA_API TArray<FNiagaraVariableBase> GetStoredDataInterfaces() const;
	
	/** Returns the actual data we have captured in the SimCache for the given data interface. */
	NIAGARA_API const UObject* GetDataInterfaceStorageObject(const FNiagaraVariableBase& DataInterface) const;

	/** Returns the debug data stored inside the cache, intended for internal use only. */
	const UNiagaraSimCacheDebugData* GetDebugData() const { return DebugData; }

#if WITH_EDITORONLY_DATA
	/** Remove the debug data from the cache */
	NIAGARA_API void RemoveDebugData();
#endif

	/**
	Get number of active instances for the emitter at the given frame.
	An EmitterIndex or INDEX_NONE will return information about the system instance.
	*/
	NIAGARA_API int GetEmitterNumInstances(int32 EmitterIndex, int32 FrameIndex) const;

	/**
	Runs a function for each attribute for the provided emitter index.
	An EmitterIndex or INDEX_NONE will return information about the system instance.
	Return true to continue iterating or false to stop
	*/
	NIAGARA_API void ForEachEmitterAttribute(int32 EmitterIndex, TFunction<bool(const FNiagaraSimCacheVariable&)> Function) const;

	/**
	Reads Niagara attributes by name from the cache frame and appends them into the relevant arrays.
	When reading using this method the attributes for a FVector would come out as AoS rather than SoA so each component is wrote one by one
	EmitterName - If left blank will return the system simulation attributes.
	*/
	NIAGARA_API void ReadAttribute(TArray<float>& OutFloats, TArray<FFloat16>& OutHalfs, TArray<int32>& OutInts, FName AttributeName, FName EmitterName, int FrameIndex = 0) const;

	/**
	Reads Niagara int attributes by name from the cache frame and appends them into the OutValues array.
	EmitterName - If left blank will return the system simulation attributes.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	NIAGARA_API void ReadIntAttribute(TArray<int32>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex = 0) const;

	/**
	Reads Niagara float attributes by name from the cache frame and appends them into the OutValues array.
	EmitterName - If left blank will return the system simulation attributes.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	NIAGARA_API void ReadFloatAttribute(TArray<float>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex = 0) const;

	/**
	Reads Niagara Vec2 attributes by name from the cache frame and appends them into the OutValues array.
	EmitterName - If left blank will return the system simulation attributes.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	NIAGARA_API void ReadVector2Attribute(TArray<FVector2D>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex = 0) const;

	/**
	Reads Niagara Vec3 attributes by name from the cache frame and appends them into the OutValues array.
	EmitterName - If left blank will return the system simulation attributes.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	NIAGARA_API void ReadVectorAttribute(TArray<FVector>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex = 0) const;

	/**
	Reads Niagara Vec4 attributes by name from the cache frame and appends them into the OutValues array.
	EmitterName - If left blank will return the system simulation attributes.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	NIAGARA_API void ReadVector4Attribute(TArray<FVector4>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex = 0) const;

	/**
	Reads Niagara Color attributes by name from the cache frame and appends them into the OutValues array.
	EmitterName - If left blank will return the system simulation attributes.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	NIAGARA_API void ReadColorAttribute(TArray<FLinearColor>& OutValues, FName AttributeName = FName("Color"), FName EmitterName = NAME_None, int FrameIndex = 0) const;

	/**
	Reads Niagara ID attributes by name from the cache frame and appends them into the OutValues array.
	EmitterName - If left blank will return the system simulation attributes.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	NIAGARA_API void ReadIDAttribute(TArray<FNiagaraID>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex = 0) const;

	/**
	Reads Niagara Position attributes by name from the cache frame and appends them into the OutValues array.
	Local space emitters provide data at local locations unless bLocalSpaceToWorld is true.
	EmitterName - If left blank will return the system simulation attributes.
	LocalSpaceToWorld - Caches are always stored in the emitters space, i.e. local or world space.  You can set this to false if you want the local position rather than the world position.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache, meta=(AdvancedDisplay="bLocalSpaceToWorld"))
	NIAGARA_API void ReadPositionAttribute(TArray<FVector>& OutValues, FName AttributeName = FName("Position"), FName EmitterName = NAME_None, bool bLocalSpaceToWorld = true, int FrameIndex = 0) const;

	/**
	Reads Niagara Position attributes by name from the cache frame and appends them into the OutValues array.
	All attributes read with this method will be re-based from the captured space into the provided transform space,
	this will occur even if the cache was not captured with re-basing enabled.
	EmitterName - If left blank will return the system simulation attributes.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	NIAGARA_API void ReadPositionAttributeWithRebase(TArray<FVector>& OutValues, FTransform Transform, FName AttributeName = FName("Position"), FName EmitterName = NAME_None, int FrameIndex = 0) const;

	/**
	Reads Niagara Quaternion attributes by name from the cache frame and appends them into the OutValues array.
	Local space emitters provide data at local rotation unless bLocalSpaceToWorld is true.
	EmitterName - If left blank will return the system simulation attributes.
	LocalSpaceToWorld - Caches are always stored in the emitters space, i.e. local or world space.  You can set this to false if you want the local Quat rather than the world Quat.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache, meta=(AdvancedDisplay="bLocalSpaceToWorld"))
	NIAGARA_API void ReadQuatAttribute(TArray<FQuat>& OutValues, FName AttributeName = FName("MeshOrientation"), FName EmitterName = NAME_None, bool bLocalSpaceToWorld = true, int FrameIndex = 0) const;

	/**
	Reads Niagara Quaternion attributes by name from the cache frame and appends them into the OutValues array.
	Only attributes that in the rebase list will be transform into the provided Quat space.  Therefore the cache
	must be captured with rebasing enabled to have any impact.
	EmitterName - If left blank will return the system simulation attributes.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	NIAGARA_API void ReadQuatAttributeWithRebase(TArray<FQuat>& OutValues, FQuat Quat, FName AttributeName = FName("MeshOrientation"), FName EmitterName = NAME_None, int FrameIndex = 0) const;

	/**
	Reads data interface data from the cache as the requested type.
	This method will return nullptr if the attribute does not exists or the requests type is not supported by the storage type.
	*/
	UFUNCTION(BlueprintCallable, Category = NiagaraSimCache)
	NIAGARA_API UObject* ReadDataInterfaceAs(UClass* RequestedType, FName AttributeName, int FrameIndex = 0) const;

	template<typename T>
	T* ReadDataInterfaceAs(FName AttributeName, int FrameIndex = 0) const { return (T*)ReadDataInterfaceAs(T::StaticClass(), AttributeName, FrameIndex); }

private:
	UPROPERTY(VisibleAnywhere, Category=SimCache)
	FGuid CacheGuid;

	UPROPERTY(VisibleAnywhere, Category=SimCache, meta=(DisplayName="Niagara System"))
	TSoftObjectPtr<UNiagaraSystem> SoftNiagaraSystem;

	UPROPERTY(VisibleAnywhere, Category=SimCache)
	float StartSeconds = 0.0f;

	UPROPERTY(VisibleAnywhere, Category=SimCache)
	float DurationSeconds = 0.0f;

	UPROPERTY()
	FNiagaraSimCacheCreateParameters CreateParameters;

	UPROPERTY(transient)
	bool bNeedsReadComponentMappingRecache = true;

	UPROPERTY(transient)
	bool bCanRunAsync = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	TArray<FNiagaraVMExecutableDataId> CachedScriptVMIds;
#endif

	UPROPERTY()
	FNiagaraSimCacheLayout CacheLayout;

	UPROPERTY()
	TArray<FNiagaraSimCacheFrame> CacheFrames;

	UPROPERTY()
	TMap<FNiagaraVariableBase, TObjectPtr<UObject>> DataInterfaceStorage;

	UPROPERTY()
	TObjectPtr<UNiagaraSimCacheDebugData> DebugData;

	int32 CaptureTickCount = INDEX_NONE;
	double CaptureStartTime = 0;

	mutable std::atomic<int32> PendingCommandsInFlight;

	mutable FRWLock RWLock;

public:
	static NIAGARA_API FOnCacheBeginWrite	OnCacheBeginWrite;
	static NIAGARA_API FOnCacheEndWrite		OnCacheEndWrite;
};
