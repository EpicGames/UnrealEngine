// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimCacheCapture.h"

#include "Containers/Ticker.h"

#include "NiagaraComponent.h"
#include "NiagaraSimCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSimCacheCapture)

FNiagaraSimCacheCapture::~FNiagaraSimCacheCapture()
{
	if (TickerHandle.IsValid())
	{
		FinishCapture();
	}
}

void FNiagaraSimCacheCapture::FinishCapture()
{
	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
		TickerHandle.Reset();
	}

	UNiagaraSimCache* CaptureSimCache = WeakCaptureSimCache.Get();
	if ( CaptureSimCache != nullptr )
	{
		CaptureSimCache->EndWrite();
	}

	// in order to support the broadcasted message resulting in us being deleted we need to
	// be sure that we don't manipulate any data post broadcast
	FOnCaptureComplete LocalDelegate = MoveTemp(CaptureComplete);
	WeakCaptureComponent = nullptr;

	LocalDelegate.Broadcast(CaptureSimCache);
}

void FNiagaraSimCacheCapture::CaptureNiagaraSimCache(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters CreateParameters, UNiagaraComponent* NiagaraComponent, FNiagaraSimCacheCaptureParameters InCaptureParameters)
{
	WeakCaptureSimCache = SimCache;
	WeakCaptureComponent = NiagaraComponent;
	CaptureParameters = InCaptureParameters;
	CaptureParameters.Sanitize();
	CaptureFrameCounter = 0;
	if(SimCache == nullptr || NiagaraComponent == nullptr)
	{
		FinishCapture();
		return;
	}
	
	if (CaptureParameters.bAppendToSimCache)
	{
		SimCache->BeginAppend(CreateParameters, NiagaraComponent);
	}
	else
	{
		SimCache->BeginWrite(CreateParameters, NiagaraComponent);
	}

	// In this path we are going to manually advance the simulation until we complete
	if (CaptureParameters.bCaptureAllFramesImmediatly)
	{
		while (WeakCaptureComponent != nullptr && NiagaraComponent != nullptr)
		{
			NiagaraComponent->AdvanceSimulation(1, CaptureParameters.ImmediateCaptureDeltaTime);
			OnFrameTick(CaptureParameters.ImmediateCaptureDeltaTime);
		}
	}
	// In this path we are going to monitor the simulation and capture each frame
	else
	{
		if (NiagaraComponent && NiagaraComponent->GetWorld())
		{
			TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FNiagaraSimCacheCapture::OnFrameTick));
		}

		if (!TickerHandle.IsValid())
		{
			FinishCapture();
		}
	}
}

bool FNiagaraSimCacheCapture::CaptureCurrentFrameImmediate(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters CreateParameters, UNiagaraComponent* NiagaraComponent, UNiagaraSimCache*& OutSimCache, bool bAdvanceSimulation, float AdvanceDeltaTime)
{
	bool bSuccess = false;
	if (SimCache != nullptr && NiagaraComponent != nullptr)
	{
		if ( bAdvanceSimulation )
		{
			NiagaraComponent->AdvanceSimulation(1, AdvanceDeltaTime);
		}

		SimCache->BeginWrite(CreateParameters, NiagaraComponent);
		SimCache->WriteFrame(NiagaraComponent);
		SimCache->EndWrite();
		bSuccess = SimCache->IsCacheValid();
	}
	OutSimCache = SimCache;
	return bSuccess;
}

bool FNiagaraSimCacheCapture::OnFrameTick(float DeltaTime)
{
	// Component invalid or not active?  If so complete the cache recording
	UNiagaraComponent* CaptureComponent = WeakCaptureComponent.Get();
	UNiagaraSimCache* CaptureSimCache = WeakCaptureSimCache.Get();
	
	if ( !CaptureComponent || !CaptureComponent->IsActive() || !CaptureSimCache )
	{
		FinishCapture();
		return true;
	}

	// Should we record this frame?
	if ( (CaptureFrameCounter % CaptureParameters.CaptureRate) == 0 )
	{
		// If we fail to capture the frame it might be because things became invalid
		// Or it might be because the simulation was not ticked since the last capture in which case don't advance the counter
		if ( CaptureSimCache->WriteFrame(CaptureComponent) == false )
		{
			if ( CaptureSimCache->IsCacheValid() == false )
			{
				FinishCapture();
			}

			// Make sure we don't keep this alive forever, if we didn't managed to capture anything in 10 ticks something has probably gone wrong so bail
			if (CaptureParameters.bUseTimeout && (TimeOutCounter++ > CaptureParameters.TimeoutFrameCount))
			{
				UE_LOGF(LogNiagara, Warning, "SimCache Write has failed too many times, abandoning capturing for (%ls)", *GetFullNameSafe(CaptureSimCache));
				FinishCapture();
			}
			return true;
		}
	}

	TimeOutCounter = 0;
	++CaptureFrameCounter;

	// Have we recorded all the frames we need?
	// Note: the -1 is because T0 was the initial frame
	if ( CaptureParameters.bCaptureFixedNumberOfFrames && (CaptureFrameCounter > (CaptureParameters.CaptureRate * (CaptureParameters.NumFrames - 1))) )
	{
		FinishCapture();
		return true;
	}

	return true;
}
