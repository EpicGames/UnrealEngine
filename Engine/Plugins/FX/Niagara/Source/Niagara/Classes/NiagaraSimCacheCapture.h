// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraSimCache.h"
#include "Containers/Ticker.h"
#include "NiagaraSimCacheCapture.generated.h"

class UNiagaraComponent;
class UNiagaraSimCache;
struct FNiagaraSimCacheCreateParameters;

USTRUCT(BlueprintType)
struct FNiagaraSimCacheCaptureParameters
{
	GENERATED_BODY()

	FNiagaraSimCacheCaptureParameters()
		: bAppendToSimCache(false)
		, bCaptureFixedNumberOfFrames(true)
		, bUseTimeout(true)
		, bCaptureAllFramesImmediatly(false)
	{

	}

	/** When enabled we will append to the existing simulation cache rather than destroying the existing contents. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache")
	uint32 bAppendToSimCache : 1;

	/** When enabled we capture NumFrames number of frames, otherwise the capture will continue until cancelled or the simulation is complete. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache")
	uint32 bCaptureFixedNumberOfFrames : 1;

	/**
	 The max number of frames to capture. The capture will stop when the simulation completes or we have rendered this many frames, whichever happens first.
	 Set to 0 to capture until simulation completes.
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache", meta = (ClampMin = "1"))
	int32 NumFrames = 16;

	/** Allows for reducing the frequency of captured frames in relation to the simulation's frames. The ratio is 1 / CaptureRate, so a CaptureRate of 2 would captures frames 0, 2, 4, etc. */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache", meta = (ClampMin = "1"))
	int32 CaptureRate = 1;

	/** When enabled the capture will time out if we reach the defined number of frames without anything new to capture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache")
	uint32 bUseTimeout : 1;

	/** When we fail to capture a new frame after this many frames the capture will complete automatically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache", meta = (ClampMin = "1"))
	int32 TimeoutFrameCount = 10;

	/**
	When enabled we will capture all the requested frames immediatly.
	This will capture the simulation outside of the main work tick, i.e. if you request a 10 frame capture we will loop capturing 10 frames on the capture call rather than over 10 world ticks.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache")
	uint32 bCaptureAllFramesImmediatly : 1;

	/** The delta time in seconds to use when manually advancing the simulation.Defaults to 1 / 60th of a second(0.01666). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SimCache", meta = (ClampMin = "0.0001"))
	float ImmediateCaptureDeltaTime = 0.01666f;

	void Sanitize()
	{
		NumFrames = FMath::Max(NumFrames, 1);
		CaptureRate = FMath::Max(CaptureRate, 1);
		TimeoutFrameCount = FMath::Max(TimeoutFrameCount, 1);
		ImmediateCaptureDeltaTime = FMath::Max(ImmediateCaptureDeltaTime, 0.0001f);
	}
};

class FNiagaraSimCacheCapture
{
public:
	UE_NONCOPYABLE(FNiagaraSimCacheCapture);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCaptureComplete, UNiagaraSimCache*);

	FNiagaraSimCacheCapture() = default;
	NIAGARA_API ~FNiagaraSimCacheCapture();

	FOnCaptureComplete& OnCaptureComplete() {return CaptureComplete;}
	
	// Captures a niagara sim cache. The caller is responsible for managing the lifetime of the provided component and sim cache. 
	NIAGARA_API void CaptureNiagaraSimCache(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters CreateParameters, UNiagaraComponent* NiagaraComponent, FNiagaraSimCacheCaptureParameters CaptureParameters);

	/**
	Captures the simulations current frame data into the SimCache.
	This happens immediately so you may need to be careful with tick order of the component you are capturing from.
	The return can be invalid if the component can not be captured for some reason (i.e. not active).
	When AdvanceSimulation is true we will manually advance the simulation one frame using the provided AdvanceDeltaTime before capturing.
	*/
	NIAGARA_API bool CaptureCurrentFrameImmediate(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters CreateParameters, UNiagaraComponent* NiagaraComponent, UNiagaraSimCache*& OutSimCache, bool bAdvanceSimulation=false, float AdvanceDeltaTime=0.01666f);

	NIAGARA_API void FinishCapture();

private:
	TWeakObjectPtr<UNiagaraSimCache> WeakCaptureSimCache;

	TWeakObjectPtr<UNiagaraComponent> WeakCaptureComponent;
	
	FOnCaptureComplete CaptureComplete;

	FTSTicker::FDelegateHandle TickerHandle;

	FNiagaraSimCacheCaptureParameters CaptureParameters;

	int32 CaptureFrameCounter = 0;

	int32 TimeOutCounter = 0;
	
	NIAGARA_API bool OnFrameTick(float DeltaTime);
};
