// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "NiagaraSimCache.h"

/**
Allows you to compare two simulation caches for equality, or approximate equality.
Due to platform differences simulations may not be equal when comparing one generates from one platform to another.
Therefore you may need to be more flexible with some attributes, like positions, velocity, when comparing caches and adjust the tolerance accordingly.
Note: This class is Experimental and the API may change
*/
struct FNiagaraSimCacheCompare
{
public:
	/** Default float tolerance when comparing caches */
	float DefaultFloatTolerance = 0.01f;

	/** Tolerance when comparing quaternions, if not set uses the default UE tolerance. */
	TOptional<float> DefaultQuaternionTolerance;

	/** Should we exclude known none deterministic variables, i.e. ID / MaterialRandom. */
	bool bExcludeNoneDeterministicVariables = true;

	/**
	Should we include data interface as part of the comparison.
	Note: If the data interface does not implement the comparison function an error will be generated.
	*/
	bool bIncludeDataInterfaces = true;

	/** Add an attribute to exclude from the comparison. */
	NIAGARA_API void AddExcludeAttribute(FNiagaraVariableBase Variable);

	/** Adds an exclude rule to match against attreibute names. */
	NIAGARA_API void AddExcludeAttributesByWildcard(FString Wildcard);

	/** Setup comparison tolerances for variables that match the provided type. */
	NIAGARA_API void SetToleranceByType(const UNiagaraSimCache& SimCache, const FNiagaraTypeDefinition& TypeDef, float Tolerance);

	/** Setup comparison tolerances for variables that match the provided name wildcard. */
	NIAGARA_API void SetToleranceByName(const UNiagaraSimCache& SimCache, const FString& NameWildcard, float Tolerance);

	/**
	Compare the provided caches.
	returns true if they match, false if they do not.
	If they do not match OutDifferences will contain a subset of the information.
	*/
	NIAGARA_API bool Compare(const UNiagaraSimCache& LhsCache, const UNiagaraSimCache& RhsCache, FString& OutDifferences);

private:
	bool CompareEmitter(const UNiagaraSimCache& LhsCache, const UNiagaraSimCache& Rhs, int EmitterIndex, int FrameIndex, FString& OutDifferences);

private:
	TSet<FNiagaraVariableBase>			AttributesToExclude_SystemScript;
	TSet<FNiagaraVariableBase>			AttributesToExclude_ParticleScript;
	TArray<FString>						AttributesToExclude_ByWildcard;
	TMap<FNiagaraVariableBase, float>	VariableToFloatTolerances_SystemScript;
	TMap<FNiagaraVariableBase, float>	VariableToFloatTolerances_ParticleScript;
};
