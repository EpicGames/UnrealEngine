// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "UObject/Interface.h"
#include "Dom/JsonObject.h"
#include "NiagaraSimCacheCustomStorageInterface.generated.h"

struct FNiagaraSimCacheFeedbackContext;
class FNiagaraSystemInstance;

// Context used when reading data for a data interface
// Contains all the information requires to find the data / interpolate / rebase information
struct FNiagaraSimCacheDataInterfaceReadContext
{
	friend class UNiagaraSimCache;

public:
	// Get the stored data interface object, will assert on nullptr
	template<typename T> T* GetStorageObject() const  { return CastChecked<T>(StorageObject); }
	// Get the stored data interface object, can return nullptr if the cast fails
	template<typename T> T* GetOptionalStorageObject() const { return Cast<T>(StorageObject); }

	// Get the system instance the cache read is attached to
	FNiagaraSystemInstance* GetSystemInstance() const { return SystemInstance; }

	// Get the system instance data interface per instance data, will assert on nullptr
	template<typename T> T* GetPerInstanceData() const { check(OptionalPerInstanceData); return static_cast<T*>(OptionalPerInstanceData); }

	// Test to see if we should interpolate the data or not
	// The default value is expected to be the 'just works' case for when the user did not set force include / exlucde and interpolation is enabled
	bool ShouldInterpolate(bool DefaultValue) const { return InterpolateData.Get(DefaultValue); }

	// Get the currrent frame we are interpolating from
	int32 GetFrameIndexA() const { return(FrameIndexA); }
	// Get the currrent frame we are interpolating to
	int32 GetFrameIndexB() const { return(FrameIndexB); }
	// Get the currrent frame interpolation fraction
	float GetFrameInterp() const { return(FrameInterp); }

	// Test to see if we should rebase the data or not
	// The default value is expected to be the 'just works' case for when the user did not set force include / exlucde and rebasing is enabled
	bool ShouldRebaseData(bool DefaultValue) const { return RebaseData.Get(DefaultValue); }

	// Get the transform to rebase data, this is not world space but LWC tile space
	const FTransform& GetRebaseTransformA() const { return RebaseTransformA; }
	// Get the transform to rebase data, this is not world space but LWC tile space
	const FTransform& GetRebaseTransformB() const { return RebaseTransformB; }

	// Get the LWC tile the instance was in during caching for frame A
	FVector3f GetLWCTileA() const { return LWCTileA; }
	// Get the LWC tile the instance was in during caching for frame B
	FVector3f GetLWCTileB() const { return LWCTileB; }

private:
	UObject*				StorageObject = nullptr;
	FNiagaraSystemInstance*	SystemInstance = nullptr;
	void*					OptionalPerInstanceData = nullptr;
	TOptional<bool>			InterpolateData;
	int						FrameIndexA = 0;
	int						FrameIndexB = 0;
	float					FrameInterp = 0.0f;
	TOptional<bool>			RebaseData;
	FTransform				RebaseTransformA;
	FTransform				RebaseTransformB;
	FVector3f				LWCTileA = FVector3f::ZeroVector;
	FVector3f				LWCTileB = FVector3f::ZeroVector;
};

// Interface for UObjects to implement renderable mesh
UINTERFACE(MinimalAPI)
class UNiagaraSimCacheCustomStorageInterface : public UInterface
{
	GENERATED_BODY()
};

/**
The current API for storing data inside a simulation cache.
This is highly experimental and the API will change as we split editor / runtime data storage.

See INiagaraDataInterfaceSimCacheVisualizer to implement a custom visualizer widget for the stored data.
*/
class INiagaraSimCacheCustomStorageInterface
{
	GENERATED_BODY()

public:
	/**
	Called when we begin to write data into a simulation cache.
	Returning nullptr means you are not going to cache any data for the simulation.
	The object returned will be stored directly into the cache file, so you are expected to manage the size of the object and store data appropriately.
	*/
	virtual UObject* SimCacheBeginWrite(UObject* SimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const { return nullptr; }

	/**
	Called when we are ready to write data into the simulation cache.
	This is always called in sequence, i.e. 0, 1, 2, etc, we will never jump around frames.
	*/
	virtual bool SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const { return true; }

	/**
	Called when we complete writing data into the simulation cache.
	Note: This is called using the Class Default Object not the instance the object was created from
	*/
	virtual bool SimCacheEndWrite(UObject* StorageObject) const { return true; }

	/**
	Read a frame of data from the simulation cache.
	*/
	virtual bool SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SimCacheReadFrame(ReadContext.GetStorageObject<UObject>(), ReadContext.GetFrameIndexA(), ReadContext.GetFrameIndexB(), ReadContext.GetFrameInterp(), ReadContext.GetSystemInstance(), ReadContext.GetPerInstanceData<void>());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.6, "This function has been deprecated in favor of one which uses a context")
	virtual bool SimCacheReadFrame(UObject* StorageObject, int FrameA, int FrameB, float Interp, FNiagaraSystemInstance* SystemInstance, void* OptionalPerInstanceData) { return true; }

	/**
	Called when the simulation cache has finished reading a frame.
	Only DataInterfaces with PerInstanceData will have this method called on them.
	*/
	virtual void SimCachePostReadFrame(void* OptionalPerInstanceData, FNiagaraSystemInstance* SystemInstance) {}

	/**
	Called to compare a frame between two separate simulation cache storages. Mainly useful for unit testing.
	This will be called on the CDO object since we do not have the actual data interface.
	*/
	virtual bool SimCacheCompareFrame(const UObject* LhsStorageObject, const UObject* RhsStorageObject, int FrameIndex, TOptional<float> Tolerance, FString& OutErrors) const { OutErrors = TEXT("Compare not implemented"); return false; }

	/**
	This function allows you to preserve a list of attributes when building a renderer only cache.
	The UsageContext will be either a UNiagaraSystem or a UNiagaraEmitter and can be used to scope your variables accordingly.
	For example, if you were to require 'Particles.MyAttribute' in order to process the cache results you would need to convert
	this into 'MyEmitter.Particles.MyAttribute' by checking the UsageContext is a UNiagaraEmitter and then creating the variable from the unique name.
	*/
	virtual TArray<FNiagaraVariableBase> GetSimCacheRendererAttributes(const UObject* UsageContext) const { return TArray<FNiagaraVariableBase>(); }

	/**
	 This converts the content of the storage object to a json representation. If another interchange format (e.g. an image format) is better, then the json this method produces should link to the secondary files.
	@param TargetFolder (optional) the folder where to save auxiliary data from this frame. Might not be set if external files are not supported (e.g. when called over network).
	@param FilenamePrefix (optional) unique name for this data interface - can either be used directly as filename or add extensions like .png or even to create a folder and put in multiple files related to this data interface
	@return a json string representing this data interface object
	 */
	virtual TSharedPtr<FJsonObject> SimCacheToJson(const UObject* StorageObject, int FrameIndex, TOptional<FString> TargetFolder, TOptional<FString> FilenamePrefix) const { return TSharedPtr<FJsonObject>(); }
};
