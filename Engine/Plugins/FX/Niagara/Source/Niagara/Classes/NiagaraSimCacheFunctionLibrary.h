// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraSimCache.h"
#include "NiagaraSimCacheCapture.h"

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/CancellableAsyncAction.h"

#include "NiagaraSimCacheFunctionLibrary.generated.h"

UCLASS()
class UAsyncNiagaraCaptureSimCache : public UCancellableAsyncAction
{
	GENERATED_BODY()

public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnCaptureComplete, bool, bSuccess);

	UPROPERTY()
	TObjectPtr<UNiagaraSimCache> CaptureSimCache;

	UPROPERTY()
	TObjectPtr<UNiagaraComponent> CaptureComponent;
	
	UPROPERTY(BlueprintAssignable)
	FOnCaptureComplete CaptureComplete;

	bool bIsRunning = false;
	FNiagaraSimCacheCapture SimCacheCapture;
	FNiagaraSimCacheCaptureParameters CaptureParameters;
	FNiagaraSimCacheCreateParameters CreateParameters;

	virtual void Activate() override;
	virtual void Cancel() override;
	virtual void SetReadyToDestroy() override;

	/**
	Capture multiple frames from the provided simulation into a SimCache until the simulation becomes inactive, completes or we hit the NumFrames limit.
	Capture occurs at the end of each frame with the first frame being this frame.
	CaptureRate allows you to reduce the rate of capture, i.e. a rate of 2 would capture frames 0, 2, 4, etc.
	When AdvanceSimulation is true we will manually advance the simulation in a loop until we have captured the number of frames request, rather than reading from the component each frame.
	*/
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly="true", Category=NiagaraSimCache, AdvancedDisplay = "CaptureRate, bAdvanceSimulation, AdvanceDeltaTime"))
	static UAsyncNiagaraCaptureSimCache* CaptureNiagaraSimCacheMultiFrame(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters CreateParameters, UNiagaraComponent* NiagaraComponent, UNiagaraSimCache*& OutSimCache, int32 NumFrames = 16, int32 CaptureRate = 1, bool bAdvanceSimulation=false, float AdvanceDeltaTime=0.01666f);
	
	/**
	Capture frames from the provided simulation into a SimCache until the simulation becomes inactive or completes.
	Capture occurs at the end of each frame with the first frame being this frame.
	CaptureRate allows you to reduce the rate of capture, i.e. a rate of 2 would capture frames 0, 2, 4, etc.
	When AdvanceSimulation is true we will manually advance the simulation until the capture is complete inside a loop, rather than reading from the component each frame.
	*/
	UFUNCTION(BlueprintCallable, meta=(BlueprintInternalUseOnly="true", Category=NiagaraSimCache, AdvancedDisplay = "CaptureRate, bAdvanceSimulation, AdvanceDeltaTime"))
	static UAsyncNiagaraCaptureSimCache* CaptureNiagaraSimCacheUntilComplete(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters CreateParameters, UNiagaraComponent* NiagaraComponent, UNiagaraSimCache*& OutSimCache, int32 CaptureRate = 1, bool bAdvanceSimulation=false, float AdvanceDeltaTime=0.01666f);

	/**
	Capture a simulation cache with customizable capture parameters.
	*/
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", Category = NiagaraSimCache))
	static UAsyncNiagaraCaptureSimCache* CaptureNiagaraSimCache(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters CreateParameters, UNiagaraComponent* NiagaraComponent, FNiagaraSimCacheCaptureParameters CaptureParameters, UNiagaraSimCache*& OutSimCache);

private:
	static UAsyncNiagaraCaptureSimCache* CaptureNiagaraSimCacheImpl(UNiagaraSimCache* SimCache, const FNiagaraSimCacheCreateParameters& CreateParameters, UNiagaraComponent* NiagaraComponent, const FNiagaraSimCacheCaptureParameters& CaptureParameters, UNiagaraSimCache*& OutSimCache);

	void CaptureFinished(UNiagaraSimCache* CapturedSimCache);
};

UCLASS(MinimalAPI)
class UNiagaraSimCacheFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:
	/**
	Captures the simulations current frame data into the SimCache.
	This happens immediately so you may need to be careful with tick order of the component you are capturing from.
	The return can be invalid if the component can not be captured for some reason (i.e. not active).
	When AdvanceSimulation is true we will manually advance the simulation one frame using the provided AdvanceDeltaTime before capturing.
	*/
	UFUNCTION(BlueprintCallable, Category=NiagaraSimCache, meta=(ReturnDisplayName="Success"))
	static NIAGARA_API bool CaptureNiagaraSimCacheImmediate(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters CreateParameters, UNiagaraComponent* NiagaraComponent, UNiagaraSimCache*& OutSimCache, bool bAdvanceSimulation=false, float AdvanceDeltaTime=0.01666f);

	/**
	Captures the simulation cache object that can be captured into using the various API calls.
	*/
	UFUNCTION(BlueprintCallable, Category=NiagaraSimCache, meta=(WorldContext="WorldContextObject", ReturnDisplayName="SimCache"))
	static NIAGARA_API UNiagaraSimCache* CreateNiagaraSimCache(UObject* WorldContextObject);
};
