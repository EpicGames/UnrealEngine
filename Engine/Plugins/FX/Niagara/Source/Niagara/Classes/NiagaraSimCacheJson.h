// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "NiagaraSimCache.h"

class FJsonObject;

/** Experimental Json interop for Niagara Sim Caches. */
struct FNiagaraSimCacheJson
{
	enum class EExportType
	{
		SingleJsonFile,
		SeparateEachFrame
	};
	
	NIAGARA_API static bool DumpToFile(const UNiagaraSimCache& SimCache, const FString& FullPath, EExportType ExportType = EExportType::SingleJsonFile);

	NIAGARA_API static TSharedPtr<FJsonObject> ToJson(const UNiagaraSimCache& SimCache);
	NIAGARA_API static TSharedPtr<FJsonObject> SystemDataToJson(const UNiagaraSimCache& SimCache);
	NIAGARA_API static TSharedPtr<FJsonObject> EmitterFrameToJson(const UNiagaraSimCache& SimCache, int EmitterIndex, int FrameIndex);

private:
	static bool DumpFramesToFolder(const UNiagaraSimCache& SimCache, const FString& TargetFolder);
};
