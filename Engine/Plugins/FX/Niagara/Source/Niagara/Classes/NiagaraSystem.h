// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "NiagaraDataSetCompiledData.h"
#include "NiagaraDataSetAccessor.h"
#include "NiagaraEffectType.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraMessageStore.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraParameterDefinitionsSubscriber.h"
#include "NiagaraScriptRuntimeCookedData.h"
#include "NiagaraUserRedirectionParameterStore.h"
#include "Particles/ParticleSystem.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Particles/FXBudget.h"
#include "NiagaraSystem.generated.h"

struct FNiagaraAssetTagDefinitionReference;
class FNiagaraActiveCompilation;
class UNiagaraEffectType;
enum class ENiagaraCullProxyMode : uint32;
enum class ERendererStencilMask : uint8;
struct FNiagaraSystemStateData;
struct FNiagaraSystemScalabilityOverride;
struct FNiagaraSystemScalabilitySettings;
struct FNiagaraSystemStaticBuffers;
struct FNiagaraScriptRuntimeCompiledDataHandle;
class  FNiagaraScriptRuntimeData;

#if WITH_EDITORONLY_DATA
class UNiagaraBakerSettings;
class UNiagaraEditorDataBase;
class UNiagaraMessageDataBase;
class UNiagaraScriptRuntimeCompiledDataEditorReference;
enum class ENiagaraScriptLibraryVisibility : uint8;
enum class ENiagaraScriptTemplateSpecification : uint8;
struct FNiagaraGraphCachedDataBase;
#endif

class UNiagaraDataChannel;
class ITargetPlatform;

enum class ENiagaraPerformanceStateScopeMode : int
{
	/** No stats exist and we should not show the option. */
	None,
	/** Only basic (i.e. whole script) stats exist. */
	Basic,
	/** Detailed (i.e. per module) stats exist. */
	Detailed,
};

USTRUCT()
struct FNiagaraEmitterCompiledData
{
	GENERATED_USTRUCT_BODY()

	FNiagaraEmitterCompiledData();
	
	/** Attribute names in the data set that are driving each emitter's spawning. */
	UPROPERTY()
	TArray<FName> SpawnAttributes;

	/** Explicit list of Niagara Variables to bind to Emitter instances. */
	UPROPERTY()
	FNiagaraVariable EmitterSpawnIntervalVar;

	UPROPERTY()
	FNiagaraVariable EmitterInterpSpawnStartDTVar;

	UPROPERTY()
	FNiagaraVariable EmitterSpawnGroupVar;

	UPROPERTY()
	FNiagaraVariable EmitterAgeVar;

	UPROPERTY()
	FNiagaraVariable EmitterRandomSeedVar;

	UPROPERTY()
	FNiagaraVariable EmitterInstanceSeedVar;

	UPROPERTY()
	FNiagaraVariable EmitterTotalSpawnedParticlesVar;

	/** Per-Emitter DataSet Data. */
	UPROPERTY()
	FNiagaraDataSetCompiledData DataSetCompiledData;

	#if NIAGARA_SYSTEM_CAPTURE
	/** Debug only data for caputuring from GPU sims, laziliy initialized when needed via GetGPUCaptureDataSetCompiledData().*/
	mutable FNiagaraDataSetCompiledData GPUCaptureDataSetCompiledData;
	const FNiagaraDataSetCompiledData& GetGPUCaptureDataSetCompiledData()const;
	#endif
};

USTRUCT()
struct FNiagaraParameterDataSetBinding
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 ParameterOffset = 0;

	UPROPERTY()
	int32 DataSetComponentOffset = 0;
};

USTRUCT()
struct FNiagaraParameterDataSetBindingCollection
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FNiagaraParameterDataSetBinding> FloatOffsets;

	UPROPERTY()
	TArray<FNiagaraParameterDataSetBinding> Int32Offsets;

#if WITH_EDITORONLY_DATA
	template<typename BufferType>
	void Build(const FNiagaraDataSetCompiledData& DataSet)
	{
		BuildInternal(BufferType::GetVariables(), DataSet, TEXT(""), TEXT(""));
	}

	template<typename BufferType>
	void Build(const FNiagaraDataSetCompiledData& DataSet, const FString& NamespaceBase, const FString& NamespaceReplacement)
	{
		BuildInternal(BufferType::GetVariables(), DataSet, NamespaceBase, NamespaceReplacement);
	}

protected:
	void BuildInternal(const TArray<FNiagaraVariable>& ParameterVars, const FNiagaraDataSetCompiledData& DataSet, const FString& NamespaceBase, const FString& NamespaceReplacement);

#endif
};

USTRUCT()
struct FNiagaraSystemCompiledData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FNiagaraParameterStore InstanceParamStore;

	UPROPERTY()
	FNiagaraDataSetCompiledData DataSetCompiledData;

	UPROPERTY()
	FNiagaraDataSetCompiledData SpawnInstanceParamsDataSetCompiledData;

	UPROPERTY()
	FNiagaraDataSetCompiledData UpdateInstanceParamsDataSetCompiledData;

	UPROPERTY()
	FNiagaraParameterDataSetBindingCollection SpawnInstanceGlobalBinding;
	UPROPERTY()
	FNiagaraParameterDataSetBindingCollection SpawnInstanceSystemBinding;
	UPROPERTY()
	FNiagaraParameterDataSetBindingCollection SpawnInstanceOwnerBinding;
	UPROPERTY()
	TArray<FNiagaraParameterDataSetBindingCollection> SpawnInstanceEmitterBindings;

	UPROPERTY()
	FNiagaraParameterDataSetBindingCollection UpdateInstanceGlobalBinding;
	UPROPERTY()
	FNiagaraParameterDataSetBindingCollection UpdateInstanceSystemBinding;
	UPROPERTY()
	FNiagaraParameterDataSetBindingCollection UpdateInstanceOwnerBinding;
	UPROPERTY()
	TArray<FNiagaraParameterDataSetBindingCollection> UpdateInstanceEmitterBindings;
};

struct FNiagaraEmitterExecutionIndex
{
	FNiagaraEmitterExecutionIndex() { bStartNewOverlapGroup = false; EmitterIndex = 0; }

	/** Flag to denote if the batcher should start a new overlap group, i.e. when we have a dependency ensure we don't overlap with the emitter we depend on. */
	uint32 bStartNewOverlapGroup : 1;
	/** Emitter index to use */
	uint32 EmitterIndex : 31;
};

struct FNiagaraRendererExecutionIndex
{
	/** The index of the emitter */
	uint32 EmitterIndex = INDEX_NONE;
	/** The index of the renderer in the emitter's list */
	uint32 EmitterRendererIndex = INDEX_NONE;
	/** The index of the renderer in the entire system */
	uint32 SystemRendererIndex = INDEX_NONE;
};

/** Container struct for an array of system scalability overrides. Enables details customization and data validation. */
USTRUCT()
struct FNiagaraSystemScalabilityOverrides
{
	GENERATED_USTRUCT_BODY()

	NIAGARA_API FNiagaraSystemScalabilityOverrides();
	NIAGARA_API ~FNiagaraSystemScalabilityOverrides();

	UPROPERTY(EditAnywhere, Category = "Override")
	TArray<FNiagaraSystemScalabilityOverride> Overrides;
};

UENUM()
enum class ENiagaraLwcTileUpdateMode : uint8
{
	/**
	When a simulation moves greater or equal to the number of tiles specified in fx.LWCTileRecache the simulation will be reset.
	This means all particles are cleared and we start the system from time 0 again.
	*/
	ResetSimulation,
	/**
	When a simulation moves greater or equal to the number of tiles specified in fx.LWCTileRecache we will trigger a rebase.
	A rebase will move all Position type attributes by the difference in tiles.
	This is not guaranteed to cover everything, such as data inteface, matrix attributes, etc, and is therefore could result in simulation errors.
	*/
	Rebase,
	/**
	This option is similar to Rebase with the exception that when we cross fx.LWCTileForceReset number of tiles we will reset the simulation.
	This can provide some additional safety if you were to cross a large number of tiles and move back the same number of tiles,
	as in situations like the floating point numbers may have lost significant precision resulting in an invalid simulation.
	*/
	RebaseOrResetSimulation,
};

/** A Niagara System contains multiple Niagara Emitters to create various effects.
 * Niagara Systems can be placed in the world, unlike Emitters, and expose User Parameters to configure an effect at runtime.*/
UCLASS(BlueprintType, MinimalAPI, meta = (LoadBehavior = "LazyOnDemand"))
class UNiagaraSystem : public UFXSystemAsset, public INiagaraParameterDefinitionsSubscriber
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSystemCompiled, UNiagaraSystem*);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSystemPostEditChange, UNiagaraSystem*);
	DECLARE_MULTICAST_DELEGATE(FOnScalabilityChanged)
#endif
	//TestChange

	NIAGARA_API UNiagaraSystem(FVTableHelper& Helper);
	NIAGARA_API ~UNiagaraSystem();

	//~ UObject interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void Serialize(FArchive& Ar) override;
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual bool CanBeClusterRoot() const override { return true; }
	NIAGARA_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
#if WITH_EDITORONLY_DATA
	static NIAGARA_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	NIAGARA_API virtual void BeginDestroy() override;
	NIAGARA_API virtual bool IsReadyForFinishDestroy() override;
	NIAGARA_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#if WITH_EDITOR
	NIAGARA_API virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext) override;
	NIAGARA_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	NIAGARA_API virtual void PreEditChange(FProperty* PropertyThatWillChange)override;
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override; 
	NIAGARA_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform *TargetPlatform) override;
	NIAGARA_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	//~ End UObject interface

	/** Helper method to handle when an internal variable has been renamed. Renames any downstream dependencies in the emitters or exposed variables.*/
	NIAGARA_API void HandleVariableRenamed(const FNiagaraVariable& InOldVariable, const FNiagaraVariable& InNewVariable, bool bUpdateContexts);
	/** Helper method to handle when an internal variable has been removed. Resets any downstream dependencies in the emitters or exposed variables.*/
	NIAGARA_API void HandleVariableRemoved(const FNiagaraVariable& InOldVariable, bool bUpdateContexts);
#endif

#if WITH_EDITORONLY_DATA
	//~ Begin INiagaraParameterDefinitionsSubscriber interface
	virtual const TArray<FParameterDefinitionsSubscription>& GetParameterDefinitionsSubscriptions() const override { return ParameterDefinitionsSubscriptions; }
	
	virtual TArray<FParameterDefinitionsSubscription>& GetParameterDefinitionsSubscriptions() override { return ParameterDefinitionsSubscriptions; };

	/** Get all UNiagaraScriptSourceBase of this subscriber. */
	NIAGARA_API virtual TArray<UNiagaraScriptSourceBase*> GetAllSourceScripts() override;

	/** Get the path to the UObject of this subscriber. */
	NIAGARA_API virtual FString GetSourceObjectPathName() const override;

	/** Get All adapters to editor only script vars owned directly by this subscriber. */
	NIAGARA_API virtual TArray<UNiagaraEditorParametersAdapterBase*> GetEditorOnlyParametersAdapters() override;

	/** Get all subscribers that are owned by this subscriber.
	 *  Note: Implemented for synchronizing UNiagaraSystem. UNiagaraSystem returns all UNiagaraEmitters it owns to call SynchronizeWithParameterDefinitions for each.
	 */
	NIAGARA_API virtual TArray<INiagaraParameterDefinitionsSubscriber*> GetOwnedParameterDefinitionsSubscribers() override;
	//~ End INiagaraParameterDefinitionsSubscriber interface

	NIAGARA_API virtual bool ChangeEmitterVersion(const FVersionedNiagaraEmitter& Emitter, const FGuid& NewVersion);

	/**
	 * Append config values or settings that can change how instances of the class are cooked, including especially
	 * values that determine how version upgraded are conducted. Can also append a unique guid when necessary to
	 * invalidate previous results because serialization changed and no custom version was updated.
	 */
	NIAGARA_API static void AppendToClassSchema(FAppendToClassSchemaContext& Context);
#endif 

	/** Gets an array of the emitter handles. */
	NIAGARA_API TArray<FNiagaraEmitterHandle>& GetEmitterHandles();
	NIAGARA_API const TArray<FNiagaraEmitterHandle>& GetEmitterHandles()const;
	
	FNiagaraSystemScalabilityOverrides& GetScalabilityOverrides(){return SystemScalabilityOverrides; }

private:
	NIAGARA_API bool IsValidInternal() const;

#if WITH_EDITORONLY_DATA
	NIAGARA_API void FixupPositionUserParameters();
#endif
	
public:
	/** Returns true if this system is valid and can be instanced. False otherwise. */
	bool IsValid() const { return FPlatformProperties::RequiresCookedData() ? bIsValidCached : IsValidInternal(); }

#if WITH_EDITORONLY_DATA
	/** Adds a new emitter handle to this System.  The new handle exposes an Instance value which is a copy of the
		original asset. */
	NIAGARA_API FNiagaraEmitterHandle AddEmitterHandle(UNiagaraEmitter& SourceEmitter, FName EmitterName, FGuid EmitterVersion);

	/** Adds a new emitter handle to this system without copying the original asset. This should only be used for temporary systems and never for live assets. */
	NIAGARA_API void AddEmitterHandleDirect(FNiagaraEmitterHandle& EmitterHandleToAdd);

	/** Duplicates an existing emitter handle and adds it to the System.  The new handle will reference the same source asset,
		but will have a copy of the duplicated Instance value. */
	NIAGARA_API FNiagaraEmitterHandle DuplicateEmitterHandle(const FNiagaraEmitterHandle& EmitterHandleToDuplicate, FName EmitterName);

	/** Removes the provided emitter handle. */
	NIAGARA_API void RemoveEmitterHandle(const FNiagaraEmitterHandle& EmitterHandleToDelete);

	/** Removes the emitter handles which have an Id in the supplied set. */
	NIAGARA_API void RemoveEmitterHandlesById(const TSet<FGuid>& HandlesToRemove);

#endif


	FNiagaraEmitterHandle& GetEmitterHandle(int Idx)
	{
		check(Idx < EmitterHandles.Num());
		return EmitterHandles[Idx];
	};

	const FNiagaraEmitterHandle& GetEmitterHandle(int Idx) const
	{
		check(Idx < EmitterHandles.Num());
		return EmitterHandles[Idx];
	};

	int GetNumEmitters() const
	{
		return EmitterHandles.Num();
	}

	/** From the last compile, what are the variables that were exported out of the system for external use?*/
	const FNiagaraUserRedirectionParameterStore& GetExposedParameters() const {	return ExposedParameters; }
	FNiagaraUserRedirectionParameterStore& GetExposedParameters()  { return ExposedParameters; }

	/** Gets the System script which is used to populate the System parameters and parameter bindings. */
	NIAGARA_API UNiagaraScript* GetSystemSpawnScript();
	NIAGARA_API UNiagaraScript* GetSystemUpdateScript();
	NIAGARA_API const UNiagaraScript* GetSystemSpawnScript() const;
	NIAGARA_API const UNiagaraScript* GetSystemUpdateScript() const;

	TOptional<float> GetMaxDeltaTime() const { return MaxDeltaTime; }
	const FNiagaraDataSetAccessor<ENiagaraExecutionState>& GetSystemExecutionStateAccessor() const { return SystemExecutionStateAccessor; }
	TConstArrayView<FNiagaraDataSetAccessor<ENiagaraExecutionState>> GetEmitterExecutionStateAccessors() const { return MakeArrayView(EmitterExecutionStateAccessors); }
	TConstArrayView<FNiagaraDataSetAccessor<FNiagaraSpawnInfo>> GetEmitterSpawnInfoAccessors(int32 EmitterIndex) const { return MakeArrayView(EmitterSpawnInfoAccessors[EmitterIndex]);  }
	
	/** Performs the passed action for all scripts in this system.
	    Include NiagaraSystemImpl.h to be able to call this function */
	template<typename TAction>
	inline void ForEachScript(TAction Func) const;

	/** Performs the passed action for all scripts in this system and provides the script runtime data.
		Include NiagaraSystemImpl.h to be able to call this function */
	template<typename TAction>
	inline void ForEachScriptWithRuntimeData(TAction Func) const;

	/** Performs the passed action for all scripts in this system and provides the owning system and emitter handle.
		Include NiagaraSystemImpl.h to be able to call this function */
	template<typename TAction>
	inline void ForEachScriptWithOwningContext(TAction Func) const;

	/** Performs the passed action for all FNiagaraPlatformSets used by this system. Some may not be owned by this system.
	    Include NiagaraSystemImpl.h to be able to call this function */
	template<typename TAction>
	inline void ForEachPlatformSet(TAction Func);

	NIAGARA_API bool AllowScalabilityForLocalPlayerFX()const;

	NIAGARA_API virtual void PrecacheAssetPSOs() override;

private:
	NIAGARA_API bool IsReadyToRunInternal() const;

public:
	NIAGARA_API bool IsReadyToRun() const;

	inline bool NeedsWarmup()const { return WarmupTickCount > 0 && WarmupTickDelta > SMALL_NUMBER; }
	inline float GetWarmupTime()const { return WarmupTime; }
	inline int32 GetWarmupTickCount()const { return WarmupTickCount; }
	inline float GetWarmupTickDelta()const { return WarmupTickDelta; }

	inline bool HasFixedTickDelta() const { return bFixedTickDelta; }
	inline float GetFixedTickDeltaTime()const { return FixedTickDeltaTime; }
	NIAGARA_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	NIAGARA_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

	inline bool NeedsDeterminism() const { return bDeterminism; }
	inline int32 GetRandomSeed() const { return RandomSeed; }

	NIAGARA_API void SetWarmupTime(float InWarmupTime);
	NIAGARA_API void SetWarmupTickDelta(float InWarmupTickDelta);
	NIAGARA_API void ResolveWarmupTickCount();

#if STATS
	FNiagaraStatDatabase& GetStatData() { return StatDatabase; }
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;

	/** Are there any pending compile requests?*/
	NIAGARA_API bool HasOutstandingCompilationRequests(bool bIncludingGPUShaders = false) const;

	NIAGARA_API bool CompileRequestsShouldBlockGC() const;

	/** Determines if this system has the supplied emitter as an editable and simulating emitter instance. */
	NIAGARA_API bool ReferencesInstanceEmitter(const FVersionedNiagaraEmitter& Emitter) const;

	/** Request that any dirty scripts referenced by this system be compiled.*/
	NIAGARA_API bool RequestCompile(bool bForce, FNiagaraSystemUpdateContext* OptionalUpdateContext = nullptr, const ITargetPlatform* TargetPlatform = nullptr);

	/** If we have a pending compile request, is it done with yet? */
	NIAGARA_API bool PollForCompilationComplete(bool bFlushRequestCompile = true);

	/** Blocks until all active compile jobs have finished */
	NIAGARA_API void WaitForCompilationComplete(bool bIncludingGPUShaders = false, bool bShowProgress = true);

	/** Blocks until all active compile jobs have finished but does not flush any requests that were deferred from loading. */
	NIAGARA_API void WaitForCompilationComplete_SkipPendingOnDemand(bool bIncludingGPUShaders = false, bool bShowProgress = true);

	/** Tries to abort all running shader compilations */
	NIAGARA_API void KillAllActiveCompilations();

	/** Invalidates any active compilation requests which will ignore their results. */
	NIAGARA_API void InvalidateActiveCompiles();

	/** Returns true if there are any active compilations. */
	NIAGARA_API bool HasActiveCompilations() const;

	/** Delegate called when the system's dependencies have all been compiled.*/
	NIAGARA_API FOnSystemCompiled& OnSystemCompiled();

	/** Delegate called on PostEditChange.*/
	NIAGARA_API FOnSystemPostEditChange& OnSystemPostEditChange();

	/** Delegate called on effect type or effect type value change */
	NIAGARA_API FOnScalabilityChanged& OnScalabilityChanged();

	/** Gets editor specific data stored with this system. */
	NIAGARA_API UNiagaraEditorDataBase* GetEditorData();

	/** Gets editor specific parameters stored with this system */
	NIAGARA_API UNiagaraEditorParametersAdapterBase* GetEditorParameters();

	/** Gets editor specific data stored with this system. */
	NIAGARA_API const UNiagaraEditorDataBase* GetEditorData() const;

	/** Internal: The thumbnail image.*/
	UPROPERTY()
	TObjectPtr<class UTexture2D> ThumbnailImage;

	/** Deprecated library exposure bool. Use the LibraryVisibility enum instead. FNiagaraEditorUtilities has accessor functions that takes deprecation into account */
	UPROPERTY()
	bool bExposeToLibrary_DEPRECATED;

	/** If this system is exposed to the library, or should be explicitly hidden. */
	UPROPERTY(AssetRegistrySearchable, meta = (SkipSystemResetOnChange = "true"))
	ENiagaraScriptLibraryVisibility LibraryVisibility;
	
	/** Deprecated template asset bool. Use the TemplateSpecification enum instead. */
	UPROPERTY()
	bool bIsTemplateAsset_DEPRECATED;

	/** If this system is a regular system, a template or a behavior example. */
	UPROPERTY()
	ENiagaraScriptTemplateSpecification TemplateSpecification_DEPRECATED;

	UPROPERTY(meta=(DeprecatedProperty))
	TArray<FNiagaraAssetTagDefinitionReference> AssetTags_DEPRECATED;
	
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Asset Options", AssetRegistrySearchable, DisplayName="Asset Description", meta = (SkipSystemResetOnChange = "true"))
	FText TemplateAssetDescription;

	/** Category of this system. */
	UPROPERTY(AssetRegistrySearchable, meta = (SkipSystemResetOnChange = "true"))
	FText Category;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Asset Options", AssetRegistrySearchable, meta = (SkipSystemResetOnChange = "true", AllowedClasses="/Script/MediaAssets.FileMediaSource"))
	FSoftObjectPath PreviewMoviePath;
	
	UPROPERTY()
	TArray<TObjectPtr<UNiagaraScript>> ScratchPadScripts;

	UPROPERTY()
	TObjectPtr<UObject> TypeContainer;

	UPROPERTY(transient)
	FNiagaraParameterStore EditorOnlyAddedParameters;

	NIAGARA_API bool GetIsolateEnabled() const;
	NIAGARA_API void SetIsolateEnabled(bool bIsolate);
	
	UPROPERTY(transient)
	FNiagaraSystemUpdateContext UpdateContext;

	NIAGARA_API const static FGuid ResolveDIsMessageId;
	NIAGARA_API const static FGuid ComputeEmitterExecutionOrderMessageId;
#endif

	NIAGARA_API void UpdateSystemAfterLoad();
	NIAGARA_API void EnsureFullyLoaded() const;

	NIAGARA_API bool IsLooping() const;

	const TArray<TSharedRef<const FNiagaraEmitterCompiledData>>& GetEmitterCompiledData() const { return EmitterCompiledData; };

	/** Gets the script that will have the compiled data for the supplied emitter handle, script usage, and usage id. */
	const UNiagaraScript* GetTargetCompiledScript(const FGuid& EmitterHandleId, ENiagaraScriptUsage ScriptUsage, const FGuid& ScriptUsageId) const
	{
		return GetTargetCompiledScript(FNiagaraScriptDataKey(EmitterHandleId, ScriptUsage, ScriptUsageId));
	}

	/** Gets the script that will have the compiled data for the supplied script data key. */
	UNiagaraScript* GetTargetCompiledScript(const FNiagaraScriptDataKey& ScriptDataKey) const;

	const FNiagaraSystemCompiledData& GetSystemCompiledData() const { return SystemCompiledData; };

	TSharedPtr<const FNiagaraScriptRuntimeData> GetScriptRuntimeData(const FNiagaraEmitterHandle* EmitterHandle, const UNiagaraScript& Script) const
	{
		return GetScriptRuntimeData(FNiagaraScriptDataKey(EmitterHandle, Script));
	}

	TSharedPtr<const FNiagaraScriptRuntimeData> GetScriptRuntimeData(const FGuid& EmitterHandleId, ENiagaraScriptUsage ScriptUsage, const FGuid& ScriptUsageId) const
	{
		return GetScriptRuntimeData(FNiagaraScriptDataKey(EmitterHandleId, ScriptUsage, ScriptUsageId));
	}

	NIAGARA_API TSharedPtr<const FNiagaraScriptRuntimeData> GetScriptRuntimeData(const FNiagaraScriptDataKey& ScriptDataKey) const;

private:
#if WITH_EDITORONLY_DATA
	struct FEditorRuntimeCompiledDataHandleDeleter
	{
		FEditorRuntimeCompiledDataHandleDeleter(const UNiagaraSystem* System);

		void operator()(FNiagaraScriptRuntimeCompiledDataHandle* ObjectToDelete) const;

		TWeakObjectPtr<const UNiagaraSystem> SystemWeak;
	};

	void NotifyScriptRuntimeCompiledDataHandleDeleted(const FObjectKey& OwnerEditorReferenceKey, const FNiagaraScriptDataKey& OwnerScriptDataKey) const;
#endif

	TSharedPtr<const FNiagaraScriptRuntimeCompiledDataHandle> CreateScriptRuntimeCompiledDataHandle(const FNiagaraScriptDataKey& ScriptDataKey) const;

public:
	NIAGARA_API bool UsesCollection(const UNiagaraParameterCollection* Collection)const;

	bool SupportsLargeWorldCoordinates() const { return bSupportLargeWorldCoordinates && bLwcEnabledSettingCached; }
	ENiagaraLwcTileUpdateMode GetLargeWorldCoordinateTileUpdateMode() const;

	inline bool IsInitialOwnerVelocityFromActor() const { return bInitialOwnerVelocityFromActor; }

#if WITH_EDITORONLY_DATA
	NIAGARA_API void ReportAnalyticsData(bool bIsCooking);
	NIAGARA_API bool UsesEmitter(UNiagaraEmitterBase* Emitter) const;
	NIAGARA_API bool UsesEmitter(const FVersionedNiagaraEmitter& VersionedEmitter) const;
	NIAGARA_API bool UsesEmitter(const FVersionedNiagaraEmitterBase& VersionedEmitter) const;
	NIAGARA_API bool UsesScript(const UNiagaraScript* Script)const; 
	NIAGARA_API void ForceGraphToRecompileOnNextCheck();
	NIAGARA_API void SetCompileForEdit(bool bNewCompileForEdit);
	inline bool GetCompileForEdit() const { return bCompileForEdit; }

	static NIAGARA_API void RequestCompileForEmitter(const FVersionedNiagaraEmitter& InEmitter);
	static NIAGARA_API void RecomputeExecutionOrderForEmitter(const FVersionedNiagaraEmitter& InEmitter);
	static NIAGARA_API void RecomputeExecutionOrderForDataInterface(class UNiagaraDataInterface* DataInterface);

	inline bool ShouldUseRapidIterationParameters() const { return bCompileForEdit ? !bBakeOutRapidIteration : !bBakeOutRapidIterationOnCook; }
	inline bool ShouldTrimAttributes() const { return bCompileForEdit ? bTrimAttributes : bTrimAttributesOnCook; }
	inline bool ShouldIgnoreParticleReadsForAttributeTrim() const { return bIgnoreParticleReadsForAttributeTrim; }
	inline bool ShouldDisableDebugSwitches() const { return bCompileForEdit ? bDisableDebugSwitches : bDisableDebugSwitchesOnCook; }
	inline bool ShouldCompressAttributes() const { return bCompressAttributes; }

	inline void SetBakeOutRapidIterationOnCook(bool bBakeOut) { bBakeOutRapidIteration = bBakeOut; bBakeOutRapidIterationOnCook = bBakeOut; }
	inline void SetTrimAttributesOnCook(bool bTrim) { bTrimAttributes = bTrim; bTrimAttributesOnCook = bTrim; }

	/** Returns information on the availability of performance stat scopes */
	NIAGARA_API ENiagaraPerformanceStateScopeMode GetPerformanceStateScopeMode() const;

	/** Do we have any pending compilation requests or not */
	bool NeedsRequestCompile() const { return RequestCompileStatus != ERequestCompileStatus::None; }

protected:
	/** When enable constant values are baked into the scripts while editing the system, this will increase iteration times but improve performance. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Performance", meta=(DisplayName="Bake Rapid Iteration Parameters During Edit"))
	uint32 bBakeOutRapidIteration : 1;

	/** When enabled constant values are baked into scripts to improve performance. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Performance", meta=(DisplayName="Bake Rapid Iteration Parameters"))
	uint32 bBakeOutRapidIterationOnCook : 1;

	/** Toggles whether or not emitters within this system will try and compress their particle attributes.
	In some cases, this precision change can lead to perceivable differences, but memory costs and or performance (especially true for GPU emitters) can improve. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Performance")
	uint32 bCompressAttributes : 1;

	/** When enabled we trim particle attributes while editing the system. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Performance", meta=(DisplayName="Trim Attributes During Edit"))
	uint32 bTrimAttributes : 1;
	/** If true Particle attributes will be removed from the DataSet if they are unnecessary (are never read by ParameterMap) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Performance", meta = (DisplayName="Trim Attributes"))
	uint32 bTrimAttributesOnCook : 1;

	/** If true ParticleReads will not absolutely prevent attribute trimming - User must ensure that the appropriate attributes are preserved on the source emitter! */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Performance")
	uint32 bIgnoreParticleReadsForAttributeTrim : 1;

	/** When enable debug switches are disabled while editing the system. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Performance", meta = (DisplayName="Disable Debug Switches During Edit"))
	uint32 bDisableDebugSwitches : 1;
	/** When enabled debug switches are disabled when compiling the system. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Performance", meta = (DisplayName="Disable Debug Switches"))
	uint32 bDisableDebugSwitchesOnCook : 1;

	/** When enabled we compile for the edit path, which can result in slower system performance but faster editor responsiveness. */
	uint32 bCompileForEdit : 1;

	/** Used to track the status of compilation request, when using on demand they will initially be in RequestPendingOnDemand. */
	enum class ERequestCompileStatus
	{
		None,
		RequestPending,
		RequestPendingOnDemand,
		RequestPendingForcedRetry,
	};
	ERequestCompileStatus RequestCompileStatus = ERequestCompileStatus::None;

public:
	/** Subscriptions to definitions of parameters. */
	UPROPERTY()
	TArray<FParameterDefinitionsSubscription> ParameterDefinitionsSubscriptions;
#endif

	/** If true then position type values will be rebased on system activation to fit into a float precision vector. This needs to be turned off when using a custom data interface or renderer that does not support the rebasing. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Rendering")
	uint8 bSupportLargeWorldCoordinates : 1;

	//////////////////////////////////////////////////////////////////////////
	// Various optional overrides for component properties when spawning a system
	UPROPERTY(EditAnywhere, Category="Rendering", meta=(InlineEditConditionToggle="bCastShadow"))
	uint8 bOverrideCastShadow : 1;

	UPROPERTY(EditAnywhere, Category="Rendering", meta=(InlineEditConditionToggle="bReceivesDecals"))
	uint8 bOverrideReceivesDecals : 1;

	UPROPERTY(EditAnywhere, Category="Rendering", meta=(InlineEditConditionToggle="bRenderCustomDepth"))
	uint8 bOverrideRenderCustomDepth : 1;

	UPROPERTY(EditAnywhere, Category="Rendering", meta=(InlineEditConditionToggle="CustomDepthStencilValue"))
	uint8 bOverrideCustomDepthStencilValue : 1;

	UPROPERTY(EditAnywhere, Category="Rendering", meta=(InlineEditConditionToggle="CustomDepthStencilWriteMask"))
	uint8 bOverrideCustomDepthStencilWriteMask : 1;

	UPROPERTY(EditAnywhere, Category="Rendering", meta=(InlineEditConditionToggle="TranslucencySortPriority"))
	uint8 bOverrideTranslucencySortPriority : 1;

	UPROPERTY(EditAnywhere, Category="Rendering", meta=(InlineEditConditionToggle="TranslucencySortDistanceOffset"))
	uint8 bOverrideTranslucencySortDistanceOffset : 1;

	/**
	When enabled this is the default value set on the component.
	Controls whether the primitive component should cast a shadow or not.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Rendering", meta=(DisplayName="Default Cast Shadows", EditCondition="bOverrideCastShadow"))
	uint8 bCastShadow : 1;

	/**
	When enabled this is the default value set on the component.
	Whether the primitive receives decals.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Rendering", meta=(DisplayName="Default Receives Decals", EditCondition="bOverrideReceivesDecals"))
	uint8 bReceivesDecals : 1;

	/**
	When enabled this is the default value set on the component.
	This primitive has bRenderCustomDepth enabled.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Rendering", meta=(DisplayName="Default Render CustomDepth Pass", EditCondition="bOverrideRenderCustomDepth"))
	uint8 bRenderCustomDepth : 1;

private:
	/*
	When enabled we use the owner actor's velocity for the first frame.
	If we do not have an owner actor, or this is disable then the first frame's velocity will be zero.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "System")
	uint8 bInitialOwnerVelocityFromActor : 1 = true;

	UPROPERTY(EditAnywhere, Category = Rendering, AdvancedDisplay, meta = (EditCondition = "bSupportLargeWorldCoordinates", EditConditionHides, DisplayAfter=bSupportLargeWorldCoordinates))
	TOptional<ENiagaraLwcTileUpdateMode> LargeWorldCoordinateTileUpdateMode;

public:
	/**
	When enabled this is the default value set on the component.
	Mask used for stencil buffer writes.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category="Rendering", meta=(DisplayName="Default CustomDepthStencil Write Mask", editcondition="bOverrideCustomDepthStencilValue"))
	ERendererStencilMask CustomDepthStencilWriteMask;

	/**
	When enabled this is the default value set on the component.
	Optionally write this 0-255 value to the stencil buffer in CustomDepth pass (Requires project setting or r.CustomDepth == 3)
	*/

	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category="Rendering", meta=(DisplayName="Default CustomDepthStencil Value", editcondition="bOverrideCustomDepthStencilWriteMask", UIMin = "0", UIMax = "255"))
	int32 CustomDepthStencilValue = 0;

	/**
	When enabled this is the default value set on the component.
	Adjusts the translucent object sorting priority, see PrimitiveComponent description for more details.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category="Rendering", meta=(editcondition="bOverrideTranslucencySortPriority"))
	int32 TranslucencySortPriority = 0;

	/**
	When enabled this is the default value set on the component.
	Modifies the sort distance for translucent objects, see PrimitiveComponent description for more details.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category="Rendering", meta=(editcondition="bOverrideTranslucencySortDistanceOffset"))
	float TranslucencySortDistanceOffset = 0.0f;

	//UPROPERTY()
	//TArray<TArray<uint8>> CachedDataInterfaceEmitterDependencies;

	/** Computes emitter priorities based on the dependency information. */
	NIAGARA_API bool ComputeEmitterPriority(int32 EmitterIdx, TArray<int32, TInlineAllocator<32>>& EmitterPriorities, const TBitArray<TInlineAllocator<32>>& EmitterDependencyGraph);

	/** Queries all the data interfaces in the array for emitter dependencies. */
	NIAGARA_API void FindDataInterfaceDependencies(const FGuid& EmitterHandleId, const UNiagaraScript* Script, TArray<FVersionedNiagaraEmitter>& Dependencies);

	/** Looks at all the event handlers in the emitter to determine which other emitters it depends on. */
	NIAGARA_API void FindEventDependencies(FVersionedNiagaraEmitterData* EmitterData, TArray<FVersionedNiagaraEmitter>& Dependencies);

	/** Computes the order in which the emitters in the Emitters array will be ticked and stores the results in EmitterExecutionOrder. */
	NIAGARA_API void ComputeEmittersExecutionOrder();

	/** Computes the order in which renderers will render */
	NIAGARA_API void ComputeRenderersDrawOrder();

	/** Cache data & accessors from the compiled data, allows us to avoid per instance. */
	NIAGARA_API void CacheFromCompiledData();

	inline TConstArrayView<FNiagaraEmitterExecutionIndex> GetEmitterExecutionOrder() const { return MakeArrayView(EmitterExecutionOrder); }
	inline TConstArrayView<FNiagaraRendererExecutionIndex> GetRendererPostTickOrder() const { return MakeArrayView(RendererPostTickOrder); }
	inline TConstArrayView<FNiagaraRendererExecutionIndex> GetRendererCompletionOrder() const { return MakeArrayView(RendererCompletionOrder); }

	inline TConstArrayView<int32> GetRendererDrawOrder() const { return MakeArrayView(RendererDrawOrder); }

	/** When an index inside the EmitterExecutionOrder array has this bit set, it means the corresponding emitter cannot execute in parallel with the previous emitters due to a data dependency. */
	static constexpr int32 kStartNewOverlapGroupBit = (1 << 31);

	inline UNiagaraParameterCollectionInstance* GetParameterCollectionOverride(UNiagaraParameterCollection* Collection)
	{
		auto* Found = ParameterCollectionOverrides.FindByPredicate(
			[&](const UNiagaraParameterCollectionInstance* CheckInst)
		{
			return CheckInst && Collection == CheckInst->Collection;
		});

		return Found ? *Found : nullptr;
	}
	
	UPROPERTY(EditAnywhere, Category = "Debug", Transient, AdvancedDisplay)
	uint8 bDumpDebugSystemInfo: 1 = false;

	UPROPERTY(EditAnywhere, Category = "Debug", Transient, AdvancedDisplay)
	uint8 bDumpDebugEmitterInfo: 1  = false;

	uint8 bFullyLoaded: 1  = false;

	/** When enabled, we follow the settings on the UNiagaraComponent for tick order. When this option is disabled, we ignore any dependencies from data interfaces or other variables and instead fire off the simulation as early in the frame as possible. This greatly
	reduces overhead and allows the game thread to run faster, but comes at a tradeoff if the dependencies might leave gaps or other visual artifacts.*/
	UPROPERTY(EditAnywhere, Category = "Performance", AdvancedDisplay)
	uint8 bRequireCurrentFrameData: 1  = true;

protected:
	UPROPERTY(EditAnywhere, Category = "Scalability", meta=(DisplayInScalabilityContext))
	uint8 bOverrideScalabilitySettings : 1;
public:
	
	inline bool HasDIsWithPostSimulateTick() const { return bHasDIsWithPostSimulateTick; }
	inline bool AllDIsPostSimulateCanOverlapFrames() const { return bAllDIsPostSimulateCanOverlapFrames; }
	inline bool AsyncWorkCanOverlapTickGroups() const { return bAllDIsPostStageCanOverlapTickGroups; }
	inline bool HasAnyGPUEmitters()const{ return bHasAnyGPUEmitters; }
	inline bool NeedsGPUContextInitForDataInterfaces() const { return bNeedsGPUContextInitForDataInterfaces; }

	NIAGARA_API FBox GetFixedBounds() const;
	inline void SetFixedBounds(const FBox& Box) { FixedBounds = Box;  }

	FBox GetInitialStreamingBounds() const { return bUseInitialStreamingBounds ? InitialStreamingBounds : FBox(); }

#if WITH_EDITOR
	NIAGARA_API void SetEffectType(UNiagaraEffectType* EffectType);

	FNiagaraSystemScalabilityOverrides& GetSystemScalabilityOverrides() { return SystemScalabilityOverrides; }
	inline bool GetOverrideScalabilitySettings()const { return bOverrideScalabilitySettings; }
	inline void SetOverrideScalabilitySettings(bool bOverride) { bOverrideScalabilitySettings = bOverride; }

	NIAGARA_API void GatherStaticVariables(TArray<FNiagaraVariable>& OutVars, TArray<FNiagaraVariable>& OutEmitterVars) const;
#endif
	NIAGARA_API UNiagaraEffectType* GetEffectType()const;
	inline const FNiagaraSystemScalabilitySettings& GetScalabilitySettings()const { return CurrentScalabilitySettings; }
	NIAGARA_API const FNiagaraSystemScalabilityOverride& GetCurrentOverrideSettings() const;
	inline bool NeedsSortedSignificanceCull()const{ return bNeedsSortedSignificanceCull; }

	const FNiagaraReadinessSettings& GetReadinessSettings() const;
	ENiagaraEmitterImportance GetHighestEmitterImportance() const { return MaxEmitterImportance; }

	inline void GetMaxInstanceCounts(int32& OutSystemInstanceMax, int32& OutFXTypeInstanceMax, bool bBudgetAdjusted)const;
	
	FNiagaraPlatformSet& GetScalabilityPlatformSet() { return Platforms; }
	const FNiagaraPlatformSet& GetScalabilityPlatformSet() const { return Platforms; }

	/** Returns true if this emitter's platform filter allows it on this platform and quality level. */
	NIAGARA_API bool IsAllowedByScalability() const;

	NIAGARA_API void UpdateScalability();

	NIAGARA_API ENiagaraCullProxyMode GetCullProxyMode()const;

	/** Whether or not fixed bounds are enabled. */
	UPROPERTY(EditAnywhere, Category = "System", meta = (SkipSystemResetOnChange = "true", InlineEditConditionToggle))
	uint32 bFixedBounds : 1;

	NIAGARA_API TStatId GetStatID(bool bGameThread, bool bConcurrent)const;
	NIAGARA_API void AddToInstanceCountStat(int32 NumInstances, bool bSolo)const;

	NIAGARA_API const FString& GetCrashReporterTag()const;
	NIAGARA_API bool CanObtainEmitterAttribute(const FNiagaraVariableBase& InVarWithUniqueNameNamespace, FNiagaraTypeDefinition& OutBoundType) const;
	NIAGARA_API bool CanObtainSystemAttribute(const FNiagaraVariableBase& InVar, FNiagaraTypeDefinition& OutBoundType) const;
	NIAGARA_API bool CanObtainUserVariable(const FNiagaraVariableBase& InVar) const;

#if WITH_EDITORONLY_DATA
	FNiagaraMessageStore& GetMessageStore() { return MessageStore; }
	const FGuid& GetAssetGuid() const {return AssetGuid;};
#endif

	inline void RegisterActiveInstance();
	inline void UnregisterActiveInstance();
	inline int32& GetActiveInstancesCount() { return ActiveInstances; }

#if WITH_EDITORONLY_DATA
	NIAGARA_API UNiagaraBakerSettings* GetBakerSettings();
	const UNiagaraBakerSettings* GetBakerGeneratedSettings() const { return BakerGeneratedSettings; }
	NIAGARA_API void SetBakerGeneratedSettings(UNiagaraBakerSettings* Settings);

	/** Get the cached parameter map traversal for this emitter.  */
	NIAGARA_API const TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe>& GetCachedTraversalData() const;
	NIAGARA_API void InvalidateCachedData();
	NIAGARA_API void GraphSourceChanged();

	/** Resets internal data leaving it in a state which would have minimal cost to exist in headless builds (servers) */
	NIAGARA_API void ResetToEmptySystem();

	/** Updates any post compile data based upon data interfaces. */
	NIAGARA_API void OnCompiledDataInterfaceChanged();

	/** Updates the system post UObject change. */
	NIAGARA_API void OnCompiledUObjectChanged();

	/** Updates the rapid iteration parameters for all scripts referenced by the system. */
	NIAGARA_API void PrepareRapidIterationParametersForCompilation();

	bool AllowValidation() const { return bAllowValidation; }
#endif

	/** Access the code system state data. */
	NIAGARA_API const FNiagaraSystemStateData& GetSystemStateData() const;

	/** Used for debug HUD / viewport to convey what mode we are running in */
	NIAGARA_API const TCHAR* GetSystemStateModeString() const;

private:
#if WITH_EDITORONLY_DATA


	/** Since the shader compilation is done in another process, this is used to check if the result for any ongoing compilations is done.
	*   If bWait is true then this *blocks* the game thread (and ui) until all shader compilations are finished.
	*	Results from the compilation will be applied to the system unless there's already another compilation queued up
	*/
	NIAGARA_API bool QueryCompileComplete(bool bWait);

	NIAGARA_API void BroadcastOnSystemCompiled();

	NIAGARA_API void EvaluateCompileResultDependencies() const;

	NIAGARA_API void InitEmitterCompiledData();

	NIAGARA_API void InitSystemCompiledData();

	/** Helper for filling in precomputed variable names per emitter. Converts an emitter parameter "Emitter.XXXX" into it's real parameter name. */
	NIAGARA_API void InitEmitterVariableAliasNames(FNiagaraEmitterCompiledData& EmitterCompiledDataToInit, const UNiagaraEmitter* InAssociatedEmitter);

	/** Helper for generating aliased FNiagaraVariable names for the Emitter they are associated with. */
	NIAGARA_API const FName GetEmitterVariableAliasName(const FNiagaraVariableBase& InEmitterVar, const UNiagaraEmitter* InEmitter) const;

	/** Helper for filling in attribute datasets per emitter. */
	NIAGARA_API void InitEmitterDataSetCompiledData(FNiagaraDataSetCompiledData& DataSetToInit, const FNiagaraEmitterHandle& InAssociatedEmitterHandle);

	NIAGARA_API void InitScriptCompiledData();

	void InvalidateScriptRuntimeData();

	void PreSaveScriptRuntimeData(FObjectPreSaveContext ObjectSaveContext);
#endif


	void ResolveRequiresScripts();
	NIAGARA_API void ResolveScalabilitySettings();
	NIAGARA_API void UpdatePostCompileDIInfo();
	NIAGARA_API void UpdateDITickFlags();
	NIAGARA_API void UpdateHasGPUEmitters();

	void PostLoadScriptRuntimeData();
	
protected:
	/** An effect types defines settings shared between systems, for example scalability and validation rules.
	 * Things like environment fx usually have a different effect type than gameplay relevant fx such as weapon impacts.
	 * This way whole classes of effects can be adjusted at once. */
	UPROPERTY(EditAnywhere, Category = "System")
	TObjectPtr<UNiagaraEffectType> EffectType;

	/** Controls whether we should override the Effect Type value for bAllowCullingForLocalPlayers. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (InlineEditConditionToggle, EditCondition = bOverrideScalabilitySettings))
	uint32 bOverrideAllowCullingForLocalPlayers : 1;
	
	/** The override value for bAllowCullingForLocalPlayers from the Effect Type. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition = bOverrideAllowCullingForLocalPlayers))
	uint32 bAllowCullingForLocalPlayersOverride : 1;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition="bOverrideScalabilitySettings", DisplayInScalabilityContext))
	FNiagaraSystemScalabilityOverrides SystemScalabilityOverrides;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayInScalabilityContext))
	FNiagaraPlatformSet Platforms;

	/** Handles to the emitter this System will simulate. */
	UPROPERTY()
	TArray<FNiagaraEmitterHandle> EmitterHandles;

	UPROPERTY(EditAnywhere, Category="System", AdvancedDisplay)
	TArray<TObjectPtr<UNiagaraParameterCollectionInstance>> ParameterCollectionOverrides;

#if WITH_EDITORONLY_DATA
	bool bCompilationReentrantGuard = false;

	UPROPERTY()
	TArray<FNiagaraSystemScalabilityOverride> ScalabilityOverrides_DEPRECATED;

	TArray<TUniquePtr<FNiagaraActiveCompilation>> ActiveCompilations;

	mutable TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalData;
#endif

	/** The script which defines the System parameters, and which generates the bindings from System
		parameter to emitter parameter. */
	UPROPERTY()
	TObjectPtr<UNiagaraScript> SystemSpawnScript;

	/** The script which defines the System parameters, and which generates the bindings from System
	parameter to emitter parameter. */
	UPROPERTY()
	TObjectPtr<UNiagaraScript> SystemUpdateScript;

	//** Post compile generated data used for initializing Emitter Instances during runtime. */
	TArray<TSharedRef<const FNiagaraEmitterCompiledData>> EmitterCompiledData;

	//** Post compile generated data used for initializing System Instances during runtime. */
	UPROPERTY()
	FNiagaraSystemCompiledData SystemCompiledData;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UNiagaraScriptRuntimeCompiledDataEditorReference> ScriptRuntimeCompiledDataForEditor;

	UPROPERTY(transient)
	mutable TArray<TObjectPtr<UNiagaraScriptRuntimeCompiledDataEditorReference>> StaleRuntimeCompiledDataForEditor;

	mutable TMap<FObjectKey, TArray<FNiagaraScriptDataKey>> StaleRuntimeCompiledDataUsageMap;

#endif
	mutable TMap<FNiagaraScriptDataKey, TSharedPtr<FNiagaraScriptRuntimeData>> ScriptRuntimeDataMap;

	UPROPERTY()
	TMap<FNiagaraScriptDataKey, FNiagaraScriptRuntimeCookedData> ScriptRuntimeCookedDataMap;

	/** Variables exposed to the outside work for tweaking*/
	UPROPERTY()
	FNiagaraUserRedirectionParameterStore ExposedParameters;

#if WITH_EDITORONLY_DATA
	/** Data used by the editor to maintain UI state etc.. */
	UPROPERTY()
	TObjectPtr<UNiagaraEditorDataBase> EditorData;

	/** Wrapper for editor only parameters. */
	UPROPERTY()
	TObjectPtr<UNiagaraEditorParametersAdapterBase> EditorParameters;

	bool bIsolateEnabled;

	/** A multicast delegate which is called whenever the script has been compiled (successfully or not). */
	FOnSystemCompiled OnSystemCompiledDelegate;

	/** A multicast delegate which is called whenever this system's properties are changed. */
	FOnSystemPostEditChange OnSystemPostEditChangeDelegate;

	/** A multicast delegate that is called whenever the effect type or the effect type values are changed */
	FOnScalabilityChanged OnScalabilityChangedDelegate;

	// If true then we need to manually update the asset registry after the system is fully loaded
	mutable bool bUpdateTagsAfterLoad = false;
#endif

	/** The fixed bounding box value for the whole system. When placed in the level and the bounding box is not visible to the camera, the effect is culled from rendering. */
	UPROPERTY(EditAnywhere, Category = "System", meta = (SkipSystemResetOnChange = "true", EditCondition = "bFixedBounds"))
	FBox FixedBounds;

	UPROPERTY(EditAnywhere, Category = "System", meta = (InlineEditConditionToggle, SkipSystemResetOnChange = "true"))
	bool bUseInitialStreamingBounds = false;

	/**
	Initial streaming bounds are only included when the streaming system asks for the initial bounds,
	it is the union of the currently calculated generated bounds and these bounds.
	Any future updates to the bounds in the streamer will use the component bounds only and not these bounds.
	Setting initial streaming bounds can help in situations where textures do not stream in correctly as the
	heuristic to updating the bounds inside the streaming system does not update them at all, or frequently enough.
	This is generally an issue with dynamic bounds, if your system has fixed bounds it is unlikely you will need to use this.
	*/
	UPROPERTY(EditAnywhere, Category = "System", meta = (SkipSystemResetOnChange = "true", EditCondition = "bUseInitialStreamingBounds"))
	FBox InitialStreamingBounds = FBox(FVector(-100), FVector(100));

	UPROPERTY()
	bool bNeedsGPUContextInitForDataInterfaces;

	/**
	When disabled we will generate a RandomSeed per instance on reset which is not deterministic.
	When enabled we will always use the RandomSeed from the system plus the components RandomSeedOffset, this allows for determinism but variance between components.
	*/
	UPROPERTY(EditAnywhere, Category = "System")
	bool bDeterminism = false;
	
	UPROPERTY(EditAnywhere, Category = "System", meta = (InlineEditConditionToggle, DisplayAfter="WarmupTickDelta"))
	bool bFixedTickDelta = false;

	/** Seed used for system script random number generator. */
	UPROPERTY(EditAnywhere, Category = "System", meta = (EditCondition = "bDeterminism", EditConditionHides))
	int32 RandomSeed = 0;

	/** Warm up time in seconds. Used to calculate WarmupTickCount. Rounds down to the nearest multiple of WarmupTickDelta. */
	UPROPERTY(EditAnywhere, Category = "System", meta = (ForceUnits=s))
	float WarmupTime = 0.0f;

	/** Number of ticks to process for warmup. You can set by this or by time via WarmupTime. */
	UPROPERTY(EditAnywhere, Category = "System", meta = (EditCondition = "WarmupTime > 0.0", EditConditionHides))
	int32 WarmupTickCount = 0;

	/** Delta time to use for warmup ticks. */
	UPROPERTY(EditAnywhere, Category = "System", meta = (ForceUnits=s, EditCondition = "WarmupTime > 0.0", EditConditionHides))
	float WarmupTickDelta = 1.0f / 15.0f;

	/**
	If activated, the system ticks with a fixed delta time instead of the varying game thread delta time. This leads to much more stable simulations.
	When the fixed tick delta is smaller than the game thread tick time the simulation is substepping by executing multiple ticks per frame.
	Note that activating this feature forces the system to tick on the game thread instead of an async task in parallel.

	The max number of substeps per frame can be set via fx.Niagara.SystemSimulation.MaxTickSubsteps
	*/
	UPROPERTY(EditAnywhere, Category = "System", meta = (EditCondition = "bFixedTickDelta", ForceUnits=s))
	float FixedTickDeltaTime = 0.01667f;

#if WITH_EDITORONLY_DATA
	/** Settings used inside the baker */
	UPROPERTY(Export)
	TObjectPtr<UNiagaraBakerSettings> BakerSettings;

	/** Generated data baker settings, will be null until we have generated at least once. */
	UPROPERTY(Export)
	TObjectPtr<UNiagaraBakerSettings> BakerGeneratedSettings;
#endif

	UPROPERTY()
	ENiagaraEmitterImportance MaxEmitterImportance = ENiagaraEmitterImportance::Low;
	
	/** Array of emitter indices sorted by execution priority. The emitters will be ticked in this order. Please note that some indices may have the top bit set (kStartNewOverlapGroupBit)
	* to indicate synchronization points in parallel execution, so mask it out before using the values as indices in the emitters array.
	*/
	TArray<FNiagaraEmitterExecutionIndex> EmitterExecutionOrder;

	/** Array of renderer indices to notify system PostTick, in order of execution */
	TArray<FNiagaraRendererExecutionIndex> RendererPostTickOrder;
	/** Array of renderer indices to notify system Completion, in order of execution */
	TArray<FNiagaraRendererExecutionIndex> RendererCompletionOrder;

	/** Precomputed emitter renderer draw order, since emitters & renderers are not dynamic we can do this. */
	TArray<int32> RendererDrawOrder;

	uint32 bIsValidCached : 1;
	uint32 bIsReadyToRunCached : 1;
	uint32 bNeedsAsyncOptimize : 1;
	uint32 bLwcEnabledSettingCached : 1;

	TOptional<float> MaxDeltaTime;
	FNiagaraDataSetAccessor<ENiagaraExecutionState> SystemExecutionStateAccessor;
	TArray<FNiagaraDataSetAccessor<ENiagaraExecutionState>> EmitterExecutionStateAccessors;
	TArray<TArray<FNiagaraDataSetAccessor<FNiagaraSpawnInfo>>> EmitterSpawnInfoAccessors;

	FGraphEventRef PSOPrecacheCompletionEvent;

public:
	bool AllowCullingForLocalPlayers() const { return bAllowCullingForLocalPlayers; }

	const FNiagaraSystemStaticBuffers* GetStaticBuffers() const { return StaticBuffers.Get(); }

protected:
	void UpdateStatID() const;
	void GenerateStatID() const;
#if STATS
	mutable TStatId StatID_GT;
	mutable TStatId StatID_GT_CNC;
	mutable TStatId StatID_RT;
	mutable TStatId StatID_RT_CNC;

	mutable TStatId StatID_InstanceCount;
	mutable TStatId StatID_InstanceCountSolo;
	FNiagaraStatDatabase StatDatabase;
#endif

	//Scalability settings
	FNiagaraSystemScalabilitySettings& CurrentScalabilitySettings;

	mutable FString CrashReporterTag;

	uint32 bAllowCullingForLocalPlayers : 1 = false;
	uint32 bHasDIsWithPostSimulateTick : 1;
	uint32 bAllDIsPostSimulateCanOverlapFrames : 1;
	uint32 bAllDIsPostStageCanOverlapTickGroups : 1;
	uint32 bHasAnyGPUEmitters : 1;
	uint32 bNeedsSortedSignificanceCull : 1;

	/** When enabled if all emitters don't require script execution and the system script is empty / constant we can invoke a faster CPU path. */
	UPROPERTY(EditAnywhere, Category = "Performance", AdvancedDisplay)
	uint32 bAllowSystemStateFastPath : 1 = true;

	UPROPERTY()
	uint32 bSystemStateFastPathEnabled : 1 = true;
	
#if WITH_EDITORONLY_DATA
	/* In some cases we can not run validation as the data has been stripped.  This is a temporary workaround to skip validation in those cases. */
	uint32 bAllowValidation : 1 = true;

	/** Messages associated with the System asset. */
	UPROPERTY()
	TMap<FGuid, TObjectPtr<UNiagaraMessageDataBase>> MessageKeyToMessageMap_DEPRECATED;

	UPROPERTY()
	FNiagaraMessageStore MessageStore;

	FGuid AssetGuid;
#endif

	/** Total active instances of this system. */
	int32 ActiveInstances;

	struct FStaticBuffersCachingFlags
	{
		bool bIsExternalDataInterface = false;
		bool bIsUsedByCpu = false;
		bool bIsUsedByGpu = false;
	};

	struct FStaticBuffersDeletor { void operator()(FNiagaraSystemStaticBuffers* Ptr) const; };
	TUniquePtr<FNiagaraSystemStaticBuffers, FStaticBuffersDeletor> StaticBuffers;

	UPROPERTY()
	FInstancedStruct SystemStateDataStruct;
	
	FRenderCommandFence WaitRenderCommandsFence;
};

inline void UNiagaraSystem::RegisterActiveInstance()
{
	++ActiveInstances;
}

inline void UNiagaraSystem::UnregisterActiveInstance()
{
	--ActiveInstances;
}


inline void UNiagaraSystem::GetMaxInstanceCounts(int32& OutSystemInstanceMax, int32& OutFXTypeInstanceMax, bool bBudgetAdjusted)const 
{
	OutSystemInstanceMax = CurrentScalabilitySettings.MaxSystemInstances;
	OutFXTypeInstanceMax = CurrentScalabilitySettings.MaxInstances;

	if (bBudgetAdjusted && (CurrentScalabilitySettings.BudgetScaling.bScaleMaxInstanceCountByGlobalBudgetUse || CurrentScalabilitySettings.BudgetScaling.bScaleSystemInstanceCountByGlobalBudgetUse))
	{
		float Usage = FFXBudget::GetWorstAdjustedUsage();

		if (CurrentScalabilitySettings.bCullMaxInstanceCount && CurrentScalabilitySettings.BudgetScaling.bScaleMaxInstanceCountByGlobalBudgetUse)
		{
			const float Scale = CurrentScalabilitySettings.BudgetScaling.MaxInstanceCountScaleByGlobalBudgetUse.Evaluate(Usage);
			OutFXTypeInstanceMax = int32(float(OutFXTypeInstanceMax) * Scale);
		}
		if (CurrentScalabilitySettings.bCullPerSystemMaxInstanceCount && CurrentScalabilitySettings.BudgetScaling.bScaleSystemInstanceCountByGlobalBudgetUse)
		{
			const float Scale = CurrentScalabilitySettings.BudgetScaling.MaxSystemInstanceCountScaleByGlobalBudgetUse.Evaluate(Usage);
			OutSystemInstanceMax = int32(float(OutSystemInstanceMax) * Scale);
		}
	}
}
