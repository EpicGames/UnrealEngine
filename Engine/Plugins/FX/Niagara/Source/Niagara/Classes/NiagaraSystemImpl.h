// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfacePlatformSet.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterImpl.h"
#include "NiagaraScript.h"
#include "NiagaraSystem.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceUtilities.h"


template<typename TAction>
void UNiagaraSystem::ForEachScript(TAction Func) const
{
	Func(SystemSpawnScript);
	Func(SystemUpdateScript);

	for (const FNiagaraEmitterHandle& Handle : EmitterHandles)
	{
		if (Handle.GetIsEnabled())
		{
			if (FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData())
			{
				EmitterData->ForEachScript(Func);
			}
		}
	}
}

template<typename TAction>
void UNiagaraSystem::ForEachScriptWithOwningContext(TAction Func) const
{
	Func(this, nullptr, SystemSpawnScript);
	Func(this, nullptr, SystemUpdateScript);

	for (const FNiagaraEmitterHandle& Handle : EmitterHandles)
	{
		if (Handle.GetIsEnabled())
		{
			if (FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData())
			{
				EmitterData->ForEachScript([this, &Handle, &Func](UNiagaraScript* Script) { Func(this, &Handle, Script); });
			}
		}
	}
}

template<typename TAction>
void UNiagaraSystem::ForEachScriptWithRuntimeData(TAction Func) const
{
	auto RuntimeDataFunc = [&Func](const UNiagaraSystem* OwningSystem, const FNiagaraEmitterHandle* OwningEmitterHandle, UNiagaraScript* Script)
	{
		if (Script == nullptr)
		{
			return;
		}

		FGuid EmitterHandleId = OwningEmitterHandle != nullptr ? OwningEmitterHandle->GetId() : FGuid();
		const FNiagaraScriptRuntimeData* ScriptRuntimeData = OwningSystem->GetScriptRuntimeData(OwningEmitterHandle, *Script).Get();
		if (ScriptRuntimeData != nullptr)
		{ 
			Func(Script, ScriptRuntimeData);
		}
	};

	ForEachScriptWithOwningContext(RuntimeDataFunc);
}

/** Performs the passed action for all FNiagaraPlatformSets in this system. */
template<typename TAction>
void UNiagaraSystem::ForEachPlatformSet(TAction Func)
{
	//Handle our scalability overrides
	for (FNiagaraSystemScalabilityOverride& Override : SystemScalabilityOverrides.Overrides)
	{
		Func(Override.Platforms);
	}

	//Handle and platform set User DIs.
	for (UNiagaraDataInterface* DI : GetExposedParameters().GetDataInterfaces())
	{
		if (UNiagaraDataInterfacePlatformSet* PlatformSetDI = Cast<UNiagaraDataInterfacePlatformSet>(DI))
		{
			Func(PlatformSetDI->Platforms);
		}
	}

	//Handle all platform set DIs held in scripts for this system.
	auto HandleScript = [this,Func](UNiagaraScript* NiagaraScript)
	{
		if (NiagaraScript)
		{
			const FNiagaraScriptRuntimeData* ScriptRuntimeData = GetScriptRuntimeData(nullptr, *NiagaraScript).Get();
			if (ScriptRuntimeData != nullptr)
			{
				for (const FNiagaraScriptResolvedDataInterfaceInfo& DataInterfaceInfo : ScriptRuntimeData->GetResolvedDataInterfaces())
				{
					if (UNiagaraDataInterfacePlatformSet* PlatformSetDI = Cast<UNiagaraDataInterfacePlatformSet>(DataInterfaceInfo.ResolvedDataInterface))
					{
						Func(PlatformSetDI->Platforms);
					}
				}
			}
		}
	};
	HandleScript(SystemSpawnScript);
	HandleScript(SystemUpdateScript);

	//Finally handle all our emitters.
	for (FNiagaraEmitterHandle& Handle : EmitterHandles)
	{
		if (FVersionedNiagaraEmitterData* Emitter = Handle.GetEmitterData())
		{
			Emitter->ForEachPlatformSet(*this, Handle.GetId(), Func);
		}
	}
}
