// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef UE_USE_OPENVDB
	#define UE_USE_OPENVDB 0
#endif
