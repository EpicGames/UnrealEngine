// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "NiagaraValidationRule.generated.h"

class FNiagaraSystemViewModel;
class UNiagaraStackEntry;

UENUM()
enum class ENiagaraValidationSeverity
{
	/** Just an info message for the user. */
	Info,
	/** Could be a potential problem, for example bad performance. */
	Warning,
	/** A problem that must be fixed for the content to be valid. */
	Error,
};

DECLARE_DELEGATE(FNiagaraValidationFixDelegate);

/** Delegate wrapper to automatically fix content that fails validation checks. */
struct FNiagaraValidationFix
{
	FNiagaraValidationFix() {}
	FNiagaraValidationFix(FText InDescription, FNiagaraValidationFixDelegate InFixDelegate)
	: Description(InDescription)
	, FixDelegate(InFixDelegate)
	{
	}
	
	FText Description;
	FNiagaraValidationFixDelegate FixDelegate;
};

struct FNiagaraValidationResult
{
	FNiagaraValidationResult() {}
	FNiagaraValidationResult(ENiagaraValidationSeverity InSeverity, FText InSummaryText, FText InDescription, TWeakObjectPtr<UObject> InSourceObject)
		: Severity(InSeverity), SummaryText(InSummaryText), Description(InDescription), SourceObject(InSourceObject) {}

	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Info;

	FText SummaryText;
	FText Description;
	TWeakObjectPtr<UObject> SourceObject;
	TArray<FNiagaraValidationFix> Fixes;
	TArray<FNiagaraValidationFix> Links;
};

struct FNiagaraValidationContext
{
	// the view model for the whole Niagara system
	TSharedPtr<FNiagaraSystemViewModel> ViewModel;

	// if set then this stack entry requested the validation
	UNiagaraStackEntry* Source = nullptr;
};

/**
Base class for system validation logic. 
These allow Niagara systems to be inspected for content validation either at save time or from a commandlet.
*/
UCLASS(abstract, EditInlineNew, MinimalAPI, Config=Editor)
class UNiagaraValidationRule : public UObject
{
	GENERATED_BODY()

public:
	bool IsEnabled() const { return bIsConfigDisabled == false && GetClass()->HasAnyClassFlags(CLASS_Deprecated) == false; }
#if WITH_EDITOR
	NIAGARA_API virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const;
#endif

protected:
	// Allows disabling validation rules from config
	UPROPERTY(Config)
	bool bIsConfigDisabled = false;
};
