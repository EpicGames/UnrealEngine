// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraValidationRule.h"
#include "NiagaraValidationRuleSet.generated.h"

/** A set of reusable validation rules to check Niagara System assets.
 * Once a number of rules are added to the rule set, it can be used either in effect types or configured as a global rule set in the Niagara plugin settings.
 */
UCLASS(BlueprintType, MinimalAPI)
class UNiagaraValidationRuleSet : public UObject
{
	GENERATED_BODY()
public:
	bool HasAnyRules() const
	{
		return ValidationRules.ContainsByPredicate([](UNiagaraValidationRule* Rule) { return Rule && Rule->IsEnabled(); });
	}

	UPROPERTY(EditAnywhere, Category = "Validation", Instanced)
	TArray<TObjectPtr<UNiagaraValidationRule>> ValidationRules;
};
