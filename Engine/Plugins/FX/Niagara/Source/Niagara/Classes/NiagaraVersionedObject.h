// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraTypes.h"

class UNiagaraVersionUpgradeBase;

struct FNiagaraVersionDataAccessor
{
	virtual ~FNiagaraVersionDataAccessor() = default;

	NIAGARA_API virtual FNiagaraAssetVersion& GetObjectVersion();
	NIAGARA_API virtual FText& GetVersionChangeDescription();
	NIAGARA_API virtual bool& IsDeprecated();
	NIAGARA_API virtual FText& GetDeprecationMessage();
	NIAGARA_API virtual ENiagaraPythonUpdateScriptReference& GetUpdateScriptExecutionType();
	NIAGARA_API virtual FString& GetPythonUpdateScript();
	NIAGARA_API virtual FFilePath& GetScriptAsset();
	NIAGARA_API virtual TSubclassOf<UNiagaraVersionUpgradeBase>& GetUpdateUtilityClass();
};

class FNiagaraVersionedObject
{
public:
	virtual ~FNiagaraVersionedObject() = default;

	/** Returns all available versions for this object. */
	NIAGARA_API virtual TArray<FNiagaraAssetVersion> GetAllAvailableVersions() const;

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual TSharedPtr<FNiagaraVersionDataAccessor> GetVersionDataAccessor(const FGuid& Version);

	/** If true then this script asset uses active version control to track changes. */
	NIAGARA_API virtual bool IsVersioningEnabled() const;
	
	/** Returns the version of the exposed version data (i.e. the version used when adding a module to the stack) */
	NIAGARA_API virtual FNiagaraAssetVersion GetExposedVersion() const;

	/** Returns the version data for the given guid, if it exists. Otherwise returns nullptr. */
	NIAGARA_API virtual FNiagaraAssetVersion const* FindVersionData(const FGuid& VersionGuid) const;
	
	/** Creates a new data entry for the given version number. The version must be > 1.0 and must not collide with an already existing version. The data will be a copy of the previous minor version. */
	NIAGARA_API virtual FGuid AddNewVersion(int32 MajorVersion, int32 MinorVersion);

	/** Deletes the version data for an existing version. The exposed version cannot be deleted and will result in an error. Does nothing if the guid does not exist in the object's version data. */
	NIAGARA_API virtual void DeleteVersion(const FGuid& VersionGuid);

	/** Changes the exposed version. Does nothing if the guid does not exist in the object's version data. */
	NIAGARA_API virtual void ExposeVersion(const FGuid& VersionGuid);

	/** Enables versioning for this object. */
	NIAGARA_API virtual void EnableVersioning();

	/** Disables versioning and keeps only the data from the given version guid. Note that this breaks ALL references from existing assets and should only be used when creating a copy of an object, as the effect is very destructive.  */
	NIAGARA_API virtual void DisableVersioning(const FGuid& VersionGuidToUse);
#endif //WITH_EDITORONLY_DATA
};
