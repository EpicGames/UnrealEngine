// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RHI.h"
#include "NiagaraUseOpenVDB.h"

#include "VolumeCache.generated.h"

class FVolumeCacheData;

UENUM()
enum class EVolumeCacheType : uint8
{
	OpenVDB
};

UCLASS(Experimental, MinimalAPI)
class UVolumeCache : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** File path to load */
	UPROPERTY(EditAnywhere, Category=File, meta=(DisplayName="File Path"))
	FString FilePath;
	
	UPROPERTY(EditAnywhere, Category = File, meta = (DisplayName = "Cache Type"))
	EVolumeCacheType CacheType;

	UPROPERTY(EditAnywhere, Category = File, meta = (DisplayName = "Resolution"))
	FIntVector Resolution;

	UPROPERTY(EditAnywhere, Category = File, meta = (DisplayName = "Frame Range Start"))
	int32 FrameRangeStart;	

	UPROPERTY(EditAnywhere, Category = File, meta = (DisplayName = "Frame Range End"))
	int32 FrameRangeEnd;
		
	NIAGARA_API void InitData();

	NIAGARA_API bool LoadFile(int frame);
	NIAGARA_API bool UnloadFile(int frame);		
	NIAGARA_API bool LoadRange();
	NIAGARA_API void UnloadAll();
	NIAGARA_API FString GetAssetPath(int frame);
	
	TSharedPtr<FVolumeCacheData> GetData() { return CachedVolumeFiles;  }

	// @todo: high level fill volume texture method that works on GT calls
	// bool Fill3DTexture(int frame, FTextureRHIRef TextureToFill, FRHICommandListImmediate& RHICmdList);
	
private:
	TSharedPtr<FVolumeCacheData> CachedVolumeFiles;
};

class FVolumeCacheData
{
public:		
	FVolumeCacheData() : DenseResolution(-1, -1, -1) {}
	virtual ~FVolumeCacheData() = default;

	NIAGARA_API FString GetAssetPath(FString PathFormat, int32 FrameIndex) const;

	FIntVector GetDenseResolution() { return DenseResolution;  }

	virtual void Init(FIntVector Resolution) = 0;
	virtual bool LoadFile(FString Path, int frame) = 0;
	virtual bool UnloadFile(int frame) = 0;
	virtual bool LoadRange(FString Path, int Start, int End) = 0;
	virtual void UnloadAll() = 0;
	virtual bool Fill3DTexture_RenderThread(int frame, FTextureRHIRef TextureToFill, FRHICommandListImmediate& RHICmdList) = 0;
	virtual bool Fill3DTexture(int frame, FTextureRHIRef TextureToFill) = 0;

protected:
	FIntVector DenseResolution;
};

#if UE_USE_OPENVDB
namespace OpenVDBTools
{
	NIAGARA_API bool WriteImageDataToOpenVDBFile(FStringView FilePath, FIntVector ImageSize, TArrayView<FFloat16Color> ImageData, bool UseFloatGrids = false);
}
#endif
