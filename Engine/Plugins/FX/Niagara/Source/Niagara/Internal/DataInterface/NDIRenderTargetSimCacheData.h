// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RHIDefinitions.h"
#include "Serialization/BulkData.h"
#include "PixelFormat.h"

#include "NDIRenderTargetSimCacheData.generated.h"

class FJsonObject;

USTRUCT()
struct FNDIRenderTargetSimCacheFrame
{
	GENERATED_BODY();

	UPROPERTY()
	FIntVector Size = FIntVector(EForceInit::ForceInitToZero);

	UPROPERTY()
	TEnumAsByte<EPixelFormat> Format = EPixelFormat::PF_A16B16G16R16;

	UPROPERTY()
	int32 UncompressedSize = 0;

	UPROPERTY()
	int32 CompressedSize = 0;

	mutable FByteBulkData BulkData;
};

UCLASS(MinimalAPI)
class UNDIRenderTargetSimCacheData : public UObject
{
	GENERATED_BODY()

public:
	// Begin UObject Interface
	NIAGARA_API virtual void BeginDestroy() override;
	NIAGARA_API virtual void Serialize(FArchive& Ar) override;

	bool IsValidFrame(int32 FrameIndex) const { return Frames.IsValidIndex(FrameIndex); }
	bool HasPixelData(int32 FrameIndex) const { return Frames.IsValidIndex(FrameIndex) && Frames[FrameIndex].CompressedSize > 0; }
	FIntVector GetTextureSize(int32 FrameIndex) const { check(Frames.IsValidIndex(FrameIndex)); return Frames[FrameIndex].Size; }
	EPixelFormat GetTextureFormat(int32 FrameIndex) const { check(Frames.IsValidIndex(FrameIndex)); return Frames[FrameIndex].Format; }
	int32 GetCompressedSize(int32 FrameIndex) const { check(Frames.IsValidIndex(FrameIndex)); return Frames[FrameIndex].CompressedSize; }
	int32 GetUncompressedSize(int32 FrameIndex) const { check(Frames.IsValidIndex(FrameIndex)); return Frames[FrameIndex].UncompressedSize; }

	bool CompareFrame(const UNDIRenderTargetSimCacheData* Other, int32 FrameIndex, TOptional<float> Tolerance, FString& OutErrors) const;

	TSharedPtr<FJsonObject> FrameToJson(int FrameIndex, TOptional<FString> TargetFolder, TOptional<FString> FilenamePrefix) const;

	// Get a copy of the uncompressed pixel data
	NIAGARA_API TArray<uint8> GetPixelData(int32 FrameIndex) const;

	// Copy the pixel data into the provided buffer
	// Note:  you must have allocated the appropriate space
	NIAGARA_API void GetPixelData(int32 FrameIndex, uint8* DestPixelData, int32 DestRowPitch, int32 DestSlicePitch) const;

	// Release the bulk data copy we are holding onto
	NIAGARA_API void ReleasePixelData(int32 FrameIndex);

	// Release all of the bulk data copies
	NIAGARA_API void ReleaseAllPixelData();

	// Set pixel data for the given frame
	NIAGARA_API void SetPixelData(int32 FrameIndex, FIntVector Size, EPixelFormat Format, TArrayView<uint8> PixelData);

	UPROPERTY()
	FName CompressionType;

protected:
	UPROPERTY()
	TArray<FNDIRenderTargetSimCacheFrame> Frames;

	mutable TArray<uint8*> PixelDataFrames;
};
