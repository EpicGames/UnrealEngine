// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"
#include "NiagaraParameterStore.h"
#include "NiagaraDataInterfaceActorComponent.generated.h"

UENUM()
enum class ENDIActorComponentSourceMode : uint8
{
	/**
	The default binding mode first we look for a valid binding on the ActorOrComponentParameter.
	If this it no valid we then look at the SourceActor.
	If these both fail we are bound to nothing.
	*/
	Default,
	/**
	We will first look at the attach parent.
	If this is not valid we fallback to the Default binding mode.
	*/
	AttachParent,
	/**
	We will look for a ULocalPlayer with the provided index.
	If this is not valid we fallback to the Default binding mode.
	*/
	LocalPlayer,
};

/**
Data interface that gives you access to actor & component information.
*/
UCLASS(EditInlineNew, Category = "Actor", CollapseCategories, meta = (DisplayName = "Actor Component Interface"), MinimalAPI)
class UNiagaraDataInterfaceActorComponent : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(uint32,		Valid)
		SHADER_PARAMETER(FMatrix44f,	Matrix)
		SHADER_PARAMETER(FQuat4f,		Rotation)
		SHADER_PARAMETER(FVector3f,		Scale)
		SHADER_PARAMETER(FVector3f,		Velocity)
	END_SHADER_PARAMETER_STRUCT();

public:
	/** Controls how we find the actor / component we want to bind to. */
	UPROPERTY(EditAnywhere, Category = "ActorComponent")
	ENDIActorComponentSourceMode SourceMode = ENDIActorComponentSourceMode::Default;

	UPROPERTY(EditAnywhere, Category = "ActorComponent", meta=(EditCondition="SourceMode==ENDIActorComponentSourceMode::LocalPlayer"))//, EditConditionHides))
	int32 LocalPlayerIndex = 0;

	/** Optional source actor to use, if the user parameter binding is valid this will be ignored. */
	UPROPERTY(EditAnywhere, Category = "ActorComponent")
	TLazyObjectPtr<AActor> SourceActor;

	/** User parameter binding to use, overrides SourceActor.  Can be set by Blueprint, etc. */
	UPROPERTY(EditAnywhere, Category = "ActorComponent")
	FNiagaraUserParameterBinding ActorOrComponentParameter;

	/** When this option is disabled, we use the previous frame's data for the skeletal mesh and can often issue the simulation early. This greatly
	reduces overhead and allows the game thread to run faster, but comes at a tradeoff if the dependencies might leave gaps or other visual artifacts.*/
	UPROPERTY(EditAnywhere, Category = "Performance")
	bool bRequireCurrentFrameData = true;
	
	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	//UObject Interface End

	NIAGARA_API class UActorComponent* ResolveComponent(FNiagaraSystemInstance* SystemInstance, const void* PerInstanceData) const;

	//UNiagaraDataInterface Interface
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;

	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

	virtual bool HasTickGroupPrereqs() const override { return true; }
	NIAGARA_API virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	virtual bool HasPreSimulateTick() const override { return true; }
	//UNiagaraDataInterface Interface

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

private:
	NIAGARA_API void VMGetMatrix(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetTransform(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetPhysicsVelocity(FVectorVMExternalFunctionContext& Context);
};
