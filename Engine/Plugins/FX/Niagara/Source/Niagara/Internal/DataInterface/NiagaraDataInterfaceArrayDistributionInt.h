// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceArray.h"
#include "NiagaraDataInterfaceArrayImpl.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "NiagaraDataInterfaceArrayDistributionInt.generated.h"

USTRUCT(BlueprintType, meta = (DisplayName = "Weighted Distribution Int Entry"))
struct FNDIDistributionIntArrayEntry
{
	GENERATED_BODY()

	// The value returned if we randomly select this value
	UPROPERTY(EditAnywhere, Category="Entry")
	int Value = 0;

	// Used to determine the probability this value will be selected when using random
	// The higher the weight relative to the other weights the higher the chance this value is selected
	UPROPERTY(EditAnywhere, Category="Entry")
	float Weight = 1.0f;

	bool operator==(const FNDIDistributionIntArrayEntry& Other) const
	{
		return
			Value == Other.Value &&
			FMath::IsNearlyEqual(Weight, Other.Weight);
	}
};

namespace NDIArrayDistributionIntPrivate
{
	struct FNDIProxy;
}

// Array data interface used for randomly selecting a weighted integer
UCLASS(EditInlineNew, Category = "Array", meta = (DisplayName = "Weighted Distribution Int Array", Experimental))
class UNiagaraDataInterfaceArrayDistributionInt : public UNiagaraDataInterface
{
	friend NDIArrayDistributionIntPrivate::FNDIProxy;

public:
	GENERATED_BODY()

	UNiagaraDataInterfaceArrayDistributionInt(FObjectInitializer const& ObjectInitializer);

	// UObject Interface
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// UObject Interface End

	// UNiagaraDataInterface Interface Begin
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
public:
	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
#if WITH_EDITORONLY_DATA
	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL) override;
	virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
#endif
	virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	// UNiagaraDataInterface Interface End

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Distribution Int Array", UnsafeDuringActorConstruction = "true"))
	static void SetNiagaraArrayDistributionInt(UNiagaraComponent* NiagaraComponent, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<FNDIDistributionIntArrayEntry>& ArrayData);

private:
	void BuildTableData();

	int32 GetBuiltTableLength() const;

	void VMLength(FVectorVMExternalFunctionContext& Context) const;
	void VMIsValidIndex(FVectorVMExternalFunctionContext& Context) const;
	void VMGetLastIndex(FVectorVMExternalFunctionContext& Context) const;
	void VMGet(FVectorVMExternalFunctionContext& Context) const;
	void VMGetProbAlias(FVectorVMExternalFunctionContext& Context) const;
	void VMGetRandomValue(FVectorVMExternalFunctionContext& Context) const;

private:
	mutable FTransactionallySafeRWLock BuiltTableDataGuard;

	UPROPERTY(EditAnywhere, Category = "Array")
	TArray<FNDIDistributionIntArrayEntry> ArrayData;

	UPROPERTY()
	TArray<uint8> BuiltTableData;
};
