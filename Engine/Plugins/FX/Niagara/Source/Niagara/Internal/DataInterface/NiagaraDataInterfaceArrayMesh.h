// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceArray.h"
#include "NiagaraDataInterfaceArrayImpl.h"
#include "NiagaraRenderableMeshArrayInterface.h"
#include "NiagaraDataInterfaceArrayMesh.generated.h"

class FNiagaraSystemInstance;

UCLASS(EditInlineNew, Category = "Array", meta = (DisplayName = "Mesh Array", Experimental), Blueprintable, BlueprintType, MinimalAPI)
class UNiagaraDataInterfaceArrayMesh : public UNiagaraDataInterfaceArray, public INiagaraRenderableMeshArrayInterface
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Array")
	TArray<FNiagaraMeshRendererMeshPropertiesBase> MeshData;

	NDIARRAY_GENERATE_BODY(UNiagaraDataInterfaceArrayMesh, FNiagaraMeshRendererMeshPropertiesBase, MeshData)

	//UObject Interface Begin
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API virtual void BeginDestroy() override;
	NIAGARA_API virtual void PreEditChange(class FProperty* PropertyThatWillChange) override;
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif// WITH_EDITORONLY_DATA
	//UObject Interface End

#if WITH_EDITORONLY_DATA
	void OnMeshChanged();
	void OnMeshPostBuild(UStaticMesh*);
	void OnAssetReimported(UObject* Object);

	void AddMeshChangedDelegates();
	void RemoveMeshChangedDelegates();
#endif //WITH_EDITORONLY_DATA

	//INiagaraRenderableMeshArrayInterface Interface Begin
	virtual void ForEachMesh(FNiagaraSystemInstance* SystemInstance, TFunction<void(int32)> NumMeshesDelegate, TFunction<void(const FNiagaraMeshRendererMeshProperties&)> IterateDelegate) const override;
	//INiagaraRenderableMeshArrayInterface Interface End

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Mesh Array", UnsafeDuringActorConstruction = "true"))
	static NIAGARA_API void SetNiagaraArrayMesh(UNiagaraComponent* NiagaraComponent, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<FNiagaraMeshRendererMeshPropertiesBase>& ArrayData);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Mesh Array (Static Mesh)", UnsafeDuringActorConstruction = "true"))
	static NIAGARA_API void SetNiagaraArrayMeshSM(UNiagaraComponent* NiagaraComponent, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<UStaticMesh*>& ArrayData);
};
