// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataSetCompiledData.h"
#include "NiagaraCompileHash.h"
#include "NiagaraDataInterface.h"
#include "RHIUtilities.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "NiagaraDataInterfaceDataChannelCommon.generated.h"

UENUM()
enum class ENiagaraDataChannelAllocationMode : uint8
{
	/** Fixed number of elements available to write per frame. */
	Static,

	/** Allocation count is determined by DI script calls to Allocate in Emitter Scripts. */
	Dynamic UMETA(Hidden)
};

//TODO: Possible we may want to do reads and writes using data channels in a single system in future, avoiding the need to push data out to any manager class etc.
// UENUM()
// enum class ENiagaraDataChannelScope
// {
// 	/** Data is read or written internally to other DIs in the same Niagara System. */
// 	Local,
// 	/** Data is read or written externally to system, into the outside world. */
// 	World,
// };

//Enable various invasive debugging features that will bloat memory and incur overhead.
#define DEBUG_NDI_DATACHANNEL (!(UE_BUILD_TEST||UE_BUILD_SHIPPING))

/** 
Stores info for a function called on a DataChannel DI.
Describes a function call which is used when generating binding information between the data and the VM & GPU scripts.
*/
USTRUCT()
struct FNDIDataChannelFunctionInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FName FunctionName;

	UPROPERTY()
	TArray<FNiagaraVariableBase> Inputs;

	UPROPERTY()
	TArray<FNiagaraVariableBase> Outputs;

	bool operator==(const FNDIDataChannelFunctionInfo& Other)const;
	bool operator!=(const FNDIDataChannelFunctionInfo& Other)const{ return !operator==(Other);}
	bool CheckHashConflict(const FNDIDataChannelFunctionInfo& Other)const;
};

uint32 GetTypeHash(const FNDIDataChannelFunctionInfo& FuncInfo);

/** Binding between registers accessed in the data channel DI function calls and the relevant data in a dataset. */
struct FNDIDataChannelRegisterBinding
{
	static const uint32 RegisterBits = 30;
	static const uint32 DataTypeBits = 2;
	FNDIDataChannelRegisterBinding(uint32 InFunctionRegisterIndex, uint32 InDataSetRegisterIndex, ENiagaraBaseTypes InDataType)
	: DataSetRegisterIndex(InDataSetRegisterIndex)
	, FunctionRegisterIndex(InFunctionRegisterIndex) 
	, DataType((uint32)InDataType)
	{
		check(FunctionRegisterIndex <= (1u << RegisterBits) - 1);
		check((uint32)InDataType <= (1u << DataTypeBits) - 1);
	}

	uint32 GetDataSetRegisterIndex()const { return DataSetRegisterIndex; }
	uint32 GetFunctionRegisterIndex()const { return FunctionRegisterIndex; }
	ENiagaraBaseTypes GetDataType()const { return (ENiagaraBaseTypes)DataType; }

private:
	uint32 DataSetRegisterIndex;
	uint32 FunctionRegisterIndex : RegisterBits;
	uint32 DataType : DataTypeBits;
};


/** Layout info mapping from a function called by a data channel DI to the actual data set register. */
struct FNDIDataChannel_FunctionToDataSetBinding
{
	/** Bindings used by the VM calls to map dataset registers to the relevant function call registers. */
	TArray<FNDIDataChannelRegisterBinding> VMRegisterBindings;

	uint32 NumFloatComponents = 0;
	uint32 NumInt32Components = 0;
	uint32 NumHalfComponents = 0;
	uint32 FunctionLayoutHash = 0;
	uint32 DataSetLayoutHash = 0;

	#if DEBUG_NDI_DATACHANNEL
	FNDIDataChannelFunctionInfo DebugFunctionInfo;
	FNiagaraDataSetCompiledData DebugCompiledData;
	#endif

	FNDIDataChannel_FunctionToDataSetBinding(const FNDIDataChannelFunctionInfo& FunctionInfo, const FNiagaraDataSetCompiledData& DataSetLayout, TArray<FNiagaraVariableBase>& OutMissingParams);

	bool IsValid()const { return DataSetLayoutHash != 0; }
	void GenVMBindings(const FNiagaraVariableBase& Var, const UStruct* Struct, uint32& FuncFloatRegister, uint32& FuncIntRegister, uint32& FuncHalfRegister, uint32& DataSetFloatRegister, uint32& DataSetIntRegister, uint32& DataSetHalfRegister);
};

typedef TSharedPtr<FNDIDataChannel_FunctionToDataSetBinding,ESPMode::ThreadSafe> FNDIDataChannel_FuncToDataSetBindingPtr;

/** 
Manager class that generates and allows access to layout information used by the Data Channel DIs. 
These layout buffers will map from a DI's function calls to the register offsets of the relevant data inside the DataSet buffers.
Each combination of dataset layout and function info will need a unique mapping but these will be used by many instances.
This manager class allows the de-duplication and sharing of such binding data that would otherwise have to be generated and stored per DI instance.
*/
struct FNDIDataChannelLayoutManager
{
	/** Map containing binding information for each function info/dataset layout pair. */
	TMap<uint32, FNDIDataChannel_FuncToDataSetBindingPtr> FunctionToDataSetLayoutMap;

	/** 
	Typically this map will be accessed from the game thread and then the shared ptrs of actual layout information passed off to various threads. 
	Though for additional safety we'll use a lock. It should be very low contention.
	*/
	FRWLock FunctionToDataSetMapLock;

public:

	//TLazySingleton interface
	static FNDIDataChannelLayoutManager& Get();
	static void TearDown();

	void Reset();

	/** Generates a key that can be used to retrieve layout information on both the GT and RT. */
	uint32 GetLayoutKey(const FNDIDataChannelFunctionInfo& FunctionInfo, const FNiagaraDataSetCompiledData& DataSetLayout) const
	{
		return HashCombine(GetTypeHash(FunctionInfo), DataSetLayout.GetLayoutHash());
	}
	
	/** 
	Retrieves, or generates, the layout information that maps from the given function to the data in the given dataset.
	*/
	FNDIDataChannel_FuncToDataSetBindingPtr GetLayoutInfo(const FNDIDataChannelFunctionInfo& FunctionInfo, const FNiagaraDataSetCompiledData& DataSetLayout, TArray<FNiagaraVariableBase>& OutMissingParams);
};

/** A sorted table of parameters accessed by each GPU script */
USTRUCT()
struct FNDIDataChannel_GPUScriptParameterAccessInfo 
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FNiagaraVariableBase> SortedParameters;
};

/** 
Compile time data used by Data Channel interfaces.
*/
USTRUCT()
struct FNDIDataChannelCompiledData
{
	GENERATED_BODY()

	/** Initializes function bindings and binds if there is a valid DataSetCompiledData. */
	bool Init(const UNiagaraSystem* System, UNiagaraDataInterface* OwnerDI);

	int32 FindFunctionInfoIndex(FName Name, const TArray<FNiagaraVariableBase>& VariadicInputs, const TArray<FNiagaraVariableBase>& VariadicOutputs)const;

	const TArray<FNDIDataChannelFunctionInfo>& GetFunctionInfo()const { return FunctionInfo; }
	const TMap<FNiagaraCompileHash, FNDIDataChannel_GPUScriptParameterAccessInfo>& GetGPUScriptParameterInfos()const{ return GPUScriptParameterInfos; }

	bool UsedByCPU()const{ return bUsedByCPU; }
	bool UsedByGPU()const{ return bUsedByGPU; }
	bool NeedSpawnDataTable()const { return bNeedsSpawnDataTable; }
	bool SpawnsParticles()const { return bSpawnsParticles; }
	bool CallsWriteFunction()const { return bCallsWrite; }
	int32 GetTotalParams()const{ return TotalParams; }

protected:

	/**
	Data describing every function call for this DI in VM scripts. 
	VM Access to data channels uses a binding from script to DataSet per function call (de-duped by layout).
	*/
	UPROPERTY()
	TArray<FNDIDataChannelFunctionInfo> FunctionInfo;

	/** 
	Info about which parameters are accessed for each GPU script. 
	GPU access to data channels uses a binding from script to DataSet per script via a mapping of param<-->data set offsets.
	*/
	UPROPERTY()
	TMap<FNiagaraCompileHash, FNDIDataChannel_GPUScriptParameterAccessInfo> GPUScriptParameterInfos;

	/** Total param count across all scripts. Allows easy pre-allocation for the buffers at runtime. */
	UPROPERTY()
	uint32 TotalParams = 0;

	UPROPERTY()
	bool bUsedByCPU = false;

	UPROPERTY()
	bool bUsedByGPU = false;
	
	UPROPERTY()
	bool bNeedsSpawnDataTable = true;

	UPROPERTY()
	bool bSpawnsParticles = false;

	//If we call Write() on our CPU buffers we must do some extra buffer book keeping.
	UPROPERTY()
	bool bCallsWrite = false;

	/** Iterates over all scripts for the owning system and gathers all functions and parameters accessing this DI. Building the FunctionInfoTable and GPUScriptParameterInfos map.  */
	void GatherAccessInfo(const UNiagaraSystem* System, UNiagaraDataInterface* Owner);
};

class FNDIDummyUAV : public FRenderResource
{
private:

	EPixelFormat PixelFmt;
	uint32 Size;

public:
	FNDIDummyUAV(EPixelFormat Fmt, uint32 InSize) :PixelFmt(Fmt), Size(InSize) {}

	FRWBuffer Buffer;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		Buffer.Initialize(RHICmdList, TEXT("FNDIDummyUAV"), Size, 1, PixelFmt, BUF_Static);
	}

	virtual void ReleaseRHI() override
	{
		Buffer.Release();
	}
};

namespace NDIDataChannelUtilities
{
	extern const FName GetNDCSpawnDataName;
	extern const FName SpawnConditionalName;
	extern const FName SpawnDirectName;
	extern const FName WriteName;

	const TGlobalResource<FNDIDummyUAV>& GetDummyUAVFloat();
	const TGlobalResource<FNDIDummyUAV>& GetDummyUAVInt32();
	const TGlobalResource<FNDIDummyUAV>& GetDummyUAVHalf();

	void SortParameters(TArray<FNiagaraVariableBase>& Parameters);

#if WITH_EDITORONLY_DATA
	void GenerateDataChannelAccessHlsl(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, TConstArrayView<FString> CommonTemplateShaderCode, const TMap<FName, FString>& TemplateShaderMap, FString& OutHLSL);
#endif

}



/** Handles any number of variadic parameter inputs. */
template<int32 EXPECTED_NUM_INPUTS>
struct FNDIVariadicInputHandler
{
	TArray<FNDIInputParam<float>, TInlineAllocator<EXPECTED_NUM_INPUTS>> FloatInputs;
	TArray<FNDIInputParam<int32>, TInlineAllocator<EXPECTED_NUM_INPUTS>> IntInputs;
	TArray<FNDIInputParam<FFloat16>, TInlineAllocator<EXPECTED_NUM_INPUTS>> HalfInputs;

	FNDIVariadicInputHandler(FVectorVMExternalFunctionContext& Context, const FNDIDataChannel_FunctionToDataSetBinding* BindingPtr)
	{
		if(BindingPtr)
		{
			FloatInputs.Reserve(BindingPtr->NumFloatComponents);
			IntInputs.Reserve(BindingPtr->NumInt32Components);
			HalfInputs.Reserve(BindingPtr->NumHalfComponents);
			for (const FNDIDataChannelRegisterBinding& VMBinding : BindingPtr->VMRegisterBindings)
			{
				switch (VMBinding.GetDataType())
				{
				case ENiagaraBaseTypes::Float: FloatInputs.Emplace(Context); break;
				case ENiagaraBaseTypes::Int32: IntInputs.Emplace(Context); break;
				case ENiagaraBaseTypes::Bool: IntInputs.Emplace(Context); break;
				case ENiagaraBaseTypes::Half: HalfInputs.Emplace(Context); break;
				default: check(0);
				};
			}
		}
	}

	void Reset()
	{
		for (FNDIInputParam<float>& Input : FloatInputs) { Input.Reset(); }
		for (FNDIInputParam<int32>& Input : IntInputs) { Input.Reset(); }
		for (FNDIInputParam<FFloat16>& Input : HalfInputs) { Input.Reset(); }
	}

	void Advance(int32 Count = 1)
	{
		for (FNDIInputParam<float>& Input : FloatInputs) { Input.Advance(Count); }
		for (FNDIInputParam<int32>& Input : IntInputs) { Input.Advance(Count); }
		for (FNDIInputParam<FFloat16>& Input : HalfInputs) { Input.Advance(Count); }
	}

	template<typename TFloatAction, typename TIntAction, typename THalfAction>
	bool Process(bool bProcess, int32 Count, const FNDIDataChannel_FunctionToDataSetBinding* BindingInfo, TFloatAction FloatFunc, TIntAction IntFunc, THalfAction HalfFunc)
	{
		if (BindingInfo && bProcess)
		{
			//TODO: Optimize for long runs of writes to reduce binding/lookup overhead.
			for (const FNDIDataChannelRegisterBinding& VMBinding : BindingInfo->VMRegisterBindings)
			{
				switch(VMBinding.GetDataType())
				{
					case ENiagaraBaseTypes::Float: FloatFunc(VMBinding, FloatInputs[VMBinding.GetFunctionRegisterIndex()]); break;
					case ENiagaraBaseTypes::Int32: IntFunc(VMBinding, IntInputs[VMBinding.GetFunctionRegisterIndex()]); break;
					case ENiagaraBaseTypes::Bool: IntFunc(VMBinding, IntInputs[VMBinding.GetFunctionRegisterIndex()]); break;
					case ENiagaraBaseTypes::Half: HalfFunc(VMBinding, HalfInputs[VMBinding.GetFunctionRegisterIndex()]); break;
					default: check(0);
				};
			}
			return true;
		}
		
		Advance(Count);
		return false;
	}
};


template<int32 EXPECTED_NUM_INPUTS>
struct FNDIVariadicOutputHandler
{
	TArray<VectorVM::FExternalFuncRegisterHandler<float>, TInlineAllocator<EXPECTED_NUM_INPUTS>> FloatOutputs;
	TArray<VectorVM::FExternalFuncRegisterHandler<int32>, TInlineAllocator<EXPECTED_NUM_INPUTS>> IntOutputs;
	TArray<VectorVM::FExternalFuncRegisterHandler<FFloat16>, TInlineAllocator<EXPECTED_NUM_INPUTS>> HalfOutputs;

	FNDIVariadicOutputHandler(FVectorVMExternalFunctionContext& Context, const FNDIDataChannel_FunctionToDataSetBinding* BindingPtr)
	{
		//Parse the VM bytecode inputs in order, mapping them to the correct DataChannel data 
		if(BindingPtr)
		{
			FloatOutputs.Reserve(BindingPtr->NumFloatComponents);
			IntOutputs.Reserve(BindingPtr->NumInt32Components);
			HalfOutputs.Reserve(BindingPtr->NumHalfComponents);
			for (const FNDIDataChannelRegisterBinding& VMBinding : BindingPtr->VMRegisterBindings)
			{
				switch (VMBinding.GetDataType())
				{
				case ENiagaraBaseTypes::Float: FloatOutputs.Emplace(Context); break;
				case ENiagaraBaseTypes::Int32: IntOutputs.Emplace(Context); break;
				case ENiagaraBaseTypes::Bool: IntOutputs.Emplace(Context); break;
				case ENiagaraBaseTypes::Half: HalfOutputs.Emplace(Context); break;
				default: check(0);
				};
			}
		}
	}

	template<typename TFloatAction, typename TIntAction, typename THalfAction>
	bool Process(bool bProcess, int32 Count, const FNDIDataChannel_FunctionToDataSetBinding* BindingInfo, TFloatAction FloatFunc, TIntAction IntFunc, THalfAction HalfFunc)
	{
		if (BindingInfo && bProcess)
		{
			//TODO: Optimize for long runs of writes to reduce binding/lookup overhead.
			for (const FNDIDataChannelRegisterBinding& VMBinding : BindingInfo->VMRegisterBindings)
			{
				switch (VMBinding.GetDataType())
				{
				case ENiagaraBaseTypes::Float: FloatFunc(VMBinding, FloatOutputs[VMBinding.GetFunctionRegisterIndex()]); break;
				case ENiagaraBaseTypes::Int32: IntFunc(VMBinding, IntOutputs[VMBinding.GetFunctionRegisterIndex()]); break;
				case ENiagaraBaseTypes::Bool: IntFunc(VMBinding, IntOutputs[VMBinding.GetFunctionRegisterIndex()]); break;
				case ENiagaraBaseTypes::Half: HalfFunc(VMBinding, HalfOutputs[VMBinding.GetFunctionRegisterIndex()]); break;
				default: check(0);
				};
			}
			return true;
		}

		Fallback(Count);

		return false;
	}

	void Fallback(int32 Count)
	{
		if(Count > 0)
		{
			for (int32 OutIdx = 0; OutIdx < FloatOutputs.Num(); ++OutIdx)
			{
				if(FloatOutputs[OutIdx].IsValid())
				{
					FMemory::Memzero(FloatOutputs[OutIdx].GetDest(), sizeof(float) * Count);
					FloatOutputs[OutIdx].Advance(Count);
				}
			}

			for (int32 OutIdx = 0; OutIdx < IntOutputs.Num(); ++OutIdx)
			{
				if (IntOutputs[OutIdx].IsValid())
				{
					FMemory::Memzero(IntOutputs[OutIdx].GetDest(), sizeof(int32) * Count);
					IntOutputs[OutIdx].Advance(Count);
				}
			}

			for (int32 OutIdx = 0; OutIdx < HalfOutputs.Num(); ++OutIdx)
			{
				if (HalfOutputs[OutIdx].IsValid())
				{
					FMemory::Memzero(HalfOutputs[OutIdx].GetDest(), sizeof(FFloat16) * Count);
					HalfOutputs[OutIdx].Advance(Count);
				}
			}
		}
	}
};


struct FVariadicParameterGPUScriptInfo
{
	/**
	Table of all parameter offsets used by each GPU script using this DI.
	Each script has to have it's own section of this table as the offsets into this table are embedded in the hlsl.
	At hlsl gen time we only have the context of each script individually to generate these indexes.
	TODO: Can possible elevate this up to the LayoutManager and have a single layout buffer for all scripts
	*/
	TResourceArray<uint32> GPUScriptParameterOffsetTable;

	/**
	Offsets into the parameter table are embedded in the gpu script hlsl.
	At hlsl gen time we can only know which parameters are accessed by each script individually so each script must have it's own parameter binding table.
	We provide the offset into the above table via a shader param.
	TODO: Can just as easily be an offset into a global buffer in the Layout manager.
	*/
	TMap<FNiagaraCompileHash, uint32> GPUScriptParameterTableOffsets;

	bool bDirty = false;

	void Init(const FNDIDataChannelCompiledData& DICompiledData, const FNiagaraDataSetCompiledData& GPUDataSetCompiledData)
	{
		bDirty = true;

		//For every GPU script, we append it's parameter access info to the table.
		GPUScriptParameterTableOffsets.Reset();
		constexpr int32 ElemsPerParam = 3;
		GPUScriptParameterOffsetTable.Reset(DICompiledData.GetTotalParams() * ElemsPerParam);
		for (auto& GPUParameterAccessInfoPair : DICompiledData.GetGPUScriptParameterInfos())
		{
			const FNDIDataChannel_GPUScriptParameterAccessInfo& ParamAccessInfo = GPUParameterAccessInfoPair.Value;

			//First get the offset for this script in the table.
			GPUScriptParameterTableOffsets.FindOrAdd(GPUParameterAccessInfoPair.Key) = GPUScriptParameterOffsetTable.Num();

			//Now fill the table for this script
			for (const FNiagaraVariableBase& Param : ParamAccessInfo.SortedParameters)
			{
				if (const FNiagaraVariableLayoutInfo* LayoutInfo = GPUDataSetCompiledData.FindVariableLayoutInfo(Param))
				{
					GPUScriptParameterOffsetTable.Add(LayoutInfo->GetNumFloatComponents() > 0 ? LayoutInfo->GetFloatComponentStart() : INDEX_NONE);
					GPUScriptParameterOffsetTable.Add(LayoutInfo->GetNumInt32Components() > 0 ? LayoutInfo->GetInt32ComponentStart() : INDEX_NONE);
					//TODO: Half Support | GPUScriptParameterOffsetTable.Add(LayoutInfo->GetNumHalfComponents() > 0 ? LayoutInfo->GetHalfComponentStart() : INDEX_NONE);
				}
				else
				{
					GPUScriptParameterOffsetTable.Add(INDEX_NONE);
					GPUScriptParameterOffsetTable.Add(INDEX_NONE);
					//TODO: Half Support | GPUScriptParameterOffsetTable.Add(INDEX_NONE);
				}
			}
		}
	}
};