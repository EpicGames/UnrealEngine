// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

/**
Data Channel Read Interface.
Enabled Niagara Systems to read data from a NiagaraDataChannel that has been previously generated by another System or Game code/BP.

Read DIs will grab a buffer from the handler for their bound Data Channel.
This could be a global buffer or some localized buffer for their region of the map
Or any other buffer sub division scheme that the handler chooses.
The main point being that this buffer can change from frame to frame and instance to instance.
The handler is free to store and distribute it's Channel Data as it pleases.

Accessor functions on the Data Channel Read and Write DIs can have any number of parameters, allowing a single function call to access arbitrary data from the Channel.
This avoids cumbersome work in the graph to access data but requires special handling inside the DI.

TODO: Local support - We can read directly from a writer DI in the same system to avoid having to publish that data in some cases. 
There is working code for this already but the concept/API needs more fleshing out before we include it.
*/

#include "Containers/DynamicRHIResourceArray.h"
#include "NiagaraDataInterfaceDataChannelCommon.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraDataChannelAccessContext.h"

#include "NiagaraDataInterfaceDataChannelRead.generated.h"

struct FNDIDataChannelWriteInstanceData;

/** Mode controlling the behavior of the Spawn function for Niagara's Data Channel Read Data Interface.*/
UENUM()
enum class ENDIDataChannelSpawnMode
{
	/** This mode will override any previous spawn counts set or accumulated this tick. */
	Override,

	/** This mode will accumulate spawn counts with other spawns. */
	Accumulate,

	/** Does nothing. This mode is useful for dynamically skipping spawning based on script logic. */
	None,

	Max UMETA(Hidden),
};

/** Mode controlling the behavior of the ScaleSpawnCount function for Niagara's Data Channel Read Data Interface.*/
UENUM()
enum class ENDIDataChannelSpawnScaleMode
{
	/** This mode will override any previously set scale values. */
	Override,

	/** This mode will combine with previously set scales. e.g. Two calls that scale by 0.5 will result in a final spawn scale of 0.25. */
	Scale,

	Max UMETA(Hidden),
};

/**
The Data Channel Reader Data Interface allows us to read from a Niagara Data Channel.
It also allows us to spawn particles into emitters in this system based upon the entries in a Niagara Data Channel.
*/
UCLASS(EditInlineNew, Category = "Data Channels", CollapseCategories, meta = (DisplayName = "Data Channel Reader"), MinimalAPI)
class UNiagaraDataInterfaceDataChannelRead : public UNiagaraDataInterfaceRWBase
{
	GENERATED_UCLASS_BODY()
protected:

public:

	//TODO: In future we may allow reads and writes that stay local to a single system.	
// 	/** The scope at which to read data. When reading locally we'll read directly from a Data Channel Write Interface in this or another emitter. When reading from World scope, we'll read from a named Data Channel. */
// 	UPROPERTY(EditAnywhere, Category = "Data Channel")
// 	ENiagaraDataChannelScope Scope;
// 
// 	/** Name of the source */
// 	UPROPERTY(EditAnywhere, Category="Data Channel", meta=(EditCondition = "Scope == ENiagaraDataChannelScope::Local"))
// 	FName Source;

	/** The data channel to access and read from. */
	UPROPERTY(EditAnywhere, Category="Data Channel")
	TObjectPtr<UNiagaraDataChannelAsset> Channel;

	/** Access Context used when accessing NDC data. Controls which of the NDCs internal data it will return for this Interface to read. Can be empty for Systems using bAutoLinkToSpawningNDC. */
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (ShowOnlyInnerProperties, EditCondition="!bAutoLinkToSpawningNDC"))
	FNDCAccessContextInst AccessContext;

	/** If true this DI will attempt to automatically link itself to the NDC it's being spawned by. */
	UPROPERTY(EditAnywhere, Category = "Data Channel", AdvancedDisplay)
	bool bAutoLinkToSpawningNDC = true;

	/** 
	* If this interface should read the current frame's data from the Data Channel. If false, the interface will read from the previous frame if it's available.
	* 
	* Reading the current frame allows us to use the most current data and have the least possible latency from the source.
	* However, it introduces a tick order dependency between this read and the Blueprints, Game Code or other Niagara Systems writing into this Data Channel.
	* If this interface reads data before those writing to the Data Channel have executed, then that data will be missed.
	* 
	* Reading the previous frame allows us to avoid this tick order dependency.
	* We can be sure we are reading all data that is written to the Data Channel, regardless of when the writes happened in the frame.
	* However it does introduce a 1 frame delay in the data being read and so can cause latency/lag.
	*/
	UPROPERTY(EditAnywhere, Category = "Data Channel", AdvancedDisplay)
	bool bReadCurrentFrame = false;

	/**
	The source Data Channel data for this interface will be refreshed every frame.
	Some Data Channels have multiple separate data elements for things such as spatial subdivision. 
	Each DI will request the correct one for it's owning system instance from the data channel. 
	Depending on the Data Channel this could be an expensive search so we should avoid doing this every tick if possible.
	However it may be required. For example if the Niagara System is moving and reading from a spatially sub-divided NDC such as the Islands type.
	*/
	UPROPERTY(EditAnywhere, Category = "Data Channel", AdvancedDisplay)
	bool bUpdateSourceDataEveryTick = true;


	/**
	When true, Emitter.Spawn group for any spawned particles will be overridden to the index of the data channel element that generated that spawn.
	Doing this will submit all NDC spawns individually and will be less performant.
	However it will allow particles to access the NDC data that generated then via the SpawnGroup value.
	It will also mean that Exec Index will be correct on a per NDC Entry level. 
	Without this settings ExecIndex will be 0...TotalSpawnCount-1. With this it will be 0...SpawnCount for each NDC item individually.
	Unless absolutely needed this is discouraged as it comes at significant performance cost when spawning and GPU emitters can currently only handle 8 individual spawns per frame.
	Calling GetNDCSpawnInfo() in the particle spawn script to get the spawning NDC Index is preferred.
	*/
	UPROPERTY(EditAnywhere, Category = "Spawning", AdvancedDisplay)
	bool bOverrideSpawnGroupToDataChannelIndex = false;

	/**
	 * If a system is ticking multiple times in a single frame (for example due to fixed tick delta time or because the user is scrubbing the desired time in the timeline),
	 * it is usually not desirable to also spawn new particles every tick for the data channel entries, as the data channel itself doesn't change.
	 *
	 * When enabled, the emitter will only spawn particles on the last subtick of the frame.
	 */
	UPROPERTY(EditAnywhere, Category = "Data Channel", AdvancedDisplay)
	bool bOnlySpawnOnceOnSubticks = true;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void BeginDestroy() override;
	NIAGARA_API virtual void Serialize(FArchive& Ar) override;
	NIAGARA_API virtual void PostLoad() override;
	#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)override;
	#endif
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL)override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL) override;

	virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)override;

	NIAGARA_API virtual void PostCompile(const UNiagaraSystem& OwningSystem) override;
#endif	

#if WITH_EDITOR	
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo) override;
	NIAGARA_API virtual void ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors) override;
#endif

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }
	virtual bool HasPostStageTick(ENiagaraScriptUsage Usage) const override { return Usage == ENiagaraScriptUsage::EmitterUpdateScript || Usage == ENiagaraScriptUsage::SystemUpdateScript; }
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual void PostStageTick(FNDICpuPostStageContext& Context)override;
	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;
	NIAGARA_API virtual void GetEmitterDependencies(UNiagaraSystem* Asset, TArray<FVersionedNiagaraEmitter>& Dependencies) const;

	virtual bool HasTickGroupPrereqs() const override;
	virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const override;

	//We cannot overlap frames as we must correctly sync up with the data channel manager on Begin/End frame etc.
	virtual bool PostSimulateCanOverlapFrames() const { return false; }
	virtual bool PostStageCanOverlapTickGroups() const { return false; }

	virtual bool RequiresCurrentFrameNDC() const { return bReadCurrentFrame; }

	virtual uint32 GetGpuCountBufferEstimate() const { return 1; }
	//UNiagaraDataInterface Interface

	//Functions usable anywhere.
	NIAGARA_API void Num(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void Read(FVectorVMExternalFunctionContext& Context, int32 FuncIdx);
	NIAGARA_API void Consume(FVectorVMExternalFunctionContext& Context, int32 FuncIdx);
	
	NIAGARA_API void GetNDCSpawnData(FVectorVMExternalFunctionContext& Context);

	//Emitter only functions.
	NIAGARA_API void SpawnConditional(FVectorVMExternalFunctionContext& Context, int32 FuncIndex);

	template<typename T>
	NIAGARA_API void SpawnDirect(FVectorVMExternalFunctionContext& Context, FName NDCVarName);
	
	template<typename T>
	NIAGARA_API void ScaleSpawnCount(FVectorVMExternalFunctionContext& Context, FName NDCVarName);

	FNDIDataChannelCompiledData& GetCompiledData() { return CompiledData; }

	NIAGARA_API virtual void InitAccessContext();

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	NIAGARA_API UNiagaraDataInterfaceDataChannelWrite* FindSourceDI()const;

 	UPROPERTY()
 	FNDIDataChannelCompiledData CompiledData;
};

struct FNDIDataChannelRead_EmitterSpawnData
{
	TArray<int32> NDCSpawnData;
	int32 NDCSpawnDataBuckets[16];

	void Reset()
	{
		NDCSpawnData.Reset();
		FMemory::Memzero(NDCSpawnDataBuckets, sizeof(int32) * 16);
	}
};

struct FNDIDataChannelRead_EmitterSpawnInfo
{
private:
	uint32 Count = 0;
	float Scale = 1.0f;

public:
	uint32 Get() const
	{
		float Value = Count * Scale;
		return static_cast<uint32>(Value > MAX_int32 ? MAX_int32 : Value);
	}

	void SetCount(uint32 NewCount) { Count = NewCount; }
	void Append(uint32 NewCount) { Count += NewCount; }
	void SetScale(float NewScale) { Scale = NewScale; }
	void ApplyScale(float NewScale) { Scale *= NewScale; }
};

struct FNDIDataChannelRead_EmitterInstanceData
{
	//Spawn data buffers needed for accessing the correct NDCIndex and spawn data from a particle during CPU or GPU execution.
	//Rebuilt every frame and only valid inside the spawn execution generated by this DI.
	FNDIDataChannelRead_EmitterSpawnData NDCSpawnData;

	//Spawn Counts for each entry in the NDC.
	TArray<FNDIDataChannelRead_EmitterSpawnInfo> NDCSpawnCounts;

	void Reset()
	{
		NDCSpawnCounts.Reset();
		NDCSpawnData.Reset();
	}
};

struct FNDIDataChannelReadInstanceData
{
	//TODO: Local reads.
// 	/** The local DataChannel writer we're bounds to, if any. */
// 	TWeakObjectPtr<UNiagaraDataInterfaceDataChannelWrite> SourceDI;
// 
// 	/** The instance data of the local DataChannel writer we're bound to, if any. */
// 	FNDIDataChannelWriteInstanceData* SourceInstData = nullptr;
	
	/** Pointer to the world DataChannel Channel we're reading from, if any. */
	TWeakObjectPtr<UNiagaraDataChannelHandler> DataChannel;

	/** Shared pointer to the actual data channel data pulled from the data channel. */
	FNiagaraDataChannelDataPtr DataChannelData = nullptr;
	
	/** Cached hash to check if the layout of our source data has changed. */
	uint64 ChachedDataSetLayoutHash = INDEX_NONE;

	/** When true we should update our function binding info on the RT next tick. */
	mutable bool bUpdateFunctionBindingRTData = false;

	/** Binding info from the Data Channel to each of our function's parameters. */
	//FNDIDataChannelBindingInfo BindingInfo;

	TArray<FNDIDataChannel_FuncToDataSetBindingPtr, TInlineAllocator<8>> FuncToDataSetBindingInfo;

	/** Keys for each of the above function infos. */
	//TArray<uint32> FuncToDataSetLayoutKeys;
	
	/** Index we use to consume DataChannel data.
		Reset at each tick.
		All consumers of the DI use the same index.
		TODO: Allow ConsumeUnique that has a unique counter per call.
	*/
	std::atomic<int32> ConsumeIndex = 0;

	// Num Elements in the NDC at the time we generated our NDCSpawnData.
	int32 NDCElementCountAtSpawn = 0;

	/** 
	Instance data for each emitter using this DI.
	*/
	TMap<FNiagaraEmitterInstance*,FNDIDataChannelRead_EmitterInstanceData> EmitterInstanceData;

	FNiagaraSystemInstance* Owner = nullptr;

	virtual ~FNDIDataChannelReadInstanceData();
	FNiagaraDataBuffer* GetReadBufferCPU(bool bPrevFrame)const;
	bool Init(UNiagaraDataInterfaceDataChannelRead* Interface, FNiagaraSystemInstance* Instance);
	void Cleanup(UNiagaraDataInterfaceDataChannelRead* Interface, FNiagaraSystemInstance* Instance);
	bool Tick(UNiagaraDataInterfaceDataChannelRead* Interface, FNiagaraSystemInstance* Instance, bool bIsInit = false);	
	bool PostTick(UNiagaraDataInterfaceDataChannelRead* Interface, FNiagaraSystemInstance* Instance);

	//Set the current NDC Data. In some cases we must unregister ourselves from the current data and register with the new.
	//TODO: move to a more handle based approach that will make this bookkeeping more robust.
	void SetDataChannelData(FNiagaraDataChannelDataPtr NewData, UNiagaraDataInterfaceDataChannelRead* Interface);
};

struct FNiagaraDataInterfaceProxy_DataChannelRead : public FNiagaraDataInterfaceProxyRW
{
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override;
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override;
	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override;

	virtual void PreStage(const FNDIGpuComputePreStageContext& Context)override;
	virtual void PostStage(const FNDIGpuComputePostStageContext& Context)override;
	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context)override;

	/** Persistent per instance data on the RT. Constructed when consuming data passed from GT->RT. */
	struct FInstanceData
	{
		//GPU Dataset from the channel handler. We'll grab the current buffer from this on the RT.
		//This must be grabbed fresh from the handler each frame as it's lifetime cannot be ensured.
		FNiagaraDataChannelDataProxyPtr ChannelDataRTProxy = nullptr;

		bool bReadPrevFrame = false;

		int32 NDCElementCountAtSpawn = 0;

		/**
		A buffer containing layout information needed to access parameters for each script using this DI.
		*/
		FReadBuffer ParameterLayoutBuffer;

		TResourceArray<uint32> ParameterLayoutData;

		/**
		Offsets into the parameter table are embedded in the gpu script hlsl.
		At hlsl gen time we can only know which parameter are accessed by each script individually so each script must have it's own parameter binding table.
		*/
		TMap<FNiagaraCompileHash, uint32> GPUScriptParameterTableOffsets;

		/** Buffer containing packed data for all emitters NDC spawning data for use on the GPU. */
		TArray<int32> NDCSpawnData;

		FRDGBufferRef NDCSpawnDataBuffer;

		uint32 ConsumeInstanceCountOffset = INDEX_NONE;

		FNiagaraDataBufferRef GPUBuffer;
	};

	TMap<FNiagaraSystemInstanceID, FInstanceData> SystemInstancesToProxyData_RT;
};
