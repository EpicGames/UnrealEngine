// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

/**

Data Channel Write Interface.
Enables Niagara Systems to push data into a NiagaraDataChannel for access later by another System or Game code/BP.

We write into an intermediate buffer on the DI which is then published to the Data Channel on post tick.
This is simple and avoids race conditions and synchronization headaches but does introduce additional copying work that in many cases may be avoided.
In the future we may allow for direct writes into the data channel buffers and/or avoiding separate writes entirely by publishing the owning emitter particle buffers.
Though these other options have their own downsides.

Accessor functions on the Data Channel Read and Write DIs can have any number of parameters, allowing a single function call to access arbitrary data from the Channel.
This avoids cumbersome work in the graph to access data but requires special handling inside the DI.

*/


#include "NiagaraDataInterfaceDataChannelCommon.h"
#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataInterface.h"
#include "NiagaraSimCacheCustomStorageInterface.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataInterfaceDataChannelWrite.generated.h"

/** Additional compile time information used by the Write DI. */
USTRUCT()
struct FNDIDataChannelWriteCompiledData : public FNDIDataChannelCompiledData
{
	GENERATED_BODY()

	/** Internal buffer layout. Contains only the data actually written by this DI. */
	UPROPERTY()
	FNiagaraDataSetCompiledData DataLayout;

	bool Init(const UNiagaraSystem* System, UNiagaraDataInterfaceDataChannelWrite* OwnerDI);
};

USTRUCT()
struct FNDIDataChannelWriteSimCacheFrameBuffer
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<uint8> Data;

	UPROPERTY()
	int32 Size = 0;

	UPROPERTY()
	FNiagaraVariableBase SourceVar;
};

USTRUCT()
struct FNDIDataChannelWriteSimCacheFrame
{
	GENERATED_BODY()

	UPROPERTY()
	int32 NumElements = 0;

	UPROPERTY()
	TArray<FNDIDataChannelWriteSimCacheFrameBuffer> VariableData;

	UPROPERTY()
	bool bVisibleToGame = false;

	UPROPERTY()
	bool bVisibleToCPUSims = false;

	UPROPERTY()
	bool bVisibleToGPUSims = false;
};

UCLASS(MinimalAPI)
class UNDIDataChannelWriteSimCacheData : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TArray<FNDIDataChannelWriteSimCacheFrame> FrameData;

	UPROPERTY()
	FSoftObjectPath DataChannelReference;

	UPROPERTY(Transient)
	TObjectPtr<const UNiagaraDataInterfaceDataChannelWrite> DataInterface = nullptr;

	FNiagaraSystemInstanceID InstanceID; 
};

/**
The Data Channel Writer Data Interface allows us to write data into a Niagara Data Channel.
We can specify where the data is made visible with visibility flags for Game, CPU and GPU.
These can help reduce costs by copying data only where it's needed.
*/
UCLASS(EditInlineNew, Category = "Data Channels", CollapseCategories, meta = (DisplayName = "Data Channel Writer"), MinimalAPI)
class UNiagaraDataInterfaceDataChannelWrite : public UNiagaraDataInterface, public INiagaraSimCacheCustomStorageInterface
{
	GENERATED_UCLASS_BODY()

public:

	/** How should we allocate the buffer into which we write data. Currently only supports static allocation but in future will support more flexible allocation modes. */
	UPROPERTY(EditAnywhere, Category = "Data Channel")
	ENiagaraDataChannelAllocationMode AllocationMode = ENiagaraDataChannelAllocationMode::Static;

	/** How many elements to allocate for writing per frame? Usage depends on AllocationMode. */
	UPROPERTY(EditAnywhere, Category = "Data Channel", meta = (EditCondition="AllocationMode == ENiagaraDataChannelAllocationMode::Static"))
	uint32 AllocationCount = 0;

	/** 
	The data generated by this interface is made visible the Game code and BP. This is required to allow game C++ and BP to read this data. 
	*/
	UPROPERTY(EditAnywhere, Category = "Data Channel", meta = (DisplayName = "Visible to Blueprint"))
	bool bPublishToGame = false;

	/** 
	Data generated by this interface is made visible to other CPU Niagara emitters. 
	Allows reading this data in CPU Particle scripts as well as Emitter and System scripts.
	Also required for this data to be used in Spawning functions on the Data Channel Read Interface such as Spawn Conditional and Spawn Direct.
	If these spawning functions are used to spawn into a GPU emitter, the relevant data will also be made visible to the GPU. 
	There is no need to also set "Visible to GPU Systems".
	*/
	UPROPERTY(EditAnywhere, Category = "Data Channel", meta = (DisplayName = "Visible to CPU systems"))
	bool bPublishToCPU = false;

	/** 
	Data generated by this interface will be made visible to GPU Niagara emitters.
	Set this only if you need to directly read this data from Niagara emitters on the GPU.
	If you are intending to use this data for Spawning particles via Spawn Conditional or Spawn Direct then you should not set this.
	In that case, set only "Visible To CPU systems" and if required by a GPU emitter, the data will be passed back to the GPU automatically for use in initializing spawned particles.
	*/
	UPROPERTY(EditAnywhere, Category = "Data Channel", meta = (DisplayName = "Visible to GPU systems"))
	bool bPublishToGPU = false;
	
	/**
	The target Data Channel data for this interface will be refreshed every frame.
	Some Data Channels have multiple separate data elements for things such as spatial subdivision. 
	Each DI will request the correct one for it's owning system instance from the data channel. 
	Depending on the Data Channel this could be an expensive search so we should avoid doing this every tick if possible.
	*/
	UPROPERTY(EditAnywhere, Category = "Data Channel", AdvancedDisplay)
	bool bUpdateDestinationDataEveryTick = true;

	/**
	 * If a system is ticking multiple times in a single frame (for example due to fixed tick delta time or because the user is scrubbing the desired time in the timeline),
	 * it is usually not desirable to also write data channel entries for each subtick, as consumers reading on the next frame will potentially see a huge number of entries.
	 *
	 * When enabled, the emitter will only execute write functions on the last subtick of the frame.
	 */
	UPROPERTY(EditAnywhere, Category = "Data Channel", AdvancedDisplay)
	bool bOnlyWriteOnceOnSubticks = true;

	/** The Data Channel to write to. */
	UPROPERTY(EditAnywhere, Category = "Data Channel")
	TObjectPtr<UNiagaraDataChannelAsset> Channel;

	/** Access Context used when accessing NDC data. Controls which of the NDCs internal data it will return for this Interface to write into. */
	UPROPERTY(EditAnywhere, Category = "Data Channel")
	FNDCAccessContextInst AccessContext;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;

#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL)override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL) override;

	NIAGARA_API virtual void PostCompile(const UNiagaraSystem& OwningSystem) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

#if WITH_EDITOR
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo);
	NIAGARA_API virtual void ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors);
#endif

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }

	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

	virtual bool HasTickGroupPostreqs() const override;
	virtual ETickingGroup CalculateFinalTickGroup(const void* PerInstanceData) const override;

	//We cannot overlap frames as we must correctly sync up with the data channel manager on Begin/End frame etc.
	virtual bool PostSimulateCanOverlapFrames() const override { return false; }
	//We cannot have post stage overlap tick groups so that the write DI can publish it's contents to the data channel at the correct time to allow same frame reads.
	virtual bool PostStageCanOverlapTickGroups() const override { return false; }

	virtual uint32 GetGpuCountBufferEstimate() const { return 1; }
	//UNiagaraDataInterface Interface

	NIAGARA_API virtual void InitAccessContext();

	NIAGARA_API void Num(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void Allocate(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void Write(FVectorVMExternalFunctionContext& Context, int32 FuncIdx);
	NIAGARA_API void Append(FVectorVMExternalFunctionContext& Context, int32 FuncIdx);

	const FNDIDataChannelWriteCompiledData& GetCompiledData()const { return CompiledData; }

	bool ShouldPublish()const { return bPublishToGame || bPublishToCPU || bPublishToGPU; }

	//sim cache functions
	virtual UObject* SimCacheBeginWrite(UObject* SimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const override;
	virtual bool SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const override;
	virtual bool SimCacheEndWrite(UObject* StorageObject) const override;
	virtual bool SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext) override;
	virtual void SimCachePostReadFrame(void* OptionalPerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual bool SimCacheCompareFrame(const UObject* LhsStorageObject, const UObject* RhsStorageObject, int FrameIndex, TOptional<float> Tolerance, FString& OutErrors) const override;
protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	UPROPERTY()
	FNDIDataChannelWriteCompiledData CompiledData;
};

struct FNiagaraDataInterfaceProxy_DataChannelWrite : public FNiagaraDataInterfaceProxyRW
{
	FNiagaraDataInterfaceProxy_DataChannelWrite()
	: FNiagaraDataInterfaceProxyRW()
	{
	}

	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override;
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override;

	virtual void PreStage(const FNDIGpuComputePreStageContext& Context)override;
	virtual void PostStage(const FNDIGpuComputePostStageContext& Context)override;
	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context)override;

	/** Persistent per instance data on the RT. Constructed when consuming data passed from GT->RT. */
	struct FInstanceData
	{
		//GPU Dataset from the channel handler. We'll grab the current buffer from this on the RT.
		//This must be grabbed fresh from the handler each frame as it's lifetime cannot be ensured.
		FNiagaraDataChannelDataProxyPtr ChannelDataRTProxy = nullptr;

		/**
		A buffer containing layout information needed to access parameters for each script using this DI.
		*/
		FReadBuffer ParameterLayoutBuffer;

		TResourceArray<uint32> ParameterLayoutData;

		/**
		Offsets into the parameter table are embedded in the gpu script hlsl.
		At hlsl gen time we can only know which parameter are accessed by each script individually so each script must have it's own parameter binding table.
		*/
		TMap<FNiagaraCompileHash, uint32> GPUScriptParameterTableOffsets;

		//This is the buffer we'll write into for shipping data back to the CPU & Game.
		FNiagaraDataBufferRef BufferForCPU;

		//This is the main GPU buffer we'll be writing into.
		FNiagaraDataBufferRef GPUBuffer;

		bool bPublishToGame = false;
		bool bPublishToCPU = false;
		bool bPublishToGPU = false;

		int32 AllocationCount = 0;
		FVector3f LwcTile;

		mutable bool bCapturingSimCache = false;
		mutable TArray<FNiagaraDataBufferRef> PendingSimCacheReadbacks;
	};

	TMap<FNiagaraSystemInstanceID, FInstanceData> SystemInstancesToProxyData_RT;
};
