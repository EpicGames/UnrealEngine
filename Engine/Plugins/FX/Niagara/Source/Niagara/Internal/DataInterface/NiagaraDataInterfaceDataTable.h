// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"

#include "NiagaraDataInterfaceDataTable.generated.h"

class UDataTable;

/**
Data interface that allows you to read rows from data tables.
You can read data either using a list of filtered row names, directly by index.
*/
UCLASS(Experimental, EditInlineNew, CollapseCategories, meta = (DisplayName = "Data Table"), MinimalAPI)
class UNiagaraDataInterfaceDataTable : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	// UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	// UObject Interface End

	// UNiagaraDataInterface Interface Begin
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
public:
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL) override;
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void PostCompile(const UNiagaraSystem& OwningSystem) override;
#endif
#if WITH_EDITOR
	virtual bool GetGpuUseIndirectDispatch() const override { return true; }
#endif
	NIAGARA_API FNiagaraDataInterfaceParametersCS* CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const override;
	NIAGARA_API const FTypeLayoutDesc* GetShaderStorageType() const override;
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	// UNiagaraDataInterface Interface End

#if WITH_EDITOR
	NIAGARA_API static bool IsReadFunction(const FNiagaraFunctionSignature& Signature);
	NIAGARA_API static TArray<FNiagaraVariableBase> GetVariablesFromDataTable(const UDataTable* DataTable);
#endif

private:
	/** Default DataTable to use can be overriden using the parameter binding. */
	UPROPERTY(EditAnywhere, Category = "DataTable")
	TObjectPtr<UDataTable> DataTable;

	/** List of Row Names to read from the DataTable. */
	UPROPERTY(EditAnywhere, Category = "DataTable")
	TArray<FName> FilteredRowNames;

	/** Parameter binding that can be used to override the default table. */
	UPROPERTY(EditAnywhere, Category = "DataTable")
	FNiagaraUserParameterBinding ObjectParameterBinding;

	UPROPERTY()
	bool bCreateFilteredTable = true;
};
