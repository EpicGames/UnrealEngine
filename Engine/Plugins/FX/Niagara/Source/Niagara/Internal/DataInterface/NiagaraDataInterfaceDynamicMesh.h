// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraDataInterface.h"
#include "NiagaraRenderableMeshInterface.h"
#include "NiagaraSimCacheCustomStorageInterface.h"
#include "VectorVM.h"

#include "NiagaraDataInterfaceDynamicMesh.generated.h"

USTRUCT()
struct FNiagaraDynamicMeshSection
{
	GENERATED_BODY()

	/** Number of triangles in this section. */
	UPROPERTY(EditAnywhere, Category = "Section")
	int32 NumTriangles = 0;

	/** Index of the material to use. */
	UPROPERTY(EditAnywhere, Category = "Section")
	int32 MaterialIndex = 0;

	bool operator==(const FNiagaraDynamicMeshSection& Other) const
	{
		return NumTriangles == Other.NumTriangles && MaterialIndex == Other.MaterialIndex;
	}
};

USTRUCT()
struct FNiagaraDynamicMeshMaterial
{
	GENERATED_BODY()

	FNiagaraDynamicMeshMaterial();

	/** Optional material to use for this section. */
	UPROPERTY(EditAnywhere, Category = "Material")
	TObjectPtr<UMaterialInterface> Material;

	/** Optional binding to a user parameter */
	UPROPERTY(EditAnywhere, Category = "Material")
	FNiagaraUserParameterBinding MaterialUserParamBinding;

	bool operator==(const FNiagaraDynamicMeshMaterial& Other) const
	{
		return Material == Other.Material && MaterialUserParamBinding == Other.MaterialUserParamBinding;
	}
};

/** Data Interface that can create mesh geometry at runtime from triangle data (e.g. to render a custom generated mesh per particle) */
UCLASS(Experimental, EditInlineNew, Category = "Mesh", CollapseCategories, meta = (DisplayName = "Dynamic Mesh"), MinimalAPI)
class UNiagaraDataInterfaceDynamicMesh : public UNiagaraDataInterface, public INiagaraRenderableMeshInterface, public INiagaraSimCacheCustomStorageInterface
{
	GENERATED_UCLASS_BODY()

public:
	/**
	Sections to render, each section will generally result in a draw call.
	Triangles are contiguous between sections, i.e. Section[1] triangles will begin after Section[0].NumTriangles
	*/
	UPROPERTY(EditAnywhere, Category = "Dynamic Mesh")
	TArray<FNiagaraDynamicMeshSection> Sections;

	/** List of materials to use */
	UPROPERTY(EditAnywhere, Category = "Dynamic Mesh")
	TArray<FNiagaraDynamicMeshMaterial> Materials;

	/** Allocates space for the number of vertices we will use, leave as zero if you intend to allocate dynamically via VM calls. */
	UPROPERTY(EditAnywhere, Category = "Dynamic Mesh")
	int32 NumVertices = 0;

	/** Allocates space for the number of texture coordinates requested. */
	UPROPERTY(EditAnywhere, Category = "Dynamic Mesh")
	int NumTexCoords = 0;

	/** Allocates space for vertex colors when enabled. */
	UPROPERTY(EditAnywhere, Category = "Dynamic Mesh")
	bool bHasColors = false;

	/** Allocates space for tangent basis when enabled. */
	UPROPERTY(EditAnywhere, Category = "Dynamic Mesh")
	bool bHasTangentBasis = false;

	/** Should we auto clear section triangle allocations per frame or not. */
	UPROPERTY(EditAnywhere, Category = "Dynamic Mesh")
	bool bClearTrianglesPerFrame = false;

	/** Should we auto clear section triangle allocations per frame or not. */
	UPROPERTY(EditAnywhere, Category = "Dynamic Mesh")
	FBox DefaultBounds = FBox(FVector(-100.0f), FVector(100.0f));

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;

	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	//UNiagaraDataInterface Interface End

	//INiagaraRenderableMeshInterface Interface Begin
	NIAGARA_API virtual FNiagaraRenderableMeshPtr GetRenderableMesh(FNiagaraSystemInstanceID SystemInstanceID) override;
	NIAGARA_API virtual void GetUsedMaterials(FNiagaraSystemInstanceID SystemInstanceID, TArray<UMaterialInterface*>& OutMaterials) const override;
	//INiagaraRenderableMeshInterface Interface End

	//~ INiagaraSimCacheCustomStorageInterface interface BEGIN
	virtual UObject* SimCacheBeginWrite(UObject* SimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const override;
	virtual bool SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const override;
	virtual bool SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext) override;
	virtual bool SimCacheCompareFrame(const UObject* LhsStorageObject, const UObject* RhsStorageObject, int FrameIndex, TOptional<float> Tolerance, FString& OutErrors) const override;
	//~ UNiagaraDataInterface interface END

	// VM Functions
	static NIAGARA_API void VMGetMeshProperties(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMGetSectionCount(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMGetSectionData(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMGetLocalBounds(FVectorVMExternalFunctionContext& Context);

	static NIAGARA_API void VMSetMeshProperties(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMSetSectionCount(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMSetSectionData(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMSetLocalBounds(FVectorVMExternalFunctionContext& Context);

	static NIAGARA_API void VMClearAllSectionTriangles(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMClearSectionTriangles(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMAllocateSectionTriangles(FVectorVMExternalFunctionContext& Context);

	static NIAGARA_API void VMGetTriangleVertices(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMGetVertexPosition(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMGetVertexTangentBasis(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMGetVertexTexCoord(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMGetVertexColor(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMGetVertexData(FVectorVMExternalFunctionContext& Context);

	static NIAGARA_API void VMSetTriangleVertices(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMSetVertexPosition(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMSetVertexTangentBasis(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMSetVertexTexCoord(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMSetVertexColor(FVectorVMExternalFunctionContext& Context);
	static NIAGARA_API void VMSetVertexData(FVectorVMExternalFunctionContext& Context);

	static NIAGARA_API void VMAppendTriangle(FVectorVMExternalFunctionContext& Context);

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
};
