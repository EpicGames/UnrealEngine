// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"

#include "NiagaraDataInterfaceDynamicMeshSimCacheData.generated.h"

class UMaterialInterface;

USTRUCT()
struct FNDIDynamicMeshSimCacheSection
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Section")
	uint32 TriangleOffset = 0;

	UPROPERTY(EditAnywhere, Category = "Section")
	uint32 MaxTriangles = 0;

	UPROPERTY(EditAnywhere, Category = "Section")
	uint32 AllocatedTriangles = 0;

	UPROPERTY(EditAnywhere, Category = "Section")
	TObjectPtr<UMaterialInterface> Material;

	bool operator==(const FNDIDynamicMeshSimCacheSection& Other) const
	{
		return
			TriangleOffset == Other.TriangleOffset &&
			MaxTriangles == Other.MaxTriangles &&
			AllocatedTriangles == Other.AllocatedTriangles &&
			Material == Other.Material;
	}
};

USTRUCT()
struct FNDIDynamicMeshSimCacheFrame
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category ="Mesh")
	uint32 NumTriangles = 0;

	UPROPERTY(EditAnywhere, Category = "Mesh")
	uint32 NumVertices = 0;

	UPROPERTY(EditAnywhere, Category = "Mesh")
	uint32 NumTexCoords = 0;

	UPROPERTY(EditAnywhere, Category = "Mesh")
	FBox LocalBounds = FBox(FVector(-100.0f), FVector(100.0f));

	UPROPERTY(EditAnywhere, Category = "Mesh")
	uint32 VertexBufferSize = 0;

	UPROPERTY(EditAnywhere, Category = "Mesh")
	uint32 PositionOffset = INDEX_NONE;

	UPROPERTY(EditAnywhere, Category = "Mesh")
	uint32 TangentBasisOffset = INDEX_NONE;

	UPROPERTY(EditAnywhere, Category = "Mesh")
	uint32 TexCoordOffset = INDEX_NONE;

	UPROPERTY(EditAnywhere, Category = "Mesh")
	uint32 ColorOffset = INDEX_NONE;

	UPROPERTY()
	TArray<uint8> IndexData;

	UPROPERTY()
	TArray<uint8> VertexData;

	UPROPERTY(EditAnywhere, Category = "Mesh")
	TArray<FNDIDynamicMeshSimCacheSection> Sections;

	bool Equals(const FNDIDynamicMeshSimCacheFrame& Other, TConstArrayView<uint8> OtherIndexData, TConstArrayView<uint8> OtherVertexData) const
	{
		return
			NumTriangles == Other.NumTriangles &&
			NumVertices == Other.NumVertices &&
			NumTexCoords == Other.NumTexCoords &&
			LocalBounds == Other.LocalBounds &&
			VertexBufferSize == Other.VertexBufferSize &&
			PositionOffset == Other.PositionOffset &&
			TangentBasisOffset == Other.TangentBasisOffset &&
			TexCoordOffset == Other.TexCoordOffset &&
			ColorOffset == Other.ColorOffset&&
			Sections == Other.Sections &&
			IndexData == OtherIndexData &&
			VertexData == OtherVertexData;
	}

	bool Equals(const FNDIDynamicMeshSimCacheFrame& Other) const
	{
		return Equals(Other, IndexData, VertexData);
	}
};

UCLASS(MinimalAPI)
class UNiagaraDataInterfaceDynamicMeshSimCacheData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<FNDIDynamicMeshSimCacheFrame> UniqueFrames;

	// Indicies into UniqueFrames where INDEX_NONE means no data
	UPROPERTY()
	TArray<int> CpuFrameIndices;

	UPROPERTY()
	TArray<int> GpuFrameIndices;

	void ResizeFrames(int FrameIndex)
	{
		while (CpuFrameIndices.Num() <= FrameIndex)
		{
			CpuFrameIndices.Add(INDEX_NONE);
		}
		while (GpuFrameIndices.Num() <= FrameIndex)
		{
			GpuFrameIndices.Add(INDEX_NONE);
		}
	}

	int32 FindOrCreateMesh(const FNDIDynamicMeshSimCacheFrame& MeshDesc, TConstArrayView<uint8> IndexData, TConstArrayView<uint8>  VertexData)
	{
		if (MeshDesc.VertexBufferSize == 0 || IndexData.Num() == 0 || VertexData.Num() == 0)
		{
			return INDEX_NONE;
		}

		if (UniqueFrames.Num() > 0)
		{
			const FNDIDynamicMeshSimCacheFrame& LastFrame = UniqueFrames.Last();
			if (LastFrame.Equals(MeshDesc, IndexData, VertexData))
			{
				return UniqueFrames.Num() - 1;
			}
		}

		const int32 MeshIndex = UniqueFrames.Add(MeshDesc);
		UniqueFrames[MeshIndex].IndexData = IndexData;
		UniqueFrames[MeshIndex].VertexData = VertexData;
		return MeshIndex;
	}

	const FNDIDynamicMeshSimCacheFrame* GetFrameData(int32 FrameIndex, ENiagaraSimTarget SimTarget) const
	{
		int32 MeshIndex = INDEX_NONE;
		if (SimTarget == ENiagaraSimTarget::CPUSim)
		{
			MeshIndex = CpuFrameIndices.IsValidIndex(FrameIndex) ? CpuFrameIndices[FrameIndex] : INDEX_NONE;
		}
		else if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			MeshIndex = GpuFrameIndices.IsValidIndex(FrameIndex) ? GpuFrameIndices[FrameIndex] : INDEX_NONE;
		}
		return UniqueFrames.IsValidIndex(MeshIndex) ? &UniqueFrames[MeshIndex] : nullptr;
	}
};
