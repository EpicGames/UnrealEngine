// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraDataInterface.h"
#include "NiagaraParameterBinding.h"
#include "NiagaraRenderableMeshInterface.h"
#include "NiagaraSimCacheCustomStorageInterface.h"

#include "VectorVM.h"

#include "NiagaraDataInterfaceMemoryBuffer.generated.h"

struct FNiagaraSimCacheFeedbackContext;

/**
Data interface used to access a memory buffer.
The user is responsible for how data is read / wrote.
The DI will ensure no out of bounds access but not that the elements are of the correct type (i.e. float or int)
Elements are considered to be 4 bytes in size.
*/
UCLASS(MinimalAPI, Experimental, EditInlineNew, Category = "Data Interface", meta = (DisplayName = "Memory Buffer"))
class UNiagaraDataInterfaceMemoryBuffer : public UNiagaraDataInterface, public INiagaraSimCacheCustomStorageInterface
{
	GENERATED_UCLASS_BODY()

public:
	/** Default space we will allocate for the memory buffer. */
	UPROPERTY(EditAnywhere, Category = "Byte Buffer")
	int32 DefaultNumElements = 0;

	/** How should we synhronize the data between CPU and GPU memory. */
	UPROPERTY(EditAnywhere, Category = "Byte Buffer")
	ENiagaraGpuSyncMode GpuSyncMode = ENiagaraGpuSyncMode::SyncCpuToGpu;

	//UObject Interface
	virtual void PostInitProperties() override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	virtual int32 PerInstanceDataSize() const override;
	virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;

	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }
	virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;

	virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	//UNiagaraDataInterface Interface End

	//~ INiagaraSimCacheCustomStorageInterface interface BEGIN
	virtual UObject* SimCacheBeginWrite(UObject* SimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const override;
	virtual bool SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const override;
	//virtual bool SimCacheEndWrite(UObject* StorageObject) const override;
	virtual bool SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext) override;
	//~ UNiagaraDataInterface interface END
};

USTRUCT()
struct FNDIMemoryBufferSimCacheDataFrame
{
	GENERATED_BODY()
		
	UPROPERTY()
	int32 CpuBufferSize = 0;

	UPROPERTY()
	int32 CpuDataOffset = INDEX_NONE;

	UPROPERTY()
	int32 GpuBufferSize = 0;

	UPROPERTY()
	int32 GpuDataOffset = INDEX_NONE;
};

UCLASS(MinimalAPI)
class UNDIMemoryBufferSimCacheData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<FNDIMemoryBufferSimCacheDataFrame> FrameData;

	UPROPERTY()
	TArray<uint32> BufferData;
};
