// Copyright Epic Games, Inc. All Rights Reserved.
// NOTE: Temporary code do not use as it will likely be removed in a future version

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"

#include "NiagaraDataInterfacePropertyInterface.generated.h"

/**
Data interface that can read from property interfaces.
*/
UCLASS(Experimental, EditInlineNew, CollapseCategories, meta = (DisplayName = "Property Interface"), MinimalAPI)
class UNiagaraDataInterfacePropertyInterface : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	// UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	// UObject Interface End

	// UNiagaraDataInterface Interface Begin
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;

	virtual bool HasPreSimulateTick() const override { return true; }
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;

	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
public:
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL) override;
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void PostCompile(const UNiagaraSystem& OwningSystem) override;
#endif
	NIAGARA_API FNiagaraDataInterfaceParametersCS* CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const override;
	NIAGARA_API const FTypeLayoutDesc* GetShaderStorageType() const override;
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
#if WITH_NIAGARA_DEBUGGER
	virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const override;
#endif
	// UNiagaraDataInterface Interface End

	bool HasValidReference() const
	{
		return !InterfaceName.IsNone() && !InterfacePath.IsNone() && !InterfacePackage.IsNone();
	}

	NIAGARA_API TArray<FNiagaraVariableBase> FindUsedVariables(UNiagaraSystem* OwningSystem);

public:
	/** The Class name for the interface */
	UPROPERTY(EditAnywhere, Category = "PropertyInterface")
	FName InterfaceName;

	/** Fully qualified path to the interface, i.e. /MyPath.org/MySubPath/MySubPath */
	UPROPERTY(EditAnywhere, Category = "PropertyInterface")
	FName InterfacePath;

	/** Package the interface comes from, i.e. /MyPackage/MyPackage */
	UPROPERTY(EditAnywhere, Category = "PropertyInterface")
	FName InterfacePackage;

	/** For HasA interfaces the member name */
	//UPROPERTY(EditAnywhere, Category = "PropertyInterface")
	//FName MemberName;
};
