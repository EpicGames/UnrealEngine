// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraSimCacheCustomStorageInterface.h"

#include "VectorVM.h"
#include "NiagaraDataInterfaceRW.h"
#include "HAL/PlatformAtomics.h"

#include "NiagaraDataInterfaceSimpleCounter.generated.h"

struct FNiagaraDataInterfaceGeneratedFunction;

/**
Thread safe counter starts at the initial value on start / reset.
When operating between CPU & GPU ensure you set the appropriate sync mode.
*/
UCLASS(EditInlineNew, Category = "Counting", CollapseCategories, meta = (DisplayName = "Simple Counter"), MinimalAPI)
class UNiagaraDataInterfaceSimpleCounter : public UNiagaraDataInterfaceRWBase, public INiagaraSimCacheCustomStorageInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(int32,		CountOffset)
	END_SHADER_PARAMETER_STRUCT();

public:
	// UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// UObject Interface End

	// UNiagaraDataInterface Interface Begin
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;

	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
#endif
#if WITH_EDITOR
	virtual bool GetGpuUseIndirectDispatch() const override { return true; }
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	NIAGARA_API virtual void PushToRenderThreadImpl() override;
	// UNiagaraDataInterface Interface End

	//~ INiagaraSimCacheCustomStorageInterface interface BEGIN
	virtual UObject* SimCacheBeginWrite(UObject* SimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const override;
	virtual bool SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const override;
	virtual bool SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext) override;
	virtual bool SimCacheCompareFrame(const UObject* LhsStorageObject, const UObject* RhsStorageObject, int FrameIndex, TOptional<float> Tolerance, FString& OutErrors) const override;
	//~ UNiagaraDataInterface interface END

	void UpdateDIProxy();

	// VM functions
	NIAGARA_API void VMGet(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMSet(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMExchange(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMAdd(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMIncrement(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMDecrement(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void GetNextValue_Deprecated(FVectorVMExternalFunctionContext& Context);

	/** Select how we should synchronize the counter between Cpu & Gpu */
	UPROPERTY(EditAnywhere, Category = "Simple Counter")
	ENiagaraGpuSyncMode GpuSyncMode = ENiagaraGpuSyncMode::None;

	/** This is the value the counter will have when the instance is reset / created. */
	UPROPERTY(EditAnywhere, Category = "Simple Counter")
	int32 InitialValue = 0;

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
};

UCLASS(MinimalAPI)
class UNDISimpleCounterSimCacheData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<int32> Values;
};
