// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"
#include "NiagaraParameterStore.h"
#include "NiagaraDataInterfaceSocketReader.generated.h"

class AActor;

UENUM()
enum class ENDISocketReaderSourceMode : uint8
{
	// Reads in the order of Parameter Binding -> Attached Parent -> Source
	Default,
	// Read from the parameter binding only
	ParameterBindingOnly,
	// Read from the attached parent only
	// This will traverse the attachment hierarchy
	AttachedParentOnly,
	// Read from the source only
	// This will read the Source Actor first then Source Asset
	SourceOnly,
};

/**
Data interface for reading sockets from various sources.
This can be from a live component in the scene or from a static / skeletal mesh asset.
*/
UCLASS(EditInlineNew, Category = "Actor", CollapseCategories, meta = (DisplayName = "Socket Reader"), MinimalAPI)
class UNiagaraDataInterfaceSocketReader : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(uint32,							IsDataValid)
		SHADER_PARAMETER(float,								InvDeltaSeconds)
		SHADER_PARAMETER(int32,								NumSockets)
		SHADER_PARAMETER(int32,								NumFilteredSockets)
		SHADER_PARAMETER(int32,								NumUnfilteredSockets)
		SHADER_PARAMETER(FVector3f,							ComponentToTranslatedWorld_Translation)
		SHADER_PARAMETER(FQuat4f,							ComponentToTranslatedWorld_Rotation)
		SHADER_PARAMETER(FVector3f,							ComponentToTranslatedWorld_Scale)
		SHADER_PARAMETER(FVector3f,							PreviousComponentToTranslatedWorld_Translation)
		SHADER_PARAMETER(FQuat4f,							PreviousComponentToTranslatedWorld_Rotation)
		SHADER_PARAMETER(FVector3f,							PreviousComponentToTranslatedWorld_Scale)
		SHADER_PARAMETER(uint32,							SocketTransformOffset)
		SHADER_PARAMETER(uint32,							PreviousSocketTransformOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer,	SocketData)
	END_SHADER_PARAMETER_STRUCT();

public:
	/** Controls how we find the object we want to read sockets from. */
	UPROPERTY(EditAnywhere, Category = "SocketReader")
	ENDISocketReaderSourceMode SourceMode = ENDISocketReaderSourceMode::Default;

	/** List of filtered sockets. */
	UPROPERTY(EditAnywhere, Category = "SocketReader")
	TArray<FName> FilteredSockets;

#if WITH_EDITORONLY_DATA
	/** When previewing in the editor this is the asset to use to gather the socket information. */
	UPROPERTY(EditAnywhere, Category = "SocketReader", meta = (AllowedClasses = "/Script/Engine.StaticMesh,/Script/Engine.SkeletalMesh"))
	TSoftObjectPtr<UObject> EditorPreviewAsset;
#endif

	/** Source actor to read sockets from. */
	UPROPERTY(EditAnywhere, Category = "SocketReader", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISocketReaderSourceMode::Default || SourceMode == ENDISocketReaderSourceMode::SourceOnly"))
	TLazyObjectPtr<AActor> SourceActor;

	/** Source object asset to read sockets from, the transsform for these would be in relation to the Niagara system. */
	UPROPERTY(EditAnywhere, Category = "SocketReader", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISocketReaderSourceMode::Default || SourceMode == ENDISocketReaderSourceMode::SourceOnly", AllowedClasses="/Script/Engine.StaticMesh,/Script/Engine.SkeletalMesh"))
	TObjectPtr<UObject> SourceAsset;

	/** When looking for an attached parent component only accept this type of component. */
	UPROPERTY(EditAnywhere, Category = "SocketReader", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISocketReaderSourceMode::Default || SourceMode == ENDISocketReaderSourceMode::AttachedParentOnly", AllowedClasses="/Script/Engine.SceneComponent,/EntityFramework/_Verse/VNI/Entity.component"))
	TObjectPtr<UClass> AttachComponentClass;

	/** When looking for an attached parent component it must have this tag to be considered. */
	UPROPERTY(EditAnywhere, Category = "SocketReader", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISocketReaderSourceMode::Default || SourceMode == ENDISocketReaderSourceMode::AttachedParentOnly"))
	FName AttachComponentTag;

	/**
	Source object parameter binding.
	Note: Source Mode impacts the order of operations.
	*/
	UPROPERTY(EditAnywhere, Category = "SocketReader", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISocketReaderSourceMode::Default || SourceMode == ENDISocketReaderSourceMode::ParameterBindingOnly"))
	FNiagaraUserParameterBinding ObjectParameterBinding;

	/**
	When enabled we will read update the sockets transforms each frame.
	This is not required in all cases as the sockets might not be able to move.
	*/
	UPROPERTY(EditAnywhere, Category = "Performance")
	bool bUpdateSocketsPerFrame = true;

	/** When this option is disabled, we use the previous frame's data for the skeletal mesh and can often issue the simulation early. This greatly
	reduces overhead and allows the game thread to run faster, but comes at a tradeoff if the dependencies might leave gaps or other visual artifacts.*/
	UPROPERTY(EditAnywhere, Category = "Performance")
	bool bRequireCurrentFrameData = true;
	
	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;

	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

	virtual bool HasTickGroupPrereqs() const override { return true; }
	NIAGARA_API virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	virtual bool HasPreSimulateTick() const override { return true; }

#if WITH_NIAGARA_DEBUGGER
	virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const override;
#endif
	//UNiagaraDataInterface Interface

#if WITH_EDITORONLY_DATA
	// Returns a list of sockets if the editor preview asset is valid
	NIAGARA_API TArray<FName> GetEditorSocketNames() const;
#endif

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
};
