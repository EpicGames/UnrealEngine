// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterfaceMeshCommon.h"
#include "StaticMeshResources.h"
#include "Engine/StaticMesh.h"
#include "Experimental/NiagaraMeshUvMappingHandle.h"
#include "NiagaraDataInterfaceStaticMesh.generated.h"

UENUM()
enum class ENDIStaticMesh_SourceMode : uint8
{
	/**
	Default behavior follows the order of.
	- Use "Source" when specified (either set explicitly or via blueprint with Set Niagara Static Mesh Component).
	- Use Mesh Parameter Binding if valid
	- Find Static Mesh Component, Attached Actor, Attached Component
	- Falls back to 'Default Mesh' specified on the data interface
	*/
	Default,

	/**	Only use "Source" (either set explicitly or via blueprint with Set Niagara Static Mesh Component). */
	Source,

	/**	Only use the parent actor or component the system is attached to. */
	AttachParent,

	/** Only use the "Default Mesh" specified. */
	DefaultMeshOnly,

	/** Only use the mesh parameter binding. */
	MeshParameterBinding,
};

USTRUCT()
struct FNDIStaticMeshSectionFilter
{
	GENERATED_USTRUCT_BODY();

	/** Only allow sections these material slots. */
	UPROPERTY(EditAnywhere, Category="Section Filter")
	TArray<int32> AllowedMaterialSlots;

	//Others?
	//Banned material slots
	
	void Init(class UNiagaraDataInterfaceStaticMesh* Owner, bool bAreaWeighted);
	FORCEINLINE bool CanEverReject()const { return AllowedMaterialSlots.Num() > 0; }
};

/** Data Interface allowing sampling of static meshes. */
UCLASS(EditInlineNew, Category = "Meshes", meta = (DisplayName = "Static Mesh Reader"), MinimalAPI)
class UNiagaraDataInterfaceStaticMesh : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	/** Controls how to retrieve the Static Mesh Component to attach to. */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	ENDIStaticMesh_SourceMode SourceMode = ENDIStaticMesh_SourceMode::Default;

#if WITH_EDITORONLY_DATA
	/** Mesh used to sample from when not overridden by a source actor from the scene. Only available in editor for previewing. This is removed in cooked builds. */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	TSoftObjectPtr<UStaticMesh> PreviewMesh;
#endif

	/** Mesh used to sample from when not overridden by a source actor from the scene. This mesh is NOT removed from cooked builds. */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	TObjectPtr<UStaticMesh> DefaultMesh;

protected:
	/** The source actor from which to sample. Takes precedence over the direct mesh. Note that this can only be set when used as a user variable on a component in the world. */
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (DisplayName = "Source Actor"))
	TSoftObjectPtr<AActor> SoftSourceActor;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<AActor> Source_DEPRECATED;
#endif

	/** The source component from which to sample. Takes precedence over the direct mesh. Not exposed to the user, only indirectly accessible from blueprints. */
	UPROPERTY(Transient)
	TWeakObjectPtr<UStaticMeshComponent> SourceComponent;
public:

	/** Array of filters the can be used to limit sampling to certain sections of the mesh. */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	FNDIStaticMeshSectionFilter SectionFilter;

	/** If true we capture the transforms from the mesh component each frame. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Mesh")
	bool bCaptureTransformsPerFrame = true;

	/** If true then the mesh velocity is taken from the mesh component's physics data. Otherwise it will be calculated by diffing the component transforms between ticks, which is more reliable but won't work on the first frame. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Mesh")
    bool bUsePhysicsBodyVelocity = false;

	/** When true, we allow this DI to sample from streaming LODs. Selectively overriding the CVar fx.Niagara.NDIStaticMesh.UseInlineLODsOnly. */
	UPROPERTY(EditAnywhere, Category = "LOD")
	bool bAllowSamplingFromStreamingLODs = false;

	/** 
	Static Mesh LOD to sample.
	When the desired LOD is not available, the next available LOD is used.
	When LOD Index is negative, Desired LOD = Num LODs - LOD Index.
	*/
	UPROPERTY(EditAnywhere, Category = "LOD")
	int32 LODIndex = 0;

	/** Reference to a user parameter if we're reading one. */
	UPROPERTY(EditAnywhere, Category = "LOD")
	FNiagaraUserParameterBinding LODIndexUserParameter;

	/** Mesh parameter binding can be either an Actor (in which case we find the component), static mesh component or a static mesh. */
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (AdditionalAllowedClasses="/Script/Engine.StaticMesh"))
	FNiagaraUserParameterBinding MeshParameterBinding;

	/** When attached to an Instanced Static Mesh, which instance should be read from. */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	int32 InstanceIndex = INDEX_NONE;

	/** List of filtered sockets to use. */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	TArray<FName> FilteredSockets;

    /** Changed within the editor on PostEditChangeProperty. Should be changed whenever a refresh is desired.*/
	uint32 ChangeId = 0;

	//~ UObject interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	NIAGARA_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif
	virtual bool CanBeInCluster() const override { return false; }	// Note: Due to BP functionality we can change a UObject property on this DI we can not put into a cluster
	//~ UObject interface

	//~ UNiagaraDataInterface interface
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual int32 PerInstanceDataSize() const override;

	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

	static NIAGARA_API bool FunctionNeedsCpuAccess(FName InName);

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
	virtual bool RequiresEarlyViewData() const override;
	virtual bool HasPreSimulateTick() const override { return true; }

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void ModifyCompilationEnvironment(EShaderPlatform ShaderPlatform, struct FShaderCompilerEnvironment& OutEnvironment) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
	NIAGARA_API void GetVertexSamplingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const;
	NIAGARA_API void GetTriangleSamplingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const;
	NIAGARA_API void GetSocketSamplingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const;
	NIAGARA_API void GetSectionFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const;
	NIAGARA_API void GetMiscFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const;
	NIAGARA_API void GetUVMappingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const;
	NIAGARA_API void GetDistanceFieldFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const;
	NIAGARA_API void GetDeprecatedFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
public:
#if WITH_NIAGARA_DEBUGGER
	NIAGARA_API virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const override;
#endif
#if WITH_EDITOR
	NIAGARA_API virtual void GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors,
		TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo) override;
#endif
	//~ UNiagaraDataInterface interface

	NIAGARA_API UStaticMesh* ResolveStaticMesh(UObject*& OutComponent, class FNiagaraSystemInstance* SystemInstance = nullptr, UObject* ParameterBindingValue = nullptr) const;
	NIAGARA_API void SetSourceComponentFromBlueprints(UStaticMeshComponent* ComponentToUse);
	NIAGARA_API void SetDefaultMeshFromBlueprints(UStaticMesh* MeshToUse);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Static Mesh DI Instance Index"))
	static NIAGARA_API void SetNiagaraStaticMeshDIInstanceIndex(UNiagaraComponent* NiagaraSystem, const FName UserParameterName, int32 NewInstanceIndex);

protected:
	// Bind/unbind delegates to release references to the source actor & component.
	NIAGARA_API void UnbindSourceDelegates();
	NIAGARA_API void BindSourceDelegates();

	UFUNCTION()
	NIAGARA_API void OnSourceEndPlay(AActor* InSource, EEndPlayReason::Type Reason);

	// VM Vertex Sampling
	NIAGARA_API void VMIsValidVertex(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMRandomVertex(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetVertexCount(FVectorVMExternalFunctionContext& Context);

	template<typename TTransformHandler>
	void VMGetVertex(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetVertexInterpolated(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetVertexColor(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetVertexUV(FVectorVMExternalFunctionContext& Context);

	// VM Triangle Sampling
	NIAGARA_API void VMIsValidTriangle(FVectorVMExternalFunctionContext& Context);
	template<typename TRandomHelper>
	void VMRandomTriangle(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetTriangleCount(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void VMIsValidFilteredTriangle(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMRandomFilteredTriangle(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetFilteredTriangleCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetFilteredTriangleAt(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void VMIsValidUnfilteredTriangle(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMRandomUnfilteredTriangle(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetUnfilteredTriangleCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetUnfilteredTriangleAt(FVectorVMExternalFunctionContext& Context);

	template<typename TTransformHandler>
	void VMGetTriangle(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetTriangleInterpolated(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetTriangleColor(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetTriangleUV(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void VMGetTriangleIndices(FVectorVMExternalFunctionContext& Context);

	// Socket Functions
	NIAGARA_API void VMGetSocketCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetFilteredSocketCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetUnfilteredSocketCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMRandomSocket(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMRandomFilteredSocket(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMRandomUnfilteredSocket(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetSocketTransform(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetSocketTransformInterpolated(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetFilteredSocketTransform(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetFilteredSocketTransformInterpolated(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetUnfilteredSocketTransform(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetUnfilteredSocketTransformInterpolated(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetFilteredSocket(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetUnfilteredSocket(FVectorVMExternalFunctionContext& Context);

	// Section functions
	NIAGARA_API void VMIsValidSection(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetSectionTriangleCount(FVectorVMExternalFunctionContext& Context);
	template<typename TRandomHandler>
	void VMRandomSectionTriangle(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetSectionTriangleAt(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void VMGetFilteredSectionAt(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetUnfilteredSectionAt(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void VMGetSectionCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetFilteredSectionCount(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetUnfilteredSectionCount(FVectorVMExternalFunctionContext& Context);

	template<typename TRandomHandler>
	void VMRandomSection(FVectorVMExternalFunctionContext& Context);
	template<typename TRandomHandler>
	void VMRandomFilteredSection(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMRandomUnfilteredSection(FVectorVMExternalFunctionContext& Context);

	// VM Misc Functions
	NIAGARA_API void VMIsValid(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void VMGetPreSkinnedLocalBounds(FVectorVMExternalFunctionContext& Context);
	template<bool bLocalSpace>
	void VMGetMeshBounds(FVectorVMExternalFunctionContext& Context);

	NIAGARA_API void VMGetLocalToWorld(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetLocalToWorldInverseTransposed(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetWorldVelocity(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetInstanceIndex(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMSetInstanceIndex(FVectorVMExternalFunctionContext& Context);

	// VM UV mapping functions
	NIAGARA_API void VMGetTriangleCoordAtUV(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMGetTriangleCoordInAabb(FVectorVMExternalFunctionContext& Context);
	NIAGARA_API void VMBuildUvMapping(FVectorVMExternalFunctionContext& Context);;

	// Deprecated VM Functions
	template<typename TTransformHandler>
	void VMGetVertexPosition_Deprecated(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetTriPosition_Deprecated(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetTriPositionAndVelocity_Deprecated(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetTriangleTangentBasis_Deprecated(FVectorVMExternalFunctionContext& Context);
	template<typename TTransformHandler>
	void VMGetTriangleNormal_Deprecated(FVectorVMExternalFunctionContext& Context);
};

class FNDI_StaticMesh_GeneratedData : public FNDI_GeneratedData
{
	FRWLock CachedUvMappingGuard;
	TArray<TSharedPtr<FStaticMeshUvMapping>> CachedUvMapping;

public:
	NIAGARA_API FStaticMeshUvMappingHandle GetCachedUvMapping(TWeakObjectPtr<UStaticMesh>& InMeshObject, int32 InLodIndex, int32 InUvSetIndex, FMeshUvMappingUsage Usage, bool bNeedsDataImmediately);

	NIAGARA_API virtual void Tick(ETickingGroup TickGroup, float DeltaSeconds) override;

	static TypeHash GetTypeHash()
	{
		static const TypeHash Hash = FCrc::Strihash_DEPRECATED(TEXT("FNDI_StaticMesh_GeneratedData"));
		return Hash;
	}
};
