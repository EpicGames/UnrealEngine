// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "NiagaraDataInterfaceStaticMeshIndirect.generated.h"

class UNiagaraSystem;

USTRUCT()
struct FNDIStaticMeshIndirectCompiledData
{
	GENERATED_BODY()
	
	/** True if this DI is used by a CPU emitter. */
	UPROPERTY()
	uint32 bUsedByCPU : 1 = false;
	
	/** True if this DI is used by a GPU emitter. */
	UPROPERTY()
	uint32 bUsedByGPU : 1 = false;
	
	/** True if this DI requires that it's referenced mesh has CPU Access Enabled. */
	UPROPERTY()
	uint32 bNeedsCpuAccess : 1 = false;

	/** True if this DI requires static mesh distance field data. */
	UPROPERTY()
	uint32 bRequiresDistanceFields : 1 = false;
	
	void Init(const UNiagaraSystem* System, UNiagaraDataInterface* OwnerDI);
};

/** 
Data Interface allowing limited access of static mesh info indirectly via an FNiagaraSharedDataID_StaticMesh. 
Available data and features are far more limited than a regular Static Mesh DI but its indirect nature allows it to be used without direct binding to a single static mesh.
*/
UCLASS(EditInlineNew, Category = "Meshes", meta = (DisplayName = "Static Mesh Reader Indirect"), MinimalAPI)
class UNiagaraDataInterfaceStaticMeshIndirect : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:

	//~ UObject interface
	NIAGARA_API virtual void PostInitProperties() override;
	//~ UObject interface
	
	//~ UNiagaraDataInterface interface
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	NIAGARA_API virtual void PostCompile(const UNiagaraSystem& OwningSystem) override;
#endif
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
	virtual bool RequiresEarlyViewData() const override;

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void ModifyCompilationEnvironment(EShaderPlatform ShaderPlatform, struct FShaderCompilerEnvironment& OutEnvironment) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;	
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
	NIAGARA_API void GetDistanceFieldFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const;
#endif
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
public:
	//~ UNiagaraDataInterface interface

	const FNDIStaticMeshIndirectCompiledData& GetCompiledData()const { return CompiledData; }
	
private:

 	UPROPERTY()
	FNDIStaticMeshIndirectCompiledData CompiledData;
};
