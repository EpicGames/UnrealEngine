// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraAnalytics.h"
#include "EngineAnalytics.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/Class.h"

void NiagaraAnalytics::RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Niagara.") + MoveTemp(EventName), Attributes);
	}
}

void NiagaraAnalytics::RecordEvent(FString&& EventName, const FString& AttributeName, const FString& AttributeValue)
{
	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Add(FAnalyticsEventAttribute(AttributeName, AttributeValue));
	RecordEvent(MoveTemp(EventName), Attributes);
}

bool NiagaraAnalytics::IsPluginAsset(const UObject* Obj)
{
	if (IsValid(Obj) && Obj->IsAsset() && Obj->GetPackage())
	{
		FString Name = Obj->GetPackage()->GetName();
		return Name.StartsWith(TEXT("/Niagara/"), ESearchCase::CaseSensitive) || Name.StartsWith(TEXT("/NiagaraFluids/"), ESearchCase::CaseSensitive);
	}
	return false;
}

bool NiagaraAnalytics::IsPluginClass(const UClass* Class)
{
	if (Class)
	{
		FString Name = Class->GetPackage()->GetName();
		return Name == TEXT("/Script/Niagara") || Name == TEXT("/Script/NiagaraFluids");
	}
	return false;
}
