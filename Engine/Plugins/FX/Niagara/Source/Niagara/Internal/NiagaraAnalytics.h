// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnalyticsEventAttribute.h"

namespace NiagaraAnalytics
{
	NIAGARA_API void RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes = TArray<FAnalyticsEventAttribute>());
	
	NIAGARA_API void RecordEvent(FString&& EventName, const FString& AttributeName, const FString& AttributeValue);

	// Checks if the provided object is part of a Niagara plugin and can safely be reported on (e.g. the asset name)
	NIAGARA_API bool IsPluginAsset(const UObject* Obj);

	// Checks if the provided class is part of a Niagara plugin and can safely be reported on (e.g. the asset name)
	NIAGARA_API bool IsPluginClass(const UClass* Class);
};