// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "NiagaraBoundsCalculator.h"
#include "NiagaraDataSet.h"
#include "NiagaraDataSetAccessor.h"

template<bool bUsedWithSprites, bool bUsedWithMeshes, bool bUsedWithRibbons>
class FNiagaraBoundsCalculatorHelper : public FNiagaraBoundsCalculator
{
public:

	FNiagaraBoundsCalculatorHelper() = default;
	FNiagaraBoundsCalculatorHelper(const FVector& InMeshExtents, const FVector& InMaxLocalMeshOffset, const FVector& InMaxWorldMeshOffset, bool bInLocalSpace)
		: MeshExtents(InMeshExtents)
		, MaxLocalMeshOffset(InMaxLocalMeshOffset)
		, MaxWorldMeshOffset(InMaxWorldMeshOffset)
		, bLocalSpace(bInLocalSpace)
	{}

	virtual void InitAccessors(const FNiagaraDataSetCompiledData* CompiledData) override final
	{
		static const FName PositionName(TEXT("Position"));
		static const FName SpriteSizeName(TEXT("SpriteSize"));
		static const FName ScaleName(TEXT("Scale"));
		static const FName RibbonWidthName(TEXT("RibbonWidth"));

		PositionAccessor.Init(CompiledData, PositionName);
		if (bUsedWithSprites)
		{
			SpriteSizeAccessor.Init(CompiledData, SpriteSizeName);
		}
		if (bUsedWithMeshes)
		{
			ScaleAccessor.Init(CompiledData, ScaleName);
		}
		if (bUsedWithRibbons)
		{
			RibbonWidthAccessor.Init(CompiledData, RibbonWidthName);
		}
	}

	virtual FBox CalculateBounds(const FTransform& SystemTransform, const FNiagaraDataSet& DataSet, const int32 NumInstances) const override final
	{
		if (!NumInstances || !PositionAccessor.IsValid())
		{
			return FBox(ForceInit);
		}

		constexpr float kDefaultSize = 50.0f;

		FNiagaraPosition BoundsMin(ForceInitToZero);
		FNiagaraPosition BoundsMax(ForceInitToZero);
		PositionAccessor.GetReader(DataSet).GetMinMax(BoundsMin, BoundsMax);
		FBox Bounds(BoundsMin, BoundsMax);

		float MaxSize = KINDA_SMALL_NUMBER;
		if (bUsedWithMeshes)
		{
			FVector3f MaxScale(1.0f, 1.0f, 1.0f);
			if (ScaleAccessor.IsValid())
			{
				MaxScale = ScaleAccessor.GetReader(DataSet).GetMax();
			}

			// NOTE: Since we're not taking particle rotation into account we have to treat the extents like a sphere,
			// which is a little bit more conservative, but saves us having to rotate the extents per particle
			const FVector ScaledExtents = MeshExtents * FVector(MaxScale);
			MaxSize = FMath::Max(MaxSize, float(ScaledExtents.Size()));
			
			FVector MaxTransformedOffset;
			if (bLocalSpace)
			{
				MaxTransformedOffset = MaxLocalMeshOffset.ComponentMax(SystemTransform.InverseTransformVector(MaxWorldMeshOffset));
			}
			else
			{
				MaxTransformedOffset = MaxWorldMeshOffset.ComponentMax(SystemTransform.TransformVector(MaxLocalMeshOffset));
			}

			// We have to extend the min/max by the offset
			Bounds.Max = Bounds.Max.ComponentMax(Bounds.Max + MaxTransformedOffset);
			Bounds.Min = Bounds.Min.ComponentMin(Bounds.Min - MaxTransformedOffset);
		}

		if (bUsedWithSprites)
		{
			float MaxSpriteSize = kDefaultSize;

			if (SpriteSizeAccessor.IsValid())
			{
				const FVector2f MaxSpriteSize2D = SpriteSizeAccessor.GetReader(DataSet).GetMax();
				MaxSpriteSize = FMath::Max(MaxSpriteSize2D.X, MaxSpriteSize2D.Y);
			}
			MaxSize = FMath::Max(MaxSize, FMath::IsNearlyZero(MaxSpriteSize) ? 1.0f : MaxSpriteSize);
		}

		if (bUsedWithRibbons)
		{
			float MaxRibbonWidth = kDefaultSize;
			if (RibbonWidthAccessor.IsValid())
			{
				MaxRibbonWidth = RibbonWidthAccessor.GetReader(DataSet).GetMax();
			}

			MaxSize = FMath::Max(MaxSize, FMath::IsNearlyZero(MaxRibbonWidth) ? 1.0f : MaxRibbonWidth);
		}

		return Bounds.ExpandBy(MaxSize);
	}

	FNiagaraDataSetAccessor<FNiagaraPosition> PositionAccessor;
	FNiagaraDataSetAccessor<FVector2f> SpriteSizeAccessor;
	FNiagaraDataSetAccessor<FVector3f> ScaleAccessor;
	FNiagaraDataSetAccessor<float> RibbonWidthAccessor;

	const FVector MeshExtents = FVector::OneVector;
	const FVector MaxLocalMeshOffset = FVector::ZeroVector;
	const FVector MaxWorldMeshOffset = FVector::ZeroVector;
	const bool bLocalSpace = false;
};
