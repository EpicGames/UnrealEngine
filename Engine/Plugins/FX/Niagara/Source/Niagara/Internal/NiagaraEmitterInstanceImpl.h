// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSet.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraScriptExecutionContext.h"

class FNiagaraSystemInstance;
struct FNiagaraEmitterHandle;
class UNiagaraParameterCollection;
class UNiagaraParameterCollectionInstance;
class FNiagaraGpuComputeDispatchInterface;
struct FNiagaraEmitterCompiledData;

/**
* Implementation of a stateful Niagara particle simulation
*/
class FNiagaraEmitterInstanceImpl final : public FNiagaraEmitterInstance
{
	using Super = FNiagaraEmitterInstance;

	friend class UNiagaraSimCache;

private:
	struct FEventInstanceData
	{
		TArray<FNiagaraScriptExecutionContext> EventExecContexts;
		TArray<FNiagaraParameterDirectBinding<int32>> EventExecCountBindings;

		TArray<FNiagaraDataSet*> UpdateScriptEventDataSets;
		TArray<FNiagaraDataSet*> SpawnScriptEventDataSets;

		TArray<bool> UpdateEventGeneratorIsSharedByIndex;
		TArray<bool> SpawnEventGeneratorIsSharedByIndex;
		
		/** Data required for handling events. */
		TArray<FNiagaraEventHandlingInfo> EventHandlingInfo;
		int32 EventSpawnTotal = 0;
	};

public:
	explicit FNiagaraEmitterInstanceImpl(FNiagaraSystemInstance* InParentSystemInstance);
	virtual ~FNiagaraEmitterInstanceImpl();

	// FNiagaraEmitterInstance Impl
	virtual void Init(int32 InEmitterIndex) override;
	virtual void ResetSimulation(bool bKillExisting = true) override;
	virtual void SetEmitterEnable(bool bNewEnableState) override;
	virtual bool HandleCompletion(bool bForce = false) override;
	virtual void OnPooledReuse() override;
	virtual FNiagaraEmitterInstanceImpl* AsStateful() override { return this; }
	virtual TConstArrayView<UNiagaraRendererProperties*> GetRenderers() const override;
	virtual void BindParameters(bool bExternalOnly) override;
	virtual void UnbindParameters(bool bExternalOnly) override;
	virtual void RebindParameterCollection(UNiagaraParameterCollectionInstance* OldCollection, UNiagaraParameterCollectionInstance* NewCollection) override;
	virtual bool ShouldTick() const override;
	virtual void PreTick() override;
	virtual void Tick(float DeltaSeconds) override;
	// FNiagaraEmitterInstance Impl

	void PostTick();

	void InitDITickLists();

	void DirtyDataInterfaces();

	bool IsAllowedToExecute() const;

#if WITH_EDITOR
	void TickRapidIterationParameters();
#endif

	bool RequiresPersistentIDs() const;

	uint32 CalculateEventSpawnCount(const FNiagaraEventScriptProperties &EventHandlerProps, TArray<int32, TInlineAllocator<16>>& EventSpawnCounts, FNiagaraDataSet *EventSet);

#if WITH_EDITOR
	/** Potentially reads back data from the GPU which will introduce a stall and should only be used for debug purposes. */
	NIAGARA_API void CalculateFixedBounds(const FTransform& ToWorldSpace);
#endif

	FORCEINLINE const FNiagaraEmitterScalabilitySettings& GetScalabilitySettings() const { check(VersionedEmitter.GetEmitterData() != nullptr); return VersionedEmitter.GetEmitterData()->GetScalabilitySettings(); }

	//ENiagaraExecutionState GetExecutionState() { return ExecutionState; }
	void NIAGARA_API SetExecutionState(ENiagaraExecutionState InState);

	FNiagaraScriptExecutionContext& GetSpawnExecutionContext() { return SpawnExecContext; }
	FNiagaraScriptExecutionContext& GetUpdateExecutionContext() { return UpdateExecContext; }
	TArrayView<FNiagaraScriptExecutionContext> GetEventExecutionContexts();

	FORCEINLINE FName GetCachedIDName() const { return CachedIDName; }

	TArray<FNiagaraSpawnInfo>& GetSpawnInfo() { return SpawnInfos; }

	NIAGARA_API bool IsReadyToRun() const;

	void Dump()const;

	bool WaitForDebugInfo();

	bool HasTicked() const { return TickCount > 0;  }

	int32 GetRandomSeed() const { return RandomSeed; }
	int32 GetInstanceSeed() const { return InstanceSeed; }

	void SetParticleComponentActive(FObjectKey ComponentKey, int32 ParticleID) const;

	bool IsParticleComponentActive(FObjectKey ComponentKey, int32 ParticleID) const;

	virtual void UpdatePSOStatus() override;

private:
	void CheckForErrors();

	void BuildConstantBufferTable(
		const FNiagaraScriptExecutionContext& ExecContext,
		FScriptExecutionConstantBufferTable& ConstantBufferTable) const;

	/** Generate emitter bounds */
	FBox InternalCalculateDynamicBounds(int32 ParticleCount) const;

	/** Array of all spawn info driven by our owning emitter script. */
	TArray<FNiagaraSpawnInfo> SpawnInfos;

	FNiagaraScriptExecutionContext SpawnExecContext;
	FNiagaraScriptExecutionContext UpdateExecContext;

	FNiagaraParameterDirectBinding<float> SpawnIntervalBinding;
	FNiagaraParameterDirectBinding<float> InterpSpawnStartBinding;
	FNiagaraParameterDirectBinding<int32> SpawnGroupBinding;

	FNiagaraParameterDirectBinding<int32> SpawnExecCountBinding;
	FNiagaraParameterDirectBinding<int32> UpdateExecCountBinding;

	TSharedPtr<const FNiagaraEmitterCompiledData> EmitterCompiledData;

	TUniquePtr<FEventInstanceData> EventInstanceData;

	TSharedPtr<FNiagaraGpuComputeDispatchInterface> ComputeDispatchInterfacePtr;

	/** Raw pointer to the emitter that we're instanced from. Raw ptr should be safe here as we check for the validity of the system and it's emitters higher up before any ticking. */
	//FVersionedNiagaraEmitter CachedEmitter;
	FName CachedIDName;

	/* The age of the emitter*/
	float EmitterAge = 0.0f;

	int32 RandomSeed = 0;
	int32 InstanceSeed = FGenericPlatformMath::Rand();
	int32 TickCount = 0;

	uint32 MaxRuntimeAllocation = 0;

	int32 MaxAllocationCount = 0;
	int32 MinOverallocation = -1;
	int32 ReallocationCount = 0;

	uint32 MaxInstanceCount = 0;

	/** Typical resets must be deferred until the tick as the RT could still be using the current buffer. */
	uint32 bResetPending : 1 = false;

	/** Used with SetEmitterEnable */
	uint32 bAllowSpawning_GT : 1 = true;
	/** Used with SetEmitterEnable */
	uint32 bAllowSpawning_CNC : 1 = true;

	// This is used to keep track which particles have spawned a component. This is needed when the bOnlyCreateComponentsOnParticleSpawn flag is set in the renderer.
	// Without this bookkeeping, the particles would lose their components when the render state is recreated or the visibility tag flips them off and on again.
	mutable TMap<FObjectKey, TSet<int32>> ParticlesWithComponents;
};
