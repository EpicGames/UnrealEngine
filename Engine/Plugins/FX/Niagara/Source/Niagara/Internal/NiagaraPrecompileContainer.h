// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraSystem.h"
#include "NiagaraScript.h"
#include "NiagaraPrecompileContainer.generated.h"


UCLASS(MinimalAPI)
class UNiagaraPrecompileContainer : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY()
	TArray<TObjectPtr<UNiagaraScript>> Scripts;

	UPROPERTY()
	TObjectPtr<UNiagaraSystem> System;
};
