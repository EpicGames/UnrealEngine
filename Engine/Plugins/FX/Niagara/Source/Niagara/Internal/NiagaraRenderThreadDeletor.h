// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "RHICommandList.h"
#include "RenderingThread.h"

// Use with unique / shared pointers to ensure the class is deleted on the rendering thread once no references exist
// Example usage: MakeShareable(new FMyClass(), FNiagaraRenderThreadDeletor<FMyClass>())
template<typename TType>
struct FNiagaraRenderThreadDeletor
{
	void operator()(TType* ObjectToDelete) const
	{
		if (IsInRenderingThread())
		{
			delete ObjectToDelete;
		}
		else
		{
			ENQUEUE_RENDER_COMMAND(NiagaraRenderThreadDeletor)
			(
				[ObjectToDelete](FRHICommandListImmediate&)
				{
					delete ObjectToDelete;
				}
			);
		}
	}
};
