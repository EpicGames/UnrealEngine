// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PrimitiveComponentId.h"
#include "MeshBatch.h"

#include "NiagaraRendererReadback.generated.h"

class UNiagaraComponent;
class FNiagaraSceneProxy;
class FPrimitiveSceneProxy;
class FSceneView;

#ifndef WITH_NIAGARA_RENDERER_READBACK
	// Console variable CVarNiagarVertexFactoryExportEnabledMode is used to control if the permutations are generated for the taget platform
	// You will need to modify both that and this define in order to run on none editor targets
	#define WITH_NIAGARA_RENDERER_READBACK WITH_EDITOR
#endif

UENUM()
enum class ENiagaraRendererReadbackCoordinateSpace : uint8
{
	/**
	Data is captured in world space.
	large world coordinates will be truncated.
	*/
	World,
	/** 
	Data is captured in local space to the primitive.
	*/
	Local,
};

// Note: Changing this requires changing NiagaraVertexFactoryExport.usf
UENUM()
enum class ENiagaraRendererReadbackAttribute : uint32
{
	DynamicParameter0X,
	DynamicParameter0Y,
	DynamicParameter0Z,
	DynamicParameter0W,

	DynamicParameter1X,
	DynamicParameter1Y,
	DynamicParameter1Z,
	DynamicParameter1W,

	DynamicParameter2X,
	DynamicParameter2Y,
	DynamicParameter2Z,
	DynamicParameter2W,

	DynamicParameter3X,
	DynamicParameter3Y,
	DynamicParameter3Z,
	DynamicParameter3W,

	TexCoord0U,
	TexCoord0V,

	TexCoord1U,
	TexCoord1V,

	TexCoord2U,
	TexCoord2V,

	TexCoord3U,
	TexCoord3V,

	TexCoord4U,
	TexCoord4V,

	TexCoord5U,
	TexCoord5V,

	TexCoord6U,
	TexCoord6V,

	TexCoord7U,
	TexCoord7V,

	Num UMETA(Hidden)
};


USTRUCT(BlueprintType)
struct FNiagaraRendererReadbackParameters
{
	GENERATED_BODY()

	// What coordinate space to use when capturing positions / tangents / etc.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	ENiagaraRendererReadbackCoordinateSpace CoordinateSpace = ENiagaraRendererReadbackCoordinateSpace::World;

	// When enabled vertex positions will be exported
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	bool bExportPosition = true;

	// When enabled the vertex tangent basis will be exported
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	bool bExportTangentBasis = true;

	// When enabled vertex colors will be exported
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	bool bExportColor = true;
	//-TODO: Add Color mode (i.e. Vertex Color / Particle Color / Disabled)

	// How many vertex texture coordinates to export
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(UIMin="0", UIMax="8", ClampMin="0", ClampMax="8"));
	int32 ExportNumTexCoords = 1;

	// Optional custom bindings for texture coordinate packing from the export process
	// TexCoord0.X -> TexCoord0.Y, TexCoord1.X -> etc
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	TArray<ENiagaraRendererReadbackAttribute> ExportTexCoordBindings;

	// When enabled we will attempt to export the materials used with each section
	// When disabled no materials will be assigned, so the default material will be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	bool bExportMaterials = true;

	// When enabled material WPO will be included in the exported data
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	bool bApplyWPO = false;

	// When set we capture the batches from the view index provided
	// When unset we capture batches from all views
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	TOptional<int> ViewIndexToCapture = 0;
};

#if WITH_NIAGARA_RENDERER_READBACK
struct FNiagaraRendererReadbackResult
{
	struct FSection
	{
		TWeakObjectPtr<UMaterialInterface>	WeakMaterialInterface;
		uint32								FirstTriangle = 0;
		uint32								NumTriangles = 0;
	};

	FNiagaraRendererReadbackResult()
	{
		for (int i = 0; i < UE_ARRAY_COUNT(VertexTexCoordBindings); ++i)
		{
			VertexTexCoordBindings[i] = int(ENiagaraRendererReadbackAttribute::TexCoord0U) + i;
		}
	}

	bool HasPosition() const { return VertexPositionOffset != INDEX_NONE; }
	bool HasColor() const { return VertexColorOffset != INDEX_NONE; }
	bool HasTangentBasis() const { return VertexTangentBasisOffset != INDEX_NONE; }
	bool HasTexCoords() const { return VertexTexCoordOffset != INDEX_NONE; }

	NIAGARA_API FVector3f GetPosition(uint32 Vertex) const;
	NIAGARA_API FLinearColor GetColor(uint32 Vertex) const;
	NIAGARA_API FVector3f GetTangentX(uint32 Vertex) const;
	NIAGARA_API FVector3f GetTangentY(uint32 Vertex) const;
	NIAGARA_API FVector3f GetTangentZ(uint32 Vertex) const;
	NIAGARA_API FVector2f GetTexCoord(uint32 Vertex, uint32 TexCoordIndex) const;

	uint32				NumVertices = 0;
	uint32				VertexStride = 0;
	uint32				VertexPositionOffset = INDEX_NONE;
	uint32				VertexColorOffset = INDEX_NONE;
	uint32				VertexTangentBasisOffset = INDEX_NONE;
	uint32				VertexTexCoordOffset = INDEX_NONE;
	uint32				VertexTexCoordNum = 0;
	uint32				VertexTexCoordBindings[8 * 2];
	TArray<uint8>		VertexData;
	TArray<FSection>	Sections;

	TArray<FString>		Errors;
};

using FNiagaraRendererReadbackComplete = TFunction<void(const FNiagaraRendererReadbackResult& Result)>;

namespace NiagaraRendererReadback
{
	extern uint32 GIsCapturing;

	// Capture the next frame's renderer data for the provided components
	// These functions are expected to be called on the Game Thread
	// The Callback will also be invoked on the GameThread once complete
	NIAGARA_API void EnqueueReadback(TConstArrayView<UNiagaraComponent*> Components, FNiagaraRendererReadbackComplete Callback, const FNiagaraRendererReadbackParameters& Parameters = FNiagaraRendererReadbackParameters());
	NIAGARA_API void EnqueueReadback(UNiagaraComponent* Component, FNiagaraRendererReadbackComplete Callback, const FNiagaraRendererReadbackParameters& Parameters = FNiagaraRendererReadbackParameters());
	// Capture the next frame's renderer data for the provided world
	NIAGARA_API void EnqueueReadback(UWorld* World, FNiagaraRendererReadbackComplete Callback, const FNiagaraRendererReadbackParameters& Parameters = FNiagaraRendererReadbackParameters());

	inline void BeginCapture() { ++GIsCapturing; }
	inline bool IsCapturing() { return GIsCapturing != 0; }
	void CaptureMeshBatch(const FSceneView* View, const FNiagaraSceneProxy* SceneProxy, const FMeshBatch& MeshBatch, uint32 NumInstances, uint32 NumVerticesPerInstance);
	inline void EndCapture() { --GIsCapturing; }
}
#endif //WITH_NIAGARA_RENDERER_READBACK
