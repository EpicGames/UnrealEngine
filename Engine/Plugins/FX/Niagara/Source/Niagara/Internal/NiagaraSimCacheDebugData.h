// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraScript.h"
#include "NiagaraSystemEmitterState.h"

#include "NiagaraSimCacheDebugData.generated.h"

struct FNiagaraSimCacheHelper;

USTRUCT()
struct FNiagaraSimCacheDebugDataFrame
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FString, FNiagaraParameterStore> DebugParameterStores;
};

// Contains data useful for debugging a Niagara system
UCLASS(MinimalAPI)
class UNiagaraSimCacheDebugData : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	void CaptureFrame(FNiagaraSimCacheHelper& Helper, int FrameNumber);

public:
	UPROPERTY(VisibleAnywhere, Category="DebugData")
	FNiagaraSystemStateData SystemStateData;

	UPROPERTY()
	TArray<FNiagaraSimCacheDebugDataFrame> Frames;
};
