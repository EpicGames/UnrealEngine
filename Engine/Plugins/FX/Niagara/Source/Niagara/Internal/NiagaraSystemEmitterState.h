// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "Stateless/NiagaraStatelessDistribution.h"

#include "NiagaraSystemEmitterState.generated.h"

UENUM()
enum class ENiagaraSystemInactiveResponse : uint8
{
	/** Let Emitters Finish Then Kill Emitter */
	Complete,
	/** Emitter & Particles Die Immediatly */
	Kill,
};

UENUM()
enum class ENiagaraEmitterInactiveResponse : uint8
{
	/** Let Particles Finish Then Kill Emitter */
	Complete,
	/** Emitter & Particles Die Immediatly */
	Kill,
	/** Emitter deactivates but doesn't die until the system does */
	//Continue,
};

UENUM()
enum class ENiagaraLoopBehavior : uint8
{
	Infinite,
	Multiple,
	Once,
};

UENUM()
enum class ENiagaraLoopDurationMode : uint8
{
	Fixed,
	Infinite,
};

USTRUCT()
struct FNiagaraSystemStateData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "System State")
	uint32 bRunSpawnScript : 1 = true;

	UPROPERTY(EditAnywhere, Category = "System State")
	uint32 bRunUpdateScript : 1 = true;

	UPROPERTY(EditAnywhere, Category = "System State")
	uint32 bIgnoreSystemState : 1 = true;

	UPROPERTY(EditAnywhere, Category = "System State", meta = (DisplayAfter = "LoopDuration", EditCondition = "LoopBehavior != ENiagaraLoopBehavior::Once", EditConditionHides))
	uint32 bRecalculateDurationEachLoop : 1 = false;

	UPROPERTY(EditAnywhere, Category = "System State", meta = (InlineEditConditionToggle))
	uint32 bLoopDelayEnabled : 1 = false;

	UPROPERTY(EditAnywhere, Category = "System State", meta = (DisplayAfter = "LoopDelay", EditCondition = "bLoopDelayEnabled && LoopBehavior != ENiagaraLoopBehavior::Once", EditConditionHides))
	uint32 bDelayFirstLoopOnly : 1 = false;

	UPROPERTY(EditAnywhere, Category = "System State", meta = (DisplayAfter = "bDelayFirstLoopOnly", EditCondition = "bLoopDelayEnabled && LoopBehavior != ENiagaraLoopBehavior::Once && !bDelayFirstLoopOnly", EditConditionHides))
	uint32 bRecalculateDelayEachLoop : 1 = false;

	UPROPERTY(EditAnywhere, Category = "System State")
	ENiagaraSystemInactiveResponse InactiveResponse = ENiagaraSystemInactiveResponse::Complete;

	UPROPERTY(EditAnywhere, Category = "System State")
	ENiagaraLoopBehavior LoopBehavior = ENiagaraLoopBehavior::Once;

	UPROPERTY(EditAnywhere, Category = "System State", meta = (ClampMin = "0.0", Units="s"))
	FNiagaraDistributionRangeFloat LoopDuration = FNiagaraDistributionRangeFloat(0.0f);

	UPROPERTY(EditAnywhere, Category = "System State", meta = (ClampMin = "1", EditCondition = "LoopBehavior == ENiagaraLoopBehavior::Multiple", EditConditionHides))
	int LoopCount = 1;

	UPROPERTY(EditAnywhere, Category = "System State", meta = (ClampMin = "0.0", Units="s", EditCondition = "bLoopDelayEnabled"))
	FNiagaraDistributionRangeFloat LoopDelay = FNiagaraDistributionRangeFloat(0.0f);

#if WITH_EDITORONLY_DATA
	// The script resolver is only needed in the editor to pull data from RI parameters
	class IScriptResolver
	{
	public:
		virtual ~IScriptResolver() = default;
		virtual void ResolveData(FNiagaraSystemStateData&) const = 0;
	};

	TSharedPtr<IScriptResolver> ScriptResolver;
#endif
};

USTRUCT()
struct FNiagaraEmitterStateData
{
	GENERATED_BODY()
		
	//UPROPERTY(EditAnywhere, Category="Emitter State")
	//ENiagaraStatelessEmitterState_SelfSystem LifeCycleMode = ENiagaraStatelessEmitterState_SelfSystem::Self;

	UPROPERTY(EditAnywhere, Category="Life Cycle", meta=(SegmentedDisplay))
	ENiagaraEmitterInactiveResponse InactiveResponse = ENiagaraEmitterInactiveResponse::Complete;

	UPROPERTY(EditAnywhere, Category="Life Cycle", meta=(SegmentedDisplay))
	ENiagaraLoopBehavior LoopBehavior = ENiagaraLoopBehavior::Infinite;

	UPROPERTY(EditAnywhere, Category = "Life Cycle", meta = (ClampMin = "1", EditCondition = "LoopBehavior == ENiagaraLoopBehavior::Multiple", EditConditionHides))
	int32 LoopCount = 1;

	UPROPERTY(EditAnywhere, Category = "Life Cycle", meta = (EditCondition = "LoopBehavior == ENiagaraLoopBehavior::Once", EditConditionHides, SegmentedDisplay))
	ENiagaraLoopDurationMode LoopDurationMode = ENiagaraLoopDurationMode::Fixed;

	UPROPERTY(EditAnywhere, Category = "Life Cycle", meta = (EditCondition = "LoopBehavior != ENiagaraLoopBehavior::Once || (LoopBehavior == ENiagaraLoopBehavior::Once && LoopDurationMode == ENiagaraLoopDurationMode::Fixed)", EditConditionHides, ClampMin = "0.0", Units="s"))
	FNiagaraDistributionRangeFloat LoopDuration = FNiagaraDistributionRangeFloat(1.0f);

	UPROPERTY(EditAnywhere, Category = "Life Cycle", meta = (ClampMin = "0.0", Units="s", EditCondition="bLoopDelayEnabled"))
	FNiagaraDistributionRangeFloat LoopDelay = FNiagaraDistributionRangeFloat(0.0f);

	UPROPERTY(EditAnywhere, Category = "Life Cycle", meta = (InlineEditConditionToggle))
	uint32 bLoopDelayEnabled : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Life Cycle", meta = (EditCondition = "LoopBehavior != ENiagaraLoopBehavior::Once", EditConditionHides, DisplayAfter = "LoopDuration"))
	uint32 bRecalculateDurationEachLoop : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Life Cycle", meta = (EditCondition = "LoopBehavior != ENiagaraLoopBehavior::Once && bLoopDelayEnabled", EditConditionHides))
	uint32 bDelayFirstLoopOnly : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Life Cycle", meta = (EditCondition = "LoopBehavior != ENiagaraLoopBehavior::Once && bLoopDelayEnabled && !bDelayFirstLoopOnly", EditConditionHides))
	uint32 bRecalculateDelayEachLoop : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Scalability")
	uint32 bEnableDistanceCulling : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayAfter = "MaxDistanceReaction"))
	uint32 bEnableVisibilityCulling : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (InlineEditConditionToggle))
	uint32 bMinDistanceEnabled : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (InlineEditConditionToggle))
	uint32 bMaxDistanceEnabled : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayAfter = "VisibilityCullDelay"))
	uint32 bResetAgeOnAwaken : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayAfter = "bEnableDistanceCulling", EditCondition = "bMinDistanceEnabled"))
	float MinDistance = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayAfter = "MinDistance"))
	ENiagaraExecutionStateManagement MinDistanceReaction = ENiagaraExecutionStateManagement::Awaken;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayAfter = "bEnableDistanceCulling", EditCondition = "bMaxDistanceEnabled"))
	float MaxDistance = 5000.0f;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayAfter = "MaxDistance"))
	ENiagaraExecutionStateManagement MaxDistanceReaction = ENiagaraExecutionStateManagement::SleepAndLetParticlesFinish;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition = "bEnableVisibilityCulling", EditConditionHides))
	ENiagaraExecutionStateManagement VisibilityCullReaction = ENiagaraExecutionStateManagement::SleepAndLetParticlesFinish;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (EditCondition = "bEnableVisibilityCulling", EditConditionHides))
	float VisibilityCullDelay = 1.0f;

#if WITH_EDITORONLY_DATA
	void PostSerialize(const FArchive& Ar);
#endif
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FNiagaraEmitterStateData> : public TStructOpsTypeTraitsBase2<FNiagaraEmitterStateData>
{
	enum
	{
		WithPostSerialize = true,
	};
};
#endif
