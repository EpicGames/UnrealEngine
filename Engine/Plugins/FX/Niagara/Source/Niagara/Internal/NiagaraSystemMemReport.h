// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UObject;
class UNiagaraSystem;

struct FNiagaraSystemMemReport
{
	enum class EReportType
	{
		Basic,
		Verbose,
		Max
	};

	struct FNode
	{
		explicit FNode(UObject* Object, uint32 InDepth);
		explicit FNode(FName CustomName, uint32 InDepth, uint32 ByteSize);

		FName	ObjectName;
		FName	ObjectClass;
		uint32	Depth = 0;
		uint64	ExclusiveSizeBytes = 0;
		uint64	InclusiveSizeBytes = 0;
	};

	NIAGARA_API void GenerateReport(EReportType ReportType, UNiagaraSystem* System);

	TConstArrayView<FNode> GetNodes() const { return Nodes; }
	uint64 GetDataInterfaceSizeBytes() const { return DataInterfaceSizeBytes; }

private:
	uint64 GatherResourceMemory(EReportType ReportType, UNiagaraSystem* OwningSystem, UObject* Object, uint32 Depth);

	TArray<FNode> Nodes;
	uint64 DataInterfaceSizeBytes = 0;
};
