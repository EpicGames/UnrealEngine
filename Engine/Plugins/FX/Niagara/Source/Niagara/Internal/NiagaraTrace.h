// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Trace/Trace.h"

#if !defined(WITH_NIAGARA_INSIGHTS)
	#define WITH_NIAGARA_INSIGHTS (UE_TRACE_ENABLED && !IS_PROGRAM && !UE_BUILD_SHIPPING)
#endif

class UNiagaraComponent;
struct FNiagaraDataChannelPublishRequest;

UE_TRACE_CHANNEL_EXTERN(NiagaraChannel, NIAGARA_API)

#if WITH_NIAGARA_INSIGHTS
namespace FNiagaraInsights
{
	void Update();
	bool IsChannelActive();

	void NotifyComponentActivate(const UNiagaraComponent* Component, bool bReset, bool bIsScalabilityCull, bool bAwaitingActivationDueToNotReady);
	void NotifyComponentDeactivate(const UNiagaraComponent* Component, bool bImmediate, bool bIsScalabilityCull);
	void NotifyComponentComplete(const UNiagaraComponent* Component, bool bExternalCompletion);

	void NotifyDataChannelPublish(const FNiagaraDataChannelPublishRequest& Request, bool bGpuRequest);
	void NotifyDataChannelWrite(const UObject* DataChannel, const TCHAR* SourceName, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU);
}
#else
namespace FNiagaraInsights
{
	inline void Update() {}
	inline bool IsChannelActive() { return false; }

	inline void NotifyComponentActivate(const UNiagaraComponent* Component, bool bReset, bool bIsScalabilityCull, bool bAwaitingActivationDueToNotReady) {}
	inline void NotifyComponentDeactivate(const UNiagaraComponent* Component, bool bImmediate, bool bIsScalabilityCull) {}
	inline void NotifyComponentComplete(const UNiagaraComponent* Component, bool bExternalCompletion) {}

	inline void NotifyDataChannelPublish(const FNiagaraDataChannelPublishRequest& Request, bool bGpuRequest) {}
	inline void NotifyDataChannelWrite(const UObject* DataChannel, const TCHAR* SourceName, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU) {}
}
#endif //WITH_NIAGARA_INSIGHTS
