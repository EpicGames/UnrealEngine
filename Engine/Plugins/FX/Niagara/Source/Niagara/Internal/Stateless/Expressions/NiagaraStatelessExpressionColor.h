// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessExpression.h"

#include "NiagaraStatelessExpressionColor.generated.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FNiagaraStatelessExpressionColor : public FNiagaraStatelessExpression
{
	GENERATED_BODY()

	virtual FNiagaraTypeDefinition GetOutputTypeDef() const override final { return FNiagaraTypeDefinition::GetColorDef(); }
	virtual FInstancedStruct Build(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override final;
	virtual void Evaluate(const FEvaluateContext& Context, void* ValueAddress) const override final { *static_cast<FLinearColor*>(ValueAddress) = EvaluateInternal(Context); }

	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const { checkNoEntry(); return FInstancedStruct(); }
	virtual FLinearColor EvaluateInternal(const FEvaluateContext& Context) const { checkNoEntry(); return FLinearColor::Black; }
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (ValueExpression))
struct FNiagaraStatelessExpressionColorConstant final : public FNiagaraStatelessExpressionColor
{
	GENERATED_BODY()

	virtual ~FNiagaraStatelessExpressionColorConstant() = default;
	virtual FLinearColor EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values")
	FLinearColor A = FLinearColor::Black;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (BindingExpression))
struct FNiagaraStatelessExpressionColorBinding final : public FNiagaraStatelessExpressionColor
{
	GENERATED_BODY()

	virtual ~FNiagaraStatelessExpressionColorBinding() = default;
	virtual bool IsConstant() const override { return false; }
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FLinearColor EvaluateInternal(const FEvaluateContext& Context) const override;

	UPROPERTY(EditAnywhere, Category = "Values")
	FName A;

	int32 ParameterOffset = INDEX_NONE;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Add Color"))
struct FNiagaraStatelessExpressionColorAdd final : public FNiagaraStatelessExpressionColor
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionColorAdd();
	virtual ~FNiagaraStatelessExpressionColorAdd() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FLinearColor EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionColor"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionColor"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Subtract Color"))
struct FNiagaraStatelessExpressionColorSubtract final : public FNiagaraStatelessExpressionColor
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionColorSubtract();
	virtual ~FNiagaraStatelessExpressionColorSubtract() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FLinearColor EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionColor"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionColor"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Multiply Color"))
struct FNiagaraStatelessExpressionColorMultiply final : public FNiagaraStatelessExpressionColor
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionColorMultiply();
	virtual ~FNiagaraStatelessExpressionColorMultiply() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FLinearColor EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionColor"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionColor"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Divide Color"))
struct FNiagaraStatelessExpressionColorDivide final : public FNiagaraStatelessExpressionColor
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionColorDivide();
	virtual ~FNiagaraStatelessExpressionColorDivide() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FLinearColor EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionColor"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionColor"))
	FInstancedStruct B;
};
