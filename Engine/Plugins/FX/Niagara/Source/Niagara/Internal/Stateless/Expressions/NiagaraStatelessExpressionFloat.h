// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessExpression.h"

#include "NiagaraStatelessExpressionFloat.generated.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FNiagaraStatelessExpressionFloat : public FNiagaraStatelessExpression
{
	GENERATED_BODY()

	virtual FNiagaraTypeDefinition GetOutputTypeDef() const override final { return FNiagaraTypeDefinition::GetFloatDef(); }
	virtual FInstancedStruct Build(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override final;
	virtual void Evaluate(const FEvaluateContext& Context, void* ValueAddress) const override final { *static_cast<float*>(ValueAddress) = EvaluateInternal(Context); }

	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const { checkNoEntry(); return FInstancedStruct(); }
	virtual float EvaluateInternal(const FEvaluateContext& Context) const { checkNoEntry(); return 0.0f; }
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (ValueExpression))
struct FNiagaraStatelessExpressionFloatConstant final : public FNiagaraStatelessExpressionFloat
{
	GENERATED_BODY()

	virtual ~FNiagaraStatelessExpressionFloatConstant() = default;
	virtual float EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values")
	float A = 0.0f;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (BindingExpression))
struct FNiagaraStatelessExpressionFloatBinding final : public FNiagaraStatelessExpressionFloat
{
	GENERATED_BODY()

	virtual ~FNiagaraStatelessExpressionFloatBinding() = default;
	virtual bool IsConstant() const override { return false; }
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual float EvaluateInternal(const FEvaluateContext& Context) const override;

	UPROPERTY(EditAnywhere, Category = "Values")
	FName A;

	int32 ParameterOffset = INDEX_NONE;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Add Float"))
struct FNiagaraStatelessExpressionFloatAdd final : public FNiagaraStatelessExpressionFloat
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionFloatAdd();
	virtual ~FNiagaraStatelessExpressionFloatAdd() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual float EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionFloat"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionFloat"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Subtract Float"))
struct FNiagaraStatelessExpressionFloatSubtract final : public FNiagaraStatelessExpressionFloat
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionFloatSubtract();
	virtual ~FNiagaraStatelessExpressionFloatSubtract() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual float EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionFloat"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionFloat"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Multiply Float"))
struct FNiagaraStatelessExpressionFloatMultiply final : public FNiagaraStatelessExpressionFloat
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionFloatMultiply();
	virtual ~FNiagaraStatelessExpressionFloatMultiply() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual float EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionFloat"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionFloat"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Divide Float"))
struct FNiagaraStatelessExpressionFloatDivide final : public FNiagaraStatelessExpressionFloat
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionFloatDivide();
	virtual ~FNiagaraStatelessExpressionFloatDivide() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual float EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionFloat"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionFloat"))
	FInstancedStruct B;
};
