// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessExpression.h"

#include "NiagaraStatelessExpressionVec2.generated.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FNiagaraStatelessExpressionVec2 : public FNiagaraStatelessExpression
{
	GENERATED_BODY()

	virtual FNiagaraTypeDefinition GetOutputTypeDef() const override final { return FNiagaraTypeDefinition::GetVec2Def(); }
	virtual FInstancedStruct Build(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override final;
	virtual void Evaluate(const FEvaluateContext& Context, void* ValueAddress) const override final { *static_cast<FVector2f*>(ValueAddress) = EvaluateInternal(Context); }

	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const { checkNoEntry(); return FInstancedStruct(); }
	virtual FVector2f EvaluateInternal(const FEvaluateContext& Context) const { checkNoEntry(); return FVector2f::ZeroVector; }
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (ValueExpression))
struct FNiagaraStatelessExpressionVec2Constant final : public FNiagaraStatelessExpressionVec2
{
	GENERATED_BODY()

	virtual ~FNiagaraStatelessExpressionVec2Constant() = default;
	virtual FVector2f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values")
	FVector2f A = FVector2f::ZeroVector;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (BindingExpression))
struct FNiagaraStatelessExpressionVec2Binding final : public FNiagaraStatelessExpressionVec2
{
	GENERATED_BODY()

	virtual ~FNiagaraStatelessExpressionVec2Binding() = default;
	virtual bool IsConstant() const override { return false; }
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector2f EvaluateInternal(const FEvaluateContext& Context) const override;

	UPROPERTY(EditAnywhere, Category = "Values")
	FName A;

	int32 ParameterOffset = INDEX_NONE;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Add Vec2"))
struct FNiagaraStatelessExpressionVec2Add final : public FNiagaraStatelessExpressionVec2
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec2Add();
	virtual ~FNiagaraStatelessExpressionVec2Add() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector2f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec2"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec2"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Subtract Vec2"))
struct FNiagaraStatelessExpressionVec2Subtract final : public FNiagaraStatelessExpressionVec2
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec2Subtract();
	virtual ~FNiagaraStatelessExpressionVec2Subtract() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector2f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec2"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec2"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Multiply Vec2"))
struct FNiagaraStatelessExpressionVec2Multiply final : public FNiagaraStatelessExpressionVec2
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec2Multiply();
	virtual ~FNiagaraStatelessExpressionVec2Multiply() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector2f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec2"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec2"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Divide Vec2"))
struct FNiagaraStatelessExpressionVec2Divide final : public FNiagaraStatelessExpressionVec2
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec2Divide();
	virtual ~FNiagaraStatelessExpressionVec2Divide() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector2f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec2"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec2"))
	FInstancedStruct B;
};
