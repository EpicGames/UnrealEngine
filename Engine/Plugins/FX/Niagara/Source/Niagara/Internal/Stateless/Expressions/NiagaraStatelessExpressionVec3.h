// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessExpression.h"

#include "NiagaraStatelessExpressionVec3.generated.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FNiagaraStatelessExpressionVec3 : public FNiagaraStatelessExpression
{
	GENERATED_BODY()

	virtual FNiagaraTypeDefinition GetOutputTypeDef() const override final { return FNiagaraTypeDefinition::GetVec3Def(); }
	virtual FInstancedStruct Build(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override final;
	virtual void Evaluate(const FEvaluateContext& Context, void* ValueAddress) const override final { *static_cast<FVector3f*>(ValueAddress) = EvaluateInternal(Context); }

	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const { checkNoEntry(); return FInstancedStruct(); }
	virtual FVector3f EvaluateInternal(const FEvaluateContext& Context) const { checkNoEntry(); return FVector3f::ZeroVector; }
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (ValueExpression))
struct FNiagaraStatelessExpressionVec3Constant final : public FNiagaraStatelessExpressionVec3
{
	GENERATED_BODY()

	virtual ~FNiagaraStatelessExpressionVec3Constant() = default;
	virtual FVector3f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values")
	FVector3f A = FVector3f::ZeroVector;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (BindingExpression))
struct FNiagaraStatelessExpressionVec3Binding final : public FNiagaraStatelessExpressionVec3
{
	GENERATED_BODY()

	virtual ~FNiagaraStatelessExpressionVec3Binding() = default;
	virtual bool IsConstant() const override { return false; }
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector3f EvaluateInternal(const FEvaluateContext& Context) const override;

	UPROPERTY(EditAnywhere, Category = "Values")
	FName A;

	int32 ParameterOffset = INDEX_NONE;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Add Vec3"))
struct FNiagaraStatelessExpressionVec3Add final : public FNiagaraStatelessExpressionVec3
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec3Add();
	virtual ~FNiagaraStatelessExpressionVec3Add() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector3f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec3"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec3"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Subtract Vec3"))
struct FNiagaraStatelessExpressionVec3Subtract final : public FNiagaraStatelessExpressionVec3
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec3Subtract();
	virtual ~FNiagaraStatelessExpressionVec3Subtract() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector3f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec3"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec3"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Multiply Vec3"))
struct FNiagaraStatelessExpressionVec3Multiply final : public FNiagaraStatelessExpressionVec3
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec3Multiply();
	virtual ~FNiagaraStatelessExpressionVec3Multiply() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector3f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec3"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec3"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Divide Vec3"))
struct FNiagaraStatelessExpressionVec3Divide final : public FNiagaraStatelessExpressionVec3
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec3Divide();
	virtual ~FNiagaraStatelessExpressionVec3Divide() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector3f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec3"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec3"))
	FInstancedStruct B;
};
