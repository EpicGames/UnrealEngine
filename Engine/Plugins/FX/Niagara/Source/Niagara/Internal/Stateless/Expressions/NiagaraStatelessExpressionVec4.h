// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessExpression.h"

#include "NiagaraStatelessExpressionVec4.generated.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FNiagaraStatelessExpressionVec4 : public FNiagaraStatelessExpression
{
	GENERATED_BODY()

	virtual FNiagaraTypeDefinition GetOutputTypeDef() const override final { return FNiagaraTypeDefinition::GetVec4Def(); }
	virtual FInstancedStruct Build(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override final;
	virtual void Evaluate(const FEvaluateContext& Context, void* ValueAddress) const override final { *static_cast<FVector4f*>(ValueAddress) = EvaluateInternal(Context); }

	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const { checkNoEntry(); return FInstancedStruct(); }
	virtual FVector4f EvaluateInternal(const FEvaluateContext& Context) const { checkNoEntry(); return FVector4f::Zero(); }
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (ValueExpression))
struct FNiagaraStatelessExpressionVec4Constant final : public FNiagaraStatelessExpressionVec4
{
	GENERATED_BODY()

	virtual ~FNiagaraStatelessExpressionVec4Constant() = default;
	virtual FVector4f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values")
	FVector4f A = FVector4f::Zero();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (BindingExpression))
struct FNiagaraStatelessExpressionVec4Binding final : public FNiagaraStatelessExpressionVec4
{
	GENERATED_BODY()

	virtual ~FNiagaraStatelessExpressionVec4Binding() = default;
	virtual bool IsConstant() const override { return false; }
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector4f EvaluateInternal(const FEvaluateContext& Context) const override;

	UPROPERTY(EditAnywhere, Category = "Values")
	FName A;

	int32 ParameterOffset = INDEX_NONE;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Add Vec4"))
struct FNiagaraStatelessExpressionVec4Add final : public FNiagaraStatelessExpressionVec4
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec4Add();
	virtual ~FNiagaraStatelessExpressionVec4Add() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector4f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec4"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec4"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Subtract Vec4"))
struct FNiagaraStatelessExpressionVec4Subtract final : public FNiagaraStatelessExpressionVec4
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec4Subtract();
	virtual ~FNiagaraStatelessExpressionVec4Subtract() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector4f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec4"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec4"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Multiply Vec4"))
struct FNiagaraStatelessExpressionVec4Multiply final : public FNiagaraStatelessExpressionVec4
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec4Multiply();
	virtual ~FNiagaraStatelessExpressionVec4Multiply() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector4f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec4"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec4"))
	FInstancedStruct B;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(meta = (OperationExpression, DisplayName = "Divide Vec4"))
struct FNiagaraStatelessExpressionVec4Divide final : public FNiagaraStatelessExpressionVec4
{
	GENERATED_BODY()

	FNiagaraStatelessExpressionVec4Divide();
	virtual ~FNiagaraStatelessExpressionVec4Divide() = default;
	virtual FInstancedStruct BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual FVector4f EvaluateInternal(const FEvaluateContext& Context) const override;
	virtual bool IsConstant() const override;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec4"))
	FInstancedStruct A;

	UPROPERTY(EditAnywhere, Category = "Values", meta = (BaseStruct = "/Script/Niagara.NiagaraStatelessExpressionVec4"))
	FInstancedStruct B;
};
