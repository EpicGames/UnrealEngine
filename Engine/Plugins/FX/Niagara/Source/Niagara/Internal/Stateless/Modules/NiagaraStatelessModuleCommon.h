// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraStatelessModuleCommon.generated.h"

USTRUCT()
struct FNiagaraStatelessSystemScaleBuildData
{
	GENERATED_BODY()

	static FName GetName();

	////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Enable bits
	UPROPERTY(EditAnywhere, Category = "Camera", meta = (DisplayPriority = 600))
	uint32 bScaleCameraOffset : 1 = false;

	// Not including drag scale right now as we have separate solver passes which is harder to handle
	//UPROPERTY(EditAnywhere, Category = "Drag", meta = (DisplayPriority = 500))
	//uint32 bScaleDrag : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Forces", meta = (DisplayPriority = 200))
	uint32 bScaleForces : 1 = false;

	// Mesh scale in a no-op for local space with standard emitters
	//UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayPriority = 400))
	//uint32 bScaleInitialMeshSize : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Ribbon", meta = (DisplayPriority = 700))
	uint32 bScaleInitialRibbonWidth : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Sprite", meta = (DisplayPriority = 300))
	uint32 bScaleInitialSpriteSize : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Velocity", meta = (DisplayPriority = 100))
	uint32 bScaleInitialVelocity : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Decal", meta = (DisplayPriority = 800))
	uint32 bScaleInitialDecalSize : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Lights", meta = (DisplayPriority = 900))
	uint32 bScaleInitialLightRadius : 1 = false;

	////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Bitfields
	UPROPERTY(EditAnywhere, Category = "Sprite", meta = (DisplayPriority = 302, EditCondition = "bScaleInitialSpriteSize", EditConditionHides))
	uint32 bScaleInitialSpriteSizeBySingleAxis : 1 = false;

	//UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayPriority = 403, EditCondition = "bScaleInitialMeshSize", EditConditionHides))
	//uint32 bScaleInitialMeshSizeAbsoluteScale : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Ribbon", meta = (DisplayPriority = 702, EditCondition = "bScaleInitialRibbonWidth", EditConditionHides))
	uint32 bScaleInitialRibbonWidthBySingleAxis : 1 = false;

	//UPROPERTY(EditAnywhere, Category = "Drag", meta = (DisplayPriority = 502))
	//uint32 bInverseDragScaling : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Lights", meta = (DisplayPriority = 902, EditCondition = "bScaleInitialLightRadius", EditConditionHides))
	uint32 bScaleInitialLightRadiusBySingleAxis : 1 = false;

	////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Velocity
	UPROPERTY(EditAnywhere, Category = "Velocity", meta = (DisplayPriority = 101, EditCondition = "bScaleInitialVelocity", EditConditionHides))
	FVector3f ScaleInitialVelocityScaleAmount = FVector3f::OneVector;

	////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Forces
	UPROPERTY(EditAnywhere, Category = "Forces", meta = (DisplayPriority = 201, EditCondition = "bScaleForces", EditConditionHides))
	FVector3f ScaleForcesScaleAmount = FVector3f::OneVector;

	UPROPERTY(EditAnywhere, Category = "Forces", meta = (DisplayPriority = 202, EditCondition = "bScaleForces", EditConditionHides))
	uint32 bScaleForcesInverseScaling : 1 = false;

	////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Sprites
	UPROPERTY(EditAnywhere, Category = "Sprite", meta = (DisplayPriority = 301, EditCondition = "bScaleInitialSpriteSize", EditConditionHides))
	float ScaleInitialSpriteSizeScaleAmount = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Sprite", meta = (DisplayPriority = 303, EditCondition = "bScaleInitialSpriteSize && bScaleInitialSpriteSizeBySingleAxis", EditConditionHides))
	ENiagaraOrientationAxis ScaleInitialSpriteSizeAxis = ENiagaraOrientationAxis::XAxis;

	////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Meshes
	//UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayPriority = 401, EditCondition = "bScaleInitialMeshSize", EditConditionHides))
	//FVector3f ScaleInitialMeshSizeScaleAmount = FVector3f::OneVector;

	//UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayPriority = 402, EditCondition = "bScaleInitialMeshSize", EditConditionHides))
	//ENiagaraCoordinateSpace ScaleInitialMeshSizeScaleSpace = ENiagaraCoordinateSpace::Simulation;

	////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Drag
	//UPROPERTY(EditAnywhere, Category = "Drag", meta = (DisplayPriority = 501, EditCondition = "bScaleDrag", EditConditionHides))
	//float ScaleDragScaleAmount = 1.0f;

	////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Camera
	UPROPERTY(EditAnywhere, Category = "Camera", meta = (DisplayPriority = 601, EditCondition = "bScaleCameraOffset", EditConditionHides))
	float ScaleCameraOffsetAmount = 1.0f;

	////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Ribbons
	UPROPERTY(EditAnywhere, Category = "Ribbon", meta = (DisplayPriority = 701, EditCondition = "bScaleInitialRibbonWidth", EditConditionHides))
	float ScaleInitialRibbonWidthScaleAmount = 1.0f;
	UPROPERTY(EditAnywhere, Category = "Ribbon", meta = (DisplayPriority = 703, EditCondition = "bScaleInitialRibbonWidth && bScaleInitialRibbonWidthBySingleAxis", EditConditionHides))
	ENiagaraOrientationAxis ScaleInitialRibbonWidthScalingAxis = ENiagaraOrientationAxis::XAxis;

	////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Decals
	UPROPERTY(EditAnywhere, Category = "Decal", meta = (DisplayPriority = 801, EditCondition = "bScaleInitialDecalSize", EditConditionHides))
	FVector3f ScaleInitialDecalSizeScaleAmount = FVector3f::OneVector;

	////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Lights
	UPROPERTY(EditAnywhere, Category = "Lights", meta = (DisplayPriority = 901, EditCondition = "bScaleInitialLightRadius", EditConditionHides))
	float ScaleInitialLightRadiusScaleAmount = 1.0f;
	UPROPERTY(EditAnywhere, Category = "Lights", meta = (DisplayPriority = 903, EditCondition = "bScaleInitialLightRadius && bScaleInitialLightRadiusBySingleAxis", EditConditionHides))
	ENiagaraOrientationAxis ScaleInitialLightRadiusScalingAxis = ENiagaraOrientationAxis::XAxis;
};

namespace NiagaraStateless
{
	//-TODO: Examine structure padding, FQuat4f / UObject*
	struct FPhysicsBuildData
	{
		static FName GetName();

		FNiagaraStatelessRangeFloat		MassRange = FNiagaraStatelessRangeFloat(1.0f);
		FNiagaraStatelessRangeFloat		DragRange = FNiagaraStatelessRangeFloat(0.0f);

		ENiagaraCoordinateSpace			LinearVelocityCoordinateSpace = ENiagaraCoordinateSpace::Local;
		FNiagaraStatelessRangeVector3	LinearVelocityRange = FNiagaraStatelessRangeVector3(FVector3f::ZeroVector);
		FNiagaraStatelessRangeFloat		LinearVelocityScale = FNiagaraStatelessRangeFloat(1.0f);

		ENiagaraCoordinateSpace			WindCoordinateSpace = ENiagaraCoordinateSpace::Local;
		FNiagaraStatelessRangeVector3	WindRange = FNiagaraStatelessRangeVector3(FVector3f::ZeroVector);

		ENiagaraCoordinateSpace			AccelerationCoordinateSpace = ENiagaraCoordinateSpace::Local;
		FNiagaraStatelessRangeVector3	AccelerationRange = FNiagaraStatelessRangeVector3(FVector3f::ZeroVector);

		FNiagaraStatelessRangeVector3	GravityRange = FNiagaraStatelessRangeVector3(FVector3f::ZeroVector);

		bool							bConeVelocity = false;
		ENiagaraCoordinateSpace			ConeCoordinateSpace = ENiagaraCoordinateSpace::Local;
		bool							bUseConeConeRotator = true;
		FNiagaraStatelessRangeRotator	ConeRotator = FNiagaraStatelessRangeRotator(FRotator3f::ZeroRotator);
		FNiagaraStatelessRangeVector3	ConeDirection = FNiagaraStatelessRangeVector3(FVector3f::XAxisVector);
		FNiagaraStatelessRangeFloat		ConeVelocityRange = FNiagaraStatelessRangeFloat(0.0f);
		FNiagaraStatelessRangeFloat		ConeVelocityScale = FNiagaraStatelessRangeFloat(1.0f);
		float							ConeOuterAngle = 0.0f;
		float							ConeInnerAngle = 0.0f;
		float							ConeVelocityFalloff = 0.0f;

		bool							bPointVelocity = false;
		ENiagaraCoordinateSpace			PointCoordinateSpace = ENiagaraCoordinateSpace::Local;
		FNiagaraStatelessRangeFloat		PointVelocityRange = FNiagaraStatelessRangeFloat(0.0f);
		FNiagaraStatelessRangeFloat		PointVelocityScale = FNiagaraStatelessRangeFloat(1.0f);
		float							PointVelocityMax = 0.0f;
		FVector3f						PointOrigin = FVector3f::ZeroVector;

		bool							bNoiseEnabled = false;
		float							NoiseStrength = 0.0f;
		float							NoiseFrequency = 0.0f;
		FNiagaraStatelessRangeVector3	NoiseFieldOffset = FNiagaraStatelessRangeVector3(FVector3f::ZeroVector);
	};

	extern FQuat4f DirectionToQuat(FVector3f Direction);
}
