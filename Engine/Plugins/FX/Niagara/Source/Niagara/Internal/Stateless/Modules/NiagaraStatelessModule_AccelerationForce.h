// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"

#include "NiagaraStatelessModule_AccelerationForce.generated.h"

// Apply an acceleration force each frame (in cm/s). This factors in mass, so particles with a high mass will accelerate slower. 
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Acceleration Force"))
class UNiagaraStatelessModule_AccelerationForce : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Acceleration", DisableUniformDistribution, DisableBindingDistribution))
	FNiagaraDistributionRangeVector3 AccelerationDistribution = FNiagaraDistributionRangeVector3(FVector3f::ZeroVector);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (SegmentedDisplay))
	ENiagaraCoordinateSpace CoordinateSpace = ENiagaraCoordinateSpace::Local;

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
};
