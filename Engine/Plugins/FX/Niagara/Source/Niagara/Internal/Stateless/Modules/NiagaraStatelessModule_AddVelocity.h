// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"

#include "NiagaraStatelessModule_AddVelocity.generated.h"

UENUM()
enum class ENSM_VelocityType : uint8
{
	Linear,			// Velocity Min / Max -> Velocity Speed Scale
	FromPoint,		// Min / Max / Origin
	InCone,			// Max, Min, Cone| Cone Axis, Cone Angle, Inner Cone
};

UENUM()
enum class ENSM_ConeRotationType : uint8
{
	Rotation,
	Direction,
};

// Add to the velocity applied to all particles
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Add Velocity"))
class UNiagaraStatelessModule_AddVelocity : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta=(SegmentedDisplay))
	ENSM_VelocityType VelocityType = ENSM_VelocityType::Linear;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Linear Velocity", DisableUniformDistribution, EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::Linear"))
	FNiagaraDistributionRangeVector3 LinearVelocityDistribution = FNiagaraDistributionRangeVector3(FVector3f(0.0f, 0.0f, 100.0f));
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Linear Velocity Scale", DisableRangeDistribution, EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::Linear"))
	FNiagaraDistributionRangeFloat LinearVelocityScale = FNiagaraDistributionRangeFloat(1.0f);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Cone Velocity", EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::InCone"))
	FNiagaraDistributionRangeFloat ConeVelocityDistribution = FNiagaraDistributionRangeFloat(500.0f);
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Cone Velocity Scale", DisableRangeDistribution, EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::InCone"))
	FNiagaraDistributionRangeFloat ConeVelocityScale = FNiagaraDistributionRangeFloat(1.0f);
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (SegmentedDisplay, EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::InCone"))
	ENSM_ConeRotationType ConeRotationType = ENSM_ConeRotationType::Rotation;
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::InCone && ConeRotationType == ENSM_ConeRotationType::Rotation", DisableRangeDistribution, DisableUniformDistribution))
	FNiagaraDistributionRangeRotator ConeRotation = FNiagaraDistributionRangeRotator(FRotator3f::ZeroRotator);
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::InCone && ConeRotationType == ENSM_ConeRotationType::Direction", DisableRangeDistribution, DisableUniformDistribution))
	FNiagaraDistributionRangeVector3 ConeDirection = FNiagaraDistributionRangeVector3(FVector3f::ZAxisVector);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::InCone", ClampMin = "0.0", ClampMax = "360.0", UIMin = "0.0", UIMax = "360.0"))
	float ConeAngle = 45.0f;
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::InCone", ClampMin = "0.0", ClampMax = "360.0", UIMin = "0.0", UIMax = "360.0"))
	float InnerCone = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Point Velocity", EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::FromPoint"))
	FNiagaraDistributionRangeFloat PointVelocityDistribution = FNiagaraDistributionRangeFloat(100.0f);
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Point Velocity Scale", DisableRangeDistribution, EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::FromPoint"))
	FNiagaraDistributionRangeFloat PointVelocityScale = FNiagaraDistributionRangeFloat(1.0f);
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition = "VelocityType == ENSM_VelocityType::FromPoint"))
	FVector3f PointOrigin = FVector3f::ZeroVector;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "VelocityType == ENSM_VelocityType::InCone"))
	bool bSpeedFalloffFromConeAxisEnabled = false;
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bSpeedFalloffFromConeAxisEnabled && VelocityType == ENSM_VelocityType::InCone", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float SpeedFalloffFromConeAxis = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (SegmentedDisplay))
	ENiagaraCoordinateSpace CoordinateSpace = ENiagaraCoordinateSpace::Local;

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }

	virtual bool CanDebugDraw() const { return true; }
	virtual void DrawDebug(const FNiagaraStatelessDrawDebugContext& DrawDebugContext) const override;
#endif
};
