// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"

#include "NiagaraStatelessModule_ApplyOwnerScaleToAttributes.generated.h"

// Allows you to control how Engine.Owner.Scale is applied to attributes
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Apply Owner Scale To Attributes", ShowTopLevelCategories))
class UNiagaraStatelessModule_ApplyOwnerScaleToAttributes : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (ShowOnlyInnerProperties, FullyExpand = true))
	FNiagaraStatelessSystemScaleBuildData SystemScaleData;

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
};
