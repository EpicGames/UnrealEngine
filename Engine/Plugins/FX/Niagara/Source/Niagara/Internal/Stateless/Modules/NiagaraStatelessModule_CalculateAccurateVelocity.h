// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"

#include "NiagaraStatelessModule_CalculateAccurateVelocity.generated.h"

UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Calculate Accurate Velocity"))
class UNiagaraStatelessModule_CalculateAccurateVelocity : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
