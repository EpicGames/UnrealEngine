// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraStatelessModule_CameraOffset.generated.h"

// Offset the particle along the vector between the particle and the camera.
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Camera Offset"))
class UNiagaraStatelessModule_CameraOffset : public UNiagaraStatelessModule
{
public:
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "CameraOffset"))
	FNiagaraDistributionFloat CameraOffsetDistribution = FNiagaraDistributionFloat(0.0f);

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
