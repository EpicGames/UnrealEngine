// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"

#include "NiagaraStatelessModule_CurlNoiseForce.generated.h"

UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Curl Noise Force"))
class UNiagaraStatelessModule_CurlNoiseForce : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters")
	float NoiseStrength = 10.0f;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	float NoiseFrequency = 50.0f;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bBiasNoiseFieldEnabled : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bBiasNoiseFieldEnabled"))
	FNiagaraDistributionRangeVector3 BiasNoiseField = FNiagaraDistributionRangeVector3(FVector3f::ZeroVector);

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
};
