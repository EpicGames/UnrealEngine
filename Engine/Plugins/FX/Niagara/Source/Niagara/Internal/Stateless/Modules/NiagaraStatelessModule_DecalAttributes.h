// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessCommon.h"
#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraStatelessModule_DecalAttributes.generated.h"

UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Decal Attributes"))
class UNiagaraStatelessModule_DecalAttributes : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	bool bApplyOrientation = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	bool bApplySize = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	bool bApplyFade = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Orientation", EditCondition = "bApplyOrientation"))
	FNiagaraDistributionVector3 Orientation = FNiagaraDistributionVector3(FVector3f::ZeroVector);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "IsOrientationEnabled()", EditConditionHides, SegmentedDisplay))
	ENiagaraCoordinateSpace OrientationCoordinateSpace = ENiagaraCoordinateSpace::Local;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Size", EditCondition = "bApplySize"))
	FNiagaraDistributionVector3 Size = FNiagaraDistributionVector3(FVector3f(50.0f, 50.0f, 50.0f));

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Fade", EditCondition = "bApplyFade"))
	FNiagaraDistributionFloat Fade = FNiagaraDistributionFloat(1.0f);

	virtual ENiagaraStatelessFeatureMask GetFeatureMask() const override;
	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

	UFUNCTION()
	bool IsOrientationEnabled() const { return bApplyOrientation; }

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
#if WITH_EDITORONLY_DATA
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
