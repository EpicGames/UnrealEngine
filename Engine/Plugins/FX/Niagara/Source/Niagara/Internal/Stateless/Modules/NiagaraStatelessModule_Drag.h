// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"

#include "NiagaraStatelessModule_Drag.generated.h"

// Applies Drag directly to particle velocity, irrespective of Mass.
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Drag"))
class UNiagaraStatelessModule_Drag : public UNiagaraStatelessModule
{
	GENERATED_BODY()

	static constexpr float DefaultDrag = 1.0f;

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Drag", DisableBindingDistribution))
	FNiagaraDistributionRangeFloat DragDistribution = FNiagaraDistributionRangeFloat(DefaultDrag);

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
};
