// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraStatelessModule_DynamicMaterialParameters.generated.h"

USTRUCT()
struct FNiagaraStatelessDynamicParameterSet
{
	GENERATED_BODY()
		
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bXChannelEnabled : 1 = true;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bYChannelEnabled : 1 = true;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bZChannelEnabled : 1 = true;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bWChannelEnabled : 1 = true;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "X Channel", EditCondition = "bXChannelEnabled"))
	FNiagaraDistributionFloat XChannelDistribution = FNiagaraDistributionFloat(FNiagaraStatelessGlobals::GetDefaultDynamicMaterialParametersValue().X);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Y Channel", EditCondition = "bYChannelEnabled"))
	FNiagaraDistributionFloat YChannelDistribution = FNiagaraDistributionFloat(FNiagaraStatelessGlobals::GetDefaultDynamicMaterialParametersValue().Y);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Z Channel", EditCondition = "bZChannelEnabled"))
	FNiagaraDistributionFloat ZChannelDistribution = FNiagaraDistributionFloat(FNiagaraStatelessGlobals::GetDefaultDynamicMaterialParametersValue().Z);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "W Channel", EditCondition = "bWChannelEnabled"))
	FNiagaraDistributionFloat WChannelDistribution = FNiagaraDistributionFloat(FNiagaraStatelessGlobals::GetDefaultDynamicMaterialParametersValue().W);
};

// Write to the Dynamic Parameters that can be read in the material vertex & pixel shader vertex
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Dynamic Material Parameters"))
class UNiagaraStatelessModule_DynamicMaterialParameters : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bParameter0Enabled : 1 = true;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bParameter1Enabled : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bParameter2Enabled : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bParameter3Enabled : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bParameter0Enabled"))
	FNiagaraStatelessDynamicParameterSet Parameter0;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bParameter1Enabled"))
	FNiagaraStatelessDynamicParameterSet Parameter1;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bParameter2Enabled"))
	FNiagaraStatelessDynamicParameterSet Parameter2;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bParameter3Enabled"))
	FNiagaraStatelessDynamicParameterSet Parameter3;

	const FNiagaraStatelessDynamicParameterSet& GetParameterSet(int32 ParameterIndex) const;
	uint32 GetParameterChannelMask(int32 ParameterIndex) const;
	const FNiagaraDistributionFloat& GetParameterChannelDistribution(int32 ParameterIndex, int32 ChannelIndex) const;
	const FNiagaraVariableBase& GetParameterVariable(int32 ParameterIndex) const;
	int32 GetRendererChannelMask() const;

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
