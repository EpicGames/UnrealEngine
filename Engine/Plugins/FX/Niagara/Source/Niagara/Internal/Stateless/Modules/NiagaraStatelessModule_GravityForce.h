// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraStatelessModule_GravityForce.generated.h"

// Applies a gravitational force (in cm/s). This acceleration is the same regardless of mass, so particles with high and low mass will reach the same velocity.
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Gravity Force"))
class UNiagaraStatelessModule_GravityForce : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Gravity", DisableUniformDistribution, DisableBindingDistribution))
	FNiagaraDistributionRangeVector3 GravityDistribution = FNiagaraDistributionRangeVector3(GetDefaultValue());

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;

	static FVector3f GetDefaultValue() { return FVector3f(0.0f, 0.0f, -980.0f); }

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
};
