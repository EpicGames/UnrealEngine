// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"

#include "NiagaraStatelessModule_InitialMeshOrientation.generated.h"

UENUM()
enum class ENSMInitialMeshOrientationMode
{
	None,
	Random,
	//System,
	OrientToAxis UMETA(DisplayName="Orient To Vector"),
	//OrientToMatrix,
	//OrientToQuaternion,
};

// Set the initial mesh orientation, directly, randomly or by orienting by axis
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Initial Mesh Orientation"))
class UNiagaraStatelessModule_InitialMeshOrientation : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	using FParameters = NiagaraStateless::FInitialMeshOrientationModule_ShaderParameters;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	ENSMInitialMeshOrientationMode	MeshOrientationMode = ENSMInitialMeshOrientationMode::None;

	// Establish an initial orientation around which to yaw, pitch, or roll. Can be overriden with any vector, for instance the normalized velocity vector, to accomplish more elaborate behavior.
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition  = "MeshOrientationMode == ENSMInitialMeshOrientationMode::OrientToAxis", DisableRangeDistribution, DisableUniformDistribution))
	FNiagaraDistributionRangeVector3 OrientationVector = FNiagaraDistributionRangeVector3(FVector3f::XAxisVector);

	// This represents the Axis on which the model was first imported from your DCC package.
	// This vector is then rotated in the direction of the Orientation Vector input.
	// If your mesh was imported on a different axis than X forward, you can change it here.
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition = "MeshOrientationMode == ENSMInitialMeshOrientationMode::OrientToAxis", DisableRangeDistribution, DisableUniformDistribution))
	FNiagaraDistributionRangeVector3 MeshAxisToOrient = FNiagaraDistributionRangeVector3(FVector3f::XAxisVector);

	// Rotation in Degrees, this is applied after any other orientation is calculated and in the space of that orientation
	UPROPERTY(EditAnywhere, Category = "Parameters", meta=(EditConditionHides, EditCondition = "MeshOrientationMode != ENSMInitialMeshOrientationMode::Random", DisableUniformDistribution, Units = "deg"))
	FNiagaraDistributionRangeVector3 Rotation = FNiagaraDistributionRangeVector3(FVector3f::ZeroVector);

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
	virtual void PostLoad() override;

private:
	UPROPERTY()
	FVector3f	RandomRotationRange_DEPRECATED = FVector3f(360.0f, 360.0f, 360.0f);
#endif
};
