// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"

#include "NiagaraParameterBinding.h"
#include "NiagaraParameterStore.h"

#include "NiagaraStatelessModule_InitializeParticle.generated.h"

// Initialize common particle attributes using common settings and options.
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Initialize Particle", ShowTopLevelCategories))
class UNiagaraStatelessModule_InitializeParticle : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Point Attributes", meta = (DisplayName = "Lifetime", Units="s"))
	FNiagaraDistributionRangeFloat LifetimeDistribution = FNiagaraDistributionRangeFloat(FNiagaraStatelessGlobals::GetDefaultLifetimeValue());

	UPROPERTY(EditAnywhere, Category = "Point Attributes", meta = (DisplayName = "Color", DistributionLookupValueEnumPath = "/Script/Niagara.ENiagaraDistributionInitialLookupValueMode"))
	FNiagaraDistributionColor ColorDistribution = FNiagaraDistributionColor(FNiagaraStatelessGlobals::GetDefaultColorValue());

	UPROPERTY(EditAnywhere, Category = "Point Attributes", meta = (DisplayName = "Mass"))
	FNiagaraDistributionRangeFloat MassDistribution = FNiagaraDistributionRangeFloat(FNiagaraStatelessGlobals::GetDefaultMassValue());

	UPROPERTY(EditAnywhere, Category = "Sprite Attributes", meta = (DisplayName = "Sprite Size"))
	FNiagaraDistributionRangeVector2 SpriteSizeDistribution = FNiagaraDistributionRangeVector2(FNiagaraStatelessGlobals::GetDefaultSpriteSizeValue());

	UPROPERTY(EditAnywhere, Category = "Sprite Attributes", meta = (DisplayName = "Sprite Rotation", Units="deg"))
	FNiagaraDistributionRangeFloat SpriteRotationDistribution = FNiagaraDistributionRangeFloat(FNiagaraStatelessGlobals::GetDefaultSpriteRotationValue());

	UPROPERTY(EditAnywhere, Category = "Mesh Attributes", meta = (DisplayName = "Mesh Scale"))
	FNiagaraDistributionRangeVector3 MeshScaleDistribution = FNiagaraDistributionRangeVector3(FNiagaraStatelessGlobals::GetDefaultScaleValue());

	UPROPERTY(EditAnywhere, Category = "Ribbon Attributes", meta = (InlineEditConditionToggle))
	bool bWriteRibbonWidth = false;

	UPROPERTY(EditAnywhere, Category = "Ribbon Attributes", meta = (DisplayName = "Ribbon Width", EditCondition="bWriteRibbonWidth"))
	FNiagaraDistributionRangeFloat RibbonWidthDistribution = FNiagaraDistributionRangeFloat(FNiagaraStatelessGlobals::GetDefaultRibbonWidthValue());

	UPROPERTY(EditAnywhere, Category = "Point Attributes", meta = (DistributionLookupValueEnumPath = "/Script/Niagara.ENiagaraDistributionInitialLookupValueMode"))
	FNiagaraDistributionPosition InitialPositionDistribution = FNiagaraDistributionPosition(FVector3f::ZeroVector);

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
