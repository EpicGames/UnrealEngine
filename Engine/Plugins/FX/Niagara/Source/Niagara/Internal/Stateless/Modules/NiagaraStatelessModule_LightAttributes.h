// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraStatelessModule_LightAttributes.generated.h"

// Sets attributes that the light renderer can consume
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Light Attributes"))
class UNiagaraStatelessModule_LightAttributes : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bApplyRadius : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bApplyFalloffExponent : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bApplyDiffuseScale : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bApplySpecularScale : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	uint32 bApplyVolumetricScattering : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bApplyRadius"))
	FNiagaraDistributionFloat Radius = FNiagaraDistributionFloat(100.0f);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bApplyFalloffExponent"))
	FNiagaraDistributionFloat FalloffExponent = FNiagaraDistributionFloat(1.0f);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bApplyDiffuseScale"))
	FNiagaraDistributionFloat DiffuseScale = FNiagaraDistributionFloat(1.0f);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bApplySpecularScale"))
	FNiagaraDistributionFloat SpecularScale = FNiagaraDistributionFloat(1.0f);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bApplyVolumetricScattering"))
	FNiagaraDistributionFloat VolumetricScattering = FNiagaraDistributionFloat(0.0f);

	virtual ENiagaraStatelessFeatureMask GetFeatureMask() const override;
	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
#if WITH_EDITORONLY_DATA
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
