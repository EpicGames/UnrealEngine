// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"

#include "NiagaraStatelessModule_MeshIndex.generated.h"

// Sets the mesh index to use with mesh renderers
UCLASS(MinimalAPI, EditInlineNew, Experimental, meta = (DisplayName = "Mesh Index"))
class UNiagaraStatelessModule_MeshIndex : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	using FParameters = NiagaraStateless::FMeshIndexModule_ShaderParameters;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FNiagaraDistributionRangeInt MeshIndex = FNiagaraDistributionRangeInt(0);

	/* Weight for each potential mesh index when using a range. */
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition="NeedsMeshIndexWeights()"))
	TArray<float> MeshIndexWeight;

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

	UFUNCTION()
	bool NeedsMeshIndexWeights() const { return MeshIndex.IsRange(); }

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
