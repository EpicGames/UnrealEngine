// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraStatelessModule_MeshRotationRate.generated.h"

// Applies a constant rotation rate to mesh orientation
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Mesh Rotation Rate"))
class UNiagaraStatelessModule_MeshRotationRate : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	bool bUseRateScale = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Rotation Rate", Units="deg"))
	FNiagaraDistributionRangeVector3 RotationRateDistribution = FNiagaraDistributionRangeVector3(FVector3f::ZeroVector);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Rate Scale", EditCondition = "bUseRateScale"))
	FNiagaraDistributionCurveVector3 RateScaleDistribution = FNiagaraDistributionCurveVector3(ENiagaraDistributionCurveLUTMode::Accumulate);

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
