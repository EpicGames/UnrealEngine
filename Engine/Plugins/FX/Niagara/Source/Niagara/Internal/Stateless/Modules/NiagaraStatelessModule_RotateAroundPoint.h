// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraStatelessModule_RotateAroundPoint.generated.h"

UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Rotate Around Point"))
class UNiagaraStatelessModule_RotateAroundPoint : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta=(DisableBindingDistribution, Units="deg/s"))
	FNiagaraDistributionRangeFloat Rate = FNiagaraDistributionRangeFloat(360.0f);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisableBindingDistribution))
	FNiagaraDistributionRangeFloat Radius = FNiagaraDistributionRangeFloat(100.0f);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisableBindingDistribution))
	FNiagaraDistributionRangeFloat InitialPhase = FNiagaraDistributionRangeFloat(0.0f);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (SegmentedDisplay))
	ENiagaraCoordinateSpace CenterCoordinateSpace = ENiagaraCoordinateSpace::Simulation;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta=(DisableUniformDistribution))
	FNiagaraDistributionRangeVector3 Center = FNiagaraDistributionRangeVector3(FVector3f::ZeroVector);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (SegmentedDisplay))
	ENiagaraCoordinateSpace RotationCoordinateSpace = ENiagaraCoordinateSpace::Simulation;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisableUniformDistribution))
	FNiagaraDistributionRangeVector3 RotationAxis = FNiagaraDistributionRangeVector3(FVector3f::ZeroVector);

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;
#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }

	virtual bool CanDebugDraw() const override { return true; }
	virtual void DrawDebug(const FNiagaraStatelessDrawDebugContext& DrawDebugContext) const override;
#endif
#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
