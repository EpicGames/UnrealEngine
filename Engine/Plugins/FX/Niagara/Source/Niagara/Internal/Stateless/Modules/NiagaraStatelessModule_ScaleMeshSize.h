// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraParameterBinding.h"

#include "NiagaraStatelessModule_ScaleMeshSize.generated.h"

// Multiply Particle.Scale by the module calculated scale value
// This can be a constant, random or curve indexed by Particle.NormalizedAge
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Scale Mesh Size"))
class UNiagaraStatelessModule_ScaleMeshSize : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Scale"))
	FNiagaraDistributionVector3 ScaleDistribution = FNiagaraDistributionVector3(FVector3f::OneVector);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition = "UseScaleCurveRange()"))
	FNiagaraParameterBindingWithValue ScaleCurveRange;

	virtual void PostInitProperties() override;
	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

	UFUNCTION()
	bool UseScaleCurveRange() const { return ScaleDistribution.IsCurve(); }

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
