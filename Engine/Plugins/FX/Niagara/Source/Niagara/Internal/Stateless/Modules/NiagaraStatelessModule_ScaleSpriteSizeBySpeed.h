// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"

#include "NiagaraStatelessModule_ScaleSpriteSizeBySpeed.generated.h"

// Applies a modifier to sprite size based on the velocity of the particle
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Scale Sprite Size By Speed"))
class UNiagaraStatelessModule_ScaleSpriteSizeBySpeed : public UNiagaraStatelessModule
{
	GENERATED_BODY()

	static constexpr float DefaultVelocity = 1000.0f;

public:
	using FParameters = NiagaraStateless::FScaleSpriteSizeBySpeedModule_ShaderParameters;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (ClampMin = "0.01", UIMin = "0.01", DisableRangeDistribution))
	FNiagaraDistributionRangeFloat VelocityThreshold = FNiagaraDistributionRangeFloat(DefaultVelocity);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (ClampMin = "0.01", UIMin = "0.01", DisableRangeDistribution))
	FNiagaraDistributionRangeVector2 MinScaleFactor = FNiagaraDistributionRangeVector2(FVector2f(1.0f, 1.0f));

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (ClampMin = "0.01", UIMin = "0.01", DisableRangeDistribution))
	FNiagaraDistributionRangeVector2 MaxScaleFactor = FNiagaraDistributionRangeVector2(FVector2f(2.0f, 2.0f));

	UPROPERTY(EditAnywhere, Category = "Parameters")
	bool bSampleScaleFactorByCurve = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisableRangeDistribution, DisableDistributionLookupValueMode, EditCondition = "bSampleScaleFactorByCurve", EditConditionHides))
	FNiagaraDistributionFloat SampleFactorCurve = FNiagaraDistributionFloat({0.0f, 1.0f});

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
