// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include "NiagaraStatelessModule_ShapeLocation.generated.h"

UENUM()
enum class ENSM_ShapePrimitive : uint8
{
	Box,
	Cylinder,
	Plane,
	Ring,
	Sphere,
	Max UMETA(Hidden)
};

UENUM()
enum class ENSM_SurfaceExpansionMode : uint8
{
	Inner,
	Centered,
	Outside,
};

UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Shape Location", ShowTopLevelCategories))
class UNiagaraStatelessModule_ShapeLocation : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	using FParameters = NiagaraStateless::FShapeLocationModule_ShaderParameters;

	UPROPERTY(EditAnywhere, Category = "Shape", meta = (ShowInStackItemHeader, StackItemHeaderAlignment = "Left"))
	ENSM_ShapePrimitive ShapePrimitive = ENSM_ShapePrimitive::Sphere;

	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Box"))
	FNiagaraDistributionRangeVector3 BoxSize = FNiagaraDistributionRangeVector3(FVector3f(100.0f));
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Box"))
	bool bBoxSurfaceOnly = false;
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Box && bBoxSurfaceOnly"))
	ENSM_SurfaceExpansionMode BoxSurfaceExpansion = ENSM_SurfaceExpansionMode::Centered;
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Box && bBoxSurfaceOnly"))
	FNiagaraDistributionRangeFloat BoxSurfaceThickness = FNiagaraDistributionRangeFloat(0.0f);

	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Plane"))
	FNiagaraDistributionRangeVector2 PlaneSize = FNiagaraDistributionRangeVector2(FVector2f(100.0f));
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Plane"))
	bool bPlaneEdgesOnly = false;
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Plane && bPlaneEdgesOnly"))
	ENSM_SurfaceExpansionMode PlaneEdgeExpansion = ENSM_SurfaceExpansionMode::Centered;
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Plane && bPlaneEdgesOnly"))
	FNiagaraDistributionRangeFloat PlaneEdgeThickness = FNiagaraDistributionRangeFloat(0.0f);

	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Cylinder"))
	FNiagaraDistributionRangeFloat CylinderHeight = FNiagaraDistributionRangeFloat(100.0f);
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Cylinder"))
	FNiagaraDistributionRangeFloat CylinderRadius = FNiagaraDistributionRangeFloat(100.0f);
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Cylinder"))
	FNiagaraDistributionRangeFloat CylinderHeightMidpoint = FNiagaraDistributionRangeFloat(0.5f);
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Cylinder"))
	bool bCylinderSurfaceOnly = false;
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Cylinder && bCylinderSurfaceOnly"))
	bool bCylinderSurfaceOnlyIncludeEndCaps = true;
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Cylinder && bCylinderSurfaceOnly"))
	ENSM_SurfaceExpansionMode CylinderSurfaceExpansion = ENSM_SurfaceExpansionMode::Centered;
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Cylinder && bCylinderSurfaceOnly"))
	FNiagaraDistributionRangeFloat CylinderSurfaceThickness = FNiagaraDistributionRangeFloat(0.0f);

	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Ring"))
	FNiagaraDistributionRangeFloat RingRadius = FNiagaraDistributionRangeFloat(100.0f);
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Ring", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FNiagaraDistributionRangeFloat DiscCoverage = FNiagaraDistributionRangeFloat(0.0f);
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, EditConditionHides, EditCondition = "ShapePrimitive == ENSM_ShapePrimitive::Ring", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FNiagaraDistributionRangeFloat RingUDistribution = FNiagaraDistributionRangeFloat(0.0f);

	UPROPERTY(EditAnywhere, Category = "Shape", meta=(EditConditionHides, EditCondition="ShapePrimitive == ENSM_ShapePrimitive::Sphere"))
	FNiagaraDistributionRangeFloat SphereRadius = FNiagaraDistributionRangeFloat(100.0f);

	UPROPERTY(EditAnywhere, Category = "Shape", meta = (SegmentedDisplay))
	ENiagaraCoordinateSpace CoordinateSpace = ENiagaraCoordinateSpace::Simulation;

	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, DisableUniformDistribution))
	FNiagaraDistributionRangeRotator ShapeRotation;

	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DisableRangeDistribution, DisableUniformDistribution))
	FNiagaraDistributionRangeVector3 ShapeScale = FNiagaraDistributionRangeVector3(FVector3f::OneVector);

	UPROPERTY(EditAnywhere, Category = "Transform")
	FNiagaraDistributionPosition TransformOffset = FNiagaraDistributionPosition(FVector3f::ZeroVector);

	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DisableRangeDistribution))
	FNiagaraDistributionRangeVector3 TransformOffsetScale = FNiagaraDistributionRangeVector3(FVector3f::OneVector);

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float SphereMin_DEPRECATED = 0.0f;
	UPROPERTY()
	float SphereMax_DEPRECATED = 100.0f;
#endif

public:
	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }

	virtual bool CanDebugDraw() const override { return true; }
	virtual void DrawDebug(const FNiagaraStatelessDrawDebugContext& DrawDebugContext) const override;
#endif
#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;

	virtual void PostLoad() override;
#endif
};
