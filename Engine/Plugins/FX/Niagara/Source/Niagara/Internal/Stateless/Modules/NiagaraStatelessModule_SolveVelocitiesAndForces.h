// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"
#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"

#include "NiagaraStatelessModule_SolveVelocitiesAndForces.generated.h"

// Integrates all the forces applying them to position
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Solve Forces And Velocity"))
class UNiagaraStatelessModule_SolveVelocitiesAndForces : public UNiagaraStatelessModule
{
	GENERATED_BODY()

	struct FModuleBuiltData
	{
		NiagaraStateless::FPhysicsBuildData	PhysicsData;
		int32								PositionVariableOffset = INDEX_NONE;
		int32								VelocityVariableOffset = INDEX_NONE;
		int32								PreviousPositionVariableOffset = INDEX_NONE;
		int32								PreviousVelocityVariableOffset = INDEX_NONE;
	};

public:
	using FParameters = NiagaraStateless::FSolveVelocitiesAndForcesModule_ShaderParameters;

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
