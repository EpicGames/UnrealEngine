// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraStatelessModule_SpriteFacingAndAlignment.generated.h"

// Sets the sprite facing and alignment attributes
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Sprite Facing Alignment"))
class UNiagaraStatelessModule_SpriteFacingAndAlignment : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	bool bSpriteFacingEnabled = true;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	bool bSpriteAlignmentEnabled = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisableRangeDistribution, DisableUniformDistribution, EditCondition = "bSpriteFacingEnabled"))
	FNiagaraDistributionRangeVector3	SpriteFacing = FNiagaraDistributionRangeVector3(FVector3f::XAxisVector);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisableRangeDistribution, DisableUniformDistribution, EditCondition = "bSpriteAlignmentEnabled"))
	FNiagaraDistributionRangeVector3	SpriteAlignment = FNiagaraDistributionRangeVector3(FVector3f::YAxisVector);

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
