// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraStatelessModule_SpriteRotationRate.generated.h"

// Applies a constant value to sprite rotation
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Sprite Rotation Rate"))
class UNiagaraStatelessModule_SpriteRotationRate : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	bool bUseRateScale = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Rotation Rate", Units = "deg"))
	FNiagaraDistributionRangeFloat RotationRateDistribution = FNiagaraDistributionRangeFloat(FNiagaraStatelessGlobals::GetDefaultSpriteRotationValue());

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayName = "Rate Scale", EditCondition = "bUseRateScale"))
	FNiagaraDistributionCurveFloat RateScaleDistribution = FNiagaraDistributionCurveFloat(ENiagaraDistributionCurveLUTMode::Accumulate);

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif
#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
