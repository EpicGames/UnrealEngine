// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessModule.h"

#include "NiagaraStatelessModule_SubUVAnimation.generated.h"

UENUM()
enum class ENSMSubUVAnimation_Mode
{
	DirectSet,
	InfiniteLoop,
	Linear,
	Random,
};

// Sets the sub image frame index value based on the select animation mode
// The sub image index is a float value where the fractional part can be used to blend frames together
UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Sub UV Animation"))
class UNiagaraStatelessModule_SubUVAnimation : public UNiagaraStatelessModule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Parameters")
	int32	NumFrames = 16;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition = "AnimationMode == ENSMSubUVAnimation_Mode::DirectSet", DisableRange))
	FNiagaraDistributionRangeInt FrameIndex = FNiagaraDistributionRangeInt(0);

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	bool	bStartFrameRangeOverride_Enabled = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (InlineEditConditionToggle))
	bool	bEndFrameRangeOverride_Enabled = false;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bStartFrameRangeOverride_Enabled"))
	int32	StartFrameRangeOverride = 0;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditCondition = "bEndFrameRangeOverride_Enabled"))
	int32	EndFrameRangeOverride = 0;
	
	UPROPERTY(EditAnywhere, Category = "Parameters", meta=(SegmentedDisplay))
	ENSMSubUVAnimation_Mode AnimationMode = ENSMSubUVAnimation_Mode::Linear;

	//-Note: Main module has PlaybackMode (Loops / FPS) to choose between loops or frames per second
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition = "AnimationMode == ENSMSubUVAnimation_Mode::InfiniteLoop"))
	float LoopsPerSecond = 1.0f;

	//-Note: Main module has a few more options
	//UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition = "AnimationMode == ENSMSubUVAnimation_Mode::Linear"))
	//bool bRandomStartFrame = false;
	//int32 StartFrameOffset = 0;
	//float LoopupIndexScale = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (EditConditionHides, EditCondition = "AnimationMode == ENSMSubUVAnimation_Mode::Random"))
	float RandomChangeInterval = 0.1f;

	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const override;
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const override;

#if WITH_EDITOR
	virtual bool CanDisableModule() const override { return true; }
#endif

#if WITH_EDITORONLY_DATA
	virtual const TCHAR* GetShaderTemplatePath() const override;
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const override;
#endif
};
