// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraStatelessDefinitions.h"
#include "NiagaraStatelessCommon.generated.h"

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ENiagaraStatelessFeatureMask : uint32
{
	// We can execute on the GPU (Might be broken down into GPUCompute | GPUGraphics | GPUAsyncCompute in future but this will remain the master mask)
	ExecuteGPU			= 1 << 0,
	// We can execute on the CPU
	ExecuteCPU			= 1 << 1,

	None			= 0							UMETA(Hidden),
	ExecuteAll		= ExecuteGPU | ExecuteCPU	UMETA(Hidden),
	All				= ExecuteAll				UMETA(Hidden),
};
ENUM_CLASS_FLAGS(ENiagaraStatelessFeatureMask);

// Helper structure to transform in / out of various spaces
// The transforms are all expected to be in tile relative space, i.e. not LWC space
struct FNiagaraStatelessSpaceTransforms
{
	FNiagaraStatelessSpaceTransforms() { InitializeTransforms(true, FTransform3f::Identity); }

	void InitializeTransforms(bool bIsLocalSpace, const FTransform3f& LocalToWorld);
	bool UpdateTransforms(const FTransform3f& LocalToWorld);
	
	const FTransform3f& GetLocalToWorld() const { return Transforms[LocalToWorldIndex]; }
	const FTransform3f& GetWorldToLocal() const { return Transforms[WorldToLocalIndex]; }
	const FTransform3f& GetTransform(ENiagaraCoordinateSpace SourceSpace, ENiagaraCoordinateSpace DestinationSpace) const;

	FQuat4f   TransformRotation(ENiagaraCoordinateSpace SourceSpace, ENiagaraCoordinateSpace DestinationSpace, FQuat4f Rotation) const;
	FVector3f TransformPosition(ENiagaraCoordinateSpace SourceSpace, ENiagaraCoordinateSpace DestinationSpace, FVector3f Position) const;
	FVector3f TransformVector(ENiagaraCoordinateSpace SourceSpace, ENiagaraCoordinateSpace DestinationSpace, FVector3f Vector) const;
	FVector3f TransformVectorNoScale(ENiagaraCoordinateSpace SourceSpace, ENiagaraCoordinateSpace DestinationSpace, FVector3f Vector) const;

	FQuat4f   TransformRotation(ENiagaraCoordinateSpace SourceSpace, FQuat4f Rotation) const { return TransformRotation(SourceSpace, ENiagaraCoordinateSpace::Simulation, Rotation); }
	FVector3f TransformPosition(ENiagaraCoordinateSpace SourceSpace, FVector3f Position) const { return TransformPosition(SourceSpace, ENiagaraCoordinateSpace::Simulation, Position); }
	FVector3f TransformVector(ENiagaraCoordinateSpace SourceSpace, FVector3f Vector) const { return TransformVector(SourceSpace, ENiagaraCoordinateSpace::Simulation, Vector); }
	FVector3f TransformVectorNoScale(ENiagaraCoordinateSpace SourceSpace, FVector3f Vector) const { return TransformVectorNoScale(SourceSpace, ENiagaraCoordinateSpace::Simulation, Vector); }

private:
	static constexpr int LocalToWorldIndex = 0;
	static constexpr int WorldToLocalIndex = 1;
	static constexpr int TransformRemapSize = 3;	// Local / World / Simulation

	int8			TransformRemap[TransformRemapSize][TransformRemapSize];	// Remaps from transforms from world / local / simulation into an index below, where INDEX_NONE is null op
	FTransform3f	Transforms[2];			// World to Local & Local to World
};


struct FNiagaraStatelessGlobals
{
	FNiagaraVariableBase	CameraOffsetVariable;
	FNiagaraVariableBase	ColorVariable;
	FNiagaraVariableBase	DynamicMaterialParameters0Variable;
	FNiagaraVariableBase	DynamicMaterialParameters1Variable;
	FNiagaraVariableBase	DynamicMaterialParameters2Variable;
	FNiagaraVariableBase	DynamicMaterialParameters3Variable;
	FNiagaraVariableBase	MaterialRandomVariable;
	FNiagaraVariableBase	MeshIndexVariable;
	FNiagaraVariableBase	MeshOrientationVariable;
	FNiagaraVariableBase	PositionVariable;
	FNiagaraVariableBase	RibbonWidthVariable;
	FNiagaraVariableBase	ScaleVariable;
	FNiagaraVariableBase	SpriteAlignmentVariable;
	FNiagaraVariableBase	SpriteFacingVariable;
	FNiagaraVariableBase	SpriteSizeVariable;
	FNiagaraVariableBase	SpriteRotationVariable;
	FNiagaraVariableBase	SubImageIndexVariable;
	FNiagaraVariableBase	UniqueIDVariable;
	FNiagaraVariableBase	VelocityVariable;

	FNiagaraVariableBase	PreviousCameraOffsetVariable;
	//FNiagaraVariableBase	PreviousColorVariable;
	//FNiagaraVariableBase	PreviousDynamicMaterialParameters0Variable;
	FNiagaraVariableBase	PreviousMeshOrientationVariable;
	FNiagaraVariableBase	PreviousPositionVariable;
	FNiagaraVariableBase	PreviousRibbonWidthVariable;
	FNiagaraVariableBase	PreviousScaleVariable;
	FNiagaraVariableBase	PreviousSpriteAlignmentVariable;
	FNiagaraVariableBase	PreviousSpriteFacingVariable;
	FNiagaraVariableBase	PreviousSpriteSizeVariable;
	FNiagaraVariableBase	PreviousSpriteRotationVariable;
	FNiagaraVariableBase	PreviousVelocityVariable;

	ENiagaraStatelessFeatureMask	FeatureMask = ENiagaraStatelessFeatureMask::All;

	inline static FLinearColor	GetDefaultColorValue() { return FLinearColor::White; }
	inline static FVector4f		GetDefaultDynamicMaterialParametersValue() { return FVector4f::Zero(); }
	inline static float			GetDefaultLifetimeValue() { return 1.0f; }
	inline static float			GetDefaultMassValue() { return 1.0f; }
	inline static FQuat4f		GetDefaultMeshOrientationValue() { return FQuat4f::Identity; }
	inline static float			GetDefaultRibbonWidthValue() { return 10.0f; }
	inline static FVector3f		GetDefaultScaleValue() { return FVector3f::OneVector; }
	inline static FVector2f		GetDefaultSpriteSizeValue() { return FVector2f(10.0f); }
	inline static float			GetDefaultSpriteRotationValue() { return 0.0f; }

	NIAGARA_API static const FNiagaraStatelessGlobals& Get();
};

namespace NiagaraStatelessCommon
{
	extern void Initialize();
	extern void UpdateSettings();

	extern FNiagaraVariableBase ConvertParticleVariableToStateless(const FNiagaraVariableBase& InVariable);
}
