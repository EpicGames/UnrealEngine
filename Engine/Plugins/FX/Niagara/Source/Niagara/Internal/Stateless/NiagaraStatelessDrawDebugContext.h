// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraStatelessCommon.h"

#if WITH_EDITOR
struct FNiagaraStatelessDrawDebugContext
{
public:
	FNiagaraStatelessDrawDebugContext(UWorld* InWorld, const FTransform& LocalToWorldIn, bool bIsLocalSpace);

	// Debug draw functions that all operate in world space
	NIAGARA_API void DrawAxis(const FVector& Origin, const FQuat& Rotation, float Scale) const;
	NIAGARA_API void DrawArrow(const FVector& Origin, const FVector& DirectionWithLength, const FColor& Color = FColor::White) const;
	NIAGARA_API void DrawBox(const FVector& Center, const FQuat& Rotation, const FVector& Extent, const FColor& Color = FColor::White) const;
	NIAGARA_API void DrawCone(const FVector& Origin, const FQuat& Rotation, float Angle, float Length, const FColor& Color = FColor::White) const;
	NIAGARA_API void DrawCylinder(const FVector& Center, const FQuat& Rotation, const FVector& Scale, float CylinderHeight, float CylinderRadius, float CylinderHeightMidpoint, const FColor& Color = FColor::White) const;
	NIAGARA_API void DrawCircle(const FVector& Center, const FQuat& Rotation, const FVector& Scale, const float Radius, const FColor& Color = FColor::White) const;
	NIAGARA_API void DrawRing(const FVector& Center, const FQuat& Rotation, const FVector& Scale, const float InnerRadius, const float OuterRadius, float Distribution, const FColor& Color = FColor::White) const;
	NIAGARA_API void DrawSphere(const FVector& Center, const FQuat& Rotation, const FVector& Scale, const float Radius, const FColor& Color = FColor::White) const;

	void DrawCylinder(const FVector& Center, const FQuat& Rotation, float CylinderHeight, float CylinderRadius, float CylinderHeightMidpoint, const FColor& Color = FColor::White) const
	{
		DrawCylinder(Center, Rotation, FVector::OneVector, CylinderHeight, CylinderRadius, CylinderHeightMidpoint, Color);
	}

	void DrawCircle(const FVector& Center, const FQuat& Rotation, const float Radius, const FColor& Color = FColor::White) const
	{
		DrawCircle(Center, Rotation, FVector::OneVector, Radius, Color);
	}

	void DrawRing(const FVector& Center, const FQuat& Rotation, const float InnerRadius, const float OuterRadius, float Distribution, const FColor& Color = FColor::White) const
	{
		DrawRing(Center, Rotation, FVector::OneVector, InnerRadius, OuterRadius, Distribution, Color);
	}

	void DrawSphere(const FVector& Center, const float Radius, const FColor& Color = FColor::White) const
	{
		DrawSphere(Center, FQuat::Identity, FVector::OneVector, Radius, Color);
	}

	// Helper functions to transform into World space for debug rendering
	NIAGARA_API FQuat   TransformRotation(ENiagaraCoordinateSpace SourceSpace, const FQuat4f& Rotation) const;
	NIAGARA_API FVector TransformPosition(ENiagaraCoordinateSpace SourceSpace, const FVector3f& Position) const;
	NIAGARA_API FVector TransformVector(ENiagaraCoordinateSpace SourceSpace, const FVector3f& Vector) const;
	NIAGARA_API FVector TransformVectorNoScale(ENiagaraCoordinateSpace SourceSpace, const FVector3f& Vector) const;

	FQuat   TransformRotation(const FQuat4f& Rotation) const { return TransformRotation(ENiagaraCoordinateSpace::Local, Rotation); }
	FVector TransformPosition(const FVector3f& Position) const { return TransformPosition(ENiagaraCoordinateSpace::Local, Position); }
	FVector TransformVector(const FVector3f& Vector) const { return TransformVector(ENiagaraCoordinateSpace::Local, Vector); }
	FVector TransformVectorNoScale(const FVector3f& Vector) const { return TransformVectorNoScale(ENiagaraCoordinateSpace::Local, Vector); }

private:
	UWorld*		World = nullptr;
	FTransform	LocalToWorld = FTransform::Identity;
	FVector		LWCTileOffset = FVector::ZeroVector;
	bool		bApplyLocalToWorld[3] = {};
};
#endif
