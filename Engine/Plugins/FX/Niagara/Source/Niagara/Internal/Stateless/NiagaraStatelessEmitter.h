// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraStatelessCommon.h"
#include "NiagaraStatelessSpawnInfo.h"
#include "NiagaraEmitterBase.h"
#include "NiagaraDataSet.h"
#include "NiagaraEffectType.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraSystemEmitterState.h"
#include "Stateless/NiagaraStatelessEmitterTemplate.h"

#include "NiagaraStatelessEmitter.generated.h"

class UNiagaraParameterCollection;
struct FNiagaraStatelessEmitterData;
class UNiagaraStatelessModule;
class UNiagaraRendererProperties;
namespace NiagaraStateless
{
	class FCommonShaderParameters;
}

using FNiagaraStatelessEmitterDataPtr = TSharedPtr<const FNiagaraStatelessEmitterData, ESPMode::ThreadSafe>;

/**
* Editor data for stateless emitters
* Generates runtime data to be consumed by the game
*/
UCLASS(MinimalAPI, EditInlineNew, meta = (ShowTopLevelCategories))
class UNiagaraStatelessEmitter : public UNiagaraEmitterBase
{
	GENERATED_BODY()

	friend class UNiagaraStatelessEmitterTemplate;

public:
	DECLARE_MULTICAST_DELEGATE(FOnTemplateChanged);

public:
	//~Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual bool NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End UObject Interface

	//~Begin UNiagaraEmitterBase Interface
	virtual bool CanObtainParticleAttribute(const FNiagaraVariableBase& InVar, const FGuid& EmitterVersion, FNiagaraTypeDefinition& OutBoundType) const override;
	virtual bool CanObtainEmitterAttribute(const FNiagaraVariableBase& InVarWithUniqueNameNamespace, FNiagaraTypeDefinition& OutBoundType) const override;
#if WITH_EDITOR
	virtual void HandleVariableRenamed(const FNiagaraVariable& InOldVariable, const FNiagaraVariable& InNewVariable, bool bUpdateContexts, FGuid EmitterVersion) override;
	virtual void HandleVariableRemoved(const FNiagaraVariable& InOldVariable, bool bUpdateContexts, FGuid EmitterVersion) override;
#endif
	//~End UNiagaraEmitterBase Interface

#if WITH_EDITOR
	// Callback is owner name, variable, read, write
	NIAGARA_API void GatherSimulationAttributeReferences(const FNiagaraStatelessSpawnInfo& SpawnInfo, TFunction<void(FName, const FNiagaraVariableBase&, bool, bool)> ReferenceFound) const;
	NIAGARA_API void GatherSimulationAttributeReferences(const UNiagaraStatelessModule* Module, TFunction<void(FName, const FNiagaraVariableBase&, bool, bool)> ReferenceFound) const;
	NIAGARA_API void GatherSimulationAttributeReferences(TFunction<void(FName, const FNiagaraVariableBase&, bool, bool)> ReferenceFound) const;

	FOnTemplateChanged& GetOnTemplateChanged() { return OnTemplateChanged; }

protected:
	void PostTemplateChanged();
	void CacheParameterCollectionReferences();
#endif

public:
	NIAGARA_API const UNiagaraStatelessEmitterTemplate* GetEmitterTemplate() const;

	bool UsesCollection(const UNiagaraParameterCollection* Collection) const;

	const TArray<UNiagaraRendererProperties*>& GetRenderers() { return RendererProperties; }
	const TArray<UNiagaraRendererProperties*>& GetRenderers() const { return RendererProperties; }

	void CacheFromCompiledData();
protected:
	ENiagaraSimTarget ComputeSimTarget() const;
	void BuildCompiledDataSet();
	void ResolveScalabilitySettings();

public:
	template<typename TAction>
	void ForEachEnabledRenderer(TAction Func) const
	{
		if (RendererProperties.Num() == 0)
		{
			return;
		}

		const ENiagaraSimTarget SimTarget = ComputeSimTarget();
		for (UNiagaraRendererProperties* Renderer : RendererProperties)
		{
			if (Renderer && Renderer->GetIsEnabled() && Renderer->IsSimTargetSupported(SimTarget))
			{
				Func(Renderer);
			}
		}
	}
	template<typename TAction>
	void ForEachEnabledRendererWithIndex(TAction Func) const
	{
		if (RendererProperties.Num() == 0)
		{
			return;
		}

		const ENiagaraSimTarget SimTarget = ComputeSimTarget();
		for (int32 i=0; i < RendererProperties.Num(); ++i)
		{
			UNiagaraRendererProperties* Renderer = RendererProperties[i];
			if (Renderer && Renderer->GetIsEnabled() && Renderer->IsSimTargetSupported(SimTarget))
			{
				Func(Renderer, i);
			}
		}
	}

	template<typename TAction>
	void ForEachRenderer(TAction Func) const
	{
		for (UNiagaraRendererProperties* Renderer : RendererProperties)
		{
			if (Renderer)
			{
				Func(Renderer);
			}
		}
	}

	FNiagaraStatelessEmitterDataPtr GetEmitterData() const { return StatelessEmitterData; }
	NiagaraStateless::FCommonShaderParameters* AllocateShaderParameters(const FNiagaraStatelessSpaceTransforms& SpaceTransforms, const FNiagaraParameterStore& RendererBindings) const;

	NIAGARA_API bool IsAllowedByScalability() const;

#if WITH_EDITOR
	NIAGARA_API void SetEmitterTemplate(UNiagaraStatelessEmitterTemplate* Template);

	NIAGARA_API void AddRenderer(UNiagaraRendererProperties* Renderer, FGuid EmitterVersion);
	NIAGARA_API void RemoveRenderer(UNiagaraRendererProperties* Renderer, FGuid EmitterVersion);
	NIAGARA_API void MoveRenderer(UNiagaraRendererProperties* Renderer, int32 NewIndex, FGuid EmitterVersion);

	NIAGARA_API FNiagaraStatelessSpawnInfo& AddSpawnInfo();

	NIAGARA_API void RemoveSpawnInfoBySourceId(FGuid& InSourceIdToRemove);

	NIAGARA_API int32 IndexOfSpawnInfoBySourceId(const FGuid& InSourceId);

	NIAGARA_API FNiagaraStatelessSpawnInfo* FindSpawnInfoBySourceId(const FGuid& InSourceId);

	int32 GetNumSpawnInfos() const { return SpawnInfos.Num(); }

	NIAGARA_API FNiagaraStatelessSpawnInfo* GetSpawnInfoByIndex(int32 Index);

	const TArray<TObjectPtr<UNiagaraStatelessModule>>& GetModules() const { return Modules; }

	template<typename TType>
	const TType* GetModule() const { return (TType*)GetModule(TType::StaticClass()); }
	NIAGARA_API UNiagaraStatelessModule* GetModule(UClass* Class) const;

	FNiagaraPlatformSet& GetPlatformSet() { return Platforms; }
	FNiagaraEmitterScalabilityOverrides& GetScalabilityOverrides() { return ScalabilityOverrides; }

	UNiagaraStatelessEmitter* CreateAsDuplicate(FName InDuplicateName, UNiagaraSystem& InDuplicateOwnerSystem) const;

	NIAGARA_API void DrawModuleDebug(UWorld* World, const FTransform& LocalToWorld) const;
#endif //WITH_NIAGARA_DEBUGGER

protected:
	TSharedPtr<FNiagaraStatelessEmitterData, ESPMode::ThreadSafe> StatelessEmitterData;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UClass> EmitterTemplateClass_DEPRECATED;
#endif

	UPROPERTY(EditAnywhere, Category = "Emitter")
	TObjectPtr<UNiagaraStatelessEmitterTemplate> EmitterTemplate;

	UPROPERTY(EditAnywhere, Category = "Emitter")
	uint32 bDeterministic : 1 = false;

#if WITH_EDITORONLY_DATA
	/**
	When enabled the emitter will output all available attributes.
	You should not need to modify this with the exception of debugging / testing and as it will impact cooked performance and memory
	*/
	UPROPERTY(EditAnywhere, Category = "Emitter", AdvancedDisplay)
	uint32 bForceOutputAllAttributes : 1 = false;

	/**
	When enabled the emitter will always include UniqueID in the output attributes.
	You should not need to modify this with the exception of debugging / testing and as it will impact cooked performance and memory
	*/
	UPROPERTY(EditAnywhere, Category = "Emitter", AdvancedDisplay, meta = (EditCondition = "!bForceOutputAllAttributes"))
	uint32 bForceOutputUniqueID : 1 = false;
#endif

	UPROPERTY(EditAnywhere, Category = "Emitter", AdvancedDisplay, meta = (Bitmask, BitMaskEnum = "/Script/Niagara.ENiagaraStatelessFeatureMask"))
	uint32 AllowedFeatureMask = uint32(ENiagaraStatelessFeatureMask::All);

	UPROPERTY(EditAnywhere, Category = "Emitter")
	int32 RandomSeed = 0;

	UPROPERTY(EditAnywhere, Category = "Emitter")
	FBox FixedBounds = FBox(FVector(-100), FVector(100));

	UPROPERTY(EditAnywhere, Category = "Emitter State")
	FNiagaraEmitterStateData EmitterState;

	UPROPERTY(EditAnywhere, Category = "Spawn Info", meta = (HideInStack))
	TArray<FNiagaraStatelessSpawnInfo> SpawnInfos;

	UPROPERTY(EditAnywhere, Category = "Modules", Instanced, meta = (HideInStack))
	TArray<TObjectPtr<UNiagaraStatelessModule>> Modules;

	UPROPERTY(EditAnywhere, Category = "Renderer", Instanced, meta = (HideInStack))
	TArray<TObjectPtr<UNiagaraRendererProperties>> RendererProperties;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayInScalabilityContext))
	FNiagaraPlatformSet Platforms;

	UPROPERTY(EditAnywhere, Category = "Scalability", meta = (DisplayInScalabilityContext))
	FNiagaraEmitterScalabilityOverrides ScalabilityOverrides;

	UPROPERTY()
	FNiagaraDataSetCompiledData ParticleDataSetCompiledData;

	UPROPERTY()
	TArray<int32> ShaderOutputVariableOffsets;

	UPROPERTY()
	TArray<TObjectPtr<UNiagaraParameterCollection>> CachedParameterCollectionReferences;

#if WITH_EDITOR
	FOnTemplateChanged OnTemplateChanged;
#endif
};
