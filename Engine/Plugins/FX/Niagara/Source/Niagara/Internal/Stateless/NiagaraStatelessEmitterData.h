// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraStatelessCommon.h"
#include "NiagaraStatelessSpawnInfo.h"
#include "NiagaraDataSet.h"
#include "NiagaraParameterStore.h"
#include "NiagaraSystemEmitterState.h"

#include "Shader.h"

class UNiagaraParameterCollection;
class UNiagaraRendererProperties;
class UNiagaraStatelessEmitterTemplate;
namespace NiagaraStateless
{
	class FSimulationShader;
	class FSpawnInfoShaderParameters;
	class FParticleSimulationExecData;
}
struct FInstancedStruct;

struct FNiagaraStatelessEmitterData
{
	friend class FNiagaraStatelessEmitterDataBuildContext;

	UE_NONCOPYABLE(FNiagaraStatelessEmitterData);

	FNiagaraStatelessEmitterData() = default;
	~FNiagaraStatelessEmitterData();

#if WITH_NIAGARA_DEBUG_EMITTER_NAME
	FName											DebugSimulationName;
	FName											DebugEmitterName;
#endif

	TSharedPtr<FNiagaraDataSetCompiledData>			ParticleDataSetCompiledData;
	TArray<int32>									ShaderOutputVariableOffsets;

	bool											bCanEverExecute = false;
	bool											bDeterministic = false;
	ENiagaraSimTarget								SimTarget = ENiagaraSimTarget::GPUComputeSim;
	ENiagaraStatelessFeatureMask					FeatureMask = ENiagaraStatelessFeatureMask::None;
	int32											RandomSeed = 0;
	FNiagaraStatelessRangeFloat						LifetimeRange = FNiagaraStatelessRangeFloat(0.0f, 0.0f);
	FBox											FixedBounds = FBox(ForceInit);

	FNiagaraEmitterStateData						EmitterState;
	TArray<FNiagaraStatelessSpawnInfo>				SpawnInfos;
	float											SpawnCountScale = 1.0f;

	TArray<TObjectPtr<UNiagaraRendererProperties>>	RendererProperties;
	TArray<TObjectPtr<UNiagaraParameterCollection>>	BoundParameterCollections;

	bool											bModulesHaveRendererBindings = false;
	FNiagaraParameterStore							RendererBindings;			// Contains all bindings for modules & renderers
	TArray<TPair<int32, FInstancedStruct>>			Expressions;				// Contains a mapping of expression to paramater store entry

	const UNiagaraStatelessEmitterTemplate*			EmitterTemplate = nullptr;	// Used to access shader information

	TArray<uint8>									BuiltData;					// Built data, generally allocated by modules if any
	TArray<uint8>									SharedBuiltData;			// Shared built data, allocated by modules generally used during parameter setting
	TArray<TPair<FName, uint32>>					SharedBuiltDataOffsets;		// Shared build data offsets, used to find the data
	TArray<uint8>									StaticDataBufferCpu;		// Used with CPU generation, must be valid if ParticleSimExecData is also valid
	FByteAddressBuffer								StaticDataBufferGpu;		// Used with GPU generation, guaranteed to be minimum safe size

	NiagaraStateless::FParticleSimulationExecData*	ParticleSimExecData = nullptr;	// CPU simulation execution data, when null we don't provide a CPU path

	void InitRenderResources();

	TShaderRef<NiagaraStateless::FSimulationShader>	GetShader() const;
	const FShaderParametersMetadata* GetShaderParametersMetadata() const;

	// Calculates the completion age based on the spawn infos / maximum potential lifetime
	//float CalculateCompletionAge(int32 InRandomSeed, TConstArrayView<FNiagaraStatelessRuntimeSpawnInfo> RuntimeSpawnInfos) const;

	// Calculcate the active particle count for all the spawn infos
	// Optionally fills out GPU spawning data into SpawnParameters
	// If no Age is provided we are calculating the maximum number of particles we could ever spawn
	uint32 CalculateActiveParticles(int32 InRandomSeed, TConstArrayView<FNiagaraStatelessRuntimeSpawnInfo> RuntimeSpawnInfos, TOptional<float> Age = TOptional<float>(), NiagaraStateless::FSpawnInfoShaderParameters* SpawnParameters = nullptr) const;
};

using FNiagaraStatelessEmitterDataPtr = TSharedPtr<const FNiagaraStatelessEmitterData, ESPMode::ThreadSafe>;
