// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessCommon.h"
#include "Stateless/NiagaraStatelessDistribution.h"
#include "NiagaraStatelessBuiltDistribution.h"

struct FNiagaraDataSetCompiledData;
struct FNiagaraParameterBinding;
struct FNiagaraParameterBindingWithValue;
struct FNiagaraParameterStore;
struct FNiagaraStatelessEmitterData;
namespace NiagaraStateless
{
	class FParticleSimulationContext;
	class FParticleSimulationExecData;
}
struct FInstancedStruct;

class FNiagaraStatelessEmitterDataBuildContext
{
public:
	UE_NONCOPYABLE(FNiagaraStatelessEmitterDataBuildContext);

	explicit FNiagaraStatelessEmitterDataBuildContext(FNiagaraStatelessEmitterData& InEmitterData);
	~FNiagaraStatelessEmitterDataBuildContext();

	void PreModuleBuild(int32 InShaderParameterOffset);

	uint32 AddStaticData(TConstArrayView<uint32> StaticData) const;
	uint32 AddStaticData(TConstArrayView<int32> StaticData) const;
	uint32 AddStaticData(TConstArrayView<float> StaticData) const;
	uint32 AddStaticData(TConstArrayView<FVector2f> StaticData) const;
	uint32 AddStaticData(TConstArrayView<FVector3f> StaticData) const;
	uint32 AddStaticData(TConstArrayView<FVector4f> StaticData) const;
	uint32 AddStaticData(TConstArrayView<FLinearColor> StaticData) const;

	NIAGARA_API void* AllocateBuiltData(uint32 Size, uint32 Alingment) const;
	NIAGARA_API void* AllocateSharedBuiltData(FName DataName, uint32 Size, uint32 Alingment) const;

	template<typename T>
	T* AllocateBuiltData() const
	{
		static_assert(TIsTrivial<T>::Value, "Only trivial types can be used for built data");
		void* NewData = AllocateBuiltData(sizeof(T), alignof(T));
		return new(NewData) T();
	}

	template<typename T>
	T* AllocateSharedBuiltData() const
	{
		void* NewData = AllocateSharedBuiltData(T::GetName(), sizeof(T), alignof(T));
		return new(NewData) T();
	}

	template<typename T>
	T& GetTransientBuildData() const
	{
		TUniquePtr<FTransientObject>& TransientObj = TransientBuildData.FindOrAdd(T::GetName());
		if (TransientObj.IsValid() == false)
		{
			TransientObj.Reset(new TTransientObject<T>);
		}
		return *reinterpret_cast<T*>(TransientObj->GetObject());
	}

	// Adds a binding to the renderer parameter store
	// This allows you to read the parameter data inside the simulation process
	// The returned value is INDEX_NONE is the variables is index otherwise the offset in DWORDs
	int32 AddRendererBinding(const FNiagaraVariableBase& Variable) const;
	int32 AddRendererBinding(const FNiagaraParameterBinding& Binding) const;
	int32 AddRendererBinding(const FNiagaraParameterBindingWithValue& Binding) const;

private:
	// Adds a binding to the renderer parameter store
	// The returned value is INDEX_NONE is the variables is index otherwise the offset in DWORDs
	int32 AddExpression(const FInstancedStruct& Expression) const;

private:
	template<typename TType>
	FNiagaraStatelessBuiltDistributionType InternalAddDistribution(ENiagaraDistributionMode Mode, TConstArrayView<TType> Values, const FVector2f& TimeRange, const uint8 LookupValueMode, TType DefaultValue) const
	{
		using namespace NiagaraStateless;
		
		if (Values.Num() == 0)
		{
			Values = MakeArrayView(&DefaultValue, 1);
		}

		FNiagaraStatelessBuiltDistributionType BuiltDistribution = FNiagaraStatelessBuiltDistribution::GetDefault();

		FNiagaraStatelessBuiltDistribution::SetBufferReadMode(BuiltDistribution, true);
		FNiagaraStatelessBuiltDistribution::SetInterpolationMode(BuiltDistribution, true);
		FNiagaraStatelessBuiltDistribution::SetAddressMode(BuiltDistribution, true);

		// Uniform random controls if we should interpolate per channel or not
		// ColorGradient is here as we would not want to interpolate between keys differently per chnanel like the other Uniform types
		const bool bUniformRandom = 
			Mode == ENiagaraDistributionMode::UniformConstant ||
			Mode == ENiagaraDistributionMode::UniformRange ||
			Mode == ENiagaraDistributionMode::UniformCurve ||
			Mode == ENiagaraDistributionMode::ColorGradient;

		const bool bIsRandom = 
			Mode == ENiagaraDistributionMode::UniformRange ||
			Mode == ENiagaraDistributionMode::NonUniformRange;

		switch (Mode)
		{
			case ENiagaraDistributionMode::UniformConstant:
				Values = MakeArrayView(Values.GetData(), 1);
				break;

			case ENiagaraDistributionMode::UniformRange:
				check(Values.Num() >= 2);
				Values = MakeArrayView(Values.GetData(), 2);
				break;

			case ENiagaraDistributionMode::Array:
			case ENiagaraDistributionMode::Binding:
			case ENiagaraDistributionMode::Expression:
				checkNoEntry();
				break;

			default:
				break;
		}

		FNiagaraStatelessBuiltDistribution::SetUniformRandom(BuiltDistribution, bUniformRandom);
		FNiagaraStatelessBuiltDistribution::SetLookupValueMode(BuiltDistribution, bIsRandom ? uint8(ENiagaraDistributionLookupValueMode::Random) : LookupValueMode);

		FNiagaraStatelessBuiltDistribution::SetLookupParameters(BuiltDistribution, AddStaticData(Values), Values.Num(), TimeRange.X, TimeRange.Y);
		return BuiltDistribution;
	}

	template<typename TType>
	FNiagaraStatelessBuiltDistributionType InternalAddDistribution(uint32 DataOffset, uint32 NumEntries, TType DefaultValue, bool bDataIsStaticBuffer) const
	{
		using namespace NiagaraStateless;

		FNiagaraStatelessBuiltDistributionType BuiltDistribution = FNiagaraStatelessBuiltDistribution::GetDefault();
		FNiagaraStatelessBuiltDistribution::SetInterpolationMode(BuiltDistribution, false);
		FNiagaraStatelessBuiltDistribution::SetAddressMode(BuiltDistribution, true);
		FNiagaraStatelessBuiltDistribution::SetUniformRandom(BuiltDistribution, true);
		FNiagaraStatelessBuiltDistribution::SetLookupValueMode(BuiltDistribution, 0);

		if ( DataOffset == INDEX_NONE || NumEntries == 0 )
		{
			FNiagaraStatelessBuiltDistribution::SetBufferReadMode(BuiltDistribution, true);
			FNiagaraStatelessBuiltDistribution::SetLookupParameters(BuiltDistribution, AddStaticData(MakeArrayView(&DefaultValue, 1)), 1);
		}
		else
		{
			FNiagaraStatelessBuiltDistribution::SetBufferReadMode(BuiltDistribution, bDataIsStaticBuffer);
			FNiagaraStatelessBuiltDistribution::SetLookupParameters(BuiltDistribution, DataOffset, NumEntries);
		}
		return BuiltDistribution;
	}

public:
	// Adds a distribution into the LUT if enabled and returns the packed information to send to the shader
	template<typename TDistribution>
	FNiagaraStatelessBuiltDistributionType AddDistribution(const TDistribution& Distribution) const
	{
		if (Distribution.Mode == ENiagaraDistributionMode::Array)
		{
			FNiagaraStatelessBuiltDistributionType BuiltDistribution = InternalAddDistribution(AddStaticData(Distribution.Values), Distribution.Values.Num(), TDistribution::GetDefaultValue(), true);

			FNiagaraStatelessBuiltDistribution::SetInterpolationMode(BuiltDistribution, Distribution.GetInterpolationMode() == ENiagaraDistributionInterpolationMode::Linear);
			FNiagaraStatelessBuiltDistribution::SetAddressMode(BuiltDistribution, Distribution.GetAddressMode() == ENiagaraDistributionAddressMode::Clamp);
			FNiagaraStatelessBuiltDistribution::SetUniformRandom(BuiltDistribution, true);

			const uint8 LookupValueMode = Distribution.GetLookupValueMode();
			FNiagaraStatelessBuiltDistribution::SetLookupValueMode(BuiltDistribution, LookupValueMode);
			FNiagaraStatelessBuiltDistribution::SetTimeScaleBias(BuiltDistribution, Distribution.ValuesTimeRange.X, Distribution.ValuesTimeRange.Y);
			return BuiltDistribution;
		}
		else if (Distribution.Mode == ENiagaraDistributionMode::Binding)
		{
			return InternalAddDistribution(AddRendererBinding(Distribution.ParameterBinding), 1, TDistribution::GetDefaultValue(), false);
		}
		else if (Distribution.Mode == ENiagaraDistributionMode::Expression)
		{
			return InternalAddDistribution(AddExpression(Distribution.ParameterExpression), 1, TDistribution::GetDefaultValue(), false);
		}
		return InternalAddDistribution(Distribution.Mode, MakeArrayView(Distribution.Values), Distribution.ValuesTimeRange, Distribution.GetLookupValueMode(), TDistribution::GetDefaultValue());
	}

	// Adds a distribution and forces it to generate as a curve for lookup
	template<typename TDistribution, typename TType>
	FNiagaraStatelessBuiltDistributionType AddDistributionAsCurve(const TDistribution& Distribution, TType DefaultValue) const
	{
		FNiagaraStatelessBuiltDistributionType BuiltDistribution = FNiagaraStatelessBuiltDistribution::GetDefault();
		if (ensure((Distribution.IsCurve() || Distribution.IsGradient()) && Distribution.Values.Num() > 1))
		{
			return InternalAddDistribution<TType>(Distribution.Mode, MakeArrayView(Distribution.Values), Distribution.ValuesTimeRange, Distribution.GetLookupValueMode(), DefaultValue);
		}
		else
		{
			const TType DefaultValues[] = {DefaultValue , DefaultValue};
			return InternalAddDistribution(ENiagaraDistributionMode::NonUniformCurve, MakeArrayView(DefaultValues), Distribution.ValuesTimeRange, Distribution.GetLookupValueMode(), DefaultValue);
		}
	}

	template<typename TRange, typename TDistribution, typename TDefaultValue>
	TRange ConvertDistributionToRangeHelper(const TDistribution& Distribution, const TDefaultValue& DefaultValue) const
	{
		TRange Range(DefaultValue);
		if (Distribution.Mode == ENiagaraDistributionMode::Binding)
		{
			//-OPT: If the expression is constant we can just resolve to a range rather than adding to the parameter store
			Range.ParameterOffset = AddRendererBinding(Distribution.ParameterBinding);
		}
		else if (Distribution.Mode == ENiagaraDistributionMode::Expression)
		{
			//-OPT: If the expression is constant we can just resolve to a range rather than adding to the parameter store
			Range.ParameterOffset = AddExpression(Distribution.ParameterExpression);
		}
		else
		{
			Range = Distribution.CalculateRange(DefaultValue);
		}
		return Range;
	}

	FNiagaraStatelessRangeFloat   ConvertDistributionToRange(const FNiagaraDistributionFloat& Distribution, float DefaultValue) const { return ConvertDistributionToRangeHelper<FNiagaraStatelessRangeFloat>(Distribution, DefaultValue); }
	FNiagaraStatelessRangeVector2 ConvertDistributionToRange(const FNiagaraDistributionVector2& Distribution, const FVector2f& DefaultValue) const { return ConvertDistributionToRangeHelper<FNiagaraStatelessRangeVector2>(Distribution, DefaultValue); }
	FNiagaraStatelessRangeVector3 ConvertDistributionToRange(const FNiagaraDistributionVector3& Distribution, const FVector3f& DefaultValue) const { return ConvertDistributionToRangeHelper<FNiagaraStatelessRangeVector3>(Distribution, DefaultValue); }
	FNiagaraStatelessRangeColor   ConvertDistributionToRange(const FNiagaraDistributionColor& Distribution, const FLinearColor& DefaultValue) const { return ConvertDistributionToRangeHelper<FNiagaraStatelessRangeColor>(Distribution, DefaultValue); }

	FNiagaraStatelessRangeFloat   ConvertDistributionToRange(const FNiagaraDistributionRangeFloat& Distribution, float DefaultValue) const { return ConvertDistributionToRangeHelper<FNiagaraStatelessRangeFloat>(Distribution, DefaultValue); }
	FNiagaraStatelessRangeVector2 ConvertDistributionToRange(const FNiagaraDistributionRangeVector2& Distribution, const FVector2f& DefaultValue) const { return ConvertDistributionToRangeHelper<FNiagaraStatelessRangeVector2>(Distribution, DefaultValue); }
	FNiagaraStatelessRangeVector3 ConvertDistributionToRange(const FNiagaraDistributionRangeVector3& Distribution, const FVector3f& DefaultValue) const { return ConvertDistributionToRangeHelper<FNiagaraStatelessRangeVector3>(Distribution, DefaultValue); }
	FNiagaraStatelessRangeColor   ConvertDistributionToRange(const FNiagaraDistributionRangeColor& Distribution, const FLinearColor& DefaultValue) const { return ConvertDistributionToRangeHelper<FNiagaraStatelessRangeColor>(Distribution, DefaultValue); }
	FNiagaraStatelessRangeRotator ConvertDistributionToRange(const FNiagaraDistributionRangeRotator& Distribution, const FRotator3f& DefaultValue) const { return ConvertDistributionToRangeHelper<FNiagaraStatelessRangeRotator>(Distribution, DefaultValue); }
	FNiagaraStatelessRangeInt     ConvertDistributionToRange(const FNiagaraDistributionRangeInt& Distribution, int32 DefaultValue) const { return ConvertDistributionToRangeHelper<FNiagaraStatelessRangeInt>(Distribution, DefaultValue); }

	NIAGARA_API void AddParticleSimulationExecSimulate(TFunction<void(const NiagaraStateless::FParticleSimulationContext&)> Func) const;

	NIAGARA_API int32 FindParticleVariableIndex(const FNiagaraVariableBase& Variable) const;

private:
	FNiagaraStatelessEmitterData&	EmitterData;

	int32							ModuleBuiltDataOffset = 0;
	int32							ShaderParameterOffset = 0;
	int32							RandomSeedOffest = 0;

	struct FTransientObject
	{
		virtual ~FTransientObject() = default;
		virtual void* GetObject() = 0;
	};

	template <typename T>
	struct TTransientObject final : FTransientObject
	{
		template <typename... TArgs>
		FORCEINLINE TTransientObject(TArgs&&... Args) : TheObject(Forward<TArgs&&>(Args)...) {}
		virtual void* GetObject() { return &TheObject; }

		T TheObject;
	};

	mutable TMap<FName, TUniquePtr<FTransientObject>>	TransientBuildData;
};
