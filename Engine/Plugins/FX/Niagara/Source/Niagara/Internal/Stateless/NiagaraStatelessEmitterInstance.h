// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraComputeExecutionContext.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraStatelessEmitter.h"
#include "Math/RandomStream.h"

class FNiagaraStatelessComputeManager;
class FShaderParametersMetadata;

namespace NiagaraStateless
{
	struct FShaderParametersDeletor
	{
		void operator()(NiagaraStateless::FCommonShaderParameters* Ptr) const
		{
			FMemory::Free(Ptr);
		}
	};

	using FShaderParametersPtr = TUniquePtr<NiagaraStateless::FCommonShaderParameters, FShaderParametersDeletor>;

	class FEmitterInstance_RT : public INiagaraComputeDataBufferInterface
	{
	public:
		virtual ~FEmitterInstance_RT() = default;

		FNiagaraStatelessComputeManager*						ComputeManager = nullptr;
		FNiagaraStatelessEmitterDataPtr							EmitterData;

		int32													RandomSeed = 0;
		float													Age = 0.0f;
		float													DeltaTime = 0.0f;		//-OPT: We should be able to pull this from the view which is important for zero tick path
		ENiagaraExecutionState									ExecutionState = ENiagaraExecutionState::Active;
		TArray<FNiagaraStatelessRuntimeSpawnInfo>				SpawnInfos;
		FShaderParametersPtr									ShaderParameters;

		mutable bool											bBindingBufferDirty = true;
		mutable TArray<uint8>									BindingBufferData;
		mutable FByteAddressBuffer								BindingBuffer;

		// Begin: INiagaraComputeDataBufferInterface
		virtual bool HasTranslucentDataToRender() const override { return false; }
		virtual FNiagaraDataBuffer* GetDataToRender(FRHICommandListBase& RHICmdList, bool bIsLowLatencyTranslucent) const override;
		// End: INiagaraComputeDataBufferInterface
	};
}

/**
* Implementation of a stateless Niagara particle simulation
*/
class FNiagaraStatelessEmitterInstance final : public FNiagaraEmitterInstance
{
	using Super = FNiagaraEmitterInstance;

	struct FActiveSpawnRate
	{
		FNiagaraDistributionRangeFloat				SpawnRate = FNiagaraDistributionRangeFloat(0.0f);
		float										ResidualSpawnTime = 0.0f;
		TOptional<FNiagaraDistributionRangeFloat>	SpawnProbability;
	};

public:
	explicit FNiagaraStatelessEmitterInstance(FNiagaraSystemInstance* InParentSystemInstance);
	virtual ~FNiagaraStatelessEmitterInstance();

	// FNiagaraEmitterInstance Impl
	virtual void Init(int32 InEmitterIndex) override;
	virtual void ResetSimulation(bool bKillExisting) override;
	virtual void SetEmitterEnable(bool bNewEnableState) override;
	virtual void OnPooledReuse() override {}
	virtual bool HandleCompletion(bool bForce) override;
	virtual int32 GetNumParticles() const override;
	virtual uint32 GetGpuCountBufferEstimate() const { return 1; }
	virtual FNiagaraStatelessEmitterInstance* AsStateless() override { return this; }
	virtual TConstArrayView<UNiagaraRendererProperties*> GetRenderers() const override;
	virtual void BindParameters(bool bExternalOnly) override;
	virtual void UnbindParameters(bool bExternalOnly) override;
	virtual void RebindParameterCollection(UNiagaraParameterCollectionInstance* OldCollection, UNiagaraParameterCollectionInstance* NewCollection) override;
	virtual bool ShouldTick() const override;
	virtual void Tick(float DeltaSeconds) override;
	// FNiagaraEmitterInstance Impl

	int32 GetRandomSeed() const { return RandomSeed; }

	const FNiagaraStatelessEmitterData* GetEmitterData() const { check(EmitterData.IsValid()); return EmitterData.Get(); }

private:
	void InitEmitterData();

	void InitEmitterState();
	void TickEmitterState();
	void CalculateBounds();
	void UpdateSimulationData(float DeltaSeconds, bool bForceParametersUpdate);

	void InitSpawnInfos(float InitializationAge);
	void InitSpawnInfosForLoop(float InitializationAge);
	void TickSpawnInfos();
	void CropSpawnInfos();
	void KillSpawnInfos();
	void RestartSpawnInfos();

	//-TODO: This can be shared perhaps?
	void SetExecutionStateInternal(ENiagaraExecutionState InExecutionState);

public:
	void EnqueueParticleDataBufferReadback(FNiagaraDataBuffer* DataBuffer, TFunction<void()> OnReadbackComplete) const;
	void CaptureForDebugging(FNiagaraDataBuffer* DataBuffer) const;

private:
	uint32										bCanEverExecute : 1 = false;
	uint32										bEmitterEnabled_GT : 1 = true;
	uint32										bEmitterEnabled_CNC : 1 = true;
	uint32										bSpawnInfosDirty : 1 = false;
	uint32										bNeedsEmitterStateInit : 1 = false;

	int32										RandomSeed = 0;
	FRandomStream								RandomStream;

	FNiagaraStatelessEmitterDataPtr				EmitterData;
	TWeakObjectPtr<UNiagaraStatelessEmitter>	WeakStatelessEmitter;

	FNiagaraStatelessSpaceTransforms			EmitterTransforms;

	float										Age = 0.0f;

	uint32										UniqueIndexOffset = 0;
	TArray<FNiagaraStatelessRuntimeSpawnInfo>	SpawnInfos;
	TArray<FActiveSpawnRate>					ActiveSpawnRates;

	ENiagaraExecutionState						InternalExecutionState = ENiagaraExecutionState::Active;
	ENiagaraExecutionStateManagement			ScalabilityState = ENiagaraExecutionStateManagement::Awaken;
	int32										LoopCount = 0;
	float										CurrentLoopDuration = 0.0f;
	float										CurrentLoopDelay = 0.0f;
	float										CurrentLoopAgeStart = 0.0f;
	float										CurrentLoopAgeEnd = 0.0f;

	TUniquePtr<NiagaraStateless::FEmitterInstance_RT>	RenderThreadDataPtr;

	NiagaraStateless::FShaderParametersPtr			ShaderParameters;
};
