// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "Stateless/NiagaraStatelessSimulationShader.h"

#include "RenderGraphFwd.h"
#include "ShaderParameterStruct.h"

#include "NiagaraStatelessEmitterTemplate.generated.h"

struct FAssetData;
class FAssetRegistryTagsContext;

UCLASS(MinimalAPI)
class UNiagaraStatelessEmitterTemplate : public UObject
{
	GENERATED_BODY()

public:
	//~Begin: UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	//~End: UObject Interface

	static void InitCDOPropertiesAfterModuleStartup();
	virtual void InitModulesAndAttributes();
#if WITH_EDITOR
	NIAGARA_API void ModifyTemplate(UNiagaraStatelessEmitterTemplate* FromTemplate);

	NIAGARA_API static bool IsExposedToLibrary(const FAssetData& AssetData);
	bool IsExposedToLibrary() const { return bExposedToLibrary; }
#endif

	TConstArrayView<TObjectPtr<UClass>> GetModules() const { return Modules; }
#if WITH_EDITORONLY_DATA
	TConstArrayView<FNiagaraVariableBase> GetImplicitVariables() const { return ImplicitVariables; }
	TConstArrayView<FNiagaraVariableBase> GetModuleVariables() const { return ModuleVariables; }
	TConstArrayView<FNiagaraVariableBase> GetShaderOutputVariables() const { return ShaderOutputVariables; }
#endif

	virtual const FShaderParametersMetadata* GetShaderParametersMetadata() const { check(ShaderParametersMetadata.IsValid()); return ShaderParametersMetadata.Get(); }
	virtual TShaderRef<NiagaraStateless::FSimulationShader> GetSimulationShader() const { return TShaderRef<NiagaraStateless::FSimulationShader>(); }
	virtual void SetShaderParameters(uint8* ShaderParametersBase, TConstArrayView<int32> ShaderOutputVariableOffsets) const;

protected:
	NIAGARA_API void BuildVariables();
	NIAGARA_API void BuildShaderParameters();

protected:
	UPROPERTY(EditAnywhere, Category = "Modules", meta = (DisallowCreateNew, AllowedClasses = "/Script/Niagara.NiagaraStatelessModule"))
	TArray<TObjectPtr<UClass>> Modules;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category="Modules")
	bool bExposedToLibrary = true;

	TArray<FNiagaraVariableBase> ImplicitVariables;				// Variables that are implicit to this template (i.e. are not from modules)
	TArray<FNiagaraVariableBase> ModuleVariables;				// All available variables from modules
	TArray<FNiagaraVariableBase> ShaderOutputVariables;			// Variables that can be output from the shader, this is tied into the mappings sent into the shader.  A subset or exact match of ImplicitVariables & ModuleVariables.
#endif
	TSharedPtr<FShaderParametersMetadata> ShaderParametersMetadata;
};

UCLASS(Category = "Statless", DisplayName = "Default Template")
class UNiagaraStatelessEmitterDefault : public UNiagaraStatelessEmitterTemplate
{
	GENERATED_BODY()

	virtual void InitModulesAndAttributes() override;

	virtual const FShaderParametersMetadata* GetShaderParametersMetadata() const override;
	virtual TShaderRef<NiagaraStateless::FSimulationShader> GetSimulationShader() const override;

	//-TODO: Add a way to set them directly, we should know that the final struct is a series of ints in the order of the provided variables
	virtual void SetShaderParameters(uint8* ShaderParametersBase, TConstArrayView<int32> ShaderOutputVariableOffsets) const override;
	//-TODO: Add a way to set them directly, we should know that the final struct is a series of ints in the order of the provided variables
};
