// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "StructUtils/InstancedStruct.h"

#include "NiagaraStatelessExpression.generated.h"

class FNiagaraStatelessEmitterDataBuildContext;
class FNiagaraStatelessSetShaderParameterContext;

USTRUCT()
struct FNiagaraStatelessExpression
{
	struct FEvaluateContext
	{
		UE_NONCOPYABLE(FEvaluateContext)

		explicit FEvaluateContext(const FNiagaraParameterStore& InParameterStore)
			: ParameterStore(InParameterStore)
		{
		}

		const FNiagaraParameterStore&	ParameterStore;
	};

	GENERATED_BODY()

	virtual ~FNiagaraStatelessExpression() = default;

	virtual FInstancedStruct Build(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const { checkNoEntry(); return FInstancedStruct();  }
	virtual void Evaluate(const FEvaluateContext& Context, void* ValueAddress) const { checkNoEntry(); }
	virtual FNiagaraTypeDefinition GetOutputTypeDef() const { checkNoEntry(); return FNiagaraTypeDefinition(); }
	virtual bool IsConstant() const { return false; }

#if WITH_EDITORONLY_DATA
	static void ForEachBinding(const FInstancedStruct& ExpressionStruct, const TFunction<void(const FNiagaraVariableBase&)>& Delegate);
#endif
};
