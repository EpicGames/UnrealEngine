// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraMergeable.h"
#include "Stateless/NiagaraStatelessCommon.h"
#include "Stateless/NiagaraStatelessDistribution.h"
#include "Stateless/NiagaraStatelessSetShaderParameterContext.h"
#include "Stateless/NiagaraStatelessShaderParametersBuilder.h"

#include "NiagaraStatelessModule.generated.h"

class FNiagaraStatelessEmitterDataBuildContext;
#if WITH_EDITOR
struct FNiagaraStatelessDrawDebugContext;
#endif

UCLASS(MinimalAPI, abstract, EditInlineNew)
class UNiagaraStatelessModule : public UNiagaraMergeable
{
	GENERATED_BODY()

public:
	enum class EVariableFilter
	{
		None,
		Used,
	};

protected:
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DisplayPriority = 0, HideInStack))
	uint32 bModuleEnabled : 1 = true;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Transient, Category = "Parameters", meta = (DisplayPriority = 0, HideInStack, StackItemHeaderIcon = "Icons.Visibility"))
	uint32 bDebugDrawEnabled : 1 = false;
#endif

public:
	NIAGARA_API bool IsModuleEnabled() const;
#if WITH_EDITOR
	bool IsDebugDrawEnabled() const { return bModuleEnabled && bDebugDrawEnabled; }

	struct PrivateMemberNames
	{
		static NIAGARA_API const FName bModuleEnabled;
		static NIAGARA_API const FName bDebugDrawEnabled;
	};
#endif

	// The feature mask is used to define when they can be used
	// For example if the module does not support GPU make sure to remove the GPU feature mask
	// Module & template authors should be careful to not get into a situation where the feature mask is none
	virtual ENiagaraStatelessFeatureMask GetFeatureMask() const { return ENiagaraStatelessFeatureMask::All; }

	// Called during the data build process on each module sequentially
	// Modules add build data that contain all the information needed to setup shader parameters for simulation
	virtual void BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const {}

	// Called during the data build process to create the shader parameter strucutre
	// Modules pack structure or loose parameters that will be used to simulate
	virtual void BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const {}

	// Called at runtime to set the shader parameter structures
	// Modules should keep this simple and used the build process to optimize data as much as possible
	virtual void SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const {}

#if WITH_EDITOR
	virtual bool CanDisableModule() const { return false; }
	void SetIsModuleEnabled(bool bInIsEnabled) { bModuleEnabled = bInIsEnabled; }

	virtual bool CanDebugDraw() const { return false; }
	virtual void DrawDebug(const FNiagaraStatelessDrawDebugContext& DrawDebugContext) const {}
#endif

#if WITH_EDITORONLY_DATA
	// Optional virtual path to shader template
	// Not required for CPU modules or GPU modules that aggregate into another module and build time
	virtual const TCHAR* GetShaderTemplatePath() const { return nullptr; }

	// Gather the output variables for this module
	// EVariableFilter specifies if we should filter to used variables or not
	virtual void GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const {}
#endif

	//~UObject interface Begin
#if WITH_EDITOR
	NIAGARA_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~UObject interface End
};
