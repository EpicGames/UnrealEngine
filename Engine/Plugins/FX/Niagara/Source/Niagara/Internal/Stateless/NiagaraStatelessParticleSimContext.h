// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessCommon.h"
#include "Stateless/NiagaraStatelessRange.h"
#include "NiagaraStatelessBuiltDistribution.h"

#include "ShaderParameterMacros.h"

class FNiagaraDataBuffer;
struct FNiagaraStatelessEmitterData;
struct FNiagaraStatelessRuntimeSpawnInfo;
class FRHICommandListBase;

namespace NiagaraStateless
{
class FSpawnInfoShaderParameters;

enum class EParticleComponent
{
	Alive,
	Lifetime,
	Age,
	NormalizedAge,
	PreviousAge,
	PreviousNormalizedAge,
	UniqueID,
	MaterialRandom,
	Num
};

class FParticleSimulationContext
{
public:
	explicit FParticleSimulationContext(const FNiagaraStatelessEmitterData* EmitterData, const void* InShaderParametersData, TConstArrayView<uint8> DynamicBufferData);

	static TConstArrayView<FNiagaraVariableBase> GetRequiredComponents();

	void Simulate(int32 EmitterRandomSeed, float EmitterAge, float DeltaTime, TConstArrayView<FNiagaraStatelessRuntimeSpawnInfo> SpawnInfos, FNiagaraDataBuffer* DestinationData);
	void SimulateGPU(FRHICommandListBase& RHICmdList, int32 EmitterRandomSeed, float EmitterAge, float DeltaTime, TConstArrayView<FNiagaraStatelessRuntimeSpawnInfo> SpawnInfos, FNiagaraDataBuffer* DestinationData);

private:
	void SimulateInternal(int32 EmitterRandomSeed, float EmitterAge, float DeltaTime, FSpawnInfoShaderParameters& SpawnParameters, uint32 ActiveParticles);
	void CompactDeadParticles();

public:
	uint32 GetNumInstances() const { return NumInstances; }
	uint32 GetParticleComponentOffset(int32 iComponent) const { return BufferStride * iComponent; };

	float GetDeltaTime() const { return DeltaTime; }
	float GetInvDeltaTime() const { return InvDeltaTime; }

	const FVector3f GetLocalToWorldScale() const;
	const FQuat4f& GetToSimulationRotation(ENiagaraCoordinateSpace SourceSpace) const;

	// When modifying particle alive you need to notify the simulation that it needs to compact killed instances or not
	void SetNeedsParticleKillCompaction(bool bNeedsCompaction) const { bNeedsParticleKillCompaction |= bNeedsCompaction; }

	// Get a pointer to required particle components
	int32* GetParticleAlive() const { return static_cast<int32*>(RequiredComponents[int32(EParticleComponent::Alive)]); }
	float* GetParticleLifetime() { return static_cast<float*>(RequiredComponents[int32(EParticleComponent::Lifetime)]); }
	float* GetParticleAge() { return static_cast<float*>(RequiredComponents[int32(EParticleComponent::Age)]); }
	float* GetParticleNormalizedAge() { return static_cast<float*>(RequiredComponents[int32(EParticleComponent::NormalizedAge)]); }
	float* GetParticlePreviousAge() { return static_cast<float*>(RequiredComponents[int32(EParticleComponent::PreviousAge)]); }
	float* GetParticlePreviousNormalizedAge() { return static_cast<float*>(RequiredComponents[int32(EParticleComponent::PreviousNormalizedAge)]); }
	int32* GetParticleUniqueID() { return static_cast<int32*>(RequiredComponents[int32(EParticleComponent::UniqueID)]); }
	float* GetParticleMaterialRandom() { return static_cast<float*>(RequiredComponents[int32(EParticleComponent::MaterialRandom)]); }

	const float* GetParticleLifetime() const { return static_cast<float*>(RequiredComponents[int32(EParticleComponent::Lifetime)]); }
	const float* GetParticleAge() const { return static_cast<float*>(RequiredComponents[int32(EParticleComponent::Age)]); }
	const float* GetParticleNormalizedAge() const { return static_cast<float*>(RequiredComponents[int32(EParticleComponent::NormalizedAge)]); }
	const float* GetParticlePreviousAge() const { return static_cast<float*>(RequiredComponents[int32(EParticleComponent::PreviousAge)]); }
	const float* GetParticlePreviousNormalizedAge() const { return static_cast<float*>(RequiredComponents[int32(EParticleComponent::PreviousNormalizedAge)]); }
	const int32* GetParticleUniqueID() const { return static_cast<int32*>(RequiredComponents[int32(EParticleComponent::UniqueID)]); }

	// Get a pointer to an optional particle component (i.e. one not required in the data set)
	float* GetParticleVariableFloat(int32 iVariable) const { return iVariable >= 0 ? reinterpret_cast<float*>(VariableComponents[iVariable]) : nullptr; }
	int32* GetParticleVariableInt32(int32 iVariable) const { return iVariable >= 0 ? reinterpret_cast<int32*>(VariableComponents[iVariable]) : nullptr; }

	template<typename TType>
	const TType ReadParticleVariableComponent(int32 iVariable, int32 iInstance, int32 iComponent) const { return *reinterpret_cast<const TType*>(VariableComponents[iVariable] + (BufferStride * iComponent) + (iInstance * sizeof(float))); }
	int32 ReadParticleVariable(int32 iVariable, uint32 iInstance, int32 DefaultValue) const { check(iInstance < NumInstances); return iVariable >= 0 ? ReadParticleVariableComponent<int32>(iVariable, iInstance, 0) : DefaultValue; }
	float ReadParticleVariable(int32 iVariable, uint32 iInstance, float DefaultValue) const { check(iInstance < NumInstances); return iVariable >= 0 ? ReadParticleVariableComponent<float>(iVariable, iInstance, 0) : DefaultValue; }
	FVector2f ReadParticleVariable(int32 iVariable, uint32 iInstance, const FVector2f& DefaultValue) const { check(iInstance < NumInstances); return iVariable >= 0 ? FVector2f(ReadParticleVariableComponent<float>(iVariable, iInstance, 0), ReadParticleVariableComponent<float>(iVariable, iInstance, 1)) : DefaultValue; }
	FVector3f ReadParticleVariable(int32 iVariable, uint32 iInstance, const FVector3f& DefaultValue) const { check(iInstance < NumInstances); return iVariable >= 0 ? FVector3f(ReadParticleVariableComponent<float>(iVariable, iInstance, 0), ReadParticleVariableComponent<float>(iVariable, iInstance, 1), ReadParticleVariableComponent<float>(iVariable, iInstance, 2)) : DefaultValue; }
	FVector4f ReadParticleVariable(int32 iVariable, uint32 iInstance, const FVector4f& DefaultValue) const { check(iInstance < NumInstances); return iVariable >= 0 ? FVector4f(ReadParticleVariableComponent<float>(iVariable, iInstance, 0), ReadParticleVariableComponent<float>(iVariable, iInstance, 1), ReadParticleVariableComponent<float>(iVariable, iInstance, 2), ReadParticleVariableComponent<float>(iVariable, iInstance, 3)) : DefaultValue; }
	FQuat4f ReadParticleVariable(int32 iVariable, uint32 iInstance, const FQuat4f& DefaultValue) const { check(iInstance < NumInstances); return iVariable >= 0 ? FQuat4f(ReadParticleVariableComponent<float>(iVariable, iInstance, 0), ReadParticleVariableComponent<float>(iVariable, iInstance, 1), ReadParticleVariableComponent<float>(iVariable, iInstance, 2), ReadParticleVariableComponent<float>(iVariable, iInstance, 3)) : DefaultValue; }
	FLinearColor ReadParticleVariable(int32 iVariable, uint32 iInstance, const FLinearColor& DefaultValue) const { check(iInstance < NumInstances); return iVariable >= 0 ? FLinearColor(ReadParticleVariableComponent<float>(iVariable, iInstance, 0), ReadParticleVariableComponent<float>(iVariable, iInstance, 1), ReadParticleVariableComponent<float>(iVariable, iInstance, 2), ReadParticleVariableComponent<float>(iVariable, iInstance, 3)) : DefaultValue; }

	void WriteParticleVariableComponent(int32 iVariable, int32 iInstance, int32 iComponent, float Value) const { *reinterpret_cast<float*>(VariableComponents[iVariable] + (BufferStride * iComponent) + (iInstance * sizeof(float))) = Value; }
	void WriteParticleVariableComponent(int32 iVariable, int32 iInstance, int32 iComponent, int32 Value) const { *reinterpret_cast<int32*>(VariableComponents[iVariable] + (BufferStride * iComponent) + (iInstance * sizeof(float))) = Value; }
	void WriteParticleVariable(int32 iVariable, uint32 iInstance, int32 Value) const { check(iInstance < NumInstances); if (iVariable >= 0 ) { WriteParticleVariableComponent(iVariable, iInstance, 0, Value); } }
	void WriteParticleVariable(int32 iVariable, uint32 iInstance, float Value) const { check(iInstance < NumInstances); if (iVariable >= 0 ) { WriteParticleVariableComponent(iVariable, iInstance, 0, Value); } }
	void WriteParticleVariable(int32 iVariable, uint32 iInstance, const FVector2f& Value) const { check(iInstance < NumInstances); if (iVariable >= 0 ) { WriteParticleVariableComponent(iVariable, iInstance, 0, Value.X); WriteParticleVariableComponent(iVariable, iInstance, 1, Value.Y); } }
	void WriteParticleVariable(int32 iVariable, uint32 iInstance, const FVector3f& Value) const { check(iInstance < NumInstances); if (iVariable >= 0 ) { WriteParticleVariableComponent(iVariable, iInstance, 0, Value.X); WriteParticleVariableComponent(iVariable, iInstance, 1, Value.Y); WriteParticleVariableComponent(iVariable, iInstance, 2, Value.Z); } }
	void WriteParticleVariable(int32 iVariable, uint32 iInstance, const FVector4f& Value) const { check(iInstance < NumInstances); if (iVariable >= 0) { WriteParticleVariableComponent(iVariable, iInstance, 0, Value.X); WriteParticleVariableComponent(iVariable, iInstance, 1, Value.Y); WriteParticleVariableComponent(iVariable, iInstance, 2, Value.Z); WriteParticleVariableComponent(iVariable, iInstance, 3, Value.W); } }
	void WriteParticleVariable(int32 iVariable, uint32 iInstance, const FQuat4f& Value) const { check(iInstance < NumInstances); if (iVariable >= 0) { WriteParticleVariableComponent(iVariable, iInstance, 0, Value.X); WriteParticleVariableComponent(iVariable, iInstance, 1, Value.Y); WriteParticleVariableComponent(iVariable, iInstance, 2, Value.Z); WriteParticleVariableComponent(iVariable, iInstance, 3, Value.W); } }
	void WriteParticleVariable(int32 iVariable, uint32 iInstance, const FLinearColor& Value) const { check(iInstance < NumInstances); if (iVariable >= 0 ) { WriteParticleVariableComponent(iVariable, iInstance, 0, Value.R); WriteParticleVariableComponent(iVariable, iInstance, 1, Value.G); WriteParticleVariableComponent(iVariable, iInstance, 2, Value.B); WriteParticleVariableComponent(iVariable, iInstance, 3, Value.A); } }

	template<typename TType>
	TType GetParameterBufferValue(int32 Offset, int32 Element) const
	{
		TType Value;
		FMemory::Memcpy(&Value, &DynamicBufferData[(Offset * 4) + (Element * sizeof(TType))], sizeof(TType));
		return Value;
	}

	int32 GetParameterBufferInt(int32 Offset, int32 Element) const { return GetParameterBufferValue<int32>(Offset, Element); }

	template<typename TType>
	TType GetParameterBufferFloat(int32 Offset, int32 Element) const { return GetParameterBufferValue<TType>(Offset, Element); }
	template<typename TType>
	TType GetParameterBufferFloatDefaulted(int32 Offset, const TType& DefaultValue) const { return Offset >= 0 ? GetParameterBufferValue<TType>(Offset, 0) : DefaultValue; }

	template<typename TType>
	TType GetStaticFloat(uint32 Offset, uint32 Element) const;
	template<> float GetStaticFloat<float>(uint32 Offset, uint32 Element) const { return StaticFloatData[Offset + Element]; }
	template<> FVector2f GetStaticFloat<FVector2f>(uint32 Offset, uint32 Element) const { Offset += Element * 2; return FVector2f(StaticFloatData[Offset], StaticFloatData[Offset + 1]); }
	template<> FVector3f GetStaticFloat<FVector3f>(uint32 Offset, uint32 Element) const { Offset += Element * 3; return FVector3f(StaticFloatData[Offset], StaticFloatData[Offset + 1], StaticFloatData[Offset + 2]); }
	template<> FVector4f GetStaticFloat<FVector4f>(uint32 Offset, uint32 Element) const { Offset += Element * 4; return FVector4f(StaticFloatData[Offset], StaticFloatData[Offset + 1], StaticFloatData[Offset + 2], StaticFloatData[Offset + 3]); }
	template<> FLinearColor GetStaticFloat<FLinearColor>(uint32 Offset, uint32 Element) const { Offset += Element * 4; return FLinearColor(StaticFloatData[Offset], StaticFloatData[Offset + 1], StaticFloatData[Offset + 2], StaticFloatData[Offset + 3]); }

	float		Lerp(const float& Lhs, const float& Rhs, const float& U, bool bUniform) const { return FMath::Lerp(Lhs, Rhs, U); }
	FVector2f	Lerp(const FVector2f& Lhs, const FVector2f& Rhs, const FVector2f& U, bool bUniform) const { return FMath::Lerp(Lhs, Rhs, bUniform ? FVector2f(U.X, U.X) : U); }
	FVector3f	Lerp(const FVector3f& Lhs, const FVector3f& Rhs, const FVector3f& U, bool bUniform) const { return FMath::Lerp(Lhs, Rhs, bUniform ? FVector3f(U.X, U.X, U.X) : U); }
	FVector4f	Lerp(const FVector4f& Lhs, const FVector4f& Rhs, const FVector4f& U, bool bUniform) const { return FMath::Lerp(Lhs, Rhs, bUniform ? FVector4f(U.X, U.X, U.X, U.X) : U); }

	template<typename TType>
	TType SampleCurve(const FNiagaraStatelessBuiltDistributionType& Distribution, float Time) const
	{
		const uint32 DataOffset = FNiagaraStatelessBuiltDistribution::GetBufferOffset(Distribution);
		uint32 IndexA, IndexB;
		float Interp;
		FNiagaraStatelessBuiltDistribution::ConvertTimeToLookup(Distribution, Time, IndexA, IndexB, Interp);
		const TType Value0 = GetStaticFloat<TType>(DataOffset, IndexA);
		const TType Value1 = GetStaticFloat<TType>(DataOffset, IndexB);
		return FMath::Lerp(Value0, Value1, Interp);
	}

private:
	template<typename TType>
	void GetDistributionTableValues(FNiagaraStatelessBuiltDistributionTypeIn BuiltDistribution, uint32 IndexA, uint32 IndexB, TType& OutValueA, TType& OutValueB) const
	{
		const uint32 BaseOffset = FNiagaraStatelessBuiltDistribution::GetBufferOffset(BuiltDistribution);
		if (FNiagaraStatelessBuiltDistribution::IsBufferReadModeStatic(BuiltDistribution))
		{
			OutValueA = GetStaticFloat<TType>(BaseOffset, IndexA);
			OutValueB = GetStaticFloat<TType>(BaseOffset, IndexB);
		}
		else
		{
			OutValueA = GetParameterBufferFloat<TType>(BaseOffset, IndexA);
			OutValueB = GetParameterBufferFloat<TType>(BaseOffset, IndexB);
		}
	}

	template<typename TType>
	TType GetDistributionRandomValue(FNiagaraStatelessBuiltDistributionTypeIn BuiltDistribution, uint32 iInstance, uint32 RandomSeedOffset) const
	{
		const float Random = RandomFloat(iInstance, RandomSeedOffset);

		uint32 Indicies[2];
		float Interp;
		FNiagaraStatelessBuiltDistribution::ConvertRandomToLookup(BuiltDistribution, Random, Indicies[0], Indicies[1], Interp);

		TType Values[2];
		GetDistributionTableValues(BuiltDistribution, Indicies[0], Indicies[1], Values[0], Values[1]);

		if (FNiagaraStatelessBuiltDistribution::IsInterpolationModeLinear(BuiltDistribution))
		{
			const bool bUniformRandom = FNiagaraStatelessBuiltDistribution::IsUniformRandomEnabled(BuiltDistribution);
			return Lerp(Values[0], Values[1], TRandomFloat<TType>(iInstance, RandomSeedOffset), bUniformRandom);
		}
		else
		{
			return Values[0];
		}
	}

	template<typename TType>
	TType GetDistributionTimeValue(FNiagaraStatelessBuiltDistributionTypeIn BuiltDistribution, float Time) const
	{
		uint32 Indicies[2];
		float Interp;
		FNiagaraStatelessBuiltDistribution::ConvertTimeToLookup(BuiltDistribution, Time, Indicies[0], Indicies[1], Interp);

		TType Values[2];
		GetDistributionTableValues(BuiltDistribution, Indicies[0], Indicies[1], Values[0], Values[1]);

		return FMath::Lerp(Values[0], Values[1], Interp);
	}

public:
	template<typename TType>
	TType SampleDistributionValue(FNiagaraStatelessBuiltDistributionTypeIn BuiltDistribution, uint32 iInstance, uint32 RandomSeedOffset, float Time) const
	{
		if (FNiagaraStatelessBuiltDistribution::IsLookupValueModeRandom(BuiltDistribution))
		{
			return GetDistributionRandomValue<TType>(BuiltDistribution, iInstance, RandomSeedOffset);
		}
		else
		{
			return GetDistributionTimeValue<TType>(BuiltDistribution, Time);
		}
	}

	template<typename TType>
	void SampleDistributionValues(FNiagaraStatelessBuiltDistributionTypeIn BuiltDistribution, uint32 iInstance, uint32 RandomSeedOffset, float TimeA, float TimeB, TType& OutValueA, TType& OutValueB) const
	{
		if (FNiagaraStatelessBuiltDistribution::IsLookupValueModeRandom(BuiltDistribution))
		{
			OutValueA = GetDistributionRandomValue<TType>(BuiltDistribution, iInstance, RandomSeedOffset);
			OutValueB = OutValueA;
		}
		else
		{
			OutValueA = GetDistributionTimeValue<TType>(BuiltDistribution, TimeA);
			OutValueB = GetDistributionTimeValue<TType>(BuiltDistribution, TimeB);
		}
	}

	template<typename TType>
	TType					TRandomFloat(uint32 iInstance, uint32 RandomSeedOffset) const;
	template<> float		TRandomFloat<float>(uint32 iInstance, uint32 RandomSeedOffset) const { return RandomFloat(iInstance, RandomSeedOffset); }
	template<> FVector2f	TRandomFloat<FVector2f>(uint32 iInstance, uint32 RandomSeedOffset) const { return RandomFloat2(iInstance, RandomSeedOffset); }
	template<> FVector3f	TRandomFloat<FVector3f>(uint32 iInstance, uint32 RandomSeedOffset) const { return RandomFloat3(iInstance, RandomSeedOffset); }
	template<> FVector4f	TRandomFloat<FVector4f>(uint32 iInstance, uint32 RandomSeedOffset) const { return RandomFloat4(iInstance, RandomSeedOffset); }
	template<> FLinearColor	TRandomFloat<FLinearColor>(uint32 iInstance, uint32 RandomSeedOffset) const { return FLinearColor(RandomFloat4(iInstance, RandomSeedOffset)); }

	uint32			RandomUInt(uint32 iInstance, uint32 RandomSeedOffset) const;
	FUintVector2	RandomUInt2(uint32 iInstance, uint32 RandomSeedOffset) const;
	FUintVector3	RandomUInt3(uint32 iInstance, uint32 RandomSeedOffset) const;
	FUintVector4	RandomUInt4(uint32 iInstance, uint32 RandomSeedOffset) const;

	float			RandomFloat(uint32 iInstance, uint32 RandomSeedOffset) const;
	FVector2f		RandomFloat2(uint32 iInstance, uint32 RandomSeedOffset) const;
	FVector3f		RandomFloat3(uint32 iInstance, uint32 RandomSeedOffset) const;
	FVector4f		RandomFloat4(uint32 iInstance, uint32 RandomSeedOffset) const;

	float			RandomScaleBiasFloat(uint32 iInstance, uint32 RandomSeedOffset, const float Scale, const float Bias) const { return Bias + (RandomFloat(iInstance, RandomSeedOffset) * Scale); }
	FVector2f		RandomScaleBiasFloat(uint32 iInstance, uint32 RandomSeedOffset, const FVector2f& Scale, const FVector2f& Bias) const { return Bias + (RandomFloat2(iInstance, RandomSeedOffset) * Scale); }
	FVector3f		RandomScaleBiasFloat(uint32 iInstance, uint32 RandomSeedOffset, const FVector3f& Scale, const FVector3f& Bias) const { return Bias + (RandomFloat3(iInstance, RandomSeedOffset) * Scale); }
	FVector4f		RandomScaleBiasFloat(uint32 iInstance, uint32 RandomSeedOffset, const FVector4f& Scale, const FVector4f& Bias) const { return Bias + (RandomFloat4(iInstance, RandomSeedOffset) * Scale); }
	FLinearColor	RandomScaleBiasFloat(uint32 iInstance, uint32 RandomSeedOffset, const FLinearColor& Scale, const FLinearColor& Bias) const { return Bias + (RandomFloat4(iInstance, RandomSeedOffset) * Scale); }

	float			RandomScaleBiasFloat(uint32 iInstance, uint32 RandomSeedOffset, const float Scale, const float Bias, bool bUniform) const { return RandomScaleBiasFloat(iInstance, RandomSeedOffset, Scale, Bias); }
	FVector2f		RandomScaleBiasFloat(uint32 iInstance, uint32 RandomSeedOffset, const FVector2f& Scale, const FVector2f& Bias, bool bUniform) const { const FVector2f Random = RandomScaleBiasFloat(iInstance, RandomSeedOffset, Scale, Bias); return bUniform ? FVector2f(Random.X, Random.X) : Random; }
	FVector3f		RandomScaleBiasFloat(uint32 iInstance, uint32 RandomSeedOffset, const FVector3f& Scale, const FVector3f& Bias, bool bUniform) const { const FVector3f Random = RandomScaleBiasFloat(iInstance, RandomSeedOffset, Scale, Bias); return bUniform ? FVector3f(Random.X, Random.X, Random.X) : Random; }
	FVector4f		RandomScaleBiasFloat(uint32 iInstance, uint32 RandomSeedOffset, const FVector4f& Scale, const FVector4f& Bias, bool bUniform) const { const FVector4f Random = RandomScaleBiasFloat(iInstance, RandomSeedOffset, Scale, Bias); return bUniform ? FVector4f(Random.X, Random.X, Random.X) : Random; }
	FLinearColor	RandomScaleBiasFloat(uint32 iInstance, uint32 RandomSeedOffset, const FLinearColor& Scale, const FLinearColor& Bias, bool bUniform) const { const FLinearColor Random = RandomScaleBiasFloat(iInstance, RandomSeedOffset, Scale, Bias); return bUniform ? FLinearColor(Random.R, Random.R, Random.R, Random.R) : Random; }

	FVector2f		RandomUnitFloat2(uint32 iInstance, uint32 RandomSeedOffset) const { return SafeNormalize(RandomFloat2(iInstance, RandomSeedOffset) - 0.5f); }
	FVector3f		RandomUnitFloat3(uint32 iInstance, uint32 RandomSeedOffset) const { return SafeNormalize(RandomFloat3(iInstance, RandomSeedOffset) - 0.5f); }

	static FVector2f	SafeNormalize(const FVector2f& v, const FVector2f& Fallback) { const float l2 = v.SquaredLength(); return l2 < UE_KINDA_SMALL_NUMBER ? Fallback : v * (1.0f / FMath::Sqrt(l2)); }
	static FVector3f	SafeNormalize(const FVector3f& v, const FVector3f& Fallback) { const float l2 = v.SquaredLength(); return l2 < UE_KINDA_SMALL_NUMBER ? Fallback : v * (1.0f / FMath::Sqrt(l2)); }
	static FVector2f	SafeNormalize(const FVector2f& v) { return SafeNormalize(v, FVector2f(1.0f, 0.0f)); }
	static FVector3f	SafeNormalize(const FVector3f& v) { return SafeNormalize(v, FVector3f(1.0f, 0.0f, 0.0f)); }

	static float		SafeRcpSqrt(const float v) { return v > UE_SMALL_NUMBER ? FMath::InvSqrt(v) : 0.0f; }
	static FVector3f	SafeRcpSqrt(const FVector3f& v) { return FVector3f(SafeRcpSqrt(v.X), SafeRcpSqrt(v.Y), SafeRcpSqrt(v.Z)); }

	void			ConvertRangeToScaleBias(const FNiagaraStatelessRangeInt&     Range, int32& OutScale, int32& OutBias) const { OutScale = Range.ParameterOffset == INDEX_NONE ? Range.GetScale() : 0; OutBias = Range.ParameterOffset == INDEX_NONE ? Range.Min : GetParameterBufferValue<int32>(Range.ParameterOffset, 0); }
	void			ConvertRangeToScaleBias(const FNiagaraStatelessRangeFloat&   Range, float& OutScale, float& OutBias) const { OutScale = Range.ParameterOffset == INDEX_NONE ? Range.GetScale() : 0.0f; OutBias = Range.ParameterOffset == INDEX_NONE ? Range.Min : GetParameterBufferValue<float>(Range.ParameterOffset, 0); }
	void			ConvertRangeToScaleBias(const FNiagaraStatelessRangeVector2& Range, FVector2f& OutScale, FVector2f& OutBias) const { OutScale = Range.ParameterOffset == INDEX_NONE ? Range.GetScale() : FVector2f::ZeroVector; OutBias = Range.ParameterOffset == INDEX_NONE ? Range.Min : GetParameterBufferValue<FVector2f>(Range.ParameterOffset, 0); }
	void			ConvertRangeToScaleBias(const FNiagaraStatelessRangeVector3& Range, FVector3f& OutScale, FVector3f& OutBias) const { OutScale = Range.ParameterOffset == INDEX_NONE ? Range.GetScale() : FVector3f::ZeroVector; OutBias = Range.ParameterOffset == INDEX_NONE ? Range.Min : GetParameterBufferValue<FVector3f>(Range.ParameterOffset, 0); }
	void			ConvertRangeToScaleBias(const FNiagaraStatelessRangeVector4& Range, FVector4f& OutScale, FVector4f& OutBias) const { OutScale = Range.ParameterOffset == INDEX_NONE ? Range.GetScale() : FVector4f::Zero(); OutBias = Range.ParameterOffset == INDEX_NONE ? Range.Min : GetParameterBufferValue<FVector4f>(Range.ParameterOffset, 0); }
	void			ConvertRangeToScaleBias(const FNiagaraStatelessRangeColor&   Range, FLinearColor& OutScale, FLinearColor& OutBias) const { OutScale = Range.ParameterOffset == INDEX_NONE ? Range.GetScale() : FLinearColor(0.0f, 0.0f, 0.0f, 0.0f); OutBias = Range.ParameterOffset == INDEX_NONE ? Range.Min : GetParameterBufferValue<FLinearColor>(Range.ParameterOffset, 0); }

	template<typename T>
	const T* ReadBuiltData() const
	{
		const int32 Offset = Align(BuiltDataOffset, alignof(T));
		BuiltDataOffset = Offset + sizeof(T);
		check(BuiltDataOffset <= BuiltData.Num());
		return reinterpret_cast<const T*>(BuiltData.GetData() + Offset);
	}

	template<typename T>
	const T* ReadParameterNestedStruct() const
	{
		const uint32 Offset = Align(ShaderParameterOffset, TShaderParameterStructTypeInfo<T>::Alignment);
		ShaderParameterOffset = Offset + TShaderParameterStructTypeInfo<T>::GetStructMetadata()->GetSize();
		return reinterpret_cast<const T*>(ShaderParametersData + Offset);
	}

	template<typename T>
	const T* ReadSharedBuiltData() const
	{
		const FName DataName = T::GetName();
		for (const TPair<FName, uint32>& Pair : SharedBuiltDataOffsets)
		{
			if (Pair.Key == DataName)
			{
				return reinterpret_cast<const T*>(SharedBuiltData.GetData() + Pair.Value);
			}
		}
		return nullptr;
	}

	static FQuat4f RotatorToQuat(FVector3f Rotator)
	{
		Rotator.X = FMath::Fractional(Rotator.X) * UE_PI;
		Rotator.Y = FMath::Fractional(Rotator.Y) * UE_PI;
		Rotator.Z = FMath::Fractional(Rotator.Z) * UE_PI;

		float SP, CP, SY, CY, SR, CR;
		FMath::SinCos(&SR, &CR, Rotator.X);
		FMath::SinCos(&SP, &CP, Rotator.Y);
		FMath::SinCos(&SY, &CY, Rotator.Z);

		return FQuat4f(
			 CR * SP * SY - SR * CP * CY,
			-CR * SP * CY - SR * CP * SY,
			 CR * CP * SY - SR * SP * CY,
			 CR * CP * CY + SR * SP * SY
		);
	}

private:
	const FNiagaraStatelessEmitterData*		EmitterData = nullptr;
	uint32									NumInstances = 0;
	float									DeltaTime = 0.0f;
	float									InvDeltaTime = 0.0f;

	uint32									EmitterRandomSeed = 0;
	uint32									ModuleRandomSeed = 0;

	uint32									BufferStride = 0;
	uint8*									BufferFloatData = nullptr;
	uint8*									BufferInt32Data = nullptr;

	mutable bool							bNeedsParticleKillCompaction = false;

	void*									RequiredComponents[int32(EParticleComponent::Num)] = {};
	TArray<uint8*, TInlineAllocator<16>>	VariableComponents;

	const TConstArrayView<uint8>			BuiltData;
	mutable int32							BuiltDataOffset = 0;
	TConstArrayView<uint8>					SharedBuiltData;
	TConstArrayView<TPair<FName, uint32>>	SharedBuiltDataOffsets;
	const uint8*							ShaderParametersData = nullptr;
	mutable int32							ShaderParameterOffset = 0;
	const TConstArrayView<float>			StaticFloatData;
	const TConstArrayView<uint8>			DynamicBufferData;
};

} //namespace NiagaraStateless
