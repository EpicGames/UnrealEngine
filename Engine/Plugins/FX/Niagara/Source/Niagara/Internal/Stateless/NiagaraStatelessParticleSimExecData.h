// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraStatelessCommon.h"

struct FNiagaraDataSetCompiledData;

namespace NiagaraStateless
{
class FParticleSimulationContext;

class FParticleSimulationExecData
{
public:
	struct FCallback
	{
		FCallback() = default;
		explicit FCallback(TFunction<void(const FParticleSimulationContext&)> InFunc, int32 InBuiltDataOffset, int32 InShaderParameterOffset, uint32 InRandomSeedOffset)
			: Function(MoveTemp(InFunc))
			, BuiltDataOffset(InBuiltDataOffset)
			, ShaderParameterOffset(InShaderParameterOffset)
			, RandomSeedOffset(InRandomSeedOffset)
		{
		}

		TFunction<void(const FParticleSimulationContext&)>	Function;
		int32	BuiltDataOffset = 0;
		int32	ShaderParameterOffset = 0;
		uint32	RandomSeedOffset = 0;
	};

	struct FVariableOffset
	{
		const bool IsFloat() const { return Type == 0; }
		const bool IsInt32() const { return Type == 1; }
		const uint32 GetOffset() const { return Offset; }
		const uint32 GetNum() const { return Num; }

		uint16	Type	: 1	 = 0;
		uint16	Offset	: 15 = 0;
		uint16	Num = 0;
	};

	struct FRequiredComponentOffset
	{
		const bool IsTransient() const { return bTransient != 0; }
		const uint32 GetOffset() const { return Offset; }
	
		uint16	bTransient : 1 = 0;
		uint16	Offset : 15 = 0;
	};

	explicit FParticleSimulationExecData(const FNiagaraDataSetCompiledData& ParticleDataSetCompiledData);

	TArray<FVariableOffset>				VariableComponentOffsets;		// Stored offsets per variable
	TArray<FRequiredComponentOffset>	RequiredComponentOffsets;		// Stored offsets per required variable, these may be output or transient
	int32								RequiredComponentByteSize = 0;	// Required byte size for all required components that are transient
	TArray<FCallback>					SimulateFunctions;				// Series of functions to simulate particles
};

} //namespace NiagaraStateless
