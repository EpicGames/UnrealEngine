// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessCommon.h"
#include "Stateless/NiagaraStatelessRange.h"
#include "ShaderParameterStruct.h"

class FNiagaraStatelessSetShaderParameterContext
{
public:
	UE_NONCOPYABLE(FNiagaraStatelessSetShaderParameterContext);

	explicit FNiagaraStatelessSetShaderParameterContext(const FNiagaraStatelessSpaceTransforms& InSpaceTransforms, TConstArrayView<uint8> InRendererParameterData, TConstArrayView<uint8> InBuiltData, TConstArrayView<uint8> InSharedBuiltData, TConstArrayView<TPair<FName, uint32>> InSharedBuiltDataOffsets, const FShaderParametersMetadata* InShaderParametersMetadata, uint8* InShaderParameters);

	const FNiagaraStatelessSpaceTransforms& GetSpaceTransforms() const { return SpaceTransforms; }

	template<typename T>
	T* GetParameterNestedStruct() const
	{
		const uint32 StructOffset = Align(ParameterOffset, TShaderParameterStructTypeInfo<T>::Alignment);
#if DO_CHECK
		ValidateIncludeStructType(StructOffset, TShaderParameterStructTypeInfo<T>::GetStructMetadata());
#endif
		ParameterOffset = StructOffset + TShaderParameterStructTypeInfo<T>::GetStructMetadata()->GetSize();
		return reinterpret_cast<T*>(ShaderParametersBase + StructOffset);
	}

	template<typename T>
	const T* ReadBuiltData() const
	{
		const int32 Offset = Align(BuiltDataOffset, alignof(T));
		BuiltDataOffset = Offset + sizeof(T);
		check(BuiltDataOffset <= BuiltData.Num());
		return reinterpret_cast<const T*>(BuiltData.GetData() + Offset);
	}

	template<typename T>
	const T* ReadSharedBuiltData() const
	{
		const FName DataName = T::GetName();
		for (const TPair<FName, uint32>& Pair : SharedBuiltDataOffsets)
		{
			if (Pair.Key == DataName)
			{
				return reinterpret_cast<const T*>(SharedBuiltData.GetData() + Pair.Value);
			}
		}
		return nullptr;
	}

	template<typename TValueType>
	TValueType GetRendererParameterValue(int32 Offset, const TValueType& DefaultValue) const
	{
		if (Offset != INDEX_NONE)
		{
			TValueType Value;
			Offset *= sizeof(uint32);
			check(Offset >= 0 && Offset + sizeof(TValueType) <= RendererParameterData.Num());
			FMemory::Memcpy(&Value, RendererParameterData.GetData() + Offset, sizeof(TValueType));
			return Value;
		}
		return DefaultValue;
	}

	template<typename TValueType>
	void ConvertRangeToScaleBias(const FNiagaraStatelessRange<TValueType>& Range, TValueType& OutScale, TValueType& OutBias) const
	{
		OutBias = GetRendererParameterValue(Range.ParameterOffset, Range.Min);
		OutScale = Range.ParameterOffset == INDEX_NONE ? Range.GetScale() : FNiagaraStatelessRangeDefaultValue<TValueType>::Zero();
	}

	template<typename TValueType>
	TValueType ConvertRangeToValue(const FNiagaraStatelessRange<TValueType>& Range) const
	{
		return GetRendererParameterValue(Range.ParameterOffset, Range.Min);
	}

	void TransformVectorRangeToScaleBias(const FNiagaraStatelessRangeVector3& Range, ENiagaraCoordinateSpace SourceSpace, FVector3f& OutScale, FVector3f& OutBias) const;
	void TransformVectorNoScaleRangeToScaleBias(const FNiagaraStatelessRangeVector3& Range, ENiagaraCoordinateSpace SourceSpace, FVector3f& OutScale, FVector3f& OutBias) const;
	void TransformPositionRangeToScaleBias(const FNiagaraStatelessRangeVector3& Range, ENiagaraCoordinateSpace SourceSpace, FVector3f& OutScale, FVector3f& OutBias) const;
	void TransformRotationRangeToScaleBias(const FNiagaraStatelessRangeVector3& Range, ENiagaraCoordinateSpace SourceSpace, FVector3f& OutScale, FVector3f& OutBias) const;
	FVector3f TransformPositionRangeToValue(const FNiagaraStatelessRangeVector3& Range, ENiagaraCoordinateSpace SourceSpace) const;

protected:
#if DO_CHECK
	NIAGARA_API void ValidateIncludeStructType(uint32 StructOffset, const FShaderParametersMetadata* StructMetaData) const;
#endif

private:
	const FNiagaraStatelessSpaceTransforms&	SpaceTransforms;
	TConstArrayView<uint8>					RendererParameterData;
	TConstArrayView<uint8>					BuiltData;
	mutable int32							BuiltDataOffset = 0;
	TConstArrayView<uint8>					SharedBuiltData;
	TConstArrayView<TPair<FName, uint32>>	SharedBuiltDataOffsets;
	uint8*									ShaderParametersBase = nullptr;
	mutable uint32							ParameterOffset = 0;
	const FShaderParametersMetadata*		ShaderParametersMetadata = nullptr;
};
