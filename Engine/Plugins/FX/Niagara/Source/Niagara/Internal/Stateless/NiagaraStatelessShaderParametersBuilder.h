// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraStatelessCommon.h"
#include "ShaderParameterMetadataBuilder.h"

class FNiagaraStatelessShaderParametersBuilder
{
public:
	explicit FNiagaraStatelessShaderParametersBuilder(FShaderParametersMetadataBuilder* OptionalMetadataBuilder = nullptr)
		: MetadataBuilder(OptionalMetadataBuilder)
	{
	}

	// Find the current size of the shader parameter structure
	int32 GetParametersStructSize() const
	{
		return ParameterOffset;
	}

	// Adds a shader parameters structure that is scoped to the data interface
	// i.e. if the structured contained "MyFloat" the shader variable would be "UniqueDataInterfaceName_MyFloat"
	template<typename T> void AddParameterNestedStruct()
	{
		using TParamTypeInfo = TShaderParameterStructTypeInfo<T>;

		const int32 StructOffset = Align(ParameterOffset, TParamTypeInfo::Alignment);
		if (MetadataBuilder)
		{
			MetadataBuilder->AddIncludedStruct(TParamTypeInfo::GetStructMetadata());
		}
		ParameterOffset = StructOffset + TParamTypeInfo::GetStructMetadata()->GetSize();
	}

private:
	FShaderParametersMetadataBuilder*	MetadataBuilder = nullptr;
	int32								ParameterOffset = 0;
};
