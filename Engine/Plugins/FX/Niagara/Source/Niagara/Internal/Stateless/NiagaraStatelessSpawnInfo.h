// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessCommon.h"
#include "Stateless/NiagaraStatelessDistribution.h"

#include "NiagaraStatelessSpawnInfo.generated.h"

UENUM()
enum class ENiagaraStatelessSpawnInfoType
{
	Burst,
	Rate
};

USTRUCT()
struct FNiagaraStatelessSpawnInfo
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid SourceId;
#endif

	UPROPERTY(EditAnywhere, Category = "Spawn", meta = (ShowInStackItemHeader, StackItemHeaderAlignment = "Left"))
	ENiagaraStatelessSpawnInfoType	Type = ENiagaraStatelessSpawnInfoType::Burst;

	// Time to spawn particles at
	UPROPERTY(EditAnywhere, Category = "Spawn", meta = (EditConditionHides, EditCondition = "Type == ENiagaraStatelessSpawnInfoType::Burst"))
	float SpawnTime = 0.0f;

	// Number of particles to spawn at the provided time
	UPROPERTY(EditAnywhere, Category = "Spawn", meta = (EditConditionHides, EditCondition = "Type == ENiagaraStatelessSpawnInfoType::Burst", ClampMin = "0", DisplayName = "Spawn Count"))
	FNiagaraDistributionRangeInt Amount = FNiagaraDistributionRangeInt(1);

	// Number of particles to spawn per second
	// Note: In the case of a random range or binding the value is only evaluated at the start of each loop.  This varies from regular emitters.
	UPROPERTY(EditAnywhere, Category = "Spawn", meta = (EditConditionHides, EditCondition = "Type == ENiagaraStatelessSpawnInfoType::Rate", ClampMin = "0.0", DisplayName = "Spawn Rate"))
	FNiagaraDistributionRangeFloat Rate = FNiagaraDistributionRangeFloat(60.0f);

	UPROPERTY(EditAnywhere, Category = "Spawn", meta = (InlineEditConditionToggle))
	uint8 bEnabled : 1 = true;

	UPROPERTY(EditAnywhere, Category = "Spawn", meta = (InlineEditConditionToggle))
	uint8 bSpawnProbabilityEnabled : 1 = false;

	UPROPERTY(EditAnywhere, Category = "Spawn", meta = (InlineEditConditionToggle))
	uint8 bLoopCountLimitEnabled : 1 = false;

	// 0 - 1 value that can be viewed as a percentage chance that the spawn will generated particles or not.
	// A value of 0.5 can be viewed as a 50% chance that the spawn will trigger.
	// Note: Rate spawning will only evaluate the probability at the start of each loop.  This varies from regular emitters which evaluate each time particles are spawned.
	UPROPERTY(EditAnywhere, Category = "Spawn", meta = (EditCondition = "bSpawnProbabilityEnabled", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FNiagaraDistributionRangeFloat SpawnProbability = FNiagaraDistributionRangeFloat(1.0f);


	// For Emitters that loop multiple times (or infinitely), this can constrain the bursting to only happen within this loop count limit.
	// For example, an emitter might loop 5 times, but bursting should only happen on the first two loops.  Setting Loop Count Limit to 2 will allow the burst to fire twice, once after each of the first two loops, and then stop.
	//-TODO:EditConditionHides, EditCondition = "Type == ENiagaraStatelessSpawnInfoType::Burst",
	UPROPERTY(EditAnywhere, Category = "Spawn", meta = (EditCondition = "bLoopCountLimitEnabled", ClampMin = "1", UIMin = "0"))
	FNiagaraDistributionRangeInt LoopCountLimit = FNiagaraDistributionRangeInt(1);

	bool IsValid(TOptional<float> LoopDuration) const;
};

struct FNiagaraStatelessRuntimeSpawnInfo
{
	ENiagaraStatelessSpawnInfoType Type = ENiagaraStatelessSpawnInfoType::Burst;
	uint32	UniqueOffset	= 0;
	float	SpawnTimeStart	= 0.0f;
	float	SpawnTimeEnd	= 0.0f;
	float	Rate			= 0.0f;
	float	Probability		= 1.0f;
	int32	Amount			= 0;		// Note: When a burst this is the absolute amount burst, when rate this is the amount over the spawn duration
	float	LifetimeMin		= 0.0f;
	float	LifetimeMax		= 0.0f;
};
