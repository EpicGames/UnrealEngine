// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class Niagara : ModuleRules
{
    public Niagara(ReadOnlyTargetRules Target) : base(Target)
	{
		NumIncludedBytesPerUnityCPPOverride = 491520; // best unity size found from using UBT ProfileUnitySizes mode
		
		PrivateDependencyModuleNames.AddRange(
            new string[] {
                "ApplicationCore",
                "AudioPlatformConfiguration",
                "Core",
                "ImageCore",
                "DeveloperSettings",
                "Engine",
                "Json",
                "JsonUtilities",
                "Landscape",
                "Projects",
                "Renderer",
                "SignalProcessing",
                "TimeManagement",
                "TraceLog",
            }
        );


        PublicDependencyModuleNames.AddRange(
            new string[] {
                "CoreUObject",
                "IntelISPC",
                "MovieScene",
                "MovieSceneTracks",
                "NiagaraCore",
                "NiagaraShader",
                "NiagaraVertexFactories",
                "PhysicsCore",
                "RenderCore",
                "RHI",
		"SlateCore",
                "VectorVM",
		"GameplayTags"
            }
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"DerivedDataCache",
				"Shaders"
			});

        if (Target.bBuildEditor == true)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[] {
                "TargetPlatform",
                "DerivedDataCache",
				"EditorFramework",
                "UnrealEd",
				"SlateCore",
				"Slate"
            });
        }
	}
}
