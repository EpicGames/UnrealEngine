<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!-- 
  *
  * Niagara Visualizers 
  *
  -->

  <!-- FNiagaraVariable visualizer -->
  <Type Name="FNiagaraVariableBase">
    <DisplayString>Name={Name}, Type={TypeDefHandle}</DisplayString>
  </Type>

  <Type Name="UNiagaraNodeFunctionCall">
    <DisplayString>Name={NamePrivate}, FunctionName={FunctionDisplayName}</DisplayString>
  </Type>

  <Type Name="FNiagaraTypeDefinitionHandle">
    <DisplayString>{((FNiagaraTypeDefinition*)(NiagaraDebugVisHelper::GTypeRegistrySingletonPtr->RegisteredTypes.AllocatorInstance.InlineData))[RegisteredTypeIndex]}</DisplayString>
  </Type>

  <Type Name="FNiagaraTypeDefinition">
    <DisplayString>({ClassStructOrEnum->NamePrivate}, Flags={Flags})</DisplayString>
  </Type>
  
</AutoVisualizer>
