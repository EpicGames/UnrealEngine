// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NDIRenderTargetSimCacheData.h"
#include "NiagaraCommon.h"

#include "Dom/JsonObject.h"
#include "Misc/Compression.h"
#include "Misc/Paths.h"
#include "ImageCoreUtils.h"
#include "ImageUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NDIRenderTargetSimCacheData)

namespace NDIRenderTargetSimCacheDataPrivate
{
	bool CanConvertPixelFormat(EPixelFormat Format)
	{
		switch (Format)
		{
			case PF_FloatRGBA:		return true;
			case PF_G16R16F:
			case PF_G16R16F_FILTER:	return true;
			default:				return false;
		}
	}

	FLinearColor ConvertPixel(const uint8* SourceData, EPixelFormat Format)
	{
		FLinearColor Color;
		switch (Format)
		{
			case PF_FloatRGBA:
				Color.R = *reinterpret_cast<const FFloat16*>(SourceData);
				Color.G = *reinterpret_cast<const FFloat16*>(SourceData + 2);
				Color.B = *reinterpret_cast<const FFloat16*>(SourceData + 4);
				Color.A = *reinterpret_cast<const FFloat16*>(SourceData + 6);
				break;

			case PF_G16R16F:
			case PF_G16R16F_FILTER:
				Color.R = *reinterpret_cast<const FFloat16*>(SourceData);
				Color.G = *reinterpret_cast<const FFloat16*>(SourceData + 2);
				Color.B = 0.0f;
				Color.A = 1.0f;
				break;

			default: // do nothing;
				Color.R = 0.0f;
				Color.G = 0.0f;
				Color.B = 0.0f;
				Color.A = 1.0f;
				break;
		}
		return Color;
	}
}


void UNDIRenderTargetSimCacheData::BeginDestroy()
{
	Super::BeginDestroy();

	ReleaseAllPixelData();
	Frames.Empty();
}

void UNDIRenderTargetSimCacheData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	for (FNDIRenderTargetSimCacheFrame& Frame : Frames)
	{
		if (Frame.UncompressedSize == 0)
		{
			continue;
		}

		Frame.BulkData.Serialize(Ar, this);
	}
}

bool UNDIRenderTargetSimCacheData::CompareFrame(const UNDIRenderTargetSimCacheData* Other, int32 FrameIndex, TOptional<float> InTolerance, FString& OutErrors) const
{
	if (HasPixelData(FrameIndex) != Other->HasPixelData(FrameIndex))
	{
		OutErrors = TEXT("HasPixelData mismatched between caches");
		return false;
	}

	const FIntVector LhsTextureSize = GetTextureSize(FrameIndex);
	const FIntVector RhsTextureSize = Other->GetTextureSize(FrameIndex);
	if (LhsTextureSize != RhsTextureSize)
	{
		OutErrors = FString::Printf(
			TEXT("TextureSizess (%dx%dx%d & %dx%dx%d) mismatched between caches"),
			LhsTextureSize.X, LhsTextureSize.Y, LhsTextureSize.Z,
			RhsTextureSize.X, RhsTextureSize.Y, RhsTextureSize.Z
		);
		return false;
	}

	const EPixelFormat LhsPixelFormat = GetTextureFormat(FrameIndex);
	const EPixelFormat RhsPixelFormat = Other->GetTextureFormat(FrameIndex);
	if (LhsPixelFormat != RhsPixelFormat)
	{
		OutErrors = FString::Printf(
			TEXT("PixelFormats (%s & %s) mismatched between caches"),
			GPixelFormats[LhsPixelFormat].Name,
			GPixelFormats[RhsPixelFormat].Name
		);
		return false;
	}

	if (!NDIRenderTargetSimCacheDataPrivate::CanConvertPixelFormat(LhsPixelFormat))
	{
		OutErrors = FString::Printf(TEXT("PixelFormat (%s) is not supported for comparison"), GPixelFormats[LhsPixelFormat].Name);
		return false;
	}

	const TArray<uint8> LhsPixelData = GetPixelData(FrameIndex);
	const TArray<uint8> RhsPixelData = Other->GetPixelData(FrameIndex);
	if (LhsPixelData.Num() != RhsPixelData.Num())
	{
		OutErrors = TEXT("Arraysize mismatch between caches, but texture size matches?");
		return false;
	}

	const float Tolerance = InTolerance.Get(1.e-3f);

	const uint32 BlockBytes = GPixelFormats[LhsPixelFormat].BlockBytes;
	const uint32 PixelCount = LhsPixelData.Num() / BlockBytes;

	for ( uint32 i=0; i < PixelCount; ++i)
	{
		const uint32 Offset = i * BlockBytes;
		const FLinearColor LhsPixel = NDIRenderTargetSimCacheDataPrivate::ConvertPixel(LhsPixelData.GetData() + Offset, LhsPixelFormat);
		const FLinearColor RhsPixel = NDIRenderTargetSimCacheDataPrivate::ConvertPixel(RhsPixelData.GetData() + Offset, RhsPixelFormat);
		if (!LhsPixel.Equals(RhsPixel, Tolerance))
		{
			const int32 PixelX = i % LhsTextureSize.X;
			const int32 PixelY = (i / LhsTextureSize.X) % LhsTextureSize.Y;
			const int32 PixelZ = i / (LhsTextureSize.X * LhsTextureSize.Y);
			
			OutErrors = FString::Printf(
				TEXT("Pixel (%dx%dx%d) is different (R:%g G:%g B:%g A:%g) vs (R:%g G:%g B:%g A:%g)"),
				PixelX, PixelY, PixelZ,
				LhsPixel.R, LhsPixel.G, LhsPixel.B, LhsPixel.A,
				RhsPixel.R, RhsPixel.G, RhsPixel.B, RhsPixel.A
			);
			return false;
		}
	}

	return true;
}

TSharedPtr<FJsonObject> UNDIRenderTargetSimCacheData::FrameToJson(int FrameIndex, TOptional<FString> TargetFolder, TOptional<FString> FilenamePrefix) const
{
	if (!HasPixelData(FrameIndex))
	{
		return TSharedPtr<FJsonObject>();
	}

	const FNDIRenderTargetSimCacheFrame& CacheFrame = Frames[FrameIndex];

	TSharedPtr<FJsonObject> JsonCacheObject = MakeShared<FJsonObject>();
	JsonCacheObject->SetStringField(TEXT("CompressionType"), CompressionType.ToString());
	JsonCacheObject->SetStringField(TEXT("PixelFormat"), GPixelFormats[CacheFrame.Format].Name);
	JsonCacheObject->SetNumberField(TEXT("CompressedSize"), CacheFrame.CompressedSize);
	JsonCacheObject->SetNumberField(TEXT("UncompressedSize"), CacheFrame.UncompressedSize);
	JsonCacheObject->SetNumberField(TEXT("SizeX"), CacheFrame.Size.X);
	JsonCacheObject->SetNumberField(TEXT("SizeY"), CacheFrame.Size.Y);
	JsonCacheObject->SetNumberField(TEXT("SizeZ"), CacheFrame.Size.Z);

	TArray<uint8> PixelData = GetPixelData(FrameIndex);
	if (PixelData.Num() == 0 || !TargetFolder.IsSet() || !FilenamePrefix.IsSet())
	{
		return JsonCacheObject;
	}

	const ERawImageFormat::Type ImageFormat = FImageCoreUtils::GetRawImageFormatForPixelFormat(CacheFrame.Format);
	if (ImageFormat != ERawImageFormat::RGBA16F)
	{
		UE_LOGF(LogNiagara, Error, "Unable to save render target to file with current pixel format");
		return JsonCacheObject;
	}

	const uint32 PixelCount = CacheFrame.Size.X * CacheFrame.Size.Y * CacheFrame.Size.Z;
	const uint32 BlockBytes = GPixelFormats[CacheFrame.Format].BlockBytes;

	// we need to convert the raw pixel data into a format that the image utils understand 
	TArray<FFloat16Color> ImagePixelData;
	ImagePixelData.AddZeroed(PixelCount);

	for (uint32 Index = 0; Index < PixelCount; Index++)
	{
		const uint32 Offset = Index * BlockBytes;
		ImagePixelData[Index] = NDIRenderTargetSimCacheDataPrivate::ConvertPixel(PixelData.GetData() + Offset, CacheFrame.Format);
	}
		
	FImageView ImageView(ImagePixelData.GetData(), CacheFrame.Size.X, CacheFrame.Size.Y, CacheFrame.Size.Z, ImageFormat, EGammaSpace::Linear);
	if (FImageUtils::SaveImageByExtension(*FPaths::Combine(TargetFolder.GetValue(), FilenamePrefix.GetValue() + ".exr"), ImageView))
	{
		JsonCacheObject->SetStringField(TEXT("TextureData"), FilenamePrefix.GetValue() + ".exr");
	}		
	return JsonCacheObject;
}

TArray<uint8> UNDIRenderTargetSimCacheData::GetPixelData(int32 FrameIndex) const
{
	if (!Frames.IsValidIndex(FrameIndex))
	{
		return TArray<uint8>();
	}

	const FNDIRenderTargetSimCacheFrame& FrameData = Frames[FrameIndex];

	PixelDataFrames.SetNum(Frames.Num());
	if (PixelDataFrames[FrameIndex] == nullptr)
	{
		FrameData.BulkData.GetCopy(reinterpret_cast<void**>(&PixelDataFrames[FrameIndex]), true);
	}

	TArray<uint8> OutPixelData;
	OutPixelData.AddUninitialized(FrameData.UncompressedSize);
	if (FrameData.CompressedSize > 0)
	{
		const bool bUncompressSucceeded = FCompression::UncompressMemory(CompressionType, OutPixelData.GetData(), FrameData.UncompressedSize, PixelDataFrames[FrameIndex], FrameData.CompressedSize);
		check(bUncompressSucceeded);
	}
	else
	{
		FMemory::Memcpy(OutPixelData.GetData(), PixelDataFrames[FrameIndex], FrameData.UncompressedSize);
	}

	return OutPixelData;
}

void UNDIRenderTargetSimCacheData::GetPixelData(int32 FrameIndex, uint8* DestPixelData, int32 DestRowPitch, int32 DestSlicePitch) const
{
	if (!Frames.IsValidIndex(FrameIndex))
	{
		return;
	}

	const FNDIRenderTargetSimCacheFrame& FrameData = Frames[FrameIndex];

	PixelDataFrames.SetNum(Frames.Num());
	if (PixelDataFrames[FrameIndex] == nullptr)
	{
		FrameData.BulkData.GetCopy(reinterpret_cast<void**>(&PixelDataFrames[FrameIndex]), true);
	}

	const uint32 BlockBytes = GPixelFormats[FrameData.Format].BlockBytes;
	const int32 SrcRowPitch = FrameData.Size.X * BlockBytes;
	const int32 SrcSlicePitch = FrameData.Size.Y * SrcRowPitch;

	// Fast path (direct copy / decompress)
	if (DestRowPitch == SrcRowPitch && (FrameData.Size.Z == 1 || DestSlicePitch == SrcSlicePitch))
	{
		if (FrameData.CompressedSize > 0)
		{
			const bool bUncompressSucceeded = FCompression::UncompressMemory(CompressionType, DestPixelData, FrameData.UncompressedSize, PixelDataFrames[FrameIndex], FrameData.CompressedSize);
			check(bUncompressSucceeded);
		}
		else
		{
			FMemory::Memcpy(DestPixelData, PixelDataFrames[FrameIndex], FrameData.UncompressedSize);
		}
	}
	// Slow path as we have to adjust for row / slice size differences
	else
	{
		TArray<uint8> Decompressed;
		if (FrameData.CompressedSize > 0)
		{
			Decompressed.AddUninitialized(BlockBytes * FrameData.Size.X * FrameData.Size.Y * FrameData.Size.Z);
			const bool bUncompressSucceeded = FCompression::UncompressMemory(CompressionType, Decompressed.GetData(), Decompressed.Num(), PixelDataFrames[FrameIndex], FrameData.CompressedSize);
			check(bUncompressSucceeded);
		}

		const uint8* SrcData = Decompressed.Num() > 0 ? Decompressed.GetData() : PixelDataFrames[FrameIndex];
		for (int32 Slice=0; Slice < FrameData.Size.Z; ++Slice)
		{
			uint8* DstData = DestPixelData + (Slice * DestSlicePitch);
			for (int32 Row=0; Row < FrameData.Size.Y; ++Row)
			{
				FMemory::Memcpy(DstData, SrcData, SrcRowPitch);
				SrcData += SrcRowPitch;
				DstData += DestRowPitch;
			}
		}
	}
}

void UNDIRenderTargetSimCacheData::ReleasePixelData(int32 FrameIndex)
{
	if (PixelDataFrames.IsValidIndex(FrameIndex))
	{
		FMemory::Free(PixelDataFrames[FrameIndex]);
		PixelDataFrames[FrameIndex] = nullptr;
	}
}

void UNDIRenderTargetSimCacheData::ReleaseAllPixelData()
{
	for (uint8* PixelData : PixelDataFrames)
	{
		FMemory::Free(PixelData);
	}
	PixelDataFrames.Empty();
}

void UNDIRenderTargetSimCacheData::SetPixelData(int32 FrameIndex, FIntVector Size, EPixelFormat Format, TArrayView<uint8> PixelData)
{
	ReleaseAllPixelData();

	Frames.SetNum(FMath::Max(Frames.Num(), FrameIndex) + 1);

	FNDIRenderTargetSimCacheFrame& FrameData = Frames[FrameIndex];
	FrameData.Size				= Size;
	FrameData.Format			= Format;
	FrameData.UncompressedSize	= PixelData.Num();
	FrameData.CompressedSize	= 0;

	// Compressed
	if (!CompressionType.IsNone())
	{
		const ECompressionFlags CompressionFlags = COMPRESS_BiasMemory;
		int CompressedSize = FCompression::CompressMemoryBound(CompressionType, PixelData.Num(), CompressionFlags);

		TArray<uint8> CompressedPixelData;
		CompressedPixelData.AddUninitialized(CompressedSize);
		if (FCompression::CompressMemory(CompressionType, CompressedPixelData.GetData(), CompressedSize, PixelData.GetData(), PixelData.Num(), CompressionFlags))
		{
			FrameData.BulkData.Lock(LOCK_READ_WRITE);
			FMemory::Memcpy(FrameData.BulkData.Realloc(CompressedSize), CompressedPixelData.GetData(), CompressedSize);
			FrameData.BulkData.Unlock();

			FrameData.CompressedSize = CompressedSize;
			return;
		}
	}

	// Uncompressed
	FrameData.BulkData.Lock(LOCK_READ_WRITE);
	FMemory::Memcpy(FrameData.BulkData.Realloc(PixelData.Num()), PixelData.GetData(), PixelData.Num());
	FrameData.BulkData.Unlock();
}
