// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceActorComponent.h"
#include "Components/PrimitiveComponent.h"
#include "GameFramework/Pawn.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraTypes.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraWorldManager.h"

#include "Components/SceneComponent.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/Actor.h"
#include "GameFramework/PlayerController.h"
#include "ShaderCompilerCore.h"
#include "Misc/LargeWorldRenderPosition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceActorComponent)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceActorComponent"

struct FNiagaraActorDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		LWCConversion = 1,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

namespace NDIActorComponentLocal
{
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceActorComponentTemplate.ush");

	static const FName	GetMatrixName(TEXT("GetMatrix"));
	static const FName	GetTransformName(TEXT("GetTransform"));
	static const FName	GetVelocityName(TEXT("GetVelocity"));

	static const FString ValidString(TEXT("Valid"));
	static const FString MatrixString(TEXT("Matrix"));
	static const FString RotationString(TEXT("Rotation"));
	static const FString ScaleString(TEXT("Scale"));

	struct FInstanceData_GameThread
	{
		FNiagaraParameterDirectBinding<UObject*>	UserParamBinding;
		bool										bCachedValid = false;
		FTransform									CachedTransform = FTransform::Identity;
		FVector3f									CachedVelocity = FVector3f::ZeroVector;

		// our use of UserParamBinding can occur within CalculateTickGroup which occurs before we tick our parameter stores.  This
		// can lead to a stale UObject reference being accessed (if the actor we're pointing at is deleted).  For now we cache
		// the results during PerInstanceTick and re-use the result (if it remains valid) for calculating the tick group.
		TWeakObjectPtr<UActorComponent>				CachedActorForCalcTickGroup;
	};

	struct FGameToRenderInstanceData
	{
		bool		bCachedValid = false;
		FTransform	CachedTransform = FTransform::Identity;
		FVector3f	CachedVelocity = FVector3f::ZeroVector;
	};

	struct FInstanceData_RenderThread
	{
		bool		bCachedValid = false;
		FTransform	CachedTransform = FTransform::Identity;
		FVector3f	CachedVelocity = FVector3f::ZeroVector;
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FGameToRenderInstanceData); }

		static void ProvidePerInstanceDataForRenderThread(void* InDataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
		{
			const FInstanceData_GameThread* InstanceData = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
			FGameToRenderInstanceData* DataForRenderThread = reinterpret_cast<FGameToRenderInstanceData*>(InDataForRenderThread);
			DataForRenderThread->bCachedValid = InstanceData->bCachedValid;
			DataForRenderThread->CachedTransform = InstanceData->CachedTransform;
			DataForRenderThread->CachedVelocity = InstanceData->CachedVelocity;
		}

		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override
		{
			FGameToRenderInstanceData* InstanceDataFromGT = reinterpret_cast<FGameToRenderInstanceData*>(PerInstanceData);

			FInstanceData_RenderThread& InstanceData = SystemInstancesToInstanceData_RT.FindOrAdd(InstanceID);
			InstanceData.bCachedValid = InstanceDataFromGT->bCachedValid;
			InstanceData.CachedTransform = InstanceDataFromGT->CachedTransform;
			InstanceData.CachedVelocity = InstanceDataFromGT->CachedVelocity;
		}

		TMap<FNiagaraSystemInstanceID, FInstanceData_RenderThread> SystemInstancesToInstanceData_RT;
	};

	USceneComponent* FindLocalPlayer(UWorld* World, int LocalPlayerIndex)
	{
		if ( World == nullptr )
		{
			return nullptr;
		}

		for (FConstPlayerControllerIterator Iterator=World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if ( PlayerController == nullptr || PlayerController->Player == nullptr )
			{
				continue;
			}
			
			ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(PlayerController->Player);
			if ( LocalPlayer == nullptr )
			{
				continue;
			}

			if (LocalPlayerIndex == LocalPlayer->GetLocalPlayerIndex())
			{
				AActor* PlayerPawn = PlayerController->GetPawn();
				return PlayerPawn ? PlayerPawn->GetRootComponent() : nullptr;
			}
		}
		return nullptr;
	}
}

//////////////////////////////////////////////////////////////////////////
// Data Interface
UNiagaraDataInterfaceActorComponent::UNiagaraDataInterfaceActorComponent(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new NDIActorComponentLocal::FNDIProxy());

	FNiagaraTypeDefinition Def(UObject::StaticClass());
	ActorOrComponentParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceActorComponent::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

UActorComponent* UNiagaraDataInterfaceActorComponent::ResolveComponent(FNiagaraSystemInstance* SystemInstance, const void* PerInstanceData) const
{
	using namespace NDIActorComponentLocal;

	if (SourceMode == ENDIActorComponentSourceMode::AttachParent)
	{
		if (USceneComponent* AttachComponent = SystemInstance->GetAttachComponent())
		{
			return AttachComponent;
		}
	}
	else if (SourceMode == ENDIActorComponentSourceMode::LocalPlayer)
	{
		if (USceneComponent* RootComponent = FindLocalPlayer(SystemInstance->GetWorldManager()->GetWorld(), LocalPlayerIndex))
		{
			return RootComponent;
		}
	}

	FInstanceData_GameThread* InstanceData = (FInstanceData_GameThread*)PerInstanceData;
	if (UObject* ObjectBinding = InstanceData->UserParamBinding.GetValue())
	{
		if (UActorComponent* ComponentBinding = Cast<UActorComponent>(ObjectBinding))
		{
			return ComponentBinding;
		}
		else if (AActor* ActorBinding = Cast<AActor>(ObjectBinding))
		{
			return ActorBinding->GetRootComponent();
		}
	}

	return SourceActor.IsValid() ? SourceActor->GetRootComponent() : nullptr;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceActorComponent::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIActorComponentLocal;
	FNiagaraFunctionSignature DefaultSig;
	DefaultSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("ActorComponent"));
	DefaultSig.bMemberFunction = true;
	DefaultSig.bRequiresContext = false;
	DefaultSig.bSupportsGPU = true;
	DefaultSig.SetFunctionVersion(FNiagaraActorDIFunctionVersion::LatestVersion);
	{
		FNiagaraFunctionSignature& FunctionSignature = OutFunctions.Add_GetRef(DefaultSig);
		FunctionSignature.Name = GetMatrixName;
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Matrix"));
		FunctionSignature.SetDescription(LOCTEXT("GetMatrix", "Returns the actor/component to world matrix if valid, otherwise identity."));
	}
	{
		FNiagaraFunctionSignature& FunctionSignature = OutFunctions.Add_GetRef(DefaultSig);
		FunctionSignature.Name = GetTransformName;
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation"));
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale"));
		FunctionSignature.SetDescription(LOCTEXT("GetTransform", "Returns the actor/component to world transform if valid, otherwise identity."));
	}
	{
		FNiagaraFunctionSignature& FunctionSignature = OutFunctions.Add_GetRef(DefaultSig);
		FunctionSignature.Name = GetVelocityName;
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity"));
		FunctionSignature.SetDescription(LOCTEXT("GetVelocityDesc", "Returns the actor/component world space velocity if valid, otherwise zero."));
	}
}
#endif

void UNiagaraDataInterfaceActorComponent::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIActorComponentLocal;
	if (BindingInfo.Name == GetMatrixName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceActorComponent::VMGetMatrix);
	}
	else if (BindingInfo.Name == GetTransformName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceActorComponent::VMGetTransform);
	}
	else if (BindingInfo.Name == GetVelocityName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceActorComponent::VMGetPhysicsVelocity);
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceActorComponent::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIActorComponentLocal::TemplateShaderFile);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceActorComponent::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIActorComponentLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceActorComponent::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIActorComponentLocal;
	return (FunctionInfo.DefinitionName == GetMatrixName) || (FunctionInfo.DefinitionName == GetTransformName) || (FunctionInfo.DefinitionName == GetVelocityName);
}

bool UNiagaraDataInterfaceActorComponent::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	// LWC upgrades
	if (FunctionSignature.FunctionVersion < FNiagaraActorDIFunctionVersion::LWCConversion)
	{
		TArray<FNiagaraFunctionSignature> AllFunctions;
		GetFunctionsInternal(AllFunctions);
		for (const FNiagaraFunctionSignature& Sig : AllFunctions)
		{
			if (FunctionSignature.Name == Sig.Name)
			{
				FunctionSignature = Sig;
				return true;
			}
		}
	}
	return false;
}
#endif

void UNiagaraDataInterfaceActorComponent::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceActorComponent::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIActorComponentLocal;

	FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	FInstanceData_RenderThread& InstanceData = DIProxy.SystemInstancesToInstanceData_RT.FindChecked(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->Valid		= InstanceData.bCachedValid ? 1 : 0;
	ShaderParameters->Matrix	= (FMatrix44f)InstanceData.CachedTransform.ToMatrixWithScale();
	ShaderParameters->Rotation	= (FQuat4f)InstanceData.CachedTransform.GetRotation();
	ShaderParameters->Scale		= (FVector3f)InstanceData.CachedTransform.GetScale3D();
	ShaderParameters->Velocity	= InstanceData.CachedVelocity;
}

bool UNiagaraDataInterfaceActorComponent::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIActorComponentLocal;

	FInstanceData_GameThread* InstanceData = new (PerInstanceData) FInstanceData_GameThread;
	InstanceData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), ActorOrComponentParameter.Parameter);
	InstanceData->CachedActorForCalcTickGroup = ResolveComponent(SystemInstance, InstanceData);

	return true;
}

void UNiagaraDataInterfaceActorComponent::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIActorComponentLocal;

	FInstanceData_GameThread* InstanceData = (FInstanceData_GameThread*)PerInstanceData;
	InstanceData->~FInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(RemoveProxy)
	(
		[RT_Proxy=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->SystemInstancesToInstanceData_RT.Remove(InstanceID);
		}
	);
}

int32 UNiagaraDataInterfaceActorComponent::PerInstanceDataSize() const
{
	return sizeof(NDIActorComponentLocal::FInstanceData_GameThread);
}

bool UNiagaraDataInterfaceActorComponent::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIActorComponentLocal;

	check(SystemInstance);
	FInstanceData_GameThread* InstanceData = (FInstanceData_GameThread*)PerInstanceData;
	if (!InstanceData)
	{
		return true;
	}

	InstanceData->bCachedValid = false;
	InstanceData->CachedTransform = FTransform::Identity;
	InstanceData->CachedVelocity = FVector3f::ZeroVector;

	UActorComponent* ActorComponent = ResolveComponent(SystemInstance, PerInstanceData);
	if (ActorComponent)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(ActorComponent) )
		{
			InstanceData->bCachedValid = true;
			InstanceData->CachedTransform = SceneComponent->GetComponentToWorld();
			InstanceData->CachedTransform.AddToTranslation(FVector(SystemInstance->GetLWCTile()) * -FLargeWorldRenderScalar::GetTileSize());
			if ( AActor* OwnerActor = ActorComponent->GetOwner() )
			{
				InstanceData->CachedVelocity = FVector3f(OwnerActor->GetVelocity());
			}
			else if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(ActorComponent))
			{
				InstanceData->CachedVelocity = FVector3f(PrimitiveComponent->GetPhysicsLinearVelocity());
			}
		}
		else if (AActor* OwnerActor = ActorComponent->GetOwner())
		{
			InstanceData->bCachedValid = true;
			InstanceData->CachedTransform = OwnerActor->GetTransform();
			InstanceData->CachedTransform.AddToTranslation(FVector(SystemInstance->GetLWCTile()) * -FLargeWorldRenderScalar::GetTileSize());
			InstanceData->CachedVelocity = FVector3f(OwnerActor->GetVelocity());
		}
	}

	if (bRequireCurrentFrameData)
	{
		InstanceData->CachedActorForCalcTickGroup = ActorComponent;
	}

	return false;
}

void UNiagaraDataInterfaceActorComponent::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	NDIActorComponentLocal::FNDIProxy::ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, SystemInstance);
}

ETickingGroup UNiagaraDataInterfaceActorComponent::CalculateTickGroup(const void* PerInstanceData) const
{
	if ( bRequireCurrentFrameData )
	{
		const NDIActorComponentLocal::FInstanceData_GameThread* InstanceData = (NDIActorComponentLocal::FInstanceData_GameThread*)PerInstanceData;
		const UActorComponent* ActorComponent = InstanceData ? InstanceData->CachedActorForCalcTickGroup.Get() : nullptr;
		if (ActorComponent)
		{
			ETickingGroup FinalTickGroup = FMath::Max(ActorComponent->PrimaryComponentTick.TickGroup, ActorComponent->PrimaryComponentTick.EndTickGroup);
			//-TODO: Do we need to do this?
			//if ( USkeletalMeshComponent* SkelMeshComponent = Cast<USkeletalMeshComponent>(ActorComponent) )
			//{
			//	if (SkelMeshComponent->bBlendPhysics)
			//	{
			//		FinalTickGroup = FMath::Max(FinalTickGroup, TG_EndPhysics);
			//	}
			//}
			FinalTickGroup = FMath::Clamp(ETickingGroup(FinalTickGroup + 1), NiagaraFirstTickGroup, NiagaraLastTickGroup);
			return FinalTickGroup;
		}
	}
	return NiagaraFirstTickGroup;
}

bool UNiagaraDataInterfaceActorComponent::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfaceActorComponent* OtherTyped = CastChecked<const UNiagaraDataInterfaceActorComponent>(Other);
	return OtherTyped->SourceActor == SourceActor
		&& OtherTyped->ActorOrComponentParameter == ActorOrComponentParameter
		&& OtherTyped->SourceMode == SourceMode
		&& OtherTyped->LocalPlayerIndex == LocalPlayerIndex
		&& OtherTyped->bRequireCurrentFrameData == bRequireCurrentFrameData;
}

bool UNiagaraDataInterfaceActorComponent::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceActorComponent* OtherTyped = CastChecked<UNiagaraDataInterfaceActorComponent>(Destination);
	OtherTyped->SourceMode = SourceMode;
	OtherTyped->LocalPlayerIndex = LocalPlayerIndex;
	OtherTyped->SourceActor = SourceActor;
	OtherTyped->ActorOrComponentParameter = ActorOrComponentParameter;
	OtherTyped->bRequireCurrentFrameData = bRequireCurrentFrameData;
	return true;
}

void UNiagaraDataInterfaceActorComponent::VMGetMatrix(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIActorComponentLocal;

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<bool>		OutValid(Context);
	FNDIOutputParam<FMatrix44f>	OutMatrix(Context);

	const FMatrix44f InstanceMatrix = FMatrix44f(InstanceData->CachedTransform.ToMatrixWithScale());		// LWC_TODO: Precision loss
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutValid.SetAndAdvance(InstanceData->bCachedValid);
		OutMatrix.SetAndAdvance(InstanceMatrix);
	}
}

void UNiagaraDataInterfaceActorComponent::VMGetTransform(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIActorComponentLocal;

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<bool>		OutValid(Context);
	FNDIOutputParam<FVector3f>	OutPosition(Context);
	FNDIOutputParam<FQuat4f>	OutRotation(Context);
	FNDIOutputParam<FVector3f>	OutScale(Context);

	for (int32 i=0; i < Context.GetNumInstances(); ++i)
	{
		OutValid.SetAndAdvance(InstanceData->bCachedValid);
		OutPosition.SetAndAdvance((FVector3f)InstanceData->CachedTransform.GetLocation());
		OutRotation.SetAndAdvance((FQuat4f)InstanceData->CachedTransform.GetRotation());
		OutScale.SetAndAdvance((FVector3f)InstanceData->CachedTransform.GetScale3D());
	}
}

void UNiagaraDataInterfaceActorComponent::VMGetPhysicsVelocity(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIActorComponentLocal;

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<bool>		OutValid(Context);
	FNDIOutputParam<FVector3f>	OutVelocity(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutValid.SetAndAdvance(InstanceData->bCachedValid);
		OutVelocity.SetAndAdvance(InstanceData->CachedVelocity);
	}
}

#undef LOCTEXT_NAMESPACE

