// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceArrayDistributionInt.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraShaderParametersBuilder.h"

#include "Misc/ScopeRWLock.h"
#include "RenderGraphBuilder.h"
#include "WeightedRandomSampler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceArrayDistributionInt)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceArrayDistributionInt"

namespace NDIArrayDistributionIntPrivate
{
	static const FName NAME_GetProbabilityAlias("GetProbabilityAlias");
	static const FName NAME_GetRandomValue("GetRandomValue");

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(int,					Length)
		SHADER_PARAMETER_SRV(ByteAddressBuffer,	BuiltTableData)
	END_SHADER_PARAMETER_STRUCT()

	const TCHAR* TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceArrayDistributionIntTemplate.ush");

	constexpr int32 GetBuiltTableEntrySize()
	{
		return sizeof(int32) * 4;
	}

	void EncodeBuiltTableEntry(TArray<uint8>& TableData, int32 Index, float Prob, int32 Alias, int32 Value, float Weight)
	{
		const int32 EntrySize = GetBuiltTableEntrySize();
		const int32 Offset = Index * EntrySize;
		check(TableData.Num() >= Offset + EntrySize);
		uint8* Data = TableData.GetData() + Offset;
		FMemory::Memcpy(Data +  0, &Prob, sizeof(float));
		FMemory::Memcpy(Data +  4, &Alias, sizeof(int32));
		FMemory::Memcpy(Data +  8, &Value, sizeof(int32));
		FMemory::Memcpy(Data + 12, &Weight, sizeof(float));
	}

	void DecodeTableDirect(const TArray<uint8>& TableData, int32 Index, int32& OutValue, float& OutWeight)
	{
		const int32 EntrySize = GetBuiltTableEntrySize();
		const int32 Offset = Index * EntrySize;
		check(TableData.Num() >= Offset + EntrySize);
		const uint8* Data = TableData.GetData() + Offset;
		FMemory::Memcpy(&OutValue, Data + 8, sizeof(int32));
		FMemory::Memcpy(&OutWeight, Data + 12, sizeof(float));
	}

	void DecodeTableProbAlias(const TArray<uint8>& TableData, int32 Index, float& OutProb, int32& OutAlias)
	{
		const int32 EntrySize = GetBuiltTableEntrySize();
		const int32 Offset = Index * EntrySize;
		check(TableData.Num() >= Offset + EntrySize);
		const uint8* Data = TableData.GetData() + Offset;
		FMemory::Memcpy(&OutProb, Data + 0, sizeof(float));
		FMemory::Memcpy(&OutAlias, Data + 4, sizeof(int32));
	}

	void DecodeTableIndirect(const TArray<uint8>& TableData, int32 Index, float Probability, int32& OutValue, float& OutWeight)
	{
		const int32 EntrySize = GetBuiltTableEntrySize();
		const int32 Offset = Index * EntrySize;
		check(TableData.Num() >= Offset + EntrySize);
		const uint8* Data = TableData.GetData() + Offset;

		float EntryProb;
		int32 EntryAlias;
		FMemory::Memcpy(&EntryProb, Data + 0, sizeof(float));
		FMemory::Memcpy(&EntryAlias, Data + 4, sizeof(int32));
		Index = Probability > EntryProb ? EntryAlias : Index;

		DecodeTableDirect(TableData, Index, OutValue, OutWeight);
	}

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }
		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override {}

		void UpdateGpuDataIfNeeded(FRDGBuilder& GraphBuilder)
		{
			if (bNeedsGpuDataUpdate == false)
			{
				return;
			}

			UE::TRWScopeLock ScopeLock(OwnerDI->BuiltTableDataGuard, SLT_ReadOnly);

			bNeedsGpuDataUpdate = false;

			const bool bDataValue = OwnerDI->BuiltTableData.Num() > 0;
			const int32 TableSizeBytes = bDataValue ? OwnerDI->BuiltTableData.Num() : GetBuiltTableEntrySize();

			BuiltTableDataLength = TableSizeBytes / GetBuiltTableEntrySize();
			BuiltTableData.Initialize(GraphBuilder.RHICmdList, TEXT("NiagaraDataInterfaceArrayDistributionInt"), TableSizeBytes);

			void* UploadMemory = GraphBuilder.RHICmdList.LockBuffer(BuiltTableData.Buffer, 0, TableSizeBytes, RLM_WriteOnly);
			if (OwnerDI->BuiltTableData.Num() > 0)
			{
				FMemory::Memcpy(UploadMemory, OwnerDI->BuiltTableData.GetData(), TableSizeBytes);
			}
			else
			{
				FMemory::Memzero(UploadMemory, TableSizeBytes);
			}
			GraphBuilder.RHICmdList.UnlockBuffer(BuiltTableData.Buffer);
		}

		UNiagaraDataInterfaceArrayDistributionInt*	OwnerDI = nullptr;
		bool										bNeedsGpuDataUpdate = true;
		int32										BuiltTableDataLength = 0;
		FByteAddressBuffer							BuiltTableData;
	};
}

UNiagaraDataInterfaceArrayDistributionInt::UNiagaraDataInterfaceArrayDistributionInt(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDIArrayDistributionIntPrivate;

	Proxy.Reset(new FNDIProxy());
	GetProxyAs<FNDIProxy>()->OwnerDI = this;
}

void UNiagaraDataInterfaceArrayDistributionInt::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceArrayDistributionInt::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITORONLY_DATA
	BuildTableData();
#endif
}

#if WITH_EDITOR
void UNiagaraDataInterfaceArrayDistributionInt::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	BuildTableData();
}
#endif

bool UNiagaraDataInterfaceArrayDistributionInt::Equals(const UNiagaraDataInterface* InOther) const
{
	using namespace NDIArrayDistributionIntPrivate;

	const UNiagaraDataInterfaceArrayDistributionInt* Other = CastChecked<const UNiagaraDataInterfaceArrayDistributionInt>(InOther);
	return
		Super::Equals(Other) &&
		Other->ArrayData == ArrayData;
}

bool UNiagaraDataInterfaceArrayDistributionInt::CopyToInternal(UNiagaraDataInterface* InDestination) const
{
	using namespace NDIArrayDistributionIntPrivate;

	if (!Super::CopyToInternal(InDestination))
	{
		return false;
	}

	UNiagaraDataInterfaceArrayDistributionInt* Destination = CastChecked<UNiagaraDataInterfaceArrayDistributionInt>(InDestination);
	Destination->ArrayData = ArrayData;
	Destination->BuildTableData();
	return true;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceArrayDistributionInt::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIArrayDistributionIntPrivate;

	FNiagaraFunctionSignature DefaultImmutableSig;
	DefaultImmutableSig.bMemberFunction = true;
	DefaultImmutableSig.bRequiresContext = false;
	DefaultImmutableSig.bSupportsCPU = true;
	DefaultImmutableSig.bSupportsGPU = true;
	DefaultImmutableSig.Inputs.Emplace(FNiagaraTypeDefinition(UNiagaraDataInterfaceArrayDistributionInt::StaticClass()), TEXT("Distribution Int Array"));

	FNiagaraDataInterfaceArrayImplInternal::GetImmutableFunctions(OutFunctions, DefaultImmutableSig, FNiagaraTypeDefinition(FNDIDistributionIntArrayEntry::StaticStruct()));
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultImmutableSig);
		Sig.Name = NAME_GetProbabilityAlias;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Probability"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Alias"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultImmutableSig);
		Sig.Name = NAME_GetRandomValue;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value"));
	}
}
#endif

void UNiagaraDataInterfaceArrayDistributionInt::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIArrayDistributionIntPrivate;

	if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_LengthName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceArrayDistributionInt::VMLength);
	}
	else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_IsValidIndexName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceArrayDistributionInt::VMIsValidIndex);
	}
	else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_LastIndexName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceArrayDistributionInt::VMGetLastIndex);
	}
	else if (BindingInfo.Name == FNiagaraDataInterfaceArrayImplInternal::Function_GetName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceArrayDistributionInt::VMGet);
	}
	else if (BindingInfo.Name == NDIArrayDistributionIntPrivate::NAME_GetProbabilityAlias)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceArrayDistributionInt::VMGetProbAlias);
	}
	else if (BindingInfo.Name == NDIArrayDistributionIntPrivate::NAME_GetRandomValue)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceArrayDistributionInt::VMGetRandomValue);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceArrayDistributionInt::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	using namespace NDIArrayDistributionIntPrivate;

	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceArrayDistributionInt::GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL)
{
	using namespace NDIArrayDistributionIntPrivate;

	return true;
}

bool UNiagaraDataInterfaceArrayDistributionInt::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace NDIArrayDistributionIntPrivate;

	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	return bSuccess;
}
#endif

void UNiagaraDataInterfaceArrayDistributionInt::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NDIArrayDistributionIntPrivate;

	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceArrayDistributionInt::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIArrayDistributionIntPrivate;

	FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	DIProxy.UpdateGpuDataIfNeeded(Context.GetGraphBuilder());

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->Length			= DIProxy.BuiltTableDataLength;
	ShaderParameters->BuiltTableData	= DIProxy.BuiltTableData.SRV;
}

void UNiagaraDataInterfaceArrayDistributionInt::BuildTableData()
{
	using namespace NDIArrayDistributionIntPrivate;

	struct FArrayWeightedSampler : public FWeightedRandomSampler
	{
		FArrayWeightedSampler(TConstArrayView<FNDIDistributionIntArrayEntry> InArrayData)
			: ArrayData(InArrayData)
		{
		}

		virtual float GetWeights(TArray<float>& OutWeights) override
		{
			float SumWeights = 0.0f;

			OutWeights.SetNumUninitialized(ArrayData.Num());
			for (int32 i = 0; i < ArrayData.Num(); ++i)
			{
				const FNDIDistributionIntArrayEntry& Entry = ArrayData[i];
				OutWeights[i] = FMath::Max(Entry.Weight, UE_SMALL_NUMBER);
				SumWeights += OutWeights[i];
			}

			return SumWeights;
		}

		TConstArrayView<FNDIDistributionIntArrayEntry> ArrayData;
	};

	FArrayWeightedSampler ArraySampler(ArrayData);
	ArraySampler.Initialize();

	TArray<uint8> NewBuiltTableData;
	NewBuiltTableData.SetNumUninitialized(FMath::Max(ArrayData.Num(), 1) * GetBuiltTableEntrySize());

	if (ArrayData.Num() > 0)
	{
		NewBuiltTableData.SetNumUninitialized(ArrayData.Num() * GetBuiltTableEntrySize());

		for (int32 i = 0; i < ArrayData.Num(); ++i)
		{
			const float Prob = ArraySampler.GetProb()[i];
			const int32 Alias = ArraySampler.GetAlias()[i];
			const int32 Value = ArrayData[i].Value;
			const float Weight = ArrayData[i].Weight;
			EncodeBuiltTableEntry(NewBuiltTableData, i, Prob, Alias, Value, Weight);
		}
	}
	else
	{
		EncodeBuiltTableEntry(NewBuiltTableData, 0, 0.0f, 0, 0, 0.0f);
	}

	UE::TRWScopeLock ScopeLock(BuiltTableDataGuard, SLT_Write);
	Swap(BuiltTableData, NewBuiltTableData);

	GetProxyAs<FNDIProxy>()->bNeedsGpuDataUpdate = true;
}

int32 UNiagaraDataInterfaceArrayDistributionInt::GetBuiltTableLength() const
{
	using namespace NDIArrayDistributionIntPrivate;

	UE::TRWScopeLock ScopeLock(BuiltTableDataGuard, SLT_ReadOnly);
	return BuiltTableData.Num() / GetBuiltTableEntrySize();
}

void UNiagaraDataInterfaceArrayDistributionInt::VMLength(FVectorVMExternalFunctionContext& Context) const
{
	FNDIOutputParam<int32> OutLength(Context);

	const int32 Length = GetBuiltTableLength();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutLength.SetAndAdvance(Length);
	}
}

void UNiagaraDataInterfaceArrayDistributionInt::VMIsValidIndex(FVectorVMExternalFunctionContext& Context) const
{
	FNDIInputParam<int32> InIndex(Context);
	FNDIOutputParam<bool> OutValid(Context);

	const int32 Length = GetBuiltTableLength();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 Index = InIndex.GetAndAdvance();
		const bool bIsValid = Index >= 0 && Index < Length;
		OutValid.SetAndAdvance(bIsValid);
	}
}

void UNiagaraDataInterfaceArrayDistributionInt::VMGetLastIndex(FVectorVMExternalFunctionContext& Context) const
{
	FNDIOutputParam<int32> OutLastIndex(Context);

	const int32 LastIndex = GetBuiltTableLength() - 1;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutLastIndex.SetAndAdvance(LastIndex);
	}
}

void UNiagaraDataInterfaceArrayDistributionInt::VMGet(FVectorVMExternalFunctionContext& Context) const
{
	using namespace NDIArrayDistributionIntPrivate;

	FNDIInputParam<int32>	InIndex(Context);
	FNDIOutputParam<int32>	OutIndex(Context);
	FNDIOutputParam<float>	OutWeight(Context);

	UE::TRWScopeLock ScopeLock(BuiltTableDataGuard, SLT_ReadOnly);
	const int32 LengthMinusOne = GetBuiltTableLength() - 1;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 Index = InIndex.GetAndAdvance();
		const int32 ClampedIndex = FMath::Clamp(Index, 0, LengthMinusOne);
		
		int32 EntryValue = 0;
		float EntryWeight = 0.0f;
		DecodeTableDirect(BuiltTableData, ClampedIndex, EntryValue, EntryWeight);

		OutIndex.SetAndAdvance(EntryValue);
		OutWeight.SetAndAdvance(EntryWeight);
	}
}

void UNiagaraDataInterfaceArrayDistributionInt::VMGetProbAlias(FVectorVMExternalFunctionContext& Context) const
{
	using namespace NDIArrayDistributionIntPrivate;

	FNDIInputParam<int32>	InIndex(Context);
	FNDIOutputParam<float>	OutProb(Context);
	FNDIOutputParam<int32>	OutAlias(Context);

	UE::TRWScopeLock ScopeLock(BuiltTableDataGuard, SLT_ReadOnly);
	const int32 LengthMinusOne = GetBuiltTableLength() - 1;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 Index = InIndex.GetAndAdvance();
		const int32 ClampedIndex = FMath::Clamp(Index, 0, LengthMinusOne);

		float EntryProb = 0.0f;
		int32 EntryAlias = 0;
		DecodeTableProbAlias(BuiltTableData, ClampedIndex, EntryProb, EntryAlias);

		OutProb.SetAndAdvance(EntryProb);
		OutAlias.SetAndAdvance(EntryAlias);
	}
}

void UNiagaraDataInterfaceArrayDistributionInt::VMGetRandomValue(FVectorVMExternalFunctionContext& Context) const
{
	using namespace NDIArrayDistributionIntPrivate;

	FNDIRandomHelper		RandHelper(Context);
	FNDIOutputParam<int32>	OutValue(Context);

	UE::TRWScopeLock ScopeLock(BuiltTableDataGuard, SLT_ReadOnly);
	const int32 LengthMinusOne = GetBuiltTableLength() - 1;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		RandHelper.GetAndAdvance();
		int32 Index = RandHelper.RandRange(i, 0, LengthMinusOne);

		int32 EntryValue = 0;
		float EntryWeight = 0.0f;
		DecodeTableIndirect(BuiltTableData, Index, RandHelper.RandRange(i, 0.0f, 1.0f), EntryValue, EntryWeight);

		OutValue.SetAndAdvance(EntryValue);
	}
}

void UNiagaraDataInterfaceArrayDistributionInt::SetNiagaraArrayDistributionInt(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<FNDIDistributionIntArrayEntry>& InArrayData)
{
	UNiagaraDataInterfaceArrayDistributionInt* ArrayDI = UNiagaraFunctionLibrary::GetDataInterface<UNiagaraDataInterfaceArrayDistributionInt>(NiagaraComponent, OverrideName);
	if (ArrayDI == nullptr)
	{
		return;
	}

	ArrayDI->ArrayData = InArrayData;
	ArrayDI->BuildTableData();
}

#undef LOCTEXT_NAMESPACE
