// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceArrayMesh.h"
#include "NiagaraDataInterfaceArrayFunctionLibrary.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraSystemInstance.h"

#include "Engine/StaticMesh.h"
#if WITH_EDITORONLY_DATA
#include "Editor/EditorEngine.h"
#include "Subsystems/ImportSubsystem.h"
#include "Editor.h"
#endif //WITH_EDITORONLY_DATA

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceArrayMesh)

template<>
struct FNDIArrayImplHelper<FNiagaraMeshRendererMeshPropertiesBase> : public FNDIArrayImplHelperBase<FNiagaraMeshRendererMeshPropertiesBase>
{
	typedef FNiagaraMeshRendererMeshPropertiesBase TVMArrayType;

	static constexpr bool bSupportsCPU = false;
	static constexpr bool bSupportsGPU = false;

	static const FNiagaraTypeDefinition GetTypeDefinition() { return FNiagaraTypeDefinition(FNiagaraMeshRendererMeshPropertiesBase::StaticStruct()); }
	static const FNiagaraMeshRendererMeshPropertiesBase GetDefaultValue() { return FNiagaraMeshRendererMeshPropertiesBase(); }

	static void CopyCpuToCpuMemory(FNiagaraMeshRendererMeshPropertiesBase* Dest, const FNiagaraMeshRendererMeshPropertiesBase* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(FNiagaraMeshRendererMeshPropertiesBase));
	}

	static void CopyCpuToCpuMemory(FNiagaraMeshRendererMeshPropertiesBase* Dest, UStaticMesh*const* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = FNiagaraMeshRendererMeshPropertiesBase();
			Dest[i].Mesh = Src[i];
		}
	}

	static void AppendValueToString(const FNiagaraMeshRendererMeshPropertiesBase& Value, FString& OutString)
	{
		GetFNameSafe(Value.Mesh).AppendString(OutString);
	}

	static bool IsNearlyEqual(const FNiagaraMeshRendererMeshPropertiesBase& Lhs, const FNiagaraMeshRendererMeshPropertiesBase& Rhs, float Tolerance)
	{
		return Lhs.IsNearlyEqual(Rhs, Tolerance);
	}
};

NDIARRAY_GENERATE_IMPL(UNiagaraDataInterfaceArrayMesh, FNiagaraMeshRendererMeshPropertiesBase, MeshData)

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceArrayMesh::PostLoad()
{
	Super::PostLoad();
	AddMeshChangedDelegates();
}

void UNiagaraDataInterfaceArrayMesh::BeginDestroy()
{
	RemoveMeshChangedDelegates();
	Super::BeginDestroy();
}

void UNiagaraDataInterfaceArrayMesh::PreEditChange(class FProperty* PropertyThatWillChange)
{
	RemoveMeshChangedDelegates();
	Super::PreEditChange(PropertyThatWillChange);
}

void UNiagaraDataInterfaceArrayMesh::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	AddMeshChangedDelegates();
}

void UNiagaraDataInterfaceArrayMesh::OnMeshChanged()
{
	FProxyType* ArrayProxy = static_cast<FProxyType*>(GetProxy());
	ArrayProxy->RecreateRenderState();
}

void UNiagaraDataInterfaceArrayMesh::OnMeshPostBuild(UStaticMesh*)
{
	OnMeshChanged();
}

void UNiagaraDataInterfaceArrayMesh::OnAssetReimported(UObject* Object)
{
	for (const FNiagaraMeshRendererMeshPropertiesBase& MeshProperties : MeshData)
	{
		if (MeshProperties.Mesh == Object)
		{
			OnMeshChanged();
			break;
		}
	}
}

void UNiagaraDataInterfaceArrayMesh::AddMeshChangedDelegates()
{
	if (!GIsEditor || HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	bool bAnyAdded = false;
	for (const FNiagaraMeshRendererMeshPropertiesBase& MeshProperties : MeshData)
	{
		if (MeshProperties.Mesh)
		{
			MeshProperties.Mesh->GetOnMeshChanged().AddUObject(this, &UNiagaraDataInterfaceArrayMesh::OnMeshChanged);
			MeshProperties.Mesh->OnPreMeshBuild().AddUObject(this, &UNiagaraDataInterfaceArrayMesh::OnMeshPostBuild);
			MeshProperties.Mesh->OnPostMeshBuild().AddUObject(this, &UNiagaraDataInterfaceArrayMesh::OnMeshPostBuild);
			bAnyAdded = true;
		}
	}

	if (bAnyAdded)
	{
		if (UImportSubsystem* ImportSubsystem = GEditor->GetEditorSubsystem<UImportSubsystem>())
		{
			ImportSubsystem->OnAssetReimport.AddUObject(this, &UNiagaraDataInterfaceArrayMesh::OnAssetReimported);
		}
	}
}

void UNiagaraDataInterfaceArrayMesh::RemoveMeshChangedDelegates()
{
	if (!GIsEditor || HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	if (UImportSubsystem* ImportSubsystem = GEditor->GetEditorSubsystem<UImportSubsystem>())
	{
		ImportSubsystem->OnAssetReimport.RemoveAll(this);
	}

	for (const FNiagaraMeshRendererMeshPropertiesBase& MeshProperties : MeshData)
	{
		if (MeshProperties.Mesh)
		{
			MeshProperties.Mesh->GetOnMeshChanged().RemoveAll(this);
			MeshProperties.Mesh->OnPreMeshBuild().RemoveAll(this);
			MeshProperties.Mesh->OnPostMeshBuild().RemoveAll(this);
		}
	}
}
#endif// WITH_EDITORONLY_DATA

void UNiagaraDataInterfaceArrayMesh::ForEachMesh(FNiagaraSystemInstance* SystemInstance, TFunction<void(int32)> NumMeshesDelegate, TFunction<void(const FNiagaraMeshRendererMeshProperties&)> IterateDelegate) const
{
	const FNDIArrayInstanceData_GameThread<FNiagaraMeshRendererMeshPropertiesBase>* PerInstanceData = nullptr;
	if (SystemInstance)
	{
		PerInstanceData = GetProxyAs<FProxyType>()->GetPerInstanceData_GameThread(SystemInstance->GetId());
	}

	FProxyType::FReadArrayRef ArrayRef(this, PerInstanceData);
	if (NumMeshesDelegate.IsSet())
	{
		NumMeshesDelegate(ArrayRef.GetArray().Num());
	}

	if (IterateDelegate.IsSet())
	{
		FNiagaraMeshRendererMeshProperties MeshProperties;
		for (const FNiagaraMeshRendererMeshPropertiesBase& MeshEntry : ArrayRef.GetArray())
		{
			MeshProperties.Mesh				= MeshEntry.Mesh;
			MeshProperties.Scale			= MeshEntry.Scale;
			MeshProperties.Rotation			= MeshEntry.Rotation;
			MeshProperties.PivotOffset		= MeshEntry.PivotOffset;
			MeshProperties.PivotOffsetSpace	= MeshEntry.PivotOffsetSpace;
			IterateDelegate(MeshProperties);
		}
	}
}

void UNiagaraDataInterfaceArrayMesh::SetNiagaraArrayMesh(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<FNiagaraMeshRendererMeshPropertiesBase>& ArrayData)
{
	if (UNiagaraDataInterfaceArrayMesh* ArrayDI = UNiagaraFunctionLibrary::GetDataInterface<UNiagaraDataInterfaceArrayMesh>(NiagaraComponent, OverrideName))
	{
	#if WITH_EDITORONLY_DATA
		ArrayDI->RemoveMeshChangedDelegates();
	#endif
		auto* ArrayProxy = static_cast<typename UNiagaraDataInterfaceArrayMesh::FProxyType*>(ArrayDI->GetProxy());
		ArrayProxy->SetArrayDataAndRecreateRenderState(MakeArrayView(ArrayData));
	#if WITH_EDITORONLY_DATA
		ArrayDI->AddMeshChangedDelegates();
	#endif
	#if WITH_EDITOR
		ArrayDI->CreateAndSetVariant(
			NiagaraComponent,
			OverrideName,
			[ArrayData](UNiagaraDataInterfaceArray* VariantDI)
			{
				CastChecked<UNiagaraDataInterfaceArrayMesh>(VariantDI)->SetVariantArrayData(MakeConstArrayView(ArrayData));
			}
		);
	#endif
	}
}

void UNiagaraDataInterfaceArrayMesh::SetNiagaraArrayMeshSM(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<UStaticMesh*>& ArrayData)
{
	if (UNiagaraDataInterfaceArrayMesh* ArrayDI = UNiagaraFunctionLibrary::GetDataInterface<UNiagaraDataInterfaceArrayMesh>(NiagaraComponent, OverrideName))
	{
	#if WITH_EDITORONLY_DATA
		ArrayDI->RemoveMeshChangedDelegates();
	#endif
		auto* ArrayProxy = static_cast<typename UNiagaraDataInterfaceArrayMesh::FProxyType*>(ArrayDI->GetProxy());
		ArrayProxy->SetArrayDataAndRecreateRenderState(MakeArrayView(ArrayData));
	#if WITH_EDITORONLY_DATA
		ArrayDI->AddMeshChangedDelegates();
	#endif
	#if WITH_EDITOR
		ArrayDI->CreateAndSetVariant(
			NiagaraComponent,
			OverrideName,
			[ArrayData](UNiagaraDataInterfaceArray* VariantDI)
			{
				CastChecked<UNiagaraDataInterfaceArrayMesh>(VariantDI)->SetVariantArrayData(MakeConstArrayView(ArrayData));
			}
		);
	#endif
	}
}
