// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceAsyncGpuTrace.h"

#include "NiagaraAsyncGpuTraceHelper.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSimStageData.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystem.h"
#include "NiagaraTypes.h"
#include "NiagaraSystemInstance.h"
#include "SceneManagement.h"
#include "RenderGraphBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceAsyncGpuTrace)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceAsyncGpuTrace"

namespace NDIAsyncGpuTraceLocal
{
	static const TCHAR* CommonShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceAsyncGpuTrace.ush");
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceAsyncGpuTraceTemplate.ush");

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
		SHADER_PARAMETER(uint32,											MaxRayTraceCount)
		SHADER_PARAMETER_UAV(RWStructuredBuffer<FNiagaraAsyncGpuTrace>,		RWRayRequests)
		SHADER_PARAMETER(uint32,											RayRequestsOffset)
		SHADER_PARAMETER_SRV(StructuredBuffer<FNiagaraAsyncGpuTraceResult>,	IntersectionResults)
		SHADER_PARAMETER(uint32,											IntersectionResultsOffset)
		SHADER_PARAMETER_UAV(RWBuffer<uint>,								RWRayTraceCounts)
		SHADER_PARAMETER(uint32,											RayTraceCountsOffset)
	END_SHADER_PARAMETER_STRUCT()

	static const FName IssueAsyncRayTraceName(TEXT("IssueAsyncRayTraceGpu"));
	static const FName CreateAsyncRayTraceName(TEXT("CreateAsyncRayTraceGpu"));
	static const FName ReserveAsyncRayTraceName(TEXT("ReserveAsyncRayTraceGpu"));
	static const FName ReadAsyncRayTraceName(TEXT("ReadAsyncRayTraceGpu"));

	struct FPerInstanceData
	{
		FNiagaraSystemInstanceID InstanceID;
		ENDICollisionQuery_AsyncGpuTraceProvider::Type ProviderType;
		bool RequiresAsyncTraces;
	};
}

struct FNiagaraAsyncGpuTraceDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		AddOptionalExecution = 1,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

struct FNiagaraDataIntefaceProxyAsyncGpuTrace : public FNiagaraDataInterfaceProxy
{
	struct FInstanceData
	{
		int32 MaxTracesPerParticle = 0;
		uint32 MaxRetraces = 0;
		TEnumAsByte<ENDICollisionQuery_AsyncGpuTraceProvider::Type> ProviderType = ENDICollisionQuery_AsyncGpuTraceProvider::Default;
	};

	using ProxyDataMap = TMap<FNiagaraSystemInstanceID, FInstanceData>;
	ProxyDataMap SystemInstancesToProxyData_RT;

	virtual ~FNiagaraDataIntefaceProxyAsyncGpuTrace()
	{
	}

	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return 0;
	}

	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override
	{
		FNiagaraDataInterfaceProxy::PreStage(Context);

		if (const FInstanceData* InstanceData_RT = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()))
		{
			if (InstanceData_RT->MaxTracesPerParticle > 0)
			{
				//Accumulate the total ray requests for this DI for all dispatches in the stage.
				const int32 RayRequests = InstanceData_RT->MaxTracesPerParticle * Context.GetSimStageData().DestinationNumInstances;
				Context.GetComputeDispatchInterface().GetAsyncGpuTraceHelper().AddToDispatch(
					this,
					RayRequests,
					InstanceData_RT->MaxRetraces,
					InstanceData_RT->ProviderType
				);
			}
		}
	}

	virtual bool RequiresPreStageFinalize() const override
	{
		return true;
	}

	virtual void FinalizePreStage(FRDGBuilder& GraphBuilder, const FNiagaraGpuComputeDispatchInterface& ComputeDispatchInterface) override
	{
		FNiagaraAsyncGpuTraceHelper& TraceHelper = ComputeDispatchInterface.GetAsyncGpuTraceHelper();
		if (SystemInstancesToProxyData_RT.Num())
		{
			TraceHelper.BuildDispatch(GraphBuilder.RHICmdList, this);
		}
		else
		{
			TraceHelper.BuildDummyDispatch(GraphBuilder.RHICmdList);
		}
	}
};

////////////////////////////////////////////////////////////////////////////////
/// 
////////////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceAsyncGpuTrace::UNiagaraDataInterfaceAsyncGpuTrace(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
    Proxy.Reset(new FNiagaraDataIntefaceProxyAsyncGpuTrace());
}

int32 UNiagaraDataInterfaceAsyncGpuTrace::PerInstanceDataSize() const
{
	return sizeof(NDIAsyncGpuTraceLocal::FPerInstanceData);
}

bool UNiagaraDataInterfaceAsyncGpuTrace::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance)
{
	NDIAsyncGpuTraceLocal::FPerInstanceData* PIData = new (PerInstanceData) NDIAsyncGpuTraceLocal::FPerInstanceData;
	PIData->InstanceID = InSystemInstance->GetId();
	PIData->RequiresAsyncTraces = false;

	if (TraceProvider == ENDICollisionQuery_AsyncGpuTraceProvider::None)
	{
		return true;
	}
	
	FNiagaraDataInterfaceUtilities::ForEachGpuFunction(this, InSystemInstance, [&](const UNiagaraScript* Script, const FNiagaraDataInterfaceGeneratedFunction& Function)
	{
		if (Function.DefinitionName == NDIAsyncGpuTraceLocal::IssueAsyncRayTraceName
			|| Function.DefinitionName == NDIAsyncGpuTraceLocal::CreateAsyncRayTraceName
			|| Function.DefinitionName == NDIAsyncGpuTraceLocal::ReserveAsyncRayTraceName
			|| Function.DefinitionName == NDIAsyncGpuTraceLocal::ReadAsyncRayTraceName)
		{
			PIData->RequiresAsyncTraces = true;
		}

		return !PIData->RequiresAsyncTraces;
	});

	PIData->ProviderType = FNiagaraAsyncGpuTraceProvider::ResolveSupportedType(TraceProvider, GetDefault<UNiagaraSettings>()->NDICollisionQuery_AsyncGpuTraceProviderOrder);

	// if nothing is supported then we shouldn't bother with initializing things
	if (PIData->ProviderType == ENDICollisionQuery_AsyncGpuTraceProvider::None)
	{
		UE_LOGF(LogNiagara, Warning, "Failed to find a supported provider when initializing %ls with TraceProvider = %ls.  Failed to initialize System %ls.",
			*GetPathNameSafe(this),
			*StaticEnum<ENDICollisionQuery_AsyncGpuTraceProvider::Type>()->GetDisplayValueAsText(TraceProvider).ToString(),
			*GetPathNameSafe(InSystemInstance->GetSystem()));
		return false;
	}

	if (PIData->RequiresAsyncTraces && MaxTracesPerParticle)
	{
		// Push Updates to Proxy.
		FNiagaraDataIntefaceProxyAsyncGpuTrace* RT_Proxy = GetProxyAs<FNiagaraDataIntefaceProxyAsyncGpuTrace>();
		ENQUEUE_RENDER_COMMAND(FUpdateData)(
			[RT_Proxy,
			RT_MaxTraceCount = MaxTracesPerParticle,
			RT_MaxRetraceCount = MaxRetraces,
			RT_ProviderType = PIData->ProviderType,
			RT_InstanceID = InSystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
		{
			check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(RT_InstanceID));
			FNiagaraDataIntefaceProxyAsyncGpuTrace::FInstanceData& TargetData = RT_Proxy->SystemInstancesToProxyData_RT.Add(RT_InstanceID);

			TargetData.MaxTracesPerParticle = RT_MaxTraceCount;
			TargetData.MaxRetraces = RT_MaxRetraceCount;
			TargetData.ProviderType = RT_ProviderType;
		});
	}

	return true;
}

void UNiagaraDataInterfaceAsyncGpuTrace::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance)
{
	NDIAsyncGpuTraceLocal::FPerInstanceData* PIData = reinterpret_cast<NDIAsyncGpuTraceLocal::FPerInstanceData*>(PerInstanceData);
	PIData->~FPerInstanceData();

	FNiagaraDataIntefaceProxyAsyncGpuTrace* RT_Proxy = GetProxyAs<FNiagaraDataIntefaceProxyAsyncGpuTrace>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[RT_Proxy, RT_InstanceID = InSystemInstance->GetId()](FRHICommandListImmediate& CmdList)
	{
		RT_Proxy->SystemInstancesToProxyData_RT.Remove(RT_InstanceID);
	});
}

void UNiagaraDataInterfaceAsyncGpuTrace::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceAsyncGpuTrace::PostLoad()
{
	Super::PostLoad();

	MarkRenderDataDirty();
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceAsyncGpuTrace::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	const FText RayTraceStartWorldDescription = LOCTEXT("TraceStartWorldDescription", "Ray starting point in world space");
	const FText RayTraceEndWorldDescription = LOCTEXT("TraceEndWorldDescription", "Ray end point in world space");
	const FText CollisionGroupDescription = LOCTEXT("CollisionGroupDescription", "Collision group index of the primitives we wish to skip");
	const FText QueryIDDescription = LOCTEXT("QueryIDDescription", "Unique (for this frame) index of the query being enqueued (used in subsequent frames to retrieve results).");
	const FText CollisionPosWorldDescription = LOCTEXT("CollisionPosWorldDescription", "If the collision is valid, this returns the location of the blocking hit.");
	const FText CollisionNormalDescription = LOCTEXT("CollisionNormalDescription", "The surface normal of the world geometry at the point of intersection");
	const FText PreviousFrameQueryIDDescription = LOCTEXT("PreviousFrameQueryIDDescription", "The query ID returned from the last frame's async trace call.\nRegardless if it is a valid ID or not this function call with issue a new async line trace, but it will only return results with a valid ID.");
	const FText ExperimentalMessage = LOCTEXT("AsyncRayTraceExperimental", "Feature remains in an experimental stage, with some features (eg. collision groups) limited to a subset of the providers.");

	FNiagaraVariable CollisionGroupVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("CollisionGroup"));
	CollisionGroupVariable.SetValue(INDEX_NONE);

	{
		const FText IssueValidDescription = LOCTEXT("IssueValidDescription", "Returns true if the query was issued");

		FNiagaraFunctionSignature& IssueRayTrace = OutFunctions.AddDefaulted_GetRef();
		IssueRayTrace.Name = NDIAsyncGpuTraceLocal::IssueAsyncRayTraceName;
		IssueRayTrace.bRequiresExecPin = true;
		IssueRayTrace.bMemberFunction = true;
		IssueRayTrace.bSupportsCPU = false;
		IssueRayTrace.bExperimental = true;
#if WITH_EDITORONLY_DATA
		IssueRayTrace.FunctionVersion = FNiagaraAsyncGpuTraceDIFunctionVersion::LatestVersion;
		IssueRayTrace.Description = LOCTEXT("IssueAsyncRayTraceDescription", "Enqueues a GPU raytrace with the result being available the following frame");
		IssueRayTrace.ExperimentalMessage = ExperimentalMessage;
#endif
		IssueRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("AsyncGpuTrace")));
		IssueRayTrace.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		IssueRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("QueryID")), QueryIDDescription);
		IssueRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceStartWorld")), RayTraceStartWorldDescription);
		IssueRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceEndWorld")), RayTraceEndWorldDescription);
		IssueRayTrace.AddInput(CollisionGroupVariable, CollisionGroupDescription);
		IssueRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsQueryValid")), IssueValidDescription);
	}

	{
		const FText CreateValidDescription = LOCTEXT("CreateValidDescription", "Returns true if the query was created");

		FNiagaraFunctionSignature& CreateRayTrace = OutFunctions.AddDefaulted_GetRef();
		CreateRayTrace.Name = NDIAsyncGpuTraceLocal::CreateAsyncRayTraceName;
		CreateRayTrace.bRequiresExecPin = true;
		CreateRayTrace.bMemberFunction = true;
		CreateRayTrace.bSupportsCPU = false;
		CreateRayTrace.bExperimental = true;
		CreateRayTrace.FunctionVersion = FNiagaraAsyncGpuTraceDIFunctionVersion::LatestVersion;
		CreateRayTrace.Description = LOCTEXT("CreateAsyncRayTraceDescription", "Creates a GPU raytrace with the result being available the following frame (index is returned)");
		CreateRayTrace.ExperimentalMessage = ExperimentalMessage;
		CreateRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("AsyncGpuTrace")));
		CreateRayTrace.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		CreateRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceStartWorld")), RayTraceStartWorldDescription);
		CreateRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceEndWorld")), RayTraceEndWorldDescription);
		CreateRayTrace.AddInput(CollisionGroupVariable, CollisionGroupDescription);
		CreateRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("QueryID")), QueryIDDescription);
		CreateRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsQueryValid")), CreateValidDescription);
	}

	{
		const FText TraceCountDescription = LOCTEXT("QueryIDDescription2", "Number of async raytrace requests to be reserved");
		const FText FirstQueryIDValidDescription = LOCTEXT("FirstQueryIDValidDescription", "The first index in the block reserved through this call");
		const FText ReserveValidDescription = LOCTEXT("ReserveValidDescription", "Returns true if the requested indices were reserved");

		FNiagaraFunctionSignature& ReserveRayTrace = OutFunctions.AddDefaulted_GetRef();
		ReserveRayTrace.Name = NDIAsyncGpuTraceLocal::ReserveAsyncRayTraceName;
		ReserveRayTrace.bRequiresExecPin = true;
		ReserveRayTrace.bMemberFunction = true;
		ReserveRayTrace.bSupportsCPU = false;
		ReserveRayTrace.bExperimental = true;
		ReserveRayTrace.FunctionVersion = FNiagaraAsyncGpuTraceDIFunctionVersion::LatestVersion;
		ReserveRayTrace.Description = LOCTEXT("ReserveAsyncRayTraceDescription", "Reserves a number of ray trace request slots");
		ReserveRayTrace.ExperimentalMessage = ExperimentalMessage;
		ReserveRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("AsyncGpuTrace")));
		ReserveRayTrace.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		ReserveRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("TraceCount")), TraceCountDescription);
		ReserveRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("FirstQueryID")), FirstQueryIDValidDescription);
		ReserveRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsQueryValid")), ReserveValidDescription);
	}

	{
		const FText CollisionValidDescription = LOCTEXT("AsyncCollisionValidDescription", "Returns true if the a Hit was encountered");
		const FText CollisionDistanceDescription = LOCTEXT("CollisionDistanceDescription", "The distance in world space from the ray starting point to the intersection");

		FNiagaraFunctionSignature& ReadRayTrace = OutFunctions.AddDefaulted_GetRef();
		ReadRayTrace.Name = NDIAsyncGpuTraceLocal::ReadAsyncRayTraceName;
		ReadRayTrace.bMemberFunction = true;
		ReadRayTrace.bSupportsCPU = false;
		ReadRayTrace.bExperimental = true;
		ReadRayTrace.FunctionVersion = FNiagaraAsyncGpuTraceDIFunctionVersion::LatestVersion;
		ReadRayTrace.Description = LOCTEXT("ReadAsyncRayTraceDescription", "Reads the results of a previously enqueued GPU ray trace");
		ReadRayTrace.ExperimentalMessage = ExperimentalMessage;
		ReadRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("AsyncGpuTrace")));
		ReadRayTrace.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		ReadRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PreviousFrameQueryID")), PreviousFrameQueryIDDescription);
		ReadRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("CollisionValid")), CollisionValidDescription);
		ReadRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CollisionDistance")), CollisionDistanceDescription);
		ReadRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("CollisionPosWorld")), CollisionPosWorldDescription);
		ReadRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CollisionNormal")), CollisionNormalDescription);
	}
}
#endif

bool UNiagaraDataInterfaceAsyncGpuTrace::RequiresGlobalDistanceField() const
{
	return FNiagaraAsyncGpuTraceHelper::RequiresGlobalDistanceField(TraceProvider);
}

bool UNiagaraDataInterfaceAsyncGpuTrace::RequiresRayTracingScene() const
{
	return FNiagaraAsyncGpuTraceHelper::RequiresRayTracingScene(TraceProvider);
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceAsyncGpuTrace::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if ( (FunctionInfo.DefinitionName == NDIAsyncGpuTraceLocal::IssueAsyncRayTraceName) ||
		 (FunctionInfo.DefinitionName == NDIAsyncGpuTraceLocal::CreateAsyncRayTraceName) ||
		 (FunctionInfo.DefinitionName == NDIAsyncGpuTraceLocal::ReserveAsyncRayTraceName) ||
		 (FunctionInfo.DefinitionName == NDIAsyncGpuTraceLocal::ReadAsyncRayTraceName) )
	{
		return true;
	}

	return false;
}

bool UNiagaraDataInterfaceAsyncGpuTrace::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	// always upgrade to the latest version
	if (FunctionSignature.FunctionVersion < FNiagaraAsyncGpuTraceDIFunctionVersion::LatestVersion) // -V547
	{
		TArray<FNiagaraFunctionSignature> AllFunctions;
		GetFunctionsInternal(AllFunctions);
		for (const FNiagaraFunctionSignature& Sig : AllFunctions)
		{
			if (FunctionSignature.Name == Sig.Name)
			{
				FunctionSignature = Sig;
				return true;
			}
		}
	}

	return false;
}

void UNiagaraDataInterfaceAsyncGpuTrace::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};

	AppendTemplateHLSL(OutHLSL, NDIAsyncGpuTraceLocal::TemplateShaderFile, TemplateArgs);
}

void UNiagaraDataInterfaceAsyncGpuTrace::GetCommonHLSL(FString& OutHlsl)
{
	OutHlsl.Appendf(TEXT("#include \"%s\"\n"), NDIAsyncGpuTraceLocal::CommonShaderFile);
}

bool UNiagaraDataInterfaceAsyncGpuTrace::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}

	InVisitor->UpdateShaderFile(NDIAsyncGpuTraceLocal::CommonShaderFile);
	InVisitor->UpdateShaderFile(NDIAsyncGpuTraceLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<NDIAsyncGpuTraceLocal::FShaderParameters>();

	return true;
}

#endif

void UNiagaraDataInterfaceAsyncGpuTrace::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIAsyncGpuTraceLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceAsyncGpuTrace::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	NDIAsyncGpuTraceLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDIAsyncGpuTraceLocal::FShaderParameters>();

	const bool bRWRayRequestsBound = Context.IsResourceBound(&ShaderParameters->RWRayRequests);
	const bool bIntersectionResultsBound = Context.IsResourceBound(&ShaderParameters->IntersectionResults);
	const bool bRWRayTraceCountsBound = Context.IsResourceBound(&ShaderParameters->RWRayTraceCounts);
	const bool HasRayTracingParametersBound = bRWRayRequestsBound || bIntersectionResultsBound || bRWRayTraceCountsBound;

	if (HasRayTracingParametersBound)
	{
		FNiagaraDataIntefaceProxyAsyncGpuTrace& DIProxy = Context.GetProxy<FNiagaraDataIntefaceProxyAsyncGpuTrace>();
		const FNiagaraDataIntefaceProxyAsyncGpuTrace::FInstanceData* InstanceData = DIProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

		FNiagaraAsyncGpuTraceHelper& TraceHelper = Context.GetComputeDispatchInterface().GetAsyncGpuTraceHelper();
		const FNiagaraAsyncGpuTraceDispatchInfo* DispatchInfo = nullptr;
		if (InstanceData && InstanceData->MaxTracesPerParticle > 0)
		{
			DispatchInfo = &TraceHelper.GetDispatch(&DIProxy);
		}
		else
		{
			DispatchInfo = &TraceHelper.GetDummyDispatch();
		}

		ShaderParameters->MaxRayTraceCount = DispatchInfo->MaxTraces;

		if (bRWRayRequestsBound)
		{
			check(DispatchInfo->TraceRequests.IsValid());

			ShaderParameters->RWRayRequests = DispatchInfo->TraceRequests.Buffer->UAV;
			ShaderParameters->RayRequestsOffset = DispatchInfo->TraceRequests.Offset;
		}

		if (bIntersectionResultsBound)
		{
			check(DispatchInfo->LastFrameTraceResults.IsValid());

			ShaderParameters->IntersectionResults = DispatchInfo->LastFrameTraceResults.Buffer->SRV;
			ShaderParameters->IntersectionResultsOffset = DispatchInfo->LastFrameTraceResults.Offset;
		}

		if (bRWRayTraceCountsBound)
		{
			check(DispatchInfo->TraceCounts.IsValid());

			ShaderParameters->RWRayTraceCounts = DispatchInfo->TraceCounts.Buffer->UAV;
			ShaderParameters->RayTraceCountsOffset = DispatchInfo->TraceCounts.Offset;
		}
	}
	else
	{
		ShaderParameters->MaxRayTraceCount = 0;
	}
}

bool UNiagaraDataInterfaceAsyncGpuTrace::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfaceAsyncGpuTrace* OtherTyped = CastChecked<const UNiagaraDataInterfaceAsyncGpuTrace>(Other);
	return OtherTyped->MaxTracesPerParticle == MaxTracesPerParticle
		&& OtherTyped->MaxRetraces == MaxRetraces
		&& OtherTyped->TraceProvider == TraceProvider;
}

bool UNiagaraDataInterfaceAsyncGpuTrace::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceAsyncGpuTrace* OtherTyped = CastChecked<UNiagaraDataInterfaceAsyncGpuTrace>(Destination);
	OtherTyped->MaxTracesPerParticle = MaxTracesPerParticle;
	OtherTyped->MaxRetraces = MaxRetraces;
	OtherTyped->TraceProvider = TraceProvider;
	OtherTyped->MarkRenderDataDirty();
	return true;
}

void UNiagaraDataInterfaceAsyncGpuTrace::PushToRenderThreadImpl()
{
	FNiagaraDataIntefaceProxyAsyncGpuTrace* RT_Proxy = GetProxyAs<FNiagaraDataIntefaceProxyAsyncGpuTrace>();

	ENDICollisionQuery_AsyncGpuTraceProvider::Type ResolvedType = FNiagaraAsyncGpuTraceHelper::ResolveSupportedType(TraceProvider);

	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Proxy,
		RT_MaxTraceCount = MaxTracesPerParticle,
		RT_MaxRetraceCount = MaxRetraces,
		RT_ProviderType = ResolvedType](FRHICommandListImmediate& RHICmdList)
	{
		for (FNiagaraDataIntefaceProxyAsyncGpuTrace::ProxyDataMap::TIterator ProxyDataIt(RT_Proxy->SystemInstancesToProxyData_RT); ProxyDataIt; ++ProxyDataIt)
		{
			if (RT_MaxTraceCount)
			{
				FNiagaraDataIntefaceProxyAsyncGpuTrace::FInstanceData& InstanceData = ProxyDataIt.Value();
				InstanceData.MaxTracesPerParticle = RT_MaxTraceCount;
				InstanceData.MaxRetraces = RT_MaxRetraceCount;
				InstanceData.ProviderType = RT_ProviderType;
			}
			else
			{
				ProxyDataIt.RemoveCurrent();
			}
		}
	});
}

#if WITH_EDITOR
void UNiagaraDataInterfaceAsyncGpuTrace::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();

		if ((PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceAsyncGpuTrace, MaxTracesPerParticle))
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceAsyncGpuTrace, MaxRetraces))
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceAsyncGpuTrace, TraceProvider)))
		{
			MarkRenderDataDirty();
		}
	}
}

void UNiagaraDataInterfaceAsyncGpuTrace::GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	Super::GetFeedback(InAsset, InComponent, OutErrors, OutWarnings, OutInfo);
	IConsoleVariable* EnableRayTrace = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.Niagara.AsyncGpuTrace.HWRayTraceEnabled"));
	IConsoleVariable* EnableDistanceField = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.Niagara.AsyncGpuTrace.GlobalSdfEnabled"));
	if (TraceProvider == ENDICollisionQuery_AsyncGpuTraceProvider::HWRT && (EnableRayTrace == nullptr || !EnableRayTrace->GetBool() || !IsRayTracingEnabled()))
	{
		FText ErrorDescription = LOCTEXT("NiagaraAsyncGpuTraceNoHWRayTraceWarning", "Hardware raytracing is currently disabled");
		FText ErrorSummary = LOCTEXT("NiagaraAsyncGpuTraceNoHWRayTraceWarningSummary", "The data interface is configured to use hardware raytracing, which is currently disabled.\nThe data interface will NOT work with the current project settings.\n\nTo enable ray tracing support, all of the following must be true:\n* Default RHI is DirectX 12 in the project settings\n* Ray Tracing is enabled in the project settings\n* CVar fx.Niagara.AsyncGpuTrace.HWRayTraceEnabled is set to 1\n* CVar r.Raytracing.Enable is set to 1");
		OutWarnings.Emplace(ErrorDescription, ErrorSummary, FNiagaraDataInterfaceFix());
	}
	if (TraceProvider == ENDICollisionQuery_AsyncGpuTraceProvider::GSDF && (EnableDistanceField == nullptr || !EnableDistanceField->GetBool() || !DoesProjectSupportDistanceFields()))
	{
		FText ErrorDescription = LOCTEXT("NiagaraAsyncGpuTraceNoDFTraceWarning", "Distance field tracing is currently disabled");
		FText ErrorSummary = LOCTEXT("NiagaraAsyncGpuTraceNoDFWarningSummary", "The data interface is configured to use distance fielde tracing, which is currently disabled.\nThe data interface will NOT work with the current project settings.\n\nAside from project settings, check the following cvars:\n* fx.Niagara.AsyncGpuTrace.GlobalSdfEnabled\n* r.GenerateMeshDistanceFields\n* r.DistanceFields.SupportEvenIfHardwareRayTracingSupported (only if ray tracing is disabled)");
		OutWarnings.Emplace(ErrorDescription, ErrorSummary, FNiagaraDataInterfaceFix());
	}
}
#endif

#undef LOCTEXT_NAMESPACE

