// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceConsoleVariable.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceConsoleVariable)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceConsoleVariable"

//////////////////////////////////////////////////////////////////////////

namespace NDIConsoleVariableLocal
{
	static const FName GetConsoleVariableFloat("GetConsoleVariableFloat");
	static const FName GetConsoleVariableInt("GetConsoleVariableInt");
	static const FName GetConsoleVariableBool("GetConsoleVariableBool");

	struct FInstanceData_GameToRender
	{
		TArray<uint32>	ConsoleVariableData;
	};

	struct FInstanceData_GameThread
	{
		TArray<TPair<IConsoleVariable*, int32>>	ConsoleVariableFloats;
		TArray<TPair<IConsoleVariable*, int32>>	ConsoleVariableInts;
		TArray<TPair<IConsoleVariable*, int32>>	ConsoleVariableBools;
		TArray<uint32>							ConsoleVariableData;
		TArray<TPair<bool, int32>>				GpuConsoleVariableDataValidAndRemap;

		TPair<bool, int32> AddConsoleVariable(FName FunctionName, FName ConsoleVariableName)
		{
			bool bValid = false;
			int32 ValueOffset = INDEX_NONE;

			if (FunctionName == GetConsoleVariableFloat)
			{
				IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName.ToString());
				bValid = ConsoleVariable && ConsoleVariable->IsVariableFloat();
				ValueOffset = ConsoleVariableData.AddZeroed(1);
				if (bValid)
				{
					ConsoleVariableFloats.Emplace(ConsoleVariable, ValueOffset);
				}
			}
			else if (FunctionName == GetConsoleVariableInt)
			{
				IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName.ToString());
				bValid = ConsoleVariable && ConsoleVariable->IsVariableInt();
				ValueOffset = ConsoleVariableData.AddZeroed(1);
				if (bValid)
				{
					ConsoleVariableInts.Emplace(ConsoleVariable, ValueOffset);
				}
			}
			else if (FunctionName == GetConsoleVariableBool)
			{
				IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName.ToString());
				bValid = ConsoleVariable && ConsoleVariable->IsVariableBool();
				ValueOffset = ConsoleVariableData.AddZeroed(1);
				if (bValid)
				{
					ConsoleVariableBools.Emplace(ConsoleVariable, ValueOffset);
				}
			}
			return MakeTuple(bValid, ValueOffset);
		}
	};

	struct FInstanceData_RenderThread
	{
		FReadBuffer		ConsoleVariableData;
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		void ProvidePerInstanceDataForRenderThread(void* InDataForRenderThread, void* InPerInstanceData, const FNiagaraSystemInstanceID& InstanceID)
		{
			FInstanceData_GameToRender* InstanceData_GameToRender = new (InDataForRenderThread) FInstanceData_GameToRender();
			FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(InPerInstanceData);

			const int32 NumGpuVariableReads = InstanceData_GT->GpuConsoleVariableDataValidAndRemap.Num();
			InstanceData_GameToRender->ConsoleVariableData.AddZeroed(FMath::Max(NumGpuVariableReads, 1) * 2);
			for (int32 i=0; i < NumGpuVariableReads; ++i)
			{
				const bool bValid = InstanceData_GT->GpuConsoleVariableDataValidAndRemap[i].Key;
				const int32 RemapIndex = InstanceData_GT->GpuConsoleVariableDataValidAndRemap[i].Value;

				InstanceData_GameToRender->ConsoleVariableData[i * 2 + 0] = bValid ? 1 : 0;
				if (RemapIndex != INDEX_NONE)
				{
					InstanceData_GameToRender->ConsoleVariableData[i * 2 + 1] = InstanceData_GT->ConsoleVariableData[RemapIndex];
				}
			}
		}

		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override
		{
			FInstanceData_GameToRender* InstanceData_GameToRender = static_cast<FInstanceData_GameToRender*>(PerInstanceData);
			FInstanceData_RenderThread* InstanceData_RT = &PerInstanceData_RenderThread.FindOrAdd(InstanceID);

			FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

			const int32 NumBytes = InstanceData_GameToRender->ConsoleVariableData.Num() * InstanceData_GameToRender->ConsoleVariableData.GetTypeSize();
			if ( InstanceData_RT->ConsoleVariableData.NumBytes != NumBytes)
			{
				InstanceData_RT->ConsoleVariableData.Release();
				InstanceData_RT->ConsoleVariableData.Initialize(RHICmdList, TEXT("NiagaraConsoleVariable"), sizeof(int32), NumBytes, EPixelFormat::PF_R32_UINT);
			}

			void* GpuMemory = RHICmdList.LockBuffer(InstanceData_RT->ConsoleVariableData.Buffer, 0, NumBytes, RLM_WriteOnly);
			FMemory::Memcpy(GpuMemory, InstanceData_GameToRender->ConsoleVariableData.GetData(), NumBytes);
			RHICmdList.UnlockBuffer(InstanceData_RT->ConsoleVariableData.Buffer);

			InstanceData_GameToRender->~FInstanceData_GameToRender();
		}

		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
		{
			return sizeof(FInstanceData_GameToRender);
		}
	
		TMap<FNiagaraSystemInstanceID, FInstanceData_RenderThread>	PerInstanceData_RenderThread;
	};

	template<typename TVariableType>
	void VMGetConsoleVariable(FVectorVMExternalFunctionContext& Context, int32 ValueOffset, bool bValid)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GameThread>	InstanceData_GT(Context);
		FNDIOutputParam<bool>								SuccessValue(Context);
		FNDIOutputParam<TVariableType>						OutValue(Context);

		TVariableType Value;
		FMemory::Memcpy(&Value, &InstanceData_GT->ConsoleVariableData[ValueOffset], sizeof(TVariableType));
		for (int32 i=0; i < Context.GetNumInstances(); ++i)
		{
			SuccessValue.SetAndAdvance(bValid);
			OutValue.SetAndAdvance(Value);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceConsoleVariable::UNiagaraDataInterfaceConsoleVariable(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDIConsoleVariableLocal;

	Proxy.Reset(new FNDIProxy());
}

void UNiagaraDataInterfaceConsoleVariable::PostInitProperties()
{
	Super::PostInitProperties();

	// Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceConsoleVariable::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIConsoleVariableLocal;

	FInstanceData_GameThread* InstanceData_GT = new(PerInstanceData) FInstanceData_GameThread();

	if ( IsUsedWithGPUScript() )
	{
		// We shouldn't need to do this per init, we should be able to cache once and once only
		for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
		{
			if (EmitterInstance->IsDisabled() || EmitterInstance->GetEmitter() == nullptr || EmitterInstance->GetSimTarget() != ENiagaraSimTarget::GPUComputeSim)
			{
				continue;
			}
			const FNiagaraScriptInstanceParameterStore& ParameterStore = EmitterInstance->GetGPUContext()->CombinedParamStore;
			const TArray<UNiagaraDataInterface*>& DataInterfaces = ParameterStore.GetDataInterfaces();
			const TSharedRef<FNiagaraShaderScriptParametersMetadata> ScriptParametersMetadata = EmitterInstance->GetGPUContext()->GPUScript_RT->GetScriptParametersMetadata();
			const TArray<FNiagaraDataInterfaceGPUParamInfo>& DataInterfaceParamInfo = ScriptParametersMetadata->DataInterfaceParamInfo;
			for ( int32 iDataInterface=0; iDataInterface < DataInterfaces.Num(); ++iDataInterface)
			{
				if ( (DataInterfaces[iDataInterface] == this) && DataInterfaceParamInfo.IsValidIndex(iDataInterface) )
				{
					const TArray<FNiagaraDataInterfaceGeneratedFunction>& GeneratedFunctions = DataInterfaceParamInfo[iDataInterface].GeneratedFunctions;
					InstanceData_GT->GpuConsoleVariableDataValidAndRemap.AddZeroed(GeneratedFunctions.Num());

					for ( int32 iFunctionInfo=0; iFunctionInfo < GeneratedFunctions.Num(); ++iFunctionInfo )
					{
						const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo = GeneratedFunctions[iFunctionInfo];

						const TPair<bool, int32> ValidAndOffset = InstanceData_GT->AddConsoleVariable(FunctionInfo.DefinitionName, FunctionInfo.Specifiers.Num() > 0 ? FunctionInfo.Specifiers[0].Value : NAME_None);
						InstanceData_GT->GpuConsoleVariableDataValidAndRemap[iFunctionInfo] = ValidAndOffset;
					}
				}
			}
		}
	}

	return true; 
}

void UNiagaraDataInterfaceConsoleVariable::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIConsoleVariableLocal;

	FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
	InstanceData_GT->~FInstanceData_GameThread();

	if ( IsUsedWithGPUScript() )
	{
		ENQUEUE_RENDER_COMMAND(NDIConsoleVariable_InitRT)
		(
			[Proxy_RT=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandList& CmdList)
			{
				Proxy_RT->PerInstanceData_RenderThread.Remove(InstanceID);
			}
		);
	}
}

int32 UNiagaraDataInterfaceConsoleVariable::PerInstanceDataSize() const
{
	using namespace NDIConsoleVariableLocal;
	return sizeof(FInstanceData_GameThread);
}

void UNiagaraDataInterfaceConsoleVariable::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID)
{
	using namespace NDIConsoleVariableLocal;
	FNDIProxy* DIProxy = GetProxyAs<FNDIProxy>();
	DIProxy->ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, InstanceID);
}

bool UNiagaraDataInterfaceConsoleVariable::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIConsoleVariableLocal;

	check(PerInstanceData && SystemInstance);

	FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);

	for (const TPair<IConsoleVariable*, int32>& ConsoleVariable : InstanceData_GT->ConsoleVariableFloats)
	{
		reinterpret_cast<float&>(InstanceData_GT->ConsoleVariableData[ConsoleVariable.Value]) = ConsoleVariable.Key->GetFloat();
	}

	for (const TPair<IConsoleVariable*, int32>& ConsoleVariable : InstanceData_GT->ConsoleVariableInts)
	{
		reinterpret_cast<int32&>(InstanceData_GT->ConsoleVariableData[ConsoleVariable.Value]) = ConsoleVariable.Key->GetInt();
	}

	for (const TPair<IConsoleVariable*, int32>& ConsoleVariable : InstanceData_GT->ConsoleVariableBools)
	{
		reinterpret_cast<bool&>(InstanceData_GT->ConsoleVariableData[ConsoleVariable.Value]) = ConsoleVariable.Key->GetBool();
	}

	return false;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceConsoleVariable::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIConsoleVariableLocal;

	// Build default signature
	FNiagaraFunctionSignature DefaultSignature;
	DefaultSignature.bMemberFunction = true;
	DefaultSignature.bRequiresContext = false;
	DefaultSignature.bSupportsCPU = true;
	DefaultSignature.bSupportsGPU = true;
	DefaultSignature.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("ConsoleVariable"));
	DefaultSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success"));
	DefaultSignature.FunctionSpecifiers.Emplace("ConsoleVariableName");

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetConsoleVariableFloat;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetConsoleVariableInt;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetConsoleVariableBool;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Value"));
	}
}
#endif

void UNiagaraDataInterfaceConsoleVariable::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* PerInstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIConsoleVariableLocal;

	FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);

	IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*BindingInfo.FunctionSpecifiers[0].Value.ToString());
	if (BindingInfo.Name == GetConsoleVariableFloat)
	{
		const bool bValid = ConsoleVariable && ConsoleVariable->IsVariableFloat();
		const int32 ValueOffset = InstanceData_GT->ConsoleVariableData.AddZeroed(1);
		if (bValid)
		{
			InstanceData_GT->ConsoleVariableFloats.Emplace(ConsoleVariable, ValueOffset);
		}
		OutFunc.BindStatic(&VMGetConsoleVariable<float>, ValueOffset, bValid);
	}
	else if (BindingInfo.Name == GetConsoleVariableInt)
	{
		const bool bValid = ConsoleVariable && ConsoleVariable->IsVariableInt();
		const int32 ValueOffset = InstanceData_GT->ConsoleVariableData.AddZeroed(1);
		if (bValid)
		{
			InstanceData_GT->ConsoleVariableInts.Emplace(ConsoleVariable, ValueOffset);
		}
		OutFunc.BindStatic(&VMGetConsoleVariable<int32>, ValueOffset, bValid);
	}
	else if (BindingInfo.Name == GetConsoleVariableBool)
	{
		const bool bValid = ConsoleVariable && ConsoleVariable->IsVariableBool();
		const int32 ValueOffset = InstanceData_GT->ConsoleVariableData.AddZeroed(1);
		if (bValid)
		{
			InstanceData_GT->ConsoleVariableBools.Emplace(ConsoleVariable, ValueOffset);
		}
		OutFunc.BindStatic(&VMGetConsoleVariable<bool>, ValueOffset, bValid);
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceConsoleVariable::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceConsoleVariable::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, FString& OutHLSL)
{
	OutHLSL.Appendf(TEXT("Buffer<uint>	%s_ConsoleVariableData;\n"), *ParameterInfo.DataInterfaceHLSLSymbol);
}

bool UNiagaraDataInterfaceConsoleVariable::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIConsoleVariableLocal;

	if (FunctionInfo.DefinitionName == GetConsoleVariableFloat)
	{
		OutHLSL.Appendf(TEXT("void %s(out bool bSuccess, out float Value)\n"), *FunctionInfo.InstanceName);
		OutHLSL.Append(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	bSuccess = %s_ConsoleVariableData[%d] != 0;\n"), *ParamInfo.DataInterfaceHLSLSymbol, FunctionInstanceIndex * 2 + 0);
		OutHLSL.Appendf(TEXT("	Value = asfloat(%s_ConsoleVariableData[%d]);\n"), *ParamInfo.DataInterfaceHLSLSymbol, FunctionInstanceIndex * 2 + 1);
		OutHLSL.Append(TEXT("}\n"));
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetConsoleVariableInt)
	{
		OutHLSL.Appendf(TEXT("void %s(out bool bSuccess, out int Value)\n"), *FunctionInfo.InstanceName);
		OutHLSL.Append(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	bSuccess = %s_ConsoleVariableData[%d] != 0;\n"), *ParamInfo.DataInterfaceHLSLSymbol, FunctionInstanceIndex * 2 + 0);
		OutHLSL.Appendf(TEXT("	Value = %s_ConsoleVariableData[%d];\n"), *ParamInfo.DataInterfaceHLSLSymbol, FunctionInstanceIndex * 2 + 1);
		OutHLSL.Append(TEXT("}\n"));
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetConsoleVariableBool)
	{
		OutHLSL.Appendf(TEXT("void %s(out bool bSuccess, out bool Value)\n"), *FunctionInfo.InstanceName);
		OutHLSL.Append(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	bSuccess = %s_ConsoleVariableData[%d] != 0;\n"), *ParamInfo.DataInterfaceHLSLSymbol, FunctionInstanceIndex * 2 + 0);
		OutHLSL.Appendf(TEXT("	Value = %s_ConsoleVariableData[%d] != 0;\n"), *ParamInfo.DataInterfaceHLSLSymbol, FunctionInstanceIndex * 2 + 1);
		OutHLSL.Append(TEXT("}\n"));
		return true;
	}

	return false;
}
#endif

void UNiagaraDataInterfaceConsoleVariable::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceConsoleVariable::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIConsoleVariableLocal;

	const FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	const FInstanceData_RenderThread& InstanceData_RT = DIProxy.PerInstanceData_RenderThread.FindChecked(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters		= Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->ConsoleVariableData	= InstanceData_RT.ConsoleVariableData.SRV;
}

#undef LOCTEXT_NAMESPACE
