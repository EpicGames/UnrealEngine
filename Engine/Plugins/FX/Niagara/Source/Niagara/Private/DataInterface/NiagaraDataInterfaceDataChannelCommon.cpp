// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceDataChannelCommon.h"
#include "Misc/LazySingleton.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraScript.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceDataChannelCommon)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceDataChannelCommon"


FAutoConsoleCommandWithWorldAndArgs ResetDataChannelLayouts(
	TEXT("fx.Niagara.DataChannels.ResetLayoutInfo"),
	TEXT("Resets all data channel layout info used by data interfaces to access data channels."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args, UWorld* World)
		{
			FNDIDataChannelLayoutManager::Get().Reset();
		}
	)
);


FNDIDataChannel_FunctionToDataSetBinding::FNDIDataChannel_FunctionToDataSetBinding(const FNDIDataChannelFunctionInfo& FunctionInfo, const FNiagaraDataSetCompiledData& DataSetLayout, TArray<FNiagaraVariableBase>& OutMissingParams)
{
#if DEBUG_NDI_DATACHANNEL
	DebugFunctionInfo = FunctionInfo;
	DebugCompiledData = DataSetLayout;
#endif

	FunctionLayoutHash = GetTypeHash(FunctionInfo);
	DataSetLayoutHash = DataSetLayout.GetLayoutHash();

	VMRegisterBindings.Reset();
	auto DoGenVMBindings = [&](const TArray<FNiagaraVariableBase>& Parameters)
	{
		for (int32 ParamIdx = 0; ParamIdx < Parameters.Num(); ++ParamIdx)
		{
			const FNiagaraVariableBase& Param = Parameters[ParamIdx];
			uint32 DataSetFloatRegister = INDEX_NONE;
			uint32 DataSetIntRegister = INDEX_NONE;
			uint32 DataSetHalfRegister = INDEX_NONE;
			if (const FNiagaraVariableLayoutInfo* DataSetVariableLayout = DataSetLayout.FindVariableLayoutInfo(Param))
			{
				DataSetFloatRegister = DataSetVariableLayout->GetFloatComponentStart();
				DataSetIntRegister = DataSetVariableLayout->GetInt32ComponentStart();
				DataSetHalfRegister = DataSetVariableLayout->GetHalfComponentStart();
			}
			else
			{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
				OutMissingParams.Emplace(Param);
#endif
			}

			GenVMBindings(Param, Param.GetType().GetStruct(), NumFloatComponents, NumInt32Components, NumHalfComponents, DataSetFloatRegister, DataSetIntRegister, DataSetHalfRegister);
		}
	};

	DoGenVMBindings(FunctionInfo.Inputs);
	DoGenVMBindings(FunctionInfo.Outputs);
}

void FNDIDataChannel_FunctionToDataSetBinding::GenVMBindings(const FNiagaraVariableBase& Var, const UStruct* Struct, uint32& FuncFloatRegister, uint32& FuncIntRegister, uint32& FuncHalfRegister, uint32& DataSetFloatRegister, uint32& DataSetIntRegister, uint32& DataSetHalfRegister)
{
	for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		if (Property->IsA(FFloatProperty::StaticClass()))
		{
			VMRegisterBindings.Emplace(FuncFloatRegister++, DataSetFloatRegister == INDEX_NONE ? INDEX_NONE : DataSetFloatRegister++, ENiagaraBaseTypes::Float);
		}
		else if (Property->IsA(FUInt16Property::StaticClass()))
		{
			VMRegisterBindings.Emplace(FuncHalfRegister++, DataSetHalfRegister == INDEX_NONE ? INDEX_NONE : DataSetHalfRegister++, ENiagaraBaseTypes::Half);
		}
		else if (Property->IsA(FIntProperty::StaticClass()))
		{
			VMRegisterBindings.Emplace(FuncIntRegister++, DataSetIntRegister == INDEX_NONE ? INDEX_NONE : DataSetIntRegister++, ENiagaraBaseTypes::Int32);
		}
		else if (Property->IsA(FBoolProperty::StaticClass()))
		{
			VMRegisterBindings.Emplace(FuncIntRegister++, DataSetIntRegister == INDEX_NONE ? INDEX_NONE : DataSetIntRegister++, ENiagaraBaseTypes::Bool);
		}
		//Should be able to support double easily enough
		else if (FStructProperty* StructProp = CastField<FStructProperty>(Property))
		{
			GenVMBindings(Var, FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProp->Struct, ENiagaraStructConversion::Simulation), FuncFloatRegister, FuncIntRegister, FuncHalfRegister, DataSetFloatRegister, DataSetIntRegister, DataSetHalfRegister);
		}
		else
		{
			checkf(false, TEXT("Property(%s) Class(%s) is not a supported type"), *Property->GetName(), *Property->GetClass()->GetName());
			DataSetLayoutHash = 0;
			return;
		}
	}
}


//////////////////////////////////////////////////////////////////////////
// FNDIDataChannelFunctionInfo

uint32 GetTypeHash(const FNDIDataChannelFunctionInfo& FuncInfo)
{
	uint32 Ret = GetTypeHash(FuncInfo.FunctionName);
	for (const FNiagaraVariableBase& Param : FuncInfo.Inputs)
	{
		Ret = HashCombine(Ret, GetTypeHash(Param));
	}
	for (const FNiagaraVariableBase& Param : FuncInfo.Outputs)
	{
		Ret = HashCombine(Ret, GetTypeHash(Param));
	}
	return Ret;
}

bool FNDIDataChannelFunctionInfo::operator==(const FNDIDataChannelFunctionInfo& Other)const
{
	return FunctionName == Other.FunctionName && Inputs == Other.Inputs && Outputs == Other.Outputs;
}

bool FNDIDataChannelFunctionInfo::CheckHashConflict(const FNDIDataChannelFunctionInfo& Other)const
{
	//If we have the same hash ensure we have the same data.
	if (GetTypeHash(*this) == GetTypeHash(Other))
	{
		return *this != Other;
	}
	return false;
}

//FNDIDataChannelFunctionInfo End
//////////////////////////////////////////////////////////////////////////

//////////////////////////////////////////////////////////////////////////
//FNDIDataChannelLayoutManager

FNDIDataChannelLayoutManager& FNDIDataChannelLayoutManager::Get()
{
	return TLazySingleton<FNDIDataChannelLayoutManager>::Get();
}

void FNDIDataChannelLayoutManager::TearDown()
{
	TLazySingleton<FNDIDataChannelLayoutManager>::TearDown();
}

void FNDIDataChannelLayoutManager::Reset()
{
	FRWScopeLock WriteLock(FunctionToDataSetMapLock, SLT_Write);
	FunctionToDataSetLayoutMap.Reset();
}

FNDIDataChannel_FuncToDataSetBindingPtr FNDIDataChannelLayoutManager::GetLayoutInfo(const FNDIDataChannelFunctionInfo& FunctionInfo, const FNiagaraDataSetCompiledData& DataSetLayout, TArray<FNiagaraVariableBase>& OutMissingParams)
{
	uint32 Key = GetLayoutKey(FunctionInfo, DataSetLayout);
	
	//Attempt to find a valid existing layout.
	{
		FRWScopeLock ReadLock(FunctionToDataSetMapLock, SLT_ReadOnly);
		FNDIDataChannel_FuncToDataSetBindingPtr* FuncLayout = FunctionToDataSetLayoutMap.Find(Key);
		if (FuncLayout && FuncLayout->IsValid())
		{
			return *FuncLayout;
		}
	}

	//No valid existing layout so generate a new one.
	FRWScopeLock WriteLock(FunctionToDataSetMapLock, SLT_Write);
	FNDIDataChannel_FuncToDataSetBindingPtr FuncLayout = MakeShared<FNDIDataChannel_FunctionToDataSetBinding, ESPMode::ThreadSafe>(FunctionInfo, DataSetLayout, OutMissingParams);
	FunctionToDataSetLayoutMap.Add(Key) = FuncLayout;

	return FuncLayout;
}

//FNDIDataChannelLayoutManager END
//////////////////////////////////////////////////////////////////////////

//////////////////////////////////////////////////////////////////////////
//FNDIDataChannelCompiledData

void FNDIDataChannelCompiledData::GatherAccessInfo(const UNiagaraSystem* System, UNiagaraDataInterface* Owner)
{
	//We search all VM functions called on this DI to generate an appropriate FNDIDataChannelFunctionInfo that can later be used in binding to actual dataset Data.
	auto HandleVMFunc = [&](const UNiagaraScript* Script, const FVMExternalFunctionBindingInfo& BindingInfo)
	{
		if (BindingInfo.VariadicInputs.Num() > 0 || BindingInfo.VariadicOutputs.Num() > 0)
		{
			//Ensure we have a binding with valid input and outputs.		
			int32 FuncInfoIdx = FindFunctionInfoIndex(BindingInfo.Name, BindingInfo.VariadicInputs, BindingInfo.VariadicOutputs);
			if (FuncInfoIdx == INDEX_NONE)
			{
				FNDIDataChannelFunctionInfo& FuncInfo = FunctionInfo.AddDefaulted_GetRef();
				FuncInfo.FunctionName = BindingInfo.Name;
				FuncInfo.Inputs = BindingInfo.VariadicInputs;
				FuncInfo.Outputs = BindingInfo.VariadicOutputs;
			}
		}
		bUsedByCPU = true;

		if(BindingInfo.Name == NDIDataChannelUtilities::GetNDCSpawnDataName)
		{
			bNeedsSpawnDataTable = true;
		}
		
		if (BindingInfo.Name == NDIDataChannelUtilities::SpawnConditionalName || BindingInfo.Name == NDIDataChannelUtilities::SpawnDirectName)
		{
			bSpawnsParticles = true;			
		}

		if(BindingInfo.Name == NDIDataChannelUtilities::WriteName)
		{
			bCallsWrite = true;
		}

		return true;
	};
	FNiagaraDataInterfaceUtilities::ForEachVMFunction(Owner, System, HandleVMFunc);

	//For every GPU script we iterate over the functions it calls and add each of them to the mapping.
	//This will then be placed in a buffer for the RT to pass to the GPU so that each script can look up the correct function layout info.
	GPUScriptParameterInfos.Empty();
	TotalParams = 0;
	auto HandleGpuFunc = [&](const UNiagaraScript* Script, const FNiagaraDataInterfaceGeneratedFunction& BindingInfo)
	{
		if (BindingInfo.VariadicInputs.Num() > 0 || BindingInfo.VariadicOutputs.Num() > 0)
		{
			const FNiagaraCompileHash ScriptCompileHash = Script->GetComputedVMCompilationId().BaseScriptCompileHash;
			FNDIDataChannel_GPUScriptParameterAccessInfo& ScriptParamAccessInfo = GPUScriptParameterInfos.FindOrAdd(ScriptCompileHash);

			for (const auto& Var : BindingInfo.VariadicInputs)
			{
				ScriptParamAccessInfo.SortedParameters.AddUnique(Var);
			}
			for (const auto& Var : BindingInfo.VariadicOutputs)
			{
				ScriptParamAccessInfo.SortedParameters.AddUnique(Var);
			}
		}
		bUsedByGPU = true;

		if (BindingInfo.DefinitionName == TEXT("GetNDCSpawnData"))
		{
			bNeedsSpawnDataTable = true;
		}
		return true;
	};
	FNiagaraDataInterfaceUtilities::ForEachGpuFunction(Owner, System, HandleGpuFunc);

	//Now we've generated the complete set of parameters accessed by each GPU script, we sort them to ensure identical access between the hlsl and the table we generate.
	for (auto& Pair : GPUScriptParameterInfos)
	{
		FNDIDataChannel_GPUScriptParameterAccessInfo& ScriptParamAccessInfo = Pair.Value;
		NDIDataChannelUtilities::SortParameters(ScriptParamAccessInfo.SortedParameters);
		TotalParams += ScriptParamAccessInfo.SortedParameters.Num();
	}
}

bool FNDIDataChannelCompiledData::Init(const UNiagaraSystem* System, UNiagaraDataInterface* OwnerDI)
{
	FunctionInfo.Empty();

	check(System);
	check(OwnerDI);

	GatherAccessInfo(System, OwnerDI);
	return true;
}

int32 FNDIDataChannelCompiledData::FindFunctionInfoIndex(FName Name, const TArray<FNiagaraVariableBase>& VariadicInputs, const TArray<FNiagaraVariableBase>& VariadicOutputs)const
{
	for (int32 FuncIndex = 0; FuncIndex < FunctionInfo.Num(); ++FuncIndex)
	{
		const FNDIDataChannelFunctionInfo& FuncInfo = FunctionInfo[FuncIndex];
		if (FuncInfo.FunctionName == Name && VariadicInputs == FuncInfo.Inputs && VariadicOutputs == FuncInfo.Outputs)
		{
			return FuncIndex;
		}
	}
	return INDEX_NONE;
}

//FNDIDataChannelCompiledData END
//////////////////////////////////////////////////////////////////////////



namespace NDIDataChannelUtilities
{
	const FName GetNDCSpawnDataName(TEXT("GetNDCSpawnData"));
	const FName SpawnConditionalName(TEXT("SpawnConditional"));
	const FName SpawnDirectName(TEXT("SpawnDirect"));
	const FName WriteName(TEXT("Write"));
	const TGlobalResource<FNDIDummyUAV> DummyUAVFloat(PF_R32_FLOAT, sizeof(float));
	const TGlobalResource<FNDIDummyUAV> DummyUAVInt32(PF_R32_SINT, sizeof(int32));
	const TGlobalResource<FNDIDummyUAV> DummyUAVHalf(PF_R16F, sizeof(FFloat16));

	const TGlobalResource<FNDIDummyUAV>& GetDummyUAVFloat()
	{
		return DummyUAVFloat;
	}

	const TGlobalResource<FNDIDummyUAV>& GetDummyUAVInt32()
	{
		return DummyUAVInt32;
	}

	const TGlobalResource<FNDIDummyUAV>& GetDummyUAVHalf()
	{
		return DummyUAVHalf;
	}

	void SortParameters(TArray<FNiagaraVariableBase>& Parameters)
	{
		Parameters.Sort([](const FNiagaraVariableBase& Lhs, const FNiagaraVariableBase& Rhs)
			{
				int32 ComparisonDiff = Lhs.GetName().Compare(Rhs.GetName());
				if (ComparisonDiff == 0)
				{
					ComparisonDiff = Lhs.GetType().GetFName().Compare(Rhs.GetType().GetFName());
				}
				return ComparisonDiff < 0;
			});
	}

#if WITH_EDITORONLY_DATA

	void GenerateDataChannelAccessHlsl(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, TConstArrayView<FString> CommonTemplateShaderCode, const TMap<FName, FString>& FunctionTemplateMap, FString& OutHLSL)
	{
		//Desc:
		// This function iterates over all functions called for this DI in each script and builds the correct hlsl.
		// The main part of the complexity here is dealing with Variadic function parameters.
		// We must interrogate the script to see what parameters are actually accessed and generate hlsl accordingly.
		// Ideally at some future point we can refactor most of this out to a utility helper that will do most of the heavy lifting.
		// Allowing users to simply provide some details and lambdas etc to specify what exactly they'd like to do in the function body with each variadic param.

		//////////////////////////////////////////////////////////////////////////
		//Initially we'll do some preamble, setting up various template strings and args etc.

		//Map of all arguments for various pieces of template code.
		//We have some common code that is shared by all functions.
		//Some code is unique for each function.
		//Some is unique per parameter to each function.
		//Finally there is some that is unique for each sub component of each parameter until we've hit a base type. float/2/3/4 etc.
		//We add to the map and take refs for  params we may want to change as we iterate over parameters etc.
		
		TMap<FString, FStringFormatArg> HlslTemplateArgs =
		{
			//Common args for all functions
			{TEXT("ParameterName"),							HlslGenContext.ParameterInfo.DataInterfaceHLSLSymbol},

			//Per function args. These will be changed with each function written
			{TEXT("FunctionSymbol"),						FString(TEXT("FunctionSymbol"))},//Function symbol which will be a mangled form from the translator.

			{TEXT("FunctionInputParameters"),				FString(TEXT("FunctionInputParameters"))},//Function input parameters written into the function signature.
			{TEXT("FunctionOutputParameters"),				FString(TEXT("FunctionOutputParameters"))},//Function output parameters written into the function signature.

			{TEXT("DefaultOutputsShaderCode"),				FString(TEXT("DefaultOutputsShaderCode"))},
			{TEXT("PerParameterFunctionDefinitions"),		FString(TEXT("PerParameterFunctionDefinitions"))},//Per parameter helper/access functions
			{TEXT("PerFunctionParameterShaderCode"),		FString(TEXT("PerFunctionParameterShaderCode"))},//Per paraemter shader code embeddded in the DI function body

			//Per function parameter args. These will be changed with each parameter written
			{TEXT("FunctionParameterIndex"),				TEXT("FunctionParameterIndex")},//Function parameter index allowing us to look up the layout information for the correct parameter to the function.
			{TEXT("FunctionParameterName"),					FString(TEXT("FunctionParameterName"))},//Name of this function parameter.
			{TEXT("FunctionParameterType"),					FString(TEXT("FunctionParameterType"))},//Type of this function parameter.
			{TEXT("FuncParamShaderCode"),						FString(TEXT("PerParamRWCode"))},//Code that does the actual reading or writing to the data channel buffers.
			//Per component/base type args. These will change with every base type we I/O from the Data Channel.
			{TEXT("FunctionParameterComponentBufferType"),	FString(TEXT("FunctionParameterComponentBufferType"))},//The actual base data buffer type being accessed by a particular DataChannel access code line. Float, Int32, Half etc
			{TEXT("FunctionParameterComponentName"),		FString(TEXT("FunctionParameterComponentName"))},//The name/symbol of the actual member of a parameter that we can I/O from the DataChannel via a standard getter/setter.
			{TEXT("FunctionParameterComponentType"),		FString(TEXT("FunctionParameterComponentType"))},//The type of the actual member of a parameter that we can I/O from the DataChannel via a standard getter/setter.
			{TEXT("FunctionParameterComponentDefault"),		FString(TEXT("FunctionParameterComponentDefault"))},
		};

		//Grab refs to per function args we'll change with each function written.
		FStringFormatArg& FunctionSymbol = HlslTemplateArgs.FindChecked(TEXT("FunctionSymbol"));
		
		FStringFormatArg& FunctionInputParameters = HlslTemplateArgs.FindChecked(TEXT("FunctionInputParameters"));
		FStringFormatArg& FunctionOutputParameters = HlslTemplateArgs.FindChecked(TEXT("FunctionOutputParameters"));

		FStringFormatArg& DefaultOutputsShaderCode = HlslTemplateArgs.FindChecked(TEXT("DefaultOutputsShaderCode"));
		FStringFormatArg& PerParameterFunctionDefinitions = HlslTemplateArgs.FindChecked(TEXT("PerParameterFunctionDefinitions"));
		FStringFormatArg& PerFunctionParameterShaderCode = HlslTemplateArgs.FindChecked(TEXT("PerFunctionParameterShaderCode"));

		//Grab refs to per function args we'll change with each parameter written.
		FStringFormatArg& FunctionParameterIndex = HlslTemplateArgs.FindChecked(TEXT("FunctionParameterIndex"));
		FStringFormatArg& FunctionParameterName = HlslTemplateArgs.FindChecked(TEXT("FunctionParameterName"));
		FStringFormatArg& FunctionParameterType = HlslTemplateArgs.FindChecked(TEXT("FunctionParameterType"));
		FStringFormatArg& FuncParamShaderCode = HlslTemplateArgs.FindChecked(TEXT("FuncParamShaderCode"));

		//Grab refs to per component/base type args we'll change for every parameter component we access in the Data Channel.
		FStringFormatArg& FunctionParameterComponentBufferType = HlslTemplateArgs.FindChecked(TEXT("FunctionParameterComponentBufferType"));
		FStringFormatArg& FunctionParameterComponentName = HlslTemplateArgs.FindChecked(TEXT("FunctionParameterComponentName"));
		FStringFormatArg& FunctionParameterComponentType = HlslTemplateArgs.FindChecked(TEXT("FunctionParameterComponentType"));
		FStringFormatArg& FunctionParameterComponentDefault = HlslTemplateArgs.FindChecked(TEXT("FunctionParameterComponentDefault"));


		//////////////////////////////////////////////////////////////////////////
		//Next we'll define some template code blocks and helper lambdas that we'll use with the above args map to build various pieces of shader code.
		
		//Template code for writing handling code for each of the function parameters.
		//Some preamble and an replacement arg into which we write all the actual I/O with the DataChannel buffers.
		FString PerParameterReadTemplate = TEXT("\n\
void Read_{FunctionParameterName}_{ParameterName}(FNDCAccessContext_{ParameterName} Context, inout bool bSuccess, inout {FunctionParameterType} {FunctionParameterName})\n\
{\n\
	if(Context.InitForParameter({FunctionParameterIndex}))\n\
	{\n\
		{FuncParamShaderCode}\
	}\n\
	else\n\
	{\n\
		bSuccess = false;\n\
	}\n\
}\n");

		FString PerParameterReadCallTemplate = TEXT("Read_{FunctionParameterName}_{ParameterName}(Context, bOutSuccess, {FunctionParameterName});\n");

		FString PerParameterWriteTemplate = TEXT("\n\
void Write_{FunctionParameterName}_{ParameterName}(FNDCAccessContext_{ParameterName} Context, inout bool bSuccess, {FunctionParameterType} {FunctionParameterName})\n\
{\n\
	if(Context.InitForParameter({FunctionParameterIndex}))\n\
	{\n\
		if(Context.InitForGPUWrite())\n\
		{\n\
			{FuncParamShaderCode}\n\
		}\n\
		if (Context.InitForCPUWrite())\n\
		{\n\
			{FuncParamShaderCode}\n\
		}\n\
	}\n\
	else\n\
	{\n\
		bSuccess = false;\n\
	}\n\
}\n");
		FString PerParameterWriteCallTemplate = TEXT("Write_{FunctionParameterName}_{ParameterName}(Context, bOutSuccess, {FunctionParameterName});\n");

		//Template code for accessing data from the Data Channel's buffers.
		static const FString ReadDataTemplate = TEXT("Context.Read_{FunctionParameterComponentBufferType}({FunctionParameterComponentName});\n");
		static const FString WriteDataTemplate = TEXT("Context.Write_{FunctionParameterComponentBufferType}({FunctionParameterComponentName});\n");
		static const FString DefaultsDataTemplate = TEXT("{FunctionParameterComponentName} = {FunctionParameterComponentDefault};\n");

		//Function that will recurse down a parameter's type and generate the appropriate IO code for all of it's members.
		TFunction<void(bool, UScriptStruct*, FString&, FString&)> GeneratePerParamShaderCode = [&](bool bRead, UScriptStruct* Struct, FString& OutCode, FString& OutDefaultsCode)
		{
			//Intercept positions and replace with FVector3fs
			if (Struct == FNiagaraPosition::StaticStruct())
			{
				Struct = FNiagaraTypeDefinition::GetVec3Struct();
			}

			if (Struct == FNiagaraTypeDefinition::GetFloatStruct() || Struct == FNiagaraTypeDefinition::GetVec2Struct() || Struct == FNiagaraTypeDefinition::GetVec3Struct()
				|| Struct == FNiagaraTypeDefinition::GetVec4Struct() || Struct == FNiagaraTypeDefinition::GetColorStruct() || Struct == FNiagaraTypeDefinition::GetQuatStruct())
			{
				FunctionParameterComponentBufferType = TEXT("Float");
				FNiagaraTypeDefinition Type = FNiagaraTypeDefinition(Struct);

				FunctionParameterComponentType = HlslGenContext.GetStructHlslTypeName(Type);
				OutCode += FString::Format(bRead ? *ReadDataTemplate : *WriteDataTemplate, HlslTemplateArgs);

				FunctionParameterComponentDefault = HlslGenContext.GetHlslDefaultForType(Type);
				OutDefaultsCode += FString::Format(*DefaultsDataTemplate, HlslTemplateArgs);
			}
			else if (Struct == FNiagaraTypeDefinition::GetIntStruct())
			{
				FunctionParameterComponentBufferType = TEXT("Int32");
				FNiagaraTypeDefinition Type = FNiagaraTypeDefinition(Struct);

				FunctionParameterComponentType = HlslGenContext.GetStructHlslTypeName(Type);
				OutCode += FString::Format(bRead ? *ReadDataTemplate : *WriteDataTemplate, HlslTemplateArgs);

				FunctionParameterComponentDefault = HlslGenContext.GetHlslDefaultForType(Type);
				OutDefaultsCode += FString::Format(*DefaultsDataTemplate, HlslTemplateArgs);
			}
			else if (Struct == FNiagaraTypeDefinition::GetBoolStruct())
			{
				FunctionParameterComponentBufferType = TEXT("Bool");
				FunctionParameterComponentType = HlslGenContext.GetStructHlslTypeName(FNiagaraTypeDefinition::GetIntDef());
				OutCode += FString::Format(bRead ? *ReadDataTemplate : *WriteDataTemplate, HlslTemplateArgs);

				FunctionParameterComponentDefault = HlslGenContext.GetHlslDefaultForType(FNiagaraTypeDefinition::GetIntDef());
				OutDefaultsCode += FString::Format(*DefaultsDataTemplate, HlslTemplateArgs);
			}
			else
			{
				FString PropertyBaseName = FunctionParameterComponentName.StringValue;

				for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
				{
					FunctionParameterComponentName.StringValue = PropertyBaseName;
					const FProperty* Property = *PropertyIt;

					if (Property->IsA(FFloatProperty::StaticClass()))
					{
						FunctionParameterComponentBufferType = TEXT("Float");
						FunctionParameterComponentType = HlslGenContext.GetStructHlslTypeName(FNiagaraTypeDefinition::GetFloatDef());
						FunctionParameterComponentName.StringValue += FString::Printf(TEXT(".%s"), *HlslGenContext.GetSanitizedSymbolName(Property->GetName()));
						OutCode += FString::Format(bRead ? *ReadDataTemplate : *WriteDataTemplate, HlslTemplateArgs);

						FunctionParameterComponentDefault = HlslGenContext.GetHlslDefaultForType(FNiagaraTypeDefinition::GetFloatDef());
						OutDefaultsCode += FString::Format(*DefaultsDataTemplate, HlslTemplateArgs);
					}
					else if (Property->IsA(FIntProperty::StaticClass()))
					{
						FunctionParameterComponentBufferType = TEXT("Int32");
						FunctionParameterComponentType = HlslGenContext.GetStructHlslTypeName(FNiagaraTypeDefinition::GetIntDef());
						FunctionParameterComponentName.StringValue += FString::Printf(TEXT(".%s"), *HlslGenContext.GetSanitizedSymbolName(Property->GetName()));
						OutCode += FString::Format(bRead ? *ReadDataTemplate : *WriteDataTemplate, HlslTemplateArgs);

						FunctionParameterComponentDefault = HlslGenContext.GetHlslDefaultForType(FNiagaraTypeDefinition::GetIntDef());
						OutDefaultsCode += FString::Format(*DefaultsDataTemplate, HlslTemplateArgs);
					}
					else if (const FStructProperty* StructProp = CastFieldChecked<const FStructProperty>(Property))
					{
						FunctionParameterComponentName.StringValue += FString::Printf(TEXT(".%s"), *HlslGenContext.GetSanitizedSymbolName(Property->GetName()));
						FunctionParameterComponentType = HlslGenContext.GetPropertyHlslTypeName(Property);
						GeneratePerParamShaderCode(bRead, StructProp->Struct, OutCode, OutDefaultsCode);
					}
					else
					{
						check(false);
						OutCode += FString::Printf(TEXT("Error! - DataChannel Interface encountered an invalid type when generating it's function hlsl. %s"), *Property->GetName());
					}
				}
			}
		};

		//////////////////////////////////////////////////////////////////////////

		//We may call the same function multiple times so avoid duplicating the same function impl.
		TMap<uint32, FString> FunctionHlslMap;

		//Now lets build some hlsl!
		// 
		//First build the shader code common to all functions.
		for(const FString& TemplateShader : CommonTemplateShaderCode)
		{
			OutHLSL += FString::Format(*TemplateShader, HlslTemplateArgs);
		}

		auto GetSignatureHash = [](const FNiagaraFunctionSignature& Sig)
		{
			uint32 Ret = GetTypeHash(Sig.Name);
			for (const FNiagaraVariable& Input : Sig.Inputs)
			{
				Ret = HashCombine(Ret, GetTypeHash(Input));
			}
			for (const FNiagaraVariableBase& Output : Sig.Outputs)
			{
				Ret = HashCombine(Ret, GetTypeHash(Output));
			}
			return Ret;
		};

		struct FParamAccessInfo
		{			
			bool bRead;
			bool bWrite;
			int32 SortedOffset;
		};

		TMap<FNiagaraVariableBase, FParamAccessInfo> ParametersAccessed;
		TArray<FNiagaraVariableBase> SortedParameters;

		//First iterate over the generated functions to gather all used parameters so we can generate the sorted parameter list for all functions called by this script.
		for (int32 FuncIdx = 0; FuncIdx < HlslGenContext.ParameterInfo.GeneratedFunctions.Num(); ++FuncIdx)
		{
			const FNiagaraDataInterfaceGeneratedFunction& Func = HlslGenContext.ParameterInfo.GeneratedFunctions[FuncIdx];
			const FNiagaraFunctionSignature& Signature = HlslGenContext.Signatures[FuncIdx];

			if(Signature.VariadicInput())
			{
				int32 StartInput = Signature.VariadicInputStartIndex();
				for (int32 InputIdx = StartInput; InputIdx < Signature.Inputs.Num(); ++InputIdx)
				{
					const FNiagaraVariable& InputParam = Signature.Inputs[InputIdx];
					FParamAccessInfo& Access = ParametersAccessed.FindOrAdd(InputParam);
					Access.bWrite = true;
				}
			}			

			if(Signature.VariadicOutput())
			{
				int32 StartOutput = Signature.VariadicOutputStartIndex();
				for (int32 OutputIdx = StartOutput; OutputIdx < Signature.Outputs.Num(); ++OutputIdx)
				{
					const FNiagaraVariable& OutputParam = Signature.Outputs[OutputIdx];
					FParamAccessInfo& Access = ParametersAccessed.FindOrAdd(OutputParam);
					Access.bRead = true;
				}
			}			
		}
		
		//Sort the parameters so that that generated hlsl can exactly match the runtime ordering of parameters.
		ParametersAccessed.GenerateKeyArray(SortedParameters);
		NDIDataChannelUtilities::SortParameters(SortedParameters);
		
		DefaultOutputsShaderCode.StringValue.Reset();

		PerParameterFunctionDefinitions.StringValue.Reset();
		FunctionInputParameters.StringValue.Reset();
		FunctionOutputParameters.StringValue.Reset();

		//Next we iterate over each of the accessed parameters and generate specific read or write functions for that parameter.
		//This is placed in the template arg {PerParameterAccessFunctions}
		//Each of these functions performing any reads or writes needed for each parameter.
		for (int32 ParamIdx = 0; ParamIdx < SortedParameters.Num(); ++ParamIdx)
		{
			const FNiagaraVariableBase& Param = SortedParameters[ParamIdx];
			FParamAccessInfo& Access = 	ParametersAccessed.FindChecked(Param);
			Access.SortedOffset = ParamIdx;

			FunctionParameterIndex = LexToString(ParamIdx);
			FunctionParameterName = HlslGenContext.GetSanitizedSymbolName(Param.GetName().ToString());
			FunctionParameterType = HlslGenContext.GetStructHlslTypeName(Param.GetType());

			FunctionParameterComponentName = HlslGenContext.GetSanitizedSymbolName(Param.GetName().ToString());

			//Generate read function for this parameter if needed.
			if(Access.bRead)
			{
				FuncParamShaderCode.StringValue.Reset();
				FunctionParameterComponentType.StringValue.Reset();
				FunctionParameterComponentBufferType.StringValue.Reset();
				//Generate the per component/base type access code that will be used in the subsequent per parameters shader code template.
				GeneratePerParamShaderCode(true, Param.GetType().GetScriptStruct(), FuncParamShaderCode.StringValue, DefaultOutputsShaderCode.StringValue);
				PerParameterFunctionDefinitions.StringValue += FString::Format(*PerParameterReadTemplate, HlslTemplateArgs);
			}

			//Generate write function for this parameter if needed.
			if (Access.bWrite)
			{
				FuncParamShaderCode.StringValue.Reset();
				FunctionParameterComponentType.StringValue.Reset();
				FunctionParameterComponentBufferType.StringValue.Reset();
				//Generate the per component/base type access code that will be used in the subsequent per parameters shader code template.
				GeneratePerParamShaderCode(false, Param.GetType().GetScriptStruct(), FuncParamShaderCode.StringValue, DefaultOutputsShaderCode.StringValue);
				PerParameterFunctionDefinitions.StringValue += FString::Format(*PerParameterWriteTemplate, HlslTemplateArgs);
			}
		}

		//Now to iterate on the functions and build the hlsl for each as needed.
		static FString FirstParamPrefix = TEXT("");
		static FString ParamPrefix = TEXT(", ");
		static FString OutputParamPrefix = TEXT("out");
		static FString InputParamPrefix = TEXT("in");
		uint32 CurrParamIdx = 0;
		for (int32 FuncIdx = 0; FuncIdx < HlslGenContext.ParameterInfo.GeneratedFunctions.Num(); ++FuncIdx)
		{
			const FNiagaraDataInterfaceGeneratedFunction& Func = HlslGenContext.ParameterInfo.GeneratedFunctions[FuncIdx];
			const FNiagaraFunctionSignature& Signature = HlslGenContext.Signatures[FuncIdx];

			//Init/Reset our per function hlsl template args.
			FunctionSymbol.StringValue = HlslGenContext.GetFunctionSignatureSymbol(Signature);
			PerFunctionParameterShaderCode.StringValue.Reset(); //Reset function parameters ready to rebuild when we iterate over the parameters.

			const FString* FunctionTemplate = FunctionTemplateMap.Find(Signature.Name);

			//Generate the function hlsl if we've not done so already for this signature.
			uint32 FuncHash = GetSignatureHash(Signature);
			FString& FunctionHlsl = FunctionHlslMap.FindOrAdd(FuncHash);
			if (FunctionHlsl.IsEmpty() && FunctionTemplate != nullptr)
			{
				if (Signature.VariadicInput())
				{
					int32 StartInput = Signature.VariadicInputStartIndex();
					for (int32 InputIdx = StartInput; InputIdx < Signature.Inputs.Num(); ++InputIdx)
					{
						const FNiagaraVariable& InputParam = Signature.Inputs[InputIdx];
						FParamAccessInfo& Access = ParametersAccessed.FindChecked(InputParam);

						FunctionParameterName = HlslGenContext.GetSanitizedSymbolName(InputParam.GetName().ToString());
						FunctionParameterType = HlslGenContext.GetStructHlslTypeName(InputParam.GetType());
						PerFunctionParameterShaderCode.StringValue += FString::Format(*PerParameterWriteCallTemplate, HlslTemplateArgs);


						bool bFirstParam = CurrParamIdx++ == 0;

						FunctionInputParameters.StringValue += FString::Printf(TEXT("%s%s %s %s"),
							bFirstParam ? *FirstParamPrefix : *ParamPrefix,
							*InputParamPrefix,
							*FunctionParameterType.StringValue, 
							*FunctionParameterName.StringValue);
					}
				}

				if (Signature.VariadicOutput())
				{
					int32 StartOutput = Signature.VariadicOutputStartIndex();
					for (int32 OutputIdx = StartOutput; OutputIdx < Signature.Outputs.Num(); ++OutputIdx)
					{
						const FNiagaraVariable& OutputParam = Signature.Outputs[OutputIdx];
						FParamAccessInfo& Access = ParametersAccessed.FindChecked(OutputParam);
						
						FunctionParameterName = HlslGenContext.GetSanitizedSymbolName(OutputParam.GetName().ToString());
						FunctionParameterType = HlslGenContext.GetStructHlslTypeName(OutputParam.GetType());
						PerFunctionParameterShaderCode.StringValue += FString::Format(*PerParameterReadCallTemplate, HlslTemplateArgs);


						bool bFirstParam = CurrParamIdx++ == 0;

						FunctionOutputParameters.StringValue += FString::Printf(TEXT("%s%s %s %s"),
							bFirstParam ? *FirstParamPrefix : *ParamPrefix,
							*OutputParamPrefix,
							*FunctionParameterType.StringValue, 
							*FunctionParameterName.StringValue);
					}
				}


				//Finally generate the final code for this function and add it to the final hlsl.
				OutHLSL += FString::Format(**FunctionTemplate, HlslTemplateArgs);
				OutHLSL += TEXT("\n");
			}
		}
	}
#endif //WITH_EDITORONLY_DATA
}

#undef LOCTEXT_NAMESPACE
