// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceDataChannelRead.h"

#include "NiagaraCompileHashVisitor.h"
#include "ShaderCompilerCore.h"
#include "RenderGraphUtils.h"
#include "SystemTextures.h"
#include "NiagaraModule.h"
#include "NiagaraShaderParametersBuilder.h"

#include "NiagaraCustomVersion.h"
#include "NiagaraCommon.h"
#include "NiagaraSystem.h"
#include "NiagaraWorldManager.h"

#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelHandler.h"
#include "NiagaraDataChannelManager.h"
#include "NiagaraDataChannelData.h"

#include "NiagaraEmitterInstanceImpl.h"

#include "NiagaraRenderer.h"
#include "NiagaraGPUSystemTick.h"

#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraGpuComputeDispatchInterface.h"

#include "Misc/StringOutputDevice.h"

#if WITH_EDITOR
#include "INiagaraEditorOnlyDataUtlities.h"
#include "Modules/ModuleManager.h"
#endif

//////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceDataChannelRead)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceDataChannelRead"

DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("NDIDataChannelRead Read"), STAT_NDIDataChannelRead_Read, STATGROUP_NiagaraDataChannels, EStatFlags::Verbose);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("NDIDataChannelRead Consume"), STAT_NDIDataChannelRead_Consume, STATGROUP_NiagaraDataChannels, EStatFlags::Verbose);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("NDIDataChannelRead Spawn"), STAT_NDIDataChannelRead_Spawn, STATGROUP_NiagaraDataChannels, EStatFlags::Verbose);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("NDIDataChannelRead Tick"), STAT_NDIDataChannelRead_Tick, STATGROUP_NiagaraDataChannels, EStatFlags::Verbose);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("NDIDataChannelRead PostTick"), STAT_NDIDataChannelRead_PostTick, STATGROUP_NiagaraDataChannels, EStatFlags::Verbose);

static int GNDCReadForceTG = -1;
static FAutoConsoleVariableRef CVarNDCReadForceTG(
	TEXT("fx.Niagara.DataChannels.ForceReadTickGroup"),
	GNDCReadForceTG,
	TEXT("When >= 0 this will force Niagara systems with NDC read DIs to tick in the given Tick Group."),
	ECVF_Default
);

static bool GNDCReadForcePrevFrame = false;
static FAutoConsoleVariableRef CVarNDCReadForcePrevFrame(
	TEXT("fx.Niagara.DataChannels.ForceReadPrevFrame"),
	GNDCReadForcePrevFrame,
	TEXT("When true this will force Niagara systems with NDC read DIs to read from the previous frame."),
	ECVF_Default
);

const static FName VarNameKey(TEXT("VarName"));
const static FName VarTypeKey(TEXT("VarType"));

//Decode a function specifier value back into a Niagara Variable.
FNiagaraVariableBase DecodeVariableFromSpecifiers(const FVMExternalFunctionBindingInfo& BindingInfo)
{
	const FVMFunctionSpecifier* NameSpec = BindingInfo.FindSpecifier(VarNameKey);
	const FVMFunctionSpecifier* TypeSpec = BindingInfo.FindSpecifier(VarTypeKey);
	if(NameSpec && TypeSpec)
	{
		FStringOutputDevice ErrorOut;
		const UScriptStruct* TypeStruct = FNiagaraTypeDefinition::StaticStruct();
		FNiagaraTypeDefinition TypeDef;
		if (TypeStruct->ImportText(*TypeSpec->Value.ToString(), &TypeDef, nullptr, PPF_None, &ErrorOut, TypeStruct->GetName(), true))
		{
			return FNiagaraVariableBase(TypeDef, NameSpec->Value);
		}

		UE_LOGF(LogNiagara, Error, "%ls", *ErrorOut);		
	}
	return FNiagaraVariableBase();
}

namespace NDIDataChannelReadLocal
{
	static const TCHAR* CommonShaderFile = TEXT("/Plugin/FX/Niagara/Private/DataChannel/NiagaraDataInterfaceDataChannelCommon.ush");
	static const TCHAR* TemplateShaderFile_Common = TEXT("/Plugin/FX/Niagara/Private/DataChannel/NiagaraDataInterfaceDataChannelTemplateCommon.ush");
	static const TCHAR* TemplateShaderFile_ReadCommon = TEXT("/Plugin/FX/Niagara/Private/DataChannel/NiagaraDataInterfaceDataChannelTemplateReadCommon.ush");
	static const TCHAR* TemplateShaderFile_Read = TEXT("/Plugin/FX/Niagara/Private/DataChannel/NiagaraDataInterfaceDataChannelTemplate_Read.ush");
	static const TCHAR* TemplateShaderFile_Consume = TEXT("/Plugin/FX/Niagara/Private/DataChannel/NiagaraDataInterfaceDataChannelTemplate_Consume.ush");

	static const FName NAME_SpawnDirect("SpawnDirect");
	static const FName NAME_SpawnConditional("SpawnConditional");
	static const FName NAME_Consume("Consume");
	static const FName NAME_Read("Read");
	static const FName NAME_Num("Num");
	static const FName NAME_ScaleSpawnCount("ScaleSpawnCount");

	//////////////////////////////////////////////////////////////////////////
	//Function definitions
	const FNiagaraFunctionSignature& GetFunctionSig_Num()
	{
		static FNiagaraFunctionSignature Sig;
		if(!Sig.IsValid())
		{
			Sig.Name = NAME_Num;
#if WITH_EDITORONLY_DATA
			Sig.Description = LOCTEXT("NumFunctionDescription", "Returns the current number of elements in the Data Channel being read.");
			NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)
#endif
			Sig.bMemberFunction = true;
			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()), TEXT("DataChannel interface")));
			Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num")));
		}		
		return Sig;
	}

	const FNiagaraFunctionSignature& GetFunctionSig_GetNDCSpawnData()
	{
		static FNiagaraFunctionSignature Sig;
		if (!Sig.IsValid())
		{
			Sig.Name = NDIDataChannelUtilities::GetNDCSpawnDataName;
#if WITH_EDITORONLY_DATA
			Sig.Description = LOCTEXT("GetNDCSpawnInfoFunctionDescription", "Returns data in relation the the NDC item that spawned this particle. Only valid for particles spawned from NDC and only on the frame in which they're spawned.");
			NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)
#endif		
			Sig.bMemberFunction = true;
			Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()), TEXT("DataChannel interface")));
			Sig.AddInputWithoutDefault(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraEmitterID::StaticStruct()), TEXT("Emitter ID")), LOCTEXT("EmitterIDDesc", "ID of the emitter we'd like to spawn into. This can be obtained from Engine.Emitter.ID."));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Spawned Particle Exec Index")), 0, LOCTEXT("GetNDCSpawnData_InExecIndexDesc","The execution index of the spawned particle."));
			Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NDC Index")), LOCTEXT("GetNDCSpawnData_OutNDCIndexDesc","Index of the NDC item that spawned this particle. Can be used to read the NDC data and initialize the spawning particle with data from the NDC."));
			Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NDC Spawn Index")), LOCTEXT("GetNDCSpawnData_OutNDCSpawnIndexDesc","The index of this particle in relation to all the particle spawned by the same NDC item. Similar to Exec Index but for particles spawned by the same NDC item."));
			Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NDC Spawn Count")), LOCTEXT("GetNDCSpawnData_OutNDCSpawnCountDesc","The number of particles spawned by the same NDC item."));
		}
		return Sig;
	}

	const FNiagaraFunctionSignature& GetFunctionSig_Read()
	{
		static FNiagaraFunctionSignature Sig;
		if(!Sig.IsValid())
		{
			Sig.Name = NAME_Read;
#if WITH_EDITORONLY_DATA
			Sig.Description = LOCTEXT("ReadFunctionDescription", "Reads Data Channel data at a specific index. Any values we read that are not in the Data Channel data are set to their default values. Returns success if there was a valid Data Channel entry to read from at the given index.");
			NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)
#endif
			Sig.bMemberFunction = true;
			Sig.bReadFunction = true;
			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()), TEXT("DataChannel interface")));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")), 0, LOCTEXT("ConsumeIndexInputDesc", "The index to read."));
			Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")), LOCTEXT("ConsumeSuccessOutputDesc", "True if all reads succeeded."));
			Sig.RequiredOutputs = IntCastChecked<int16>(Sig.Outputs.Num()); //The user defines what we read in the graph.			
		}
		return Sig;
	}

	const FNiagaraFunctionSignature& GetFunctionSig_Consume()
	{
		static FNiagaraFunctionSignature Sig;
		if(!Sig.IsValid())
		{
			Sig.Name = NAME_Consume;
#if WITH_EDITORONLY_DATA
			Sig.Description = LOCTEXT("ConsumeFunctionDescription", "Consumes an item from the Data Channel and reads the specified values. Any values we read that are not in the Data Channel data are set to their default values. Returns success if an entry was available to be consumed in the Data Channel.");
			NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)
#endif
			Sig.bMemberFunction = true;
			Sig.bReadFunction = true;
			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()), TEXT("DataChannel interface")));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Consume")), FNiagaraBool(true), LOCTEXT("ConsumeInputDesc", "True if this instance (particle/emitter etc) should consume data from the Data Channel in this call."));
			Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")), LOCTEXT("ConsumeSuccessOutputDesc", "True if all reads succeeded."));
			Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")), LOCTEXT("ConsumeIndexOutputDesc", "The index we actually read from. If reading failed this can be -1. This allows subsequent reads of the Data Channel at this index."));
			Sig.RequiredOutputs = IntCastChecked<int16>(Sig.Outputs.Num()); //The user defines what we read in the graph.			
		}
		return Sig;
	}

	enum class FunctionVersion_SpawnConditional : uint32
	{
		Initial = 0,
		EmitterIDParameter = 1,
	};

	const FNiagaraFunctionSignature& GetFunctionSig_SpawnConditional()
	{
		static FNiagaraFunctionSignature Sig;
		if(!Sig.IsValid())
		{
			Sig.Name = NAME_SpawnConditional;
#if WITH_EDITORONLY_DATA
			NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)
			Sig.Description = LOCTEXT("SpawnCustomFunctionDescription", "Will Spawn particles into the given Emitter between Min and Max counts for every element in the Data Channel.\n\
		Can take optional additional parameters as conditions on spawning. The data passed into the function will be compared against the contents of each Data Channel element.\n\
		For example, you could spawn only for NDC items that match a particular value of an enum.\n\
		For compound data types that contain multiple component floats or ints, comparisons are done on a per component basis.\n\
		For example if you add a Vector condition parameter it will be compared against each component of the corresponding Vector in the Data Channel.\n\
		Result = (Param.X == ChannelValue.X) && (Param.Y == ChannelValue.Y) && (Param.Z == ChannelValue.Z)");
			Sig.FunctionVersion = static_cast<uint32>(FunctionVersion_SpawnConditional::EmitterIDParameter);
#endif
			Sig.bMemberFunction = true;
			Sig.bRequiresExecPin = true;
			Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()), TEXT("DataChannel interface")));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable")), FNiagaraBool(true), LOCTEXT("SpawnEnableInputDesc", "Enable or disable this function call. If false, this call with have no effect."));
			Sig.AddInputWithoutDefault(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraEmitterID::StaticStruct()), TEXT("Emitter ID")), LOCTEXT("EmitterIDDesc", "ID of the emitter we'd like to spawn into. This can be obtained from Engine.Emitter.ID."));
			Sig.AddInput(FNiagaraVariable(StaticEnum<ENDIDataChannelSpawnMode>(), TEXT("Mode")), LOCTEXT("SpawnCondModeInputDesc", "Controls how this function will interact with other calls to spawn functions. Spawn counts for each NDC can be accumulated or overwritten."));
			Sig.AddInput(FNiagaraVariable(StaticEnum<ENiagaraConditionalOperator>(), TEXT("Operator")), LOCTEXT("SpawnCondOpInputDesc", "Compare the input against the Data Channel value:\n\n[Input] [Condition] [Data Channel Value]\n\nFor example:\nSpawn if [100] is [greater than] [Data Channel 'Height' Variable]"));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Min Spawn Count")), 1, LOCTEXT("MinSpawnCountInputDesc", "Minimum number of particles to spawn for each element in the Data Channel."));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Max Spawn Count")), 1, LOCTEXT("MaxSpawnCountInputDesc", "Maximum number of particles to spawn for each element in the Data Channel."));
			Sig.RequiredInputs = IntCastChecked<int16>(Sig.Inputs.Num());
		}
		return Sig;
	}

	const FNiagaraFunctionSignature& GetFunctionSig_SpawnDirect()
	{
		static FNiagaraFunctionSignature Sig;
		if (!Sig.IsValid())
		{
			Sig.Name = NAME_SpawnDirect;
#if WITH_EDITORONLY_DATA
			Sig.Description = LOCTEXT("SpawnDirectFunctionDescription", "Spawns particles into a given emitter for each entry in the Data Channel. Spawn count is determined directly from a value in the Data Channel. Additional per NDC item random scale and a clamp is available.");
			NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)
#endif
			Sig.FunctionSpecifiers.Add(VarNameKey);
			Sig.FunctionSpecifiers.Add(VarTypeKey);
			Sig.bMemberFunction = true;
			Sig.bRequiresExecPin = true;
			Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()), TEXT("DataChannel interface")));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable")), FNiagaraBool(true), LOCTEXT("SpawnEnableInputDesc", "Enable or disable this function call. If false, this call with have no effect."));
			Sig.AddInputWithoutDefault(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraEmitterID::StaticStruct()), TEXT("Emitter ID")), LOCTEXT("EmitterIDDesc", "ID of the emitter we'd like to spawn into. This can be obtained from Engine.Emitter.ID."));
			Sig.AddInput(FNiagaraVariable(StaticEnum<ENDIDataChannelSpawnMode>(), TEXT("Mode")), LOCTEXT("SpawnCondModeInputDesc", "Controls how this function will interact with other calls to spawn functions. Spawn counts for each NDC can be accumulated or overwritten."));		
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("RandomScaleMin")), 1.0f, LOCTEXT("SpawnDirectRandomScaleMinInputDesc", "Minimum value for an additional random scale applied to each NDC spawn count."));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("RandomScaleMax")), 1.0f, LOCTEXT("SpawnDirectRandomScaleMaxInputDesc", "Maximum value for an additional random scale applied to each NDC spawn count."));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ClampMin")), 0, LOCTEXT("SpawnDirectClampMinInputDesc", "Minimum Spawn Count to use after random scale is applied."));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ClampMax")), 1, LOCTEXT("SpawnDirectClampMaxInputDesc", "Maximum Spawn Count to use after random scale is applied. If < 0, No max clamp is applied."));
		}
		return Sig;
	}

	const FNiagaraFunctionSignature& GetFunctionSig_ScaleSpawnCount()
	{
		static FNiagaraFunctionSignature Sig;
		if (!Sig.IsValid())
		{
			Sig.Name = NAME_ScaleSpawnCount;
#if WITH_EDITORONLY_DATA
			Sig.Description = LOCTEXT("ScaleSpawnCountFunctionDescription", "Applies a scaling value for each NDC Item spawn count based on a variable in the NDC data. Optional additional random scale and clamp operations to the value read from each NDC entry.");
			NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)
#endif
			Sig.FunctionSpecifiers.Add(VarNameKey);
			Sig.FunctionSpecifiers.Add(VarTypeKey);
			Sig.bMemberFunction = true;
			Sig.bRequiresExecPin = true;
			Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()), TEXT("DataChannel interface")));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable")), FNiagaraBool(true), LOCTEXT("SpawnEnableInputDesc", "Enable or disable this function call. If false, this call with have no effect."));
			Sig.AddInputWithoutDefault(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraEmitterID::StaticStruct()), TEXT("Emitter ID")), LOCTEXT("EmitterIDDesc", "ID of the emitter we'd like to spawn into. This can be obtained from Engine.Emitter.ID."));
			Sig.AddInput(FNiagaraVariable(StaticEnum<ENDIDataChannelSpawnScaleMode>(), TEXT("Mode")), LOCTEXT("SpawnScaleModeInputDesc", "Control whether to override or combine this scale with previously set scales when calling this function multiple times."));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("RandomScaleMin")), 1.0f, LOCTEXT("ScaleSpawnCountRandomScaleMinInputDesc", "Minimum value for a random additional scale applied to each NDC spawn."));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("RandomScaleMax")), 1.0f, LOCTEXT("ScaleSpawnCountRandomScaleMaxInputDesc", "Maximum value for a random additional scale applied to each NDC spawn."));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("ClampMin")), 0.0f, LOCTEXT("ScaleSpawnCountClampMinInputDesc", "A minimum value for the scale value. If < 0 then no minimum is applied to the final scale."));
			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("ClampMax")), 1.0f, LOCTEXT("ScaleSpawnCountClampMaxInputDesc", "A maximum value for the scale value. If < 0 then no maximum is applied to the final scale."));
		}
		return Sig;
	}
	// Function definitions END
	//////////////////////////////////////////////////////////////////////////

	void BuildFunctionTemplateMap(TArray<FString>& OutCommonTemplateShaders, TMap<FName, FString>& OutMap)
	{
		LoadShaderSourceFile(TemplateShaderFile_Common, EShaderPlatform::SP_PCD3D_SM5, &OutCommonTemplateShaders.AddDefaulted_GetRef(), nullptr);
		LoadShaderSourceFile(TemplateShaderFile_ReadCommon, EShaderPlatform::SP_PCD3D_SM5, &OutCommonTemplateShaders.AddDefaulted_GetRef(), nullptr);

		LoadShaderSourceFile(TemplateShaderFile_Read, EShaderPlatform::SP_PCD3D_SM5, &OutMap.Add(NAME_Read), nullptr);
		LoadShaderSourceFile(TemplateShaderFile_Consume, EShaderPlatform::SP_PCD3D_SM5, &OutMap.Add(NAME_Consume), nullptr);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_SRV(Buffer<uint>, ParamOffsetTable)
		SHADER_PARAMETER(int32, ParameterOffsetTableIndex)
		SHADER_PARAMETER(int32, FloatStride)
		SHADER_PARAMETER(int32, Int32Stride)
		//TODO: Half Support | SHADER_PARAMETER(int32, HalfStride)

		SHADER_PARAMETER_SRV(Buffer<float>, DataFloat)
		SHADER_PARAMETER_SRV(Buffer<int>, DataInt32)
		//TODO: Half Support | SHADER_PARAMETER_SRV(Buffer<float>, DataHalf)
		SHADER_PARAMETER(int32, InstanceCountOffset)
		SHADER_PARAMETER(int32, ConsumeInstanceCountOffset)
		SHADER_PARAMETER(int32, BufferSize)

		SHADER_PARAMETER(int32, NDCElementCountAtSpawn)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int32>, NDCSpawnDataBuffer)
	END_SHADER_PARAMETER_STRUCT()
}

/** Render thread copy of current instance data. */
struct FNDIDataChannelReadInstanceData_RT
{
	//RT proxy for game channel data from which we're reading.
	FNiagaraDataChannelDataProxyPtr ChannelDataRTProxy = nullptr;

	bool bReadPrevFrame = false;

	/** Parameter mapping info for every function in every script used by this DI. */
	FVariadicParameterGPUScriptInfo ScriptParamInfo;

	/** Buffer containing packed data for all emitters NDC spawning data for use on the GPU. */
	TArray<int32> NDCSpawnData;

	/** Number of NDC elements at the point of spawning. More could have been added after this. */
	int32 NDCElementCountAtSpawn = 0;
};

//////////////////////////////////////////////////////////////////////////

FNDIDataChannelReadInstanceData::~FNDIDataChannelReadInstanceData()
{
	//We must have cleared this by now in Cleanup so that we can unregister if needed.
	check(DataChannelData == nullptr);
}

FNiagaraDataBuffer* FNDIDataChannelReadInstanceData::GetReadBufferCPU(bool bPrevFrame)const
{
	if (DataChannelData)
	{
		return DataChannelData->GetCPUData(bPrevFrame);
	}

	//TODO: Local reads.
// 	if (SourceInstData)
// 	{
// 		if (SourceInstData->Data && SourceInstData->Data->GetSimTarget() == ENiagaraSimTarget::CPUSim)
// 		{
// 			return SourceInstData->Data->GetDestinationData();
// 		}
// 	}

	return nullptr;
}

bool FNDIDataChannelReadInstanceData::Init(UNiagaraDataInterfaceDataChannelRead* Interface, FNiagaraSystemInstance* Instance)
{
	bool bSuccess = Tick(Interface, Instance, true);
	bSuccess &= PostTick(Interface, Instance);
	return bSuccess;
}

void FNDIDataChannelReadInstanceData::Cleanup(UNiagaraDataInterfaceDataChannelRead* Interface, FNiagaraSystemInstance* Instance)
{
	SetDataChannelData(nullptr, Interface);
}

void FNDIDataChannelReadInstanceData::SetDataChannelData(FNiagaraDataChannelDataPtr NewData, UNiagaraDataInterfaceDataChannelRead* Interface)
{
	if(DataChannelData == NewData)
	{
		return;
	}

	FNDIDataChannelCompiledData& CompiledData = Interface->GetCompiledData();
	//If this interface spawns into a GPU emitter then we need to inform the NDC data so that it can automatically upload all CPU data to the GPU.
	//Otherwise we get mis-matching data used for spawn and the subsequent reads on the GPU to init particles.
	//It can create some very confusing janky behavior.
	//Unfortunately we can't detect if this is directly spawning into a GPU emitter without some additional compiler/translator work.
	//For now we can make do with checking if it's used to spawn particles and if any GPU emitters read the NDC data.
	if(CompiledData.UsedByGPU() && CompiledData.SpawnsParticles())
	{
		if(DataChannelData)
		{
			DataChannelData->UnregisterGPUSpawningReader();
		}
		if(NewData)
		{
			NewData->RegisterGPUSpawningReader();
		}
	}

	DataChannelData = NewData;
}

bool FNDIDataChannelReadInstanceData::Tick(UNiagaraDataInterfaceDataChannelRead* Interface, FNiagaraSystemInstance* Instance, bool bIsInit)
{
	ConsumeIndex = 0;
	for(auto& EmitterInstDataPair : EmitterInstanceData)
	{
		EmitterInstDataPair.Value.Reset();
	}

	const FNDIDataChannelCompiledData& CompiledData = Interface->GetCompiledData();

	//Interface is never used so do not create any instance data.
	if (CompiledData.UsedByCPU() == false && CompiledData.UsedByGPU() == false)
	{
		//UE_LOGF(LogNiagara, Warning, "Data Channel Interface is being initialized but it is never used.\nSystem: %ls\nInterface: %ls", *Instance->GetSystem()->GetFullName(), *Interface->GetFullName());
		return true;
	}
	
	Owner = Instance;

	//In non test/shipping builds we gather and log and missing parameters that cause us to fail to find correct bindings.
	TArray<FNiagaraVariableBase> MissingParams;

	//TODO: Reads directly from a local writer DI.
	//For local readers, we find the source DI inside the same system and bind our functions to it's data layout.
// 		if (Interface->Scope == ENiagaraDataChannelScope::Local)
// 		{
// 			//Find our source DI and it's inst data if needed.
// 			//We have generated our compiled data already in PostCompile but we have to get the actual DI and inst data at runtime as they may not be the same as compile time.
// 			//Though the layout of the data should be the same etc.
// 			if (SourceDI.IsValid() == false && Interface->Source != NAME_None)
// 			{
// 				//Should not be able to bind to ourselves!
// 				check(Interface->GetFName() != Interface->Source);
// 
// 				UNiagaraDataInterface* FoundDI = nullptr;
// 				auto FindSourceDI = [&](const FNiagaraDataInterfaceUtilities::FDataInterfaceUsageContext& Context)
// 				{
// 					if (Context.Variable.GetName() == Interface->Source)
// 					{
// 						check(FoundDI == nullptr);
// 						FoundDI = Context.DataInterface;
// 						return false;
// 					}
// 					return true;
// 				};
// 
// 				if (UNiagaraSystem* System = Interface->GetTypedOuter<UNiagaraSystem>())
// 				{
// 					FNiagaraDataInterfaceUtilities::FDataInterfaceSearchFilter Filter;
// 					Filter.bIncludeInternal = true;
// 					FNiagaraDataInterfaceUtilities::ForEachDataInterface(System, Filter, FindSourceDI);
// 
// 
// 					if (UNiagaraDataInterfaceDataChannelWrite* SrcWriter = Cast<UNiagaraDataInterfaceDataChannelWrite>(FoundDI))
// 					{
// 						SourceDI = SrcWriter;
// 						SourceInstData = static_cast<FNDIDataChannelWriteInstanceData*>(Instance->FindDataInterfaceInstanceData(FoundDI));
// 
// 						uint32 NumFuncs = CompiledData.GetFunctionInfo().Num();
// 						FuncToDataSetBindingInfo.Reset(NumFuncs);
// 						for(const FNDIDataChannelFunctionInfo& FuncInfo : CompiledData.GetFunctionInfo())
// 						{
// 							FuncToDataSetBindingInfo.Add(FNDIDataChannelLayoutManager::Get().GetLayoutInfo(FuncInfo, SourceDI->GetCompiledData().DataLayout));
// 						}
// 					}
// 					else
// 					{
// 						UE_LOGF(LogNiagara, Warning, "Failed to find the Source DataChannel Writer %ls for DataChannel Reader %ls", *Interface->Source.ToString(), *Interface->GetFName().ToString());
// 						return false;
// 					}
// 				}				
// 			}
// 		}
// 		else
	{
		//For external readers, we find the DataChannel channel in the current world and bind our functions to it's data layout so we can read directly from the channel data.
		//check(Interface->Scope == ENiagaraDataChannelScope::World);

		//Grab our DataChannel channel and init the compiled data if needed.
		UNiagaraDataChannelHandler* DataChannelPtr = DataChannel.Get();
		if (DataChannelPtr == nullptr)
		{
			SetDataChannelData(nullptr, Interface);
			ChachedDataSetLayoutHash = INDEX_NONE;
			UWorld* World = Instance->GetWorld();
			if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World))
			{
				if (UNiagaraDataChannelHandler* NewChannelHandler = WorldMan->GetDataChannelManager().FindDataChannelHandler(Interface->Channel))
				{
					DataChannelPtr = NewChannelHandler;
					DataChannel = NewChannelHandler;
				}
				else
				{
					UE_LOGF(LogNiagara, Warning, "Failed to find or add Naigara DataChannel Channel: %ls", *Interface->Channel.GetName());
					return false;
				}
			}
		}

		//Grab the world DataChannel data if we're reading from there.	
		if (DataChannelPtr)
		{
			bool bNDCDataIsValid = DataChannelData && DataChannelData->IsLayoutValid(DataChannelPtr);
			if(!bIsInit && (bNDCDataIsValid == false || Interface->bUpdateSourceDataEveryTick))
			{
				FNDCAccessContextInst& AccessContext = DataChannelPtr->GetTransientAccessContext();
				if(AccessContext.IsValid() && AccessContext.GetScriptStruct() != FNDCAccessContextLegacy::StaticStruct())
				{
					if(!Interface->bAutoLinkToSpawningNDC)
					{
						//TODO: Copy over the interface custom AccessContext. If this proves slow we can possibly cache in per instance data.
						AccessContext = Interface->AccessContext;
					}
					
					FNDCAccessContextBase& ContextTyped = AccessContext.GetChecked<FNDCAccessContextBase>();
					ContextTyped.OwningComponent = Instance->GetAttachComponent();
					ContextTyped.bIsAutoLinkingSystem = Interface->bAutoLinkToSpawningNDC;
					SetDataChannelData(DataChannelPtr->FindData(AccessContext, ENiagaraResourceAccess::ReadOnly), Interface);//TODO: Maybe should have two paths, one for system instances and another for SceneComponents...
				}
				else
				{
					//Legacy NDC so just use old path.
					FNiagaraDataChannelSearchParameters SearchParams(Instance->GetAttachComponent());
					SetDataChannelData(DataChannelPtr->FindData(SearchParams, ENiagaraResourceAccess::ReadOnly), Interface);
				}
			}	

			if(const UNiagaraDataChannel* ChannelPtr = DataChannelPtr->GetDataChannel())
			{
				if(!bIsInit && ChannelPtr->ShouldEnforceTickGroupReadWriteOrder() && Interface->bReadCurrentFrame)
				{
					ETickingGroup CurrTG = DataChannelPtr->GetCurrentTickGroup();
					ETickingGroup MinTickGroup = Interface->CalculateTickGroup(nullptr);//We don't use the per instance data...
					if(CurrTG < MinTickGroup)
					{
						static UEnum* TGEnum = StaticEnum<ETickingGroup>();
						UE_LOGF(LogNiagara, Warning, "NDC Read DI is required to tick on or after %ls but is reading in %ls. This may cause us to have incorrectly ordered reads and writes to this NDC and thereform miss data."
						, *TGEnum->GetDisplayNameTextByValue((int32)MinTickGroup).ToString()
						, *TGEnum->GetDisplayNameTextByValue((int32)CurrTG).ToString());
					}
				}
			}

			const FNiagaraDataSetCompiledData& CPUSourceDataCompiledData = DataChannelPtr->GetDataChannel()->GetLayoutInfo()->GetDataSetCompiledData();
			const FNiagaraDataSetCompiledData& GPUSourceDataCompiledData = DataChannelPtr->GetDataChannel()->GetLayoutInfo()->GetDataSetCompiledDataGPU();
			check(CPUSourceDataCompiledData.GetLayoutHash() && CPUSourceDataCompiledData.GetLayoutHash() == GPUSourceDataCompiledData.GetLayoutHash());			
			uint64 SourceDataLayoutHash = CPUSourceDataCompiledData.GetLayoutHash();
			bool bChanged = SourceDataLayoutHash != ChachedDataSetLayoutHash;

			//If our CPU or GPU source data has changed then regenerate our binding info.
			//TODO: Multi-source buffer support. 
			//TODO: Variable input layout support. i.e. allow source systems to publish their particle buffers without the need for a separate write.
			if (bChanged)
			{
				ChachedDataSetLayoutHash = SourceDataLayoutHash;

				//We can likely be more targeted here.
				//Could probably only update the RT when the GPU data changes and only update the bindings if the function hashes change etc.
				bUpdateFunctionBindingRTData = CompiledData.UsedByGPU();
				int32 NumFuncs = CompiledData.GetFunctionInfo().Num();
				FuncToDataSetBindingInfo.SetNum(NumFuncs);
				//FuncToDataSetLayoutKeys.SetNumZeroed(NumFuncs);
				for (int32 BindingIdx = 0; BindingIdx < NumFuncs; ++BindingIdx)
				{
					const FNDIDataChannelFunctionInfo& FuncInfo = CompiledData.GetFunctionInfo()[BindingIdx];

					FNDIDataChannel_FuncToDataSetBindingPtr& BindingPtr = FuncToDataSetBindingInfo[BindingIdx];
					BindingPtr = FNDIDataChannelLayoutManager::Get().GetLayoutInfo(FuncInfo, CPUSourceDataCompiledData, MissingParams);
				}
			}
		}
	}

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	if(MissingParams.Num() > 0)
	{
		FString MissingParamsString;
		for (FNiagaraVariableBase& MissingParam : MissingParams)
		{
			MissingParamsString += FString::Printf(TEXT("%s %s\n"), *MissingParam.GetType().GetName(), *MissingParam.GetName().ToString());
		}

		UE_LOGF(LogNiagara, Warning, "Niagara Data Channel Reader Interface is trying to read parameters that do not exist in this channel.\nIt's likely that the Data Channel Definition has been changed and this system needs to be updated.\nData Channel: %ls\nSystem: %ls\nComponent:%ls\nMissing Parameters:\n%ls\n"
			, *DataChannel->GetDataChannel()->GetName()
			, *Instance->GetSystem()->GetPathName()
			, *Instance->GetAttachComponent()->GetPathName()
			, *MissingParamsString);		
	}	
#endif

	if (!DataChannel.IsValid() /*&& !SourceDI.IsValid()*/)//TODO: Local reads
	{
		UE_LOGF(LogNiagara, Warning, "Niagara Data Channel Reader Interface could not find a valid Data Channel.\nData Channel: %ls\nSystem: %ls\nComponent:%ls\n"
			, Interface->Channel ? *Interface->Channel->GetName() : TEXT("None")
			, *Instance->GetSystem()->GetPathName()
			, *Instance->GetAttachComponent()->GetPathName());
		return false;
	}

	//Verify we have valid binding info. If not, we have to bail as we cannot properly parse the vm bytecode.
	if (FuncToDataSetBindingInfo.Num() != Interface->GetCompiledData().GetFunctionInfo().Num())
	{
		return false;
	}

	for (const FNDIDataChannel_FuncToDataSetBindingPtr& FuncBinding : FuncToDataSetBindingInfo)
	{
		if (FuncBinding.IsValid() == false || FuncBinding->IsValid() == false)
		{
			return false;
		}
	}

	return true;
}

bool FNDIDataChannelReadInstanceData::PostTick(UNiagaraDataInterfaceDataChannelRead* Interface, FNiagaraSystemInstance* Instance)
{
	return true;
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceDataChannelRead::UNiagaraDataInterfaceDataChannelRead(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxy_DataChannelRead());
}

void UNiagaraDataInterfaceDataChannelRead::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) && INiagaraModule::DataChannelsEnabled())
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowNotUserVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceDataChannelRead::BeginDestroy()
{
	Super::BeginDestroy();
}

void UNiagaraDataInterfaceDataChannelRead::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);
	const int32 NiagaraVersion = Ar.CustomVer(FNiagaraCustomVersion::GUID);

	//Before we serialize in the properties we will restore any old default values from previous versions.
	if(NiagaraVersion < FNiagaraCustomVersion::NDCSpawnGroupOverrideDisabledByDefault)
	{
		bOverrideSpawnGroupToDataChannelIndex = true;
	}

	Super::Serialize(Ar);
}

void UNiagaraDataInterfaceDataChannelRead::PostLoad()
{
	Super::PostLoad();
	
	if(Channel)
	{
		Channel->ConditionalPostLoad();
	}

	InitAccessContext();
}

#if WITH_EDITOR
void UNiagaraDataInterfaceDataChannelRead::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceDataChannelRead, Channel) ||
		PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceDataChannelRead, bAutoLinkToSpawningNDC))
	{
		InitAccessContext();
	}
}
#endif

void UNiagaraDataInterfaceDataChannelRead::InitAccessContext()
{
	UNiagaraDataChannel* NDC = Channel ? Channel->Get() : nullptr;
	if (NDC && !bAutoLinkToSpawningNDC)
	{
		const UScriptStruct* CtxStruct = NDC->GetAccessContextType().Get();
		//Special case legacy type. We avoid creating an access context for this type here and just use the old search params path.
		if (CtxStruct == FNDCAccessContextLegacy::StaticStruct())
		{
			AccessContext.Reset();
		}
		else if (AccessContext.IsValid() == false || CtxStruct != AccessContext.GetScriptStruct())
		{
			AccessContext.Init(NDC->GetAccessContextType());
		}
	}
	else
	{
		AccessContext.Reset();
	}
}

bool UNiagaraDataInterfaceDataChannelRead::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIDataChannelReadInstanceData* InstanceData = new (PerInstanceData) FNDIDataChannelReadInstanceData;

	if (INiagaraModule::DataChannelsEnabled() == false)
	{
		return false;
	}

	if (InstanceData->Init(this, SystemInstance) == false)
	{
		return false;
	}

	return true;
}

void UNiagaraDataInterfaceDataChannelRead::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIDataChannelReadInstanceData* InstanceData = static_cast<FNDIDataChannelReadInstanceData*>(PerInstanceData);
	InstanceData->Cleanup(this, SystemInstance);
	InstanceData->~FNDIDataChannelReadInstanceData();

	ENQUEUE_RENDER_COMMAND(RemoveProxy)
	(
		[RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxy_DataChannelRead>(), InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		}
	);
}

int32 UNiagaraDataInterfaceDataChannelRead::PerInstanceDataSize() const
{
	return sizeof(FNDIDataChannelReadInstanceData);
}

bool UNiagaraDataInterfaceDataChannelRead::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_NDIDataChannelRead_Tick);
	check(SystemInstance);

	if (INiagaraModule::DataChannelsEnabled() == false)
	{
		return true;
	}
	
	FNDIDataChannelReadInstanceData* InstanceData = static_cast<FNDIDataChannelReadInstanceData*>(PerInstanceData);
	if (!InstanceData)
	{
		return true;
	}

	if (InstanceData->Tick(this, SystemInstance) == false)
	{
		return true;
	}

	return false;
}

bool UNiagaraDataInterfaceDataChannelRead::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_NDIDataChannelRead_PostTick);
	check(SystemInstance);

	if (INiagaraModule::DataChannelsEnabled() == false)
	{
		return true;
	}

	FNDIDataChannelReadInstanceData* InstanceData = static_cast<FNDIDataChannelReadInstanceData*>(PerInstanceData);
	if (!InstanceData)
	{
		return true;
	}

	if (InstanceData->PostTick(this, SystemInstance) == false)
	{
		return true;
	}

	return false;
}

void UNiagaraDataInterfaceDataChannelRead::PostStageTick(FNDICpuPostStageContext& Context)
{
	FNDIDataChannelReadInstanceData* InstanceData = Context.GetPerInstanceData<FNDIDataChannelReadInstanceData>();

	check(InstanceData);
	check(Context.Usage == ENiagaraScriptUsage::EmitterUpdateScript || Context.Usage == ENiagaraScriptUsage::SystemUpdateScript);

	InstanceData->NDCElementCountAtSpawn = INDEX_NONE;
	
	for (auto& EmitterInstanceDataPair : InstanceData->EmitterInstanceData)
	{
		FNDIDataChannelRead_EmitterInstanceData& EmitterInstData = EmitterInstanceDataPair.Value;
		FNiagaraEmitterInstance* TargetEmitter = EmitterInstanceDataPair.Key;
		if(TargetEmitter == nullptr || EmitterInstData.NDCSpawnCounts.Num() == 0)
		{
			EmitterInstData.Reset();
			continue;
		}
		
		if (InstanceData->NDCElementCountAtSpawn == INDEX_NONE)
		{
			InstanceData->NDCElementCountAtSpawn = EmitterInstData.NDCSpawnCounts.Num();
		}
		//All emitters with a non-zero spawn count should agree about how many NDC entries there were.
		check(EmitterInstData.NDCSpawnCounts.Num() == InstanceData->NDCElementCountAtSpawn);

		TArray<uint32> PerNDCSpawnCounts;
		PerNDCSpawnCounts.Reserve(EmitterInstData.NDCSpawnCounts.Num());
		for(FNDIDataChannelRead_EmitterSpawnInfo& Info : EmitterInstData.NDCSpawnCounts)
		{
			PerNDCSpawnCounts.Add(Info.Get());
		}

		EmitterInstData.Reset();

		//-TODO:Stateless:
		if (FNiagaraEmitterInstanceImpl* StatefulEmitter = TargetEmitter->AsStateful())
		{
			if (bOverrideSpawnGroupToDataChannelIndex)
			{
				//If we're overriding the spawn group then we must submit one SpawnInfo per NDC entry.
				FNiagaraSpawnInfo NewSpawnInfo(0, 0.0f, 0.0f, 0);
				for (int32 i = 0; i < PerNDCSpawnCounts.Num(); ++i)
				{
					uint32 SpawnCount = PerNDCSpawnCounts[i];
					if (SpawnCount > 0)
					{
						NewSpawnInfo.Count = SpawnCount;
						NewSpawnInfo.SpawnGroup = i;
						StatefulEmitter->GetSpawnInfo().Emplace(NewSpawnInfo);
					}
				}
			}
			else
			{
				//No need for indirection table but we're not overriding the spawn group either so still push a single combined spawn info.
				FNiagaraSpawnInfo NewSpawnInfo(0, 0.0f, 0.0f, 0);
				for (int32 i = 0; i < PerNDCSpawnCounts.Num(); ++i)
				{
					NewSpawnInfo.Count += PerNDCSpawnCounts[i];
				}
				StatefulEmitter->GetSpawnInfo().Emplace(NewSpawnInfo);
			}

			if (CompiledData.NeedSpawnDataTable())
			{
				//Build an indirection table that allows us to map from ExecIndex back to the NDCIndex that generated it.
				//The indirection table is arranged in power of two buckets.
				//An NDC that spawns say 37 particles would add an entry to the 32, 4 and 1 buckets.
				//This allows us to spawn any number of particles from each NDC and only have a max of 16 indirection table entries.
				//Vs the naive per particle approach of 1 entry per particle.
				//Buckets are processed in descending size order.

				//TODO: It should be possible to write this from the GPU too as long as we allocate fixed size buckets.
				int32* SpawnDataBuckets = EmitterInstData.NDCSpawnData.NDCSpawnDataBuckets;

				TArray<int32>& NDCSpawnData = EmitterInstData.NDCSpawnData.NDCSpawnData;

				//Start of the buffer is the per NDC spawn counts.
				uint32 TotalNDCSpawnDataSize = PerNDCSpawnCounts.Num();

				for (int32 i = 0; i < PerNDCSpawnCounts.Num(); ++i)
				{
					uint32 Count = PerNDCSpawnCounts[i];

					//First section is the per NDC counts.
					NDCSpawnData.Add(Count);

					for (uint32 Bucket = 0; Bucket < 16; ++Bucket)
					{
						uint32 BucketSize = (1 << 15) >> Bucket;
						uint32 Mask = (0xFFFF >> (Bucket + 1));
						uint32 CountMasked = Count & ~Mask;
						Count &= Mask;
						uint32 NumBucketEntries = CountMasked / BucketSize;
						SpawnDataBuckets[Bucket] += NumBucketEntries;
						TotalNDCSpawnDataSize += NumBucketEntries;
					}
				}

				//Second part is the counts decomposed into power of two buckets that allows us to map ExecIndex at runtime to an NDCIndex entry in this table.
				for (int32 Bucket = 0; Bucket < 16; ++Bucket)
				{
					uint32 BucketSize = (1 << 15) >> Bucket;
					uint32 StartSize = NDCSpawnData.Num();
					for (int32 i = 0; i < PerNDCSpawnCounts.Num(); ++i)
					{
						uint32& Count = PerNDCSpawnCounts[i];
						while (Count >= BucketSize)
						{
							Count -= BucketSize;
							NDCSpawnData.Add(i);
						}
					}

					uint32 EndSize = NDCSpawnData.Num();
					check(EndSize - StartSize == SpawnDataBuckets[Bucket]);
				}
			}
		}
	}
}

void UNiagaraDataInterfaceDataChannelRead::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	const FNDIDataChannelReadInstanceData& SourceData = *reinterpret_cast<const FNDIDataChannelReadInstanceData*>(PerInstanceData);
	FNDIDataChannelReadInstanceData_RT* TargetData = new(DataForRenderThread) FNDIDataChannelReadInstanceData_RT();

	//Always update the dataset, this may change without triggering a full update if it's layout is the same.
	TargetData->ChannelDataRTProxy = SourceData.DataChannelData ? SourceData.DataChannelData->GetRTProxy() : nullptr;

	bool bReadPrevFrame = bReadCurrentFrame == false || GNDCReadForcePrevFrame;
	TargetData->bReadPrevFrame = bReadPrevFrame;
	TargetData->NDCElementCountAtSpawn = SourceData.NDCElementCountAtSpawn;

	if (SourceData.bUpdateFunctionBindingRTData && INiagaraModule::DataChannelsEnabled())
	{
		SourceData.bUpdateFunctionBindingRTData = false;
		
		const FNiagaraDataSetCompiledData& GPUCompiledData = SourceData.DataChannel->GetDataChannel()->GetLayoutInfo()->GetDataSetCompiledDataGPU();
		TargetData->ScriptParamInfo.Init(CompiledData, GPUCompiledData);
	}

	//NDC SpawnData Handling. TODO: Refactor into function
	{
		//Always need to fill in the NDCSpawnData array as it will change every frame and be pushed into an RDG buffer.

		//New buffer is every emitter continuous NDCSpawnDataArray. We need to store an offset that we pass in as a uniform.
		//The buckets come first, then the per NDC SpawnCounts, Then the bucket back ptrs.

		//Do one pass to calculate size.
		auto GetEmitterNDCSpawnDataSize = [](const FNDIDataChannelRead_EmitterInstanceData& EmitterInstData)
		{
			return 16 + EmitterInstData.NDCSpawnData.NDCSpawnData.Num();
		};

		uint32 NumEmitters = SourceData.EmitterInstanceData.Num();
		uint32 TotalPacckedNDCSpawnDataSize = 0;
		int32 MaxEmitterIndex = 0;
		for (const TPair<FNiagaraEmitterInstance*, FNDIDataChannelRead_EmitterInstanceData>& EmitterInstDataPair : SourceData.EmitterInstanceData)
		{
			if (const FNiagaraEmitterInstance* EmitterInst = EmitterInstDataPair.Key)
			{
				const FNDIDataChannelRead_EmitterInstanceData& EmitterInstData = EmitterInstDataPair.Value;

				TotalPacckedNDCSpawnDataSize += GetEmitterNDCSpawnDataSize(EmitterInstData);
				FNiagaraEmitterID ID = EmitterInst->GetEmitterID();
				MaxEmitterIndex = FMath::Max(MaxEmitterIndex, ID.ID);
			}
		}

		//First section of the NDCSpawnDataBuffer is an offset into the buffer for each emitter.
		TotalPacckedNDCSpawnDataSize += (MaxEmitterIndex + 1);

		TargetData->NDCSpawnData.Reset(TotalPacckedNDCSpawnDataSize);
		TArray<int32>& TargetNDCSpawnData = TargetData->NDCSpawnData;

		//First grab space for the per emitter offset table. We'll fill this in as we go.
		TargetNDCSpawnData.AddZeroed(MaxEmitterIndex + 1);

		uint32 CurrentSpawnDataOffset = TargetNDCSpawnData.Num();

		for (const TPair<FNiagaraEmitterInstance*, FNDIDataChannelRead_EmitterInstanceData>& EmitterInstDataPair : SourceData.EmitterInstanceData)
		{
			if (const FNiagaraEmitterInstance* EmitterInst = EmitterInstDataPair.Key)
			{
				const FNDIDataChannelRead_EmitterInstanceData& EmitterInstData = EmitterInstDataPair.Value;
				if (EmitterInst->GetGPUContext() == nullptr)
				{
					continue;
				}


				uint32 EmitterNDCSpawnDataSize = GetEmitterNDCSpawnDataSize(EmitterInstData);

				//First fill in the current offset for this emitter.
				FNiagaraEmitterID EmitterID = EmitterInst->GetEmitterID();
				TargetNDCSpawnData[EmitterID.ID] = CurrentSpawnDataOffset;

				CurrentSpawnDataOffset += EmitterNDCSpawnDataSize;

				//Next fill in bucket counts
				for (int32 i = 0; i < 16; ++i)
				{
					TargetNDCSpawnData.Add(EmitterInstData.NDCSpawnData.NDCSpawnDataBuckets[i]);
				}
				TargetNDCSpawnData.Append(EmitterInstData.NDCSpawnData.NDCSpawnData);
			}
		}
	}
}

void UNiagaraDataInterfaceDataChannelRead::GetEmitterDependencies(UNiagaraSystem* Asset, TArray<FVersionedNiagaraEmitter>& Dependencies)const
{
	//TODO: Local support.
	// When reading directly from a local writer DI we modify the tick order so the readers come after writers.
	//Find our source DI and add a dependency for any emitter that writes to it.
// 	if (UNiagaraDataInterfaceDataChannelWrite* SourceDI = FindSourceDI())
// 	{
// 		auto HandleVMFunc = [&](const UNiagaraScript* Script, const FVMExternalFunctionBindingInfo& BindingInfo)
// 		{
// 			if (UNiagaraEmitter* Emitter = Script->GetTypedOuter<UNiagaraEmitter>())
// 			{
// 				if (Script->IsParticleScript())
// 				{
// 					for (const FNiagaraEmitterHandle& EmitterHandle : Asset->GetEmitterHandles())
// 					{
// 						FVersionedNiagaraEmitter EmitterInstance = EmitterHandle.GetInstance();
// 						if (EmitterInstance.Emitter.Get() == Emitter)
// 						{
// 							Dependencies.AddUnique(EmitterInstance);
// 						}
// 					}
// 				}
// 			}
// 			return true;
// 		};
// 		if (SourceDI)
// 		{
// 			FNiagaraDataInterfaceUtilities::ForEachVMFunctionEquals(SourceDI, Asset, HandleVMFunc);
// 		}
// 	}
}

bool UNiagaraDataInterfaceDataChannelRead::HasTickGroupPrereqs() const
{
	if (GNDCReadForceTG >= 0 && GNDCReadForceTG < (int32)ETickingGroup::TG_MAX)
	{
		return true;
	}
	else if (Channel && Channel->Get())
	{
		return Channel->Get()->ShouldEnforceTickGroupReadWriteOrder();
	}
	return false;
}

ETickingGroup UNiagaraDataInterfaceDataChannelRead::CalculateTickGroup(const void* PerInstanceData) const
{
	if(GNDCReadForceTG >= 0 && GNDCReadForceTG < (int32)ETickingGroup::TG_MAX)
	{
		return (ETickingGroup)GNDCReadForceTG;
	}
	else if(Channel && Channel->Get() && Channel->Get()->ShouldEnforceTickGroupReadWriteOrder())
	{
		return (ETickingGroup)((int32)Channel->Get()->GetFinalWriteTickGroup() + 1);
	}

	return NiagaraFirstTickGroup; 
}

#if WITH_EDITORONLY_DATA

void UNiagaraDataInterfaceDataChannelRead::PostCompile(const UNiagaraSystem& OwningSystem)
{
	CompiledData.Init(&OwningSystem, this);
}

#endif


#if WITH_EDITOR	

void UNiagaraDataInterfaceDataChannelRead::GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	Super::GetFeedback(InAsset, InComponent, OutErrors, OutWarnings, OutInfo);

	INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");
	const INiagaraEditorOnlyDataUtilities& EditorOnlyDataUtilities = NiagaraModule.GetEditorOnlyDataUtilities();
	UNiagaraDataInterface* RuntimeInstanceOfThis = InAsset && EditorOnlyDataUtilities.IsEditorDataInterfaceInstance(this)
		? EditorOnlyDataUtilities.GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(*InAsset, *this)
		: this;

	UNiagaraDataInterfaceDataChannelRead* RuntimeReadDI = Cast<UNiagaraDataInterfaceDataChannelRead>(RuntimeInstanceOfThis);
	
	if(!RuntimeReadDI)
	{
		return;	
	}

	if (RuntimeReadDI->Channel == nullptr)
	{
		OutErrors.Emplace(LOCTEXT("DataChannelMissingFmt", "Data Channel Interface has no valid Data Channel."),
			LOCTEXT("DataChannelMissingErrorSummaryFmt", "Missing Data Channel."),
			FNiagaraDataInterfaceFix());

		return;
	}

	//if(Scope == ENiagaraDataChannelScope::World)
	{
		if (const UNiagaraDataChannel* DataChannel = RuntimeReadDI->Channel->Get())
		{
			//Ensure the Data Channel contains all the parameters this function is requesting.
			TConstArrayView<FNiagaraDataChannelVariable> ChannelVars = DataChannel->GetVariables();
			for (const FNDIDataChannelFunctionInfo& FuncInfo : RuntimeReadDI->GetCompiledData().GetFunctionInfo())
			{
				TArray<FNiagaraVariableBase> MissingParams;

				auto VerifyChannelContainsParams = [&](const TArray<FNiagaraVariableBase>& Parameters)
				{
					for (const FNiagaraVariableBase& FuncParam : Parameters)
					{
						bool bParamFound = false;
						for (const FNiagaraDataChannelVariable& ChannelVar : ChannelVars)
						{
							//We have to convert each channel var to SWC for comparison with the function variables as there is no reliable way to go back from the SWC function var to the originating LWC var.
							FNiagaraVariable SWCVar(ChannelVar);
							if(ChannelVar.GetType().IsEnum() == false)
							{
								UScriptStruct* ChannelSWCStruct = FNiagaraTypeHelper::GetSWCStruct(ChannelVar.GetType().GetScriptStruct());
								if (ChannelSWCStruct)
								{
									FNiagaraTypeDefinition SWCType(ChannelSWCStruct, FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Deny);
									SWCVar = FNiagaraVariable(SWCType, ChannelVar.GetName());
								}
							}
						
							if (SWCVar == FuncParam)
							{
								bParamFound = true;
								break;
							}
						}

						if (bParamFound == false)
						{
							MissingParams.Add(FuncParam);
						}
					}
				};
				VerifyChannelContainsParams(FuncInfo.Inputs);
				VerifyChannelContainsParams(FuncInfo.Outputs);

				if(MissingParams.Num() > 0)
				{
					FTextBuilder Builder;
					Builder.AppendLineFormat(LOCTEXT("FuncParamMissingFromDataChannelErrorFmt", "Accessing variables that do not exist in Data Channel {0}."), FText::FromString(Channel.GetName()));
					for(FNiagaraVariableBase& Param : MissingParams)
					{
						Builder.AppendLineFormat(LOCTEXT("FuncParamMissingFromDataChannelErrorLineFmt", "{0} {1}"), Param.GetType().GetNameText(), FText::FromName(Param.GetName()));
					}					
					
					OutErrors.Emplace(Builder.ToText(), LOCTEXT("FuncParamMissingFromDataChannelErrorSummaryFmt", "Data Channel DI function is accessing invalid parameters."), FNiagaraDataInterfaceFix());
				}
			}
		}
		else
		{
			OutErrors.Emplace(FText::Format(LOCTEXT("DataChannelDoesNotExistErrorFmt", "Data Channel {0} does not exist. It may have been deleted."), FText::FromString(Channel.GetName())),
				LOCTEXT("DataChannelDoesNotExistErrorSummaryFmt", "Data Channel DI is accesssinga a Data Channel that doesn't exist."),
				FNiagaraDataInterfaceFix());
		}
	}
	//TODO: Local support.
// 	else
// 	{
// 		check(Scope == ENiagaraDataChannelScope::Local);
// 		UNiagaraDataInterfaceDataChannelWrite* SourceDI = FindSourceDI();
// 		if (SourceDI == nullptr)
// 		{
// 			OutErrors.Emplace(FText::Format(LOCTEXT("DataChannelCouldNotFindSourceErrorFmt", "Could not find local Data Channel Writer DI with name {0}."), FText::FromName(Source)),
// 				LOCTEXT("DataChannelCouldNotFindSourceErrorSummaryFmt", "Could not find source Data Channel Writer DI for local Data Channel Reader DI."),
// 				FNiagaraDataInterfaceFix());
// 		}
// 	}
}

void UNiagaraDataInterfaceDataChannelRead::ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors)
{
	Super::ValidateFunction(Function, OutValidationErrors);

	//It would be great to be able to validate the parameters on the function calls here but this is only called on the DI CDO. We don't have the context of which Data Channel we'll be accessing.
	//The translator should have all the required data to use the actual DIs when validating functions. We just need to do some wrangling to pull it from the pre compiled data correctly.
	//This would probably also allow us to actually call hlsl generation functions on the actual DIs rather than their CDOs. Which would allow for a bunch of better optimized code gen for things like fluids.
	//TODO!!!
}

#endif


UNiagaraDataInterfaceDataChannelWrite* UNiagaraDataInterfaceDataChannelRead::FindSourceDI()const
{
	//TODO: Local Read/Write?
// 	if (Scope == ENiagaraDataChannelScope::Local)
// 	{
// 		UNiagaraDataInterface* FoundDI = nullptr;
// 		auto FindSourceDI = [Interface=this, &FoundDI](const FNiagaraDataInterfaceUtilities::FDataInterfaceUsageContext& Context)
// 		{
// 			if (Context.Variable.GetName() == Interface->Source)
// 			{
// 				check(FoundDI == nullptr);				
// 				FoundDI = Context.DataInterface;
// 				return false;
// 			}
// 			return true;
// 		};
// 
// 		if (UNiagaraSystem* System = GetTypedOuter<UNiagaraSystem>())
// 		{
// 			FNiagaraDataInterfaceUtilities::FDataInterfaceSearchFilter Filter;
// 			Filter.bIncludeInternal = true;
// 			FNiagaraDataInterfaceUtilities::ForEachDataInterface(System, Filter, FindSourceDI);
// 
// 			if (UNiagaraDataInterfaceDataChannelWrite* SrcWriter = Cast<UNiagaraDataInterfaceDataChannelWrite>(FoundDI))
// 			{
// 				return SrcWriter;
// 			}
// 		}
// 	}
	return nullptr;
}

bool UNiagaraDataInterfaceDataChannelRead::Equals(const UNiagaraDataInterface* Other)const
{
	//UE_LOGF(LogNiagara, Warning, "Checking Equality DCRead DI %ls == %ls", *GetPathName(), *Other->GetPathName());
	if (const UNiagaraDataInterfaceDataChannelRead* OtherTyped = CastChecked<UNiagaraDataInterfaceDataChannelRead>(Other))
	{
		if (Super::Equals(Other) &&
			//Scope == OtherTyped->Scope &&
			//Source == OtherTyped->Source &&
			Channel == OtherTyped->Channel &&
			AccessContext == OtherTyped->AccessContext &&
			bAutoLinkToSpawningNDC == OtherTyped->bAutoLinkToSpawningNDC &&
			bReadCurrentFrame == OtherTyped->bReadCurrentFrame &&
			bUpdateSourceDataEveryTick == OtherTyped->bUpdateSourceDataEveryTick &&
			bOverrideSpawnGroupToDataChannelIndex == OtherTyped->bOverrideSpawnGroupToDataChannelIndex &&
			bOnlySpawnOnceOnSubticks == OtherTyped->bOnlySpawnOnceOnSubticks)
		{
			return true;
		}
	}

	return false;
}

bool UNiagaraDataInterfaceDataChannelRead::CopyToInternal(UNiagaraDataInterface* Destination)const
{
	//UE_LOGF(LogNiagara, Warning, "Coping DCRead DI %ls --> %ls", *GetPathName(), *Destination->GetPathName());
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	if (UNiagaraDataInterfaceDataChannelRead* DestTyped = CastChecked<UNiagaraDataInterfaceDataChannelRead>(Destination))
	{
		//DestTyped->Scope = Scope;
		//DestTyped->Source = Source;
		DestTyped->Channel = Channel;
		DestTyped->CompiledData = CompiledData;
		DestTyped->AccessContext = AccessContext;
		DestTyped->bAutoLinkToSpawningNDC = bAutoLinkToSpawningNDC;
		DestTyped->bReadCurrentFrame = bReadCurrentFrame;
		DestTyped->bUpdateSourceDataEveryTick = bUpdateSourceDataEveryTick;
		DestTyped->bOverrideSpawnGroupToDataChannelIndex = bOverrideSpawnGroupToDataChannelIndex;
		DestTyped->bOnlySpawnOnceOnSubticks = bOnlySpawnOnceOnSubticks;
		return true;
	}

	return false;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceDataChannelRead::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	/////
	// NOTE: *any* changes to function inputs or outputs needs to be included in FReadNDCModel::GenerateNewModuleContent()
	/////
	OutFunctions.Add(NDIDataChannelReadLocal::GetFunctionSig_Num());
	OutFunctions.Add(NDIDataChannelReadLocal::GetFunctionSig_GetNDCSpawnData());
	OutFunctions.Add(NDIDataChannelReadLocal::GetFunctionSig_Read());
	OutFunctions.Add(NDIDataChannelReadLocal::GetFunctionSig_Consume());
	OutFunctions.Add(NDIDataChannelReadLocal::GetFunctionSig_SpawnConditional());
	OutFunctions.Add(NDIDataChannelReadLocal::GetFunctionSig_SpawnDirect());
	OutFunctions.Add(NDIDataChannelReadLocal::GetFunctionSig_ScaleSpawnCount());
}
#endif

void UNiagaraDataInterfaceDataChannelRead::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	if (BindingInfo.Name == NDIDataChannelReadLocal::NAME_Num)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::Num);
	}
	else if (BindingInfo.Name == NDIDataChannelUtilities::GetNDCSpawnDataName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::GetNDCSpawnData);
	}
	else if (BindingInfo.Name == NDIDataChannelReadLocal::NAME_SpawnDirect)
	{
		//Find the appropriate variable from the function binding and determine it's type to bind to the correct impl for ScaleSpawnCount and provide the variable index.
		FNiagaraVariableBase FuncSpecVariable = DecodeVariableFromSpecifiers(BindingInfo);

		//TODO: We are currently storing these as FNames and encoding/decoding the variable.
		//This is slow, clunky, brittle and generally bad. Ideally imo we could change function specifiers to be FInstancedStructs and allow and DI to provide and handle their own types.
		const FNiagaraDataChannelVariable* NDCVar = Channel->Get()->GetVariables().FindByPredicate(
			[&](const FNiagaraDataChannelVariable& Var)
			{
				return Var.GetName() == FuncSpecVariable.GetName() && Var.GetType() == FuncSpecVariable.GetType();
			});

		if (NDCVar)
		{
			FName NDCVarName = NDCVar->GetName();
			FNiagaraTypeDefinition NDCVarType = NDCVar->GetType();
			if (NDCVarType == FNiagaraTypeDefinition::GetIntDef()) 
			{
				OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::SpawnDirect<int32>, NDCVarName);
			}
			else
			{
				UE_LOGF(LogTemp, Display, "Failed to bind Data Interface function as this is not a valid variable type for SpawnDirect.\nDI: %ls.\nReceived Name: %ls\nNDC: %ls", *GetPathNameSafe(this), *BindingInfo.Name.ToString(), *GetPathNameSafe(Channel));
			}
		}
		else
		{
			UE_LOGF(LogTemp, Display, "Failed to bind Data Interface function due to missing NDC variable for %ls.\nReceived Name: %ls\nNDC: %ls", *GetPathNameSafe(this), *BindingInfo.Name.ToString(), *GetPathNameSafe(Channel));
		}
	}
	else if (BindingInfo.Name == NDIDataChannelReadLocal::NAME_ScaleSpawnCount)
	{
		//Find the appropriate variable from the function binding and determine it's type to bind to the correct impl for ScaleSpawnCount and provide the variable index.
			//Can we do this decode on the PostCompile? Also would be good to validate/warn on missing or invalid types.
		FNiagaraVariableBase FuncSpecVariable = DecodeVariableFromSpecifiers(BindingInfo);

		//TODO: We are currently storing these as FNames and encoding/decoding the variable.
		//This is slow, clunky, brittle and generally bad. Ideally imo we could change function specifiers to be FInstancedStructs and allow and DI to provide and handle their own types.
		const FNiagaraDataChannelVariable* NDCVar = Channel->Get()->GetVariables().FindByPredicate(
			[&](const FNiagaraDataChannelVariable& Var)
			{
				return Var.GetName() == FuncSpecVariable.GetName() && Var.GetType() == FuncSpecVariable.GetType();
			});

		if (NDCVar)
		{
			FName NDCVarName = NDCVar->GetName();
			FNiagaraTypeDefinition NDCVarType = NDCVar->GetType();
			if (NDCVarType == FNiagaraTypeDefinition::GetIntDef()) OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::ScaleSpawnCount<int32>, NDCVarName);
			else if (NDCVarType == FNiagaraTypeHelper::GetDoubleDef()) OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::ScaleSpawnCount<double>, NDCVarName);
			else if (NDCVarType == FNiagaraTypeHelper::GetVector2DDef()) OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::ScaleSpawnCount<FVector2D>, NDCVarName);
			else if (NDCVarType == FNiagaraTypeHelper::GetVectorDef()) OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::ScaleSpawnCount<FVector>, NDCVarName);
			else if (NDCVarType == FNiagaraTypeHelper::GetVector4Def()) OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::ScaleSpawnCount<FVector4>, NDCVarName);
			else if (NDCVarType == FNiagaraTypeDefinition::GetPositionDef()) OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::ScaleSpawnCount<FNiagaraPosition>, NDCVarName);
			else
			{
				UE_LOGF(LogTemp, Display, "Failed to bind Data Interface function as this is not a valid variable type for ScaleSpawnCount.\nDI: %ls.\nReceived Name: %ls\nNDC: %ls", *GetPathNameSafe(this), *BindingInfo.Name.ToString(), *GetPathNameSafe(Channel));
			}
		}
		else
		{
			UE_LOGF(LogTemp, Display, "Failed to bind Data Interface function due to missing NDC variable for %ls.\nReceived Name: %ls\nNDC: %ls", *GetPathNameSafe(this), *BindingInfo.Name.ToString(), *GetPathNameSafe(Channel));
		}
	}
	else
	{
		int32 FuncIndex = CompiledData.FindFunctionInfoIndex(BindingInfo.Name, BindingInfo.VariadicInputs, BindingInfo.VariadicOutputs);
		if (BindingInfo.Name == NDIDataChannelReadLocal::NAME_Read)
		{
			OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::Read, FuncIndex);
		}
		else if (BindingInfo.Name == NDIDataChannelReadLocal::NAME_Consume)
		{
			OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::Consume, FuncIndex);
		}
		else if (BindingInfo.Name == NDIDataChannelReadLocal::NAME_SpawnConditional)
		{
			OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelRead::SpawnConditional, FuncIndex);
		}
		else
		{
			UE_LOGF(LogTemp, Display, "Could not find data interface external function in %ls. Received Name: %ls", *GetPathNameSafe(this), *BindingInfo.Name.ToString());
		}
	}
}

void UNiagaraDataInterfaceDataChannelRead::Num(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIDataChannelReadInstanceData> InstData(Context);

	FNDIOutputParam<int32> OutNum(Context);

	bool bReadPrevFrame = bReadCurrentFrame == false || GNDCReadForcePrevFrame;
	FNiagaraDataBuffer* Buffer = InstData->GetReadBufferCPU(bReadPrevFrame);

	int32 Num = 0;
	if (Buffer && INiagaraModule::DataChannelsEnabled())
	{
		Num = (int32)Buffer->GetNumInstances();
	}

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutNum.SetAndAdvance(Num);
	}
}

/** 
GetNDCSpawnData - Retrieves spawn data about the NDC that spawned a particular particle.
Uses an indirection table that decomposes each NDC spawn into power of 2 buckets of particles.
We do this to strike a balance between allowing many particles per NDC entry and allowing many NDC entries to spawn particles.
We have a max of 16 buckets with the highest being for spawns with 1<15 particles or more and the lowest being for individual particles.
An example with two NDC entries. The first spawning 10 and the second 8.
The first's spawn count decomposes into an entry in the 8 bucket and 2 bucket.
The seconds just has an entry in the 8 bucket.
We have 16 buckets so the bucket counts array looks like
0,0,0,0,0,0,0,0,0,0,0,0,2,0,1,0
This means the we have 2 buckets with data so the rest of our buffer is.
0,1,0,

We have two entries in the 8 bucket. 
As we spawn particles we use our bucket sizes counts and the exec index to see which bucket entry each exec index should use.
The first 8 particles processed, exec index 0-7 will lookup the first entry and so use NDC 0.
THe next 8 particles, exec index 8-15 will use the next and so use NDC 1.
Finally the last two particle spawned will use the next entry and so also use NDC 0.

So in total we do have 10 particles from NDC 0 and 8 from NDC 1.
However they will not be processed all together with their own spawning NDC.

In the worst case an NDC entry could add to all 16 buckets and so we'd have 16 entries for that NDC entry.
Which may seem like a lot but consider that is spawning 1<<15 particles so not all that bad really.

It also means the lookup does not need to search an arbitrary sized list.
It just has to loop over a size 16 array and do some math to get an index into the main buffer from which to retreive the NDC Index.

Once we have the NDCIndex we the do another similar pass and use the total spawn counts for that NDC to work out a SpawnIndex within the NDC.
*/
void UNiagaraDataInterfaceDataChannelRead::GetNDCSpawnData(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIDataChannelReadInstanceData> InstData(Context);

	FNDIInputParam<FNiagaraEmitterID> InEmitterID(Context);
	FNDIInputParam<int32> InExecIndex(Context);
	FNDIOutputParam<int32> OutNDCIndex(Context);
	FNDIOutputParam<int32> OutNDCSpawnIndex(Context);
	FNDIOutputParam<int32> OutNDCSpawnCount(Context);

	FNiagaraSystemInstance* SystemInstance = InstData->Owner;
	check(SystemInstance);

	bool bReadPrevFrame = bReadCurrentFrame == false || GNDCReadForcePrevFrame;
	FNiagaraDataBuffer* Buffer = InstData->GetReadBufferCPU(bReadPrevFrame);

	if(Buffer && INiagaraModule::DataChannelsEnabled())
	{
		auto CalculateNDCSpawnInfo = [&](const FNDIDataChannelRead_EmitterInstanceData& EmitterInstanceData)
		{
			const int32* NDCSpawnBukets = EmitterInstanceData.NDCSpawnData.NDCSpawnDataBuckets;
			TConstArrayView<int32> NDCSpawnData(EmitterInstanceData.NDCSpawnData.NDCSpawnData);
			uint32 NumNDCEntries = InstData->NDCElementCountAtSpawn;

			uint32 ExecIndex = InExecIndex.GetAndAdvance();
			uint32 NDCIndex = INDEX_NONE;

			//First we find which bucket this exec index is in.	
			uint32 MaxBucketExecIndex = 0;
			uint32 BucketEntryStart = NumNDCEntries;
			for (uint32 BucketIdx = 0; BucketIdx < 16; ++BucketIdx)
			{
				uint32 BucketSize = (1 << 15) >> BucketIdx;
				uint32 NumEntriesInBucket = NDCSpawnBukets[BucketIdx];
				uint32 MinBucketExecIndex = MaxBucketExecIndex;
				MaxBucketExecIndex += BucketSize * NumEntriesInBucket;
				if (ExecIndex < MaxBucketExecIndex)
				{
					//We found our bucket.
					//Now we need to find our NDCIndex Entry Index.
					uint32 NDCIndexEntry = (ExecIndex - MinBucketExecIndex) >> (15 - BucketIdx);

					NDCIndex = NDCSpawnData[BucketEntryStart + NDCIndexEntry];
					break;
				}

				BucketEntryStart += NumEntriesInBucket;
			}

			if (NDCIndex >= 0 && NDCIndex < NumNDCEntries)
			{
				OutNDCIndex.SetAndAdvance(NDCIndex);

				uint32 NDCSpawnCount = NDCSpawnData[NDCIndex];
				OutNDCSpawnCount.SetAndAdvance(NDCSpawnCount);

				//Do another pass to calculate our SpawnIndex for this NDC within the total count for this NDC.
				if (OutNDCSpawnIndex.IsValid())
				{
					uint32 NDCSpawnIndex = 0;
					uint32 Count = NDCSpawnCount;
					MaxBucketExecIndex = 0;
					for (int32 BucketIdx = 0; BucketIdx < 16; ++BucketIdx)
					{
						uint32 BucketSize = (1 << 15) >> BucketIdx;
						uint32 Mask = (0xFFFF >> (BucketIdx + 1));
						uint32 CountMasked = Count & ~Mask;
						Count &= Mask;
						uint32 NumNDCEntriesInBucket = CountMasked >> (15 - BucketIdx);
						uint32 NumEntriesInBucket = NDCSpawnBukets[BucketIdx];
						uint32 NumNDCInstancesInBucket = NumNDCEntriesInBucket * BucketSize;

						int32 MinBucketExecIndex = MaxBucketExecIndex;
						MaxBucketExecIndex += BucketSize * NumEntriesInBucket;
						if (ExecIndex < MaxBucketExecIndex && NumNDCInstancesInBucket > 0)
						{
							//Find our NDC entry. There is one entry for each bucket load of instances. So we divide our current adjusted exec index in this bucket by the bucket size.
							//As buckets are power of 2 we can do this faster by just shifting down.
							uint32 NDCIndexEntry = (ExecIndex - MinBucketExecIndex) >> (15 - BucketIdx);

							uint32 MinNDCBucketExecIndex = MinBucketExecIndex + (BucketSize * NDCIndexEntry);

							NDCSpawnIndex += (ExecIndex - MinNDCBucketExecIndex);
							break;
						}
						else
						{
							NDCSpawnIndex += NumNDCInstancesInBucket;
						}
					}
					OutNDCSpawnIndex.SetAndAdvance(NDCSpawnIndex);
				}
			}
			else
			{
				OutNDCIndex.SetAndAdvance(INDEX_NONE);
				OutNDCSpawnCount.SetAndAdvance(INDEX_NONE);
				OutNDCSpawnIndex.SetAndAdvance(INDEX_NONE);
			}
		};
		
		if(InEmitterID.IsConstant())
		{		
			//TODO: Can likely vectorize all this.
			const FNiagaraEmitterID EmitterID = InEmitterID.GetAndAdvance();
			FNiagaraEmitterInstance* EmitterInst = SystemInstance->GetEmitterByID(EmitterID);
			if(const FNDIDataChannelRead_EmitterInstanceData* EmitterInstData = InstData->EmitterInstanceData.Find(EmitterInst))
			{
				for (int32 i = 0; i < Context.GetNumInstances(); ++i)
				{
					CalculateNDCSpawnInfo(*EmitterInstData);
				}
			}
			else
			{
				for (int32 i = 0; i < Context.GetNumInstances(); ++i)
				{
					OutNDCIndex.SetAndAdvance(INDEX_NONE);
					OutNDCSpawnCount.SetAndAdvance(INDEX_NONE);
					OutNDCSpawnIndex.SetAndAdvance(INDEX_NONE);
				}
			}
		}
		else
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				const FNiagaraEmitterID EmitterID = InEmitterID.GetAndAdvance();
				FNiagaraEmitterInstance* EmitterInst = SystemInstance->GetEmitterByID(EmitterID);
				if (const FNDIDataChannelRead_EmitterInstanceData* EmitterInstData = InstData->EmitterInstanceData.Find(EmitterInst))
				{
					CalculateNDCSpawnInfo(*EmitterInstData);
				}
				else
				{
					OutNDCIndex.SetAndAdvance(INDEX_NONE);
					OutNDCSpawnCount.SetAndAdvance(INDEX_NONE);
					OutNDCSpawnIndex.SetAndAdvance(INDEX_NONE);
				}
			}			
		}
	}	
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutNDCIndex.SetAndAdvance(INDEX_NONE);
			OutNDCSpawnCount.SetAndAdvance(INDEX_NONE);
			OutNDCSpawnIndex.SetAndAdvance(INDEX_NONE);
		}
	}
}

void UNiagaraDataInterfaceDataChannelRead::Read(FVectorVMExternalFunctionContext& Context, int32 FuncIdx)
{
	SCOPE_CYCLE_COUNTER(STAT_NDIDataChannelRead_Read);
	VectorVM::FUserPtrHandler<FNDIDataChannelReadInstanceData> InstData(Context);
	FNDIInputParam<int32> InIndex(Context);

	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	const FNDIDataChannelFunctionInfo& FuncInfo = CompiledData.GetFunctionInfo()[FuncIdx];
	const FNDIDataChannel_FunctionToDataSetBinding* BindingInfo = InstData->FuncToDataSetBindingInfo.IsValidIndex(FuncIdx) ? InstData->FuncToDataSetBindingInfo[FuncIdx].Get() : nullptr;
	FNDIVariadicOutputHandler<16> VariadicOutputs(Context, BindingInfo);//TODO: Make static / avoid allocation

	bool bReadPrevFrame = bReadCurrentFrame == false || GNDCReadForcePrevFrame;
	FNiagaraDataBuffer* Data = InstData->GetReadBufferCPU(bReadPrevFrame);
	if (Data && BindingInfo && INiagaraModule::DataChannelsEnabled())
 	{
 		//FString Label = TEXT("NDIDataChannelRead::Read() - ");
 		//Data->Dump(0, Data->GetNumInstances(), Label);

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			int32 Index = InIndex.GetAndAdvance();

			bool bProcess = (uint32)Index < Data->GetNumInstances();
			bool bAllReadsSuccess = true;

			auto FloatFunc = [Data, Index, &bAllReadsSuccess](const FNDIDataChannelRegisterBinding& VMBinding, VectorVM::FExternalFuncRegisterHandler<float>& FloatData)
			{
				if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
				{
					float* Src = Data->GetInstancePtrFloat(VMBinding.GetDataSetRegisterIndex(), (uint32)Index);
					*FloatData.GetDestAndAdvance() = *Src;
				}
				else
				{
					bAllReadsSuccess = false;
					*FloatData.GetDestAndAdvance() = 0.0f;
				}
			};
			auto IntFunc = [Data, Index, &bAllReadsSuccess](const FNDIDataChannelRegisterBinding& VMBinding, VectorVM::FExternalFuncRegisterHandler<int32>& IntData)
			{
				if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
				{
					int32* Src = Data->GetInstancePtrInt32(VMBinding.GetDataSetRegisterIndex(), (uint32)Index);
					*IntData.GetDestAndAdvance() = *Src;
				}
				else
				{
					bAllReadsSuccess = false;
					*IntData.GetDestAndAdvance() = 0;
				}
			};
			auto HalfFunc = [Data, Index, &bAllReadsSuccess](const FNDIDataChannelRegisterBinding& VMBinding, VectorVM::FExternalFuncRegisterHandler<FFloat16>& HalfData)
			{
				if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
				{
					*HalfData.GetDestAndAdvance() = *Data->GetInstancePtrHalf(VMBinding.GetDataSetRegisterIndex(), (uint32)Index);
				}
				else
				{
					bAllReadsSuccess = false;
					*HalfData.GetDestAndAdvance() = 0.0f;
				}				
			};
			bool bFinalSuccess = VariadicOutputs.Process(bProcess, 1, BindingInfo, FloatFunc, IntFunc, HalfFunc) && bAllReadsSuccess;

			if (OutSuccess.IsValid())
			{
				OutSuccess.SetAndAdvance(bFinalSuccess);
			}
		}
	}
	else
	{
		VariadicOutputs.Fallback(Context.GetNumInstances());
		if (OutSuccess.IsValid())
		{
			FMemory::Memzero(OutSuccess.Data.GetDest(), sizeof(int32) * Context.GetNumInstances());
		}
	}
}

void UNiagaraDataInterfaceDataChannelRead::Consume(FVectorVMExternalFunctionContext& Context, int32 FuncIdx)
{
	SCOPE_CYCLE_COUNTER(STAT_NDIDataChannelRead_Consume);
	VectorVM::FUserPtrHandler<FNDIDataChannelReadInstanceData> InstData(Context);
	FNDIInputParam<bool> InConsume(Context);

	FNDIOutputParam<bool> OutSuccess(Context);
	FNDIOutputParam<int32> OutIndex(Context);

	const FNDIDataChannelFunctionInfo& FuncInfo = CompiledData.GetFunctionInfo()[FuncIdx];
	const FNDIDataChannel_FunctionToDataSetBinding* BindingInfo = InstData->FuncToDataSetBindingInfo.IsValidIndex(FuncIdx) ? InstData->FuncToDataSetBindingInfo[FuncIdx].Get() : nullptr;
	FNDIVariadicOutputHandler<16> VariadicOutputs(Context, BindingInfo);//TODO: Make static / avoid allocation

	//TODO: Optimize for constant bConsume.
	//TODO: Optimize for long runs of bConsume==true;
	bool bReadPrevFrame = bReadCurrentFrame == false || GNDCReadForcePrevFrame;
	FNiagaraDataBuffer* Data = InstData->GetReadBufferCPU(bReadPrevFrame);
	if (Data && BindingInfo && INiagaraModule::DataChannelsEnabled())
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			bool bConsume = InConsume.GetAndAdvance();

			bool bSuccess = false;
			bool bNeedsFallback = true;
			int32 Index = INDEX_NONE;

			if (bConsume)
			{
				//Increment counter and enforce max if the result is over acceptable values.
				//Note: This allows the index to temporarily exceed the max limits so is unsafe if we access this concurrently anywhere else without checking the limits.
				//Note: However it does avoid a more expensive looping compare exchange.
				Index = InstData->ConsumeIndex++;
				bool bAllReadsSuccess = true;

				if(Index >= 0 && Index < (int32)Data->GetNumInstances())
				{
					//TODO: Wrap/clamp modes etc

					auto FloatFunc = [Data, Index, &bAllReadsSuccess](const FNDIDataChannelRegisterBinding& VMBinding, VectorVM::FExternalFuncRegisterHandler<float>& FloatData)
					{
						if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
						{
							*FloatData.GetDestAndAdvance() = *Data->GetInstancePtrFloat(VMBinding.GetDataSetRegisterIndex(), (uint32)Index);
						}
						else
						{
							bAllReadsSuccess = false;
							*FloatData.GetDestAndAdvance() = 0.0f;
						}
					};
					auto IntFunc = [Data, Index, &bAllReadsSuccess](const FNDIDataChannelRegisterBinding& VMBinding, VectorVM::FExternalFuncRegisterHandler<int32>& IntData)
					{
						if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
						{
							*IntData.GetDestAndAdvance() = *Data->GetInstancePtrInt32(VMBinding.GetDataSetRegisterIndex(), (uint32)Index);
						}
						else
						{
							bAllReadsSuccess = false;
							*IntData.GetDestAndAdvance() = 0;
						}
					};
					auto HalfFunc = [Data, Index, &bAllReadsSuccess](const FNDIDataChannelRegisterBinding& VMBinding, VectorVM::FExternalFuncRegisterHandler<FFloat16>& HalfData)
					{
						if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
						{
							*HalfData.GetDestAndAdvance() = *Data->GetInstancePtrHalf(VMBinding.GetDataSetRegisterIndex(), (uint32)Index);
						}
						else
						{
							bAllReadsSuccess = false;
							*HalfData.GetDestAndAdvance() = 0.0f;
						}
					};
					
					bNeedsFallback = VariadicOutputs.Process(bConsume, 1, BindingInfo, FloatFunc, IntFunc, HalfFunc) == false;
					bSuccess = !bNeedsFallback && bAllReadsSuccess;
				}
				else
				{
					Index = INDEX_NONE;
					bSuccess = false;
					InstData->ConsumeIndex = Data->GetNumInstances();
				}
			}
			
			if(bNeedsFallback)
			{
				VariadicOutputs.Fallback(1);
			}

			if (OutSuccess.IsValid())
			{
				OutSuccess.SetAndAdvance(bSuccess);
			}
			if (OutIndex.IsValid())
			{
				OutIndex.SetAndAdvance(Index);
			}
		}
	}
	else
	{
		VariadicOutputs.Fallback(Context.GetNumInstances());

		if (OutSuccess.IsValid())
		{
			FMemory::Memzero(OutSuccess.Data.GetDest(), sizeof(int32) * Context.GetNumInstances());
		}

		if (OutIndex.IsValid())
		{
			FMemory::Memset(OutIndex.Data.GetDest(), -1, sizeof(int32) * Context.GetNumInstances());
		}
	}
}

void UNiagaraDataInterfaceDataChannelRead::SpawnConditional(FVectorVMExternalFunctionContext& Context, int32 FuncIdx)
{
	SCOPE_CYCLE_COUNTER(STAT_NDIDataChannelRead_Spawn);

	//UE_LOGF(LogNiagara, Warning, "UNiagaraDataInterfaceDataChannelRead::SpawnConditional - %u", FuncIdx);

	//This should only be called from emitter scripts and since it has per instance data then we process them individually.
	check(Context.GetNumInstances() == 1);

	VectorVM::FUserPtrHandler<FNDIDataChannelReadInstanceData> InstData(Context);

	FNiagaraSystemInstance* SystemInstance = InstData->Owner;

	// SystemInstance could be null because of the DI being a user parameter, in which case it is not fully initilaized
	// because it doesn't have access to all the compile time data
	if (!SystemInstance)
	{
		return;
	}

	//Binding info can be null here as we can be spawning without any conditions, i.e. no variadic parameters to the function.
	const FNDIDataChannel_FunctionToDataSetBinding* BindingInfo = InstData->FuncToDataSetBindingInfo.IsValidIndex(FuncIdx) ? InstData->FuncToDataSetBindingInfo[FuncIdx].Get() : nullptr;

	FNDIInputParam<FNiagaraBool> InEnabled(Context);
	FNDIInputParam<FNiagaraEmitterID> InEmitterID(Context);	

	FNDIInputParam<int32> InMode(Context);
	FNDIInputParam<int32> InOp(Context);
	FNDIInputParam<int32> InSpawnMin(Context);
	FNDIInputParam<int32> InSpawnMax(Context);

	FNDIVariadicInputHandler<16> VariadicInputs(Context, BindingInfo);//TODO: Make static / avoid allocation

	const FNiagaraEmitterID EmitterID = InEmitterID.GetAndAdvance();
	FNiagaraEmitterInstance* EmitterInst = SystemInstance->GetEmitterByID(EmitterID);

	FNiagaraDataChannelData* DataChannelData = InstData->DataChannelData.Get();

	const bool bReadPrevFrame = bReadCurrentFrame == false || GNDCReadForcePrevFrame;
	FNiagaraDataBuffer* Data = DataChannelData ? DataChannelData->GetCPUData(bReadPrevFrame).GetReference() : nullptr;
	const FNiagaraTickInfo& TickInfo = SystemInstance->GetSystemSimulation()->GetTickInfo();
	bool bProcessCurrentTick = bOnlySpawnOnceOnSubticks ? TickInfo.TickNumber == TickInfo.TickCount - 1 : true;
	
	const bool bSpawn = INiagaraModule::DataChannelsEnabled() && Data && Data->GetNumInstances() > 0 && EmitterInst && EmitterInst->IsActive() && InEnabled.GetAndAdvance() && bProcessCurrentTick;
	if(bSpawn)
	{
		FNDIRandomHelperFromStream RandHelper(Context);

		ENDIDataChannelSpawnMode Mode = static_cast<ENDIDataChannelSpawnMode>(InMode.GetAndAdvance());
		ENiagaraConditionalOperator Op = static_cast<ENiagaraConditionalOperator>(InOp.GetAndAdvance());
		int32 NumDataChannelInstances = Data->GetNumInstances();

		//Is mode none or invalid?
		if (NumDataChannelInstances == 0 || (int32)Mode == (int32)ENDIDataChannelSpawnMode::None || (int32)Mode < 0 || (int32)Mode >= (int32)ENDIDataChannelSpawnMode::Max)
		{
			return;
		}

		int32 SpawnMin = FMath::Max(0, InSpawnMin.GetAndAdvance());
		int32 SpawnMax = FMath::Max(0, InSpawnMax.GetAndAdvance());

		//Each Data Channel element has an additional spawn entry which accumulates across all spawning calls and can be nulled independently by a suppression call.
		FNDIDataChannelRead_EmitterInstanceData& EmitterInstData = InstData->EmitterInstanceData.FindOrAdd(EmitterInst);
		TArray<FNDIDataChannelRead_EmitterSpawnInfo>& EmitterConditionalSpawns = EmitterInstData.NDCSpawnCounts;
		EmitterConditionalSpawns.SetNum(NumDataChannelInstances);

		for (int32 DataChannelIdx = 0; DataChannelIdx < NumDataChannelInstances; ++DataChannelIdx)
		{
			bool bConditionsPass = true;
			auto FloatFunc = [&bConditionsPass, Op, Data, DataChannelIdx](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<float>& FloatData)
			{
				if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
				{
					bConditionsPass &= EvalConditional(Op, FloatData.GetAndAdvance(), *Data->GetInstancePtrFloat(VMBinding.GetDataSetRegisterIndex(), (uint32)DataChannelIdx));
				}
				else
				{
					bConditionsPass = false;
				}
				
			};
			auto IntFunc = [&bConditionsPass, Op, Data, DataChannelIdx](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<int32>& IntData)
			{
				if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
				{
					bConditionsPass &= EvalConditional(Op, IntData.GetAndAdvance(), *Data->GetInstancePtrInt32(VMBinding.GetDataSetRegisterIndex(), (uint32)DataChannelIdx));
				}
				else
				{
					bConditionsPass = false;
				}
			};
			auto HalfFunc = [&bConditionsPass, Op, Data, DataChannelIdx](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<FFloat16>& HalfData)
			{
				if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
				{
					bConditionsPass &= EvalConditional(Op, HalfData.GetAndAdvance(), *Data->GetInstancePtrHalf(VMBinding.GetDataSetRegisterIndex(), (uint32)DataChannelIdx));
				}
				else
				{
					bConditionsPass = false;
				}
			};
			VariadicInputs.Process(true, 1, BindingInfo, FloatFunc, IntFunc, HalfFunc);
			VariadicInputs.Reset();

			if (bConditionsPass)
			{
				int32 Count = RandHelper.RandRange(DataChannelIdx, SpawnMin, SpawnMax);
				if (Mode == ENDIDataChannelSpawnMode::Accumulate)
				{
					EmitterConditionalSpawns[DataChannelIdx].Append(Count);
				}
				else if (Mode == ENDIDataChannelSpawnMode::Override)
				{
					EmitterConditionalSpawns[DataChannelIdx].SetCount(Count);
				}
			}
		}
	}
}

template<typename T> 
float NDCValueSize(const T& Value){ return 1.0f; }

template<> float NDCValueSize<int32>(const int32& Value){ return static_cast<float>(FMath::Abs(Value)); }
template<> float NDCValueSize<float>(const float& Value) { return FMath::Abs(Value); }
template<> float NDCValueSize<FVector2f>(const FVector2f& Value) { return Value.Size(); }
template<> float NDCValueSize<FVector3f>(const FVector3f& Value) { return Value.Size(); }
template<> float NDCValueSize<FVector4f>(const FVector4f& Value) { return Value.Size(); }

template<typename T>
T NDCValueDefault() { return 1.0f; }
template<> int32 NDCValueDefault<int32>() { return 1; }
template<> float NDCValueDefault<float>() { return 1.0f; }
template<> FVector2f NDCValueDefault<FVector2f>() { return FVector2f(1.0f); }
template<> FVector3f NDCValueDefault<FVector3f>() { return FVector3f(1.0f); }
template<> FVector4f NDCValueDefault<FVector4f>() { return FVector4f(1.0f); }


template<typename T>
void UNiagaraDataInterfaceDataChannelRead::SpawnDirect(FVectorVMExternalFunctionContext& Context, FName NDCVarName)
{
	//This should only be called from emitter scripts and since it has per instance data then we process them individually.
	check(Context.GetNumInstances() == 1);

	using TSimType = typename TNiagaraTypeHelper<T>::TSimType;
	VectorVM::FUserPtrHandler<FNDIDataChannelReadInstanceData> InstData(Context);

	FNDIInputParam<FNiagaraBool> InEnabled(Context);
	FNDIInputParam<FNiagaraEmitterID> InEmitterID(Context);

	FNDIInputParam<int32> InMode(Context);

	FNDIInputParam<float> InRandMinScale(Context);
	FNDIInputParam<float> InRandMaxScale(Context);
	FNDIInputParam<int32> InClampMin(Context);
	FNDIInputParam<int32> InClampMax(Context);

	FNiagaraSystemInstance* SystemInstance = InstData->Owner;
	check(SystemInstance);

	const FNiagaraEmitterID EmitterID = InEmitterID.GetAndAdvance();
	const int32 NumEmitters = SystemInstance->GetEmitters().Num();
	FNiagaraEmitterInstance* EmitterInst = SystemInstance->GetEmitterByID(EmitterID);

	FNiagaraDataChannelData* DataChannelData = InstData->DataChannelData.Get();

	const bool bReadPrevFrame = bReadCurrentFrame == false || GNDCReadForcePrevFrame;
	FNiagaraDataBuffer* Data = DataChannelData ? DataChannelData->GetCPUData(bReadPrevFrame).GetReference() : nullptr;
	const FNiagaraTickInfo& TickInfo = SystemInstance->GetSystemSimulation()->GetTickInfo();
	bool bProcessCurrentTick = bOnlySpawnOnceOnSubticks ? TickInfo.TickNumber == TickInfo.TickCount - 1 : true;
	
	const bool bApply = INiagaraModule::DataChannelsEnabled() && Data && Data->GetNumInstances() > 0 && EmitterInst && EmitterInst->IsActive() && InEnabled.GetAndAdvance() && bProcessCurrentTick;
	if (bApply)
	{
		FNDIRandomHelperFromStream RandHelper(Context);

		int32 NumDataChannelInstances = Data->GetNumInstances();

		ENDIDataChannelSpawnMode Mode = (ENDIDataChannelSpawnMode)InMode.GetAndAdvance();

		float RandMinScale = InRandMinScale.GetAndAdvance();
		float RandMaxScale = InRandMaxScale.GetAndAdvance();
		int32 ClampMin = FMath::Max(0, InClampMin.GetAndAdvance());
		int32 ClampMax = InClampMax.GetAndAdvance();
		ClampMax = ClampMax < 0 ? INT_MAX : ClampMax;

		//Each Data Channel element has an additional spawn entry which accumulates across all spawning calls and can be nulled independently by a suppression call.
		FNDIDataChannelRead_EmitterInstanceData& EmitterInstData = InstData->EmitterInstanceData.FindOrAdd(EmitterInst);
		TArray<FNDIDataChannelRead_EmitterSpawnInfo>& EmitterConditionalSpawns = EmitterInstData.NDCSpawnCounts;
		EmitterConditionalSpawns.SetNum(NumDataChannelInstances);

		const auto ValueData = FNiagaraDataSetAccessor<TSimType>::CreateReader(Data, NDCVarName);

		if (ValueData.IsValid())
		{
			for (int32 DataChannelIndex = 0; DataChannelIndex < NumDataChannelInstances; DataChannelIndex++)
			{
				TSimType NDCValue = ValueData.GetSafe(DataChannelIndex, NDCValueDefault<TSimType>());
				double VarSize = NDCValueSize(NDCValue);
				float Scale = RandHelper.RandRange(DataChannelIndex, RandMinScale, RandMaxScale);
				int32 ScaledCount = FMath::TruncToInt32(VarSize * Scale);
				int32 FinalCount = FMath::Clamp(ScaledCount, ClampMin, ClampMax);

				if (Mode == ENDIDataChannelSpawnMode::Accumulate)
				{
					EmitterConditionalSpawns[DataChannelIndex].Append(FinalCount);
				}
				else if (Mode == ENDIDataChannelSpawnMode::Override)
				{
					EmitterConditionalSpawns[DataChannelIndex].SetCount(FinalCount);
				}
			}
		}
	}
}

template<typename T>
void UNiagaraDataInterfaceDataChannelRead::ScaleSpawnCount(FVectorVMExternalFunctionContext& Context, FName NDCVarName)
{
	//This should only be called from emitter scripts and since it has per instance data then we process them individually.
	check(Context.GetNumInstances() == 1);

	using TSimType = typename TNiagaraTypeHelper<T>::TSimType;
	VectorVM::FUserPtrHandler<FNDIDataChannelReadInstanceData> InstData(Context);

	FNDIInputParam<FNiagaraBool> InEnabled(Context);
	FNDIInputParam<FNiagaraEmitterID> InEmitterID(Context);
	FNDIInputParam<int32> InMode(Context);

	FNDIInputParam<float> InRandMinScale(Context);
	FNDIInputParam<float> InRandMaxScale(Context);
	FNDIInputParam<float> InClampMin(Context);
	FNDIInputParam<float> InClampMax(Context);
	
	FNiagaraSystemInstance* SystemInstance = InstData->Owner;
	check(SystemInstance);

	const FNiagaraEmitterID EmitterID = InEmitterID.GetAndAdvance();
	const int32 NumEmitters = SystemInstance->GetEmitters().Num();
	FNiagaraEmitterInstance* EmitterInst = SystemInstance->GetEmitterByID(EmitterID);

	FNiagaraDataChannelData* DataChannelData = InstData->DataChannelData.Get();

	const bool bReadPrevFrame = bReadCurrentFrame == false || GNDCReadForcePrevFrame;
	FNiagaraDataBuffer* Data = DataChannelData ? DataChannelData->GetCPUData(bReadPrevFrame).GetReference() : nullptr;

	const bool bApplyScale = INiagaraModule::DataChannelsEnabled() && Data && Data->GetNumInstances() > 0 && EmitterInst && EmitterInst->IsActive() && InEnabled.GetAndAdvance();
	if (bApplyScale)
	{
		int32 Mode = InMode.GetAndAdvance();
		bool bOverrideScale = Mode == (int32)ENDIDataChannelSpawnScaleMode::Override;

		FNDIRandomHelperFromStream RandHelper(Context);

		int32 NumDataChannelInstances = Data->GetNumInstances();

		float RandMinScale = InRandMinScale.GetAndAdvance();
		float RandMaxScale = InRandMaxScale.GetAndAdvance();
		float ClampMin = FMath::Max(0.0f, InClampMin.GetAndAdvance());
		float ClampMax = InClampMax.GetAndAdvance();
		ClampMax = ClampMax < 0.0f ? FLT_MAX : ClampMax;

		//Each Data Channel element has an additional spawn entry which accumulates across all spawning calls and can be nulled independently by a suppression call.
		FNDIDataChannelRead_EmitterInstanceData& EmitterInstData = InstData->EmitterInstanceData.FindOrAdd(EmitterInst);
		TArray<FNDIDataChannelRead_EmitterSpawnInfo>& EmitterConditionalSpawns = EmitterInstData.NDCSpawnCounts;
		EmitterConditionalSpawns.SetNumZeroed(NumDataChannelInstances);

		const auto ValueData = FNiagaraDataSetAccessor<TSimType>::CreateReader(Data, NDCVarName);

		if(ValueData.IsValid())
		{
			for (int32 DataChannelIndex = 0; DataChannelIndex < NumDataChannelInstances; DataChannelIndex++)
			{
				TSimType NDCValue = ValueData.GetSafe(DataChannelIndex, NDCValueDefault<TSimType>());
				float VarSize = NDCValueSize(NDCValue);
				float Scale = RandHelper.RandRange(DataChannelIndex, RandMinScale, RandMaxScale);
				float FinalScale = VarSize * Scale;
				FinalScale = FMath::Clamp(FinalScale, ClampMin, ClampMax);

				//TODO: Either change this to a float/double or add a separate scale value applied at the end so that multiple scales will combine correctly.
				if (bOverrideScale)
				{
					EmitterConditionalSpawns[DataChannelIndex].SetScale(FinalScale);
				}
				else
				{
					EmitterConditionalSpawns[DataChannelIndex].ApplyScale(FinalScale);
				}
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceDataChannelRead::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIDataChannelReadLocal::CommonShaderFile);
	bSuccess &= InVisitor->UpdateShaderFile(NDIDataChannelReadLocal::TemplateShaderFile_Common);
	bSuccess &= InVisitor->UpdateShaderFile(NDIDataChannelReadLocal::TemplateShaderFile_ReadCommon);
	bSuccess &= InVisitor->UpdateShaderFile(NDIDataChannelReadLocal::TemplateShaderFile_Read);
	bSuccess &= InVisitor->UpdateShaderFile(NDIDataChannelReadLocal::TemplateShaderFile_Consume);

	bSuccess &= InVisitor->UpdateShaderParameters<NDIDataChannelReadLocal::FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceDataChannelRead::GetCommonHLSL(FString& OutHLSL)
{
	Super::GetCommonHLSL(OutHLSL);
	OutHLSL.Appendf(TEXT("#include \"%s\"\n"), NDIDataChannelReadLocal::CommonShaderFile);
}

bool UNiagaraDataInterfaceDataChannelRead::GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL)
{
	return	HlslGenContext.GetFunctionInfo().DefinitionName == GET_FUNCTION_NAME_CHECKED(UNiagaraDataInterfaceDataChannelRead, Num) ||
		HlslGenContext.GetFunctionInfo().DefinitionName == GET_FUNCTION_NAME_CHECKED(UNiagaraDataInterfaceDataChannelRead, GetNDCSpawnData) ||
		HlslGenContext.GetFunctionInfo().DefinitionName == GET_FUNCTION_NAME_CHECKED(UNiagaraDataInterfaceDataChannelRead, Read) ||
		HlslGenContext.GetFunctionInfo().DefinitionName == GET_FUNCTION_NAME_CHECKED(UNiagaraDataInterfaceDataChannelRead, Consume);
}

void UNiagaraDataInterfaceDataChannelRead::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(HlslGenContext, OutHLSL);

	TArray<FString> CommonTemplateShaders;
	TMap<FName, FString> TemplateShaderMap;
	NDIDataChannelReadLocal::BuildFunctionTemplateMap(CommonTemplateShaders, TemplateShaderMap);
	
	NDIDataChannelUtilities::GenerateDataChannelAccessHlsl(HlslGenContext, CommonTemplateShaders, TemplateShaderMap, OutHLSL);
}

bool UNiagaraDataInterfaceDataChannelRead::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	TArray<FNiagaraFunctionSignature> Funcs;
	GetFunctionsInternal(Funcs);

	for (const FNiagaraFunctionSignature& Func : Funcs)
	{
		if (Func.Name == FunctionSignature.Name && Func.FunctionVersion > FunctionSignature.FunctionVersion)
		{
			//We need to add back any variadic params from the source signature.
			TArray<FNiagaraVariableBase> VariadicInputs;
			FunctionSignature.GetVariadicInputs(VariadicInputs);
			TArray<FNiagaraVariableBase> VariadicOutputs;
			FunctionSignature.GetVariadicOutputs(VariadicOutputs);

			FunctionSignature = Func;
			for (FNiagaraVariableBase& Param : VariadicInputs)
			{
				FunctionSignature.AddInput(Param);
			}
			for (FNiagaraVariableBase& Param : VariadicOutputs)
			{
				FunctionSignature.AddOutput(Param);
			}
			return true;
		}
	}

	return false;
}

#endif

void UNiagaraDataInterfaceDataChannelRead::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIDataChannelReadLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceDataChannelRead::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxy_DataChannelRead& DataInterfaceProxy = Context.GetProxy<FNiagaraDataInterfaceProxy_DataChannelRead>();
	FNiagaraDataInterfaceProxy_DataChannelRead::FInstanceData* InstanceData = DataInterfaceProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

	NDIDataChannelReadLocal::FShaderParameters* InstParameters = Context.GetParameterNestedStruct<NDIDataChannelReadLocal::FShaderParameters>();

	bool bSuccess = false;
	if (InstanceData)
	{
		//Find the start offset in the parameter table for this script.
		const FNiagaraCompileHash& ScriptCompileHash = Context.GetComputeInstanceData().Context->GPUScript_RT->GetBaseCompileHash();
		uint32 ParameterOffsetTableIndex = INDEX_NONE;
		if (uint32* ParameterOffsetTableIndexPtr = InstanceData->GPUScriptParameterTableOffsets.Find(ScriptCompileHash))
		{
			ParameterOffsetTableIndex = *ParameterOffsetTableIndexPtr;
		}

		if (InstanceData->ChannelDataRTProxy && ParameterOffsetTableIndex != INDEX_NONE)
		{
			FNiagaraDataBuffer* Data = InstanceData->GPUBuffer;
			if (Data)
			{
				const FReadBuffer& ParameterLayoutBuffer = InstanceData->ParameterLayoutBuffer;

				FRDGBufferSRVRef NDCSpawnDataBufferSRV = Context.GetGraphBuilder().CreateSRV(InstanceData->NDCSpawnDataBuffer, PF_R32_SINT);
				if (NDCSpawnDataBufferSRV && ParameterLayoutBuffer.SRV.IsValid() && ParameterLayoutBuffer.NumBytes > 0)
				{
					InstParameters->ParamOffsetTable = ParameterLayoutBuffer.SRV.IsValid() ? ParameterLayoutBuffer.SRV.GetReference() : FNiagaraRenderer::GetDummyUIntBuffer();
					InstParameters->ParameterOffsetTableIndex = ParameterOffsetTableIndex;

					InstParameters->FloatStride = Data->GetFloatStride() / sizeof(float);
					InstParameters->Int32Stride = Data->GetInt32Stride() / sizeof(int32);
					//TODO: Half Support | InstParameters->HalfStride = Data->GetHalfStride() / sizeof(FFloat16);

					InstParameters->DataFloat = Data->GetGPUBufferFloat().SRV.IsValid() ? Data->GetGPUBufferFloat().SRV.GetReference() : FNiagaraRenderer::GetDummyFloatBuffer();
					InstParameters->DataInt32 = Data->GetGPUBufferInt().SRV.IsValid() ? Data->GetGPUBufferInt().SRV.GetReference() : FNiagaraRenderer::GetDummyIntBuffer();
					//TODO: Half Support | InstParameters->DataHalf = Data->GetGPUBufferHalf().SRV.IsValid() ? Data->GetGPUBufferHalf().SRV.GetReference() : FNiagaraRenderer::GetDummyHalfBuffer();
					InstParameters->InstanceCountOffset = Data->GetGPUInstanceCountBufferOffset();
					InstParameters->ConsumeInstanceCountOffset = InstanceData->ConsumeInstanceCountOffset;
					InstParameters->BufferSize = Data ? Data->GetNumInstancesAllocated() : INDEX_NONE;

					InstParameters->NDCElementCountAtSpawn = InstanceData->NDCElementCountAtSpawn;
					InstParameters->NDCSpawnDataBuffer = NDCSpawnDataBufferSRV;

					bSuccess = true;
				}
			}
		}
	}

	if (bSuccess == false)
	{
		InstParameters->ParamOffsetTable = FNiagaraRenderer::GetDummyUIntBuffer();
		InstParameters->ParameterOffsetTableIndex = INDEX_NONE;

		InstParameters->FloatStride = 0;
		InstParameters->Int32Stride = 0;
		//TODO: Half Support | InstParameters->HalfStride = 0;

		InstParameters->DataFloat = FNiagaraRenderer::GetDummyFloatBuffer();
		InstParameters->DataInt32 = FNiagaraRenderer::GetDummyIntBuffer();
		//TODO: Half Support | InstParameters->DataHalf = FNiagaraRenderer::GetDummyHalfBuffer();

		InstParameters->InstanceCountOffset = INDEX_NONE;
		InstParameters->ConsumeInstanceCountOffset = INDEX_NONE;
		InstParameters->BufferSize = INDEX_NONE;
		
		InstParameters->NDCElementCountAtSpawn = 0;
		FRDGBufferRef DummyBuffer = GSystemTextures.GetDefaultBuffer(Context.GetGraphBuilder(), 4, 0u);
		InstParameters->NDCSpawnDataBuffer = Context.GetGraphBuilder().CreateSRV(DummyBuffer, PF_R32_SINT);		
	}
}

void FNiagaraDataInterfaceProxy_DataChannelRead::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	FNiagaraDataInterfaceProxy_DataChannelRead::FInstanceData* InstanceData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

	if(InstanceData)
	{
		if(InstanceData->ChannelDataRTProxy && InstanceData->GPUBuffer == nullptr)
		{
			InstanceData->GPUBuffer = InstanceData->ChannelDataRTProxy->PrepareForReadAccess(Context.GetGraphBuilder(), InstanceData->bReadPrevFrame == false);
		}

		//TODO: Should grab just one for the whole frame...
		//TODO: Add some wrap behavior...		
		if(InstanceData->ConsumeInstanceCountOffset == INDEX_NONE)
		{
			InstanceData->ConsumeInstanceCountOffset = Context.GetInstanceCountManager().AcquireEntry();
		}

		if(InstanceData->NDCSpawnDataBuffer == nullptr)
		{
			InstanceData->NDCSpawnDataBuffer = CreateUploadBuffer<int32>(
				Context.GetGraphBuilder(),
				TEXT("Niagara_NDCReadDI_NDCSpawnData"),
				InstanceData->NDCSpawnData);
		}
	}
}

void FNiagaraDataInterfaceProxy_DataChannelRead::PostStage(const FNDIGpuComputePostStageContext& Context)
{
	FNiagaraDataInterfaceProxy_DataChannelRead::FInstanceData* InstanceData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	if (InstanceData && InstanceData->ChannelDataRTProxy)
	{
		if (InstanceData->GPUBuffer)
		{
			InstanceData->ChannelDataRTProxy->EndReadAccess(Context.GetGraphBuilder(), InstanceData->bReadPrevFrame == false);
			InstanceData->GPUBuffer = nullptr;
		}	
	}
}

void FNiagaraDataInterfaceProxy_DataChannelRead::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	FNiagaraDataInterfaceProxy_DataChannelRead::FInstanceData* InstanceData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	if(InstanceData && InstanceData->ChannelDataRTProxy)
	{
		check(InstanceData->GPUBuffer == nullptr);

		if (Context.IsFinalPostSimulate())
		{
			InstanceData->NDCSpawnDataBuffer = nullptr;
			Context.GetInstanceCountManager().FreeEntry(InstanceData->ConsumeInstanceCountOffset);
			InstanceData->ConsumeInstanceCountOffset = INDEX_NONE;//This should already be done inside FreeEntry but just to be sure.
		}
	}
}

void FNiagaraDataInterfaceProxy_DataChannelRead::ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance)
{
	FNDIDataChannelReadInstanceData_RT& SourceData = *reinterpret_cast<FNDIDataChannelReadInstanceData_RT*>(PerInstanceData);
	FInstanceData& InstData = SystemInstancesToProxyData_RT.FindOrAdd(Instance);

	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	InstData.ChannelDataRTProxy = SourceData.ChannelDataRTProxy;
	InstData.bReadPrevFrame = SourceData.bReadPrevFrame;
	InstData.NDCElementCountAtSpawn = SourceData.NDCElementCountAtSpawn;

	InstData.NDCSpawnData = SourceData.NDCSpawnData;
	InstData.NDCSpawnDataBuffer = nullptr;//Clear the RDG buffer ready for re-up to the GPU.

	if (SourceData.ScriptParamInfo.bDirty) 
	{
		//Take the offset map from the source data.
		//This maps from GPU script to that scripts offset into the ParameterLayoutBuffer.
		//Allows us to look up and pass in at SetShaderParameters time.
		InstData.GPUScriptParameterTableOffsets = MoveTemp(SourceData.ScriptParamInfo.GPUScriptParameterTableOffsets);

		//Now generate the ParameterLayoutBuffer
		//This contains a table of all parameters used by each GPU script that uses this DI.
		//TODO: This buffer can likely be shared among many instances and stored in the layout manager or in the DI proxy.
		{
			if (InstData.ParameterLayoutBuffer.NumBytes > 0)
			{
				InstData.ParameterLayoutBuffer.Release();
			}

			if(SourceData.ScriptParamInfo.GPUScriptParameterOffsetTable.Num() > 0)
			{
				InstData.ParameterLayoutData = SourceData.ScriptParamInfo.GPUScriptParameterOffsetTable;
				InstData.ParameterLayoutBuffer.Initialize(RHICmdList, TEXT("NDIDataChannel_ParameterLayoutBuffer"), sizeof(uint32), SourceData.ScriptParamInfo.GPUScriptParameterOffsetTable.Num(), EPixelFormat::PF_R32_UINT, BUF_Static, &InstData.ParameterLayoutData);
			}
		}
	}

	SourceData.~FNDIDataChannelReadInstanceData_RT();
}

int32 FNiagaraDataInterfaceProxy_DataChannelRead::PerInstanceDataPassedToRenderThreadSize() const
{
	return sizeof(FNDIDataChannelReadInstanceData_RT);
}

void FNiagaraDataInterfaceProxy_DataChannelRead::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context)
{
	const FNiagaraDataInterfaceProxy_DataChannelRead::FInstanceData* InstanceData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	if (InstanceData && InstanceData->ChannelDataRTProxy)
	{
		FNiagaraDataBuffer* Data = InstanceData->bReadPrevFrame ? InstanceData->ChannelDataRTProxy->PrevFrameData.GetReference() : InstanceData->ChannelDataRTProxy->GetCurrentData().GetReference();
		if (Data)
		{
			//Indirect args via the instance count buffer is not working. TODO.
			//Running for all allocated elements will execute more than needed but should allow things to work.
			//Context.CreateIndirect(Data->GetGPUInstanceCountBufferOffset());
			Context.SetDirect(Data->GetNumInstancesAllocated());
		}
	}
}

#undef LOCTEXT_NAMESPACE
