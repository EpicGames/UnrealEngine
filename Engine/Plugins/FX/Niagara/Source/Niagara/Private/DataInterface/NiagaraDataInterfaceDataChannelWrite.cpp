// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceDataChannelWrite.h"

#include "NiagaraModule.h"
#include "Stats/Stats.h"
#include "NiagaraCommon.h"
#include "NiagaraShared.h"

#include "NiagaraSimCache.h"
#include "NiagaraSystem.h"
#include "NiagaraWorldManager.h"
#include "NiagaraSystemInstance.h"

#include "NiagaraCompileHashVisitor.h"
#include "ShaderCompilerCore.h"
#include "NiagaraShaderParametersBuilder.h"

#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelHandler.h"
#include "NiagaraDataChannelManager.h"
#include "NiagaraDataChannelData.h"
#include "NiagaraDataChannelGameData.h"

#include "NiagaraRenderer.h"
#include "NiagaraGPUSystemTick.h"
#include "NiagaraGpuComputeDispatchInterface.h"

#include "NiagaraDataInterfaceUtilities.h"

#include "NiagaraDataSetReadback.h"
#include "NiagaraGpuReadbackManager.h"

#if WITH_EDITOR
#include "INiagaraEditorOnlyDataUtlities.h"
#include "Modules/ModuleManager.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceDataChannelWrite)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceDataChannelWrite"

DECLARE_CYCLE_STAT(TEXT("NDIDataChannelWrite Write"), STAT_NDIDataChannelWrite_Write, STATGROUP_NiagaraDataChannels);
DECLARE_CYCLE_STAT(TEXT("NDIDataChannelWrite Append"), STAT_NDIDataChannelWrite_Append, STATGROUP_NiagaraDataChannels);
DECLARE_CYCLE_STAT(TEXT("NDIDataChannelWrite Tick"), STAT_NDIDataChannelWrite_Tick, STATGROUP_NiagaraDataChannels);
DECLARE_CYCLE_STAT(TEXT("NDIDataChannelWrite PostTick"), STAT_NDIDataChannelWrite_PostTick, STATGROUP_NiagaraDataChannels);
DECLARE_CYCLE_STAT(TEXT("NDIDataChannelWrite PreStageTick"), STAT_NDIDataChannelWrite_PreStageTick, STATGROUP_NiagaraDataChannels);
DECLARE_CYCLE_STAT(TEXT("NDIDataChannelWrite PostStageTick"), STAT_NDIDataChannelWrite_PostStageTick, STATGROUP_NiagaraDataChannels);

int32 GbDebugDumpWriter = 0;
static FAutoConsoleVariableRef CVarDebugDumpWriterDI(
	TEXT("fx.Niagara.DataChannels.DebugDumpWriterDI"),
	GbDebugDumpWriter,
	TEXT(" \n"),
	ECVF_Default
);

int32 GbNDCWriteDIZeroCPUBufferMode = 1;
static FAutoConsoleVariableRef CVarNDCWriteDIZeroCPUBufferMode(
	TEXT("fx.Niagara.DataChannels.WriteDIZeroCPUBuffersMode"),
	GbNDCWriteDIZeroCPUBufferMode,
	TEXT("Controls how CPU buffers are zeroed for the NDC Write DI\n0 = Do not Zero CPU buffers.\n1 = Zero only when calling \"Write\" function.\n2 = Zero always.\n"),
	ECVF_Default
);

namespace NDIDataChannelWriteLocal
{
	static const TCHAR* CommonShaderFile = TEXT("/Plugin/FX/Niagara/Private/DataChannel/NiagaraDataInterfaceDataChannelCommon.ush");	
	static const TCHAR* TemplateShaderFile_Common = TEXT("/Plugin/FX/Niagara/Private/DataChannel/NiagaraDataInterfaceDataChannelTemplateCommon.ush");
	static const TCHAR* TemplateShaderFile_WriteCommon = TEXT("/Plugin/FX/Niagara/Private/DataChannel/NiagaraDataInterfaceDataChannelTemplateWriteCommon.ush");
	static const TCHAR* TemplateShaderFile_Write = TEXT("/Plugin/FX/Niagara/Private/DataChannel/NiagaraDataInterfaceDataChannelTemplate_Write.ush");
	static const TCHAR* TemplateShaderFile_Append = TEXT("/Plugin/FX/Niagara/Private/DataChannel/NiagaraDataInterfaceDataChannelTemplate_Append.ush");

	//////////////////////////////////////////////////////////////////////////
	//Function definitions

	/////
	// NOTE: *any* changes to function inputs or outputs here needs to be included in FWriteNDCModel::GenerateNewModuleContent()
	/////

	const FNiagaraFunctionSignature& GetFunctionSig_Num()
	{
		static FNiagaraFunctionSignature Sig;
		if(!Sig.IsValid())
		{
			Sig.Name = TEXT("Num");
#if WITH_EDITORONLY_DATA
			NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)
			Sig.Description = LOCTEXT("NumFunctionDescription", "Returns the number of instances allocated for writing into the Data Channel from this interface. Writes at an index beyond this will fail.");
#endif
			Sig.bMemberFunction = true;
			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelWrite::StaticClass()), TEXT("DataChannel interface")));
			Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num")));
		}
		return Sig;
	}

	//For now, disable the allocate function as we don't have time to test it thoroughly.
	// TODO: Make experimental?
// 	const FNiagaraFunctionSignature& GetFunctionSig_Allocate()
// 	{
// 		static FNiagaraFunctionSignature Sig;
// 		if (!Sig.IsValid())
// 		{
// 			Sig.Name = TEXT("Allocate");
// #if WITH_EDITORONLY_DATA
// 			NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)
// 			Sig.Description = LOCTEXT("AllocateFunctionDescription", "Adds an amount to allocated into the bound NDC data for the given emitter to write into.");
// #endif
// 			Sig.bMemberFunction = true;
// 			Sig.bRequiresExecPin = true;
// 			Sig.bWriteFunction = true;
// 			Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
// 			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelWrite::StaticClass()), TEXT("DataChannel interface")));
// 			Sig.AddInputWithoutDefault(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraEmitterID::StaticStruct()), TEXT("Emitter ID")), LOCTEXT("EmitterIDDesc", "ID of the emitter that will be writing into this allocated space."));
// 			Sig.AddInputWithDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Allocation Count")), 0, LOCTEXT("AllocationCountDesc", "The number of elements to allocate in the NDC for writing from the given emitter."));
// 		}
// 		return Sig;
// 	}

	const FNiagaraFunctionSignature& GetFunctionSig_Write()
	{
		static FNiagaraFunctionSignature Sig;
		if (!Sig.IsValid())
		{
			static FNiagaraVariable EmitVar(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emit"));
			EmitVar.SetValue(FNiagaraBool(true));
			Sig.Name = TEXT("Write");
#if WITH_EDITORONLY_DATA
			NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)
			Sig.Description = LOCTEXT("WriteFunctionDescription", "Writes data into the Data Channel at a specific index.  Values in the DataChannel that are not written here are set to their defaults. Returns success if the index was valid and data was written into the Data Channel.");
#endif
			Sig.bMemberFunction = true;
			Sig.bRequiresExecPin = true;
			Sig.bWriteFunction = true;
			Sig.bSupportsGPU = false;//Cannot use direct index writes on GPU as we write into one shared buffer with all DIs using the same NDC data.
			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelWrite::StaticClass()), TEXT("DataChannel interface")));
			Sig.AddInput(EmitVar, LOCTEXT("ExecuteWriteFlagTooltip", "If true then the write is executed, if false then this call is ignored and no write occurs."));
			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));
			Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));
			Sig.RequiredInputs = IntCastChecked<int16>(Sig.Inputs.Num());//The user defines what we write in the graph.
		}
		return Sig;
	}

	const FNiagaraFunctionSignature& GetFunctionSig_Append()
	{
		static FNiagaraFunctionSignature Sig;
		if (!Sig.IsValid())
		{
			static FNiagaraVariable EmitVar(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emit"));
			EmitVar.SetValue(FNiagaraBool(true));

			Sig.Name = TEXT("Append");
#if WITH_EDITORONLY_DATA
			NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)
			Sig.Description = LOCTEXT("AppendFunctionDescription", "Appends a new DataChannel to the end of the DataChannel array and writes the specified values. Values in the DataChannel that are not written here are set to their defaults. Returns success if an DataChannel was successfully pushed.");
#endif
			Sig.bMemberFunction = true;
			Sig.bRequiresExecPin = true;
			Sig.bWriteFunction = true;
			Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelWrite::StaticClass()), TEXT("DataChannel interface")));
			Sig.AddInput(EmitVar, LOCTEXT("ExecuteAppendFlagTooltip", "If true then the append is executed, if false then this call is skipped and no append occurs."));
			Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));
			Sig.RequiredInputs = IntCastChecked<int16>(Sig.Inputs.Num());//The user defines what we write in the graph.
		}
		return Sig;
	}

	void BuildFunctionTemplateMap(TArray<FString>& OutCommonTemplateShaders, TMap<FName, FString>& OutMap)
	{
		//Add common template shaders 
		LoadShaderSourceFile(TemplateShaderFile_Common, EShaderPlatform::SP_PCD3D_SM5, &OutCommonTemplateShaders.AddDefaulted_GetRef(), nullptr);
		LoadShaderSourceFile(TemplateShaderFile_WriteCommon, EShaderPlatform::SP_PCD3D_SM5, &OutCommonTemplateShaders.AddDefaulted_GetRef(), nullptr);

		//Add per function template shaders
//		LoadShaderSourceFile(TemplateShaderFile_Write, EShaderPlatform::SP_PCD3D_SM5, &OutMap.Add(GetFunctionSig_Write().Name), nullptr);//Wite is not supported on GPU
		LoadShaderSourceFile(TemplateShaderFile_Append, EShaderPlatform::SP_PCD3D_SM5, &OutMap.Add(GetFunctionSig_Append().Name), nullptr);
	}


	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_SRV(Buffer<uint>, ParamOffsetTable)
		SHADER_PARAMETER(int32, ParameterOffsetTableIndex)
		SHADER_PARAMETER(int32, FloatStride)
		SHADER_PARAMETER(int32, Int32Stride)
		//TODO: Half Support | SHADER_PARAMETER(int32, HalfStride)

		SHADER_PARAMETER_UAV(RWBuffer<float>, GPUBufferFloat)
		SHADER_PARAMETER_UAV(RWBuffer<int>, GPUBufferInt32)
		//TODO: Half Support | SHADER_PARAMETER_UAV(RWBuffer<float>, GPUBufferHalf)
		SHADER_PARAMETER(int32, GPUInstanceCountOffset)
		SHADER_PARAMETER(int32, GPUBufferSize)

		SHADER_PARAMETER_UAV(RWBuffer<float>, CPUBufferFloat)
		SHADER_PARAMETER_UAV(RWBuffer<int>, CPUBufferInt32)
		//TODO: Half Support | SHADER_PARAMETER_UAV(RWBuffer<float>, CPUBufferHalf)
		SHADER_PARAMETER(int32, CPUInstanceCountOffset)
		SHADER_PARAMETER(int32, CPUBufferSize)
		SHADER_PARAMETER(int32, CPUFloatStride)
		SHADER_PARAMETER(int32, CPUInt32Stride)
		//TODO: Half Support | SHADER_PARAMETER(int32, CPUHalfStride)
	END_SHADER_PARAMETER_STRUCT()
}

/** Render thread copy of current instance data. */
struct FNDIDataChannelWriteInstanceData_RT
{
	//RT proxy for game channel data from which we're reading.
	FNiagaraDataChannelDataProxyPtr ChannelDataRTProxy = nullptr;

	/** Parameter mapping info for every function in every script used by this DI. */
	FVariadicParameterGPUScriptInfo ScriptParamInfo;

	/** How many instances should we allocate in the NDC for this DI. */
	uint32 AllocationCount = 0;

	bool bPublishToGame = false;
	bool bPublishToCPU = false;
	bool bPublishToGPU = false;

	FVector3f LwcTile;
};

//////////////////////////////////////////////////////////////////////////
//FNDIDataChannelWriteCompiledData

bool FNDIDataChannelWriteCompiledData::Init(const UNiagaraSystem* System, UNiagaraDataInterfaceDataChannelWrite* OwnerDI)
{
	FunctionInfo.Reset();

	DataLayout.Empty();

	GatherAccessInfo(System, OwnerDI);

	for (FNDIDataChannelFunctionInfo& FuncInfo : FunctionInfo)
	{
		for (FNiagaraVariableBase& Param : FuncInfo.Inputs)
		{
			DataLayout.Variables.AddUnique(Param);
		}
	}

	DataLayout.BuildLayout();

	return true;
}

//FNDIDataChannelWriteCompiledData END
//////////////////////////////////////////////////////////////////////////




/**
 The data channel write interface allows one Niagara System to write out arbitrary data to be later read by some other Niagara System or Game code/BP.

 Currently this is done by writing the data to a local buffer and then copying into a global buffer when the data channel next ticks.
 In the future we may add alternatives to this that allow for less copying etc.
 Though for now this method allows the system to work without any synchronization headaches for the Read/Write or data races accessing a shared buffer concurrently etc.

 Write DIs can also write in "Local" mode, which means their data is defined by whatever they write rather than any predefined

*/
struct FNDIDataChannelWriteInstanceData
{
	/** Pointer to the world DataChannel Channel we'll push our DataChannel into. Can be null if DI is not set to publish it's DataChannel. */
	TWeakObjectPtr<UNiagaraDataChannelHandler> DataChannel;
	
	/** Access context object we'll use for finding the correct source data from the Data Channel. */	
	FNDCAccessContextInst AccessContext;

	//Shared pointer to the actual data we'll be pushing into for this data channel.
	FNiagaraDataChannelDataPtr DataChannelData;

	/** Cached hash to check if the layout of our source data has changed. */
	uint64 ChachedDataSetLayoutHash = INDEX_NONE;

	TArray<FNDIDataChannel_FuncToDataSetBindingPtr, TInlineAllocator<8>> FunctionToDatSetBindingInfo;

	//Atomic uint for tracking num instances of the target data buffer when writing from multiple threads in the VM.
	std::atomic<uint32> AtomicNumInstances;

	/** When true we should update our function binding info on the RT next tick. */
	mutable bool bUpdateFunctionBindingRTData = false;

	int32 DynamicAllocationCount = 0;

	FVector3f LwcTile;

	//Buffer we're currently writing into this frame.
	FNiagaraDataBuffer* DestinationData = nullptr;

	FNiagaraSystemInstance* Owner = nullptr;

	~FNDIDataChannelWriteInstanceData()
	{
	}

	bool Init(UNiagaraDataInterfaceDataChannelWrite* Interface, FNiagaraSystemInstance* Instance)
	{
		Owner = Instance;
		
		//In non test/shipping builds we gather and log and missing parameters that cause us to fail to find correct bindings.
		TArray<FNiagaraVariableBase> MissingParams;

		//Grab the correct function binding infos for this DI.
		const FNDIDataChannelCompiledData& CompiledData = Interface->GetCompiledData();
		FunctionToDatSetBindingInfo.Reset(CompiledData.GetFunctionInfo().Num());
		for (const FNDIDataChannelFunctionInfo& FuncInfo : CompiledData.GetFunctionInfo())
		{
			FunctionToDatSetBindingInfo.Add(FNDIDataChannelLayoutManager::Get().GetLayoutInfo(FuncInfo, Interface->GetCompiledData().DataLayout, MissingParams));			
		}

		//Init our access context for finding the correct source data from the NDC.
		if (Interface->AccessContext.IsValid())
		{
			AccessContext = Interface->AccessContext;
			FNDCAccessContextBase& BaseContext = AccessContext.GetChecked<FNDCAccessContextBase>();
			BaseContext.OwningComponent = Instance->GetAttachComponent();
		}

		return true;
	}

	bool Tick(UNiagaraDataInterfaceDataChannelWrite* Interface, FNiagaraSystemInstance* Instance)
	{
		DynamicAllocationCount = 0;
		AtomicNumInstances = 0;
		DestinationData = nullptr;
		Owner = Instance;
		
		LwcTile = Instance->GetLWCTile();
		if (Interface->ShouldPublish())
		{
			UNiagaraDataChannelHandler* DataChannelPtr = DataChannel.Get();
			if (DataChannelPtr == nullptr)
			{
				UWorld* World = Instance->GetWorld();
				if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World))
				{
					if (UNiagaraDataChannelHandler* NewChannelHandler = WorldMan->GetDataChannelManager().FindDataChannelHandler(Interface->Channel))
					{
						DataChannelPtr = NewChannelHandler;
						DataChannel = NewChannelHandler;

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
						//In non test/shipping builds we gather and log and missing parameters that cause us to fail to find correct bindings.
						TArray<FNiagaraVariableBase> MissingParams;
						const FNDIDataChannelCompiledData& CompiledData = Interface->GetCompiledData();
						for (const FNDIDataChannelFunctionInfo& FuncInfo : CompiledData.GetFunctionInfo())
						{
							FNDIDataChannelLayoutManager::Get().GetLayoutInfo(FuncInfo, DataChannelPtr->GetDataChannel()->GetLayoutInfo()->GetDataSetCompiledData(), MissingParams);
						}

						if (MissingParams.Num() > 0)
						{
							FString MissingParamsString;
							for (FNiagaraVariableBase& MissingParam : MissingParams)
							{
								MissingParamsString += FString::Printf(TEXT("%s %s\n"), *MissingParam.GetType().GetName(), *MissingParam.GetName().ToString());
							}

							UE_LOGF(LogNiagara, Warning, "Niagara Data Channel Writer Interface is trying to write parameters that do not exist in this channel.\nIt's likely that the Data Channel Definition has been changed and this system needs to be updated.\nData Channel: %ls\nSystem: %ls\nComponent:%ls\nMissing Parameters:\n%ls\n"
								, *DataChannel->GetDataChannel()->GetName()
								, *Instance->GetSystem()->GetPathName()
								, *Instance->GetAttachComponent()->GetPathName()
								, *MissingParamsString);
						}
#endif
					}
					else
					{
						UE_LOGF(LogNiagara, Warning, "Failed to find or add Naigara DataChannel Channel: %ls", *Interface->Channel.GetName());
						return false;
					}
				}
			}

			if (DataChannelPtr)
			{
				//In non test/shipping builds we gather and log and missing parameters that cause us to fail to find correct bindings.
				TArray<FNiagaraVariableBase> MissingParams;

				bool bNDCDataIsValid = DataChannelData && DataChannelData->IsLayoutValid(DataChannelPtr);
				if (bNDCDataIsValid == false || Interface->bUpdateDestinationDataEveryTick)
				{	
					if (AccessContext.IsValid() && AccessContext.GetScriptStruct() != FNDCAccessContextLegacy::StaticStruct())
					{
						DataChannelData = DataChannelPtr->FindData(AccessContext, ENiagaraResourceAccess::WriteOnly);
					}
					else
					{
						//If the access context is not valid then this must be a legacy NDC so just use legacy access code.
						FNiagaraDataChannelSearchParameters SearchParams(Instance->GetAttachComponent());
						DataChannelData = DataChannelPtr->FindData(SearchParams, ENiagaraResourceAccess::WriteOnly);
					}
				}

				const FNDIDataChannelCompiledData& CompiledData = Interface->GetCompiledData();

				if(DataChannelData)
				{
					if(CompiledData.UsedByCPU())
					{
						//Grab the buffer we'll be writing into for cpu sims. Must be done on the GT but actual buffer alloc can be done concurrently.
						DestinationData = DataChannelData->GetBufferForCPUWrite();
						if(DestinationData && Interface->AllocationMode == ENiagaraDataChannelAllocationMode::Static)
						{
							DestinationData->Allocate(Interface->AllocationCount);
							
							//We choose whether to zero the CPU buffers or not.
							//By default we only do this when we are calling "Write" (fx.Niagara.DataChannels.WriteDIZeroCPUBuffersMode==1) as calling Append should handle uninitialized buffers fine.
							//However we can use fx.Niagara.DataChannels.WriteDIZeroCPUBuffersMode==2 to do this always or fx.Niagara.DataChannels.WriteDIZeroCPUBuffersMode==0 to never do it.
							bool bZeroBuffers = GbNDCWriteDIZeroCPUBufferMode == 2 || (GbNDCWriteDIZeroCPUBufferMode == 1 && CompiledData.CallsWriteFunction());							
							if(bZeroBuffers)
							{
								DestinationData->ZeroCPUBuffers();
							}
						}
					}
				}

				const FNiagaraDataSetCompiledData& CPUSourceDataCompiledData = DataChannelPtr->GetDataChannel()->GetLayoutInfo()->GetDataSetCompiledData();
				const FNiagaraDataSetCompiledData& GPUSourceDataCompiledData = DataChannelPtr->GetDataChannel()->GetLayoutInfo()->GetDataSetCompiledDataGPU();
				check(CPUSourceDataCompiledData.GetLayoutHash() && CPUSourceDataCompiledData.GetLayoutHash() == GPUSourceDataCompiledData.GetLayoutHash());
				uint64 SourceDataLayoutHash = CPUSourceDataCompiledData.GetLayoutHash();
				bool bChanged = SourceDataLayoutHash != ChachedDataSetLayoutHash;

				//If our CPU or GPU source data has changed then regenerate our binding info.
				//TODO: Multi-source buffer support. 
				//TODO: Variable input layout support. i.e. allow source systems to publish their particle buffers without the need for a separate write.
				if (bChanged)
				{
					ChachedDataSetLayoutHash = SourceDataLayoutHash;

					//We can likely be more targeted here.
					//Could probably only update the RT when the GPU data changes and only update the bindings if the function hashes change etc.
					bUpdateFunctionBindingRTData = CompiledData.UsedByGPU();
					int32 NumFuncs = CompiledData.GetFunctionInfo().Num();
					FunctionToDatSetBindingInfo.SetNum(NumFuncs);
					//FuncToDataSetLayoutKeys.SetNumZeroed(NumFuncs);
					for (int32 BindingIdx = 0; BindingIdx < NumFuncs; ++BindingIdx)
					{
						const FNDIDataChannelFunctionInfo& FuncInfo = CompiledData.GetFunctionInfo()[BindingIdx];

						FNDIDataChannel_FuncToDataSetBindingPtr& BindingPtr = FunctionToDatSetBindingInfo[BindingIdx];
						BindingPtr = FNDIDataChannelLayoutManager::Get().GetLayoutInfo(FuncInfo, CPUSourceDataCompiledData, MissingParams);
					}
				}
			}
		}		

		//Verify our function info.
		if(!ensure(Interface->GetCompiledData().GetFunctionInfo().Num() == FunctionToDatSetBindingInfo.Num()))
		{
			UE_LOGF(LogNiagara, Warning, "Invalid Bindings for Niagara Data Interface Data Channel Write: %ls", *Interface->Channel.GetName());
			return false;			
		}

		for(const auto& Binding : FunctionToDatSetBindingInfo)
		{
			if(Binding.IsValid() == false)
			{
				UE_LOGF(LogNiagara, Warning, "Invalid Bindings for Niagara Data Interface Data Channel Write: %ls", *Interface->Channel.GetName());
				return false;			
			}
		}

		return true;
	}
	
	bool PostSimTick(UNiagaraDataInterfaceDataChannelWrite* Interface, FNiagaraSystemInstance* Instance)
	{
		if (DestinationData)
		{
			//The count here can overrun the num allocated but we should never actually write beyond the max allocated.
			uint32 WrittenInstances = AtomicNumInstances.load(std::memory_order_seq_cst);
			WrittenInstances = FMath::Min(WrittenInstances, DestinationData->GetNumInstancesAllocated());
			DestinationData->SetNumInstances(WrittenInstances);

			if (GbDebugDumpWriter)
			{
				DestinationData->Dump(0, DestinationData->GetNumInstances(), FString::Printf(TEXT("=== Data Channle Write: %d Elements --> %s ==="), DestinationData->GetNumInstances(), *Interface->Channel.GetName()));
			}

			if (DataChannelData && Interface->ShouldPublish() && DestinationData->GetNumInstances() > 0)
			{
				FNiagaraDataChannelPublishRequest PublishRequest(DestinationData->UnlockForRead());
				PublishRequest.bVisibleToGame = Interface->bPublishToGame;
				PublishRequest.bVisibleToCPUSims = Interface->bPublishToCPU;
				PublishRequest.bVisibleToGPUSims = Interface->bPublishToGPU;
				PublishRequest.LwcTile = Instance->GetLWCTile();
#if WITH_NIAGARA_DEBUGGER
				PublishRequest.DebugSource = FString::Format(TEXT("{0} ({1})"), { Instance->GetSystem()->GetName(), GetPathNameSafe(Interface) });
#endif
				DataChannelData->Publish(PublishRequest);
			}
			else
			{
				DestinationData->Unlock();
			}
			
			AtomicNumInstances = 0;
		}
		return true;
	}
};

bool UNiagaraDataInterfaceDataChannelWrite::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIDataChannelWriteInstanceData* InstanceData = new (PerInstanceData) FNDIDataChannelWriteInstanceData;

	//If data channels are disabled we just skip and return ok so that systems can continue to function.
	if (INiagaraModule::DataChannelsEnabled() == false)
	{
		return false;
	}

	if (InstanceData->Init(this, SystemInstance) == false)
	{
		return false;
	}

	return true;
}

void UNiagaraDataInterfaceDataChannelWrite::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIDataChannelWriteInstanceData* InstanceData = static_cast<FNDIDataChannelWriteInstanceData*>(PerInstanceData);
	InstanceData->~FNDIDataChannelWriteInstanceData();

	ENQUEUE_RENDER_COMMAND(RemoveProxy)
	(
		[RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxy_DataChannelWrite>(), InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		}
	);
}


UNiagaraDataInterfaceDataChannelWrite::UNiagaraDataInterfaceDataChannelWrite(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxy_DataChannelWrite());
}

void UNiagaraDataInterfaceDataChannelWrite::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) && INiagaraModule::DataChannelsEnabled())
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowNotUserVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceDataChannelWrite::PostLoad()
{
	Super::PostLoad();

	if(Channel)
	{
		Channel->ConditionalPostLoad();
		UNiagaraDataChannel* NDC = Channel->Get();
		if (NDC)
		{
			const UScriptStruct* CtxStruct = NDC->GetAccessContextType().Get();
			//Special case legacy type. We avoid creating an access context for this type here and just use the old search params path.
			if (CtxStruct == FNDCAccessContextLegacy::StaticStruct())
			{
				AccessContext.Reset();
			}
			else if (AccessContext.GetScriptStruct() != CtxStruct)
			{
				AccessContext.Init(NDC->GetAccessContextType());
			}
		}
		else
		{
			AccessContext.Reset();
		}
	}
	else
	{
		AccessContext.Reset();
	}
}

#if WITH_EDITOR
void UNiagaraDataInterfaceDataChannelWrite::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceDataChannelWrite, Channel))
	{
		UNiagaraDataChannel* NDC = Channel ? Channel->Get() : nullptr;
		if (NDC)
		{
			const UScriptStruct* CtxStruct = NDC->GetAccessContextType().Get();
			//Special case legacy type. We avoid creating an access context for this type here and just use the old search params path.
			if (CtxStruct == FNDCAccessContextLegacy::StaticStruct())
			{
				AccessContext.Reset();
			}
			else if (AccessContext.GetScriptStruct() != CtxStruct)
			{
				AccessContext.Init(NDC->GetAccessContextType());
			}
		}
		else
		{
			AccessContext.Reset();
		}
	}
}
#endif

int32 UNiagaraDataInterfaceDataChannelWrite::PerInstanceDataSize() const
{
	return sizeof(FNDIDataChannelWriteInstanceData);
}

bool UNiagaraDataInterfaceDataChannelWrite::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	if (INiagaraModule::DataChannelsEnabled() == false)
	{
		return true;
	}
	
	SCOPE_CYCLE_COUNTER(STAT_NDIDataChannelWrite_Tick);
	check(SystemInstance);
	FNDIDataChannelWriteInstanceData* InstanceData = static_cast<FNDIDataChannelWriteInstanceData*>(PerInstanceData);
	if (!InstanceData)
	{
		return true;
	}

	if (InstanceData->Tick(this, SystemInstance) == false)
	{
		return true;
	}

	return false;
}

bool UNiagaraDataInterfaceDataChannelWrite::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FNDIDataChannelWriteInstanceData* InstanceData = static_cast<FNDIDataChannelWriteInstanceData*>(PerInstanceData);
	if (!InstanceData)
	{
		return true;
	}

	if (InstanceData->PostSimTick(this, SystemInstance) == false)
	{
		return true;
	}

 	return false;
}

void UNiagaraDataInterfaceDataChannelWrite::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	const FNDIDataChannelWriteInstanceData& SourceData = *reinterpret_cast<const FNDIDataChannelWriteInstanceData*>(PerInstanceData);
	FNDIDataChannelWriteInstanceData_RT* TargetData = new(DataForRenderThread) FNDIDataChannelWriteInstanceData_RT();

	//Always update the dataset, this may change without triggering a full update if it's layout is the same.
	TargetData->ChannelDataRTProxy = SourceData.DataChannelData ? SourceData.DataChannelData->GetRTProxy() : nullptr;

	if (SourceData.bUpdateFunctionBindingRTData && INiagaraModule::DataChannelsEnabled())
	{
		SourceData.bUpdateFunctionBindingRTData = false;

		const FNiagaraDataSetCompiledData& GPUCompiledData = SourceData.DataChannel->GetDataChannel()->GetLayoutInfo()->GetDataSetCompiledDataGPU();
		TargetData->ScriptParamInfo.Init(CompiledData, GPUCompiledData);
	}

	TargetData->AllocationCount = 0;
	if(AllocationMode == ENiagaraDataChannelAllocationMode::Static)
	{
		TargetData->AllocationCount = AllocationCount;
	}
	else if(AllocationMode == ENiagaraDataChannelAllocationMode::Dynamic)
	{
		TargetData->AllocationCount = SourceData.DynamicAllocationCount;
	}

	TargetData->bPublishToGame = bPublishToGame;
	TargetData->bPublishToCPU = bPublishToCPU;
	TargetData->bPublishToGPU = bPublishToGPU;
	TargetData->LwcTile = SourceData.LwcTile;
}

bool UNiagaraDataInterfaceDataChannelWrite::HasTickGroupPostreqs() const
{
	if (Channel && Channel->Get())
	{
		return Channel->Get()->ShouldEnforceTickGroupReadWriteOrder();
	}
	return false;
}

ETickingGroup UNiagaraDataInterfaceDataChannelWrite::CalculateFinalTickGroup(const void* PerInstanceData) const
{
	if(Channel && Channel->Get() && Channel->Get()->ShouldEnforceTickGroupReadWriteOrder())
	{
		return Channel->Get()->GetFinalWriteTickGroup();
	}
	return NiagaraLastTickGroup;
}

#if WITH_EDITORONLY_DATA

void UNiagaraDataInterfaceDataChannelWrite::PostCompile(const UNiagaraSystem& OwningSystem)
{
	CompiledData.Init(&OwningSystem, this);
}

#endif



#if WITH_EDITOR	

void UNiagaraDataInterfaceDataChannelWrite::GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");
	const INiagaraEditorOnlyDataUtilities& EditorOnlyDataUtilities = NiagaraModule.GetEditorOnlyDataUtilities();
	UNiagaraDataInterface* RuntimeInstanceOfThis = InAsset && EditorOnlyDataUtilities.IsEditorDataInterfaceInstance(this)
		? EditorOnlyDataUtilities.GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(*InAsset, *this)
		: this;

	UNiagaraDataInterfaceDataChannelWrite* RuntimeDI = Cast<UNiagaraDataInterfaceDataChannelWrite>(RuntimeInstanceOfThis);

	if (!RuntimeDI)
	{
		return;
	}

	Super::GetFeedback(InAsset, InComponent, OutErrors, OutWarnings, OutInfo);

	if (Channel == nullptr || RuntimeDI->Channel == nullptr)
	{
		OutErrors.Emplace(LOCTEXT("DataChannelMissingFmt", "Data Channel Interface has no valid Data Channel."),
			LOCTEXT("DataChannelMissingErrorSummaryFmt", "Missing Data Channel."),
			FNiagaraDataInterfaceFix());	

		return;
	}

	if(ShouldPublish() == false)
	{
		OutErrors.Emplace(FText::Format(LOCTEXT("DataChannelDoesNotPublishtErrorFmt", "Data Channel {0} does not publish it's data to the Game, CPU Simulations or GPU simulations."), FText::FromName(Channel.GetFName())),
			LOCTEXT("DataChannelDoesNotPublishErrorSummaryFmt", "Data Channel DI does not publish."),
			FNiagaraDataInterfaceFix());
	}

	if (const UNiagaraDataChannel* DataChannel = RuntimeDI->Channel->Get())
	{
		//Ensure the data channel contains all the parameters this function is requesting.
		TConstArrayView<FNiagaraDataChannelVariable> ChannelVars = DataChannel->GetVariables();
		for (const FNDIDataChannelFunctionInfo& FuncInfo : RuntimeDI->GetCompiledData().GetFunctionInfo())
		{
			TArray<FNiagaraVariableBase> MissingParams;

			auto VerifyChannelContainsParams = [&](const TArray<FNiagaraVariableBase>& Parameters)
			{
				for (const FNiagaraVariableBase& FuncParam : Parameters)
				{
					bool bParamFound = false;
					for (const FNiagaraDataChannelVariable& ChannelVar : ChannelVars)
					{
						FNiagaraVariable SWCVar(ChannelVar);

						//We have to convert each channel var to SWC for comparison with the function variables as there is no reliable way to go back from the SWC function var to the originating LWC var.
						if (ChannelVar.GetType().IsEnum() == false)
						{
							UScriptStruct* ChannelSWCStruct = FNiagaraTypeHelper::GetSWCStruct(ChannelVar.GetType().GetScriptStruct());
							if (ChannelSWCStruct)
							{
								FNiagaraTypeDefinition SWCType(ChannelSWCStruct, FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Deny);
								SWCVar = FNiagaraVariable(SWCType, ChannelVar.GetName());
							}
						}

						if (SWCVar == FuncParam)
						{
							bParamFound = true;
							break;
						}
					}

					if (bParamFound == false)
					{
						MissingParams.Add(FuncParam);
					}
				}
			};
			VerifyChannelContainsParams(FuncInfo.Inputs);
			VerifyChannelContainsParams(FuncInfo.Outputs);

			if (MissingParams.Num() > 0)
			{
				FTextBuilder Builder;
				Builder.AppendLineFormat(LOCTEXT("FuncParamMissingFromDataChannelWriteErrorFmt", "Accessing variables that do not exist in Data Channel {0}."), FText::FromName(Channel.GetFName()));
				for (FNiagaraVariableBase& Param : MissingParams)
				{
					Builder.AppendLineFormat(LOCTEXT("FuncParamMissingFromDataChannelWriteErrorLineFmt", "{0} {1}"), Param.GetType().GetNameText(), FText::FromName(Param.GetName()));
				}

				OutErrors.Emplace(Builder.ToText(), LOCTEXT("FuncParamMissingFromDataChannelWriteErrorSummaryFmt", "Data Channel DI function is accessing invalid parameters."), FNiagaraDataInterfaceFix());
			}
		}
	}
	else
	{
		OutErrors.Emplace(FText::Format(LOCTEXT("DataChannelDoesNotExistErrorFmt", "Data Channel {0} does not exist. It may have been deleted."), FText::FromName(Channel.GetFName())),
			LOCTEXT("DataChannelDoesNotExistErrorSummaryFmt", "Data Channel DI is accesssinga a Data Channel that doesn't exist."),
			FNiagaraDataInterfaceFix());
	}
}

void UNiagaraDataInterfaceDataChannelWrite::ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors)
{
	Super::ValidateFunction(Function, OutValidationErrors);

	//It would be great to be able to validate the parameters on the function calls here but this is only called on the DI CDO. We don't have the context of which data channel we'll be accessing.
	//The translator should have all the required data to use the actual DIs when validating functions. We just need to do some wrangling to pull it from the pre compiled data correctly.
	//This would probably also allow us to actually call hlsl generation functions on the actual DIs rather than their CDOs. Which would allow for a bunch of better optimized code gen for things like fluids.
	//TODO!!!
}

#endif

bool UNiagaraDataInterfaceDataChannelWrite::Equals(const UNiagaraDataInterface* Other)const
{
	if (const UNiagaraDataInterfaceDataChannelWrite* OtherTyped = CastChecked<UNiagaraDataInterfaceDataChannelWrite>(Other))
	{
		if (Super::Equals(Other) &&
			AllocationMode == OtherTyped->AllocationMode &&
			AllocationCount == OtherTyped->AllocationCount &&
			bPublishToGame == OtherTyped->bPublishToGame &&
			bPublishToCPU == OtherTyped->bPublishToCPU &&
			bPublishToGPU == OtherTyped->bPublishToGPU &&
			Channel == OtherTyped->Channel &&
			AccessContext == OtherTyped->AccessContext &&
			bOnlyWriteOnceOnSubticks == OtherTyped->bOnlyWriteOnceOnSubticks &&
			bUpdateDestinationDataEveryTick == OtherTyped->bUpdateDestinationDataEveryTick)
		{
			return true;
		}
	}
	return false;
}

UObject* UNiagaraDataInterfaceDataChannelWrite::SimCacheBeginWrite(UObject* SimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	UNDIDataChannelWriteSimCacheData* NewSimCacheStorage = NewObject<UNDIDataChannelWriteSimCacheData>(SimCache);
	NewSimCacheStorage->DataInterface = this;
	NewSimCacheStorage->InstanceID = NiagaraSystemInstance->GetId();

	const FNiagaraDataInterfaceProxy_DataChannelWrite* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxy_DataChannelWrite>();
	if(RT_Proxy)
	{
		ENQUEUE_RENDER_COMMAND(NDISimCacheGPUBeginWrite)
			(
				[RT_Proxy, RT_InstanceID = NiagaraSystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
				{
					const FNiagaraDataInterfaceProxy_DataChannelWrite::FInstanceData* InstanceData = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID);
					if (InstanceData)
					{
						InstanceData->bCapturingSimCache = true;
					}
				}
			);
	}
	return NewSimCacheStorage;
}

bool UNiagaraDataInterfaceDataChannelWrite::SimCacheEndWrite(UObject* StorageObject) const
{
	if (UNDIDataChannelWriteSimCacheData* Storage = Cast<UNDIDataChannelWriteSimCacheData>(StorageObject))
	{
		if(Storage->DataInterface)
		{
			const FNiagaraDataInterfaceProxy_DataChannelWrite* RT_Proxy = Storage->DataInterface->GetProxyAs<FNiagaraDataInterfaceProxy_DataChannelWrite>();
			if(RT_Proxy)
			{
				ENQUEUE_RENDER_COMMAND(NDISimCacheGPUBeginWrite)
				(
					[RT_Proxy, RT_InstanceID = Storage->InstanceID](FRHICommandListImmediate& RHICmdList)
					{
						const FNiagaraDataInterfaceProxy_DataChannelWrite::FInstanceData* InstanceData = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID);
						if (InstanceData)
						{
							InstanceData->bCapturingSimCache = false;
						}
					}
				);
			}
		}

		Storage->DataInterface = nullptr;
		Storage->InstanceID = 0;
	}

	return true;
}

bool UNiagaraDataInterfaceDataChannelWrite::SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	if (!Channel || !Channel->Get())
	{
		FeedbackContext.Errors.Add(TEXT("Missing data channel asset for data channel writer DI"));
		return false;
	}
	if (OptionalPerInstanceData == nullptr)
	{
		FeedbackContext.Errors.Add(TEXT("Missing per instance data for data channel writer DI"));
		return false;
	}

	UWorld* World = SystemInstance->GetWorld();
	if(World == nullptr)
	{
		FeedbackContext.Errors.Add(TEXT("Missing world for data channel writer DI's System Instace."));
		return false;
	}


	const FNDIDataChannelWriteInstanceData* InstanceData = static_cast<const FNDIDataChannelWriteInstanceData*>(OptionalPerInstanceData);
	if (UNDIDataChannelWriteSimCacheData* Storage = Cast<UNDIDataChannelWriteSimCacheData>(StorageObject))
	{
		ensure(Storage->FrameData.Num() == FrameIndex);
		Storage->DataChannelReference = Channel.Get();
		FNDIDataChannelWriteSimCacheFrame& FrameData = Storage->FrameData.AddDefaulted_GetRef();

		if (InstanceData->DataChannelData && ShouldPublish())
		{
			FNiagaraDataChannelGameData GameData(Channel->Get()->GetLayoutInfo());

			if(InstanceData->DestinationData && InstanceData->DestinationData->GetNumInstances() > 0)
			{
				GameData.AppendFromDataSet(InstanceData->DestinationData, SystemInstance->GetLWCTile());
			}

			FNiagaraGpuComputeDispatchInterface*  DispatchInterface = FNiagaraGpuComputeDispatchInterface::Get(World);
			if (CompiledData.UsedByGPU() && DispatchInterface)
			{
				const FNiagaraDataInterfaceProxy_DataChannelWrite* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxy_DataChannelWrite>();

				TArray<TSharedPtr<FNiagaraDataBufferReadback, ESPMode::ThreadSafe>> FrameReadbacks;

				ENQUEUE_RENDER_COMMAND(NDISimCacheGPUWriteFrame)
				(
					[RT_Proxy, RT_InstanceID = SystemInstance->GetId(), NDCData_RT = InstanceData->DataChannelData, &FrameReadbacks, DispatchInterface](FRHICommandListImmediate& RHICmdList)
					{
						const FNiagaraDataInterfaceProxy_DataChannelWrite::FInstanceData* InstanceData_RT = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID);
						if (InstanceData_RT)
						{
							//Instance data already has a buffer we can use to read back this frame's data.
							for (const FNiagaraDataBufferRef& ReadbackBuffer : InstanceData_RT->PendingSimCacheReadbacks)
							{
								//We enqueue a readback for the data if we're wanting to store in the sim cache
								TSharedPtr<FNiagaraDataBufferReadback, ESPMode::ThreadSafe>& NewReadback = FrameReadbacks.Add_GetRef(MakeShared<FNiagaraDataBufferReadback, ESPMode::ThreadSafe>());
								//TODO: Rework to avoid additional readback. With some refactoring we can just re-use the existing readback if available.
								NewReadback->EnqueueReadback(RHICmdList, ReadbackBuffer, DispatchInterface->GetGpuReadbackManager(), DispatchInterface->GetGPUInstanceCounterManager());
							}
							InstanceData_RT->PendingSimCacheReadbacks.Reset();
						}

						DispatchInterface->GetGpuReadbackManager()->WaitCompletion(RHICmdList);
					}
				);

				//Is it enough that we flush the RT here?
				//Can we be sure all work has been submitted and the dispatcher has finished processing the frame etc?
				FlushRenderingCommands();

				for(TSharedPtr<FNiagaraDataBufferReadback, ESPMode::ThreadSafe>& Readback : FrameReadbacks)
				{
					//TODO: Direct copy path, for now copy over to a data set then to the frame data.
					FNiagaraDataBuffer* ReadbackBuffer = InstanceData->DataChannelData->GetBufferForCPUWrite();
					Readback->ReadResultsToDataBuffer(ReadbackBuffer);
					GameData.AppendFromDataSet(ReadbackBuffer->UnlockForRead(), SystemInstance->GetLWCTile());
				}
			}

			FrameData.NumElements = GameData.Num();
			for (const FNiagaraDataChannelVariableBuffer& VarBuffer : GameData.GetVariableBuffers())
			{
				FNDIDataChannelWriteSimCacheFrameBuffer& FrameBuffer = FrameData.VariableData.AddDefaulted_GetRef();
				FrameBuffer.Size = VarBuffer.Size;
				FrameBuffer.Data = VarBuffer.Data;
			}
			const FNiagaraDataChannelGameDataLayout& Layout = Channel->Get()->GetLayoutInfo()->GetGameDataLayout();
			for (const TPair<FNiagaraVariableBase, int32>& VarPair : Layout.VariableIndices)
			{
				FrameData.VariableData[VarPair.Value].SourceVar = VarPair.Key;
			}

			FrameData.bVisibleToGame = bPublishToGame;
			FrameData.bVisibleToCPUSims = bPublishToCPU;
			FrameData.bVisibleToGPUSims = bPublishToGPU;
		}

		return true;
	} 
	return false;
}

bool UNiagaraDataInterfaceDataChannelWrite::SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext)
{
	if (Channel == nullptr || Channel->Get() == nullptr)
	{
		return false;
	}

	FNiagaraSystemInstance* SystemInstance = ReadContext.GetSystemInstance();
	UNiagaraDataChannel* NDC = Channel->Get();

	FNiagaraDataChannelDataPtr DataChannelData;
	if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(SystemInstance->GetWorld()))
	{
		if (UNiagaraDataChannelHandler* Handler = WorldMan->GetDataChannelManager().FindDataChannelHandler(NDC))
		{
			if(AccessContext.IsValid() && AccessContext.GetScriptStruct() != FNDCAccessContextLegacy::StaticStruct())
			{
				FNDCAccessContextInst& TransientAccessContext = NDC->GetTransientAccessContext();
				TransientAccessContext = AccessContext;
				FNDCAccessContextBase& TypedContext = TransientAccessContext.GetChecked<FNDCAccessContextBase>();
				TypedContext.OwningComponent = SystemInstance->GetAttachComponent();
				DataChannelData = Handler->FindData(TransientAccessContext, ENiagaraResourceAccess::WriteOnly);
			}
			else
			{
				FNiagaraDataChannelSearchParameters SearchParams(SystemInstance->GetAttachComponent());
				DataChannelData = Handler->FindData(SearchParams, ENiagaraResourceAccess::WriteOnly);
			}
		}
	}
	if (!DataChannelData.IsValid())
	{
		return false;
	}
	
	if (UNDIDataChannelWriteSimCacheData* Storage = ReadContext.GetOptionalStorageObject<UNDIDataChannelWriteSimCacheData>())
	{
		const int32 FrameIndex = ReadContext.GetFrameIndexA();
		if (Storage->FrameData.IsValidIndex(FrameIndex))
		{
			FNDIDataChannelWriteSimCacheFrame& Frame = Storage->FrameData[FrameIndex];
			FNiagaraDataChannelPublishRequest PublishRequest;
			PublishRequest.bVisibleToGame = Frame.bVisibleToGame;
			PublishRequest.bVisibleToCPUSims = Frame.bVisibleToCPUSims;
			PublishRequest.bVisibleToGPUSims = Frame.bVisibleToGPUSims;
#if WITH_NIAGARA_DEBUGGER
			PublishRequest.DebugSource = FString::Format(TEXT("{0} (Sim cache {1})"), {SystemInstance->GetSystem()->GetName(), GetPathNameSafe(Storage->GetOuter())});
#endif

			PublishRequest.GameData = MakeShared<FNiagaraDataChannelGameData>(NDC->GetLayoutInfo());
			PublishRequest.GameData->SetNum(Frame.NumElements);
			PublishRequest.LwcTile = ReadContext.GetLWCTileA();

			if (ReadContext.ShouldRebaseData(true))
			{
				const FNiagaraTypeDefinition& PositionTypeDef = FNiagaraTypeDefinition::GetPositionDef();
				for (int32 i = 0; i < Frame.VariableData.Num(); ++i)
				{
					const FNDIDataChannelWriteSimCacheFrameBuffer& Buffer = Frame.VariableData[i];
					if (Buffer.SourceVar.GetType() == PositionTypeDef)
					{
						check(Buffer.Size == sizeof(FVector));

						//-OPT: Could set directly to avoid the copy
						TArray<uint8> TempData;
						TempData.AddUninitialized(Buffer.Data.Num());
						for (int32 Element=0; Element < Frame.NumElements; ++Element)
						{
							FVector Position;
							FMemory::Memcpy(&Position, Buffer.Data.GetData() + (sizeof(FVector) * Element), sizeof(FVector));
							Position  = ReadContext.GetRebaseTransformA().TransformPosition(Position);
							FMemory::Memcpy(TempData.GetData() + (sizeof(FVector) * Element), &Position, sizeof(FVector));
						}
						PublishRequest.GameData->SetFromSimCache(Buffer.SourceVar, TempData, sizeof(FVector));
					}
					else
					{
						PublishRequest.GameData->SetFromSimCache(Buffer.SourceVar, Buffer.Data, Buffer.Size);
					}
				}
			}
			else
			{
				for (int32 i = 0; i < Frame.VariableData.Num(); ++i)
				{
					const FNDIDataChannelWriteSimCacheFrameBuffer& Buffer = Frame.VariableData[i];
					PublishRequest.GameData->SetFromSimCache(Buffer.SourceVar, Buffer.Data, Buffer.Size);
				}
			}
			
			DataChannelData->Publish(PublishRequest);
			return true;
		}
	}
	return false;
}

void UNiagaraDataInterfaceDataChannelWrite::SimCachePostReadFrame(void* OptionalPerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	// send data to data channel
}

bool UNiagaraDataInterfaceDataChannelWrite::SimCacheCompareFrame(const UObject* LhsStorageObject, const UObject* RhsStorageObject, int FrameIndex, TOptional<float> Tolerance, FString& OutErrors) const
{
	const UNDIDataChannelWriteSimCacheData* Storage1 = Cast<const UNDIDataChannelWriteSimCacheData>(LhsStorageObject);
	const UNDIDataChannelWriteSimCacheData* Storage2 = Cast<const UNDIDataChannelWriteSimCacheData>(RhsStorageObject);

	if (Storage1 == nullptr && Storage2 == nullptr)
	{
		return true;
	}
	if (Storage1 == nullptr || Storage2 == nullptr)
	{
		OutErrors = TEXT("Recevied nullptr storage object for comparison");
		return false;
	}
	if (Storage1->DataChannelReference != Storage2->DataChannelReference)
	{
		OutErrors = TEXT("Different source data channel assets");
		return false;
	}
	if (Storage1->FrameData.Num() != Storage2->FrameData.Num())
	{
		OutErrors = FString::Format(TEXT("Different frame data count. {0} vs {1}"), {Storage1->FrameData.Num(), Storage2->FrameData.Num()});
		return false;
	}

	bool bEqual = true;
	for (int i = 0; i < Storage1->FrameData.Num(); i++)
	{
		const FNDIDataChannelWriteSimCacheFrame& Frame1 = Storage1->FrameData[i];
		const FNDIDataChannelWriteSimCacheFrame& Frame2 = Storage2->FrameData[i];

		if (Frame1.NumElements != Frame2.NumElements)
		{
			bEqual = false;
			OutErrors += FString::Format(TEXT("Frame {0}: different number of elements in data channel store, {1} vs {2}\n"), {i, Frame1.NumElements, Frame2.NumElements});
		}
		else if (Frame1.VariableData.Num() != Frame2.VariableData.Num())
		{
			bEqual = false;
			OutErrors += FString::Format(TEXT("Frame {0}: different number of variables in data channel store, {1} vs {2}\n"), {i, Frame1.VariableData.Num(), Frame2.VariableData.Num()});
		}
		else
		{
			for (int k = 0; k < Frame1.VariableData.Num(); k++)
			{
				const FNDIDataChannelWriteSimCacheFrameBuffer& Buffer1 = Frame1.VariableData[k];
				const FNDIDataChannelWriteSimCacheFrameBuffer& Buffer2 = Frame2.VariableData[k];
				if (Buffer1.SourceVar != Buffer2.SourceVar || Buffer1.Data != Buffer2.Data)
				{
					OutErrors += FString::Format(TEXT("Frame {0}: different buffers in data channel store for source var {1}\n"), {i, *Buffer1.SourceVar.GetName().ToString()});
				}
			}
		}
	}
	return bEqual;
}

bool UNiagaraDataInterfaceDataChannelWrite::CopyToInternal(UNiagaraDataInterface* Destination)const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	if (UNiagaraDataInterfaceDataChannelWrite* DestTyped = CastChecked<UNiagaraDataInterfaceDataChannelWrite>(Destination))
	{
		DestTyped->AllocationMode = AllocationMode;
		DestTyped->AllocationCount = AllocationCount;
		DestTyped->bPublishToGame = bPublishToGame;
		DestTyped->bPublishToCPU = bPublishToCPU;
		DestTyped->bPublishToGPU = bPublishToGPU;
		DestTyped->Channel = Channel;
		DestTyped->AccessContext = AccessContext;
		DestTyped->CompiledData = CompiledData;
		DestTyped->bUpdateDestinationDataEveryTick = bUpdateDestinationDataEveryTick;
		DestTyped->bOnlyWriteOnceOnSubticks = bOnlyWriteOnceOnSubticks;
		return true;
	}

	return false;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceDataChannelWrite::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	OutFunctions.Add(NDIDataChannelWriteLocal::GetFunctionSig_Num());
	//OutFunctions.Add(NDIDataChannelWriteLocal::GetFunctionSig_Allocate());
	OutFunctions.Add(NDIDataChannelWriteLocal::GetFunctionSig_Write());
	OutFunctions.Add(NDIDataChannelWriteLocal::GetFunctionSig_Append());
}
#endif

void UNiagaraDataInterfaceDataChannelWrite::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	if (BindingInfo.Name == NDIDataChannelWriteLocal::GetFunctionSig_Num().Name)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelWrite::Num);
	}
// 	else if (BindingInfo.Name == NDIDataChannelWriteLocal::GetFunctionSig_Allocate().Name)
// 	{
//		OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelWrite::Allocate);
// 	}
	else
	{
		int32 FuncIndex = CompiledData.FindFunctionInfoIndex(BindingInfo.Name, BindingInfo.VariadicInputs, BindingInfo.VariadicOutputs);
		if (BindingInfo.Name == NDIDataChannelWriteLocal::GetFunctionSig_Write().Name)
		{
			OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelWrite::Write, FuncIndex);
		}
		else 
		if (BindingInfo.Name == NDIDataChannelWriteLocal::GetFunctionSig_Append().Name)
		{
			OutFunc.BindUObject(this, &UNiagaraDataInterfaceDataChannelWrite::Append, FuncIndex);
		}
		else
		{
			UE_LOGF(LogTemp, Display, "Could not find data interface external function in %ls. Received Name: %ls", *GetPathNameSafe(this), *BindingInfo.Name.ToString());
		}
	}
}

void UNiagaraDataInterfaceDataChannelWrite::InitAccessContext()
{
	UNiagaraDataChannel* NDC = Channel ? Channel->Get() : nullptr;
	if (NDC)
	{
		const UScriptStruct* CtxStruct = NDC->GetAccessContextType().Get();
		//Special case legacy type. We avoid creating an access context for this type here and just use the old search params path.
		if (CtxStruct == FNDCAccessContextLegacy::StaticStruct())
		{
			AccessContext.Reset();
		}
		else if (AccessContext.IsValid() == false || CtxStruct != AccessContext.GetScriptStruct())
		{
			AccessContext.Init(NDC->GetAccessContextType());
		}
	}
	else
	{
		AccessContext.Reset();
	}
}

void UNiagaraDataInterfaceDataChannelWrite::Num(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIDataChannelWriteInstanceData> InstData(Context);

	FNDIOutputParam<int32> OutNum(Context);

	FNiagaraDataBuffer* Buffer = InstData->DestinationData;	
	int32 Num = 0;
	if (Buffer && INiagaraModule::DataChannelsEnabled())
	{
		Num = (int32)Buffer->GetNumInstancesAllocated();
	}

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutNum.SetAndAdvance(Num);
	}
}

void UNiagaraDataInterfaceDataChannelWrite::Allocate(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIDataChannelWriteInstanceData> InstData(Context);

	FNDIInputParam<FNiagaraEmitterID> InEmitterID(Context);
	FNDIInputParam<int32> InAllocationCount(Context);

	check(Context.GetNumInstances() == 1);

	uint32 Count = InAllocationCount.GetAndAdvance();
	
	//Store the count so we can pass this to the GPU for allocating space in the main GPU write buffer.
	InstData->DynamicAllocationCount += Count;
	
	//If we have a CPU write buffer, allocate that now. Do this here so the emitter/system script itself can write data if it wants to.
	if(InstData->DestinationData)
	{
		InstData->DestinationData->Allocate(Count, true);//On the off chance we alloc->write->alloc, re-alloc and keep existing data.
	}
}

void UNiagaraDataInterfaceDataChannelWrite::Write(FVectorVMExternalFunctionContext& Context, int32 FuncIdx)
{
	SCOPE_CYCLE_COUNTER(STAT_NDIDataChannelWrite_Write);
	VectorVM::FUserPtrHandler<FNDIDataChannelWriteInstanceData> InstData(Context);
	FNDIInputParam<FNiagaraBool> InEmit(Context);
	FNDIInputParam<int32> InIndex(Context);

	const FNDIDataChannel_FunctionToDataSetBinding* BindingInfo = InstData->FunctionToDatSetBindingInfo.IsValidIndex(FuncIdx) ? InstData->FunctionToDatSetBindingInfo[FuncIdx].Get() : nullptr;
	FNDIVariadicInputHandler<16> VariadicInputs(Context, BindingInfo);//TODO: Make static / avoid allocation

	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	std::atomic<uint32>& AtomicNumInstances = InstData->AtomicNumInstances;
	bool bProcessCurrentTick = true;
	if (bOnlyWriteOnceOnSubticks && InstData->Owner)
	{
		const FNiagaraTickInfo& TickInfo = InstData->Owner->GetSystemSimulation()->GetTickInfo();
		bProcessCurrentTick = TickInfo.TickNumber == TickInfo.TickCount - 1;
	}

	bool bAllFailedFallback = true;
	if (InstData->DestinationData && BindingInfo && INiagaraModule::DataChannelsEnabled() && bProcessCurrentTick)
	{
		if(FNiagaraDataBuffer* Data = InstData->DestinationData)
		{			
			bAllFailedFallback = false;
			int32 MaxLocalIndex = INDEX_NONE;
			int32 NumAllocated = IntCastChecked<int32>(Data->GetNumInstancesAllocated());
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				int32 Index = InIndex.GetAndAdvance();
				bool bEmit = InEmit.GetAndAdvance() && Index >= 0 && Index < NumAllocated;

				if(!bEmit)
				{
					VariadicInputs.Advance(1);
					if (OutSuccess.IsValid())
					{
						OutSuccess.SetAndAdvance(false);
					}
					continue;
				}

				MaxLocalIndex = FMath::Max(Index, MaxLocalIndex);

				bool bAllWritesSuccess = true;

				//TODO: Optimize case where emit is constant
				//TODO: Optimize for runs of sequential true emits.
				auto FloatFunc = [Data, Index, &bAllWritesSuccess](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<float>& FloatData)
				{
					if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
					{
						*Data->GetInstancePtrFloat(VMBinding.GetDataSetRegisterIndex(), Index) = FloatData.GetAndAdvance();
					}
					else
					{
						bAllWritesSuccess = false;
					}
				};
				auto IntFunc = [Data, Index, &bAllWritesSuccess](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<int32>& IntData)
				{
					if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
					{
						*Data->GetInstancePtrInt32(VMBinding.GetDataSetRegisterIndex(), Index) = IntData.GetAndAdvance();
					}
					else
					{
						bAllWritesSuccess = false;
					}
				};
				auto HalfFunc = [Data, Index, &bAllWritesSuccess](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<FFloat16>& HalfData)
				{
					if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
					{
						*Data->GetInstancePtrHalf(VMBinding.GetDataSetRegisterIndex(), Index) = HalfData.GetAndAdvance();
					}
					else
					{
						bAllWritesSuccess = false;
					}
				};

				bool bFinalSuccess = VariadicInputs.Process(bEmit, 1, BindingInfo, FloatFunc, IntFunc, HalfFunc) && bAllWritesSuccess;

				if (OutSuccess.IsValid())
				{
					OutSuccess.SetAndAdvance(bFinalSuccess);
				}
			}

			if(MaxLocalIndex != INDEX_NONE)
			{
				//Update the shared instance count with an updated max.
				uint32 CurrNumInstances = AtomicNumInstances;
				uint32 MaxLocalNumInstances = MaxLocalIndex + 1;
				while (CurrNumInstances < MaxLocalNumInstances && !AtomicNumInstances.compare_exchange_weak(CurrNumInstances, MaxLocalNumInstances))
				{
					CurrNumInstances = AtomicNumInstances;
				}
			}
		}
	}

	if (bAllFailedFallback)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			if (OutSuccess.IsValid())
			{
				OutSuccess.SetAndAdvance(false);
			}
		}
	}
}

void UNiagaraDataInterfaceDataChannelWrite::Append(FVectorVMExternalFunctionContext& Context, int32 FuncIdx)
{
	SCOPE_CYCLE_COUNTER(STAT_NDIDataChannelWrite_Append);
	VectorVM::FUserPtrHandler<FNDIDataChannelWriteInstanceData> InstData(Context);
	FNDIInputParam<FNiagaraBool> InEmit(Context);

	const FNDIDataChannel_FunctionToDataSetBinding* BindingInfo = InstData->FunctionToDatSetBindingInfo.IsValidIndex(FuncIdx) ? InstData->FunctionToDatSetBindingInfo[FuncIdx].Get() : nullptr;
	FNDIVariadicInputHandler<16> VariadicInputs(Context, BindingInfo);//TODO: Make static / avoid allocation

	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	std::atomic<uint32>& AtomicNumInstances = InstData->AtomicNumInstances;

	bool bProcessCurrentTick = true;
	if (bOnlyWriteOnceOnSubticks && InstData->Owner)
	{
		const FNiagaraTickInfo& TickInfo = InstData->Owner->GetSystemSimulation()->GetTickInfo();
		bProcessCurrentTick = TickInfo.TickNumber == TickInfo.TickCount - 1;
	}

	bool bAllFailedFallback = true;
	if (InstData->DestinationData && BindingInfo && INiagaraModule::DataChannelsEnabled() && bProcessCurrentTick)
	{
		if(FNiagaraDataBuffer* Data = InstData->DestinationData)
		{
			//Get the total number to emit.
			//Allows going via a faster write path if we're emiting every instance.
			//Also needed to update the atomic num instances and get our start index for writing.
			uint32 LocalNumToEmit = 0;
			if(InEmit.IsConstant())
			{
				bool bEmit = InEmit.GetAndAdvance();
				LocalNumToEmit = bEmit ? Context.GetNumInstances() : 0;
			}
			else
			{
				for (int32 i = 0; i < Context.GetNumInstances(); ++i)
				{
					if (InEmit.GetAndAdvance())
					{
						++LocalNumToEmit;
					}
				}
			}			

			if (LocalNumToEmit > 0)
			{
				uint32 NumAllocated = Data->GetNumInstancesAllocated();
				InEmit.Reset();

				//Update the shared atomic instance count and grab the current index at which we can write.
				uint32 CurrNumInstances = AtomicNumInstances.fetch_add(LocalNumToEmit);

				bAllFailedFallback = false;

				bool bEmitAll = LocalNumToEmit == Context.GetNumInstances();

				if(bEmitAll)
				{
					//limit the number to emit so we do not write over the end of the buffers.
					uint32 MaxWriteCount = NumAllocated - FMath::Min(CurrNumInstances, NumAllocated);
					LocalNumToEmit = FMath::Min(LocalNumToEmit, MaxWriteCount);

					//If we're writing all instances then we can do a memcpy instead of slower loop copies.
					bool bAllWritesSuccess = true;
					uint32 Index = CurrNumInstances;
					auto FloatFunc = [Data, Index, LocalNumToEmit, &bAllWritesSuccess](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<float>& FloatData)
					{
						if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
						{
							float* Dest = Data->GetInstancePtrFloat(VMBinding.GetDataSetRegisterIndex(), Index);
							if (FloatData.IsConstant())
							{
								float Value = FloatData.GetAndAdvance();
								for(uint32 i=0; i<LocalNumToEmit; ++i){ Dest[i] = Value; }
							}
							else
							{
								 const float* Src = FloatData.Data.GetDest();
								 FMemory::Memcpy(Dest, Src, LocalNumToEmit * sizeof(float));
							}
						}
						else
						{
							bAllWritesSuccess = false;
						}
					};
					auto IntFunc = [Data, Index, LocalNumToEmit, &bAllWritesSuccess](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<int32>& IntData)
					{
						if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
						{
							int32* Dest = Data->GetInstancePtrInt32(VMBinding.GetDataSetRegisterIndex(), Index);
							if (IntData.IsConstant())
							{
								int32 Value = IntData.GetAndAdvance();
								for (uint32 i = 0; i < LocalNumToEmit; ++i) { Dest[i] = Value; }
							}
							else
							{
								const int32* Src = IntData.Data.GetDest();
								FMemory::Memcpy(Dest, Src, LocalNumToEmit * sizeof(int32));
							}
						}
						else
						{
							bAllWritesSuccess = false;
						}
					};
					auto HalfFunc = [Data, Index, LocalNumToEmit, &bAllWritesSuccess](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<FFloat16>& HalfData)
					{
						if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
						{
							FFloat16* Dest = Data->GetInstancePtrHalf(VMBinding.GetDataSetRegisterIndex(), Index);
							if (HalfData.IsConstant())
							{
								FFloat16 Value = HalfData.GetAndAdvance();
								for (uint32 i = 0; i < LocalNumToEmit; ++i) { Dest[i] = Value; }
							}
							else
							{
								const FFloat16* Src = HalfData.Data.GetDest();
								FMemory::Memcpy(Dest, Src, LocalNumToEmit * sizeof(FFloat16));
							}
						}
						else
						{
							bAllWritesSuccess = false;
						}
					};

					bool bFinalSuccess = VariadicInputs.Process(true, Context.GetNumInstances(), BindingInfo, FloatFunc, IntFunc, HalfFunc) && bAllWritesSuccess;

					if (OutSuccess.IsValid())
					{
						for (int32 i = 0; i < Context.GetNumInstances(); ++i)
						{
							OutSuccess.SetAndAdvance(bFinalSuccess);
						}
					}
				}
				else
				{
					for (int32 i = 0; i < Context.GetNumInstances() && CurrNumInstances < NumAllocated; ++i)
					{						
						uint32 Index = CurrNumInstances;

						bool bEmit = InEmit.GetAndAdvance();
						bool bAllWritesSuccess = true;

						if(bEmit)
						{
							++CurrNumInstances;
						}

						auto FloatFunc = [Data, Index, &bAllWritesSuccess](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<float>& FloatData)
						{
							if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
							{
								*Data->GetInstancePtrFloat(VMBinding.GetDataSetRegisterIndex(), Index) = FloatData.GetAndAdvance();
							}
							else
							{
								bAllWritesSuccess = false;
							}
						};
						auto IntFunc = [Data, Index, &bAllWritesSuccess](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<int32>& IntData)
						{
							if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
							{
								*Data->GetInstancePtrInt32(VMBinding.GetDataSetRegisterIndex(), Index) = IntData.GetAndAdvance();
							}
							else
							{
								bAllWritesSuccess = false;
							}
						};
						auto HalfFunc = [Data, Index, &bAllWritesSuccess](const FNDIDataChannelRegisterBinding& VMBinding, FNDIInputParam<FFloat16>& HalfData)
						{
							if (VMBinding.GetDataSetRegisterIndex() != INDEX_NONE)
							{
								*Data->GetInstancePtrHalf(VMBinding.GetDataSetRegisterIndex(), Index) = HalfData.GetAndAdvance();
							}
							else
							{
								bAllWritesSuccess = false;
							}
						};

						bool bFinalSuccess = VariadicInputs.Process(bEmit, 1, BindingInfo, FloatFunc, IntFunc, HalfFunc) && bAllWritesSuccess;

						if (OutSuccess.IsValid())
						{
							OutSuccess.SetAndAdvance(bFinalSuccess);
						}
					}
				}
			}
		}
	}

	if(bAllFailedFallback)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			if (OutSuccess.IsValid())
			{
				OutSuccess.SetAndAdvance(false);
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceDataChannelWrite::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{ 
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIDataChannelWriteLocal::CommonShaderFile);
	bSuccess &= InVisitor->UpdateShaderFile(NDIDataChannelWriteLocal::TemplateShaderFile_Common);
	bSuccess &= InVisitor->UpdateShaderFile(NDIDataChannelWriteLocal::TemplateShaderFile_WriteCommon);
	//bSuccess &= InVisitor->UpdateShaderFile(NDIDataChannelWriteLocal::TemplateShaderFile_Write);
	bSuccess &= InVisitor->UpdateShaderFile(NDIDataChannelWriteLocal::TemplateShaderFile_Append);
 
 	bSuccess &= InVisitor->UpdateShaderParameters<NDIDataChannelWriteLocal::FShaderParameters>();
 	return bSuccess;
}

void UNiagaraDataInterfaceDataChannelWrite::GetCommonHLSL(FString& OutHLSL)
{
 	Super::GetCommonHLSL(OutHLSL);
 	OutHLSL.Appendf(TEXT("#include \"%s\"\n"), NDIDataChannelWriteLocal::CommonShaderFile);
}

bool UNiagaraDataInterfaceDataChannelWrite::GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL)
{
 	return	HlslGenContext.GetFunctionInfo().DefinitionName == GET_FUNCTION_NAME_CHECKED(UNiagaraDataInterfaceDataChannelWrite, Num) ||
 		//HlslGenContext.GetFunctionInfo().DefinitionName == GET_FUNCTION_NAME_CHECKED(UNiagaraDataInterfaceDataChannelWrite, Write) ||
 		HlslGenContext.GetFunctionInfo().DefinitionName == GET_FUNCTION_NAME_CHECKED(UNiagaraDataInterfaceDataChannelWrite, Append);
}

void UNiagaraDataInterfaceDataChannelWrite::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(HlslGenContext, OutHLSL);

	TArray<FString> CommonTemplateShaders;
	TMap<FName, FString> TemplateShaderMap;
	NDIDataChannelWriteLocal::BuildFunctionTemplateMap(CommonTemplateShaders, TemplateShaderMap);

	NDIDataChannelUtilities::GenerateDataChannelAccessHlsl(HlslGenContext, CommonTemplateShaders, TemplateShaderMap, OutHLSL);
}

#endif
void UNiagaraDataInterfaceDataChannelWrite::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIDataChannelWriteLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceDataChannelWrite::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxy_DataChannelWrite& DataInterfaceProxy = Context.GetProxy<FNiagaraDataInterfaceProxy_DataChannelWrite>();
	FNiagaraDataInterfaceProxy_DataChannelWrite::FInstanceData* InstanceData = DataInterfaceProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

	NDIDataChannelWriteLocal::FShaderParameters* InstParameters = Context.GetParameterNestedStruct<NDIDataChannelWriteLocal::FShaderParameters>();

	bool bSuccess = false;
	if (InstanceData)
	{
		//Find the start offset in the parameter table for this script.
		const FNiagaraCompileHash& ScriptCompileHash = Context.GetComputeInstanceData().Context->GPUScript_RT->GetBaseCompileHash();
		uint32 ParameterOffsetTableIndex = INDEX_NONE;
		if (uint32* ParameterOffsetTableIndexPtr = InstanceData->GPUScriptParameterTableOffsets.Find(ScriptCompileHash))
		{
			ParameterOffsetTableIndex = *ParameterOffsetTableIndexPtr;
		}

		if (InstanceData->ChannelDataRTProxy && ParameterOffsetTableIndex != INDEX_NONE)
		{
			FNiagaraDataBuffer* GPUBuffer = InstanceData->GPUBuffer.GetReference();
			FNiagaraDataBuffer* BufferForCPU = InstanceData->BufferForCPU.GetReference();
			if (GPUBuffer || BufferForCPU)
			{
				const FReadBuffer& ParameterLayoutBuffer = InstanceData->ParameterLayoutBuffer;

				if (ParameterLayoutBuffer.SRV.IsValid() && ParameterLayoutBuffer.NumBytes > 0)
				{
					InstParameters->ParamOffsetTable = ParameterLayoutBuffer.SRV.IsValid() ? ParameterLayoutBuffer.SRV.GetReference() : FNiagaraRenderer::GetDummyUIntBuffer();
					InstParameters->ParameterOffsetTableIndex = ParameterOffsetTableIndex;
					InstParameters->FloatStride = (GPUBuffer ? GPUBuffer->GetFloatStride() : sizeof(float)) / sizeof(float);
					InstParameters->Int32Stride = (GPUBuffer ? GPUBuffer->GetInt32Stride() : sizeof(int32)) / sizeof(int32);
					//TODO: Half Support | InstParameters->HalfStride = (GPUBuffer ? GPUBuffer->GetHalfStride() : sizeof(FFloat16)) / sizeof(FFloat16);
					
					InstParameters->GPUBufferFloat = GPUBuffer && GPUBuffer->GetGPUBufferFloat().UAV.IsValid() ? GPUBuffer->GetGPUBufferFloat().UAV : NDIDataChannelUtilities::GetDummyUAVFloat().Buffer.UAV;
					InstParameters->GPUBufferInt32 = GPUBuffer && GPUBuffer->GetGPUBufferInt().UAV.IsValid() ? GPUBuffer->GetGPUBufferInt().UAV : NDIDataChannelUtilities::GetDummyUAVInt32().Buffer.UAV;
					//TODO: Half Support | InstParameters->GPUBufferHalf = GPUBuffer && GPUBuffer->GetGPUBufferHalf().UAV.IsValid() ? GPUBuffer->GetGPUBufferHalf().UAV : NDIDataChannelUtilities::GetDummyUAVHalf().Buffer.UAV;
					InstParameters->GPUInstanceCountOffset = GPUBuffer ? GPUBuffer->GetGPUInstanceCountBufferOffset() : INDEX_NONE;
					InstParameters->GPUBufferSize = GPUBuffer ? GPUBuffer->GetNumInstancesAllocated() : INDEX_NONE;


					InstParameters->CPUBufferFloat = BufferForCPU && BufferForCPU->GetGPUBufferFloat().UAV.IsValid() ? BufferForCPU->GetGPUBufferFloat().UAV : NDIDataChannelUtilities::GetDummyUAVFloat().Buffer.UAV;
					InstParameters->CPUBufferInt32 = BufferForCPU && BufferForCPU->GetGPUBufferInt().UAV.IsValid() ? BufferForCPU->GetGPUBufferInt().UAV : NDIDataChannelUtilities::GetDummyUAVInt32().Buffer.UAV;
					//TODO: Half Support | InstParameters->CPUBufferHalf = BufferForCPU && BufferForCPU->GetGPUBufferHalf().UAV.IsValid() ? BufferForCPU->GetGPUBufferHalf().UAV : NDIDataChannelUtilities::GetDummyUAVHalf().Buffer.UAV;
					InstParameters->CPUInstanceCountOffset = BufferForCPU ? BufferForCPU->GetGPUInstanceCountBufferOffset() : INDEX_NONE;
					InstParameters->CPUBufferSize = BufferForCPU ? BufferForCPU->GetNumInstancesAllocated() : INDEX_NONE;

					InstParameters->CPUFloatStride = (BufferForCPU ? BufferForCPU->GetFloatStride() : sizeof(float)) / sizeof(float);
					InstParameters->CPUInt32Stride = (BufferForCPU ? BufferForCPU->GetInt32Stride() : sizeof(int32)) / sizeof(int32);
					//TODO: Half Support | InstParameters->CPUHalfStride = (BufferForCPU ? BufferForCPU->GetHalfStride() : sizeof(FFloat16)) / sizeof(FFloat16);

					bSuccess = true;
				}
			}
		}
	}

	if (bSuccess == false)
	{
		InstParameters->ParamOffsetTable = FNiagaraRenderer::GetDummyUIntBuffer();
		InstParameters->ParameterOffsetTableIndex = INDEX_NONE;
		InstParameters->FloatStride = 0;
		InstParameters->Int32Stride = 0;
		//TODO: Half Support | InstParameters->HalfStride = 0;

		InstParameters->GPUBufferFloat = NDIDataChannelUtilities::GetDummyUAVFloat().Buffer.UAV;
		InstParameters->GPUBufferInt32 = NDIDataChannelUtilities::GetDummyUAVInt32().Buffer.UAV;
		//TODO: Half Support | InstParameters->GPUBufferHalf = NDIDataChannelUtilities::GetDummyUAVHalf().Buffer.UAV;
		InstParameters->GPUInstanceCountOffset = INDEX_NONE;
		InstParameters->GPUBufferSize = INDEX_NONE;


		InstParameters->CPUBufferFloat = NDIDataChannelUtilities::GetDummyUAVFloat().Buffer.UAV;
		InstParameters->CPUBufferInt32 = NDIDataChannelUtilities::GetDummyUAVInt32().Buffer.UAV;
		//TODO: Half Support | InstParameters->CPUBufferHalf = NDIDataChannelUtilities::GetDummyUAVHalf().Buffer.UAV;
		InstParameters->CPUInstanceCountOffset = INDEX_NONE;
		InstParameters->CPUBufferSize = INDEX_NONE;
	}
}

void FNiagaraDataInterfaceProxy_DataChannelWrite::ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance)
{
	FNDIDataChannelWriteInstanceData_RT& SourceData = *reinterpret_cast<FNDIDataChannelWriteInstanceData_RT*>(PerInstanceData);
	FInstanceData& InstData = SystemInstancesToProxyData_RT.FindOrAdd(Instance);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	if(SourceData.ChannelDataRTProxy)
	{
		InstData.ChannelDataRTProxy = SourceData.ChannelDataRTProxy;
		
		if(SourceData.bPublishToGPU)
		{
			//Allocate space in the GPU buffers for writing...
			InstData.ChannelDataRTProxy->AddGPUAllocationForNextTick(SourceData.AllocationCount);
		}

		InstData.AllocationCount = SourceData.AllocationCount;
		InstData.bPublishToGame = SourceData.bPublishToGame;
		InstData.bPublishToCPU = SourceData.bPublishToCPU;
		InstData.bPublishToGPU = SourceData.bPublishToGPU;
		InstData.LwcTile = SourceData.LwcTile;
	}
	else
	{
		InstData.ChannelDataRTProxy = nullptr;
	}

	if (SourceData.ScriptParamInfo.bDirty)
	{
		SourceData.ScriptParamInfo.bDirty = false;

		//Take the offset map from the source data.
		//This maps from GPU script to that scripts offset into the ParameterLayoutBuffer.
		//Allows us to look up and pass in at SetShaderParameters time.
		InstData.GPUScriptParameterTableOffsets = MoveTemp(SourceData.ScriptParamInfo.GPUScriptParameterTableOffsets);

		//Now generate the ParameterLayoutBuffer
		//This contains a table of all parameters used by each GPU script that uses this DI.
		//TODO: This buffer can likely be shared among many instances and stored in the layout manager or in the DI proxy.
		{
			if (InstData.ParameterLayoutBuffer.NumBytes > 0)
			{
				InstData.ParameterLayoutBuffer.Release();
			}

			if (SourceData.ScriptParamInfo.GPUScriptParameterOffsetTable.Num() > 0)
			{
				InstData.ParameterLayoutData = SourceData.ScriptParamInfo.GPUScriptParameterOffsetTable;
				InstData.ParameterLayoutBuffer.Initialize(RHICmdList, TEXT("NDIDataChannel_ParameterLayoutBuffer"), sizeof(uint32), SourceData.ScriptParamInfo.GPUScriptParameterOffsetTable.Num(), EPixelFormat::PF_R32_UINT, BUF_Static, &InstData.ParameterLayoutData);
			}
		}
	}

	SourceData.~FNDIDataChannelWriteInstanceData_RT();
}

int32 FNiagaraDataInterfaceProxy_DataChannelWrite::PerInstanceDataPassedToRenderThreadSize() const
{
	return sizeof(FNDIDataChannelWriteInstanceData_RT);
}

void FNiagaraDataInterfaceProxy_DataChannelWrite::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	FNiagaraDataInterfaceProxy_DataChannelWrite::FInstanceData* InstanceData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

	if (InstanceData && InstanceData->ChannelDataRTProxy)
	{
		if( InstanceData->BufferForCPU == nullptr && (InstanceData->bPublishToGame || InstanceData->bPublishToCPU || InstanceData->bCapturingSimCache) )
		{
			//Allocate a separate buffer that we will write into and ship back to the CPU.			 
			InstanceData->BufferForCPU = InstanceData->ChannelDataRTProxy->AllocateBufferForCPU(Context.GetGraphBuilder(), Context.GetComputeDispatchInterface().GetFeatureLevel(), InstanceData->AllocationCount, InstanceData->bPublishToGame, InstanceData->bPublishToCPU, InstanceData->LwcTile);

			//Get a new instance count. This is later released by the ndc proxy
			uint32 Offset = InstanceData->BufferForCPU->GetGPUInstanceCountBufferOffset();
			Context.GetInstanceCountManager().FreeEntry(Offset);
			InstanceData->BufferForCPU->SetGPUInstanceCountBufferOffset(Context.GetInstanceCountManager().AcquireEntry());

			if(InstanceData->bCapturingSimCache)
			{
				InstanceData->PendingSimCacheReadbacks.Emplace(InstanceData->BufferForCPU);
			}
		}

		if(InstanceData->bPublishToGPU && InstanceData->GPUBuffer == nullptr)
		{
			InstanceData->GPUBuffer = InstanceData->ChannelDataRTProxy->PrepareForWriteAccess(Context.GetGraphBuilder());
		}
	}
}


void FNiagaraDataInterfaceProxy_DataChannelWrite::PostStage(const FNDIGpuComputePostStageContext& Context)
{
	FNiagaraDataInterfaceProxy_DataChannelWrite::FInstanceData* InstanceData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

	if(InstanceData && InstanceData->ChannelDataRTProxy)
	{
		if (InstanceData->GPUBuffer)
		{
			InstanceData->ChannelDataRTProxy->EndWriteAccess(Context.GetGraphBuilder());
			InstanceData->GPUBuffer = nullptr;
		}
	}
}

void FNiagaraDataInterfaceProxy_DataChannelWrite::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	FNiagaraDataInterfaceProxy_DataChannelWrite::FInstanceData* InstanceData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	
	if (InstanceData && InstanceData->ChannelDataRTProxy && Context.IsFinalPostSimulate())
	{
		if(InstanceData->BufferForCPU)
		{
			InstanceData->ChannelDataRTProxy->AddTransition(Context.GetGraphBuilder(), ERHIAccess::UAVCompute, ERHIAccess::SRVMask, InstanceData->BufferForCPU.GetReference());
			InstanceData->BufferForCPU = nullptr;
		}
	}
}

#undef LOCTEXT_NAMESPACE
