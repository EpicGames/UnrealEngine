// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceDataTable.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraParameterStore.h"
#include "NiagaraRenderThreadDeletor.h"
#include "NiagaraSystemInstance.h"

#include "Engine/DataTable.h"
#include "RenderGraphUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceDataTable)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceDataTable"

// Some additional things to consider
//-TODO: How do we handle updates to data tables?  Need a notification from update functions in UDataTable (see GetOrCreateBuiltDataTable)
//-TODO: No position type handling currently, likely need to use FDFScalar and move into tile space on read
//-TODO: Implement a picker on the node to pre-populate the outputs in GetRow / GetFilteredRow

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace NDIDataTableLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(int32,					NumRows)
		SHADER_PARAMETER(uint32,				RowStride)
		SHADER_PARAMETER(uint32,				InvalidRowReadOffset)
		SHADER_PARAMETER(int32,					NumFilteredRows)
		SHADER_PARAMETER_SRV(ByteAddressBuffer,	TableDataBuffer)
		SHADER_PARAMETER_SRV(Buffer<uint>,		FilteredRowDataOffsetBuffer)
	END_SHADER_PARAMETER_STRUCT()

	const TCHAR* TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceDataTableTemplate.ush");

	static const FName NAME_IsValid(TEXT("IsValid"));
	static const FName NAME_GetNumRows(TEXT("GetNumRows"));
	static const FName NAME_GetNumFilteredRows(TEXT("GetNumFilteredRows"));
	static const FName NAME_GetRow(TEXT("GetRow"));
	static const FName NAME_GetFilteredRow(TEXT("GetFilteredRow"));

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FShaderStorage : public FNiagaraDataInterfaceParametersCS
	{
		DECLARE_TYPE_LAYOUT(FShaderStorage, NonVirtual);

		LAYOUT_FIELD(TMemoryImageArray<FMemoryImageName>,	AttributeNames);
	};
	IMPLEMENT_TYPE_LAYOUT(FShaderStorage);

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FBuiltDataTable
	{
	public:
		using FPropertyConversionFunc = TFunction<void(const uint8*, uint8*)>;
		static constexpr uint32 kMaxElementReadSize = 16;

		explicit FBuiltDataTable(UDataTable* DataTable, TOptional<TConstArrayView<FName>> FilterRowNames = {})
			: SourceTable(DataTable)
		{
			UScriptStruct* RowStruct = DataTable ? DataTable->RowStruct.Get() : nullptr;
			if (RowStruct)
			{
				BuildRowStructColumns(RowStruct);
				if (FilterRowNames.IsSet())
				{
					BuildRows(DataTable, FilterRowNames.GetValue());
				}
				else
				{
					BuildRows(DataTable);
				}
			}

			// Make sure we have space to read our maximum element size, avoids branching
			TableRowData.AddZeroed(FMath::Max(RowStride, kMaxElementReadSize));
		}

		~FBuiltDataTable()
		{
			check(IsInRenderingThread());
			GpuTableRowData.Release();
		}

		uint32 GetRowNameByteOffset(FName RowName) const
		{
			const int32 RowIndex = RowNames.IndexOfByKey(RowName);
			return (RowIndex == INDEX_NONE ? NumRows : RowIndex) * RowStride;
		}

		uint32 GetColumnByteOffset(const FNiagaraVariableBase& Variable) const
		{
			for (const TPair<FNiagaraVariableBase, uint32>& Column : Columns)
			{
				if (Column.Key == Variable)
				{
					return Column.Value;
				}
			}
			return INDEX_NONE;
		}

		uint32 GetColumnByteOffset(const FName& AttributeName) const
		{
			for (const TPair<FNiagaraVariableBase, uint32>& Column : Columns)
			{
				if (Column.Key.GetName() == AttributeName)
				{
					return Column.Value;
				}
			}
			return INDEX_NONE;
		}

		uint32 GetRowStride() const { return RowStride; }
		uint32 GetNumRows() const { return NumRows; }
		uint32 GetInvalidRowReadOffset() const { return RowStride * NumRows; }

		const uint8* GetRowData() const { return TableRowData.GetData(); }
		TWeakObjectPtr<UDataTable> GetSourceTable() const { return SourceTable; }
		
		TConstArrayView<FName> GetRowNames() const { return RowNames; }

		bool RowNamesMatch(TConstArrayView<FName> InRows) const { return RowNames == InRows; }

		FRHIShaderResourceView* GetGpuRowDataSrv(FRDGBuilder& GraphBuilder)
		{
			if (GpuTableRowData.NumBytes == 0)
			{
				GpuTableRowData.Initialize(GraphBuilder.RHICmdList, TEXT("NiagaraDataTable::BuiltDataTable"), TableRowData.Num());

				void* UploadMemory = GraphBuilder.RHICmdList.LockBuffer(GpuTableRowData.Buffer, 0, TableRowData.Num(), RLM_WriteOnly);
				FMemory::Memcpy(UploadMemory, TableRowData.GetData(), TableRowData.Num());
				GraphBuilder.RHICmdList.UnlockBuffer(GpuTableRowData.Buffer);
			}
			return GpuTableRowData.SRV;
		}

	private:
		template<typename TTypeFrom, typename TTypeTo>
		void AddRowColum(UScriptStruct* RowStruct, FProperty* Property, const FNiagaraTypeDefinition& TypeDef)
		{
			const FName PropertyName(*RowStruct->GetAuthoredNameForField(Property));
			Columns.Emplace(FNiagaraVariableBase(TypeDef, PropertyName), RowStride);
			RowStructConversionFuncs.Emplace(
				[SrcOffset = Property->GetOffset_ForInternal(), DestOffset = RowStride](const uint8* Src, uint8* Dest)
				{
					TTypeFrom FromValue;
					FMemory::Memcpy(&FromValue, Src + SrcOffset, sizeof(TTypeFrom));
					const TTypeTo ToValue = TTypeTo(FromValue);
					FMemory::Memcpy(Dest + DestOffset, &ToValue, sizeof(TTypeTo));
				}
			);
			RowStride += sizeof(TTypeTo);
		}

		void AddRow(FName RowName, const uint8* DataTableRowData)
		{
			RowNames.Add(RowName);

			const uint32 RowOffset = TableRowData.AddUninitialized(RowStride);
			uint8* DestData = TableRowData.GetData() + RowOffset;
			for (const FPropertyConversionFunc& ConversionFunc : RowStructConversionFuncs)
			{
				ConversionFunc(DataTableRowData, DestData);
			}

			++NumRows;
		}

		void BuildRowStructColumns(UScriptStruct* RowStruct)
		{
			for (FProperty* Property = RowStruct->PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext)
			{
				if (Property->IsA<const FIntProperty>())
				{
					AddRowColum<int32, int32>(RowStruct, Property, FNiagaraTypeDefinition::GetIntDef());
				}
				else if (Property->IsA<const FFloatProperty>())
				{
					AddRowColum<float, float>(RowStruct, Property, FNiagaraTypeDefinition::GetFloatDef());
				}
				else if (Property->IsA<const FDoubleProperty>())
				{
					AddRowColum<double, float>(RowStruct, Property, FNiagaraTypeDefinition::GetFloatDef());
				}
				else if ( const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
				{
					if (StructProperty->Struct == TBaseStructure<FVector2D>::Get())
					{
						AddRowColum<FVector2D, FVector2f>(RowStruct, Property, FNiagaraTypeDefinition::GetVec2Def());
					}
					else if (StructProperty->Struct == TBaseStructure<FVector>::Get())
					{
						AddRowColum<FVector, FVector3f>(RowStruct, Property, FNiagaraTypeDefinition::GetVec3Def());
					}
					else if (StructProperty->Struct == TBaseStructure<FVector4>::Get())
					{
						AddRowColum<FVector4, FVector4f>(RowStruct, Property, FNiagaraTypeDefinition::GetVec4Def());
					}
					else if (StructProperty->Struct == TBaseStructure<FQuat>::Get())
					{
						AddRowColum<FQuat, FQuat4f>(RowStruct, Property, FNiagaraTypeDefinition::GetQuatDef());
					}
					else if (StructProperty->Struct == TBaseStructure<FLinearColor>::Get())
					{
						AddRowColum<FLinearColor, FLinearColor>(RowStruct, Property, FNiagaraTypeDefinition::GetColorDef());
					}
				}
			}
		}

		void BuildRows(UDataTable* DataTable)
		{
			const TMap<FName, uint8*>& DataTableRowMap = DataTable->GetRowMap();

			RowNames.Reserve(DataTableRowMap.Num());
			TableRowData.Reserve((DataTableRowMap.Num() + 1) * RowStride);

			for (auto It = DataTableRowMap.CreateConstIterator(); It; ++It)
			{
				AddRow(It.Key(), It.Value());
			}
		}

		void BuildRows(UDataTable* DataTable, TConstArrayView<FName> FilterRowNames)
		{
			RowNames.Reserve(FilterRowNames.Num());
			TableRowData.Reserve((FilterRowNames.Num() + 1) * RowStride);
			for ( FName RowName : FilterRowNames )
			{
				const uint8* DataTableRowData = static_cast<const uint8*>(DataTable->FindRowUnchecked(RowName));
				if (DataTableRowData)
				{
					AddRow(RowName, DataTableRowData);
				}
			}
		}

	private:
		TWeakObjectPtr<UDataTable>											SourceTable;				// Reference to source table
		TArray<FPropertyConversionFunc, TInlineAllocator<16>>				RowStructConversionFuncs;	// Functions used to convert from the original table to the Niagara version of the table
		TArray<TPair<FNiagaraVariableBase, uint32>, TInlineAllocator<16>>	Columns;					// Mapping from Column to Byte Offset in a Row
		uint32																RowStride = 0;				// Stride for each row of the table
		uint32																NumRows = 0;				// Number of rows stored in the table, note that we always store +1 row as the 'invalid' read row
		TArray<FName>														RowNames;					// Names for each row in the table
		TArray<uint8>														TableRowData;				// Table data blob, basically series of Rows
		FByteAddressBuffer													GpuTableRowData;			// Gpu Table Data, can be null if never used on the GPU
	};

	using FBuiltDataTablePtr = TSharedPtr<FBuiltDataTable>;

	static FBuiltDataTablePtr GetOrCreateBuiltDataTable(UDataTable* DataTable, TOptional<TConstArrayView<FName>> FilteredRowNames = {})
	{
		// Try and find existing table
		static TArray<TWeakPtr<FBuiltDataTable>> GBuiltDataTables;

		for ( auto It=GBuiltDataTables.CreateIterator(); It; ++It )
		{
			TSharedPtr<FBuiltDataTable> ExistingTable = It->Pin();
			if ( ExistingTable == nullptr )
			{
				It.RemoveCurrent();
				continue;
			}

			if (ExistingTable->GetSourceTable() == DataTable)
			{
				// If a row filter was provided then make sure the table onlt contains those rows
				if ( FilteredRowNames.IsSet() == false || ExistingTable->RowNamesMatch(FilteredRowNames.GetValue()) )
				{
					return ExistingTable;
				}
			}
		}

		// We need to create a new Table
		FBuiltDataTablePtr BuiltTable = MakeShareable(new FBuiltDataTable(DataTable, FilteredRowNames), FNiagaraRenderThreadDeletor<FBuiltDataTable>());
		GBuiltDataTables.Add(BuiltTable);
		return BuiltTable;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FGpuAttributeHelper
	{
		explicit FGpuAttributeHelper(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo)
			: FGpuAttributeHelper(ParamInfo.GeneratedFunctions)
		{			
		}

		explicit FGpuAttributeHelper(TConstArrayView<FNiagaraDataInterfaceGeneratedFunction> GeneratedFunctions)
		{
			for (const FNiagaraDataInterfaceGeneratedFunction& Function : GeneratedFunctions)
			{
				for (const FNiagaraVariableCommonReference& OutputVariable : Function.VariadicOutputs)
				{
					Attributes.AddUnique(OutputVariable);
				}
			}
		}

		int32 GetAttributeIndex(const FNiagaraVariableBase& Variable) const
		{
			return Attributes.IndexOfByKey(Variable);
		}

		TArray<FNiagaraVariableBase> Attributes;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FInstanceData_RenderThread
	{
		using FAttributeArray = TArray<uint32, TInlineAllocator<FBuiltDataTable::kMaxElementReadSize>>;

		FBuiltDataTablePtr					BuiltDataTable;
		uint32								NumFilteredRows = 0;
		FReadBuffer							FilteredRowDataOffsetBuffer;
		TMap<uintptr_t, FAttributeArray>	AttributeReadOffsets;
	};

	struct FGameToRenderData
	{
		FNiagaraSystemInstanceID	SystemInstanceID = {};

		FBuiltDataTablePtr			BuiltDataTable;
		uint32						NumFilteredRows = 0;
		TArray<uint32>				FilteredRowDataOffset;
	};

	struct FInstanceData_GameThread
	{
		FNiagaraParameterDirectBinding<UObject*>	UserParamBinding;
		TWeakObjectPtr<UDataTable>					WeakDataTable;

		FBuiltDataTablePtr							BuiltDataTable;
		uint32										InvalidRowReadOffset = 0;

		TArray<FName, TInlineAllocator<16>>			FilteredRowNames;			// Filtered Row Names
		TArray<uint32, TInlineAllocator<16>>		FilteredRowDataOffset;		// Byte offset into RowData
		int32										NumFilteredRows = 0;

		void Initialize(TConstArrayView<FName> InFilteredRowNames, UDataTable* DefaultDataTable, bool bCreateFilteredTable)
		{
			// Get the data table to read
			UDataTable* DataTable = Cast<UDataTable>(UserParamBinding.GetValue());
			DataTable = DataTable ? DataTable : DefaultDataTable;
			WeakDataTable = DataTable;

			// Initialize the table data
			BuiltDataTable = bCreateFilteredTable ? GetOrCreateBuiltDataTable(DataTable, InFilteredRowNames) : GetOrCreateBuiltDataTable(DataTable);
			InvalidRowReadOffset = BuiltDataTable->GetInvalidRowReadOffset();

			// Build mapping of RowName -> Byte Offset
			FilteredRowNames = InFilteredRowNames;
			FilteredRowDataOffset.Reserve(FilteredRowNames.Num() + 1);
			for (FName RowName : FilteredRowNames)
			{
				FilteredRowDataOffset.Add(BuiltDataTable->GetRowNameByteOffset(RowName));
			}
			NumFilteredRows = FilteredRowDataOffset.Num();
			FilteredRowDataOffset.Add(InvalidRowReadOffset);
		}

		TArray<uint32> CreateVariadicReadTable(const FVMExternalFunctionBindingInfo& BindingInfo) const
		{
			TArray<uint32> VariadicReadOffsets;
			VariadicReadOffsets.Reserve(BindingInfo.VariadicOutputs.Num());
			for (const FNiagaraVariableBase& Variable : BindingInfo.VariadicOutputs)
			{
				const uint32 ColumnOffset = BuiltDataTable->GetColumnByteOffset(Variable);
				const uint32 NumRegisters = Variable.GetType().GetSize() / sizeof(uint32);	// This won't work with struct types or ones that contain complex alignment
				check(NumRegisters * sizeof(uint32) == Variable.GetType().GetSize());

				for (uint32 i = 0; i < NumRegisters; ++i)
				{
					const uint32 ElementOffset = ColumnOffset == INDEX_NONE ? ColumnOffset : ColumnOffset + (i * sizeof(uint32));
					VariadicReadOffsets.Add(ElementOffset);
				}
			}
			return VariadicReadOffsets;
		}
	};

	static void VMIsValid(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);
		FNDIOutputParam<bool> OutIsValid(Context);

		const bool bIsValid = InstanceData->BuiltDataTable->GetNumRows() > 0;
		for (int32 iInstance = 0; iInstance < Context.GetNumInstances(); ++iInstance)
		{
			OutIsValid.SetAndAdvance(bIsValid);
		}
	}

	template<bool bIsFilteredRow>
	static void VMGetNumRows(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);
		FNDIOutputParam<int> OutNumRows(Context);

		const bool bIsValid = InstanceData->BuiltDataTable->GetNumRows() > 0;
		const int32 NumRows = bIsFilteredRow ? InstanceData->NumFilteredRows : InstanceData->BuiltDataTable->GetNumRows();
		for (int32 iInstance = 0; iInstance < Context.GetNumInstances(); ++iInstance)
		{
			OutNumRows.SetAndAdvance(NumRows);
		}
	}

	template<bool bIsFilteredRow>
	static void VMGetRow(FVectorVMExternalFunctionContext& Context, TConstArrayView<uint32> VariadicReadOffsets)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<int>				InFilteredRowIndex(Context);
		FNDIOutputParam<bool>			OutIsValid(Context);

		TArray<FNDIOutputParam<int32>, TInlineAllocator<16>> OutVariadics;
		OutVariadics.Reserve(VariadicReadOffsets.Num());
		for (int32 i=0; i < VariadicReadOffsets.Num(); ++i)
		{
			OutVariadics.Emplace(Context);
		}

		const uint32 InvalidRowReadOffset = InstanceData->InvalidRowReadOffset;
		const uint8* RawTableData = InstanceData->BuiltDataTable->GetRowData();
		const int32 NumRows = bIsFilteredRow ? InstanceData->NumFilteredRows : InstanceData->BuiltDataTable->GetNumRows();
		const uint32 RowStride = InstanceData->BuiltDataTable->GetRowStride();
		for (int32 iInstance=0; iInstance < Context.GetNumInstances(); ++iInstance)
		{
			const int32 RawRowIndex = InFilteredRowIndex.GetAndAdvance();
			const bool bValidRowIndex = RawRowIndex >= 0 && RawRowIndex < NumRows;
			const int32 RowIndex = bValidRowIndex ? RawRowIndex : NumRows;
			const uint32 RowReadOffset = bIsFilteredRow ? InstanceData->FilteredRowDataOffset[RowIndex] : RowIndex * RowStride;

			OutIsValid.SetAndAdvance(bValidRowIndex);
			for (int32 iOutput = 0; iOutput < VariadicReadOffsets.Num(); ++iOutput)
			{
				const uint32 VariableReadOffset = VariadicReadOffsets[iOutput] == INDEX_NONE ? InvalidRowReadOffset : RowReadOffset + VariadicReadOffsets[iOutput];
				const int32* Value = reinterpret_cast<const int32*>(RawTableData + VariableReadOffset);
				OutVariadics[iOutput].SetAndAdvance(*Value);
			}
		}
	}

	static void VMGetFilteredRow(FVectorVMExternalFunctionContext& Context, TConstArrayView<uint32> VariadicReadOffsets)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<int>				InFilteredRowIndex(Context);
		FNDIOutputParam<bool>			OutIsValid(Context);

		TArray<FNDIOutputParam<int32>, TInlineAllocator<16>> OutVariadics;
		OutVariadics.Reserve(VariadicReadOffsets.Num());
		for (int32 i=0; i < VariadicReadOffsets.Num(); ++i)
		{
			OutVariadics.Emplace(Context);
		}

		const uint32 InvalidRowReadOffset = InstanceData->InvalidRowReadOffset;
		const uint8* RawTableData = InstanceData->BuiltDataTable->GetRowData();
		for (int32 iInstance=0; iInstance < Context.GetNumInstances(); ++iInstance)
		{
			const int32 RawRowIndex = InFilteredRowIndex.GetAndAdvance();
			const bool bValidRowIndex = RawRowIndex >= 0 && RawRowIndex < InstanceData->NumFilteredRows;
			const int32 RowIndex = bValidRowIndex ? RawRowIndex : InstanceData->NumFilteredRows;
			const uint32 RowReadOffset = InstanceData->FilteredRowDataOffset[RowIndex];

			OutIsValid.SetAndAdvance(bValidRowIndex);
			for (int32 iOutput = 0; iOutput < VariadicReadOffsets.Num(); ++iOutput)
			{
				const uint32 VariableReadOffset = VariadicReadOffsets[iOutput] == INDEX_NONE ? InvalidRowReadOffset : RowReadOffset + VariadicReadOffsets[iOutput];
				const int32* Value = reinterpret_cast<const int32*>(RawTableData + VariableReadOffset);
				OutVariadics[iOutput].SetAndAdvance(*Value);
			}
		}
	}

	struct FNDIProxy : public FNiagaraDataInterfaceProxy//RW
	{
		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override {}
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }
	
		void SendGameToRender(FInstanceData_GameThread* InstanceData_GT, FNiagaraSystemInstanceID SystemInstanceID)
		{
			FGameToRenderData GameToRenderData;
			GameToRenderData.SystemInstanceID		= SystemInstanceID;
			GameToRenderData.BuiltDataTable			= InstanceData_GT->BuiltDataTable;
			GameToRenderData.NumFilteredRows		= InstanceData_GT->NumFilteredRows;
			GameToRenderData.FilteredRowDataOffset	= InstanceData_GT->FilteredRowDataOffset;

			ENQUEUE_RENDER_COMMAND(FNDISimpleCounter_RemoveProxy)
			(
				[this, GameToRenderData=MoveTemp(GameToRenderData)](FRHICommandListImmediate& RHICmdList)
				{
					FInstanceData_RenderThread* InstanceData_RT = &PerInstanceData_RenderThread.FindOrAdd(GameToRenderData.SystemInstanceID);
					InstanceData_RT->BuiltDataTable		= GameToRenderData.BuiltDataTable;
					InstanceData_RT->NumFilteredRows	= GameToRenderData.NumFilteredRows;
					InstanceData_RT->FilteredRowDataOffsetBuffer.Release();
					InstanceData_RT->AttributeReadOffsets.Empty();

					InstanceData_RT->FilteredRowDataOffsetBuffer.InitializeWithData(
						RHICmdList,
						TEXT("NiagaraDataTable::PerDataDI"),
						sizeof(uint32),
						GameToRenderData.FilteredRowDataOffset.Num(),
						PF_R32_UINT,
						EBufferUsageFlags::None,
						[&](FRHIBufferInitializer& Initializer)
						{
							Initializer.WriteData(GameToRenderData.FilteredRowDataOffset.GetData(), GameToRenderData.FilteredRowDataOffset.Num() * GameToRenderData.FilteredRowDataOffset.GetTypeSize());
						}
					);
				}
			);
		}
	
		TMap<FNiagaraSystemInstanceID, FInstanceData_RenderThread>	PerInstanceData_RenderThread;
	};

} //namespace NDIDataTableLocal

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceDataTable::UNiagaraDataInterfaceDataTable(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDIDataTableLocal;

	Proxy.Reset(new FNDIProxy());

	FNiagaraTypeDefinition Def(UObject::StaticClass());
	ObjectParameterBinding.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceDataTable::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceDataTable::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfaceDataTable* OtherTyped = CastChecked<const UNiagaraDataInterfaceDataTable>(Other);
	return
		OtherTyped->DataTable == DataTable &&
		OtherTyped->FilteredRowNames == FilteredRowNames &&
		OtherTyped->ObjectParameterBinding == ObjectParameterBinding;
}

bool UNiagaraDataInterfaceDataTable::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceDataTable* DestinationTyped = CastChecked<UNiagaraDataInterfaceDataTable>(Destination);
	DestinationTyped->DataTable = DataTable;
	DestinationTyped->FilteredRowNames = FilteredRowNames;
	DestinationTyped->ObjectParameterBinding = ObjectParameterBinding;
	DestinationTyped->bCreateFilteredTable = bCreateFilteredTable;

	return true;
}

bool UNiagaraDataInterfaceDataTable::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIDataTableLocal;

	FInstanceData_GameThread* InstanceData_GT = new(PerInstanceData) FInstanceData_GameThread();
	InstanceData_GT->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), ObjectParameterBinding.Parameter);
	InstanceData_GT->Initialize(FilteredRowNames, DataTable, bCreateFilteredTable);

	if ( IsUsedWithGPUScript() )
	{
		GetProxyAs<FNDIProxy>()->SendGameToRender(InstanceData_GT, SystemInstance->GetId());
	}
	return true;
}

void UNiagaraDataInterfaceDataTable::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIDataTableLocal;

	FInstanceData_GameThread* InstanceData = static_cast<FInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FInstanceData_GameThread();

	if ( IsUsedWithGPUScript() )
	{
		ENQUEUE_RENDER_COMMAND(FNDISimpleCounter_RemoveProxy)
		(
			[Proxy=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
			{
				Proxy->PerInstanceData_RenderThread.Remove(InstanceID);
			}
		);
	}
}

int32 UNiagaraDataInterfaceDataTable::PerInstanceDataSize() const
{
	using namespace NDIDataTableLocal;

	return sizeof(FInstanceData_GameThread);
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceDataTable::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIDataTableLocal;

	FNiagaraFunctionSignature DefaultSig;
	DefaultSig.bMemberFunction = true;
	DefaultSig.bRequiresContext = false;
	DefaultSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("DataTable"));

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultSig);
		Sig.Name = NAME_IsValid;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Sig.SetDescription(LOCTEXT("IsValidDesc", "Returns true if the table is valid and has at least 1 row."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultSig);
		Sig.Name = NAME_GetNumRows;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumRows"));
		Sig.SetDescription(LOCTEXT("GetNumRowsDesc", "Returns the total number of rows in table."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultSig);
		Sig.Name = NAME_GetNumFilteredRows;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumFilteredRows"));
		Sig.SetDescription(LOCTEXT("GetNumFilteredRowsDesc", "Returns the number of filtered rows, this matches the number of values in the filter list even if the table data is invalid or filtered rows do not exist."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultSig);
		Sig.Name = NAME_GetRow;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("RowIndex"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Sig.RequiredOutputs = Sig.Outputs.Num();
		Sig.SetDescription(LOCTEXT("GetRowDesc", "Returns data from the table using the provided row index.  If the row is invalid or the column is invalid the output for the attributes will be 0."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultSig);
		Sig.Name = NAME_GetFilteredRow;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("FilteredRowIndex"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Sig.RequiredOutputs = Sig.Outputs.Num();
		Sig.SetDescription(LOCTEXT("GetRowFilteredDesc", "Returns data from the table using the provided filtered row index.  If the filtered row is invalid or the column is invalid the output for the attributes will be 0."));
	}
}
#endif

void UNiagaraDataInterfaceDataTable::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* PerInstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIDataTableLocal;

	if ( BindingInfo.Name == NAME_IsValid )
	{
		OutFunc.BindStatic(&VMIsValid);
	}
	else if ( BindingInfo.Name == NAME_GetNumRows )
	{
		OutFunc.BindStatic(&VMGetNumRows<false>);
	}
	else if (BindingInfo.Name == NAME_GetNumFilteredRows)
	{
		OutFunc.BindStatic(&VMGetNumRows<true>);
	}
	else if ( BindingInfo.Name == NAME_GetRow )
	{
		const FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
		OutFunc = FVMExternalFunction::CreateLambda(
			[VariadicReadOffsets = InstanceData_GT->CreateVariadicReadTable(BindingInfo)](FVectorVMExternalFunctionContext& Context)
			{
				VMGetRow<false>(Context, VariadicReadOffsets);
			}
		);
	}
	else if ( BindingInfo.Name == NAME_GetFilteredRow )
	{
		const FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
		OutFunc = FVMExternalFunction::CreateLambda(
			[VariadicReadOffsets=InstanceData_GT->CreateVariadicReadTable(BindingInfo)](FVectorVMExternalFunctionContext& Context)
			{
				VMGetRow<true>(Context, VariadicReadOffsets);
			}
		);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceDataTable::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	using namespace NDIDataTableLocal;

	const FGpuAttributeHelper AttributeHelper(ParamInfo);
	OutHLSL.Appendf(
		TEXT("uint4 %s_AttributeReadOffset[%d];\n"),
		*ParamInfo.DataInterfaceHLSLSymbol,
		FMath::DivideAndRoundUp(FMath::Max(AttributeHelper.Attributes.Num(), 1), 4)
	);

	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceDataTable::GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL)
{
	using namespace NDIDataTableLocal;

	const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo = HlslGenContext.GetFunctionInfo();
	const FString& DataInterfaceHLSLSymbol = HlslGenContext.ParameterInfo.DataInterfaceHLSLSymbol;

	if (FunctionInfo.DefinitionName == NAME_GetFilteredRow || FunctionInfo.DefinitionName == NAME_GetRow)
	{
		const FGpuAttributeHelper AttributeHelper(HlslGenContext.ParameterInfo);

		const TCHAR* RowOffsetGetter = FunctionInfo.DefinitionName == NAME_GetFilteredRow ? TEXT("GetFilteredRowReadOffset_") : TEXT("GetRowReadOffset_");
		const TCHAR* RowIndexValue = FunctionInfo.DefinitionName == NAME_GetFilteredRow ? TEXT("In_FilteredRowIndex") : TEXT("In_RowIndex");

		OutHLSL.Appendf(TEXT("void %s%s\n"), *FunctionInfo.InstanceName, *HlslGenContext.GetSanitizedFunctionParameters(HlslGenContext.GetFunctionSignature()));
		OutHLSL.Append(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("\tconst uint RowReadOffset = %s%s(%s, Out_IsValid);\n"), RowOffsetGetter, *DataInterfaceHLSLSymbol, RowIndexValue);
		for (const FNiagaraVariableCommonReference& OutputVariable : FunctionInfo.VariadicOutputs)
		{
			OutHLSL.Appendf(
				TEXT("\tReadValue_%s(RowReadOffset, %d, Out_%s);\n"),
				*DataInterfaceHLSLSymbol,
				AttributeHelper.GetAttributeIndex(OutputVariable),
				*HlslGenContext.GetSanitizedSymbolName(OutputVariable.Name.ToString())
			);
		}
		OutHLSL.Append(TEXT("}\n"));
		return true;
	}

	// Functions inside the template file
	static TArray<FName> HlslTemplateFunctions =
	{
		NAME_IsValid,
		NAME_GetNumRows,
		NAME_GetNumFilteredRows,
	};
	return HlslTemplateFunctions.Contains(FunctionInfo.DefinitionName);
}

bool UNiagaraDataInterfaceDataTable::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace NDIDataTableLocal;

	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceDataTable::PostCompile(const UNiagaraSystem& OwningSystem)
{
	using namespace NDIDataTableLocal;

	Super::PostCompile(OwningSystem);

	bCreateFilteredTable = true;
	{
		static const TArray<FName> AllTableFunctions = { NAME_GetRow, NAME_GetNumRows };

		FNiagaraDataInterfaceUtilities::ForEachVMFunction(
			this,
			&OwningSystem,
			[this](const UNiagaraScript* Script, const FVMExternalFunctionBindingInfo& FunctionBinding) -> bool
			{
				if (AllTableFunctions.Contains(FunctionBinding.Name))
				{
					bCreateFilteredTable = false;
					return false;
				}
				return true;
			}
		);
		if (bCreateFilteredTable == true)
		{
			FNiagaraDataInterfaceUtilities::ForEachGpuFunction(
				this,
				&OwningSystem,
				[this](const UNiagaraScript* Script, const FNiagaraDataInterfaceGeneratedFunction& FunctionBinding) -> bool
				{
					if (AllTableFunctions.Contains(FunctionBinding.DefinitionName))
					{
						bCreateFilteredTable = false;
						return false;
					}
					return true;
				}
			);
		}
	}
}
#endif

FNiagaraDataInterfaceParametersCS* UNiagaraDataInterfaceDataTable::CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const
{
	using namespace NDIDataTableLocal;

	const FGpuAttributeHelper AttributeHelper(ParameterInfo);

	FShaderStorage* ShaderStorage = new FShaderStorage();
	ShaderStorage->AttributeNames.Reserve(AttributeHelper.Attributes.Num());
	for (const FNiagaraVariableBase& Attribute : AttributeHelper.Attributes)
	{
		ShaderStorage->AttributeNames.Add(Attribute.GetName());
	}
	return ShaderStorage;
}

const FTypeLayoutDesc* UNiagaraDataInterfaceDataTable::GetShaderStorageType() const
{
	using namespace NDIDataTableLocal;
	return &StaticGetTypeLayoutDesc<FShaderStorage>();
}

void UNiagaraDataInterfaceDataTable::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NDIDataTableLocal;

	const FGpuAttributeHelper AttributeHelper(ShaderParametersBuilder.GetGeneratedFunctions());
	const int32 NumAttributes = FMath::DivideAndRoundUp(FMath::Max(AttributeHelper.Attributes.Num(), 1), 4);
	ShaderParametersBuilder.AddLooseParamArray<FUintVector4>(TEXT("AttributeReadOffset"), NumAttributes);

	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceDataTable::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIDataTableLocal;

	FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	FInstanceData_RenderThread& InstanceData = DIProxy.PerInstanceData_RenderThread.FindChecked(Context.GetSystemInstanceID());

	const FShaderStorage& ShaderStorage = Context.GetShaderStorage<FShaderStorage>();
	const uint32 NumAttributes4 = FMath::DivideAndRoundUp(FMath::Max(ShaderStorage.AttributeNames.Num(), 1), 4);

	FInstanceData_RenderThread::FAttributeArray& AttributeReadOffsets = InstanceData.AttributeReadOffsets.FindOrAdd(uintptr_t(&ShaderStorage));
	if (AttributeReadOffsets.IsEmpty())
	{
		AttributeReadOffsets.AddZeroed(NumAttributes4 * 4);
		for ( int32 i=0; i < ShaderStorage.AttributeNames.Num(); ++i )
		{
			AttributeReadOffsets[i] = InstanceData.BuiltDataTable->GetColumnByteOffset(ShaderStorage.AttributeNames[i]);
		}
	}
	TArrayView<FUintVector4> AttributeIndices = Context.GetParameterLooseArray<FUintVector4>(NumAttributes4);
	FMemory::Memcpy(AttributeIndices.GetData(), AttributeReadOffsets.GetData(), AttributeReadOffsets.Num() * AttributeReadOffsets.GetTypeSize());

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->NumRows						= InstanceData.BuiltDataTable->GetNumRows();
	ShaderParameters->RowStride						= InstanceData.BuiltDataTable->GetRowStride();
	ShaderParameters->InvalidRowReadOffset			= InstanceData.BuiltDataTable->GetInvalidRowReadOffset();
	ShaderParameters->NumFilteredRows				= InstanceData.NumFilteredRows;
	ShaderParameters->TableDataBuffer				= InstanceData.BuiltDataTable->GetGpuRowDataSrv(Context.GetGraphBuilder());
	ShaderParameters->FilteredRowDataOffsetBuffer	= InstanceData.FilteredRowDataOffsetBuffer.SRV;
}

#if WITH_EDITOR
bool UNiagaraDataInterfaceDataTable::IsReadFunction(const FNiagaraFunctionSignature& Signature)
{
	using namespace NDIDataTableLocal;

	return Signature.Name == NAME_GetRow || Signature.Name == NAME_GetFilteredRow;
}

TArray<FNiagaraVariableBase> UNiagaraDataInterfaceDataTable::GetVariablesFromDataTable(const UDataTable* DataTable)
{
	UScriptStruct* RowStruct = DataTable ? DataTable->RowStruct.Get() : nullptr;
	if (RowStruct == nullptr)
	{
		return {};
	}

	TArray<FNiagaraVariableBase> Variables;
	for (FProperty* Property = RowStruct->PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext)
	{
		const FName PropertyName(*RowStruct->GetAuthoredNameForField(Property));

		if (Property->IsA<const FIntProperty>())
		{
			Variables.Emplace(FNiagaraTypeDefinition::GetIntDef(), PropertyName);
		}
		else if (Property->IsA<const FFloatProperty>())
		{
			Variables.Emplace(FNiagaraTypeDefinition::GetFloatDef(), PropertyName);
		}
		else if (Property->IsA<const FDoubleProperty>())
		{
			Variables.Emplace(FNiagaraTypeDefinition::GetFloatDef(), PropertyName);
		}
		else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
		{
			if (StructProperty->Struct == TBaseStructure<FVector2D>::Get())
			{
				Variables.Emplace(FNiagaraTypeDefinition::GetVec2Def(), PropertyName);
			}
			else if (StructProperty->Struct == TBaseStructure<FVector>::Get())
			{
				Variables.Emplace(FNiagaraTypeDefinition::GetVec3Def(), PropertyName);
			}
			else if (StructProperty->Struct == TBaseStructure<FVector4>::Get())
			{
				Variables.Emplace(FNiagaraTypeDefinition::GetVec4Def(), PropertyName);
			}
			else if (StructProperty->Struct == TBaseStructure<FQuat>::Get())
			{
				Variables.Emplace(FNiagaraTypeDefinition::GetQuatDef(), PropertyName);
			}
			else if (StructProperty->Struct == TBaseStructure<FLinearColor>::Get())
			{
				Variables.Emplace(FNiagaraTypeDefinition::GetColorDef(), PropertyName);
			}
		}
	}

	return Variables;
}
#endif

#undef LOCTEXT_NAMESPACE

