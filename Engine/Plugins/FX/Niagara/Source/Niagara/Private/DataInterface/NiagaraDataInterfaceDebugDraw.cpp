// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceDebugDraw.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraTypes.h"
#include "NiagaraEmitter.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGpuComputeDebug.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstance.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceDebugDraw)

//////////////////////////////////////////////////////////////////////////

struct FNiagaraDebugDrawDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		AddedNonUniformScale = 1,
		AddedHemiSpheres = 2,
		AddedAdditionalRotation = 3,
		UnifiedParameters = 4,
		AddCylinderHalfHeight = 5,
		AddedSphereExtraRotation = 6,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

BEGIN_SHADER_PARAMETER_STRUCT(FNDIDebugDrawShaderParameters,)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>,	RWNDIDebugDrawArgs)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>,	RWNDIDebugDrawLineVertex)
	SHADER_PARAMETER(uint32,						NDIDebugDrawLineMaxInstances)
END_SHADER_PARAMETER_STRUCT()

struct FNDIDebugDrawInstanceData_GameThread
{
	FNDIDebugDrawInstanceData_GameThread()
	{

	}


#if NIAGARA_COMPUTEDEBUG_ENABLED
	void AddLine(const FVector3f& Start, const FVector3f& End, const FLinearColor& Color)
	{
		//-OPT: Need to improve this
		FScopeLock RWLock(&LineBufferLock);

		if (OverrideMaxLineInstances == 0 || uint32(LineBuffer.Num()) < OverrideMaxLineInstances)
		{
			auto& Line = LineBuffer.AddDefaulted_GetRef();
			Line.Start = Start;
			Line.End = End;
			Line.Color = uint32(FMath::Clamp(Color.R, 0.0f, 1.0f) * 255.0f) << 24;
			Line.Color |= uint32(FMath::Clamp(Color.G, 0.0f, 1.0f) * 255.0f) << 16;
			Line.Color |= uint32(FMath::Clamp(Color.B, 0.0f, 1.0f) * 255.0f) << 8;
			Line.Color |= uint32(FMath::Clamp(Color.A, 0.0f, 1.0f) * 255.0f) << 0;
		}
	}

	void AddSphere(const FVector3f& Location, float Radius, int32 Segments, const FLinearColor& Color)
	{
		AddSphere(Location, FVector3f(Radius), FQuat4f::Identity, FQuat4f::Identity, FVector3f::OneVector, Segments, true, true, true, Color);
	}

	void AddSphere(const FVector3f& Location, const FVector3f& Radius, const FQuat4f& AxisRotator, const FQuat4f& WorldRotate, const FVector3f& WorldScale, int32 Segments, bool bHemiX, bool bHemiY, bool bHemiZ, const FLinearColor& Color)
	{
		const float uinc = 2.0f * PI / float(Segments);

		float ux = 0.0f;
		float SinX0 = FMath::Sin(ux);
		float CosX0 = FMath::Cos(ux);
		FVector3f LerpVector = FVector3f(bHemiX ? 0.0f : 1.0f, bHemiY ? 0.0f : 1.0f, bHemiZ ? 0.0f : 1.0f);
		FTransform3f LocalToWorld;
		LocalToWorld.SetComponents(WorldRotate, FVector3f::ZeroVector, WorldScale);

		for (int x = 0; x < Segments; ++x)
		{
			ux += uinc;
			const float SinX1 = FMath::Sin(ux);
			const float CosX1 = FMath::Cos(ux);

			float uy = 0.0f;
			float SinY0 = FMath::Sin(uy);
			float CosY0 = FMath::Cos(uy);
			for (int y = 0; y < Segments; ++y)
			{
				uy += uinc;
				const float SinY1 = FMath::Sin(uy);
				const float CosY1 = FMath::Cos(uy);

				// Get uniform sphere location...
				FVector3f A = FVector3f(CosX0 * CosY0, SinY0, SinX0 * CosY0);
				FVector3f B = FVector3f(CosX1 * CosY0, SinY0, SinX1 * CosY0);
				FVector3f C = FVector3f(CosX0 * CosY1, SinY1, SinX0 * CosY1);

				// Handle hemisphere mapping and scaling to radii/nonuniform... SphereVector = Lerp(SphereVector, Abs(SphereVector), (HemiX, HemiY, HemiZ)) * Radius
				A = FMath::Lerp(A, A.GetAbs(), LerpVector) * Radius;
				B = FMath::Lerp(B, B.GetAbs(), LerpVector) * Radius;
				C = FMath::Lerp(C, C.GetAbs(), LerpVector) * Radius;

				// Handle sphere axis alignment rotation.
				A = AxisRotator.RotateVector(A);
				B = AxisRotator.RotateVector(B);
				C = AxisRotator.RotateVector(C);

				const FVector3f Point0 = Location + LocalToWorld.TransformVector(A);
				const FVector3f Point1 = Location + LocalToWorld.TransformVector(B);
				const FVector3f Point2 = Location + LocalToWorld.TransformVector(C);

				AddLine(Point0, Point1, Color);
				AddLine(Point0, Point2, Color);

				SinY0 = SinY1;
				CosY0 = CosY1;
			}

			SinX0 = SinX1;
			CosX0 = CosX1;
		}
	}

	void AddBox(const FVector3f& Location, const FQuat4f& Rotation, const FVector3f& Extents, const FLinearColor& Color)
	{
		AddBox(Location, Rotation, Extents, FQuat4f::Identity, FVector3f::OneVector, Color);
	}

	void AddBox(const FVector3f& Location, const FQuat4f& Rotation, const FVector3f& Extents, const FQuat4f& WorldRotate, const FVector3f& WorldScale, const FLinearColor& Color)
	{
		FTransform3f LocalToWorld;
		LocalToWorld.SetComponents(WorldRotate, FVector3f::ZeroVector, WorldScale);
		const FVector3f Points[] =
		{
			Location + LocalToWorld.TransformVector(Rotation.RotateVector(FVector3f(Extents.X,  Extents.Y,  Extents.Z))),
			Location + LocalToWorld.TransformVector(Rotation.RotateVector(FVector3f(-Extents.X,  Extents.Y,  Extents.Z))),
			Location + LocalToWorld.TransformVector(Rotation.RotateVector(FVector3f(-Extents.X, -Extents.Y,  Extents.Z))),
			Location + LocalToWorld.TransformVector(Rotation.RotateVector(FVector3f(Extents.X, -Extents.Y,  Extents.Z))),
			Location + LocalToWorld.TransformVector(Rotation.RotateVector(FVector3f(Extents.X,  Extents.Y, -Extents.Z))),
			Location + LocalToWorld.TransformVector(Rotation.RotateVector(FVector3f(-Extents.X,  Extents.Y, -Extents.Z))),
			Location + LocalToWorld.TransformVector(Rotation.RotateVector(FVector3f(-Extents.X, -Extents.Y, -Extents.Z))),
			Location + LocalToWorld.TransformVector(Rotation.RotateVector(FVector3f(Extents.X, -Extents.Y, -Extents.Z))),
		};
		AddLine(Points[0], Points[1], Color);
		AddLine(Points[1], Points[2], Color);
		AddLine(Points[2], Points[3], Color);
		AddLine(Points[3], Points[0], Color);
		AddLine(Points[4], Points[5], Color);
		AddLine(Points[5], Points[6], Color);
		AddLine(Points[6], Points[7], Color);
		AddLine(Points[7], Points[4], Color);
		AddLine(Points[0], Points[4], Color);
		AddLine(Points[1], Points[5], Color);
		AddLine(Points[2], Points[6], Color);
		AddLine(Points[3], Points[7], Color);
	}

	void AddCircle(const FVector3f& Location, const FVector3f& XAxis, const FVector3f& YAxis, float Scale, int32 Segments, const FLinearColor& Color)
	{
		const FVector3f X = XAxis * Scale;
		const FVector3f Y = YAxis * Scale;

		const float d = 2.0f * PI / float(Segments);
		float u = 0.0f;

		FVector3f LastPoint = Location + (X * FMath::Cos(u)) + (Y * FMath::Sin(u));

		for (int32 x = 0; x < Segments; ++x)
		{
			u += d;
			const FVector3f CurrPoint = Location + (X * FMath::Cos(u)) + (Y * FMath::Sin(u));
			AddLine(LastPoint, CurrPoint, Color);
			LastPoint = CurrPoint;
		}
	}

	void AddRectangle(const FVector3f& Location, const FVector3f& XAxis, const FVector3f& YAxis, const FVector2f& Extents, const FIntPoint& Segments, const FLinearColor& Color, bool bUnbounded)
	{
		const FVector3f XAxisNorm = XAxis.GetSafeNormal();
		const FVector3f YAxisNorm = YAxis.GetSafeNormal();

		const FIntPoint NumSegments = FIntPoint(FMath::Clamp(Segments.X, bUnbounded ? 2 : 1, 16), FMath::Clamp(Segments.Y, bUnbounded ? 2 : 1, 16));

		const FVector3f StartPosition = Location - ((XAxisNorm * Extents.X) + (YAxisNorm * Extents.Y));
		const FVector2f Step = (Extents * 2) / NumSegments;
		const FVector3f StepX = XAxisNorm * Step.X;
		const FVector3f StepY = YAxisNorm * Step.Y;

		const int32 LowerBound = bUnbounded ? 1 : 0;
		const int32 UpperBoundX = bUnbounded ? FMath::Max(NumSegments.X - 1, 1) : NumSegments.X;
		const int32 UpperBoundY = bUnbounded ? FMath::Max(NumSegments.Y - 1, 1) : NumSegments.Y;

		// Add all Y axis lines
		for (int32 X = LowerBound; X <= UpperBoundX; X++)
		{
			FVector3f LineStart = StartPosition + (StepX * X);
			FVector3f LineEnd = LineStart + (StepY * NumSegments.Y);

			AddLine(LineStart, LineEnd, Color);
		}

		// Add all X axis lines
		for (int32 Y = LowerBound; Y <= UpperBoundY; Y++)
		{
			FVector3f LineStart = StartPosition + (StepY * Y);
			FVector3f LineEnd = LineStart + (StepX * NumSegments.X);

			AddLine(LineStart, LineEnd, Color);
		}
	}

	void AddCylinder(const FVector3f& Location, const FVector3f& Axis, float Height, float Radius, int32 NumHeightSegments, int32 NumRadiusSegments, const FLinearColor& Color)
	{
		FQuat4f Rotation = FQuat4f::FindBetweenNormals(FVector3f::UpVector, Axis);

		AddCylinder(Location, Rotation, Height, Radius, NumHeightSegments, NumRadiusSegments, Color, false, false, FVector3f::OneVector, true);
	}

	void AddCylinder(const FVector3f& Location, const FQuat4f& Rotation, float Height, float Radius, int32 NumHeightSegments, int32 NumRadiusSegments, const FLinearColor& Color, bool bHemiXDraw, bool bHemiYDraw, const FVector3f& NonUniformScale, bool bCenterVertically)
	{
		if (NumHeightSegments == 0 || NumRadiusSegments == 0)
		{
			return;
		}

		const FVector3f HeightVector = FVector3f::ZAxisVector * Height;
		const FVector3f HalfHeightOffset = bCenterVertically? HeightVector * 0.5f : FVector3f::ZeroVector;
		const FVector3f AxisStep = FVector3f::ZAxisVector * Height / NumHeightSegments;

		const FVector3f HemisphereMask = FVector3f(bHemiXDraw ? 1.0f : 0.0f, bHemiYDraw ? 1.0f : 0.0f, 0.0f);

		float CurrentRotation = 0.0f;
		FVector3f LastPointNormal = (FVector3f::YAxisVector * FMath::Cos(CurrentRotation)) + (FVector3f::XAxisVector * FMath::Sin(CurrentRotation));

		for (int32 Idx = 1; Idx <= NumRadiusSegments; Idx++)
		{
			CurrentRotation = 2.0f * PI / float(NumRadiusSegments) * Idx;

			FVector3f CurrentPointNormal = (FVector3f::YAxisVector * FMath::Cos(CurrentRotation)) + (FVector3f::XAxisVector * FMath::Sin(CurrentRotation));
			CurrentPointNormal = FMath::Lerp(CurrentPointNormal, CurrentPointNormal.GetAbs(), HemisphereMask);

			const FVector3f LastBottom = LastPointNormal * Radius - HalfHeightOffset;
			const FVector3f LastTop = LastPointNormal * Radius + HeightVector - HalfHeightOffset;

			const FVector3f CurrentBottom = CurrentPointNormal * Radius - HalfHeightOffset;
			const FVector3f CurrentTop = CurrentPointNormal * Radius + HeightVector - HalfHeightOffset;


			// Add Height line
			AddLine(Location + Rotation.RotateVector(CurrentBottom * NonUniformScale), Location + Rotation.RotateVector(CurrentTop * NonUniformScale), Color);

			// Add Bottom
			AddLine(Location + Rotation.RotateVector(LastBottom * NonUniformScale), Location + Rotation.RotateVector(CurrentBottom * NonUniformScale), Color);

			// Add all Rings + top
			for (int32 HeightIdx = 1; HeightIdx <= NumHeightSegments; HeightIdx++)
			{
				AddLine(Location + Rotation.RotateVector((LastBottom + AxisStep * HeightIdx) * NonUniformScale), Location + Rotation.RotateVector((CurrentBottom + AxisStep * HeightIdx) * NonUniformScale), Color);
			}

			LastPointNormal = CurrentPointNormal;
		}
	}

	void AddCone(const FVector3f& Location, const FVector3f& Axis, float Height, float RadiusTop, float RadiusBottom, int32 NumHeightSegments, int32 NumRadiusSegments, const FLinearColor& Color)
	{
		FQuat4f Rotation = FQuat4f::FindBetweenVectors(FVector3f::UpVector, Axis);

		AddCone(Location, Rotation, Height, RadiusTop, RadiusBottom, NumHeightSegments, NumRadiusSegments, Color, FVector3f::OneVector, true);
	}

	void AddCone(const FVector3f& Location, const FQuat4f& Rotation, float Height, float RadiusTop, float RadiusBottom, int32 NumHeightSegments, int32 NumRadiusSegments, const FLinearColor& Color, const FVector3f& NonUniformScale, bool bCenterVertically)
	{
		NumHeightSegments = FMath::Clamp(NumHeightSegments, 2, 32);
		NumRadiusSegments = FMath::Clamp(NumRadiusSegments, 3, 32);

		const FVector3f HeightVector = FVector3f::ZAxisVector * Height;
		const FVector3f HalfHeightOffset = bCenterVertically ? HeightVector * 0.5f : FVector3f::ZeroVector;
		const FVector3f AxisStep = FVector3f::ZAxisVector * Height / float(NumHeightSegments - 1);

		const float HeightSegmentStepAlpha = 1.0f / float(NumHeightSegments - 1);

		float CurrentRotation = 0.0f;
		FVector3f LastPointNormal = (FVector3f::YAxisVector * FMath::Cos(CurrentRotation)) + (FVector3f::XAxisVector * FMath::Sin(CurrentRotation));

		for (int32 Idx = 1; Idx <= NumRadiusSegments; Idx++)
		{
			CurrentRotation = 2.0f * PI / float(NumRadiusSegments) * Idx;

			FVector3f CurrentPointNormal = (FVector3f::YAxisVector * FMath::Cos(CurrentRotation)) + (FVector3f::XAxisVector * FMath::Sin(CurrentRotation));

			const FVector3f CurrentBottom = CurrentPointNormal * RadiusBottom - HalfHeightOffset;
			const FVector3f CurrentTop = CurrentPointNormal * RadiusTop + HeightVector - HalfHeightOffset;

			// Add Height line
			AddLine(Location + Rotation.RotateVector(CurrentBottom * NonUniformScale), Location + Rotation.RotateVector(CurrentTop * NonUniformScale), Color);

			// Add Rings
			for (int32 HeightIdx = 0; HeightIdx < NumHeightSegments; HeightIdx++)
			{
				float RingRadius = FMath::Lerp(RadiusBottom, RadiusTop, HeightSegmentStepAlpha * HeightIdx);

				const FVector3f RingOffset = AxisStep * HeightIdx;

				const FVector3f LastRingPosition = LastPointNormal * RingRadius + RingOffset - HalfHeightOffset;
				const FVector3f CurrentRingPosition = CurrentPointNormal * RingRadius + RingOffset - HalfHeightOffset;

				AddLine(Location + Rotation.RotateVector(LastRingPosition * NonUniformScale), Location + Rotation.RotateVector(CurrentRingPosition * NonUniformScale), Color);
			}

			LastPointNormal = CurrentPointNormal;
		}
	}


	void AddTorus(const FVector3f& Location, const FVector3f& Axis, float MajorRadius, float MinorRadius, int32 MajorRadiusSegments, int32 MinorRadiusSegments, const FLinearColor& Color)
	{
		MinorRadiusSegments = FMath::Clamp(MinorRadiusSegments, 3, 32);
		MajorRadiusSegments = FMath::Clamp(MajorRadiusSegments, 3, 32);

		const FVector3f AxisNorm = Axis.GetSafeNormal();

		const FQuat4f AxisRotation = FQuat4f::FindBetweenNormals(FVector3f::UpVector, AxisNorm);

		const FVector3f TangentX = AxisRotation.RotateVector(FVector3f::XAxisVector);
		const FVector3f TangentY = AxisRotation.RotateVector(FVector3f::YAxisVector);

		const float MajorRotationDelta = 2.0f * PI / float(MajorRadiusSegments);
		const float MinorRotationDelta = 2.0f * PI / float(MinorRadiusSegments);

		float CurrentMajorRotation = 0.0f;
		FVector3f PreviousMajorNormal = (TangentX * FMath::Cos(CurrentMajorRotation)) + (TangentY * FMath::Sin(CurrentMajorRotation));

		for (int32 MajorIdx = 0; MajorIdx < MajorRadiusSegments; ++MajorIdx)
		{
			CurrentMajorRotation += MajorRotationDelta;
			const FVector3f CurrentMajorNormal = (TangentX * FMath::Cos(CurrentMajorRotation)) + (TangentY * FMath::Sin(CurrentMajorRotation));

			float CurrentMinorRotation = 0.0f;
			FVector3f PreviousMinorNormal = (CurrentMajorNormal * FMath::Cos(CurrentMinorRotation)) + (AxisNorm * FMath::Sin(CurrentMinorRotation));

			for (int32 MinorIdx = 0; MinorIdx < MinorRadiusSegments; ++MinorIdx)
			{
				CurrentMinorRotation += MinorRotationDelta;
				FVector3f CurrentMinorNormal = (CurrentMajorNormal * FMath::Cos(CurrentMinorRotation)) + (AxisNorm * FMath::Sin(CurrentMinorRotation));

				AddLine(Location + CurrentMajorNormal * MajorRadius + PreviousMinorNormal * MinorRadius, Location + CurrentMajorNormal * MajorRadius + CurrentMinorNormal * MinorRadius, Color);

				FVector3f PreviousMajorMinorNormal = (PreviousMajorNormal * FMath::Cos(CurrentMinorRotation)) + (AxisNorm * FMath::Sin(CurrentMinorRotation));
				AddLine(Location + PreviousMajorNormal * MajorRadius + PreviousMajorMinorNormal * MinorRadius, Location + CurrentMajorNormal * MajorRadius + CurrentMinorNormal * MinorRadius, Color);


				PreviousMinorNormal = CurrentMinorNormal;

			}

			PreviousMajorNormal = CurrentMajorNormal;
		}
	}

	void AddCoordinateSystem(const FVector3f& Location, const FQuat4f& Rotation, float Scale)
	{
		const FVector3f XAxis = Rotation.RotateVector(FVector3f(Scale, 0.0f, 0.0f));
		const FVector3f YAxis = Rotation.RotateVector(FVector3f(0.0f, Scale, 0.0f));
		const FVector3f ZAxis = Rotation.RotateVector(FVector3f(0.0f, 0.0f, Scale));

		AddLine(Location, Location + XAxis, FLinearColor::Red);
		AddLine(Location, Location + YAxis, FLinearColor::Green);
		AddLine(Location, Location + ZAxis, FLinearColor::Blue);
	}

	void AddGrid2D(const FVector3f& Center, const FQuat4f& Rotation, const FVector2f& Extents, const FIntPoint& NumCells, const FLinearColor& Color)
	{
		const FVector3f Corner = Center - Rotation.RotateVector(FVector3f(Extents.X, Extents.Y, 0.0f));
		const FVector3f XLength = Rotation.RotateVector(FVector3f(Extents.X * 2.0f, 0.0f, 0.0f));
		const FVector3f YLength = Rotation.RotateVector(FVector3f(0.0f, Extents.Y * 2.0f, 0.0f));
		const FVector3f XDelta = XLength / float(NumCells.X);
		const FVector3f YDelta = YLength / float(NumCells.Y);

		for (int X = 0; X <= NumCells.X; ++X)
		{
			const FVector3f XOffset = XDelta * float(X);
			for (int Y = 0; Y <= NumCells.Y; ++Y)
			{
				const FVector3f YOffset = YDelta * float(Y);
				AddLine(Corner + XOffset, Corner + XOffset + YLength, Color);
				AddLine(Corner + YOffset, Corner + YOffset + XLength, Color);
			}
		}
	}

	void AddGrid3D(const FVector3f& Center, const FQuat4f& Rotation, const FVector3f& Extents, const FIntVector& NumCells, const FLinearColor& Color)
	{
		const FVector3f Corner = Center - Rotation.RotateVector(Extents);
		const FVector3f XLength = Rotation.RotateVector(FVector3f(Extents.X * 2.0f, 0.0f, 0.0f));
		const FVector3f YLength = Rotation.RotateVector(FVector3f(0.0f, Extents.Y * 2.0f, 0.0f));
		const FVector3f ZLength = Rotation.RotateVector(FVector3f(0.0f, 0.0f, Extents.Z * 2.0f));
		const FVector3f XDelta = XLength / float(NumCells.X);
		const FVector3f YDelta = YLength / float(NumCells.Y);
		const FVector3f ZDelta = ZLength / float(NumCells.Z);

		for (int X = 0; X <= NumCells.X; ++X)
		{
			const FVector3f XOffset = XDelta * float(X);
			for (int Y = 0; Y <= NumCells.Y; ++Y)
			{
				const FVector3f YOffset = YDelta * float(Y);
				for (int Z = 0; Z <= NumCells.Z; ++Z)
				{
					const FVector3f ZOffset = ZDelta * float(Z);

					AddLine(Corner + ZOffset + XOffset, Corner + ZOffset + XOffset + YLength, Color);		// Z Slice: X -> Y
					AddLine(Corner + ZOffset + YOffset, Corner + ZOffset + YOffset + XLength, Color);		// Z Slice: Y -> X

					AddLine(Corner + XOffset + YOffset, Corner + XOffset + YOffset + ZLength, Color);		// X Slice: Y -> Z
					AddLine(Corner + XOffset + ZOffset, Corner + XOffset + ZOffset + YLength, Color);		// X Slice: Z -> Y
				}
			}
		}
	}

	bool bResolvedPersistentShapes = false;
	FCriticalSection LineBufferLock;
	TArray<FNiagaraSimulationDebugDrawData::FGpuLine> LineBuffer;
	uint32 OverrideMaxLineInstances = 0;

	template<typename TYPE>
	static inline TOptional<TYPE> GetCompileTag(FNiagaraSystemInstance* SystemInstance, const UNiagaraScript* Script, const FNiagaraTypeDefinition& VarType, const FName& ParameterName)
	{
		return Script->GetCompilerTag<TYPE>(FNiagaraVariable(VarType, ParameterName), SystemInstance->GetOverrideParameters());
	}


	struct FDebugPrim_PersistentShape
	{
		UNiagaraDataInterfaceDebugDraw::ShapeId ShapeId;
		const UNiagaraScript* Script = nullptr;
		bool bSimSpaceIsLocal = false;
		TArray<FName, TInlineAllocator<16>> ParameterNames;

		ENiagaraCoordinateSpace GetConcreteSource(bool bVectorWasSet, const TOptional<ENiagaraCoordinateSpace>& SourceSpace)
		{
			ENiagaraCoordinateSpace SourceSpaceConcrete = ENiagaraCoordinateSpace::Simulation;
			if (SourceSpace.IsSet())
				SourceSpaceConcrete = SourceSpace.GetValue();

			if (SourceSpaceConcrete == ENiagaraCoordinateSpace::Simulation && bSimSpaceIsLocal)
				SourceSpaceConcrete = ENiagaraCoordinateSpace::Local;
			else if (SourceSpaceConcrete == ENiagaraCoordinateSpace::Simulation)
				SourceSpaceConcrete = ENiagaraCoordinateSpace::World;

			// Override it all as local space if the source vector wasn't set...
			if (bVectorWasSet)
				return SourceSpaceConcrete;
			else
				return ENiagaraCoordinateSpace::Local;
		}

		void TransformVector(bool bVectorWasSet, FVector3f& Vector, const TOptional<ENiagaraCoordinateSpace>& SourceSpace, ENiagaraCoordinateSpace DestSpace, const FNiagaraSystemInstance* SystemInstance)
		{
			ENiagaraCoordinateSpace SourceSpaceConcrete = GetConcreteSource(bVectorWasSet, SourceSpace);
			ENiagaraCoordinateSpace DestSpaceConcrete = GetConcreteSource(true, DestSpace);

			// We are always going to world, so if wer'e already world, just do nothing.
			if (SourceSpaceConcrete == DestSpaceConcrete)
				return;

			if (SourceSpaceConcrete == ENiagaraCoordinateSpace::Local && DestSpaceConcrete == ENiagaraCoordinateSpace::World)
			{
				Vector = (FVector3f)SystemInstance->GetWorldTransform().TransformVector((FVector)Vector);	// LWC_TODO: Precision Loss
			}
			else if (SourceSpaceConcrete == ENiagaraCoordinateSpace::World && DestSpaceConcrete == ENiagaraCoordinateSpace::Local)
			{
				Vector = (FVector3f)SystemInstance->GetWorldTransform().InverseTransformVector((FVector)Vector);	// LWC_TODO: Precision Loss
			}
			else
			{
				UE_LOGF(LogNiagara, Warning, "Unknown space!");
			}
		}

		void ScaleVector(bool bVectorWasSet, FVector3f& Vector, const TOptional<ENiagaraCoordinateSpace>& SourceSpace, ENiagaraCoordinateSpace DestSpace, const FNiagaraSystemInstance* SystemInstance)
		{
			ENiagaraCoordinateSpace SourceSpaceConcrete = GetConcreteSource(bVectorWasSet, SourceSpace);
			ENiagaraCoordinateSpace DestSpaceConcrete = GetConcreteSource(true, DestSpace);

			// We are always going to world, so if wer'e already world, just do nothing.
			if (SourceSpaceConcrete == DestSpaceConcrete)
				return;

			if (SourceSpaceConcrete == ENiagaraCoordinateSpace::Local && DestSpaceConcrete == ENiagaraCoordinateSpace::World)
			{
				Vector = (FVector3f)SystemInstance->GetWorldTransform().GetScale3D() * Vector;
			}
			else if (SourceSpaceConcrete == ENiagaraCoordinateSpace::World && DestSpaceConcrete == ENiagaraCoordinateSpace::Local)
			{
				Vector = Vector / (FVector3f)SystemInstance->GetWorldTransform().GetScale3D();
			}
			else
			{
				UE_LOGF(LogNiagara, Warning, "Unknown space!");
			}
		}

		void TransformPosition(bool bPointWasSet, FVector3f& Point, const TOptional<ENiagaraCoordinateSpace>& SourceSpace, ENiagaraCoordinateSpace DestSpace, const FNiagaraSystemInstance* SystemInstance)
		{
			ENiagaraCoordinateSpace SourceSpaceConcrete = GetConcreteSource(bPointWasSet, SourceSpace);
			ENiagaraCoordinateSpace DestSpaceConcrete = GetConcreteSource(true, DestSpace);

			// We are always going to world, so if wer'e already world, just do nothing.
			if (SourceSpaceConcrete == DestSpaceConcrete)
				return;

			if (SourceSpaceConcrete == ENiagaraCoordinateSpace::Local && DestSpaceConcrete == ENiagaraCoordinateSpace::World)
			{
				Point = (FVector3f)SystemInstance->GetWorldTransform().TransformPosition((FVector)Point);	// LWC_TODO: Precision Loss
			}
			else if (SourceSpaceConcrete == ENiagaraCoordinateSpace::World && DestSpaceConcrete == ENiagaraCoordinateSpace::Local)
			{
				Point = (FVector3f)SystemInstance->GetWorldTransform().InverseTransformPosition((FVector)Point);	// LWC_TODO: Precision Loss
			}
			else
			{
				UE_LOGF(LogNiagara, Warning, "Unknown space!");
			}
		}

		void TransformQuat(bool bRotationWasSet, FQuat4f& Quat, const TOptional<ENiagaraCoordinateSpace>& SourceSpace, ENiagaraCoordinateSpace DestSpace, const FNiagaraSystemInstance* SystemInstance)
		{
			ENiagaraCoordinateSpace SourceSpaceConcrete = GetConcreteSource(bRotationWasSet, SourceSpace);
			ENiagaraCoordinateSpace DestSpaceConcrete = GetConcreteSource(true, DestSpace);

			// We are always going to world, so if wer'e already world, just do nothing.
			if (SourceSpaceConcrete == DestSpaceConcrete)
				return;

			if (SourceSpaceConcrete == ENiagaraCoordinateSpace::Local && DestSpaceConcrete == ENiagaraCoordinateSpace::World)
			{
				Quat = (FQuat4f)SystemInstance->GetWorldTransform().Rotator().Quaternion() * Quat;
			}
			else if (SourceSpaceConcrete == ENiagaraCoordinateSpace::World && DestSpaceConcrete == ENiagaraCoordinateSpace::Local)
			{
				Quat = (FQuat4f)SystemInstance->GetWorldTransform().Rotator().GetInverse().Quaternion() * Quat;
			}
			else
			{
				UE_LOGF(LogNiagara, Warning, "Unknown space!");
			}
		}

		void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			switch (ShapeId)
			{
			case UNiagaraDataInterfaceDebugDraw::Line:
				DrawLine(InstanceData, SystemInstance, DeltaSeconds);
				break;
			case UNiagaraDataInterfaceDebugDraw::Rectangle:
				DrawRectangle(InstanceData, SystemInstance, DeltaSeconds);
				break;
			case UNiagaraDataInterfaceDebugDraw::Circle:
				DrawCircle(InstanceData, SystemInstance, DeltaSeconds);
				break;
			case UNiagaraDataInterfaceDebugDraw::Box:
				DrawBox(InstanceData, SystemInstance, DeltaSeconds);
				break;
			case UNiagaraDataInterfaceDebugDraw::Sphere:
				DrawSphere(InstanceData, SystemInstance, DeltaSeconds);
				break;
			case UNiagaraDataInterfaceDebugDraw::Cylinder:
				DrawCylinder(InstanceData, SystemInstance, DeltaSeconds);
				break;
			case UNiagaraDataInterfaceDebugDraw::Cone:
				DrawCone(InstanceData, SystemInstance, DeltaSeconds);
				break;
			case UNiagaraDataInterfaceDebugDraw::Torus:
				DrawTorus(InstanceData, SystemInstance, DeltaSeconds);
				break;
			case UNiagaraDataInterfaceDebugDraw::CoordinateSystem:
				DrawCoordinateSystem(InstanceData, SystemInstance, DeltaSeconds);
				break;
			case UNiagaraDataInterfaceDebugDraw::Grid2D:
				DrawGrid2D(InstanceData, SystemInstance, DeltaSeconds);
				break;
			case UNiagaraDataInterfaceDebugDraw::Grid3D:
				DrawGrid3D(InstanceData, SystemInstance, DeltaSeconds);
				break;

			};				
		}

		template<typename TYPE>
		inline TOptional<TYPE> GetTag(const FNiagaraSystemInstance* SystemInstance, const FNiagaraTypeDefinition& VarType, int32 Index)
		{
			return Script->GetCompilerTag<TYPE>(FNiagaraVariableBase(VarType, ParameterNames[Index]), SystemInstance->GetOverrideParameters());
		}


		void DrawLine(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			if (Script == nullptr)
				return;
			static FNiagaraTypeDefinition CoordTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());

			TOptional<FVector3f> StartLocation = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 0);
			TOptional<ENiagaraCoordinateSpace> StartLocationCoordinateSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 1);
			TOptional<FVector3f> EndLocation = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 2);
			TOptional<ENiagaraCoordinateSpace> EndLocationCoordinateSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 3);
			TOptional<FLinearColor> Color = GetTag<FLinearColor>(SystemInstance, FNiagaraTypeDefinition::GetColorDef(), 4);

			FVector3f DrawStartLocation = StartLocation.Get(FVector3f::ZeroVector);
			FVector3f DrawEndLocation = EndLocation.Get(FVector3f::ZeroVector);
			FLinearColor DrawColor = Color.Get(FLinearColor::Red);

			TransformPosition(StartLocation.IsSet(), DrawStartLocation, StartLocationCoordinateSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformPosition(EndLocation.IsSet(), DrawEndLocation, EndLocationCoordinateSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);

			if (DrawStartLocation != DrawEndLocation)
			{
				InstanceData->AddLine(DrawStartLocation, DrawEndLocation, DrawColor);
			}
		}

		void DrawRectangle(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			if (Script == nullptr)
				return;
			static FNiagaraTypeDefinition CoordTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());

			TOptional<FVector3f> Center = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 0);
			TOptional<ENiagaraCoordinateSpace> CenterWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 1);
			TOptional<FVector3f> Offset = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 2);
			TOptional<ENiagaraCoordinateSpace> OffsetWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 3);
			TOptional<FVector2f> Extents = GetTag<FVector2f>(SystemInstance, FNiagaraTypeDefinition::GetVec2Def(), 4);
			TOptional<FNiagaraBool> HalfExtents = GetTag<FNiagaraBool>(SystemInstance, FNiagaraTypeDefinition::GetBoolDef(), 5);
			TOptional<FVector3f> XAxis = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 6);
			TOptional<FVector3f> YAxis = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 7);
			TOptional<ENiagaraCoordinateSpace> AxisCoordinateSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 8);
			TOptional<int32> NumXSegments = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 9);
			TOptional<int32> NumYSegments = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 10);
			TOptional<FNiagaraBool> UnboundedPlane = GetTag<FNiagaraBool>(SystemInstance, FNiagaraTypeDefinition::GetBoolDef(), 11);
			TOptional<FLinearColor> Color = GetTag<FLinearColor>(SystemInstance, FNiagaraTypeDefinition::GetColorDef(), 12);

			FVector3f DrawCenter = Center.Get(FVector3f::ZeroVector);
			FVector3f DrawOffset = Offset.Get(FVector3f::ZeroVector);
			FVector2f DrawExtents = Extents.Get(FVector2f(10));
			FVector3f DrawXAxis = XAxis.Get(FVector3f::XAxisVector);
			FVector3f DrawYAxis = YAxis.Get(FVector3f::YAxisVector);
			int32 DrawNumXSegments = NumXSegments.Get(1);
			int32 DrawNumYSegments = NumYSegments.Get(1);
			bool bDrawUnbounded = UnboundedPlane.Get(FNiagaraBool(false)).GetValue();
			FLinearColor DrawColor = Color.Get(FLinearColor::Green);

			TransformPosition(Center.IsSet(), DrawCenter, CenterWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformVector(Offset.IsSet(), DrawOffset, OffsetWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);

			TransformVector(XAxis.IsSet(), DrawXAxis, AxisCoordinateSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformVector(YAxis.IsSet(), DrawYAxis, AxisCoordinateSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);

			if ((HalfExtents.IsSet() && HalfExtents.GetValue().GetValue()) || !HalfExtents.IsSet())
			{
				DrawExtents *= 0.5f;
			}

			if (Extents.IsSet())
			{
				InstanceData->AddRectangle(DrawCenter + DrawOffset, DrawXAxis, DrawYAxis, DrawExtents, FIntPoint(DrawNumXSegments, DrawNumYSegments), DrawColor, bDrawUnbounded);
			}
		}

		void DrawCircle(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			if (Script == nullptr)
				return;
			static FNiagaraTypeDefinition CoordTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());

			TOptional<FVector3f> Center = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 0);
			TOptional<ENiagaraCoordinateSpace> CenterWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 1);
			TOptional<FVector3f> Offset = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 2);
			TOptional<ENiagaraCoordinateSpace> OffsetWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 3);
			TOptional<float> Radius = GetTag<float>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 4);
			TOptional<FVector3f> XAxis = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 5);
			TOptional<FVector3f> YAxis = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 6);
			TOptional<ENiagaraCoordinateSpace> AxisCoordinateSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 7);
			TOptional<int32> NumSegments = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 8);
			TOptional<FLinearColor> Color = GetTag<FLinearColor>(SystemInstance, FNiagaraTypeDefinition::GetColorDef(), 9);

			FVector3f DrawCenter = Center.Get(FVector3f::ZeroVector);
			FVector3f DrawOffset = Offset.Get(FVector3f::ZeroVector);
			float DrawRadius = Radius.Get(10);
			FVector3f DrawXAxis = XAxis.Get(FVector3f::XAxisVector);
			FVector3f DrawYAxis = YAxis.Get(FVector3f::YAxisVector);
			int32 DrawNumSegments = NumSegments.Get(1);
			FLinearColor DrawColor = Color.Get(FLinearColor::Green);

			TransformPosition(Center.IsSet(), DrawCenter, CenterWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformVector(Offset.IsSet(), DrawOffset, OffsetWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);

			TransformVector(XAxis.IsSet(), DrawXAxis, AxisCoordinateSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformVector(YAxis.IsSet(), DrawYAxis, AxisCoordinateSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);

			if (Radius.IsSet())
			{
				InstanceData->AddCircle(DrawCenter + DrawOffset, DrawXAxis, DrawYAxis, DrawRadius, DrawNumSegments, DrawColor);
			}

		}

		void DrawBox(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			if (Script == nullptr)
				return;
			static FNiagaraTypeDefinition CoordTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());

			TOptional<FVector3f> Center = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 0);
			TOptional<ENiagaraCoordinateSpace> CenterWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 1);
			TOptional<FVector3f> Offset = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 2);
			TOptional<ENiagaraCoordinateSpace> OffsetWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 3);
			TOptional<FVector3f> RotationAxis = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 4);
			TOptional<float> RotationNormalizedAngle = GetTag<float>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 5);
			TOptional<ENiagaraCoordinateSpace> RotationWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 6);
			TOptional<FVector3f> Extents = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 7);
			TOptional<FNiagaraBool> HalfExtents = GetTag<FNiagaraBool>(SystemInstance, FNiagaraTypeDefinition::GetBoolDef(), 8);
			TOptional<FLinearColor> Color = GetTag<FLinearColor>(SystemInstance, FNiagaraTypeDefinition::GetColorDef(), 9);


			FVector3f DrawCenter = Center.Get(FVector3f::ZeroVector);
			FVector3f DrawOffset = Offset.Get(FVector3f::ZeroVector);
			FVector3f DrawExtents = Extents.Get(FVector3f(10.0f));
			FVector3f DrawRotationAxis = RotationAxis.Get(FVector3f::ZAxisVector);
			float DrawRotationNormalizedAngle = RotationNormalizedAngle.Get(0.0f);
			FLinearColor DrawColor = Color.Get(FLinearColor::Green);
			FQuat4f DrawRotation = FQuat4f::Identity;


			// DrawCenter and DrawOffset both are brought in and are used to locate the center of the box.
			// Assume that the extents are in local space and expand out to world space from the center.

			TransformPosition(Center.IsSet(), DrawCenter, CenterWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformVector(Offset.IsSet(), DrawOffset, OffsetWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);

			DrawRotation = FQuat4f(DrawRotationAxis, FMath::DegreesToRadians(DrawRotationNormalizedAngle * 360.0f));

			FQuat4f WorldRot = FQuat4f::Identity;
			TransformQuat(true, WorldRot, RotationWorldSpace, ENiagaraCoordinateSpace::World, SystemInstance);
			FVector3f WorldScale = FVector3f::OneVector;
			ScaleVector(true, WorldScale, RotationWorldSpace, ENiagaraCoordinateSpace::World, SystemInstance);

			if ((HalfExtents.IsSet() && HalfExtents.GetValue().GetValue()) || !HalfExtents.IsSet())
			{
				DrawExtents /= 2.0f;
			}

			if (Extents.IsSet())
			{
				FVector3f DebugPoint = DrawCenter + DrawOffset;
				TransformPosition(true, DebugPoint, TOptional<ENiagaraCoordinateSpace>(ENiagaraCoordinateSpace::Simulation), ENiagaraCoordinateSpace::World, SystemInstance);

				InstanceData->AddBox(DebugPoint, DrawRotation, DrawExtents, WorldRot, WorldScale, DrawColor);
			}
		}

		void DrawSphere(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			if (Script == nullptr)
				return;
			static FNiagaraTypeDefinition CoordTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());

			TOptional<FVector3f> Center = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 0);
			TOptional<ENiagaraCoordinateSpace> CenterWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 1);
			TOptional<FVector3f> Offset = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 2);
			TOptional<ENiagaraCoordinateSpace> OffsetWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 3);
			TOptional<float> Radius = GetTag<float>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 4);
			TOptional<ENiagaraCoordinateSpace> RadiusWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 5);
			TOptional<FNiagaraBool> HemiX = GetTag<FNiagaraBool>(SystemInstance, FNiagaraTypeDefinition::GetBoolDef(), 6);
			TOptional<FNiagaraBool> HemiY = GetTag<FNiagaraBool>(SystemInstance, FNiagaraTypeDefinition::GetBoolDef(), 7);
			TOptional<FNiagaraBool> HemiZ = GetTag<FNiagaraBool>(SystemInstance, FNiagaraTypeDefinition::GetBoolDef(), 8);
			TOptional<FVector3f> OrientationAxisForSphere = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 9);
			TOptional<ENiagaraCoordinateSpace> RotationCoordinateSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 10);
			TOptional<FQuat4f> AdditionalRotation = GetTag<FQuat4f>(SystemInstance, FNiagaraTypeDefinition::GetQuatDef(), 11);
			TOptional<FVector3f> NonUniformScale = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 12);
			TOptional<int32> NumSegments = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 13);
			TOptional<FLinearColor> Color = GetTag<FLinearColor>(SystemInstance, FNiagaraTypeDefinition::GetColorDef(), 14);

			FVector3f DrawCenter = Center.Get(FVector3f::ZeroVector);
			FVector3f DrawOffset = Offset.Get(FVector3f::ZeroVector);
			FVector3f DrawRadius = FVector3f(Radius.Get(1.0f));
			FLinearColor DrawColor = Color.Get(FLinearColor::Green);
			int32 DrawNumSegments = NumSegments.Get(36);
			FVector3f DrawNonUniformScale = NonUniformScale.Get(FVector3f::OneVector);

			FVector3f DrawOrientationAxis = OrientationAxisForSphere.Get(FVector3f::XAxisVector);
			bool bHemiXDraw = HemiX.IsSet() ? HemiX.GetValue().GetValue() : true;
			bool bHemiYDraw = HemiY.IsSet() ? HemiY.GetValue().GetValue() : true;
			bool bHemiZDraw = HemiZ.IsSet() ? HemiZ.GetValue().GetValue() : true;


			FVector3f DrawRadii = DrawNonUniformScale * DrawRadius;


			TransformPosition(Center.IsSet(), DrawCenter, CenterWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformVector(Offset.IsSet(), DrawOffset, OffsetWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);

			FQuat4f OrientationAxis = FQuat4f::FindBetweenVectors(FVector3f(1.0f, 0.0f, 0.0f), DrawOrientationAxis) * AdditionalRotation.Get(FQuat4f::Identity);

			FQuat4f WorldRot = FQuat4f::Identity;
			TransformQuat(true, WorldRot, RadiusWorldSpace, ENiagaraCoordinateSpace::World, SystemInstance);
			FVector3f WorldScale = FVector3f::OneVector;
			ScaleVector(true, WorldScale, RadiusWorldSpace, ENiagaraCoordinateSpace::World, SystemInstance);


			if (Radius.IsSet())
			{
				FVector3f DebugPoint = DrawCenter + DrawOffset;
				TransformPosition(true, DebugPoint, TOptional<ENiagaraCoordinateSpace>(ENiagaraCoordinateSpace::Simulation), ENiagaraCoordinateSpace::World, SystemInstance);

				InstanceData->AddSphere(DebugPoint, DrawRadii, OrientationAxis, WorldRot, WorldScale, DrawNumSegments, bHemiXDraw, bHemiYDraw, bHemiZDraw, DrawColor);
			}
		}

		void DrawCylinder(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			if (Script == nullptr)
				return;
			static FNiagaraTypeDefinition CoordTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());
			static FNiagaraTypeDefinition OrientationAxisTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetOrientationAxisEnum());

			TOptional<FVector3f> Center = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 0);
			TOptional<ENiagaraCoordinateSpace> CenterWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 1);
			TOptional<FVector3f> Offset = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 2);
			TOptional<ENiagaraCoordinateSpace> OffsetWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 3);
			TOptional<float> Radius = GetTag<float>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 4);
			TOptional<float> Height = GetTag<float>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 5);
			TOptional<bool> IsHalfHeight = GetTag<bool>(SystemInstance, FNiagaraTypeDefinition::GetBoolDef(), 6);
			TOptional<FNiagaraBool> HemiX = GetTag<FNiagaraBool>(SystemInstance, FNiagaraTypeDefinition::GetBoolDef(), 7);
			TOptional<FNiagaraBool> HemiY = GetTag<FNiagaraBool>(SystemInstance, FNiagaraTypeDefinition::GetBoolDef(), 8);
			TOptional<ENiagaraOrientationAxis> OrientationAxis = GetTag<ENiagaraOrientationAxis>(SystemInstance, OrientationAxisTypeDef, 9);
			TOptional<ENiagaraCoordinateSpace> OrientationAxisCoordinateSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 10);
			TOptional<FVector3f> NonUniformScale = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 11);
			TOptional<int32> NumRadiusSegments = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 12);
			TOptional<int32> NumHeightSegments = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 13);
			TOptional<FLinearColor> Color = GetTag<FLinearColor>(SystemInstance, FNiagaraTypeDefinition::GetColorDef(), 14);

			FVector3f DrawCenter = Center.Get(FVector3f::ZeroVector);
			FVector3f DrawOffset = Offset.Get(FVector3f::ZeroVector);
			float DrawRadius = Radius.Get(10.0f);
			float DrawHeight = Height.Get(5.0f);
			bool DrawHalfHeight = IsHalfHeight.Get(false);
			bool bHemiXDraw = HemiX.Get(FNiagaraBool(false)).GetValue();
			bool bHemiYDraw = HemiY.Get(FNiagaraBool(false)).GetValue();
			ENiagaraOrientationAxis DrawOrientationAxisEnum = OrientationAxis.Get(ENiagaraOrientationAxis::ZAxis);
			FVector3f DrawNonUniformScale = NonUniformScale.Get(FVector3f::OneVector);
			int32 DrawNumRadiusSegmentss = NumRadiusSegments.Get(16);
			int32 DrawNumHeightSegments = NumHeightSegments.Get(1);
			FLinearColor DrawColor = Color.Get(FLinearColor::Green);

			TransformPosition(Center.IsSet(), DrawCenter, CenterWorldSpace, ENiagaraCoordinateSpace::World, SystemInstance);
			TransformVector(Offset.IsSet(), DrawOffset, OffsetWorldSpace, ENiagaraCoordinateSpace::World, SystemInstance);

			FQuat4f WorldRot =
				DrawOrientationAxisEnum == ENiagaraOrientationAxis::XAxis ? FQuat4f(FRotator3f(0, 90, -90)) :
				DrawOrientationAxisEnum == ENiagaraOrientationAxis::YAxis ? FQuat4f(FRotator3f(0, 0, -90)) :
				FQuat4f::Identity;

			TransformQuat(true, WorldRot, OrientationAxisCoordinateSpace, ENiagaraCoordinateSpace::World, SystemInstance);

			if (DrawHalfHeight)
			{
				DrawHeight *= 2.0f;
			}

			if (Radius.IsSet())
			{
				InstanceData->AddCylinder(DrawCenter + DrawOffset, WorldRot, DrawHeight, DrawRadius, DrawNumHeightSegments, DrawNumRadiusSegmentss, DrawColor, bHemiXDraw, bHemiYDraw, DrawNonUniformScale, true);
			}
		}

		void DrawCone(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			if (Script == nullptr)
				return;
			static FNiagaraTypeDefinition CoordTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());

			TOptional<FVector3f> Center = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 0);
			TOptional<ENiagaraCoordinateSpace> CenterWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 1);
			TOptional<FVector3f> Offset = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 2);
			TOptional<ENiagaraCoordinateSpace> OffsetWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 3);
			TOptional<float> Angle = GetTag<float>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 4);
			TOptional<float> Length = GetTag<float>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 5);
			TOptional<FVector3f> OrientationAxis = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 6);
			TOptional<ENiagaraCoordinateSpace> OrientationAxisCoordinateSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 7);
			TOptional<FVector3f> NonUniformScale = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 8);
			TOptional<int32> NumRadiusSegments = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 9);
			TOptional<int32> NumHeightSegments = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 10);
			TOptional<FLinearColor> Color = GetTag<FLinearColor>(SystemInstance, FNiagaraTypeDefinition::GetColorDef(), 11);

			FVector3f DrawCenter = Center.Get(FVector3f::ZeroVector);
			FVector3f DrawOffset = Offset.Get(FVector3f::ZeroVector);
			float DrawAngle = Angle.Get(25.0f);
			float DrawLength = Length.Get(10.0f);
			FVector3f DrawOrientationAxis = OrientationAxis.Get(FVector3f::ZAxisVector);;
			int32 DrawNumRadiusSegmentss = NumRadiusSegments.Get(16);
			int32 DrawNumHeightSegments = NumHeightSegments.Get(1);
			FLinearColor DrawColor = Color.Get(FLinearColor::Green);

			FVector3f DrawNonUniformScale = NonUniformScale.Get(FVector3f::OneVector);
			TransformPosition(Center.IsSet(), DrawCenter, CenterWorldSpace, ENiagaraCoordinateSpace::World, SystemInstance);
			TransformVector(Offset.IsSet(), DrawOffset, OffsetWorldSpace, ENiagaraCoordinateSpace::World, SystemInstance);


			FQuat4f WorldRot = FQuat4f::FindBetweenNormals(FVector3f::ZAxisVector, DrawOrientationAxis);
			TransformQuat(true, WorldRot, OrientationAxisCoordinateSpace, ENiagaraCoordinateSpace::World, SystemInstance);

			FVector3f HalfOffset = DrawOrientationAxis.GetSafeNormal() * DrawLength * 0.5f;
			TransformVector(Offset.IsSet(), HalfOffset, OrientationAxisCoordinateSpace, ENiagaraCoordinateSpace::World, SystemInstance);


			float DrawRadius = DrawLength * FMath::Tan(FMath::DegreesToRadians(DrawAngle));


			if (Length.IsSet() || Angle.IsSet())
			{
				InstanceData->AddCone(DrawCenter + DrawOffset, WorldRot, DrawLength, DrawRadius, 0, DrawNumHeightSegments, DrawNumRadiusSegmentss, DrawColor, DrawNonUniformScale, false);
			}
		}

		void DrawTorus(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			if (Script == nullptr)
				return;
			static FNiagaraTypeDefinition CoordTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());

			TOptional<FVector3f> Center = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 0);
			TOptional<ENiagaraCoordinateSpace> CenterWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 1);
			TOptional<FVector3f> Offset = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 2);
			TOptional<ENiagaraCoordinateSpace> OffsetWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 3);
			TOptional<float> MajorRadius = GetTag<float>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 4);
			TOptional<float> MinorRadius = GetTag<float>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 5);
			TOptional<FVector3f> OrientationAxis = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 6);
			TOptional<ENiagaraCoordinateSpace> OrientationAxisCoordinateSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 7);
			TOptional<FVector3f> NonUniformScale = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 8);
			TOptional<int32> MajorRadiusSegments = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 9);
			TOptional<int32> MinorRadiusSegments = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 10);
			TOptional<FLinearColor> Color = GetTag<FLinearColor>(SystemInstance, FNiagaraTypeDefinition::GetColorDef(), 11);

			FVector3f DrawCenter = Center.Get(FVector3f::ZeroVector);
			FVector3f DrawOffset = Offset.Get(FVector3f::ZeroVector);
			float DrawMajorRadius = MajorRadius.Get(25.0f);
			float DrawMinorRadius = MinorRadius.Get(5.0f);
			FVector3f DrawOrientationAxis = OrientationAxis.Get(FVector3f::ZAxisVector);;
			int32 DrawMajorRadiusSegments = MajorRadiusSegments.Get(16);
			int32 DrawMinorRadiusSegments = MinorRadiusSegments.Get(8);
			FLinearColor DrawColor = Color.Get(FLinearColor::Green);

			FVector3f DrawNonUniformScale = NonUniformScale.Get(FVector3f::OneVector);
			TransformPosition(Center.IsSet(), DrawCenter, CenterWorldSpace, ENiagaraCoordinateSpace::World, SystemInstance);
			TransformVector(Offset.IsSet(), DrawOffset, OffsetWorldSpace, ENiagaraCoordinateSpace::World, SystemInstance);

			FQuat4f WorldRot = FQuat4f::Identity;
			TransformQuat(true, WorldRot, OrientationAxisCoordinateSpace, ENiagaraCoordinateSpace::World, SystemInstance);

			DrawOrientationAxis = WorldRot.RotateVector(DrawOrientationAxis);


			if (MinorRadius.IsSet() || MajorRadius.IsSet())
			{
				InstanceData->AddTorus(DrawCenter + DrawOffset, DrawOrientationAxis, DrawMajorRadius, DrawMinorRadius, DrawMajorRadiusSegments, DrawMinorRadiusSegments, DrawColor);
			}
		}

		void DrawCoordinateSystem(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			if (Script == nullptr)
				return;
			static FNiagaraTypeDefinition CoordTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());

			TOptional<FVector3f> Center = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 0);
			TOptional<ENiagaraCoordinateSpace> CenterWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 1);
			TOptional<FVector3f> Offset = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 2);
			TOptional<ENiagaraCoordinateSpace> OffsetWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 3);
			TOptional<FQuat4f> Rotation = GetTag<FQuat4f>(SystemInstance, FNiagaraTypeDefinition::GetQuatDef(), 4);
			TOptional<ENiagaraCoordinateSpace> RotationWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 5);
			TOptional<float> Scale = GetTag<float>(SystemInstance, FNiagaraTypeDefinition::GetFloatDef(), 6);

			FVector3f DrawCenter = Center.Get(FVector3f::ZeroVector);
			FVector3f DrawOffset = Offset.Get(FVector3f::ZeroVector);
			FQuat4f DrawRotation = Rotation.Get(FQuat4f::Identity);
			float DrawScale = Scale.Get(1.0f);

			TransformPosition(Center.IsSet(), DrawCenter, CenterWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformVector(Offset.IsSet(), DrawOffset, OffsetWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformQuat(Rotation.IsSet(), DrawRotation, RotationWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);

			if (Scale.IsSet())
			{
				InstanceData->AddCoordinateSystem(DrawCenter + DrawOffset, DrawRotation, DrawScale);
			}
		}

		void DrawGrid2D(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			if (Script == nullptr)
				return;
			static FNiagaraTypeDefinition CoordTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());

			TOptional<FVector3f> Center = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 0);
			TOptional<ENiagaraCoordinateSpace> CenterWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 1);
			TOptional<FVector3f> Offset = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 2);
			TOptional<ENiagaraCoordinateSpace> OffsetWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 3);
			TOptional<FQuat4f> Rotation = GetTag<FQuat4f>(SystemInstance, FNiagaraTypeDefinition::GetQuatDef(), 4);
			TOptional<ENiagaraCoordinateSpace> RotationWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 5);
			TOptional<FVector2f> Extents = GetTag<FVector2f>(SystemInstance, FNiagaraTypeDefinition::GetVec2Def(), 6);
			TOptional<int32> NumCellsX = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 7);
			TOptional<int32> NumCellsY = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 8);
			TOptional<FLinearColor> Color = GetTag<FLinearColor>(SystemInstance, FNiagaraTypeDefinition::GetColorDef(), 9);



			FVector3f DrawCenter = Center.Get(FVector3f::ZeroVector);
			FVector3f DrawOffset = Offset.Get(FVector3f::ZeroVector);
			FQuat4f DrawRotation = Rotation.Get(FQuat4f::Identity);
			FVector2f DrawExtents = Extents.Get(FVector2f(10.f));
			int32 DrawNumCellsX = NumCellsX.Get(10);
			int32 DrawNumCellsY = NumCellsY.Get(10);
			FLinearColor DrawColor = Color.Get(FLinearColor::Green);

			TransformPosition(Center.IsSet(), DrawCenter, CenterWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformVector(Offset.IsSet(), DrawOffset, OffsetWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformQuat(Rotation.IsSet(), DrawRotation, RotationWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);

			if (Extents.IsSet())
			{
				InstanceData->AddGrid2D(DrawCenter + DrawOffset, DrawRotation, DrawExtents, FIntPoint(DrawNumCellsX, DrawNumCellsY), DrawColor);
			}
		}

		void DrawGrid3D(FNDIDebugDrawInstanceData_GameThread* InstanceData, const FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
		{
			if (Script == nullptr)
				return;
			static FNiagaraTypeDefinition CoordTypeDef = FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());

			TOptional<FVector3f> Center = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 0);
			TOptional<ENiagaraCoordinateSpace> CenterWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 1);
			TOptional<FVector3f> Offset = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 2);
			TOptional<ENiagaraCoordinateSpace> OffsetWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 3);
			TOptional<FQuat4f> Rotation = GetTag<FQuat4f>(SystemInstance, FNiagaraTypeDefinition::GetQuatDef(), 4);
			TOptional<ENiagaraCoordinateSpace> RotationWorldSpace = GetTag<ENiagaraCoordinateSpace>(SystemInstance, CoordTypeDef, 5);
			TOptional<FVector3f> Extents = GetTag<FVector3f>(SystemInstance, FNiagaraTypeDefinition::GetVec3Def(), 6);
			TOptional<int32> NumCellsX = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 7);
			TOptional<int32> NumCellsY = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 8);
			TOptional<int32> NumCellsZ = GetTag<int32>(SystemInstance, FNiagaraTypeDefinition::GetIntDef(), 9);
			TOptional<FLinearColor> Color = GetTag<FLinearColor>(SystemInstance, FNiagaraTypeDefinition::GetColorDef(), 10);


			FVector3f DrawCenter = Center.Get(FVector3f::ZeroVector);
			FVector3f DrawOffset = Offset.Get(FVector3f::ZeroVector);
			FQuat4f DrawRotation = Rotation.Get(FQuat4f::Identity);
			FVector3f DrawExtents = Extents.Get(FVector3f(10.f));
			int32 DrawNumCellsX = NumCellsX.Get(10);
			int32 DrawNumCellsY = NumCellsY.Get(10);
			int32 DrawNumCellsZ = NumCellsZ.Get(10);
			FLinearColor DrawColor = Color.Get(FLinearColor::Green);

			TransformPosition(Center.IsSet(), DrawCenter, CenterWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformVector(Offset.IsSet(), DrawOffset, OffsetWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);
			TransformQuat(Rotation.IsSet(), DrawRotation, RotationWorldSpace, ENiagaraCoordinateSpace::Simulation, SystemInstance);

			if (Extents.IsSet())
			{
				InstanceData->AddGrid3D(DrawCenter + DrawOffset, DrawRotation, DrawExtents, FIntVector(DrawNumCellsX, DrawNumCellsY, DrawNumCellsZ), DrawColor);
			}
		}

	};



	FName GenerateParameterName(const TPair<FName, UNiagaraDataInterfaceDebugDraw::ShapeId>& Shape, const FString& Param)
	{
		return FName(*(Shape.Key.ToString() + TEXT(".") + Param));
	}
	

	TArray<TPair<FName, UNiagaraDataInterfaceDebugDraw::ShapeId>> PersistentShapeIds;
	TArray<FDebugPrim_PersistentShape> PersistentShapes;

	void AddNamedPersistentShape(const FName& InName, UNiagaraDataInterfaceDebugDraw::ShapeId InShapeId)
	{
		for (const TPair<FName, UNiagaraDataInterfaceDebugDraw::ShapeId>& ExistingShape : PersistentShapeIds)
		{
			if (ExistingShape.Key == InName && ExistingShape.Value == InShapeId)
				return;
		}


		PersistentShapeIds.Add(TPair<FName, UNiagaraDataInterfaceDebugDraw::ShapeId>(InName, InShapeId));
		bResolvedPersistentShapes = false;
	}


	void HandlePersistentShapes(FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
	{

		if (!bResolvedPersistentShapes && PersistentShapeIds.Num() != 0)
		{
			TArray<UNiagaraScript*> Scripts;
			TArray<bool> ScriptIsLocal;
			UNiagaraSystem* System = SystemInstance->GetSystem();
			if (System)
			{
				Scripts.Add(System->GetSystemSpawnScript());
				Scripts.Add(System->GetSystemUpdateScript());
				ScriptIsLocal.Add(false);
				ScriptIsLocal.Add(false);

				for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
				{
					FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData();
					if (EmitterData)
					{
						bool bEmitterIsLocal = EmitterData->bLocalSpace;
						if (EmitterData->SimTarget == ENiagaraSimTarget::CPUSim)
						{
							int32 ScriptCap = Scripts.Num();
							EmitterData->GetScripts(Scripts, true, true);

							for (int32 i = ScriptCap; i < Scripts.Num(); i++)
								ScriptIsLocal.Add(bEmitterIsLocal);
						}
						else
						{
							// It's a little weird to do this, but ultimately all the rapid iteration values are 
							// referenced by the compile tags from these scripts and we want to get the most up-to-date 
							// values here. If we reference the GPU script here, it will have stale values for some reason.
							Scripts.Add(EmitterData->SpawnScriptProps.Script);
							Scripts.Add(EmitterData->UpdateScriptProps.Script);
							ScriptIsLocal.Add(bEmitterIsLocal);
							ScriptIsLocal.Add(bEmitterIsLocal);
						}
					}
				}
			}

			ensure(ScriptIsLocal.Num() == Scripts.Num());

			PersistentShapes.Empty();
			for (const TPair<FName, UNiagaraDataInterfaceDebugDraw::ShapeId>& ExistingShape : PersistentShapeIds)
			{
				for (int32 i = 0; i< Scripts.Num(); i++)
				{
					UNiagaraScript* Script = Scripts[i];
					bool bIsLocal = ScriptIsLocal[i];
					if (Script)
					{
						TArray<FName, TInlineAllocator<16>> ParamNames;
						bool bIsValid = false;

						switch (ExistingShape.Value)
						{
						case UNiagaraDataInterfaceDebugDraw::Line:
						{
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("StartLocation")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("StartLocationCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("EndLocation")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("EndLocationCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Color")));

							TOptional<FVector3f> StartLocation = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[0]);
							TOptional<FVector3f> EndLocation = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[2]);

							bIsValid = StartLocation.IsSet() || EndLocation.IsSet();

							break;
						}
						case UNiagaraDataInterfaceDebugDraw::Rectangle:
						{
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Center")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("CenterCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Offset")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OffsetCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Extents")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("HalfExtents")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("XAxis")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("YAxis")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("AxisVectorsCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NumXSegments")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NumYSegments")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("UnboundedPlane")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Color")));

							TOptional<FVector3f> Center = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[0]);
							TOptional<FVector2f> Extents = GetCompileTag<FVector2f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec2Def(), ParamNames[4]);

							bIsValid = Center.IsSet() || Extents.IsSet();

							break;
						}
						case UNiagaraDataInterfaceDebugDraw::Circle:
						{
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Center")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("CenterCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Offset")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OffsetCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Radius")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("XAxis")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("YAxis")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("AxisVectorsCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NumSegments")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Color")));

							TOptional<FVector3f> Center = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[0]);
							TOptional<float> Radius = GetCompileTag<float>(SystemInstance, Script, FNiagaraTypeDefinition::GetFloatDef(), ParamNames[4]);

							bIsValid = Center.IsSet() || Radius.IsSet();

							break;
						}
						case UNiagaraDataInterfaceDebugDraw::Box:
						{
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Center")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("CenterCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Offset")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OffsetCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("RotationAxis")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("RotationNormalizedAngle")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("RotationCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Extents")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("HalfExtents")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Color")));

							TOptional<FVector3f> Center = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[0]);
							TOptional<FVector3f> Extents = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[7]);

							bIsValid = Center.IsSet() || Extents.IsSet();

							break;
						}
						case UNiagaraDataInterfaceDebugDraw::Sphere:
						{
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Center")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Center Coordinate Space")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Offset From Center")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Offset Coordinate Space")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Radius")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Radius Coordinate Space")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Hemisphere X")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Hemisphere Y")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Hemisphere Z")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Sphere Orientation Axis")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("RotationCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Additional Rotation")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NonUniform Scale")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Num Segments")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Color")));							

							TOptional<FVector3f> Center = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[0]);
							TOptional<float> Radius = GetCompileTag<float>(SystemInstance, Script, FNiagaraTypeDefinition::GetFloatDef(), ParamNames[4]);

							bIsValid = Center.IsSet() || Radius.IsSet();

							break;
						}
						case UNiagaraDataInterfaceDebugDraw::Cylinder:
						{
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Center")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("CenterCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Offset")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OffsetCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Radius")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Height")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("IsHalfHeight")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Hemisphere X")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Hemisphere Y")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Orientation Axis")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OrientationAxisCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NonUniform Scale")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Num Radius Segments")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Num Height Segments")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Color")));

							TOptional<FVector3f> Center = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[0]);
							TOptional<float> Radius = GetCompileTag<float>(SystemInstance, Script, FNiagaraTypeDefinition::GetFloatDef(), ParamNames[4]);

							bIsValid = Center.IsSet() || Radius.IsSet();

							break;
						}
						case UNiagaraDataInterfaceDebugDraw::Cone:
						{
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Center")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("CenterCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Offset")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OffsetCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Angle")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Length")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Orientation Axis")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OrientationAxisCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NonUniform Scale")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Num Radius Segments")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Num Height Segments")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Color")));

							TOptional<FVector3f> Center = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[0]);
							TOptional<float> Radius = GetCompileTag<float>(SystemInstance, Script, FNiagaraTypeDefinition::GetFloatDef(), ParamNames[4]);

							bIsValid = Center.IsSet() || Radius.IsSet();

							break;
						}
						case UNiagaraDataInterfaceDebugDraw::Torus:
						{
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Center")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("CenterCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Offset")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OffsetCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("MajorRadius")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("MinorRadius")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Orientation Axis")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OrientationAxisCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NonUniform Scale")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("MajorRadiusSegments")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("MinorRadiusSegments")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Color")));

							TOptional<FVector3f> Center = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[0]);
							TOptional<float> MajorRadius = GetCompileTag<float>(SystemInstance, Script, FNiagaraTypeDefinition::GetFloatDef(), ParamNames[4]);
							TOptional<float> MinorRadius = GetCompileTag<float>(SystemInstance, Script, FNiagaraTypeDefinition::GetFloatDef(), ParamNames[5]);

							bIsValid = Center.IsSet() || MajorRadius.IsSet() || MinorRadius.IsSet();

							break;
						}
						case UNiagaraDataInterfaceDebugDraw::CoordinateSystem:
						{
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Center")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("CenterCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Offset")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OffsetCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Rotation")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("RotationCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Scale")));

							TOptional<FVector3f> Center = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[0]);

							bIsValid = Center.IsSet();

							break;
						}
						case UNiagaraDataInterfaceDebugDraw::Grid2D:
						{
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Center")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("CenterCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Offset")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OffsetCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Rotation")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("RotationCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Extents")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NumCellsX")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NumCellsY")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Color")));

							TOptional<FVector3f> Center = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[0]);

							bIsValid = Center.IsSet();

							break;
						}
						case UNiagaraDataInterfaceDebugDraw::Grid3D:
						{
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Center")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("CenterCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Offset")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("OffsetCoordinateSpace")));							
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Rotation")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("RotationCoordinateSpace")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Extents")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NumCellsX")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NumCellsY")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("NumCellsZ")));
							ParamNames.Add(GenerateParameterName(ExistingShape, TEXT("Color")));

							TOptional<FVector3f> Center = GetCompileTag<FVector3f>(SystemInstance, Script, FNiagaraTypeDefinition::GetVec3Def(), ParamNames[0]);

							bIsValid = Center.IsSet();

							break;
						}
						}		

						if (bIsValid)
						{
							int32 Index = PersistentShapes.Emplace();
							PersistentShapes[Index].ShapeId = ExistingShape.Value;
							PersistentShapes[Index].Script = Script;
							PersistentShapes[Index].bSimSpaceIsLocal = bIsLocal;
							PersistentShapes[Index].ParameterNames = MoveTemp(ParamNames);
						}
					}
				}
			}


			bResolvedPersistentShapes = true;
		}

		if (bResolvedPersistentShapes)
		{
			for (FDebugPrim_PersistentShape& Shape : PersistentShapes)
			{
				Shape.Draw(this, SystemInstance, DeltaSeconds);
			}
		}
	}

#endif //NIAGARA_COMPUTEDEBUG_ENABLED
};

#if NIAGARA_COMPUTEDEBUG_ENABLED
struct FNDIDebugDrawInstanceData_RenderThread
{
	FNiagaraGpuComputeDebug* GpuComputeDebug = nullptr;
	uint32 OverrideMaxLineInstances = 0;
};
#endif //NIAGARA_COMPUTEDEBUG_ENABLED

struct FNDIDebugDrawProxy : public FNiagaraDataInterfaceProxy
{
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override {}
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

#if NIAGARA_COMPUTEDEBUG_ENABLED
	TMap<FNiagaraSystemInstanceID, FNDIDebugDrawInstanceData_RenderThread> SystemInstancesToProxyData_RT;
#endif
};

//////////////////////////////////////////////////////////////////////////

namespace NDIDebugDrawLocal
{
	static const FName CompileTagKey = TEXT("CompilerTagKey");
	static const TCHAR* CommonShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceDebugDraw.ush");
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceDebugDrawTemplate.ush");

	static const FName DrawLineName(TEXT("DrawLine"));
	static const FName DrawRectangleName(TEXT("DrawRectangle"));
	static const FName DrawCircleName(TEXT("DrawCircle"));
	static const FName DrawBoxName(TEXT("DrawBox"));
	static const FName DrawSphereName(TEXT("DrawSphere"));
	static const FName DrawCylinderName(TEXT("DrawCylinder"));
	static const FName DrawConeName(TEXT("DrawCone"));
	static const FName DrawTorusName(TEXT("DrawTorus"));
	static const FName DrawCoordinateSystemName(TEXT("DrawCoordinateSystem"));
	static const FName DrawGrid2DName(TEXT("DrawGrid2D"));
	static const FName DrawGrid3DName(TEXT("DrawGrid3D"));

	static const FName DrawLinePersistentName(TEXT("DrawLinePersistent"));
	static const FName DrawRectanglePersistentName(TEXT("DrawRectanglePersistent"));
	static const FName DrawCirclePersistentName(TEXT("DrawCirclePersistent"));
	static const FName DrawBoxPersistentName(TEXT("DrawBoxPersistent"));
	static const FName DrawSpherePersistentName(TEXT("DrawSpherePersistent"));
	static const FName DrawCylinderPersistentName(TEXT("DrawCylinderPersistent"));
	static const FName DrawConePersistentName(TEXT("DrawConePersistent"));
	static const FName DrawTorusPersistentName(TEXT("DrawTorusPersistent"));
	static const FName DrawCoordinateSystemPersistentName(TEXT("DrawCoordinateSystemPersistent"));
	static const FName DrawGrid2DPersistentName(TEXT("DrawGrid2DPersistent"));
	static const FName DrawGrid3DPersistentName(TEXT("DrawGrid3DPersistent"));

	static int32 GNiagaraDebugDrawEnabled = 1;
	static FAutoConsoleVariableRef CVarNiagaraDebugDrawEnabled(
		TEXT("fx.Niagara.DebugDraw.Enabled"),
		GNiagaraDebugDrawEnabled,
		TEXT("Enable or disable the Debug Draw Data Interface, note does not fully disable the overhead."),
		ECVF_Default
	);

	struct FDebugPrim_Line
	{
		struct VMBindings
		{
			VMBindings(FVectorVMExternalFunctionContext& Context)
				: LineStartParam(Context)
				, LineEndParam(Context)
				, ColorParam(Context)
			{
			}

			FNDIInputParam<FVector3f> LineStartParam;
			FNDIInputParam<FVector3f> LineEndParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

		struct PersistentVMBindings
		{
			PersistentVMBindings(FVectorVMExternalFunctionContext& Context)
				: StartLocationParam(Context)
				, StartLocationWSParam(Context)
				, EndLocationParam(Context)
				, EndLocationWSParam(Context)
				, ColorParam(Context)
			{

			}

			FNDIInputParam<FVector3f> StartLocationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> StartLocationWSParam;
			FNDIInputParam<FVector3f> EndLocationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> EndLocationWSParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

#if NIAGARA_COMPUTEDEBUG_ENABLED
		static void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, VMBindings& Bindings, bool bExecute)
		{
			const FVector3f LineStart = Bindings.LineStartParam.GetAndAdvance();
			const FVector3f LineEnd = Bindings.LineEndParam.GetAndAdvance();
			const FLinearColor Color = Bindings.ColorParam.GetAndAdvance();

			if (bExecute)
			{
				InstanceData->AddLine(LineStart, LineEnd, Color);
			}
		}
#endif
	};

	struct FDebugPrim_Rectangle
	{
		struct VMBindings
		{
			VMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, XAxisParam(Context)
				, YAxisParam(Context)
				, ExtentsParam(Context)
				, NumXSegments(Context)
				, NumYSegments(Context)
				, ColorParam(Context)
				, UnboundedParam(Context)
			{
			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<FVector3f> XAxisParam;
			FNDIInputParam<FVector3f> YAxisParam;
			FNDIInputParam<FVector2f> ExtentsParam;
			FNDIInputParam<int32> NumXSegments;
			FNDIInputParam<int32> NumYSegments;
			FNDIInputParam<FLinearColor> ColorParam;
			FNDIInputParam<FNiagaraBool> UnboundedParam;
		};

		struct PersistentVMBindings
		{
			PersistentVMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, LocationWSParam(Context)
				, OffsetParam(Context)
				, OffsetWSParam(Context)
				, ExtentsParam(Context)
				, HalfExtentsParam(Context)
				, XAxisParam(Context)
				, YAxisParam(Context)
				, AxisCoordinateSpace(Context)
				, NumXSegmentsParam(Context)
				, NumYSegmentsParam(Context)
				, UnboundedParam(Context)
				, ColorParam(Context)
			{

			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> LocationWSParam;
			FNDIInputParam<FVector3f> OffsetParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OffsetWSParam;
			FNDIInputParam<FVector2f> ExtentsParam;
			FNDIInputParam<FNiagaraBool> HalfExtentsParam;
			FNDIInputParam<FVector3f> XAxisParam;
			FNDIInputParam<FVector3f> YAxisParam;
			FNDIInputParam<ENiagaraCoordinateSpace> AxisCoordinateSpace;
			FNDIInputParam<int32> NumXSegmentsParam;
			FNDIInputParam<int32> NumYSegmentsParam;
			FNDIInputParam<FNiagaraBool> UnboundedParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

#if NIAGARA_COMPUTEDEBUG_ENABLED
		static void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, VMBindings& Bindings, bool bExecute)
		{
			const FVector3f Location = Bindings.LocationParam.GetAndAdvance();
			const FVector3f XAxis = Bindings.XAxisParam.GetAndAdvance();
			const FVector3f YAxis = Bindings.YAxisParam.GetAndAdvance();
			const FVector2f Extents = Bindings.ExtentsParam.GetAndAdvance();
			const FIntPoint NumSegments = FIntPoint(FMath::Clamp(Bindings.NumXSegments.GetAndAdvance(), 1, 16), FMath::Clamp(Bindings.NumYSegments.GetAndAdvance(), 1, 16));
			const FLinearColor Color = Bindings.ColorParam.GetAndAdvance();

			const bool bUnbounded = Bindings.UnboundedParam.GetAndAdvance();

			if (bExecute)
			{
				InstanceData->AddRectangle(Location, XAxis, YAxis, Extents, NumSegments, Color, bUnbounded);
			}
		}
#endif
	};

	struct FDebugPrim_Circle
	{
		struct VMBindings
		{
			VMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, XAxisParam(Context)
				, YAxisParam(Context)
				, ScaleParam(Context)
				, SegmentsParam(Context)
				, ColorParam(Context)
			{
			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<FVector3f> XAxisParam;
			FNDIInputParam<FVector3f> YAxisParam;
			FNDIInputParam<float> ScaleParam;
			FNDIInputParam<int32> SegmentsParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

		struct PersistentVMBindings
		{
			PersistentVMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, LocationWSParam(Context)
				, OffsetParam(Context)
				, OffsetWSParam(Context)
				, RadiusParam(Context)
				, XAxisParam(Context)
				, YAxisParam(Context)
				, AxisCoordinateSpace(Context)
				, NumSegmentsParam(Context)
				, ColorParam(Context)
			{

			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> LocationWSParam;
			FNDIInputParam<FVector3f> OffsetParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OffsetWSParam;
			FNDIInputParam<float> RadiusParam;
			FNDIInputParam<FVector3f> XAxisParam;
			FNDIInputParam<FVector3f> YAxisParam;
			FNDIInputParam<ENiagaraCoordinateSpace> AxisCoordinateSpace;
			FNDIInputParam<int32> NumSegmentsParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

#if NIAGARA_COMPUTEDEBUG_ENABLED
		static void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, VMBindings& Bindings, bool bExecute)
		{
			const FVector3f Location = Bindings.LocationParam.GetAndAdvance();
			const FVector3f XAxis = Bindings.XAxisParam.GetAndAdvance();
			const FVector3f YAxis = Bindings.YAxisParam.GetAndAdvance();
			const float Scale = Bindings.ScaleParam.GetAndAdvance();
			const int32 Segments = FMath::Clamp(Bindings.SegmentsParam.GetAndAdvance(), 4, 16);
			const FLinearColor Color = Bindings.ColorParam.GetAndAdvance();

			if (bExecute)
			{
				InstanceData->AddCircle(Location, XAxis, YAxis, Scale, Segments, Color);
			}
		}
#endif
	};

	struct FDebugPrim_Box
	{
		struct VMBindings
		{
			VMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, RotationParam(Context)
				, ExtentsParam(Context)
				, ColorParam(Context)
			{
			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<FQuat4f> RotationParam;
			FNDIInputParam<FVector3f> ExtentsParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

		struct PersistentVMBindings
		{
			PersistentVMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, LocationWSParam(Context)
				, OffsetParam(Context)
				, OffsetWSParam(Context)
				, ExtentsParam(Context)
				, HalfExtentsParam(Context)
				, RotationAxisParam(Context)
				, RotationAngleParam(Context)
				, RotationWSParam(Context)
				, ColorParam(Context)
			{

			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> LocationWSParam;
			FNDIInputParam<FVector3f> OffsetParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OffsetWSParam;
			FNDIInputParam<FVector3f> ExtentsParam;
			FNDIInputParam<FNiagaraBool> HalfExtentsParam;
			FNDIInputParam<FVector3f> RotationAxisParam;
			FNDIInputParam<float> RotationAngleParam;
			FNDIInputParam<ENiagaraCoordinateSpace> RotationWSParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

#if NIAGARA_COMPUTEDEBUG_ENABLED
		static void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, VMBindings& Bindings, bool bExecute)
		{
			const FVector3f Location = Bindings.LocationParam.GetAndAdvance();
			const FQuat4f Rotation = Bindings.RotationParam.GetAndAdvance();
			const FVector3f Extents = Bindings.ExtentsParam.GetAndAdvance();
			const FLinearColor Color = Bindings.ColorParam.GetAndAdvance();

			if (bExecute)
			{
				InstanceData->AddBox(Location, Rotation, Extents, Color);
			}
		}
#endif
	};

	struct FDebugPrim_Sphere
	{
		struct VMBindings
		{
			VMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, RadiusParam(Context)
				, SegmentsParam(Context)
				, ColorParam(Context)
			{
			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<float> RadiusParam;
			FNDIInputParam<int32> SegmentsParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

		struct PersistentVMBindings
		{
			PersistentVMBindings(FVectorVMExternalFunctionContext& Context)
				: CenterParam(Context)
				, CenterWSParam(Context)
				, OffsetParam(Context)
				, OffsetWSParam(Context)
				, RadiusParam(Context)
				, RadiusWSParam(Context)
				, HemiXParam(Context)
				, HemiYParam(Context)
				, HemiZParam(Context)
				, OrientationAxisParam(Context)
				, RotationWSParam(Context)
				, AdditionalRotationParam(Context)
				, NonUniformScaleParam(Context)
				, SegmentsParam(Context)
				, ColorParam(Context)
			{
			}

			FNDIInputParam<FVector3f> CenterParam;
			FNDIInputParam<ENiagaraCoordinateSpace> CenterWSParam;
			FNDIInputParam<FVector3f> OffsetParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OffsetWSParam;
			FNDIInputParam<float> RadiusParam;
			FNDIInputParam<ENiagaraCoordinateSpace> RadiusWSParam;
			FNDIInputParam<FNiagaraBool> HemiXParam;
			FNDIInputParam<FNiagaraBool> HemiYParam;
			FNDIInputParam<FNiagaraBool> HemiZParam;
			FNDIInputParam<FVector3f> OrientationAxisParam;
			FNDIInputParam<ENiagaraCoordinateSpace> RotationWSParam;
			FNDIInputParam<FQuat4f> AdditionalRotationParam;
			FNDIInputParam<FVector3f> NonUniformScaleParam;
			FNDIInputParam<int32> SegmentsParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

#if NIAGARA_COMPUTEDEBUG_ENABLED
		static void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, VMBindings& Bindings, bool bExecute)
		{
			const FVector3f Location = Bindings.LocationParam.GetAndAdvance();
			const float Radius = Bindings.RadiusParam.GetAndAdvance();
			const int32 Segments = FMath::Clamp(Bindings.SegmentsParam.GetAndAdvance(), 4, 16);
			const FLinearColor Color = Bindings.ColorParam.GetAndAdvance();

			if (bExecute)
			{
				InstanceData->AddSphere(Location, Radius, Segments, Color);
			}
		}
#endif
	};

	struct FDebugPrim_Cylinder
	{
		struct VMBindings
		{
			VMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, AxisParam(Context)
				, HeightParam(Context)
				, RadiusParam(Context)
				, NumHeightSegmentsParam(Context)
				, NumRadiusSegmentsParam(Context)
				, ColorParam(Context)
			{
			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<FVector3f> AxisParam;
			FNDIInputParam<float> HeightParam;
			FNDIInputParam<float> RadiusParam;
			FNDIInputParam<int32> NumHeightSegmentsParam;
			FNDIInputParam<int32> NumRadiusSegmentsParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

		struct PersistentVMBindings
		{
			PersistentVMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, LocationWSParam(Context)
				, OffsetParam(Context)
				, OffsetWSParam(Context)
				, RadiusParam(Context)
				, HeightParam(Context)
				, IsHalfHeightParam(Context)
				, HemisphereXParam(Context)
				, HemisphereYParam(Context)
				, OrientationAxisParam(Context)
				, OrientationAxisCoordinateSpaceParam(Context)
				, NonUniformScaleParam(Context)
				, NumRadiusSegmentsParam(Context)
				, NumHeightSegmentsParam(Context)
				, ColorParam(Context)
			{

			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> LocationWSParam;
			FNDIInputParam<FVector3f> OffsetParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OffsetWSParam;
			FNDIInputParam<float> RadiusParam;
			FNDIInputParam<float> HeightParam;
			FNDIInputParam<FNiagaraBool> IsHalfHeightParam;
			FNDIInputParam<FNiagaraBool> HemisphereXParam;
			FNDIInputParam<FNiagaraBool> HemisphereYParam;
			FNDIInputParam<ENiagaraOrientationAxis> OrientationAxisParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OrientationAxisCoordinateSpaceParam;
			FNDIInputParam<FVector3f> NonUniformScaleParam;
			FNDIInputParam<int32> NumRadiusSegmentsParam;
			FNDIInputParam<int32> NumHeightSegmentsParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};


#if NIAGARA_COMPUTEDEBUG_ENABLED
		static void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, VMBindings& Bindings, bool bExecute)
		{
			const FVector3f Location = Bindings.LocationParam.GetAndAdvance();
			const FVector3f Axis = Bindings.AxisParam.GetAndAdvance();
			const float Height = Bindings.HeightParam.GetAndAdvance();
			const float Radius = Bindings.RadiusParam.GetAndAdvance();
			const int32 NumHeightSegments = Bindings.NumHeightSegmentsParam.GetAndAdvance();
			const int32 NumRadiusSegments = Bindings.NumRadiusSegmentsParam.GetAndAdvance();
			const FLinearColor Color = Bindings.ColorParam.GetAndAdvance();

			if (bExecute)
			{
				InstanceData->AddCylinder(Location, Axis, Height, Radius, NumHeightSegments, NumRadiusSegments, Color);
			}
		}
#endif
	};

	struct FDebugPrim_Cone
	{
		struct VMBindings
		{
			VMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, AxisParam(Context)
				, HeightParam(Context)
				, RadiusTopParam(Context)
				, RadiusBottomParam(Context)
				, NumHeightSegmentsParam(Context)
				, NumRadiusSegmentsParam(Context)
				, ColorParam(Context)
			{
			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<FVector3f> AxisParam;
			FNDIInputParam<float> HeightParam;
			FNDIInputParam<float> RadiusTopParam;
			FNDIInputParam<float> RadiusBottomParam;
			FNDIInputParam<int32> NumHeightSegmentsParam;
			FNDIInputParam<int32> NumRadiusSegmentsParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

		struct PersistentVMBindings
		{
			PersistentVMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, LocationWSParam(Context)
				, OffsetParam(Context)
				, OffsetWSParam(Context)
				, AngleParam(Context)
				, LengthParam(Context)
				, OrientationAxisParam(Context)
				, OrientationAxisCoordinateSpaceParam(Context)
				, NonUniformScaleParam(Context)
				, NumRadiusSegmentsParam(Context)
				, NumHeightSegmentsParam(Context)
				, ColorParam(Context)
			{

			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> LocationWSParam;
			FNDIInputParam<FVector3f> OffsetParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OffsetWSParam;
			FNDIInputParam<float> AngleParam;
			FNDIInputParam<float> LengthParam;
			FNDIInputParam<FVector3f> OrientationAxisParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OrientationAxisCoordinateSpaceParam;
			FNDIInputParam<FVector3f> NonUniformScaleParam;
			FNDIInputParam<int32> NumRadiusSegmentsParam;
			FNDIInputParam<int32> NumHeightSegmentsParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

#if NIAGARA_COMPUTEDEBUG_ENABLED
		static void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, VMBindings& Bindings, bool bExecute)
		{
			const FVector3f Location = Bindings.LocationParam.GetAndAdvance();
			const FVector3f Axis = Bindings.AxisParam.GetAndAdvance();
			const float Height = Bindings.HeightParam.GetAndAdvance();
			const float RadiusTop = Bindings.RadiusTopParam.GetAndAdvance();
			const float RadiusBottom = Bindings.RadiusBottomParam.GetAndAdvance();
			const int32 NumHeightSegments = Bindings.NumHeightSegmentsParam.GetAndAdvance();
			const int32 NumRadiusSegments = Bindings.NumRadiusSegmentsParam.GetAndAdvance();
			const FLinearColor Color = Bindings.ColorParam.GetAndAdvance();

			if (bExecute)
			{
				InstanceData->AddCone(Location, Axis, Height, RadiusTop, RadiusBottom, NumHeightSegments, NumRadiusSegments, Color);
			}
		}
#endif
	};

	struct FDebugPrim_Torus
	{
		struct VMBindings
		{
			VMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, AxisParam(Context)
				, MajorRadiusParam(Context)
				, MinorRadiusParam(Context)
				, MajorRadiusSegmentsParam(Context)
				, MinorRadiusSegmentsParam(Context)
				, ColorParam(Context)
			{
			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<FVector3f> AxisParam;
			FNDIInputParam<float> MajorRadiusParam;
			FNDIInputParam<float> MinorRadiusParam;
			FNDIInputParam<int32> MajorRadiusSegmentsParam;
			FNDIInputParam<int32> MinorRadiusSegmentsParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

		struct PersistentVMBindings
		{
			PersistentVMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, LocationWSParam(Context)
				, OffsetParam(Context)
				, OffsetWSParam(Context)
				, MajorRadiusParam(Context)
				, MinorRadiusParam(Context)
				, OrientationAxisParam(Context)
				, OrientationAxisCoordinateSpaceParam(Context)
				, NonUniformScaleParam(Context)
				, MajorRadiusSegments(Context)
				, MinorRadiusSegments(Context)
				, ColorParam(Context)
			{

			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> LocationWSParam;
			FNDIInputParam<FVector3f> OffsetParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OffsetWSParam;
			FNDIInputParam<float> MajorRadiusParam;
			FNDIInputParam<float> MinorRadiusParam;
			FNDIInputParam<FVector3f> OrientationAxisParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OrientationAxisCoordinateSpaceParam;
			FNDIInputParam<FVector3f> NonUniformScaleParam;
			FNDIInputParam<int32> MajorRadiusSegments;
			FNDIInputParam<int32> MinorRadiusSegments;
			FNDIInputParam<FLinearColor> ColorParam;
		};

#if NIAGARA_COMPUTEDEBUG_ENABLED
		static void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, VMBindings& Bindings, bool bExecute)
		{
			const FVector3f Location = Bindings.LocationParam.GetAndAdvance();
			const FVector3f Axis = Bindings.AxisParam.GetAndAdvance();
			const float MajorRadius = Bindings.MajorRadiusParam.GetAndAdvance();
			const float MinorRadius = Bindings.MinorRadiusParam.GetAndAdvance();
			const int32 MajorRadiusSegments = Bindings.MajorRadiusSegmentsParam.GetAndAdvance();
			const int32 MinorRadiusSegments = Bindings.MinorRadiusSegmentsParam.GetAndAdvance();
			const FLinearColor Color = Bindings.ColorParam.GetAndAdvance();

			if (bExecute)
			{
				InstanceData->AddTorus(Location, Axis, MajorRadius, MinorRadius, MajorRadiusSegments, MinorRadiusSegments, Color);
			}
		}
#endif
	};

	struct FDebugPrim_CoordinateSystem
	{
		struct VMBindings
		{
			VMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, RotationParam(Context)
				, ScaleParam(Context)
			{
			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<FQuat4f> RotationParam;
			FNDIInputParam<float> ScaleParam;
		};		
		
		struct PersistentVMBindings
		{
			PersistentVMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, LocationWSParam(Context)
				, OffsetParam(Context)
				, OffsetWSParam(Context)
				, RotationParam(Context)
				, RotationCoordinateSpace(Context)
				, ScaleParam(Context)
			{

			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> LocationWSParam;
			FNDIInputParam<FVector3f> OffsetParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OffsetWSParam;
			FNDIInputParam<FQuat4f> RotationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> RotationCoordinateSpace;
			FNDIInputParam<float> ScaleParam;
		};


#if NIAGARA_COMPUTEDEBUG_ENABLED
		static void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, VMBindings& Bindings, bool bExecute)
		{
			const FVector3f Location = Bindings.LocationParam.GetAndAdvance();
			const FQuat4f Rotation = Bindings.RotationParam.GetAndAdvance();
			const float Scale = Bindings.ScaleParam.GetAndAdvance();

			if (bExecute)
			{
				InstanceData->AddCoordinateSystem(Location, Rotation, Scale);
			}
		}
#endif
	};

	struct FDebugPrim_Grid2D
	{
		struct VMBindings
		{
			VMBindings(FVectorVMExternalFunctionContext& Context)
				: CenterParam(Context)
				, RotationParam(Context)
				, ExtentsParam(Context)
				, NumCellsXParam(Context)
				, NumCellsYParam(Context)
				, ColorParam(Context)
			{
			}

			FNDIInputParam<FVector3f> CenterParam;
			FNDIInputParam<FQuat4f> RotationParam;
			FNDIInputParam<FVector2f> ExtentsParam;
			FNDIInputParam<int32> NumCellsXParam;
			FNDIInputParam<int32> NumCellsYParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

		struct PersistentVMBindings
		{
			PersistentVMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, LocationWSParam(Context)
				, OffsetParam(Context)
				, OffsetWSParam(Context)
				, RotationParam(Context)
				, RotationCoordinateSpace(Context)
				, ExtentsParam(Context)
				, NumCellsXParam(Context)
				, NumCellsYParam(Context)
				, ColorParam(Context)
			{

			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> LocationWSParam;
			FNDIInputParam<FVector3f> OffsetParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OffsetWSParam;
			FNDIInputParam<FQuat4f> RotationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> RotationCoordinateSpace;
			FNDIInputParam<FVector2f> ExtentsParam;
			FNDIInputParam<int32> NumCellsXParam;
			FNDIInputParam<int32> NumCellsYParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

#if NIAGARA_COMPUTEDEBUG_ENABLED
		static void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, VMBindings& Bindings, bool bExecute)
		{
			const FVector3f Center = Bindings.CenterParam.GetAndAdvance();
			const FQuat4f Rotation = Bindings.RotationParam.GetAndAdvance();
			const FVector2f Extents = Bindings.ExtentsParam.GetAndAdvance();
			const FIntPoint NumCells(Bindings.NumCellsXParam.GetAndAdvance(), Bindings.NumCellsYParam.GetAndAdvance());
			const FLinearColor Color = Bindings.ColorParam.GetAndAdvance();

			if (bExecute && NumCells.X > 0 && NumCells.Y > 0)
			{
				InstanceData->AddGrid2D(Center, Rotation, Extents, NumCells, Color);
			}
		}
#endif
	};

	struct FDebugPrim_Grid3D
	{
		struct VMBindings
		{
			VMBindings(FVectorVMExternalFunctionContext& Context)
				: CenterParam(Context)
				, RotationParam(Context)
				, ExtentsParam(Context)
				, NumCellsXParam(Context)
				, NumCellsYParam(Context)
				, NumCellsZParam(Context)
				, ColorParam(Context)
			{
			}

			FNDIInputParam<FVector3f> CenterParam;
			FNDIInputParam<FQuat4f> RotationParam;
			FNDIInputParam<FVector3f> ExtentsParam;
			FNDIInputParam<int32> NumCellsXParam;
			FNDIInputParam<int32> NumCellsYParam;
			FNDIInputParam<int32> NumCellsZParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};

		struct PersistentVMBindings
		{
			PersistentVMBindings(FVectorVMExternalFunctionContext& Context)
				: LocationParam(Context)
				, LocationWSParam(Context)
				, OffsetParam(Context)
				, OffsetWSParam(Context)
				, RotationParam(Context)
				, RotationCoordinateSpace(Context)
				, ExtentsParam(Context)
				, NumCellsXParam(Context)
				, NumCellsYParam(Context)
				, NumCellsZParam(Context)
				, ColorParam(Context)
			{

			}

			FNDIInputParam<FVector3f> LocationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> LocationWSParam;
			FNDIInputParam<FVector3f> OffsetParam;
			FNDIInputParam<ENiagaraCoordinateSpace> OffsetWSParam;
			FNDIInputParam<FQuat4f> RotationParam;
			FNDIInputParam<ENiagaraCoordinateSpace> RotationCoordinateSpace;
			FNDIInputParam<FVector3f> ExtentsParam;
			FNDIInputParam<int32> NumCellsXParam;
			FNDIInputParam<int32> NumCellsYParam;
			FNDIInputParam<int32> NumCellsZParam;
			FNDIInputParam<FLinearColor> ColorParam;
		};


#if NIAGARA_COMPUTEDEBUG_ENABLED
		static void Draw(FNDIDebugDrawInstanceData_GameThread* InstanceData, VMBindings& Bindings, bool bExecute)
		{
			const FVector3f Center = Bindings.CenterParam.GetAndAdvance();
			const FQuat4f Rotation = Bindings.RotationParam.GetAndAdvance();
			const FVector3f Extents = Bindings.ExtentsParam.GetAndAdvance();
			const FIntVector NumCells(Bindings.NumCellsXParam.GetAndAdvance(), Bindings.NumCellsYParam.GetAndAdvance(), Bindings.NumCellsZParam.GetAndAdvance());
			const FLinearColor Color = Bindings.ColorParam.GetAndAdvance();

			if (bExecute && NumCells.X > 0 && NumCells.Y > 0 && NumCells.Z > 0)
			{
				InstanceData->AddGrid3D(Center, Rotation, Extents, NumCells, Color);
			}
		}
#endif
	};


	template<typename TPrimType>
	void DrawDebug(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIDebugDrawInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<FNiagaraBool> ExecuteParam(Context);
		typename TPrimType::VMBindings Bindings(Context);

#if NIAGARA_COMPUTEDEBUG_ENABLED
		if ( !NDIDebugDrawLocal::GNiagaraDebugDrawEnabled )
		{
			return;
		}

		for ( int i=0; i < Context.GetNumInstances(); ++i )
		{
			const bool bExecute = ExecuteParam.GetAndAdvance();
			TPrimType::Draw(InstanceData, Bindings, bExecute);
		}
#endif
	}

	template<typename TPrimType>
	void DrawDebugPersistent(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIDebugDrawInstanceData_GameThread> InstanceData(Context);
		typename TPrimType::PersistentVMBindings Bindings(Context);

#if NIAGARA_COMPUTEDEBUG_ENABLED
		if (!NDIDebugDrawLocal::GNiagaraDebugDrawEnabled)
		{
			return;
		}

		// Do nothing here... will draw this later on..
#endif
	}
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceDebugDraw::UNiagaraDataInterfaceDebugDraw(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNDIDebugDrawProxy());
}

void UNiagaraDataInterfaceDebugDraw::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA

// Codegen optimization degenerates for very long functions like GetFunctions when combined with the invocation of lots of FORCEINLINE methods.
BEGIN_FUNCTION_BUILD_OPTIMIZATION

void UNiagaraDataInterfaceDebugDraw::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	OutFunctions.Reserve(OutFunctions.Num() + 22);
	const int32 FirstFunction = OutFunctions.Num();

	FNiagaraFunctionSignature DefaultSignature;
	DefaultSignature.bMemberFunction = true;
	DefaultSignature.bRequiresContext = false;
	DefaultSignature.bSupportsGPU = true;
	DefaultSignature.bExperimental = true;
	DefaultSignature.bRequiresExecPin = true;

	// Line
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawLineName;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Start Location")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("End Location")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Rectangle
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawRectangleName;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("XAxis"))).SetValue(FVector3f::XAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("YAxis"))).SetValue(FVector3f::YAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Extents"))).SetValue(FVector2f(10.0f));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumXSegments"))).SetValue(1);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumYSegments"))).SetValue(1);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("UnboundedPlane"))).SetValue(false);
	}

	// Circle
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawCircleName;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("XAxis"))).SetValue(FVector3f::XAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("YAxis"))).SetValue(FVector3f::YAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Radius"))).SetValue(10.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Segments"))).SetValue(6);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Box
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawBoxName;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Extents"))).SetValue(FVector3f(10.0f));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Sphere
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawSphereName;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Radius"))).SetValue(10.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Segments"))).SetValue(6);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Cylinder
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawCylinderName;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Axis"))).SetValue(FVector3f::ZAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Height"))).SetValue(10.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Radius"))).SetValue(10.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumHeightSegments"))).SetValue(1);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumRadiusSegments"))).SetValue(6);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Cone
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawConeName;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Axis"))).SetValue(FVector3f::ZAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Height"))).SetValue(10.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("RadiusTop"))).SetValue(0.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("RadiusBottom"))).SetValue(10.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumHeightSegments"))).SetValue(1);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumRadiusSegments"))).SetValue(6);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Torus
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawTorusName;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Axis"))).SetValue(FVector3f::ZAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MajorRadius"))).SetValue(100.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MinorRadius"))).SetValue(25.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MajorRadiusSegments"))).SetValue(12);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MinorRadiusSegments"))).SetValue(6);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Coordinate System
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawCoordinateSystemName;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Location")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Scale"))).SetValue(1.0f);
	}

	// Grid2D
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawGrid2DName;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Extents"))).SetValue(FVector2f(10.0f));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX"))).SetValue(1);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY"))).SetValue(1);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Grid3D
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawGrid3DName;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Extents"))).SetValue(FVector3f(10.0f));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX"))).SetValue(1);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY"))).SetValue(1);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsZ"))).SetValue(1);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	FNiagaraTypeDefinition CoordTypeDef(FNiagaraTypeDefinition::GetCoordinateSpaceEnum());
	FNiagaraTypeDefinition AxisTypeDef(FNiagaraTypeDefinition::GetOrientationAxisEnum());

	// Line Persistent
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawLinePersistentName;
		Signature.FunctionSpecifiers.Add(TEXT("Identifier"));
		Signature.bIsCompileTagGenerator = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("StartLocation")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("StartLocationCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("EndLocation")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("EndLocationCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Rectangle Persistent
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawRectanglePersistentName;
		Signature.FunctionSpecifiers.Add(TEXT("Identifier"));
		Signature.bIsCompileTagGenerator = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("CenterCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Offset"))).SetValue(FVector3f::ZeroVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OffsetCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Extents"))).SetValue(FVector2f(5, 5));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("HalfExtents"))).SetValue(true);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("XAxis"))).SetValue(FVector3f::XAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("YAxis"))).SetValue(FVector3f::YAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("AxisVectorsCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumXSegments"))).SetValue(1);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumYSegments"))).SetValue(1);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("UnboundedPlane"))).SetValue(false);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Circle Persistent
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawCirclePersistentName;
		Signature.FunctionSpecifiers.Add(TEXT("Identifier"));
		Signature.bIsCompileTagGenerator = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("CenterCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Offset"))).SetValue(FVector3f::ZeroVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OffsetCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Radius"))).SetValue(10.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("XAxis"))).SetValue(FVector3f::XAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("YAxis"))).SetValue(FVector3f::YAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("AxisVectorsCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumSegments"))).SetValue(6);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Box Persistent
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawBoxPersistentName;
		Signature.FunctionSpecifiers.Add(TEXT("Identifier"));
		Signature.bIsCompileTagGenerator = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("CenterCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Offset"))).SetValue(FVector3f::ZeroVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OffsetCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Extents"))).SetValue(FVector3f(10.0f));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("HalfExtents"))).SetValue(true);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("RotationAxis"))).SetValue(FVector3f::ZAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("RotationNormalizedAngle"))).SetValue(0.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("RotationCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Sphere Persistent
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawSpherePersistentName;
		Signature.FunctionSpecifiers.Add(TEXT("Identifier"));
		Signature.bIsCompileTagGenerator = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("Center Coordinate Space"))).SetValue(ENiagaraCoordinateSpace::Simulation);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Offset From Center"))).SetValue(FVector3f::ZeroVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("Offset Coordinate Space"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Radius"))).SetValue(10.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("Radius Coordinate Space"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Hemisphere X"))).SetValue(true);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Hemisphere Y"))).SetValue(true);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Hemisphere Z"))).SetValue(true);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Sphere Orientation Axis"))).SetValue(FVector3f::ZAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("RotationCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Additional Rotation"))).SetValue(FQuat4f::Identity);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("NonUniform Scale"))).SetValue(FVector3f::OneVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Segments"))).SetValue(12);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Cylinder Persistent
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawCylinderPersistentName;
		Signature.FunctionSpecifiers.Add(TEXT("Identifier"));
		Signature.bIsCompileTagGenerator = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("CenterCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Offset"))).SetValue(FVector3f::ZeroVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OffsetCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Radius"))).SetValue(5.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Height"))).SetValue(10.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsHalfHeight"))).SetValue(false);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Hemisphere X"))).SetValue(true);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Hemisphere Y"))).SetValue(true);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(AxisTypeDef, TEXT("Orientation Axis"))).SetValue(ENiagaraOrientationAxis::ZAxis);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OrientationAxisCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("NonUniform Scale"))).SetValue(FVector3f::OneVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Radius Segments"))).SetValue(12);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Height Segments"))).SetValue(1);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Cone Persistent
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawConePersistentName;
		Signature.FunctionSpecifiers.Add(TEXT("Identifier"));
		Signature.bIsCompileTagGenerator = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("CenterCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Offset"))).SetValue(FVector3f::ZeroVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OffsetCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Angle"))).SetValue(10.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Length"))).SetValue(10.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Orientation Axis"))).SetValue(FVector3f::ZAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OrientationAxisCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("NonUniform Scale"))).SetValue(FVector3f::OneVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Radius Segments"))).SetValue(12);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Height Segments"))).SetValue(1);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Torus
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawTorusPersistentName;
		Signature.FunctionSpecifiers.Add(TEXT("Identifier"));
		Signature.bIsCompileTagGenerator = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("CenterCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Offset"))).SetValue(FVector3f::ZeroVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OffsetCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MajorRadius"))).SetValue(100.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MinorRadius"))).SetValue(25.0f);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Orientation Axis"))).SetValue(FVector3f::ZAxisVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OrientationAxisCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("NonUniform Scale"))).SetValue(FVector3f::OneVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MajorRadiusSegments"))).SetValue(12);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MinorRadiusSegments"))).SetValue(6);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Coordinate System
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawCoordinateSystemPersistentName;
		Signature.FunctionSpecifiers.Add(TEXT("Identifier"));
		Signature.bIsCompileTagGenerator = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("CenterCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Offset"))).SetValue(FVector3f::ZeroVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OffsetCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("RotationCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Scale"))).SetValue(1.0f);
	}

	// Grid2D
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawGrid2DPersistentName;
		Signature.FunctionSpecifiers.Add(TEXT("Identifier"));
		Signature.bIsCompileTagGenerator = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("CenterCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Offset"))).SetValue(FVector3f::ZeroVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OffsetCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("RotationCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Extents"))).SetValue(FVector2f(10.0f));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX"))).SetValue(1);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY"))).SetValue(1);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	// Grid3D
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(DefaultSignature);
		Signature.Name = NDIDebugDrawLocal::DrawGrid3DPersistentName;
		Signature.FunctionSpecifiers.Add(TEXT("Identifier"));
		Signature.bIsCompileTagGenerator = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DebugDrawInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("CenterCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Offset"))).SetValue(FVector3f::ZeroVector);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("OffsetCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(CoordTypeDef, TEXT("RotationCoordinateSpace"))).SetValue(ENiagaraCoordinateSpace::Local);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Extents"))).SetValue(FVector3f(10.0f));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX"))).SetValue(1);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY"))).SetValue(1);
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsZ"))).SetValue(1);
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
	}

	for (int i = FirstFunction; i < OutFunctions.Num(); ++i)
	{
		OutFunctions[i].FunctionVersion = FNiagaraDebugDrawDIFunctionVersion::LatestVersion;
	}
}

END_FUNCTION_BUILD_OPTIMIZATION

bool UNiagaraDataInterfaceDebugDraw::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	bool bWasChanged = false;

	// Early out for version matching
	if (FunctionSignature.FunctionVersion == FNiagaraDebugDrawDIFunctionVersion::LatestVersion)
	{
		return bWasChanged;
	}

	if (FunctionSignature.FunctionVersion < FNiagaraDebugDrawDIFunctionVersion::AddedSphereExtraRotation && FunctionSignature.bIsCompileTagGenerator)
	{
		TArray<FNiagaraFunctionSignature> Sigs;
		GetFunctionsInternal(Sigs);
		for (const FNiagaraFunctionSignature& Sig : Sigs)
		{
			if (Sig.Name == FunctionSignature.Name)
			{
				FunctionSignature.Inputs = Sig.Inputs;
				bWasChanged = true;
			}
		}
	}

	FunctionSignature.FunctionVersion = FNiagaraDebugDrawDIFunctionVersion::LatestVersion;

	return bWasChanged;

}
#endif

void UNiagaraDataInterfaceDebugDraw::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddIncludedStruct<FNDIDebugDrawShaderParameters>();
}

void UNiagaraDataInterfaceDebugDraw::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	FNDIDebugDrawShaderParameters* ShaderParameters = Context.GetParameterIncludedStruct<FNDIDebugDrawShaderParameters>();

#if NIAGARA_COMPUTEDEBUG_ENABLED
	FNDIDebugDrawProxy& DIProxy = Context.GetProxy<FNDIDebugDrawProxy>();
	FNDIDebugDrawInstanceData_RenderThread& InstanceData = DIProxy.SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());

	FNiagaraSimulationDebugDrawData* DebugDraw = nullptr;
	if (InstanceData.GpuComputeDebug)
	{
		DebugDraw = InstanceData.GpuComputeDebug->GetSimulationDebugDrawData(GraphBuilder, Context.GetSystemInstanceID(), InstanceData.OverrideMaxLineInstances);
	}

	const bool bIsValid =
		NDIDebugDrawLocal::GNiagaraDebugDrawEnabled &&
		(DebugDraw != nullptr) &&
		(DebugDraw->GpuLineMaxInstances > 0) &&
		Context.IsResourceBound(&ShaderParameters->RWNDIDebugDrawArgs) &&
		Context.IsResourceBound(&ShaderParameters->RWNDIDebugDrawLineVertex);

	if ( bIsValid )
	{
		ShaderParameters->RWNDIDebugDrawArgs			= DebugDraw->GpuLineBufferArgs.GetOrCreateUAV(GraphBuilder);
		ShaderParameters->RWNDIDebugDrawLineVertex		= DebugDraw->GpuLineVertexBuffer.GetOrCreateUAV(GraphBuilder);
		ShaderParameters->NDIDebugDrawLineMaxInstances	= DebugDraw->GpuLineMaxInstances;
	}
	else
#endif
	{
		ShaderParameters->RWNDIDebugDrawArgs			= Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_UINT);
		ShaderParameters->RWNDIDebugDrawLineVertex		= Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_UINT);
		ShaderParameters->NDIDebugDrawLineMaxInstances	= 0;
	}
}

void UNiagaraDataInterfaceDebugDraw::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	if (BindingInfo.Name == NDIDebugDrawLocal::DrawBoxName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebug<NDIDebugDrawLocal::FDebugPrim_Box>);
	}
	else if (BindingInfo.Name == NDIDebugDrawLocal::DrawCircleName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebug<NDIDebugDrawLocal::FDebugPrim_Circle>);
	}
	else if (BindingInfo.Name == NDIDebugDrawLocal::DrawRectangleName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebug<NDIDebugDrawLocal::FDebugPrim_Rectangle>);
	}
	else if (BindingInfo.Name == NDIDebugDrawLocal::DrawCylinderName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebug<NDIDebugDrawLocal::FDebugPrim_Cylinder>);
	}
	else if (BindingInfo.Name == NDIDebugDrawLocal::DrawConeName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebug<NDIDebugDrawLocal::FDebugPrim_Cone>);
	}
	else if (BindingInfo.Name == NDIDebugDrawLocal::DrawTorusName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebug<NDIDebugDrawLocal::FDebugPrim_Torus>);
	}
	else if (BindingInfo.Name == NDIDebugDrawLocal::DrawCoordinateSystemName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebug<NDIDebugDrawLocal::FDebugPrim_CoordinateSystem>);
	}
	else if (BindingInfo.Name == NDIDebugDrawLocal::DrawGrid2DName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebug<NDIDebugDrawLocal::FDebugPrim_Grid2D>);
	}
	else if (BindingInfo.Name == NDIDebugDrawLocal::DrawGrid3DName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebug<NDIDebugDrawLocal::FDebugPrim_Grid3D>);
	}
	else if (BindingInfo.Name == NDIDebugDrawLocal::DrawLineName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebug<NDIDebugDrawLocal::FDebugPrim_Line>);
	}
	else if (BindingInfo.Name == NDIDebugDrawLocal::DrawSphereName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebug<NDIDebugDrawLocal::FDebugPrim_Sphere>);
	}
	else if (BindingInfo.FunctionSpecifiers.Num() != 0)
	{
		// The HLSL translator adds this function specifier in so that we have a unqiue key during compilation.
		const FVMFunctionSpecifier* Specifier = BindingInfo.FunctionSpecifiers.FindByPredicate([&](const FVMFunctionSpecifier& Info) { return Info.Key == NDIDebugDrawLocal::CompileTagKey; });

		if (Specifier && !Specifier->Value.IsNone())
		{
			FNDIDebugDrawInstanceData_GameThread* PerInstanceData = reinterpret_cast<FNDIDebugDrawInstanceData_GameThread*>(InstanceData);

			if (BindingInfo.Name == NDIDebugDrawLocal::DrawLinePersistentName)
			{
#if NIAGARA_COMPUTEDEBUG_ENABLED
				PerInstanceData->AddNamedPersistentShape(Specifier->Value, UNiagaraDataInterfaceDebugDraw::Line);
#endif
				OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebugPersistent<NDIDebugDrawLocal::FDebugPrim_Line>);
			}
			else if (BindingInfo.Name == NDIDebugDrawLocal::DrawRectanglePersistentName)
			{
#if NIAGARA_COMPUTEDEBUG_ENABLED
				PerInstanceData->AddNamedPersistentShape(Specifier->Value, UNiagaraDataInterfaceDebugDraw::Rectangle);
#endif
				OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebugPersistent<NDIDebugDrawLocal::FDebugPrim_Rectangle>);
			}
			else if (BindingInfo.Name == NDIDebugDrawLocal::DrawCirclePersistentName)
			{
#if NIAGARA_COMPUTEDEBUG_ENABLED
				PerInstanceData->AddNamedPersistentShape(Specifier->Value, UNiagaraDataInterfaceDebugDraw::Circle);
#endif
				OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebugPersistent<NDIDebugDrawLocal::FDebugPrim_Circle>);
			}
			else if (BindingInfo.Name == NDIDebugDrawLocal::DrawBoxPersistentName)
			{
#if NIAGARA_COMPUTEDEBUG_ENABLED
				PerInstanceData->AddNamedPersistentShape(Specifier->Value, UNiagaraDataInterfaceDebugDraw::Box);
#endif
				OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebugPersistent<NDIDebugDrawLocal::FDebugPrim_Box>);
			}
			else if (BindingInfo.Name == NDIDebugDrawLocal::DrawSpherePersistentName)
			{
#if NIAGARA_COMPUTEDEBUG_ENABLED
				PerInstanceData->AddNamedPersistentShape(Specifier->Value, UNiagaraDataInterfaceDebugDraw::Sphere);
#endif
				OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebugPersistent<NDIDebugDrawLocal::FDebugPrim_Sphere>);
			}
			else if (BindingInfo.Name == NDIDebugDrawLocal::DrawCylinderPersistentName)
			{
#if NIAGARA_COMPUTEDEBUG_ENABLED
				PerInstanceData->AddNamedPersistentShape(Specifier->Value, UNiagaraDataInterfaceDebugDraw::Cylinder);
#endif
				OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebugPersistent<NDIDebugDrawLocal::FDebugPrim_Cylinder>);
			}
			else if (BindingInfo.Name == NDIDebugDrawLocal::DrawConePersistentName)
			{
#if NIAGARA_COMPUTEDEBUG_ENABLED
				PerInstanceData->AddNamedPersistentShape(Specifier->Value, UNiagaraDataInterfaceDebugDraw::Cone);
#endif
				OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebugPersistent<NDIDebugDrawLocal::FDebugPrim_Cone>);
			}
			else if (BindingInfo.Name == NDIDebugDrawLocal::DrawTorusPersistentName)
			{
#if NIAGARA_COMPUTEDEBUG_ENABLED
				PerInstanceData->AddNamedPersistentShape(Specifier->Value, UNiagaraDataInterfaceDebugDraw::Torus);
#endif
				OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebugPersistent<NDIDebugDrawLocal::FDebugPrim_Torus>);
			}
			else if (BindingInfo.Name == NDIDebugDrawLocal::DrawCoordinateSystemPersistentName)
			{
#if NIAGARA_COMPUTEDEBUG_ENABLED
				PerInstanceData->AddNamedPersistentShape(Specifier->Value, UNiagaraDataInterfaceDebugDraw::CoordinateSystem);
#endif
				OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebugPersistent<NDIDebugDrawLocal::FDebugPrim_CoordinateSystem>);
			}
			else if (BindingInfo.Name == NDIDebugDrawLocal::DrawGrid2DPersistentName)
			{
#if NIAGARA_COMPUTEDEBUG_ENABLED
				PerInstanceData->AddNamedPersistentShape(Specifier->Value, UNiagaraDataInterfaceDebugDraw::Grid2D);
#endif
				OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebugPersistent<NDIDebugDrawLocal::FDebugPrim_Grid2D>);
			}
			else if (BindingInfo.Name == NDIDebugDrawLocal::DrawGrid3DPersistentName)
			{
#if NIAGARA_COMPUTEDEBUG_ENABLED
				PerInstanceData->AddNamedPersistentShape(Specifier->Value, UNiagaraDataInterfaceDebugDraw::Grid3D);
#endif
				OutFunc = FVMExternalFunction::CreateStatic(&NDIDebugDrawLocal::DrawDebugPersistent<NDIDebugDrawLocal::FDebugPrim_Grid3D>);
			}
		}
	}
	
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceDebugDraw::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
		return false;

	InVisitor->UpdateShaderFile(NDIDebugDrawLocal::CommonShaderFile);
	InVisitor->UpdateShaderFile(NDIDebugDrawLocal::TemplateShaderFile);

	InVisitor->UpdatePOD<int32>(TEXT("NiagaraDataInterfaceDebugDrawVersion"), FNiagaraDebugDrawDIFunctionVersion::LatestVersion);	
	InVisitor->UpdateShaderParameters<FNDIDebugDrawShaderParameters>();

	return true;
}

void UNiagaraDataInterfaceDebugDraw::GetCommonHLSL(FString& OutHLSL)
{
	OutHLSL.Appendf(TEXT("#include \"%s\"\n"), NDIDebugDrawLocal::CommonShaderFile);
}


void UNiagaraDataInterfaceDebugDraw::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIDebugDrawLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceDebugDraw::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	static const TSet<FName> PersistentFunctions =
	{
		NDIDebugDrawLocal::DrawLinePersistentName,
		NDIDebugDrawLocal::DrawRectanglePersistentName,
		NDIDebugDrawLocal::DrawCirclePersistentName,
		NDIDebugDrawLocal::DrawBoxPersistentName,
		NDIDebugDrawLocal::DrawSpherePersistentName,
		NDIDebugDrawLocal::DrawCylinderPersistentName,
		NDIDebugDrawLocal::DrawConePersistentName,
		NDIDebugDrawLocal::DrawTorusPersistentName,
		NDIDebugDrawLocal::DrawCoordinateSystemPersistentName,
		NDIDebugDrawLocal::DrawGrid2DPersistentName,
		NDIDebugDrawLocal::DrawGrid3DPersistentName,
	};
	if (PersistentFunctions.Contains(FunctionInfo.DefinitionName))
	{
		OutHLSL.Appendf(TEXT("Make%s(%s)\r\n"), *FunctionInfo.DefinitionName.ToString(), *FunctionInfo.InstanceName);
		return true;
	}

	static const TSet<FName> ValidGpuFunctions =
	{
		NDIDebugDrawLocal::DrawLineName,
		NDIDebugDrawLocal::DrawRectangleName,
		NDIDebugDrawLocal::DrawCircleName,
		NDIDebugDrawLocal::DrawBoxName,
		NDIDebugDrawLocal::DrawSphereName,
		NDIDebugDrawLocal::DrawCylinderName,
		NDIDebugDrawLocal::DrawConeName,
		NDIDebugDrawLocal::DrawTorusName,
		NDIDebugDrawLocal::DrawCoordinateSystemName,
		NDIDebugDrawLocal::DrawGrid2DName,
		NDIDebugDrawLocal::DrawGrid3DName,
	};
	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

bool UNiagaraDataInterfaceDebugDraw::GenerateCompilerTagPrefix(const FNiagaraFunctionSignature& InSignature, FString& OutPrefix) const 
{
	if (InSignature.bIsCompileTagGenerator && InSignature.FunctionSpecifiers.Num() == 1)
	{
		for (auto Specifier : InSignature.FunctionSpecifiers)
		{
			if (!Specifier.Value.IsNone())
			{
				OutPrefix = Specifier.Value.ToString();
				return true;
			}
		}
	}
	return false;
}

#endif

bool UNiagaraDataInterfaceDebugDraw::GPUContextInit(const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDataInterfaceData, const TArray<FVMExternalFunctionBindingInfo>& CalledFunctions, void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) const
{
	if (PerInstanceData == nullptr)
	{
		return true;
	}
#if NIAGARA_COMPUTEDEBUG_ENABLED && WITH_EDITORONLY_DATA

	FNDIDebugDrawInstanceData_GameThread* InstanceData = reinterpret_cast<FNDIDebugDrawInstanceData_GameThread*>(PerInstanceData);
	for (const FVMExternalFunctionBindingInfo& Sig : CalledFunctions)
	{
		if (Sig.OwnerName == ResolvedDataInterfaceData.CompileName && Sig.FunctionSpecifiers.Num() > 0)
		{
			// The HLSL translator adds this function specifier in so that we have a unqiue key during compilation.
			const FVMFunctionSpecifier* SpecifierPtr = Sig.FindSpecifier(NDIDebugDrawLocal::CompileTagKey);
			const FName* Specifier = SpecifierPtr != nullptr ? &SpecifierPtr->Value : nullptr;
			if (Specifier && !Specifier->IsNone())
			{
				if (Sig.Name == NDIDebugDrawLocal::DrawLinePersistentName)
					InstanceData->AddNamedPersistentShape(*Specifier, UNiagaraDataInterfaceDebugDraw::Line);
				else if (Sig.Name == NDIDebugDrawLocal::DrawRectanglePersistentName)
					InstanceData->AddNamedPersistentShape(*Specifier, UNiagaraDataInterfaceDebugDraw::Rectangle);
				else if (Sig.Name == NDIDebugDrawLocal::DrawCirclePersistentName)
					InstanceData->AddNamedPersistentShape(*Specifier, UNiagaraDataInterfaceDebugDraw::Circle);
				else if (Sig.Name == NDIDebugDrawLocal::DrawBoxPersistentName)
					InstanceData->AddNamedPersistentShape(*Specifier, UNiagaraDataInterfaceDebugDraw::Box);
				else if (Sig.Name == NDIDebugDrawLocal::DrawSpherePersistentName)
					InstanceData->AddNamedPersistentShape(*Specifier, UNiagaraDataInterfaceDebugDraw::Sphere);
				else if (Sig.Name == NDIDebugDrawLocal::DrawCylinderPersistentName)
					InstanceData->AddNamedPersistentShape(*Specifier, UNiagaraDataInterfaceDebugDraw::Cylinder);
				else if (Sig.Name == NDIDebugDrawLocal::DrawConePersistentName)
					InstanceData->AddNamedPersistentShape(*Specifier, UNiagaraDataInterfaceDebugDraw::Cone);
				else if (Sig.Name == NDIDebugDrawLocal::DrawTorusPersistentName)
					InstanceData->AddNamedPersistentShape(*Specifier, UNiagaraDataInterfaceDebugDraw::Torus);
				else if (Sig.Name == NDIDebugDrawLocal::DrawCoordinateSystemPersistentName)
					InstanceData->AddNamedPersistentShape(*Specifier, UNiagaraDataInterfaceDebugDraw::CoordinateSystem);
				else if (Sig.Name == NDIDebugDrawLocal::DrawGrid2DPersistentName)
					InstanceData->AddNamedPersistentShape(*Specifier, UNiagaraDataInterfaceDebugDraw::Grid2D);
				else if (Sig.Name == NDIDebugDrawLocal::DrawGrid3DPersistentName)
					InstanceData->AddNamedPersistentShape(*Specifier, UNiagaraDataInterfaceDebugDraw::Grid3D);
	
			}
		}
	}
#endif
	return true;
}


bool UNiagaraDataInterfaceDebugDraw::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
#if NIAGARA_COMPUTEDEBUG_ENABLED
	FNDIDebugDrawInstanceData_GameThread* InstanceData = reinterpret_cast<FNDIDebugDrawInstanceData_GameThread*>(PerInstanceData);
	InstanceData->LineBuffer.Reset();
	InstanceData->OverrideMaxLineInstances = FMath::Min<uint32>(OverrideMaxLineInstances, TNumericLimits<int32>::Max());
#endif
	return false;
}

bool UNiagaraDataInterfaceDebugDraw::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	if (!SystemInstance)
		return false;

#if NIAGARA_COMPUTEDEBUG_ENABLED
	FNDIDebugDrawInstanceData_GameThread* InstanceData = reinterpret_cast<FNDIDebugDrawInstanceData_GameThread*>(PerInstanceData);

	if (InstanceData)
	{
		if (NDIDebugDrawLocal::GNiagaraDebugDrawEnabled)
		{
			InstanceData->HandlePersistentShapes(SystemInstance, DeltaSeconds);
		}

		// Dispatch information to the RT proxy
		ENQUEUE_RENDER_COMMAND(NDIDebugDrawUpdate)(
			[RT_Proxy=GetProxyAs<FNDIDebugDrawProxy>(), RT_InstanceID=SystemInstance->GetId(), RT_TickCount=SystemInstance->GetTickCount(), RT_LineBuffer=MoveTemp(InstanceData->LineBuffer)](FRHICommandListImmediate& RHICmdList) mutable
			{
				if (RT_Proxy)
				{
					FNDIDebugDrawInstanceData_RenderThread* RT_InstanceData = &RT_Proxy->SystemInstancesToProxyData_RT.FindChecked(RT_InstanceID);

					if (RT_InstanceData && RT_InstanceData->GpuComputeDebug)
					{
						if (FNiagaraSimulationDebugDrawData* DebugDraw = RT_InstanceData->GpuComputeDebug->GetSimulationDebugDrawData(RT_InstanceID))
						{
							if (DebugDraw->LastUpdateTickCount != RT_TickCount)
							{
								DebugDraw->LastUpdateTickCount = RT_TickCount;
								DebugDraw->bRequiresUpdate = true;
								DebugDraw->StaticLines = MoveTemp(RT_LineBuffer);
							}
							else
							{
								DebugDraw->StaticLines += MoveTemp(RT_LineBuffer);
							}
						}
					}
				}
			}
		);
	}
#endif
	return false;
}

int32 UNiagaraDataInterfaceDebugDraw::PerInstanceDataSize() const
{
	return sizeof(FNDIDebugDrawInstanceData_GameThread);
}

bool UNiagaraDataInterfaceDebugDraw::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	auto* InstanceData = new(PerInstanceData) FNDIDebugDrawInstanceData_GameThread();

#if NIAGARA_COMPUTEDEBUG_ENABLED
	ENQUEUE_RENDER_COMMAND(NDIDebugDrawInit)(
		[RT_Proxy=GetProxyAs<FNDIDebugDrawProxy>(), RT_InstanceID=SystemInstance->GetId(), RT_ComputeDispatchInterface=SystemInstance->GetComputeDispatchInterface(), RT_OverrideMaxLineInstances = OverrideMaxLineInstances](FRHICommandListImmediate& RHICmdList)
		{
			check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(RT_InstanceID));
			FNDIDebugDrawInstanceData_RenderThread* RT_InstanceData = &RT_Proxy->SystemInstancesToProxyData_RT.Add(RT_InstanceID);
			RT_InstanceData->GpuComputeDebug = RT_ComputeDispatchInterface->GetGpuComputeDebugPrivate();
			RT_InstanceData->OverrideMaxLineInstances = RT_OverrideMaxLineInstances;
		}
	);
#endif
	return true;
}

void UNiagaraDataInterfaceDebugDraw::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
#if NIAGARA_COMPUTEDEBUG_ENABLED
	auto* InstanceData = reinterpret_cast<FNDIDebugDrawInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FNDIDebugDrawInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(NDIDebugDrawInit)(
		[RT_Proxy=GetProxyAs<FNDIDebugDrawProxy>(), RT_InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
		{
			FNDIDebugDrawInstanceData_RenderThread OriginalData;
			if ( ensure(RT_Proxy->SystemInstancesToProxyData_RT.RemoveAndCopyValue(RT_InstanceID, OriginalData)) )
			{
				if ( OriginalData.GpuComputeDebug )
				{
					OriginalData.GpuComputeDebug->RemoveSimulationDebugDrawData(RT_InstanceID);
				}
			}
		}
	);
#endif
}

bool UNiagaraDataInterfaceDebugDraw::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceDebugDraw* OtherTyped = CastChecked<const UNiagaraDataInterfaceDebugDraw>(Other);

	return OtherTyped->OverrideMaxLineInstances == OverrideMaxLineInstances;
}

bool UNiagaraDataInterfaceDebugDraw::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceDebugDraw* OtherTyped = CastChecked<UNiagaraDataInterfaceDebugDraw>(Destination);

	OtherTyped->OverrideMaxLineInstances = OverrideMaxLineInstances;	

	return true;
}
