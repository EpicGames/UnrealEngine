// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceDebugDraw.generated.h"

enum class ENiagaraSimTarget : uint8;
struct FNiagaraDataInterfaceGeneratedFunction;
struct FNiagaraFunctionSignature;

UCLASS(EditInlineNew, Category = "Debug", CollapseCategories, meta = (DisplayName = "DebugDraw"), MinimalAPI)
class UNiagaraDataInterfaceDebugDraw : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	// Number of debug lines is set as the max of OverrideMaxLineInstances and fx.Niagara.GpuComputeDebug.MaxLineInstances
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	uint32 OverrideMaxLineInstances = 0;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;

	enum ShapeId
	{
		Box = 0,
		Circle,
		CoordinateSystem,
		Grid2D,
		Grid3D,
		Line,
		Sphere,
		Cylinder,
		Cone,
		Rectangle,
		Torus,
		Max
	};

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	NIAGARA_API virtual bool GenerateCompilerTagPrefix(const FNiagaraFunctionSignature& InSignature, FString& OutPrefix) const override;	
	NIAGARA_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;

	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }

	NIAGARA_API int32 PerInstanceDataSize() const;
	NIAGARA_API bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance);
	NIAGARA_API void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance);

	virtual bool NeedsGPUContextInit() const override { return true; }
	NIAGARA_API virtual bool GPUContextInit(const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDataInterfaceData, const TArray<FVMExternalFunctionBindingInfo>& CalledFunctions, void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) const override;

	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	/** Copy one niagara DI to this */
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	//UNiagaraDataInterface Interface
};
