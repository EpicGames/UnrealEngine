// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceDynamicMesh.h"
#include "DataInterface/NiagaraDataInterfaceDynamicMeshSimCacheData.h"

#include "NiagaraCompileHashVisitor.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGpuReadbackManager.h"
#include "NiagaraMeshVertexFactory.h"
#include "NiagaraParameterStore.h"
#include "NiagaraSimStageData.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraStats.h"

#include "Materials/MaterialRenderProxy.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "UnifiedBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceDynamicMesh)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterfaceDynamicMesh"

//-OPT: Allocating triangles doesn't need to lock / unlock, use atomics / batches to improve performance on both CPU & GPU
//-OPT: We need to be able to pass an GPU generated count for the number of triangles to mesh renderers to improve draw performance
namespace NDIDynamicMeshLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(uint32,				NumSections)
		SHADER_PARAMETER(uint32,				NumTriangles)
		SHADER_PARAMETER(uint32,				NumVertices)
		SHADER_PARAMETER(uint32,				NumTexCoords)

		SHADER_PARAMETER(uint32,				PositionOffset)
		SHADER_PARAMETER(uint32,				TangentBasisOffset)
		SHADER_PARAMETER(uint32,				TexCoordOffset)
		SHADER_PARAMETER(uint32,				ColorOffset)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>,	SectionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>,	IndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>,	VertexBuffer)
	END_SHADER_PARAMETER_STRUCT()

	const TCHAR*	TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceDynamicMeshTemplate.ush");

	static const FName GetMeshPropertiesName("GetMeshProperties");
	static const FName GetSectionCountName("GetSectionCount");
	static const FName GetSectionDataName("GetSectionData");
	static const FName GetLocalBoundsName("GetLocalBounds");

	static const FName SetMeshPropertiesName("SetMeshProperties");
	static const FName SetSectionCountName("SetSectionCount");
	static const FName SetSectionDataName("SetSectionData");
	static const FName SetLocalBoundsName("SetLocalBounds");

	static const FName ClearAllSectionTrianglesName("ClearAllSectionTriangles");
	static const FName ClearSectionTrianglesName("ClearSectionTriangles");
	static const FName AllocateSectionTrianglesName("AllocateSectionTriangles");

	static const FName GetTriangleVerticesName("GetTriangleVertices");
	static const FName GetVertexPositionName("GetVertexPosition");
	static const FName GetVertexTangentBasisName("GetVertexTangentBasis");
	static const FName GetVertexTexCoordName("GetVertexTexCoord");
	static const FName GetVertexColorName("GetVertexColor");
	static const FName GetVertexDataName("GetVertexData");
	static const FName SetTriangleVerticesName("SetTriangleVertices");
	static const FName SetVertexPositionName("SetVertexPosition");
	static const FName SetVertexTangentBasisName("SetVertexTangentBasis");
	static const FName SetVertexTexCoordName("SetVertexTexCoord");
	static const FName SetVertexColorName("SetVertexColor");
	static const FName SetVertexDataName("SetVertexData");

	static const FName AppendTriangleName("AppendTriangle");

	static constexpr EPixelFormat PixelFormatPosition = EPixelFormat::PF_R32_FLOAT;
	static constexpr EPixelFormat PixelFormatTangentBasis = EPixelFormat::PF_R8G8B8A8_SNORM;
	static constexpr EPixelFormat PixelFormatTexCoord = EPixelFormat::PF_G32R32F;
	static constexpr EPixelFormat PixelFormatColor = EPixelFormat::PF_R8G8B8A8;

	struct FVertexData
	{
		FVector3f		Position = FVector3f::ZeroVector;
		FVector3f		TangentX = FVector3f::ZeroVector;
		FVector3f		TangentY = FVector3f::ZeroVector;
		FVector3f		TangentZ = FVector3f::ZeroVector;
		FVector2f		TexCoord = FVector2f::ZeroVector;
		FLinearColor	Color = FLinearColor::White;
	};

	struct FVMVertexInput
	{
		explicit FVMVertexInput(FVectorVMExternalFunctionContext& Context)
			: Position(Context)
			, TangentX(Context)
			, TangentY(Context)
			, TangentZ(Context)
			, TexCoord(Context)
			, Color(Context)
		{
		}

		[[nodiscard]] FVertexData GetAndAdvance()
		{
			FVertexData Data;
			Data.Position = Position.GetAndAdvance();
			Data.TangentX = TangentX.GetAndAdvance();
			Data.TangentY = TangentY.GetAndAdvance();
			Data.TangentZ = TangentZ.GetAndAdvance();
			Data.TexCoord = TexCoord.GetAndAdvance();
			Data.Color = Color.GetAndAdvance();
			return Data;
		}

		FNDIInputParam<FVector3f>		Position;
		FNDIInputParam<FVector3f>		TangentX;
		FNDIInputParam<FVector3f>		TangentY;
		FNDIInputParam<FVector3f>		TangentZ;
		FNDIInputParam<FVector2f>		TexCoord;
		FNDIInputParam<FLinearColor>	Color;
	};

	struct FNDIInstanceMaterial
	{
		TObjectPtr<UMaterialInterface> Material = nullptr;
		FNiagaraParameterDirectBinding<UObject*> MaterialBinding;
	};

	struct FNDIInstanceSection
	{
		int32	TriangleOffset = 0;		// Offset into the triangle buffer
		int32	MaxTriangles = 0;		// Number of triangles we can store
		int32	AllocatedTriangles = 0;	// Number of triangles we have allocated
		int32	MaterialIndex = 0;		// Material Index
	};

	struct FNDIInstanceData_GameThread
	{
		bool	bHasChanged = true;
		bool	bClearTrianglesPerFrame = false;
		bool	bGpuUsesDynamicAllocation = false;

		FNiagaraSystemInstanceID SystemInstanceID;

		uint32	NumTriangles = 0;
		uint32	NumVertices = 0;
		uint32	NumTexCoords = 0;
		FBox	LocalBounds = FBox(FVector(-100.0f), FVector(100.0f));

		uint32	VertexBufferSize = 0;
		uint32	PositionOffset = INDEX_NONE;
		uint32	TangentBasisOffset = INDEX_NONE;
		uint32	TexCoordOffset = INDEX_NONE;
		uint32	ColorOffset = INDEX_NONE;

		TArray<FNDIInstanceSection>			Sections;
		TArray<FNDIInstanceMaterial>		Materials;
		TArray<uint8>						IndexData;
		TArray<uint8>						VertexData;

		FCriticalSection					CpuDataGuard;

		void UpdateMesh()
		{
			bHasChanged = true;
			NumTriangles = 0;

			for (FNDIInstanceSection& Section : Sections)
			{
				Section.TriangleOffset = NumTriangles;
				Section.AllocatedTriangles = bClearTrianglesPerFrame ? 0 : Section.MaxTriangles;
				NumTriangles += Section.MaxTriangles;
			}

			IndexData.Empty();
			VertexData.Empty();
		}

		void UpdateMesh(uint32 InNumVertices, uint32 InNumTexCoords, bool InbHasTangentBasis, bool InbHasColors)
		{
			UpdateMesh();

			NumVertices		= InNumVertices;
			NumTexCoords	= InNumTexCoords;

			VertexBufferSize = 0;

			PositionOffset = VertexBufferSize;
			VertexBufferSize += sizeof(FVector3f) * NumVertices;

			if ( InbHasTangentBasis )
			{
				VertexBufferSize = Align(VertexBufferSize, RHIGetMinimumAlignmentForBufferBackedSRV(PixelFormatTangentBasis));
				TangentBasisOffset = VertexBufferSize;
				VertexBufferSize +=  sizeof(FPackedNormal) * 2 * NumVertices;
			}
			else
			{
				TangentBasisOffset = INDEX_NONE;
			}
			

			if ( NumTexCoords > 0 )
			{
				VertexBufferSize = Align(VertexBufferSize, RHIGetMinimumAlignmentForBufferBackedSRV(PixelFormatTexCoord));
				TexCoordOffset = VertexBufferSize;
				VertexBufferSize +=  sizeof(FVector2f) * NumTexCoords * NumVertices;
			}
			else
			{
				TexCoordOffset = INDEX_NONE;
			}

			if ( InbHasColors )
			{
				VertexBufferSize = Align(VertexBufferSize, RHIGetMinimumAlignmentForBufferBackedSRV(PixelFormatColor));
				ColorOffset = VertexBufferSize;
				VertexBufferSize +=  sizeof(FColor) * NumVertices;
			}
			else
			{
				ColorOffset = INDEX_NONE;
			}
		}

		void ModifyCpuData()
		{
			bHasChanged = true;

			FScopeLock DataGuard(&CpuDataGuard);
			IndexData.SetNumZeroed(NumTriangles * 3 * sizeof(uint32));
			VertexData.SetNum(VertexBufferSize);
		}

		FVector3f GetVertexPosition(uint32 Vertex) const
		{
			check(Vertex >= 0 && Vertex < NumVertices && PositionOffset != INDEX_NONE);
			return *reinterpret_cast<const FVector3f*>(VertexData.GetData() + (Vertex * sizeof(FVector3f)) + PositionOffset);
		}

		void GetVertexTangentBasis(uint32 Vertex, FVector3f& OutTangentX, FVector3f& OutTangentY, FVector3f& OutTangentZ)
		{
			check(Vertex >= 0 && Vertex < NumVertices && TangentBasisOffset != INDEX_NONE);
			const uint32* TangentData = reinterpret_cast<const uint32*>(VertexData.GetData() + (Vertex * sizeof(FPackedNormal) * 2) + TangentBasisOffset);

			OutTangentX = FVector3f(float((TangentData[0] >> 0) & 0xff), float((TangentData[0] >> 8) & 0xff), float((TangentData[0] >> 16) & 0xff));
			OutTangentZ = FVector3f(float((TangentData[1] >> 0) & 0xff), float((TangentData[1] >> 8) & 0xff), float((TangentData[1] >> 16) & 0xff));
			OutTangentX = (OutTangentX / 127.5f) - 1.0f;
			OutTangentZ = (OutTangentZ / 127.5f) - 1.0f;

			const float TangentSign = (((TangentData[1] >> 24) & 0xff) / 127.5f) - 1.0f;

			OutTangentY = FVector3f::CrossProduct(OutTangentZ, OutTangentX) * TangentSign;
			OutTangentX = FVector3f::CrossProduct(OutTangentY, OutTangentZ) * TangentSign;
		}
		
		FVector2f GetVertexTexCoord(uint32 Vertex, uint32 TexCoord)
		{
			check(Vertex >= 0 && Vertex < NumVertices && TexCoord >= 0 && TexCoord < NumTexCoords && TexCoordOffset != INDEX_NONE);
			return reinterpret_cast<const FVector2f*>(VertexData.GetData() + (Vertex * sizeof(FVector2f)) + TexCoordOffset)[TexCoord];
		}

		FLinearColor GetVertexColor(uint32 Vertex)
		{
			check(Vertex >= 0 && Vertex < NumVertices && ColorOffset != INDEX_NONE);
			return FLinearColor(*reinterpret_cast<const FColor*>(VertexData.GetData() + (Vertex * sizeof(FColor)) + ColorOffset));
		}

		void SetVertexPosition(uint32 Vertex, const FVector3f& Position)
		{
			check(Vertex >= 0 && Vertex < NumVertices&& PositionOffset != INDEX_NONE);
			*reinterpret_cast<FVector3f*>(VertexData.GetData() + (Vertex * sizeof(FVector3f)) + PositionOffset) = Position;
		}

		void SetVertexTangentBasis(uint32 Vertex, const FVector3f& TangentX, const FVector3f& TangentY, const FVector3f& TangentZ)
		{
			check(Vertex >= 0 && Vertex < NumVertices && TangentBasisOffset != INDEX_NONE);
			uint32* TangentData = reinterpret_cast<uint32*>(VertexData.GetData() + (Vertex * sizeof(FPackedNormal) * 2) + TangentBasisOffset);

			const uint32 TangentSign = FVector3f::DotProduct(FVector3f::CrossProduct(TangentX, TangentZ), TangentY) < 0 ? 0x80000000 : 0x7f000000;
			TangentData[0] =
				((int(TangentX.X * 127.499f) & 0xff) << 0) |
				((int(TangentX.Y * 127.499f) & 0xff) << 8) |
				((int(TangentX.Z * 127.499f) & 0xff) << 16) |
				TangentSign;

			TangentData[1] =
				((int(TangentZ.X * 127.499f) & 0xff) << 0) |
				((int(TangentZ.Y * 127.499f) & 0xff) << 8) |
				((int(TangentZ.Z * 127.499f) & 0xff) << 16) |
				TangentSign;
		}

		void SetVertexTexCoord(uint32 Vertex, uint32 TexCoord, const FVector2f& Coord)
		{
			check(Vertex >= 0 && Vertex < NumVertices && TexCoord >= 0 && TexCoord < NumTexCoords && TexCoordOffset != INDEX_NONE);
			reinterpret_cast<FVector2f*>(VertexData.GetData() + (Vertex * sizeof(FVector2f)) + TexCoordOffset)[TexCoord] = Coord;
		}

		void SetVertexColor(uint32 Vertex, const FLinearColor& Color)
		{
			check(Vertex >= 0 && Vertex < NumVertices && ColorOffset != INDEX_NONE);
			*reinterpret_cast<FColor*>(VertexData.GetData() + (Vertex * sizeof(FColor)) + ColorOffset) = Color.ToFColor(false);
		}
	};

	struct FGameToRenderData
	{
		explicit FGameToRenderData(const FNDIInstanceData_GameThread& InstanceData)
		{
			bGpuUsesDynamicAllocation	= InstanceData.bGpuUsesDynamicAllocation;
			NumTriangles				= InstanceData.NumTriangles;
			NumVertices					= InstanceData.NumVertices;
			NumTexCoords				= InstanceData.NumTexCoords;
			LocalBounds					= InstanceData.LocalBounds;

			VertexBufferSize			= InstanceData.VertexBufferSize;
			PositionOffset				= InstanceData.PositionOffset;
			TangentBasisOffset			= InstanceData.TangentBasisOffset;
			TexCoordOffset				= InstanceData.TexCoordOffset;
			ColorOffset					= InstanceData.ColorOffset;

			IndexData.AddUninitialized(InstanceData.IndexData.Num());
			FMemory::Memcpy(IndexData.GetData(), InstanceData.IndexData.GetData(), IndexData.Num());

			VertexData.AddUninitialized(InstanceData.VertexData.Num());
			FMemory::Memcpy(VertexData.GetData(), InstanceData.VertexData.GetData(), VertexData.Num());

			MeshSections = InstanceData.Sections;

			// Clamp material indices to avoid user error, we always have at least 1 material even if none are specified
			const int32 NumMaterials = FMath::Max(InstanceData.Materials.Num() - 1, 0);
			for (FNDIInstanceSection& Section : MeshSections)
			{
				Section.MaterialIndex = FMath::Clamp(Section.MaterialIndex, 0, NumMaterials);
			}
		}

		bool			bGpuUsesDynamicAllocation = false;
		uint32			NumTriangles = 0;
		uint32			NumVertices = 0;
		uint32			NumTexCoords = 0;
		FBox			LocalBounds = FBox(FVector(-100.0f), FVector(100.0f));

		uint32			VertexBufferSize = 0;
		uint32			PositionOffset = INDEX_NONE;
		uint32			TangentBasisOffset = INDEX_NONE;
		uint32			TexCoordOffset = INDEX_NONE;
		uint32			ColorOffset = INDEX_NONE;

		TResourceArray<uint8>	IndexData;
		TResourceArray<uint8>	VertexData;

		TArray<FNDIInstanceSection> MeshSections;
	};

	struct FNDIInstanceData_RenderThread
	{
		uint32			NumSections = 0;
		uint32			NumTriangles = 0;
		uint32			NumVertices = 0;
		uint32			NumTexCoords = 0;
		FBox			LocalBounds = FBox(FVector(-100.0f), FVector(100.0f));

		uint32			VertexBufferSize = 0;
		uint32			PositionOffset = INDEX_NONE;
		uint32			TangentBasisOffset = INDEX_NONE;
		uint32			TexCoordOffset = INDEX_NONE;
		uint32			ColorOffset = INDEX_NONE;

		TRefCountPtr<FRDGPooledBuffer>	SectionDefaultPooledBuffer;
		TRefCountPtr<FRDGPooledBuffer>	SectionPooledBuffer;
		TRefCountPtr<FRDGPooledBuffer>	IndexPooledBuffer;
		TRefCountPtr<FRDGPooledBuffer>	VertexPooledBuffer;

		FIndexBuffer				IndexBuffer;
		FShaderResourceViewRHIRef	IndexBufferSRV;
		FVertexBuffer				VertexBuffer;
		FShaderResourceViewRHIRef	VertexBufferPositionSRV;
		FShaderResourceViewRHIRef	VertexBufferTangentBasisSRV;
		FShaderResourceViewRHIRef	VertexBufferTexCoordSRV;
		FShaderResourceViewRHIRef	VertexBufferColorSRV;

		FRDGBufferRef				RDGTransientSectionBuffer = nullptr;
		FRDGBufferUAVRef			RDGTransientSectionBufferUAV = nullptr;
		FRDGBufferRef				RDGTransientIndexBuffer = nullptr;
		FRDGBufferUAVRef			RDGTransientIndexBufferUAV = nullptr;
		FRDGBufferRef				RDGTransientVertexBuffer = nullptr;
		FRDGBufferUAVRef			RDGTransientVertexBufferUAV = nullptr;

		bool						bGpuUsesDynamicAllocation = true;
		bool						bNeedsClearIndexBuffer = true;
		bool						bNeedsGpuSectionUpdate = false;
		TArray<FStaticMeshSection>	MeshSections;

		~FNDIInstanceData_RenderThread()
		{
			ReleaseData();
		}

		void UpdateData(FRHICommandListImmediate& RHICmdList, FGameToRenderData& GameToRenderData)
		{
			ReleaseData();

			bGpuUsesDynamicAllocation	= GameToRenderData.bGpuUsesDynamicAllocation;
			NumSections					= GameToRenderData.MeshSections.Num();
			NumTriangles				= GameToRenderData.NumTriangles;
			NumVertices					= GameToRenderData.NumVertices;
			NumTexCoords				= GameToRenderData.NumTexCoords;
			LocalBounds					= GameToRenderData.LocalBounds;

			VertexBufferSize			= GameToRenderData.VertexBufferSize;
			PositionOffset				= GameToRenderData.PositionOffset;
			TangentBasisOffset			= GameToRenderData.TangentBasisOffset;
			TexCoordOffset				= GameToRenderData.TexCoordOffset;
			ColorOffset					= GameToRenderData.ColorOffset;

			{
				FRDGBuilder GraphBuilder(RHICmdList);

				FRDGBufferDesc IndexBufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), FMath::Max(NumTriangles * 3u, 1u));
				IndexBufferDesc.Usage = (IndexBufferDesc.Usage & ~EBufferUsageFlags::VertexBuffer) | EBufferUsageFlags::IndexBuffer;

				//-OPT: We don't need to create a SectionDefaultPooledBuffer if we aren't doing dynamic allocations
				FRDGBufferRef SectionDefaultBuffer = ResizeBufferIfNeeded(GraphBuilder, SectionDefaultPooledBuffer, EPixelFormat::PF_R32_UINT, FMath::Max(NumSections * 2u, 1u), TEXT("NiagaraDynamicMeshSectionBuffer"));
				ResizeBufferIfNeeded(GraphBuilder, SectionPooledBuffer, EPixelFormat::PF_R32_UINT, FMath::Max(NumSections * 2u, 1u), TEXT("NiagaraDynamicMeshSectionBuffer"));
				FRDGBufferRef RDGIndexBuffer = ResizeBufferIfNeeded(GraphBuilder, IndexPooledBuffer, IndexBufferDesc, TEXT("NiagaraDynamicMeshIndexBuffer"));
				FRDGBufferRef RDGVertexBuffer = ResizeBufferIfNeeded(GraphBuilder, VertexPooledBuffer, EPixelFormat::PF_R32_UINT, FMath::Max(GameToRenderData.VertexBufferSize >> 2u, 1u), TEXT("NiagaraDynamicMeshVertexBuffer"));

				MeshSections.SetNum(GameToRenderData.MeshSections.Num());
				if (NumSections > 0)
				{
					bNeedsGpuSectionUpdate = true;

					const uint32 GpuMeshSectionsBytes = GameToRenderData.MeshSections.Num() * 2 * sizeof(uint32);
					uint32* GpuMeshSections = reinterpret_cast<uint32*>(GraphBuilder.Alloc(GpuMeshSectionsBytes));
					for (int32 i = 0; i < MeshSections.Num(); ++i)
					{
						const FNDIInstanceSection& InSection = GameToRenderData.MeshSections[i];
						FStaticMeshSection& OutSection = MeshSections[i];

						OutSection.MaterialIndex = InSection.MaterialIndex;
						OutSection.FirstIndex = InSection.TriangleOffset * 3;
						OutSection.NumTriangles = bGpuUsesDynamicAllocation ? InSection.MaxTriangles : InSection.AllocatedTriangles;
						OutSection.MinVertexIndex = 0;
						OutSection.MaxVertexIndex = NumVertices;
						OutSection.bEnableCollision = false;
						OutSection.bCastShadow = false;
						OutSection.bVisibleInRayTracing = false;
						OutSection.bForceOpaque = false;

						GpuMeshSections[i * 2 + 0] = InSection.MaxTriangles;
						GpuMeshSections[i * 2 + 1] = InSection.AllocatedTriangles;
					}
					GraphBuilder.QueueBufferUpload(SectionDefaultBuffer, GpuMeshSections, GpuMeshSectionsBytes, ERDGInitialDataFlags::NoCopy);
				}
				//-OPT: We can remove the copying here.
				if (GameToRenderData.IndexData.Num() > 0)
				{
					GraphBuilder.QueueBufferUpload(RDGIndexBuffer, GameToRenderData.IndexData.GetData(), GameToRenderData.IndexData.Num());
				}

				if (GameToRenderData.VertexData.Num() > 0)
				{
					GraphBuilder.QueueBufferUpload(RDGVertexBuffer, GameToRenderData.VertexData.GetData(), GameToRenderData.VertexData.Num());
				}

				FRDGExternalAccessQueue ExternalAccessQueue;
				if (RDGIndexBuffer != nullptr)
				{
					ExternalAccessQueue.Add(RDGIndexBuffer, ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);
				}
				if (RDGVertexBuffer != nullptr)
				{
					ExternalAccessQueue.Add(RDGVertexBuffer, ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);
				}
				ExternalAccessQueue.Submit(GraphBuilder);

				GraphBuilder.Execute();
			}

			if (NumTriangles > 0)
			{
				bNeedsClearIndexBuffer = GameToRenderData.IndexData.Num() == 0;

				IndexBuffer.InitResource(RHICmdList);
				IndexBuffer.IndexBufferRHI = IndexPooledBuffer->GetRHI();
				IndexBufferSRV = RHICmdList.CreateShaderResourceView(
					IndexBuffer.IndexBufferRHI, 
					FRHIViewDesc::CreateBufferSRV()
						.SetType(FRHIViewDesc::EBufferType::Typed)
						.SetFormat(EPixelFormat::PF_R32_UINT));
			}

			if (NumVertices > 0)
			{
				VertexBuffer.InitResource(RHICmdList);
				VertexBuffer.VertexBufferRHI = VertexPooledBuffer->GetRHI();
				VertexBufferPositionSRV = RHICmdList.CreateShaderResourceView(FShaderResourceViewInitializer(VertexBuffer.VertexBufferRHI, PixelFormatPosition, PositionOffset, NumVertices * 3));
				if (TangentBasisOffset != INDEX_NONE)
				{
					VertexBufferTangentBasisSRV = RHICmdList.CreateShaderResourceView(FShaderResourceViewInitializer(VertexBuffer.VertexBufferRHI, PixelFormatTangentBasis, TangentBasisOffset, NumVertices * 2));
				}
				if (TexCoordOffset != INDEX_NONE)
				{
					VertexBufferTexCoordSRV = RHICmdList.CreateShaderResourceView(FShaderResourceViewInitializer(VertexBuffer.VertexBufferRHI, PixelFormatTexCoord, TexCoordOffset, NumVertices * NumTexCoords));
				}
				if (ColorOffset != INDEX_NONE)
				{
					VertexBufferColorSRV = RHICmdList.CreateShaderResourceView(FShaderResourceViewInitializer(VertexBuffer.VertexBufferRHI, PixelFormatColor, ColorOffset, NumVertices));
				}
			}
		#if STATS
			UpdateMemoryStats();
		#endif
		}

		void ReleaseData()
		{
		#if STATS
			ResetMemoryStats();
		#endif

			IndexBuffer.ReleaseResource();
			IndexBufferSRV.SafeRelease();
			VertexBuffer.ReleaseResource();
			VertexBufferPositionSRV.SafeRelease();
			VertexBufferTangentBasisSRV.SafeRelease();
			VertexBufferTexCoordSRV.SafeRelease();
			VertexBufferColorSRV.SafeRelease();

			NumTriangles = 0;
			NumVertices = 0;
			NumTexCoords = 0;

			VertexBufferSize = 0;
			PositionOffset = INDEX_NONE;
			TangentBasisOffset = INDEX_NONE;
			TexCoordOffset = INDEX_NONE;
			ColorOffset = INDEX_NONE;
		}

#if STATS
		void UpdateMemoryStats()
		{
			DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, MemorySize);
			MemorySize = 0;
						
			if (IndexBuffer.GetRHI())
			{
				MemorySize += IndexBuffer.GetRHI()->GetSize();
			}
			
			if (VertexBuffer.GetRHI())
			{
				MemorySize += VertexBuffer.GetRHI()->GetSize();
			}

			INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, MemorySize);
		}
		
		void ResetMemoryStats()
		{
			DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, MemorySize);
			MemorySize = 0;
		}
		uint64 MemorySize = 0;
#endif
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { checkNoEntry(); }
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

		virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override
		{
			FNDIInstanceData_RenderThread& InstanceData = InstanceData_RT.FindChecked(Context.GetSystemInstanceID());
			const FNiagaraSimStageData& SimStageData = Context.GetSimStageData();
			if (SimStageData.bFirstStage)
			{
				FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

				// Do we need to register our graph resources
				// Note: We are assuming they will be used we could possibly defer until later on
				if (InstanceData.RDGTransientSectionBuffer == nullptr)
				{
					InstanceData.RDGTransientSectionBuffer		= GraphBuilder.RegisterExternalBuffer(InstanceData.SectionPooledBuffer);
					InstanceData.RDGTransientSectionBufferUAV	= GraphBuilder.CreateUAV(InstanceData.RDGTransientSectionBuffer, EPixelFormat::PF_R32_UINT);
					InstanceData.RDGTransientIndexBuffer		= GraphBuilder.RegisterExternalBuffer(InstanceData.IndexPooledBuffer);
					InstanceData.RDGTransientIndexBufferUAV		= GraphBuilder.CreateUAV(InstanceData.RDGTransientIndexBuffer, EPixelFormat::PF_R32_UINT);
					InstanceData.RDGTransientVertexBuffer		= GraphBuilder.RegisterExternalBuffer(InstanceData.VertexPooledBuffer);
					InstanceData.RDGTransientVertexBufferUAV	= GraphBuilder.CreateUAV(InstanceData.RDGTransientVertexBuffer, EPixelFormat::PF_R32_UINT);

					Context.GetRDGExternalAccessQueue().Add(InstanceData.RDGTransientIndexBuffer, ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);
					Context.GetRDGExternalAccessQueue().Add(InstanceData.RDGTransientVertexBuffer, ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);
				}

				if (InstanceData.bNeedsClearIndexBuffer)
				{
					InstanceData.bNeedsClearIndexBuffer = false;
					AddClearUAVPass(Context.GetGraphBuilder(), InstanceData.RDGTransientIndexBufferUAV, 0);
				}

				if (InstanceData.bNeedsGpuSectionUpdate && InstanceData.NumSections > 0)
				{
					AddCopyBufferPass(
						GraphBuilder,
						InstanceData.RDGTransientSectionBuffer,
						GraphBuilder.RegisterExternalBuffer(InstanceData.SectionDefaultPooledBuffer)
					);
				}
			}
			else if (SimStageData.bLastStage && InstanceData.bGpuUsesDynamicAllocation)
			{
				//InstanceData.bNeedsClearIndexBuffer = true;
				InstanceData.bNeedsGpuSectionUpdate = true;
			}
		}

		virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override
		{
			if (Context.IsFinalPostSimulate())
			{
				FNDIInstanceData_RenderThread& InstanceData = InstanceData_RT.FindChecked(Context.GetSystemInstanceID());
				InstanceData.RDGTransientSectionBuffer = nullptr;
				InstanceData.RDGTransientSectionBufferUAV = nullptr;
				InstanceData.RDGTransientIndexBuffer = nullptr;
				InstanceData.RDGTransientIndexBufferUAV = nullptr;
				InstanceData.RDGTransientVertexBuffer = nullptr;
				InstanceData.RDGTransientVertexBufferUAV = nullptr;
			}
		}

		TMap<FNiagaraSystemInstanceID, FNDIInstanceData_GameThread*> InstanceData_GT;
		TMap<FNiagaraSystemInstanceID, FNDIInstanceData_RenderThread> InstanceData_RT;
	};

	struct FNDIRenderableMesh : public INiagaraRenderableMesh
	{
		explicit FNDIRenderableMesh(const FNDIProxy* InOwnerProxy, FNiagaraSystemInstanceID InSystemInstanceID)
			: OwnerProxy(InOwnerProxy)
			, SystemInstanceID(InSystemInstanceID)
		{
			check(OwnerProxy != nullptr);
		}

		// INiagaraRenderableMesh Impl Begin
		virtual FBox GetLocalBounds() const override
		{
			if ( const FNDIInstanceData_RenderThread* InstanceData = OwnerProxy->InstanceData_RT.Find(SystemInstanceID) )
			{
				return InstanceData->LocalBounds;
			}

			return FBox(FVector(-100.0f), FVector(100.0f));
		}

		virtual void GetLODModelData(FLODModelData& OutLODModelData, int32 LODLevel) const
		{
			const FNDIInstanceData_RenderThread* InstanceData = OwnerProxy->InstanceData_RT.Find(SystemInstanceID);
			if (InstanceData && InstanceData->NumTriangles > 0 && InstanceData->NumVertices > 0)
			{
				OutLODModelData.LODIndex = 0;
				OutLODModelData.NumVertices = InstanceData->NumVertices;
				OutLODModelData.IndexBuffer = &InstanceData->IndexBuffer;
				OutLODModelData.NumIndices = InstanceData->NumTriangles * 3;
				OutLODModelData.Sections = MakeArrayView(InstanceData->MeshSections);
			}
			else
			{
				OutLODModelData.LODIndex = INDEX_NONE;
			}
		}

		virtual void SetupVertexFactory(FRHICommandListBase& RHICmdList, class FNiagaraMeshVertexFactory& VertexFactory, const FLODModelData& LODModelData) const override
		{
			const FNDIInstanceData_RenderThread* InstanceData = &OwnerProxy->InstanceData_RT.FindChecked(SystemInstanceID);

			FStaticMeshDataType StaticMeshData;

			// Setup Position Data
			if (InstanceData->PositionOffset != INDEX_NONE)
			{
				StaticMeshData.PositionComponent = FVertexStreamComponent(&InstanceData->VertexBuffer, InstanceData->PositionOffset, sizeof(FVector3f), VET_Float3);
				StaticMeshData.PositionComponentSRV = InstanceData->VertexBufferPositionSRV;
			}
			else
			{
				StaticMeshData.PositionComponent = FVertexStreamComponent(&GNullVertexBuffer, 0, 0, VET_Float3);
				StaticMeshData.PositionComponentSRV = GNullVertexBuffer.VertexBufferSRV;
			}
			
			// Setup Tangent Data
			if (InstanceData->TangentBasisOffset != INDEX_NONE)
			{
				StaticMeshData.TangentBasisComponents[0] = FVertexStreamComponent(&InstanceData->VertexBuffer, /*InstanceData->TangentBasisOffset + 0*/0, sizeof(FPackedNormal) * 2, VET_PackedNormal, EVertexStreamUsage::ManualFetch);
				StaticMeshData.TangentBasisComponents[1] = FVertexStreamComponent(&InstanceData->VertexBuffer, /*InstanceData->TangentBasisOffset + 4*/0, sizeof(FPackedNormal) * 2, VET_PackedNormal, EVertexStreamUsage::ManualFetch);
				StaticMeshData.TangentsSRV = InstanceData->VertexBufferTangentBasisSRV;
			}
			else
			{
				StaticMeshData.TangentBasisComponents[0] = FVertexStreamComponent(&GNullVertexBuffer, 0, 0, VET_PackedNormal, EVertexStreamUsage::ManualFetch);
				StaticMeshData.TangentBasisComponents[1] = FVertexStreamComponent(&GNullVertexBuffer, 0, 0, VET_PackedNormal, EVertexStreamUsage::ManualFetch);
				StaticMeshData.TangentsSRV = GNullVertexBuffer.VertexBufferSRV;
			}
			
			// Setup TexCoord Data
			if (InstanceData->TexCoordOffset != INDEX_NONE)
			{
				StaticMeshData.NumTexCoords = IntCastChecked<uint8>(InstanceData->NumTexCoords);
				for (uint32 i=0; i < InstanceData->NumTexCoords; ++i)
				{
					StaticMeshData.TextureCoordinates.Emplace(&InstanceData->VertexBuffer, /*InstanceData->TexCoordOffset + (i * sizeof(FVector2f))*/0, int32(sizeof(FVector2f) * InstanceData->NumTexCoords), VET_Float2, EVertexStreamUsage::ManualFetch);
				}
				StaticMeshData.TextureCoordinatesSRV = InstanceData->VertexBufferTexCoordSRV;
			}
			else
			{
				StaticMeshData.NumTexCoords = 0;
				StaticMeshData.TextureCoordinates.Emplace(&GNullVertexBuffer, 0, 0, VET_Float2, EVertexStreamUsage::ManualFetch);
				StaticMeshData.TextureCoordinatesSRV = GNullVertexBuffer.VertexBufferSRV;
			}
			
			// Setup Color Data
			if (InstanceData->ColorOffset != INDEX_NONE)
			{
				StaticMeshData.ColorIndexMask = ~0u;
				StaticMeshData.ColorComponent = FVertexStreamComponent(&InstanceData->VertexBuffer, /*InstanceData->ColorOffset*/0, sizeof(FColor), VET_Color, EVertexStreamUsage::ManualFetch);
				StaticMeshData.ColorComponentsSRV = InstanceData->VertexBufferColorSRV;
			}
			else
			{
				StaticMeshData.ColorIndexMask = 0;
				StaticMeshData.ColorComponent = FVertexStreamComponent(&GNullColorVertexBuffer, 0, 0, VET_Color, EVertexStreamUsage::ManualFetch);
				StaticMeshData.ColorComponentsSRV = GNullColorVertexBuffer.VertexBufferSRV;
			}

			VertexFactory.SetData(RHICmdList, StaticMeshData);
		}

		virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials) const override
		{
			if (const FNDIInstanceData_GameThread* InstanceData = OwnerProxy->InstanceData_GT.FindRef(SystemInstanceID))
			{
				OutMaterials.Reserve(OutMaterials.Num() + InstanceData->Materials.Num());
				for (const FNDIInstanceMaterial& InstanceMaterial : InstanceData->Materials)
				{
					OutMaterials.Add(InstanceMaterial.Material);
				}
			}
			if (OutMaterials.Num() == 0)
			{
				OutMaterials.Add(nullptr);
			}
		}
		// INiagaraRenderableMesh Impl End

		const FNDIProxy*				OwnerProxy = nullptr;
		const FNiagaraSystemInstanceID	SystemInstanceID;
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraDynamicMeshMaterial::FNiagaraDynamicMeshMaterial()
	: Material(nullptr)
	, MaterialUserParamBinding(FNiagaraTypeDefinition(UMaterialInterface::StaticClass()))
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceDynamicMesh::UNiagaraDataInterfaceDynamicMesh(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDIDynamicMeshLocal;

	Proxy.Reset(new FNDIProxy());
}

void UNiagaraDataInterfaceDynamicMesh::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceDynamicMesh::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIDynamicMeshLocal;

	FNiagaraFunctionSignature ImmutableSignature;
	ImmutableSignature.bMemberFunction = true;
	ImmutableSignature.bRequiresContext = false;
	ImmutableSignature.ModuleUsageBitmask = ENiagaraScriptUsageMask::System | ENiagaraScriptUsageMask::Emitter;
	ImmutableSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Dynamic Mesh Interface")));

	FNiagaraFunctionSignature MutableSignature = ImmutableSignature;
	MutableSignature.bRequiresExecPin = true;

	//////////////////////////////////////////////////////////////////////////
	// Mesh Functions
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ImmutableSignature);
		Signature.Name = GetMeshPropertiesName;
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumTriangles"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumVertices"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumTexCoords"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("HasColors"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("HasTangentBasis"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ImmutableSignature);
		Signature.Name = GetSectionCountName;
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SectionCount"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ImmutableSignature);
		Signature.Name = GetSectionDataName;
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SectionIndex"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumTriangles"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MaterialIndex"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ImmutableSignature);
		Signature.Name = GetLocalBoundsName;
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BoundsMin"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BoundsMax"));
	}

	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = SetMeshPropertiesName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumVertices"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumTexCoords"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("HasColors"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("HasTangentBasis"));
		Signature.SetDescription(LOCTEXT("SetMeshPropertiesDesc", "Modifies the mesh proprties, any changes here will clear the existing mesh data."));
		Signature.bSupportsGPU = false;
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = SetSectionCountName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SectionCount"));
		Signature.SetDescription(LOCTEXT("SetSectionCountDesc", "Sets the number of sections for the mesh, any changes here will clear existing mesh data."));
		Signature.bSupportsGPU = false;
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = SetSectionDataName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SectionIndex"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumTriangles"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MaterialIndex"));
		Signature.SetDescription(LOCTEXT("SetSectionDataDesc", "Sets the mesh section data, any changes here will clear existing mesh data."));
		Signature.bSupportsGPU = false;
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = SetLocalBoundsName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BoundsMin"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BoundsMax"));
		Signature.bSupportsGPU = false;
	}

	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = ClearAllSectionTrianglesName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.SetDescription(LOCTEXT("ClearAllSectionTrianglesDesc", "Clears all section triangle counts back to 0."));
		Signature.bSupportsGPU = false;
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = ClearSectionTrianglesName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SectionIndex"));
		Signature.SetDescription(LOCTEXT("ClearSectionTrianglesDesc", "Clears the sections triangle count back to 0."));
		Signature.bSupportsGPU = false;
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = AllocateSectionTrianglesName;
		Signature.ModuleUsageBitmask = ENiagaraScriptUsageMask::System | ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::Particle;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SectionIndex"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumTriangles"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TriangleIndex"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumAllocated"));
		Signature.SetDescription(LOCTEXT("AllocateSectionTrianglesDesc", "Allocates the provided number of triangles from the section.  Returns the first triangle index allocated along with the number actually allocated."));
	}

	//////////////////////////////////////////////////////////////////////////
	// Index / Vertex Functions
	MutableSignature.ModuleUsageBitmask = ENiagaraScriptUsageMask::System | ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::Particle;
	ImmutableSignature.ModuleUsageBitmask = ENiagaraScriptUsageMask::System | ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::Particle;
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ImmutableSignature);
		Signature.Name = GetTriangleVerticesName;
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TriangleIndex"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index0"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index1"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index2"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ImmutableSignature);
		Signature.Name = GetVertexPositionName;
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("VertexIndex"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ImmutableSignature);
		Signature.Name = GetVertexTangentBasisName;
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("VertexIndex"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentX"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentY"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentZ"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ImmutableSignature);
		Signature.Name = GetVertexTexCoordName;
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("VertexIndex"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexCoordIndex"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("TexCoord"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ImmutableSignature);
		Signature.Name = GetVertexColorName;
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("VertexIndex"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ImmutableSignature);
		Signature.Name = GetVertexDataName;
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("VertexIndex"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentX"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentY"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentZ"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("TexCoord"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color"));
	}

	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = SetTriangleVerticesName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TriangleIndex"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index0"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index1"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index2"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = SetVertexPositionName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("VertexIndex"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = SetVertexTangentBasisName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("VertexIndex"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentX"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentY"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentZ"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = SetVertexTexCoordName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("VertexIndex"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexCoordIndex"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("TexCoord"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = SetVertexColorName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("VertexIndex"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = SetVertexDataName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("VertexIndex"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentX"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentY"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentZ"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("TexCoord"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color"));
	}

	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = AppendTriangleName;
		Signature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SectionIndex"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position0"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentX0"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentY0"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentZ0"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("TexCoord0"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color0"));

		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position1"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentX1"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentY1"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentZ1"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("TexCoord1"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color1"));

		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position2"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentX2"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentY2"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TangentZ2"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("TexCoord2"));
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color2"));

		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TriangleIndex"));

		Signature.SetDescription(LOCTEXT("AppendTriangleDesc", "Appends a triangle to the section.  This assumes that triangles are stored sequentially in the section, no vertex sharing, etc."));
	}
}
#endif

void UNiagaraDataInterfaceDynamicMesh::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIDynamicMeshLocal;

	static const TPair<FName, FVMExternalFunction> StaticBindings[] =
	{
		{GetMeshPropertiesName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMGetMeshProperties)},
		{GetSectionCountName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMGetSectionCount)},
		{GetSectionDataName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMGetSectionData)},
		{GetLocalBoundsName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMGetLocalBounds)},
		{SetMeshPropertiesName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMSetMeshProperties)},
		{SetSectionCountName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMSetSectionCount)},
		{SetSectionDataName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMSetSectionData)},
		{SetLocalBoundsName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMSetLocalBounds)},
		{ClearAllSectionTrianglesName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMClearAllSectionTriangles)},
		{ClearSectionTrianglesName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMClearSectionTriangles)},
		{AllocateSectionTrianglesName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMAllocateSectionTriangles)},

		{GetTriangleVerticesName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMGetTriangleVertices)},
		{GetVertexPositionName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMGetVertexPosition)},
		{GetVertexTangentBasisName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMGetVertexTangentBasis)},
		{GetVertexTexCoordName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMGetVertexTexCoord)},
		{GetVertexColorName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMGetVertexColor)},
		{GetVertexDataName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMGetVertexData)},
		{SetTriangleVerticesName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMSetTriangleVertices)},
		{SetVertexPositionName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMSetVertexPosition)},
		{SetVertexTangentBasisName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMSetVertexTangentBasis)},
		{SetVertexTexCoordName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMSetVertexTexCoord)},
		{SetVertexColorName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMSetVertexColor)},
		{SetVertexDataName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMSetVertexData)},

		{AppendTriangleName, FVMExternalFunction::CreateStatic(UNiagaraDataInterfaceDynamicMesh::VMAppendTriangle)},
	};

	for (const auto& StaticBinding : StaticBindings)
	{
		if (StaticBinding.Key == BindingInfo.Name)
		{
			OutFunc = StaticBinding.Value;
			return;
		}
	}
}

int32 UNiagaraDataInterfaceDynamicMesh::PerInstanceDataSize() const
{
	using namespace NDIDynamicMeshLocal;
	return sizeof(FNDIInstanceData_GameThread);
}

bool UNiagaraDataInterfaceDynamicMesh::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIDynamicMeshLocal;

	FNDIInstanceData_GameThread* InstanceData = new(PerInstanceData) FNDIInstanceData_GameThread();
	InstanceData->SystemInstanceID = SystemInstance->GetId();
	InstanceData->bClearTrianglesPerFrame = bClearTrianglesPerFrame;
	InstanceData->bGpuUsesDynamicAllocation = IsUsedWithGPUScript();		//-OPT: We can improve this by looking for used functions
	InstanceData->LocalBounds = DefaultBounds;

	InstanceData->Sections.SetNum(Sections.Num());
	for (int32 i = 0; i < Sections.Num(); ++i)
	{
		const FNiagaraDynamicMeshSection& SrcSection = Sections[i];
		FNDIInstanceSection& DstSection = InstanceData->Sections[i];
		DstSection.MaxTriangles = SrcSection.NumTriangles;
		DstSection.MaterialIndex = SrcSection.MaterialIndex;
	}
	InstanceData->UpdateMesh(NumVertices, NumTexCoords, bHasTangentBasis, bHasColors);

	InstanceData->Materials.AddDefaulted(Materials.Num());
	for ( int32 i=0; i < Materials.Num(); ++i )
	{
		InstanceData->Materials[i].Material = Materials[i].Material;
		InstanceData->Materials[i].MaterialBinding.Init(SystemInstance->GetInstanceParameters(), Materials[i].MaterialUserParamBinding.Parameter);
	}

	FNDIProxy* TypedProxy = GetProxyAs<FNDIProxy>();
	check(!TypedProxy->InstanceData_GT.Contains(InstanceData->SystemInstanceID));
	TypedProxy->InstanceData_GT.Emplace(InstanceData->SystemInstanceID, InstanceData);

	return true;
}

void UNiagaraDataInterfaceDynamicMesh::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIDynamicMeshLocal;

	FNDIInstanceData_GameThread* InstanceData = reinterpret_cast<FNDIInstanceData_GameThread*>(PerInstanceData);
	GetProxyAs<FNDIProxy>()->InstanceData_GT.Remove(SystemInstance->GetId());

	ENQUEUE_RENDER_COMMAND(FNDIDynamicMesh_RemoveProxy)
	(
		[RT_Proxy=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->InstanceData_RT.Remove(InstanceID);
		}
	);

	InstanceData->~FNDIInstanceData_GameThread();
}

bool UNiagaraDataInterfaceDynamicMesh::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIDynamicMeshLocal;

	FNDIInstanceData_GameThread* InstanceData = reinterpret_cast<FNDIInstanceData_GameThread*>(PerInstanceData);
	if (InstanceData->bClearTrianglesPerFrame)
	{
		InstanceData->bHasChanged = true;
		for (FNDIInstanceSection& Section : InstanceData->Sections)
		{
			Section.AllocatedTriangles = 0;
		}
	}

	return false;
}

bool UNiagaraDataInterfaceDynamicMesh::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIDynamicMeshLocal;

	FNDIInstanceData_GameThread* InstanceData = reinterpret_cast<FNDIInstanceData_GameThread*>(PerInstanceData);

	// Update material list
	check(InstanceData->Materials.Num() == Materials.Num());
	for (int32 i = 0; i < Materials.Num(); ++i)
	{
		InstanceData->Materials[i].Material = InstanceData->Materials[i].MaterialBinding.GetValue<UMaterialInterface>();
		if (InstanceData->Materials[i].Material == nullptr)
		{
			InstanceData->Materials[i].Material = Materials[i].Material ? Materials[i].Material : UMaterial::GetDefaultMaterial(MD_Surface);
		}
	}

	// Update RenderThread mirror
	if (InstanceData->bHasChanged)
	{
		InstanceData->bHasChanged = false;
		
		ENQUEUE_RENDER_COMMAND(FNDIDynamicMesh_UpdateData)
		(
			[RT_Proxy=GetProxyAs<FNDIProxy>(), RT_SystemInstanceID=InstanceData->SystemInstanceID, RT_GameToRenderData=FGameToRenderData(*InstanceData)](FRHICommandListImmediate& RHICmdList) mutable
			{
				FNDIInstanceData_RenderThread& InstanceData_RT = RT_Proxy->InstanceData_RT.FindOrAdd(RT_SystemInstanceID);
				InstanceData_RT.UpdateData(RHICmdList, RT_GameToRenderData);
			}
		);
	}
	return false;
}

bool UNiagaraDataInterfaceDynamicMesh::Equals(const UNiagaraDataInterface* Other) const
{
	const UNiagaraDataInterfaceDynamicMesh* OtherTyped = CastChecked<const UNiagaraDataInterfaceDynamicMesh>(Other);
	return
		Super::Equals(Other) &&
		OtherTyped->Sections == Sections &&
		OtherTyped->Materials == Materials &&
		OtherTyped->NumVertices == NumVertices &&
		OtherTyped->NumTexCoords == NumTexCoords &&
		OtherTyped->bHasColors == bHasColors &&
		OtherTyped->bHasTangentBasis == bHasTangentBasis &&
		OtherTyped->bClearTrianglesPerFrame == bClearTrianglesPerFrame &&
		OtherTyped->DefaultBounds == DefaultBounds;
}

bool UNiagaraDataInterfaceDynamicMesh::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceDynamicMesh* OtherTyped = CastChecked<UNiagaraDataInterfaceDynamicMesh>(Destination);
	OtherTyped->Sections = Sections;
	OtherTyped->Materials = Materials;
	OtherTyped->NumVertices = NumVertices;
	OtherTyped->NumTexCoords = NumTexCoords;
	OtherTyped->bHasColors = bHasColors;
	OtherTyped->bHasTangentBasis = bHasTangentBasis;
	OtherTyped->bClearTrianglesPerFrame = bClearTrianglesPerFrame;
	OtherTyped->DefaultBounds = DefaultBounds;
	return true;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceDynamicMesh::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace NDIDynamicMeshLocal;

	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceDynamicMesh::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	using namespace NDIDynamicMeshLocal;

	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceDynamicMesh::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIDynamicMeshLocal;

	static const TSet<FName> ValidGpuFunctions =
	{
		//GetMeshPropertiesName,
		//GetSectionCountName,
		//GetSectionDataName,
		//GetLocalBoundsName,
		//SetMeshPropertiesName,
		//SetSectionCountName,
		//SetSectionDataName,
		//SetLocalBoundsName,
		AllocateSectionTrianglesName,
		GetTriangleVerticesName,
		GetVertexPositionName,
		GetVertexTangentBasisName,
		GetVertexTexCoordName,
		GetVertexColorName,
		GetVertexDataName,
		SetTriangleVerticesName,
		SetVertexPositionName,
		SetVertexTangentBasisName,
		SetVertexTexCoordName,
		SetVertexColorName,
		SetVertexDataName,
		AppendTriangleName,
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}
#endif

void UNiagaraDataInterfaceDynamicMesh::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NDIDynamicMeshLocal;

	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceDynamicMesh::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIDynamicMeshLocal;

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	FNDIProxy& DataInterfaceProxy = Context.GetProxy<FNDIProxy>();
	FNDIInstanceData_RenderThread& InstanceData = DataInterfaceProxy.InstanceData_RT.FindChecked(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->NumSections			= InstanceData.NumSections;
	ShaderParameters->NumTriangles			= InstanceData.NumTriangles;
	ShaderParameters->NumVertices			= InstanceData.NumVertices;
	ShaderParameters->NumTexCoords			= InstanceData.NumTexCoords;
	ShaderParameters->PositionOffset		= InstanceData.PositionOffset >> 2;
	ShaderParameters->TangentBasisOffset	= InstanceData.TangentBasisOffset >> 2;
	ShaderParameters->TexCoordOffset		= InstanceData.TexCoordOffset >> 2;
	ShaderParameters->ColorOffset			= InstanceData.ColorOffset >> 2;
	ShaderParameters->SectionBuffer			= InstanceData.RDGTransientSectionBufferUAV;
	ShaderParameters->IndexBuffer			= InstanceData.RDGTransientIndexBufferUAV;
	ShaderParameters->VertexBuffer			= InstanceData.RDGTransientVertexBufferUAV;
}

FNiagaraRenderableMeshPtr UNiagaraDataInterfaceDynamicMesh::GetRenderableMesh(FNiagaraSystemInstanceID SystemInstanceID)
{
	using namespace NDIDynamicMeshLocal;

	return MakeShared<FNDIRenderableMesh>(GetProxyAs<FNDIProxy>(), SystemInstanceID);
}

void UNiagaraDataInterfaceDynamicMesh::GetUsedMaterials(FNiagaraSystemInstanceID SystemInstanceID, TArray<UMaterialInterface*>& OutMaterials) const
{
	using namespace NDIDynamicMeshLocal;

	const FNDIProxy* DataInterfaceProxy = GetProxyAs<FNDIProxy>();
	if (const FNDIInstanceData_GameThread* InstanceData = DataInterfaceProxy->InstanceData_GT.FindRef(SystemInstanceID))
	{
		OutMaterials.Reserve(OutMaterials.Num() + InstanceData->Materials.Num());
		for (const FNDIInstanceMaterial& InstanceMaterial : InstanceData->Materials)
		{
			OutMaterials.Add(InstanceMaterial.Material);
		}
	}
}

UObject* UNiagaraDataInterfaceDynamicMesh::SimCacheBeginWrite(UObject* SimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	return OptionalPerInstanceData ? NewObject<UNiagaraDataInterfaceDynamicMeshSimCacheData>(SimCache) : nullptr;
}

bool UNiagaraDataInterfaceDynamicMesh::SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	using namespace NDIDynamicMeshLocal;

	check(OptionalPerInstanceData && StorageObject);
	UNiagaraDataInterfaceDynamicMeshSimCacheData* CacheData = CastChecked<UNiagaraDataInterfaceDynamicMeshSimCacheData>(StorageObject);
	CacheData->ResizeFrames(FrameIndex);

	// Build MeshDesc (shared between RT / GT)
	const FNDIInstanceData_GameThread* InstanceData_GT = reinterpret_cast<const FNDIInstanceData_GameThread*>(OptionalPerInstanceData);

	FNDIDynamicMeshSimCacheFrame MeshDesc;
	MeshDesc.NumTriangles		= InstanceData_GT->NumTriangles;
	MeshDesc.NumVertices		= InstanceData_GT->NumVertices;
	MeshDesc.NumTexCoords		= InstanceData_GT->NumTexCoords;
	MeshDesc.LocalBounds		= InstanceData_GT->LocalBounds;
	MeshDesc.VertexBufferSize	= InstanceData_GT->VertexBufferSize;
	MeshDesc.PositionOffset		= InstanceData_GT->PositionOffset;
	MeshDesc.TangentBasisOffset	= InstanceData_GT->TangentBasisOffset;
	MeshDesc.TexCoordOffset		= InstanceData_GT->TexCoordOffset;
	MeshDesc.ColorOffset		= InstanceData_GT->ColorOffset;
	MeshDesc.Sections.Reserve(InstanceData_GT->Sections.Num());
	for (const FNDIInstanceSection& Section : InstanceData_GT->Sections)
	{
		FNDIDynamicMeshSimCacheSection& CacheSection = MeshDesc.Sections.AddDefaulted_GetRef();
		CacheSection.TriangleOffset		= Section.TriangleOffset;
		CacheSection.MaxTriangles		= Section.MaxTriangles;
		CacheSection.AllocatedTriangles	= Section.AllocatedTriangles;
		CacheSection.Material			= InstanceData_GT->Materials.IsValidIndex(Section.MaterialIndex) ? InstanceData_GT->Materials[Section.MaterialIndex].Material : nullptr;
	}

	// Add to CPU meshes
	CacheData->CpuFrameIndices[FrameIndex] = CacheData->FindOrCreateMesh(MeshDesc, InstanceData_GT->IndexData, InstanceData_GT->VertexData);

	// Gather from the GPU
	if (IsUsedWithGPUScript())
	{
		FNiagaraGpuComputeDispatchInterface* ComputeInterface = FNiagaraGpuComputeDispatchInterface::Get(SystemInstance->GetWorld());
		ENQUEUE_RENDER_COMMAND(NDIDynamicMesh_SimCacheWrite)(
			[Proxy_RT=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId(), CacheData, MeshDesc, FrameIndex, ComputeInterface](FRHICommandListImmediate& RHICmdList) mutable
			{
				const FNDIInstanceData_RenderThread* InstanceData_RT = Proxy_RT->InstanceData_RT.Find(InstanceID);
				if (InstanceData_RT == nullptr)
				{
					return;
				}

				// Right now we are assuming that the GT data must match if it doesn't don't attempt to record the frame
				if ( (MeshDesc.NumTriangles != InstanceData_RT->NumTriangles) ||
					 (MeshDesc.NumVertices != InstanceData_RT->NumVertices) ||
					 (MeshDesc.NumTexCoords != InstanceData_RT->NumTexCoords) ||
					 //(MeshDesc.LocalBounds != InstanceData_RT->LocalBounds) ||
					 (MeshDesc.VertexBufferSize != InstanceData_RT->VertexBufferSize) ||
					 (MeshDesc.PositionOffset != InstanceData_RT->PositionOffset) ||
					 (MeshDesc.TangentBasisOffset != InstanceData_RT->TangentBasisOffset) ||
					 (MeshDesc.TexCoordOffset != InstanceData_RT->TexCoordOffset) ||
					 (MeshDesc.ColorOffset != InstanceData_RT->ColorOffset) ||
					 (MeshDesc.Sections.Num() != InstanceData_RT->NumSections) )
				{
					return;
				}

				if (InstanceData_RT->SectionPooledBuffer == nullptr || InstanceData_RT->IndexPooledBuffer == nullptr || InstanceData_RT->VertexPooledBuffer == nullptr)
				{
					return;
				}

				FRDGBuilder GraphBuilder(RHICmdList);

				FRDGBufferRef BuffersToReadback[3];
				BuffersToReadback[0] = GraphBuilder.RegisterExternalBuffer(InstanceData_RT->SectionPooledBuffer);
				BuffersToReadback[1] = GraphBuilder.RegisterExternalBuffer(InstanceData_RT->IndexPooledBuffer);
				BuffersToReadback[2] = GraphBuilder.RegisterExternalBuffer(InstanceData_RT->VertexPooledBuffer);

				TArray<uint32> GpuSectionData;
				TArray<uint8> GpuIndexData;
				TArray<uint8> GpuVertexData;

				FNiagaraGpuReadbackManager* ReadbackManager = ComputeInterface->GetGpuReadbackManager();
				ReadbackManager->EnqueueReadbacks(
					GraphBuilder,
					BuffersToReadback,
					[&](TConstArrayView<TPair<void*, uint32>> ReadbackData)
					{
						GpuSectionData.AddUninitialized(ReadbackData[0].Value / sizeof(uint32));
						FMemory::Memcpy(GpuSectionData.GetData(), ReadbackData[0].Key, GpuSectionData.NumBytes());
			
						GpuIndexData.AddUninitialized(ReadbackData[1].Value);
						FMemory::Memcpy(GpuIndexData.GetData(), ReadbackData[1].Key, ReadbackData[1].Value);

						GpuVertexData.AddUninitialized(ReadbackData[2].Value);
						FMemory::Memcpy(GpuVertexData.GetData(), ReadbackData[2].Key, ReadbackData[2].Value);
					}
				);

				GraphBuilder.Execute();
				ReadbackManager->WaitCompletion(RHICmdList);

				for (int iSection = 0; iSection < MeshDesc.Sections.Num(); ++iSection)
				{
					FNDIDynamicMeshSimCacheSection& Section = MeshDesc.Sections[iSection];
					Section.AllocatedTriangles = GpuSectionData[iSection * 2 + 1];
				}

				CacheData->GpuFrameIndices[FrameIndex] = CacheData->FindOrCreateMesh(MeshDesc, GpuIndexData, GpuVertexData);
			}
		);

		FlushRenderingCommands();
	}

	return true;
}

bool UNiagaraDataInterfaceDynamicMesh::SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext)
{
	using namespace NDIDynamicMeshLocal;

	const UNiagaraDataInterfaceDynamicMeshSimCacheData* CacheData = ReadContext.GetOptionalStorageObject<UNiagaraDataInterfaceDynamicMeshSimCacheData>();
	const int32 FrameIndex = ReadContext.GetFrameIndexA();

	// Copy data from cache if we have GPU data we take that as the authoritative version
	ENiagaraSimTarget SimTarget = ENiagaraSimTarget::GPUComputeSim;
	const FNDIDynamicMeshSimCacheFrame* CacheFrame = CacheData->GetFrameData(FrameIndex, SimTarget);
	if (CacheFrame == nullptr)
	{
		SimTarget = ENiagaraSimTarget::CPUSim;
		CacheFrame = CacheData->GetFrameData(FrameIndex, SimTarget);
	}

	FNDIInstanceData_GameThread* InstanceData_GT = ReadContext.GetPerInstanceData<FNDIInstanceData_GameThread>();
	if (CacheFrame)
	{
		InstanceData_GT->bHasChanged		= false;
		InstanceData_GT->NumTriangles		= CacheFrame->NumTriangles;
		InstanceData_GT->NumVertices		= CacheFrame->NumVertices;
		InstanceData_GT->NumTexCoords		= CacheFrame->NumTexCoords;
		InstanceData_GT->VertexBufferSize	= CacheFrame->VertexBufferSize;
		InstanceData_GT->PositionOffset		= CacheFrame->PositionOffset;
		InstanceData_GT->TangentBasisOffset	= CacheFrame->TangentBasisOffset;
		InstanceData_GT->TexCoordOffset		= CacheFrame->TexCoordOffset;
		InstanceData_GT->ColorOffset		= CacheFrame->ColorOffset;
		InstanceData_GT->IndexData			= CacheFrame->IndexData;
		InstanceData_GT->VertexData			= CacheFrame->VertexData;
		InstanceData_GT->Sections.SetNum(CacheFrame->Sections.Num());
		InstanceData_GT->Materials.Empty();

		for (int iSection = 0; iSection < CacheFrame->Sections.Num(); ++iSection)
		{
			const FNDIDynamicMeshSimCacheSection& CacheMeshSection = CacheFrame->Sections[iSection];
			FNDIInstanceSection& MeshSection = InstanceData_GT->Sections[iSection];
			UMaterialInterface* SectionMaterial = CacheMeshSection.Material ? CacheMeshSection.Material.Get() : UMaterial::GetDefaultMaterial(MD_Surface);

			MeshSection.TriangleOffset		= CacheMeshSection.TriangleOffset;
			MeshSection.MaxTriangles		= CacheMeshSection.MaxTriangles;
			MeshSection.AllocatedTriangles	= CacheMeshSection.AllocatedTriangles;
			MeshSection.MaterialIndex		= InstanceData_GT->Materials.IndexOfByPredicate([SectionMaterial](const FNDIInstanceMaterial& ExistingMaterial) { return ExistingMaterial.Material == SectionMaterial; });
			if (MeshSection.MaterialIndex == INDEX_NONE)
			{
				MeshSection.MaterialIndex = InstanceData_GT->Materials.AddDefaulted();
				InstanceData_GT->Materials[MeshSection.MaterialIndex].Material = SectionMaterial;
			}
		}

		TUniquePtr<FGameToRenderData> GameToRenderData;
		GameToRenderData.Reset(new FGameToRenderData(*InstanceData_GT));
		if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			// Clear out the index / vertex data if it came from a GPU simulation
			// Note: We could do something more optimal here and avoid the copy
			InstanceData_GT->IndexData.Empty();
			InstanceData_GT->VertexData.Empty();
		}

		ENQUEUE_RENDER_COMMAND(NDIDynamicMesh_SimCacheRead)(
			[Proxy_RT=GetProxyAs<FNDIProxy>(), InstanceID=ReadContext.GetSystemInstance()->GetId(), GameToRenderData_RT=MoveTemp(GameToRenderData)](FRHICommandListImmediate& RHICmdList)
			{
				FNDIInstanceData_RenderThread& InstanceData_RT = Proxy_RT->InstanceData_RT.FindOrAdd(InstanceID);
				InstanceData_RT.UpdateData(RHICmdList, *GameToRenderData_RT.Get());
			}
		);
	}
	else
	{
		InstanceData_GT->bHasChanged		= false;
		InstanceData_GT->NumTriangles		= 0;
		InstanceData_GT->NumVertices		= 0;
		InstanceData_GT->NumTexCoords		= 0;
		InstanceData_GT->VertexBufferSize	= 0;
		InstanceData_GT->PositionOffset		= INDEX_NONE;
		InstanceData_GT->TangentBasisOffset	= INDEX_NONE;
		InstanceData_GT->TexCoordOffset		= INDEX_NONE;
		InstanceData_GT->ColorOffset		= INDEX_NONE;
		InstanceData_GT->Sections.Empty();
		InstanceData_GT->Materials.Empty();
		InstanceData_GT->IndexData.Empty();
		InstanceData_GT->VertexData.Empty();
	}

	return true;
}

bool UNiagaraDataInterfaceDynamicMesh::SimCacheCompareFrame(const UObject* LhsStorageObject, const UObject* RhsStorageObject, int FrameIndex, TOptional<float> Tolerance, FString& OutErrors) const
{
	const UNiagaraDataInterfaceDynamicMeshSimCacheData* LhsCacheData = CastChecked<UNiagaraDataInterfaceDynamicMeshSimCacheData>(LhsStorageObject);
	const UNiagaraDataInterfaceDynamicMeshSimCacheData* RhsCacheData = CastChecked<UNiagaraDataInterfaceDynamicMeshSimCacheData>(RhsStorageObject);

	const bool LhsValid = LhsCacheData->CpuFrameIndices.IsValidIndex(FrameIndex) && LhsCacheData->GpuFrameIndices.IsValidIndex(FrameIndex);
	const bool RhsValid = RhsCacheData->CpuFrameIndices.IsValidIndex(FrameIndex) && RhsCacheData->GpuFrameIndices.IsValidIndex(FrameIndex);
	if (LhsValid != RhsValid)
	{
		OutErrors = FString::Format(TEXT("Frame Validity different LhsValid = {0} RhsValid = {1}"), { LhsValid, RhsValid });
		return false;
	}

	if (!LhsValid)
	{
		return true;
	}

	for (ENiagaraSimTarget SimTarget : {ENiagaraSimTarget::CPUSim, ENiagaraSimTarget::GPUComputeSim})
	{
		const FNDIDynamicMeshSimCacheFrame* LhsMeshData = LhsCacheData->GetFrameData(FrameIndex, SimTarget);
		const FNDIDynamicMeshSimCacheFrame* RhsMeshData = RhsCacheData->GetFrameData(FrameIndex, SimTarget);

		const bool LhsMeshDataValid = LhsMeshData != nullptr;
		const bool RhsMeshDataValid = RhsMeshData != nullptr;
		if (LhsMeshDataValid != RhsMeshDataValid)
		{
			OutErrors = FString::Format(TEXT("Mesh Data Validity different LhsMeshData = {0} RhsMeshData = {1}"), { LhsMeshDataValid, RhsMeshDataValid });
			return false;
		}

		if (LhsMeshDataValid)
		{
			if (!LhsMeshData->Equals(*RhsMeshData))
			{
				OutErrors = TEXT("Mesh Data Different");
				return false;
			}
		}
	}

	return true;
}

void UNiagaraDataInterfaceDynamicMesh::VMGetMeshProperties(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<int32> OutNumTriangles(Context);
	FNDIOutputParam<int32> OutNumVertices(Context);
	FNDIOutputParam<int32> OutNumTexCoords(Context);
	FNDIOutputParam<bool>  OutHasColors(Context);
	FNDIOutputParam<bool>  OutHasTangentBasis(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutNumTriangles.SetAndAdvance(InstanceData->NumTriangles);
		OutNumVertices.SetAndAdvance(InstanceData->NumVertices);
		OutNumTexCoords.SetAndAdvance(InstanceData->NumTexCoords);
		OutHasColors.SetAndAdvance(InstanceData->ColorOffset != INDEX_NONE);
		OutHasTangentBasis.SetAndAdvance(InstanceData->TangentBasisOffset != INDEX_NONE);
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMGetSectionCount(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<int32> OutSectionCount(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutSectionCount.SetAndAdvance(InstanceData->Sections.Num());
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMGetSectionData(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<int32>	InSectionIndex(Context);
	FNDIOutputParam<bool>	OutValid(Context);
	FNDIOutputParam<int32>	OutNumTriangles(Context);
	FNDIOutputParam<int32>	OutMaterialIndex(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 SectionIndex = InSectionIndex.GetAndAdvance();
		if (InstanceData->Sections.IsValidIndex(SectionIndex))
		{
			const FNDIInstanceSection& InstanceSection = InstanceData->Sections[SectionIndex];
			OutValid.SetAndAdvance(true);
			OutNumTriangles.SetAndAdvance(InstanceSection.MaxTriangles);
			OutMaterialIndex.SetAndAdvance(InstanceSection.MaterialIndex);
		}
		else
		{
			OutValid.SetAndAdvance(false);
			OutNumTriangles.SetAndAdvance(0);
			OutMaterialIndex.SetAndAdvance(0);
		}
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMGetLocalBounds(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<FVector3f>	OutBoundsMin(Context);
	FNDIOutputParam<FVector3f>	OutBoundsMax(Context);

	const FVector3f BoundsMin = FVector3f(InstanceData->LocalBounds.Min);
	const FVector3f BoundsMax = FVector3f(InstanceData->LocalBounds.Max);
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutBoundsMin.SetAndAdvance(BoundsMin);
		OutBoundsMax.SetAndAdvance(BoundsMax);
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMSetMeshProperties(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>	InExecute(Context);
	FNDIInputParam<int32>	InNumVertices(Context);
	FNDIInputParam<int32>	InNumTexCoords(Context);
	FNDIInputParam<bool>	InHasColors(Context);
	FNDIInputParam<bool>	InHasTangentBasis(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool	bExecute = InExecute.GetAndAdvance();
		const int32	NumVertices = InNumVertices.GetAndAdvance();
		const int32	NumTexCoords = InNumTexCoords.GetAndAdvance();
		const bool	bHasColors = InHasColors.GetAndAdvance();
		const bool	bHasTangentBasis = InHasTangentBasis.GetAndAdvance();
		if (bExecute == false)
		{
			continue;
		}

		if ((NumVertices != InstanceData->NumVertices) ||
			(NumTexCoords != InstanceData->NumTexCoords) ||
			(bHasColors != (InstanceData->ColorOffset != INDEX_NONE)) ||
			(bHasTangentBasis != (InstanceData->TangentBasisOffset != INDEX_NONE)) )
		{
			InstanceData->UpdateMesh(NumVertices, NumTexCoords, bHasTangentBasis, bHasColors);
		}
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMSetSectionCount(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>	InExecute(Context);
	FNDIInputParam<int32>	InSectionCount(Context);

	bool bUpdateMesh = false;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool bExecute = InExecute.GetAndAdvance();
		const int32 NumSections = InSectionCount.GetAndAdvance();
		if (bExecute && (NumSections != InstanceData->Sections.Num()))
		{
			bUpdateMesh = true;
			InstanceData->Sections.SetNum(NumSections);
		}
	}
	if (bUpdateMesh)
	{
		InstanceData->UpdateMesh();
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMSetSectionData(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>	InExecute(Context);
	FNDIInputParam<int32>	InSectionIndex(Context);
	FNDIInputParam<int32>	InNumTriangles(Context);
	FNDIInputParam<int32>	InMaterialIndex(Context);

	bool bUpdateMesh = false;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool	bExecute = InExecute.GetAndAdvance();
		const int32 SectionIndex = InSectionIndex.GetAndAdvance();
		const int32 NumTriangles = InNumTriangles.GetAndAdvance();
		const int32 MaterialIndex = InMaterialIndex.GetAndAdvance();
		if (bExecute && InstanceData->Sections.IsValidIndex(SectionIndex))
		{
			FNDIInstanceSection& MeshSection = InstanceData->Sections[SectionIndex];
			bUpdateMesh |= MeshSection.MaxTriangles != NumTriangles;
			MeshSection.MaxTriangles = NumTriangles;
			MeshSection.MaterialIndex = MaterialIndex;
		}
	}
	if (bUpdateMesh)
	{
		InstanceData->UpdateMesh();
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMSetLocalBounds(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>		InExecute(Context);
	FNDIInputParam<FVector3f>	InBoundsMin(Context);
	FNDIInputParam<FVector3f>	InBoundsMax(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool		bExecute = InExecute.GetAndAdvance();
		const FVector3f	BoundsMin = InBoundsMin.GetAndAdvance();
		const FVector3f	BoundsMax = InBoundsMax.GetAndAdvance();
		if (bExecute)
		{
			const FVector BoundsMinV(BoundsMin);
			const FVector BoundsMaxV(BoundsMax);
			if (!InstanceData->LocalBounds.Min.Equals(BoundsMinV) || !InstanceData->LocalBounds.Max.Equals(BoundsMaxV))
			{
				InstanceData->LocalBounds.Min = BoundsMinV;
				InstanceData->LocalBounds.Max = BoundsMaxV;
			}
		}
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMClearAllSectionTriangles(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>	InExecute(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool	bExecute = InExecute.GetAndAdvance();
		if (bExecute)
		{
			for (FNDIInstanceSection& MeshSection : InstanceData->Sections)
			{
				MeshSection.AllocatedTriangles = 0;
			}
		}
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMClearSectionTriangles(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>	InExecute(Context);
	FNDIInputParam<int32>	InSectionIndex(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool	bExecute = InExecute.GetAndAdvance();
		const int32	SectionIndex = InSectionIndex.GetAndAdvance();
		if (bExecute && InstanceData->Sections.IsValidIndex(SectionIndex))
		{
			FNDIInstanceSection& MeshSection = InstanceData->Sections[SectionIndex];
			MeshSection.AllocatedTriangles = 0;
		}
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMAllocateSectionTriangles(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>	InExecute(Context);
	FNDIInputParam<int32>	InSectionIndex(Context);
	FNDIInputParam<int32>	InNumTriangles(Context);
	FNDIOutputParam<int32>	OutTriangleIndex(Context);
	FNDIOutputParam<int32>	OutNumAllocated(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool	bExecute = InExecute.GetAndAdvance();
		const int32	SectionIndex = InSectionIndex.GetAndAdvance();
		const int32 NumTriangles = InNumTriangles.GetAndAdvance();
		int32 TriangleIndex = -1;
		int32 NumAllocated = 0;
		if (bExecute && InstanceData->Sections.IsValidIndex(SectionIndex) && NumTriangles > 0 )
		{
			FScopeLock DataGuard(&InstanceData->CpuDataGuard);

			FNDIInstanceSection& Section = InstanceData->Sections[SectionIndex];
			const int32 Remaining = Section.MaxTriangles - Section.AllocatedTriangles;
			TriangleIndex = Remaining > 0 ? Section.TriangleOffset + Section.AllocatedTriangles : -1;
			NumAllocated = FMath::Min(Remaining, NumTriangles);
			Section.AllocatedTriangles += NumAllocated;
		}
		OutTriangleIndex.SetAndAdvance(TriangleIndex);
		OutNumAllocated.SetAndAdvance(NumAllocated);
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMGetTriangleVertices(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<int32>	InTriangleIndex(Context);
	FNDIOutputParam<bool>	OutValid(Context);
	FNDIOutputParam<int32>	OutIndex0(Context);
	FNDIOutputParam<int32>	OutIndex1(Context);
	FNDIOutputParam<int32>	OutIndex2(Context);

	const bool bHasCpuData		= InstanceData->IndexData.Num() > 0;
	const int32 NumTriangles	= bHasCpuData ? InstanceData->NumTriangles : 0;
	const uint32* IndexBuffer	= reinterpret_cast<const uint32*>(InstanceData->IndexData.GetData());
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 TriangleIndex = InTriangleIndex.GetAndAdvance();
		const bool bValid = TriangleIndex >= 0 && TriangleIndex < NumTriangles;
		OutValid.SetAndAdvance(bValid);
		OutIndex0.SetAndAdvance(bValid ? IndexBuffer[TriangleIndex * 3 + 0] : 0);
		OutIndex1.SetAndAdvance(bValid ? IndexBuffer[TriangleIndex * 3 + 1] : 0);
		OutIndex2.SetAndAdvance(bValid ? IndexBuffer[TriangleIndex * 3 + 2] : 0);
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMGetVertexPosition(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<int32>		InVertexIndex(Context);
	FNDIOutputParam<bool>		OutValid(Context);
	FNDIOutputParam<FVector3f>	OutPosition(Context);

	const bool bHasCpuData	= InstanceData->VertexData.Num() > 0 && InstanceData->PositionOffset != INDEX_NONE;
	const int32 NumVertices	= bHasCpuData ? InstanceData->NumVertices : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 VertexIndex = InVertexIndex.GetAndAdvance();
		const bool bValid = VertexIndex >= 0 && VertexIndex < NumVertices;
		OutValid.SetAndAdvance(bValid);
		OutPosition.SetAndAdvance(bValid ? InstanceData->GetVertexPosition(VertexIndex) : FVector3f::ZeroVector);
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMGetVertexTangentBasis(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<int32>		InVertexIndex(Context);
	FNDIOutputParam<bool>		OutValid(Context);
	FNDIOutputParam<FVector3f>	OutTangentX(Context);
	FNDIOutputParam<FVector3f>	OutTangentY(Context);
	FNDIOutputParam<FVector3f>	OutTangentZ(Context);

	const bool bHasCpuData = InstanceData->VertexData.Num() > 0 && InstanceData->TangentBasisOffset != INDEX_NONE;
	const int32 NumVertices = bHasCpuData ? InstanceData->NumVertices : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 VertexIndex = InVertexIndex.GetAndAdvance();
		const bool bValid = VertexIndex >= 0 && VertexIndex < NumVertices;
		FVector3f TangentX = FVector3f::ZeroVector;
		FVector3f TangentY = FVector3f::ZeroVector;
		FVector3f TangentZ = FVector3f::ZeroVector;
		if (bValid)
		{
			InstanceData->GetVertexTangentBasis(VertexIndex, TangentX, TangentY, TangentZ);
		}

		OutValid.SetAndAdvance(bValid);
		OutTangentX.SetAndAdvance(TangentX);
		OutTangentY.SetAndAdvance(TangentY);
		OutTangentZ.SetAndAdvance(TangentZ);
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMGetVertexTexCoord(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<int32>		InVertexIndex(Context);
	FNDIInputParam<int32>		InTexCoordIndex(Context);
	FNDIOutputParam<bool>		OutValid(Context);
	FNDIOutputParam<FVector2f>	OutTexCoord(Context);

	const bool bHasCpuData = InstanceData->VertexData.Num() > 0 && InstanceData->TexCoordOffset != INDEX_NONE;
	const int32 NumVertices = bHasCpuData ? InstanceData->NumVertices : 0;
	const int32 NumTexCoords = bHasCpuData ? InstanceData->NumTexCoords : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 VertexIndex = InVertexIndex.GetAndAdvance();
		const int32 TexCoordIndex = InTexCoordIndex.GetAndAdvance();
		const bool bValid = VertexIndex >= 0 && VertexIndex < NumVertices && TexCoordIndex >= 0 && TexCoordIndex < NumTexCoords;
		OutValid.SetAndAdvance(bValid);
		OutTexCoord.SetAndAdvance(bValid ? InstanceData->GetVertexTexCoord(VertexIndex, TexCoordIndex) : FVector2f::ZeroVector);
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMGetVertexColor(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<int32>		InVertexIndex(Context);
	FNDIOutputParam<bool>		OutValid(Context);
	FNDIOutputParam<FLinearColor>	OutColor(Context);

	const bool bHasCpuData = InstanceData->VertexData.Num() > 0 && InstanceData->ColorOffset != INDEX_NONE;
	const int32 NumVertices = bHasCpuData ? InstanceData->NumVertices : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 VertexIndex = InVertexIndex.GetAndAdvance();
		const bool bValid = VertexIndex >= 0 && VertexIndex < NumVertices;
		OutValid.SetAndAdvance(bValid);
		OutColor.SetAndAdvance(bValid ? InstanceData->GetVertexColor(VertexIndex) : FLinearColor::White);
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMGetVertexData(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<int32>		InVertexIndex(Context);
	FNDIOutputParam<bool>		OutValid(Context);
	FNDIOutputParam<FVector3f>	OutPosition(Context);
	FNDIOutputParam<FVector3f>	OutTangentX(Context);
	FNDIOutputParam<FVector3f>	OutTangentY(Context);
	FNDIOutputParam<FVector3f>	OutTangentZ(Context);
	FNDIOutputParam<FVector2f>	OutTexCoord(Context);
	FNDIOutputParam<FLinearColor>	OutColor(Context);

	const bool bHasCpuData = InstanceData->VertexData.Num() > 0;
	const int32 NumVertices = bHasCpuData ? InstanceData->NumVertices : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 VertexIndex = InVertexIndex.GetAndAdvance();
		const bool bValid = VertexIndex >= 0 && VertexIndex < NumVertices;
		FVector3f Position = FVector3f::ZeroVector;
		FVector3f TangentX = FVector3f::ZeroVector;
		FVector3f TangentY = FVector3f::ZeroVector;
		FVector3f TangentZ = FVector3f::ZeroVector;
		FVector2f TexCoord = FVector2f::ZeroVector;
		FLinearColor Color = FLinearColor::White;
		if (bValid)
		{
			if (InstanceData->PositionOffset != INDEX_NONE)
			{
				Position = InstanceData->GetVertexPosition(VertexIndex);
			}
			if (InstanceData->TangentBasisOffset != INDEX_NONE)
			{
				InstanceData->GetVertexTangentBasis(VertexIndex, TangentX, TangentY, TangentZ);
			}
			if (InstanceData->TexCoordOffset != INDEX_NONE)
			{
				TexCoord = InstanceData->GetVertexTexCoord(VertexIndex, 0);
			}
			if (InstanceData->ColorOffset != INDEX_NONE)
			{
				Color = InstanceData->GetVertexColor(VertexIndex);
			}
		}
		OutValid.SetAndAdvance(bValid);
		OutPosition.SetAndAdvance(Position);
		OutTangentX.SetAndAdvance(TangentX);
		OutTangentY.SetAndAdvance(TangentY);
		OutTangentZ.SetAndAdvance(TangentZ);
		OutTexCoord.SetAndAdvance(TexCoord);
		OutColor.SetAndAdvance(Color);
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMSetTriangleVertices(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>	InExecute(Context);
	FNDIInputParam<int32>	InTriangleIndex(Context);
	FNDIInputParam<int32>	InIndex0(Context);
	FNDIInputParam<int32>	InIndex1(Context);
	FNDIInputParam<int32>	InIndex2(Context);

	InstanceData->ModifyCpuData();

	const int32 NumTriangles = InstanceData->NumTriangles;
	uint32* IndexBuffer = reinterpret_cast<uint32*>(InstanceData->IndexData.GetData());
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool bExecute = InExecute.GetAndAdvance();
		const int32 TriangleIndex = InTriangleIndex.GetAndAdvance();
		const int32 Index0 = InIndex0.GetAndAdvance();
		const int32 Index1 = InIndex1.GetAndAdvance();
		const int32 Index2 = InIndex2.GetAndAdvance();
		if (bExecute && TriangleIndex >= 0 && TriangleIndex < NumTriangles)
		{
			IndexBuffer[TriangleIndex * 3 + 0] = Index0;
			IndexBuffer[TriangleIndex * 3 + 1] = Index1;
			IndexBuffer[TriangleIndex * 3 + 2] = Index2;
		}
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMSetVertexPosition(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>		InExecute(Context);
	FNDIInputParam<int32>		InVertexIndex(Context);
	FNDIInputParam<FVector3f>	InPosition(Context);

	InstanceData->ModifyCpuData();

	const int32 NumVertices = InstanceData->PositionOffset != INDEX_NONE ? InstanceData->NumVertices : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool		bExecute = InExecute.GetAndAdvance();
		const int32		VertexIndex = InVertexIndex.GetAndAdvance();
		const FVector3f	Position = InPosition.GetAndAdvance();

		if (bExecute && VertexIndex >= 0 && VertexIndex < NumVertices)
		{
			InstanceData->SetVertexPosition(VertexIndex, Position);
		}
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMSetVertexTangentBasis(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>		InExecute(Context);
	FNDIInputParam<int32>		InVertexIndex(Context);
	FNDIInputParam<FVector3f>	InTangentX(Context);
	FNDIInputParam<FVector3f>	InTangentY(Context);
	FNDIInputParam<FVector3f>	InTangentZ(Context);

	InstanceData->ModifyCpuData();

	const int32 NumVertices = InstanceData->TangentBasisOffset != INDEX_NONE ? InstanceData->NumVertices : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool		bExecute = InExecute.GetAndAdvance();
		const int32		VertexIndex = InVertexIndex.GetAndAdvance();
		const FVector3f	TangentX = InTangentX.GetAndAdvance();
		const FVector3f	TangentY = InTangentY.GetAndAdvance();
		const FVector3f	TangentZ = InTangentZ.GetAndAdvance();

		if (bExecute && VertexIndex >= 0 && VertexIndex < NumVertices)
		{
			InstanceData->SetVertexTangentBasis(VertexIndex, TangentX, TangentY, TangentZ);
		}
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMSetVertexTexCoord(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>		InExecute(Context);
	FNDIInputParam<int32>		InVertexIndex(Context);
	FNDIInputParam<int32>		InTexCoord(Context);
	FNDIInputParam<FVector2f>	InCoord(Context);

	InstanceData->ModifyCpuData();

	const int32 NumVertices = InstanceData->TexCoordOffset != INDEX_NONE ? InstanceData->NumVertices : 0;
	const int32 NumTexCoords = InstanceData->TexCoordOffset != INDEX_NONE ? InstanceData->NumTexCoords : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool		bExecute = InExecute.GetAndAdvance();
		const int32		VertexIndex = InVertexIndex.GetAndAdvance();
		const int32		TexCoord = InTexCoord.GetAndAdvance();
		const FVector2f	Coord = InCoord.GetAndAdvance();

		if (bExecute && VertexIndex >= 0 && VertexIndex < NumVertices && TexCoord >= 0 && TexCoord < NumTexCoords)
		{
			InstanceData->SetVertexTexCoord(VertexIndex, TexCoord, Coord);
		}
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMSetVertexColor(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>		InExecute(Context);
	FNDIInputParam<int32>		InVertexIndex(Context);
	FNDIInputParam<FLinearColor>	InColor(Context);

	InstanceData->ModifyCpuData();

	const int32 NumVertices = InstanceData->ColorOffset != INDEX_NONE ? InstanceData->NumVertices : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool		bExecute = InExecute.GetAndAdvance();
		const int32		VertexIndex = InVertexIndex.GetAndAdvance();
		const FLinearColor	Color = InColor.GetAndAdvance();

		if (bExecute && VertexIndex >= 0 && VertexIndex < NumVertices)
		{
			InstanceData->SetVertexColor(VertexIndex, Color);
		}
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMSetVertexData(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>		InExecute(Context);
	FNDIInputParam<int32>		InVertexIndex(Context);
	FNDIInputParam<FVector3f>	InPosition(Context);
	FNDIInputParam<FVector3f>	InTangentX(Context);
	FNDIInputParam<FVector3f>	InTangentY(Context);
	FNDIInputParam<FVector3f>	InTangentZ(Context);
	FNDIInputParam<FVector2f>	InTexCoord(Context);
	FNDIInputParam<FLinearColor>	InColor(Context);

	InstanceData->ModifyCpuData();

	const int32 NumVertices = InstanceData->NumVertices;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool bExecute = InExecute.GetAndAdvance();
		const int32 VertexIndex = InVertexIndex.GetAndAdvance();
		const FVector3f Position = InPosition.GetAndAdvance();
		const FVector3f TangentX = InTangentX.GetAndAdvance();
		const FVector3f TangentY = InTangentY.GetAndAdvance();
		const FVector3f TangentZ = InTangentZ.GetAndAdvance();
		const FVector2f TexCoord = InTexCoord.GetAndAdvance();
		const FLinearColor Color = InColor.GetAndAdvance();
		if (bExecute)
		{
			if (InstanceData->PositionOffset != INDEX_NONE)
			{
				InstanceData->SetVertexPosition(VertexIndex, Position);
			}
			if (InstanceData->TangentBasisOffset != INDEX_NONE)
			{
				InstanceData->SetVertexTangentBasis(VertexIndex, TangentX, TangentY, TangentZ);
			}
			if (InstanceData->TexCoordOffset != INDEX_NONE)
			{
				InstanceData->SetVertexTexCoord(VertexIndex, 0, TexCoord);
			}
			if (InstanceData->ColorOffset != INDEX_NONE)
			{
				InstanceData->SetVertexColor(VertexIndex, Color);
			}
		}
	}
}

void UNiagaraDataInterfaceDynamicMesh::VMAppendTriangle(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIDynamicMeshLocal;

	VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool>	InExecute(Context);
	FNDIInputParam<int32>	InSectionIndex(Context);
	FVMVertexInput			InVertex0(Context);
	FVMVertexInput			InVertex1(Context);
	FVMVertexInput			InVertex2(Context);
	FNDIOutputParam<int32>	OutTriangleIndex(Context);

	InstanceData->ModifyCpuData();

	uint32* IndexBuffer = reinterpret_cast<uint32*>(InstanceData->IndexData.GetData());
	const int32 NumVertices = InstanceData->NumVertices;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool bExecute = InExecute.GetAndAdvance();
		const int32 SectionIndex = InSectionIndex.GetAndAdvance();
		const FVertexData Vertex0 = InVertex0.GetAndAdvance();
		const FVertexData Vertex1 = InVertex1.GetAndAdvance();
		const FVertexData Vertex2 = InVertex2.GetAndAdvance();
		if (!bExecute || !InstanceData->Sections.IsValidIndex(SectionIndex))
		{
			continue;
		}
		int32 TriangleIndex = INDEX_NONE;
		{
			FScopeLock DataGuard(&InstanceData->CpuDataGuard);
			FNDIInstanceSection& Section = InstanceData->Sections[SectionIndex];
			if (Section.AllocatedTriangles >= Section.MaxTriangles)
			{
				OutTriangleIndex.SetAndAdvance(TriangleIndex);
				continue;
			}
			TriangleIndex = Section.AllocatedTriangles++;
			OutTriangleIndex.SetAndAdvance(TriangleIndex);
		}

		IndexBuffer[TriangleIndex * 3 + 0] = TriangleIndex * 3 + 0;
		IndexBuffer[TriangleIndex * 3 + 1] = TriangleIndex * 3 + 1;
		IndexBuffer[TriangleIndex * 3 + 2] = TriangleIndex * 3 + 2;

		if (InstanceData->PositionOffset != INDEX_NONE)
		{
			InstanceData->SetVertexPosition(TriangleIndex * 3 + 0, Vertex0.Position);
			InstanceData->SetVertexPosition(TriangleIndex * 3 + 1, Vertex1.Position);
			InstanceData->SetVertexPosition(TriangleIndex * 3 + 2, Vertex2.Position);
		}
		if (InstanceData->TangentBasisOffset != INDEX_NONE)
		{
			InstanceData->SetVertexTangentBasis(TriangleIndex * 3 + 0, Vertex0.TangentX, Vertex0.TangentY, Vertex0.TangentZ);
			InstanceData->SetVertexTangentBasis(TriangleIndex * 3 + 1, Vertex1.TangentX, Vertex1.TangentY, Vertex1.TangentZ);
			InstanceData->SetVertexTangentBasis(TriangleIndex * 3 + 2, Vertex2.TangentX, Vertex2.TangentY, Vertex2.TangentZ);
		}
		if (InstanceData->TexCoordOffset != INDEX_NONE)
		{
			InstanceData->SetVertexTexCoord(TriangleIndex * 3 + 0, 0, Vertex0.TexCoord);
			InstanceData->SetVertexTexCoord(TriangleIndex * 3 + 1, 0, Vertex1.TexCoord);
			InstanceData->SetVertexTexCoord(TriangleIndex * 3 + 2, 0, Vertex2.TexCoord);
		}
		if (InstanceData->ColorOffset != INDEX_NONE)
		{
			InstanceData->SetVertexColor(TriangleIndex * 3 + 0, Vertex0.Color);
			InstanceData->SetVertexColor(TriangleIndex * 3 + 1, Vertex1.Color);
			InstanceData->SetVertexColor(TriangleIndex * 3 + 2, Vertex2.Color);
		}
	}
}

#undef LOCTEXT_NAMESPACE
