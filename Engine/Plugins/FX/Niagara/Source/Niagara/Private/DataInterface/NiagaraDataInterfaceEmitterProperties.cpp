// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceEmitterProperties.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"

#include "Internationalization/Internationalization.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceEmitterProperties)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceEmitterProperties"

//////////////////////////////////////////////////////////////////////////

namespace NDIEmitterPropertiesLocal
{
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceEmitterPropertiesTemplate.ush");

	static const FName GetLocalSpaceName("GetLocalSpace");
	static const FName GetBoundsName("GetBounds");
	static const FName GetFixedBoundsName("GetFixedBounds");
	static const FName SetFixedBoundsName("SetFixedBounds");

	struct FInstanceData_GameToRender
	{
		FBox	Bounds = FBox(EForceInit::ForceInit);
		FBox	FixedBounds = FBox(EForceInit::ForceInit);
	};

	struct FInstanceData_GameThread
	{
		FNiagaraEmitterInstance*	EmitterInstance = nullptr;
		bool						bLocalSpace = false;
	};

	struct FInstanceData_RenderThread
	{
		bool	bLocalSpace = false;
		FBox	Bounds = FBox(EForceInit::ForceInit);
		FBox	FixedBounds = FBox(EForceInit::ForceInit);
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID)
		{
			FInstanceData_GameToRender* InstanceData_ForRT = new(DataForRenderThread) FInstanceData_GameToRender();
			FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);

			if (InstanceData_GT->EmitterInstance)
			{
				InstanceData_ForRT->Bounds = InstanceData_GT->EmitterInstance->GetBounds();
				InstanceData_ForRT->FixedBounds = InstanceData_GT->EmitterInstance->GetFixedBounds();
			}
		}

		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override
		{
			FInstanceData_GameToRender* InstanceData_FromGT = static_cast<FInstanceData_GameToRender*>(PerInstanceData);
			if ( FInstanceData_RenderThread* InstanceData_RT = PerInstanceData_RenderThread.Find(InstanceID) )
			{
				InstanceData_RT->Bounds = InstanceData_FromGT->Bounds;
				InstanceData_RT->FixedBounds = InstanceData_FromGT->FixedBounds;
			}
			InstanceData_FromGT->~FInstanceData_GameToRender();
		}

		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
		{
			return sizeof(FInstanceData_GameToRender);
		}
	
		TMap<FNiagaraSystemInstanceID, FInstanceData_RenderThread>	PerInstanceData_RenderThread;
	};
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceEmitterProperties::UNiagaraDataInterfaceEmitterProperties(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDIEmitterPropertiesLocal;

	Proxy.Reset(new FNDIProxy());
}

void UNiagaraDataInterfaceEmitterProperties::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowEmitterVariable | ENiagaraTypeRegistryFlags::AllowParticleVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceEmitterProperties::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	auto* OtherTyped = CastChecked<const UNiagaraDataInterfaceEmitterProperties>(Other);
	return
		OtherTyped->EmitterBinding == EmitterBinding;
}

bool UNiagaraDataInterfaceEmitterProperties::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	auto* DestinationTyped = CastChecked<UNiagaraDataInterfaceEmitterProperties>(Destination);
	DestinationTyped->EmitterBinding = EmitterBinding;

	return true;
}

bool UNiagaraDataInterfaceEmitterProperties::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIEmitterPropertiesLocal;

	FInstanceData_GameThread* InstanceData_GT = new(PerInstanceData) FInstanceData_GameThread();
	InstanceData_GT->EmitterInstance = EmitterBinding.Resolve(SystemInstance, this);
	if (InstanceData_GT->EmitterInstance)
	{
		InstanceData_GT->bLocalSpace = InstanceData_GT->EmitterInstance->IsLocalSpace();
	}

	if ( IsUsedWithGPUScript() )
	{
		// Initialize render side instance data
		ENQUEUE_RENDER_COMMAND(NDIEmitter_InitRT)
		(
			[Proxy_RT=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId(), bLocalSpace_RT=InstanceData_GT->bLocalSpace](FRHICommandList& CmdList)
			{
				FInstanceData_RenderThread* InstanceData_RT = &Proxy_RT->PerInstanceData_RenderThread.Add(InstanceID);
				InstanceData_RT->bLocalSpace = bLocalSpace_RT;
			}
		);
	}

	return true;
}

void UNiagaraDataInterfaceEmitterProperties::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIEmitterPropertiesLocal;

	FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
	InstanceData_GT->~FInstanceData_GameThread();

	if ( IsUsedWithGPUScript() )
	{
		ENQUEUE_RENDER_COMMAND(NDIEmitter_InitRT)
		(
			[Proxy_RT=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandList& CmdList)
			{
				Proxy_RT->PerInstanceData_RenderThread.Remove(InstanceID);
			}
		);
	}
}

int32 UNiagaraDataInterfaceEmitterProperties::PerInstanceDataSize() const
{
	using namespace NDIEmitterPropertiesLocal;
	return sizeof(FInstanceData_GameThread);
}

void UNiagaraDataInterfaceEmitterProperties::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID)
{
	using namespace NDIEmitterPropertiesLocal;
	FNDIProxy* DIProxy = GetProxyAs<FNDIProxy>();
	DIProxy->ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, InstanceID);
}

bool UNiagaraDataInterfaceEmitterProperties::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIEmitterPropertiesLocal;

	check(PerInstanceData && SystemInstance);

	FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
	//-TODO:

	return false;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceEmitterProperties::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIEmitterPropertiesLocal;

	// Reserve space
	OutFunctions.Reserve(OutFunctions.Num() + 4);

	// Build default signature
	FNiagaraFunctionSignature DefaultSignature;
	DefaultSignature.bMemberFunction = true;
	DefaultSignature.bRequiresContext = false;
	DefaultSignature.bSupportsCPU = true;
	DefaultSignature.bSupportsGPU = true;
	DefaultSignature.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Emitter"));

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetLocalSpaceName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsLocalSpace"));
#if WITH_EDITORONLY_DATA
		Sig.SetDescription(LOCTEXT("GetEmitterLocalSpace", "Returns if the emitter is using local space or world space."));
#endif
	}
	{
		FNiagaraFunctionSignature & Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetBoundsName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Local Min"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Local Max"));
#if WITH_EDITORONLY_DATA
		Sig.SetDescription(LOCTEXT("GetEmitterBounds", "Get local space bounds for the emitter, this is the bounds calculated from the last frame, either dynamic or fixed bounds depending on what is set."));
#endif
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetFixedBoundsName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Local Min"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Local Max"));
#if WITH_EDITORONLY_DATA
		Sig.SetDescription(LOCTEXT("GetEmitterFixedBounds", "Get the runtime set local space fixed bounds for the emitter.  If Valid is false then no runtime fixed bounds has been set, the emitter will be use the asset fixed bounds or dynamically generated bounds."));
#endif
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = SetFixedBoundsName;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsGPU = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Local Min"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Local Max"));
#if WITH_EDITORONLY_DATA
		Sig.SetDescription(LOCTEXT("SetEmitterFixedBounds", "Set the runtime local space fixed bounds for the emitter, this stomps any fixed bounds set from Blueprint.  If Valid is set to false the emitter will fallback to the asset bounds setup, i.e. fixed / dynamic / programmable."));
#endif
	}
}
#endif

void UNiagaraDataInterfaceEmitterProperties::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* PerInstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIEmitterPropertiesLocal;

	if ( BindingInfo.Name == GetLocalSpaceName )
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceEmitterProperties::VMGetLocalSpace);
	}
	else if (BindingInfo.Name == GetBoundsName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceEmitterProperties::VMGetBounds);
	}
	else if (BindingInfo.Name == GetFixedBoundsName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceEmitterProperties::VMGetFixedBounds);
	}
	else if (BindingInfo.Name == SetFixedBoundsName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceEmitterProperties::VMSetFixedBounds);
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceEmitterProperties::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(NDIEmitterPropertiesLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceEmitterProperties::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParameterInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIEmitterPropertiesLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceEmitterProperties::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIEmitterPropertiesLocal;

	if ((FunctionInfo.DefinitionName == GetLocalSpaceName) ||
		(FunctionInfo.DefinitionName == GetBoundsName) ||
		(FunctionInfo.DefinitionName == GetFixedBoundsName) )
	{
		return true;
	}

	return false;
}
#endif

void UNiagaraDataInterfaceEmitterProperties::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceEmitterProperties::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIEmitterPropertiesLocal;

	FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	FInstanceData_RenderThread& InstanceData_RT = DIProxy.PerInstanceData_RenderThread.FindChecked(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters	= Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->LocalSpace		= InstanceData_RT.bLocalSpace ? 1 : 0;
	ShaderParameters->BoundsValid		= InstanceData_RT.Bounds.IsValid;
	ShaderParameters->BoundsMin			= FVector3f(InstanceData_RT.Bounds.Min);
	ShaderParameters->BoundsMax			= FVector3f(InstanceData_RT.Bounds.Max);
	ShaderParameters->FixedBoundsValid	= InstanceData_RT.FixedBounds.IsValid;
	ShaderParameters->FixedBoundsMin	= FVector3f(InstanceData_RT.FixedBounds.Min);
	ShaderParameters->FixedBoundsMax	= FVector3f(InstanceData_RT.FixedBounds.Max);
}

#if WITH_EDITOR	
void UNiagaraDataInterfaceEmitterProperties::GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	if (Asset == nullptr)
	{
		return;
	}

	// See if we are resolve the source emitter
	if ( EmitterBinding.BindingMode == ENiagaraDataInterfaceEmitterBindingMode::Other )
	{
		UNiagaraEmitter* NiagaraEmitter = EmitterBinding.Resolve(this);
		if (NiagaraEmitter == nullptr)
		{
			OutWarnings.Emplace(
				LOCTEXT("SourceEmitterNotFound", "Source emitter was not found."),
				FText::Format(LOCTEXT("SourceEmitterNotFoundSummary", "Source emitter '{0}' could not be found"), FText::FromName(EmitterBinding.EmitterName)),
				FNiagaraDataInterfaceFix()
			);
		}
	}
}
#endif

void UNiagaraDataInterfaceEmitterProperties::VMGetLocalSpace(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIEmitterPropertiesLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<bool> OutIsLocalSpace(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutIsLocalSpace.SetAndAdvance(InstanceData->bLocalSpace);
	}
}

void UNiagaraDataInterfaceEmitterProperties::VMGetBounds(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIEmitterPropertiesLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<FNiagaraBool> OutIsValid(Context);
	FNDIOutputParam<FVector3f> OutMin(Context);
	FNDIOutputParam<FVector3f> OutMax(Context);

	const FNiagaraEmitterInstance* EmitterInstance = InstanceData->EmitterInstance;
	const FBox Bounds = EmitterInstance ? EmitterInstance->GetBounds() : FBox(EForceInit::ForceInit);
	const FNiagaraBool BoundsValid(Bounds.IsValid ? true : false);
	const FVector3f BoundsMin = Bounds.IsValid ? FVector3f(Bounds.Min) : FVector3f::Zero();
	const FVector3f BoundsMax = Bounds.IsValid ? FVector3f(Bounds.Max) : FVector3f::Zero();

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutIsValid.SetAndAdvance(BoundsValid);
		OutMin.SetAndAdvance(BoundsMin);
		OutMax.SetAndAdvance(BoundsMax);
	}
}

void UNiagaraDataInterfaceEmitterProperties::VMGetFixedBounds(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIEmitterPropertiesLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<FNiagaraBool> OutIsValid(Context);
	FNDIOutputParam<FVector3f> OutMin(Context);
	FNDIOutputParam<FVector3f> OutMax(Context);

	const FNiagaraEmitterInstance* EmitterInstance = InstanceData->EmitterInstance;
	const FBox FixedBounds = EmitterInstance ? EmitterInstance->GetFixedBounds() : FBox(EForceInit::ForceInit);
	const FNiagaraBool BoundsValid(FixedBounds.IsValid ? true : false);
	const FVector3f BoundsMin = FixedBounds.IsValid ? FVector3f(FixedBounds.Min) : FVector3f::Zero();
	const FVector3f BoundsMax = FixedBounds.IsValid ? FVector3f(FixedBounds.Max) : FVector3f::Zero();

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutIsValid.SetAndAdvance(BoundsValid);
		OutMin.SetAndAdvance(BoundsMin);
		OutMax.SetAndAdvance(BoundsMax);
	}
}

void UNiagaraDataInterfaceEmitterProperties::VMSetFixedBounds(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIEmitterPropertiesLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool> InExecute(Context);
	FNDIInputParam<FNiagaraBool> InIsValid(Context);
	FNDIInputParam<FVector3f> InMin(Context);
	FNDIInputParam<FVector3f> InMax(Context);

	if (FNiagaraEmitterInstance* EmitterInstance = InstanceData->EmitterInstance)
	{
		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			FBox FixedBounds;
			FixedBounds.IsValid = InIsValid.GetAndAdvance();
			FixedBounds.Min = FVector(InMin.GetAndAdvance());
			FixedBounds.Max = FVector(InMax.GetAndAdvance());
			const bool bExecute = InExecute.GetAndAdvance();
			if (bExecute)
			{
				EmitterInstance->SetFixedBounds(FixedBounds);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

