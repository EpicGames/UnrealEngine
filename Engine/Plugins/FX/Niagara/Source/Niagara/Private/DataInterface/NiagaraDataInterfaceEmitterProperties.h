// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceEmitterBinding.h"
#include "NiagaraDataInterfaceEmitterProperties.generated.h"

enum class ENiagaraSimTarget : uint8;
struct FNiagaraDataInterfaceGeneratedFunction;
struct FNiagaraFunctionSignature;
struct FVMExternalFunctionBindingInfo;

/**
Allows access to various emitter properties that are not part of the simulation data.
*/
UCLASS(EditInlineNew, Category = "DataInterface", CollapseCategories, meta=(DisplayName="Emitter Properties"))
class UNiagaraDataInterfaceEmitterProperties : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(uint32,	LocalSpace)
		SHADER_PARAMETER(uint32,	BoundsValid)
		SHADER_PARAMETER(FVector3f,	BoundsMin)
		SHADER_PARAMETER(FVector3f,	BoundsMax)
		SHADER_PARAMETER(uint32,	FixedBoundsValid)
		SHADER_PARAMETER(FVector3f,	FixedBoundsMin)
		SHADER_PARAMETER(FVector3f,	FixedBoundsMax)
	END_SHADER_PARAMETER_STRUCT();

public:
	/** Selects which emitter the data interface will bind to, i.e the emitter we are contained within or a named emitter. */
	UPROPERTY(EditAnywhere, Category = "Emitter")
	FNiagaraDataInterfaceEmitterBinding EmitterBinding;

	//UNiagaraDataInterface Interface
	virtual void PostInitProperties() override;
	// UObject Interface End

	// UNiagaraDataInterface Interface Begin
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual int32 PerInstanceDataSize() const override;

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override;

	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;

	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
#if WITH_EDITORONLY_DATA
	virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
#if WITH_EDITOR	
	virtual void GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& Warnings, TArray<FNiagaraDataInterfaceFeedback>& Info) override;
#endif
	//UNiagaraDataInterface Interface

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

private:
	void VMGetLocalSpace(FVectorVMExternalFunctionContext& Context);
	void VMGetBounds(FVectorVMExternalFunctionContext& Context);
	void VMGetFixedBounds(FVectorVMExternalFunctionContext& Context);
	void VMSetFixedBounds(FVectorVMExternalFunctionContext& Context);
};
