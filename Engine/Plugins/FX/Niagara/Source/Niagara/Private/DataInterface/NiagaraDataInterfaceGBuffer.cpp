// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceGBuffer.h"
#include "Containers/StridedView.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraTypes.h"
#include "NiagaraShaderParametersBuilder.h"

#include "Internationalization/Internationalization.h"
#include "FXRenderingUtils.h"
#include "RHIStaticStates.h"
#include "RenderGraphResources.h"
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceGBuffer)

//////////////////////////////////////////////////////////////////////////

namespace NiagaraDataInterfaceGBufferLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VelocityTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, VelocityTextureSampler)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceGBufferTemplate.ush");

	struct EDIFunctionVersion
	{
		enum Type
		{
			InitialVersion = 0,
			AddedApplyViewportOffset = 1,

			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};
	};

	struct FGBufferAttribute
	{
		FGBufferAttribute(const TCHAR* InAttributeName, const TCHAR* InAttributeType, FNiagaraTypeDefinition InTypeDef, FText InDescription, uint16 InMiscUsageBitMask)
			: AttributeName(InAttributeName)
			, AttributeType(InAttributeType)
			, TypeDef(InTypeDef)
			, Description(InDescription)
			, MiscUsageBitMask(InMiscUsageBitMask)
		{
			FString TempName;
			TempName = TEXT("Decode");
			TempName += AttributeName;
			ScreenUVFunctionName = FName(TempName);
		}

		const TCHAR*			AttributeName;
		const TCHAR*			AttributeType;
		FName					ScreenUVFunctionName;
		FNiagaraTypeDefinition	TypeDef;
		FText					Description;
		uint16					MiscUsageBitMask;
	};

	static FText GetDescription_ScreenVelocity()
	{
#if WITH_EDITORONLY_DATA
		return NSLOCTEXT("Niagara", "GBuffer_ScreenVelocity", "Get the screen space velocity in UV space.  This is a per frame value, to get per second you must divide by delta time.");
#else
		return FText::GetEmpty();
#endif
	}

	static FText GetDescription_WorldVelocity()
	{
#if WITH_EDITORONLY_DATA
		return NSLOCTEXT("Niagara", "GBuffer_WorldVelocity", "Get the world space velocity estimate (not accurate due to reconstrucion).  This is a per frame value, to get per second you must divide by delta time.");
#else
		return FText::GetEmpty();
#endif
	}

	static FText GetDescription_SceneColor()
	{
#if WITH_EDITORONLY_DATA
		return NSLOCTEXT("Niagara", "GBuffer_SceneColor", "Gets the current frames scene color buffer, this will not include translucency since we run PostOpaque.");
#else
		return FText::GetEmpty();
#endif
	}

	static TConstArrayView<FGBufferAttribute> GetGBufferAttributes()
	{
		static const TArray<FGBufferAttribute> GBufferAttributes =
		{
			FGBufferAttribute(TEXT("DiffuseColor"),		TEXT("float3"),	FNiagaraTypeDefinition::GetVec3Def(), FText::GetEmpty(), 0),
			FGBufferAttribute(TEXT("WorldNormal"),		TEXT("float3"),	FNiagaraTypeDefinition::GetVec3Def(), FText::GetEmpty(), 0),
			FGBufferAttribute(TEXT("ScreenVelocity"),	TEXT("float3"),	FNiagaraTypeDefinition::GetVec3Def(), GetDescription_ScreenVelocity(), 0),
			FGBufferAttribute(TEXT("WorldVelocity"),	TEXT("float3"),	FNiagaraTypeDefinition::GetVec3Def(), GetDescription_WorldVelocity(), 0),
			FGBufferAttribute(TEXT("BaseColor"),		TEXT("float3"),	FNiagaraTypeDefinition::GetVec3Def(), FText::GetEmpty(), 0),
			//FGBufferAttribute(TEXT("SpecularColor"),	TEXT("float3"),	FNiagaraTypeDefinition::GetVec3Def(), FText::GetEmpty(), 0),
			FGBufferAttribute(TEXT("Metallic"),			TEXT("float"),	FNiagaraTypeDefinition::GetFloatDef(), FText::GetEmpty(), 0),
			FGBufferAttribute(TEXT("Specular"),			TEXT("float"),	FNiagaraTypeDefinition::GetFloatDef(), FText::GetEmpty(), 0),
			FGBufferAttribute(TEXT("Roughness"),		TEXT("float"),	FNiagaraTypeDefinition::GetFloatDef(), FText::GetEmpty(), 0),
			FGBufferAttribute(TEXT("Depth"),			TEXT("float"),	FNiagaraTypeDefinition::GetFloatDef(), FText::GetEmpty(), 0),
			//FGBufferAttribute(TEXT("Stencil"),			TEXT("int"),	FNiagaraTypeDefinition::GetIntDef(), FText::GetEmpty(), 0),

			FGBufferAttribute(TEXT("PartialDepth"),		TEXT("float"),	FNiagaraTypeDefinition::GetFloatDef(), FText::GetEmpty(), uint16(ENiagaraScriptMiscUsageMask::UsesPartialDepthCollisionQuery)),

			FGBufferAttribute(TEXT("CustomDepth"),		TEXT("float"),	FNiagaraTypeDefinition::GetFloatDef(), FText::GetEmpty(), 0),
			FGBufferAttribute(TEXT("CustomStencil"),	TEXT("int"),	FNiagaraTypeDefinition::GetIntDef(), FText::GetEmpty(), 0),

			FGBufferAttribute(TEXT("SceneColor"),		TEXT("float4"),	FNiagaraTypeDefinition::GetVec4Def(), GetDescription_SceneColor(), 0),
			FGBufferAttribute(TEXT("ShadingModelID"),	TEXT("int"),	FNiagaraTypeDefinition::GetIntDef(), FText::GetEmpty(), 0),
		};

		return MakeArrayView(GBufferAttributes);
	}
}

//////////////////////////////////////////////////////////////////////////

struct FNiagaraDataIntefaceProxyGBuffer : public FNiagaraDataInterfaceProxy
{
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }
};

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceGBuffer::UNiagaraDataInterfaceGBuffer(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataIntefaceProxyGBuffer());
}

void UNiagaraDataInterfaceGBuffer::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceGBuffer::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NiagaraDataInterfaceGBufferLocal;

	TConstArrayView<FGBufferAttribute> GBufferAttributes = GetGBufferAttributes();

	OutFunctions.Reserve(OutFunctions.Num() + (GBufferAttributes.Num()));

	for ( const FGBufferAttribute& Attribute : GBufferAttributes )
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.AddDefaulted_GetRef();
		Signature.Name = Attribute.ScreenUVFunctionName;
		Signature.Description = Attribute.Description;
		Signature.bMemberFunction = true;
		Signature.bRequiresContext = false;
		Signature.bSupportsCPU = false;
		Signature.MiscUsageBitMask = Attribute.MiscUsageBitMask;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("GBufferInterface")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("ScreenUV")));
		Signature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("ApplyViewportOffset"))).SetValue(true);
		Signature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
		Signature.Outputs.Add(FNiagaraVariable(Attribute.TypeDef, Attribute.AttributeName));
		Signature.FunctionVersion = EDIFunctionVersion::LatestVersion;
	}
}
#endif

void UNiagaraDataInterfaceGBuffer::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NiagaraDataInterfaceGBufferLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceGBuffer::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FRDGTextureRef VelocityTexture = nullptr;
	NiagaraDataInterfaceGBufferLocal::FShaderParameters* Parameters = Context.GetParameterNestedStruct<NiagaraDataInterfaceGBufferLocal::FShaderParameters>();
	if (Context.IsResourceBound(&Parameters->VelocityTexture))
	{
		TConstStridedView<FSceneView> SimulationSceneViews = Context.GetComputeDispatchInterface().GetSimulationSceneViews();
		if (SimulationSceneViews.Num() > 0)
		{
			VelocityTexture = UE::FXRenderingUtils::GetSceneVelocityTexture(SimulationSceneViews[0]);
		}
		
		if (VelocityTexture == nullptr || !VelocityTexture->HasBeenProduced())
		{
			VelocityTexture = Context.GetComputeDispatchInterface().GetBlackTexture(Context.GetGraphBuilder(), ETextureDimension::Texture2D);
		}
	}

	Parameters->VelocityTexture = VelocityTexture;
	Parameters->VelocityTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceGBuffer::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(NiagaraDataInterfaceGBufferLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<NiagaraDataInterfaceGBufferLocal::FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceGBuffer::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NiagaraDataInterfaceGBufferLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceGBuffer::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NiagaraDataInterfaceGBufferLocal;

	for ( const FGBufferAttribute& Attribute : GetGBufferAttributes() )
	{
		if (FunctionInfo.DefinitionName == Attribute.ScreenUVFunctionName)
		{
			return true;
		}
	}

	return false;
}

bool UNiagaraDataInterfaceGBuffer::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	using namespace NiagaraDataInterfaceGBufferLocal;

	bool bWasChanged = false;

	// Early out for version matching
	if (FunctionSignature.FunctionVersion == EDIFunctionVersion::LatestVersion)
	{
		return bWasChanged;
	}

	// AddedApplyViewportOffset
	if ( FunctionSignature.FunctionVersion < EDIFunctionVersion::AddedApplyViewportOffset )
	{
		FunctionSignature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("ApplyViewportOffset"))).SetValue(false);
		bWasChanged = true;
	}

	FunctionSignature.FunctionVersion = EDIFunctionVersion::LatestVersion;
	return bWasChanged;
}
#endif

