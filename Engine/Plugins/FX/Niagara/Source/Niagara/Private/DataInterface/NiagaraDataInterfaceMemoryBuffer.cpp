// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceMemoryBuffer.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraMeshVertexFactory.h"
#include "NiagaraParameterStore.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGpuReadbackManager.h"
#include "NiagaraSimStageData.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraWorldManager.h"

#include "Materials/MaterialRenderProxy.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "UnifiedBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceMemoryBuffer)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterfaceMemoryBuffer"

namespace NDIMemoryBufferLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(int32,									NumElements)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer,	Buffer)
	END_SHADER_PARAMETER_STRUCT()

	const TCHAR*	TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceMemoryBufferTemplate.ush");

	const FName NAME_SetNumElements("SetNumElements");
	const FName NAME_GetNumElements("GetNumElements");

	const FName NAME_ClearBufferInt("ClearBufferInt");
	const FName NAME_ClearBufferFloat("ClearBufferFloat");

	template<typename T>
	T AlignMemoryBuffer(T Size)
	{
		return Size + T(4);
	}

	struct FNDIInstanceData_GameThread
	{
		bool				bNeedsGpuSync = true;		// We have a pending writes on the CPU that can be sent to the GPU
		bool				bUsedWithCpu = false;		// Only true if we ever read / write to the buffer on the CPU, clearing for example does not qualify
		bool				bUsedWithGpu = false;		// Are we used on the GPU
		FRWLock				CpuBufferGuard;
		TArray<uint32>		CpuBuffer;
		int32				NumElements = 0;

		ENiagaraGpuSyncMode	GpuSyncMode = ENiagaraGpuSyncMode::None;
		TOptional<uint32>	GpuPendingClear;

		void UpdateCpuBuffer()
		{
			if (bUsedWithCpu)
			{
				const int32 AlignedNumElements = AlignMemoryBuffer(NumElements);
				if (AlignedNumElements != CpuBuffer.Num())
				{
					CpuBuffer.SetNumZeroed(AlignedNumElements);
				}
			}
		}
	};

	struct FGameToRenderData
	{
		explicit FGameToRenderData(FNDIInstanceData_GameThread& InstanceData)
			: NumElements(InstanceData.NumElements)
			, GpuSyncMode(InstanceData.GpuSyncMode)
		{
			if (InstanceData.bNeedsGpuSync)
			{
				if ( FNiagaraUtilities::ShouldSyncCpuToGpu(GpuSyncMode) )
				{
					InitialCpuBufferData = InstanceData.CpuBuffer;
				}
				GpuPendingClear = InstanceData.GpuPendingClear;
				InstanceData.GpuPendingClear.Reset();
			}
		}

		explicit FGameToRenderData(TConstArrayView<uint32> DataToSend)
		{
			NumElements = DataToSend.Num();
			InitialCpuBufferData = DataToSend;
		}

		int32				NumElements = 0;
		ENiagaraGpuSyncMode	GpuSyncMode = ENiagaraGpuSyncMode::None;
		TOptional<uint32>	GpuPendingClear;
		TArray<uint32>		InitialCpuBufferData;
	};

	struct FNDIInstanceData_RenderThread
	{
		int32							NumElements = 0;
		TRefCountPtr<FRDGPooledBuffer>	PooledBuffer;
		FRDGBufferRef					RDGTransientBuffer = nullptr;
		FRDGBufferUAVRef				RDGTransientBufferUAV = nullptr;

		ENiagaraGpuSyncMode				GpuSyncMode = ENiagaraGpuSyncMode::None;
		TArray<uint32>					InitialCpuBufferData;
		TOptional<uint32>				GpuPendingClear;

		~FNDIInstanceData_RenderThread()
		{
			ReleaseData();
		}

		void UpdateData(FGameToRenderData& GameToRenderData)
		{
			ReleaseData();

			NumElements				= GameToRenderData.NumElements;
			GpuSyncMode				= GameToRenderData.GpuSyncMode;
			InitialCpuBufferData	= MoveTemp(GameToRenderData.InitialCpuBufferData);
			GpuPendingClear			= GameToRenderData.GpuPendingClear;
		}

		void ReleaseData()
		{
			NumElements = 0;
			PooledBuffer.SafeRelease();
		}
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		FNDIProxy(UObject* Owner)
			: WeakOwner(Owner)
		{
		}

		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { checkNoEntry(); }
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

		virtual void PreStage(const FNDIGpuComputePreStageContext& Context)
		{
			const FNiagaraSystemInstanceID SystemInstanceID = Context.GetSystemInstanceID();
			FNDIInstanceData_RenderThread& InstanceData = PerInstanceData_RenderThread.FindChecked(SystemInstanceID);
			const FNiagaraSimStageData& SimStageData = Context.GetSimStageData();

			if (SimStageData.bFirstStage)
			{
				FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

				const int32 AlignedNumElements = AlignMemoryBuffer(InstanceData.NumElements);
				FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateByteAddressDesc(AlignedNumElements << 2);
				BufferDesc.Usage |= BUF_SourceCopy;
				ResizeBufferIfNeeded(GraphBuilder, InstanceData.PooledBuffer, BufferDesc, TEXT("NiagaraMemoryBuffer"));

				if (InstanceData.InitialCpuBufferData.Num() > 0)
				{
					if (ensure(InstanceData.InitialCpuBufferData.Num() == AlignedNumElements))
					{
						const void* InitialData = InstanceData.InitialCpuBufferData.GetData();
						const uint64 InitialDataSize = InstanceData.InitialCpuBufferData.Num() * InstanceData.InitialCpuBufferData.GetTypeSize();
						GraphBuilder.QueueBufferUpload(
							GraphBuilder.RegisterExternalBuffer(InstanceData.PooledBuffer),
							InitialData,
							InitialDataSize,
							[DataBuffer=MoveTemp(InstanceData.InitialCpuBufferData)](const void*) { }
						);
					}
					else
					{
						InstanceData.InitialCpuBufferData.Empty();
					}
				}
				else if (InstanceData.GpuPendingClear.IsSet())
				{
					AddClearUAVPass(Context.GetGraphBuilder(), GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalBuffer(InstanceData.PooledBuffer)), InstanceData.GpuPendingClear.GetValue());
				}
				InstanceData.GpuPendingClear.Reset();
			}
		}

		virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
		{
			if (!Context.IsFinalPostSimulate())
			{
				return;
			}

			const FNiagaraSystemInstanceID SystemInstanceID = Context.GetSystemInstanceID();
			FNDIInstanceData_RenderThread& InstanceData = PerInstanceData_RenderThread.FindChecked(SystemInstanceID);

			// Should we send the buffer back over
			if (FNiagaraUtilities::ShouldSyncGpuToCpu(InstanceData.GpuSyncMode) && InstanceData.NumElements > 0 && InstanceData.RDGTransientBuffer)
			{
				//-TODO: Currently we don't know if a buffer was written to, we can extract that from the used functions to reduce readback pressure
				FNiagaraGpuReadbackManager* ReadbackManager = Context.GetComputeDispatchInterface().GetGpuReadbackManager();

				const int32 AlignedNumElements = AlignMemoryBuffer(InstanceData.NumElements);
				ReadbackManager->EnqueueReadback(
					Context.GetGraphBuilder(),
					InstanceData.RDGTransientBuffer,
					0,
					AlignedNumElements * sizeof(uint32),
					[SystemInstanceID, WeakOwnerObject=WeakOwner, Proxy=this, ExpectedElements=AlignedNumElements](TConstArrayView<TPair<void*, uint32>> ReadbackData)
					{
						if (ReadbackData.Num() != 1 || (ExpectedElements << 2 != ReadbackData[0].Value))
						{
							// Readback error
							return;
						}

						TArray<uint32> BufferData_RT;
						BufferData_RT.AddUninitialized(ExpectedElements);
						FMemory::Memcpy(BufferData_RT.GetData(), ReadbackData[0].Key, ExpectedElements * sizeof(uint32));

						FNiagaraWorldManager::EnqueueGlobalDeferredCallback(
							[SystemInstanceID, BufferData=MoveTemp(BufferData_RT), WeakOwnerObject, Proxy]() mutable
							{
								// FNiagaraDataInterfaceProxy do not outlive UNiagaraDataInterface so if our Object is valid so is the proxy
								// Equally because we do not share instance IDs (monotonically increasing number) we won't ever stomp something that has 'gone away'
								if (!WeakOwnerObject.IsValid())
								{
									return;
								}
								if (FNDIInstanceData_GameThread* InstanceData_GT = Proxy->PerInstanceData_GameThread.FindRef(SystemInstanceID))
								{
									FRWScopeLock ScopeLock(InstanceData_GT->CpuBufferGuard, SLT_Write);
									if (InstanceData_GT->CpuBuffer.Num() == BufferData.Num())
									{
										Swap(InstanceData_GT->CpuBuffer, BufferData);
									}
								}
							}
						);
					}
				);
			}

			// Clear out transient pointers we cache for graph usage
			InstanceData.RDGTransientBuffer = nullptr;
			InstanceData.RDGTransientBufferUAV = nullptr;
		}

		TWeakObjectPtr<UObject> WeakOwner;
		TMap<FNiagaraSystemInstanceID, FNDIInstanceData_RenderThread>	PerInstanceData_RenderThread;
		TMap<FNiagaraSystemInstanceID, FNDIInstanceData_GameThread*>	PerInstanceData_GameThread;
	};

	//////////////////////////////////////////////////////////////////////////////////////////
	// VM functions
	void VMSetNumElements(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<int32>		InNumElements(Context);
		const uint64 MaxViewDimensionForTypedBuffer = GRHIGlobals.MaxViewDimensionForTypedBuffer;
		const int32 MaxElements = (int32)FMath::Min<int64>(MaxViewDimensionForTypedBuffer, TNumericLimits<int32>::Max());

		FRWScopeLock ScopeLock(InstanceData->CpuBufferGuard, SLT_Write);
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 NumElements = FMath::Clamp(InNumElements.GetAndAdvance(), 0, MaxElements);
			InstanceData->bNeedsGpuSync |= InstanceData->NumElements != NumElements;
			InstanceData->NumElements = NumElements;
		}
		InstanceData->UpdateCpuBuffer();
	}

	void VMGetNumElements(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIOutputParam<int32>		OutNumElements(Context);

		FRWScopeLock ScopeLock(InstanceData->CpuBufferGuard, SLT_ReadOnly);
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutNumElements.SetAndAdvance(InstanceData->NumElements);
		}
	}

	template<typename TType>
	void VMLoadData(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<int32>		InElementOffset(Context);
		FNDIOutputParam<bool>		OutSuccess(Context);
		FNDIOutputParam<TType>		OutValue(Context);

		FRWScopeLock ScopeLock(InstanceData->CpuBufferGuard, SLT_ReadOnly);
		const int32 TypeSize = FMath::DivideAndRoundUp(sizeof(TType), sizeof(uint32));
		const int32 ValidEnd = InstanceData->NumElements - TypeSize;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Offset			= InElementOffset.GetAndAdvance();
			const int32 ClampedOffset	= FMath::Clamp(Offset, 0, InstanceData->NumElements);
			TType Value;
			FMemory::Memcpy(&Value, &InstanceData->CpuBuffer[ClampedOffset], sizeof(TType));

			OutSuccess.SetAndAdvance(Offset >= 0 && Offset <= ValidEnd);
			OutValue.SetAndAdvance(Value);
		}
	}

	template<typename TType>
	void VMStoreData(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<bool>		InExecute(Context);
		FNDIInputParam<int32>		InElementOffset(Context);
		FNDIInputParam<TType>		InValue(Context);
		FNDIOutputParam<bool>		OutSuccess(Context);

		FRWScopeLock ScopeLock(InstanceData->CpuBufferGuard, SLT_Write);
		InstanceData->bNeedsGpuSync = true;

		const int32 TypeSize = FMath::DivideAndRoundUp(sizeof(TType), sizeof(uint32));
		const int32 ValidEnd = InstanceData->NumElements - TypeSize;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool bExecute			= InExecute.GetAndAdvance();
			const int32 Offset			= InElementOffset.GetAndAdvance();
			const int32 ClampedOffset	= FMath::Clamp(Offset, 0, InstanceData->NumElements);
			const TType Value			= InValue.GetAndAdvance();
			const bool bSuccess			= bExecute && Offset >= 0 && Offset <= ValidEnd;
			if (bSuccess)
			{
				FMemory::Memcpy(&InstanceData->CpuBuffer[ClampedOffset], &Value, sizeof(TType));
			}
			OutSuccess.SetAndAdvance(bSuccess);
		}
	}

	template<typename TType>
	void VMClearBuffer(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<bool>	InExecute(Context);
		FNDIInputParam<TType>	InClearValue(Context);

		FRWScopeLock ScopeLock(InstanceData->CpuBufferGuard, SLT_Write);
		InstanceData->bNeedsGpuSync = true;

		bool bExecuteClear = false;
		uint32 ClearValue = 0;
		for (int32 i=0; i < Context.GetNumInstances(); ++i)
		{
			const bool bExecute = InExecute.GetAndAdvance();
			const TType NewClearValue = InClearValue.GetAndAdvance();
			bExecuteClear |= bExecute;
			ClearValue = bExecute ? reinterpret_cast<const uint32&>(NewClearValue) : ClearValue;
		}

		if (bExecuteClear)
		{
			if (InstanceData->bUsedWithCpu)
			{
				for (uint32& Value : InstanceData->CpuBuffer)
				{
					Value = ClearValue;
				}
			}
			InstanceData->GpuPendingClear = ClearValue;
		}
	}

	struct FAtomicAdd
	{
		static int32 InterlockedOp(int32* Location, int32 Value) { return FPlatformAtomics::InterlockedAdd(Location, Value); }
		static int32 RawOp(int32 PreviousValue, int32 Value) { return PreviousValue += Value; }
	};

	struct FAtomicAnd
	{
		static int32 InterlockedOp(int32* Location, int32 Value) { return FPlatformAtomics::InterlockedAnd(Location, Value); }
		static int32 RawOp(int32 PreviousValue, int32 Value) { return PreviousValue &= Value; }
	};

	struct FAtomicMin
	{
		static int32 InterlockedOp(int32* Location, int32 Value)
		{
			int32 PreviousValue = *Location;
			while (PreviousValue > Value)
			{
				PreviousValue = FPlatformAtomics::InterlockedCompareExchange(Location, Value, PreviousValue);
			}
			return PreviousValue;
		}
		static int32 RawOp(int32 PreviousValue, int32 Value) { return FMath::Min(PreviousValue, Value); }
	};

	struct FAtomicMax
	{
		static int32 InterlockedOp(int32* Location, int32 Value)
		{
			int32 PreviousValue = *Location;
			while (PreviousValue < Value)
			{
				PreviousValue = FPlatformAtomics::InterlockedCompareExchange(Location, Value, PreviousValue);
			}
			return PreviousValue;
		}
		static int32 RawOp(int32 PreviousValue, int32 Value) { return FMath::Max(PreviousValue, Value); }
	};

	struct FAtomicOr
	{
		static int32 InterlockedOp(int32* Location, int32 Value) { return FPlatformAtomics::InterlockedOr(Location, Value); }
		static int32 RawOp(int32 PreviousValue, int32 Value) { return PreviousValue |= Value; }
	};

	struct FAtomicXor
	{
		static int32 InterlockedOp(int32* Location, int32 Value) { return FPlatformAtomics::InterlockedXor(Location, Value); }
		static int32 RawOp(int32 PreviousValue, int32 Value) { return PreviousValue ^= Value; }
	};

	template<typename TType, typename TOperation>
	void VMAtomicOp(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<bool>	InExecute(Context);
		FNDIInputParam<int32>	InOffset(Context);
		FNDIInputParam<TType>	InValue(Context);

		FNDIOutputParam<TType>	OutPreviousValue(Context);
		FNDIOutputParam<TType>	OutCurrentValue(Context);

		FRWScopeLock ScopeLock(InstanceData->CpuBufferGuard, SLT_ReadOnly);
		InstanceData->bNeedsGpuSync = true;

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool bExecute = InExecute.GetAndAdvance();
			const int32 Offset = InOffset.GetAndAdvance();
			const TType Value = InValue.GetAndAdvance();
			TType PreviousValue = TType{};
			TType CurrentValue = TType{};
			if (bExecute)
			{
				const int32 ClampedOffset = FMath::Clamp(Offset, 0, InstanceData->NumElements);
				PreviousValue = TOperation::InterlockedOp(reinterpret_cast<int32*>(&InstanceData->CpuBuffer[ClampedOffset]), Value);
				CurrentValue = TOperation::RawOp(PreviousValue, Value);
			}
			OutPreviousValue.SetAndAdvance(PreviousValue);
			OutCurrentValue.SetAndAdvance(CurrentValue);
		}
	}

	template<typename TType>
	void VMAtomicCAE(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<bool>	InExecute(Context);
		FNDIInputParam<int32>	InOffset(Context);
		FNDIInputParam<TType>	InValue(Context);
		FNDIInputParam<TType>	InComperand(Context);

		FNDIOutputParam<TType>	OutPreviousValue(Context);
		FNDIOutputParam<TType>	OutCurrentValue(Context);

		FRWScopeLock ScopeLock(InstanceData->CpuBufferGuard, SLT_ReadOnly);
		InstanceData->bNeedsGpuSync = true;

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool bExecute = InExecute.GetAndAdvance();
			const int32 Offset = InOffset.GetAndAdvance();
			const TType Value = InValue.GetAndAdvance();
			const TType Comperand = InComperand.GetAndAdvance();
			TType PreviousValue = TType{};
			TType CurrentValue = TType{};
			if (bExecute)
			{
				const int32 ClampedOffset = FMath::Clamp(Offset, 0, InstanceData->NumElements);

				PreviousValue = FPlatformAtomics::InterlockedCompareExchange(reinterpret_cast<int32*>(&InstanceData->CpuBuffer[ClampedOffset]), Value, Comperand);
				CurrentValue = PreviousValue == Comperand ? Value : PreviousValue;
			}
			OutPreviousValue.SetAndAdvance(PreviousValue);
			OutCurrentValue.SetAndAdvance(CurrentValue);
		}
	}

	//////////////////////////////////////////////////////////////////////////////////////////
	// Supported load / store types
	struct FLoadStoreTypes
	{
		FName					LoadFunctionName;
		FName					StoreFunctionName;
		FNiagaraTypeDefinition	TypeDef;
		int32					NumTypeDefSigCopies = 0;

		FVMExternalFunction		VMLoadFunction;
		FVMExternalFunction		VMStoreFunction;
	};

	static TConstArrayView<FLoadStoreTypes> GetLoadStoreTypes()
	{
		static const FLoadStoreTypes LoadStoreTypes[] =
		{
			{"LoadFloat",	"StoreFloat",	FNiagaraTypeDefinition::GetFloatDef(),	1, FVMExternalFunction::CreateStatic(VMLoadData<float>),		FVMExternalFunction::CreateStatic(VMStoreData<float>)		},
			{"LoadFloat2",	"StoreFloat2",	FNiagaraTypeDefinition::GetVec2Def(),	1, FVMExternalFunction::CreateStatic(VMLoadData<FVector2f>),	FVMExternalFunction::CreateStatic(VMStoreData<FVector2f>)	},
			{"LoadFloat3",	"StoreFloat3",	FNiagaraTypeDefinition::GetVec3Def(),	1, FVMExternalFunction::CreateStatic(VMLoadData<FVector3f>),	FVMExternalFunction::CreateStatic(VMStoreData<FVector3f>)	},
			{"LoadFloat4",	"StoreFloat4",	FNiagaraTypeDefinition::GetVec4Def(),	1, FVMExternalFunction::CreateStatic(VMLoadData<FVector4f>),	FVMExternalFunction::CreateStatic(VMStoreData<FVector4f>)	},
			{"LoadInt",		"StoreInt",		FNiagaraTypeDefinition::GetIntDef(),	1, FVMExternalFunction::CreateStatic(VMLoadData<int32>),		FVMExternalFunction::CreateStatic(VMStoreData<int32>)		},
			{"LoadInt2",	"StoreInt2",	FNiagaraTypeDefinition::GetIntDef(),	2, FVMExternalFunction::CreateStatic(VMLoadData<FIntVector2>),	FVMExternalFunction::CreateStatic(VMStoreData<FIntVector2>)	},
			{"LoadInt3",	"StoreInt3",	FNiagaraTypeDefinition::GetIntDef(),	3, FVMExternalFunction::CreateStatic(VMLoadData<FIntVector3>),	FVMExternalFunction::CreateStatic(VMStoreData<FIntVector3>)	},
			{"LoadInt4",	"StoreInt4",	FNiagaraTypeDefinition::GetIntDef(),	4, FVMExternalFunction::CreateStatic(VMLoadData<FIntVector4>),	FVMExternalFunction::CreateStatic(VMStoreData<FIntVector4>)	},
		};

		return MakeArrayView(LoadStoreTypes);
	};

	//////////////////////////////////////////////////////////////////////////////////////////
	// Supported interlocked ops types
	const FName NAME_AtomicAdd("AtomicAdd");
	const FName NAME_AtomicAnd("AtomicAnd");
	const FName NAME_AtomicCompareAndExchange("AtomicCompareAndExchange");
	const FName NAME_AtomicMax("AtomicMax");
	const FName NAME_AtomicMin("AtomicMin");
	const FName NAME_AtomicOr("AtomicOr");
	const FName NAME_AtomicXor("AtomicXor");

	static TConstArrayView<TPair<FName, FVMExternalFunction>> GetAtomicOps()
	{
		static const TArray<TPair<FName, FVMExternalFunction>> AtomicOps =
		{
			{NAME_AtomicAdd,				FVMExternalFunction::CreateStatic(VMAtomicOp<int32, FAtomicAdd>)},
			{NAME_AtomicAnd,				FVMExternalFunction::CreateStatic(VMAtomicOp<int32, FAtomicAnd>)},
			{NAME_AtomicCompareAndExchange,	FVMExternalFunction::CreateStatic(VMAtomicCAE<int32>)},
			{NAME_AtomicMax,				FVMExternalFunction::CreateStatic(VMAtomicOp<int32, FAtomicMax>)},
			{NAME_AtomicMin,				FVMExternalFunction::CreateStatic(VMAtomicOp<int32, FAtomicMin>)},
			{NAME_AtomicOr,					FVMExternalFunction::CreateStatic(VMAtomicOp<int32, FAtomicOr>)},
			{NAME_AtomicXor,				FVMExternalFunction::CreateStatic(VMAtomicOp<int32, FAtomicXor>)},
			{NAME_ClearBufferInt,			FVMExternalFunction::CreateStatic(VMClearBuffer<int32>)},
			{NAME_ClearBufferFloat,			FVMExternalFunction::CreateStatic(VMClearBuffer<float>)},
		};
		return AtomicOps;
	}

	//////////////////////////////////////////////////////////////////////////////////////////
	// Cache Helper
	int32 FindOrAddCacheData(UNDIMemoryBufferSimCacheData* CacheData, TConstArrayView<uint32> CurrentData)
	{
		if (CurrentData.Num() == 0)
		{
			return INDEX_NONE;
		}

		for (const FNDIMemoryBufferSimCacheDataFrame& ExistingFrame : CacheData->FrameData)
		{
			if (ExistingFrame.CpuBufferSize == CurrentData.Num())
			{
				if (FMemory::Memcmp(CacheData->BufferData.GetData() + ExistingFrame.CpuDataOffset, CurrentData.GetData(), CurrentData.Num() * sizeof(uint32)) == 0)
				{
					return ExistingFrame.CpuDataOffset;
				}
			}
			if (ExistingFrame.GpuBufferSize == CurrentData.Num())
			{
				if (FMemory::Memcmp(CacheData->BufferData.GetData() + ExistingFrame.GpuDataOffset, CurrentData.GetData(), CurrentData.Num() * sizeof(uint32)) == 0)
				{
					return ExistingFrame.GpuDataOffset;
				}
			}
		}
		const int32 NewOffset = CacheData->BufferData.AddUninitialized(CurrentData.Num());
		FMemory::Memcpy(CacheData->BufferData.GetData() + NewOffset, CurrentData.GetData(), CurrentData.Num() * sizeof(uint32));
		return NewOffset;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceMemoryBuffer::UNiagaraDataInterfaceMemoryBuffer(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDIMemoryBufferLocal;

	Proxy.Reset(new FNDIProxy(this));
}

void UNiagaraDataInterfaceMemoryBuffer::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceMemoryBuffer::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIMemoryBufferLocal;

	FNiagaraFunctionSignature ImmutableSignature;
	ImmutableSignature.bMemberFunction = true;
	ImmutableSignature.bRequiresContext = false;
	ImmutableSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Byte Buffer")));

	FNiagaraFunctionSignature MutableSignature = ImmutableSignature;
	MutableSignature.bRequiresExecPin = true;

	////////////////////////////////////////////////////////////////////////////
	// Helper Functions
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(MutableSignature);
		Signature.Name = NAME_SetNumElements;
		Signature.ModuleUsageBitmask = ENiagaraScriptUsageMask::System | ENiagaraScriptUsageMask::Emitter;
		Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumElements"));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ImmutableSignature);
		Signature.Name = NAME_GetNumElements;
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumElements"));
	}

	////////////////////////////////////////////////////////////////////////////
	// Load Functions
	{
		FNiagaraFunctionSignature LoadSignature = ImmutableSignature;
		LoadSignature.bReadFunction = true;
		LoadSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("ElementOffset"));
		LoadSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("bValid"));

		for (const FLoadStoreTypes& SupportedType : GetLoadStoreTypes())
		{
			FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(LoadSignature);
			Signature.Name = SupportedType.LoadFunctionName;
			for ( int32 i=0; i < SupportedType.NumTypeDefSigCopies; ++i )
			{
				Signature.Outputs.Emplace(
					SupportedType.TypeDef,
					SupportedType.NumTypeDefSigCopies == 1 ? TEXT("Value") : *FString::Printf(TEXT("Value%d"), i)
				);
			}
		}
	}

	////////////////////////////////////////////////////////////////////////////
	// Store Functions
	{
		FNiagaraFunctionSignature StoreSignature = MutableSignature;
		StoreSignature.bWriteFunction = true;
		StoreSignature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		StoreSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("ElementOffset"));
		StoreSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("bSuccess"));

		for (const FLoadStoreTypes& SupportedType : GetLoadStoreTypes())
		{
			FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(StoreSignature);
			Signature.Name = SupportedType.StoreFunctionName;
			for (int32 i = 0; i < SupportedType.NumTypeDefSigCopies; ++i)
			{
				Signature.Inputs.Emplace(
					SupportedType.TypeDef,
					SupportedType.NumTypeDefSigCopies == 1 ? TEXT("Value") : *FString::Printf(TEXT("Value%d"), i)
				);
			}
		}
	}

	////////////////////////////////////////////////////////////////////////////
	// Atomic Operations
	{
		FNiagaraFunctionSignature AtomicSignature = MutableSignature;
		AtomicSignature.bWriteFunction = true;
		AtomicSignature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		AtomicSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("ElementOffset"));
		AtomicSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value"));
		AtomicSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("PreviousValue"));
		AtomicSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("CurrentValue"));

		OutFunctions.Add_GetRef(AtomicSignature).Name = NAME_AtomicAdd;
		OutFunctions.Add_GetRef(AtomicSignature).Name = NAME_AtomicAnd;
		OutFunctions.Add_GetRef(AtomicSignature).Name = NAME_AtomicMax;
		OutFunctions.Add_GetRef(AtomicSignature).Name = NAME_AtomicMin;
		OutFunctions.Add_GetRef(AtomicSignature).Name = NAME_AtomicOr;
		OutFunctions.Add_GetRef(AtomicSignature).Name = NAME_AtomicXor;

		FNiagaraFunctionSignature& CASSignature = OutFunctions.Add_GetRef(AtomicSignature);
		CASSignature.Name = NAME_AtomicCompareAndExchange;
		CASSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Comperand"));
	}

	////////////////////////////////////////////////////////////////////////////
	// Buffer Clear Operations
	{
		FNiagaraFunctionSignature ClearSignature = MutableSignature;
		ClearSignature.ModuleUsageBitmask = ENiagaraScriptUsageMask::System | ENiagaraScriptUsageMask::Emitter;
		ClearSignature.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		{
			FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ClearSignature);
			Signature.Name = NAME_ClearBufferInt;
			Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("ClearValue"));
		}
		{
			FNiagaraFunctionSignature& Signature = OutFunctions.Add_GetRef(ClearSignature);
			Signature.Name = NAME_ClearBufferFloat;
			Signature.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("ClearValue"));
		}
	}
}
#endif //WITH_EDITORONLY_DATA

void UNiagaraDataInterfaceMemoryBuffer::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIMemoryBufferLocal;

	static const TPair<FName, FVMExternalFunction> StaticBindings[] =
	{
		{NAME_SetNumElements, FVMExternalFunction::CreateStatic(VMSetNumElements)},
		{NAME_GetNumElements, FVMExternalFunction::CreateStatic(VMGetNumElements)},
	};

	for (const auto& StaticBinding : StaticBindings)
	{
		if (StaticBinding.Key == BindingInfo.Name)
		{
			OutFunc = StaticBinding.Value;
			return;
		}
	}

	for (const FLoadStoreTypes& SupportedType : GetLoadStoreTypes())
	{
		if (BindingInfo.Name == SupportedType.LoadFunctionName)
		{
			OutFunc = SupportedType.VMLoadFunction;
			return;
		}
		if (BindingInfo.Name == SupportedType.StoreFunctionName)
		{
			OutFunc = SupportedType.VMStoreFunction;
			return;
		}
	}

	for (const TPair<FName, FVMExternalFunction>& AtomicOp : GetAtomicOps())
	{
		if (BindingInfo.Name == AtomicOp.Key)
		{
			OutFunc = AtomicOp.Value;
			return;
		}
	}
}

int32 UNiagaraDataInterfaceMemoryBuffer::PerInstanceDataSize() const
{
	using namespace NDIMemoryBufferLocal;
	return sizeof(FNDIInstanceData_GameThread);
}

bool UNiagaraDataInterfaceMemoryBuffer::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIMemoryBufferLocal;

	FNDIInstanceData_GameThread* InstanceData	= new(PerInstanceData) FNDIInstanceData_GameThread();
	InstanceData->NumElements					= DefaultNumElements;
	InstanceData->GpuSyncMode					= GpuSyncMode;

	if (IsUsedWithCPUScript())
	{
		FNiagaraDataInterfaceUtilities::ForEachVMFunction(
			this,
			SystemInstance,
			[InstanceData](const UNiagaraScript*, const FVMExternalFunctionBindingInfo& BindingInfo)
			{
				for (const FLoadStoreTypes& LoadStoreType : GetLoadStoreTypes())
				{
					if (BindingInfo.Name == LoadStoreType.LoadFunctionName || BindingInfo.Name == LoadStoreType.StoreFunctionName)
					{
						InstanceData->bUsedWithCpu = true;
						return false;
					}
				}
				for (const TPair<FName, FVMExternalFunction>& AtomicOp : GetAtomicOps())
				{
					if (AtomicOp.Key == BindingInfo.Name)
					{
						InstanceData->bUsedWithCpu = true;
						return false;
					}
				}
				return true;
			}
		);
	}

	InstanceData->UpdateCpuBuffer();

	if (IsUsedWithGPUScript())
	{
		InstanceData->bUsedWithGpu = true;

		FNDIProxy* Proxy_GT = GetProxyAs<FNDIProxy>();
		Proxy_GT->PerInstanceData_GameThread.Add(SystemInstance->GetId(), InstanceData);
	}

	return true;
}

void UNiagaraDataInterfaceMemoryBuffer::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIMemoryBufferLocal;

	FNDIInstanceData_GameThread* InstanceData = reinterpret_cast<FNDIInstanceData_GameThread*>(PerInstanceData);
	if (InstanceData->bUsedWithGpu)
	{
		FNDIProxy* Proxy_GT = GetProxyAs<FNDIProxy>();
		Proxy_GT->PerInstanceData_GameThread.Remove(SystemInstance->GetId());

		ENQUEUE_RENDER_COMMAND(FNDIMemoryBuffer_RemoveProxy)
		(
			[RT_Proxy=Proxy_GT, InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
			{
				RT_Proxy->PerInstanceData_RenderThread.Remove(InstanceID);
			}
		);
	}

	InstanceData->~FNDIInstanceData_GameThread();
}

bool UNiagaraDataInterfaceMemoryBuffer::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIMemoryBufferLocal;

	FNDIInstanceData_GameThread* InstanceData = reinterpret_cast<FNDIInstanceData_GameThread*>(PerInstanceData);

	if (InstanceData->bUsedWithGpu && InstanceData->bNeedsGpuSync)
	{
		ENQUEUE_RENDER_COMMAND(FNDIMemoryBuffer_UpdateData)
		(
			[RT_Proxy=GetProxyAs<FNDIProxy>(), RT_SystemInstanceID=SystemInstance->GetId(), RT_GameToRenderData=FGameToRenderData(*InstanceData)](FRHICommandListImmediate& RHICmdList) mutable
			{
				FNDIInstanceData_RenderThread& InstanceData_RT = RT_Proxy->PerInstanceData_RenderThread.FindOrAdd(RT_SystemInstanceID);
				InstanceData_RT.UpdateData(RT_GameToRenderData);
			}
		);
		InstanceData->bNeedsGpuSync = false;
	}

	return false;
}

bool UNiagaraDataInterfaceMemoryBuffer::Equals(const UNiagaraDataInterface* Other) const
{
	const UNiagaraDataInterfaceMemoryBuffer* OtherTyped = CastChecked<const UNiagaraDataInterfaceMemoryBuffer>(Other);
	return
		Super::Equals(Other) &&
		OtherTyped->DefaultNumElements == DefaultNumElements &&
		OtherTyped->GpuSyncMode == GpuSyncMode;
}

bool UNiagaraDataInterfaceMemoryBuffer::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceMemoryBuffer* OtherTyped = CastChecked<UNiagaraDataInterfaceMemoryBuffer>(Destination);
	OtherTyped->DefaultNumElements			= DefaultNumElements;
	OtherTyped->GpuSyncMode					= GpuSyncMode;

	return true;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceMemoryBuffer::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace NDIMemoryBufferLocal;

	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceMemoryBuffer::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	using namespace NDIMemoryBufferLocal;

	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceMemoryBuffer::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIMemoryBufferLocal;

	static const TSet<FName> ValidGpuFunctions =
	{
		NAME_GetNumElements,
		NAME_AtomicAdd,
		NAME_AtomicAnd,
		NAME_AtomicCompareAndExchange,
		NAME_AtomicMax,
		NAME_AtomicMin,
		NAME_AtomicOr,
		NAME_AtomicXor,
	};

	return
		ValidGpuFunctions.Contains(FunctionInfo.DefinitionName) ||
		GetLoadStoreTypes().ContainsByPredicate(
			[&](const FLoadStoreTypes& SupportedType)
			{
				return
					SupportedType.LoadFunctionName == FunctionInfo.DefinitionName ||
					SupportedType.StoreFunctionName == FunctionInfo.DefinitionName;
			}
		);
}
#endif

void UNiagaraDataInterfaceMemoryBuffer::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NDIMemoryBufferLocal;

	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceMemoryBuffer::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIMemoryBufferLocal;

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	FNDIProxy& DataInterfaceProxy = Context.GetProxy<FNDIProxy>();
	FNDIInstanceData_RenderThread& InstanceData = DataInterfaceProxy.PerInstanceData_RenderThread.FindChecked(Context.GetSystemInstanceID());

	if (InstanceData.RDGTransientBuffer == nullptr)
	{
		InstanceData.RDGTransientBuffer = GraphBuilder.RegisterExternalBuffer(InstanceData.PooledBuffer);
		InstanceData.RDGTransientBufferUAV = GraphBuilder.CreateUAV(InstanceData.RDGTransientBuffer);
	}

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->NumElements	= InstanceData.NumElements;
	ShaderParameters->Buffer		= InstanceData.RDGTransientBufferUAV;
}

UObject* UNiagaraDataInterfaceMemoryBuffer::SimCacheBeginWrite(UObject* SimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	return OptionalPerInstanceData ? NewObject<UNDIMemoryBufferSimCacheData>(SimCache) : nullptr;
}

bool UNiagaraDataInterfaceMemoryBuffer::SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	using namespace NDIMemoryBufferLocal;

	check(OptionalPerInstanceData && StorageObject);
	UNDIMemoryBufferSimCacheData* CacheData = CastChecked<UNDIMemoryBufferSimCacheData>(StorageObject);
	if (CacheData->FrameData.Num() <= FrameIndex)
	{
		CacheData->FrameData.AddDefaulted(FrameIndex + 1 - CacheData->FrameData.Num());
	}

	const FNDIInstanceData_GameThread* InstanceData_GT = static_cast<const FNDIInstanceData_GameThread*>(OptionalPerInstanceData);
	if (InstanceData_GT->CpuBuffer.Num() > 0)
	{
		CacheData->FrameData[FrameIndex].CpuDataOffset = FindOrAddCacheData(CacheData, InstanceData_GT->CpuBuffer);
		CacheData->FrameData[FrameIndex].CpuBufferSize = InstanceData_GT->CpuBuffer.Num();
	}

	if (InstanceData_GT->bUsedWithGpu)
	{
		FNiagaraGpuComputeDispatchInterface* ComputeInterface = FNiagaraGpuComputeDispatchInterface::Get(SystemInstance->GetWorld());

		ENQUEUE_RENDER_COMMAND(NDIRenderTargetVolumeUpdate)(
			[RT_Proxy=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId(), CacheData, FrameIndex, ComputeInterface](FRHICommandListImmediate& RHICmdList)
			{
				const FNDIInstanceData_RenderThread* InstanceData_RT = RT_Proxy->PerInstanceData_RenderThread.Find(InstanceID);
				if (!InstanceData_RT || !InstanceData_RT->NumElements)
				{
					return;
				}

				const int32 BufferSize = InstanceData_RT->NumElements * sizeof(uint32);

				FNiagaraGpuReadbackManager* ReadbackManager = ComputeInterface->GetGpuReadbackManager();
				FRDGBuilder GraphBuilder(RHICmdList);
				ReadbackManager->EnqueueReadback(
					GraphBuilder,
					GraphBuilder.RegisterExternalBuffer(InstanceData_RT->PooledBuffer), 0, BufferSize,
					[CacheData, FrameIndex](TConstArrayView<TPair<void*, uint32>> ReadbackData)
					{
						const int32 BufferNumElements = ReadbackData[0].Value / sizeof(uint32);
						const uint32* BufferData = static_cast<const uint32*>(ReadbackData[0].Key);
						CacheData->FrameData[FrameIndex].GpuDataOffset = FindOrAddCacheData(CacheData, MakeArrayView<const uint32>(BufferData, BufferNumElements));
						CacheData->FrameData[FrameIndex].GpuBufferSize = BufferNumElements;
					}
				);
				GraphBuilder.Execute();
				ReadbackManager->WaitCompletion(RHICmdList);
			}
		);

		FlushRenderingCommands();
	}

	return true;
}

bool UNiagaraDataInterfaceMemoryBuffer::SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext)
{
	using namespace NDIMemoryBufferLocal;

	UNDIMemoryBufferSimCacheData* CacheData = ReadContext.GetStorageObject<UNDIMemoryBufferSimCacheData>();

	const int FrameIndex = ReadContext.GetFrameIndexA();
	if (!CacheData->FrameData.IsValidIndex(FrameIndex))
	{
		return false;
	}

	const FNDIMemoryBufferSimCacheDataFrame& FrameData = CacheData->FrameData[FrameIndex];

	FNDIInstanceData_GameThread* InstanceData_GT = ReadContext.GetPerInstanceData<FNDIInstanceData_GameThread>();
	if (FrameData.CpuBufferSize > 0 && InstanceData_GT->bUsedWithCpu)
	{		
		InstanceData_GT->NumElements = FrameData.CpuBufferSize;
		InstanceData_GT->UpdateCpuBuffer();
		FMemory::Memcpy(InstanceData_GT->CpuBuffer.GetData(), &CacheData->BufferData[FrameData.CpuDataOffset], FrameData.CpuBufferSize * sizeof(uint32));
	}

	if (FrameData.GpuBufferSize > 0 && InstanceData_GT->bUsedWithGpu)
	{
		FNiagaraSystemInstance* SystemInstance = ReadContext.GetSystemInstance();
		TConstArrayView<uint32> GpuData(&CacheData->BufferData[FrameData.GpuDataOffset], FrameData.GpuBufferSize);
		ENQUEUE_RENDER_COMMAND(FNDIMemoryBuffer_UpdateData)
		(
			[RT_Proxy=GetProxyAs<FNDIProxy>(), RT_SystemInstanceID=SystemInstance->GetId(), RT_GameToRenderData=FGameToRenderData(GpuData)](FRHICommandListImmediate& RHICmdList) mutable
			{
				FNDIInstanceData_RenderThread& InstanceData_RT = RT_Proxy->PerInstanceData_RenderThread.FindOrAdd(RT_SystemInstanceID);
				InstanceData_RT.UpdateData(RT_GameToRenderData);
			}
		);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
