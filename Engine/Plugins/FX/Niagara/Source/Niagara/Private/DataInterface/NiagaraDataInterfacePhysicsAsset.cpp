// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfacePhysicsAsset.h"
#include "Animation/SkeletalMeshActor.h"
#include "Components/SkeletalMeshComponent.h"
#include "AnimationRuntime.h"
#include "Engine/SkeletalMesh.h"
#include "NiagaraRenderer.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraSimStageData.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstance.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "RenderGraphBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfacePhysicsAsset)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfacePhysicsAsset"
DEFINE_LOG_CATEGORY_STATIC(LogPhysicsAsset, Log, All);

namespace NDIPhysicsAssetLocal
{

BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
	SHADER_PARAMETER(FUintVector4,			ElementOffsets)
	SHADER_PARAMETER_SRV(Buffer<float4>,	WorldTransformBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>,	InverseTransformBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>,	ElementExtentBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint>,		PhysicsTypeBuffer)
	SHADER_PARAMETER(FVector3f,				BoxOrigin)
	SHADER_PARAMETER(FVector3f,				BoxExtent)
END_SHADER_PARAMETER_STRUCT()

static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfacePhysicsAsset.ush");

//------------------------------------------------------------------------------------------------------------

static const FName GetNumBoxesName(TEXT("GetNumBoxes"));
static const FName GetNumSpheresName(TEXT("GetNumSpheres"));
static const FName GetNumCapsulesName(TEXT("GetNumCapsules"));

//------------------------------------------------------------------------------------------------------------

static const FName GetClosestElementName(TEXT("GetClosestElement"));
static const FName GetElementPointName(TEXT("GetElementPoint"));
static const FName GetElementDistanceName(TEXT("GetElementDistance"));
static const FName GetClosestPointName(TEXT("GetClosestPoint"));
static const FName GetClosestDistanceName(TEXT("GetClosestDistance"));
static const FName GetRestDistanceName(TEXT("GetRestDistance"));
static const FName GetTexturePointName(TEXT("GetTexturePoint"));
static const FName GetProjectionPointName(TEXT("GetProjectionPoint"));

//------------------------------------------------------------------------------------------------------------

template<typename BufferType, EPixelFormat PixelFormat, uint32 ElementCount, uint32 BufferCount = 1>
void CreateInternalBuffer(FRHICommandListBase& RHICmdList, FReadBuffer& OutputBuffer)
{
	if (ElementCount > 0)
	{
		OutputBuffer.Initialize(RHICmdList, TEXT("FNDIPhysicsAssetBuffer"), sizeof(BufferType), ElementCount * BufferCount, PixelFormat, BUF_Static);
	}
}

template<typename BufferType, EPixelFormat PixelFormat, uint32 ElementCount, uint32 BufferCount = 1>
void UpdateInternalBuffer(FRHICommandListBase& RHICmdList, const TStaticArray<BufferType,ElementCount*BufferCount>& InputData, FReadBuffer& OutputBuffer)
{
	if (ElementCount > 0 && OutputBuffer.Buffer.IsValid())
	{
		const uint32 BufferBytes = sizeof(BufferType) * ElementCount * BufferCount;

		void* OutputData = RHICmdList.LockBuffer(OutputBuffer.Buffer, 0, BufferBytes, RLM_WriteOnly);

		FMemory::Memcpy(OutputData, InputData.GetData(), BufferBytes);
		RHICmdList.UnlockBuffer(OutputBuffer.Buffer);
	}
}

void FillCurrentTransforms(const FTransform& ElementTransform, uint32& ElementCount,
	TStaticArray<FVector4f,PHYSICS_ASSET_MAX_TRANSFORMS>& OutCurrentTransform, TStaticArray<FVector4f, PHYSICS_ASSET_MAX_TRANSFORMS>& OutCurrentInverse)
{
	const uint32 ElementOffset = 3 * ElementCount;

	// LWC_TODO: precision loss
	const FMatrix44f ElementMatrix = FMatrix44f(ElementTransform.ToMatrixWithScale());
	const FMatrix44f ElementInverse = FMatrix44f(ElementMatrix.Inverse());

	ElementMatrix.To3x4MatrixTranspose(&OutCurrentTransform[ElementOffset].X);
	ElementInverse.To3x4MatrixTranspose(&OutCurrentInverse[ElementOffset].X);
	++ElementCount;
}

void GetNumPrimitives(const TArray<TWeakObjectPtr<UPhysicsAsset>>& PhysicsAssets, const TArray<TWeakObjectPtr<USkeletalMeshComponent>>& SkeletalMeshs, uint32& NumBoxes, uint32& NumSpheres, uint32& NumCapsules)
{
	NumBoxes = 0;
	NumSpheres = 0;
	NumCapsules = 0;

	for (int32 ComponentIndex = 0; ComponentIndex < SkeletalMeshs.Num(); ++ComponentIndex)
	{
		TWeakObjectPtr<UPhysicsAsset> PhysicsAsset = PhysicsAssets[ComponentIndex];
		if (PhysicsAsset.IsValid() && PhysicsAsset.Get() != nullptr)
		{
			USkeletalMesh* SkelMesh = (SkeletalMeshs[ComponentIndex].Get() && SkeletalMeshs[ComponentIndex]->GetSkeletalMeshAsset()) ? ToRawPtr(SkeletalMeshs[ComponentIndex]->GetSkeletalMeshAsset()) : PhysicsAsset->GetPreviewMesh();
			if (!SkelMesh)
			{
				continue;
			}

			const FReferenceSkeleton& RefSkeleton = SkelMesh->GetRefSkeleton();
			if (RefSkeleton.GetNum() > 0)
			{
				for (const UBodySetup* BodySetup : PhysicsAsset->SkeletalBodySetups)
				{
					const FName BoneName = BodySetup->BoneName;
					const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
					if (BoneIndex != INDEX_NONE && BoneIndex < RefSkeleton.GetNum())
					{
						for (const FKBoxElem& BoxElem : BodySetup->AggGeom.BoxElems)
						{
							if (CollisionEnabledHasPhysics(BoxElem.GetCollisionEnabled()))
							{
								NumBoxes += 1;
							}
						}
						for (const FKSphereElem& SphereElem : BodySetup->AggGeom.SphereElems)
						{
							if (CollisionEnabledHasPhysics(SphereElem.GetCollisionEnabled()))
							{
								NumSpheres += 1;
							}
						}
						for (const FKSphylElem& CapsuleElem : BodySetup->AggGeom.SphylElems)
						{
							if (CollisionEnabledHasPhysics(CapsuleElem.GetCollisionEnabled()))
							{
								NumCapsules += 1;
							}
						}
					}
				}
			}
		}
	}
}

void CompactInternalArrays(FNDIPhysicsAssetArrays* OutAssetArrays)
{
	for (uint32 TransformIndex = 0; TransformIndex < PHYSICS_ASSET_MAX_TRANSFORMS; ++TransformIndex)
	{
		uint32 OffsetIndex = TransformIndex;
		OutAssetArrays->WorldTransform[OffsetIndex] = OutAssetArrays->CurrentTransform[TransformIndex];
		OutAssetArrays->InverseTransform[OffsetIndex] = OutAssetArrays->CurrentInverse[TransformIndex];

		OffsetIndex += PHYSICS_ASSET_MAX_TRANSFORMS;
		OutAssetArrays->WorldTransform[OffsetIndex] = OutAssetArrays->PreviousTransform[TransformIndex];
		OutAssetArrays->InverseTransform[OffsetIndex] = OutAssetArrays->PreviousInverse[TransformIndex];

		OffsetIndex += PHYSICS_ASSET_MAX_TRANSFORMS;
		OutAssetArrays->WorldTransform[OffsetIndex] = OutAssetArrays->RestTransform[TransformIndex];
		OutAssetArrays->InverseTransform[OffsetIndex] = OutAssetArrays->RestInverse[TransformIndex];
	}
}

// Due to large world coordinate we compute the relative world transform in double precision
inline FTransform ComputeRelativeTransform(const FTransform& TransformA, const FTransform& TransformB)
{
	const FMatrix44d TransformAMatrix = TransformA.ToMatrixWithScale();
	const FMatrix44d TransformBMatrix = TransformB.ToMatrixWithScale();

	const FMatrix44d RelativeMatrix  = TransformAMatrix * TransformBMatrix.Inverse();
	return FTransform(RelativeMatrix);
}

void CreateInternalArrays(const TArray<TWeakObjectPtr<UPhysicsAsset>>& PhysicsAssets, const TArray<TWeakObjectPtr<USkeletalMeshComponent>>& SkeletalMeshs,
	FNDIPhysicsAssetArrays* OutAssetArrays, const FTransform& InWorldTransform, const FTransform& LocalTransform)
{
	if (OutAssetArrays != nullptr)
	{
		OutAssetArrays->ElementOffsets.BoxOffset = 0;
		OutAssetArrays->ElementOffsets.SphereOffset = 0;
		OutAssetArrays->ElementOffsets.CapsuleOffset = 0;
		OutAssetArrays->ElementOffsets.NumElements = 0;

		uint32 NumBoxes = 0;
		uint32 NumSpheres = 0;
		uint32 NumCapsules = 0;

		GetNumPrimitives(PhysicsAssets, SkeletalMeshs, NumBoxes, NumSpheres, NumCapsules);
		
		if ((NumBoxes + NumSpheres + NumCapsules) < PHYSICS_ASSET_MAX_PRIMITIVES)
		{
			OutAssetArrays->ElementOffsets.BoxOffset = 0;
			OutAssetArrays->ElementOffsets.SphereOffset = OutAssetArrays->ElementOffsets.BoxOffset + NumBoxes;
			OutAssetArrays->ElementOffsets.CapsuleOffset = OutAssetArrays->ElementOffsets.SphereOffset + NumSpheres;
			OutAssetArrays->ElementOffsets.NumElements = OutAssetArrays->ElementOffsets.CapsuleOffset + NumCapsules;

			const uint32 NumTransforms = OutAssetArrays->ElementOffsets.NumElements * 3;
			const uint32 NumExtents = OutAssetArrays->ElementOffsets.NumElements;

			uint32 BoxCount = OutAssetArrays->ElementOffsets.BoxOffset;
			uint32 SphereCount = OutAssetArrays->ElementOffsets.SphereOffset;
			uint32 CapsuleCount = OutAssetArrays->ElementOffsets.CapsuleOffset;

			for (int32 ComponentIndex = 0; ComponentIndex < SkeletalMeshs.Num(); ++ComponentIndex)
			{
				TWeakObjectPtr<USkeletalMeshComponent> SkeletalMesh = SkeletalMeshs[ComponentIndex];
				const bool IsSkelMeshValid = SkeletalMesh.IsValid() && SkeletalMesh.Get() != nullptr;
				const FTransform WorldTransform = IsSkelMeshValid ? SkeletalMesh->GetComponentTransform() : InWorldTransform;

				TWeakObjectPtr<UPhysicsAsset> PhysicsAsset = PhysicsAssets[ComponentIndex];
				if (PhysicsAsset.IsValid() && PhysicsAsset.Get() != nullptr)
				{
					USkeletalMesh* SkelMesh = (SkeletalMeshs[ComponentIndex].Get() && SkeletalMeshs[ComponentIndex]->GetSkeletalMeshAsset()) ? ToRawPtr(SkeletalMeshs[ComponentIndex]->GetSkeletalMeshAsset()) : PhysicsAsset->GetPreviewMesh();
					if (!SkelMesh)
					{
						continue;
					}
					const FReferenceSkeleton& RefSkeleton = SkelMesh->GetRefSkeleton();
					TArray<FTransform> RestTransforms;
					FAnimationRuntime::FillUpComponentSpaceTransforms(RefSkeleton, RefSkeleton.GetRefBonePose(), RestTransforms);

					if (RefSkeleton.GetNum() > 0)
					{
						for (const UBodySetup* BodySetup : PhysicsAsset->SkeletalBodySetups)
						{
							const FName BoneName = BodySetup->BoneName;
							const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
							if (BoneIndex != INDEX_NONE && BoneIndex < RestTransforms.Num())
							{
								const FTransform RestTransform = RestTransforms[BoneIndex];
								const FTransform BoneTransform = IsSkelMeshValid ? ComputeRelativeTransform(SkeletalMesh->GetBoneTransform(BoneIndex), LocalTransform) : 
									RestTransform * ComputeRelativeTransform(WorldTransform, LocalTransform);

								for (const FKBoxElem& BoxElem : BodySetup->AggGeom.BoxElems)
								{
									if (CollisionEnabledHasPhysics(BoxElem.GetCollisionEnabled()))
									{
										const FTransform RestElement = FTransform(BoxElem.Rotation, BoxElem.Center) * RestTransform;
										FillCurrentTransforms(RestElement, BoxCount, OutAssetArrays->RestTransform, OutAssetArrays->RestInverse);
										--BoxCount;

										OutAssetArrays->PhysicsType[BoxCount] = (BoxElem.GetCollisionEnabled() == ECollisionEnabled::QueryAndPhysics);

										const FTransform ElementTransform = FTransform(BoxElem.Rotation, BoxElem.Center) * BoneTransform;
										OutAssetArrays->ElementExtent[BoxCount] = FVector4f(BoxElem.X, BoxElem.Y, BoxElem.Z, 0);
										FillCurrentTransforms(ElementTransform, BoxCount, OutAssetArrays->CurrentTransform, OutAssetArrays->CurrentInverse);
									}
								}

								for (const FKSphereElem& SphereElem : BodySetup->AggGeom.SphereElems)
								{
									if (CollisionEnabledHasPhysics(SphereElem.GetCollisionEnabled()))
									{
										const FTransform RestElement = FTransform(SphereElem.Center) * RestTransform;
										FillCurrentTransforms(RestElement, SphereCount, OutAssetArrays->RestTransform, OutAssetArrays->RestInverse);
										--SphereCount;

										OutAssetArrays->PhysicsType[SphereCount] = (SphereElem.GetCollisionEnabled() == ECollisionEnabled::QueryAndPhysics);

										const FTransform ElementTransform = FTransform(SphereElem.Center) * BoneTransform;
										OutAssetArrays->ElementExtent[SphereCount] = FVector4f(SphereElem.Radius, 0, 0, 0);
										FillCurrentTransforms(ElementTransform, SphereCount, OutAssetArrays->CurrentTransform, OutAssetArrays->CurrentInverse);
									}
								}

								for (const FKSphylElem& CapsuleElem : BodySetup->AggGeom.SphylElems)
								{
									if (CollisionEnabledHasPhysics(CapsuleElem.GetCollisionEnabled()))
									{
										const FTransform RestElement = FTransform(CapsuleElem.Rotation, CapsuleElem.Center) * RestTransform;
										FillCurrentTransforms(RestElement, CapsuleCount, OutAssetArrays->RestTransform, OutAssetArrays->RestInverse);
										--CapsuleCount;

										OutAssetArrays->PhysicsType[CapsuleCount] = (CapsuleElem.GetCollisionEnabled() == ECollisionEnabled::QueryAndPhysics);

										const FTransform ElementTransform = FTransform(CapsuleElem.Rotation, CapsuleElem.Center) * BoneTransform;
										OutAssetArrays->ElementExtent[CapsuleCount] = FVector4f(CapsuleElem.Radius, CapsuleElem.Length, 0, 0);
										FillCurrentTransforms(ElementTransform, CapsuleCount, OutAssetArrays->CurrentTransform, OutAssetArrays->CurrentInverse);
									}
								}
							}
						}
					}
				}
			}
			OutAssetArrays->PreviousTransform = OutAssetArrays->CurrentTransform;
			OutAssetArrays->PreviousInverse = OutAssetArrays->CurrentInverse;

			CompactInternalArrays(OutAssetArrays);
		}
		else
		{
			UE_LOGF(LogPhysicsAsset, Warning, "Number of physics asset primitives is higher than the niagara %d limit", PHYSICS_ASSET_MAX_PRIMITIVES);
		}
	}
}

void UpdateInternalArrays(const TArray<TWeakObjectPtr<UPhysicsAsset>>& PhysicsAssets, const TArray<TWeakObjectPtr<USkeletalMeshComponent>>& SkeletalMeshs,
	FNDIPhysicsAssetArrays* OutAssetArrays, const FTransform& InWorldTransform, const FTransform& LocalTransform)
{
	if (OutAssetArrays != nullptr && OutAssetArrays->ElementOffsets.NumElements < PHYSICS_ASSET_MAX_PRIMITIVES)
	{
		uint32 NumBoxes = 0;
		uint32 NumSpheres = 0;
		uint32 NumCapsules = 0;

		GetNumPrimitives(PhysicsAssets, SkeletalMeshs, NumBoxes, NumSpheres, NumCapsules);

		if ((NumBoxes + NumSpheres + NumCapsules) < PHYSICS_ASSET_MAX_PRIMITIVES)
		{
			if (((OutAssetArrays->ElementOffsets.SphereOffset - OutAssetArrays->ElementOffsets.BoxOffset) != NumBoxes) || 
				((OutAssetArrays->ElementOffsets.CapsuleOffset - OutAssetArrays->ElementOffsets.SphereOffset) != NumSpheres) ||
				((OutAssetArrays->ElementOffsets.NumElements - OutAssetArrays->ElementOffsets.CapsuleOffset) != NumCapsules))
			{
				CreateInternalArrays(PhysicsAssets, SkeletalMeshs, OutAssetArrays, InWorldTransform, LocalTransform);
			}

			OutAssetArrays->PreviousTransform = OutAssetArrays->CurrentTransform;
			OutAssetArrays->PreviousInverse = OutAssetArrays->CurrentInverse;

			uint32 BoxCount = OutAssetArrays->ElementOffsets.BoxOffset;
			uint32 SphereCount = OutAssetArrays->ElementOffsets.SphereOffset;
			uint32 CapsuleCount = OutAssetArrays->ElementOffsets.CapsuleOffset;

			for (int32 ComponentIndex = 0; ComponentIndex < SkeletalMeshs.Num(); ++ComponentIndex)
			{
				TWeakObjectPtr<USkeletalMeshComponent> SkeletalMesh = SkeletalMeshs[ComponentIndex];
				const bool IsSkelMeshValid = SkeletalMesh.IsValid() && SkeletalMesh.Get() != nullptr;
				const FTransform WorldTransform = IsSkelMeshValid ? SkeletalMesh->GetComponentTransform() : InWorldTransform;

				TWeakObjectPtr<UPhysicsAsset> PhysicsAsset = PhysicsAssets[ComponentIndex];
				if (PhysicsAsset.IsValid() && PhysicsAsset.Get() != nullptr)
				{
					USkeletalMesh* SkelMesh = (SkeletalMeshs[ComponentIndex].Get() && SkeletalMeshs[ComponentIndex]->GetSkeletalMeshAsset()) ? ToRawPtr(SkeletalMeshs[ComponentIndex]->GetSkeletalMeshAsset()) : PhysicsAsset->GetPreviewMesh();
					if (!SkelMesh)
					{
						continue;
					}
					const FReferenceSkeleton& RefSkeleton = SkelMesh->GetRefSkeleton();

					{
						TArray<FTransform> RestTransforms;
						FAnimationRuntime::FillUpComponentSpaceTransforms(RefSkeleton, RefSkeleton.GetRefBonePose(), RestTransforms);
						
						for (const UBodySetup* BodySetup : PhysicsAsset->SkeletalBodySetups)
						{
							const FName BoneName = BodySetup->BoneName;
							const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
							if (BoneIndex != INDEX_NONE && BoneIndex < RestTransforms.Num())
							{
								const FTransform BoneTransform = IsSkelMeshValid ? ComputeRelativeTransform(SkeletalMesh->GetBoneTransform(BoneIndex), LocalTransform) : 
									RestTransforms[BoneIndex] * ComputeRelativeTransform(WorldTransform, LocalTransform);

								for (const FKBoxElem& BoxElem : BodySetup->AggGeom.BoxElems)
								{
									if (CollisionEnabledHasPhysics(BoxElem.GetCollisionEnabled()))
									{
										const FTransform ElementTransform = FTransform(BoxElem.Rotation, BoxElem.Center) * BoneTransform;
										FillCurrentTransforms(ElementTransform, BoxCount, OutAssetArrays->CurrentTransform, OutAssetArrays->CurrentInverse);
									}
								}

								for (const FKSphereElem& SphereElem : BodySetup->AggGeom.SphereElems)
								{
									if (CollisionEnabledHasPhysics(SphereElem.GetCollisionEnabled()))
									{
										const FTransform ElementTransform = FTransform(SphereElem.Center) * BoneTransform;
										FillCurrentTransforms(ElementTransform, SphereCount, OutAssetArrays->CurrentTransform, OutAssetArrays->CurrentInverse);
									}
								}

								for (const FKSphylElem& CapsuleElem : BodySetup->AggGeom.SphylElems)
								{
									if (CollisionEnabledHasPhysics(CapsuleElem.GetCollisionEnabled()))
									{
										const FTransform ElementTransform = FTransform(CapsuleElem.Rotation, CapsuleElem.Center) * BoneTransform;
										FillCurrentTransforms(ElementTransform, CapsuleCount, OutAssetArrays->CurrentTransform, OutAssetArrays->CurrentInverse);
									}
								}
							}
						}
					}
				}
			}
		}
		CompactInternalArrays(OutAssetArrays);
	}
}

} //namespace NDIPhysicsAssetLocal

//------------------------------------------------------------------------------------------------------------

ETickingGroup ComputeTickingGroup(const TArray<TWeakObjectPtr<class USkeletalMeshComponent>> SkeletalMeshes)
{
	ETickingGroup TickingGroup = NiagaraFirstTickGroup;
	for (int32 ComponentIndex = 0; ComponentIndex < SkeletalMeshes.Num(); ++ComponentIndex)
	{
		if (SkeletalMeshes[ComponentIndex].Get() != nullptr)
		{
			const USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(SkeletalMeshes[ComponentIndex].Get());

			const ETickingGroup ComponentTickGroup = FMath::Max(Component->PrimaryComponentTick.TickGroup, Component->PrimaryComponentTick.EndTickGroup);
			const ETickingGroup PhysicsTickGroup = Component->bBlendPhysics ? FMath::Max(ComponentTickGroup, TG_EndPhysics) : ComponentTickGroup;
			const ETickingGroup ClampedTickGroup = FMath::Clamp(ETickingGroup(PhysicsTickGroup + 1), NiagaraFirstTickGroup, NiagaraLastTickGroup);

			TickingGroup = FMath::Max(TickingGroup, ClampedTickGroup);
		}
	}
	return TickingGroup;
}

//------------------------------------------------------------------------------------------------------------

void FNDIPhysicsAssetBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	using namespace NDIPhysicsAssetLocal;

	CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F, PHYSICS_ASSET_MAX_TRANSFORMS, 3>(RHICmdList, WorldTransformBuffer);
	CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F, PHYSICS_ASSET_MAX_TRANSFORMS, 3>(RHICmdList, InverseTransformBuffer);

	CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F, PHYSICS_ASSET_MAX_PRIMITIVES>(RHICmdList, ElementExtentBuffer);
	CreateInternalBuffer<uint32, EPixelFormat::PF_R32_UINT, PHYSICS_ASSET_MAX_PRIMITIVES>(RHICmdList, PhysicsTypeBuffer);
}

void FNDIPhysicsAssetBuffer::ReleaseRHI()
{
	WorldTransformBuffer.Release();
	InverseTransformBuffer.Release();
	ElementExtentBuffer.Release();
	PhysicsTypeBuffer.Release();
}

void FNDIPhysicsAssetData::Release()
{
	if (AssetBuffer)
	{
		ENQUEUE_RENDER_COMMAND(DeleteResource)(
			[ParamPointerToRelease = AssetBuffer](FRHICommandListImmediate& RHICmdList)
			{
				ParamPointerToRelease->ReleaseResource();
				delete ParamPointerToRelease;
			});
		AssetBuffer = nullptr;
	}
}

void FNDIPhysicsAssetData::Init(UNiagaraDataInterfacePhysicsAsset* Interface, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIPhysicsAssetLocal;

	AssetBuffer = nullptr;

	if (Interface != nullptr && SystemInstance != nullptr)
	{
		FTransform BoneTransform = FTransform::Identity;
		Interface->ExtractSourceComponent(SystemInstance, BoneTransform);
		TickingGroup = ComputeTickingGroup(Interface->SourceComponents);

		if(0 < Interface->PhysicsAssets.Num() && Interface->PhysicsAssets[0].IsValid() && Interface->PhysicsAssets[0].Get() != nullptr && 
			Interface->PhysicsAssets.Num() == Interface->SourceComponents.Num() )
		{
			CreateInternalArrays(Interface->PhysicsAssets, Interface->SourceComponents, &AssetArrays, SystemInstance->GetWorldTransform(), BoneTransform);
		}

		AssetBuffer = new FNDIPhysicsAssetBuffer();
		BeginInitResource(AssetBuffer);

		FVector BoxMax(-FLT_MAX, -FLT_MAX, -FLT_MAX);
		FVector BoxMin = -BoxMax;

		FBox BoundingBox(BoxMin, BoxMax);
		for (int32 ComponentIndex = 0; ComponentIndex < Interface->PhysicsAssets.Num(); ++ComponentIndex)
		{
			TWeakObjectPtr<UPhysicsAsset> PhysicsAsset = Interface->PhysicsAssets[ComponentIndex];
			if (PhysicsAsset.IsValid() && PhysicsAsset.Get() != nullptr)
			{
				USkeletalMesh* SkelMesh = (Interface->SourceComponents[ComponentIndex].IsValid() && Interface->SourceComponents[ComponentIndex].Get() && Interface->SourceComponents[ComponentIndex]->GetSkeletalMeshAsset()) ?
					ToRawPtr(Interface->SourceComponents[ComponentIndex]->GetSkeletalMeshAsset()) : PhysicsAsset->GetPreviewMesh();
				if (SkelMesh)
				{
					BoundingBox += SkelMesh->GetImportedBounds().GetBox();
				}
			}
		}
		BoxOrigin = 0.5 * (BoundingBox.Max + BoundingBox.Min);
		BoxExtent = (BoundingBox.Max - BoundingBox.Min);
	}
}

void FNDIPhysicsAssetData::Update(UNiagaraDataInterfacePhysicsAsset* Interface, FNiagaraSystemInstance* SystemInstance)
{
	if (Interface != nullptr && SystemInstance != nullptr)
	{
		FTransform BoneTransform = FTransform::Identity;
		Interface->ExtractSourceComponent(SystemInstance, BoneTransform);
		TickingGroup = ComputeTickingGroup(Interface->SourceComponents);

		if (0 < Interface->PhysicsAssets.Num() && Interface->PhysicsAssets[0].IsValid() && Interface->PhysicsAssets[0].Get() != nullptr &&
			Interface->PhysicsAssets.Num() == Interface->SourceComponents.Num())
		{
			NDIPhysicsAssetLocal::UpdateInternalArrays(Interface->PhysicsAssets, Interface->SourceComponents, &AssetArrays, SystemInstance->GetWorldTransform(), BoneTransform);
		}
	}
}

//------------------------------------------------------------------------------------------------------------

void FNDIPhysicsAssetProxy::ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance)
{
	FNDIPhysicsAssetData* SourceData = static_cast<FNDIPhysicsAssetData*>(PerInstanceData);
	FNDIPhysicsAssetData* TargetData = &(SystemInstancesToProxyData.FindOrAdd(Instance));

	ensure(TargetData);
	if (TargetData)
	{
		TargetData->AssetBuffer = SourceData->AssetBuffer;
		TargetData->BoxOrigin = SourceData->BoxOrigin;
		TargetData->BoxExtent = SourceData->BoxExtent;
		TargetData->AssetArrays = SourceData->AssetArrays;
		TargetData->TickingGroup = SourceData->TickingGroup;
	}
	else
	{
		UE_LOGF(LogPhysicsAsset, Log, "ConsumePerInstanceDataFromGameThread() ... could not find %lld", Instance);
	}
	SourceData->~FNDIPhysicsAssetData();
}

void FNDIPhysicsAssetProxy::InitializePerInstanceData(const FNiagaraSystemInstanceID& SystemInstance)
{
	check(IsInRenderingThread());

	FNDIPhysicsAssetData* TargetData = SystemInstancesToProxyData.Find(SystemInstance);
	TargetData = &SystemInstancesToProxyData.Add(SystemInstance);
}

void FNDIPhysicsAssetProxy::DestroyPerInstanceData(const FNiagaraSystemInstanceID& SystemInstance)
{
	check(IsInRenderingThread());
	SystemInstancesToProxyData.Remove(SystemInstance);
}

void FNDIPhysicsAssetProxy::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	using namespace NDIPhysicsAssetLocal;

	FNDIPhysicsAssetData* ProxyData = SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());

	if (ProxyData != nullptr && ProxyData->AssetBuffer)
	{
		if (Context.GetSimStageData().bFirstStage)
		{
			FRHICommandListBase& RHICmdList = Context.GetGraphBuilder().RHICmdList;

			UpdateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F, PHYSICS_ASSET_MAX_TRANSFORMS, 3>(RHICmdList, ProxyData->AssetArrays.WorldTransform, ProxyData->AssetBuffer->WorldTransformBuffer);
			UpdateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F, PHYSICS_ASSET_MAX_TRANSFORMS, 3>(RHICmdList, ProxyData->AssetArrays.InverseTransform, ProxyData->AssetBuffer->InverseTransformBuffer);
			UpdateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F, PHYSICS_ASSET_MAX_PRIMITIVES>(RHICmdList, ProxyData->AssetArrays.ElementExtent, ProxyData->AssetBuffer->ElementExtentBuffer);
			UpdateInternalBuffer<uint32, EPixelFormat::PF_R32_UINT, PHYSICS_ASSET_MAX_PRIMITIVES>(RHICmdList, ProxyData->AssetArrays.PhysicsType, ProxyData->AssetBuffer->PhysicsTypeBuffer);
		}
	}
}

//------------------------------------------------------------------------------------------------------------

UNiagaraDataInterfacePhysicsAsset::UNiagaraDataInterfacePhysicsAsset(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, DefaultSource(nullptr)
	, SoftSourceActor(nullptr)
	, SourceComponents()
	, PhysicsAssets()
{
	FNiagaraTypeDefinition Def(UObject::StaticClass());
	MeshUserParameter.Parameter.SetType(Def);

	Proxy.Reset(new FNDIPhysicsAssetProxy());
}

void UNiagaraDataInterfacePhysicsAsset::ExtractSourceComponent(FNiagaraSystemInstance* SystemInstance, FTransform& BoneTransform)
{
	check(SystemInstance);

	// Helper to scour an actor (or its parents) for a valid skeletal mesh component
	auto FindActorSkelMeshComponent = [](AActor* Actor, bool bRecurseParents = false) -> USkeletalMeshComponent*
	{
		if (ASkeletalMeshActor* SkelMeshActor = Cast<ASkeletalMeshActor>(Actor))
		{
			USkeletalMeshComponent* Comp = SkelMeshActor->GetSkeletalMeshComponent();
			if (IsValid(Comp))
			{
				return Comp;
			}
		}

		// Fall back on any valid component on the actor
		while (Actor)
		{
			for (UActorComponent* ActorComp : Actor->GetComponents())
			{
				USkeletalMeshComponent* Comp = Cast<USkeletalMeshComponent>(ActorComp);
				if (IsValid(Comp) && Comp->GetSkeletalMeshAsset() != nullptr)
				{
					return Comp;
				}
			}

			if (bRecurseParents)
			{
				Actor = Actor->GetParentActor();
			}
			else
			{
				break;
			}
		}

		return nullptr;
	};
		
	BoneTransform = FTransform::Identity;

	// Track down the source component
	TWeakObjectPtr<USkeletalMeshComponent> SourceComponent;
	
	if (MeshUserParameter.Parameter.IsValid())
	{
		// Initialize the binding and retrieve the object. If a valid object is bound, we'll try and retrieve the SkelMesh component from it.
		// If it's not valid yet, we'll reset and do this again when/if a valid object is set on the binding
		FNiagaraParameterDirectBinding<UObject*> UserParamBinding;
		UObject* UserParamObject = UserParamBinding.Init(SystemInstance->GetInstanceParameters(), MeshUserParameter.Parameter);

		if (UserParamObject)
		{
			if (USkeletalMeshComponent* UserSkelMeshComp = Cast<USkeletalMeshComponent>(UserParamObject))
			{
				if (IsValid(UserSkelMeshComp))
				{
					SourceComponent = UserSkelMeshComp;
				}
			}
			else if (AActor* Actor = Cast<AActor>(UserParamObject))
			{
				SourceComponent = FindActorSkelMeshComponent(Actor);
			}
			else
			{
				//We have a valid, non-null UObject parameter type but it is not a type we can use to get a skeletal mesh from.
				UE_LOGF(LogPhysicsAsset, Warning, "SkeletalMesh data interface using object parameter with invalid type. Skeletal Mesh Data Interfaces can only get a valid mesh from SkeletalMeshComponents, SkeletalMeshActors or Actors.");
				UE_LOGF(LogPhysicsAsset, Warning, "Invalid Parameter : %ls", *UserParamObject->GetFullName());
				UE_LOGF(LogPhysicsAsset, Warning, "System : %ls", *GetFullNameSafe(SystemInstance->GetSystem()));
			}
		}
		else
		{
			// The binding exists, but no object is bound. Not warning here in case the user knows what they're doing.
		}
	}
	else if (AActor* SourceActor = SoftSourceActor.Get())
	{
		ASkeletalMeshActor* SkeletalMeshActor = Cast<ASkeletalMeshActor>(SourceActor);
		if (SkeletalMeshActor != nullptr)
		{
			SourceComponent = SkeletalMeshActor->GetSkeletalMeshComponent();
		}
		else
		{
			SourceComponent = SourceActor->FindComponentByClass<USkeletalMeshComponent>();
		}
	}
	else if (USceneComponent* AttachComponent = SystemInstance->GetAttachComponent())
	{
		// Try to find the first component by walking the attachment hierarchy
		for (USceneComponent* Curr = AttachComponent; Curr; Curr = Curr->GetAttachParent())
		{
			USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(Curr);
			if (SkelMeshComp && SkelMeshComp->GetSkeletalMeshAsset())
			{
				SourceComponent = SkelMeshComp;
				break;
			}
		}

		if (!SourceComponent.IsValid())
		{
			// Fall back on the attach component's outer chain if we aren't attached to the skeletal mesh 
			if (USkeletalMeshComponent* OuterComp = AttachComponent->GetTypedOuter<USkeletalMeshComponent>())
			{
				SourceComponent = OuterComp;
			}
		}
	}
	
	SourceComponents.Empty();
	PhysicsAssets.Empty();
	
	// Try to find the groom physics asset by walking the attachment hierarchy
	UPhysicsAsset* PhysicsAsset = DefaultSource;
	for (USceneComponent* Curr = SystemInstance->GetAttachComponent(); Curr; Curr = Curr->GetAttachParent())
	{
		if (INiagaraPhysicsAssetDICollectorInterface* RetrieverInterface = Cast<INiagaraPhysicsAssetDICollectorInterface>(Curr))
		{
			PhysicsAsset = RetrieverInterface->BuildAndCollect(BoneTransform, SourceComponents, PhysicsAssets);;
			break;
		}
	}
	
	if (SourceComponent != nullptr)
	{
		if (PhysicsAsset || SourceComponent->GetPhysicsAsset())
		{
			SourceComponents.Add(SourceComponent);
			if (PhysicsAsset)
			{
				PhysicsAssets.Add(PhysicsAsset);
			}
			else
			{
				PhysicsAssets.Add(SourceComponent->GetPhysicsAsset());
			}
		}
	}
	else if (PhysicsAsset != nullptr)
	{
		SourceComponents.Add(nullptr);
		PhysicsAssets.Add(PhysicsAsset);
	}
}

bool UNiagaraDataInterfacePhysicsAsset::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIPhysicsAssetData* InstanceData = new (PerInstanceData) FNDIPhysicsAssetData();

	check(InstanceData);
	InstanceData->Init(this, SystemInstance);

	return true;
}

ETickingGroup UNiagaraDataInterfacePhysicsAsset::CalculateTickGroup(const void* PerInstanceData) const
{
	const FNDIPhysicsAssetData* InstanceData = static_cast<const FNDIPhysicsAssetData*>(PerInstanceData);

	if (InstanceData)
	{
		return InstanceData->TickingGroup;
	}
	return NiagaraFirstTickGroup;
}

void UNiagaraDataInterfacePhysicsAsset::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIPhysicsAssetData* InstanceData = static_cast<FNDIPhysicsAssetData*>(PerInstanceData);

	InstanceData->Release();
	InstanceData->~FNDIPhysicsAssetData();

	FNDIPhysicsAssetProxy* ThisProxy = GetProxyAs<FNDIPhysicsAssetProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			ThisProxy->SystemInstancesToProxyData.Remove(InstanceID);
		}
	);
}

bool UNiagaraDataInterfacePhysicsAsset::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float InDeltaSeconds)
{
	FNDIPhysicsAssetData* InstanceData = static_cast<FNDIPhysicsAssetData*>(PerInstanceData);
	if (InstanceData && InstanceData->AssetBuffer && SystemInstance)
	{
		InstanceData->Update(this, SystemInstance);
	}
	return false;
}

bool UNiagaraDataInterfacePhysicsAsset::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfacePhysicsAsset* OtherTyped = CastChecked<UNiagaraDataInterfacePhysicsAsset>(Destination);
	OtherTyped->PhysicsAssets = PhysicsAssets;
	OtherTyped->SoftSourceActor = SoftSourceActor;
	OtherTyped->SourceComponents = SourceComponents;
	OtherTyped->DefaultSource = DefaultSource;
	OtherTyped->MeshUserParameter = MeshUserParameter;

	return true;
}

bool UNiagaraDataInterfacePhysicsAsset::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfacePhysicsAsset* OtherTyped = CastChecked<const UNiagaraDataInterfacePhysicsAsset>(Other);

	return  (OtherTyped->PhysicsAssets == PhysicsAssets) && 
		(OtherTyped->SoftSourceActor == SoftSourceActor) &&
		(OtherTyped->SourceComponents == SourceComponents) && 
		(OtherTyped->DefaultSource == DefaultSource) &&
		(OtherTyped->MeshUserParameter == MeshUserParameter);
}

void UNiagaraDataInterfacePhysicsAsset::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfacePhysicsAsset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (SourceActor_DEPRECATED != nullptr)
	{
		SoftSourceActor = SourceActor_DEPRECATED;
	}
#endif
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfacePhysicsAsset::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIPhysicsAssetLocal;
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNumBoxesName;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Asset")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Boxes")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNumSpheresName;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Asset")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Spheres")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNumCapsulesName;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Asset")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Capsules")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetClosestPointName;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Asset")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Velocity")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetClosestElementName;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Asset")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Closest Element")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetElementPointName;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Asset")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Velocity")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetElementDistanceName;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Asset")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetClosestDistanceName;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Asset")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetRestDistanceName;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Asset")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Rest Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Distance")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetTexturePointName;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Asset")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Texture Position")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetProjectionPointName;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Asset")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Texture Value")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Texture Gradient")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));

		OutFunctions.Add(Sig);
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetNumBoxes);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetNumSpheres);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetNumCapsules);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetClosestPoint);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetClosestElement);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetElementPoint);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetElementDistance);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetClosestDistance);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetTexturePoint);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetProjectionPoint);

void UNiagaraDataInterfacePhysicsAsset::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIPhysicsAssetLocal;

	if (BindingInfo.Name == GetNumBoxesName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetNumBoxes)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetNumSpheresName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetNumSpheres)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetNumCapsulesName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetNumCapsules)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetClosestPointName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 9);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetClosestPoint)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetClosestElementName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetClosestElement)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetElementPointName)
	{
		check(BindingInfo.GetNumInputs() == 7 && BindingInfo.GetNumOutputs() == 9);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetElementPoint)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetElementDistanceName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetElementDistance)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetClosestDistanceName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetClosestDistance)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetTexturePointName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 4);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetTexturePoint)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetProjectionPointName)
	{
		check(BindingInfo.GetNumInputs() == 10 && BindingInfo.GetNumOutputs() == 10);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsAsset, GetProjectionPoint)::Bind(this, OutFunc);
	}
}

void UNiagaraDataInterfacePhysicsAsset::GetNumBoxes(FVectorVMExternalFunctionContext& Context)
{
}

void UNiagaraDataInterfacePhysicsAsset::GetNumSpheres(FVectorVMExternalFunctionContext& Context)
{
}

void UNiagaraDataInterfacePhysicsAsset::GetNumCapsules(FVectorVMExternalFunctionContext& Context)
{
}

void UNiagaraDataInterfacePhysicsAsset::GetClosestPoint(FVectorVMExternalFunctionContext& Context)
{
}

void UNiagaraDataInterfacePhysicsAsset::GetClosestElement(FVectorVMExternalFunctionContext& Context)
{
}

void UNiagaraDataInterfacePhysicsAsset::GetElementPoint(FVectorVMExternalFunctionContext& Context)
{
}

void UNiagaraDataInterfacePhysicsAsset::GetElementDistance(FVectorVMExternalFunctionContext& Context)
{
}

void UNiagaraDataInterfacePhysicsAsset::GetClosestDistance(FVectorVMExternalFunctionContext& Context)
{
}

void UNiagaraDataInterfacePhysicsAsset::GetTexturePoint(FVectorVMExternalFunctionContext& Context)
{
}

void UNiagaraDataInterfacePhysicsAsset::GetProjectionPoint(FVectorVMExternalFunctionContext& Context)
{
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfacePhysicsAsset::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIPhysicsAssetLocal;

	static const TSet<FName> ValidGpuFunctions =
	{
		GetNumBoxesName,
		GetNumCapsulesName,
		GetNumSpheresName,
		GetClosestPointName,
		GetClosestElementName,
		GetElementPointName,
		GetElementDistanceName,
		GetClosestDistanceName,
		GetRestDistanceName,
		GetTexturePointName,
		GetProjectionPointName,
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

bool UNiagaraDataInterfacePhysicsAsset::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}

	InVisitor->UpdateShaderFile(NDIPhysicsAssetLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<NDIPhysicsAssetLocal::FShaderParameters>();

	return true;
}

void UNiagaraDataInterfacePhysicsAsset::GetCommonHLSL(FString& OutHLSL)
{
	OutHLSL += TEXT("#include \"/Plugin/FX/Niagara/Private/NiagaraQuaternionUtils.ush\"\n");
}

void UNiagaraDataInterfacePhysicsAsset::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIPhysicsAssetLocal::TemplateShaderFile, TemplateArgs);
}
#endif

void UNiagaraDataInterfacePhysicsAsset::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIPhysicsAssetLocal::FShaderParameters>();
}

void UNiagaraDataInterfacePhysicsAsset::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNDIPhysicsAssetProxy& DIProxy = Context.GetProxy<FNDIPhysicsAssetProxy>();
	FNDIPhysicsAssetData* ProxyData = DIProxy.SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());
	NDIPhysicsAssetLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDIPhysicsAssetLocal::FShaderParameters>();

	if (ProxyData != nullptr && ProxyData->AssetBuffer && ProxyData->AssetBuffer->IsInitialized())
	{
		FNDIPhysicsAssetBuffer* AssetBuffer = ProxyData->AssetBuffer;

		ShaderParameters->WorldTransformBuffer = FNiagaraRenderer::GetSrvOrDefaultFloat4(AssetBuffer->WorldTransformBuffer.SRV);
		ShaderParameters->InverseTransformBuffer = FNiagaraRenderer::GetSrvOrDefaultFloat4(AssetBuffer->InverseTransformBuffer.SRV);
		ShaderParameters->ElementExtentBuffer = FNiagaraRenderer::GetSrvOrDefaultFloat4(AssetBuffer->ElementExtentBuffer.SRV);
		ShaderParameters->PhysicsTypeBuffer = FNiagaraRenderer::GetSrvOrDefaultUInt(AssetBuffer->PhysicsTypeBuffer.SRV);

		ShaderParameters->ElementOffsets.X = ProxyData->AssetArrays.ElementOffsets.BoxOffset;
		ShaderParameters->ElementOffsets.Y = ProxyData->AssetArrays.ElementOffsets.SphereOffset;
		ShaderParameters->ElementOffsets.Z = ProxyData->AssetArrays.ElementOffsets.CapsuleOffset;
		ShaderParameters->ElementOffsets.W = ProxyData->AssetArrays.ElementOffsets.NumElements;
		ShaderParameters->BoxOrigin = FVector3f(ProxyData->BoxOrigin);
		ShaderParameters->BoxExtent = FVector3f(ProxyData->BoxExtent);
	}
	else
	{
		ShaderParameters->WorldTransformBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->InverseTransformBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->ElementExtentBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->PhysicsTypeBuffer = FNiagaraRenderer::GetDummyUIntBuffer();

		ShaderParameters->ElementOffsets = FUintVector4(0, 0, 0, 0);
		ShaderParameters->BoxOrigin = FVector3f::ZeroVector;
		ShaderParameters->BoxExtent = FVector3f::ZeroVector;
	}
}

void UNiagaraDataInterfacePhysicsAsset::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	FNDIPhysicsAssetData* GameThreadData = static_cast<FNDIPhysicsAssetData*>(PerInstanceData);
	FNDIPhysicsAssetData* RenderThreadData = static_cast<FNDIPhysicsAssetData*>(DataForRenderThread);

	if (GameThreadData != nullptr && RenderThreadData != nullptr)
	{
		RenderThreadData->AssetBuffer = GameThreadData->AssetBuffer;
		RenderThreadData->BoxOrigin = GameThreadData->BoxOrigin;
		RenderThreadData->BoxExtent = GameThreadData->BoxExtent;
		RenderThreadData->AssetArrays = GameThreadData->AssetArrays;
		RenderThreadData->TickingGroup = GameThreadData->TickingGroup;
	}
	check(Proxy);
}

UNiagaraPhysicsAssetDICollectorInterface::UNiagaraPhysicsAssetDICollectorInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

#undef LOCTEXT_NAMESPACE
