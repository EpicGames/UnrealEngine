// Copyright Epic Games, Inc. All Rights Reserved.
// NOTE: Temporary code do not use as it will likely be removed in a future version

#include "DataInterface/NiagaraDataInterfacePropertyInterface.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraParameterStore.h"
#include "NiagaraRenderThreadDeletor.h"
#include "NiagaraRuntimeTypeUtilities.h"
#include "NiagaraSceneComponentUtils.h"
#include "NiagaraSystemInstance.h"

#if WITH_EDITORONLY_DATA
#include "INiagaraEditorOnlyDataUtlities.h"
#include "Modules/ModuleManager.h"
#endif

#include "VerseVM/VVMNames.h"

#include "RenderGraphUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfacePropertyInterface)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfacePropertyInterface"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace NDIPropertyInterfaceLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_SRV(ByteAddressBuffer,	PropertyInterfaceData)
	END_SHADER_PARAMETER_STRUCT()

	static const FName NAME_GetInterfaceData("GetInterfaceData");

	const TCHAR* TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfacePropertyInterfaceTemplate.ush");

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FShaderStorage : public FNiagaraDataInterfaceParametersCS
	{
		DECLARE_TYPE_LAYOUT(FShaderStorage, NonVirtual);

		LAYOUT_FIELD(TMemoryImageArray<FMemoryImageName>, AttributeNames);
	};
	IMPLEMENT_TYPE_LAYOUT(FShaderStorage);

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FBuiltPropertyPropertyInterfaceData
	{
	public:
		using FPropertyConversionFunc = TFunction<void(const uint8*, uint8*)>;
		static constexpr uint32 kMaxElementReadSize = 16;

		//explicit FBuiltPropertyPropertyInterfaceData(UClass* InterfaceClass, UObject* SourceObject)
		//	: WeakInterfaceClass(InterfaceClass)
		//	, WeakSourceObject(SourceObject)
		//{
		//	if (InterfaceClass && SourceObject)
		//	{
		//		UClass* SourceClass = SourceObject->GetClass();
		//		for (const FImplementedInterface& Interface : SourceClass->Interfaces)
		//		{
		//			if (Interface.Class == InterfaceClass)
		//			{
		//				//-TODO: Find interface inside source object
		//				BuildProperties(InterfaceClass, reinterpret_cast<uint8*>(SourceObject));

		//				break;
		//			}
		//		}
		//	}

		//	// Always add padding to end for sampling invalid attributes
		//	CpuPropertyInterfaceData.AddZeroed(kMaxElementReadSize);
		//}

		explicit FBuiltPropertyPropertyInterfaceData(UObject* SourceObject, UClass* InterfaceClass)
			: WeakSourceObject(SourceObject)
		{
			if (SourceObject && InterfaceClass)
			{
				WeakInterfaceClass = InterfaceClass;
				BuildProperties(SourceObject->GetClass(), InterfaceClass, reinterpret_cast<uint8*>(SourceObject));
			}

			// Always add padding to end for sampling invalid attributes
			CpuPropertyInterfaceData.AddZeroed(kMaxElementReadSize);
		}

		~FBuiltPropertyPropertyInterfaceData()
		{
			check(IsInRenderingThread());
			GpuPropertyInterfaceData.Release();
		}

		uint32 GetVariableByteOffset(const FNiagaraVariableBase& Variable) const
		{
			for (const TPair<FNiagaraVariableBase, uint32>& VariableToOffset : VariableToOffsets)
			{
				if (VariableToOffset.Key == Variable)
				{
					return VariableToOffset.Value;
				}
			}
			return CpuPropertyInterfaceData.Num() - kMaxElementReadSize;
		}

		uint32 GetVariableByteOffset(const FName& VariableName) const
		{
			for (const TPair<FNiagaraVariableBase, uint32>& VariableToOffset : VariableToOffsets)
			{
				if (VariableToOffset.Key.GetName() == VariableName)
				{
					return VariableToOffset.Value;
				}
			}
			return CpuPropertyInterfaceData.Num() - kMaxElementReadSize;
		}

		FRHIShaderResourceView* GetGpuDataSrv(FRDGBuilder& GraphBuilder)
		{
			if (GpuPropertyInterfaceData.NumBytes == 0)
			{
				GpuPropertyInterfaceData.Initialize(GraphBuilder.RHICmdList, TEXT("NiagaraPropertyInterface::GpuData"), CpuPropertyInterfaceData.Num());

				void* UploadMemory = GraphBuilder.RHICmdList.LockBuffer(GpuPropertyInterfaceData.Buffer, 0, CpuPropertyInterfaceData.Num(), RLM_WriteOnly);
				FMemory::Memcpy(UploadMemory, CpuPropertyInterfaceData.GetData(), CpuPropertyInterfaceData.Num());
				GraphBuilder.RHICmdList.UnlockBuffer(GpuPropertyInterfaceData.Buffer);
			}
			return GpuPropertyInterfaceData.SRV;
		}

		TConstArrayView<uint8> GetCpuPropertyInterfaceData() const { return CpuPropertyInterfaceData; }

	private:
		void AddProperty(UStruct* Struct, FProperty* InterfaceProperty, FProperty* ClassProperty, const uint8* SourceObject, const FNiagaraRuntimeTypeUtilities::FTypeConverter* TypeConverter)
		{
			check(TypeConverter && TypeConverter->ToNiagara);

			const FString UnmangledPropertyName = Verse::Names::Private::UnmangleCasedName(InterfaceProperty->GetFName());
			const FName PropertyName(*UnmangledPropertyName);
			VariableToOffsets.Emplace(FNiagaraVariableBase(TypeConverter->TypeDef, PropertyName), CpuPropertyInterfaceData.Num());

			const uint32 SrcOffset = ClassProperty->GetOffset_ForInternal();
			const uint32 DstOffset = CpuPropertyInterfaceData.Num();
			
			CpuPropertyInterfaceData.AddUninitialized(TypeConverter->TypeDef.GetSize());
			TypeConverter->ToNiagara(SourceObject + SrcOffset, CpuPropertyInterfaceData.GetData() + DstOffset);
		}

		void BuildProperties(UStruct* ClassStruct, UStruct* InterfaceStruct, const uint8* SourceObject)
		{
			for (FProperty* InterfaceProperty = InterfaceStruct->PropertyLink; InterfaceProperty != nullptr; InterfaceProperty = InterfaceProperty->PropertyLinkNext)
			{
				FNameBuilder PropertyNameBuilder;
				InterfaceProperty->GetFName().ToString(PropertyNameBuilder);
				PropertyNameBuilder.Append(TEXT("_Internal"));

				FName PropertyName(PropertyNameBuilder.ToString());

				// Interface proprties do not have the correct offset to the data so we need to find the property in the base class
				FProperty* Property = ClassStruct->FindPropertyByName(PropertyName);
				if (Property == nullptr)
				{
					// Note this should never happen
					continue;
				}

				if (const FNiagaraRuntimeTypeUtilities::FTypeConverter* TypeConverter = FNiagaraRuntimeTypeUtilities::FindTypeConverterForProperty(Property))
				{
					AddProperty(InterfaceStruct, InterfaceProperty, Property, SourceObject, TypeConverter);
				}
			}
		}

	private:
		TWeakObjectPtr<UClass>												WeakInterfaceClass;			// Weak reference to the interface class this data came from
		TWeakObjectPtr<UObject>												WeakSourceObject;			// Weak reference to the interface source object this data came from
		TArray<TPair<FNiagaraVariableBase, uint32>, TInlineAllocator<16>>	VariableToOffsets;			// Mapping from Variable to Byte Offset in the data blob
		TArray<uint8>														CpuPropertyInterfaceData;	// Cpu data blob for supported properties
		FByteAddressBuffer													GpuPropertyInterfaceData;	// Gpu data blob for supported properties
	};

	using FBuiltPropertyPropertyInterfaceDataPtr = TSharedPtr<FBuiltPropertyPropertyInterfaceData>;

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FGpuAttributeHelper
	{
		explicit FGpuAttributeHelper(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo)
			: FGpuAttributeHelper(ParamInfo.GeneratedFunctions)
		{			
		}

		explicit FGpuAttributeHelper(TConstArrayView<FNiagaraDataInterfaceGeneratedFunction> GeneratedFunctions)
		{
			for (const FNiagaraDataInterfaceGeneratedFunction& Function : GeneratedFunctions)
			{
				for (const FNiagaraVariableCommonReference& OutputVariable : Function.VariadicOutputs)
				{
					Attributes.AddUnique(OutputVariable);
				}
			}
		}

		int32 GetAttributeIndex(const FNiagaraVariableBase& Variable) const
		{
			return Attributes.IndexOfByKey(Variable);
		}

		TArray<FNiagaraVariableBase> Attributes;
	};

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FInstanceData_RenderThread
	{
		using FAttributeArray = TArray<uint32, TInlineAllocator<8>>;

		FBuiltPropertyPropertyInterfaceDataPtr	PropertyInterfaceData;
		TMap<uintptr_t, FAttributeArray>		AttributeReadOffsets;
	};

	struct FGameToRenderData
	{
		FNiagaraSystemInstanceID				SystemInstanceID = {};
		FBuiltPropertyPropertyInterfaceDataPtr	PropertyInterfaceData;
	};

	struct FInstanceData_GameThread
	{
		//FNiagaraParameterDirectBinding<UObject*>	UserParamBinding;
		//TWeakObjectPtr<UClass>						WeakInterfaceClass;
		FName										InterfaceFullPath;
		const INiagaraSceneComponentUtils*			SceneComponentUtils = nullptr;
		TWeakObjectPtr<UObject>						WeakSourceObject;
		TWeakObjectPtr<UObject>						WeakOwnerObject;

		FBuiltPropertyPropertyInterfaceDataPtr		PropertyInterfaceData;

		void Initialize(FNiagaraSystemInstance* SystemInstance, FName InterfacePath, FName InterfaceName)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			SceneComponentUtils = &FNiagaraSystemInstance::GetSceneComponentUtils(SystemInstance);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			FNameBuilder PathBuilder;
			InterfacePath.ToString(PathBuilder);
			PathBuilder.AppendChar('/');
			InterfaceName.AppendString(PathBuilder);

			InterfaceFullPath = FName(PathBuilder.ToString());

			Tick();
		}

		void Tick()
		{
			UObject* SourceObject = SceneComponentUtils->GetOwner();
			WeakSourceObject = SourceObject;

			UClass* InterfaceClass = SceneComponentUtils->FindPropertyInterface(SourceObject, InterfaceFullPath);

			//PropertyInterfaceData = MakeShareable(new FBuiltPropertyPropertyInterfaceData(WeakInterfaceClass.Get(), SourceObject), FNiagaraRenderThreadDeletor<FBuiltPropertyPropertyInterfaceData>());
			PropertyInterfaceData = MakeShareable(new FBuiltPropertyPropertyInterfaceData(SourceObject, InterfaceClass), FNiagaraRenderThreadDeletor<FBuiltPropertyPropertyInterfaceData>());
		}

		TArray<uint32> CreateVariadicReadTable(const FVMExternalFunctionBindingInfo& BindingInfo) const
		{
			TArray<uint32> VariadicReadOffsets;
			VariadicReadOffsets.Reserve(BindingInfo.VariadicOutputs.Num());
			for (const FNiagaraVariableBase& Variable : BindingInfo.VariadicOutputs)
			{
				const uint32 ByteOffset = PropertyInterfaceData->GetVariableByteOffset(Variable);
				const uint32 NumRegisters = Variable.GetType().GetSize() / sizeof(uint32);	// This won't work with struct types or ones that contain complex alignment
				check(NumRegisters * sizeof(uint32) == Variable.GetType().GetSize());

				for (uint32 i = 0; i < NumRegisters; ++i)
				{
					const uint32 ElementOffset = ByteOffset + (i * sizeof(uint32));
					VariadicReadOffsets.Add(ElementOffset);
				}
			}
			return VariadicReadOffsets;
		}
	};

	static void VMGetInterfaceData(FVectorVMExternalFunctionContext& Context, TConstArrayView<uint32> VariadicReadOffsets)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);

		TArray<FNDIOutputParam<int32>, TInlineAllocator<16>> OutVariadics;
		OutVariadics.Reserve(VariadicReadOffsets.Num());
		for (int32 i=0; i < VariadicReadOffsets.Num(); ++i)
		{
			OutVariadics.Emplace(Context);
		}

		const uint8* PropertyInterfaceData = InstanceData->PropertyInterfaceData->GetCpuPropertyInterfaceData().GetData();
		for (int32 iInstance=0; iInstance < Context.GetNumInstances(); ++iInstance)
		{
			for (int32 iOutput = 0; iOutput < VariadicReadOffsets.Num(); ++iOutput)
			{
				const int32 VariableReadOffset = VariadicReadOffsets[iOutput];
				const int32* Value = reinterpret_cast<const int32*>(PropertyInterfaceData + VariableReadOffset);
				OutVariadics[iOutput].SetAndAdvance(*Value);
			}
		}
	}

	struct FNDIProxy : public FNiagaraDataInterfaceProxy//RW
	{
		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override {}
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }
	
		void SendGameToRender(FInstanceData_GameThread* InstanceData_GT, FNiagaraSystemInstanceID SystemInstanceID)
		{
			FGameToRenderData GameToRenderData;
			GameToRenderData.SystemInstanceID		= SystemInstanceID;
			GameToRenderData.PropertyInterfaceData	= InstanceData_GT->PropertyInterfaceData;

			ENQUEUE_RENDER_COMMAND(FNDIPropertyInterface_SendData)
			(
				[this, GameToRenderData=MoveTemp(GameToRenderData)](FRHICommandListImmediate& RHICmdList)
				{
					FInstanceData_RenderThread* InstanceData_RT = &PerInstanceData_RenderThread.FindOrAdd(GameToRenderData.SystemInstanceID);
					InstanceData_RT->PropertyInterfaceData = GameToRenderData.PropertyInterfaceData;
					InstanceData_RT->AttributeReadOffsets.Empty();
				}
			);
		}
	
		TMap<FNiagaraSystemInstanceID, FInstanceData_RenderThread>	PerInstanceData_RenderThread;
	};

} //namespace NDIPropertyInterfaceLocal

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfacePropertyInterface::UNiagaraDataInterfacePropertyInterface(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDIPropertyInterfaceLocal;

	Proxy.Reset(new FNDIProxy());

	//FNiagaraTypeDefinition Def(UObject::StaticClass());
	//ObjectParameterBinding.Parameter.SetType(Def);
}

void UNiagaraDataInterfacePropertyInterface::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfacePropertyInterface::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfacePropertyInterface* OtherTyped = CastChecked<const UNiagaraDataInterfacePropertyInterface>(Other);
	return
		OtherTyped->InterfaceName == InterfaceName &&
		OtherTyped->InterfacePath == InterfacePath &&
		OtherTyped->InterfacePackage == InterfacePackage;
		//OtherTyped->InterfaceClass == InterfaceClass &&
		//OtherTyped->ObjectParameterBinding == ObjectParameterBinding;
}

bool UNiagaraDataInterfacePropertyInterface::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfacePropertyInterface* DestinationTyped = CastChecked<UNiagaraDataInterfacePropertyInterface>(Destination);
	DestinationTyped->InterfaceName = InterfaceName;
	DestinationTyped->InterfacePath = InterfacePath;
	DestinationTyped->InterfacePackage = InterfacePackage;
	//DestinationTyped->InterfaceClass = InterfaceClass;
	//DestinationTyped->ObjectParameterBinding = ObjectParameterBinding;

	return true;
}

bool UNiagaraDataInterfacePropertyInterface::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIPropertyInterfaceLocal;

	FInstanceData_GameThread* InstanceData_GT = new(PerInstanceData) FInstanceData_GameThread();
	InstanceData_GT->Initialize(SystemInstance, InterfacePath, InterfaceName);

	if ( IsUsedWithGPUScript() )
	{
		GetProxyAs<FNDIProxy>()->SendGameToRender(InstanceData_GT, SystemInstance->GetId());
	}
	return true;
}

void UNiagaraDataInterfacePropertyInterface::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIPropertyInterfaceLocal;

	FInstanceData_GameThread* InstanceData = static_cast<FInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FInstanceData_GameThread();

	if ( IsUsedWithGPUScript() )
	{
		ENQUEUE_RENDER_COMMAND(FNDIPropertyInterface_RemoveProxy)
		(
			[Proxy=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
			{
				Proxy->PerInstanceData_RenderThread.Remove(InstanceID);
			}
		);
	}
}

bool UNiagaraDataInterfacePropertyInterface::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIPropertyInterfaceLocal;

	FInstanceData_GameThread* InstanceData_GT = static_cast<FInstanceData_GameThread*>(PerInstanceData);
	InstanceData_GT->Tick();

	if (IsUsedWithGPUScript())
	{
		GetProxyAs<FNDIProxy>()->SendGameToRender(InstanceData_GT, SystemInstance->GetId());
	}

	return false;
}

int32 UNiagaraDataInterfacePropertyInterface::PerInstanceDataSize() const
{
	using namespace NDIPropertyInterfaceLocal;

	return sizeof(FInstanceData_GameThread);
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfacePropertyInterface::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIPropertyInterfaceLocal;

	FNiagaraFunctionSignature DefaultSig;
	DefaultSig.bMemberFunction = true;
	DefaultSig.bRequiresContext = false;
	DefaultSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("PropertyInterface"));

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultSig);
		Sig.Name = NAME_GetInterfaceData;
		Sig.RequiredOutputs = Sig.Outputs.Num();
	}
}
#endif

void UNiagaraDataInterfacePropertyInterface::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* PerInstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIPropertyInterfaceLocal;

	if ( BindingInfo.Name == NAME_GetInterfaceData )
	{
		const FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
		OutFunc = FVMExternalFunction::CreateLambda(
			[VariadicReadOffsets=InstanceData_GT->CreateVariadicReadTable(BindingInfo)](FVectorVMExternalFunctionContext& Context)
			{
				VMGetInterfaceData(Context, VariadicReadOffsets);
			}
		);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfacePropertyInterface::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	using namespace NDIPropertyInterfaceLocal;

	const FGpuAttributeHelper AttributeHelper(ParamInfo);
	OutHLSL.Appendf(
		TEXT("uint4 %s_AttributeReadOffset[%d];\n"),
		*ParamInfo.DataInterfaceHLSLSymbol,
		FMath::DivideAndRoundUp(FMath::Max(AttributeHelper.Attributes.Num(), 1), 4)
	);

	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfacePropertyInterface::GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL)
{
	using namespace NDIPropertyInterfaceLocal;

	const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo = HlslGenContext.GetFunctionInfo();
	const FString& DataInterfaceHLSLSymbol = HlslGenContext.ParameterInfo.DataInterfaceHLSLSymbol;

	if (FunctionInfo.DefinitionName == NAME_GetInterfaceData)
	{
		const FGpuAttributeHelper AttributeHelper(HlslGenContext.ParameterInfo);

		OutHLSL.Appendf(TEXT("void %s%s\n"), *FunctionInfo.InstanceName, *HlslGenContext.GetSanitizedFunctionParameters(HlslGenContext.GetFunctionSignature()));
		OutHLSL.Append(TEXT("{\n"));
		for (const FNiagaraVariableCommonReference& OutputVariable : FunctionInfo.VariadicOutputs)
		{
			OutHLSL.Appendf(
				TEXT("\tReadValue_%s(%d, Out_%s);\n"),
				*DataInterfaceHLSLSymbol,
				AttributeHelper.GetAttributeIndex(OutputVariable),
				*HlslGenContext.GetSanitizedSymbolName(OutputVariable.Name.ToString())
			);
		}
		OutHLSL.Append(TEXT("}\n"));
		return true;
	}

	return false;
}

bool UNiagaraDataInterfacePropertyInterface::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace NDIPropertyInterfaceLocal;

	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfacePropertyInterface::PostCompile(const UNiagaraSystem& OwningSystem)
{
	using namespace NDIPropertyInterfaceLocal;

	Super::PostCompile(OwningSystem);
}
#endif

FNiagaraDataInterfaceParametersCS* UNiagaraDataInterfacePropertyInterface::CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const
{
	using namespace NDIPropertyInterfaceLocal;

	const FGpuAttributeHelper AttributeHelper(ParameterInfo);

	FShaderStorage* ShaderStorage = new FShaderStorage();
	ShaderStorage->AttributeNames.Reserve(AttributeHelper.Attributes.Num());
	for (const FNiagaraVariableBase& Attribute : AttributeHelper.Attributes)
	{
		ShaderStorage->AttributeNames.Add(Attribute.GetName());
	}
	return ShaderStorage;
}

const FTypeLayoutDesc* UNiagaraDataInterfacePropertyInterface::GetShaderStorageType() const
{
	using namespace NDIPropertyInterfaceLocal;
	return &StaticGetTypeLayoutDesc<FShaderStorage>();
}

void UNiagaraDataInterfacePropertyInterface::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NDIPropertyInterfaceLocal;

	const FGpuAttributeHelper AttributeHelper(ShaderParametersBuilder.GetGeneratedFunctions());
	const int32 NumAttributes = FMath::DivideAndRoundUp(FMath::Max(AttributeHelper.Attributes.Num(), 1), 4);
	ShaderParametersBuilder.AddLooseParamArray<FUintVector4>(TEXT("AttributeReadOffset"), NumAttributes);

	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfacePropertyInterface::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIPropertyInterfaceLocal;

	FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	FInstanceData_RenderThread& InstanceData = DIProxy.PerInstanceData_RenderThread.FindChecked(Context.GetSystemInstanceID());

	const FShaderStorage& ShaderStorage = Context.GetShaderStorage<FShaderStorage>();
	const uint32 NumAttributes4 = FMath::DivideAndRoundUp(FMath::Max(ShaderStorage.AttributeNames.Num(), 1), 4);

	FInstanceData_RenderThread::FAttributeArray& AttributeReadOffsets = InstanceData.AttributeReadOffsets.FindOrAdd(uintptr_t(&ShaderStorage));
	if (AttributeReadOffsets.IsEmpty())
	{
		AttributeReadOffsets.AddZeroed(NumAttributes4 * 4);
		for ( int32 i=0; i < ShaderStorage.AttributeNames.Num(); ++i )
		{
			AttributeReadOffsets[i] = InstanceData.PropertyInterfaceData->GetVariableByteOffset(ShaderStorage.AttributeNames[i]);
		}
	}
	TArrayView<FUintVector4> AttributeIndices = Context.GetParameterLooseArray<FUintVector4>(NumAttributes4);
	FMemory::Memcpy(AttributeIndices.GetData(), AttributeReadOffsets.GetData(), AttributeReadOffsets.Num() * AttributeReadOffsets.GetTypeSize());

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->PropertyInterfaceData = InstanceData.PropertyInterfaceData->GetGpuDataSrv(Context.GetGraphBuilder());
}

#if WITH_NIAGARA_DEBUGGER
void UNiagaraDataInterfacePropertyInterface::DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const
{
	using namespace NDIPropertyInterfaceLocal;

	const FInstanceData_GameThread* InstanceData_GT = DebugHudContext.GetSystemInstance()->FindTypedDataInterfaceInstanceData<FInstanceData_GameThread>(this);
	if (InstanceData_GT == nullptr)
	{
		return;
	}

	DebugHudContext.GetOutputString().Appendf(TEXT("SourceObject(%s)"), *GetNameSafe(InstanceData_GT->WeakSourceObject.Get()));
}
#endif //WITH_NIAGARA_DEBUGGER

TArray<FNiagaraVariableBase> UNiagaraDataInterfacePropertyInterface::FindUsedVariables(UNiagaraSystem* OwningSystem)
{
	using namespace NDIPropertyInterfaceLocal;

	TArray<FNiagaraVariableBase> Variables;

#if WITH_EDITORONLY_DATA
	INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");
	const INiagaraEditorOnlyDataUtilities& EditorOnlyDataUtilities = NiagaraModule.GetEditorOnlyDataUtilities();
	UNiagaraDataInterface* RuntimeInstanceOfThis =
		EditorOnlyDataUtilities.IsEditorDataInterfaceInstance(this)
		? EditorOnlyDataUtilities.GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(*OwningSystem, *this)
		: this;

	FNiagaraDataInterfaceUtilities::ForEachVMFunction(
		RuntimeInstanceOfThis,
		OwningSystem,
		[&Variables](const UNiagaraScript* Script, const FVMExternalFunctionBindingInfo& FunctionBinding) -> bool
		{
			if (FunctionBinding.Name == NAME_GetInterfaceData)
			{
				for (const FNiagaraVariableBase& Variable : FunctionBinding.VariadicOutputs)
				{
					Variables.AddUnique(Variable);
				}
			}
			return true;
		}
	);

	FNiagaraDataInterfaceUtilities::ForEachGpuFunction(
		RuntimeInstanceOfThis,
		OwningSystem,
		[&Variables](const UNiagaraScript* Script, const FNiagaraDataInterfaceGeneratedFunction& FunctionBinding) -> bool
		{
			if (FunctionBinding.DefinitionName == NAME_GetInterfaceData)
			{
				for (const FNiagaraVariableCommonReference& Variable : FunctionBinding.VariadicOutputs)
				{
					Variables.AddUnique(Variable);
				}
			}
			return true;
		}
	);
#endif

	return Variables;
}

#undef LOCTEXT_NAMESPACE
