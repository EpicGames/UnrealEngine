// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSceneCapture2D.h"
#include "NiagaraComponent.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"

#include "Components/LineBatchComponent.h"
#include "Engine/Canvas.h"
#include "Engine/SceneCapture2D.h"
#include "UObject/Package.h"
#include "DrawDebugHelpers.h"
#include "GlobalRenderResources.h"
#include "RHIStaticStates.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceSceneCapture2D)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterfaceSceneCapture2D"

namespace NDISceneCapture2DLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D,			Texture)
		SHADER_PARAMETER_SAMPLER(SamplerState,		TextureSampler)
		SHADER_PARAMETER(FIntVector3,				TextureSize)
		SHADER_PARAMETER(FMatrix44f,				TextureViewMatrix)
		SHADER_PARAMETER(FMatrix44f,				TextureProjMatrix)
		SHADER_PARAMETER(FMatrix44f,				InvTextureProjMatrix)
		SHADER_PARAMETER(uint32,					IsPerspective)
	END_SHADER_PARAMETER_STRUCT()

	const TCHAR*	TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceSceneCapture2DTemplate.ush");

	static const FName NAME_GetTextureSize("GetTextureSize");
	static const FName NAME_Project("Project");
	static const FName NAME_Deproject("Deproject");
	static const FName NAME_GetCaptureTransform("GetCaptureTransform");
	static const FName NAME_SetCaptureTransform("SetCaptureTransform");
	static const FName NAME_SetRequestCapture("SetRequestCapture");
	static const FName NAME_TextureLoad("TextureLoad");
	static const FName NAME_TextureSample("TextureSample");
	static const FName NAME_TextureProject("TextureProject");
	static const FName NAME_TextureProjectDepth("TextureProjectDepth");
	static const FName NAME_TextureDeprojectDepth("TextureDeprojectDepth");

	FNiagaraVariableBase ExposedVariable_SceneCapture;

	struct FNDIInstanceDataCommon
	{
		FIntVector3	TextureSize = FIntVector3(0, 0, 0);
		FMatrix44f	TextureViewMatrix = FMatrix44f::Identity;
		FMatrix44f	TextureProjMatrix = FMatrix44f::Identity;
		FMatrix44f	InvTextureProjMatrix = FMatrix44f::Identity;
		ECameraProjectionMode::Type	ProjectType = ECameraProjectionMode::Perspective;
	};

	struct FNDIGameToRenderData : public FNDIInstanceDataCommon
	{
	};

	struct FNDIInstanceData_GameThread : public FNDIInstanceDataCommon
	{
		uint32										ChangeId = 0;
		FNiagaraParameterDirectBinding<UObject*>	UserParamBinding;
		TWeakObjectPtr<USceneCaptureComponent2D>	WeakCaptureComponent;

		bool						bNeedsCaptureUpdate = false;
		bool						bNeedsTransformUpdate = false;
		FVector3f					CaptureLocation = FVector3f::ZeroVector;
		FQuat4f						CaptureRotation = FQuat4f::Identity;
		float						FOV = 90.0f;
		float						OrthoWidth = 512.0f;

		void UpdateTransformData(float DeltaSeconds, FNiagaraSystemInstance* SystemInstance, USceneCaptureComponent2D* CaptureComponent)
		{
			FMinimalViewInfo ViewInfo;
			CaptureComponent->GetCameraView(DeltaSeconds, ViewInfo);

			const FNiagaraLWCConverter LWCConverter = SystemInstance->GetLWCConverter();
			ViewInfo.Location = FVector(LWCConverter.ConvertWorldToSimulationVector(ViewInfo.Location));

			const FMatrix44f ViewMatrix = FRotationTranslationMatrix44f::Make(FRotator3f(ViewInfo.Rotation), FVector3f(ViewInfo.Location));
			const FMatrix44f UnrealAxis(FPlane4f(0, 0, 1, 0), FPlane4f(1, 0, 0, 0), FPlane4f(0, 1, 0, 0), FPlane4f(0, 0, 0, 1));
			const FMatrix44f ProjMatrix = FMatrix44f(ViewInfo.CalculateProjectionMatrix());
			TextureViewMatrix = ViewMatrix.Inverse() * UnrealAxis;
			TextureProjMatrix = TextureViewMatrix * ProjMatrix;

			const bool bPerspectiveProjection = ViewInfo.ProjectionMode == ECameraProjectionMode::Perspective;
			const FMatrix44f ScreenToClip(
				FPlane4f(1, 0, 0, 0),
				FPlane4f(0, 1, 0, 0),
				FPlane4f(0, 0, ProjMatrix.M[2][2], bPerspectiveProjection ? 1.0f : 0.0f),
				FPlane4f(0, 0, ProjMatrix.M[3][2], bPerspectiveProjection ? 0.0f : 1.0f)
			);
			InvTextureProjMatrix = ScreenToClip * TextureProjMatrix.Inverse();

			UTextureRenderTarget2D* CaptureTexture = CaptureComponent->TextureTarget;
			TextureSize = FIntVector3(int32(CaptureTexture->GetSurfaceWidth()), int32(CaptureTexture->GetSurfaceHeight()), CaptureTexture->GetNumMips());

			CaptureLocation			= FVector3f(ViewInfo.Location);
			CaptureRotation			= FQuat4f(ViewInfo.Rotation.Quaternion());
			ProjectType				= ViewInfo.ProjectionMode;
			FOV						= ViewInfo.FOV;
			OrthoWidth				= ViewInfo.OrthoWidth;
		}
	};

	struct FNDInstanceData_RenderThread : public FNDIInstanceDataCommon
	{
		FTextureReferenceRHIRef	TextureReferenceRHI;
		FSamplerStateRHIRef		SamplerStateRHI;
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		static void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
		{
			FNDIGameToRenderData* InstanceDataToRender = new(DataForRenderThread) FNDIGameToRenderData();
			FNDIInstanceData_GameThread* InstanceData = reinterpret_cast<FNDIInstanceData_GameThread*>(PerInstanceData);
			InstanceDataToRender->TextureSize			= InstanceData->TextureSize;
			InstanceDataToRender->TextureViewMatrix		= InstanceData->TextureViewMatrix;
			InstanceDataToRender->TextureProjMatrix		= InstanceData->TextureProjMatrix;
			InstanceDataToRender->InvTextureProjMatrix	= InstanceData->InvTextureProjMatrix;
			InstanceDataToRender->ProjectType			= InstanceData->ProjectType;
		}

		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override
		{
			FNDIGameToRenderData* InstanceDataFromGame = reinterpret_cast<FNDIGameToRenderData*>(PerInstanceData);
			FNDInstanceData_RenderThread* InstanceData = &InstanceData_RT.FindOrAdd(Instance);
			InstanceData->TextureSize			= InstanceDataFromGame->TextureSize;
			InstanceData->TextureViewMatrix		= InstanceDataFromGame->TextureViewMatrix;
			InstanceData->TextureProjMatrix		= InstanceDataFromGame->TextureProjMatrix;
			InstanceData->InvTextureProjMatrix	= InstanceDataFromGame->InvTextureProjMatrix;
			InstanceData->ProjectType			= InstanceDataFromGame->ProjectType;
			InstanceDataFromGame->~FNDIGameToRenderData();
		}

		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
		{
			return sizeof(FNDIGameToRenderData);
		}

		TMap<FNiagaraSystemInstanceID, FNDInstanceData_RenderThread> InstanceData_RT;
	};

	USceneCaptureComponent2D* ResolveComponent(ENDISceneCapture2DSourceMode SourceMode, FNiagaraSystemInstance* SystemInstance, const FNDIInstanceData_GameThread& InstanceData)
	{
		// Find from user parameter binding
		if (SourceMode != ENDISceneCapture2DSourceMode::AttachParentOnly)
		{
			if (UObject* ObjectBinding = InstanceData.UserParamBinding.GetValue())
			{
				if (ObjectBinding->IsA<USceneCaptureComponent2D>())
				{
					return CastChecked<USceneCaptureComponent2D>(ObjectBinding);
				}
				else if ( ObjectBinding->IsA<ASceneCapture2D>() )
				{
					return CastChecked<ASceneCapture2D>(ObjectBinding)->GetCaptureComponent2D();
				}
			}
		}

		// Find from attach parent
		if (SourceMode != ENDISceneCapture2DSourceMode::UserParameterOnly)
		{
			if (USceneComponent* OwnerComponent = SystemInstance->GetAttachComponent())
			{
				for (USceneComponent* AttachComponent=OwnerComponent; AttachComponent; AttachComponent=AttachComponent->GetAttachParent())
				{
					if (USceneCaptureComponent2D* AttachSceneCaptureComponent = Cast<USceneCaptureComponent2D>(AttachComponent))
					{
						return AttachSceneCaptureComponent;
					}
				}
			}
		}

		return nullptr;
	}

	void VMGetTextureSize(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIOutputParam<int32>	OutSizeX(Context);
		FNDIOutputParam<int32>	OutSizeY(Context);
		FNDIOutputParam<int32>	OutNumMips(Context);

		for (int32 i=0; i < Context.GetNumInstances(); ++i)
		{
			OutSizeX.SetAndAdvance(InstanceData->TextureSize.X);
			OutSizeY.SetAndAdvance(InstanceData->TextureSize.Y);
			OutNumMips.SetAndAdvance(InstanceData->TextureSize.Z);
		}
	}

	void VMProject(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<FNiagaraPosition>	InPosition(Context);
		FNDIOutputParam<FVector3f>			OutUVW(Context);

		for (int32 i=0; i < Context.GetNumInstances(); ++i)
		{
			const FVector3f WorldPosition = InPosition.GetAndAdvance();
			const FVector4f ProjPosition = InstanceData->TextureProjMatrix.TransformPosition(WorldPosition);
			const float RcpW = ProjPosition.W > 0.0f ? 1.0f / ProjPosition.W : 1.0f;
			const FVector3f UVW(
				((ProjPosition.X * RcpW) *  0.5f) + 0.5f,
				((ProjPosition.Y * RcpW) * -0.5f) + 0.5f,
				ProjPosition.Z * RcpW
			);

			OutUVW.SetAndAdvance(UVW);
		}
	}

	void VMDeproject(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<FVector2f>			InUV(Context);
		FNDIInputParam<float>				InDepth(Context);
		FNDIOutputParam<FNiagaraPosition>	OutPosition(Context);

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const FVector2f UV = InUV.GetAndAdvance();
			const float Depth = InDepth.GetAndAdvance();

			FVector3f Position;
			Position.X = (UV.X - 0.5f) *  2.0f;
			Position.Y = (UV.Y - 0.5f) * -2.0f;
			Position.Z = Depth;
			if ( InstanceData->ProjectType == ECameraProjectionMode::Perspective )
			{
				Position.X *= Depth;
				Position.Y *= Depth;
			}

			Position = InstanceData->InvTextureProjMatrix.TransformFVector4(FVector4f(Position, 1.0f));

			OutPosition.SetAndAdvance(Position);
		}
	}

	void VMGetCaptureTransform(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIOutputParam<FNiagaraPosition>	OutLocation(Context);
		FNDIOutputParam<FQuat4f>			OutRotation(Context);

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutLocation.SetAndAdvance(InstanceData->CaptureLocation);
			OutRotation.SetAndAdvance(InstanceData->CaptureRotation);
		}
	}

	void VMSetCaptureTransform(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<bool>				InExecute(Context);
		FNDIInputParam<FNiagaraPosition>	InLocation(Context);
		FNDIInputParam<FQuat4f>				InRotation(Context);

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool bExecute = InExecute.GetAndAdvance();
			const FNiagaraPosition Location = InLocation.GetAndAdvance();
			const FQuat4f Rotation = InRotation.GetAndAdvance();

			if (bExecute)
			{
				InstanceData->CaptureLocation = Location;
				InstanceData->CaptureRotation = Rotation;
				InstanceData->bNeedsTransformUpdate = true;
			}
		}
	}

	void VMSetRequestCapture(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<bool>	InRequestCapture(Context);

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool bRequestCapture = InRequestCapture.GetAndAdvance();
			if (bRequestCapture)
			{
				InstanceData->bNeedsCaptureUpdate = true;
			}
		}
	}
}

UNiagaraDataInterfaceSceneCapture2D::UNiagaraDataInterfaceSceneCapture2D(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDISceneCapture2DLocal;

	Proxy.Reset(new FNDIProxy());

	FNiagaraTypeDefinition Def(UObject::StaticClass());
	SceneCaptureUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceSceneCapture2D::PostInitProperties()
{
	using namespace NDISceneCapture2DLocal;

	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);

		ExposedVariable_SceneCapture = FNiagaraVariableBase(FNiagaraTypeDefinition(UTexture::StaticClass()), TEXT("SceneCapture"));
	}
}

bool UNiagaraDataInterfaceSceneCapture2D::CopyToInternal(UNiagaraDataInterface* InDestination) const
{
	if (!Super::CopyToInternal(InDestination))
	{
		return false;
	}
	UNiagaraDataInterfaceSceneCapture2D* Destination = CastChecked<UNiagaraDataInterfaceSceneCapture2D>(InDestination);
	Destination->SourceMode = SourceMode;
	Destination->SceneCaptureUserParameter = SceneCaptureUserParameter;

	Destination->ManagedCaptureSource = ManagedCaptureSource;
	Destination->ManagedTextureSize = ManagedTextureSize;
	Destination->ManagedTextureFormat = ManagedTextureFormat;
	Destination->ManagedProjectionType = ManagedProjectionType;
	Destination->ManagedFOVAngle = ManagedFOVAngle;
	Destination->ManagedOrthoWidth = ManagedOrthoWidth;
	Destination->ManagedLODDistanceFactor = ManagedLODDistanceFactor;
	Destination->bManagedCaptureEveryFrame = bManagedCaptureEveryFrame;
	Destination->bManagedCaptureOnMovement = bManagedCaptureOnMovement;
	Destination->bManagedHideAttachComponent = bManagedHideAttachComponent;
	Destination->ManagedShowOnlyActors = ManagedShowOnlyActors;
	Destination->ManagedHidenActors = ManagedHidenActors;
	Destination->bAutoMoveWithComponent = bAutoMoveWithComponent;
	Destination->AutoMoveOffsetLocationMode = AutoMoveOffsetLocationMode;
	Destination->AutoMoveOffsetLocation = AutoMoveOffsetLocation;
	Destination->AutoMoveOffsetRotationMode = AutoMoveOffsetRotationMode;
	Destination->AutoMoveOffsetRotation = AutoMoveOffsetRotation;

	return true;
}

bool UNiagaraDataInterfaceSceneCapture2D::Equals(const UNiagaraDataInterface* InOther) const
{
	if (!Super::Equals(InOther))
	{
		return false;
	}
	const UNiagaraDataInterfaceSceneCapture2D* Other = CastChecked<const UNiagaraDataInterfaceSceneCapture2D>(InOther);
	return
		Other->SourceMode == SourceMode &&
		Other->SceneCaptureUserParameter == SceneCaptureUserParameter &&
		Other->ManagedCaptureSource == ManagedCaptureSource &&
		Other->ManagedTextureSize == ManagedTextureSize &&
		Other->ManagedTextureFormat == ManagedTextureFormat &&
		Other->ManagedProjectionType == ManagedProjectionType &&
		Other->ManagedFOVAngle == ManagedFOVAngle &&
		Other->ManagedOrthoWidth == ManagedOrthoWidth &&
		Other->ManagedLODDistanceFactor == ManagedLODDistanceFactor &&
		Other->bManagedCaptureEveryFrame == bManagedCaptureEveryFrame &&
		Other->bManagedCaptureOnMovement == bManagedCaptureOnMovement &&
		Other->bManagedHideAttachComponent == bManagedHideAttachComponent &&
		Other->ManagedShowOnlyActors == ManagedShowOnlyActors &&
		Other->ManagedHidenActors == ManagedHidenActors &&
		Other->bAutoMoveWithComponent == bAutoMoveWithComponent &&
		Other->AutoMoveOffsetLocationMode == AutoMoveOffsetLocationMode &&
		Other->AutoMoveOffsetLocation == AutoMoveOffsetLocation &&
		Other->AutoMoveOffsetRotationMode == AutoMoveOffsetRotationMode &&
		Other->AutoMoveOffsetRotation == AutoMoveOffsetRotation;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSceneCapture2D::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDISceneCapture2DLocal;

	FNiagaraFunctionSignature DefaultSignature;
	DefaultSignature.bMemberFunction	= true;
	DefaultSignature.bRequiresContext	= false;
	DefaultSignature.bSupportsCPU		= true;
	DefaultSignature.bSupportsGPU		= true;
	DefaultSignature.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("SceneCapture2D"));

	FNiagaraFunctionSignature DefaultMutableSignature = DefaultSignature;
	DefaultMutableSignature.bRequiresExecPin	= true;
	DefaultMutableSignature.bSupportsGPU		= false;

	FNiagaraFunctionSignature DefaultGpuSignature = DefaultSignature;
	DefaultGpuSignature.bSupportsCPU		= false;

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultSignature);
		Sig.Name = NAME_GetTextureSize;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SizeX"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SizeY"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumMipLevels"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultSignature);
		Sig.Name = NAME_Project;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UVW"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultSignature);
		Sig.Name = NAME_Deproject;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Depth"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultSignature);
		Sig.Name = NAME_GetCaptureTransform;
		Sig.bSupportsGPU = false;	//-TODO: Technically we can support this
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Location"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultMutableSignature);
		Sig.Name = NAME_SetCaptureTransform;
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Location"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultMutableSignature);
		Sig.Name = NAME_SetRequestCapture;
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("RequestCapture")).SetValue(true);
	}	

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultGpuSignature);
		Sig.Name = NAME_TextureLoad;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("TextureLoadDesc", "Load a value from the texture at the pixel location"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultGpuSignature);
		Sig.Name = NAME_TextureSample;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("TextureSampleDesc", "Sample a value from the texture at the UV location"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultGpuSignature);
		Sig.Name = NAME_TextureProject;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("ValidateTextureBounds")).SetValue(true);
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("ValidateDepthBounds")).SetValue(true);
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetColorDef(), TEXT("DefaultColor")).SetValue(FLinearColor::White);
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("InBounds"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Color"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UVW"));
		Sig.SetDescription(LOCTEXT("TextureProjectDesc", "Project the position into the scene capture and sample from the texture."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultGpuSignature);
		Sig.Name = NAME_TextureProjectDepth;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("PointSample")).SetValue(true);
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("ValidateTextureBounds")).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("DefaultDepth"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("InBounds"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CaptureDepth"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("PositionDepth"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
		Sig.SetDescription(LOCTEXT("TextureProjectDepthDesc", "Project the position into the scene capture and sample from the texture."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(DefaultGpuSignature);
		Sig.Name = NAME_TextureDeprojectDepth;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("PointSample")).SetValue(true);
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CaptureDepth"));
		Sig.SetDescription(LOCTEXT("TextureDeprojectDepthDesc", "Deproject the depth sample from the texture into world space."));
	}
}
#endif

void UNiagaraDataInterfaceSceneCapture2D::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	using namespace NDISceneCapture2DLocal;
	if (BindingInfo.Name == NAME_GetTextureSize)
	{
		OutFunc = FVMExternalFunction::CreateStatic(VMGetTextureSize);
	}
	else if (BindingInfo.Name == NAME_Project)
	{
		OutFunc = FVMExternalFunction::CreateStatic(VMProject);
	}
	else if (BindingInfo.Name == NAME_Deproject)
	{
		OutFunc = FVMExternalFunction::CreateStatic(VMDeproject);
	}
	else if (BindingInfo.Name == NAME_GetCaptureTransform)
	{
		OutFunc = FVMExternalFunction::CreateStatic(VMGetCaptureTransform);
	}
	else if (BindingInfo.Name == NAME_SetCaptureTransform)
	{
		OutFunc = FVMExternalFunction::CreateStatic(VMSetCaptureTransform);
	}
	else if (BindingInfo.Name == NAME_SetRequestCapture)
	{
		OutFunc = FVMExternalFunction::CreateStatic(VMSetRequestCapture);
	}
}

int32 UNiagaraDataInterfaceSceneCapture2D::PerInstanceDataSize() const
{
	using namespace NDISceneCapture2DLocal;
	return sizeof(FNDIInstanceData_GameThread);
}

bool UNiagaraDataInterfaceSceneCapture2D::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDISceneCapture2DLocal;
	FNDIInstanceData_GameThread* InstanceData = new (PerInstanceData) FNDIInstanceData_GameThread();
	InstanceData->ChangeId = ChangeId;
	InstanceData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), SceneCaptureUserParameter.Parameter);

	const FTransform InstanceTransform = SystemInstance->GetWorldTransform();
	const FNiagaraLWCConverter LWCConverter = SystemInstance->GetLWCConverter();
	InstanceData->CaptureLocation	= LWCConverter.ConvertWorldToSimulationVector(InstanceTransform.GetLocation());
	InstanceData->CaptureRotation	= FQuat4f(InstanceTransform.GetRotation());
	InstanceData->ProjectType		= ManagedProjectionType;
	InstanceData->FOV				= ManagedFOVAngle;
	InstanceData->OrthoWidth		= ManagedOrthoWidth;

	return true;
}

void UNiagaraDataInterfaceSceneCapture2D::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDISceneCapture2DLocal;

	FNDIInstanceData_GameThread* InstanceData = static_cast<FNDIInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FNDIInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(NDITexture_RemoveInstance)
	(
		[RT_Proxy=GetProxyAs<FNDIProxy>(), RT_InstanceID=SystemInstance->GetId()](FRHICommandListImmediate&)
		{
			RT_Proxy->InstanceData_RT.Remove(RT_InstanceID);
		}
	);

	if (SourceMode == ENDISceneCapture2DSourceMode::Managed)
	{
		TObjectPtr<USceneCaptureComponent2D> SceneCaptureComponent = nullptr;
		ManagedCaptureComponents.RemoveAndCopyValue(SystemInstance->GetId(), SceneCaptureComponent);
		if ( ::IsValid(SceneCaptureComponent) )
		{
			SceneCaptureComponent->UnregisterComponent();
			SceneCaptureComponent->MarkAsGarbage();
		}
	}
}

bool UNiagaraDataInterfaceSceneCapture2D::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDISceneCapture2DLocal;

	// Check to see if a change should force us to recreate the data interface
	// Note: we check the capture component here because if it's not valid we will be recaching anyway
	FNDIInstanceData_GameThread* InstanceData = static_cast<FNDIInstanceData_GameThread*>(PerInstanceData);
	if (InstanceData->WeakCaptureComponent.IsValid() && InstanceData->ChangeId != ChangeId)
	{
		return true;
	}

	InstanceData->ChangeId = ChangeId;
	InstanceData->bNeedsCaptureUpdate = false;
	InstanceData->bNeedsTransformUpdate = bAutoMoveWithComponent;

	// Find the scene capture / texture we are using
	USceneCaptureComponent2D* CaptureComponent = nullptr;
	if (SourceMode == ENDISceneCapture2DSourceMode::Managed)
	{
		InstanceData->bNeedsCaptureUpdate |= bManagedCaptureEveryFrame;
		CaptureComponent = ManagedCaptureComponents.FindOrAdd(SystemInstance->GetId());

		// Create our scene capture component
		if (!::IsValid(CaptureComponent))
		{
			CaptureComponent = NewObject<USceneCaptureComponent2D>(GetTransientPackage(), NAME_None, RF_Transient);
			CaptureComponent->SetComponentTickEnabled(false);

			CaptureComponent->CaptureSource = ManagedCaptureSource;
			CaptureComponent->bCaptureEveryFrame = false;
			CaptureComponent->bCaptureOnMovement = bManagedCaptureOnMovement;
			CaptureComponent->ProjectionType = ManagedProjectionType;
			CaptureComponent->FOVAngle = ManagedFOVAngle;
			CaptureComponent->OrthoWidth = ManagedOrthoWidth;
			CaptureComponent->LODDistanceFactor = ManagedLODDistanceFactor;
			if (ManagedShowOnlyActors.Num() > 0)
			{
				CaptureComponent->ShowOnlyActors = ManagedShowOnlyActors;
				CaptureComponent->PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;
			}
			else
			{
				CaptureComponent->HiddenActors = ManagedHidenActors;
				if (bManagedHideAttachComponent)
				{
					if (UPrimitiveComponent* SceneComponent = Cast<UPrimitiveComponent>(SystemInstance->GetAttachComponent()))
					{
						CaptureComponent->HiddenComponents.Add(SceneComponent);
					}
				}
			}

			CaptureComponent->RegisterComponentWithWorld(SystemInstance->GetWorld());

			InstanceData->bNeedsCaptureUpdate = true;
			InstanceData->bNeedsTransformUpdate = true;
			ManagedCaptureComponents[SystemInstance->GetId()] = CaptureComponent;
		}

		// Create our scene capture render target
		UTextureRenderTarget2D* CaptureTexture = CaptureComponent->TextureTarget;
		if (CaptureTexture == nullptr)
		{
			CaptureTexture = NewObject<UTextureRenderTarget2D>(this);
			//CaptureTexture->bAutoGenerateMips	= InstanceData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled;
			CaptureTexture->RenderTargetFormat = ManagedTextureFormat;
			//CaptureTexture->ClearColor			= FLinearColor(0.0, 0, 0, 0);
			//CaptureTexture->Filter				= InstanceData->Filter;
			CaptureTexture->InitAutoFormat(ManagedTextureSize.X, ManagedTextureSize.Y);
			CaptureTexture->UpdateResourceImmediate(true);

			InstanceData->bNeedsCaptureUpdate = true;
			InstanceData->bNeedsTransformUpdate = true;
			CaptureComponent->TextureTarget = CaptureTexture;
		}
	}
	else
	{
		//-OPT: We can possibly be smarter with picking up this information.
		CaptureComponent = ResolveComponent(SourceMode, SystemInstance, *InstanceData);
	}
	InstanceData->WeakCaptureComponent = CaptureComponent;

	// Auto update transform if required?
	if (InstanceData->bNeedsTransformUpdate)
	{
		FTransform CaptureTransform = SystemInstance->GetWorldTransform();

		switch (AutoMoveOffsetLocationMode)
		{
			case ENDISceneCapture2DOffsetMode::RelativeLocal:	CaptureTransform.SetLocation(CaptureTransform.InverseTransformPosition(AutoMoveOffsetLocation)); break;
			case ENDISceneCapture2DOffsetMode::RelativeWorld:	CaptureTransform.SetLocation(CaptureTransform.GetLocation() + AutoMoveOffsetLocation); break;
			case ENDISceneCapture2DOffsetMode::AbsoluteWorld:	CaptureTransform.SetLocation(AutoMoveOffsetLocation); break;
			default:											CaptureTransform.SetLocation(CaptureTransform.GetLocation()); break;
		}

		switch (AutoMoveOffsetRotationMode)
		{
			case ENDISceneCapture2DOffsetMode::RelativeLocal:	CaptureTransform.SetRotation(AutoMoveOffsetRotation.Quaternion() * CaptureTransform.GetRotation()); break;
			case ENDISceneCapture2DOffsetMode::RelativeWorld:	CaptureTransform.SetRotation(CaptureTransform.GetRotation() * AutoMoveOffsetRotation.Quaternion()); break;
			case ENDISceneCapture2DOffsetMode::AbsoluteWorld:	CaptureTransform.SetRotation(AutoMoveOffsetRotation.Quaternion()); break;
			default:											CaptureTransform.SetRotation(CaptureTransform.GetRotation()); break;
		}

		if (CaptureComponent && !CaptureTransform.Equals(CaptureComponent->GetComponentToWorld()))
		{
			CaptureComponent->SetWorldTransform(CaptureTransform);
		}
		InstanceData->bNeedsTransformUpdate = false;
	}

	if (CaptureComponent)
	{
		InstanceData->UpdateTransformData(DeltaSeconds, SystemInstance, CaptureComponent);
	}

	return false;
}

bool UNiagaraDataInterfaceSceneCapture2D::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDISceneCapture2DLocal;
	FNDIInstanceData_GameThread* InstanceData = static_cast<FNDIInstanceData_GameThread*>(PerInstanceData);

	UTextureRenderTarget2D* CaptureTexture = nullptr;
	if (USceneCaptureComponent2D* CaptureComponent = InstanceData->WeakCaptureComponent.Get())
	{
		CaptureTexture = CaptureComponent->TextureTarget;

		// Do we need a transform update?
		if (InstanceData->bNeedsTransformUpdate)
		{
			const FNiagaraLWCConverter LWCConverter = SystemInstance->GetLWCConverter();
			FTransform CaptureTransform = CaptureComponent->GetComponentToWorld();
			CaptureTransform.SetLocation(LWCConverter.ConvertSimulationPositionToWorld(InstanceData->CaptureLocation));
			CaptureTransform.SetRotation(FQuat(InstanceData->CaptureRotation));
			if (!CaptureComponent->GetComponentToWorld().Equals(CaptureTransform))
			{
				CaptureComponent->SetWorldTransform(CaptureTransform);
			}

			InstanceData->UpdateTransformData(DeltaSeconds, SystemInstance, CaptureComponent);
		}

		// Do we need a capture update?
		if (InstanceData->bNeedsCaptureUpdate)
		{
			CaptureComponent->CaptureSceneDeferred();
		}
	}
	InstanceData->bNeedsTransformUpdate = false;
	InstanceData->bNeedsCaptureUpdate = false;

	// We need to update the texture ideally this is part of the tick but that isn't possible
	// since the resource could have been deleted / changed before the ticks are processed
	if (IsUsedWithGPUScript())
	{
		ENQUEUE_RENDER_COMMAND(NDISceneCapture2D_UpdateInstance)
		(
			[RT_Proxy=GetProxyAs<FNDIProxy>(), RT_InstanceID=SystemInstance->GetId(), RT_Texture=CaptureTexture](FRHICommandListImmediate&)
			{
				FNDInstanceData_RenderThread& InstanceData = RT_Proxy->InstanceData_RT.FindOrAdd(RT_InstanceID);

				if (RT_Texture)
				{
					InstanceData.TextureReferenceRHI = RT_Texture->TextureReference.TextureReferenceRHI;
					if ( RT_Texture->GetResource() )
					{
						InstanceData.SamplerStateRHI = RT_Texture->GetResource()->SamplerStateRHI.GetReference();
					}
					else
					{
						InstanceData.SamplerStateRHI = nullptr;
					}
				}
				else
				{
					InstanceData.TextureReferenceRHI = nullptr;
					InstanceData.SamplerStateRHI = nullptr;
				}
			}
		);
	}

	return false;
}

void UNiagaraDataInterfaceSceneCapture2D::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	using namespace NDISceneCapture2DLocal;
	FNDIProxy::ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, SystemInstance);
}

#if WITH_NIAGARA_DEBUGGER
void UNiagaraDataInterfaceSceneCapture2D::DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const
{
	using namespace NDISceneCapture2DLocal;

	const FNDIInstanceData_GameThread* InstanceData = DebugHudContext.GetSystemInstance()->FindTypedDataInterfaceInstanceData<FNDIInstanceData_GameThread>(this);
	if (InstanceData == nullptr)
	{
		return;
	}

	if (USceneCaptureComponent2D* CaptureComponent = InstanceData->WeakCaptureComponent.Get())
	{
		const FNiagaraLWCConverter LWCConverter = DebugHudContext.GetSystemInstance()->GetLWCConverter();
		const FVector CaptureLocation = LWCConverter.ConvertSimulationPositionToWorld(InstanceData->CaptureLocation);
		const FRotator CaptureRotation = FRotator(FQuat(InstanceData->CaptureRotation));

		DebugHudContext.GetOutputString().Appendf(
			TEXT("CaptureComponent(%s) Location(%5.2f, %5.2f, %5.2f) Rotation(%5.2f, %5.2f, %5.2f)"),
			*GetNameSafe(CaptureComponent),
			CaptureLocation.X, CaptureLocation.Y, CaptureLocation.Z,
			CaptureRotation.Pitch, CaptureRotation.Yaw, CaptureRotation.Roll
		);

		if (DebugHudContext.IsVerbose())
		{
			FMinimalViewInfo ViewInfo;
			CaptureComponent->GetCameraView(0.0f, ViewInfo);

			const FVector ForwardVector = ViewInfo.Rotation.RotateVector(FVector::XAxisVector);
			const FVector LeftVector = ViewInfo.Rotation.RotateVector(FVector::YAxisVector);
			const FVector UpVector = ViewInfo.Rotation.RotateVector(FVector::ZAxisVector);
			const bool IsPerspective = ViewInfo.ProjectionMode == ECameraProjectionMode::Perspective;

			FVector2D ClipPlanes;
			FVector2D NearSize;
			FVector2D FarSize;
			if (IsPerspective)
			{
				const float HalfFOV = FMath::DegreesToRadians(ViewInfo.FOV * 0.5f);
				ClipPlanes = FVector2D(ViewInfo.GetFinalPerspectiveNearClipPlane(), UE_LARGE_WORLD_MAX);
				NearSize = FVector2D(ClipPlanes.X, ClipPlanes.X / ViewInfo.AspectRatio) * FMath::Tan(HalfFOV);
				FarSize = FVector2D(ClipPlanes.Y, ClipPlanes.Y / ViewInfo.AspectRatio) * FMath::Tan(HalfFOV);
			}
			else
			{
				const float HalfOrthoWidth = ViewInfo.OrthoWidth * 0.5f;
				ClipPlanes = FVector2D(ViewInfo.OrthoNearClipPlane, ViewInfo.OrthoFarClipPlane);
				NearSize = FVector2D(HalfOrthoWidth, HalfOrthoWidth / ViewInfo.AspectRatio);
				FarSize = FVector2D(HalfOrthoWidth, HalfOrthoWidth / ViewInfo.AspectRatio);
			}

			const FVector Verts[8] =
			{
				ViewInfo.Location + (ForwardVector * ClipPlanes.X) + (UpVector * NearSize.Y) + (LeftVector * NearSize.X),
				ViewInfo.Location + (ForwardVector * ClipPlanes.X) + (UpVector * NearSize.Y) - (LeftVector * NearSize.X),
				ViewInfo.Location + (ForwardVector * ClipPlanes.X) - (UpVector * NearSize.Y) - (LeftVector * NearSize.X),
				ViewInfo.Location + (ForwardVector * ClipPlanes.X) - (UpVector * NearSize.Y) + (LeftVector * NearSize.X),

				ViewInfo.Location + (ForwardVector * ClipPlanes.Y) + (UpVector * FarSize.Y) + (LeftVector * FarSize.X),
				ViewInfo.Location + (ForwardVector * ClipPlanes.Y) + (UpVector * FarSize.Y) - (LeftVector * FarSize.X),
				ViewInfo.Location + (ForwardVector * ClipPlanes.Y) - (UpVector * FarSize.Y) - (LeftVector * FarSize.X),
				ViewInfo.Location + (ForwardVector * ClipPlanes.Y) - (UpVector * FarSize.Y) + (LeftVector * FarSize.X),
			};

			const FColor FrustumColor = FColor::Red;
			DrawDebugLine(DebugHudContext.GetWorld(), Verts[0], Verts[1], FrustumColor);
			DrawDebugLine(DebugHudContext.GetWorld(), Verts[1], Verts[2], FrustumColor);
			DrawDebugLine(DebugHudContext.GetWorld(), Verts[2], Verts[3], FrustumColor);
			DrawDebugLine(DebugHudContext.GetWorld(), Verts[3], Verts[0], FrustumColor);

			DrawDebugLine(DebugHudContext.GetWorld(), Verts[4], Verts[5], FrustumColor);
			DrawDebugLine(DebugHudContext.GetWorld(), Verts[5], Verts[6], FrustumColor);
			DrawDebugLine(DebugHudContext.GetWorld(), Verts[6], Verts[7], FrustumColor);
			DrawDebugLine(DebugHudContext.GetWorld(), Verts[7], Verts[4], FrustumColor);

			DrawDebugLine(DebugHudContext.GetWorld(), Verts[0], Verts[4], FrustumColor);
			DrawDebugLine(DebugHudContext.GetWorld(), Verts[1], Verts[5], FrustumColor);
			DrawDebugLine(DebugHudContext.GetWorld(), Verts[2], Verts[6], FrustumColor);
			DrawDebugLine(DebugHudContext.GetWorld(), Verts[3], Verts[7], FrustumColor);
		}
	}
	else
	{
		DebugHudContext.GetOutputString() = TEXT("Null Capture Component");
	}
}
#endif

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceSceneCapture2D::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace NDISceneCapture2DLocal;

	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceSceneCapture2D::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDISceneCapture2DLocal::TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceSceneCapture2D::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDISceneCapture2DLocal;

	static const TSet<FName> ValidGpuFunctions =
	{
		NAME_GetTextureSize,
		NAME_Project,
		NAME_Deproject,
		NAME_TextureLoad,
		NAME_TextureSample,
		NAME_TextureProject,
		NAME_TextureProjectDepth,
		NAME_TextureDeprojectDepth,
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}
#endif

void UNiagaraDataInterfaceSceneCapture2D::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NDISceneCapture2DLocal;
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceSceneCapture2D::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDISceneCapture2DLocal;

	FNDIProxy& TextureProxy = Context.GetProxy<FNDIProxy>();
	FNDInstanceData_RenderThread* InstanceData = TextureProxy.InstanceData_RT.Find(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters		= Context.GetParameterNestedStruct<FShaderParameters>();
	if (InstanceData->TextureReferenceRHI.IsValid())
	{
		ShaderParameters->Texture			= InstanceData->TextureReferenceRHI;
	}
	else
	{
		ShaderParameters->Texture			= GWhiteTexture->TextureRHI;
	}
	ShaderParameters->TextureSampler		= InstanceData->SamplerStateRHI.IsValid() ? InstanceData->SamplerStateRHI.GetReference() : TStaticSamplerState<SF_Bilinear>::GetRHI();
	ShaderParameters->TextureSize			= InstanceData->TextureSize;
	ShaderParameters->TextureViewMatrix		= InstanceData->TextureViewMatrix;
	ShaderParameters->TextureProjMatrix		= InstanceData->TextureProjMatrix;
	ShaderParameters->InvTextureProjMatrix	= InstanceData->InvTextureProjMatrix;
	ShaderParameters->IsPerspective			= InstanceData->ProjectType == ECameraProjectionMode::Perspective ? 1 : 0;
}

void UNiagaraDataInterfaceSceneCapture2D::GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const
{
	using namespace NDISceneCapture2DLocal;

	OutVariables.Emplace(ExposedVariable_SceneCapture);
}

bool UNiagaraDataInterfaceSceneCapture2D::GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const
{
	using namespace NDISceneCapture2DLocal;

	if (InVariable.IsValid() && InVariable == ExposedVariable_SceneCapture && InPerInstanceData)
	{
		const USceneCaptureComponent2D* SceneCapture = ManagedCaptureComponents.FindRef(InSystemInstance->GetId());
		if (SceneCapture && SceneCapture->TextureTarget)
		{
			UObject** Var = (UObject**)OutData;
			*Var = SceneCapture->TextureTarget;
			return true;
		}
	}
	return false;
}

void UNiagaraDataInterfaceSceneCapture2D::SetSceneCapture2DManagedShowOnlyActors(UNiagaraComponent* NiagaraComponent, const FName ParameterName, TArray<AActor*> ShowOnlyActors)
{
	if ( !NiagaraComponent )
	{
		return;
	}

	const FNiagaraParameterStore& OverrideParameters = NiagaraComponent->GetOverrideParameters();
	const FNiagaraVariable Variable(FNiagaraTypeDefinition(UNiagaraDataInterfaceSceneCapture2D::StaticClass()), ParameterName);
	UNiagaraDataInterfaceSceneCapture2D* DataInterface = Cast<UNiagaraDataInterfaceSceneCapture2D>(OverrideParameters.GetDataInterface(Variable));
	if (DataInterface == nullptr)
	{
		return;
	}

	DataInterface->ManagedShowOnlyActors = ShowOnlyActors;
	++DataInterface->ChangeId;
}

#undef LOCTEXT_NAMESPACE
