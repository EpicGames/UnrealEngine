// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "Components/SceneCaptureComponent2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "NiagaraDataInterfaceSceneCapture2D.generated.h"

struct FNiagaraDataInterfaceGeneratedFunction;

UENUM()
enum class ENDISceneCapture2DSourceMode : uint8
{
	/* Check the user parameter then the attach parent to find the scene capture component. */
	UserParameterThenAttachParent,
	/* Only check the user parameter to find the scene capture component. */
	UserParameterOnly,
	/* Only look at the attach parent to find the scene capture component. */
	AttachParentOnly,
	/* Managed mode, we will not search for any and instead create one internally. */
	Managed,
};

UENUM()
enum class ENDISceneCapture2DOffsetMode : uint8
{
	/** The offset is disabled and will not be applied. */
	Disabled,
	/** The offset is in the component local space. */
	RelativeLocal,
	/** The offset is in world space, i.e. added to the exising value directly. */
	RelativeWorld,
	/** The offset is applied directly, i.e. not added to the existing valud */
	AbsoluteWorld,
};

/** Data Interface which can control or read from a scene capture component. */
UCLASS(Experimental, EditInlineNew, Category = "SceneCapture", meta = (DisplayName = "Scene Capture 2D"), MinimalAPI)
class UNiagaraDataInterfaceSceneCapture2D : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "SceneCapture", meta = (ToolTip = "How should we find the scene capture component to use?"))
	ENDISceneCapture2DSourceMode SourceMode = ENDISceneCapture2DSourceMode::UserParameterThenAttachParent;

	UPROPERTY(EditAnywhere, Category = "SceneCapture", meta = (ToolTip = "When valid should point to either a Scene Capture 2D Component or a Scene Capture 2D Actor.", EditCondition = "SourceMode != ENDISceneCapture2DSourceMode::AttachParentOnly && SourceMode != ENDISceneCapture2DSourceMode::Managed"))
	FNiagaraUserParameterBinding SceneCaptureUserParameter;

	/** When enabled the scene capture component will be automatically moved to the location of the NiagaraComponent with an optional offset. */
	UPROPERTY(EditAnywhere, Category = "SceneCaptureAutoMove")
	bool bAutoMoveWithComponent = false;

	/** Should we apply the auto move offset in local or world space? */
	UPROPERTY(EditAnywhere, Category = "SceneCaptureAutoMove", meta = (EditConditionHides, EditCondition = "bAutoMoveWithComponent"))
	ENDISceneCapture2DOffsetMode AutoMoveOffsetLocationMode = ENDISceneCapture2DOffsetMode::Disabled;

	/** Location offset when applying auto movement. */
	UPROPERTY(EditAnywhere, Category = "SceneCaptureAutoMove", meta = (EditConditionHides, EditCondition = "bAutoMoveWithComponent && AutoMoveOffsetLocationMode != ENDISceneCapture2DOffsetMode::Disabled"))
	FVector AutoMoveOffsetLocation = FVector::ZeroVector;

	/** How we should apply the rotation */
	UPROPERTY(EditAnywhere, Category = "SceneCaptureAutoMove", meta = (EditConditionHides, EditCondition = "bAutoMoveWithComponent"))
	ENDISceneCapture2DOffsetMode AutoMoveOffsetRotationMode = ENDISceneCapture2DOffsetMode::Disabled;

	/** Rotation offset when applying auto movement. */
	UPROPERTY(EditAnywhere, Category = "SceneCaptureAutoMove", meta = (EditConditionHides, EditCondition = "bAutoMoveWithComponent && AutoMoveOffsetRotationMode != ENDISceneCapture2DOffsetMode::Disabled"))
	FRotator AutoMoveOffsetRotation = FRotator::ZeroRotator;

	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed"))
	TEnumAsByte<ESceneCaptureSource> ManagedCaptureSource = ESceneCaptureSource::SCS_SceneColorHDR;

	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed"))
	FIntPoint ManagedTextureSize = FIntPoint(128, 128);

	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed"))
	TEnumAsByte<ETextureRenderTargetFormat> ManagedTextureFormat = RTF_RGBA8;

	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed"))
	TEnumAsByte<ECameraProjectionMode::Type> ManagedProjectionType = ECameraProjectionMode::Perspective;

	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed && ManagedProjectionType == ECameraProjectionMode::Perspective"))
	float ManagedFOVAngle = 90.0f;

	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed && ManagedProjectionType != ECameraProjectionMode::Perspective"))
	float ManagedOrthoWidth = 512.0f;

	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed"))
	float ManagedLODDistanceFactor = 1.0f;

	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed"))
	bool bManagedCaptureEveryFrame = true;
	
	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed"))
	bool bManagedCaptureOnMovement = false;

	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed"))
	bool bManagedHideAttachComponent = true;

	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed"))
	TArray<TObjectPtr<AActor>> ManagedHidenActors;

	UPROPERTY(EditAnywhere, Category = "SceneCaptureManaged", meta = (EditConditionHides, EditCondition = "SourceMode == ENDISceneCapture2DSourceMode::Managed"))
	TArray<TObjectPtr<AActor>> ManagedShowOnlyActors;

	/** Used to track changes from BP for managed components */
	uint32 ChangeId = 0;

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	//UObject Interface End

	//UNiagaraDataInterface Interface
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target==ENiagaraSimTarget::GPUComputeSim; }

	NIAGARA_API virtual int32 PerInstanceDataSize() const override;
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;

	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	NIAGARA_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;

	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

#if WITH_NIAGARA_DEBUGGER
	NIAGARA_API virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const override;
#endif

	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	virtual bool CanExposeVariables() const override { return true; }
	NIAGARA_API virtual void GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const override;
	NIAGARA_API virtual bool GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const override;
	//UNiagaraDataInterface Interface End

	/** Allows you to set the show only actors when Niagara manages the component.  If Niagara does not manage the component use the regular BP methods. */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	static NIAGARA_API void SetSceneCapture2DManagedShowOnlyActors(UNiagaraComponent* NiagaraSystem, const FName ParameterName, TArray<AActor*> ShowOnlyActors);

	UPROPERTY(Transient, DuplicateTransient)
	TMap<uint64, TObjectPtr<USceneCaptureComponent2D>>	ManagedCaptureComponents;
};
