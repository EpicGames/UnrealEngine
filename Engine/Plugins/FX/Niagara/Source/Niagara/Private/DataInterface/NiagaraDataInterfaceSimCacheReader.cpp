// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSimCacheReader.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraRenderer.h"
#include "NiagaraSimCache.h"
#include "NiagaraSimCacheAttributeReaderHelper.h"
#include "NiagaraSimCacheGpuResource.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"

#include "Internationalization/Internationalization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceSimCacheReader)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceSimCacheReader"

//////////////////////////////////////////////////////////////////////////

namespace NDISimCacheReaderLocal
{
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceSimCacheReaderTemplate.ush");
	static const FName GetNumFrames("GetNumFrames");
	static const FName GetNumEmitters("GetNumEmitters");
	static const FName GetEmitterIndex("GetEmitterIndex");
	static const FName GetNumInstances("GetNumInstances");
	static const FName GetInstanceInt("GetInstanceInt");
	static const FName GetInstanceFloat("GetInstanceFloat");
	static const FName GetInstanceVector2("GetInstanceVector2");
	static const FName GetInstanceVector3("GetInstanceVector3");
	static const FName GetInstanceVector4("GetInstanceVector4");
	static const FName GetInstanceColor("GetInstanceColor");
	static const FName GetInstanceQuat("GetInstanceQuat");
	static const FName GetInstancePosition("GetInstancePosition");
	static const FName GetInstanceID("GetInstanceID");
	static const FName GetInstanceIndexFromID("GetInstanceIndexFromID");

	static const FName NAME_Attribute("Attribute");

	struct FShaderStorage : public FNiagaraDataInterfaceParametersCS
	{
		DECLARE_TYPE_LAYOUT(FShaderStorage, NonVirtual);

		LAYOUT_FIELD(TMemoryImageArray<FMemoryImageName>, AttributeReadNames);
	};
	IMPLEMENT_TYPE_LAYOUT(FShaderStorage);

	struct FInstanceData_GameThread
	{
		FNiagaraParameterDirectBinding<UObject*>	SimCacheBinding;
		TWeakObjectPtr<UNiagaraSimCache>			WeakSimCache;
		int32										EmitterIndex = INDEX_NONE;
	};

	struct FInstanceData_RenderThread
	{
		int32							EmitterIndex = INDEX_NONE;
		FNiagaraSimCacheGpuResourcePtr	SimCacheResource;
		TMap<uintptr_t, TArray<int32>>	StorageToComponentOffsets;
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const { return 0; }
		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) {}

		TMap<FNiagaraSystemInstanceID, FInstanceData_RenderThread>	PerInstanceData_RenderThread;
	};

	struct FGetFunctionInfo
	{
		explicit FGetFunctionInfo(FName InFunctionName, FNiagaraTypeDefinition InTypeDef, const TCHAR* InHlslType, const TCHAR* InHlslFunction)
			: FunctionName(InFunctionName)
			, TypeDef(InTypeDef)
			, HlslType(InHlslType)
			, HlslFunction(InHlslFunction)
		{
		}

		FName					FunctionName;
		FNiagaraTypeDefinition	TypeDef;
		const TCHAR*			HlslType = nullptr;
		const TCHAR*			HlslFunction = nullptr;
	};

	TConstArrayView<FGetFunctionInfo> GetFunctionInfos()
	{
		static const FGetFunctionInfo Infos[] =
		{
			FGetFunctionInfo(GetInstanceInt,		FNiagaraTypeDefinition::GetIntDef(),		TEXT("int"),		TEXT("GetInstanceInt")),
			FGetFunctionInfo(GetInstanceFloat,		FNiagaraTypeDefinition::GetFloatDef(),		TEXT("float"),		TEXT("GetInstanceFloat")),
			FGetFunctionInfo(GetInstanceVector2,	FNiagaraTypeDefinition::GetVec2Def(),		TEXT("float2"),		TEXT("GetInstanceFloat2")),
			FGetFunctionInfo(GetInstanceVector3,	FNiagaraTypeDefinition::GetVec3Def(),		TEXT("float3"),		TEXT("GetInstanceFloat3")),
			FGetFunctionInfo(GetInstanceVector4,	FNiagaraTypeDefinition::GetVec4Def(),		TEXT("float4"),		TEXT("GetInstanceFloat4")),
			FGetFunctionInfo(GetInstanceColor,		FNiagaraTypeDefinition::GetColorDef(),		TEXT("float4"),		TEXT("GetInstanceFloat4")),
			FGetFunctionInfo(GetInstanceQuat,		FNiagaraTypeDefinition::GetQuatDef(),		TEXT("float4"),		TEXT("GetInstanceFloat4")),
			FGetFunctionInfo(GetInstancePosition,	FNiagaraTypeDefinition::GetPositionDef(),	TEXT("float3"),		TEXT("GetInstanceFloat3")),
			FGetFunctionInfo(GetInstanceID,			FNiagaraTypeDefinition::GetIDDef(),			TEXT("NiagaraID"),	TEXT("GetInstanceID")),
		};
		return MakeArrayView(Infos);
	}

	static FNiagaraTypeDefinition FunctionNameToTypeDef(FName FunctionName)
	{
		for ( const FGetFunctionInfo& GetFuncInfo : GetFunctionInfos())
		{
			if (GetFuncInfo.FunctionName == FunctionName)
			{
				return GetFuncInfo.TypeDef;
			}
		}

		if (FunctionName == GetInstanceIndexFromID)
		{
			return FNiagaraTypeDefinition::GetIDDef();
		}

		check(false);
		return FNiagaraTypeDefinition::GetFloatDef();
	}

	static int AttributeIndexFromVariable(const UNiagaraSimCache* SimCache, FName FunctionName, FName AttributeName, int EmitterIndex)
	{
		const FNiagaraVariableBase Variable(FunctionNameToTypeDef(FunctionName), AttributeName);
		return FNiagaraSimCacheAttributeReaderHelper::FindVariableIndex(SimCache, Variable, EmitterIndex);
	}

	static int ComponentIndexFromVariable(const UNiagaraSimCache* SimCache, FName FunctionName, FName AttributeName, int EmitterIndex)
	{
		const FNiagaraVariableBase Variable(FunctionNameToTypeDef(FunctionName), AttributeName);
		if ( const FNiagaraSimCacheVariable* CacheVariable = FNiagaraSimCacheAttributeReaderHelper::FindVariable(SimCache, Variable, EmitterIndex) )
		{
			if (Variable.GetType() == FNiagaraTypeDefinition::GetIntDef() || Variable.GetType() == FNiagaraTypeDefinition::GetIDDef())
			{
				return CacheVariable->Int32Offset;
			}
			else
			{
				return CacheVariable->FloatOffset;
			}
		}
		return INDEX_NONE;
	}

	template<typename TType>
	static void VMInstanceGetFunction(FVectorVMExternalFunctionContext& Context, const TType DefaultValue, int32 AttributeIndex, TFunction<TType(FNiagaraSimCacheAttributeReaderHelper&, int32)> ReadFunction)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData_GT(Context);
		FNDIInputParam<int32> InFrameIndex(Context);
		FNDIInputParam<int32> InInstanceIndex(Context);
		FNDIOutputParam<bool> OutValid(Context);
		FNDIOutputParam<TType> OutValue(Context);

		if (UNiagaraSimCache* InstanceSimCache = InstanceData_GT->WeakSimCache.Get())
		{
			FNiagaraSimCacheAttributeReaderHelper AttributeReader;
			int CachedFrameIndex = INDEX_NONE;
			
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				const int32 FrameIndex = InFrameIndex.GetAndAdvance();
				const int32 InstanceIndex = InInstanceIndex.GetAndAdvance();

				if (CachedFrameIndex != FrameIndex)
				{
					CachedFrameIndex = FrameIndex;
					AttributeReader.Initialize(InstanceSimCache, InstanceData_GT->EmitterIndex, AttributeIndex, FrameIndex);
				}

				if ( AttributeReader.IsValid() && InstanceIndex >= 0 && InstanceIndex < int32(AttributeReader.DataBuffers->NumInstances) )
				{
					OutValid.SetAndAdvance(true);
					OutValue.SetAndAdvance(ReadFunction(AttributeReader, InstanceIndex));
				}
				else
				{
					OutValid.SetAndAdvance(false);
					OutValue.SetAndAdvance(DefaultValue);
				}
			}
		}
		else
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				OutValid.SetAndAdvance(false);
				OutValue.SetAndAdvance(DefaultValue);
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceSimCacheReader::UNiagaraDataInterfaceSimCacheReader(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDISimCacheReaderLocal;

	SimCacheBinding.Parameter.SetType(FNiagaraTypeDefinition::GetSimCacheClassDef());

	Proxy.Reset(new FNDIProxy());
}

void UNiagaraDataInterfaceSimCacheReader::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceSimCacheReader::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	auto* OtherTyped = CastChecked<const UNiagaraDataInterfaceSimCacheReader>(Other);
	return
		OtherTyped->SimCacheBinding == SimCacheBinding &&
		OtherTyped->SimCache == SimCache &&
		OtherTyped->EmitterBinding == EmitterBinding;
}

bool UNiagaraDataInterfaceSimCacheReader::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	auto* DestinationTyped = CastChecked<UNiagaraDataInterfaceSimCacheReader>(Destination);
	DestinationTyped->SimCacheBinding = SimCacheBinding;
	DestinationTyped->SimCache = SimCache;
	DestinationTyped->EmitterBinding = EmitterBinding;

	return true;
}

bool UNiagaraDataInterfaceSimCacheReader::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDISimCacheReaderLocal;

	FInstanceData_GameThread* InstanceData_GT = new(PerInstanceData) FInstanceData_GameThread();
	InstanceData_GT->SimCacheBinding.Init(SystemInstance->GetInstanceParameters(), SimCacheBinding.Parameter);

	return true;
}

void UNiagaraDataInterfaceSimCacheReader::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDISimCacheReaderLocal;

	FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
	InstanceData_GT->~FInstanceData_GameThread();

	if ( IsUsedWithGPUScript() )
	{
		ENQUEUE_RENDER_COMMAND(NDISimCacheReader_DestroyRT)
		(
			[Proxy_RT=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandList& CmdList)
			{
				Proxy_RT->PerInstanceData_RenderThread.Remove(InstanceID);
			}
		);
	}
}

int32 UNiagaraDataInterfaceSimCacheReader::PerInstanceDataSize() const
{
	using namespace NDISimCacheReaderLocal;
	return sizeof(FInstanceData_GameThread);
}

bool UNiagaraDataInterfaceSimCacheReader::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDISimCacheReaderLocal;

	check(PerInstanceData && SystemInstance);

	FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);

	// Has the cache changed at all?
	UNiagaraSimCache* InstanceSimCache = InstanceData_GT->SimCacheBinding.GetValueOrDefault<UNiagaraSimCache>(SimCache.Get());
	if (InstanceData_GT->WeakSimCache.Get() != InstanceSimCache)
	{
		InstanceData_GT->EmitterIndex = InstanceSimCache->GetEmitterIndex(EmitterBinding);

		FNiagaraSimCacheGpuResourcePtr SimCacheResource = FNiagaraSimCacheGpuResource::CreateResource(InstanceSimCache);

		ENQUEUE_RENDER_COMMAND(NDISimCacheReader_InitRT)
		(
			[Proxy_RT=GetProxyAs<FNDIProxy>(), InstanceID_RT=SystemInstance->GetId(), EmitterIndex_RT=InstanceData_GT->EmitterIndex, SimCacheResource_RT=MoveTemp(SimCacheResource)](FRHICommandList& CmdList)
			{
				FInstanceData_RenderThread* InstanceData_RT = &Proxy_RT->PerInstanceData_RenderThread.FindOrAdd(InstanceID_RT);
				InstanceData_RT->SimCacheResource = SimCacheResource_RT;
				InstanceData_RT->EmitterIndex = EmitterIndex_RT;
			}
		);
	}
	else if (InstanceSimCache == nullptr)
	{
		InstanceData_GT->EmitterIndex = INDEX_NONE;
		if ( IsUsedWithGPUScript() )
		{
			ENQUEUE_RENDER_COMMAND(NDISimCacheReader_DestroyRT)
			(
				[Proxy_RT=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandList& CmdList)
				{
					Proxy_RT->PerInstanceData_RenderThread.Remove(InstanceID);
				}
			);
		}
	}
	InstanceData_GT->WeakSimCache = InstanceSimCache;

	return false;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSimCacheReader::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDISimCacheReaderLocal;

	// Reserve space
	OutFunctions.Reserve(OutFunctions.Num() + 3 + GetFunctionInfos().Num());

	// Build default signature
	FNiagaraFunctionSignature DefaultSignature;
	DefaultSignature.bMemberFunction = true;
	DefaultSignature.bRequiresContext = false;
	DefaultSignature.bSupportsCPU = true;
	DefaultSignature.bSupportsGPU = true;
	DefaultSignature.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("SimCacheReader"));

	// Utility functions
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetNumFrames;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumFrames"));
		Sig.SetDescription(LOCTEXT("GetNumFramesDesc", "Returns the number of frames stored inside the cache"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetNumEmitters;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumEmitters"));
		Sig.SetDescription(LOCTEXT("GetNumEmittersDesc", "Returns the number of emitters stored inside the cache"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetEmitterIndex;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("EmitterIndex"));
		Sig.SetDescription(LOCTEXT("GetEmitterIndexDesc", "Returns the emitter index the data interface is bound to"));		
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetNumInstances;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("FrameIndex"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("InstanceCount"));
		Sig.SetDescription(LOCTEXT("GetNumInstancesDesc", "Returns the number of instances for the provided frame."));
	}

	// Attribute Reading functions
	FNiagaraFunctionSignature AttributeFunctionSignature = DefaultSignature;
	AttributeFunctionSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("FrameIndex"));
	AttributeFunctionSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("InstanceIndex"));
	AttributeFunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
	AttributeFunctionSignature.FunctionSpecifiers.Emplace(NAME_Attribute);
	AttributeFunctionSignature.SetDescription(LOCTEXT("GetInstanceDesc", "Returns the instance value, valid will be false if the instance / frame / emitter were not valid or out of range."));

	for (const FGetFunctionInfo& GenFuncInfo : GetFunctionInfos())
	{
		FNiagaraFunctionSignature & Sig = OutFunctions.Add_GetRef(AttributeFunctionSignature);
		Sig.Name = GenFuncInfo.FunctionName;
		Sig.Outputs.Emplace(GenFuncInfo.TypeDef, TEXT("Value"));
	}

	// Index from ID
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetInstanceIndexFromID;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("FrameIndex"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIDDef(), TEXT("ID"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("InstanceIndex"));
		Sig.FunctionSpecifiers.Emplace(NAME_Attribute);
		Sig.bExperimental = true;
	#if WITH_EDITORONLY_DATA
		Sig.ExperimentalMessage = LOCTEXT("GetInstanceIndexFromIDExperimental", "This functions is experimental and not optimal as it may perform a linear search.");
	#endif
		Sig.SetDescription(LOCTEXT("GetInstanceIndexFromIDDesc", "Returns the Instance Index if the ID was found otherwise -1."));
	}
}
#endif

void UNiagaraDataInterfaceSimCacheReader::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* PerInstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDISimCacheReaderLocal;

	FInstanceData_GameThread* InstanceData = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
	UNiagaraSimCache* InstanceSimCache = InstanceData->WeakSimCache.Get();
	if ( BindingInfo.Name == GetNumFrames )
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetNumFrames);
	}
	else if (BindingInfo.Name == GetNumEmitters)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetNumEmitters);
	}
	else if (BindingInfo.Name == GetEmitterIndex)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetEmitterIndex);
	}
	else if (BindingInfo.Name == GetNumInstances)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetNumInstances);
	}
	else if (BindingInfo.Name == GetInstanceInt)
	{
		const int32 AttributeIndex = AttributeIndexFromVariable(InstanceSimCache, BindingInfo.Name, BindingInfo.FunctionSpecifiers[0].Value, InstanceData->EmitterIndex);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetInstanceInt, AttributeIndex);
	}
	else if (BindingInfo.Name == GetInstanceFloat)
	{
		const int32 AttributeIndex = AttributeIndexFromVariable(InstanceSimCache, BindingInfo.Name, BindingInfo.FunctionSpecifiers[0].Value, InstanceData->EmitterIndex);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetInstanceFloat, AttributeIndex);
	}
	else if (BindingInfo.Name == GetInstanceVector2)
	{
		const int32 AttributeIndex = AttributeIndexFromVariable(InstanceSimCache, BindingInfo.Name, BindingInfo.FunctionSpecifiers[0].Value, InstanceData->EmitterIndex);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetInstanceVector2, AttributeIndex);
	}
	else if (BindingInfo.Name == GetInstanceVector3)
	{
		const int32 AttributeIndex = AttributeIndexFromVariable(InstanceSimCache, BindingInfo.Name, BindingInfo.FunctionSpecifiers[0].Value, InstanceData->EmitterIndex);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetInstanceVector3, AttributeIndex);
	}
	else if (BindingInfo.Name == GetInstanceVector4)
	{
		const int32 AttributeIndex = AttributeIndexFromVariable(InstanceSimCache, BindingInfo.Name, BindingInfo.FunctionSpecifiers[0].Value, InstanceData->EmitterIndex);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetInstanceVector4, AttributeIndex);
	}
	else if (BindingInfo.Name == GetInstanceColor)
	{
		const int32 AttributeIndex = AttributeIndexFromVariable(InstanceSimCache, BindingInfo.Name, BindingInfo.FunctionSpecifiers[0].Value, InstanceData->EmitterIndex);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetInstanceColor, AttributeIndex);
	}
	else if (BindingInfo.Name == GetInstanceQuat)
	{
		const int32 AttributeIndex = AttributeIndexFromVariable(InstanceSimCache, BindingInfo.Name, BindingInfo.FunctionSpecifiers[0].Value, InstanceData->EmitterIndex);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetInstanceQuat, AttributeIndex, false);
	}
	else if (BindingInfo.Name == GetInstancePosition)
	{
		const int32 AttributeIndex = AttributeIndexFromVariable(InstanceSimCache, BindingInfo.Name, BindingInfo.FunctionSpecifiers[0].Value, InstanceData->EmitterIndex);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetInstancePosition, AttributeIndex, false);
	}
	else if (BindingInfo.Name == GetInstanceID)
	{
		const int32 AttributeIndex = AttributeIndexFromVariable(InstanceSimCache, BindingInfo.Name, BindingInfo.FunctionSpecifiers[0].Value, InstanceData->EmitterIndex);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetInstanceID, AttributeIndex);
	}
	else if (BindingInfo.Name == GetInstanceIndexFromID)
	{
		const int32 AttributeIndex = AttributeIndexFromVariable(InstanceSimCache, BindingInfo.Name, BindingInfo.FunctionSpecifiers[0].Value, InstanceData->EmitterIndex);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimCacheReader::VMGetInstanceIndexFromID, AttributeIndex);
	}
	
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceSimCacheReader::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace NDISimCacheReaderLocal;

	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(NDISimCacheReaderLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<FShaderParameters>();
	InVisitor->UpdateString(TEXT("FShaderStorage"), TEXT("V0"));
	return bSuccess;
}

void UNiagaraDataInterfaceSimCacheReader::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParameterInfo.DataInterfaceHLSLSymbol},
	};

	OutHLSL.Appendf(TEXT("int4 %s_ComponentOffsets[%d];\n"), *ParameterInfo.DataInterfaceHLSLSymbol, FMath::DivideAndRoundUp(ParameterInfo.GeneratedFunctions.Num(), 4));

	AppendTemplateHLSL(OutHLSL, NDISimCacheReaderLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceSimCacheReader::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDISimCacheReaderLocal;

	const int32 Float4Index = FunctionInstanceIndex / 4;
	const int32 ElementIndex = FunctionInstanceIndex % 4;

	for (const FGetFunctionInfo& GenFuncInfo : GetFunctionInfos())
	{
		if ( GenFuncInfo.FunctionName == FunctionInfo.DefinitionName )
		{
			OutHLSL.Appendf(TEXT("void %s(int FrameIndex, int InstanceIndex, out bool bValid, out %s Value)\n"), *FunctionInfo.InstanceName, GenFuncInfo.HlslType);
			OutHLSL.Append(TEXT("{\n"));
			OutHLSL.Appendf(TEXT("	int AttributeIndex = %s_ComponentOffsets[%d][%d];\n"), *ParamInfo.DataInterfaceHLSLSymbol, Float4Index, ElementIndex);
			OutHLSL.Appendf(TEXT("	%s_%s(FrameIndex, %s_EmitterIndex, AttributeIndex, InstanceIndex, bValid, Value);\n"), GenFuncInfo.HlslFunction, *ParamInfo.DataInterfaceHLSLSymbol, *ParamInfo.DataInterfaceHLSLSymbol);
			OutHLSL.Append(TEXT("}\n"));
			return true;
		}
	}

	if (FunctionInfo.DefinitionName == GetInstanceIndexFromID)
	{
		OutHLSL.Appendf(TEXT("void %s(int FrameIndex, NiagaraID ID, out bool bValid, out int Value)\n"), *FunctionInfo.InstanceName);
		OutHLSL.Append(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	int AttributeIndex = %s_ComponentOffsets[%d][%d];\n"), *ParamInfo.DataInterfaceHLSLSymbol, Float4Index, ElementIndex);
		OutHLSL.Appendf(TEXT("	GetInstanceIndexFromID_%s(FrameIndex, %s_EmitterIndex, AttributeIndex, ID, bValid, Value);\n"), *ParamInfo.DataInterfaceHLSLSymbol, *ParamInfo.DataInterfaceHLSLSymbol);
		OutHLSL.Append(TEXT("}\n"));
		return true;
	}

	return true;
}
#endif

FNiagaraDataInterfaceParametersCS* UNiagaraDataInterfaceSimCacheReader::CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const
{
	using namespace NDISimCacheReaderLocal;

	FShaderStorage* ShaderStorage = new FShaderStorage();
	ShaderStorage->AttributeReadNames.Reserve(ParameterInfo.GeneratedFunctions.Num() * 2);
	for (const FNiagaraDataInterfaceGeneratedFunction& GeneratedFunction : ParameterInfo.GeneratedFunctions)
	{
		if (GeneratedFunction.Specifiers.Num() == 1)
		{
			ShaderStorage->AttributeReadNames.Add(GeneratedFunction.DefinitionName);
			ShaderStorage->AttributeReadNames.Add(GeneratedFunction.Specifiers[0].Value);
		}
		else
		{
			ShaderStorage->AttributeReadNames.Add(NAME_None);
			ShaderStorage->AttributeReadNames.Add(NAME_None);
		}
	}
	return ShaderStorage;
}

const FTypeLayoutDesc* UNiagaraDataInterfaceSimCacheReader::GetShaderStorageType() const
{
	using namespace NDISimCacheReaderLocal;
	return &StaticGetTypeLayoutDesc<FShaderStorage>();
}

void UNiagaraDataInterfaceSimCacheReader::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();

	const int32 ComponentOffsetsSize = FMath::Max(FMath::DivideAndRoundUp(ShaderParametersBuilder.GetGeneratedFunctions().Num(), 4), 1);
	ShaderParametersBuilder.AddLooseParamArray<FUintVector4>(TEXT("ComponentOffsets"), ComponentOffsetsSize);
}

void UNiagaraDataInterfaceSimCacheReader::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDISimCacheReaderLocal;

	FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	FInstanceData_RenderThread* InstanceData_RT = DIProxy.PerInstanceData_RenderThread.Find(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	if (InstanceData_RT && InstanceData_RT->SimCacheResource->GetNumFrames() > 0)
	{
		ShaderParameters->NumFrames		= InstanceData_RT->SimCacheResource->GetNumFrames();
		ShaderParameters->NumEmitters	= InstanceData_RT->SimCacheResource->GetNumEmitters();
		ShaderParameters->EmitterIndex	= InstanceData_RT->EmitterIndex;
		ShaderParameters->CacheData		= InstanceData_RT->SimCacheResource->GetBufferSRV();

		const FShaderStorage& ShaderStorage = Context.GetShaderStorage<FShaderStorage>();
		const uint32 NumAttributes4 = FMath::DivideAndRoundUp(FMath::Max(ShaderStorage.AttributeReadNames.Num() / 2, 1), 4);

		TArray<int32>& ComponentOffsets = InstanceData_RT->StorageToComponentOffsets.FindOrAdd(uintptr_t(&ShaderStorage));
		if (ComponentOffsets.IsEmpty())
		{
			const UNiagaraSimCache* InstanceSimCache = InstanceData_RT->SimCacheResource->GetSimCache();

			ComponentOffsets.AddZeroed(NumAttributes4 * 4);
			for (int32 i = 0; i < ShaderStorage.AttributeReadNames.Num() / 2; ++i)
			{
				const FName FunctionName = ShaderStorage.AttributeReadNames[(i * 2) + 0];
				const FName AttributeName = ShaderStorage.AttributeReadNames[(i * 2) + 1];
				const int32 ComponentIndex = FunctionName.IsNone() ? INDEX_NONE : ComponentIndexFromVariable(InstanceSimCache, FunctionName, AttributeName, InstanceData_RT->EmitterIndex);
				ComponentOffsets[i] = ComponentIndex;
			}
		}

		TArrayView<FIntVector4> ShaderComponentOffsets = Context.GetParameterLooseArray<FIntVector4>(NumAttributes4);
		FMemory::Memcpy(ShaderComponentOffsets.GetData(), ComponentOffsets.GetData(), ComponentOffsets.Num() * ComponentOffsets.GetTypeSize());
	}
	else
	{
		ShaderParameters->NumFrames		= 0;
		ShaderParameters->NumEmitters	= 0;
		ShaderParameters->EmitterIndex	= INDEX_NONE;
		ShaderParameters->CacheData		= FNiagaraRenderer::GetDummyUIntBuffer();

		TArrayView<FIntVector4> ComponentOffsets = Context.GetParameterLooseArray<FIntVector4>(1);
		FMemory::Memset(ComponentOffsets.GetData(), 0, sizeof(FIntVector4));
	}
}

void UNiagaraDataInterfaceSimCacheReader::VMGetNumFrames(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDISimCacheReaderLocal;

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData_GT(Context);
	FNDIOutputParam<int32> OutNumFrames(Context);
	
	UNiagaraSimCache* InstanceSimCache = InstanceData_GT->WeakSimCache.Get();
	const uint32 NumFrames = InstanceSimCache ? InstanceSimCache->GetNumFrames() : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutNumFrames.SetAndAdvance(NumFrames);
	}
}

void UNiagaraDataInterfaceSimCacheReader::VMGetNumEmitters(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDISimCacheReaderLocal;

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData_GT(Context);
	FNDIOutputParam<int32> OutNumEmitters(Context);

	UNiagaraSimCache* InstanceSimCache = InstanceData_GT->WeakSimCache.Get();
	const uint32 NumEmitters = InstanceSimCache ? InstanceSimCache->GetNumEmitters() : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutNumEmitters.SetAndAdvance(NumEmitters);
	}
}

void UNiagaraDataInterfaceSimCacheReader::VMGetEmitterIndex(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDISimCacheReaderLocal;

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData_GT(Context);
	FNDIOutputParam<int32> OutEmitterIndex(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutEmitterIndex.SetAndAdvance(InstanceData_GT->EmitterIndex);
	}
}

void UNiagaraDataInterfaceSimCacheReader::VMGetNumInstances(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDISimCacheReaderLocal;

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData_GT(Context);
	FNDIInputParam<int32>	InFrameIndex(Context);
	FNDIOutputParam<int32>	OutNumInstances(Context);

	UNiagaraSimCache* InstanceSimCache = InstanceData_GT->WeakSimCache.Get();
	int32 LastFrameIndex = INDEX_NONE;
	int32 LastNumInstances = 0;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 FrameIndex = InFrameIndex.GetAndAdvance();
		if (FrameIndex != LastFrameIndex)
		{
			LastFrameIndex = FrameIndex;

			if (InstanceSimCache)
			{
				FNiagaraSimCacheAttributeReaderHelper AttributeReader;
				AttributeReader.Initialize(InstanceData_GT->WeakSimCache.Get(), InstanceData_GT->EmitterIndex, 0, FrameIndex);
				LastNumInstances = AttributeReader.IsValid() ? AttributeReader.GetNumInstances() : 0;
			}
			else
			{
				LastNumInstances = 0;
			}
		}

		OutNumInstances.SetAndAdvance(LastNumInstances);
	}
}

void UNiagaraDataInterfaceSimCacheReader::VMGetInstanceInt(FVectorVMExternalFunctionContext& Context, int AttributeIndex)
{
	using namespace NDISimCacheReaderLocal;
	VMInstanceGetFunction<int32>(Context, 0, AttributeIndex, [](FNiagaraSimCacheAttributeReaderHelper& AttributeReader, int32 InstanceIndex) { return AttributeReader.ReadInt(InstanceIndex); });
}

void UNiagaraDataInterfaceSimCacheReader::VMGetInstanceFloat(FVectorVMExternalFunctionContext& Context, int AttributeIndex)
{
	using namespace NDISimCacheReaderLocal;
	VMInstanceGetFunction<float>(Context, 0.0f, AttributeIndex, [](FNiagaraSimCacheAttributeReaderHelper& AttributeReader, int32 InstanceIndex) { return AttributeReader.ReadFloat(InstanceIndex); });
}

void UNiagaraDataInterfaceSimCacheReader::VMGetInstanceVector2(FVectorVMExternalFunctionContext& Context, int AttributeIndex)
{
	using namespace NDISimCacheReaderLocal;
	VMInstanceGetFunction<FVector2f>(Context, FVector2f::ZeroVector, AttributeIndex, [](FNiagaraSimCacheAttributeReaderHelper& AttributeReader, int32 InstanceIndex) { return AttributeReader.ReadFloat2f(InstanceIndex); });
}

void UNiagaraDataInterfaceSimCacheReader::VMGetInstanceVector3(FVectorVMExternalFunctionContext& Context, int AttributeIndex)
{
	using namespace NDISimCacheReaderLocal;
	VMInstanceGetFunction<FVector3f>(Context, FVector3f::ZeroVector, AttributeIndex, [](FNiagaraSimCacheAttributeReaderHelper& AttributeReader, int32 InstanceIndex) { return AttributeReader.ReadFloat3f(InstanceIndex); });
}

void UNiagaraDataInterfaceSimCacheReader::VMGetInstanceVector4(FVectorVMExternalFunctionContext& Context, int AttributeIndex)
{
	using namespace NDISimCacheReaderLocal;
	VMInstanceGetFunction<FVector4f>(Context, FVector4f::Zero(), AttributeIndex, [](FNiagaraSimCacheAttributeReaderHelper& AttributeReader, int32 InstanceIndex) { return AttributeReader.ReadFloat4f(InstanceIndex); });
}

void UNiagaraDataInterfaceSimCacheReader::VMGetInstanceColor(FVectorVMExternalFunctionContext& Context, int AttributeIndex)
{
	using namespace NDISimCacheReaderLocal;
	VMInstanceGetFunction<FLinearColor>(Context, FLinearColor::White, AttributeIndex, [](FNiagaraSimCacheAttributeReaderHelper& AttributeReader, int32 InstanceIndex) { return AttributeReader.ReadColor(InstanceIndex); });
}

void UNiagaraDataInterfaceSimCacheReader::VMGetInstanceQuat(FVectorVMExternalFunctionContext& Context, int AttributeIndex, bool bRebase)
{
	using namespace NDISimCacheReaderLocal;
	VMInstanceGetFunction<FQuat4f>(Context, FQuat4f::Identity, AttributeIndex, [](FNiagaraSimCacheAttributeReaderHelper& AttributeReader, int32 InstanceIndex) { return AttributeReader.ReadQuat4f(InstanceIndex); });
}

void UNiagaraDataInterfaceSimCacheReader::VMGetInstancePosition(FVectorVMExternalFunctionContext& Context, int AttributeIndex, bool bRebase)
{
	using namespace NDISimCacheReaderLocal;
	VMInstanceGetFunction<FVector3f>(Context, FVector3f::ZeroVector, AttributeIndex, [](FNiagaraSimCacheAttributeReaderHelper& AttributeReader, int32 InstanceIndex) { return AttributeReader.ReadFloat3f(InstanceIndex); });
}

void UNiagaraDataInterfaceSimCacheReader::VMGetInstanceID(FVectorVMExternalFunctionContext& Context, int AttributeIndex)
{
	using namespace NDISimCacheReaderLocal;
	VMInstanceGetFunction<FNiagaraID>(Context, FNiagaraID(), AttributeIndex, [](FNiagaraSimCacheAttributeReaderHelper& AttributeReader, int32 InstanceIndex) { return AttributeReader.ReadID(InstanceIndex); });
}

void UNiagaraDataInterfaceSimCacheReader::VMGetInstanceIndexFromID(FVectorVMExternalFunctionContext& Context, int AttributeIndex)
{
	using namespace NDISimCacheReaderLocal;

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData_GT(Context);
	FNDIInputParam<int32>		InFrameIndex(Context);
	FNDIInputParam<FNiagaraID>	InID(Context);
	FNDIOutputParam<bool>		OutValid(Context);
	FNDIOutputParam<int32>		OutValue(Context);

	if (UNiagaraSimCache* InstanceSimCache = InstanceData_GT->WeakSimCache.Get())
	{
		FNiagaraSimCacheAttributeReaderHelper AttributeReader;
		int CachedFrameIndex = INDEX_NONE;

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 FrameIndex = InFrameIndex.GetAndAdvance();
			const FNiagaraID ID = InID.GetAndAdvance();

			if (CachedFrameIndex != FrameIndex)
			{
				CachedFrameIndex = FrameIndex;
				AttributeReader.Initialize(InstanceSimCache, InstanceData_GT->EmitterIndex, AttributeIndex, FrameIndex);
			}

			int32 InstanceIndex = INDEX_NONE;
			if (AttributeReader.IsValid())
			{
				for (int32 j = 0; j < AttributeReader.GetNumInstances(); ++j)
				{
					if (AttributeReader.ReadID(j) == ID)
					{
						InstanceIndex = j;
						break;
					}
				}
			}
			OutValid.SetAndAdvance(InstanceIndex != INDEX_NONE);
			OutValue.SetAndAdvance(InstanceIndex);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutValid.SetAndAdvance(false);
			OutValue.SetAndAdvance(INDEX_NONE);
		}
	}
}

#undef LOCTEXT_NAMESPACE
