// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceSimCacheReader.generated.h"

struct FNiagaraDataInterfaceGeneratedFunction;

class UNiagaraSimCache;

/**
Data interface to read properties from a Niagara Simulation Cache
*/
UCLASS(EditInlineNew, Category = "DataInterface", CollapseCategories, meta=(DisplayName="SimCache Reader", Experimental))
class UNiagaraDataInterfaceSimCacheReader : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(int32,				NumFrames)
		SHADER_PARAMETER(int32,				NumEmitters)
		SHADER_PARAMETER(int32,				EmitterIndex)
		SHADER_PARAMETER_SRV(Buffer<uint>,	CacheData)
	END_SHADER_PARAMETER_STRUCT()

public:
	/** User parameter Object binding if this is not a valid sim cache the default one will be used instead. */
	UPROPERTY(EditAnywhere, Category = "SimCacheReader")
	FNiagaraUserParameterBinding SimCacheBinding;

	/** Optional source SimCache to use, if the user parameter binding is valid this will be ignored. */
	UPROPERTY(EditAnywhere, Category = "SimCacheReader")
	TObjectPtr<UNiagaraSimCache> SimCache;

	/** Which Emitter we should read from within the SimCache. */
	UPROPERTY(EditAnywhere, Category = "Emitter")
	FName EmitterBinding;

	//UNiagaraDataInterface Interface
	virtual void PostInitProperties() override;
	// UObject Interface End

	// UNiagaraDataInterface Interface Begin
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual int32 PerInstanceDataSize() const override;

	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;

	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
#if WITH_EDITORONLY_DATA
	virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	virtual FNiagaraDataInterfaceParametersCS* CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const override;
	virtual const FTypeLayoutDesc* GetShaderStorageType() const override;
	virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	//UNiagaraDataInterface Interface

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

private:
	void VMGetNumFrames(FVectorVMExternalFunctionContext& Context);
	void VMGetNumEmitters(FVectorVMExternalFunctionContext& Context);
	void VMGetEmitterIndex(FVectorVMExternalFunctionContext& Context);
	void VMGetNumInstances(FVectorVMExternalFunctionContext& Context);
	void VMGetInstanceInt(FVectorVMExternalFunctionContext& Context, int AttributeIndex);
	void VMGetInstanceFloat(FVectorVMExternalFunctionContext& Context, int AttributeIndex);
	void VMGetInstanceVector2(FVectorVMExternalFunctionContext& Context, int AttributeIndex);
	void VMGetInstanceVector3(FVectorVMExternalFunctionContext& Context, int AttributeIndex);
	void VMGetInstanceVector4(FVectorVMExternalFunctionContext& Context, int AttributeIndex);
	void VMGetInstanceColor(FVectorVMExternalFunctionContext& Context, int AttributeIndex);
	void VMGetInstanceQuat(FVectorVMExternalFunctionContext& Context, int AttributeIndex, bool bRebase);
	void VMGetInstancePosition(FVectorVMExternalFunctionContext& Context, int AttributeIndex, bool bRebase);
	void VMGetInstanceID(FVectorVMExternalFunctionContext& Context, int AttributeIndex);
	void VMGetInstanceIndexFromID(FVectorVMExternalFunctionContext& Context, int AttributeIndex);
};
