// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceSimpleCounter.h"
#include "NiagaraClearCounts.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGpuComputeDispatch.h"
#include "NiagaraGpuReadbackManager.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraWorldManager.h"

#include "Internationalization/Internationalization.h"
#include "Async/Async.h"
#include "RenderGraphUtils.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceSimpleCounter)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceSimpleCounter"

//////////////////////////////////////////////////////////////////////////

namespace NDISimpleCounterLocal
{
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceSimpleCounterTemplate.ush");
	static const FName NAME_GetNextValue_Deprecated(TEXT("GetNextValue"));
	static const FName NAME_Get(TEXT("Get"));
	static const FName NAME_Set(TEXT("Set"));
	static const FName NAME_Exchange(TEXT("Exchange"));
	static const FName NAME_Add(TEXT("Add"));
	static const FName NAME_Increment(TEXT("Increment"));
	static const FName NAME_Decrement(TEXT("Decrement"));
}

struct FNDISimpleCounterInstanceData_RenderThread
{
	TOptional<int32>	CountValue;
	uint32				CountOffset = INDEX_NONE;
};

struct FNDISimpleCounterInstanceData_GameThread
{
	bool				bModified = true;
	std::atomic<int32>	Counter;
};

struct FNDISimpleCounterProxy : public FNiagaraDataInterfaceProxyRW
{
	FNDISimpleCounterProxy(UNiagaraDataInterfaceSimpleCounter* Owner)
		: WeakOwner(Owner)
		, GpuSyncMode(Owner->GpuSyncMode)
	{
	}

	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override {}
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override
	{
		if (const FNDISimpleCounterInstanceData_RenderThread* InstanceData = PerInstanceData_RenderThread.Find(Context.GetSystemInstanceID()))
		{
			Context.CreateIndirect(InstanceData->CountOffset);
		}
	}

	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override
	{
		if (FNDISimpleCounterInstanceData_RenderThread* InstanceData = PerInstanceData_RenderThread.Find(Context.GetSystemInstanceID()) )
		{
			if ( InstanceData->CountValue.IsSet() )
			{
				//-OPT: We could push this into the count manager and batch set as part of the clear process
				const FNiagaraGPUInstanceCountManager& CounterManager = Context.GetComputeDispatchInterface().GetGPUInstanceCounterManager();
				const FRWBuffer& CountBuffer = CounterManager.GetInstanceCountBuffer();

				//-TODO: Once the count manager is ported to RDG we won't need to track this
				const FNiagaraGpuComputeDispatch& ComputeDispatch = static_cast<const FNiagaraGpuComputeDispatch&>(Context.GetComputeDispatchInterface());
				const ERHIAccess CountRHIAccess = ComputeDispatch.IsExecutingFirstDispatchGroup() ? FNiagaraGPUInstanceCountManager::kCountBufferDefaultState : ERHIAccess::UAVCompute;

				//-TODO:RDG: Once the count buffer is a graph resource this can be changed
				AddPass(
					Context.GetGraphBuilder(),
					RDG_EVENT_NAME("NiagaraSimpleCounter::PreStage"),
					[CountBufferUAV=CountBuffer.UAV, CountOffset=InstanceData->CountOffset, CountValue=InstanceData->CountValue.GetValue(), CountRHIAccess](FRHICommandListImmediate& RHICmdList)
					{
						const TPair<uint32, uint32> DataToClear(CountOffset, reinterpret_cast<const uint32&>(CountValue));
						RHICmdList.Transition(FRHITransitionInfo(CountBufferUAV, CountRHIAccess, ERHIAccess::UAVCompute));
						NiagaraClearCounts::ClearCountsUInt(RHICmdList, CountBufferUAV, MakeArrayView(&DataToClear, 1) );
						RHICmdList.Transition(FRHITransitionInfo(CountBufferUAV, ERHIAccess::UAVCompute, CountRHIAccess));
					}
				);

				InstanceData->CountValue.Reset();
			}
		}
	}

	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override
	{
		if (FNiagaraUtilities::ShouldSyncGpuToCpu(GpuSyncMode) && Context.IsFinalPostSimulate())
		{
			FNiagaraSystemInstanceID SystemInstanceID = Context.GetSystemInstanceID();
			if (FNDISimpleCounterInstanceData_RenderThread* InstanceData = PerInstanceData_RenderThread.Find(SystemInstanceID))
			{
				//-TODO:RDG: This should be done using the graph builder
				const FNiagaraGpuComputeDispatch& ComputeDispatch = static_cast<const FNiagaraGpuComputeDispatch&>(Context.GetComputeDispatchInterface());
				const FNiagaraGPUInstanceCountManager& CountManager = ComputeDispatch.GetGPUInstanceCounterManager();
				FNiagaraGpuReadbackManager* ReadbackManager = ComputeDispatch.GetGpuReadbackManager();

				const ERHIAccess CountRHIAccess = ComputeDispatch.IsExecutingLastDispatchGroup() ? FNiagaraGPUInstanceCountManager::kCountBufferDefaultState : ERHIAccess::UAVCompute;

				AddPass(
					Context.GetGraphBuilder(),
					RDG_EVENT_NAME("NiagaraSimpleCounter::PostStage"),
					[InstanceData, SystemInstanceID, ReadbackManager, CountBuffer=CountManager.GetInstanceCountBuffer().Buffer, CountRHIAccess, Proxy=this](FRHICommandListImmediate& RHICmdList)
					{
						RHICmdList.Transition(FRHITransitionInfo(CountBuffer, CountRHIAccess, ERHIAccess::CopySrc));
						ReadbackManager->EnqueueReadback(
							RHICmdList,
							CountBuffer,
							InstanceData->CountOffset * sizeof(uint32), sizeof(uint32),
							[SystemInstanceID, WeakOwner=Proxy->WeakOwner, Proxy](TConstArrayView<TPair<void*, uint32>> ReadbackData)
							{
								const int32 CounterValue = *reinterpret_cast<const int32*>(ReadbackData[0].Key);
								FNiagaraWorldManager::EnqueueGlobalDeferredCallback(
									[SystemInstanceID, CounterValue, WeakOwner, Proxy]()
									{
										TRACE_CPUPROFILER_EVENT_SCOPE(NDISimpleCounterGTCallback);

										// FNiagaraDataInterfaceProxy do not outlive UNiagaraDataInterface so if our Object is valid so is the proxy
										// Equally because we do not share instance IDs (monotonically increasing number) we won't ever stomp something that has 'gone away'
										if ( WeakOwner.Get() )
										{
											if ( FNDISimpleCounterInstanceData_GameThread* InstanceData_GT = Proxy->PerInstanceData_GameThread.FindRef(SystemInstanceID) )
											{
												InstanceData_GT->Counter = CounterValue;
											}
										}
									}
								);
							}
						);
						RHICmdList.Transition(FRHITransitionInfo(CountBuffer, ERHIAccess::CopySrc, CountRHIAccess));
					}
				);
			}
		}
	}

	TWeakObjectPtr<UNiagaraDataInterfaceSimpleCounter> WeakOwner;
	ENiagaraGpuSyncMode GpuSyncMode = ENiagaraGpuSyncMode::None;

	TMap<FNiagaraSystemInstanceID, FNDISimpleCounterInstanceData_RenderThread>	PerInstanceData_RenderThread;
	TMap<FNiagaraSystemInstanceID, FNDISimpleCounterInstanceData_GameThread*>	PerInstanceData_GameThread;
};

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceSimpleCounter::UNiagaraDataInterfaceSimpleCounter(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNDISimpleCounterProxy(this));
}

void UNiagaraDataInterfaceSimpleCounter::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceSimpleCounter::PostLoad()
{
	Super::PostLoad();
	UpdateDIProxy();
}

#if WITH_EDITOR
void UNiagaraDataInterfaceSimpleCounter::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UpdateDIProxy();
}
#endif

bool UNiagaraDataInterfaceSimpleCounter::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	auto* OtherTyped = CastChecked<const UNiagaraDataInterfaceSimpleCounter>(Other);
	return
		OtherTyped->GpuSyncMode == GpuSyncMode &&
		OtherTyped->InitialValue == InitialValue;
}

bool UNiagaraDataInterfaceSimpleCounter::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	auto* DestinationTyped = CastChecked<UNiagaraDataInterfaceSimpleCounter>(Destination);
	DestinationTyped->GpuSyncMode = GpuSyncMode;
	DestinationTyped->InitialValue = InitialValue;

	// Ensure proxy properties are up to date
	DestinationTyped->UpdateDIProxy();

	return true;
}

bool UNiagaraDataInterfaceSimpleCounter::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDISimpleCounterInstanceData_GameThread* InstanceData_GT = new(PerInstanceData) FNDISimpleCounterInstanceData_GameThread();
	InstanceData_GT->Counter = InitialValue;

	if ( IsUsedWithGPUScript() )
	{
		FNDISimpleCounterProxy* Proxy_GT = GetProxyAs<FNDISimpleCounterProxy>();
		Proxy_GT->PerInstanceData_GameThread.Add(SystemInstance->GetId(), InstanceData_GT);

		ENQUEUE_RENDER_COMMAND(FNDISimpleCounter_RemoveProxy)
		(
			[Proxy_RT=Proxy_GT, InstanceID_RT=SystemInstance->GetId(), ComputeInterface_RT=SystemInstance->GetComputeDispatchInterface(), InitialValue_RT=InitialValue](FRHICommandListImmediate& RHICmdList)
			{
				FNDISimpleCounterInstanceData_RenderThread* InstanceData = &Proxy_RT->PerInstanceData_RenderThread.Add(InstanceID_RT);
				InstanceData->CountValue = InitialValue_RT;
				InstanceData->CountOffset = ComputeInterface_RT->GetGPUInstanceCounterManager().AcquireOrAllocateEntry(RHICmdList);
			}
		);
	}
	return true;
}

void UNiagaraDataInterfaceSimpleCounter::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	auto* InstanceData = reinterpret_cast<FNDISimpleCounterInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FNDISimpleCounterInstanceData_GameThread();

	if ( IsUsedWithGPUScript() )
	{
		FNDISimpleCounterProxy* Proxy_GT = GetProxyAs<FNDISimpleCounterProxy>();
		Proxy_GT->PerInstanceData_GameThread.Remove(SystemInstance->GetId());

		ENQUEUE_RENDER_COMMAND(FNDISimpleCounter_RemoveProxy)
		(
			[Proxy_RT=Proxy_GT, InstanceID_RT=SystemInstance->GetId(), ComputeInterface_RT=SystemInstance->GetComputeDispatchInterface()](FRHICommandListImmediate& RHICmdList)
			{
				if ( FNDISimpleCounterInstanceData_RenderThread* InstanceData = Proxy_RT->PerInstanceData_RenderThread.Find(InstanceID_RT) )
				{
					if ( ensure(InstanceData->CountOffset != INDEX_NONE) )
					{
						ComputeInterface_RT->GetGPUInstanceCounterManager().FreeEntry(InstanceData->CountOffset);
						InstanceData->CountOffset = INDEX_NONE;
					}
					Proxy_RT->PerInstanceData_RenderThread.Remove(InstanceID_RT);
				}
			}
		);
	}
}

int32 UNiagaraDataInterfaceSimpleCounter::PerInstanceDataSize() const
{
	return sizeof(FNDISimpleCounterInstanceData_GameThread);
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSimpleCounter::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDISimpleCounterLocal;

	// Deprecated function
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NAME_GetNextValue_Deprecated;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSoftDeprecatedFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Counter")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));
		Sig.SetDescription(LOCTEXT("UNiagaraDataInterfaceSimpleCounter_GetNextValue", "Increment the internal counter. Note that it is possible for this counter to roll over eventually, so make sure that your particles do not live extremely long lifetimes."));
	}

	{
		FNiagaraFunctionSignature & Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NAME_Get;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Counter")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Current Value")));
		Sig.SetDescription(LOCTEXT("GetDesc", "Gets the current value of the counter."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NAME_Set;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Counter")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("New Value")));
		Sig.SetDescription(LOCTEXT("SetDesc", "Sets the current value of the counter."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NAME_Exchange;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Counter")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("New Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Previous Value")));
		Sig.SetDescription(LOCTEXT("ExchangeDesc", "Exchanges the current value with the new one."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NAME_Add;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Counter")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Amount")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Previous Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Current Value")));
		Sig.SetDescription(LOCTEXT("AddDesc", "Adds the Amount to the counter."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NAME_Increment;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Counter")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Previous Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Current Value")));
		Sig.SetDescription(LOCTEXT("IncrementDesc", "Increments the counter by 1."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NAME_Decrement;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Counter")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Previous Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Current Value")));
		Sig.SetDescription(LOCTEXT("DecrementDesc", "Decrements the counter by 1."));
	}
}
#endif

void UNiagaraDataInterfaceSimpleCounter::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDISimpleCounterLocal;

	if ( BindingInfo.Name == NAME_GetNextValue_Deprecated )
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimpleCounter::GetNextValue_Deprecated);
	}
	else if (BindingInfo.Name == NAME_Get)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimpleCounter::VMGet);
	}
	else if (BindingInfo.Name == NAME_Set)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimpleCounter::VMSet);
	}
	else if (BindingInfo.Name == NAME_Exchange)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimpleCounter::VMExchange);
	}
	else if (BindingInfo.Name == NAME_Add)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimpleCounter::VMAdd);
	}
	else if (BindingInfo.Name == NAME_Increment)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimpleCounter::VMIncrement);
	}
	else if (BindingInfo.Name == NAME_Decrement)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSimpleCounter::VMDecrement);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSimpleCounter::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDISimpleCounterLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceSimpleCounter::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDISimpleCounterLocal;

	if ((FunctionInfo.DefinitionName == NAME_Get) ||
		(FunctionInfo.DefinitionName == NAME_Exchange) ||
		(FunctionInfo.DefinitionName == NAME_Add) ||
		(FunctionInfo.DefinitionName == NAME_Increment) ||
		(FunctionInfo.DefinitionName == NAME_Decrement) )
	{
		return true;
	}

	// Invalid function
	return true;
}

bool UNiagaraDataInterfaceSimpleCounter::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(NDISimpleCounterLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}
#endif

void UNiagaraDataInterfaceSimpleCounter::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceSimpleCounter::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDISimpleCounterLocal;

	FNDISimpleCounterProxy& DIProxy = Context.GetProxy<FNDISimpleCounterProxy>();
	FNDISimpleCounterInstanceData_RenderThread& InstanceData = DIProxy.PerInstanceData_RenderThread.FindOrAdd(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->CountOffset = InstanceData.CountOffset;
}

void UNiagaraDataInterfaceSimpleCounter::PushToRenderThreadImpl()
{
	TArray<TPair<FNiagaraSystemInstanceID, int32>, TInlineAllocator<4>> DataToPush;

	FNDISimpleCounterProxy* Proxy_GT = GetProxyAs<FNDISimpleCounterProxy>();
	for ( auto it=Proxy_GT->PerInstanceData_GameThread.CreateIterator(); it; ++it )
	{
		if ( it.Value()->bModified )
		{
			it.Value()->bModified = false;
			DataToPush.Emplace(it.Key(), it.Value()->Counter);
		}
	}

	if (DataToPush.Num() > 0)
	{
		ENQUEUE_RENDER_COMMAND(FNDISimpleCounter_PushToRender)
		(
			[Proxy_RT=GetProxyAs<FNDISimpleCounterProxy>(), DataToPush_RT=MoveTemp(DataToPush)](FRHICommandListImmediate& RHICmdList)
			{
				for (const auto& Pair : DataToPush_RT)
				{
					if ( FNDISimpleCounterInstanceData_RenderThread* InstanceData_RT=Proxy_RT->PerInstanceData_RenderThread.Find(Pair.Key) )
					{
						InstanceData_RT->CountValue = Pair.Value;
					}
				}
			}
		);
	}
}

UObject* UNiagaraDataInterfaceSimpleCounter::SimCacheBeginWrite(UObject* SimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	return OptionalPerInstanceData ? NewObject<UNDISimpleCounterSimCacheData>(SimCache) : nullptr;
}

bool UNiagaraDataInterfaceSimpleCounter::SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	check(OptionalPerInstanceData && StorageObject);

	UNDISimpleCounterSimCacheData* CacheData = CastChecked<UNDISimpleCounterSimCacheData>(StorageObject);

	const int32 ValueOffset = FrameIndex * 2;
	const int32 ExpectedValues = ValueOffset + 2;
	if (CacheData->Values.Num() < ExpectedValues)
	{
		CacheData->Values.AddDefaulted(ExpectedValues - CacheData->Values.Num());
	}

	const FNDISimpleCounterInstanceData_GameThread* InstanceData_GT = static_cast<const FNDISimpleCounterInstanceData_GameThread*>(OptionalPerInstanceData);
	CacheData->Values[ValueOffset + 0] = InstanceData_GT->Counter;
	CacheData->Values[ValueOffset + 1] = InstanceData_GT->Counter;

	if (IsUsedWithGPUScript())
	{
		ENQUEUE_RENDER_COMMAND(FNDISimpleCounter_CacheWriteFrame)
		(
			[Proxy_RT=GetProxyAs<FNDISimpleCounterProxy>(), InstanceID=SystemInstance->GetId(), ComputeInterface=SystemInstance->GetComputeDispatchInterface(), CacheData, ValueOffset](FRHICommandListImmediate& RHICmdList)
			{
				const FNDISimpleCounterInstanceData_RenderThread* InstanceData_RT = Proxy_RT->PerInstanceData_RenderThread.Find(InstanceID);
				if (!InstanceData_RT || InstanceData_RT->CountOffset == INDEX_NONE)
				{
					return;
				}

				const FNiagaraGPUInstanceCountManager& CountManager = ComputeInterface->GetGPUInstanceCounterManager();
				FNiagaraGpuReadbackManager* ReadbackManager = ComputeInterface->GetGpuReadbackManager();

				ReadbackManager->EnqueueReadback(
					RHICmdList,
					CountManager.GetInstanceCountBuffer().Buffer,
					InstanceData_RT->CountOffset * sizeof(uint32), sizeof(int32),
					[CacheData, ValueOffset](TConstArrayView<TPair<void*, uint32>> ReadbackData)
					{
						CacheData->Values[ValueOffset + 1] = *static_cast<const int32*>(ReadbackData[0].Key);
					}
				);
				ReadbackManager->WaitCompletion(RHICmdList);
			}
		);
		FlushRenderingCommands();
	}
	return true;
}

bool UNiagaraDataInterfaceSimpleCounter::SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext)
{
	const int32 ValueOffset = ReadContext.GetFrameIndexA() * 2;
	const int32 ExpectedValues = ValueOffset + 2;

	UNDISimpleCounterSimCacheData* CacheData = ReadContext.GetStorageObject<UNDISimpleCounterSimCacheData>();
	if (CacheData->Values.Num() < ExpectedValues)
	{
		return false;
	}

	// Set CPU Data
	FNDISimpleCounterInstanceData_GameThread* InstanceData_GT = ReadContext.GetPerInstanceData<FNDISimpleCounterInstanceData_GameThread>();
	InstanceData_GT->Counter = CacheData->Values[ValueOffset + 0];

	// Set GPU Data
	if (IsUsedWithGPUScript())
	{
		ENQUEUE_RENDER_COMMAND(FNDISimpleCounter_PushToRender)
		(
			[Proxy_RT=GetProxyAs<FNDISimpleCounterProxy>(), InstanceID=ReadContext.GetSystemInstance()->GetId(), NewValue = CacheData->Values[ValueOffset + 1]](FRHICommandListImmediate& RHICmdList)
			{
				if ( FNDISimpleCounterInstanceData_RenderThread* InstanceData_RT=Proxy_RT->PerInstanceData_RenderThread.Find(InstanceID) )
				{
					InstanceData_RT->CountValue = NewValue;
				}
			}
		);
	}

	return true;
}

bool UNiagaraDataInterfaceSimpleCounter::SimCacheCompareFrame(const UObject* LhsStorageObject, const UObject* RhsStorageObject, int FrameIndex, TOptional<float> Tolerance, FString& OutErrors) const
{
	const UNDISimpleCounterSimCacheData* LhsCacheData = CastChecked<const UNDISimpleCounterSimCacheData>(LhsStorageObject);
	const UNDISimpleCounterSimCacheData* RhsCacheData = CastChecked<const UNDISimpleCounterSimCacheData>(RhsStorageObject);

	const int32 ValueOffset = FrameIndex * 2;
	const int32 ExpectedValues = ValueOffset + 2;

	if (LhsCacheData->Values.Num() != RhsCacheData->Values.Num())
	{
		OutErrors = FString::Printf(TEXT("Number of LhsValues(%d) != RhsValues(%d)"), LhsCacheData->Values.Num(), RhsCacheData->Values.Num());
		return false;
	}

	if ((LhsCacheData->Values[ValueOffset + 0] != RhsCacheData->Values[ValueOffset + 0]) ||
		(LhsCacheData->Values[ValueOffset + 1] != RhsCacheData->Values[ValueOffset + 1]))
	{
		OutErrors = FString::Printf(
			TEXT("Cpu|Gpu values to do not match LhsValues(%d|%d) != RhsValues(%d|%d)"),
			LhsCacheData->Values[ValueOffset + 0], LhsCacheData->Values[ValueOffset + 1],
			RhsCacheData->Values[ValueOffset + 0], RhsCacheData->Values[ValueOffset + 1]
		);
		return false;
	}

	return true;
}

void UNiagaraDataInterfaceSimpleCounter::UpdateDIProxy()
{
	FNDISimpleCounterProxy* Proxy_GT = GetProxyAs<FNDISimpleCounterProxy>();
	Proxy_GT->GpuSyncMode = GpuSyncMode;
}

void UNiagaraDataInterfaceSimpleCounter::VMGet(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISimpleCounterInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<int32> OutValue(Context);

	const int32 CurrValue = InstanceData->Counter;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutValue.SetAndAdvance(CurrValue);
	}
}

void UNiagaraDataInterfaceSimpleCounter::VMSet(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISimpleCounterInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool> InExecute(Context);
	FNDIInputParam<int32> InValue(Context);

	for (int32 i=0; i < Context.GetNumInstances(); ++i)
	{
		const bool bExecute = InExecute.GetAndAdvance();
		const int32 NewValue = InValue.GetAndAdvance();
		if (bExecute)
		{
			const int32 PrevValue = InstanceData->Counter.exchange(NewValue);
		}
	}

	if (FNiagaraUtilities::ShouldSyncCpuToGpu(GpuSyncMode))
	{
		InstanceData->bModified = true;
		MarkRenderDataDirty();
	}
}

void UNiagaraDataInterfaceSimpleCounter::VMExchange(FVectorVMExternalFunctionContext& Context)
{	
	VectorVM::FUserPtrHandler<FNDISimpleCounterInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool> InExecute(Context);
	FNDIInputParam<int32> InValue(Context);
	FNDIOutputParam<int32> OutPreviousValue(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool bExecute = InExecute.GetAndAdvance();
		const int32 NewValue = InValue.GetAndAdvance();
		if (bExecute)
		{
			const int32 PrevValue = InstanceData->Counter.exchange(NewValue);
			OutPreviousValue.SetAndAdvance(PrevValue);
		}
		else
		{
			OutPreviousValue.SetAndAdvance(InstanceData->Counter);
		}
	}

	if (FNiagaraUtilities::ShouldSyncCpuToGpu(GpuSyncMode))
	{
		InstanceData->bModified = true;
		MarkRenderDataDirty();
	}
}

void UNiagaraDataInterfaceSimpleCounter::VMAdd(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISimpleCounterInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool> InExecute(Context);
	FNDIInputParam<int32> InValue(Context);
	FNDIOutputParam<int32> OutPreviousValue(Context);
	FNDIOutputParam<int32> OutCurrentValue(Context);

	for (int32 i=0; i < Context.GetNumInstances(); ++i)
	{
		const bool bExecute = InExecute.GetAndAdvance();
		const int32 Value = InValue.GetAndAdvance();
		if (bExecute)
		{
			const int32 PrevValue = InstanceData->Counter.fetch_add(Value);
			OutPreviousValue.SetAndAdvance(PrevValue);
			OutCurrentValue.SetAndAdvance(PrevValue + Value);
		}
		else
		{
			const int32 CurrValue = InstanceData->Counter;
			OutPreviousValue.SetAndAdvance(CurrValue);
			OutCurrentValue.SetAndAdvance(CurrValue);
		}
	}

	if (FNiagaraUtilities::ShouldSyncCpuToGpu(GpuSyncMode))
	{
		InstanceData->bModified = true;
		MarkRenderDataDirty();
	}
}

void UNiagaraDataInterfaceSimpleCounter::VMIncrement(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISimpleCounterInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool> InExecute(Context);
	FNDIOutputParam<int32> OutPreviousValue(Context);
	FNDIOutputParam<int32> OutCurrentValue(Context);

	if (InExecute.IsConstant())
	{
		const bool bIncrement = InExecute.GetAndAdvance();
		int32 PrevValue = bIncrement ? InstanceData->Counter.fetch_add(Context.GetNumInstances()) : InstanceData->Counter.load();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 CurrValue = PrevValue + (bIncrement ? 1 : 0);
			OutPreviousValue.SetAndAdvance(PrevValue);
			OutCurrentValue.SetAndAdvance(CurrValue);
			PrevValue = CurrValue;
		}
	}
	else
	{
		for (int32 i=0; i < Context.GetNumInstances(); ++i)
		{
			const bool bExecute = InExecute.GetAndAdvance();
			if (bExecute)
			{
				const int32 PrevValue = InstanceData->Counter.fetch_add(1);
				OutPreviousValue.SetAndAdvance(PrevValue);
				OutCurrentValue.SetAndAdvance(PrevValue + 1);
			}
			else
			{
				const int32 CurrValue = InstanceData->Counter;
				OutPreviousValue.SetAndAdvance(CurrValue);
				OutCurrentValue.SetAndAdvance(CurrValue);
			}
		}
	}

	if (FNiagaraUtilities::ShouldSyncCpuToGpu(GpuSyncMode))
	{
		InstanceData->bModified = true;
		MarkRenderDataDirty();
	}
}

void UNiagaraDataInterfaceSimpleCounter::VMDecrement(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISimpleCounterInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<bool> InExecute(Context);
	FNDIOutputParam<int32> OutPreviousValue(Context);
	FNDIOutputParam<int32> OutCurrentValue(Context);

	if (InExecute.IsConstant())
	{
		const bool bDecrement = InExecute.GetAndAdvance();
		int32 PrevValue = bDecrement ? InstanceData->Counter.fetch_sub(Context.GetNumInstances()) : InstanceData->Counter.load();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 CurrValue = PrevValue - (bDecrement ? 1 : 0);
			OutPreviousValue.SetAndAdvance(PrevValue);
			OutCurrentValue.SetAndAdvance(CurrValue);
			PrevValue = CurrValue;
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool bExecute = InExecute.GetAndAdvance();
			if (bExecute)
			{
				const int32 PrevValue = InstanceData->Counter.fetch_sub(1);
				OutPreviousValue.SetAndAdvance(PrevValue);
				OutCurrentValue.SetAndAdvance(PrevValue - 1);
			}
			else
			{
				const int32 CurrValue = InstanceData->Counter;
				OutPreviousValue.SetAndAdvance(CurrValue);
				OutCurrentValue.SetAndAdvance(CurrValue);
			}
		}
	}

	if (FNiagaraUtilities::ShouldSyncCpuToGpu(GpuSyncMode))
	{
		InstanceData->bModified = true;
		MarkRenderDataDirty();
	}
}

void UNiagaraDataInterfaceSimpleCounter::GetNextValue_Deprecated(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISimpleCounterInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<int32> OutValue(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutValue.SetAndAdvance(InstanceData->Counter.fetch_add(1) + 1);
	}

	if (FNiagaraUtilities::ShouldSyncCpuToGpu(GpuSyncMode))
	{
		InstanceData->bModified = true;
		MarkRenderDataDirty();
	}
}

#undef LOCTEXT_NAMESPACE

