// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceSocketReader.h"
#include "Components/PrimitiveComponent.h"
#include "GameFramework/Pawn.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraComponent.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraTypes.h"
#include "NiagaraActorSceneComponentSocketUtils.h"
#include "NiagaraSceneComponentUtils.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraWorldManager.h"

#include "Components/SceneComponent.h"
#include "Engine/Canvas.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSocket.h"
#include "ShaderCompilerCore.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "UnifiedBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceSocketReader)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceSocketReader"

namespace NDISocketReaderLocal
{
	static bool GDisableTransformReadType = 0;
	static FAutoConsoleVariableRef CVarDisableTransformReadType(
		TEXT("fx.Niagara.SocketReaderDI.DisableTransformReadType"),
		GDisableTransformReadType,
		TEXT("Disables optimization around removing reading transforms that we don't consume."),
		ECVF_Default
	);

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceSocketReaderTemplate.ush");

	static const FName	IsValidName("IsValid");

	static const FName	GetComponentToWorldName("GetComponentToWorld");

	static const FName	GetSocketCountName("GetSocketCount");
	static const FName	GetFilteredSocketCountName("GetFilteredSocketCount");
	static const FName	GetUnfilteredSocketCountName("GetUnfilteredSocketCount");

	static const FName	GetSocketTransformName("GetSocketTransform");
	static const FName	GetFilteredSocketTransformName("GetFilteredSocketTransform");
	static const FName	GetUnfilteredSocketTransformName("GetUnfilteredSocketTransform");

	static const FName	GetSocketTransformInterpolatedName("GetSocketTransformInterpolated");
	static const FName	GetFilteredSocketTransformInterpolatedName("GetFilteredSocketTransformInterpolated");
	static const FName	GetUnfilteredSocketTransformInterpolatedName("GetUnfilteredSocketTransformInterpolated");

	struct FInstanceData_GameThread
	{
		TUniquePtr<FNiagaraSceneComponentSocketUtils>	SocketUtils;

		FNiagaraParameterDirectBinding<UObject*>	UserParamBinding;
		uint32										bIsDataValid : 1 = false;
		uint32										bNeedsSocketRecache : 1 = true;
		uint32										bNeedsRenderUpdate : 1 = true;
		uint32										bReadsFilteredTransforms : 1 = false;
		uint32										bReadsUnfilteredTransforms : 1 = false;

		float										DeltaSeconds = 0.0f;
		float										InvDeltaSeconds = 0.0f;

		FTransform									ComponentToWorld;
		FTransform									PreviousComponentToWorld;
		FTransform3f								ComponentToTranslatedWorld;
		FTransform3f								PreviousComponentToTranslatedWorld;

		int32										NumSockets = 0;
		int32										NumFilteredSockets = 0;
		int32										NumUnfilteredSockets = 0;
		TArray<int32>								SocketFilterUnfilteredIndex;
		TArray<FName>								SocketNames;
		TArray<FTransform3f>						SocketTransforms;
		TArray<FTransform3f>						PreviousSocketTransforms;
	};

	struct FInstanceData_SharedData
	{
		bool							bIsDataValid = false;
		float							InvDeltaSeconds = 0.0f;
		int32							NumSockets = 0;
		int32							NumFilteredSockets = 0;
		int32							NumUnfilteredSockets = 0;
		FTransform3f					ComponentToTranslatedWorld = FTransform3f::Identity;
		FTransform3f					PreviousComponentToTranslatedWorld = FTransform3f::Identity;
		uint32							SocketTransformOffset = 0;
		uint32							PreviousSocketTransformOffset = 0;
	};

	struct FGameToRenderInstanceData : public FInstanceData_SharedData
	{
		TArray<uint8>					DataToUpload;
	};

	struct FInstanceData_RenderThread : public FInstanceData_SharedData
	{
		TArray<uint8>					DataToUpload;
		TRefCountPtr<FRDGPooledBuffer>	PooledBuffer;
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FGameToRenderInstanceData); }

		static void CopyTransforms(uint8* OutBuffer, TConstArrayView<FTransform3f> Transforms)
		{
			float* OutFloats = reinterpret_cast<float*>(OutBuffer);
			for ( const FTransform3f& Transform : Transforms )
			{
				const FVector3f Translation = Transform.GetTranslation();
				const FQuat4f Rotation = Transform.GetRotation();
				const FVector3f Scale = Transform.GetScale3D();

				OutFloats[0] = Translation.X;
				OutFloats[1] = Translation.Y;
				OutFloats[2] = Translation.Z;
				OutFloats[3] = Rotation.X;
				OutFloats[4] = Rotation.Y;
				OutFloats[5] = Rotation.Z;
				OutFloats[6] = Rotation.W;
				OutFloats[7] = Scale.X;
				OutFloats[8] = Scale.Y;
				OutFloats[9] = Scale.Z;
				OutFloats += 10;
			}
		}

		static void ProvidePerInstanceDataForRenderThread(void* InDataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
		{
			FInstanceData_GameThread* InstanceData = static_cast<FInstanceData_GameThread*>(PerInstanceData);
			FGameToRenderInstanceData* DataForRenderThread = new(InDataForRenderThread) FGameToRenderInstanceData();

			DataForRenderThread->bIsDataValid						= InstanceData->bIsDataValid;
			DataForRenderThread->InvDeltaSeconds					= InstanceData->InvDeltaSeconds;
			DataForRenderThread->NumSockets							= InstanceData->NumSockets;
			DataForRenderThread->NumFilteredSockets					= InstanceData->NumFilteredSockets;
			DataForRenderThread->NumUnfilteredSockets				= InstanceData->NumUnfilteredSockets;
			DataForRenderThread->ComponentToTranslatedWorld			= InstanceData->ComponentToTranslatedWorld;
			DataForRenderThread->PreviousComponentToTranslatedWorld	= InstanceData->PreviousComponentToTranslatedWorld;

			if (InstanceData->bNeedsRenderUpdate)
			{
				InstanceData->bNeedsRenderUpdate = false;

				const int32 TransformGpuSize = 10 * 4;
				const int32 SocketTableSize = InstanceData->SocketFilterUnfilteredIndex.Num() * InstanceData->SocketFilterUnfilteredIndex.GetTypeSize();
				const int32 SocketTransformSize = InstanceData->SocketTransforms.Num() * TransformGpuSize;
				const int32 BufferSize = SocketTableSize + (SocketTransformSize * 2);
				DataForRenderThread->SocketTransformOffset			= SocketTableSize;
				DataForRenderThread->PreviousSocketTransformOffset	= SocketTableSize + SocketTransformSize;

				DataForRenderThread->DataToUpload.SetNumUninitialized(BufferSize);
				FMemory::Memcpy(DataForRenderThread->DataToUpload.GetData(), InstanceData->SocketFilterUnfilteredIndex.GetData(), SocketTableSize);
				CopyTransforms(DataForRenderThread->DataToUpload.GetData() + DataForRenderThread->SocketTransformOffset, InstanceData->SocketTransforms);
				CopyTransforms(DataForRenderThread->DataToUpload.GetData() + DataForRenderThread->PreviousSocketTransformOffset, InstanceData->PreviousSocketTransforms);
			}
		}

		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override
		{
			FGameToRenderInstanceData* InstanceDataFromGT = static_cast<FGameToRenderInstanceData*>(PerInstanceData);
			FInstanceData_RenderThread& InstanceData = SystemInstancesToInstanceData_RT.FindOrAdd(InstanceID);

			InstanceData.bIsDataValid						= InstanceDataFromGT->bIsDataValid;
			InstanceData.InvDeltaSeconds					= InstanceDataFromGT->InvDeltaSeconds;
			InstanceData.NumSockets							= InstanceDataFromGT->NumSockets;
			InstanceData.NumFilteredSockets					= InstanceDataFromGT->NumFilteredSockets;
			InstanceData.NumUnfilteredSockets				= InstanceDataFromGT->NumUnfilteredSockets;
			InstanceData.ComponentToTranslatedWorld			= InstanceDataFromGT->ComponentToTranslatedWorld;
			InstanceData.PreviousComponentToTranslatedWorld	= InstanceDataFromGT->PreviousComponentToTranslatedWorld;
			InstanceData.SocketTransformOffset				= InstanceDataFromGT->SocketTransformOffset;
			InstanceData.PreviousSocketTransformOffset		= InstanceDataFromGT->PreviousSocketTransformOffset;

			if (InstanceDataFromGT->DataToUpload.Num() > 0)
			{
				// If we got new data then swap in for any existing data
				// We don't clear InstanceData.DataToUpload as that is consumed when we need the GPU buffer
				InstanceData.DataToUpload					= MoveTemp(InstanceDataFromGT->DataToUpload);
			}

			InstanceDataFromGT->~FGameToRenderInstanceData();
		}

		virtual void PreStage(const FNDIGpuComputePreStageContext& Context)
		{
			const FNiagaraSystemInstanceID SystemInstanceID = Context.GetSystemInstanceID();
			FInstanceData_RenderThread& InstanceData = SystemInstancesToInstanceData_RT.FindChecked(SystemInstanceID);
			if (InstanceData.DataToUpload.Num() > 0)
			{
				FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

				const void* InitialData = InstanceData.DataToUpload.GetData();
				const uint64 InitialDataSize = InstanceData.DataToUpload.Num() * InstanceData.DataToUpload.GetTypeSize();
				const uint64 BufferSize = Align(InitialDataSize, 16);

				FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateByteAddressDesc(IntCastChecked<uint32>(BufferSize));
				ResizeBufferIfNeeded(GraphBuilder, InstanceData.PooledBuffer, BufferDesc, TEXT("NiagaraSocketReader"));

				GraphBuilder.QueueBufferUpload(
					GraphBuilder.RegisterExternalBuffer(InstanceData.PooledBuffer),
					InitialData,
					InitialDataSize,
					[Data = MoveTemp(InstanceData.DataToUpload)](const void*) {}
				);

				InstanceData.DataToUpload.Empty();
			}
		}

		TMap<FNiagaraSystemInstanceID, FInstanceData_RenderThread> SystemInstancesToInstanceData_RT;
	};

	//////////////////////////////////////////////////////////////////////////

	void ResolveObject(const UNiagaraDataInterfaceSocketReader* SocketDI, FInstanceData_GameThread* InstanceData, FNiagaraSystemInstance* SystemInstance)
	{
		InstanceData->bNeedsSocketRecache  |= InstanceData->SocketUtils->ResolveObject();
		InstanceData->bIsDataValid			= InstanceData->SocketUtils->IsValid();
	}

	void UpdateSocketCache(const UNiagaraDataInterfaceSocketReader* SocketDI, FInstanceData_GameThread* InstanceData, FNiagaraSystemInstance* SystemInstance)
	{
		const bool bResetPreviousTransform = InstanceData->bNeedsSocketRecache;
		const bool bNeedsSocketUpdate = SocketDI->bUpdateSocketsPerFrame || InstanceData->bNeedsSocketRecache;

		// Recache the list of sockets
		if (InstanceData->bNeedsSocketRecache)
		{
			InstanceData->bNeedsSocketRecache = false;
			InstanceData->bNeedsRenderUpdate = true;

			InstanceData->ComponentToTranslatedWorld = FTransform3f::Identity;
			InstanceData->PreviousComponentToTranslatedWorld = FTransform3f::Identity;

			InstanceData->NumSockets = 0;
			InstanceData->NumFilteredSockets = 0;
			InstanceData->NumUnfilteredSockets = 0;

			InstanceData->SocketFilterUnfilteredIndex.Empty();
			InstanceData->SocketNames.Empty();

			InstanceData->SocketNames = InstanceData->SocketUtils->GetSocketNames();

			InstanceData->NumSockets = InstanceData->SocketNames.Num();
			if (InstanceData->NumSockets > 0 )
			{
				InstanceData->NumFilteredSockets = SocketDI->FilteredSockets.Num();

				TBitArray<TInlineAllocator<2>> IsFilteredSocket;
				IsFilteredSocket.Add(false, InstanceData->NumSockets);

				InstanceData->SocketFilterUnfilteredIndex.Reserve(FMath::Max(InstanceData->NumSockets, InstanceData->NumFilteredSockets) + 1);
				InstanceData->SocketFilterUnfilteredIndex.Add(InstanceData->NumSockets);

				for (FName SocketName : SocketDI->FilteredSockets)
				{
					const int32 SocketIndex = InstanceData->SocketNames.IndexOfByKey(SocketName);
					if (InstanceData->SocketNames.IsValidIndex(SocketIndex))
					{
						InstanceData->SocketFilterUnfilteredIndex.Add(SocketIndex);
						IsFilteredSocket[SocketIndex] = true;
					}
					else
					{
						InstanceData->SocketFilterUnfilteredIndex.Add(InstanceData->NumSockets);
					}
				}

				for (int32 i = 0; i < InstanceData->NumSockets; ++i)
				{
					if (!IsFilteredSocket[i])
					{
						InstanceData->SocketFilterUnfilteredIndex.Add(i);
						++InstanceData->NumUnfilteredSockets;
					}
				}
			}
			else
			{
				InstanceData->SocketFilterUnfilteredIndex.Add(InstanceData->NumSockets);
			}
			InstanceData->SocketTransforms.SetNum(InstanceData->NumSockets + 1);
			InstanceData->PreviousSocketTransforms.SetNum(InstanceData->NumSockets + 1);
		}

		// Update the socket data
		const bool bReadsAnyTransformData = InstanceData->bReadsFilteredTransforms || InstanceData->bReadsUnfilteredTransforms;
		if (bReadsAnyTransformData && bNeedsSocketUpdate && InstanceData->NumSockets > 0)
		{
			// If we are not resetting copy the current transform to previous
			if (!bResetPreviousTransform)
			{
				InstanceData->PreviousSocketTransforms = InstanceData->SocketTransforms;
			}

			// Read the sockets
			TOptional<TConstArrayView<int32>> SocketIndices;
			if (!InstanceData->bReadsFilteredTransforms || !InstanceData->bReadsUnfilteredTransforms)
			{
				const int32 FirstSocket = (InstanceData->bReadsFilteredTransforms ? 0 : InstanceData->NumFilteredSockets) + 1;
				const int32 NumSockets = InstanceData->bReadsFilteredTransforms ? InstanceData->NumFilteredSockets : InstanceData->NumUnfilteredSockets;
				SocketIndices = MakeArrayView<int32>(InstanceData->SocketFilterUnfilteredIndex.GetData() + FirstSocket, NumSockets);
			}
			InstanceData->bNeedsRenderUpdate = InstanceData->SocketUtils->GetSocketTransforms(InstanceData->SocketTransforms, SocketIndices);

			// Transforms are being reset so copy current -> previous
			if (bResetPreviousTransform)
			{
				InstanceData->PreviousSocketTransforms = InstanceData->SocketTransforms;
			}
		}

		// Update the transform
		const FTransform ComponentToWorld = InstanceData->SocketUtils->GetLocalToWorldTransform();
		if (bResetPreviousTransform)
		{
			InstanceData->ComponentToWorld			= ComponentToWorld;
			InstanceData->PreviousComponentToWorld	= InstanceData->ComponentToWorld;
		}
		else
		{
			InstanceData->PreviousComponentToWorld	= InstanceData->ComponentToWorld;
			InstanceData->ComponentToWorld			= ComponentToWorld;
		}
		const FNiagaraLWCConverter LWCConverter = SystemInstance->GetLWCConverter();
		InstanceData->ComponentToTranslatedWorld = LWCConverter.ConvertWorldToSimulationTransform(InstanceData->ComponentToWorld);
		InstanceData->PreviousComponentToTranslatedWorld = LWCConverter.ConvertWorldToSimulationTransform(InstanceData->PreviousComponentToWorld);
	}

	//////////////////////////////////////////////////////////////////////////

	enum class ESocketReadType
	{
		None,
		Any,
		Filtered,
		Unfiltered,
	};

	void VMIsValid(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);
		FNDIOutputParam<bool>	OutIsValid(Context);

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutIsValid.SetAndAdvance(InstanceData->bIsDataValid);
		}
	}

	void VMGetComponentToWorld(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);
		FNDIOutputParam<FVector3f>	OutPosition(Context);
		FNDIOutputParam<FQuat4f>	OutRotation(Context);
		FNDIOutputParam<FVector3f>	OutScale(Context);

		const FVector3f Translation = InstanceData->ComponentToTranslatedWorld.GetTranslation();
		const FQuat4f	Rotation	= InstanceData->ComponentToTranslatedWorld.GetRotation();
		const FVector3f	Scale		= InstanceData->ComponentToTranslatedWorld.GetScale3D();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutPosition.SetAndAdvance(Translation);
			OutRotation.SetAndAdvance(Rotation);
			OutScale.SetAndAdvance(Scale);
		}
	}

	template<ESocketReadType ReadType>
	void VMGetSocketCount(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);
		FNDIOutputParam<int32>	OutCount(Context);

		int32 SocketCount = 0;
		switch (ReadType)
		{
			case ESocketReadType::Filtered:		SocketCount = InstanceData->NumFilteredSockets; break;
			case ESocketReadType::Unfiltered:	SocketCount = InstanceData->NumUnfilteredSockets; break;
			default:							SocketCount = InstanceData->NumSockets; break;
		}

		for (int32 i=0; i < Context.GetNumInstances(); ++i)
		{
			OutCount.SetAndAdvance(SocketCount);
		}
	}

	template<ESocketReadType ReadType, bool bInterpolated>
	void VMGetSocketTransform(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData(Context);
		FNDIInputParam<int32>		InSocketIndex(Context);
		FNDIInputParam<float>		InInterpolation;
		if constexpr ( bInterpolated )
		{
			InInterpolation.Init(Context);
		}

		FNDIOutputParam<FVector3f>	OutPosition(Context);
		FNDIOutputParam<FQuat4f>	OutRotation(Context);
		FNDIOutputParam<FVector3f>	OutScale(Context);
		FNDIOutputParam<FVector3f>	OutVelocity(Context);

		for (int32 i=0; i < Context.GetNumInstances(); ++i)
		{
			int32 SocketIndex = InSocketIndex.GetAndAdvance();
			const float Interpolation = bInterpolated ? InInterpolation.GetAndAdvance() : 1.0f;

			switch (ReadType)
			{
				case ESocketReadType::Filtered:
				{
					const int32 FilteredSockedIndex = SocketIndex >= 0 && SocketIndex < InstanceData->NumFilteredSockets ? (SocketIndex + 1) : 0;
					SocketIndex =  InstanceData->SocketFilterUnfilteredIndex[FilteredSockedIndex];
					break;
				}

				case ESocketReadType::Unfiltered:
				{
					const int32 UnfilteredSockedIndex = SocketIndex >= 0 && SocketIndex < InstanceData->NumUnfilteredSockets ? (SocketIndex + 1 + InstanceData->NumFilteredSockets) : 0;
					SocketIndex = InstanceData->SocketFilterUnfilteredIndex[UnfilteredSockedIndex];
					break;
				}

				default:
					SocketIndex = SocketIndex >= 0 && SocketIndex < InstanceData->NumSockets ? SocketIndex : InstanceData->NumSockets;
					break;
			}

			const FTransform3f PreviousSocketTransform = InstanceData->PreviousSocketTransforms[SocketIndex] * InstanceData->PreviousComponentToTranslatedWorld;
			const FTransform3f SocketTransform = InstanceData->SocketTransforms[SocketIndex] * InstanceData->ComponentToTranslatedWorld;

			FVector3f Position	= SocketTransform.GetTranslation();
			FQuat4f Rotation	= SocketTransform.GetRotation();
			FVector3f Scale		= SocketTransform.GetScale3D();
			FVector3f Velocity	= (Position - PreviousSocketTransform.GetTranslation()) * InstanceData->InvDeltaSeconds;

			if constexpr (bInterpolated)
			{
				Position	= FMath::Lerp(PreviousSocketTransform.GetTranslation(), Position, Interpolation);
				Rotation	= FQuat4f::Slerp(PreviousSocketTransform.GetRotation(), Rotation, Interpolation);
				Scale		= FMath::Lerp(PreviousSocketTransform.GetScale3D(), Scale, Interpolation);
			}

			OutPosition.SetAndAdvance(Position);
			OutRotation.SetAndAdvance(Rotation);
			OutScale.SetAndAdvance(Scale);
			OutVelocity.SetAndAdvance(Velocity);
		}
	}

	struct FVMFunctionInfo
	{
		FVMExternalFunction	FunctionBinding;
		ESocketReadType		TransformReadType;
	};

	static const TMap<FName, FVMFunctionInfo> VMFunctionInfos =
	{
		{IsValidName,									FVMFunctionInfo{FVMExternalFunction::CreateStatic(VMIsValid), ESocketReadType::None}},
		{GetComponentToWorldName,						FVMFunctionInfo{FVMExternalFunction::CreateStatic(VMGetComponentToWorld), ESocketReadType::None}},
		{GetSocketCountName,							FVMFunctionInfo{FVMExternalFunction::CreateStatic(VMGetSocketCount<ESocketReadType::Any>), ESocketReadType::None}},
		{GetFilteredSocketCountName,					FVMFunctionInfo{FVMExternalFunction::CreateStatic(VMGetSocketCount<ESocketReadType::Filtered>), ESocketReadType::None}},
		{GetUnfilteredSocketCountName,					FVMFunctionInfo{FVMExternalFunction::CreateStatic(VMGetSocketCount<ESocketReadType::Unfiltered>), ESocketReadType::None}},
		{GetSocketTransformName,						FVMFunctionInfo{FVMExternalFunction::CreateStatic(VMGetSocketTransform<ESocketReadType::Any, false>), ESocketReadType::Any}},
		{GetFilteredSocketTransformName,				FVMFunctionInfo{FVMExternalFunction::CreateStatic(VMGetSocketTransform<ESocketReadType::Filtered, false>), ESocketReadType::Filtered}},
		{GetUnfilteredSocketTransformName,				FVMFunctionInfo{FVMExternalFunction::CreateStatic(VMGetSocketTransform<ESocketReadType::Unfiltered, false>), ESocketReadType::Unfiltered}},
		{GetSocketTransformInterpolatedName,			FVMFunctionInfo{FVMExternalFunction::CreateStatic(VMGetSocketTransform<ESocketReadType::Any, true>), ESocketReadType::Any}},
		{GetFilteredSocketTransformInterpolatedName,	FVMFunctionInfo{FVMExternalFunction::CreateStatic(VMGetSocketTransform<ESocketReadType::Filtered, true>), ESocketReadType::Filtered}},
		{GetUnfilteredSocketTransformInterpolatedName,	FVMFunctionInfo{FVMExternalFunction::CreateStatic(VMGetSocketTransform<ESocketReadType::Unfiltered, true>), ESocketReadType::Unfiltered}},
	};
}

//////////////////////////////////////////////////////////////////////////
// Data Interface
UNiagaraDataInterfaceSocketReader::UNiagaraDataInterfaceSocketReader(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new NDISocketReaderLocal::FNDIProxy());

	FNiagaraTypeDefinition Def(UObject::StaticClass());
	ObjectParameterBinding.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceSocketReader::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSocketReader::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDISocketReaderLocal;
	FNiagaraFunctionSignature ImmutableSig;
	ImmutableSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("SocketReader"));
	ImmutableSig.bMemberFunction = true;
	ImmutableSig.bRequiresContext = false;
	ImmutableSig.bSupportsGPU = true;
	{
		FNiagaraFunctionSignature& FunctionSignature = OutFunctions.Add_GetRef(ImmutableSig);
		FunctionSignature.Name = IsValidName;
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
	}
	{
		FNiagaraFunctionSignature& FunctionSignature = OutFunctions.Add_GetRef(ImmutableSig);
		FunctionSignature.Name = GetComponentToWorldName;
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Translation"));
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation"));
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale"));
	}
	{
		FNiagaraFunctionSignature FunctionSignature = ImmutableSig;
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count"));

		OutFunctions.Add_GetRef(FunctionSignature).Name = GetSocketCountName;
		OutFunctions.Add_GetRef(FunctionSignature).Name = GetFilteredSocketCountName;
		OutFunctions.Add_GetRef(FunctionSignature).Name = GetUnfilteredSocketCountName;
	}
	{
		FNiagaraFunctionSignature FunctionSignature = ImmutableSig;
		FunctionSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SocketIndex"));
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation"));
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale"));
		FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity"));

		OutFunctions.Add_GetRef(FunctionSignature).Name = GetSocketTransformName;
		OutFunctions.Add_GetRef(FunctionSignature).Name = GetFilteredSocketTransformName;
		OutFunctions.Add_GetRef(FunctionSignature).Name = GetUnfilteredSocketTransformName;

		FunctionSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interpolation"));
		OutFunctions.Add_GetRef(FunctionSignature).Name = GetSocketTransformInterpolatedName;
		OutFunctions.Add_GetRef(FunctionSignature).Name = GetFilteredSocketTransformInterpolatedName;
		OutFunctions.Add_GetRef(FunctionSignature).Name = GetUnfilteredSocketTransformInterpolatedName;
	}
}
#endif

void UNiagaraDataInterfaceSocketReader::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* PerInstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDISocketReaderLocal;
	if (const FVMFunctionInfo* FunctionInfo = VMFunctionInfos.Find(BindingInfo.Name))
	{
		OutFunc = FunctionInfo->FunctionBinding;
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceSocketReader::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDISocketReaderLocal::TemplateShaderFile);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceSocketReader::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDISocketReaderLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceSocketReader::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDISocketReaderLocal;
	//return (FunctionInfo.DefinitionName == GetMatrixName) || (FunctionInfo.DefinitionName == GetTransformName) || (FunctionInfo.DefinitionName == GetVelocityName);
	return true;
}
#endif

void UNiagaraDataInterfaceSocketReader::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceSocketReader::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDISocketReaderLocal;

	FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	FInstanceData_RenderThread& InstanceData = DIProxy.SystemInstancesToInstanceData_RT.FindChecked(Context.GetSystemInstanceID());

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	FRDGBufferRef RDGBuffer = GraphBuilder.RegisterExternalBuffer(InstanceData.PooledBuffer);
	FRDGBufferSRVRef RDGBufferSRV = GraphBuilder.CreateSRV(RDGBuffer);

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->IsDataValid										= InstanceData.bIsDataValid ? 1 : 0;
	ShaderParameters->InvDeltaSeconds									= InstanceData.InvDeltaSeconds;
	ShaderParameters->NumSockets										= InstanceData.NumSockets;
	ShaderParameters->NumFilteredSockets								= InstanceData.NumFilteredSockets;
	ShaderParameters->NumUnfilteredSockets								= InstanceData.NumUnfilteredSockets;
	ShaderParameters->ComponentToTranslatedWorld_Translation			= InstanceData.ComponentToTranslatedWorld.GetTranslation();
	ShaderParameters->ComponentToTranslatedWorld_Rotation				= InstanceData.ComponentToTranslatedWorld.GetRotation();
	ShaderParameters->ComponentToTranslatedWorld_Scale					= InstanceData.ComponentToTranslatedWorld.GetScale3D();
	ShaderParameters->PreviousComponentToTranslatedWorld_Translation	= InstanceData.PreviousComponentToTranslatedWorld.GetTranslation();
	ShaderParameters->PreviousComponentToTranslatedWorld_Rotation		= InstanceData.PreviousComponentToTranslatedWorld.GetRotation();
	ShaderParameters->PreviousComponentToTranslatedWorld_Scale			= InstanceData.PreviousComponentToTranslatedWorld.GetScale3D();
	ShaderParameters->SocketTransformOffset								= InstanceData.SocketTransformOffset;
	ShaderParameters->PreviousSocketTransformOffset						= InstanceData.PreviousSocketTransformOffset;
	ShaderParameters->SocketData										= RDGBufferSRV;
}

bool UNiagaraDataInterfaceSocketReader::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDISocketReaderLocal;

	FInstanceData_GameThread* InstanceData = new (PerInstanceData) FInstanceData_GameThread;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InstanceData->SocketUtils.Reset(FNiagaraSystemInstance::GetSceneComponentUtils(SystemInstance).CreateSocketUtils());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	check(InstanceData->SocketUtils.IsValid());
	if ( SourceMode == ENDISocketReaderSourceMode::Default || SourceMode == ENDISocketReaderSourceMode::ParameterBindingOnly )
	{
		InstanceData->SocketUtils->SetAllowParameterBinding(SystemInstance, ObjectParameterBinding.Parameter);
	}

	if ( SourceMode == ENDISocketReaderSourceMode::Default || SourceMode == ENDISocketReaderSourceMode::AttachedParentOnly )
	{
		InstanceData->SocketUtils->SetAllowAttachParent(AttachComponentClass, AttachComponentTag);
	}

	if ( SourceMode == ENDISocketReaderSourceMode::Default || SourceMode == ENDISocketReaderSourceMode::SourceOnly )
	{
		InstanceData->SocketUtils->SetAllowSource(SourceActor, SourceAsset);
	}
#if WITH_EDITOR
	InstanceData->SocketUtils->SetEditorPreviewAsset(EditorPreviewAsset);
#endif

	InstanceData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), ObjectParameterBinding.Parameter);
	ResolveObject(this, InstanceData, SystemInstance);

	InstanceData->bReadsFilteredTransforms |= GDisableTransformReadType;
	InstanceData->bReadsUnfilteredTransforms |= GDisableTransformReadType;

	if (IsUsedWithCPUScript() && (!InstanceData->bReadsFilteredTransforms || !InstanceData->bReadsUnfilteredTransforms))
	{
		FNiagaraDataInterfaceUtilities::ForEachVMFunction(
			this, SystemInstance,
			[&](const UNiagaraScript* Script, const FVMExternalFunctionBindingInfo& Function)
			{
				if (const FVMFunctionInfo* FunctionInfo = VMFunctionInfos.Find(Function.Name))
				{
					InstanceData->bReadsFilteredTransforms |= FunctionInfo->TransformReadType == ESocketReadType::Any || FunctionInfo->TransformReadType == ESocketReadType::Filtered;
					InstanceData->bReadsUnfilteredTransforms |= FunctionInfo->TransformReadType == ESocketReadType::Any || FunctionInfo->TransformReadType == ESocketReadType::Unfiltered;
				}
				return !InstanceData->bReadsFilteredTransforms || !InstanceData->bReadsUnfilteredTransforms;
			}
		);
	}

	if (IsUsedWithGPUScript() && (!InstanceData->bReadsFilteredTransforms || !InstanceData->bReadsUnfilteredTransforms))
	{
		FNiagaraDataInterfaceUtilities::ForEachGpuFunction(
			this, SystemInstance,
			[&](const UNiagaraScript* Script, const FNiagaraDataInterfaceGeneratedFunction& Function)
			{
				if (const FVMFunctionInfo* FunctionInfo = VMFunctionInfos.Find(Function.DefinitionName))
				{
					InstanceData->bReadsFilteredTransforms |= FunctionInfo->TransformReadType == ESocketReadType::Any || FunctionInfo->TransformReadType == ESocketReadType::Filtered;
					InstanceData->bReadsUnfilteredTransforms |= FunctionInfo->TransformReadType == ESocketReadType::Any || FunctionInfo->TransformReadType == ESocketReadType::Unfiltered;
				}
				return !InstanceData->bReadsFilteredTransforms || !InstanceData->bReadsUnfilteredTransforms;
			}
		);
	}

	return true;
}

void UNiagaraDataInterfaceSocketReader::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDISocketReaderLocal;

	FInstanceData_GameThread* InstanceData = (FInstanceData_GameThread*)PerInstanceData;
	InstanceData->~FInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(RemoveProxy)
	(
		[RT_Proxy=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->SystemInstancesToInstanceData_RT.Remove(InstanceID);
		}
	);
}

int32 UNiagaraDataInterfaceSocketReader::PerInstanceDataSize() const
{
	return sizeof(NDISocketReaderLocal::FInstanceData_GameThread);
}

bool UNiagaraDataInterfaceSocketReader::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDISocketReaderLocal;

	FInstanceData_GameThread* InstanceData = static_cast<FInstanceData_GameThread*>(PerInstanceData);
	InstanceData->DeltaSeconds = DeltaSeconds;
	InstanceData->InvDeltaSeconds = DeltaSeconds > 0.0f ? 1.0f / DeltaSeconds : 0.0f;
	ResolveObject(this, InstanceData, SystemInstance);
	UpdateSocketCache(this, InstanceData, SystemInstance);

	return false;
}

void UNiagaraDataInterfaceSocketReader::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	using namespace NDISocketReaderLocal;
	FNDIProxy::ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, SystemInstance);
}

ETickingGroup UNiagaraDataInterfaceSocketReader::CalculateTickGroup(const void* PerInstanceData) const
{
	using namespace NDISocketReaderLocal;
	if ( bRequireCurrentFrameData && PerInstanceData)
	{
		const FInstanceData_GameThread* InstanceData = static_cast<const FInstanceData_GameThread*>(PerInstanceData);
		//-TODO: Not sure how we will handle tick groups with Entity vs Actor?
		const USceneComponent* SceneComponent = Cast<USceneComponent>(InstanceData->SocketUtils->GetResolvedObject());
		if (SceneComponent)
		{
			ETickingGroup FinalTickGroup = FMath::Max(SceneComponent->PrimaryComponentTick.TickGroup, SceneComponent->PrimaryComponentTick.EndTickGroup);
	//		//-TODO: Do we need to do this?
	//		//if ( USkeletalMeshComponent* SkelMeshComponent = Cast<USkeletalMeshComponent>(ActorComponent) )
	//		//{
	//		//	if (SkelMeshComponent->bBlendPhysics)
	//		//	{
	//		//		FinalTickGroup = FMath::Max(FinalTickGroup, TG_EndPhysics);
	//		//	}
	//		//}
			FinalTickGroup = FMath::Clamp(ETickingGroup(FinalTickGroup + 1), NiagaraFirstTickGroup, NiagaraLastTickGroup);
			return FinalTickGroup;
		}
	}
	return NiagaraFirstTickGroup;
}

bool UNiagaraDataInterfaceSocketReader::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfaceSocketReader* OtherTyped = CastChecked<const UNiagaraDataInterfaceSocketReader>(Other);
	return OtherTyped->SourceMode == SourceMode
		&& OtherTyped->FilteredSockets == FilteredSockets
	#if WITH_EDITORONLY_DATA
		&& OtherTyped->EditorPreviewAsset == EditorPreviewAsset
	#endif
		&& OtherTyped->SourceActor == SourceActor
		&& OtherTyped->SourceAsset == SourceAsset
		&& OtherTyped->AttachComponentClass == AttachComponentClass
		&& OtherTyped->AttachComponentTag == AttachComponentTag
		&& OtherTyped->ObjectParameterBinding == ObjectParameterBinding
		&& OtherTyped->bUpdateSocketsPerFrame == bUpdateSocketsPerFrame
		&& OtherTyped->bRequireCurrentFrameData == bRequireCurrentFrameData;
}

bool UNiagaraDataInterfaceSocketReader::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceSocketReader* OtherTyped = CastChecked<UNiagaraDataInterfaceSocketReader>(Destination);
	OtherTyped->SourceMode = SourceMode;
	OtherTyped->FilteredSockets = FilteredSockets;
#if WITH_EDITORONLY_DATA
	OtherTyped->EditorPreviewAsset = EditorPreviewAsset;
#endif
	OtherTyped->SourceActor = SourceActor;
	OtherTyped->SourceAsset = SourceAsset;
	OtherTyped->AttachComponentClass = AttachComponentClass;
	OtherTyped->AttachComponentTag = AttachComponentTag;
	OtherTyped->ObjectParameterBinding = ObjectParameterBinding;
	OtherTyped->bUpdateSocketsPerFrame = bUpdateSocketsPerFrame;
	OtherTyped->bRequireCurrentFrameData = bRequireCurrentFrameData;
	return true;
}

#if WITH_NIAGARA_DEBUGGER
void UNiagaraDataInterfaceSocketReader::DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const
{
	using namespace NDISocketReaderLocal;

	const FInstanceData_GameThread* InstanceData_GT = DebugHudContext.GetSystemInstance()->FindTypedDataInterfaceInstanceData<FInstanceData_GameThread>(this);
	if (InstanceData_GT == nullptr)
	{
		return;
	}

	UObject* ResolvedObject = InstanceData_GT->SocketUtils->GetResolvedObject();
	DebugHudContext.GetOutputString().Appendf(TEXT("ResolvedObject(%s)"), *GetNameSafe(ResolvedObject));

	UCanvas* Canvas = DebugHudContext.GetCanvas();
	if (DebugHudContext.IsVerbose() && ResolvedObject && Canvas)
	{
		for ( const FTransform3f SocketTransform : InstanceData_GT->SocketTransforms )
		{
			const FTransform WorldTransform = FTransform(SocketTransform) * InstanceData_GT->ComponentToWorld;
			const FVector SocketLocation = WorldTransform.GetLocation();
			const FVector ScreenPos = Canvas->Project(SocketLocation, false);
			if (ScreenPos.Z <= 0.0f)
			{
				continue;
			}

			Canvas->Canvas->DrawNGon(FVector2D(ScreenPos), FColor::Red, 8, 4.0f);
		}
	}
}
#endif

#if WITH_EDITORONLY_DATA
TArray<FName> UNiagaraDataInterfaceSocketReader::GetEditorSocketNames() const
{
	TArray<FName> SocketNames;
	if (UObject* EditorAsset = EditorPreviewAsset.LoadSynchronous())
	{
		FNiagaraActorSceneComponentSocketUtils SocketUtils(nullptr);
		SocketUtils.SetAllowSource(nullptr, EditorAsset);
		SocketUtils.ResolveObject();
		SocketNames.Append(SocketUtils.GetSocketNames());
	}
	return SocketNames;
}
#endif

#undef LOCTEXT_NAMESPACE

