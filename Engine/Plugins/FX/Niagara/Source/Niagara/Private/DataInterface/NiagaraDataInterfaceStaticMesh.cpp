// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceStaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "NiagaraStats.h"
#include "Containers/StridedView.h"
#include "NiagaraSystemInstance.h"
#include "Engine/StaticMesh.h"
#include "NiagaraRenderer.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "NiagaraSceneComponentUtils.h"
#include "NiagaraSettings.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraDataInterfaceMeshCommon.h"
#include "NiagaraWorldManager.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraSystem.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "StaticMeshComponentLODInfo.h"
#include "NiagaraStats.h"
#include "PrimitiveSceneInfo.h"

#if WITH_EDITOR
#include "Editor.h"
#include "StaticMeshResources.h"
#include "Subsystems/ImportSubsystem.h"
#include "INiagaraEditorOnlyDataUtlities.h"
#include "Modules/ModuleManager.h"
#include "NiagaraModule.h"
#include "ScopedTransaction.h"
#endif

#include "Engine/StaticMeshActor.h"
#include "Engine/StaticMeshSocket.h"
#include "NiagaraDataInterfaceStaticMeshUvMapping.h"
#include "Experimental/NiagaraMeshUvMappingHandle.h"
#include "Internationalization/Internationalization.h"
#include "ShaderCompilerCore.h"

#include "NiagaraGpuComputeDispatchInterface.h"
#include "FXRenderingUtils.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceStaticMesh)

#if WITH_EDITOR
#include "Subsystems/ImportSubsystem.h"
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceStaticMesh"

//-OPT: We should be able to share data between data interfaces where possible

namespace NDIStaticMeshLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
		SHADER_PARAMETER(FIntVector,			NumTriangles)		// x = Num Sections, y = Num Filtered, z = Num Unfiltered
		SHADER_PARAMETER(int,					NumVertices)
		SHADER_PARAMETER(int,					NumUVs)
		SHADER_PARAMETER(uint32,				HasColors)
		SHADER_PARAMETER_SRV(Buffer<uint>,		IndexBuffer)
		SHADER_PARAMETER_SRV(Buffer<float>,		PositionBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	TangentBuffer)
		SHADER_PARAMETER_SRV(Buffer<float2>,	UVBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	ColorBuffer)

		SHADER_PARAMETER(uint32,				HasUniformSampling)
		SHADER_PARAMETER_SRV(Buffer<uint2>,		UniformSamplingTriangles)

		SHADER_PARAMETER(FIntVector,			SectionCounts)		// x = Num Sections, y = Num Filtered, z = Num Unfiltered
		SHADER_PARAMETER_SRV(Buffer<uint4>,		SectionInfos)		// FirstTriangle, NumTriangles, Prob, Alias
		SHADER_PARAMETER_SRV(Buffer<uint>,		FilteredAndUnfilteredSections)

		SHADER_PARAMETER(FIntVector,			SocketCounts)		// x = Num Sockets, y = Num Filtered, z = Num Unfiltered
		SHADER_PARAMETER_SRV(Buffer<float4>,	SocketTransforms)
		SHADER_PARAMETER_SRV(Buffer<uint>,		FilteredAndUnfilteredSockets)

		SHADER_PARAMETER(float,					InvDeltaSeconds)
		SHADER_PARAMETER(FMatrix44f,			InstanceTransform)
		SHADER_PARAMETER(FMatrix44f,			InstanceTransformInverseTransposed)
		SHADER_PARAMETER(FQuat4f,				InstanceRotation)
		SHADER_PARAMETER(FMatrix44f,			InstancePreviousTransform)
		SHADER_PARAMETER(FMatrix44f,			InstancePreviousTransformInverseTransposed)
		SHADER_PARAMETER(FQuat4f,				InstancePreviousRotation)
		SHADER_PARAMETER(FVector3f,				InstanceWorldVelocity)

		SHADER_PARAMETER(int,					InstanceDistanceFieldIndex)

		SHADER_PARAMETER(FVector3f,				PreSkinnedLocalBoundsCenter)
		SHADER_PARAMETER(FVector3f,				PreSkinnedLocalBoundsExtents)
		SHADER_PARAMETER(FVector3f,				MeshBoundsWSCenter)
		SHADER_PARAMETER(FVector3f,				MeshBoundsWSExtents)
		SHADER_PARAMETER(FVector3f,				OwnerToMeshVector)

		SHADER_PARAMETER_SRV(Buffer<int>,		UvMappingBuffer)
		SHADER_PARAMETER(uint32,				UvMappingBufferLength)
		SHADER_PARAMETER(uint32,				UvMappingSet)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceStaticMeshTemplate.ush");

	static int32 GNiagaraFailStaticMeshDataInterface = 0;
	static FAutoConsoleVariableRef CVarNiagaraFailStaticMeshDataInterface(
		TEXT("fx.Niagara.FailStaticMeshDataInterface"),
		GNiagaraFailStaticMeshDataInterface,
		TEXT("When enabled we will fail out using static mesh data interfaces."),
		ECVF_Default
	);

	static int32 GNDIStaticMesh_UseInlineLODsOnly = 2;
	static FAutoConsoleVariableRef CVarNDIStaticMesh_UseInlineLODsOnly(
		TEXT("fx.Niagara.NDIStaticMesh.UseInlineLODsOnly"),
		GNDIStaticMesh_UseInlineLODsOnly,
		TEXT("When enabled Niagara will never use streaming LOD levels, only inline LODs."
		"0 = Streaming LODs can be sampled."
		"1 = Only inlined LODs can be sampled."
		"2 = Only inlined LODs can be sampled by default but each DI can override this if desired. "),
		ECVF_Default
	);

	//////////////////////////////////////////////////////////////////////////
	struct EDIFunctionVersion
	{
		enum Type
		{
			InitialVersion = 0,
			RefactoredV1 = 1,
			LargeWorldCoordinates = 2,
			LargeWorldCoordinates2 = 3,
			AddSocketVelocity = 4,

			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};
	};

	//////////////////////////////////////////////////////////////////////////
	// Vertex Sampling
	static const FName	IsValidVertexName("IsValidVertex");
	static const FName	RandomVertexName("RandomVertex");
	static const FName	GetVertexCountName("GetVertexCount");

	static const FName	GetVertexName("GetVertex");
	static const FName	GetVertexWSName("GetVertexWS");
	static const FName	GetVertexWSInterpolatedName("GetVertexWSInterpolated");
	static const FName	GetVertexColorName("GetVertexColor");
	static const FName	GetVertexUVName("GetVertexUV");

	//////////////////////////////////////////////////////////////////////////
	// VM Triangle Sampling
	static const FName	IsValidTriangleName("IsValidTriangle");
	static const FName	RandomTriangleName("RandomTriangle");
	static const FName	GetTriangleCountName("GetTriangleCount");

	static const FName	IsValidFilteredTriangleName("IsValidFilteredTriangle");
	static const FName	RandomFilteredTriangleName("RandomFilteredTriangle");
	static const FName	GetFilteredTriangleCountName("GetFilteredTriangleCount");
	static const FName	GetFilteredTriangleAtName("GetFilteredTriangleAt");

	static const FName	IsValidUnfilteredTriangleName("IsValidUnfilteredTriangle");
	static const FName	RandomUnfilteredTriangleName("RandomUnfilteredTriangle");
	static const FName	GetUnfilteredTriangleCountName("GetUnfilteredTriangleCount");
	static const FName	GetUnfilteredTriangleAtName("GetUnfilteredTriangleAt");

	static const FName	GetTriangleName("GetTriangle");
	static const FName	GetTriangleWSName("GetTriangleWS");
	static const FName	GetTriangleWSInterpolatedName("GetTriangleWSInterpolated");
	static const FName	GetTriangleColorName("GetTriangleColor");
	static const FName	GetTriangleUVName("GetTriangleUV");

	static const FName	GetTriangleIndicesName("GetTriangleIndices");

	//////////////////////////////////////////////////////////////////////////
	// Socket Functions
	static const FName	GetSocketCountName("GetSocketCount");
	static const FName	GetFilteredSocketCountName("GetFilteredSocketCount");
	static const FName	GetUnfilteredSocketCountName("GetUnfilteredSocketCount");
	
	static const FName	RandomSocketName("RandomSocket");
	static const FName	RandomFilteredSocketName("RandomFilteredSocket");
	static const FName	RandomUnfilteredSocketName("RandomUnfilteredSocket");
	
	static const FName	GetSocketTransformName("GetSocketTransform");
	static const FName	GetSocketTransformWSName("GetSocketTransformWS");
	static const FName	GetSocketTransformWSInterpolatedName("GetSocketTransformWSInterpolated");

	static const FName	GetFilteredSocketTransformName("GetFilteredSocketTransform");
	static const FName	GetFilteredSocketTransformWSName("GetFilteredSocketTransformWS");
	static const FName	GetFilteredSocketTransformWSInterpolatedName("GetFilteredSocketTransformWSInterpolated");

	static const FName	GetUnfilteredSocketTransformName("GetUnfilteredSocketTransform");
	static const FName	GetUnfilteredSocketTransformWSName("GetUnfilteredSocketTransformWS");
	static const FName	GetUnfilteredSocketTransformWSInterpolatedName("GetUnfilteredSocketTransformWSInterpolated");

	static const FName	GetFilteredSocketName("GetFilteredSocket");
	static const FName	GetUnfilteredSocketName("GetUnfilteredSocket");

	//////////////////////////////////////////////////////////////////////////
	// Section functions
	static const FName	IsValidSectionName("IsValidSection");
	static const FName	GetSectionTriangleCountName("GetSectionTriangleCount");
	static const FName	RandomSectionTriangleName("RandomSectionTriangle");
	static const FName	GetSectionTriangleAtName("GetSectionTriangleAt");

	static const FName	GetFilteredSectionAtName("GetFilteredSectionAt");
	static const FName	GetUnfilteredSectionAtName("GetUnfilteredSectionAt");

	static const FName	GetSectionCountName("GetSectionCount");
	static const FName	GetFilteredSectionCountName("GetFilteredSectionCount");
	static const FName	GetUnfilteredSectionCountName("GetUnfilteredSectionCount");

	static const FName	RandomSectionName("RandomSection");
	static const FName	RandomFilteredSectionName("RandomFilteredSection");
	static const FName	RandomUnfilteredSectionName("RandomUnfilteredSection");

	//////////////////////////////////////////////////////////////////////////
	// VM Misc Functions
	static const FName	IsValidName("IsValid");

	static const FName	GetPreSkinnedLocalBoundsName("GetPreSkinnedLocalBounds");
	static const FName	GetMeshBoundsName("GetMeshBounds");
	static const FName	GetMeshBoundsWSName("GetMeshBoundsWS");

	static const FName	GetLocalToWorldName("GetLocalToWorld");
	static const FName	GetLocalToWorldInverseTransposedName("GetLocalToWorldInverseTransposed");
	static const FName	GetWorldVelocityName("GetWorldVelocity");

	static const FName	GetInstanceIndexName("GetInstanceIndex");
	static const FName	SetInstanceIndexName("SetInstanceIndex");

	//////////////////////////////////////////////////////////////////////////
	// VM UV mapping functions
	static const FName	GetTriangleCoordAtUVName("GetTriangleCoordAtUV");
	static const FName	GetTriangleCoordInAabbName("GetTriangleCoordInAabb");

	static const FName	BuildUvMappingName("BuildUvMapping");

	//////////////////////////////////////////////////////////////////////////
	// Distance Field Functions
	static const FName	QueryDistanceFieldName("QueryDistanceField");

	//////////////////////////////////////////////////////////////////////////
	// Deprecated Functions
	static const FName Deprecated_RandomSectionName("DeprecatedRandomSection");
	static const FName Deprecated_RandomTriCoordName("DeprecatedRandomTriCoord");
	static const FName Deprecated_RandomTriCoordOnSectionName("DeprecatedRandomTriCoordOnSection");
	static const FName Deprecated_RandomTriCoordVCFilteredName("DeprecatedRandomTriCoordUsingVertexColorFilter");
	static const FName Deprecated_GetVertexPositionName("DeprecatedGetVertexPosition");
	static const FName Deprecated_GetVertexPositionWSName("DeprecatedGetVertexPositionWS");
	static const FName Deprecated_GetTriPositionName("DeprecatedGetTriPosition");
	static const FName Deprecated_GetTriNormalName("DeprecatedGetTriNormal");
	static const FName Deprecated_GetTriTangentsName("DeprecatedGetTriTangents");
	static const FName Deprecated_GetTriPositionWSName("DeprecatedGetTriPositionWS");
	static const FName Deprecated_GetTriNormalWSName("DeprecatedGetTriNormalWS");
	static const FName Deprecated_GetTriTangentsWSName("DeprecatedGetTriTangentsWS");
	static const FName Deprecated_GetTriColorName("DeprecatedGetTriColor");
	static const FName Deprecated_GetTriUVName("DeprecatedGetTriUV");
	static const FName Deprecated_GetTriPositionAndVelocityWSName("DeprecatedGetTriPositionAndVelocityWS");
	//static const FName Deprecated_GetMeshLocalToWorldInverseTransposedName("DeprecatedGetMeshLocalToWorldInverseTransposed");

	//////////////////////////////////////////////////////////////////////////
	struct FNDISectionInfo
	{
		uint32 FirstTriangle = 0;
		uint32 NumTriangles = 0;
		float  Prob = 0.0f;
		uint32 Alias = 0;
	};

	struct FNDITangentBasis
	{
		FVector3f TangentX;
		FVector3f TangentY;
		FVector3f TangentZ;
	
		template<typename TTransformHandler = FNDITransformHandlerNoop>
		void TransformAndEnsureValid(const TTransformHandler& TransformHandler, const FMatrix44f& Matrix)
		{
			if (TangentY.SquaredLength() <= 0.001f)
			{
				TangentX = FVector3f(1.0f, 0.0f, 0.0f);
				TangentY = FVector3f(0.0f, 1.0f, 0.0f);
				TangentZ = FVector3f(0.0f, 0.0f, 1.0f);
			}
			TransformHandler.TransformUnitVector(TangentX, Matrix);
			TransformHandler.TransformUnitVector(TangentY, Matrix);
			TransformHandler.TransformUnitVector(TangentZ, Matrix);
		}
	};

	class FNDISectionAreaWeightedSampler : public FWeightedRandomSampler
	{
	public:
		static void Init(const FStaticMeshLODResources* LODResource, TConstArrayView<int32> Sections, const FStaticMeshSectionAreaWeightedTriangleSamplerArray* MeshSectionSamplers, TArray<FNDISectionInfo>& OutSectionInfos)
		{
			if (MeshSectionSamplers)
			{
				FNDISectionAreaWeightedSampler Sampler;
				Sampler.Init(Sections, *MeshSectionSamplers);
				for (int32 i=0; i < Sections.Num(); ++i)
				{
					const int32 SectionIndex	= Sections[i];
					FNDISectionInfo& SectionInfo= OutSectionInfos.AddDefaulted_GetRef();
					SectionInfo.FirstTriangle	= LODResource->Sections[SectionIndex].FirstIndex / 3;
					SectionInfo.NumTriangles	= LODResource->Sections[SectionIndex].NumTriangles;
					SectionInfo.Prob			= Sampler.GetProb()[i];
					SectionInfo.Alias			= Sampler.GetAlias()[i];
				}
			}
			else
			{
				for (int32 i=0; i < Sections.Num(); ++i)
				{
					const int32 SectionIndex	= Sections[i];
					FNDISectionInfo& SectionInfo= OutSectionInfos.AddDefaulted_GetRef();
					SectionInfo.FirstTriangle	= LODResource->Sections[SectionIndex].FirstIndex / 3;
					SectionInfo.NumTriangles	= LODResource->Sections[SectionIndex].NumTriangles;
					SectionInfo.Prob			= 1.0f;
					SectionInfo.Alias			= i;
				}
			}
		}

		void Init(TConstArrayView<int32> Sections, const FStaticMeshSectionAreaWeightedTriangleSamplerArray& MeshSectionSamplers)
		{
			TempTotalWeight = 0.0f;
			TempWeights.Empty();

			for (const int32 SectionIndex : Sections)
			{
				const FStaticMeshSectionAreaWeightedTriangleSampler& SectionSampler = MeshSectionSamplers[SectionIndex];
				const float SectionWeight = SectionSampler.GetTotalWeight();
				TempTotalWeight += SectionWeight;
				TempWeights.Add(SectionWeight);
			}

			FWeightedRandomSampler::Initialize();
		}

	protected:
		virtual float GetWeights(TArray<float>& OutWeights) override
		{
			OutWeights = MoveTemp(TempWeights);
			return TempTotalWeight;
		}

		float			TempTotalWeight = 0.0f;
		TArray<float>	TempWeights;
	};

	//////////////////////////////////////////////////////////////////////////
	struct FGpuInitializeData
	{
		struct FRenderProxy*				RenderProxy = nullptr;
		FNiagaraSystemInstanceID			SystemInstanceID;
		TRefCountPtr<const FStaticMeshLODResources>		LODResource = nullptr;

#if !WITH_EDITOR
		FColorVertexBuffer*					OverrideColorBuffer = nullptr;
#endif

#if DO_CHECK
		FName								SystemFName;
		FName								StaticMeshFName;
#endif

		int32								NumFilteredTriangles = 0;
		int32								NumUnfilteredTriangles = 0;

		bool								bGpuUniformDistribution = false;

		int32								NumSections = 0;
		int32								NumFilteredSections = 0;
		int32								NumUnfilteredSections = 0;
		TResourceArray<FIntVector4>			SectionInfos;
		TResourceArray<uint16>				FilteredAndUnfilteredSections;

		int32								NumSockets = 0;
		int32								NumFilteredSockets = 0;
		int32								NumUnfilteredSockets = 0;
		TResourceArray<FVector4f>			SocketTransforms;
		TResourceArray<uint16>				FilteredAndUnfilteredSockets;
	};

	//////////////////////////////////////////////////////////////////////////
	struct FInstanceData_RenderThread
	{
		bool		bIsValid = false;
		bool		bGpuUniformDistribution = false;
		FMatrix44f	Transform = FMatrix44f::Identity;
		FMatrix44f	PrevTransform = FMatrix44f::Identity;
		FQuat4f		Rotation = FQuat4f::Identity;
		FQuat4f		PrevRotation = FQuat4f::Identity;
		float		DeltaSeconds = 0.0f;
		FVector3f	PreSkinnedLocalBoundsCenter = FVector3f::ZeroVector;
		FVector3f	PreSkinnedLocalBoundsExtents = FVector3f::ZeroVector;
		FVector3f	OwnerToMeshVector = FVector3f::ZeroVector;
		FPrimitiveComponentId	DistanceFieldPrimitiveId;

		FIntVector NumTriangles = FIntVector::ZeroValue;
		int32 NumVertices = 0;
		int32 NumUVs = 0;
		FShaderResourceViewRHIRef MeshIndexBufferSRV;
		FShaderResourceViewRHIRef MeshPositionBufferSRV;
		FShaderResourceViewRHIRef MeshTangentBufferSRV;
		FShaderResourceViewRHIRef MeshUVBufferSRV;
		FShaderResourceViewRHIRef MeshColorBufferSRV;

#if WITH_EDITOR
		// in the editor, there's the potential for dynamically updating (including getting cleared) vertex buffers.
		// At runtime we can just handle things during initialization of the instance data
		FShaderResourceViewRHIRef MeshOverrideColorBufferSRV;
#endif

		FShaderResourceViewRHIRef MeshUniformSamplingTriangleSRV;

		FIntVector	SectionCounts = FIntVector::ZeroValue;	// X=NumSections, y = Num Filtered, Z=NumUnfiltered
		FReadBuffer SectionInfos;
		FReadBuffer FilteredAndUnfilteredSections;

		FIntVector	SocketCounts = FIntVector::ZeroValue;	// X=NumSockets, Y=NumFiltered, Z=NumUnfiltered
		FReadBuffer	SocketTransforms;
		FReadBuffer	FilteredAndUnfilteredSockets;

	#if STATS
		int64 GPUMemoryUsage = 0;
	#endif

		const FMeshUvMappingBufferProxy* UvMappingBuffer = nullptr;
		uint32 UvMappingSet = 0;

		~FInstanceData_RenderThread()
		{
			Release();
		}

		void Init(FRHICommandListBase& RHICmdList, FGpuInitializeData& GpuInitializeData)
		{
			// Gather mesh buffers
			if (GpuInitializeData.LODResource)
			{
				FBufferRHIRef IndexBufferRHIRef = GpuInitializeData.LODResource->IndexBuffer.IndexBufferRHI;
				const bool bAllowCPUAccess = GpuInitializeData.LODResource->IndexBuffer.GetAllowCPUAccess();
				const bool bCanCreateIndexSRV =
					IndexBufferRHIRef.IsValid() &&
					((IndexBufferRHIRef->GetUsage() & EBufferUsageFlags::ShaderResource) == EBufferUsageFlags::ShaderResource) &&
					// Added just to keep old behavior, where whether the SRV should be created has been inferred from whether GPU Skin cache is enabled or not
					// Typically if GPU Skin Cache is enabled, the platform likely does not care about memory cost from extra SRVs
					// Individual systems can further decide what to do on platforms that don't support GPU Skin Cache
					(AreBufferSRVsAlwaysCreatedByDefault(GMaxRHIShaderPlatform) || bAllowCPUAccess); 

				if (bCanCreateIndexSRV)
				{
					bool b32Bit = GpuInitializeData.LODResource->IndexBuffer.Is32Bit();
					MeshIndexBufferSRV = RHICmdList.CreateShaderResourceView(
						IndexBufferRHIRef, 
						FRHIViewDesc::CreateBufferSRV()
							.SetType(FRHIViewDesc::EBufferType::Typed)
							.SetFormat(b32Bit ? PF_R32_UINT : PF_R16_UINT));
				}
				else
				{
#if DO_CHECK
					UE_LOGF(LogNiagara, Log, "NiagaraStaticMeshDataInterface used by GPU emitter but does not have SRV access on this platform.  Enable CPU access to fix this issue. System: %ls, Mesh: %ls", *GpuInitializeData.SystemFName.ToString(), *GpuInitializeData.StaticMeshFName.ToString());
#endif
					MeshIndexBufferSRV = nullptr;
				}

				MeshPositionBufferSRV = GpuInitializeData.LODResource->VertexBuffers.PositionVertexBuffer.GetSRV();
				MeshTangentBufferSRV = GpuInitializeData.LODResource->VertexBuffers.StaticMeshVertexBuffer.GetTangentsSRV();
				MeshUVBufferSRV = GpuInitializeData.LODResource->VertexBuffers.StaticMeshVertexBuffer.GetTexCoordsSRV();

#if !WITH_EDITOR
				if (GpuInitializeData.OverrideColorBuffer)
				{
					MeshColorBufferSRV = GpuInitializeData.OverrideColorBuffer->GetColorComponentsSRV();
				}
				else
#endif
				{
					MeshColorBufferSRV = GpuInitializeData.LODResource->VertexBuffers.ColorVertexBuffer.GetColorComponentsSRV();
				}

				NumTriangles.X = GpuInitializeData.LODResource->IndexBuffer.GetNumIndices() / 3;
				NumTriangles.Y = GpuInitializeData.NumFilteredTriangles;
				NumTriangles.Z = GpuInitializeData.NumUnfilteredTriangles;
				NumVertices = GpuInitializeData.LODResource->VertexBuffers.PositionVertexBuffer.GetNumVertices();
				NumUVs = MeshUVBufferSRV.IsValid() ? GpuInitializeData.LODResource->VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() : 0;

				bIsValid = MeshIndexBufferSRV.IsValid() && MeshPositionBufferSRV.IsValid() && MeshTangentBufferSRV.IsValid();

				// Build uniform sampling buffer
				MeshUniformSamplingTriangleSRV = GpuInitializeData.bGpuUniformDistribution ? GpuInitializeData.LODResource->AreaWeightedSectionSamplersBuffer.GetBufferSRV() : nullptr;
				bGpuUniformDistribution = GpuInitializeData.bGpuUniformDistribution && MeshUniformSamplingTriangleSRV.IsValid();

				// Build section buffers
				SectionCounts.X = GpuInitializeData.NumSections;
				SectionCounts.Y = GpuInitializeData.NumFilteredSections;
				SectionCounts.Z = GpuInitializeData.NumUnfilteredSections;
				if (GpuInitializeData.SectionInfos.Num() > 0)
				{
					SectionInfos.Initialize(RHICmdList, TEXT("NDISkelMesh_SectionInfos"), sizeof(FIntVector4), GpuInitializeData.SectionInfos.Num(), EPixelFormat::PF_R32G32B32A32_UINT, BUF_Static, &GpuInitializeData.SectionInfos);
				}
				if (GpuInitializeData.FilteredAndUnfilteredSections.Num() > 0)
				{
					FilteredAndUnfilteredSections.Initialize(RHICmdList, TEXT("NDISkelMesh_FilteredAndUnfilteredSections"), sizeof(uint16), GpuInitializeData.FilteredAndUnfilteredSections.Num(), EPixelFormat::PF_R16_UINT, BUF_Static, &GpuInitializeData.FilteredAndUnfilteredSections);
				}
			}
			else
			{
				NumTriangles = FIntVector::ZeroValue;
				NumVertices = 0;
				NumUVs = 0;

				bIsValid = false;

				bGpuUniformDistribution = false;

				SectionCounts = FIntVector::ZeroValue;
			}

			// Build socket buffers
			SocketCounts.X = GpuInitializeData.NumSockets;
			SocketCounts.Y = GpuInitializeData.NumFilteredSockets;
			SocketCounts.Z = GpuInitializeData.NumUnfilteredSockets;
			if ( GpuInitializeData.SocketTransforms.Num() > 0 )
			{
				SocketTransforms.Initialize(RHICmdList, TEXT("NDISkelMesh_SocketTransforms"), sizeof(FVector4f), GpuInitializeData.SocketTransforms.Num(), EPixelFormat::PF_A32B32G32R32F, BUF_Static, &GpuInitializeData.SocketTransforms);
			}
			if ( GpuInitializeData.FilteredAndUnfilteredSockets.Num() > 0 )
			{
				FilteredAndUnfilteredSockets.Initialize(RHICmdList, TEXT("NDISkelMesh_FilteredAndUnfilteredSockets"), sizeof(uint16), GpuInitializeData.FilteredAndUnfilteredSockets.Num(), EPixelFormat::PF_R16_UINT, BUF_Static, &GpuInitializeData.FilteredAndUnfilteredSockets);
			}
		#if STATS
			GPUMemoryUsage = SectionInfos.NumBytes + FilteredAndUnfilteredSections.NumBytes + SocketTransforms.NumBytes + FilteredAndUnfilteredSockets.NumBytes;
			INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GPUMemoryUsage);
		#endif
		}

		void Release()
		{
			bIsValid = false;

		#if STATS
			DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GPUMemoryUsage);
			GPUMemoryUsage = 0;
		#endif

			MeshIndexBufferSRV.SafeRelease();
			MeshPositionBufferSRV.SafeRelease();
			MeshTangentBufferSRV.SafeRelease();
			MeshUVBufferSRV.SafeRelease();
			MeshColorBufferSRV.SafeRelease();

			SectionInfos.Release();
			FilteredAndUnfilteredSections.Release();

			SocketTransforms.Release();
			FilteredAndUnfilteredSockets.Release();
		}
	};

	struct FInstanceData_FromGameThread
	{
		FMatrix44f				Transform = FMatrix44f::Identity;
		FMatrix44f				PrevTransform = FMatrix44f::Identity;
		FQuat4f					Rotation = FQuat4f::Identity;
		FQuat4f					PrevRotation = FQuat4f::Identity;
		float					DeltaSeconds = 0.0f;
		FVector3f				PreSkinnedLocalBoundsCenter = FVector3f::ZeroVector;
		FVector3f				PreSkinnedLocalBoundsExtents = FVector3f::ZeroVector;
		FVector3f				OwnerToMeshVector = FVector3f::ZeroVector;
		FPrimitiveComponentId	DistanceFieldPrimitiveId;

		const FMeshUvMappingBufferProxy* UvMappingBuffer = nullptr;
		uint32					UvMappingSet = 0;

#if WITH_EDITOR
		FColorVertexBuffer*		OverrideVertexColors = nullptr;
#endif
	};

	struct FInstanceData_GameThread
	{
		const INiagaraSceneComponentUtils* SceneComponentUtils = nullptr;

		/** Cached ptr to StaticMeshComponent we sample from, when found. Otherwise, the SceneComponent to use to transform the Default or Preview mesh. */
		TWeakObjectPtr<UObject> ComponentWeakPtr;

		/** Cached ptr to the mesh so that we can make sure that we haven't been deleted. */
		TWeakObjectPtr<UStaticMesh> StaticMeshWeakPtr;

		/** Handle to our uv mapping data. */
		FStaticMeshUvMappingHandle UvMapping;

		/** Vector from system owners instance location to the sampled mesh location. */
		FVector3f OwnerToMeshVector = FVector3f::ZeroVector;

		/** Which instance index we are reading from. */
		int32 InstanceIndex = INDEX_NONE;

		/** Cached ComponentToWorld. (Falls back to WorldTransform of the system instance) */
		FMatrix Transform;
		/** InverseTranspose of above for transforming normals/tangents. */
		FMatrix TransformInverseTransposed;

		/** Cached ComponentToWorld from previous tick. */
		FMatrix PrevTransform;
		/** Cached Previous InverseTranspose of above for transforming normals/tangents. */
		FMatrix PrevTransformInverseTransposed;

		/** When attached to an ISM contains all the transforms, so safe to use concurrently */
		TArray<FTransform> ISMTransforms;

		/** Cached SystemInstance World Transform */
		FTransform SystemInstanceWorldTransform = FTransform::Identity;

		/** Cached LWC tile offset */
		FVector LWCTileOffset = FVector::ZeroVector;

		/** Cached Rotation. */
		FQuat4f Rotation;
		/** Cached Previous Rotation. */
		FQuat4f PrevRotation;

		/** Time separating Transform and PrevTransform. */
		float DeltaSeconds;

		/** Preskinned local bounds pulled from attached static mesh */
		FVector3f PreSkinnedLocalBoundsCenter = FVector3f::ZeroVector;
		FVector3f PreSkinnedLocalBoundsExtents = FVector3f::ZeroVector;

		/** Velocity set by the physics body of the mesh component */
		FVector PhysicsVelocity;

		/** True we capture the transforms from the mesh component each frame. */
		uint32 bCaptureTransformsPerFrame : 1;

		/** True if velocity should not be calculated via the transforms, but rather read the physics data from the mesh component */
		uint32 bUsePhysicsVelocity : 1;

		/** True if SceneComponent was valid on initialization (used to track invalidation of the component on tick) */
		uint32 bComponentValid : 1;

		/** True if StaticMesh was valid on initialization (used to track invalidation of the mesh on tick) */
		uint32 bMeshValid : 1;

		/** True if the mesh allows CPU access. Use to reset the instance in the editor*/
		uint32 bMeshAllowsCpuAccess : 1;
		/** True if the mesh we're using allows area weighted sampling on CPU. */
		uint32 bIsCpuUniformlyDistributedSampling : 1;
		/** True if the mesh we're using allows area weighted sampling on GPU. */
		uint32 bIsGpuUniformlyDistributedSampling : 1;

		/** Filtered sections followed by unfiltered sections. */
		int32 NumFilteredSections = 0;
		int32 NumUnfilteredSections = 0;
		int32 NumFilteredTriangles = 0;
		int32 NumUnfilteredTriangles = 0;
		TArray<int32> FilteredAndUnfilteredSections;
		TArray<FNDISectionInfo> FilteredAndUnfilteredSectionInfos;

		///** Allows sampling of the mesh's tris based on a dynamic color range. */
		//TSharedPtr<struct FDynamicVertexColorFilterData> DynamicVertexColorSampler;

		/** Cached change id off of the data interface.*/
		uint32 ChangeId = 0;

		/** The MinLOD, see UStaticMesh::MinLOD which is platform specific.*/
		int32 MinLOD = 0;
		/** The cached LODIdx used to initialize the FNDIStaticMesh_InstanceData.*/
		int32 CachedLODIdx = 0;

		FNiagaraParameterDirectBinding<int32> LODIndexUserBinding;

		FNiagaraParameterDirectBinding<UObject*> MeshParameterBinding;

		/** Cached socket information, if available */
		TArray<FTransform3f> CachedSockets;

		/** Number of filtered sockets. */
		int32 NumFilteredSockets = 0;

		/** Filter sockets followed by unfiltered sockets */
		TArray<uint16> FilteredAndUnfilteredSockets;

		/** The UV index to use for UV mapping, -1 indicates the use of the feature is pending the call to BuildUvMapping() */
		int32 UvMappingIndexSet = -1;

		FMeshUvMappingUsage UvMappingUsage;

		bool CalculateLODIndex(UNiagaraDataInterfaceStaticMesh* Interface, FNiagaraSystemInstance* Instance, UStaticMesh* StaticMesh, int32& OutLOD, int32& OutMinLOD)
		{
			FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();

			OutLOD = INDEX_NONE;
			OutMinLOD = INDEX_NONE;
			if(Interface == nullptr || Instance == nullptr || RenderData == nullptr)
			{
				return false;
			}

			OutMinLOD = StaticMesh->GetMinLOD().GetValue();
			if (GNDIStaticMesh_UseInlineLODsOnly == 1 || (GNDIStaticMesh_UseInlineLODsOnly == 2 && !Interface->bAllowSamplingFromStreamingLODs))
			{
				OutMinLOD = StaticMesh->GetNumLODs() - StaticMesh->GetRenderData()->NumInlinedLODs;
			}

			int32 RequestedLOD = Interface->LODIndex;
			if(LODIndexUserBinding.IsBound())
			{
				RequestedLOD = LODIndexUserBinding.GetValue();
			}
			else if(Interface->LODIndexUserParameter.Parameter.IsValid())
			{
				if (int32* UserLODIndex = LODIndexUserBinding.Init(Instance->GetInstanceParameters(), Interface->LODIndexUserParameter.Parameter))
				{
					RequestedLOD = *UserLODIndex;
				}				
			}

			int32 MinSampleLOD = RenderData->GetCurrentFirstLODIdx(MinLOD);
			int32 MaxSampleLOD = StaticMesh->GetNumLODs() - 1;

			OutLOD = RequestedLOD >= 0 ? RequestedLOD : (StaticMesh->GetNumLODs() + RequestedLOD);
			OutLOD = FMath::Clamp(OutLOD, MinSampleLOD, MaxSampleLOD);

			return StaticMesh->GetRenderData()->LODResources.IsValidIndex(OutLOD);
		}

#if WITH_EDITOR
		FDelegateHandle OnMeshChanged;
		FDelegateHandle OnPostMeshBuild;
		FDelegateHandle OnMeshReimported;

		void OnEditorMeshChanged(UObject* OptionalMesh)
		{
			UStaticMesh* StaticMesh = StaticMeshWeakPtr.Get();
			if (!OptionalMesh || (OptionalMesh == StaticMesh))
			{
				StaticMeshWeakPtr.Reset();
				if (StaticMesh)
				{
					StaticMesh->GetOnMeshChanged().Remove(OnMeshChanged);
					StaticMesh->OnPostMeshBuild().Remove(OnPostMeshBuild);
				}
				OnMeshChanged.Reset();
				OnPostMeshBuild.Reset();
				if (UImportSubsystem* ImportSubsystem = GEditor->GetEditorSubsystem<UImportSubsystem>())
				{
					ImportSubsystem->OnAssetReimport.Remove(OnMeshReimported);
				}
				OnMeshReimported.Reset();
			}
		}
#endif

		void UpdateTransforms(UObject* Component, FNiagaraSystemInstance* SystemInstance)
		{
			SystemInstanceWorldTransform = SystemInstance->GetWorldTransform();
			LWCTileOffset = FVector(SystemInstance->GetLWCTile()) * -FLargeWorldRenderScalar::GetTileSize();

			FTransform ComponentTransform;
			if (SceneComponentUtils->GetStaticMeshTransforms(Component, ComponentTransform, ISMTransforms))
			{
				if (ISMTransforms.IsValidIndex(InstanceIndex))
				{
					ComponentTransform = ISMTransforms[InstanceIndex];
				}
				OwnerToMeshVector = FVector3f(ComponentTransform.GetLocation() - SystemInstanceWorldTransform.GetLocation());
				ComponentTransform.AddToTranslation(LWCTileOffset);
			}
			else
			{
				ISMTransforms.Empty();
				ComponentTransform = SystemInstanceWorldTransform;
				OwnerToMeshVector = FVector3f::ZeroVector;
			}

			Transform = ComponentTransform.ToMatrixWithScale();
			TransformInverseTransposed = ComponentTransform.Inverse().ToMatrixWithScale().GetTransposed();
			Rotation = FQuat4f(ComponentTransform.GetRotation());
		}

		void UpdateTransformsResetPrevious(UObject* Component, FNiagaraSystemInstance* SystemInstance)
		{
			UpdateTransforms(Component, SystemInstance);
			PrevTransform = Transform;
			PrevTransformInverseTransposed = TransformInverseTransposed;
			PrevRotation = Rotation;
		}

		void UpdateTransformsCopyPrevious(UObject* Component, FNiagaraSystemInstance* SystemInstance)
		{
			PrevTransform = Transform;
			PrevTransformInverseTransposed = TransformInverseTransposed;
			PrevRotation = Rotation;
			UpdateTransforms(Component, SystemInstance);
		}

		void UpdateTransformsInstanceIndexChanged()
		{
			FTransform ComponentTransform = ISMTransforms.IsValidIndex(InstanceIndex) ? ISMTransforms[InstanceIndex] : SystemInstanceWorldTransform;

			OwnerToMeshVector = FVector3f(ComponentTransform.GetLocation() - SystemInstanceWorldTransform.GetLocation());
			ComponentTransform.AddToTranslation(LWCTileOffset);

			Transform = ComponentTransform.ToMatrixWithScale();
			TransformInverseTransposed = ComponentTransform.Inverse().ToMatrixWithScale().GetTransposed();
			Rotation = FQuat4f(ComponentTransform.GetRotation());

			PrevTransform = Transform;
			PrevTransformInverseTransposed = TransformInverseTransposed;
			PrevRotation = Rotation;
		}

		bool Init(UNiagaraDataInterfaceStaticMesh* Interface, FNiagaraSystemInstance* SystemInstance)
		{
			check(SystemInstance);

			// Initialize members
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			SceneComponentUtils = &FNiagaraSystemInstance::GetSceneComponentUtils(SystemInstance);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			ComponentWeakPtr = nullptr;
			StaticMeshWeakPtr = nullptr;
			Transform = FMatrix::Identity;
			TransformInverseTransposed = FMatrix::Identity;
			PrevTransform = FMatrix::Identity;
			PrevTransformInverseTransposed = FMatrix::Identity;
			Rotation = FQuat4f::Identity;
			PrevRotation = FQuat4f::Identity;
			DeltaSeconds = 0.0f;
			PreSkinnedLocalBoundsCenter = FVector3f::ZeroVector;
			PreSkinnedLocalBoundsExtents = FVector3f::ZeroVector;
			OwnerToMeshVector = FVector3f::ZeroVector;
			PhysicsVelocity = FVector::ZeroVector;
			bCaptureTransformsPerFrame = Interface->bCaptureTransformsPerFrame;
			bUsePhysicsVelocity = Interface->bUsePhysicsBodyVelocity;
			bComponentValid = false;
			bMeshValid = false;
			bMeshAllowsCpuAccess = false;
			bIsCpuUniformlyDistributedSampling = false;
			bIsGpuUniformlyDistributedSampling = false;
			NumFilteredSections = 0;
			NumUnfilteredSections = 0;
			FilteredAndUnfilteredSections.Empty();
			FilteredAndUnfilteredSectionInfos.Empty();
			NumFilteredTriangles = 0;
			NumUnfilteredTriangles = 0;
			ChangeId = Interface->ChangeId;
			MinLOD = 0;
			CachedLODIdx = 0;
			CachedSockets.Empty();
			NumFilteredSockets = 0;
			FilteredAndUnfilteredSockets.Empty();

			MeshParameterBinding.Init(SystemInstance->GetInstanceParameters(), Interface->MeshParameterBinding.Parameter);

			// Get component / mesh we are using
			UObject* Component = nullptr;
			UStaticMesh* StaticMesh = Interface->ResolveStaticMesh(Component, SystemInstance, MeshParameterBinding.GetValue());
			ComponentWeakPtr = Component;

			// Gather attached information
			InstanceIndex = Interface->InstanceIndex;
			UpdateTransformsResetPrevious(Component, SystemInstance);

			if (bUsePhysicsVelocity)
			{
				PhysicsVelocity = SceneComponentUtils->GetPhysicsLinearVelocity(Component);
			}

			if (GNiagaraFailStaticMeshDataInterface != 0)
			{
				UE_LOGF(LogNiagara, Log, "StaticMesh data interface aborting because \"fx.Niagara.FailStaticMeshDataInterface 1\". Failed InitPerInstanceData - %ls", *Interface->GetFullName());
				return false;
			}

			const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[SystemInstance->GetFeatureLevel()];

			// Report missing or inaccessible meshes to the log
			if (StaticMesh == nullptr)
			{
				UE_LOGF(LogNiagara, Log, "StaticMesh data interface has no valid mesh - %ls", *Interface->GetFullName());
			}
			else if (StaticMesh->HasValidNaniteData() && !StaticMesh->HasNaniteFallbackMesh(ShaderPlatform))
			{
				UE_LOGF(LogNiagara, Warning, "NiagaraStaticMeshDataInterface does not have a fallback mesh. Enable Nanite fallback mesh to fix this issue. System: %ls, Mesh: %ls", *GetFullNameSafe(SystemInstance->GetSystem()), *GetFullNameSafe(StaticMesh));
				StaticMesh = nullptr;
			}
			else if (!StaticMesh->bAllowCPUAccess)
			{
				//-TODO: This will be modified with the refactor to allow GPU to be used when CPU is also used
				if (Interface->IsUsedWithGPUScript() && !Interface->IsUsedWithCPUScript())
				{
					if (!FNiagaraUtilities::AreBufferSRVsAlwaysCreated(GMaxRHIShaderPlatform))
					{
						UE_LOGF(LogNiagara, Log, "NiagaraStaticMeshDataInterface used by GPU emitter but does not have SRV access on this platform.  Enable CPU access to fix this issue. System: %ls, Mesh: %ls", *GetFullNameSafe(SystemInstance->GetSystem()), *GetFullNameSafe(StaticMesh));
						StaticMesh = nullptr;
					}
				}
				else
				{
					bool bNeedsCpuAccess = false;
					FNiagaraDataInterfaceUtilities::ForEachVMFunction(
						Interface,
						SystemInstance,
						[&bNeedsCpuAccess](const UNiagaraScript* Script, const FVMExternalFunctionBindingInfo& VMFunction) -> bool
						{
							bNeedsCpuAccess |= UNiagaraDataInterfaceStaticMesh::FunctionNeedsCpuAccess(VMFunction.Name);
							return bNeedsCpuAccess == false;
						}
					);

					if (bNeedsCpuAccess)
					{
						if (Interface->IsUsedWithCPUScript())
						{
							UE_LOGF(LogNiagara, Log, "NiagaraStaticMeshDataInterface used by CPU emitter and does not allow CPU access. System: %ls, Mesh: %ls", *GetFullNameSafe(SystemInstance->GetSystem()), *GetFullNameSafe(StaticMesh));
						}
						StaticMesh = nullptr;
					}
				}
			}

			TRefCountPtr<const FStaticMeshLODResources> LODData;
			if (StaticMesh != nullptr)
			{
				FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();
				if (RenderData)
				{
					if (CalculateLODIndex(Interface, SystemInstance, StaticMesh, CachedLODIdx, MinLOD))
					{
						LODData = &RenderData->LODResources[CachedLODIdx];
					}
				}

				if (!LODData.IsValid())
				{
					StaticMesh = nullptr;
				}
			}

			StaticMeshWeakPtr = StaticMesh;
			bMeshValid = StaticMesh != nullptr;

			if (StaticMesh != nullptr)
			{
#if WITH_EDITOR
				if ( GIsEditor )
				{
					OnMeshChanged = StaticMesh->GetOnMeshChanged().AddLambda([this]() { OnEditorMeshChanged(nullptr); });
					OnPostMeshBuild = StaticMesh->OnPostMeshBuild().AddLambda([this](UStaticMesh* StaticMesh) { OnEditorMeshChanged(StaticMesh); });
					if (UImportSubsystem* ImportSubsystem = GEditor->GetEditorSubsystem<UImportSubsystem>())
					{
						OnMeshReimported = ImportSubsystem->OnAssetReimport.AddLambda([this](UObject* Object) { OnEditorMeshChanged(Object); });
					}
				}
#endif

				bMeshAllowsCpuAccess = StaticMesh->bAllowCPUAccess;
				bIsCpuUniformlyDistributedSampling = StaticMesh->bSupportUniformlyDistributedSampling;
				if (bIsCpuUniformlyDistributedSampling)
				{
					bIsCpuUniformlyDistributedSampling &= LODData->AreaWeightedSampler.GetNumEntries() == LODData->Sections.Num();
					bIsCpuUniformlyDistributedSampling &= LODData->AreaWeightedSectionSamplers.Num() == LODData->Sections.Num();
				}
				bIsGpuUniformlyDistributedSampling = bIsCpuUniformlyDistributedSampling && StaticMesh->bSupportGpuUniformlyDistributedSampling;

				// Init section filter
				//-OPT: I don't foresee a lot of entries here, could potentially optimize this later to avoid insert
				FilteredAndUnfilteredSections.Reserve(LODData->Sections.Num());
				FilteredAndUnfilteredSectionInfos.Reserve(LODData->Sections.Num());
				for (int32 i = 0; i < LODData->Sections.Num(); ++i)
				{
					// Skip garbage data
					if ( LODData->Sections[i].NumTriangles == 0 )
					{
						continue;
					}

					if (Interface->SectionFilter.AllowedMaterialSlots.Num() == 0 || Interface->SectionFilter.AllowedMaterialSlots.Contains(LODData->Sections[i].MaterialIndex))
					{
						FilteredAndUnfilteredSections.Insert(i, NumFilteredSections);
						++NumFilteredSections;
						NumFilteredTriangles += LODData->Sections[i].NumTriangles;
					}
					else
					{
						FilteredAndUnfilteredSections.Add(i);
						++NumUnfilteredSections;
						NumUnfilteredTriangles += LODData->Sections[i].NumTriangles;
					}
				}

				// Initialize section information
				{
					const FStaticMeshSectionAreaWeightedTriangleSamplerArray* SectionSamplerArray = bIsCpuUniformlyDistributedSampling ? &LODData->AreaWeightedSectionSamplers : nullptr;
					FNDISectionAreaWeightedSampler::Init(LODData, GetFilteredSections(), SectionSamplerArray, FilteredAndUnfilteredSectionInfos);
					FNDISectionAreaWeightedSampler::Init(LODData, GetUnfilteredSections(), SectionSamplerArray, FilteredAndUnfilteredSectionInfos);
				}

				// Init socket information
				const int32 NumMeshSockets = StaticMesh->Sockets.Num();
				if (NumMeshSockets > 0)
				{
					if (NumMeshSockets > TNumericLimits<uint16>::Max())
					{
						UE_LOGF(LogNiagara, Warning, "Static Mesh Data Interface '%ls' requires more sockets '%d' than we currently support '%d' Mesh '%ls' Component '%ls'", *GetFullNameSafe(Interface), NumMeshSockets, TNumericLimits<uint16>::Max(), *GetFullNameSafe(StaticMesh), *GetFullNameSafe(Component));
						return false;
					}

					CachedSockets.AddDefaulted(NumMeshSockets);
					for (int32 i = 0; i < NumMeshSockets; ++i)
					{
						CachedSockets[i].SetTranslation(FVector3f(StaticMesh->Sockets[i]->RelativeLocation));
						CachedSockets[i].SetRotation(FQuat4f(FRotator3f(StaticMesh->Sockets[i]->RelativeRotation)));
						CachedSockets[i].SetScale3D(FVector3f(StaticMesh->Sockets[i]->RelativeScale));
					}

					NumFilteredSockets = 0;
					if (Interface->FilteredSockets.Num() > 0)
					{
						FilteredAndUnfilteredSockets.Reserve(NumMeshSockets);
						for (int32 i = 0; i < Interface->FilteredSockets.Num(); ++i)
						{
							const FName FilteredSocketName = Interface->FilteredSockets[i];
							const int32 SocketIndex = StaticMesh->Sockets.IndexOfByPredicate([&](const UStaticMeshSocket* Socket) { return Socket->SocketName == FilteredSocketName; });
							if (SocketIndex == INDEX_NONE)
							{
								UE_LOGF(LogNiagara, Warning, "Static Mesh Data Interface '%ls' could not find socket '%ls' Mesh '%ls' Component '%ls'", *GetFullNameSafe(Interface), *FilteredSocketName.ToString(), *GetFullNameSafe(StaticMesh), *GetFullNameSafe(Component));
								continue;
							}
							++NumFilteredSockets;
							FilteredAndUnfilteredSockets.Add(uint16(SocketIndex));
						}

						for (int32 i = 0; i < NumMeshSockets; ++i)
						{
							FilteredAndUnfilteredSockets.AddUnique(uint16(i));
						}
					}
				}

				// Preskinned local bounds
				{
					const FBoxSphereBounds LocalBounds = StaticMesh->GetBounds();
					PreSkinnedLocalBoundsCenter = FVector3f(LocalBounds.Origin);
					PreSkinnedLocalBoundsExtents = FVector3f(LocalBounds.BoxExtent);
				}
			}

			// support for UV mapping
			{
				bool UsedByCpuUvMapping = false;
				bool UsedByGpuUvMapping = false;

				SystemInstance->EvaluateBoundFunction(GetTriangleCoordAtUVName, UsedByCpuUvMapping, UsedByGpuUvMapping);
				SystemInstance->EvaluateBoundFunction(GetTriangleCoordInAabbName, UsedByCpuUvMapping, UsedByGpuUvMapping);

				const bool MeshValid = IsValid(StaticMesh);
				const bool SupportUvMappingCpu = UsedByCpuUvMapping && MeshValid;
				const bool SupportUvMappingGpu = UsedByGpuUvMapping && MeshValid && Interface->IsUsedWithGPUScript();

				UvMappingUsage = FMeshUvMappingUsage(SupportUvMappingCpu, SupportUvMappingGpu);

				if (UvMappingIndexSet != INDEX_NONE)
				{
					UpdateUvMapping(SystemInstance);
				}
			}

			return true;
		}

		void UpdateUvMapping(FNiagaraSystemInstance* SystemInstance)
		{
			if (UvMappingUsage.IsValid())
			{
				const bool bNeedsDataImmediately = true;

				FNDI_StaticMesh_GeneratedData& GeneratedData = SystemInstance->GetWorldManager()->EditGeneratedData<FNDI_StaticMesh_GeneratedData>();
				UvMapping = GeneratedData.GetCachedUvMapping(StaticMeshWeakPtr, CachedLODIdx, UvMappingIndexSet, UvMappingUsage, bNeedsDataImmediately);
			}
			else
			{
				UvMapping = FStaticMeshUvMappingHandle(UvMappingUsage, nullptr, false);
			}
		}

		bool Tick(UNiagaraDataInterfaceStaticMesh* Interface, FNiagaraSystemInstance* SystemInstance, float InDeltaSeconds)
		{
			//TODO: Recache LOD info.
			//TODO: Currently we just grab a LOD on init and then fail/reset if that's changed or becomes invalid.
			//TODO: If we recache here, the DI can just continue with new LODs when desired and more gracefully handle streamed out LODs.

			if (ResetRequired(Interface, SystemInstance))
			{
				return true;
			}

			DeltaSeconds = InDeltaSeconds;

			UObject* Component = ComponentWeakPtr.Get();
			if (bCaptureTransformsPerFrame)
			{
				UpdateTransformsCopyPrevious(Component, SystemInstance);
			}

			if (bUsePhysicsVelocity)
			{
				PhysicsVelocity = SceneComponentUtils->GetPhysicsLinearVelocity(Component);
			}

			if (UvMappingIndexSet != INDEX_NONE && !UvMapping)
			{
				UpdateUvMapping(SystemInstance);
			}

			return false;
		}

		void Release()
		{
#if WITH_EDITOR
			if (GIsEditor)
			{
				// We reuse the function to clear the delegates out
				OnEditorMeshChanged(nullptr);
			}
#endif
		}

		bool ResetRequired(UNiagaraDataInterfaceStaticMesh* Interface, FNiagaraSystemInstance* Instance)
		{
			UObject* Component = ComponentWeakPtr.Get();
			if (bComponentValid && !Component)
			{
				// The component we were bound to is no longer valid so we have to trigger a reset.
				return true;
			}

			// Check to see if either the mesh has become invalid
			UStaticMesh* Mesh = StaticMeshWeakPtr.Get();
			if (bMeshValid && !Mesh)
			{
				return true;
			}

			// Has an external call invalidated the SM DI
			if (Interface != nullptr && ChangeId != Interface->ChangeId)
			{
				return true;
			}

			// Check to see if we would resolve to a different mesh / component
			UObject* ResolvedComponent = nullptr;
			UStaticMesh* ResolvedMesh = nullptr;
			if (Interface != nullptr)
			{
				ResolvedMesh = Interface->ResolveStaticMesh(ResolvedComponent, Instance, MeshParameterBinding.GetValue());
			}

			if (Component != ResolvedComponent || Mesh != ResolvedMesh)
			{
				return true;
			}

			// Has the LOD changed
			if (Mesh != nullptr)
			{
				//TODO: If the LOD index selection changes then we reset the system
				//TODO: We should just recache all the relevant data and continue on our way.
				int32 NewLODIndex = INDEX_NONE;
				int32 NewMinLOD = INDEX_NONE;
				bool bSuccess = CalculateLODIndex(Interface, Instance, Mesh, NewLODIndex, NewMinLOD);
				if (!bSuccess || NewLODIndex != CachedLODIdx)
				{
					return true;
				}
			}

			return false;
		}

		const FStaticMeshLODResources* GetCurrentLOD()
		{
			//-OPT: Perhaps we could cache this during the tick function?
			UStaticMesh* StaticMesh = StaticMeshWeakPtr.Get();
			if ( bMeshValid && StaticMesh )
			{
				int32 CurrentFirstLOD = StaticMesh->GetRenderData()->GetCurrentFirstLODIdx(MinLOD);
				if(CachedLODIdx >= CurrentFirstLOD && StaticMesh->GetRenderData()->LODResources.IsValidIndex(CachedLODIdx))
				{
					if (const FStaticMeshLODResources* LODResource = &StaticMesh->GetRenderData()->LODResources[CachedLODIdx])
					{
						if(LODResource->GetNumVertices() > 0 && LODResource->BuffersSize > 0)
						{
							return LODResource;
						}
					}
				}
			}
			return nullptr;
		}

		const FStaticMeshLODResources* GetCurrentLODWithVertexColorOverrides(FColorVertexBuffer*& OutVertexColorOverrides)
		{
			//-OPT: Perhaps we could cache this during the tick function?
			UStaticMesh* StaticMesh = StaticMeshWeakPtr.Get();
			OutVertexColorOverrides = nullptr;
			if (bMeshValid && StaticMesh)
			{
				int32 CurrentFirstLOD = StaticMesh->GetRenderData()->GetCurrentFirstLODIdx(MinLOD);

				if (CachedLODIdx >= CurrentFirstLOD && StaticMesh->GetRenderData()->LODResources.IsValidIndex(CachedLODIdx))
				{
					OutVertexColorOverrides = SceneComponentUtils->GetStaticMeshOverrideColors(ComponentWeakPtr.Get(), CachedLODIdx);
				
					if (const FStaticMeshLODResources* LODResource = &StaticMesh->GetRenderData()->LODResources[CachedLODIdx])
					{
						if (LODResource->GetNumVertices() > 0 && LODResource->BuffersSize > 0)
						{
							return LODResource;
						}
					}
				}

			}
			return nullptr;
		}

		TConstArrayView<int32> GetFilteredSections() const { return MakeArrayView(FilteredAndUnfilteredSections.GetData(), NumFilteredSections); }
		TConstArrayView<int32> GetUnfilteredSections() const { return MakeArrayView(FilteredAndUnfilteredSections.GetData() + NumFilteredSections, NumUnfilteredSections); }

		TConstArrayView<FNDISectionInfo> GetFilteredSectionInfos() const { return MakeArrayView(FilteredAndUnfilteredSectionInfos.GetData(), NumFilteredSections); }
		TConstArrayView<FNDISectionInfo> GetUnfilteredSectionInfos() const { return MakeArrayView(FilteredAndUnfilteredSectionInfos.GetData() + NumFilteredSections, NumUnfilteredSections); }
	};

	struct FRenderProxy : public FNiagaraDataInterfaceProxy
	{
		FRenderProxy() {}

		virtual void ConsumePerInstanceDataFromGameThread(void* FromGameThreadData, const FNiagaraSystemInstanceID& InstanceID) override
		{
			FInstanceData_FromGameThread* FromGameThread = reinterpret_cast<FInstanceData_FromGameThread*>(FromGameThreadData);
			FInstanceData_RenderThread* InstanceData = &PerInstanceData_RT.FindChecked(InstanceID);

			InstanceData->Transform					= FromGameThread->Transform;
			InstanceData->PrevTransform				= FromGameThread->PrevTransform;
			InstanceData->Rotation					= FromGameThread->Rotation;
			InstanceData->PrevRotation				= FromGameThread->PrevRotation;
			InstanceData->DeltaSeconds				= FromGameThread->DeltaSeconds;
			InstanceData->PreSkinnedLocalBoundsCenter = FromGameThread->PreSkinnedLocalBoundsCenter;
			InstanceData->PreSkinnedLocalBoundsExtents = FromGameThread->PreSkinnedLocalBoundsExtents;
			InstanceData->OwnerToMeshVector			= FromGameThread->OwnerToMeshVector;
			InstanceData->DistanceFieldPrimitiveId	= FromGameThread->DistanceFieldPrimitiveId;

			InstanceData->UvMappingBuffer			= FromGameThread->UvMappingBuffer;
			InstanceData->UvMappingSet				= FromGameThread->UvMappingSet;

#if WITH_EDITOR
			if (FromGameThread->OverrideVertexColors)
			{
				InstanceData->MeshOverrideColorBufferSRV = FromGameThread->OverrideVertexColors->GetColorComponentsSRV();
			}
			else
			{
				InstanceData->MeshOverrideColorBufferSRV.SafeRelease();
			}
#endif

			FromGameThread->~FInstanceData_FromGameThread();
		}

		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
		{
			return sizeof(FInstanceData_FromGameThread);
		}

		TMap<FNiagaraSystemInstanceID, FInstanceData_RenderThread> PerInstanceData_RT;
	};

	//////////////////////////////////////////////////////////////////////////

	template<typename TTransformHandler = FNDITransformHandlerNoop>
	struct FStaticMeshCpuHelper
	{
		FORCEINLINE FStaticMeshCpuHelper(FVectorVMExternalFunctionContext& Context)
			: InstanceData(Context)
		{
			LODResource = InstanceData->GetCurrentLODWithVertexColorOverrides(OverrideVertexColors);
		}


		FORCEINLINE float GetInvDeltaSeconds() const
		{
			return InstanceData->DeltaSeconds > 0.0f ? 1.0f / InstanceData->DeltaSeconds : 0.0f;
		}

		FORCEINLINE FVector3f TransformPosition(FVector3f Position) const
		{
			TransformHandler.TransformPosition(Position, FMatrix44f(InstanceData->Transform));					// LWC_TODO: Precision loss
			return Position;
		}

		FORCEINLINE FVector3f TransformUnitVector(FVector3f Vector) const
		{
			TransformHandler.TransformUnitVector(Vector, FMatrix44f(InstanceData->TransformInverseTransposed));		// LWC_TODO: Precision loss
			return Vector;
		}

		FORCEINLINE FVector3f TransformNotUnitVector(FVector3f Vector) const
		{
			TransformHandler.TransformNotUnitVector(Vector, FMatrix44f(InstanceData->TransformInverseTransposed));		// LWC_TODO: Precision loss
			return Vector;
		}

		FORCEINLINE FQuat4f TransformRotation(FQuat4f Rotation) const
		{
			TransformHandler.TransformRotation(Rotation, InstanceData->Rotation);
			return Rotation;
		}

		FORCEINLINE FVector3f PreviousTransformPosition(FVector3f Position) const
		{
			TransformHandler.TransformPosition(Position, FMatrix44f(InstanceData->PrevTransform));				// LWC_TODO: Precision loss
			return Position;
		}

		FORCEINLINE FVector3f PreviousTransformUnitVector(FVector3f Vector) const
		{
			TransformHandler.TransformUnitVector(Vector, FMatrix44f(InstanceData->PrevTransformInverseTransposed));	// LWC_TODO: Precision loss
			return Vector;
		}

		FORCEINLINE FVector3f PreviousTransformNotUnitVector(FVector3f Vector) const
		{
			TransformHandler.TransformNotUnitVector(Vector, FMatrix44f(InstanceData->PrevTransformInverseTransposed));	// LWC_TODO: Precision loss
			return Vector;
		}

		FORCEINLINE FQuat4f PreviousTransformRotation(FQuat4f Rotation) const
		{
			TransformHandler.TransformRotation(Rotation, InstanceData->PrevRotation);
			return Rotation;
		}

		FORCEINLINE FIndexArrayView GetIndexArrayView() const
		{
			return LODResource ? LODResource->IndexBuffer.GetArrayView() : FIndexArrayView();
		}

		FORCEINLINE int32 GetNumPositionVertices() const
		{
			return LODResource && LODResource->VertexBuffers.PositionVertexBuffer.GetVertexData() && LODResource->VertexBuffers.PositionVertexBuffer.GetAllowCPUAccess() ? LODResource->VertexBuffers.PositionVertexBuffer.GetNumVertices() : 0;
		}

		FORCEINLINE int32 GetNumTangentVertices() const
		{
			return LODResource && LODResource->VertexBuffers.StaticMeshVertexBuffer.GetTangentData() && LODResource->VertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess() ? LODResource->VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() : 0;
		}

		uint32 GetNumUVs() const
		{
			return LODResource && LODResource->VertexBuffers.StaticMeshVertexBuffer.GetTexCoordData() && LODResource->VertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess() ? LODResource->VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() : 0;
		}

		uint32 GetNumUVVertices() const
		{
			return LODResource && LODResource->VertexBuffers.StaticMeshVertexBuffer.GetTexCoordData() && LODResource->VertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess() ? LODResource->VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() : 0;
		}

		FORCEINLINE int32 GetNumColorVertices() const
		{
			if(OverrideVertexColors)
			{
				return OverrideVertexColors->GetAllowCPUAccess() ? OverrideVertexColors->GetNumVertices() : 0; 
			}
			return LODResource && LODResource->VertexBuffers.ColorVertexBuffer.GetVertexData() && LODResource->VertexBuffers.ColorVertexBuffer.GetAllowCPUAccess() ? LODResource->VertexBuffers.ColorVertexBuffer.GetNumVertices() : 0;
		}

		FORCEINLINE FVector3f GetLocalTrianglePosition(const FVector3f& BaryCoord, int32 Index0, int32 Index1, int32 Index2) const
		{
			FVector3f Position;
			Position = LODResource->VertexBuffers.PositionVertexBuffer.VertexPosition(Index0) * BaryCoord.X;
			Position += LODResource->VertexBuffers.PositionVertexBuffer.VertexPosition(Index1) * BaryCoord.Y;
			Position += LODResource->VertexBuffers.PositionVertexBuffer.VertexPosition(Index2) * BaryCoord.Z;
			return Position;
		}

		FORCEINLINE FVector3f GetTrianglePosition(const FVector3f& BaryCoord, int32 Index0, int32 Index1, int32 Index2) const
		{
			FVector3f Position = GetLocalTrianglePosition(BaryCoord, Index0, Index1, Index2);
			TransformHandler.TransformPosition(Position, FMatrix44f(InstanceData->Transform));					// LWC_TODO: Precision loss?
			return Position;
		}

		FORCEINLINE FVector3f GetTriangleTangentX(const FVector3f& BaryCoord, int32 Index0, int32 Index1, int32 Index2) const
		{
			FVector3f Tangent;
			Tangent  = LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(Index0) * BaryCoord.X;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(Index1) * BaryCoord.Y;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(Index2) * BaryCoord.Z;
			TransformHandler.TransformUnitVector(Tangent, FMatrix44f(InstanceData->Transform));						// LWC_TODO: Precision loss?
			return Tangent;
		}

		FORCEINLINE FVector3f GetTriangleTangentY(const FVector3f& BaryCoord, int32 Index0, int32 Index1, int32 Index2) const
		{
			FVector3f Tangent;
			Tangent  = LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(Index0) * BaryCoord.X;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(Index1) * BaryCoord.Y;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(Index2) * BaryCoord.Z;
			TransformHandler.TransformUnitVector(Tangent, FMatrix44f(InstanceData->Transform));						// LWC_TODO: Precision loss?
			return Tangent;
		}

		FORCEINLINE FVector3f GetTriangleTangentZ(const FVector3f& BaryCoord, int32 Index0, int32 Index1, int32 Index2) const
		{
			FVector3f Tangent;
			Tangent  = LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Index0) * BaryCoord.X;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Index1) * BaryCoord.Y;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Index2) * BaryCoord.Z;
			TransformHandler.TransformUnitVector(Tangent, FMatrix44f(InstanceData->Transform));						// LWC_TODO: Precision loss?
			return Tangent;
		}

		FORCEINLINE FVector3f GetTriangleTangentXInterpolated(const FVector3f& BaryCoord, float Interp, int32 Index0, int32 Index1, int32 Index2) const
		{
			FVector3f Tangent;
			Tangent = LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(Index0) * BaryCoord.X;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(Index1) * BaryCoord.Y;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(Index2) * BaryCoord.Z;
			FVector3f CurrTangent = Tangent;
			FVector3f PrevTangent = Tangent;
			TransformHandler.TransformUnitVector(PrevTangent, FMatrix44f(InstanceData->PrevTransform));						// LWC_TODO: Precision loss?
			TransformHandler.TransformUnitVector(CurrTangent, FMatrix44f(InstanceData->Transform));						// LWC_TODO: Precision loss?
			return FMath::Lerp(PrevTangent, CurrTangent, Interp);
		}

		FORCEINLINE FVector3f GetTriangleTangentYInterpolated(const FVector3f& BaryCoord, float Interp, int32 Index0, int32 Index1, int32 Index2) const
		{
			FVector3f Tangent;
			Tangent = LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(Index0) * BaryCoord.X;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(Index1) * BaryCoord.Y;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(Index2) * BaryCoord.Z;
			FVector3f CurrTangent = Tangent;
			FVector3f PrevTangent = Tangent;
			TransformHandler.TransformUnitVector(PrevTangent, FMatrix44f(InstanceData->PrevTransform));						// LWC_TODO: Precision loss?
			TransformHandler.TransformUnitVector(CurrTangent, FMatrix44f(InstanceData->Transform));						// LWC_TODO: Precision loss?
			return FMath::Lerp(PrevTangent, CurrTangent, Interp);
		}

		FORCEINLINE FVector3f GetTriangleTangentZInterpolated(const FVector3f& BaryCoord, float Interp, int32 Index0, int32 Index1, int32 Index2) const
		{
			FVector3f Tangent;
			Tangent = LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Index0) * BaryCoord.X;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Index1) * BaryCoord.Y;
			Tangent += LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Index2) * BaryCoord.Z;
			FVector3f CurrTangent = Tangent;
			FVector3f PrevTangent = Tangent;
			TransformHandler.TransformUnitVector(PrevTangent, FMatrix44f(InstanceData->PrevTransform));						// LWC_TODO: Precision loss?
			TransformHandler.TransformUnitVector(CurrTangent, FMatrix44f(InstanceData->Transform));						// LWC_TODO: Precision loss?
			return FMath::Lerp(PrevTangent, CurrTangent, Interp);
		}

		FORCEINLINE FLinearColor GetTriangleColor(const FVector3f& BaryCoord, int32 Index0, int32 Index1, int32 Index2) const
		{
			FLinearColor Color;

			if (OverrideVertexColors)
			{
				Color = FLinearColor(OverrideVertexColors->VertexColor(Index0)) * BaryCoord.X;
				Color += FLinearColor(OverrideVertexColors->VertexColor(Index1)) * BaryCoord.Y;
				Color += FLinearColor(OverrideVertexColors->VertexColor(Index2)) * BaryCoord.Z;
			}
			else
			{
				Color  = FLinearColor(LODResource->VertexBuffers.ColorVertexBuffer.VertexColor(Index0)) * BaryCoord.X;
				Color += FLinearColor(LODResource->VertexBuffers.ColorVertexBuffer.VertexColor(Index1)) * BaryCoord.Y;
				Color += FLinearColor(LODResource->VertexBuffers.ColorVertexBuffer.VertexColor(Index2)) * BaryCoord.Z;			
			}
			return Color;
		}

		FORCEINLINE FVector2D GetTriangleUV(const FVector3f& BaryCoord, int32 Index0, int32 Index1, int32 Index2, int32 UVSet) const
		{
			FVector2f UV;
			UV = LODResource->VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index0, UVSet) * BaryCoord.X;
			UV += LODResource->VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index1, UVSet) * BaryCoord.Y;
			UV += LODResource->VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index2, UVSet) * BaryCoord.Z;
			return FVector2D(UV);
		}

		FORCEINLINE FVector3f GetPosition(int32 Vertex) const
		{
			FVector3f Position = LODResource->VertexBuffers.PositionVertexBuffer.VertexPosition(Vertex);
			TransformHandler.TransformPosition(Position, FMatrix44f(InstanceData->Transform));					// LWC_TODO: Precision loss?
			return Position;
		}

		FORCEINLINE FVector3f GetLocalPosition(int32 Vertex) const
		{
			return LODResource->VertexBuffers.PositionVertexBuffer.VertexPosition(Vertex);
		}

		FORCEINLINE FNDITangentBasis GetTangentBasis(int32 Vertex) const
		{
			FNDITangentBasis TangentBasis;
			TangentBasis.TangentX = LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(Vertex);
			TangentBasis.TangentY = LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(Vertex);
			TangentBasis.TangentZ = LODResource->VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Vertex);
			TangentBasis.TransformAndEnsureValid(TransformHandler, FMatrix44f(InstanceData->TransformInverseTransposed));		// LWC_TODO: Precision loss?
			return TangentBasis;
		}

		FORCEINLINE FNDITangentBasis GetTangentBasisInterpolated(int32 Vertex, float Interp) const
		{
			FNDITangentBasis TangentBasisCurr = GetTangentBasis(Vertex);
			FNDITangentBasis TangentBasisPrev = TangentBasisCurr;
			TangentBasisCurr.TransformAndEnsureValid(TransformHandler, FMatrix44f(InstanceData->TransformInverseTransposed));		// LWC_TODO: Precision loss?
			TangentBasisPrev.TransformAndEnsureValid(TransformHandler, FMatrix44f(InstanceData->PrevTransformInverseTransposed));	// LWC_TODO: Precision loss?

			FNDITangentBasis InterpolatedBasis;
			InterpolatedBasis.TangentX = FMath::Lerp(TangentBasisPrev.TangentX, TangentBasisCurr.TangentX, Interp);
			InterpolatedBasis.TangentY = FMath::Lerp(TangentBasisPrev.TangentY, TangentBasisCurr.TangentY, Interp);
			InterpolatedBasis.TangentZ = FMath::Lerp(TangentBasisPrev.TangentZ, TangentBasisCurr.TangentZ, Interp);
			return InterpolatedBasis;
		}

		FORCEINLINE FLinearColor GetColor(int32 Vertex) const
		{
			if (OverrideVertexColors)
			{
				return OverrideVertexColors->VertexColor(Vertex);
			}
			return LODResource->VertexBuffers.ColorVertexBuffer.VertexColor(Vertex);
		}

		FORCEINLINE FVector2D GetUV(int32 Vertex, int32 UVSet) const
		{
			return FVector2D(LODResource->VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Vertex, UVSet));
		}

		FORCEINLINE bool IsCpuUniformlyDistributedSampling() const
		{
			return LODResource && InstanceData->bIsCpuUniformlyDistributedSampling;
		}

		template<typename TRandomHelper>
		FORCEINLINE int32 RandomUniformTriangle(const TRandomHelper& RandHelper, int32 InstanceIndex) const
		{
			int32 Triangle = 0;
			const int32 Section = LODResource->AreaWeightedSampler.GetNumEntries() ? LODResource->AreaWeightedSampler.GetEntryIndex(RandHelper.Rand(InstanceIndex), RandHelper.Rand(InstanceIndex)) : 0;
			if ( LODResource->AreaWeightedSectionSamplers.IsValidIndex(Section) && LODResource->AreaWeightedSectionSamplers[Section].GetNumEntries() )
			{
				const int32 SectionTriangle = LODResource->AreaWeightedSectionSamplers[Section].GetEntryIndex(RandHelper.Rand(InstanceIndex), RandHelper.Rand(InstanceIndex));
				Triangle = (LODResource->Sections[Section].FirstIndex / 3) + SectionTriangle;
			}
			return Triangle;
		}

		template<typename TRandomHelper>
		FORCEINLINE int32 RandomUniformTriangle(const TRandomHelper& RandHelper, int32 InstanceIndex, const TConstArrayView<FNDISectionInfo>& SectionInfos, TConstArrayView<int32>& SectionRemap) const
		{
			int32 Triangle = 0;
			int32 SectionIndex = int32(RandHelper.Rand(InstanceIndex) * float(SectionInfos.Num()));
			SectionIndex = RandHelper.Rand(InstanceIndex) < SectionInfos[SectionIndex].Prob ? SectionIndex : SectionInfos[SectionIndex].Alias;
			const int32 Section = SectionRemap[SectionIndex];
			if ( LODResource->AreaWeightedSectionSamplers.IsValidIndex(Section) && LODResource->AreaWeightedSectionSamplers[Section].GetNumEntries() )
			{
				const int32 SectionTriangle = LODResource->AreaWeightedSectionSamplers[Section].GetEntryIndex(RandHelper.Rand(InstanceIndex), RandHelper.Rand(InstanceIndex));
				Triangle = SectionInfos[SectionIndex].FirstTriangle + SectionTriangle;
			}
			return Triangle;
		}

		template<typename TRandomHelper>
		FORCEINLINE int32 RandomTriangle(const TRandomHelper& RandHelper, int32 InstanceIndex, const TConstArrayView<FNDISectionInfo>& SectionInfos) const
		{
			const int32 Section = int32(RandHelper.Rand(InstanceIndex) * float(SectionInfos.Num()));
			const int32 Triangle = SectionInfos[Section].FirstTriangle + int32(RandHelper.Rand(InstanceIndex) * float(SectionInfos[Section].NumTriangles));
			return Triangle;
		}

		template<typename TRandomHelper>
		FORCEINLINE int32 RandomUniformSection(const TRandomHelper& RandHelper, int32 InstanceIndex) const
		{
			return LODResource->AreaWeightedSampler.GetEntryIndex(RandHelper.Rand(InstanceIndex), RandHelper.Rand(InstanceIndex));
		}

		template<typename TRandomHelper>
		FORCEINLINE int32 RandomUniformSection(const TRandomHelper& RandHelper, int32 InstanceIndex, const TConstArrayView<FNDISectionInfo>& SectionInfos, TConstArrayView<int32>& SectionRemap) const
		{
			int32 SectionIndex = int32(RandHelper.Rand(InstanceIndex) * float(SectionInfos.Num()));
			SectionIndex = RandHelper.Rand(InstanceIndex) < SectionInfos[SectionIndex].Prob ? SectionIndex : SectionInfos[SectionIndex].Alias;
			return SectionRemap[SectionIndex];
		}

		template<typename TRandomHelper>
		FORCEINLINE int32 RandomSectionUniformTriangle(const TRandomHelper& RandHelper, int32 InstanceIndex, int32 Section) const
		{
			int32 Triangle = 0;
			if (LODResource->AreaWeightedSectionSamplers.IsValidIndex(Section) && LODResource->AreaWeightedSectionSamplers[Section].GetNumEntries())
			{
				const int32 SectionTriangle = LODResource->AreaWeightedSectionSamplers[Section].GetEntryIndex(RandHelper.Rand(InstanceIndex), RandHelper.Rand(InstanceIndex));
				Triangle = (LODResource->Sections[Section].FirstIndex / 3) + SectionTriangle;
			}
			return Triangle;
		}

		template<typename TRandomHelper>
		FORCEINLINE int32 RandomSectionTriangle(const TRandomHelper& RandHelper, int32 InstanceIndex, int32 Section) const
		{
			const int32 Triangle = (LODResource->Sections[Section].FirstIndex / 3) + int32(RandHelper.Rand(InstanceIndex) * float(LODResource->Sections[Section].NumTriangles));
			return Triangle;
		}

		FORCEINLINE int32 GetNumTriangles() const
		{
			return LODResource ? LODResource->IndexBuffer.GetArrayView().Num() / 3 : 0;
		}

		FORCEINLINE int32 GetNumFilteredTriangles() const
		{
			return LODResource ? InstanceData->NumFilteredTriangles : 0;
		}

		FORCEINLINE int32 GetNumUnfilteredTriangles() const
		{
			return LODResource ? InstanceData->NumUnfilteredTriangles : 0;
		}

		FORCEINLINE int32 SectionTriangleToTriangle(const TConstArrayView<FNDISectionInfo>& SectionInfos, int32 Index)
		{
			int32 SectionFirstTriangle = 0;
			for ( const FNDISectionInfo& SectionInfo : SectionInfos )
			{
				const int32 SectionLastTriangle = SectionFirstTriangle + SectionInfo.NumTriangles;
				if ( (Index >= SectionFirstTriangle) && (Index < SectionLastTriangle) )
				{
					return (Index - SectionFirstTriangle) + SectionInfo.FirstTriangle;
				}
				SectionFirstTriangle = SectionLastTriangle;
			}
			return 0;
		}

		FORCEINLINE void InterpolateSocket(int32 SocketIndex, float Interp, FVector3f& OutPosition, FQuat4f& OutRotation, FVector3f& OutScale, FVector3f& OutVelocity)
		{
			const FTransform3f& SocketTransform = InstanceData->CachedSockets[SocketIndex];
			const FVector3f PrevPosition = PreviousTransformPosition(SocketTransform.GetLocation());
			const FVector3f CurrPosition = TransformPosition(SocketTransform.GetLocation());
			const FQuat4f PrevRotation = PreviousTransformRotation(SocketTransform.GetRotation());
			const FQuat4f CurrRotation = TransformRotation(SocketTransform.GetRotation());
			const FVector3f PrevScale = PreviousTransformNotUnitVector(SocketTransform.GetScale3D());
			const FVector3f CurrScale = TransformNotUnitVector(SocketTransform.GetScale3D());

			OutPosition = FMath::Lerp(PrevPosition, CurrPosition, Interp);
			OutRotation = FQuat4f::Slerp(PrevRotation, CurrRotation, Interp);
			OutScale = FMath::Lerp(PrevScale, CurrScale, Interp);
			OutVelocity = (CurrPosition - PrevPosition) * GetInvDeltaSeconds();
		}

		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData;
		TRefCountPtr<const FStaticMeshLODResources> LODResource;
		FColorVertexBuffer* OverrideVertexColors;
		TTransformHandler TransformHandler;
	};
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceStaticMesh::UNiagaraDataInterfaceStaticMesh(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new NDIStaticMeshLocal::FRenderProxy());

	LODIndexUserParameter.Parameter.SetType(FNiagaraTypeDefinition::GetIntDef());
	MeshParameterBinding.Parameter.SetType(FNiagaraTypeDefinition(UObject::StaticClass()));
}

void UNiagaraDataInterfaceStaticMesh::PostInitProperties()
{
	Super::PostInitProperties();

	// Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags DIFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), DIFlags);

		// Still some issues with using custom structs. Convert node for example throws a wobbler. TODO after GDC.
		ENiagaraTypeRegistryFlags CoordFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FMeshTriCoordinate::StaticStruct(), CoordFlags);
	}
}

void UNiagaraDataInterfaceStaticMesh::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if ( Source_DEPRECATED != nullptr )
	{
		SoftSourceActor = Source_DEPRECATED;
	}
#endif
}

#if WITH_EDITOR
void UNiagaraDataInterfaceStaticMesh::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	++ChangeId;

	if (PropertyChangedEvent.Property &&
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceStaticMesh, SourceMode))
	{
		if (SourceMode != ENDIStaticMesh_SourceMode::Default && SourceMode != ENDIStaticMesh_SourceMode::Source)
		{
			// Ensure we don't have any reference to a source actor that we'll never use
			SoftSourceActor = nullptr;
			SourceComponent = nullptr;
		}
	
		if (SourceMode != ENDIStaticMesh_SourceMode::Default && SourceMode != ENDIStaticMesh_SourceMode::DefaultMeshOnly)
		{
			// Ensure we don't cook in a default mesh we'll never use
			DefaultMesh = nullptr;
		}
	}

	//Temp hack to prevent the reset to default breaking this user binding.
	LODIndexUserParameter.Parameter.SetType(FNiagaraTypeDefinition::GetIntDef());
}

bool UNiagaraDataInterfaceStaticMesh::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceStaticMesh, SoftSourceActor) &&
		SourceMode != ENDIStaticMesh_SourceMode::Default &&
		SourceMode != ENDIStaticMesh_SourceMode::Source)
	{
		// Disable Source if we'll never use it
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceStaticMesh, DefaultMesh) &&
		SourceMode != ENDIStaticMesh_SourceMode::Default &&
		SourceMode != ENDIStaticMesh_SourceMode::DefaultMeshOnly)
	{
		// Disable Default Mesh if we'll never use it
		return false;
	}

	return true;
}
#endif

bool UNiagaraDataInterfaceStaticMesh::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	NDIStaticMeshLocal::FInstanceData_GameThread* InstanceData = new(PerInstanceData) NDIStaticMeshLocal::FInstanceData_GameThread();
	const bool bIsValid = InstanceData->Init(this, SystemInstance);

	// Create render thread data?
	if (bIsValid && IsUsedWithGPUScript() )
	{
		TUniquePtr<NDIStaticMeshLocal::FGpuInitializeData> GpuInitializeData = MakeUnique<NDIStaticMeshLocal::FGpuInitializeData>();
		GpuInitializeData->RenderProxy = GetProxyAs<NDIStaticMeshLocal::FRenderProxy>();
		GpuInitializeData->SystemInstanceID = SystemInstance->GetId();

		//This is safe to ref on the RT as it's freed on the RT in FStaticMeshComponentLODInfo::BeginReleaseOverrideVertexColors()
		//However, it's seems unsafe to reference in Niagara's instance data this way as there looks to be a window between this RT command and it's actual use where the data could have been freed.
#if WITH_EDITOR
		GpuInitializeData->LODResource = InstanceData->GetCurrentLOD();
#else
		GpuInitializeData->LODResource = InstanceData->GetCurrentLODWithVertexColorOverrides(GpuInitializeData->OverrideColorBuffer);
#endif

		if ( GpuInitializeData->LODResource )
		{
		#if DO_CHECK
			GpuInitializeData->SystemFName = SystemInstance->GetSystem()->GetFName();
			if ( UStaticMesh* StaticMesh = InstanceData->StaticMeshWeakPtr.Get() )
			{
				GpuInitializeData->StaticMeshFName = StaticMesh->GetFName();
			}
		#endif

			GpuInitializeData->NumFilteredTriangles = InstanceData->NumFilteredTriangles;
			GpuInitializeData->NumUnfilteredTriangles = InstanceData->NumUnfilteredTriangles;

			GpuInitializeData->bGpuUniformDistribution = InstanceData->bIsGpuUniformlyDistributedSampling;

			// Build section data to pass
			GpuInitializeData->NumSections = GpuInitializeData->LODResource->Sections.Num();
			if (GpuInitializeData->NumSections > 0)
			{
				const FStaticMeshAreaWeightedSectionSampler* SectionSamplers = GpuInitializeData->bGpuUniformDistribution ? &GpuInitializeData->LODResource->AreaWeightedSampler : nullptr;
				for (int i = 0; i < GpuInitializeData->NumSections; ++i)
				{
					const FStaticMeshSection& MeshSection = GpuInitializeData->LODResource->Sections[i];
					const float Prob = SectionSamplers ? SectionSamplers->GetProb()[i] : 1.0f;
					const int32 Alias = SectionSamplers ? SectionSamplers->GetAlias()[i] : i;
					GpuInitializeData->SectionInfos.Emplace(MeshSection.FirstIndex / 3, MeshSection.NumTriangles, reinterpret_cast<const int32&>(Prob), Alias);
				}

				GpuInitializeData->NumFilteredSections = InstanceData->NumFilteredSections;
				for (const NDIStaticMeshLocal::FNDISectionInfo& SectionInfo : InstanceData->GetFilteredSectionInfos())
				{
					GpuInitializeData->SectionInfos.Emplace(SectionInfo.FirstTriangle, SectionInfo.NumTriangles, reinterpret_cast<const int32&>(SectionInfo.Prob), SectionInfo.Alias);
				}
				for (int Section : InstanceData->GetFilteredSections())
				{
					GpuInitializeData->FilteredAndUnfilteredSections.Add(uint16(Section));
				}

				GpuInitializeData->NumUnfilteredSections = InstanceData->NumUnfilteredSections;
				for (const NDIStaticMeshLocal::FNDISectionInfo& SectionInfo : InstanceData->GetUnfilteredSectionInfos())
				{
					GpuInitializeData->SectionInfos.Emplace(SectionInfo.FirstTriangle, SectionInfo.NumTriangles, reinterpret_cast<const int32&>(SectionInfo.Prob), SectionInfo.Alias);
				}
				for (int Section : InstanceData->GetUnfilteredSections())
				{
					GpuInitializeData->FilteredAndUnfilteredSections.Add(uint16(Section));
				}
			}
		}

		// Build socket data to pass
		GpuInitializeData->NumSockets = InstanceData->CachedSockets.Num();
		if (GpuInitializeData->NumSockets > 0)
		{
			GpuInitializeData->NumFilteredSockets = InstanceData->NumFilteredSockets;
			GpuInitializeData->NumUnfilteredSockets = GpuInitializeData->NumSockets - InstanceData->NumFilteredSockets;

			GpuInitializeData->SocketTransforms.Reserve(GpuInitializeData->NumFilteredSockets * 3);
			for (const FTransform3f& SocketTransform : InstanceData->CachedSockets)
			{
				const FQuat4f SocketRotation = SocketTransform.GetRotation();
				GpuInitializeData->SocketTransforms.Emplace(SocketTransform.GetTranslation(), 0.0f);
				GpuInitializeData->SocketTransforms.Emplace(SocketRotation.X, SocketRotation.Y, SocketRotation.Z, SocketRotation.W);
				GpuInitializeData->SocketTransforms.Emplace(SocketTransform.GetScale3D(), 0.0f);
			}

			if (InstanceData->NumFilteredSockets > 0)
			{
				GpuInitializeData->FilteredAndUnfilteredSockets.Reserve(InstanceData->FilteredAndUnfilteredSockets.Num());
				for (uint16 v : InstanceData->FilteredAndUnfilteredSockets)
				{
					GpuInitializeData->FilteredAndUnfilteredSockets.Add(v);
				}
			}
		}

		// Initialize the render thread side data
		ENQUEUE_RENDER_COMMAND(InitInstanceData)(
			[GpuInitializeData_RT=MoveTemp(GpuInitializeData)](FRHICommandListImmediate& CmdList)
			{
				NDIStaticMeshLocal::FInstanceData_RenderThread& InstanceData_RT = GpuInitializeData_RT->RenderProxy->PerInstanceData_RT.Add(GpuInitializeData_RT->SystemInstanceID);
				InstanceData_RT.Init(CmdList, *(GpuInitializeData_RT.Get()));
			}
		);
	}

	return bIsValid;
}

void UNiagaraDataInterfaceStaticMesh::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	NDIStaticMeshLocal::FInstanceData_GameThread* InstanceData = reinterpret_cast<NDIStaticMeshLocal::FInstanceData_GameThread*>(PerInstanceData);
	InstanceData->Release();
	InstanceData->~FInstanceData_GameThread();

	if ( IsUsedWithGPUScript() )
	{
		NDIStaticMeshLocal::FRenderProxy* Proxy_RT = GetProxyAs<NDIStaticMeshLocal::FRenderProxy>();
		FNiagaraSystemInstanceID InstanceID_RT = SystemInstance->GetId();
		ENQUEUE_RENDER_COMMAND(RemoveInstanceData)(
			[Proxy_RT, InstanceID_RT](FRHICommandListImmediate& CmdList)
			{
				Proxy_RT->PerInstanceData_RT.Remove(InstanceID_RT);
			}
		);
	}
}

bool UNiagaraDataInterfaceStaticMesh::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIStaticMeshLocal;

	FInstanceData_GameThread* InstanceData = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
	return InstanceData->Tick(this, SystemInstance, DeltaSeconds);
}

int32 UNiagaraDataInterfaceStaticMesh::PerInstanceDataSize() const
{
	return sizeof(NDIStaticMeshLocal::FInstanceData_GameThread);
}

void UNiagaraDataInterfaceStaticMesh::ProvidePerInstanceDataForRenderThread(void* InDataFromGT, void* InInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	NDIStaticMeshLocal::FInstanceData_GameThread* InstanceData = static_cast<NDIStaticMeshLocal::FInstanceData_GameThread*>(InInstanceData);
	NDIStaticMeshLocal::FInstanceData_FromGameThread* DataFromGT = static_cast<NDIStaticMeshLocal::FInstanceData_FromGameThread*>(InDataFromGT);

	DataFromGT->Transform					= FMatrix44f(InstanceData->Transform);			// LWC_TODO: Precision loss
	DataFromGT->PrevTransform				= FMatrix44f(InstanceData->PrevTransform);
	DataFromGT->Rotation					= InstanceData->Rotation;
	DataFromGT->PrevRotation				= InstanceData->PrevRotation;
	DataFromGT->DeltaSeconds				= InstanceData->DeltaSeconds;
	DataFromGT->PreSkinnedLocalBoundsCenter = InstanceData->PreSkinnedLocalBoundsCenter;
	DataFromGT->PreSkinnedLocalBoundsExtents = InstanceData->PreSkinnedLocalBoundsExtents;
	DataFromGT->OwnerToMeshVector			= InstanceData->OwnerToMeshVector;
	DataFromGT->DistanceFieldPrimitiveId	= FPrimitiveComponentId();
	DataFromGT->UvMappingBuffer				= InstanceData->UvMapping.GetQuadTreeProxy();
	DataFromGT->UvMappingSet				= InstanceData->UvMapping.GetUvSetIndex();

#if WITH_EDITOR
	InstanceData->GetCurrentLODWithVertexColorOverrides(DataFromGT->OverrideVertexColors);
#endif

	DataFromGT->DistanceFieldPrimitiveId = InstanceData->SceneComponentUtils->GetPrimitiveSceneId(InstanceData->ComponentWeakPtr.Get());
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceStaticMesh::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIStaticMeshLocal;

	//OutFunctions.Reserve(OutFunctions.Num() + NumFunctions);

	// Setup base signature
	FNiagaraFunctionSignature BaseSignature;
	BaseSignature.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("StaticMesh"));
	BaseSignature.bMemberFunction = true;
	BaseSignature.bRequiresContext = false;
	BaseSignature.FunctionVersion = EDIFunctionVersion::LatestVersion;

	GetVertexSamplingFunctions(OutFunctions, BaseSignature);
	GetTriangleSamplingFunctions(OutFunctions, BaseSignature);
	GetSocketSamplingFunctions(OutFunctions, BaseSignature);
	GetSectionFunctions(OutFunctions, BaseSignature);
	GetMiscFunctions(OutFunctions, BaseSignature);
	GetUVMappingFunctions(OutFunctions, BaseSignature);
	GetDistanceFieldFunctions(OutFunctions, BaseSignature);
	GetDeprecatedFunctions(OutFunctions, BaseSignature);
}

void UNiagaraDataInterfaceStaticMesh::GetVertexSamplingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const
{
	using namespace NDIStaticMeshLocal;
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = IsValidVertexName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = RandomVertexName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetRandInfoDef(), TEXT("RandomInfo"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetVertexCountName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count"));
	}

	//{
	//	FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
	//	Sig.Name = IsValidFilteredVertexName;
	//	Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex"));
	//	Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
	//}
	//{
	//	FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
	//	Sig.Name = RandomFilteredVertexName;
	//	Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex"));
	//}
	//{
	//	FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
	//	Sig.Name = GetFilteredVertexCountName;
	//	Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count"));
	//}
	//{
	//	FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
	//	Sig.Name = GetFilteredVertexAtName;
	//	Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
	//	Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex"));
	//}

	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Bitangent"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent"));
		OutFunctions.Add_GetRef(Sig).Name = GetVertexName;
		OutFunctions.Last().SetOutputDescription(FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")), LOCTEXT("LocalSpaceVelocityDesc", "Local space velocity is always zero as the mesh does not have any local deformation."));
		
		FNiagaraFunctionSignature& WsSig = OutFunctions.Add_GetRef(Sig);
		WsSig.Name = GetVertexWSName;
		WsSig.Outputs[0].SetType(FNiagaraTypeDefinition::GetPositionDef());

		FNiagaraFunctionSignature& WsInterpSig = OutFunctions.Add_GetRef(Sig);
		WsInterpSig.Name = GetVertexWSInterpolatedName;
		WsInterpSig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interp"));
		WsInterpSig.Outputs[0].SetType(FNiagaraTypeDefinition::GetPositionDef());
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetVertexColorName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetVertexUVName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("UV Set"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
	}
}

void UNiagaraDataInterfaceStaticMesh::GetTriangleSamplingFunctions(TArray<FNiagaraFunctionSignature>&OutFunctions, const FNiagaraFunctionSignature & BaseSignature) const
{
	using namespace NDIStaticMeshLocal;
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = IsValidTriangleName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = RandomTriangleName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetRandInfoDef(), TEXT("RandomInfo"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaryCoord"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetTriangleCountName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = IsValidFilteredTriangleName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = RandomFilteredTriangleName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetRandInfoDef(), TEXT("RandomInfo"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaryCoord"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetFilteredTriangleCountName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetFilteredTriangleAtName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = IsValidUnfilteredTriangleName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = RandomUnfilteredTriangleName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetRandInfoDef(), TEXT("RandomInfo"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaryCoord"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetUnfilteredTriangleCountName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetUnfilteredTriangleAtName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
	}

	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
		Sig.Inputs.Emplace_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaryCoord"))).SetValue(FVector3f(1.0f / 3.0f));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Bitangent"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent"));
		OutFunctions.Add_GetRef(Sig).Name = GetTriangleName;
		OutFunctions.Last().SetOutputDescription(FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")), LOCTEXT("LocalSpaceVelocityDesc", "Local space velocity is always zero as the mesh does not have any local deformation."));

		FNiagaraFunctionSignature& WsSig = OutFunctions.Add_GetRef(Sig);
		WsSig.Name = GetTriangleWSName;
		WsSig.Outputs[0].SetType(FNiagaraTypeDefinition::GetPositionDef());

		FNiagaraFunctionSignature& WsInterpSig = OutFunctions.Add_GetRef(Sig);
		WsInterpSig.Name = GetTriangleWSInterpolatedName;
		WsInterpSig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interp"));
		WsInterpSig.Outputs[0].SetType(FNiagaraTypeDefinition::GetPositionDef());
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetTriangleColorName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
		Sig.Inputs.Emplace_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaryCoord"))).SetValue(FVector3f(1.0f / 3.0f));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetTriangleUVName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
		Sig.Inputs.Emplace_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaryCoord"))).SetValue(FVector3f(1.0f / 3.0f));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("UV Set"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetTriangleIndicesName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index0"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index1"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index2"));
	}
}

void UNiagaraDataInterfaceStaticMesh::GetSocketSamplingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const
{
	using namespace NDIStaticMeshLocal;
	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num")));

		OutFunctions.Add_GetRef(Sig).Name = GetSocketCountName;
		OutFunctions.Add_GetRef(Sig).Name = GetFilteredSocketCountName;
		OutFunctions.Add_GetRef(Sig).Name = GetUnfilteredSocketCountName;
	}
	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetRandInfoDef(), TEXT("RandomInfo"));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));

		OutFunctions.Add_GetRef(Sig).Name = RandomSocketName;
		OutFunctions.Add_GetRef(Sig).Name = RandomFilteredSocketName;
		OutFunctions.Add_GetRef(Sig).Name = RandomUnfilteredSocketName;
	}
	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));

		FNiagaraFunctionSignature WsSig = Sig;
		WsSig.Outputs[0].SetType(FNiagaraTypeDefinition::GetPositionDef());

		FNiagaraFunctionSignature WsInterpSig = WsSig;
		WsInterpSig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interp"));

		Sig.SetOutputDescription(FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")), LOCTEXT("LocalSpaceVelocityDesc", "Local space velocity is always zero as the mesh does not have any local deformation."));

		OutFunctions.Add_GetRef(Sig).Name = GetSocketTransformName;
		OutFunctions.Add_GetRef(WsSig).Name = GetSocketTransformWSName;
		OutFunctions.Add_GetRef(WsInterpSig).Name = GetSocketTransformWSInterpolatedName;
		OutFunctions.Add_GetRef(Sig).Name = GetFilteredSocketTransformName;
		OutFunctions.Add_GetRef(WsSig).Name = GetFilteredSocketTransformWSName;
		OutFunctions.Add_GetRef(WsInterpSig).Name = GetFilteredSocketTransformWSInterpolatedName;
		OutFunctions.Add_GetRef(Sig).Name = GetUnfilteredSocketTransformName;
		OutFunctions.Add_GetRef(WsSig).Name = GetUnfilteredSocketTransformWSName;
		OutFunctions.Add_GetRef(WsInterpSig).Name = GetUnfilteredSocketTransformWSInterpolatedName;
	}

	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SocketIndex"));

		OutFunctions.Add_GetRef(Sig).Name = GetFilteredSocketName;
		OutFunctions.Add_GetRef(Sig).Name = GetUnfilteredSocketName;
	}
}

void UNiagaraDataInterfaceStaticMesh::GetSectionFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const
{
	using namespace NDIStaticMeshLocal;
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = IsValidSectionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetSectionTriangleCountName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Section"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = RandomSectionTriangleName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetRandInfoDef(), TEXT("RandomInfo"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Section"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaryCoord"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetSectionTriangleAtName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Section"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
	}

	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Section"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count"));
		OutFunctions.Add_GetRef(Sig).Name = GetFilteredSectionAtName;
		OutFunctions.Add_GetRef(Sig).Name = GetUnfilteredSectionAtName;
	}

	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count"));
		OutFunctions.Add_GetRef(Sig).Name = GetSectionCountName;
		OutFunctions.Add_GetRef(Sig).Name = GetFilteredSectionCountName;
		OutFunctions.Add_GetRef(Sig).Name = GetUnfilteredSectionCountName;
	}

	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetRandInfoDef(), TEXT("RandomInfo"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Section"));
		OutFunctions.Add_GetRef(Sig).Name = RandomSectionName;
		OutFunctions.Add_GetRef(Sig).Name = RandomFilteredSectionName;
		OutFunctions.Add_GetRef(Sig).Name = RandomUnfilteredSectionName;
	}
}

void UNiagaraDataInterfaceStaticMesh::GetMiscFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const
{
	using namespace NDIStaticMeshLocal;
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = IsValidName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetPreSkinnedLocalBoundsName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("ExtentsMin"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("ExtentsMax"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Extents"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Half Extents"));
		Sig.SetDescription(LOCTEXT("GetPreSkinnedLocalBoundsDesc", "Returns the bounding information of the mesh vertices."));
		Sig.SetOutputDescription(Sig.Outputs[0], LOCTEXT("PreSkinnedLocalBounds_CenterDesc", "Center of the mesh vertices bounding box, i.e. ((max - min) * 0.5) + min"));
		Sig.SetOutputDescription(Sig.Outputs[1], LOCTEXT("PreSkinnedLocalBounds_ExtentsMinDesc", "Min of the mesh vertices."));
		Sig.SetOutputDescription(Sig.Outputs[2], LOCTEXT("PreSkinnedLocalBounds_ExtentsMaxDesc", "Max of the mesh vertices."));
		Sig.SetOutputDescription(Sig.Outputs[3], LOCTEXT("PreSkinnedLocalBounds_ExtentsDesc", "Extents of the mesh vertices bounding box, i.e. max - min"));
		Sig.SetOutputDescription(Sig.Outputs[4], LOCTEXT("PreSkinnedLocalBounds_HalfExtentsDesc", "Extents * 0.5f of the mesh vertices bounding box, i.e. (max - min) * 0.5"));
	}
	{
		FNiagaraFunctionSignature BoundsSig = BaseSignature;
		BoundsSig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Center"));
		BoundsSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("ExtentsMin"));
		BoundsSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("ExtentsMax"));
		BoundsSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Extents"));
		BoundsSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Half Extents"));
		BoundsSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("OwnerToMeshVector"));
		BoundsSig.SetOutputDescription(BoundsSig.Outputs.Last(), LOCTEXT("GetMeshBounds_MeshComponentOffsetDesc", "Offset vector to the Source Mesh Component from the Niagara instances location."));

		FNiagaraFunctionSignature& LSSig = OutFunctions.Add_GetRef(BoundsSig);
		LSSig.Name = GetMeshBoundsName;
		LSSig.SetDescription(LOCTEXT("GetMeshBoundsDesc", "Returns the bounding information of the mesh in local space."));

		FNiagaraFunctionSignature& WSSig = OutFunctions.Add_GetRef(BoundsSig);
		WSSig.Name = GetMeshBoundsWSName;
		WSSig.SetDescription(LOCTEXT("GetMeshBoundsWSDesc", "Returns the bounding information of the mesh in world space."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetLocalToWorldName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Matrix"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetLocalToWorldInverseTransposedName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Matrix"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetWorldVelocityName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity"));
	}
	//-TBD: There's not really a benefit to getting this in the graph currently
	//{
	//	FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
	//	Sig.Name = GetInstanceIndexName;
	//	Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
	//}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = SetInstanceIndexName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.bRequiresExecPin = true;
		Sig.bSupportsGPU = false;
	}
}

void UNiagaraDataInterfaceStaticMesh::GetUVMappingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const
{
	using namespace NDIStaticMeshLocal;
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetTriangleCoordAtUVName;

		Sig.Inputs.Emplace_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled"))).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV")));
		Sig.Inputs.Emplace_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Tolerance"))).SetValue(KINDA_SMALL_NUMBER);

		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaryCoord")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetTriangleCoordInAabbName;

		Sig.Inputs.Emplace_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled"))).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UvMin")));
		Sig.Inputs.Emplace(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UvMax")));

		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaryCoord")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = BuildUvMappingName;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::System;
		Sig.bSupportsGPU = false;
		Sig.bRequiresExecPin = true;

		Sig.Inputs.Emplace(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("UvSet")));
	}
}

void UNiagaraDataInterfaceStaticMesh::GetDistanceFieldFunctions(TArray<FNiagaraFunctionSignature>&OutFunctions, const FNiagaraFunctionSignature & BaseSignature) const
{
	using namespace NDIStaticMeshLocal;
	{
		const FNiagaraVariable UseMaxDistanceVar(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Use Max Distance"));
		const FNiagaraVariable MaxDistanceVar(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Max Distance"));

		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = QueryDistanceFieldName;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position"));
		Sig.Inputs.Emplace(UseMaxDistanceVar);
		Sig.Inputs.Emplace(MaxDistanceVar);
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Distance"));
		
#if WITH_EDITORONLY_DATA
		Sig.Description = LOCTEXT("QueryDistanceFieldDescription", "Given a world position, this returns the value of the parented static mesh`s signed distance field");
		Sig.InputDescriptions.Add(
			UseMaxDistanceVar,
			LOCTEXT("UseMaxDistanceOptimizationDescription", 
			"This enables an optimization that will skip reading the SDF texture if the world position exceeds a provided max distance value.\n"
			"This is a useful optimization if you only need accurate distance information within a certain distance threshold. \n"
			"One example of a good usecase is if you only want to spawn particles inside the mesh (i.e. distance < 0.0), you should enable this with a MaxDistance of 0.0.")
		);
		Sig.InputDescriptions.Add(
			MaxDistanceVar,
			LOCTEXT("MaxDistanceDescription",
			"Only used if UseMaxDistanceOptimization is enabled. This is the max distance from the static meshes` BOUNDING BOX at which we should query it's SDF.\n"
			"This defaults to 0.0, which means it will only read the SDF if the world position is inside the meshes' bounding box")
		);
#endif
	}
}

void UNiagaraDataInterfaceStaticMesh::GetDeprecatedFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& BaseSignature) const
{
	using namespace NDIStaticMeshLocal;
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.bSoftDeprecatedFunction = true;
		Sig.Name = Deprecated_GetTriColorName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.bSoftDeprecatedFunction = true;
		Sig.Name = Deprecated_GetTriUVName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("UV Set"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
	}
	{
		FNiagaraFunctionSignature & Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.bSoftDeprecatedFunction = true;
		Sig.Name = Deprecated_RandomSectionName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Section"));
	}
	{
		FNiagaraFunctionSignature & Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.bSoftDeprecatedFunction = true;
		Sig.Name = Deprecated_RandomTriCoordName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord"));
	}
	{
		FNiagaraFunctionSignature & Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.bSoftDeprecatedFunction = true;
		Sig.Name = Deprecated_RandomTriCoordOnSectionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Section"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord"));
	}
	// Need custom code
	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.bSoftDeprecatedFunction = true;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position"));
		OutFunctions.Add_GetRef(Sig).Name = Deprecated_GetVertexPositionName;
		OutFunctions.Add_GetRef(Sig).Name = Deprecated_GetVertexPositionWSName;
	}

	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.bSoftDeprecatedFunction = true;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position"));
		OutFunctions.Add_GetRef(Sig).Name = Deprecated_GetTriPositionName;
		OutFunctions.Add_GetRef(Sig).Name = Deprecated_GetTriPositionWSName;
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.bSoftDeprecatedFunction = true;
		Sig.Name = Deprecated_GetTriPositionAndVelocityWSName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity"));
	}
	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.bSoftDeprecatedFunction = true;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal"));
		OutFunctions.Add_GetRef(Sig).Name = Deprecated_GetTriTangentsName;
		OutFunctions.Add_GetRef(Sig).Name = Deprecated_GetTriTangentsWSName;
	}
	{
		FNiagaraFunctionSignature Sig = BaseSignature;
		Sig.bSoftDeprecatedFunction = true;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal"));
		OutFunctions.Add_GetRef(Sig).Name = Deprecated_GetTriNormalName;
		OutFunctions.Add_GetRef(Sig).Name = Deprecated_GetTriNormalWSName;
	}
}
#endif

bool UNiagaraDataInterfaceStaticMesh::FunctionNeedsCpuAccess(FName InName)
{
	using namespace NDIStaticMeshLocal;

	static TSet<FName> SafeFunctions =
	{
		IsValidName,
		GetPreSkinnedLocalBoundsName,
		GetMeshBoundsName,
		GetMeshBoundsWSName,
		GetLocalToWorldName,
		GetLocalToWorldInverseTransposedName,
		GetWorldVelocityName,
		//GetInstanceIndexName,
		SetInstanceIndexName,

		// Socket functions are safe
		GetSocketCountName,
		GetFilteredSocketCountName,
		GetUnfilteredSocketCountName,
		RandomSocketName,
		RandomFilteredSocketName,
		RandomUnfilteredSocketName,
		GetSocketTransformName,
		GetSocketTransformWSName,
		GetSocketTransformWSInterpolatedName,
		GetFilteredSocketTransformName,
		GetFilteredSocketTransformWSName,
		GetFilteredSocketTransformWSInterpolatedName,
		GetUnfilteredSocketTransformName,
		GetUnfilteredSocketTransformWSName,
		GetUnfilteredSocketTransformWSInterpolatedName,
		GetFilteredSocketName,
		GetUnfilteredSocketName,
	};

	return SafeFunctions.Contains(InName) == false;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceStaticMesh::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	using namespace NDIStaticMeshLocal;

	// Early out for version matching
	if (FunctionSignature.FunctionVersion == EDIFunctionVersion::LatestVersion)
	{
		return false;
	}

	// Major refactor for consistency across the data interface
	if (FunctionSignature.FunctionVersion < EDIFunctionVersion::RefactoredV1)
	{
		static const TPair<FName, FName> NameTranslations[] =
		{
			MakeTuple("RandomSection", Deprecated_RandomSectionName),
			MakeTuple("RandomTriCoord", Deprecated_RandomTriCoordName),
			MakeTuple("RandomTriCoordOnSection", Deprecated_RandomTriCoordOnSectionName),
			MakeTuple("RandomTriCoordUsingVertexColorFilter", Deprecated_RandomTriCoordVCFilteredName),
			MakeTuple("GetVertexPosition", Deprecated_GetVertexPositionName),
			MakeTuple("GetVertexPositionWS", Deprecated_GetVertexPositionWSName),
			MakeTuple("GetTriPosition", Deprecated_GetTriPositionName),
			MakeTuple("GetTriNormal", Deprecated_GetTriNormalName),
			MakeTuple("GetTriTangents", Deprecated_GetTriTangentsName),
			MakeTuple("GetTriPositionWS", Deprecated_GetTriPositionWSName),
			MakeTuple("GetTriNormalWS", Deprecated_GetTriNormalWSName),
			MakeTuple("GetTriTangentsWS", Deprecated_GetTriTangentsWSName),
			MakeTuple("GetTriColor", Deprecated_GetTriColorName),
			MakeTuple("GetTriUV", Deprecated_GetTriUVName),
			MakeTuple("GetTriPositionAndVelocityWS", Deprecated_GetTriPositionAndVelocityWSName),
			MakeTuple("GetMeshLocalToWorldInverseTransposed", GetLocalToWorldInverseTransposedName),
		};

		for ( const auto& NameTrans : NameTranslations )
		{
			if ( FunctionSignature.Name == NameTrans.Key )
			{
				FunctionSignature.Name = NameTrans.Value;
				break;
			}
		}
	}

	if (FunctionSignature.FunctionVersion < EDIFunctionVersion::LargeWorldCoordinates2)
	{
		if (
			(FunctionSignature.Name == GetVertexName) ||
			(FunctionSignature.Name == GetVertexWSName) ||
			(FunctionSignature.Name == GetTriangleName) ||
			(FunctionSignature.Name == GetTriangleWSName) ||
			(FunctionSignature.Name == GetSocketTransformName) ||
			(FunctionSignature.Name == GetSocketTransformWSName) ||
			(FunctionSignature.Name == GetFilteredSocketTransformName) ||
			(FunctionSignature.Name == GetFilteredSocketTransformWSName) ||
			(FunctionSignature.Name == GetUnfilteredSocketTransformName) ||
			(FunctionSignature.Name == GetUnfilteredSocketTransformWSName) )
		{
			check(FunctionSignature.Outputs[0].GetName() == TEXT("Position"));
			check(FunctionSignature.Outputs[0].GetType() == FNiagaraTypeDefinition::GetVec3Def() || FunctionSignature.Outputs[0].GetType() == FNiagaraTypeDefinition::GetPositionDef());
			FunctionSignature.Outputs[0].SetType(FNiagaraTypeDefinition::GetPositionDef());
		}
	}

	if (FunctionSignature.FunctionVersion < EDIFunctionVersion::AddSocketVelocity)
	{
		if ((FunctionSignature.Name == GetSocketTransformName) ||
			(FunctionSignature.Name == GetSocketTransformWSName) ||
			(FunctionSignature.Name == GetFilteredSocketTransformName) ||
			(FunctionSignature.Name == GetFilteredSocketTransformWSName) ||
			(FunctionSignature.Name == GetUnfilteredSocketTransformName) ||
			(FunctionSignature.Name == GetUnfilteredSocketTransformWSName))
		{
			FunctionSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity"));
		}
	}

	FunctionSignature.FunctionVersion = EDIFunctionVersion::LatestVersion;

	return true;
}

void UNiagaraDataInterfaceStaticMesh::GetCommonHLSL(FString& OutHLSL)
{
	if (GetDefault<UNiagaraSettings>()->NDIStaticMesh_AllowDistanceFields)
	{
		OutHLSL += TEXT("#include \"/Engine/Private/DistanceFieldLightingShared.ush\"\n");
		OutHLSL += TEXT("#include \"/Engine/Private/MeshDistanceFieldCommon.ush\"\n");
	}

	OutHLSL += TEXT("#include \"/Plugin/FX/Niagara/Private/Experimental/NiagaraUvMappingUtils.ush\"\n");
}
#endif //WITH_EDITORONLY_DATA

void UNiagaraDataInterfaceStaticMesh::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIStaticMeshLocal;

	//////////////////////////////////////////////////////////////////////////
	// VM Vertex Sampling
	if (BindingInfo.Name == IsValidVertexName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMIsValidVertex);
	}
	else if (BindingInfo.Name == RandomVertexName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomVertex);
	}
	else if (BindingInfo.Name == GetVertexCountName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetVertexCount);
	}
	//else if (BindingInfo.Name == IsValidFilteredVertexName)
	//{
	//	OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMIsValidFilteredVertex);
	//}
	//else if (BindingInfo.Name == RandomFilteredVertexName)
	//{
	//	OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomFilteredVertexAt);
	//}
	//else if (BindingInfo.Name == GetFilteredVertexCountName)
	//{
	//	OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetFilteredVertexCount);
	//}
	//else if (BindingInfo.Name == GetFilteredVertexAtName)
	//{
	//	OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetFilteredVertexAt);
	//}
	else if (BindingInfo.Name == GetVertexName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetVertex<FNDITransformHandlerNoop>);
	}
	else if (BindingInfo.Name == GetVertexWSName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetVertex<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == GetVertexWSInterpolatedName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetVertexInterpolated<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == GetVertexColorName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetVertexColor);
	}
	else if (BindingInfo.Name == GetVertexUVName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetVertexUV);
	}

	//////////////////////////////////////////////////////////////////////////
	// VM Triangle Sampling
	else if (BindingInfo.Name == IsValidTriangleName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMIsValidTriangle);
	}
	else if (BindingInfo.Name == RandomTriangleName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomTriangle<FNDIRandomHelper>);
	}
	else if (BindingInfo.Name == GetTriangleCountName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleCount);
	}
	else if (BindingInfo.Name == IsValidFilteredTriangleName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMIsValidFilteredTriangle);
	}
	else if (BindingInfo.Name == RandomFilteredTriangleName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomFilteredTriangle);
	}
	else if (BindingInfo.Name == GetFilteredTriangleCountName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetFilteredTriangleCount);
	}
	else if (BindingInfo.Name == GetFilteredTriangleAtName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetFilteredTriangleAt);
	}
	else if (BindingInfo.Name == IsValidUnfilteredTriangleName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMIsValidUnfilteredTriangle);
	}
	else if (BindingInfo.Name == RandomUnfilteredTriangleName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomUnfilteredTriangle);
	}
	else if (BindingInfo.Name == GetUnfilteredTriangleCountName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredTriangleCount);
	}
	else if (BindingInfo.Name == GetUnfilteredTriangleAtName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredTriangleAt);
	}
	else if (BindingInfo.Name == GetTriangleName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangle<FNDITransformHandlerNoop>);
	}
	else if (BindingInfo.Name == GetTriangleWSName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangle<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == GetTriangleWSInterpolatedName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleInterpolated<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == GetTriangleColorName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleColor);
	}
	else if (BindingInfo.Name == GetTriangleUVName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleUV);
	}
	else if (BindingInfo.Name == GetTriangleIndicesName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleIndices);
	}

	//////////////////////////////////////////////////////////////////////////
	// Socket Functions
	else if (BindingInfo.Name == GetSocketCountName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetSocketCount);
	}
	else if (BindingInfo.Name == GetFilteredSocketCountName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetFilteredSocketCount);
	}
	else if (BindingInfo.Name == GetUnfilteredSocketCountName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSocketCount);
	}
	else if (BindingInfo.Name == RandomSocketName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomSocket);
	}
	else if (BindingInfo.Name == RandomFilteredSocketName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomFilteredSocket);
	}
	else if (BindingInfo.Name == RandomUnfilteredSocketName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomUnfilteredSocket);
	}
	else if (BindingInfo.Name == GetSocketTransformName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetSocketTransform<FNDITransformHandlerNoop>);
	}
	else if (BindingInfo.Name == GetSocketTransformWSName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetSocketTransform<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == GetSocketTransformWSInterpolatedName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetSocketTransformInterpolated<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == GetFilteredSocketTransformName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetFilteredSocketTransform<FNDITransformHandlerNoop>);
	}
	else if (BindingInfo.Name == GetFilteredSocketTransformWSName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetFilteredSocketTransform<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == GetFilteredSocketTransformWSInterpolatedName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetFilteredSocketTransformInterpolated<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == GetUnfilteredSocketTransformName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSocketTransform<FNDITransformHandlerNoop>);
	}
	else if (BindingInfo.Name == GetUnfilteredSocketTransformWSName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSocketTransform<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == GetUnfilteredSocketTransformWSInterpolatedName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSocketTransformInterpolated<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == GetFilteredSocketName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetFilteredSocket);
	}
	else if (BindingInfo.Name == GetUnfilteredSocketName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSocket);
	}

	//////////////////////////////////////////////////////////////////////////
	// Section functions
	else if (BindingInfo.Name == IsValidSectionName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMIsValidSection);
	}
	else if (BindingInfo.Name == GetSectionTriangleCountName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetSectionTriangleCount);
	}
	else if (BindingInfo.Name == RandomSectionTriangleName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomSectionTriangle<FNDIRandomHelper>);
	}
	else if (BindingInfo.Name == GetSectionTriangleAtName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetSectionTriangleAt);
	}
	else if (BindingInfo.Name == GetFilteredSectionAtName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetFilteredSectionAt);
	}
	else if (BindingInfo.Name == GetUnfilteredSectionAtName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSectionAt);
	}
	else if (BindingInfo.Name == GetSectionCountName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetSectionCount);
	}
	else if (BindingInfo.Name == GetFilteredSectionCountName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetFilteredSectionCount);
	}
	else if (BindingInfo.Name == GetUnfilteredSectionCountName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSectionCount);
	}
	else if (BindingInfo.Name == RandomSectionName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomSection<FNDIRandomHelper>);
	}
	else if (BindingInfo.Name == RandomFilteredSectionName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomFilteredSection<FNDIRandomHelper>);
	}
	else if (BindingInfo.Name == RandomUnfilteredSectionName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomUnfilteredSection);
	}

	//////////////////////////////////////////////////////////////////////////
	// VM Misc Functions
	else if (BindingInfo.Name == IsValidName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMIsValid);
	}
	else if (BindingInfo.Name == GetPreSkinnedLocalBoundsName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetPreSkinnedLocalBounds);
	}
	else if (BindingInfo.Name == GetMeshBoundsName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetMeshBounds<true>);
	}
	else if (BindingInfo.Name == GetMeshBoundsWSName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetMeshBounds<false>);
	}
	else if (BindingInfo.Name == GetLocalToWorldName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetLocalToWorld);
	}
	else if (BindingInfo.Name == GetLocalToWorldInverseTransposedName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetLocalToWorldInverseTransposed);
	}
	else if (BindingInfo.Name == GetWorldVelocityName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetWorldVelocity);
	}
	else if (BindingInfo.Name == GetInstanceIndexName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetInstanceIndex);
	}
	else if (BindingInfo.Name == SetInstanceIndexName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMSetInstanceIndex);
	}

	//////////////////////////////////////////////////////////////////////////
	// VM UV mapping functions
	else if (BindingInfo.Name == GetTriangleCoordAtUVName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleCoordAtUV);
	}
	else if (BindingInfo.Name == GetTriangleCoordInAabbName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleCoordInAabb);
	}
	else if (BindingInfo.Name == BuildUvMappingName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMBuildUvMapping);
	}

	//////////////////////////////////////////////////////////////////////////
	// Deprecated Functions
	else if (BindingInfo.Name == Deprecated_GetTriColorName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleColor);
	}
	else if (BindingInfo.Name == Deprecated_GetTriUVName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleUV);
	}
	// Custom VM bindings
	else if (BindingInfo.Name == Deprecated_RandomSectionName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomFilteredSection<FNDIRandomHelperFromStream>);
	}
	else if (BindingInfo.Name == Deprecated_RandomTriCoordName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomTriangle<FNDIRandomHelperFromStream>);
	}
	else if (BindingInfo.Name == Deprecated_RandomTriCoordOnSectionName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMRandomSectionTriangle<FNDIRandomHelperFromStream>);
	}
	else if (BindingInfo.Name == Deprecated_GetVertexPositionName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetVertexPosition_Deprecated<FNDITransformHandlerNoop>);
	}
	else if (BindingInfo.Name == Deprecated_GetVertexPositionWSName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetVertexPosition_Deprecated<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == Deprecated_GetTriPositionName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriPosition_Deprecated<FNDITransformHandlerNoop>);
	}
	else if (BindingInfo.Name == Deprecated_GetTriPositionWSName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriPosition_Deprecated<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == Deprecated_GetTriTangentsName)
	{	
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleTangentBasis_Deprecated<FNDITransformHandlerNoop>);
	}
	else if (BindingInfo.Name == Deprecated_GetTriTangentsWSName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleTangentBasis_Deprecated<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == Deprecated_GetTriNormalName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleNormal_Deprecated<FNDITransformHandlerNoop>);
	}
	else if (BindingInfo.Name == Deprecated_GetTriNormalWSName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriangleNormal_Deprecated<FNDITransformHandler>);
	}
	else if (BindingInfo.Name == Deprecated_GetTriPositionAndVelocityWSName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceStaticMesh::VMGetTriPositionAndVelocity_Deprecated<FNDITransformHandler>);
	}
}

bool UNiagaraDataInterfaceStaticMesh::RequiresEarlyViewData() const
{
	return GetDefault<UNiagaraSettings>()->NDIStaticMesh_AllowDistanceFields;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceStaticMesh::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIStaticMeshLocal::TemplateShaderFile);
	bSuccess &= InVisitor->UpdatePOD(TEXT("NDIStaticMesh_AllowDistanceField"), GetDefault<UNiagaraSettings>()->NDIStaticMesh_AllowDistanceFields ? 1 : 0);
	bSuccess &= InVisitor->UpdateShaderParameters<NDIStaticMeshLocal::FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceStaticMesh::ModifyCompilationEnvironment(EShaderPlatform ShaderPlatform, struct FShaderCompilerEnvironment& OutEnvironment) const
{
	Super::ModifyCompilationEnvironment(ShaderPlatform, OutEnvironment);

	OutEnvironment.SetDefine(TEXT("DISTATICMESH_ALLOWDISTANCEFIELD"), GetDefault<UNiagaraSettings>()->NDIStaticMesh_AllowDistanceFields ? 1 : 0);
}

void UNiagaraDataInterfaceStaticMesh::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIStaticMeshLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceStaticMesh::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIStaticMeshLocal;

	// Currently we support all functions on the GPU
	// Add an exclusion if the function is not supported by GPU

	TMap<FString, FStringFormatArg> ArgsSample =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("InstanceFunctionName"), FunctionInfo.InstanceName},
	};

	// We still need to go through some hoops for the old deprecated functions
	if (FunctionInfo.DefinitionName == Deprecated_RandomSectionName)
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(out int OutSection) { RandomFilteredSection_{ParameterName}(MakeRandInfo(), OutSection); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if (FunctionInfo.DefinitionName == Deprecated_RandomTriCoordName )
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(out MeshTriCoordinate OutTriCoord) { RandomTriangle_{ParameterName}(MakeRandInfo(), OutTriCoord.Tri, OutTriCoord.BaryCoord); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if (FunctionInfo.DefinitionName == Deprecated_RandomTriCoordOnSectionName )
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(int Section, out MeshTriCoordinate OutTriCoord) { RandomSectionTriangle_{ParameterName}(MakeRandInfo(), Section, OutTriCoord.Tri, OutTriCoord.BaryCoord); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if (FunctionInfo.DefinitionName == Deprecated_RandomTriCoordVCFilteredName )
	{
		// Not supported anymore
		return false;
	}
	else if (FunctionInfo.DefinitionName == Deprecated_GetVertexPositionName)
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(int Vertex, out float3 Position) { float3 Dummy; GetVertex_{ParameterName}(Vertex, Position, Dummy, Dummy, Dummy, Dummy); }");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if (FunctionInfo.DefinitionName == Deprecated_GetVertexPositionWSName)
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(int Vertex, out float3 Position) { float3 Dummy; GetVertexWS_{ParameterName}(Vertex, Position, Dummy, Dummy, Dummy, Dummy); }");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if (FunctionInfo.DefinitionName == Deprecated_GetTriPositionName )
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(MeshTriCoordinate TriCoord, out float3 Position) { float3 Dummy; GetTriangle_{ParameterName}(TriCoord.Tri, TriCoord.BaryCoord, Position, Dummy, Dummy, Dummy, Dummy); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if ( FunctionInfo.DefinitionName == Deprecated_GetTriNormalName )
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(MeshTriCoordinate TriCoord, out float3 Normal) { float3 Dummy; GetTriangle_{ParameterName}(TriCoord.Tri, TriCoord.BaryCoord, Dummy, Dummy, Normal, Dummy, Dummy); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if ( FunctionInfo.DefinitionName == Deprecated_GetTriTangentsName )
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(MeshTriCoordinate TriCoord, out float3 Tangent, out float3 Bitangent, out float3 Normal) { float3 Dummy; GetTriangle_{ParameterName}(TriCoord.Tri, TriCoord.BaryCoord, Dummy, Dummy, Normal, Bitangent, Tangent); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if ( FunctionInfo.DefinitionName == Deprecated_GetTriPositionWSName )
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(MeshTriCoordinate TriCoord, out float3 Position) { float3 Dummy; GetTriangleWS_{ParameterName}(TriCoord.Tri, TriCoord.BaryCoord, Position, Dummy, Dummy, Dummy, Dummy); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if ( FunctionInfo.DefinitionName == Deprecated_GetTriNormalWSName )
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(MeshTriCoordinate TriCoord, out float3 Normal) { float3 Dummy; GetTriangleWS_{ParameterName}(TriCoord.Tri, TriCoord.BaryCoord, Dummy, Dummy, Normal, Dummy, Dummy); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if ( FunctionInfo.DefinitionName == Deprecated_GetTriTangentsWSName )
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(MeshTriCoordinate TriCoord, out float3 Tangent, out float3 Bitangent, out float3 Normal) { float3 Dummy; GetTriangleWS_{ParameterName}(TriCoord.Tri, TriCoord.BaryCoord, Dummy, Dummy, Normal, Bitangent, Tangent); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if (FunctionInfo.DefinitionName == Deprecated_GetTriColorName)
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(MeshTriCoordinate TriCoord, out float4 Color) { GetTriangleColor_{ParameterName}(TriCoord.Tri, TriCoord.BaryCoord, Color); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if (FunctionInfo.DefinitionName == Deprecated_GetTriUVName)
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(MeshTriCoordinate TriCoord, int UVSet, out float2 UV) { GetTriangleUV_{ParameterName}(TriCoord.Tri, TriCoord.BaryCoord, UVSet, UV); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}
	else if ( FunctionInfo.DefinitionName == Deprecated_GetTriPositionAndVelocityWSName )
	{
		static const TCHAR* FormatSample = TEXT("void {InstanceFunctionName}(MeshTriCoordinate TriCoord, out float3 Position, out float3 Velocity) { float3 Dummy; GetTriangleWS_{ParameterName}(TriCoord.Tri, TriCoord.BaryCoord, Position, Velocity, Dummy, Dummy, Dummy); }\r\n");
		OutHLSL += FString::Format(FormatSample, ArgsSample);
	}

	static const TSet<FName> UnsupportedGpuFunctions =
	{
		SetInstanceIndexName,
	};

	return UnsupportedGpuFunctions.Contains(FunctionInfo.DefinitionName) ? false : true;
}
#endif

void UNiagaraDataInterfaceStaticMesh::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIStaticMeshLocal::FShaderParameters>();
	ShaderParametersBuilder.AddIncludedStruct(UE::FXRenderingUtils::DistanceFields::GetObjectBufferParametersMetadata());
	ShaderParametersBuilder.AddIncludedStruct(UE::FXRenderingUtils::DistanceFields::GetAtlasParametersMetadata());
}

void UNiagaraDataInterfaceStaticMesh::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	NDIStaticMeshLocal::FRenderProxy& DIProxy = Context.GetProxy<NDIStaticMeshLocal::FRenderProxy>();
	const NDIStaticMeshLocal::FInstanceData_RenderThread& InstanceData = DIProxy.PerInstanceData_RT.FindChecked(Context.GetSystemInstanceID());

	NDIStaticMeshLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDIStaticMeshLocal::FShaderParameters>();

	// Set mesh sampling data
	if (InstanceData.bIsValid)
	{
#if WITH_EDITOR
		const FShaderResourceViewRHIRef& ColorSRV = InstanceData.MeshOverrideColorBufferSRV.IsValid()
			? InstanceData.MeshOverrideColorBufferSRV
			: InstanceData.MeshColorBufferSRV;
#else
		const FShaderResourceViewRHIRef& ColorSRV = InstanceData.MeshColorBufferSRV;
#endif

		const bool bHasColors = ColorSRV.IsValid();
		ShaderParameters->NumTriangles		= InstanceData.NumTriangles;
		ShaderParameters->NumVertices		= InstanceData.NumVertices;
		ShaderParameters->NumUVs			= InstanceData.NumUVs;
		ShaderParameters->HasColors			= bHasColors ? 1 : 0;
		ShaderParameters->IndexBuffer		= InstanceData.MeshIndexBufferSRV;
		ShaderParameters->PositionBuffer	= InstanceData.MeshPositionBufferSRV;
		ShaderParameters->TangentBuffer		= InstanceData.MeshTangentBufferSRV;
		ShaderParameters->UVBuffer			= InstanceData.MeshUVBufferSRV.IsValid() ? InstanceData.MeshUVBufferSRV.GetReference() : FNiagaraRenderer::GetDummyFloat2Buffer();
		ShaderParameters->ColorBuffer		= bHasColors ? ColorSRV.GetReference() : FNiagaraRenderer::GetDummyWhiteColorBuffer();
		ShaderParameters->HasUniformSampling = InstanceData.bGpuUniformDistribution ? 1 : 0;
		ShaderParameters->UniformSamplingTriangles = InstanceData.bGpuUniformDistribution ? InstanceData.MeshUniformSamplingTriangleSRV.GetReference() : FNiagaraRenderer::GetDummyUInt2Buffer();
	}
	else
	{
		ShaderParameters->NumTriangles		= FIntVector::ZeroValue;
		ShaderParameters->NumVertices		= 0;
		ShaderParameters->NumUVs			= 0;
		ShaderParameters->HasColors			= 0;
		ShaderParameters->IndexBuffer		= FNiagaraRenderer::GetDummyUIntBuffer();
		ShaderParameters->PositionBuffer	= FNiagaraRenderer::GetDummyFloatBuffer();
		ShaderParameters->TangentBuffer		= FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->UVBuffer			= FNiagaraRenderer::GetDummyFloat2Buffer();
		ShaderParameters->ColorBuffer		= FNiagaraRenderer::GetDummyWhiteColorBuffer();

		ShaderParameters->HasUniformSampling = 0;
		ShaderParameters->UniformSamplingTriangles = FNiagaraRenderer::GetDummyUInt2Buffer();
	}

	// Section information
	ShaderParameters->SectionCounts	= InstanceData.SectionCounts;
	ShaderParameters->SectionInfos	= InstanceData.SectionInfos.SRV.IsValid() ? InstanceData.SectionInfos.SRV.GetReference() : FNiagaraRenderer::GetDummyUInt4Buffer();
	ShaderParameters->FilteredAndUnfilteredSections = InstanceData.FilteredAndUnfilteredSections.SRV.IsValid() ? InstanceData.FilteredAndUnfilteredSections.SRV.GetReference() : FNiagaraRenderer::GetDummyUIntBuffer();

	// Set socket sampling information
	ShaderParameters->SocketCounts		= InstanceData.SocketCounts;
	ShaderParameters->SocketTransforms	= InstanceData.SocketTransforms.SRV.IsValid() ? InstanceData.SocketTransforms.SRV.GetReference() : FNiagaraRenderer::GetDummyFloat4Buffer();
	ShaderParameters->FilteredAndUnfilteredSockets = InstanceData.FilteredAndUnfilteredSockets.SRV.IsValid() ? InstanceData.FilteredAndUnfilteredSockets.SRV.GetReference() : FNiagaraRenderer::GetDummyUIntBuffer();

	ShaderParameters->SectionInfos = InstanceData.SectionInfos.SRV.IsValid() ? InstanceData.SectionInfos.SRV.GetReference() : FNiagaraRenderer::GetDummyUInt4Buffer();
	ShaderParameters->SocketTransforms	= InstanceData.SocketTransforms.SRV.IsValid() ? InstanceData.SocketTransforms.SRV.GetReference() : FNiagaraRenderer::GetDummyFloat4Buffer();

	// Set misc data
	const float InvDeltaTime = InstanceData.DeltaSeconds > 0.0f ? 1.0f / InstanceData.DeltaSeconds : 0.0f;
	const FVector3f DeltaPosition = InstanceData.Transform.GetOrigin() - InstanceData.PrevTransform.GetOrigin();

	ShaderParameters->InvDeltaSeconds								= InvDeltaTime;
	ShaderParameters->InstanceTransform								= InstanceData.Transform;
	ShaderParameters->InstanceTransformInverseTransposed			= InstanceData.Transform.Inverse().GetTransposed();
	ShaderParameters->InstanceRotation								= InstanceData.Rotation;
	ShaderParameters->InstancePreviousTransform						= InstanceData.PrevTransform;
	ShaderParameters->InstancePreviousTransformInverseTransposed	= InstanceData.PrevTransform.Inverse().GetTransposed();
	ShaderParameters->InstancePreviousRotation						= InstanceData.PrevRotation;
	ShaderParameters->InstanceWorldVelocity							= DeltaPosition;

	TConstStridedView<FSceneView> SimulationSceneViews = Context.GetComputeDispatchInterface().GetSimulationSceneViews();
	const FSceneView* PrimaryView = SimulationSceneViews.Num() > 0 ? &SimulationSceneViews[0] : nullptr;

	const bool bDistanceFieldParametersExist = PrimaryView ? UE::FXRenderingUtils::DistanceFields::HasDataToBind(*PrimaryView) : false;

	if (Context.IsParameterBound(&ShaderParameters->InstanceDistanceFieldIndex))
	{
		int32 DistanceFieldIndex = INDEX_NONE;
		if (bDistanceFieldParametersExist && InstanceData.DistanceFieldPrimitiveId.IsValid())
		{
			const FSceneInterface* Scene = Context.GetComputeDispatchInterface().GetSceneInterface();
			if (const FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene ? Scene->GetPrimitiveSceneInfo(InstanceData.DistanceFieldPrimitiveId) : nullptr)
			{
				DistanceFieldIndex = PrimitiveSceneInfo->DistanceFieldInstanceIndices.Num() > 0 ? PrimitiveSceneInfo->DistanceFieldInstanceIndices[0] : INDEX_NONE;
			}
		}
		ShaderParameters->InstanceDistanceFieldIndex = DistanceFieldIndex;
	}

	// Bind Mesh Distance Field Data
	const FShaderParametersMetadata* ObjectBufferParametersMetadata = UE::FXRenderingUtils::DistanceFields::GetObjectBufferParametersMetadata();
	const FShaderParametersMetadata* AtlasParametersMetadata = UE::FXRenderingUtils::DistanceFields::GetAtlasParametersMetadata();

	uint8* ShaderDistanceFieldObjectParameters = Context.GetParameterIncludedStruct(ObjectBufferParametersMetadata);
	uint8* ShaderDistanceFieldAtlasParameters = Context.GetParameterIncludedStruct(AtlasParametersMetadata);

	const bool bDistanceFieldDataBound =
		Context.IsStructBound(ShaderDistanceFieldObjectParameters, ObjectBufferParametersMetadata) ||
		Context.IsStructBound(ShaderDistanceFieldAtlasParameters, AtlasParametersMetadata);

	if (bDistanceFieldDataBound)
	{
		UE::FXRenderingUtils::DistanceFields::SetupObjectBufferParameters(Context.GetGraphBuilder(), ShaderDistanceFieldObjectParameters, PrimaryView);
		UE::FXRenderingUtils::DistanceFields::SetupAtlasParameters(Context.GetGraphBuilder(), ShaderDistanceFieldAtlasParameters, PrimaryView);
	}

	const FBox3f MeshBounds(InstanceData.PreSkinnedLocalBoundsCenter - InstanceData.PreSkinnedLocalBoundsExtents, InstanceData.PreSkinnedLocalBoundsCenter + InstanceData.PreSkinnedLocalBoundsExtents);
	const FBox3f WorldBounds = MeshBounds.TransformBy(InstanceData.Transform);

	ShaderParameters->PreSkinnedLocalBoundsCenter = InstanceData.PreSkinnedLocalBoundsCenter;
	ShaderParameters->PreSkinnedLocalBoundsExtents = InstanceData.PreSkinnedLocalBoundsExtents;
	ShaderParameters->MeshBoundsWSCenter = WorldBounds.GetCenter();
	ShaderParameters->MeshBoundsWSExtents = WorldBounds.GetExtent();
	ShaderParameters->OwnerToMeshVector = InstanceData.OwnerToMeshVector;

	if (InstanceData.UvMappingBuffer)
	{
		ShaderParameters->UvMappingBuffer = InstanceData.UvMappingBuffer->GetSrv();
		ShaderParameters->UvMappingBufferLength = InstanceData.UvMappingBuffer->GetBufferSize();
		ShaderParameters->UvMappingSet = InstanceData.UvMappingSet;
	}
	else
	{
		ShaderParameters->UvMappingBuffer = FNiagaraRenderer::GetDummyIntBuffer();
		ShaderParameters->UvMappingBufferLength = 0;
		ShaderParameters->UvMappingSet = 0;
	}
}

bool UNiagaraDataInterfaceStaticMesh::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceStaticMesh* OtherTyped = CastChecked<const UNiagaraDataInterfaceStaticMesh>(Other);
	return
		OtherTyped->SourceMode == SourceMode &&
#if WITH_EDITORONLY_DATA
		OtherTyped->PreviewMesh == PreviewMesh &&
#endif
		OtherTyped->DefaultMesh == DefaultMesh &&
		OtherTyped->SoftSourceActor == SoftSourceActor &&
		OtherTyped->SourceComponent == SourceComponent &&
		OtherTyped->SectionFilter.AllowedMaterialSlots == SectionFilter.AllowedMaterialSlots &&
		OtherTyped->bCaptureTransformsPerFrame == bCaptureTransformsPerFrame &&
		OtherTyped->bUsePhysicsBodyVelocity == bUsePhysicsBodyVelocity &&
		OtherTyped->bAllowSamplingFromStreamingLODs == bAllowSamplingFromStreamingLODs &&
		OtherTyped->LODIndex == LODIndex &&
		OtherTyped->LODIndexUserParameter == LODIndexUserParameter &&
		OtherTyped->MeshParameterBinding == MeshParameterBinding &&
		OtherTyped->InstanceIndex == InstanceIndex &&
		OtherTyped->FilteredSockets == FilteredSockets;
}

bool UNiagaraDataInterfaceStaticMesh::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceStaticMesh* OtherTyped = CastChecked<UNiagaraDataInterfaceStaticMesh>(Destination);
	OtherTyped->UnbindSourceDelegates();
	OtherTyped->SourceMode = SourceMode;
#if WITH_EDITORONLY_DATA
	OtherTyped->PreviewMesh = PreviewMesh;
#endif
	OtherTyped->DefaultMesh = DefaultMesh;
	OtherTyped->SoftSourceActor = SoftSourceActor;
	OtherTyped->SourceComponent = SourceComponent;
	OtherTyped->SectionFilter = SectionFilter;
	OtherTyped->bCaptureTransformsPerFrame = bCaptureTransformsPerFrame;
	OtherTyped->bUsePhysicsBodyVelocity = bUsePhysicsBodyVelocity;
	OtherTyped->FilteredSockets = FilteredSockets;
	OtherTyped->bAllowSamplingFromStreamingLODs = bAllowSamplingFromStreamingLODs;
	OtherTyped->LODIndex = LODIndex;
	OtherTyped->LODIndexUserParameter = LODIndexUserParameter;
	OtherTyped->MeshParameterBinding = MeshParameterBinding;
	OtherTyped->InstanceIndex = InstanceIndex;
	OtherTyped->BindSourceDelegates();
	return true;
}

#if WITH_NIAGARA_DEBUGGER
void UNiagaraDataInterfaceStaticMesh::DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const
{
	const NDIStaticMeshLocal::FInstanceData_GameThread* InstanceData_GT = DebugHudContext.GetSystemInstance()->FindTypedDataInterfaceInstanceData<NDIStaticMeshLocal::FInstanceData_GameThread>(this);
	if (InstanceData_GT == nullptr)
	{
		return;
	}

	UObject* Component = InstanceData_GT->ComponentWeakPtr.Get();
	UStaticMesh* SkeletalMesh = InstanceData_GT->StaticMeshWeakPtr.Get();
	DebugHudContext.GetOutputString().Appendf(TEXT("StaticMesh(%s) StaticMeshComp(%s)"), *GetNameSafe(SkeletalMesh), *GetNameSafe(Component));
}
#endif

#if WITH_EDITOR
void UNiagaraDataInterfaceStaticMesh::GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	if (Asset == nullptr)
	{
		return;
	}

	AActor* SourceActor = SoftSourceActor.Get();
	UStaticMesh* CurrentMesh = DefaultMesh;

	bool bHasNoMeshAssignedWarning = (SourceActor == nullptr && DefaultMesh == nullptr);
	if (bHasNoMeshAssignedWarning)
	{
		CurrentMesh = PreviewMesh.LoadSynchronous();
		bHasNoMeshAssignedWarning = CurrentMesh == nullptr;
	}

	INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");
	const INiagaraEditorOnlyDataUtilities& EditorOnlyDataUtilities = NiagaraModule.GetEditorOnlyDataUtilities();
	UNiagaraDataInterface* RuntimeInstanceOfThis = EditorOnlyDataUtilities.IsEditorDataInterfaceInstance(this)
		? EditorOnlyDataUtilities.GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(*Asset, *this)
		: this;

	if (CurrentMesh != nullptr)
	{
		if (CurrentMesh->bAllowCPUAccess == false)
		{
			bool bNeedsCpuAccess = false;

			FNiagaraDataInterfaceUtilities::ForEachVMFunction(
				RuntimeInstanceOfThis,
				Asset,
				[&bNeedsCpuAccess](const UNiagaraScript* Script, const FVMExternalFunctionBindingInfo& VMFunction) -> bool
				{
					bNeedsCpuAccess |= FunctionNeedsCpuAccess(VMFunction.Name);
					return bNeedsCpuAccess == false;
				}
			);

			FNiagaraDataInterfaceUtilities::ForEachGpuFunction(
				RuntimeInstanceOfThis,
				Asset,
				[&bNeedsCpuAccess](const UNiagaraScript* Script,const FNiagaraDataInterfaceGeneratedFunction& GpuFunction) -> bool
				{
					bNeedsCpuAccess |= FunctionNeedsCpuAccess(GpuFunction.DefinitionName);
					return bNeedsCpuAccess == false;
				}
			);

			if (bNeedsCpuAccess)
			{
				OutErrors.Emplace(
					FText::Format(LOCTEXT("CPUAccessNotAllowedError", "This mesh needs CPU access in order to be used properly.({0})"), FText::FromString(CurrentMesh->GetName())),
					LOCTEXT("CPUAccessNotAllowedErrorSummary", "CPU access error"),
					FNiagaraDataInterfaceFix::CreateLambda(
						[=]()
						{
							CurrentMesh->Modify();
							CurrentMesh->bAllowCPUAccess = true;
							return true;
						}
					)
				);
			}
		}
	}

	if (bHasNoMeshAssignedWarning)
	{
		FNiagaraDataInterfaceFeedback NoMeshAssignedError(LOCTEXT("NoMeshAssignedError", "This Data Interface should be assigned a static mesh to operate correctly."),
			LOCTEXT("NoMeshAssignedErrorSummary", "No mesh assigned warning"),
			FNiagaraDataInterfaceFix());

		OutWarnings.Add(NoMeshAssignedError);
	}

	if (SourceMode == ENDIStaticMesh_SourceMode::Default && DefaultMesh != nullptr)
	{
		OutWarnings.Emplace(
			LOCTEXT("DefaultSourceAndDefaultMesh", "Default Mesh is set but Source Mode is set to Default.  Verify you have not left it set incorrectly to avoid unused meshes being referenced."),
			LOCTEXT("DefaultSourceAndDefaultMeshSummary", "Potentially unused Default Mesh set!"),
			FNiagaraDataInterfaceFix::CreateLambda(
				[this]()
				{
					FScopedTransaction ST(LOCTEXT("ClearDefaultMesh", "Clear Default Mesh"));
					Modify();
					DefaultMesh = nullptr;
					return true;
				}
			)
		);
	}

	if (GetDefault<UNiagaraSettings>()->NDIStaticMesh_AllowDistanceFields == false)
	{
		auto GenerateWarnings = 
			[&](const UNiagaraScript* Script, const FNiagaraDataInterfaceGeneratedFunction& FunctionBinding)
			{
				if (FunctionBinding.DefinitionName == NDIStaticMeshLocal::QueryDistanceFieldName )
				{
					OutWarnings.Emplace(
						LOCTEXT("DistanceFieldsNotEnabled", "Distance fields functionality is disabled, this can be enabled in Niagara project settings."),
						LOCTEXT("DistanceFieldsNotEnabledSummary", "Distance fields functionality is disabled."),
						FNiagaraDataInterfaceFix()
					);

					return false;
				}
				return true;
			};

		if (Component != nullptr)
		{
			FNiagaraDataInterfaceUtilities::ForEachGpuFunction(RuntimeInstanceOfThis, Component, GenerateWarnings);
		}
		else if (Asset != nullptr)
		{
			FNiagaraDataInterfaceUtilities::ForEachGpuFunction(RuntimeInstanceOfThis, Asset, GenerateWarnings);
		}
	}
}
#endif //WITH_EDITOR

UStaticMesh* UNiagaraDataInterfaceStaticMesh::ResolveStaticMesh(UObject*& OutComponent, FNiagaraSystemInstance* SystemInstance, UObject* ParameterBindingValue) const
{
	const bool bTrySource = SourceMode == ENDIStaticMesh_SourceMode::Default || SourceMode == ENDIStaticMesh_SourceMode::Source;
	const bool bTryMeshParameter = SourceMode == ENDIStaticMesh_SourceMode::Default || SourceMode == ENDIStaticMesh_SourceMode::MeshParameterBinding;
	const bool bTryAttachParent = SourceMode == ENDIStaticMesh_SourceMode::Default || SourceMode == ENDIStaticMesh_SourceMode::AttachParent;
	const bool bTryDefaultMesh = SourceMode == ENDIStaticMesh_SourceMode::Default || SourceMode == ENDIStaticMesh_SourceMode::DefaultMeshOnly;

	UStaticMesh* OutMesh = nullptr;
	if (bTrySource && SourceComponent.IsValid())
	{
		OutComponent = SourceComponent.Get();
		OutMesh = SourceComponent->GetStaticMesh();
	}
	else if (bTrySource && SoftSourceActor.Get())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const INiagaraSceneComponentUtils& SceneComponentUtils = FNiagaraSystemInstance::GetSceneComponentUtils(SystemInstance);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		SceneComponentUtils.ResolveStaticMesh(SoftSourceActor.Get(), false, OutComponent, OutMesh);
	}
	else
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const INiagaraSceneComponentUtils& SceneComponentUtils = FNiagaraSystemInstance::GetSceneComponentUtils(SystemInstance);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Try parameter binding, this could either be a component or a static mesh
		if (bTryMeshParameter && ParameterBindingValue)
		{
			SceneComponentUtils.ResolveStaticMesh(ParameterBindingValue, true, OutComponent, OutMesh);
		}

		// Try attached parent, only if we did not already find a valid component / mesh
		if (bTryAttachParent && SystemInstance && !OutComponent && !OutMesh)
		{
			SceneComponentUtils.ResolveStaticMesh(true, OutComponent, OutMesh);
		}
	}

	if (bTryDefaultMesh && !OutComponent && !OutMesh)
	{
		OutMesh = DefaultMesh;
	}

#if WITH_EDITORONLY_DATA
	if (!OutMesh && !OutComponent && (!SystemInstance || !SystemInstance->GetWorld()->IsGameWorld()))
	{
		// NOTE: We don't fall back on the preview mesh if we have a valid static mesh component referenced
		OutMesh = PreviewMesh.LoadSynchronous();		
	}
#endif

	return OutMesh;
}

void UNiagaraDataInterfaceStaticMesh::SetSourceComponentFromBlueprints(UStaticMeshComponent* ComponentToUse)
{
	// NOTE: When ChangeId changes the next tick will be skipped and a reset of the per-instance data will be initiated. 
	++ChangeId;
	UnbindSourceDelegates();
	SourceComponent = ComponentToUse;
	SoftSourceActor = ComponentToUse ? ComponentToUse->GetOwner() : nullptr;
	BindSourceDelegates();
}

void UNiagaraDataInterfaceStaticMesh::SetNiagaraStaticMeshDIInstanceIndex(UNiagaraComponent* NiagaraSystem, const FName UserParameterName, int32 NewInstanceIndex)
{
	if (UNiagaraDataInterfaceStaticMesh* DataInterface = UNiagaraFunctionLibrary::GetDataInterface<UNiagaraDataInterfaceStaticMesh>(NiagaraSystem, UserParameterName))
	{
		if (DataInterface->InstanceIndex != NewInstanceIndex)
		{
			++DataInterface->ChangeId;
			DataInterface->InstanceIndex = NewInstanceIndex;
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::BindSourceDelegates()
{
	if (AActor* Source = SoftSourceActor.Get())
	{
		Source->OnEndPlay.AddDynamic(this, &UNiagaraDataInterfaceStaticMesh::OnSourceEndPlay);
	}
	else if (SourceComponent.IsValid())
	{
		UE_CLOGF(!UObjectBaseUtility::IsGarbageEliminationEnabled(),
			LogNiagara, Warning, "%ls: Unable to bind OnEndPlay for actor-less source component %ls, this may extend the lifetime of the component",
			*GetFullName(), *SourceComponent->GetPathName());
	}
}

void UNiagaraDataInterfaceStaticMesh::UnbindSourceDelegates()
{
	if (AActor* Source = SoftSourceActor.Get())
	{
		Source->OnEndPlay.RemoveAll(this);
	}
}

void UNiagaraDataInterfaceStaticMesh::OnSourceEndPlay(AActor* InSource, EEndPlayReason::Type Reason)
{
	// Increment change id in case we're able to find a new source component 
	++ChangeId;
	UnbindSourceDelegates();
	SoftSourceActor = nullptr;
	SourceComponent = nullptr;
}

void UNiagaraDataInterfaceStaticMesh::SetDefaultMeshFromBlueprints(UStaticMesh* MeshToUse)
{
	// NOTE: When ChangeId changes the next tick will be skipped and a reset of the per-instance data will be initiated. 
	++ChangeId;
	UnbindSourceDelegates();
	SourceComponent = nullptr;
	SoftSourceActor = nullptr;
	DefaultMesh = MeshToUse;
}

//////////////////////////////////////////////////////////////////////////
// Misc Functions
void UNiagaraDataInterfaceStaticMesh::VMGetPreSkinnedLocalBounds(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIOutputParam<FVector3f>	OutCenter(Context);
	FNDIOutputParam<FVector3f>	OutExtentsMin(Context);
	FNDIOutputParam<FVector3f>	OutExtentsMax(Context);
	FNDIOutputParam<FVector3f>	OutExtents(Context);
	FNDIOutputParam<FVector3f>	OutHalfExtents(Context);

	const FVector3f Center = StaticMeshHelper.InstanceData->PreSkinnedLocalBoundsCenter;
	const FVector3f HalfExtents = StaticMeshHelper.InstanceData->PreSkinnedLocalBoundsExtents;
	const FVector3f Extents = HalfExtents * 2.0f;
	const FVector3f ExtentsMin = Center - HalfExtents;
	const FVector3f ExtentsMax = Center + HalfExtents;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCenter.SetAndAdvance(Center);
		OutExtentsMin.SetAndAdvance(ExtentsMin);
		OutExtentsMax.SetAndAdvance(ExtentsMax);
		OutExtents.SetAndAdvance(Extents);
		OutHalfExtents.SetAndAdvance(HalfExtents);
	}
}

template<bool bLocalSpace>
void UNiagaraDataInterfaceStaticMesh::VMGetMeshBounds(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIOutputParam<FVector3f>	OutCenter(Context);
	FNDIOutputParam<FVector3f>	OutExtentsMin(Context);
	FNDIOutputParam<FVector3f>	OutExtentsMax(Context);
	FNDIOutputParam<FVector3f>	OutExtents(Context);
	FNDIOutputParam<FVector3f>	OutHalfExtents(Context);
	FNDIOutputParam<FVector3f>	OutOwnerToMeshVector(Context);

	const FBox MeshBounds(StaticMeshHelper.InstanceData->PreSkinnedLocalBoundsCenter - StaticMeshHelper.InstanceData->PreSkinnedLocalBoundsExtents, StaticMeshHelper.InstanceData->PreSkinnedLocalBoundsCenter + StaticMeshHelper.InstanceData->PreSkinnedLocalBoundsExtents);
	const FBox WorldBounds = MeshBounds.TransformBy(StaticMeshHelper.InstanceData->Transform);

	const FVector3f Center = bLocalSpace ? FVector3f(MeshBounds.GetCenter()) : FVector3f(WorldBounds.GetCenter());
	const FVector3f HalfExtents = bLocalSpace ? FVector3f(MeshBounds.GetExtent()) : FVector3f(WorldBounds.GetExtent());
	const FVector3f Extents = HalfExtents * 2.0f;
	const FVector3f ExtentsMin = Center - HalfExtents;
	const FVector3f ExtentsMax = Center + HalfExtents;
	const FVector3f OwnerToMeshVector = StaticMeshHelper.InstanceData->OwnerToMeshVector;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCenter.SetAndAdvance(Center);
		OutExtentsMin.SetAndAdvance(ExtentsMin);
		OutExtentsMax.SetAndAdvance(ExtentsMax);
		OutExtents.SetAndAdvance(Extents);
		OutHalfExtents.SetAndAdvance(HalfExtents);
		OutOwnerToMeshVector.SetAndAdvance(OwnerToMeshVector);
	}
}

//////////////////////////////////////////////////////////////////////////
// VM Vertex Sampling
void UNiagaraDataInterfaceStaticMesh::VMIsValidVertex(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> VertexParam(Context);
	FNDIOutputParam<bool> OutValid(Context);

	const int32 NumVertices = StaticMeshHelper.GetNumPositionVertices();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 Vertex = VertexParam.GetAndAdvance();
		OutValid.SetAndAdvance(Vertex >= 0 && Vertex < NumVertices);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMRandomVertex(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIRandomHelper RandHelper(Context);
	FNDIOutputParam<int32> OutVertex(Context);

	const int32 VertexMax = FMath::Max(StaticMeshHelper.GetNumPositionVertices() - 1, 0);
	for (int32 i=0; i < Context.GetNumInstances(); ++i)
	{
		RandHelper.GetAndAdvance();
		const int32 Vertex = RandHelper.RandRange(i, 0, VertexMax);
		OutVertex.SetAndAdvance(Vertex);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetVertexCount(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIOutputParam<int32> OutNumVertices(Context);

	const int32 NumVertices = StaticMeshHelper.GetNumPositionVertices();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutNumVertices.SetAndAdvance(NumVertices);
	}
}

//void UNiagaraDataInterfaceStaticMesh::VMIsValidFilteredVertex(FVectorVMExternalFunctionContext& Context)
//{
//}

//void UNiagaraDataInterfaceStaticMesh::VMRandomFilteredVertex(FVectorVMExternalFunctionContext& Context)
//{
//}

//void UNiagaraDataInterfaceStaticMesh::VMGetFilteredVertexCount(FVectorVMExternalFunctionContext& Context)
//{
//}
//
//void UNiagaraDataInterfaceStaticMesh::VMGetFilteredVertexAt(FVectorVMExternalFunctionContext& Context)
//{
//}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetVertex(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> VertexParam(Context);
	FNDIOutputParam<FVector3f> OutPosition(Context);
	FNDIOutputParam<FVector3f> OutVelocity(Context);
	FNDIOutputParam<FVector3f> OutNormal(Context);
	FNDIOutputParam<FVector3f> OutBitangent(Context);
	FNDIOutputParam<FVector3f> OutTangent(Context);

	const int32 PositionsMax = StaticMeshHelper.GetNumPositionVertices() - 1;
	const int32 TangentsMax = StaticMeshHelper.GetNumTangentVertices() - 1;
	if ( (PositionsMax >= 0) && (PositionsMax == TangentsMax) )
	{
		const float InvDt = StaticMeshHelper.GetInvDeltaSeconds();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Vertex = FMath::Clamp(VertexParam.GetAndAdvance(), 0, PositionsMax);
			const FVector3f LocalPosition = StaticMeshHelper.GetLocalPosition(Vertex);
			const FVector3f Position = StaticMeshHelper.TransformPosition(LocalPosition);
			const FVector3f PreviousPosition = StaticMeshHelper.PreviousTransformPosition(LocalPosition);
			const FVector3f Velocity = (Position - PreviousPosition) * InvDt;
			const NDIStaticMeshLocal::FNDITangentBasis TangentBasis = StaticMeshHelper.GetTangentBasis(Vertex);
			OutPosition.SetAndAdvance(Position);
			OutVelocity.SetAndAdvance(Velocity);
			OutNormal.SetAndAdvance(TangentBasis.TangentZ);
			OutBitangent.SetAndAdvance(TangentBasis.TangentY);
			OutTangent.SetAndAdvance(TangentBasis.TangentX);
		}
	}
	else
	{
		const FVector3f Position = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		const FVector3f Velocity = FVector3f::ZeroVector;
		const FVector3f TangentX = StaticMeshHelper.TransformUnitVector(FVector3f(1.0f, 0.0f, 0.0f));
		const FVector3f TangentY = StaticMeshHelper.TransformUnitVector(FVector3f(0.0f, 1.0f, 0.0f));
		const FVector3f TangentZ = StaticMeshHelper.TransformUnitVector(FVector3f(0.0f, 0.0f, 1.0f));
		for ( int32 i=0; i < Context.GetNumInstances(); ++i )
		{
			OutPosition.SetAndAdvance(Position);
			OutVelocity.SetAndAdvance(Velocity);
			OutNormal.SetAndAdvance(TangentZ);
			OutBitangent.SetAndAdvance(TangentY);
			OutTangent.SetAndAdvance(TangentX);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetVertexInterpolated(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> VertexParam(Context);
	FNDIInputParam<float> InterpParam(Context);
	FNDIOutputParam<FVector3f> OutPosition(Context);
	FNDIOutputParam<FVector3f> OutVelocity(Context);
	FNDIOutputParam<FVector3f> OutNormal(Context);
	FNDIOutputParam<FVector3f> OutBitangent(Context);
	FNDIOutputParam<FVector3f> OutTangent(Context);

	const int32 PositionsMax = StaticMeshHelper.GetNumPositionVertices() - 1;
	const int32 TangentsMax = StaticMeshHelper.GetNumTangentVertices() - 1;
	if ((PositionsMax >= 0) && (PositionsMax == TangentsMax))
	{
		const float InvDt = StaticMeshHelper.GetInvDeltaSeconds();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const float Interp = InterpParam.GetAndAdvance();
			const int32 Vertex = FMath::Clamp(VertexParam.GetAndAdvance(), 0, PositionsMax);
			const FVector3f LocalPosition = StaticMeshHelper.GetLocalPosition(Vertex);
			const FVector3f Position = StaticMeshHelper.TransformPosition(LocalPosition);
			const FVector3f PreviousPosition = StaticMeshHelper.PreviousTransformPosition(LocalPosition);
			const FVector3f Velocity = (Position - PreviousPosition) * InvDt;
			const NDIStaticMeshLocal::FNDITangentBasis TangentBasis = StaticMeshHelper.GetTangentBasisInterpolated(Vertex, Interp);
			OutPosition.SetAndAdvance(FMath::Lerp(PreviousPosition, Position, Interp));
			OutVelocity.SetAndAdvance(Velocity);
			OutNormal.SetAndAdvance(TangentBasis.TangentZ);
			OutBitangent.SetAndAdvance(TangentBasis.TangentY);
			OutTangent.SetAndAdvance(TangentBasis.TangentX);
		}
	}
	else
	{
		const FVector3f Position = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		const FVector3f Velocity = FVector3f::ZeroVector;
		const FVector3f TangentX = StaticMeshHelper.TransformUnitVector(FVector3f(1.0f, 0.0f, 0.0f));
		const FVector3f TangentY = StaticMeshHelper.TransformUnitVector(FVector3f(0.0f, 1.0f, 0.0f));
		const FVector3f TangentZ = StaticMeshHelper.TransformUnitVector(FVector3f(0.0f, 0.0f, 1.0f));
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutPosition.SetAndAdvance(Position);
			OutVelocity.SetAndAdvance(Velocity);
			OutNormal.SetAndAdvance(TangentZ);
			OutBitangent.SetAndAdvance(TangentY);
			OutTangent.SetAndAdvance(TangentX);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetVertexColor(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper StaticMeshHelper(Context);
	FNDIInputParam<int32> VertexParam(Context);
	FNDIOutputParam<FLinearColor> OutColor(Context);

	const int32 ColorsMax = StaticMeshHelper.GetNumColorVertices() - 1;
	if ( ColorsMax >= 0 )
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Vertex = VertexParam.GetAndAdvance();
			OutColor.SetAndAdvance(StaticMeshHelper.GetColor(FMath::Clamp(Vertex, 0, ColorsMax)));
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutColor.SetAndAdvance(FLinearColor::White);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetVertexUV(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper StaticMeshHelper(Context);
	FNDIInputParam<int32> VertexParam(Context);
	FNDIInputParam<int32> UVSetParam(Context);
	FNDIOutputParam<FVector2f> OutUV(Context);

	const int32 VertexMax = StaticMeshHelper.GetNumUVVertices() - 1;
	const int32 UVSetMax = StaticMeshHelper.GetNumUVs() - 1;
	if (VertexMax >= 0 && UVSetMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Vertex = VertexParam.GetAndAdvance();
			const int32 UVSet = UVSetParam.GetAndAdvance();
			const FVector2f UV = FVector2f(StaticMeshHelper.GetUV(FMath::Clamp(Vertex, 0, VertexMax), FMath::Clamp(UVSet, 0, UVSetMax)));	// LWC_TODO: Precision loss
			OutUV.SetAndAdvance(UV);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutUV.SetAndAdvance(FVector2f::ZeroVector);
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// VM Triangle Sampling
void UNiagaraDataInterfaceStaticMesh::VMIsValidTriangle(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIOutputParam<bool> OutValid(Context);

	const int32 NumTriangles = StaticMeshHelper.GetNumTriangles();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 Triangle = TriangleParam.GetAndAdvance();
		OutValid.SetAndAdvance(Triangle >= 0 && Triangle < NumTriangles);
	}
}

template<typename TRandomHelper>
void UNiagaraDataInterfaceStaticMesh::VMRandomTriangle(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	TRandomHelper RandHelper(Context);
	FNDIOutputParam<int32> OutTriangleParam(Context);
	FNDIOutputParam<FVector3f> OutBaryCoordParam(Context);

	if ( StaticMeshHelper.IsCpuUniformlyDistributedSampling() )
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutTriangleParam.SetAndAdvance(StaticMeshHelper.RandomUniformTriangle(RandHelper, i));
			OutBaryCoordParam.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
		}
	}
	else
	{
		const int32 NumTriangles = StaticMeshHelper.GetNumTriangles();
		const int32 TriangleMax = FMath::Max(NumTriangles - 1, 0);
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutTriangleParam.SetAndAdvance(RandHelper.RandRange(i, 0, TriangleMax));
			OutBaryCoordParam.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetTriangleCount(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIOutputParam<int32> OutCountParam(Context);

	const int32 NumTriangles = StaticMeshHelper.GetNumTriangles();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCountParam.SetAndAdvance(NumTriangles);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMIsValidFilteredTriangle(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIOutputParam<bool> OutValid(Context);

	const int32 NumTriangles = StaticMeshHelper.GetNumFilteredTriangles();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 Triangle = TriangleParam.GetAndAdvance();
		OutValid.SetAndAdvance(Triangle >= 0 && Triangle < NumTriangles);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMRandomFilteredTriangle(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIRandomHelper RandHelper(Context);
	FNDIOutputParam<int32> OutTriangleParam(Context);
	FNDIOutputParam<FVector3f> OutBaryCoordParam(Context);

	TConstArrayView<NDIStaticMeshLocal::FNDISectionInfo> SectionInfos = StaticMeshHelper.InstanceData->GetFilteredSectionInfos();
	if (SectionInfos.Num() )
	{
		if ( StaticMeshHelper.IsCpuUniformlyDistributedSampling() )
		{
			TConstArrayView<int32> SectionRemap = StaticMeshHelper.InstanceData->GetFilteredSections();
			for (int32 i=0; i < Context.GetNumInstances(); ++i)
			{
				RandHelper.GetAndAdvance();
				OutTriangleParam.SetAndAdvance(StaticMeshHelper.RandomUniformTriangle(RandHelper, i, SectionInfos, SectionRemap));
				OutBaryCoordParam.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
			}
		}
		else
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				RandHelper.GetAndAdvance();
				OutTriangleParam.SetAndAdvance(StaticMeshHelper.RandomTriangle(RandHelper, i, SectionInfos));
				OutBaryCoordParam.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
			}
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutTriangleParam.SetAndAdvance(0);
			OutBaryCoordParam.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetFilteredTriangleCount(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIOutputParam<int32> OutCount(Context);

	const int32 NumTriangles = StaticMeshHelper.GetNumFilteredTriangles();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCount.SetAndAdvance(NumTriangles);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetFilteredTriangleAt(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> IndexParam(Context);
	FNDIOutputParam<int32> OutTriangle(Context);

	const int32 TriangleMax = StaticMeshHelper.GetNumFilteredTriangles() - 1;
	if ( TriangleMax > 0 )
	{
		TConstArrayView<NDIStaticMeshLocal::FNDISectionInfo> FilteredSections = StaticMeshHelper.InstanceData->GetFilteredSectionInfos();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Index = FMath::Clamp(IndexParam.GetAndAdvance(), 0, TriangleMax);
			const int32 Triangle = StaticMeshHelper.SectionTriangleToTriangle(FilteredSections, Index);
			OutTriangle.SetAndAdvance(Triangle);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTriangle.SetAndAdvance(0);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMIsValidUnfilteredTriangle(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIOutputParam<bool> OutValid(Context);

	const int32 NumTriangles = StaticMeshHelper.GetNumUnfilteredTriangles();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 Triangle = TriangleParam.GetAndAdvance();
		OutValid.SetAndAdvance(Triangle >= 0 && Triangle < NumTriangles);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMRandomUnfilteredTriangle(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIRandomHelper RandHelper(Context);
	FNDIOutputParam<int32> OutTriangleParam(Context);
	FNDIOutputParam<FVector3f> OutBaryCoordParam(Context);

	TConstArrayView<NDIStaticMeshLocal::FNDISectionInfo> SectionInfos = StaticMeshHelper.InstanceData->GetUnfilteredSectionInfos();
	if (SectionInfos.Num() )
	{
		if ( StaticMeshHelper.IsCpuUniformlyDistributedSampling() )
		{
			TConstArrayView<int32> SectionRemap = StaticMeshHelper.InstanceData->GetUnfilteredSections();
			for (int32 i=0; i < Context.GetNumInstances(); ++i)
			{
				RandHelper.GetAndAdvance();
				OutTriangleParam.SetAndAdvance(StaticMeshHelper.RandomUniformTriangle(RandHelper, i, SectionInfos, SectionRemap));
				OutBaryCoordParam.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
			}
		}
		else
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				RandHelper.GetAndAdvance();
				OutTriangleParam.SetAndAdvance(StaticMeshHelper.RandomTriangle(RandHelper, i, SectionInfos));
				OutBaryCoordParam.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
			}
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutTriangleParam.SetAndAdvance(0);
			OutBaryCoordParam.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredTriangleCount(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIOutputParam<int32> OutCount(Context);

	const int32 NumTriangles = StaticMeshHelper.GetNumUnfilteredTriangles();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCount.SetAndAdvance(NumTriangles);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredTriangleAt(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> IndexParam(Context);
	FNDIOutputParam<int32> OutTriangle(Context);

	const int32 TriangleMax = StaticMeshHelper.GetNumUnfilteredTriangles() - 1;
	if ( TriangleMax > 0 )
	{
		TConstArrayView<NDIStaticMeshLocal::FNDISectionInfo> UnfilteredSections = StaticMeshHelper.InstanceData->GetUnfilteredSectionInfos();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Index = FMath::Clamp(IndexParam.GetAndAdvance(), 0, TriangleMax);
			const int32 Triangle = StaticMeshHelper.SectionTriangleToTriangle(UnfilteredSections, Index);
			OutTriangle.SetAndAdvance(Triangle);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTriangle.SetAndAdvance(0);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetTriangle(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIInputParam<FVector3f> BaryCoordParam(Context);
	FNDIOutputParam<FVector3f> OutPositionParam(Context);
	FNDIOutputParam<FVector3f> OutVelocityParam(Context);
	FNDIOutputParam<FVector3f> OutNormalParam(Context);
	FNDIOutputParam<FVector3f> OutBitangentParam(Context);
	FNDIOutputParam<FVector3f> OutTangentParam(Context);

	const FIndexArrayView IndexArray = StaticMeshHelper.GetIndexArrayView();
	const int32 TriangleMax = StaticMeshHelper.GetNumTriangles() - 1;
	if (TriangleMax >= 0 && StaticMeshHelper.GetNumPositionVertices() > 0 && StaticMeshHelper.GetNumTangentVertices() > 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Triangle = FMath::Clamp(TriangleParam.GetAndAdvance(), 0, TriangleMax);
			const FVector3f BaryCoord = BaryCoordParam.GetAndAdvance();
			const FVector3f LocalPosition = StaticMeshHelper.GetLocalTrianglePosition(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			const FVector3f Position = StaticMeshHelper.TransformPosition(LocalPosition);
			const FVector3f PreviousPosition = StaticMeshHelper.PreviousTransformPosition(LocalPosition);
			const FVector3f Velocity = (Position - PreviousPosition) * StaticMeshHelper.GetInvDeltaSeconds();
			const FVector3f TangentX = StaticMeshHelper.GetTriangleTangentX(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			const FVector3f TangentY = StaticMeshHelper.GetTriangleTangentY(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			const FVector3f TangentZ = StaticMeshHelper.GetTriangleTangentZ(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			OutPositionParam.SetAndAdvance(Position);
			OutVelocityParam.SetAndAdvance(Velocity);
			OutNormalParam.SetAndAdvance(TangentZ);
			OutBitangentParam.SetAndAdvance(TangentY);
			OutTangentParam.SetAndAdvance(TangentX);
		}
	}
	else
	{
		const FVector3f Position = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		const FVector3f PreviousPosition = StaticMeshHelper.PreviousTransformPosition(FVector3f::ZeroVector);
		const FVector3f Velocity = FVector3f::ZeroVector;
		const FVector3f TangentX = StaticMeshHelper.TransformUnitVector(FVector3f(1.0f, 0.0f, 0.0f));
		const FVector3f TangentY = StaticMeshHelper.TransformUnitVector(FVector3f(0.0f, 1.0f, 0.0f));
		const FVector3f TangentZ = StaticMeshHelper.TransformUnitVector(FVector3f(0.0f, 0.0f, 1.0f));
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutPositionParam.SetAndAdvance(Position);
			OutVelocityParam.SetAndAdvance(Velocity);
			OutNormalParam.SetAndAdvance(TangentZ);
			OutBitangentParam.SetAndAdvance(TangentY);
			OutTangentParam.SetAndAdvance(TangentX);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetTriangleInterpolated(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIInputParam<FVector3f> BaryCoordParam(Context);
	FNDIInputParam<float> InterpParam(Context);
	FNDIOutputParam<FVector3f> OutPositionParam(Context);
	FNDIOutputParam<FVector3f> OutVelocityParam(Context);
	FNDIOutputParam<FVector3f> OutNormalParam(Context);
	FNDIOutputParam<FVector3f> OutBitangentParam(Context);
	FNDIOutputParam<FVector3f> OutTangentParam(Context);

	const FIndexArrayView IndexArray = StaticMeshHelper.GetIndexArrayView();
	const int32 TriangleMax = StaticMeshHelper.GetNumTriangles() - 1;
	if (TriangleMax >= 0 && StaticMeshHelper.GetNumPositionVertices() > 0 && StaticMeshHelper.GetNumTangentVertices() > 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Triangle = FMath::Clamp(TriangleParam.GetAndAdvance(), 0, TriangleMax);
			const FVector3f BaryCoord = BaryCoordParam.GetAndAdvance();
			const float Interp = InterpParam.GetAndAdvance();
			const FVector3f LocalPosition = StaticMeshHelper.GetLocalTrianglePosition(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			const FVector3f Position = StaticMeshHelper.TransformPosition(LocalPosition);
			const FVector3f PreviousPosition = StaticMeshHelper.PreviousTransformPosition(LocalPosition);
			const FVector3f Velocity = (Position - PreviousPosition) * StaticMeshHelper.GetInvDeltaSeconds();
			const FVector3f TangentX = StaticMeshHelper.GetTriangleTangentXInterpolated(BaryCoord, Interp, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			const FVector3f TangentY = StaticMeshHelper.GetTriangleTangentYInterpolated(BaryCoord, Interp, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			const FVector3f TangentZ = StaticMeshHelper.GetTriangleTangentZInterpolated(BaryCoord, Interp, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			OutPositionParam.SetAndAdvance(FMath::Lerp(PreviousPosition, Position, Interp));
			OutVelocityParam.SetAndAdvance(Velocity);
			OutNormalParam.SetAndAdvance(TangentZ);
			OutBitangentParam.SetAndAdvance(TangentY);
			OutTangentParam.SetAndAdvance(TangentX);
		}
	}
	else
	{
		const FVector3f Position = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		const FVector3f PreviousPosition = StaticMeshHelper.PreviousTransformPosition(FVector3f::ZeroVector);
		const FVector3f Velocity = FVector3f::ZeroVector;
		const FVector3f TangentX = StaticMeshHelper.TransformUnitVector(FVector3f(1.0f, 0.0f, 0.0f));
		const FVector3f TangentY = StaticMeshHelper.TransformUnitVector(FVector3f(0.0f, 1.0f, 0.0f));
		const FVector3f TangentZ = StaticMeshHelper.TransformUnitVector(FVector3f(0.0f, 0.0f, 1.0f));
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutPositionParam.SetAndAdvance(Position);
			OutVelocityParam.SetAndAdvance(Velocity);
			OutNormalParam.SetAndAdvance(TangentZ);
			OutBitangentParam.SetAndAdvance(TangentY);
			OutTangentParam.SetAndAdvance(TangentX);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetTriangleColor(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIInputParam<FVector3f> BaryCoordParam(Context);
	FNDIOutputParam<FLinearColor> OutColorParam(Context);

	const FIndexArrayView IndexArray = StaticMeshHelper.GetIndexArrayView();
	const int32 TriangleMax = StaticMeshHelper.GetNumTriangles() - 1;
	if (TriangleMax >= 0 && StaticMeshHelper.GetNumColorVertices() > 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Triangle = FMath::Clamp(TriangleParam.GetAndAdvance(), 0, TriangleMax);
			const FVector3f BaryCoord = BaryCoordParam.GetAndAdvance();
			const FLinearColor Color = StaticMeshHelper.GetTriangleColor(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			OutColorParam.SetAndAdvance(Color);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutColorParam.SetAndAdvance(FLinearColor::White);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetTriangleUV(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIInputParam<FVector3f> BaryCoordParam(Context);
	FNDIInputParam<int32> UVSetParam(Context);
	FNDIOutputParam<FVector2f> OutUVParam(Context);

	const FIndexArrayView IndexArray = StaticMeshHelper.GetIndexArrayView();
	const int32 TriangleMax = StaticMeshHelper.GetNumTriangles() - 1;
	const int32 UVSetMax = StaticMeshHelper.GetNumUVs() - 1;
	if (TriangleMax >= 0 && UVSetMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Triangle = FMath::Clamp(TriangleParam.GetAndAdvance(), 0, TriangleMax);
			const FVector3f BaryCoord = BaryCoordParam.GetAndAdvance();
			const int32 UVSet = FMath::Clamp(UVSetParam.GetAndAdvance(), 0, UVSetMax);
			const FVector2f UV = FVector2f(StaticMeshHelper.GetTriangleUV(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2], UVSet));	// LWC_TODO: Precision loss
			OutUVParam.SetAndAdvance(UV);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutUVParam.SetAndAdvance(FVector2f::ZeroVector);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetTriangleIndices(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIOutputParam<int32> OutIndex0Param(Context);
	FNDIOutputParam<int32> OutIndex1Param(Context);
	FNDIOutputParam<int32> OutIndex2Param(Context);

	const FIndexArrayView IndexArray = StaticMeshHelper.GetIndexArrayView();
	const int32 TriangleMax = StaticMeshHelper.GetNumTriangles() - 1;
	if (TriangleMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Triangle = FMath::Clamp(TriangleParam.GetAndAdvance(), 0, TriangleMax);
			OutIndex0Param.SetAndAdvance(IndexArray[Triangle * 3 + 0]);
			OutIndex1Param.SetAndAdvance(IndexArray[Triangle * 3 + 1]);
			OutIndex2Param.SetAndAdvance(IndexArray[Triangle * 3 + 2]);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutIndex0Param.SetAndAdvance(0);
			OutIndex1Param.SetAndAdvance(0);
			OutIndex2Param.SetAndAdvance(0);
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// Socket Functions
void UNiagaraDataInterfaceStaticMesh::VMGetSocketCount(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<int32> OutCountParam(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCountParam.SetAndAdvance(InstanceData->CachedSockets.Num());
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetFilteredSocketCount(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<int32> OutCountParam(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCountParam.SetAndAdvance(InstanceData->NumFilteredSockets);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSocketCount(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<int32> OutCountParam(Context);

	const int32 NumUnfilteredSockets = InstanceData->CachedSockets.Num() - InstanceData->NumFilteredSockets;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCountParam.SetAndAdvance(NumUnfilteredSockets);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMRandomSocket(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIRandomHelper RandHelper(Context);
	FNDIOutputParam<int32> OutSocket(Context);

	const int32 SocketRandMax = InstanceData->CachedSockets.Num() - 1;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		RandHelper.GetAndAdvance();
		OutSocket.SetAndAdvance(SocketRandMax >= 0 ? RandHelper.RandRange(i, 0, SocketRandMax) : -1);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMRandomFilteredSocket(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIRandomHelper RandHelper(Context);
	FNDIOutputParam<int32> OutSocket(Context);

	const int32 SocketRandMax = InstanceData->NumFilteredSockets - 1;
	if (SocketRandMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			const int32 SocketIndex = RandHelper.RandRange(i, 0, SocketRandMax);
			OutSocket.SetAndAdvance(InstanceData->FilteredAndUnfilteredSockets[SocketIndex]);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutSocket.SetAndAdvance(-1);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMRandomUnfilteredSocket(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIRandomHelper RandHelper(Context);
	FNDIOutputParam<int32> OutSocket(Context);

	const int32 SocketRandMax = InstanceData->FilteredAndUnfilteredSockets.Num() - InstanceData->NumFilteredSockets - 1;
	if (SocketRandMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			const int32 SocketIndex = InstanceData->NumFilteredSockets + RandHelper.RandRange(i, 0, SocketRandMax);
			OutSocket.SetAndAdvance(InstanceData->FilteredAndUnfilteredSockets[SocketIndex]);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutSocket.SetAndAdvance(-1);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetSocketTransform(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> SocketIndexParam(Context);
	FNDIOutputParam<FVector3f> OutTranslateParam(Context);
	FNDIOutputParam<FQuat4f> OutRotateParam(Context);
	FNDIOutputParam<FVector3f> OutScaleParam(Context);
	FNDIOutputParam<FVector3f> OutVelocityParam(Context);

	const int32 SocketMax = StaticMeshHelper.InstanceData->CachedSockets.Num() - 1;
	if (SocketMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 SocketIndex = FMath::Clamp(SocketIndexParam.GetAndAdvance(), 0, SocketMax);
			const FTransform3f& SocketTransform = StaticMeshHelper.InstanceData->CachedSockets[SocketIndex];
			const FVector3f Position = StaticMeshHelper.TransformPosition(SocketTransform.GetLocation());
			const FVector3f PreviousPosition = StaticMeshHelper.PreviousTransformPosition(SocketTransform.GetLocation());
			const FVector3f Velocity = (Position - PreviousPosition) * StaticMeshHelper.GetInvDeltaSeconds();
			OutTranslateParam.SetAndAdvance(StaticMeshHelper.TransformPosition(SocketTransform.GetTranslation()));
			OutRotateParam.SetAndAdvance(StaticMeshHelper.TransformRotation(SocketTransform.GetRotation()));
			OutScaleParam.SetAndAdvance(StaticMeshHelper.TransformNotUnitVector(SocketTransform.GetScale3D()));
			OutVelocityParam.SetAndAdvance(Velocity);
		}
	}
	else
	{
		const FVector3f DefaultTranslate = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		const FQuat4f DefaultRotation = StaticMeshHelper.TransformRotation(FQuat4f::Identity);
		const FVector3f DefaultScale = StaticMeshHelper.TransformNotUnitVector(FVector3f::OneVector);
		const FVector3f DefaultVelocity = FVector3f::ZeroVector;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTranslateParam.SetAndAdvance(DefaultTranslate);
			OutRotateParam.SetAndAdvance(DefaultRotation);
			OutScaleParam.SetAndAdvance(DefaultScale);
			OutVelocityParam.SetAndAdvance(DefaultVelocity);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetSocketTransformInterpolated(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> SocketIndexParam(Context);
	FNDIInputParam<float> InterpParam(Context);
	FNDIOutputParam<FVector3f> OutTranslateParam(Context);
	FNDIOutputParam<FQuat4f> OutRotateParam(Context);
	FNDIOutputParam<FVector3f> OutScaleParam(Context);
	FNDIOutputParam<FVector3f> OutVelocityParam(Context);

	const int32 SocketMax = StaticMeshHelper.InstanceData->CachedSockets.Num() - 1;
	if (SocketMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 SocketIndex = FMath::Clamp(SocketIndexParam.GetAndAdvance(), 0, SocketMax);
			FVector3f Position, Scale, Velocity;
			FQuat4f Rotation;
			StaticMeshHelper.InterpolateSocket(SocketIndex, InterpParam.GetAndAdvance(), Position, Rotation, Scale, Velocity);
			OutTranslateParam.SetAndAdvance(Position);
			OutRotateParam.SetAndAdvance(Rotation);
			OutScaleParam.SetAndAdvance(Scale);
			OutVelocityParam.SetAndAdvance(Velocity);
		}
	}
	else
	{
		const FVector3f DefaultTranslate = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		const FQuat4f DefaultRotation = StaticMeshHelper.TransformRotation(FQuat4f::Identity);
		const FVector3f DefaultScale = StaticMeshHelper.TransformNotUnitVector(FVector3f::OneVector);
		const FVector3f DefaultVelocity = FVector3f::ZeroVector;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTranslateParam.SetAndAdvance(DefaultTranslate);
			OutRotateParam.SetAndAdvance(DefaultRotation);
			OutScaleParam.SetAndAdvance(DefaultScale);
			OutVelocityParam.SetAndAdvance(DefaultVelocity);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetFilteredSocketTransform(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> SocketIndexParam(Context);
	FNDIOutputParam<FVector3f> OutTranslateParam(Context);
	FNDIOutputParam<FQuat4f> OutRotateParam(Context);
	FNDIOutputParam<FVector3f> OutScaleParam(Context);
	FNDIOutputParam<FVector3f> OutVelocityParam(Context);

	const int32 SocketMax = StaticMeshHelper.InstanceData->NumFilteredSockets - 1;
	if (SocketMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 FilteredSocketIndex = FMath::Clamp(SocketIndexParam.GetAndAdvance(), 0, SocketMax);
			const int32 SocketIndex = StaticMeshHelper.InstanceData->FilteredAndUnfilteredSockets[FilteredSocketIndex];
			const FTransform3f& SocketTransform = StaticMeshHelper.InstanceData->CachedSockets[SocketIndex];
			const FVector3f Position = StaticMeshHelper.TransformPosition(SocketTransform.GetLocation());
			const FVector3f PreviousPosition = StaticMeshHelper.PreviousTransformPosition(SocketTransform.GetLocation());
			const FVector3f Velocity = (Position - PreviousPosition) * StaticMeshHelper.GetInvDeltaSeconds();
			OutTranslateParam.SetAndAdvance(StaticMeshHelper.TransformPosition(SocketTransform.GetTranslation()));
			OutRotateParam.SetAndAdvance(StaticMeshHelper.TransformRotation(SocketTransform.GetRotation()));
			OutScaleParam.SetAndAdvance(StaticMeshHelper.TransformNotUnitVector(SocketTransform.GetScale3D()));
			OutVelocityParam.SetAndAdvance(Velocity);
		}
	}
	else
	{
		const FVector3f DefaultTranslate = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		const FQuat4f DefaultRotation = StaticMeshHelper.TransformRotation(FQuat4f::Identity);
		const FVector3f DefaultScale = StaticMeshHelper.TransformNotUnitVector(FVector3f::OneVector);
		const FVector3f DefaultVelocity = FVector3f::ZeroVector;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTranslateParam.SetAndAdvance(DefaultTranslate);
			OutRotateParam.SetAndAdvance(DefaultRotation);
			OutScaleParam.SetAndAdvance(DefaultScale);
			OutVelocityParam.SetAndAdvance(DefaultVelocity);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetFilteredSocketTransformInterpolated(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> SocketIndexParam(Context);
	FNDIInputParam<float> InterpParam(Context);
	FNDIOutputParam<FVector3f> OutTranslateParam(Context);
	FNDIOutputParam<FQuat4f> OutRotateParam(Context);
	FNDIOutputParam<FVector3f> OutScaleParam(Context);
	FNDIOutputParam<FVector3f> OutVelocityParam(Context);

	const int32 SocketMax = StaticMeshHelper.InstanceData->NumFilteredSockets - 1;
	if (SocketMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 FilteredSocketIndex = FMath::Clamp(SocketIndexParam.GetAndAdvance(), 0, SocketMax);
			const int32 SocketIndex = StaticMeshHelper.InstanceData->FilteredAndUnfilteredSockets[FilteredSocketIndex];
			FVector3f Position, Scale, Velocity;
			FQuat4f Rotation;
			StaticMeshHelper.InterpolateSocket(SocketIndex, InterpParam.GetAndAdvance(), Position, Rotation, Scale, Velocity);
			OutTranslateParam.SetAndAdvance(Position);
			OutRotateParam.SetAndAdvance(Rotation);
			OutScaleParam.SetAndAdvance(Scale);
			OutVelocityParam.SetAndAdvance(Velocity);
		}
	}
	else
	{
		const FVector3f DefaultTranslate = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		const FQuat4f DefaultRotation = StaticMeshHelper.TransformRotation(FQuat4f::Identity);
		const FVector3f DefaultScale = StaticMeshHelper.TransformNotUnitVector(FVector3f::OneVector);
		const FVector3f DefaultVelocity = FVector3f::ZeroVector;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTranslateParam.SetAndAdvance(DefaultTranslate);
			OutRotateParam.SetAndAdvance(DefaultRotation);
			OutScaleParam.SetAndAdvance(DefaultScale);
			OutVelocityParam.SetAndAdvance(DefaultVelocity);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSocketTransform(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> SocketIndexParam(Context);
	FNDIOutputParam<FVector3f> OutTranslateParam(Context);
	FNDIOutputParam<FQuat4f> OutRotateParam(Context);
	FNDIOutputParam<FVector3f> OutScaleParam(Context);
	FNDIOutputParam<FVector3f> OutVelocityParam(Context);

	const int32 UnfilteredOffset = StaticMeshHelper.InstanceData->NumFilteredSockets;
	const int32 SocketMax = StaticMeshHelper.InstanceData->FilteredAndUnfilteredSockets.Num() - UnfilteredOffset - 1;
	if (SocketMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 UnfilteredSocketIndex = FMath::Clamp(SocketIndexParam.GetAndAdvance(), 0, SocketMax) + UnfilteredOffset;
			const int32 SocketIndex = StaticMeshHelper.InstanceData->FilteredAndUnfilteredSockets[UnfilteredSocketIndex];
			const FTransform3f& SocketTransform = StaticMeshHelper.InstanceData->CachedSockets[SocketIndex];
			const FVector3f Position = StaticMeshHelper.TransformPosition(SocketTransform.GetLocation());
			const FVector3f PreviousPosition = StaticMeshHelper.PreviousTransformPosition(SocketTransform.GetLocation());
			const FVector3f Velocity = (Position - PreviousPosition) * StaticMeshHelper.GetInvDeltaSeconds();
			OutTranslateParam.SetAndAdvance(StaticMeshHelper.TransformPosition(SocketTransform.GetTranslation()));
			OutRotateParam.SetAndAdvance(StaticMeshHelper.TransformRotation(SocketTransform.GetRotation()));
			OutScaleParam.SetAndAdvance(StaticMeshHelper.TransformNotUnitVector(SocketTransform.GetScale3D()));
			OutVelocityParam.SetAndAdvance(Velocity);
		}
	}
	else
	{
		const FVector3f DefaultTranslate = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		const FQuat4f DefaultRotation = StaticMeshHelper.TransformRotation(FQuat4f::Identity);
		const FVector3f DefaultScale = StaticMeshHelper.TransformNotUnitVector(FVector3f::OneVector);
		const FVector3f DefaultVelocity = FVector3f::ZeroVector;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTranslateParam.SetAndAdvance(DefaultTranslate);
			OutRotateParam.SetAndAdvance(DefaultRotation);
			OutScaleParam.SetAndAdvance(DefaultScale);
			OutVelocityParam.SetAndAdvance(DefaultVelocity);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSocketTransformInterpolated(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> SocketIndexParam(Context);
	FNDIInputParam<float> InterpParam(Context);
	FNDIOutputParam<FVector3f> OutTranslateParam(Context);
	FNDIOutputParam<FQuat4f> OutRotateParam(Context);
	FNDIOutputParam<FVector3f> OutScaleParam(Context);
	FNDIOutputParam<FVector3f> OutVelocityParam(Context);

	const int32 UnfilteredOffset = StaticMeshHelper.InstanceData->NumFilteredSockets;
	const int32 SocketMax = StaticMeshHelper.InstanceData->FilteredAndUnfilteredSockets.Num() - UnfilteredOffset - 1;
	if (SocketMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 UnfilteredSocketIndex = FMath::Clamp(SocketIndexParam.GetAndAdvance(), 0, SocketMax) + UnfilteredOffset;
			const int32 SocketIndex = StaticMeshHelper.InstanceData->FilteredAndUnfilteredSockets[UnfilteredSocketIndex];
			FVector3f Position, Scale, Velocity;
			FQuat4f Rotation;
			StaticMeshHelper.InterpolateSocket(SocketIndex, InterpParam.GetAndAdvance(), Position, Rotation, Scale, Velocity);
			OutTranslateParam.SetAndAdvance(Position);
			OutRotateParam.SetAndAdvance(Rotation);
			OutScaleParam.SetAndAdvance(Scale);
			OutVelocityParam.SetAndAdvance(Velocity);
		}
	}
	else
	{
		const FVector3f DefaultTranslate = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		const FQuat4f DefaultRotation = StaticMeshHelper.TransformRotation(FQuat4f::Identity);
		const FVector3f DefaultScale = StaticMeshHelper.TransformNotUnitVector(FVector3f::OneVector);
		const FVector3f DefaultVelocity = FVector3f::ZeroVector;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTranslateParam.SetAndAdvance(DefaultTranslate);
			OutRotateParam.SetAndAdvance(DefaultRotation);
			OutScaleParam.SetAndAdvance(DefaultScale);
			OutVelocityParam.SetAndAdvance(DefaultVelocity);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetFilteredSocket(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<int32> InIndex(Context);
	FNDIOutputParam<int32> OutSocketIndex(Context);

	const int32 SocketMax = InstanceData->NumFilteredSockets - 1;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 Index = InIndex.GetAndAdvance();
		const int32 FilteredSocketIndex = FMath::Clamp(Index, 0, SocketMax);
		const int32 SocketIndex = SocketMax >= 0 ? InstanceData->FilteredAndUnfilteredSockets[FilteredSocketIndex] : INDEX_NONE;
		OutSocketIndex.SetAndAdvance(SocketIndex);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSocket(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<int32> InIndex(Context);
	FNDIOutputParam<int32> OutSocketIndex(Context);

	const int32 UnfilteredOffset = InstanceData->NumFilteredSockets;
	const int32 SocketMax = InstanceData->FilteredAndUnfilteredSockets.Num() - UnfilteredOffset - 1;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 Index = InIndex.GetAndAdvance();
		const int32 UnfilteredSocketIndex = FMath::Clamp(Index, 0, SocketMax) + UnfilteredOffset;
		const int32 SocketIndex = SocketMax >= 0 ? InstanceData->FilteredAndUnfilteredSockets[UnfilteredSocketIndex] : INDEX_NONE;
		OutSocketIndex.SetAndAdvance(SocketIndex);
	}
}

//////////////////////////////////////////////////////////////////////////
// Section functions
void UNiagaraDataInterfaceStaticMesh::VMIsValidSection(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> SectionIndexParam(Context);
	FNDIOutputParam<bool> OutIsValid(Context);

	const int32 NumSections = StaticMeshHelper.LODResource ? StaticMeshHelper.LODResource->Sections.Num() : 0;
	for ( int32 i=0; i < Context.GetNumInstances(); ++i )
	{
		const int32 Section = SectionIndexParam.GetAndAdvance();
		OutIsValid.SetAndAdvance(Section >= 0 && Section < NumSections);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetSectionTriangleCount(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> SectionIndexParam(Context);
	FNDIOutputParam<int32> OutTriangleCount(Context);

	const int32 SectionMax = StaticMeshHelper.LODResource ? StaticMeshHelper.LODResource->Sections.Num() : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 Section = SectionIndexParam.GetAndAdvance();
		const bool ValidSection = Section >= 0 && Section < SectionMax;
		OutTriangleCount.SetAndAdvance(ValidSection ? StaticMeshHelper.LODResource->Sections[Section].NumTriangles : 0);
	}
}

template<typename TRandomHandler>
void UNiagaraDataInterfaceStaticMesh::VMRandomSectionTriangle(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	TRandomHandler RandHelper(Context);
	FNDIInputParam<int32> SectionIndexParam(Context);
	FNDIOutputParam<int32> OutTriangle(Context);
	FNDIOutputParam<FVector3f> OutBaryCoord(Context);

	const int32 SectionMax = StaticMeshHelper.LODResource ? StaticMeshHelper.LODResource->Sections.Num() - 1 : 0;
	if (SectionMax >= 0)
	{
		if ( StaticMeshHelper.IsCpuUniformlyDistributedSampling() )
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				RandHelper.GetAndAdvance();
				const int32 Section = FMath::Clamp(SectionIndexParam.GetAndAdvance(), 0, SectionMax);
				OutTriangle.SetAndAdvance(StaticMeshHelper.RandomSectionUniformTriangle(RandHelper, i, Section));
				OutBaryCoord.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
			}
		}
		else
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				RandHelper.GetAndAdvance();
				const int32 Section = FMath::Clamp(SectionIndexParam.GetAndAdvance(), 0, SectionMax);
				OutTriangle.SetAndAdvance(StaticMeshHelper.RandomSectionTriangle(RandHelper, i, Section));
				OutBaryCoord.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
			}
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutTriangle.SetAndAdvance(0);
			OutBaryCoord.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetSectionTriangleAt(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> SectionIndexParam(Context);
	FNDIInputParam<int32> TriangleIndexParam(Context);
	FNDIOutputParam<int32> OutTriangle(Context);

	const int32 NumSections = StaticMeshHelper.LODResource ? StaticMeshHelper.LODResource->Sections.Num() : 0;
	if (NumSections >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Section = SectionIndexParam.GetAndAdvance();
			const int32 SectionTriangleIndex = TriangleIndexParam.GetAndAdvance();
			const bool SectionValid = Section >= 0 && Section < NumSections;
			const int32 SectionTriangleMax = SectionValid ? StaticMeshHelper.LODResource->Sections[Section].NumTriangles - 1 : 0;
			const int32 SectionTriangleOffset = SectionValid ? StaticMeshHelper.LODResource->Sections[Section].FirstIndex / 3 : 0;
			OutTriangle.SetAndAdvance(FMath::Clamp(SectionTriangleIndex, 0, SectionTriangleMax) + SectionTriangleOffset);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTriangle.SetAndAdvance(0);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetFilteredSectionAt(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> SectionIndexParam(Context);
	FNDIOutputParam<int32> OutSection(Context);

	TConstArrayView<int32> FilteredSections = StaticMeshHelper.InstanceData->GetFilteredSections();
	const int32 SectionMax = FilteredSections.Num() - 1;
	if ( SectionMax >= 0 )
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 SectionIndex = FMath::Clamp(SectionIndexParam.GetAndAdvance(), 0, SectionMax);
			OutSection.SetAndAdvance(FilteredSections[SectionIndex]);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutSection.SetAndAdvance(0);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSectionAt(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIInputParam<int32> SectionIndexParam(Context);
	FNDIOutputParam<int32> OutSection(Context);

	TConstArrayView<int32> UnfilteredSections = StaticMeshHelper.InstanceData->GetUnfilteredSections();
	const int32 SectionMax = UnfilteredSections.Num() - 1;
	if (SectionMax >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 SectionIndex = FMath::Clamp(SectionIndexParam.GetAndAdvance(), 0, SectionMax);
			OutSection.SetAndAdvance(UnfilteredSections[SectionIndex]);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutSection.SetAndAdvance(0);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetSectionCount(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIOutputParam<int32> OutSectionCount(Context);

	const int32 NumSections = StaticMeshHelper.LODResource ? StaticMeshHelper.LODResource->Sections.Num() : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutSectionCount.SetAndAdvance(NumSections);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetFilteredSectionCount(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIOutputParam<int32> OutSectionCount(Context);

	const int32 NumSections = StaticMeshHelper.InstanceData->NumFilteredSections;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutSectionCount.SetAndAdvance(NumSections);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetUnfilteredSectionCount(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIOutputParam<int32> OutSectionCount(Context);

	const int32 NumSections = StaticMeshHelper.InstanceData->NumUnfilteredSections;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutSectionCount.SetAndAdvance(NumSections);
	}
}

template<typename TRandomHandler>
void UNiagaraDataInterfaceStaticMesh::VMRandomSection(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	TRandomHandler RandHelper(Context);
	FNDIOutputParam<int32> OutSection(Context);

	if ( StaticMeshHelper.IsCpuUniformlyDistributedSampling() )
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutSection.SetAndAdvance(StaticMeshHelper.RandomUniformSection(RandHelper, i));
		}
	}
	else
	{
		const int32 SectionMax = StaticMeshHelper.LODResource ? StaticMeshHelper.LODResource->Sections.Num() - 1 : 0;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutSection.SetAndAdvance(RandHelper.RandRange(i, 0, SectionMax));
		}
	}
}

template<typename TRandomHandler>
void UNiagaraDataInterfaceStaticMesh::VMRandomFilteredSection(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	TRandomHandler RandHelper(Context);
	FNDIOutputParam<int32> OutSection(Context);

	TConstArrayView<int32> FilteredSections = StaticMeshHelper.InstanceData->GetFilteredSections();
	const int32 SectionMax = FilteredSections.Num() - 1;
	if (SectionMax >= 0)
	{
		if ( StaticMeshHelper.IsCpuUniformlyDistributedSampling() )
		{
			TConstArrayView<NDIStaticMeshLocal::FNDISectionInfo> FilteredSectionInfos = StaticMeshHelper.InstanceData->GetFilteredSectionInfos();
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				RandHelper.GetAndAdvance();
				const int32 Section = StaticMeshHelper.RandomUniformSection(RandHelper, i, FilteredSectionInfos, FilteredSections);
				OutSection.SetAndAdvance(FilteredSections[Section]);
			}
		}
		else
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				RandHelper.GetAndAdvance();
				OutSection.SetAndAdvance(FilteredSections[RandHelper.RandRange(i, 0, SectionMax)]);
			}
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutSection.SetAndAdvance(0);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMRandomUnfilteredSection(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<FNDITransformHandlerNoop> StaticMeshHelper(Context);
	FNDIRandomHelper RandHelper(Context);
	FNDIOutputParam<int32> OutSection(Context);

	TConstArrayView<int32> UnfilteredSections = StaticMeshHelper.InstanceData->GetUnfilteredSections();
	const int32 SectionMax = UnfilteredSections.Num() - 1;
	if (SectionMax >= 0)
	{
		if (StaticMeshHelper.IsCpuUniformlyDistributedSampling())
		{
			TConstArrayView<NDIStaticMeshLocal::FNDISectionInfo> UnfilteredSectionInfos = StaticMeshHelper.InstanceData->GetUnfilteredSectionInfos();
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				RandHelper.GetAndAdvance();
				const int32 Section = StaticMeshHelper.RandomUniformSection(RandHelper, i, UnfilteredSectionInfos, UnfilteredSections);
				OutSection.SetAndAdvance(UnfilteredSections[Section]);
			}
		}
		else
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				RandHelper.GetAndAdvance();
				OutSection.SetAndAdvance(UnfilteredSections[RandHelper.RandRange(i, 0, SectionMax)]);
			}
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutSection.SetAndAdvance(0);
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// VM Misc Functions
void UNiagaraDataInterfaceStaticMesh::VMIsValid(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<bool> OutValid(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutValid.SetAndAdvance(InstanceData->bMeshValid);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetLocalToWorld(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<FMatrix44f> OutMatrix(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutMatrix.SetAndAdvance(FMatrix44f(InstanceData->Transform));						// LWC_TODO: Precision loss
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetLocalToWorldInverseTransposed(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<FMatrix44f> OutMatrix(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutMatrix.SetAndAdvance(FMatrix44f(InstanceData->TransformInverseTransposed));		// LWC_TODO: Precision loss
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetWorldVelocity(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<FVector3f> OutVelocity(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutVelocity.SetAndAdvance((FVector3f)InstanceData->PhysicsVelocity);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetInstanceIndex(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIOutputParam<int32> OutInstanceIndex(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutInstanceIndex.SetAndAdvance(InstanceData->InstanceIndex);
	}
}

void UNiagaraDataInterfaceStaticMesh::VMSetInstanceIndex(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);
	FNDIInputParam<int32> InInstanceIndex(Context);

	bool bUpdateTransforms = false;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 NewInstanceIndex = InInstanceIndex.GetAndAdvance();
		bUpdateTransforms |= InstanceData->InstanceIndex != NewInstanceIndex;
		InstanceData->InstanceIndex = NewInstanceIndex;
	}
	if (bUpdateTransforms)
	{
		InstanceData->UpdateTransformsInstanceIndexChanged();
	}
}

//////////////////////////////////////////////////////////////////////////
// VM UV mapping functions
void UNiagaraDataInterfaceStaticMesh::VMGetTriangleCoordAtUV(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);

	FNDIInputParam<bool> InEnabled(Context);
	FNDIInputParam<FVector2f> InUV(Context);
	FNDIInputParam<float> InTolerance(Context);

	FNDIOutputParam<int32> OutTriangleIndex(Context);
	FNDIOutputParam<FVector3f> OutBaryCoord(Context);
	FNDIOutputParam<FNiagaraBool> OutIsValid(Context);

	const FStaticMeshUvMapping* UvMapping = InstanceData->UvMapping ? InstanceData->UvMapping.GetMappingData() : nullptr;

	if (UvMapping)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool Enabled = InEnabled.GetAndAdvance();
			const FVector2D SourceUv = FVector2D(InUV.GetAndAdvance());
			const float Tolerance = InTolerance.GetAndAdvance();

			FVector3f BaryCoord(ForceInitToZero);
			int32 TriangleIndex = INDEX_NONE;

			if (Enabled)
			{
				FVector BaryCoordD;
				TriangleIndex = UvMapping->FindFirstTriangle(SourceUv, Tolerance, BaryCoordD);
				BaryCoord = FVector3f(BaryCoordD);
			}

			OutTriangleIndex.SetAndAdvance(TriangleIndex);
			OutBaryCoord.SetAndAdvance(BaryCoord);
			OutIsValid.SetAndAdvance(TriangleIndex != INDEX_NONE);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTriangleIndex.SetAndAdvance(INDEX_NONE);
			OutBaryCoord.SetAndAdvance(FVector3f::ZeroVector);
			OutIsValid.SetAndAdvance(false);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMGetTriangleCoordInAabb(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);

	FNDIInputParam<bool> InEnabled(Context);
	FNDIInputParam<FVector2f> InUvMin(Context);
	FNDIInputParam<FVector2f> InUvMax(Context);

	FNDIOutputParam<int32> OutTriangleIndex(Context);
	FNDIOutputParam<FVector3f> OutBaryCoord(Context);
	FNDIOutputParam<FNiagaraBool> OutIsValid(Context);

	const FStaticMeshUvMapping* UvMapping = InstanceData->UvMapping ? InstanceData->UvMapping.GetMappingData() : nullptr;

	if (UvMapping)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool Enabled = InEnabled.GetAndAdvance();
			const FVector2D SourceUvMin = FVector2D(InUvMin.GetAndAdvance());
			const FVector2D SourceUvMax = FVector2D(InUvMax.GetAndAdvance());

			FVector3f BaryCoord(ForceInitToZero);
			int32 TriangleIndex = INDEX_NONE;

			if (Enabled)
			{
				FVector BaryCoordD;
				TriangleIndex = UvMapping->FindFirstTriangle(FBox2D(SourceUvMin, SourceUvMax), BaryCoordD);
				BaryCoord = FVector3f(BaryCoordD);
			}

			OutTriangleIndex.SetAndAdvance(TriangleIndex);
			OutBaryCoord.SetAndAdvance(BaryCoord);
			OutIsValid.SetAndAdvance(TriangleIndex != INDEX_NONE);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTriangleIndex.SetAndAdvance(INDEX_NONE);
			OutBaryCoord.SetAndAdvance(FVector3f::ZeroVector);
			OutIsValid.SetAndAdvance(false);
		}
	}
}

void UNiagaraDataInterfaceStaticMesh::VMBuildUvMapping(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<NDIStaticMeshLocal::FInstanceData_GameThread> InstanceData(Context);

	FNDIInputParam<int32> InUvSet(Context);

	const int32 InstanceCount = Context.GetNumInstances();
	if (InstanceCount > 0)
	{
		const int32 UvMappingIndexSet = InUvSet.GetAndAdvance();

		// for sanity, verify that we don't have multiple instances with conflicted reports for the uv set to use
		for (int32 i = 1; i < InstanceCount; ++i)
		{
			check(InUvSet.GetAndAdvance() == UvMappingIndexSet);
		}

		InstanceData->UvMappingIndexSet = UvMappingIndexSet;
	}
}


//////////////////////////////////////////////////////////////////////////
// Deprecated VM Functions
template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetVertexPosition_Deprecated(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> VertexParam(Context);
	FNDIOutputParam<FVector3f> OutPosition(Context);

	const int32 PositionsMax = StaticMeshHelper.GetNumPositionVertices() - 1;
	if ( PositionsMax >= 0 )
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Vertex = FMath::Clamp(VertexParam.GetAndAdvance(), 0, PositionsMax);
			const FVector3f Position = StaticMeshHelper.GetPosition(Vertex);
			OutPosition.SetAndAdvance(Position);
		}
	}
	else
	{
		const FVector3f Position = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutPosition.SetAndAdvance(Position);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetTriPosition_Deprecated(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIInputParam<FVector3f> BaryCoordParam(Context);
	FNDIOutputParam<FVector3f> OutPositionParam(Context);

	const FIndexArrayView IndexArray = StaticMeshHelper.GetIndexArrayView();
	const int32 TriangleMax = StaticMeshHelper.GetNumTriangles() - 1;
	if (TriangleMax >= 0 && StaticMeshHelper.GetNumPositionVertices() > 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Triangle = FMath::Clamp(TriangleParam.GetAndAdvance(), 0, TriangleMax);
			const FVector3f BaryCoord = BaryCoordParam.GetAndAdvance();
			const FVector3f Position = StaticMeshHelper.GetTrianglePosition(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			OutPositionParam.SetAndAdvance(Position);
		}
	}
	else
	{
		const FVector3f Position = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutPositionParam.SetAndAdvance(Position);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetTriPositionAndVelocity_Deprecated(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIInputParam<FVector3f> BaryCoordParam(Context);
	FNDIOutputParam<FVector3f> OutPositionParam(Context);
	FNDIOutputParam<FVector3f> OutVelocityParam(Context);

	const FIndexArrayView IndexArray = StaticMeshHelper.GetIndexArrayView();
	const int32 TriangleMax = StaticMeshHelper.GetNumTriangles() - 1;
	if (TriangleMax >= 0 && StaticMeshHelper.GetNumPositionVertices() > 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Triangle = FMath::Clamp(TriangleParam.GetAndAdvance(), 0, TriangleMax);
			const FVector3f BaryCoord = BaryCoordParam.GetAndAdvance();
			const FVector3f LocalPosition = StaticMeshHelper.GetLocalTrianglePosition(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			const FVector3f Position = StaticMeshHelper.TransformPosition(LocalPosition);
			const FVector3f PreviousPosition = StaticMeshHelper.PreviousTransformPosition(LocalPosition);
			const FVector3f Velocity = (Position - PreviousPosition) * StaticMeshHelper.GetInvDeltaSeconds();
			OutPositionParam.SetAndAdvance(Position);
			OutVelocityParam.SetAndAdvance(Velocity);
		}
	}
	else
	{
		const FVector3f Position = StaticMeshHelper.TransformPosition(FVector3f::ZeroVector);
		const FVector3f PreviousPosition = StaticMeshHelper.PreviousTransformPosition(FVector3f::ZeroVector);
		const FVector3f Velocity = FVector3f::ZeroVector;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutPositionParam.SetAndAdvance(Position);
			OutVelocityParam.SetAndAdvance(Velocity);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetTriangleTangentBasis_Deprecated(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIInputParam<FVector3f> BaryCoordParam(Context);
	FNDIOutputParam<FVector3f> OutTangent(Context);
	FNDIOutputParam<FVector3f> OutBitangent(Context);
	FNDIOutputParam<FVector3f> OutNormal(Context);

	const FIndexArrayView IndexArray = StaticMeshHelper.GetIndexArrayView();
	const int32 TriangleMax = StaticMeshHelper.GetNumTriangles() - 1;
	if (TriangleMax >= 0 && StaticMeshHelper.GetNumTangentVertices() > 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Triangle = FMath::Clamp(TriangleParam.GetAndAdvance(), 0, TriangleMax);
			const FVector3f BaryCoord = BaryCoordParam.GetAndAdvance();
			const FVector3f TangentX = StaticMeshHelper.GetTriangleTangentX(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			const FVector3f TangentY = StaticMeshHelper.GetTriangleTangentY(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			const FVector3f TangentZ = StaticMeshHelper.GetTriangleTangentZ(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			OutNormal.SetAndAdvance(TangentZ);
			OutBitangent.SetAndAdvance(TangentY);
			OutTangent.SetAndAdvance(TangentX);
		}
	}
	else
	{
		const FVector3f TangentX = StaticMeshHelper.TransformUnitVector(FVector3f(1.0f, 0.0f, 0.0f));
		const FVector3f TangentY = StaticMeshHelper.TransformUnitVector(FVector3f(0.0f, 1.0f, 0.0f));
		const FVector3f TangentZ = StaticMeshHelper.TransformUnitVector(FVector3f(0.0f, 0.0f, 1.0f));
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutNormal.SetAndAdvance(TangentZ);
			OutBitangent.SetAndAdvance(TangentY);
			OutTangent.SetAndAdvance(TangentX);
		}
	}
}

template<typename TTransformHandler>
void UNiagaraDataInterfaceStaticMesh::VMGetTriangleNormal_Deprecated(FVectorVMExternalFunctionContext& Context)
{
	NDIStaticMeshLocal::FStaticMeshCpuHelper<TTransformHandler> StaticMeshHelper(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIInputParam<FVector3f> BaryCoordParam(Context);
	FNDIOutputParam<FVector3f> OutNormal(Context);

	const FIndexArrayView IndexArray = StaticMeshHelper.GetIndexArrayView();
	const int32 TriangleMax = StaticMeshHelper.GetNumTriangles() - 1;
	if (TriangleMax >= 0 && StaticMeshHelper.GetNumTangentVertices() > 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Triangle = FMath::Clamp(TriangleParam.GetAndAdvance(), 0, TriangleMax);
			const FVector3f BaryCoord = BaryCoordParam.GetAndAdvance();
			const FVector3f TangentZ = StaticMeshHelper.GetTriangleTangentZ(BaryCoord, IndexArray[Triangle * 3 + 0], IndexArray[Triangle * 3 + 1], IndexArray[Triangle * 3 + 2]);
			OutNormal.SetAndAdvance(TangentZ);
		}
	}
	else
	{
		const FVector3f TangentZ = StaticMeshHelper.TransformUnitVector(FVector3f(0.0f, 0.0f, 1.0f));
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutNormal.SetAndAdvance(TangentZ);
		}
	}
}


void FNDI_StaticMesh_GeneratedData::Tick(ETickingGroup TickGroup, float DeltaSeconds)
{
	check(IsInGameThread());

	{ // handle any changes to the UV mappings
		FRWScopeLock UvMappingWriteLock(CachedUvMappingGuard, SLT_Write);

		TArray<int32, TInlineAllocator<32>> MappingsToRemove;

		const int32 MappingCount = CachedUvMapping.Num();

		for (int32 MappingIt = 0; MappingIt < MappingCount; ++MappingIt)
		{
			const TSharedPtr<FStaticMeshUvMapping>& UvMappingData = CachedUvMapping[MappingIt];;

			if (UvMappingData->CanBeDestroyed())
			{
				MappingsToRemove.Add(MappingIt);
			}
		}

		while (MappingsToRemove.Num())
		{
			CachedUvMapping.RemoveAtSwap(MappingsToRemove.Pop(EAllowShrinking::No));
		}
	}
}

FStaticMeshUvMappingHandle FNDI_StaticMesh_GeneratedData::GetCachedUvMapping(TWeakObjectPtr<UStaticMesh>& MeshObject, int32 InLodIndex, int32 InUvSetIndex, FMeshUvMappingUsage Usage, bool bNeedsDataImmediately)
{
	check(MeshObject.Get() != nullptr);

	if (!FStaticMeshUvMapping::IsValidMeshObject(MeshObject, InLodIndex, InUvSetIndex))
	{
		return FStaticMeshUvMappingHandle();
	}

	// Attempt to Find data
	auto MappingMatchPredicate = [&](const TSharedPtr<FStaticMeshUvMapping>& UvMapping)
	{
		return UvMapping->Matches(MeshObject, InLodIndex, InUvSetIndex);
	};

	{
		FRWScopeLock ReadLock(CachedUvMappingGuard, SLT_ReadOnly);
		if (TSharedPtr<FStaticMeshUvMapping>* Existing = CachedUvMapping.FindByPredicate(MappingMatchPredicate))
		{
			return FStaticMeshUvMappingHandle(Usage, *Existing, bNeedsDataImmediately);
		}
	}

	// We need to add
	FRWScopeLock WriteLock(CachedUvMappingGuard, SLT_Write);
	if (TSharedPtr<FStaticMeshUvMapping>* Existing = CachedUvMapping.FindByPredicate(MappingMatchPredicate))
	{
		return FStaticMeshUvMappingHandle(Usage, *Existing, bNeedsDataImmediately);
	}
	return FStaticMeshUvMappingHandle(
		Usage,
		CachedUvMapping.Add_GetRef(MakeShared<FStaticMeshUvMapping>(MeshObject, InLodIndex, InUvSetIndex)),
		bNeedsDataImmediately);
}

#undef LOCTEXT_NAMESPACE
