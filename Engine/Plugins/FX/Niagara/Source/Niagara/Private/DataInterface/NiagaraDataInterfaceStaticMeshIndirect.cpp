// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterface/NiagaraDataInterfaceStaticMeshIndirect.h"

#include "NiagaraWorldManagerImpl.h"

#include "NiagaraCompileHashVisitor.h"
#include "NiagaraShaderParametersBuilder.h"
#include "FXRenderingUtils.h"
#include "NiagaraGpuComputeDispatchInterface.h"

#include "NiagaraDataManager_StaticMesh.h"

#include "NiagaraDataInterfaceUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceStaticMeshIndirect)
#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceStaticMeshIndirect"

namespace NDIStaticMeshIndirectLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceStaticMeshIndirectTemplate.ush");

	//////////////////////////////////////////////////////////////////////////
	struct EDIFunctionVersion
	{
		enum Type
		{
			InitialVersion = 0,

			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};
	};

	//////////////////////////////////////////////////////////////////////////
	// Distance Field Functions
	static const FName	QueryDistanceFieldName("QueryDistanceField");
	static const FName	GetClosestPointOnMeshDistanceFieldName("GetClosestPointOnMeshDistanceField");

	struct FRenderProxy : public FNiagaraDataInterfaceProxy
	{
		virtual void ConsumePerInstanceDataFromGameThread(void* FromGameThreadData, const FNiagaraSystemInstanceID& InstanceID) override{}
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override{ return 0; }
	};
};

//////////////////////////////////////////////////////////////////////////

void FNDIStaticMeshIndirectCompiledData::Init(const UNiagaraSystem* System, UNiagaraDataInterface* OwnerDI)
{
	bUsedByCPU = false;
	bUsedByGPU = false;
	bNeedsCpuAccess = false;	
	bRequiresDistanceFields = false;

	//For every GPU script we iterate over the functions it calls and add each of them to the mapping.
	//This will then be placed in a buffer for the RT to pass to the GPU so that each script can look up the correct function layout info.
	auto HandleGpuFunc = [&](const UNiagaraScript* Script, const FNiagaraDataInterfaceGeneratedFunction& BindingInfo)
	{
		bUsedByGPU = true;

		bool bIsDistanceFieldFunction = BindingInfo.DefinitionName == NDIStaticMeshIndirectLocal::QueryDistanceFieldName ||
										BindingInfo.DefinitionName == NDIStaticMeshIndirectLocal::GetClosestPointOnMeshDistanceFieldName;
		bRequiresDistanceFields |= bIsDistanceFieldFunction;
		 
		return true;
	};
	FNiagaraDataInterfaceUtilities::ForEachGpuFunction(OwnerDI, System, HandleGpuFunc);
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceStaticMeshIndirect::UNiagaraDataInterfaceStaticMeshIndirect(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new NDIStaticMeshIndirectLocal::FRenderProxy());
}

void UNiagaraDataInterfaceStaticMeshIndirect::PostInitProperties()
{
	Super::PostInitProperties();

	// Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags DIFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), DIFlags);
	}
}

bool UNiagaraDataInterfaceStaticMeshIndirect::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceStaticMeshIndirect* OtherTyped = CastChecked<const UNiagaraDataInterfaceStaticMeshIndirect>(Other);
	return true;
}

bool UNiagaraDataInterfaceStaticMeshIndirect::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceStaticMeshIndirect* OtherTyped = CastChecked<UNiagaraDataInterfaceStaticMeshIndirect>(Destination);
	return true;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceStaticMeshIndirect::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIStaticMeshIndirectLocal;

	//OutFunctions.Reserve(OutFunctions.Num() + NumFunctions);

	// Setup base signature
	FNiagaraFunctionSignature BaseSignature;
	BaseSignature.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("StaticMeshIndirect"));
	BaseSignature.bMemberFunction = true;
	BaseSignature.bRequiresContext = false;
	BaseSignature.FunctionVersion = EDIFunctionVersion::LatestVersion;

	GetDistanceFieldFunctions(OutFunctions, BaseSignature);
}

void UNiagaraDataInterfaceStaticMeshIndirect::GetDistanceFieldFunctions(TArray<FNiagaraFunctionSignature>&OutFunctions, const FNiagaraFunctionSignature & BaseSignature) const
{
	using namespace NDIStaticMeshIndirectLocal;
	
	const FText DFIsValidText = LOCTEXT("DFIsValidTooltip","IsValid Returns true if the mesh ID is valid and the mesh has a valid Distance Field to sample.");
	const FText DFNormalIsValidText = LOCTEXT("OutNormalIsValidTooltip","OutNormalIsValid Returns true the distance field was successfully sampled and the position and normal are valid.");
	{
		const FNiagaraVariable UseMaxDistanceVar(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Use Max Distance"));
		const FNiagaraVariable MaxDistanceVar(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Max Distance"));

		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = QueryDistanceFieldName;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(FNiagaraSharedDataID_StaticMesh::StaticStruct()), TEXT("StaticMeshID"));
		Sig.Inputs.Emplace(UseMaxDistanceVar);
		Sig.Inputs.Emplace(MaxDistanceVar);
		Sig.AddOutput(FNiagaraVariableBase(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")), DFIsValidText);
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Distance"));
		
#if WITH_EDITORONLY_DATA
		Sig.Description = LOCTEXT("QueryDistanceFieldDescription", "Given a world position, this returns the value of the parented static mesh`s signed distance field");
		Sig.InputDescriptions.Add(
			UseMaxDistanceVar,
			LOCTEXT("UseMaxDistanceOptimizationDescription", 
			"This enables an optimization that will skip reading the SDF texture if the world position exceeds a provided max distance value.\n"
			"This is a useful optimization if you only need accurate distance information within a certain distance threshold. \n"
			"One example of a good usecase is if you only want to spawn particles inside the mesh (i.e. distance < 0.0), you should enable this with a MaxDistance of 0.0.")
		);
		Sig.InputDescriptions.Add(
			MaxDistanceVar,
			LOCTEXT("MaxDistanceDescription",
			"Only used if UseMaxDistanceOptimization is enabled. This is the max distance from the static meshes` BOUNDING BOX at which we should query it's SDF.\n"
			"This defaults to 0.0, which means it will only read the SDF if the world position is inside the meshes' bounding box")
		);
#endif
	}

	{
		const FNiagaraVariable UseMaxDistanceVar(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Use Max Distance"));
		const FNiagaraVariable MaxDistanceVar(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Max Distance"));

		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(BaseSignature);
		Sig.Name = GetClosestPointOnMeshDistanceFieldName;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(FNiagaraSharedDataID_StaticMesh::StaticStruct()), TEXT("StaticMeshID"));
		Sig.Inputs.Emplace(UseMaxDistanceVar);
		Sig.Inputs.Emplace(MaxDistanceVar);
		Sig.AddOutput(FNiagaraVariableBase(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")), DFIsValidText);
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("OutClosestDistance"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("OutClosestPosition"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("OutClosestNormal"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("OutMaxEncodedDistance"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("OutNormalIsValid"));

#if WITH_EDITORONLY_DATA
		Sig.Description = LOCTEXT("GetClosestPointOnMeshDistanceFieldDescription", "Given a world position, this returns the closest point on the mesh with it's distance and normal.");
#endif
	}
}

#endif //WITH_EDITORONLY_DATA

bool UNiagaraDataInterfaceStaticMeshIndirect::RequiresEarlyViewData() const
{
	return GetCompiledData().bRequiresDistanceFields;
}

#if WITH_EDITORONLY_DATA

void UNiagaraDataInterfaceStaticMeshIndirect::PostCompile(const UNiagaraSystem& OwningSystem)
{
	CompiledData.Init(&OwningSystem, this);
}

void UNiagaraDataInterfaceStaticMeshIndirect::GetCommonHLSL(FString& OutHLSL)
{
	OutHLSL += TEXT("#include \"/Engine/Private/DistanceFieldLightingShared.ush\"\n");
	OutHLSL += TEXT("#include \"/Engine/Private/MeshDistanceFieldCommon.ush\"\n");
}

bool UNiagaraDataInterfaceStaticMeshIndirect::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIStaticMeshIndirectLocal::TemplateShaderFile);
	bSuccess &= InVisitor->UpdateShaderParameters<NDIStaticMeshIndirectLocal::FShaderParameters>();
	bSuccess &= InVisitor->UpdateShaderParameters<FNiagaraDataManager_StaticMesh_Parameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceStaticMeshIndirect::ModifyCompilationEnvironment(EShaderPlatform ShaderPlatform, struct FShaderCompilerEnvironment& OutEnvironment) const
{
	Super::ModifyCompilationEnvironment(ShaderPlatform, OutEnvironment);
}

void UNiagaraDataInterfaceStaticMeshIndirect::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIStaticMeshIndirectLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceStaticMeshIndirect::GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL)
{
	return	HlslGenContext.GetFunctionInfo().DefinitionName == NDIStaticMeshIndirectLocal::GetClosestPointOnMeshDistanceFieldName ||
		HlslGenContext.GetFunctionInfo().DefinitionName == NDIStaticMeshIndirectLocal::QueryDistanceFieldName;
}
#endif

void UNiagaraDataInterfaceStaticMeshIndirect::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIStaticMeshIndirectLocal::FShaderParameters>();
	ShaderParametersBuilder.AddIncludedStruct(UE::FXRenderingUtils::DistanceFields::GetObjectBufferParametersMetadata());
	ShaderParametersBuilder.AddIncludedStruct(UE::FXRenderingUtils::DistanceFields::GetAtlasParametersMetadata());
	ShaderParametersBuilder.AddIncludedStruct(FNiagaraDataManager_StaticMesh_RTProxy::GetParametersMetadata());
}

void UNiagaraDataInterfaceStaticMeshIndirect::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	NDIStaticMeshIndirectLocal::FRenderProxy& DIProxy = Context.GetProxy<NDIStaticMeshIndirectLocal::FRenderProxy>();
	NDIStaticMeshIndirectLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDIStaticMeshIndirectLocal::FShaderParameters>();

	// Bind Mesh Distance Field Data
	const FShaderParametersMetadata* ObjectBufferParametersMetadata = UE::FXRenderingUtils::DistanceFields::GetObjectBufferParametersMetadata();
	const FShaderParametersMetadata* AtlasParametersMetadata = UE::FXRenderingUtils::DistanceFields::GetAtlasParametersMetadata();

	uint8* ShaderDistanceFieldObjectParameters = Context.GetParameterIncludedStruct(ObjectBufferParametersMetadata);
	uint8* ShaderDistanceFieldAtlasParameters = Context.GetParameterIncludedStruct(AtlasParametersMetadata);

	const bool bDistanceFieldDataBound =
		Context.IsStructBound(ShaderDistanceFieldObjectParameters, ObjectBufferParametersMetadata) ||
		Context.IsStructBound(ShaderDistanceFieldAtlasParameters, AtlasParametersMetadata);

	if (bDistanceFieldDataBound)
	{
		TConstStridedView<FSceneView> SimulationSceneViews = Context.GetComputeDispatchInterface().GetSimulationSceneViews();
		const FSceneView* PrimaryView = SimulationSceneViews.Num() > 0 ? &SimulationSceneViews[0] : nullptr;
		UE::FXRenderingUtils::DistanceFields::SetupObjectBufferParameters(Context.GetGraphBuilder(), ShaderDistanceFieldObjectParameters, PrimaryView);
		UE::FXRenderingUtils::DistanceFields::SetupAtlasParameters(Context.GetGraphBuilder(), ShaderDistanceFieldAtlasParameters, PrimaryView);
	}

	FNiagaraDataManager_StaticMesh_RTProxy::SetShaderParameters(Context);	
}

#undef LOCTEXT_NAMESPACE
