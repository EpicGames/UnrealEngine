// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceUObjectPropertyReader.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"

#include "Internationalization/Internationalization.h"
#include "UObject/EnumProperty.h"
#include "UObject/WeakFieldPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceUObjectPropertyReader)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceUObjectPropertyReader"

//////////////////////////////////////////////////////////////////////////

namespace NDIUObjectPropertyReaderLocal
{
	static const FName GetComponentTransformName("GetComponentTransform");
	static const FName GetComponentInvTransformName("GetComponentInverseTransform");

	struct FNDIPropertyGetter
	{
		FNiagaraVariableBase		Variable;
		uint32						DataOffset = INDEX_NONE;
		TWeakFieldPtr<FProperty>	WeakProperty;
		TFunction<void(const FNiagaraLWCConverter&, void*)>		PropertyCopyFunction;
	};

	struct FInstanceData_GameToRender
	{
		TOptional<FTransform>	CachedTransform;
		TArray<uint32>			PropertyOffsets;
		TArray<uint8>			PropertyData;
	};

	struct FInstanceData_GameThread
	{
		FNiagaraParameterDirectBinding<UObject*>	UObjectBinding;
		TWeakObjectPtr<UObject>						WeakUObject;
		TArray<FNDIPropertyGetter>					PropertyGetters;
		TArray<uint8>								PropertyData;
		uint32										ChangeId = 0;

		TOptional<FTransform>						CachedTransform;
		TOptional<FTransform>						CachedInvTransform;

		uint32 AddProperty(FNiagaraVariableBase PropertyType)
		{
			for ( int i=0; i < PropertyGetters.Num(); ++i )
			{
				if ( PropertyGetters[i].Variable == PropertyType )
				{
					return i;
				}
			}

			FNDIPropertyGetter& NewGetter = PropertyGetters.AddDefaulted_GetRef();
			NewGetter.Variable = PropertyType;
			NewGetter.DataOffset = PropertyData.AddZeroed(PropertyType.GetSizeInBytes());

			return PropertyGetters.Num() - 1;
		}
	};

	struct FInstanceData_RenderThread
	{
		TOptional<FTransform>	CachedTransform;
		TOptional<FTransform>	CachedInvTransform;
		FReadBuffer				PropertyData;
		TArray<uint32>			PropertyOffsets;
		TArray<uint32>			GpuFunctionToPropertyRemap;
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID)
		{
			FInstanceData_GameToRender* InstanceData_ForRT = new(DataForRenderThread) FInstanceData_GameToRender();
			FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
			InstanceData_ForRT->CachedTransform = InstanceData_GT->CachedTransform;
			if (InstanceData_GT->PropertyData.Num() > 0)
			{
				InstanceData_ForRT->PropertyOffsets.AddZeroed(InstanceData_GT->PropertyGetters.Num());
				for ( int32 i=0; i < InstanceData_GT->PropertyGetters.Num(); ++i )
				{
					const FNDIPropertyGetter& PropertyGetter = InstanceData_GT->PropertyGetters[i];
					InstanceData_ForRT->PropertyOffsets[i] = PropertyGetter.PropertyCopyFunction ? PropertyGetter.DataOffset / sizeof(int32) : INDEX_NONE;
				}
				InstanceData_ForRT->PropertyData = InstanceData_GT->PropertyData;
			}
		}

		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override
		{
			FInstanceData_GameToRender* InstanceData_FromGT = static_cast<FInstanceData_GameToRender*>(PerInstanceData);
			if ( FInstanceData_RenderThread* InstanceData_RT = PerInstanceData_RenderThread.Find(InstanceID) )
			{
				FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

				InstanceData_RT->CachedTransform = InstanceData_FromGT->CachedTransform;
				if (InstanceData_RT->CachedTransform.IsSet())
				{
					InstanceData_RT->CachedInvTransform = InstanceData_RT->CachedTransform->Inverse();
				}
				else
				{
					InstanceData_RT->CachedInvTransform.Reset();
				}

				for ( int32 i=0; i < InstanceData_RT->GpuFunctionToPropertyRemap.Num(); ++i )
				{
					const uint32 RemapIndex = InstanceData_RT->GpuFunctionToPropertyRemap[i];
					InstanceData_RT->PropertyOffsets[i] = RemapIndex == INDEX_NONE ? INDEX_NONE : InstanceData_FromGT->PropertyOffsets[RemapIndex];
				}

				if ( InstanceData_RT->PropertyData.NumBytes != InstanceData_FromGT->PropertyData.Num())
				{
					InstanceData_RT->PropertyData.Release();
					const uint32 NumElements = InstanceData_FromGT->PropertyData.Num() / 4;
					if (NumElements > 0)
					{
						InstanceData_RT->PropertyData.Initialize(RHICmdList, TEXT("NiagaraUObjectPropertyReader"), sizeof(float), NumElements, EPixelFormat::PF_R32_UINT);
					}
				}
				if (InstanceData_RT->PropertyData.NumBytes > 0)
				{
					void* GpuMemory = RHICmdList.LockBuffer(InstanceData_RT->PropertyData.Buffer, 0, InstanceData_RT->PropertyData.NumBytes, RLM_WriteOnly);
					FMemory::Memcpy(GpuMemory, InstanceData_FromGT->PropertyData.GetData(), InstanceData_FromGT->PropertyData.Num());
					RHICmdList.UnlockBuffer(InstanceData_RT->PropertyData.Buffer);
				}
			}
			InstanceData_FromGT->~FInstanceData_GameToRender();
		}

		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
		{
			return sizeof(FInstanceData_GameToRender);
		}
	
		TMap<FNiagaraSystemInstanceID, FInstanceData_RenderThread>	PerInstanceData_RenderThread;
	};

	template<typename TInputType, typename TOutputType>
	void VMReadData(FVectorVMExternalFunctionContext& Context, const uint32 GetterIndex, const TOutputType DefaultValue)
	{
		using namespace NDIUObjectPropertyReaderLocal;

		VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData_GT(Context);
		FNDIOutputParam<bool> SuccessValue(Context);
		FNDIOutputParam<TOutputType> OutValue(Context);
		
		if (InstanceData_GT->PropertyGetters[GetterIndex].PropertyCopyFunction != nullptr)
		{
			const uint32 DataOffset = InstanceData_GT->PropertyGetters[GetterIndex].DataOffset;
			TInputType Value;
			FMemory::Memcpy(&Value, InstanceData_GT->PropertyData.GetData() + DataOffset, sizeof(TInputType));
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				SuccessValue.SetAndAdvance(true);
				OutValue.SetAndAdvance(Value);
			}
		}
		else
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				SuccessValue.SetAndAdvance(false);
				OutValue.SetAndAdvance(DefaultValue);
			}
		}
	}


	template<typename TNiagaraType, typename TPropertyType>
	void CopyPropertyHelper(const void* InPropertyValue, void* Destination)
	{
		const TPropertyType PropertyValue(*reinterpret_cast<const TPropertyType*>(InPropertyValue));
		const TNiagaraType NiagaraValue = TNiagaraType(PropertyValue);
		FMemory::Memcpy(Destination, &NiagaraValue, sizeof(TNiagaraType));
	}

	template<typename TNiagaraType, typename TScriptType>
	void InvokeGetterHelper(UObject* BoundObject, UFunction* BoundFunction, void* Destination, const TScriptType& DefaultValue)
	{
		TScriptType ReturnValue = DefaultValue;
		BoundObject->ProcessEvent(BoundFunction, &ReturnValue);
		const TNiagaraType NiagaraValue = TNiagaraType(ReturnValue);
		FMemory::Memcpy(Destination, &NiagaraValue, sizeof(TNiagaraType));
	}

	template<typename TType>
	struct FTypeHelper : std::false_type
	{
	};

	template<>
	struct FTypeHelper<float> : std::true_type
	{
		static FName GetFunctionName() { return FName("GetFloatProperty"); }
		static FNiagaraTypeDefinition GetTypeDef() { return FNiagaraTypeDefinition::GetFloatDef(); }
		static void VMFunction(FVectorVMExternalFunctionContext& Context, const uint32 GetterIndex) { VMReadData<float, float>(Context, GetterIndex, 0.0f); }
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetCopyFunction(const FProperty* InProperty, const void* PropertyAddress)
		{
			if (const FFloatProperty* FloatProperty = CastField<const FFloatProperty>(InProperty))
			{
				return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<float, float>(PropertyAddress, DestAddress); };
			}
			else if (const FDoubleProperty* DoubleProperty = CastField<const FDoubleProperty>(InProperty))
			{
				return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<float, double>(PropertyAddress, DestAddress); };
			}
			return nullptr;
		}
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetGetterFunction(const FProperty* InProperty, UObject* BoundObject, UFunction* BoundFunction)
		{
			if (CastField<const FFloatProperty>(InProperty))
			{
				return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { BoundObject->ProcessEvent(BoundFunction, DestAddress); };
			}
			else if (CastField<const FDoubleProperty>(InProperty))
			{
				return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<float, double>(BoundObject, BoundFunction, DestAddress, 0.0); };
			}
			return nullptr;
		}
		static constexpr TCHAR const* HlslBufferType = TEXT("float");
		static constexpr TCHAR const* HlslBufferRead = TEXT("asfloat(BUFFER[OFFSET]) : 0.0f");
	};

	template<>
	struct FTypeHelper<FVector2f> : std::true_type
	{
		static FName GetFunctionName() { return FName("GetVec2Property"); }
		static FNiagaraTypeDefinition GetTypeDef() { return FNiagaraTypeDefinition::GetVec2Def(); }
		static void VMFunction(FVectorVMExternalFunctionContext& Context, const uint32 GetterIndex) { VMReadData<FVector2f, FVector2f>(Context, GetterIndex, FVector2f::ZeroVector); }
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetCopyFunction(const FProperty* InProperty, const void* PropertyAddress)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if ( StructProperty->Struct == TBaseStructure<FVector2D>::Get() )
				{
					return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<FVector2f, FVector2D>(PropertyAddress, DestAddress); };
				}
			}
			return nullptr;
		}
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetGetterFunction(const FProperty* InProperty, UObject* BoundObject, UFunction* BoundFunction)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if (StructProperty->Struct == TBaseStructure<FVector2D>::Get())
				{
					return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<FVector2f, FVector2D>(BoundObject, BoundFunction, DestAddress, FVector2D::ZeroVector); };
				}
			}
			return nullptr;
		}
		static constexpr TCHAR const* HlslBufferType = TEXT("float2");
		static constexpr TCHAR const* HlslBufferRead = TEXT("float2(asfloat(BUFFER[OFFSET + 0]), asfloat(BUFFER[OFFSET + 1])) : float2(0.0f, 0.0f)");
	};

	template<>
	struct FTypeHelper<FVector3f> : std::true_type
	{
		static FName GetFunctionName() { return FName("GetVec3Property"); }
		static FNiagaraTypeDefinition GetTypeDef() { return FNiagaraTypeDefinition::GetVec3Def(); }
		static void VMFunction(FVectorVMExternalFunctionContext& Context, const uint32 GetterIndex) { VMReadData<FVector3f, FVector3f>(Context, GetterIndex, FVector3f::ZeroVector); }
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetCopyFunction(const FProperty* InProperty, const void* PropertyAddress)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if (StructProperty->Struct == TBaseStructure<FVector>::Get())
				{
					return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<FVector3f, FVector>(PropertyAddress, DestAddress); };
				}
			}
			return nullptr;
		}
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetGetterFunction(const FProperty* InProperty, UObject* BoundObject, UFunction* BoundFunction)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if (StructProperty->Struct == TBaseStructure<FVector>::Get())
				{
					return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<FVector3f, FVector>(BoundObject, BoundFunction, DestAddress, FVector::ZeroVector); };
				}
			}
			return nullptr;
		}
		static constexpr TCHAR const* HlslBufferType = TEXT("float3");
		static constexpr TCHAR const* HlslBufferRead = TEXT("float3(asfloat(BUFFER[OFFSET + 0]), asfloat(BUFFER[OFFSET + 1]), asfloat(BUFFER[OFFSET + 2])) : float3(0.0f, 0.0f, 0.0f)");
	};

	template<>
	struct FTypeHelper<FVector4f> : std::true_type
	{
		static FName GetFunctionName() { return FName("GetVec4Property"); }
		static FNiagaraTypeDefinition GetTypeDef() { return FNiagaraTypeDefinition::GetVec4Def(); }
		static void VMFunction(FVectorVMExternalFunctionContext& Context, const uint32 GetterIndex) { VMReadData<FVector4f, FVector4f>(Context, GetterIndex, FVector4f::Zero()); }
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetCopyFunction(const FProperty* InProperty, const void* PropertyAddress)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if (StructProperty->Struct == TBaseStructure<FVector4>::Get())
				{
					return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<FVector4f, FVector4>(PropertyAddress, DestAddress); };
				}
			}
			return nullptr;
		}
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetGetterFunction(const FProperty* InProperty, UObject* BoundObject, UFunction* BoundFunction)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if (StructProperty->Struct == TBaseStructure<FVector4>::Get())
				{
					return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<FVector4f, FVector4>(BoundObject, BoundFunction, DestAddress, FVector4::Zero()); };
				}
			}
			return nullptr;
		}
		static constexpr TCHAR const* HlslBufferType = TEXT("float4");
		static constexpr TCHAR const* HlslBufferRead = TEXT("float4(asfloat(BUFFER[OFFSET + 0]), asfloat(BUFFER[OFFSET + 1]), asfloat(BUFFER[OFFSET + 2]), asfloat(BUFFER[OFFSET + 3])) : float4(0.0f, 0.0f, 0.0f, 0.0f)");
	};

	template<>
	struct FTypeHelper<FLinearColor> : std::true_type
	{
		static FName GetFunctionName() { return FName("GetColorProperty"); }
		static FNiagaraTypeDefinition GetTypeDef() { return FNiagaraTypeDefinition::GetColorDef(); }
		static void VMFunction(FVectorVMExternalFunctionContext& Context, const uint32 GetterIndex) { VMReadData<FLinearColor, FLinearColor>(Context, GetterIndex, FLinearColor::Black); }
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetCopyFunction(const FProperty* InProperty, const void* PropertyAddress)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if (StructProperty->Struct == TBaseStructure<FColor>::Get())
				{
					return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<FLinearColor, FColor>(PropertyAddress, DestAddress); };
				}
				else if (StructProperty->Struct == TBaseStructure<FLinearColor>::Get())
				{
					return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<FLinearColor, FLinearColor>(PropertyAddress, DestAddress); };
				}
			}
			return nullptr;
		}
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetGetterFunction(const FProperty* InProperty, UObject* BoundObject, UFunction* BoundFunction)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if (StructProperty->Struct == TBaseStructure<FColor>::Get())
				{
					return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<FLinearColor, FColor>(BoundObject, BoundFunction, DestAddress, FColor::Black); };
				}
				else if (StructProperty->Struct == TBaseStructure<FLinearColor>::Get())
				{
					return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<FLinearColor, FLinearColor>(BoundObject, BoundFunction, DestAddress, FLinearColor::Black); };
				}
			}
			return nullptr;
		}
		static constexpr TCHAR const* HlslBufferType = TEXT("float4");
		static constexpr TCHAR const* HlslBufferRead = TEXT("float4(asfloat(BUFFER[OFFSET + 0]), asfloat(BUFFER[OFFSET + 1]), asfloat(BUFFER[OFFSET + 2]), asfloat(BUFFER[OFFSET + 3])) : float4(0.0f, 0.0f, 0.0f, 0.0f)");
	};

	template<>
	struct FTypeHelper<FNiagaraPosition> : std::true_type
	{
		static FName GetFunctionName() { return FName("GetPositionProperty"); }
		static FNiagaraTypeDefinition GetTypeDef() { return FNiagaraTypeDefinition::GetPositionDef(); }
		static void VMFunction(FVectorVMExternalFunctionContext& Context, const uint32 GetterIndex) { VMReadData<FNiagaraPosition, FNiagaraPosition>(Context, GetterIndex, FNiagaraPosition(ForceInit)); }
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetCopyFunction(const FProperty* InProperty, const void* PropertyAddress)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if (StructProperty->Struct == TBaseStructure<FVector>::Get())
				{
					return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { *reinterpret_cast<FNiagaraPosition*>(DestAddress) = LwcConverter.ConvertWorldToSimulationPosition(*reinterpret_cast<const FVector*>(PropertyAddress)); };
				}
			}
			return nullptr;
		}
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetGetterFunction(const FProperty* InProperty, UObject* BoundObject, UFunction* BoundFunction)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if (StructProperty->Struct == TBaseStructure<FVector>::Get())
				{
					return
						[BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress)
						{
							FVector ReturnValue = FVector::ZeroVector;
							BoundObject->ProcessEvent(BoundFunction, &ReturnValue);
							const FVector3f NiagaraValue = LwcConverter.ConvertWorldToSimulationPosition(ReturnValue);
							FMemory::Memcpy(DestAddress, &NiagaraValue, sizeof(FVector3f));
						};
				}
			}
			return nullptr;
		}
		static constexpr TCHAR const* HlslBufferType = TEXT("float3");
		static constexpr TCHAR const* HlslBufferRead = TEXT("float3(asfloat(BUFFER[OFFSET + 0]), asfloat(BUFFER[OFFSET + 1]), asfloat(BUFFER[OFFSET + 2])) : float3(0.0f, 0.0f, 0.0f)");
	};

	template<>
	struct FTypeHelper<int32> : std::true_type
	{
		static FName GetFunctionName() { return FName("GetIntProperty"); }
		static FNiagaraTypeDefinition GetTypeDef() { return FNiagaraTypeDefinition::GetIntDef(); }
		static void VMFunction(FVectorVMExternalFunctionContext& Context, const uint32 GetterIndex) { VMReadData<int32, int32>(Context, GetterIndex, 0); }
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetCopyFunction(const FProperty* InProperty, const void* PropertyAddress)
		{
			if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
			{
				InProperty = EnumProperty->GetUnderlyingProperty();
			}

			if (InProperty->IsA<const FByteProperty>())
			{
				return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<int32, uint8>(PropertyAddress, DestAddress); };
			}
			else if (InProperty->IsA<const FUInt16Property>())
			{
				return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<int32, uint16>(PropertyAddress, DestAddress); };
			}
			else if (InProperty->IsA<const FUInt32Property>())
			{
				return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<int32, uint32>(PropertyAddress, DestAddress); };
			}
			else if (InProperty->IsA<const FInt16Property>())
			{
				return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<int32, int16>(PropertyAddress, DestAddress); };
			}
			else if (InProperty->IsA<const FIntProperty>())
			{
				return [PropertyAddress](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { CopyPropertyHelper<int32, int32>(PropertyAddress, DestAddress); };
			}
			return nullptr;
		}
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetGetterFunction(const FProperty* InProperty, UObject* BoundObject, UFunction* BoundFunction)
		{
			if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
			{
				InProperty = EnumProperty->GetUnderlyingProperty();
			}

			if (InProperty->IsA<const FByteProperty>())
			{
				return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<int32, uint8>(BoundObject, BoundFunction, DestAddress, 0); };
			}
			else if (InProperty->IsA<const FUInt16Property>())
			{
				return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<int32, uint16>(BoundObject, BoundFunction, DestAddress, 0); };
			}
			else if (InProperty->IsA<const FUInt32Property>())
			{
				return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<int32, uint32>(BoundObject, BoundFunction, DestAddress, 0); };
			}
			else if (InProperty->IsA<const FInt16Property>())
			{
				return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<int32, uint16>(BoundObject, BoundFunction, DestAddress, 0); };
			}
			else if (InProperty->IsA<const FIntProperty>())
			{
				return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<int32, int32>(BoundObject, BoundFunction, DestAddress, 0); };
			}
			return nullptr;
		}
		static constexpr TCHAR const* HlslBufferType = TEXT("int");
		static constexpr TCHAR const* HlslBufferRead = TEXT("asint(BUFFER[OFFSET]) : 0");
	};

	template<>
	struct FTypeHelper<bool> : std::true_type
	{
		static FName GetFunctionName() { return FName("GetBoolProperty"); }
		static FNiagaraTypeDefinition GetTypeDef() { return FNiagaraTypeDefinition::GetBoolDef(); }
		static void VMFunction(FVectorVMExternalFunctionContext& Context, const uint32 GetterIndex) { VMReadData<FNiagaraBool, FNiagaraBool>(Context, GetterIndex, 0.0f); }
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetCopyFunction(const FProperty* InProperty, const void* PropertyAddress)
		{
			if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(InProperty))
			{
				return [PropertyAddress, BoolProperty](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { *reinterpret_cast<FNiagaraBool*>(DestAddress) = BoolProperty->GetPropertyValue(PropertyAddress); };
			}
			return nullptr;
		}
		static TFunction<void(const FNiagaraLWCConverter&, void*)> GetGetterFunction(const FProperty* InProperty, UObject* BoundObject, UFunction* BoundFunction)
		{
			if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(InProperty))
			{
				return [BoundObject, BoundFunction](const FNiagaraLWCConverter& LwcConverter, void* DestAddress) { InvokeGetterHelper<FNiagaraBool, bool>(BoundObject, BoundFunction, DestAddress, false); };
			}
			return nullptr;
		}
		static constexpr TCHAR const* HlslBufferType = TEXT("bool");
		static constexpr TCHAR const* HlslBufferRead = TEXT("asint(BUFFER[OFFSET]) != 0 : 0");
	};

	#define NDI_PROPERTY_TYPES \
		NDI_PROPERTY_TYPE(float) \
		NDI_PROPERTY_TYPE(FVector2f) \
		NDI_PROPERTY_TYPE(FVector3f) \
		NDI_PROPERTY_TYPE(FVector4f) \
		NDI_PROPERTY_TYPE(FLinearColor) \
		NDI_PROPERTY_TYPE(FNiagaraPosition) \
		NDI_PROPERTY_TYPE(int32) \
		NDI_PROPERTY_TYPE(bool) \
		/*NDI_PROPERTY_TYPE(FQuat4f)*/ \
		/*NDI_PROPERTY_TYPE(FMatrix44f)*/ \

	void BindPropertyGetter(FNDIPropertyGetter& PropertyGetter, UObject* ObjectBinding)
	{
		PropertyGetter.WeakProperty.Reset();
		PropertyGetter.PropertyCopyFunction = nullptr;

		TStringBuilder<128> VariablePathSB;
		PropertyGetter.Variable.GetName().ToString(VariablePathSB);

		FStringView VariablePath = VariablePathSB.ToView();

		UStruct* ObjectClass = ObjectBinding->GetClass();
		void* ObjectAddress = ObjectBinding;

		// Look for a getter function, i.e. one with a single return value
		if (UFunction* GetterFunction = ObjectBinding->FindFunction(PropertyGetter.Variable.GetName()))
		{
			FProperty* ReturnProperty = nullptr;
			if (GetterFunction->NumParms == 1 && GetterFunction->PropertyLink != nullptr && GetterFunction->PropertyLink->HasAnyPropertyFlags(CPF_OutParm))
			{
				ReturnProperty = GetterFunction->PropertyLink;
			}
			//-TODO: Support native function path??
			//else if (GetterFunction->NumParms == 0)
			//{
			//	ReturnProperty = GetterFunction->GetReturnProperty();
			//}

			if (ReturnProperty)
			{
				#define NDI_PROPERTY_TYPE(TYPE) \
					else if (PropertyGetter.Variable.GetType() == FTypeHelper<TYPE>::GetTypeDef()) \
					{ \
						PropertyGetter.PropertyCopyFunction = FTypeHelper<TYPE>::GetGetterFunction(ReturnProperty, ObjectBinding, GetterFunction); \
						if (PropertyGetter.PropertyCopyFunction == nullptr && FNiagaraUtilities::LogVerboseWarnings()) \
						{ \
							const FStructProperty* StructProperty = CastField<FStructProperty>(ReturnProperty); \
							const FString PropertyType = StructProperty && StructProperty->Struct ? StructProperty->Struct->GetName() : ReturnProperty->GetClass()->GetName(); \
							UE_LOGF(LogNiagara, Warning, "Could convert return property for function '%ls' type '%ls' into expected Niagara type '%ls'", *PropertyGetter.Variable.GetName().ToString(), *PropertyType, *PropertyGetter.Variable.GetType().GetName()); \
						} \
					} \

					if (false) { } NDI_PROPERTY_TYPES
				#undef NDI_PROPERTY_TYPE
			}

			// Complete as BP can not have both a function and variable of the same type
			return;
		}

		// Look for a property
		while ( true )
		{
			FStringView PropertyName = VariablePath;
			int32 DotIndex = INDEX_NONE;
			if (VariablePath.FindChar('.', DotIndex))
			{
				PropertyName = VariablePath.Mid(0, DotIndex);
				VariablePath = VariablePath.Mid(DotIndex + 1);
			}

			// Attempt to find property
			FName PropertyFName(PropertyName, FNAME_Find);
			FProperty* Property = FindFProperty<FProperty>(ObjectClass, PropertyFName);
			if ( Property == nullptr )
			{
				return;
			}

			// Sub property?  Must be a structure of we are done
			if ( DotIndex != INDEX_NONE )
			{
				const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
				if (StructProperty == nullptr)
				{
					return;
				}

				ObjectClass = StructProperty->Struct;
				ObjectAddress = StructProperty->ContainerPtrToValuePtr<void>(ObjectAddress);
				continue;
			}

			// This is the final property
			void* PropertyAddress = Property->ContainerPtrToValuePtr<void>(ObjectAddress);

			#define NDI_PROPERTY_TYPE(TYPE) \
				else if (PropertyGetter.Variable.GetType() == FTypeHelper<TYPE>::GetTypeDef()) \
				{ \
					PropertyGetter.PropertyCopyFunction = FTypeHelper<TYPE>::GetCopyFunction(Property, PropertyAddress); \
					if (PropertyGetter.PropertyCopyFunction == nullptr && FNiagaraUtilities::LogVerboseWarnings()) \
					{ \
						const FStructProperty* StructProperty = CastField<FStructProperty>(Property); \
						const FString PropertyType = StructProperty && StructProperty->Struct ? StructProperty->Struct->GetName() : Property->GetClass()->GetName(); \
						UE_LOGF(LogNiagara, Warning, "Could not copy property '%ls' type '%ls' into expected Niagara type '%ls'", *Property->GetName(), *PropertyType, *PropertyGetter.Variable.GetType().GetName()); \
					} \
				} \

				if (false) { } NDI_PROPERTY_TYPES
			#undef NDI_PROPERTY_TYPE

			// Complete
			return;
		}
	}
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceUObjectPropertyReader::UNiagaraDataInterfaceUObjectPropertyReader(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDIUObjectPropertyReaderLocal;

	FNiagaraTypeDefinition Def(UObject::StaticClass());
	UObjectParameterBinding.Parameter.SetType(Def);

	Proxy.Reset(new FNDIProxy());
}

void UNiagaraDataInterfaceUObjectPropertyReader::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceUObjectPropertyReader::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	auto* OtherTyped = CastChecked<const UNiagaraDataInterfaceUObjectPropertyReader>(Other);
	return
		OtherTyped->SourceMode == SourceMode &&
		OtherTyped->UObjectParameterBinding == UObjectParameterBinding &&
		OtherTyped->PropertyRemap == PropertyRemap &&
		OtherTyped->SourceActor == SourceActor &&
		OtherTyped->SourceActorComponentClass == SourceActorComponentClass;
}

bool UNiagaraDataInterfaceUObjectPropertyReader::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	auto* DestinationTyped = CastChecked<UNiagaraDataInterfaceUObjectPropertyReader>(Destination);
	DestinationTyped->SourceMode = SourceMode;
	DestinationTyped->UObjectParameterBinding = UObjectParameterBinding;
	DestinationTyped->PropertyRemap = PropertyRemap;
	DestinationTyped->SourceActor = SourceActor;
	DestinationTyped->SourceActorComponentClass = SourceActorComponentClass;

	return true;
}

bool UNiagaraDataInterfaceUObjectPropertyReader::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIUObjectPropertyReaderLocal;

	FInstanceData_GameThread* InstanceData_GT = new(PerInstanceData) FInstanceData_GameThread();
	InstanceData_GT->UObjectBinding.Init(SystemInstance->GetInstanceParameters(), UObjectParameterBinding.Parameter);
	InstanceData_GT->ChangeId = ChangeId;

	if ( IsUsedWithGPUScript() )
	{
		TArray<uint32> GpuFunctionToPropertyRemap;

		// We shouldn't need to do this per init, we should be able to cache once and once only
		for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
		{
			if (EmitterInstance->IsDisabled() || EmitterInstance->GetEmitter() == nullptr || EmitterInstance->GetSimTarget() != ENiagaraSimTarget::GPUComputeSim)
			{
				continue;
			}
			const FNiagaraScriptInstanceParameterStore& ParameterStore = EmitterInstance->GetGPUContext()->CombinedParamStore;
			const TArray<UNiagaraDataInterface*>& DataInterfaces = ParameterStore.GetDataInterfaces();
			const TSharedRef<FNiagaraShaderScriptParametersMetadata> ScriptParametersMetadata = EmitterInstance->GetGPUContext()->GPUScript_RT->GetScriptParametersMetadata();
			const TArray<FNiagaraDataInterfaceGPUParamInfo>& DataInterfaceParamInfo = ScriptParametersMetadata->DataInterfaceParamInfo;
			for ( int32 iDataInterface=0; iDataInterface < DataInterfaces.Num(); ++iDataInterface)
			{
				if ( (DataInterfaces[iDataInterface] == this) && DataInterfaceParamInfo.IsValidIndex(iDataInterface) )
				{
					const TArray<FNiagaraDataInterfaceGeneratedFunction>& GeneratedFunctions = DataInterfaceParamInfo[iDataInterface].GeneratedFunctions;
					GpuFunctionToPropertyRemap.AddZeroed(GeneratedFunctions.Num());
					for ( int32 iFunctionInfo=0; iFunctionInfo < GeneratedFunctions.Num(); ++iFunctionInfo )
					{
						const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo = GeneratedFunctions[iFunctionInfo];
						GpuFunctionToPropertyRemap[iFunctionInfo] = INDEX_NONE;
						#define NDI_PROPERTY_TYPE(TYPE) \
							else if (FunctionInfo.DefinitionName == FTypeHelper<TYPE>::GetFunctionName()) { GpuFunctionToPropertyRemap[iFunctionInfo] = InstanceData_GT->AddProperty(FNiagaraVariableBase(FTypeHelper<TYPE>::GetTypeDef(), GetRemappedPropertyName(FunctionInfo.Specifiers[0].Value))); }

							if (false) {} NDI_PROPERTY_TYPES
						#undef NDI_PROPERTY_TYPE
					}
				}
			}
		}

		// Initialize render side instance data
		ENQUEUE_RENDER_COMMAND(NDIUObjectPropertyReader_InitRT)
		(
			[Proxy_RT=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId(), GpuFunctionToPropertyRemap_RT=MoveTemp(GpuFunctionToPropertyRemap)](FRHICommandList& CmdList)
			{
				FInstanceData_RenderThread* InstanceData_RT = &Proxy_RT->PerInstanceData_RenderThread.Add(InstanceID);
				InstanceData_RT->PropertyOffsets.AddUninitialized(Align(GpuFunctionToPropertyRemap_RT.Num(), 4));
				FMemory::Memset(InstanceData_RT->PropertyOffsets.GetData(), 0xff, InstanceData_RT->PropertyOffsets.GetAllocatedSize());
				InstanceData_RT->GpuFunctionToPropertyRemap = GpuFunctionToPropertyRemap_RT;
			}
		);
	}

	return true;
}

void UNiagaraDataInterfaceUObjectPropertyReader::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIUObjectPropertyReaderLocal;

	FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
	InstanceData_GT->~FInstanceData_GameThread();

	if ( IsUsedWithGPUScript() )
	{
		ENQUEUE_RENDER_COMMAND(NDIUObjectPropertyReader_InitRT)
		(
			[Proxy_RT=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandList& CmdList)
			{
				Proxy_RT->PerInstanceData_RenderThread.Remove(InstanceID);
			}
		);
	}
}

int32 UNiagaraDataInterfaceUObjectPropertyReader::PerInstanceDataSize() const
{
	using namespace NDIUObjectPropertyReaderLocal;
	return sizeof(FInstanceData_GameThread);
}

void UNiagaraDataInterfaceUObjectPropertyReader::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID)
{
	using namespace NDIUObjectPropertyReaderLocal;
	FNDIProxy* DIProxy = GetProxyAs<FNDIProxy>();
	DIProxy->ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, InstanceID);
}

bool UNiagaraDataInterfaceUObjectPropertyReader::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIUObjectPropertyReaderLocal;

	check(PerInstanceData && SystemInstance);

	FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);

	// Do we need to reinitialize the data interface?
	if ( InstanceData_GT->ChangeId != ChangeId )
	{
		return true;
	}

	// Resolve our source object
	UObject* ObjectBinding = nullptr;
	{
		// Try Object Binding?
		const bool bTryObjectBinding =
			SourceMode == ENDIObjectPropertyReaderSourceMode::Binding ||
			SourceMode == ENDIObjectPropertyReaderSourceMode::BindingThenAttachParentActor ||
			SourceMode == ENDIObjectPropertyReaderSourceMode::BindingThenAttachParent;
		if (bTryObjectBinding)
		{
			ObjectBinding = InstanceData_GT->UObjectBinding.GetValue();
			ObjectBinding = ObjectBinding ? ObjectBinding : SourceActor.Get();
		}

		// If that failed do we look at the parent / parent actor?
		if (ObjectBinding == nullptr)
		{
			if (SourceMode == ENDIObjectPropertyReaderSourceMode::AttachParent || SourceMode == ENDIObjectPropertyReaderSourceMode::BindingThenAttachParent )
			{
				ObjectBinding = SystemInstance->GetAttachComponent();
			}
			else if (SourceMode == ENDIObjectPropertyReaderSourceMode::AttachParentActor || SourceMode == ENDIObjectPropertyReaderSourceMode::BindingThenAttachParentActor)
			{
				if (USceneComponent* AttachComponent = SystemInstance->GetAttachComponent())
				{
					ObjectBinding = AttachComponent->GetAttachParentActor();
				}
			}
		}
	}

	// Do we need to rebind the reader?
	if ( InstanceData_GT->WeakUObject.Get() != ObjectBinding )
	{
		InstanceData_GT->WeakUObject = ObjectBinding;

		// If the object we are binding to is an actor we will get the root component / specified component class
		UObject* ActorComponent = nullptr;
		if ( AActor* ObjectActor = Cast<AActor>(ObjectBinding) )
		{
			if ( SourceActorComponentClass != nullptr )
			{
				ActorComponent = ObjectActor->FindComponentByClass(SourceActorComponentClass);
			}
			else
			{
				ActorComponent = ObjectActor->GetRootComponent();
			}
		}

		for (FNDIPropertyGetter& PropertyGetter : InstanceData_GT->PropertyGetters)
		{
			PropertyGetter.WeakProperty.Reset();
			PropertyGetter.PropertyCopyFunction = nullptr;

			if ( ObjectBinding != nullptr )
			{
				BindPropertyGetter(PropertyGetter, ObjectBinding);
				if (PropertyGetter.PropertyCopyFunction == nullptr && ActorComponent != nullptr)
				{
					BindPropertyGetter(PropertyGetter, ActorComponent);
				}

				if (PropertyGetter.PropertyCopyFunction == nullptr && FNiagaraUtilities::LogVerboseWarnings())
				{
					UE_LOGF(LogNiagara, Warning, "Could not find property '%ls' inside object '%ls' or component '%ls'", *PropertyGetter.Variable.GetName().ToString(), *GetNameSafe(ObjectBinding), *GetNameSafe(ActorComponent));
				}
			}
		}
	}

	// Update our data store as we can not read object's async it's unsafe
	InstanceData_GT->CachedTransform.Reset();
	InstanceData_GT->CachedInvTransform.Reset();
	if (ObjectBinding != nullptr)
	{
		// Update transform
		if (AActor* ObjectActor = Cast<AActor>(ObjectBinding))
		{
			USceneComponent* ActorComponent = Cast<USceneComponent>(SourceActorComponentClass ? ObjectActor->FindComponentByClass(SourceActorComponentClass) : ObjectActor->GetRootComponent());
			InstanceData_GT->CachedTransform = ActorComponent ? ActorComponent->GetComponentToWorld() : ObjectActor->GetTransform();
			InstanceData_GT->CachedInvTransform = InstanceData_GT->CachedTransform->Inverse();
		}
		else if ( USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectBinding) )
		{
			InstanceData_GT->CachedTransform = SceneComponent->GetComponentToWorld();
			InstanceData_GT->CachedInvTransform = InstanceData_GT->CachedTransform->Inverse();
		}

		// Update properties
		FNiagaraLWCConverter LwcConverter = SystemInstance->GetLWCConverter(false);
		for (FNDIPropertyGetter& PropertyGetter : InstanceData_GT->PropertyGetters )
		{
			if (PropertyGetter.PropertyCopyFunction != nullptr)
			{
				PropertyGetter.PropertyCopyFunction(LwcConverter, InstanceData_GT->PropertyData.GetData() + PropertyGetter.DataOffset);
			}
		}
	}

	return false;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceUObjectPropertyReader::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIUObjectPropertyReaderLocal;

	// Reserve space
	{
		int32 NumFunctions = 0;
		#define NDI_PROPERTY_TYPE(TYPE) ++NumFunctions;
			NDI_PROPERTY_TYPES
		#undef NDI_PROPERTY_TYPE
		OutFunctions.Reserve(OutFunctions.Num() + NumFunctions);
	}

	// Build default signature
	FNiagaraFunctionSignature DefaultSignature;
	DefaultSignature.bMemberFunction = true;
	DefaultSignature.bRequiresContext = false;
	DefaultSignature.bSupportsCPU = true;
	DefaultSignature.bSupportsGPU = true;
	DefaultSignature.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("UObjectReader"));
	DefaultSignature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success"));
	DefaultSignature.FunctionSpecifiers.Emplace("PropertyName");

	// Utility functions
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetComponentTransformName;
		Sig.FunctionSpecifiers.Empty();
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale"));
		Sig.SetDescription(LOCTEXT("GetComponentTransformDesc", "If the object we are bound to is an actor it will return root component transform, or the component class we bound to"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature);
		Sig.Name = GetComponentInvTransformName;
		Sig.FunctionSpecifiers.Empty();
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale"));
		Sig.SetDescription(LOCTEXT("GetComponentInvTransformDesc", "If the object we are bound to is an actor it will return root component inverse transform, or the component class we bound to"));
	}

	// Build property function list
	#define NDI_PROPERTY_TYPE(TYPE) \
		{ \
			FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSignature); \
			Sig.Name = FTypeHelper<TYPE>::GetFunctionName(); \
			Sig.Outputs.Emplace(FTypeHelper<TYPE>::GetTypeDef(), TEXT("Value")); \
			/*Sig.SetDescription(FTypeHelper<TYPE>::GetFunctionDesc());*/ \
		} \

		NDI_PROPERTY_TYPES
	#undef NDI_PROPERTY_TYPE
}
#endif

void UNiagaraDataInterfaceUObjectPropertyReader::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* PerInstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIUObjectPropertyReaderLocal;

	// Ensure we force a rebind
	FInstanceData_GameThread* InstanceData_GT = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);
	InstanceData_GT->WeakUObject = nullptr;

	if ( BindingInfo.Name == GetComponentTransformName )
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceUObjectPropertyReader::VMGetComponentTransform);
	}
	else if (BindingInfo.Name == GetComponentInvTransformName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceUObjectPropertyReader::VMGetComponentInvTransform);
	}
	// Bind property functions
	#define NDI_PROPERTY_TYPE(TYPE) \
		else if (BindingInfo.Name == FTypeHelper<TYPE>::GetFunctionName()) \
		{ \
			const uint32 GetterIndex = InstanceData_GT->AddProperty(FNiagaraVariableBase(FTypeHelper<TYPE>::GetTypeDef(), GetRemappedPropertyName(BindingInfo.FunctionSpecifiers[0].Value))); \
			OutFunc.BindStatic(&FTypeHelper<TYPE>::VMFunction, GetterIndex); \
		} \

		NDI_PROPERTY_TYPES
	#undef NDI_PROPERTY_TYPE
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceUObjectPropertyReader::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceUObjectPropertyReader::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, FString& OutHLSL)
{
	OutHLSL.Appendf(TEXT("float3	%s_TransformLocation;\n"), *ParameterInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("uint		%s_TransformValid;\n"), *ParameterInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("float4	%s_TransformRotation;\n"), *ParameterInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("float3	%s_TransformScale;\n"), *ParameterInfo.DataInterfaceHLSLSymbol);

	OutHLSL.Appendf(TEXT("float3	%s_InvTransformLocation;\n"), *ParameterInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("uint		%s_InvTransformValid;\n"), *ParameterInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("float4	%s_InvTransformRotation;\n"), *ParameterInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("float3	%s_InvTransformScale;\n"), *ParameterInfo.DataInterfaceHLSLSymbol);

	OutHLSL.Appendf(TEXT("Buffer<uint>	%s_PropertyData;\n"), *ParameterInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("uint4			%s_PropertyOffsets[%d];\n"), *ParameterInfo.DataInterfaceHLSLSymbol, FMath::DivideAndRoundUp(ParameterInfo.GeneratedFunctions.Num(), 4));
}

bool UNiagaraDataInterfaceUObjectPropertyReader::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIUObjectPropertyReaderLocal;

	const TCHAR* HlslBufferType = nullptr;
	const TCHAR* HlslBufferRead = nullptr;

	if (FunctionInfo.DefinitionName == GetComponentTransformName )
	{
		OutHLSL.Appendf(TEXT("void %s(out bool bSuccess, out float3 Position, out float4 Rotation, out float3 Scale)\n"), *FunctionInfo.InstanceName);
		OutHLSL.Append(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	bSuccess = %s_TransformValid != 0;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
		OutHLSL.Appendf(TEXT("	Position = %s_TransformLocation;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
		OutHLSL.Appendf(TEXT("	Rotation = %s_TransformRotation;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
		OutHLSL.Appendf(TEXT("	Scale = %s_TransformScale;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
		OutHLSL.Append(TEXT("}\n"));
		return true;
	}
	if (FunctionInfo.DefinitionName == GetComponentInvTransformName)
	{
		OutHLSL.Appendf(TEXT("void %s(out bool bSuccess, out float3 Position, out float4 Rotation, out float3 Scale)\n"), *FunctionInfo.InstanceName);
		OutHLSL.Append(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	bSuccess = %s_InvTransformValid != 0;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
		OutHLSL.Appendf(TEXT("	Position = %s_InvTransformLocation;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
		OutHLSL.Appendf(TEXT("	Rotation = %s_InvTransformRotation;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
		OutHLSL.Appendf(TEXT("	Scale = %s_InvTransformScale;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
		OutHLSL.Append(TEXT("}\n"));
		return true;
	}
#define NDI_PROPERTY_TYPE(TYPE) \
		else if (FunctionInfo.DefinitionName == FTypeHelper<TYPE>::GetFunctionName()) \
		{ \
			HlslBufferType = FTypeHelper<TYPE>::HlslBufferType; \
			HlslBufferRead = FTypeHelper<TYPE>::HlslBufferRead; \
		} \

		NDI_PROPERTY_TYPES
	#undef NDI_PROPERTY_TYPE

	if ( HlslBufferType && HlslBufferRead )
	{
		FString HlslBufferReadString(HlslBufferRead);
		HlslBufferReadString.ReplaceInline(TEXT("BUFFER"), *(ParamInfo.DataInterfaceHLSLSymbol + TEXT("_PropertyData")));
		HlslBufferReadString.ReplaceInline(TEXT("OFFSET"), TEXT("BufferOffset"));

		OutHLSL.Appendf(TEXT("void %s(out bool bSuccess, out %s Value)\n"), *FunctionInfo.InstanceName, HlslBufferType);
		OutHLSL.Append(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	uint BufferOffset = %s_PropertyOffsets[%d][%d];\n"), *ParamInfo.DataInterfaceHLSLSymbol, FunctionInstanceIndex / 4, FunctionInstanceIndex % 4);
		OutHLSL.Append(TEXT("	bSuccess = BufferOffset != 0xffffffff;\n"));
		OutHLSL.Appendf(TEXT("	Value = bSuccess ? %s;\n"), *HlslBufferReadString);
		OutHLSL.Append(TEXT("}\n"));
		return true;
	}

	return false;
}
#endif

void UNiagaraDataInterfaceUObjectPropertyReader::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();

	const int32 PropertyOffsetsSize = FMath::Max(FMath::DivideAndRoundUp(ShaderParametersBuilder.GetGeneratedFunctions().Num(), 4), 1);
	ShaderParametersBuilder.AddLooseParamArray<FUintVector4>(TEXT("PropertyOffsets"), PropertyOffsetsSize);
}

void UNiagaraDataInterfaceUObjectPropertyReader::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIUObjectPropertyReaderLocal;

	const FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	const FInstanceData_RenderThread& InstanceData_RT = DIProxy.PerInstanceData_RenderThread.FindChecked(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters		= Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->TransformLocation		= InstanceData_RT.CachedTransform.IsSet() ? FVector3f(InstanceData_RT.CachedTransform->GetLocation()) : FVector3f::ZeroVector;
	ShaderParameters->TransformValid		= InstanceData_RT.CachedTransform.IsSet() ? 1 : 0;
	ShaderParameters->TransformRotation		= InstanceData_RT.CachedTransform.IsSet() ? FQuat4f(InstanceData_RT.CachedTransform->GetRotation()) : FQuat4f::Identity;
	ShaderParameters->TransformScale		= InstanceData_RT.CachedTransform.IsSet() ? FVector3f(InstanceData_RT.CachedTransform->GetScale3D()) : FVector3f::OneVector;
	ShaderParameters->InvTransformLocation	= InstanceData_RT.CachedInvTransform.IsSet() ? FVector3f(InstanceData_RT.CachedInvTransform->GetLocation()) : FVector3f::ZeroVector;
	ShaderParameters->InvTransformValid		= InstanceData_RT.CachedInvTransform.IsSet() ? 1 : 0;
	ShaderParameters->InvTransformRotation	= InstanceData_RT.CachedInvTransform.IsSet() ? FQuat4f(InstanceData_RT.CachedInvTransform->GetRotation()) : FQuat4f::Identity;
	ShaderParameters->InvTransformScale		= InstanceData_RT.CachedInvTransform.IsSet() ? FVector3f(InstanceData_RT.CachedInvTransform->GetScale3D()) : FVector3f::OneVector;
	ShaderParameters->PropertyData			= InstanceData_RT.PropertyData.SRV;

	check((InstanceData_RT.PropertyOffsets.Num() % 4) == 0 && InstanceData_RT.PropertyOffsets.Num() > 0);
	const int32 PropertyOffsetsSize = InstanceData_RT.PropertyOffsets.Num() / 4;
	TArrayView<FUintVector4> PropertyOffsets = Context.GetParameterLooseArray<FUintVector4>(PropertyOffsetsSize);
	FMemory::Memcpy(PropertyOffsets.GetData(), InstanceData_RT.PropertyOffsets.GetData(), InstanceData_RT.PropertyOffsets.Num() * InstanceData_RT.PropertyOffsets.GetTypeSize());
}

void UNiagaraDataInterfaceUObjectPropertyReader::SetUObjectReaderPropertyRemap(UNiagaraComponent* NiagaraComponent, FName UserParameterName, FName GraphName, FName RemapName)
{
	UNiagaraDataInterfaceUObjectPropertyReader* ReaderDI = UNiagaraFunctionLibrary::GetDataInterface<UNiagaraDataInterfaceUObjectPropertyReader>(NiagaraComponent, UserParameterName);
	if ( ReaderDI == nullptr )
	{
		return;
	}

	FNiagaraUObjectPropertyReaderRemap* RemapEntry = ReaderDI->PropertyRemap.FindByPredicate([GraphName](const FNiagaraUObjectPropertyReaderRemap& Entry) { return Entry.GraphName == GraphName; });
	if (RemapEntry == nullptr)
	{
		RemapEntry = &ReaderDI->PropertyRemap.AddDefaulted_GetRef();
		RemapEntry->GraphName = GraphName;
	}
	RemapEntry->RemapName = RemapName;

	// Notify changed so all user recache
	++ReaderDI->ChangeId;
}

void UNiagaraDataInterfaceUObjectPropertyReader::VMGetComponentTransform(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIUObjectPropertyReaderLocal;

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData_GT(Context);
	FNDIOutputParam<bool> OutValid(Context);
	FNDIOutputParam<FNiagaraPosition> OutPosition(Context);
	FNDIOutputParam<FQuat4f> OutRotation(Context);
	FNDIOutputParam<FVector3f> OutScale(Context);

	const bool bTransformValid			= InstanceData_GT->CachedTransform.IsSet();
	const FVector3f TransformPosition	= bTransformValid ? FVector3f(InstanceData_GT->CachedTransform->GetLocation()) : FVector3f::ZeroVector;
	const FQuat4f TransformRotation		= bTransformValid ? FQuat4f(InstanceData_GT->CachedTransform->GetRotation()) : FQuat4f::Identity;
	const FVector3f TransformScale		= bTransformValid ? FVector3f(InstanceData_GT->CachedTransform->GetScale3D()) : FVector3f::OneVector;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutValid.SetAndAdvance(bTransformValid);
		OutPosition.SetAndAdvance(TransformPosition);
		OutRotation.SetAndAdvance(TransformRotation);
		OutScale.SetAndAdvance(TransformScale);
	}
}

void UNiagaraDataInterfaceUObjectPropertyReader::VMGetComponentInvTransform(FVectorVMExternalFunctionContext& Context)
{
	using namespace NDIUObjectPropertyReaderLocal;

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstanceData_GT(Context);
	FNDIOutputParam<bool> OutValid(Context);
	FNDIOutputParam<FNiagaraPosition> OutPosition(Context);
	FNDIOutputParam<FQuat4f> OutRotation(Context);
	FNDIOutputParam<FVector3f> OutScale(Context);

	const bool bInvTransformValid = InstanceData_GT->CachedInvTransform.IsSet();
	const FVector3f InvTransformPosition = bInvTransformValid ? FVector3f(InstanceData_GT->CachedInvTransform->GetLocation()) : FVector3f::ZeroVector;
	const FQuat4f InvTransformRotation = bInvTransformValid ? FQuat4f(InstanceData_GT->CachedInvTransform->GetRotation()) : FQuat4f::Identity;
	const FVector3f InvTransformScale = bInvTransformValid ? FVector3f(InstanceData_GT->CachedInvTransform->GetScale3D()) : FVector3f::OneVector;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutValid.SetAndAdvance(bInvTransformValid);
		OutPosition.SetAndAdvance(InvTransformPosition);
		OutRotation.SetAndAdvance(InvTransformRotation);
		OutScale.SetAndAdvance(InvTransformScale);
	}
}

#undef NDI_PROPERTY_TYPES
#undef LOCTEXT_NAMESPACE

