// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceUObjectPropertyReader.generated.h"

struct FNiagaraDataInterfaceGeneratedFunction;

UENUM()
enum class ENDIObjectPropertyReaderSourceMode : uint8
{
	/** Source object is found in the order of object binding, source actor. */
	Binding,
	/** Source object is found use the attached parent actor, i.e. the owner components actor. */
	AttachParentActor,
	/** Source object is found in the order of object binding, source actor, then attached parent actor. */
	BindingThenAttachParentActor,
	/** Source object is the attach parent, i.e. the owner component. */
	AttachParent,
	/** Source object is found in the order of object binding, source actor, then attached parent. */
	BindingThenAttachParent
};


USTRUCT()
struct FNiagaraUObjectPropertyReaderRemap
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Remap")
	FName GraphName = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Remap")
	FName RemapName = NAME_None;

	bool operator==(const FNiagaraUObjectPropertyReaderRemap& Other) const
	{
		return GraphName == Other.GraphName && RemapName == Other.RemapName;
	}
};

/**
Data interface to read properties from UObjects.
Rather than having BP tick functions that push data into Niagara this data interface will instead pull them.
*/
UCLASS(EditInlineNew, Category = "DataInterface", CollapseCategories, meta=(DisplayName="Object Reader"))
class UNiagaraDataInterfaceUObjectPropertyReader : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FVector3f,			TransformLocation)
		SHADER_PARAMETER(uint32,			TransformValid)
		SHADER_PARAMETER(FQuat4f,			TransformRotation)
		SHADER_PARAMETER(FVector3f,			TransformScale)

		SHADER_PARAMETER(FVector3f,			InvTransformLocation)
		SHADER_PARAMETER(uint32,			InvTransformValid)
		SHADER_PARAMETER(FQuat4f,			InvTransformRotation)
		SHADER_PARAMETER(FVector3f,			InvTransformScale)

		SHADER_PARAMETER_SRV(Buffer<uint>,	PropertyData)
	END_SHADER_PARAMETER_STRUCT()

public:
	/** Determines how we should select the source object we read from. */
	UPROPERTY(EditAnywhere, Category = "UObjectPropertyReader")
	ENDIObjectPropertyReaderSourceMode SourceMode = ENDIObjectPropertyReaderSourceMode::Binding;

	/** User parameter Object binding to read properties from. */
	UPROPERTY(EditAnywhere, Category = "UObjectPropertyReader", meta = (EditCondition = "SourceMode != ENDIObjectPropertyReaderSourceMode::AttachParent && SourceMode != ENDIObjectPropertyReaderSourceMode::AttachParentActor", EditConditionHides))
	FNiagaraUserParameterBinding UObjectParameterBinding;

	UPROPERTY(EditAnywhere, Category = "UObjectPropertyReader")
	TArray<FNiagaraUObjectPropertyReaderRemap> PropertyRemap;

	/** Optional source actor to use, if the user parameter binding is valid this will be ignored. */
	UPROPERTY(EditAnywhere, Category = "UObjectPropertyReader", meta = (EditCondition = "SourceMode != ENDIObjectPropertyReaderSourceMode::AttachParent && SourceMode != ENDIObjectPropertyReaderSourceMode::AttachParentActor", EditConditionHides))
	TSoftObjectPtr<AActor> SourceActor;

	/**
	When an actor is bound as the object we will also search for a component of this type to bind properties to.
	For example, setting this to a UPointLightComponent when binding properties we will first look at the actor
	then look for a component of UPointLightComponent and look at properties on that also.
	If no class is specified here we look at the RootComponent instead.
	*/
	UPROPERTY(EditAnywhere, Category = "UObjectPropertyReader")
	TObjectPtr<UClass> SourceActorComponentClass;

	//UNiagaraDataInterface Interface
	virtual void PostInitProperties() override;
	// UObject Interface End

	// UNiagaraDataInterface Interface Begin
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }

	virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual int32 PerInstanceDataSize() const override;

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override;

	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;

	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
#if WITH_EDITORONLY_DATA
	virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;
	//UNiagaraDataInterface Interface

	/** Remaps a property reader, where the  */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	static void SetUObjectReaderPropertyRemap(UNiagaraComponent* NiagaraComponent, FName UserParameterName, FName GraphName, FName RemapName);

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

private:
	void VMGetComponentTransform(FVectorVMExternalFunctionContext& Context);
	void VMGetComponentInvTransform(FVectorVMExternalFunctionContext& Context);

	FName GetRemappedPropertyName(FName InName) const
	{
		for ( const FNiagaraUObjectPropertyReaderRemap& RemapEntry : PropertyRemap )
		{
			if ( RemapEntry.GraphName == InName )
			{
				return RemapEntry.RemapName;
			}
		}
		return InName;
	};

	uint32 ChangeId = 0;
};
