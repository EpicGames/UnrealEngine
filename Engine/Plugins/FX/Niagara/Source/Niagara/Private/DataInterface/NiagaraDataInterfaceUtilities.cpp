// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraComponent.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraConstants.h"
#include "NiagaraDataInterface.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraSystemImpl.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemSimulation.h"

namespace FNiagaraDataInterfaceUtilities
{

void ForEachVMFunctionImpl(class UNiagaraDataInterface* RuntimeResolvedDataInterface, const class UNiagaraSystem* NiagaraSystem, const FNiagaraParameterStore& ParameterStore, TFunction<bool(const UNiagaraScript* Script, const FVMExternalFunctionBindingInfo&)> Action)
{
	if (RuntimeResolvedDataInterface == nullptr || NiagaraSystem == nullptr )
	{
		return;
	}

	// If the resolved data interface is an override DI capture the corresponding user parameter name here.
	FNiagaraVariableBase DataInterfaceUserVariable;
	for (const UNiagaraDataInterface* OverrideDI : ParameterStore.GetDataInterfaces())
	{
		if (OverrideDI != nullptr && OverrideDI == RuntimeResolvedDataInterface)
		{
			if (const FNiagaraVariableBase* Variable = ParameterStore.FindVariableFromDataInterface(OverrideDI))
			{
				DataInterfaceUserVariable = *Variable;
				break;
			}
		}
	}

	// Loop through all scripts
	bool bContinueSearching = true;
	NiagaraSystem->ForEachScriptWithRuntimeData(
		[&](const UNiagaraScript* Script, const FNiagaraScriptRuntimeData* ScriptRuntimeData)
		{
			if (bContinueSearching == false)
			{
				return;
			}

			const FNiagaraVMExecutableData& ScriptExecutableData = Script->GetVMExecutableData();
			if (!ScriptExecutableData.IsValid())
			{
				return;
			}

			for (const FVMExternalFunctionBindingInfo& FunctionBinding : ScriptExecutableData.CalledVMExternalFunctions)
			{
				for (const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDI : ScriptRuntimeData->GetResolvedDataInterfaces())
				{
					if (ResolvedDI.CompileName != FunctionBinding.OwnerName)
					{
						continue;
					}

					if (ResolvedDI.ResolvedDataInterface == RuntimeResolvedDataInterface || (DataInterfaceUserVariable.IsValid() && ResolvedDI.ResolvedVariable == DataInterfaceUserVariable))
					{
						if (Action(Script, FunctionBinding) == false)
						{
							bContinueSearching = false;
							return;
						}
					}
				}
			}
		}
	);
}

void ForEachGpuFunctionImpl(class UNiagaraDataInterface* RuntimeResolvedDataInterface, const class UNiagaraSystem* NiagaraSystem, const FNiagaraParameterStore& ParameterStore, TFunction<bool(const UNiagaraScript*, const FNiagaraDataInterfaceGeneratedFunction&)> Action)
{
	if (RuntimeResolvedDataInterface == nullptr || NiagaraSystem == nullptr)
	{
		return;
	}

	// If the resolved data interface is an override DI capture the corresponding user parameter name here.
	FNiagaraVariableBase DataInterfaceUserVariable;
	for (const UNiagaraDataInterface* OverrideDI : ParameterStore.GetDataInterfaces())
	{
		if (OverrideDI != nullptr && OverrideDI == RuntimeResolvedDataInterface)
		{
			if (const FNiagaraVariableBase* Variable = ParameterStore.FindVariableFromDataInterface(OverrideDI))
			{
				DataInterfaceUserVariable = *Variable;
				break;
			}
		}
	}

	// Loop through all scripts
	bool bContinueSearching = true;
	NiagaraSystem->ForEachScriptWithRuntimeData(
		[&](const UNiagaraScript* Script, const FNiagaraScriptRuntimeData* ScriptRuntimeData)
		{
			if (bContinueSearching == false)
			{
				return;
			}

			TConstArrayView<FNiagaraScriptResolvedDataInterfaceInfo> ResolvedDIs = ScriptRuntimeData->GetResolvedDataInterfaces();
			TConstArrayView<FNiagaraDataInterfaceGPUParamInfo> DataInterfaceParamInfos = Script->GetDataInterfaceGPUParamInfos();

			const int NumDataInterfaces = FMath::Min(ResolvedDIs.Num(), DataInterfaceParamInfos.Num());	// Note: Should always be equal but lets be safe
			for (int iDataInterface = 0; iDataInterface < NumDataInterfaces; ++iDataInterface)
			{
				const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDI = ResolvedDIs[iDataInterface];
				if (ResolvedDI.ResolvedDataInterface == nullptr)
				{
					continue;
				}
				if (ResolvedDI.ResolvedDataInterface == RuntimeResolvedDataInterface || (DataInterfaceUserVariable.IsValid() && ResolvedDI.ResolvedVariable == DataInterfaceUserVariable))
				{
					for (const FNiagaraDataInterfaceGeneratedFunction& GeneratedFunction : DataInterfaceParamInfos[iDataInterface].GeneratedFunctions)
					{
						if (Action(Script, GeneratedFunction) == false)
						{
							bContinueSearching = false;
							return;
						}
					}
				}
			}
		}
	);
}

void ForEachDataInterface(const FNiagaraParameterStore& ParameterStore, TFunction<bool(const FNiagaraVariableBase Variable, UNiagaraDataInterface* DataInterface)> Action)
{
	const TArray<UNiagaraDataInterface*>& DataInterfaces = ParameterStore.GetDataInterfaces();
	for (const FNiagaraVariableWithOffset& Variable : ParameterStore.ReadParameterVariables())
	{
		if (Variable.IsDataInterface() == false)
		{
			continue;
		}

		if ( UNiagaraDataInterface* DataInterface = DataInterfaces[Variable.Offset] )
		{
			Action(Variable, DataInterface);
		}
	}
}

void ForEachDataInterface(FDataInterfaceUsageContext& UsageContext, const FNiagaraParameterStore& ParameterStore, TFunction<bool(const FDataInterfaceUsageContext&)> Action)
{
	const TArray<UNiagaraDataInterface*>& DataInterfaces = ParameterStore.GetDataInterfaces();
	for (const FNiagaraVariableWithOffset& Variable : ParameterStore.ReadParameterVariables())
	{
		if (Variable.IsDataInterface() == false)
		{
			continue;
		}

		if (UNiagaraDataInterface* DataInterface = DataInterfaces[Variable.Offset])
		{
			UsageContext.Variable = Variable;
			UsageContext.DataInterface = DataInterface;
			Action(UsageContext);
		}
	}
}

} //namespace FNiagaraDataInterfaceUtilities

void FNiagaraDataInterfaceUtilities::ForEachVMFunction(UNiagaraDataInterface* RuntimeResolvedDataInterface, const UNiagaraSystem* NiagaraSystem, TFunction<bool(const UNiagaraScript*, const FVMExternalFunctionBindingInfo&)> Action)
{
	ForEachVMFunctionImpl(RuntimeResolvedDataInterface, NiagaraSystem, NiagaraSystem->GetExposedParameters(), Action);
}

void FNiagaraDataInterfaceUtilities::ForEachVMFunction(UNiagaraDataInterface* RuntimeResolvedDataInterface, UNiagaraComponent* Component, TFunction<bool(const UNiagaraScript*, const FVMExternalFunctionBindingInfo&)> Action)
{
	if (Component != nullptr && Component->GetAsset() != nullptr)
	{
		ForEachVMFunctionImpl(RuntimeResolvedDataInterface, Component->GetAsset(), Component->GetOverrideParameters(), Action);
	}
}

void FNiagaraDataInterfaceUtilities::ForEachVMFunction(UNiagaraDataInterface* RuntimeResolvedDataInterface, const FNiagaraSystemInstance* SystemInstance, TFunction<bool(const UNiagaraScript*, const FVMExternalFunctionBindingInfo&)> Action)
{
	if (SystemInstance != nullptr && SystemInstance->GetOverrideParameters() != nullptr && SystemInstance->GetSystem() != nullptr)
	{
		ForEachVMFunctionImpl(RuntimeResolvedDataInterface, SystemInstance->GetSystem(), *SystemInstance->GetOverrideParameters(), Action);
	}
}

void FNiagaraDataInterfaceUtilities::ForEachGpuFunction(UNiagaraDataInterface* RuntimeResolvedDataInterface, const class UNiagaraSystem* NiagaraSystem, TFunction<bool(const UNiagaraScript*, const FNiagaraDataInterfaceGeneratedFunction&)> Action)
{
	ForEachGpuFunctionImpl(RuntimeResolvedDataInterface, NiagaraSystem, NiagaraSystem->GetExposedParameters(), Action);
}

void FNiagaraDataInterfaceUtilities::ForEachGpuFunction(UNiagaraDataInterface* RuntimeResolvedDataInterface, UNiagaraComponent* Component, TFunction<bool(const UNiagaraScript*, const FNiagaraDataInterfaceGeneratedFunction&)> Action)
{
	if (Component != nullptr && Component->GetAsset() != nullptr)
	{
		ForEachGpuFunctionImpl(RuntimeResolvedDataInterface, Component->GetAsset(), Component->GetOverrideParameters(), Action);
	}
}

void FNiagaraDataInterfaceUtilities::ForEachGpuFunction(UNiagaraDataInterface* RuntimeResolvedDataInterface, const FNiagaraSystemInstance* SystemInstance, TFunction<bool(const UNiagaraScript*, const FNiagaraDataInterfaceGeneratedFunction&)> Action)
{
	if (SystemInstance != nullptr && SystemInstance->GetOverrideParameters() != nullptr && SystemInstance->GetSystem() != nullptr)
	{
		ForEachGpuFunctionImpl(RuntimeResolvedDataInterface, SystemInstance->GetSystem(), *SystemInstance->GetOverrideParameters(), Action);
	}
}

void FNiagaraDataInterfaceUtilities::ForEachDataInterface(const FNiagaraSystemInstance* SystemInstance, TFunction<bool(const FNiagaraVariableBase Variable, UNiagaraDataInterface* DataInterface)> Action)
{
	if ( SystemInstance == nullptr )
	{
		return;
	}

	FNiagaraSystemSimulationPtr SystemSimulation = SystemInstance->GetSystemSimulation();
	if (SystemSimulation == nullptr || !SystemSimulation->IsValid())
	{
		return;
	}

	ForEachDataInterface(*SystemInstance->GetOverrideParameters(), Action);

	ForEachDataInterface(SystemSimulation->GetSpawnExecutionContext()->Parameters, Action);
	ForEachDataInterface(SystemSimulation->GetUpdateExecutionContext()->Parameters, Action);

	for (const FNiagaraEmitterInstanceRef& Emitter : SystemInstance->GetEmitters())
	{
		FNiagaraEmitterInstanceImpl* StatefulEmitter = Emitter->AsStateful();
		if (!StatefulEmitter || Emitter->IsDisabled())
		{
			continue;
		}

		ForEachDataInterface(StatefulEmitter->GetSpawnExecutionContext().Parameters, Action);
		ForEachDataInterface(StatefulEmitter->GetUpdateExecutionContext().Parameters, Action);
		for (const FNiagaraScriptExecutionContext& EventExecContext : StatefulEmitter->GetEventExecutionContexts())
		{
			ForEachDataInterface(EventExecContext.Parameters, Action);
		}

		if (Emitter->GetEmitter() && Emitter->GetSimTarget() == ENiagaraSimTarget::GPUComputeSim && Emitter->GetGPUContext())
		{
			ForEachDataInterface(Emitter->GetGPUContext()->CombinedParamStore, Action);
		}
	}
}

void FNiagaraDataInterfaceUtilities::ForEachDataInterface(const FNiagaraSystemInstance* SystemInstance, TFunction<bool(const FDataInterfaceUsageContext&)> Action)
{
	if (SystemInstance == nullptr)
	{
		return;
	}

	FNiagaraSystemSimulationPtr SystemSimulation = SystemInstance->GetSystemSimulation();
	if (SystemSimulation == nullptr || !SystemSimulation->IsValid())
	{
		return;
	}

	FDataInterfaceUsageContext UsageContext;
	UsageContext.OwnerObject = SystemSimulation->GetSystem();
	ForEachDataInterface(UsageContext, *SystemInstance->GetOverrideParameters(), Action);

	ForEachDataInterface(UsageContext, SystemSimulation->GetSpawnExecutionContext()->Parameters, Action);
	ForEachDataInterface(UsageContext, SystemSimulation->GetUpdateExecutionContext()->Parameters, Action);

	for (const FNiagaraEmitterInstanceRef& Emitter : SystemInstance->GetEmitters())
	{
		FNiagaraEmitterInstanceImpl* StatefulEmitter = Emitter->AsStateful();
		if (!StatefulEmitter || Emitter->IsDisabled())
		{
			continue;
		}
		UsageContext.OwnerObject = Emitter->GetEmitter();

		ForEachDataInterface(UsageContext, StatefulEmitter->GetSpawnExecutionContext().Parameters, Action);
		ForEachDataInterface(UsageContext, StatefulEmitter->GetUpdateExecutionContext().Parameters, Action);
		for (const FNiagaraScriptExecutionContext& EventExecContext : StatefulEmitter->GetEventExecutionContexts() )
		{
			ForEachDataInterface(UsageContext, EventExecContext.Parameters, Action);
		}

		if (Emitter->GetEmitter() && Emitter->GetSimTarget() == ENiagaraSimTarget::GPUComputeSim && Emitter->GetGPUContext())
		{
			ForEachDataInterface(UsageContext, Emitter->GetGPUContext()->CombinedParamStore, Action);
		}
	}
}

void FNiagaraDataInterfaceUtilities::ForEachDataInterface(const UNiagaraSystem* NiagaraSystem, TFunction<bool(const FDataInterfaceUsageContext&)> Action)
{
	if (NiagaraSystem == nullptr)
	{
		return;
	}

	NiagaraSystem->ForEachScriptWithOwningContext([&Action](const UNiagaraSystem* OwningSystem, const FNiagaraEmitterHandle* OwningEmitterHandle, UNiagaraScript* Script)
	{
		if (Script == nullptr)
		{
			return;
		}

		FGuid EmitterHandleId;
		FDataInterfaceUsageContext UsageContext;
		if (OwningEmitterHandle != nullptr)
		{
			EmitterHandleId = OwningEmitterHandle->GetId();
			UsageContext.OwnerObject = OwningEmitterHandle->GetInstance().Emitter;
		}
		else
		{
			EmitterHandleId = FGuid();
			UsageContext.OwnerObject = OwningSystem;
		}

		const FNiagaraScriptRuntimeData* ScriptRuntimeData = OwningSystem->GetScriptRuntimeData(OwningEmitterHandle, *Script).Get();
		if (ScriptRuntimeData == nullptr)
		{
			return;
		}

		ForEachDataInterface(UsageContext, *ScriptRuntimeData->GetExecutionReadyParameterStore(), Action);
	});
}