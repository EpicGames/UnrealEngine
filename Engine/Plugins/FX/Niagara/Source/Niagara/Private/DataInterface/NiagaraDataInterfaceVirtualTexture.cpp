// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceVirtualTexture.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"

#include "GlobalRenderResources.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "RHIStaticStates.h"
#include "ShaderCompilerCore.h"
#include "VT/RuntimeVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceVirtualTexture)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterfaceVirtualTexture"

namespace NDIVirtualTextureLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_SRV(Texture2D,				VirtualTexture0)
		SHADER_PARAMETER_TEXTURE(Texture2D<uint4>,	VirtualTexture0PageTable)
		SHADER_PARAMETER(FUintVector4,				VirtualTexture0TextureUniforms)

		SHADER_PARAMETER_SRV(Texture2D,				VirtualTexture1)
		SHADER_PARAMETER_TEXTURE(Texture2D<uint4>,	VirtualTexture1PageTable)
		SHADER_PARAMETER(FUintVector4,				VirtualTexture1TextureUniforms)

		SHADER_PARAMETER_SRV(Texture2D,				VirtualTexture2)
		SHADER_PARAMETER_TEXTURE(Texture2D<uint4>,	VirtualTexture2PageTable)
		SHADER_PARAMETER(FUintVector4,				VirtualTexture2TextureUniforms)

		SHADER_PARAMETER(uint32,					ValidLayersMask)
		SHADER_PARAMETER(int32,						MaterialType)
		SHADER_PARAMETER_ARRAY(FUintVector4,		PageTableUniforms, [2])
		SHADER_PARAMETER_ARRAY(FVector4f,			UVUniforms, [3])
		SHADER_PARAMETER(FVector2f,					WorldHeightUnpack)
		SHADER_PARAMETER_SAMPLER(SamplerState,		SharedSampler)
	END_SHADER_PARAMETER_STRUCT()

	const TCHAR*	CommonShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceVirtualTexture.ush");
	const TCHAR*	TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceVirtualTextureTemplate.ush");
	const FName		GetAttributesValidName("GetAttributesValid");
	const FName		SampleRVTName("SampleRVT");
	constexpr int32	MaxRVTLayers = 3;

	struct FNDIFunctionVersion
	{
		enum Type
		{
			InitialVersion = 0,
			AddInsideVolumeToSampleRVT = 1,
			AddNewChannels = 2,

			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};
	};

	struct FNDIInstanceData_GameThread
	{
		TWeakObjectPtr<URuntimeVirtualTexture> CurrentTexture = nullptr;
		FNiagaraParameterDirectBinding<UObject*> UserParamBinding;
	};

	struct FNDInstanceData_RenderThread
	{
		FNDInstanceData_RenderThread()
		{
			SharedSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		}

		struct FLayerTexture
		{
			bool IsValid() const
			{
				return Texture.IsValid() && TextureSRV.IsValid() && PageTable.IsValid();
			}

			FTextureRHIRef				Texture;
			FRHIShaderResourceViewRef	TextureSRV;
			FTextureRHIRef				PageTable;
			FUintVector4				TextureUniforms = FUintVector4(0, 0, 0, 0);
		};

		FLayerTexture		LayerTexture[MaxRVTLayers];
		ERuntimeVirtualTextureMaterialType	MaterialType = ERuntimeVirtualTextureMaterialType::Count;
		FUintVector4		PageTableUniforms[2] = { FUintVector4(0, 0, 0, 0), FUintVector4(0, 0, 0, 0) };
		FVector4			UVUniforms[3] = { FVector4::Zero(), FVector4::Zero(), FVector4::Zero() };
		FVector2f			WorldHeightUnpack = FVector2f::ZeroVector;
		FSamplerStateRHIRef	SharedSampler;
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { checkNoEntry(); }
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

		TMap<FNiagaraSystemInstanceID, FNDInstanceData_RenderThread> InstanceData_RT;
	};
}

UNiagaraDataInterfaceVirtualTexture::UNiagaraDataInterfaceVirtualTexture(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, Texture(nullptr)
{
	using namespace NDIVirtualTextureLocal;

	Proxy.Reset(new FNDIProxy());

	FNiagaraTypeDefinition Def(UObject::StaticClass());
	TextureUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceVirtualTexture::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceVirtualTexture::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
	if (NiagaraVer < FNiagaraCustomVersion::TextureDataInterfaceUsesCustomSerialize)
	{
		if (Texture != nullptr)
		{
			Texture->ConditionalPostLoad();
		}
	}
#endif
}

void UNiagaraDataInterfaceVirtualTexture::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsLoading() == false || Ar.CustomVer(FNiagaraCustomVersion::GUID) >= FNiagaraCustomVersion::TextureDataInterfaceUsesCustomSerialize)
	{
		TArray<uint8> StreamData;
		Ar << StreamData;
	}
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);
}

bool UNiagaraDataInterfaceVirtualTexture::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceVirtualTexture* DestinationTexture = CastChecked<UNiagaraDataInterfaceVirtualTexture>(Destination);
	DestinationTexture->Texture = Texture;
	DestinationTexture->TextureUserParameter = TextureUserParameter;

	return true;
}

bool UNiagaraDataInterfaceVirtualTexture::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceVirtualTexture* OtherTexture = CastChecked<const UNiagaraDataInterfaceVirtualTexture>(Other);
	return
		OtherTexture->Texture == Texture &&
		OtherTexture->TextureUserParameter == TextureUserParameter;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceVirtualTexture::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIVirtualTextureLocal;

	{
		FNiagaraFunctionSignature & Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = GetAttributesValidName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.SetFunctionVersion(FNDIFunctionVersion::LatestVersion);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Texture"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("BaseColorValid"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("SpecularValid"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("RoughnessValid"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("NormalValid"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("WorldHeightValid"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("MaskValid"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("DisplacementValid"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Mask4Valid"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = SampleRVTName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;		
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.SetFunctionVersion(FNDIFunctionVersion::LatestVersion);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Texture"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("WorldPosition"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("InsideVolume"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaseColor"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Specular"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Roughness"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("WorldHeight"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Mask"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Displacement"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Mask4"));
	}
}
#endif

void UNiagaraDataInterfaceVirtualTexture::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	// No CPU side functions yet
}

int32 UNiagaraDataInterfaceVirtualTexture::PerInstanceDataSize() const
{
	using namespace NDIVirtualTextureLocal;
	return sizeof(FNDIInstanceData_GameThread);
}

bool UNiagaraDataInterfaceVirtualTexture::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIVirtualTextureLocal;
	FNDIInstanceData_GameThread* InstanceData = new (PerInstanceData) FNDIInstanceData_GameThread();
	InstanceData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), TextureUserParameter.Parameter);
	return true;
}

void UNiagaraDataInterfaceVirtualTexture::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIVirtualTextureLocal;

	FNDIInstanceData_GameThread* InstanceData = static_cast<FNDIInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FNDIInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(NDITexture_RemoveInstance)
	(
		[RT_Proxy=GetProxyAs<FNDIProxy>(), RT_InstanceID=SystemInstance->GetId()](FRHICommandListImmediate&)
		{
			RT_Proxy->InstanceData_RT.Remove(RT_InstanceID);
		}
	);
}

bool UNiagaraDataInterfaceVirtualTexture::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIVirtualTextureLocal;
	FNDIInstanceData_GameThread* InstanceData = static_cast<FNDIInstanceData_GameThread*>(PerInstanceData);

	URuntimeVirtualTexture* CurrentTexture = InstanceData->UserParamBinding.GetValueOrDefault<URuntimeVirtualTexture>(Texture);
	//-OPT: We shouldn't need to send this data each frame, currently playing it safe
	{
		InstanceData->CurrentTexture = CurrentTexture;

		ENQUEUE_RENDER_COMMAND(NDITexture_UpdateInstance)
		(
			[RT_Proxy=GetProxyAs<FNDIProxy>(), RT_InstanceID=SystemInstance->GetId(), RT_WeakTexture=TWeakObjectPtr<URuntimeVirtualTexture>(CurrentTexture)](FRHICommandListImmediate&)
			{
				URuntimeVirtualTexture* RT_Texture = RT_WeakTexture.Get();
				IAllocatedVirtualTexture* VirtualTexture = RT_Texture ? RT_Texture->GetAllocatedVirtualTexture() : nullptr;
				FNDInstanceData_RenderThread& InstanceData = RT_Proxy->InstanceData_RT.FindOrAdd(RT_InstanceID);

				if (VirtualTexture != nullptr)
				{
					InstanceData.MaterialType = RT_Texture->GetMaterialType();
					VirtualTexture->GetPackedPageTableUniform(InstanceData.PageTableUniforms);
					InstanceData.UVUniforms[0] = RT_Texture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldToUVTransform0);
					InstanceData.UVUniforms[1] = RT_Texture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1);
					InstanceData.UVUniforms[2] = RT_Texture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2);

					const FVector4 WorldHeightUnpack = RT_Texture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldHeightUnpack);
					InstanceData.WorldHeightUnpack = FVector2f(float(WorldHeightUnpack.X), float(WorldHeightUnpack.Y));		//LWC Precision Loss

					for ( int32 iLayer=0; iLayer < MaxRVTLayers; ++iLayer)
					{
						FNDInstanceData_RenderThread::FLayerTexture& LayerTexture = InstanceData.LayerTexture[iLayer];
						LayerTexture.Texture = VirtualTexture->GetPhysicalTexture(iLayer);
						LayerTexture.TextureSRV = VirtualTexture->GetPhysicalTextureSRV(iLayer, RT_Texture->IsLayerSRGB(iLayer));
						LayerTexture.PageTable = VirtualTexture->GetPageTableTexture(0);//-TODO:iLayer);

						VirtualTexture->GetPackedUniform(&LayerTexture.TextureUniforms, iLayer);
					}
				}
				else
				{
					InstanceData = FNDInstanceData_RenderThread();
				}
			}
		);
	}

	return false;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceVirtualTexture::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace NDIVirtualTextureLocal;

	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(CommonShaderFilePath);
	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceVirtualTexture::GetCommonHLSL(FString& OutHlsl)
{
	using namespace NDIVirtualTextureLocal;

	OutHlsl.Appendf(TEXT("#include \"%s\"\n"), CommonShaderFilePath);
}

void UNiagaraDataInterfaceVirtualTexture::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIVirtualTextureLocal::TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceVirtualTexture::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIVirtualTextureLocal;

	static const TSet<FName> ValidGpuFunctions =
	{
		GetAttributesValidName,
		SampleRVTName,
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

bool UNiagaraDataInterfaceVirtualTexture::UpgradeFunctionCall(FNiagaraFunctionSignature& Sig)
{
	using namespace NDIVirtualTextureLocal;

	bool bModified = false;
	if (Sig.FunctionVersion < FNDIFunctionVersion::AddInsideVolumeToSampleRVT)
	{
		if (Sig.Name == SampleRVTName)
		{
			Sig.Outputs.Insert(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("InsideVolume")), 0);
			bModified = true;
		}
	}
	if (Sig.FunctionVersion < FNDIFunctionVersion::AddNewChannels)
	{
		if (Sig.Name == SampleRVTName)
		{
			Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Displacement"));
			Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Mask4"));
			bModified = true;
		}
		else if (Sig.Name == GetAttributesValidName)
		{
			Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("DisplacementValid"));
			Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Mask4Valid"));
			bModified = true;
		}
	}

	Sig.SetFunctionVersion(FNDIFunctionVersion::LatestVersion);
	return bModified;
}
#endif

void UNiagaraDataInterfaceVirtualTexture::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NDIVirtualTextureLocal;
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceVirtualTexture::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIVirtualTextureLocal;

	FNDIProxy& TextureProxy = Context.GetProxy<FNDIProxy>();
	FNDInstanceData_RenderThread* InstanceData = TextureProxy.InstanceData_RT.Find(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();

	bool bLayerValid[MaxRVTLayers];
	FMemory::Memset(bLayerValid, 0);

	if ( InstanceData )
	{
		bLayerValid[0] = InstanceData->LayerTexture[0].IsValid();
		bLayerValid[1] = InstanceData->LayerTexture[1].IsValid();
		bLayerValid[2] = InstanceData->LayerTexture[2].IsValid();

		if ( bLayerValid[0] == true)
		{
			ShaderParameters->VirtualTexture0					= InstanceData->LayerTexture[0].TextureSRV;
			ShaderParameters->VirtualTexture0PageTable			= InstanceData->LayerTexture[0].PageTable;
			ShaderParameters->VirtualTexture0TextureUniforms	= InstanceData->LayerTexture[0].TextureUniforms;
		}

		if ( bLayerValid[1] == true)
		{
			ShaderParameters->VirtualTexture1					= InstanceData->LayerTexture[1].TextureSRV;
			ShaderParameters->VirtualTexture1PageTable			= InstanceData->LayerTexture[1].PageTable;
			ShaderParameters->VirtualTexture1TextureUniforms	= InstanceData->LayerTexture[1].TextureUniforms;
		}

		if ( bLayerValid[2] == true)
		{
			ShaderParameters->VirtualTexture2					= InstanceData->LayerTexture[2].TextureSRV;
			ShaderParameters->VirtualTexture2PageTable			= InstanceData->LayerTexture[2].PageTable;
			ShaderParameters->VirtualTexture2TextureUniforms	= InstanceData->LayerTexture[2].TextureUniforms;
		}

		const FNiagaraLWCConverter LWCConverter(FVector(Context.GetSystemLWCTile()) * FLargeWorldRenderScalar::GetTileSize());
		const FVector3f UVOriginRebased = LWCConverter.ConvertWorldToSimulationVector(FVector(InstanceData->UVUniforms[0]));

		ShaderParameters->ValidLayersMask		 = bLayerValid[0] ? 1 : 0;
		ShaderParameters->ValidLayersMask		|= bLayerValid[1] ? 2 : 0;
		ShaderParameters->ValidLayersMask		|= bLayerValid[2] ? 4 : 0;
		ShaderParameters->MaterialType			= int(InstanceData->MaterialType);
		ShaderParameters->PageTableUniforms[0]	= InstanceData->PageTableUniforms[0];
		ShaderParameters->PageTableUniforms[1]	= InstanceData->PageTableUniforms[1];
		ShaderParameters->UVUniforms[0]			= FVector4f(UVOriginRebased, InstanceData->UVUniforms[0].W);
		ShaderParameters->UVUniforms[1]			= FVector4f(InstanceData->UVUniforms[1]);
		ShaderParameters->UVUniforms[2]			= FVector4f(InstanceData->UVUniforms[2]);
		ShaderParameters->WorldHeightUnpack		= InstanceData->WorldHeightUnpack;
		ShaderParameters->SharedSampler			= InstanceData->SharedSampler;
	}
	else
	{
		ShaderParameters->ValidLayersMask		= 0;
		ShaderParameters->MaterialType			= int(ERuntimeVirtualTextureMaterialType::Count);
		ShaderParameters->PageTableUniforms[0]	= FUintVector4(0, 0, 0, 0);
		ShaderParameters->PageTableUniforms[1]	= FUintVector4(0, 0, 0, 0);
		ShaderParameters->UVUniforms[0]			= FVector4f::Zero();
		ShaderParameters->UVUniforms[1]			= FVector4f::Zero();
		ShaderParameters->UVUniforms[2]			= FVector4f::Zero();
		ShaderParameters->WorldHeightUnpack		= FVector2f::ZeroVector;
		ShaderParameters->SharedSampler			= TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}
	
	if ( bLayerValid[0] == false )
	{
		ShaderParameters->VirtualTexture0					= GBlackTextureWithSRV->ShaderResourceViewRHI;
		ShaderParameters->VirtualTexture0PageTable			= GBlackUintTexture->TextureRHI;
		ShaderParameters->VirtualTexture0TextureUniforms	= FUintVector4(0, 0, 0, 0);
	}
	if ( bLayerValid[1] == false )
	{
		ShaderParameters->VirtualTexture1					= GBlackTextureWithSRV->ShaderResourceViewRHI;
		ShaderParameters->VirtualTexture1PageTable			= GBlackUintTexture->TextureRHI;
		ShaderParameters->VirtualTexture1TextureUniforms	= FUintVector4(0, 0, 0, 0);
	}
	if ( bLayerValid[2] == false )
	{
		ShaderParameters->VirtualTexture2					= GBlackTextureWithSRV->ShaderResourceViewRHI;
		ShaderParameters->VirtualTexture2PageTable			= GBlackUintTexture->TextureRHI;
		ShaderParameters->VirtualTexture2TextureUniforms	= FUintVector4(0, 0, 0, 0);
	}
}

void UNiagaraDataInterfaceVirtualTexture::SetTexture(URuntimeVirtualTexture* InTexture)
{
	Texture = InTexture;
}

#undef LOCTEXT_NAMESPACE
