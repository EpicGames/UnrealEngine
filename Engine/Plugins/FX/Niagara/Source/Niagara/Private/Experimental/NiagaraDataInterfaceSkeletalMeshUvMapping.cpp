// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSkeletalMeshUvMapping.h"

#include "Experimental/NiagaraMeshUvMapping.h"
#include "NDISkeletalMeshCommon.h"
#include "Rendering/SkeletalMeshRenderData.h"

template<bool UseFullPrecisionUv>
struct FSkelMeshVertexAccessorHelper
{
public:
	FSkelMeshVertexAccessorHelper(const FSkeletalMeshLODRenderData* InLodRenderData)
		: LodRenderData(InLodRenderData)
		, IndexBuffer(*InLodRenderData->MultiSizeIndexContainer.GetIndexBuffer())
	{}

	FORCEINLINE FVector2D GetVertexUV(int32 VertexIndex, int32 UvChannel) const
	{
		return MeshVertexAccessor.GetVertexUV(LodRenderData, IndexBuffer.Get(VertexIndex), UvChannel);
	}

	FORCEINLINE int32 GetTriangleCount() const
	{
		return IndexBuffer.Num() / 3;
	}

private:
	const FSkeletalMeshLODRenderData* LodRenderData;
	const FRawStaticIndexBuffer16or32Interface& IndexBuffer;
	const FSkelMeshVertexAccessor<UseFullPrecisionUv> MeshVertexAccessor;
};

FSkeletalMeshUvMapping::FSkeletalMeshUvMapping(TWeakObjectPtr<USkinnedAsset> InMeshObject, int32 InLodIndex, int32 InUvSetIndex)
	: FMeshUvMapping(InLodIndex, InUvSetIndex)
	, MeshObject(InMeshObject)
{
}

bool FSkeletalMeshUvMapping::IsValidMeshObject(const TWeakObjectPtr<USkinnedAsset>& MeshObject, int32 InLodIndex, int32 InUvSetIndex)
{
	USkinnedAsset* Mesh = MeshObject.Get();

	if (!Mesh)
	{
		return false;
	}

	const FSkeletalMeshLODInfo* LodInfo = Mesh->GetLODInfo(InLodIndex);
	if (!LodInfo)
	{
		// invalid Lod index
		return false;
	}

	if (!LodInfo->bAllowCPUAccess)
	{
		// we need CPU access to buffers in order to generate our UV mapping quad tree
		return false;
	}

	if (!GetLodRenderData(*Mesh, InLodIndex, InUvSetIndex))
	{
		// no render data available
		return false;
	}

	return true;
}

bool FSkeletalMeshUvMapping::Matches(const TWeakObjectPtr<USkinnedAsset>& InMeshObject, int32 InLodIndex, int32 InUvSetIndex) const
{
	return LodIndex == InLodIndex && MeshObject == InMeshObject && UvSetIndex == InUvSetIndex;
}

const FSkeletalMeshLODRenderData* FSkeletalMeshUvMapping::GetLodRenderData() const
{
	if (const USkinnedAsset* Mesh = MeshObject.Get())
	{
		return GetLodRenderData(*Mesh, LodIndex, UvSetIndex);
	}
	return nullptr;
}

void FSkeletalMeshUvMapping::FindOverlappingTriangles(const FVector2D& InUv, float Tolerance, TArray<int32>& TriangleIndices) const
{
	TriangleIndices.Empty();

	if (const FSkeletalMeshLODRenderData* LodRenderData = GetLodRenderData())
	{
		if (LodRenderData->StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs())
		{
			FSkelMeshVertexAccessorHelper<true> VertexHelper(LodRenderData);
			FQuadTreeQueryHelper QueryHelper(TriangleIndexQuadTree, VertexHelper, UvSetIndex);
			QueryHelper.FindOverlappingTriangle(InUv, Tolerance, TriangleIndices);
		}
		else
		{
			FSkelMeshVertexAccessorHelper<false> VertexHelper(LodRenderData);
			FQuadTreeQueryHelper QueryHelper(TriangleIndexQuadTree, VertexHelper, UvSetIndex);
			QueryHelper.FindOverlappingTriangle(InUv, Tolerance, TriangleIndices);
		}
	}
}

int32 FSkeletalMeshUvMapping::FindFirstTriangle(const FVector2D& InUv, float Tolerance, FVector& BarycentricCoord) const
{
	if (const FSkeletalMeshLODRenderData* LodRenderData = GetLodRenderData())
	{
		if (LodRenderData->StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs())
		{
			FSkelMeshVertexAccessorHelper<true> VertexHelper(LodRenderData);
			FQuadTreeQueryHelper QueryHelper(TriangleIndexQuadTree, VertexHelper, UvSetIndex);
			return QueryHelper.FindFirstTriangle(InUv, Tolerance, BarycentricCoord);
		}
		else
		{
			FSkelMeshVertexAccessorHelper<false> VertexHelper(LodRenderData);
			FQuadTreeQueryHelper QueryHelper(TriangleIndexQuadTree, VertexHelper, UvSetIndex);
			return QueryHelper.FindFirstTriangle(InUv, Tolerance, BarycentricCoord);
		}
	}

	return INDEX_NONE;
}

int32 FSkeletalMeshUvMapping::FindFirstTriangle(const FBox2D& InUvBox, FVector& BarycentricCoord) const
{
	if (const FSkeletalMeshLODRenderData* LodRenderData = GetLodRenderData())
	{
		if (LodRenderData->StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs())
		{
			FSkelMeshVertexAccessorHelper<true> VertexHelper(LodRenderData);
			FQuadTreeQueryHelper QueryHelper(TriangleIndexQuadTree, VertexHelper, UvSetIndex);
			return QueryHelper.FindFirstTriangle(InUvBox, BarycentricCoord);
		}
		else
		{
			FSkelMeshVertexAccessorHelper<false> VertexHelper(LodRenderData);
			FQuadTreeQueryHelper QueryHelper(TriangleIndexQuadTree, VertexHelper, UvSetIndex);
			return QueryHelper.FindFirstTriangle(InUvBox, BarycentricCoord);
		}
	}

	return INDEX_NONE;
}

const FSkeletalMeshLODRenderData* FSkeletalMeshUvMapping::GetLodRenderData(const USkinnedAsset& Mesh, int32 LodIndex, int32 UvSetIndex)
{
	if (const FSkeletalMeshRenderData* RenderData = Mesh.GetResourceForRendering())
	{
		if (RenderData->LODRenderData.IsValidIndex(LodIndex))
		{
			const FSkeletalMeshLODRenderData& LodRenderData = RenderData->LODRenderData[LodIndex];
			const int32 NumTexCoords = LodRenderData.GetNumTexCoords();

			if (NumTexCoords > UvSetIndex)
			{
				return &LodRenderData;
			}
		}
	}

	return nullptr;
}

void FSkeletalMeshUvMapping::BuildQuadTree()
{
	if (const FSkeletalMeshLODRenderData* LodRenderData = GetLodRenderData())
	{
		if (LodRenderData->StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs())
		{
			FSkelMeshVertexAccessorHelper<true> VertexHelper(LodRenderData);
			FQuadTreeBuildHelper BuildHelper(VertexHelper, UvSetIndex, TriangleIndexQuadTree);
			BuildHelper.Build();
		}
		else
		{
			FSkelMeshVertexAccessorHelper<false> VertexHelper(LodRenderData);
			FQuadTreeBuildHelper BuildHelper(VertexHelper, UvSetIndex, TriangleIndexQuadTree);
			BuildHelper.Build();
		}
	}
}
