// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Experimental/NiagaraMeshUvMapping.h"
#include "NiagaraUvQuadTree.h"
#include "RenderResource.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FSkeletalMeshLODRenderData;
class USkinnedAsset;

struct FSkeletalMeshUvMapping : public FMeshUvMapping
{
	FSkeletalMeshUvMapping(TWeakObjectPtr<USkinnedAsset> InMeshObject, int32 InLodIndex, int32 InUvSetIndex);
	virtual ~FSkeletalMeshUvMapping() = default;

	static bool IsValidMeshObject(const TWeakObjectPtr<USkinnedAsset>& MeshObject, int32 InLodIndex, int32 InUvSetIndex);

	bool Matches(const TWeakObjectPtr<USkinnedAsset>& InMeshObject, int32 InLodIndex, int32 InUvSetIndex) const;

	const FSkeletalMeshLODRenderData* GetLodRenderData() const;

	void FindOverlappingTriangles(const FVector2D& InUv, float Tolerance, TArray<int32>& TriangleIndices) const;
	int32 FindFirstTriangle(const FVector2D& InUv, float Tolerance, FVector& BarycentricCoord) const;
	int32 FindFirstTriangle(const FBox2D& InUvBox, FVector& BarycentricCoord) const;

private:
	static const FSkeletalMeshLODRenderData* GetLodRenderData(const USkinnedAsset& Mesh, int32 LodIndex, int32 UvSetIndex);
	virtual void BuildQuadTree() override;

	TWeakObjectPtr<USkinnedAsset> MeshObject;
};
