// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMeshUvMapping.h"

#include "Experimental/NiagaraMeshUvMappingHandle.h"
#include "NiagaraResourceArrayWriter.h"
#include "NiagaraStats.h"
#include "RenderingThread.h"

DECLARE_CYCLE_STAT(TEXT("Niagara - UvQuadTree Cpu"), STAT_Niagara_UvMapping_Cpu, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Niagara - UvQuadTree Gpu"), STAT_Niagara_UvMapping_Gpu, STATGROUP_Niagara);

FMeshUvMapping::FMeshUvMapping(int32 InLodIndex, int32 InUvSetIndex)
: LodIndex(InLodIndex)
, UvSetIndex(InUvSetIndex)
, TriangleIndexQuadTree(8 /* Internal node capacity */, 8 /* Maximum tree depth */)
{}

FMeshUvMapping::~FMeshUvMapping()
{
	ReleaseQuadTree();
	ReleaseGpuQuadTree();
}

void FMeshUvMapping::FreezeQuadTree(TResourceArray<uint8>& OutQuadTree) const
{
	FNiagaraResourceArrayWriter Ar(OutQuadTree);
	TriangleIndexQuadTree.Freeze(Ar);
}

void FMeshUvMapping::ReleaseQuadTree()
{
	TriangleIndexQuadTree.Empty();
}

void FMeshUvMapping::BuildGpuQuadTree()
{
	check(FrozenQuadTreeProxy == nullptr);
	FrozenQuadTreeProxy.Reset(new FMeshUvMappingBufferProxy());
	FrozenQuadTreeProxy->Initialize(*this);
	BeginInitResource(FrozenQuadTreeProxy.Get());
}

void FMeshUvMapping::ReleaseGpuQuadTree()
{
	if (FMeshUvMappingBufferProxy* ProxyPtr = FrozenQuadTreeProxy.Release())
	{
		ENQUEUE_RENDER_COMMAND(BeginDestroyCommand)([RT_Proxy = ProxyPtr](FRHICommandListImmediate& RHICmdList)
			{
				RT_Proxy->ReleaseResource();
				delete RT_Proxy;
			});
	}
}

bool FMeshUvMapping::IsUsed() const
{
	return (CpuQuadTreeUserCount > 0)
		|| (GpuQuadTreeUserCount > 0);
}

bool FMeshUvMapping::CanBeDestroyed() const
{
	return !IsUsed();
}

void FMeshUvMapping::RegisterUser(FMeshUvMappingUsage Usage, bool bNeedsDataImmediately)
{
	if (Usage.RequiresCpuAccess || Usage.RequiresGpuAccess)
	{
		if (CpuQuadTreeUserCount++ == 0)
		{
			SCOPE_CYCLE_COUNTER(STAT_Niagara_UvMapping_Cpu);
			BuildQuadTree();
		}
	}

	if (Usage.RequiresGpuAccess)
	{
		if (GpuQuadTreeUserCount++ == 0)
		{
			SCOPE_CYCLE_COUNTER(STAT_Niagara_UvMapping_Gpu);
			BuildGpuQuadTree();
		}
	}
}

void FMeshUvMapping::UnregisterUser(FMeshUvMappingUsage Usage)
{
	if (Usage.RequiresCpuAccess || Usage.RequiresGpuAccess)
	{
		if (--CpuQuadTreeUserCount == 0)
		{
			ReleaseQuadTree();
		}
	}

	if (Usage.RequiresGpuAccess)
	{
		if (--GpuQuadTreeUserCount == 0)
		{
			ReleaseGpuQuadTree();
		}
	}
}

const FMeshUvMappingBufferProxy* FMeshUvMapping::GetQuadTreeProxy() const
{
	return FrozenQuadTreeProxy.Get();
}

void
FMeshUvMappingBufferProxy::Initialize(const FMeshUvMapping& UvMapping)
{
	UvMapping.FreezeQuadTree(FrozenQuadTree);
}

void
FMeshUvMappingBufferProxy::InitRHI(FRHICommandListBase& RHICmdList)
{
	const int32 BufferSize = FrozenQuadTree.GetResourceDataSize();

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(TEXT("UvMappingBuffer"), BufferSize)
		.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Static)
		.SetInitActionResourceArray(&FrozenQuadTree)
		.DetermineInitialState();

	UvMappingBuffer = RHICmdList.CreateBuffer(CreateDesc);
	UvMappingSrv = RHICmdList.CreateShaderResourceView(
		UvMappingBuffer, 
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R32_SINT));

#if STATS
	check(GpuMemoryUsage == 0);
	GpuMemoryUsage = BufferSize;
#endif

	INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GpuMemoryUsage);
}

void
FMeshUvMappingBufferProxy::ReleaseRHI()
{
	UvMappingBuffer.SafeRelease();
	UvMappingSrv.SafeRelease();

	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GpuMemoryUsage);

#if STATS
	GpuMemoryUsage = 0;
#endif
}

FMeshUvMappingHandleBase::FMeshUvMappingHandleBase(FMeshUvMappingUsage InUsage, const TSharedPtr<FMeshUvMapping>&InUvMappingData, bool bNeedsDataImmediately)
	: Usage(InUsage)
	, UvMappingData(InUvMappingData)
{
	if (FMeshUvMapping* MappingData = UvMappingData.Get())
	{
		MappingData->RegisterUser(Usage, bNeedsDataImmediately);
	}
}

FMeshUvMappingHandleBase::FMeshUvMappingHandleBase(FMeshUvMappingHandleBase&& Other) noexcept
{
	Usage = Other.Usage;
	UvMappingData = Other.UvMappingData;
	Other.UvMappingData = nullptr;
}

FMeshUvMappingHandleBase::~FMeshUvMappingHandleBase()
{
	if (FMeshUvMapping* MappingData = UvMappingData.Get())
	{
		MappingData->UnregisterUser(Usage);
	}
}

FMeshUvMappingHandleBase& FMeshUvMappingHandleBase::operator=(FMeshUvMappingHandleBase&& Other) noexcept
{
	if (this != &Other)
	{
		Usage = Other.Usage;
		UvMappingData = Other.UvMappingData;
		Other.UvMappingData = nullptr;
	}
	return *this;
}

FMeshUvMappingHandleBase::operator bool() const
{
	return UvMappingData.IsValid();
}

const FMeshUvMappingBufferProxy* FMeshUvMappingHandleBase::GetQuadTreeProxy() const
{
	if (UvMappingData)
	{
		return UvMappingData->GetQuadTreeProxy();
	}

	return nullptr;
}

int32 FMeshUvMappingHandleBase::GetUvSetIndex() const
{
	if (UvMappingData)
	{
		return UvMappingData->UvSetIndex;
	}

	return 0;
}

int32 FMeshUvMappingHandleBase::GetLodIndex() const
{
	if (UvMappingData)
	{
		return UvMappingData->LodIndex;
	}

	return 0;
}

void FMeshUvMappingHandleBase::PinAndInvalidateHandle()
{
	UvMappingData.Reset();
}
