// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/DynamicRHIResourceArray.h"
#include "Math/Vector.h"
#include "NiagaraUvQuadTree.h"
#include "RenderResource.h"
#include "RHIResources.h"

struct FMeshUvMappingUsage;

struct FMeshUvMapping;
class FNiagaraUvQuadTree;

template<typename MeshAccessorType>
struct FQuadTreeBuildHelper
{
	const MeshAccessorType& MeshAccessor;
	const int32 UvSetIndex;
	FNiagaraUvQuadTree& QuadTree;

	FQuadTreeBuildHelper(const MeshAccessorType& InMeshAccessor, int32 InUvSetIndex, FNiagaraUvQuadTree& InQuadTree)
		: MeshAccessor(InMeshAccessor)
		, UvSetIndex(InUvSetIndex)
		, QuadTree(InQuadTree)
	{
	}

	void Build()
	{
		const int32 TriangleCount = MeshAccessor.GetTriangleCount();

		for (int32 TriangleIt = 0; TriangleIt < TriangleCount; ++TriangleIt)
		{
			const FVector2D UVs[] =
			{
				MeshAccessor.GetVertexUV(TriangleIt * 3 + 0, UvSetIndex),
				MeshAccessor.GetVertexUV(TriangleIt * 3 + 1, UvSetIndex),
				MeshAccessor.GetVertexUV(TriangleIt * 3 + 2, UvSetIndex),
			};

			// we want to skip degenerate triangles
			if (FMath::Abs((UVs[1] - UVs[0]) ^ (UVs[2] - UVs[0])) < SMALL_NUMBER)
			{
				continue;
			}

			QuadTree.Insert(TriangleIt, FBox2D(UVs, UE_ARRAY_COUNT(UVs)));
		}
	}
};

template<typename MeshAccessorType>
struct FQuadTreeQueryHelper
{
	const FNiagaraUvQuadTree& QuadTree;
	const MeshAccessorType& MeshAccessor;
	const int32 UvSetIndex;

	FQuadTreeQueryHelper(const FNiagaraUvQuadTree& InQuadTree, const MeshAccessorType& InMeshAccessor, int32 InUvSetIndex)
	: QuadTree(InQuadTree)
	, MeshAccessor(InMeshAccessor)
	, UvSetIndex(InUvSetIndex)
	{
	}

	FVector BuildTriangleCoordinate(const FVector2D& InUv, int32 TriangleIndex) const
	{
		const FVector VertexUvs[] =
		{
			FVector(MeshAccessor.GetVertexUV(TriangleIndex * 3 + 0, UvSetIndex), 0.0f),
			FVector(MeshAccessor.GetVertexUV(TriangleIndex * 3 + 1, UvSetIndex), 0.0f),
			FVector(MeshAccessor.GetVertexUV(TriangleIndex * 3 + 2, UvSetIndex), 0.0f),
		};

		return FMath::GetBaryCentric2D(FVector(InUv, 0.0f), VertexUvs[0], VertexUvs[1], VertexUvs[2]);
	}

	void FindOverlappingTriangle(const FVector2D& InUv, float Tolerance, TArray<int32>& TriangleIndices) const
	{
		FBox2D UvBox(InUv, InUv);

		TArray<int32, TInlineAllocator<32>> Elements;
		QuadTree.GetElements(UvBox, Elements);

		for (int32 TriangleIndex : Elements)
		{
			// generate the barycentric coordinates using the UVs of the triangle, the result may not be in the (0,1) range
			const FVector BarycentricCoord = BuildTriangleCoordinate(InUv, TriangleIndex);

			if ((BarycentricCoord.GetMin() > -Tolerance) && (BarycentricCoord.GetMax() < (1.0f + Tolerance)))
			{
				TriangleIndices.Add(TriangleIndex);
			}
		}
	}

	int32 FindFirstTriangle(const FVector2D& InUv, float Tolerance, FVector& BarycentricCoord) const
	{
		int32 FoundTriangleIndex = INDEX_NONE;

		QuadTree.VisitElements(FBox2D(InUv, InUv), [&](int32 TriangleIndex)
		{
			// generate the barycentric coordinates using the UVs of the triangle, the result may not be in the (0,1) range
			FVector ElementCoord = BuildTriangleCoordinate(InUv, TriangleIndex);

			if ((ElementCoord.GetMin() > -Tolerance) && (ElementCoord.GetMax() < (1.0f + Tolerance)))
			{
				FoundTriangleIndex = TriangleIndex;
				BarycentricCoord = ElementCoord;
				return false;
			}

			return true;
		});

		return FoundTriangleIndex;
	}

	static bool NormalizedAabbTriangleOverlap(const FVector2D& A, const FVector2D& B, const FVector2D& C)
	{
		const FVector2D TriAabbMin = FVector2D(FMath::Min3(A.X, B.X, C.X), FMath::Min3(A.Y, B.Y, C.Y));
		const FVector2D TriAabbMax = FVector2D(FMath::Max3(A.X, B.X, C.X), FMath::Max3(A.Y, B.Y, C.Y));

		if (TriAabbMin.GetMax() > 1.0f || TriAabbMax.GetMin() < 0.0f)
		{
			return false;
		}

		const FVector2D TriangleEdges[] = { C - B, A - C, B - A };

		for (int32 i = 0; i < UE_ARRAY_COUNT(TriangleEdges); ++i)
		{
			const FVector2D SeparatingAxis(-TriangleEdges[i].Y, TriangleEdges[i].X);
			const double AabbSegmentMin = FMath::Min(0.0, FMath::Min3(SeparatingAxis.X, SeparatingAxis.Y, SeparatingAxis.X + SeparatingAxis.Y));
			const double AabbSegmentMax = FMath::Max(0.0, FMath::Max3(SeparatingAxis.X, SeparatingAxis.Y, SeparatingAxis.X + SeparatingAxis.Y));
			const double TriangleSegmentMin = FMath::Min3(FVector2D::DotProduct(A, SeparatingAxis), FVector2D::DotProduct(B, SeparatingAxis), FVector2D::DotProduct(C, SeparatingAxis));
			const double TriangleSegmentMax = FMath::Max3(FVector2D::DotProduct(A, SeparatingAxis), FVector2D::DotProduct(B, SeparatingAxis), FVector2D::DotProduct(C, SeparatingAxis));

			if (AabbSegmentMin > TriangleSegmentMax || AabbSegmentMax < TriangleSegmentMin)
			{
				return false;
			}
		}

		return true;
	}

	int32 FindFirstTriangle(const FBox2D& InUvBox, FVector& BarycentricCoord) const
	{
		int32 FoundTriangleIndex = INDEX_NONE;
		const FVector2D NormalizeScale = FVector2D(1.0f, 1.0f) / (InUvBox.Max - InUvBox.Min);
		const FVector2D NormalizeBias = FVector2D(1.0f, 1.0f) - InUvBox.Max * NormalizeScale;
		const FVector UvRef = FVector(InUvBox.GetCenter(), 0.0f);

		QuadTree.VisitElements(InUvBox, [&](int32 TriangleIndex)
		{
			const FVector2D A = MeshAccessor.GetVertexUV(TriangleIndex * 3 + 0, UvSetIndex);
			const FVector2D B = MeshAccessor.GetVertexUV(TriangleIndex * 3 + 1, UvSetIndex);
			const FVector2D C = MeshAccessor.GetVertexUV(TriangleIndex * 3 + 2, UvSetIndex);

			// evaluate if the triangle overlaps with the InUvBox
			if (!NormalizedAabbTriangleOverlap(NormalizeScale * A + NormalizeBias, NormalizeScale * B + NormalizeBias, NormalizeScale * C + NormalizeBias))
			{
				return true;
			}

			BarycentricCoord = FMath::GetBaryCentric2D(UvRef, FVector(A, 0.0f), FVector(B, 0.0f), FVector(C, 0.0f));
			FoundTriangleIndex = TriangleIndex;
			return false;
		});

		return FoundTriangleIndex;
	}
};

class FMeshUvMappingBufferProxy : public FRenderResource
{
public:
	void Initialize(const FMeshUvMapping& UvMappingData);

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	FShaderResourceViewRHIRef GetSrv() const { return UvMappingSrv; }
	uint32 GetBufferSize() const { return UvMappingBuffer ? UvMappingBuffer->GetSize() : 0; }

private:
	TResourceArray<uint8> FrozenQuadTree;

	FBufferRHIRef UvMappingBuffer;
	FShaderResourceViewRHIRef UvMappingSrv;

#if STATS
	int64 GpuMemoryUsage = 0;
#endif
};

struct FMeshUvMapping
{
	FMeshUvMapping(int32 InLodIndex, int32 InUvSetIndex);
	FMeshUvMapping() = delete;
	FMeshUvMapping(const FMeshUvMapping&) = delete;
	virtual ~FMeshUvMapping();

	bool IsUsed() const;
	bool CanBeDestroyed() const;
	void RegisterUser(FMeshUvMappingUsage Usage, bool bNeedsDataImmediately);
	void UnregisterUser(FMeshUvMappingUsage Usage);

	void FreezeQuadTree(TResourceArray<uint8>& OutQuadTree) const;
	const FMeshUvMappingBufferProxy* GetQuadTreeProxy() const;

	const int32 LodIndex;
	const int32 UvSetIndex;

protected:
	virtual void BuildQuadTree() = 0;
	void ReleaseQuadTree();

	void BuildGpuQuadTree();
	void ReleaseGpuQuadTree();

	FNiagaraUvQuadTree TriangleIndexQuadTree;
	TUniquePtr<FMeshUvMappingBufferProxy> FrozenQuadTreeProxy;

	std::atomic<int32> CpuQuadTreeUserCount = {0};
	std::atomic<int32> GpuQuadTreeUserCount = {0};
};
