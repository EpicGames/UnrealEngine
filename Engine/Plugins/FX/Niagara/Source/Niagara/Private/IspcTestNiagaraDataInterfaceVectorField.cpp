// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && INTEL_ISPC

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "NiagaraDataInterfaceVectorField.h"
#include "VectorField/VectorField.h"
#include "VectorVM.h"
#include "VectorField/VectorFieldStatic.h"

extern bool GNiagaraVectorFieldUseIspc;

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIspcTestNiagaraDataInterfaceVectorField, "Ispc.Graphics.NiagaraDataInterfaceVectorField", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FIspcTestNiagaraDataInterfaceVectorField::RunTest(const FString& Parameters)
{
	bool InitialState = GNiagaraVectorFieldUseIspc;

	FFloat16Color RepeatColor;
	RepeatColor.R = FFloat16(5.f);
	RepeatColor.G = FFloat16(6.f);
	RepeatColor.B = FFloat16(7.f);
	RepeatColor.A = FFloat16(0.f);

	float InX = 10.f;
	float InY = 10.f;
	float InZ = 10.f;

	GNiagaraVectorFieldUseIspc = true;
	float ISPCOutX = 0.f;
	float ISPCOutY = 0.f;
	float ISPCOutZ = 0.f;
	{
		TObjectPtr<UNiagaraDataInterfaceVectorField> VectorField = NewObject<UNiagaraDataInterfaceVectorField>();
		TObjectPtr<UVectorFieldStatic> StaticField = NewObject<UVectorFieldStatic>();
		if (!IsValid(VectorField) || !IsValid(StaticField))
		{
			return false;
		}

		StaticField->SizeX = 10;
		StaticField->SizeY = 10;
		StaticField->SizeZ = 10;
		StaticField->Intensity = 10.f;
		StaticField->Bounds = FBox(FVector(10.f), FVector(50.f));

		StaticField->InitResource();
		StaticField->bAllowCPUAccess = true;

		// Fill the static vector field with values.
		const uint32 VectorCount = (StaticField->SizeX * StaticField->SizeY * StaticField->SizeZ);
		const uint32 DestBufferSize = VectorCount * sizeof(FFloat16Color);
		StaticField->SourceData.Lock(LOCK_READ_WRITE);
		FFloat16Color* RESTRICT DestValues = (FFloat16Color*)StaticField->SourceData.Realloc((int64)DestBufferSize);
		for (uint32 VectorIndex = 0; VectorIndex < VectorCount; ++VectorIndex)
		{
			*DestValues = RepeatColor;
			DestValues++;
		}
		StaticField->SourceData.Unlock();

		StaticField->UpdateCPUData(true);
		VectorField->Field = StaticField;

		// HACK: Pretend we are executing with the VM.
		// This requires 6 floats to be pushed onto its registers.
		// In X, Y, Z
		// Out X, Y, Z
		// Assumes one instance
		FVectorVMExternalFunctionContext FunctionContext;
		FMemory::Memset(&FunctionContext, 0, sizeof(FunctionContext));
		const int32 RegisterCount = 6;
		FunctionContext.RegReadCount = 0;
		FunctionContext.NumInstances = 1;
		FunctionContext.NumRegisters = RegisterCount;
		FunctionContext.PerInstanceFnInstanceIdx = 0;
		float* Values[RegisterCount];
		Values[0] = &InX;
		Values[1] = &InY;
		Values[2] = &InZ;
		Values[3] = &ISPCOutX;
		Values[4] = &ISPCOutY;
		Values[5] = &ISPCOutZ;
		float** ValuePtr = Values;
		FunctionContext.RegisterData = (uint32**)ValuePtr;
		uint8 IncJunk[RegisterCount] = { 0, 1, 2, 3, 4, 5 };
		uint16 VecJunk[RegisterCount] = { 0, 1, 2, 3, 4, 5 };
		FunctionContext.RegInc = IncJunk;
		VectorField->SampleVectorField(FunctionContext);
	}

	GNiagaraVectorFieldUseIspc = false;
	float CPPOutX = 0.f;
	float CPPOutY = 0.f;
	float CPPOutZ = 0.f;
	{
		TObjectPtr<UNiagaraDataInterfaceVectorField> VectorField = NewObject<UNiagaraDataInterfaceVectorField>();
		TObjectPtr<UVectorFieldStatic> StaticField = NewObject<UVectorFieldStatic>();
		if (!IsValid(VectorField) || !IsValid(StaticField))
		{
			return false;
		}

		StaticField->SizeX = 10;
		StaticField->SizeY = 10;
		StaticField->SizeZ = 10;
		StaticField->Intensity = 10.f;
		StaticField->Bounds = FBox(FVector(10.f), FVector(50.f));

		StaticField->InitResource();
		StaticField->bAllowCPUAccess = true;

		// Fill the static vector field with values.
		const uint32 VectorCount = (StaticField->SizeX * StaticField->SizeY * StaticField->SizeZ);
		const uint32 DestBufferSize = VectorCount * sizeof(FFloat16Color);
		StaticField->SourceData.Lock(LOCK_READ_WRITE);
		FFloat16Color* RESTRICT DestValues = (FFloat16Color*)StaticField->SourceData.Realloc((int64)DestBufferSize);
		for (uint32 VectorIndex = 0; VectorIndex < VectorCount; ++VectorIndex)
		{
			*DestValues = RepeatColor;
			DestValues++;
		}
		StaticField->SourceData.Unlock();

		StaticField->UpdateCPUData(true);
		VectorField->Field = StaticField;

		// HACK: Pretend we are executing with the VM.
		// This requires 6 floats to be pushed onto its registers.
		// In X, Y, Z
		// Out X, Y, Z
		// Assumes one instance
		FVectorVMExternalFunctionContext FunctionContext;
		FMemory::Memset(&FunctionContext, 0, sizeof(FunctionContext));
		const int32 RegisterCount = 6;
		FunctionContext.RegReadCount = 0;
		FunctionContext.NumInstances = 1;
		FunctionContext.NumRegisters = RegisterCount;
		FunctionContext.PerInstanceFnInstanceIdx = 0;
		float* Values[RegisterCount];
		Values[0] = &InX;
		Values[1] = &InY;
		Values[2] = &InZ;
		Values[3] = &CPPOutX;
		Values[4] = &CPPOutY;
		Values[5] = &CPPOutZ;
		float** ValuePtr = Values;
		FunctionContext.RegisterData = (uint32**)ValuePtr;
		uint8 IncJunk[RegisterCount] = { 0, 1, 2, 3, 4, 5 };
		uint16 VecJunk[RegisterCount] = { 0, 1, 2, 3, 4, 5 };
		FunctionContext.RegInc = IncJunk;
		VectorField->SampleVectorField(FunctionContext);
	}

	GNiagaraVectorFieldUseIspc = InitialState;

	TestEqual(TEXT("X"), ISPCOutX, CPPOutX);
	TestEqual(TEXT("Y"), ISPCOutY, CPPOutY);
	TestEqual(TEXT("Z"), ISPCOutZ, CPPOutZ);

	return true;
}

#if 0
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIspcTestNiagaraDataInterfaceVectorField, "Ispc.Graphics.NiagaraDataInterfaceVectorField", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FIspcTestNiagaraDataInterfaceVectorField::RunTest(const FString& Parameters)
{
	bool InitialState = GNiagaraVectorFieldUseIspc;

	FFloat16Color RepeatColor;
	RepeatColor.R = FFloat16(5.f);
	RepeatColor.G = FFloat16(6.f);
	RepeatColor.B = FFloat16(7.f);
	RepeatColor.A = FFloat16(0.f);

	float InX = 10.f;
	float InY = 10.f;
	float InZ = 10.f;

	GNiagaraVectorFieldUseIspc = true;
	float ISPCOutX = 0.f;
	float ISPCOutY = 0.f;
	float ISPCOutZ = 0.f;
	{
		TObjectPtr<UNiagaraDataInterfaceVectorField> VectorField = NewObject<UNiagaraDataInterfaceVectorField>();
		TObjectPtr<UVectorFieldStatic> StaticField = NewObject<UVectorFieldStatic>();
		if (!IsValid(VectorField) || !IsValid(StaticField))
		{
			return false;
		}

		StaticField->SizeX = 10;
		StaticField->SizeY = 10;
		StaticField->SizeZ = 10;
		StaticField->Intensity = 10.f;
		StaticField->Bounds = FBox(FVector(10.f), FVector(50.f));

		StaticField->InitResource();
		StaticField->bAllowCPUAccess = true;

		// Fill the static vector field with values.
		const uint32 VectorCount = (StaticField->SizeX * StaticField->SizeY * StaticField->SizeZ);
		const uint32 DestBufferSize = VectorCount * sizeof(FFloat16Color);
		StaticField->SourceData.Lock(LOCK_READ_WRITE);
		FFloat16Color* RESTRICT DestValues = (FFloat16Color*)StaticField->SourceData.Realloc((int64)DestBufferSize);
		for (uint32 VectorIndex = 0; VectorIndex < VectorCount; ++VectorIndex)
		{
			*DestValues = RepeatColor;
			DestValues++;
		}
		StaticField->SourceData.Unlock();

		StaticField->UpdateCPUData(true);
		VectorField->Field = StaticField;

		// HACK: Pretend we are executing with the VM.
		// This requires 6 floats to be pushed onto its registers.
		// In X, Y, Z
		// Out X, Y, Z
		// Assumes one instance
		FVectorVMContext VectorVM;
		const int32 RegisterCount = 6;
		float* Values[RegisterCount];
		Values[0] = &InX;
		Values[1] = &InY;
		Values[2] = &InZ;
		Values[3] = &ISPCOutX;
		Values[4] = &ISPCOutY;
		Values[5] = &ISPCOutZ;
		float** ValuePtr = Values;
		TArray<FDataSetMeta> JunkData;
		VectorVM.PrepareForExec(RegisterCount,
			0,
			0,
			0,
			0,
			(void**)ValuePtr,
			JunkData,
			1,
			false);
		FVectorVMExternalFunctionContextLegacy Legacy(&VectorVM);
		VectorField->SampleVectorField(Legacy);
	}

	GNiagaraVectorFieldUseIspc = false;
	float CPPOutX = 0.f;
	float CPPOutY = 0.f;
	float CPPOutZ = 0.f;
	{
		TObjectPtr<UNiagaraDataInterfaceVectorField> VectorField = NewObject<UNiagaraDataInterfaceVectorField>();
		TObjectPtr<UVectorFieldStatic> StaticField = NewObject<UVectorFieldStatic>();
		if (!IsValid(VectorField) || !IsValid(StaticField))
		{
			return false;
		}

		StaticField->SizeX = 10;
		StaticField->SizeY = 10;
		StaticField->SizeZ = 10;
		StaticField->Intensity = 10.f;
		StaticField->Bounds = FBox(FVector(10.f), FVector(50.f));

		StaticField->InitResource();
		StaticField->bAllowCPUAccess = true;

		// Fill the static vector field with values.
		const uint32 VectorCount = (StaticField->SizeX * StaticField->SizeY * StaticField->SizeZ);
		const uint32 DestBufferSize = VectorCount * sizeof(FFloat16Color);
		StaticField->SourceData.Lock(LOCK_READ_WRITE);
		FFloat16Color* RESTRICT DestValues = (FFloat16Color*)StaticField->SourceData.Realloc((int64)DestBufferSize);
		for (uint32 VectorIndex = 0; VectorIndex < VectorCount; ++VectorIndex)
		{
			*DestValues = RepeatColor;
			DestValues++;
		}
		StaticField->SourceData.Unlock();

		StaticField->UpdateCPUData(true);
		VectorField->Field = StaticField;

		// HACK: Pretend we are executing with the VM.
		// This requires 6 floats to be pushed onto its registers.
		// In X, Y, Z
		// Out X, Y, Z
		// Assumes one instance
		FVectorVMContext VectorVM;
		const int32 RegisterCount = 6;
		float* Values[RegisterCount];
		Values[0] = &InX;
		Values[1] = &InY;
		Values[2] = &InZ;
		Values[3] = &ISPCOutX;
		Values[4] = &ISPCOutY;
		Values[5] = &ISPCOutZ;
		float** ValuePtr = Values;
		TArray<FDataSetMeta> JunkData;
		VectorVM.PrepareForExec(RegisterCount,
			0,
			0,
			0,
			0,
			(void**)ValuePtr,
			JunkData,
			1,
			false);
		FVectorVMExternalFunctionContextLegacy Legacy(&VectorVM);
		VectorField->SampleVectorField(Legacy);
	}

	GNiagaraVectorFieldUseIspc = InitialState;

	TestEqual(TEXT("X"), ISPCOutX, CPPOutX);
	TestEqual(TEXT("Y"), ISPCOutY, CPPOutY);
	TestEqual(TEXT("Z"), ISPCOutZ, CPPOutZ);

	return true;
}
#endif

#endif // WITH_DEV_AUTOMATION_TESTS
