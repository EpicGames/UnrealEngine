// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneNiagaraSystemSpawnSection.h"
#include "NiagaraCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraSystemSpawnSection)

UMovieSceneNiagaraSystemSpawnSection::UMovieSceneNiagaraSystemSpawnSection()
{
	EvalOptions.CompletionMode = EMovieSceneCompletionMode::RestoreState;
	SectionStartBehavior = ENiagaraSystemSpawnSectionStartBehavior::Activate;
	SectionEvaluateBehavior = ENiagaraSystemSpawnSectionEvaluateBehavior::ActivateIfInactive;
	SectionEndBehavior = ENiagaraSystemSpawnSectionEndBehavior::SetSystemInactive;
	AgeUpdateMode = ENiagaraAgeUpdateMode::TickDeltaTime;
	bAllowScalability = false;
}

ENiagaraSystemSpawnSectionStartBehavior UMovieSceneNiagaraSystemSpawnSection::GetSectionStartBehavior() const
{
	return SectionStartBehavior;
}

void UMovieSceneNiagaraSystemSpawnSection::SetSectionStartBehavior(ENiagaraSystemSpawnSectionStartBehavior InBehavior)
{
	SectionStartBehavior = InBehavior;
}

ENiagaraSystemSpawnSectionEvaluateBehavior UMovieSceneNiagaraSystemSpawnSection::GetSectionEvaluateBehavior() const
{
	return SectionEvaluateBehavior;
}

void UMovieSceneNiagaraSystemSpawnSection::SetSectionEvaluateBehavior(ENiagaraSystemSpawnSectionEvaluateBehavior InBehavior)
{
	SectionEvaluateBehavior = InBehavior;
}

ENiagaraSystemSpawnSectionEndBehavior UMovieSceneNiagaraSystemSpawnSection::GetSectionEndBehavior() const
{
	return SectionEndBehavior;
}

void UMovieSceneNiagaraSystemSpawnSection::SetSectionEndBehavior(ENiagaraSystemSpawnSectionEndBehavior InBehavior)
{
	SectionEndBehavior = InBehavior;
}


ENiagaraAgeUpdateMode UMovieSceneNiagaraSystemSpawnSection::GetAgeUpdateMode() const
{
	return AgeUpdateMode;
}

void UMovieSceneNiagaraSystemSpawnSection::SetAgeUpdateMode(ENiagaraAgeUpdateMode InMode)
{
	AgeUpdateMode = InMode;
}

bool UMovieSceneNiagaraSystemSpawnSection::GetAllowScalability() const
{
	return bAllowScalability;
}

void UMovieSceneNiagaraSystemSpawnSection::SetAllowScalability(bool bInAllowScalability)
{
	bAllowScalability = bInAllowScalability;
}
