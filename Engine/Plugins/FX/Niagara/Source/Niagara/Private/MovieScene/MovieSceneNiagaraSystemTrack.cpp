// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneNiagaraSystemTrack.h"
#include "MovieScene.h"
#include "MovieScene/MovieSceneNiagaraSystemSpawnSection.h"
#include "MovieScene/MovieSceneNiagaraSystemTrackTemplate.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraSystemTrack)

FMovieSceneSharedDataId UMovieSceneNiagaraSystemTrack::SharedDataId = FMovieSceneSharedDataId::Allocate();

bool UMovieSceneNiagaraSystemTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneNiagaraSystemSpawnSection::StaticClass();
}

UMovieSceneSection* UMovieSceneNiagaraSystemTrack::CreateNewSection()
{
	return NewObject<UMovieSceneNiagaraSystemSpawnSection>(this, NAME_None, RF_Transactional);
}

FMovieSceneEvalTemplatePtr UMovieSceneNiagaraSystemTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneEvalTemplatePtr();
}

void UMovieSceneNiagaraSystemTrack::PostCompile(FMovieSceneEvaluationTrack& OutTrack, const FMovieSceneTrackCompilerArgs& Args) const
{
	TObjectPtr<UMovieSceneSection> const* SpawnSectionPtr = Sections.FindByPredicate([](const UMovieSceneSection* Section) { return Section->GetClass() == UMovieSceneNiagaraSystemSpawnSection::StaticClass(); });
	if (SpawnSectionPtr != nullptr)
	{
		UMovieSceneNiagaraSystemSpawnSection* SpawnSection = CastChecked<UMovieSceneNiagaraSystemSpawnSection>(*SpawnSectionPtr);
		OutTrack.SetTrackImplementation(FMovieSceneNiagaraSystemTrackImplementation(
			SpawnSection->GetInclusiveStartFrame(), SpawnSection->GetExclusiveEndFrame(),
			SpawnSection->GetSectionStartBehavior(), SpawnSection->GetSectionEvaluateBehavior(),
			SpawnSection->GetSectionEndBehavior(), SpawnSection->GetAgeUpdateMode(), SpawnSection->GetAllowScalability()));
	}
}

bool UMovieSceneNiagaraSystemTrack::PopulateEvaluationTree(TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData) const
{
	UMovieSceneNiagaraSystemTrack* This = const_cast<UMovieSceneNiagaraSystemTrack*>(this);
	// We always just evaluate everything
	OutData.Add(TRange<FFrameNumber>::All(), FMovieSceneTrackEvaluationData::FromTrack(This));

	return true;
}

