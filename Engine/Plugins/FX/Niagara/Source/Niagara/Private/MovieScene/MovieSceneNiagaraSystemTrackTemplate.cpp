// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneNiagaraSystemTrackTemplate.h"
#include "Evaluation/MovieSceneExecutionTokens.h"
#include "MovieSceneExecutionToken.h"
#include "MovieScene/MovieSceneNiagaraSystemSpawnSection.h"
#include "NiagaraComponent.h"
#include "NiagaraSystemInstanceController.h"
#include "IMovieScenePlayer.h"

#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "MovieScene/MovieSceneNiagaraSystemTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraSystemTrackTemplate)

struct FPreAnimatedNiagaraComponentToken : IMovieScenePreAnimatedToken
{
	FPreAnimatedNiagaraComponentToken(
		bool bInComponentIsActive,
		bool bInComponentForceSolo,
		bool bInComponentRenderingEnabled,
		TOptional<ENiagaraExecutionState> InSystemInstanceExecutionState,
		ENiagaraAgeUpdateMode InComponentAgeUpdateMode,
		bool bInComponentAllowScalability,
		float InComponentSeekDelta,
		float InComponentDesiredAge,
		bool bInComponentLockDesiredAgeDeltaTimeToSeekDelta
	)
		: bComponentIsActive(bInComponentIsActive)
		, bComponentForceSolo(bInComponentForceSolo)
		, bComponentRenderingEnabled(bInComponentRenderingEnabled)
		, SystemInstanceExecutionState(InSystemInstanceExecutionState)
		, ComponentAgeUpdateMode(InComponentAgeUpdateMode)
		, bComponentAllowScalability(bInComponentAllowScalability)
		, ComponentSeekDelta(InComponentSeekDelta)
		, ComponentDesiredAge(InComponentDesiredAge)
		, bComponentLockDesiredAgeDeltaTimeToSeekDelta(bInComponentLockDesiredAgeDeltaTimeToSeekDelta)
	{ }

	virtual void RestoreState(UObject& InObject, const UE::MovieScene::FRestoreStateParams& Params)
	{
		UNiagaraComponent* NiagaraComponent = CastChecked<UNiagaraComponent>(&InObject);		
		if (bComponentIsActive)
		{
			NiagaraComponent->Activate();
		}
		else
		{
			// TODO: This is seemingly done because there is some state that isn't fully reset on Deactivate. Should be a single component call
			if (auto SystemInstanceController = NiagaraComponent->GetSystemInstanceController())		
			{
				SystemInstanceController->Reset(FNiagaraSystemInstance::EResetMode::ResetSystem);
			}
			NiagaraComponent->Deactivate();
		}
		NiagaraComponent->SetForceSolo(bComponentForceSolo);
		NiagaraComponent->SetRenderingEnabled(bComponentRenderingEnabled);
		NiagaraComponent->SetAgeUpdateMode(ComponentAgeUpdateMode);
		NiagaraComponent->SetAllowScalability(bComponentAllowScalability);
		NiagaraComponent->SetSeekDelta(ComponentSeekDelta);
		NiagaraComponent->SetDesiredAge(ComponentDesiredAge);
		NiagaraComponent->SetAllowScalability(bComponentAllowScalability);
		NiagaraComponent->SetLockDesiredAgeDeltaTimeToSeekDelta(bComponentLockDesiredAgeDeltaTimeToSeekDelta);

		// TODO: When this action is ACTUALLY deferred, just expose it to the component		
		if (SystemInstanceExecutionState.IsSet())
		{
			// NOTE: Get the controller again here because it might have been released on Deactivate
			if (auto SystemInstanceController = NiagaraComponent->GetSystemInstanceController())		
			{
				SystemInstanceController->SetRequestedExecutionState_Deferred(SystemInstanceExecutionState.GetValue());
			}
		}
	}

	bool bComponentIsActive;
	bool bComponentForceSolo;
	bool bComponentRenderingEnabled;
	TOptional<ENiagaraExecutionState> SystemInstanceExecutionState;
	ENiagaraAgeUpdateMode ComponentAgeUpdateMode;
	bool bComponentAllowScalability;
	float ComponentSeekDelta;
	float ComponentDesiredAge;
	bool bComponentLockDesiredAgeDeltaTimeToSeekDelta;
};

struct FPreAnimatedNiagaraComponentTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& InObject) const override
	{
		UNiagaraComponent* NiagaraComponent = CastChecked<UNiagaraComponent>(&InObject);
		// TODO: Shouldn't need to access the SystemInstanceController directly here, want to eventually use the component interface only
		FNiagaraSystemInstanceControllerPtr SystemInstanceController = NiagaraComponent->GetSystemInstanceController();
		return FPreAnimatedNiagaraComponentToken(
			NiagaraComponent->IsActive(),
			NiagaraComponent->GetForceSolo(),
			NiagaraComponent->GetRenderingEnabled(),
			SystemInstanceController.IsValid() ? SystemInstanceController->GetRequestedExecutionState() : TOptional<ENiagaraExecutionState>(),
			NiagaraComponent->GetAgeUpdateMode(),
			NiagaraComponent->GetAllowScalability(),
			NiagaraComponent->GetSeekDelta(),
			NiagaraComponent->GetDesiredAge(),
			NiagaraComponent->GetLockDesiredAgeDeltaTimeToSeekDelta());
	}
};

struct FNiagaraSystemUpdateDesiredAgeExecutionToken : IMovieSceneExecutionToken
{
	FNiagaraSystemUpdateDesiredAgeExecutionToken(
		FFrameNumber InSpawnSectionStartFrame, FFrameNumber InSpawnSectionEndFrame,
		ENiagaraSystemSpawnSectionStartBehavior InSpawnSectionStartBehavior, ENiagaraSystemSpawnSectionEvaluateBehavior InSpawnSectionEvaluateBehavior,
		ENiagaraSystemSpawnSectionEndBehavior InSpawnSectionEndBehavior, ENiagaraAgeUpdateMode InAgeUpdateMode, bool bInAllowScalability)
		: SpawnSectionStartFrame(InSpawnSectionStartFrame)
		, SpawnSectionEndFrame(InSpawnSectionEndFrame)
		, SpawnSectionStartBehavior(InSpawnSectionStartBehavior)
		, SpawnSectionEvaluateBehavior(InSpawnSectionEvaluateBehavior)
		, SpawnSectionEndBehavior(InSpawnSectionEndBehavior)
		, AgeUpdateMode(InAgeUpdateMode)
		, bAllowScalability(bInAllowScalability)
	{}

	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		for (TWeakObjectPtr<> Object : Player.FindBoundObjects(Operand))
		{
			UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(Object);
			if (!NiagaraComponent)
			{
				continue;
			}

			{
				static FMovieSceneAnimTypeID TypeID = TMovieSceneAnimTypeID<FNiagaraSystemUpdateDesiredAgeExecutionToken, 0>();

				FScopedPreAnimatedCaptureSource CaptureSource(&Player.PreAnimatedState, PersistentData.GetTrackKey(), true);
				Player.PreAnimatedState.SavePreAnimatedState(*NiagaraComponent, TypeID, FPreAnimatedNiagaraComponentTokenProducer());
			}

			NiagaraComponent->SetForceSolo(true);
			NiagaraComponent->SetAgeUpdateMode(AgeUpdateMode);
			NiagaraComponent->SetAllowScalability(bAllowScalability);

			UMovieSceneSequence* MovieSceneSequence = Player.GetEvaluationTemplate().GetSequence(Operand.SequenceID);
			if (MovieSceneSequence != nullptr)
			{
				UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();
				if (MovieScene != nullptr)
				{
					NiagaraComponent->SetSeekDelta((float)MovieScene->GetDisplayRate().Denominator / MovieScene->GetDisplayRate().Numerator);
					NiagaraComponent->SetLockDesiredAgeDeltaTimeToSeekDelta(MovieScene->GetEvaluationType() == EMovieSceneEvaluationType::FrameLocked);
				}
			}

			FNiagaraSystemInstanceControllerPtr SystemInstanceController = NiagaraComponent->GetSystemInstanceController();

			if (Context.GetTime() < SpawnSectionStartFrame)
			{
				if (SpawnSectionStartBehavior == ENiagaraSystemSpawnSectionStartBehavior::Activate)
				{
					if (NiagaraComponent->IsActive())
					{
						// TODO: More stuff that should be a single component call once deferred
						NiagaraComponent->DeactivateImmediate();
						if (SystemInstanceController.IsValid() && SystemInstanceController->IsValid())
						{
							SystemInstanceController->Reset(FNiagaraSystemInstance::EResetMode::ResetAll);
						}
					}
				}
			}
			else if (Context.GetRange().Overlaps(TRange<FFrameTime>(FFrameTime(SpawnSectionStartFrame))))
			{
				if (SpawnSectionStartBehavior == ENiagaraSystemSpawnSectionStartBehavior::Activate)
				{
					if (NiagaraComponent->IsActive())
					{
						// TODO: More stuff that should be a single component call once deferred
						NiagaraComponent->DeactivateImmediate();
						if (SystemInstanceController.IsValid() && SystemInstanceController->IsValid())
						{
							SystemInstanceController->Reset(FNiagaraSystemInstance::EResetMode::ResetAll);
						}
					}
					NiagaraComponent->Activate();
					SystemInstanceController = NiagaraComponent->GetSystemInstanceController();
				}
			}
			else if (Context.GetTime() < SpawnSectionEndFrame)
			{
				if (SpawnSectionEvaluateBehavior == ENiagaraSystemSpawnSectionEvaluateBehavior::ActivateIfInactive)
				{
					if (NiagaraComponent->IsActive() == false)
					{
						NiagaraComponent->Activate();
						SystemInstanceController = NiagaraComponent->GetSystemInstanceController();
					}
					
					// TODO: Once ACTUALLY deferred, should just be a component call
					if (SystemInstanceController.IsValid() && SystemInstanceController->IsValid())
					{
						SystemInstanceController->SetRequestedExecutionState_Deferred(ENiagaraExecutionState::Active);
					}
				}
			}
			else
			{
				if (SpawnSectionEndBehavior == ENiagaraSystemSpawnSectionEndBehavior::SetSystemInactive)
				{
					// TODO: Once ACTUALLY deferred, should just be a component call
					if (SystemInstanceController.IsValid() && SystemInstanceController->IsValid())
					{
						SystemInstanceController->SetRequestedExecutionState_Deferred(ENiagaraExecutionState::Inactive);
					}
				}
				else if (SpawnSectionEndBehavior == ENiagaraSystemSpawnSectionEndBehavior::Deactivate)
				{
					if (NiagaraComponent->IsActive())
					{
						// TODO: More stuff that should be a single component call once deferred
						NiagaraComponent->DeactivateImmediate();
						if (SystemInstanceController.IsValid() && SystemInstanceController->IsValid())
						{
							SystemInstanceController->Reset(FNiagaraSystemInstance::EResetMode::ResetAll);
						}
					}
				}
			}

			bool bRenderingEnabled = Context.IsPreRoll() == false;
			NiagaraComponent->SetRenderingEnabled(bRenderingEnabled);

			if (SystemInstanceController.IsValid() && SystemInstanceController->IsValid() && SystemInstanceController->IsComplete() == false)
			{
				const float DesiredAge = FMath::RoundToFloat(static_cast<float>(Context.GetFrameRate().AsSeconds(Context.GetTime() - SpawnSectionStartFrame)) * MinAgeResolution) / MinAgeResolution;
				if (DesiredAge >= 0)
				{
					// Add a quarter of a frame offset here to push the desired age into the middle of the frame since it will be automatically rounded
					// down to the nearest seek delta.  This prevents a situation where float rounding results in a value which is just slightly less than
					// the frame boundary, which results in a skipped simulation frame.
					const float FrameOffset = NiagaraComponent->GetLockDesiredAgeDeltaTimeToSeekDelta() ? (NiagaraComponent->GetSeekDelta() / 4.0f) : 0.0f;
					NiagaraComponent->SetDesiredAge(DesiredAge + FrameOffset);
				}
			}
		}
	}

	FFrameNumber SpawnSectionStartFrame;
	FFrameNumber SpawnSectionEndFrame;
	ENiagaraSystemSpawnSectionStartBehavior SpawnSectionStartBehavior;
	ENiagaraSystemSpawnSectionEvaluateBehavior SpawnSectionEvaluateBehavior;
	ENiagaraSystemSpawnSectionEndBehavior SpawnSectionEndBehavior;
	ENiagaraAgeUpdateMode AgeUpdateMode;
	bool bAllowScalability;

	//TODO (mga) move to global sim resolution config, update sim cache as well
	static constexpr float MinAgeResolution = 10000.0f;
};

FMovieSceneNiagaraSystemTrackImplementation::FMovieSceneNiagaraSystemTrackImplementation(
	FFrameNumber InSpawnSectionStartFrame, FFrameNumber InSpawnSectionEndFrame,
	ENiagaraSystemSpawnSectionStartBehavior InSpawnSectionStartBehavior, ENiagaraSystemSpawnSectionEvaluateBehavior InSpawnSectionEvaluateBehavior,
	ENiagaraSystemSpawnSectionEndBehavior InSpawnSectionEndBehavior, ENiagaraAgeUpdateMode InAgeUpdateMode, bool bInAllowScalability)
	: SpawnSectionStartFrame(InSpawnSectionStartFrame)
	, SpawnSectionEndFrame(InSpawnSectionEndFrame)
	, SpawnSectionStartBehavior(InSpawnSectionStartBehavior)
	, SpawnSectionEvaluateBehavior(InSpawnSectionEvaluateBehavior)
	, SpawnSectionEndBehavior(InSpawnSectionEndBehavior)
	, AgeUpdateMode(InAgeUpdateMode)
	, bAllowScalability(bInAllowScalability)

{
}

FMovieSceneNiagaraSystemTrackImplementation::FMovieSceneNiagaraSystemTrackImplementation()
	: SpawnSectionStartFrame(FFrameNumber())
	, SpawnSectionEndFrame(FFrameNumber())
	, SpawnSectionStartBehavior(ENiagaraSystemSpawnSectionStartBehavior::Activate)
	, SpawnSectionEvaluateBehavior(ENiagaraSystemSpawnSectionEvaluateBehavior::None)
	, SpawnSectionEndBehavior(ENiagaraSystemSpawnSectionEndBehavior::SetSystemInactive)
	, AgeUpdateMode(ENiagaraAgeUpdateMode::TickDeltaTime)
	, bAllowScalability(false)
{
}

void FMovieSceneNiagaraSystemTrackImplementation::Evaluate(const FMovieSceneEvaluationTrack& Track, TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	ExecutionTokens.SetContext(Context);

	// only add a token if there isn't one already, otherwise another track's token takes precendence
	FMovieSceneSharedDataId TokenID = UMovieSceneNiagaraSystemTrack::SharedDataId;
	FNiagaraSharedMarkerToken* SharedCacheToken = static_cast<FNiagaraSharedMarkerToken*>(ExecutionTokens.FindShared(TokenID));
	if (SharedCacheToken == nullptr || !SharedCacheToken->BoundObjectIDs.Contains(Operand.ObjectBindingID))
	{
		ExecutionTokens.Add(FNiagaraSystemUpdateDesiredAgeExecutionToken(
		SpawnSectionStartFrame, SpawnSectionEndFrame,
		SpawnSectionStartBehavior, SpawnSectionEvaluateBehavior,
		SpawnSectionEndBehavior, AgeUpdateMode, bAllowScalability));
	}
}
