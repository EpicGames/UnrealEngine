// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Evaluation/MovieSceneTrackImplementation.h"
#include "MovieSceneNiagaraSystemTrackTemplate.generated.h"

enum class ENiagaraAgeUpdateMode : uint8;
enum class ENiagaraSystemSpawnSectionEndBehavior : uint8;
enum class ENiagaraSystemSpawnSectionEvaluateBehavior : uint8;
enum class ENiagaraSystemSpawnSectionStartBehavior : uint8;

USTRUCT()
struct FMovieSceneNiagaraSystemTrackTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()

public:
	FMovieSceneNiagaraSystemTrackTemplate() { }

private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
};

USTRUCT()
struct FMovieSceneNiagaraSystemTrackImplementation : public FMovieSceneTrackImplementation
{
	GENERATED_BODY()

public:
	FMovieSceneNiagaraSystemTrackImplementation();
	FMovieSceneNiagaraSystemTrackImplementation(
		FFrameNumber InSpawnSectionStartFrame, FFrameNumber InSpawnSectionEndFrame,
		ENiagaraSystemSpawnSectionStartBehavior InSectionStartBehavior, ENiagaraSystemSpawnSectionEvaluateBehavior InSectionEvaluateBehavior,
		ENiagaraSystemSpawnSectionEndBehavior InSectionEndBehavior, ENiagaraAgeUpdateMode InAgeUpdateMode, bool bInAllowScalability);

	virtual void SetupOverrides() override
	{
		EnableOverrides(CustomEvaluateFlag);
	}

private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationTrack& Track, TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

private:
	UPROPERTY()
	FFrameNumber SpawnSectionStartFrame;
	UPROPERTY()
	FFrameNumber SpawnSectionEndFrame;
	UPROPERTY()
	ENiagaraSystemSpawnSectionStartBehavior SpawnSectionStartBehavior;
	UPROPERTY()
	ENiagaraSystemSpawnSectionEvaluateBehavior SpawnSectionEvaluateBehavior;
	UPROPERTY()
	ENiagaraSystemSpawnSectionEndBehavior SpawnSectionEndBehavior;
	UPROPERTY()
	ENiagaraAgeUpdateMode AgeUpdateMode;
	UPROPERTY()
	bool bAllowScalability;

};
