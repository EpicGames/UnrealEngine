// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneNiagaraTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraTrack)

void UMovieSceneNiagaraTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}

const TArray<UMovieSceneSection*>& UMovieSceneNiagaraTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneNiagaraTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

bool UMovieSceneNiagaraTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

void UMovieSceneNiagaraTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

void UMovieSceneNiagaraTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneNiagaraTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

