// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneNiagaraBoolParameterSectionTemplate.h"
#include "NiagaraTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraBoolParameterSectionTemplate)

FMovieSceneNiagaraBoolParameterSectionTemplate::FMovieSceneNiagaraBoolParameterSectionTemplate()
{
}

FMovieSceneNiagaraBoolParameterSectionTemplate::FMovieSceneNiagaraBoolParameterSectionTemplate(FNiagaraVariable InParameter, const FMovieSceneBoolChannel& InBoolChannel)
	: FMovieSceneNiagaraParameterSectionTemplate(InParameter)
	, BoolChannel(InBoolChannel)
{
}

void FMovieSceneNiagaraBoolParameterSectionTemplate::GetAnimatedParameterValue(FFrameTime InTime, const FNiagaraVariableBase& InTargetParameter, const TArray<uint8>& InCurrentValueData, TArray<uint8>& OutAnimatedValueData) const
{
	FNiagaraBool AnimatedNiagaraValue;
	FMemory::Memcpy(&AnimatedNiagaraValue, InCurrentValueData.GetData(), sizeof(FNiagaraBool));

	bool AnimatedValue;
	if (BoolChannel.Evaluate(InTime, AnimatedValue))
	{
		AnimatedNiagaraValue.SetValue(AnimatedValue);
	}
	
	OutAnimatedValueData.AddUninitialized(sizeof(FNiagaraBool));
	FMemory::Memcpy(OutAnimatedValueData.GetData(), (uint8*)&AnimatedNiagaraValue, sizeof(FNiagaraBool));
}
