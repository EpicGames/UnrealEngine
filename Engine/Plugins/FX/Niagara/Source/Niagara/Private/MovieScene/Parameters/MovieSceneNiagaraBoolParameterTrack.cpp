// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/Parameters/MovieSceneNiagaraBoolParameterTrack.h"
#include "MovieScene/Parameters/MovieSceneNiagaraBoolParameterSectionTemplate.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraBoolParameterTrack)

bool UMovieSceneNiagaraBoolParameterTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneBoolSection::StaticClass();
}

UMovieSceneSection* UMovieSceneNiagaraBoolParameterTrack::CreateNewSection()
{
	return NewObject<UMovieSceneBoolSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneNiagaraBoolParameterTrack::SetSectionChannelDefaults(UMovieSceneSection* Section, const TArray<uint8>& DefaultValueData) const
{
	UMovieSceneBoolSection* BoolSection = Cast<UMovieSceneBoolSection>(Section);
	if (ensureMsgf(BoolSection != nullptr, TEXT("Section must be a bool section.")) && ensureMsgf(DefaultValueData.Num() == sizeof(FNiagaraBool), TEXT("DefaultValueData must be a FNiagaraBool.")))
	{
		FMovieSceneChannelProxy& BoolChannelProxy = BoolSection->GetChannelProxy();
		FNiagaraBool DefaultValue;
		FMemory::Memcpy(&DefaultValue, DefaultValueData.GetData(), sizeof(FNiagaraBool));
		SetChannelDefault(BoolChannelProxy, BoolSection->GetChannel(), DefaultValue.GetValue());
	}
}

FMovieSceneEvalTemplatePtr UMovieSceneNiagaraBoolParameterTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	const UMovieSceneBoolSection* BoolSection = Cast<UMovieSceneBoolSection>(&InSection);
	if (BoolSection != nullptr)
	{
		return FMovieSceneNiagaraBoolParameterSectionTemplate(GetParameter(), BoolSection->GetChannel());
	}
	return FMovieSceneEvalTemplatePtr();
}
