// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneNiagaraColorParameterSectionTemplate.h"
#include "NiagaraTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraColorParameterSectionTemplate)

FMovieSceneNiagaraColorParameterSectionTemplate::FMovieSceneNiagaraColorParameterSectionTemplate()
{
}

FMovieSceneNiagaraColorParameterSectionTemplate::FMovieSceneNiagaraColorParameterSectionTemplate(FNiagaraVariable InParameter, const FMovieSceneFloatChannel& InRedChannel, const FMovieSceneFloatChannel& InGreenChannel, const FMovieSceneFloatChannel& InBlueChannel, const FMovieSceneFloatChannel& InAlphaChannel)
	: FMovieSceneNiagaraParameterSectionTemplate(InParameter)
	, RedChannel(InRedChannel)
	, GreenChannel(InGreenChannel)
	, BlueChannel(InBlueChannel)
	, AlphaChannel(InAlphaChannel)
{
}

void FMovieSceneNiagaraColorParameterSectionTemplate::GetAnimatedParameterValue(FFrameTime InTime, const FNiagaraVariableBase& InTargetParameter, const TArray<uint8>& InCurrentValueData, TArray<uint8>& OutAnimatedValueData) const
{
	FLinearColor AnimatedValue;
	FMemory::Memcpy(&AnimatedValue, InCurrentValueData.GetData(), sizeof(FLinearColor));

	RedChannel.Evaluate(InTime, AnimatedValue.R);
	GreenChannel.Evaluate(InTime, AnimatedValue.G);
	BlueChannel.Evaluate(InTime, AnimatedValue.B);
	AlphaChannel.Evaluate(InTime, AnimatedValue.A);

	OutAnimatedValueData.AddUninitialized(sizeof(FLinearColor));
	FMemory::Memcpy(OutAnimatedValueData.GetData(), (uint8*)&AnimatedValue, sizeof(FLinearColor));
}
