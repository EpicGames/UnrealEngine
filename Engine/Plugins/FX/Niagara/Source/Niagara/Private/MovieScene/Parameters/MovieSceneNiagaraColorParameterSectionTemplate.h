// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene/Parameters/MovieSceneNiagaraParameterSectionTemplate.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneNiagaraColorParameterSectionTemplate.generated.h"

struct FNiagaraVariable;
struct FNiagaraVariableBase;

USTRUCT()
struct FMovieSceneNiagaraColorParameterSectionTemplate : public FMovieSceneNiagaraParameterSectionTemplate
{
	GENERATED_BODY()

public:
	FMovieSceneNiagaraColorParameterSectionTemplate();

	FMovieSceneNiagaraColorParameterSectionTemplate(FNiagaraVariable InParameter, const FMovieSceneFloatChannel& InRedChannel, const FMovieSceneFloatChannel& InGreenChannel, const FMovieSceneFloatChannel& InBlueChannel, const FMovieSceneFloatChannel& InAlphaChannel);

private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }

protected:
	virtual void GetAnimatedParameterValue(FFrameTime InTime, const FNiagaraVariableBase& InTargetParameter, const TArray<uint8>& InCurrentValueData, TArray<uint8>& OutAnimatedValueData) const override;

private:
	UPROPERTY()
	FMovieSceneFloatChannel RedChannel;

	UPROPERTY()
	FMovieSceneFloatChannel GreenChannel;

	UPROPERTY()
	FMovieSceneFloatChannel BlueChannel;

	UPROPERTY()
	FMovieSceneFloatChannel AlphaChannel;
};
