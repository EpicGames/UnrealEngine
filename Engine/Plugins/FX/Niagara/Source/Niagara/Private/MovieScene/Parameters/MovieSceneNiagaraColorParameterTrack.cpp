// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/Parameters/MovieSceneNiagaraColorParameterTrack.h"
#include "MovieScene/Parameters/MovieSceneNiagaraColorParameterSectionTemplate.h"
#include "Sections/MovieSceneColorSection.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Channels/MovieSceneChannelProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraColorParameterTrack)

bool UMovieSceneNiagaraColorParameterTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneColorSection::StaticClass();
}

UMovieSceneSection* UMovieSceneNiagaraColorParameterTrack::CreateNewSection()
{
	return NewObject<UMovieSceneColorSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneNiagaraColorParameterTrack::SetSectionChannelDefaults(UMovieSceneSection* Section, const TArray<uint8>& DefaultValueData) const
{
	UMovieSceneColorSection* ColorSection = Cast<UMovieSceneColorSection>(Section);
	if (ensureMsgf(ColorSection != nullptr, TEXT("Section must be a color section.")) && ensureMsgf(DefaultValueData.Num() == sizeof(FLinearColor), TEXT("DefaultValueData must be a FLinearColor.")))
	{
		FLinearColor DefaultValue;
		FMemory::Memcpy(&DefaultValue, DefaultValueData.GetData(), sizeof(FLinearColor));

		FMovieSceneChannelProxy& ColorChannelProxy = ColorSection->GetChannelProxy();
		SetChannelDefault(ColorChannelProxy, ColorSection->GetRedChannel(), DefaultValue.R);
		SetChannelDefault(ColorChannelProxy, ColorSection->GetGreenChannel(), DefaultValue.G);
		SetChannelDefault(ColorChannelProxy, ColorSection->GetBlueChannel(), DefaultValue.B);
		SetChannelDefault(ColorChannelProxy, ColorSection->GetAlphaChannel(), DefaultValue.A);
	}
}

FMovieSceneEvalTemplatePtr UMovieSceneNiagaraColorParameterTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	const UMovieSceneColorSection* ColorSection = Cast<UMovieSceneColorSection>(&InSection);
	if (ColorSection != nullptr)
	{
		return FMovieSceneNiagaraColorParameterSectionTemplate(GetParameter(), ColorSection->GetRedChannel(), ColorSection->GetGreenChannel(), ColorSection->GetBlueChannel(), ColorSection->GetAlphaChannel());
	}
	return FMovieSceneEvalTemplatePtr();
}
