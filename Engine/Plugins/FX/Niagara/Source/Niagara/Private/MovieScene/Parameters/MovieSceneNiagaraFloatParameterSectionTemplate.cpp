// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneNiagaraFloatParameterSectionTemplate.h"
#include "NiagaraTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraFloatParameterSectionTemplate)

FMovieSceneNiagaraFloatParameterSectionTemplate::FMovieSceneNiagaraFloatParameterSectionTemplate()
{
}

FMovieSceneNiagaraFloatParameterSectionTemplate::FMovieSceneNiagaraFloatParameterSectionTemplate(FNiagaraVariable InParameter, const FMovieSceneFloatChannel& InFloatChannel)
	: FMovieSceneNiagaraParameterSectionTemplate(InParameter)
	, FloatChannel(InFloatChannel)
{
}

void FMovieSceneNiagaraFloatParameterSectionTemplate::GetAnimatedParameterValue(FFrameTime InTime, const FNiagaraVariableBase& InTargetParameter, const TArray<uint8>& InCurrentValueData, TArray<uint8>& OutAnimatedValueData) const
{
	FNiagaraFloat AnimatedValue;
	FMemory::Memcpy(&AnimatedValue, InCurrentValueData.GetData(), sizeof(FNiagaraFloat));

	FloatChannel.Evaluate(InTime, AnimatedValue.Value);

	OutAnimatedValueData.AddUninitialized(sizeof(FNiagaraFloat));
	FMemory::Memcpy(OutAnimatedValueData.GetData(), (uint8*)&AnimatedValue, sizeof(FNiagaraFloat));
}
