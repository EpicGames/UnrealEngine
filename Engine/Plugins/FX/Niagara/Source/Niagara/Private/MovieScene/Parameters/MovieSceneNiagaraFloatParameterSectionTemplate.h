// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene/Parameters/MovieSceneNiagaraParameterSectionTemplate.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneNiagaraFloatParameterSectionTemplate.generated.h"

struct FNiagaraVariable;
struct FNiagaraVariableBase;

USTRUCT()
struct FMovieSceneNiagaraFloatParameterSectionTemplate : public FMovieSceneNiagaraParameterSectionTemplate
{
	GENERATED_BODY()

public:
	FMovieSceneNiagaraFloatParameterSectionTemplate();

	FMovieSceneNiagaraFloatParameterSectionTemplate(FNiagaraVariable InParameter, const FMovieSceneFloatChannel& InFloatChannel);

private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }

protected:
	virtual void GetAnimatedParameterValue(FFrameTime InTime, const FNiagaraVariableBase& InTargetParameter, const TArray<uint8>& InCurrentValueData, TArray<uint8>& OutAnimatedValueData) const override;

private:
	UPROPERTY()
	FMovieSceneFloatChannel FloatChannel;
};
