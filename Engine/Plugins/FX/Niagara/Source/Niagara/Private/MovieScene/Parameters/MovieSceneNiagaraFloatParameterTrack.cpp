// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/Parameters/MovieSceneNiagaraFloatParameterTrack.h"
#include "MovieScene/Parameters/MovieSceneNiagaraFloatParameterSectionTemplate.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraFloatParameterTrack)

bool UMovieSceneNiagaraFloatParameterTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneFloatSection::StaticClass();
}

UMovieSceneSection* UMovieSceneNiagaraFloatParameterTrack::CreateNewSection()
{
	return NewObject<UMovieSceneFloatSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneNiagaraFloatParameterTrack::SetSectionChannelDefaults(UMovieSceneSection* Section, const TArray<uint8>& DefaultValueData) const
{
	UMovieSceneFloatSection* FloatSection = Cast<UMovieSceneFloatSection>(Section);
	if (ensureMsgf(FloatSection != nullptr, TEXT("Section must be a float section.")) && ensureMsgf(DefaultValueData.Num() == sizeof(float), TEXT("DefaultValueData must be a float.")))
	{
		FMovieSceneChannelProxy& FloatChannelProxy = FloatSection->GetChannelProxy();
		float DefaultValue;
		FMemory::Memcpy(&DefaultValue, DefaultValueData.GetData(), sizeof(float));

		SetChannelDefault(FloatChannelProxy, FloatSection->GetChannel(), DefaultValue);
	}
}

FMovieSceneEvalTemplatePtr UMovieSceneNiagaraFloatParameterTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	const UMovieSceneFloatSection* FloatSection = Cast<UMovieSceneFloatSection>(&InSection);
	if (FloatSection != nullptr)
	{
		return FMovieSceneNiagaraFloatParameterSectionTemplate(GetParameter(), FloatSection->GetChannel());
	}
	return FMovieSceneEvalTemplatePtr();
}
