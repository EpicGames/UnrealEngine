// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneNiagaraIntegerParameterSectionTemplate.h"
#include "NiagaraTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraIntegerParameterSectionTemplate)

FMovieSceneNiagaraIntegerParameterSectionTemplate::FMovieSceneNiagaraIntegerParameterSectionTemplate()
{
}

FMovieSceneNiagaraIntegerParameterSectionTemplate::FMovieSceneNiagaraIntegerParameterSectionTemplate(FNiagaraVariable InParameter, const FMovieSceneIntegerChannel& InIntegerChannel)
	: FMovieSceneNiagaraParameterSectionTemplate(InParameter)
	, IntegerChannel(InIntegerChannel)
{
}

void FMovieSceneNiagaraIntegerParameterSectionTemplate::GetAnimatedParameterValue(FFrameTime InTime, const FNiagaraVariableBase& InTargetParameter, const TArray<uint8>& InCurrentValueData, TArray<uint8>& OutAnimatedValueData) const
{
	FNiagaraInt32 AnimatedValue;
	FMemory::Memcpy(&AnimatedValue, InCurrentValueData.GetData(), sizeof(FNiagaraInt32));

	IntegerChannel.Evaluate(InTime, AnimatedValue.Value);

	OutAnimatedValueData.AddUninitialized(sizeof(FNiagaraInt32));
	FMemory::Memcpy(OutAnimatedValueData.GetData(), (uint8*)&AnimatedValue, sizeof(FNiagaraInt32));
}
