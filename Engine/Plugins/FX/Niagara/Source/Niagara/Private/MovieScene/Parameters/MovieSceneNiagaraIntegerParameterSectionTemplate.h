// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene/Parameters/MovieSceneNiagaraParameterSectionTemplate.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "MovieSceneNiagaraIntegerParameterSectionTemplate.generated.h"

struct FNiagaraVariable;
struct FNiagaraVariableBase;

USTRUCT()
struct FMovieSceneNiagaraIntegerParameterSectionTemplate : public FMovieSceneNiagaraParameterSectionTemplate
{
	GENERATED_BODY()

public:
	FMovieSceneNiagaraIntegerParameterSectionTemplate();

	FMovieSceneNiagaraIntegerParameterSectionTemplate(FNiagaraVariable InParameter, const FMovieSceneIntegerChannel& InIntegerChannel);

private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }

protected:
	virtual void GetAnimatedParameterValue(FFrameTime InTime, const FNiagaraVariableBase& InTargetParameter, const TArray<uint8>& InCurrentValueData, TArray<uint8>& OutAnimatedValueData) const override;

private:
	UPROPERTY()
	FMovieSceneIntegerChannel IntegerChannel;
};
