// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/Parameters/MovieSceneNiagaraIntegerParameterTrack.h"
#include "MovieScene/Parameters/MovieSceneNiagaraIntegerParameterSectionTemplate.h"
#include "Sections/MovieSceneIntegerSection.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Channels/MovieSceneChannelProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraIntegerParameterTrack)

bool UMovieSceneNiagaraIntegerParameterTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneIntegerSection::StaticClass();
}

UMovieSceneSection* UMovieSceneNiagaraIntegerParameterTrack::CreateNewSection()
{
	return NewObject<UMovieSceneIntegerSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneNiagaraIntegerParameterTrack::SetSectionChannelDefaults(UMovieSceneSection* Section, const TArray<uint8>& DefaultValueData) const
{
	UMovieSceneIntegerSection* IntegerSection = Cast<UMovieSceneIntegerSection>(Section);
	if (ensureMsgf(IntegerSection != nullptr, TEXT("Section must be an integer section.")) && ensureMsgf(DefaultValueData.Num() == sizeof(int32), TEXT("DefaultValueData must be an int32.")))
	{
		FMovieSceneChannelProxy& IntegerChannelProxy = IntegerSection->GetChannelProxy();
		int32 DefaultValue;
		FMemory::Memcpy(&DefaultValue, DefaultValueData.GetData(), sizeof(int32));

		SetChannelDefault(IntegerChannelProxy, IntegerSection->GetChannel(), DefaultValue);
	}
}

FMovieSceneEvalTemplatePtr UMovieSceneNiagaraIntegerParameterTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	const UMovieSceneIntegerSection* IntegerSection = Cast<UMovieSceneIntegerSection>(&InSection);
	if (IntegerSection != nullptr)
	{
		return FMovieSceneNiagaraIntegerParameterSectionTemplate(GetParameter(), IntegerSection->GetChannel());
	}
	return FMovieSceneEvalTemplatePtr();
}
