// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneNiagaraParameterSectionTemplate.h"
#include "Evaluation/MovieSceneExecutionTokens.h"
#include "NiagaraComponent.h"
#include "NiagaraSystem.h"
#include "NiagaraTypes.h"
#include "IMovieScenePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraParameterSectionTemplate)

struct FComponentData
{
	TWeakObjectPtr<UNiagaraComponent> Component;
	FNiagaraVariableBase TargetParameter;
	TOptional<TArray<uint8>> CurrentValue;
};

struct FParameterSectionData : IPersistentEvaluationData
{
	TArray<FComponentData> CachedComponentData;
};

struct FPreAnimatedParameterValueToken : IMovieScenePreAnimatedToken
{
	FPreAnimatedParameterValueToken(FNiagaraVariable InParameter, TOptional<TArray<uint8>>&& InPreviousValueData)
		: Parameter(InParameter)
		, PreviousValueData(InPreviousValueData)
	{
	}

	virtual void RestoreState(UObject& InObject, const UE::MovieScene::FRestoreStateParams& Params)
	{
		UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(&InObject);
		if (PreviousValueData.IsSet() == false)
		{
			NiagaraComponent->GetOverrideParameters().RemoveParameter(Parameter);
		}
		else
		{
			NiagaraComponent->GetOverrideParameters().SetParameterData(PreviousValueData.GetValue().GetData(), Parameter);
		}
	}

	FNiagaraVariable Parameter;
	TOptional<TArray<uint8>> PreviousValueData;
};

struct FPreAnimatedParameterValueTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	FPreAnimatedParameterValueTokenProducer(FNiagaraVariable InParameter)
		: Parameter(InParameter)
	{
	}

	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const override
	{
		UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(&Object);
		const uint8* ParameterData = NiagaraComponent->GetOverrideParameters().GetParameterData(Parameter);
		TOptional<TArray<uint8>> PreviousValue;
		if (ParameterData != nullptr)
		{
			TArray<uint8> ParameterDataArray;
			ParameterDataArray.AddUninitialized(Parameter.GetSizeInBytes());
			FMemory::Memcpy(ParameterDataArray.GetData(), ParameterData, Parameter.GetSizeInBytes());
			PreviousValue.Emplace(MoveTemp(ParameterDataArray));
		}
		return FPreAnimatedParameterValueToken(Parameter, MoveTemp(PreviousValue));
	}

	FNiagaraVariable Parameter;
};

struct FSetParameterValueExecutionToken : IMovieSceneExecutionToken
{
	FSetParameterValueExecutionToken(TWeakObjectPtr<UNiagaraComponent> InComponentPtr, FNiagaraVariable InParameter, TArray<uint8>&& InData)
		: ComponentPtr(InComponentPtr)
		, Parameter(InParameter)
		, Data(InData)
	{
	}

	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		static TMovieSceneAnimTypeIDContainer<FNiagaraVariableBase> ParameterToTypeID;
		FMovieSceneAnimTypeID TypeID = ParameterToTypeID.GetAnimTypeID(Parameter);
		UNiagaraComponent* NiagaraComponent = ComponentPtr.Get();
		if (NiagaraComponent != nullptr)
		{
			Player.PreAnimatedState.SavePreAnimatedState(*NiagaraComponent, TypeID, FPreAnimatedParameterValueTokenProducer(Parameter));
			NiagaraComponent->GetOverrideParameters().AddParameter(Parameter, false);
			if (Parameter.GetType() == FNiagaraTypeDefinition::GetPositionDef())
			{
				FVector* PositionValue = reinterpret_cast<FVector*>(Data.GetData());
				NiagaraComponent->GetOverrideParameters().SetPositionParameterValue(*PositionValue, Parameter.GetName(), false);
			}
			else
			{
				NiagaraComponent->GetOverrideParameters().SetParameterData(Data.GetData(), Parameter);
			}
		}
	}

	TWeakObjectPtr<UNiagaraComponent> ComponentPtr;
	FNiagaraVariableBase Parameter;
	TArray<uint8> Data;
};

FMovieSceneNiagaraParameterSectionTemplate::FMovieSceneNiagaraParameterSectionTemplate()
{
}

FMovieSceneNiagaraParameterSectionTemplate::FMovieSceneNiagaraParameterSectionTemplate(FNiagaraVariable InParameter)
	: Parameter(InParameter)
{}

void FMovieSceneNiagaraParameterSectionTemplate::Initialize(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
{
	FParameterSectionData& SectionData = PersistentData.GetOrAddSectionData<FParameterSectionData>();
	SectionData.CachedComponentData.Empty();

	for (TWeakObjectPtr<> ObjectPtr : Player.FindBoundObjects(Operand))
	{
		UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(ObjectPtr.Get());
		if (NiagaraComponent != nullptr && NiagaraComponent->GetAsset() != nullptr)
		{
			FNiagaraVariableBase TargetParameter;
			const uint8* ParameterData = NiagaraComponent->GetOverrideParameters().GetParameterData(Parameter);
			if (ParameterData == nullptr)
			{
				ParameterData = NiagaraComponent->GetAsset()->GetExposedParameters().GetParameterData(Parameter);
			};

			if (ParameterData != nullptr)
			{
				TargetParameter = Parameter;
			}
			else
			{
				// If a matching parameter was not found in the component overrides or the assets 
				// exposed parameters check if there are alternate types which are supported for this section.
				for (const FNiagaraTypeDefinition& AlternateParameterType : GetAlternateParameterTypes())
				{
					FNiagaraVariable AlternateParameter(AlternateParameterType, Parameter.GetName());
					ParameterData = NiagaraComponent->GetOverrideParameters().GetParameterData(AlternateParameter);
					if (ParameterData == nullptr)
					{
						ParameterData = NiagaraComponent->GetAsset()->GetExposedParameters().GetParameterData(AlternateParameter);
					}

					if (ParameterData != nullptr)
					{
						TargetParameter = AlternateParameter;
						break;
					}
				}
			}

			if (TargetParameter.IsValid() && ParameterData != nullptr)
			{
				TArray<uint8> CurrentValueData;
				CurrentValueData.AddUninitialized(Parameter.GetSizeInBytes());
				FMemory::Memcpy(CurrentValueData.GetData(), ParameterData, Parameter.GetSizeInBytes());

				FComponentData Data;
				Data.Component = NiagaraComponent;
				Data.TargetParameter = TargetParameter;
				Data.CurrentValue.Emplace(CurrentValueData);
				SectionData.CachedComponentData.Add(Data);
			}
		}
	}
}

void FMovieSceneNiagaraParameterSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	FParameterSectionData const* SectionData = PersistentData.FindSectionData<FParameterSectionData>();
	if (SectionData != nullptr)
	{
		for (const FComponentData& ComponentData : SectionData->CachedComponentData)
		{
			if (ComponentData.CurrentValue.IsSet())
			{
				TArray<uint8> AnimatedValueData;
				GetAnimatedParameterValue(Context.GetTime(), ComponentData.TargetParameter, ComponentData.CurrentValue.GetValue(),  AnimatedValueData);
				ExecutionTokens.Add(FSetParameterValueExecutionToken(ComponentData.Component, ComponentData.TargetParameter, MoveTemp(AnimatedValueData)));
			}
		}
	}
}
