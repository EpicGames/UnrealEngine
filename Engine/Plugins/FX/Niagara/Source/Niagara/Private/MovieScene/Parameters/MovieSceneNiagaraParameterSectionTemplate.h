// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneEvalTemplate.h"
#include "NiagaraTypes.h"
#include "MovieSceneNiagaraParameterSectionTemplate.generated.h"

USTRUCT()
struct FMovieSceneNiagaraParameterSectionTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()

public:
	FMovieSceneNiagaraParameterSectionTemplate();

	FMovieSceneNiagaraParameterSectionTemplate(FNiagaraVariable InParameter);

	virtual void SetupOverrides() override
	{
		EnableOverrides(RequiresInitializeFlag);
	}

private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Initialize(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const override;
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

protected:
	virtual void GetAnimatedParameterValue(FFrameTime InTime, const FNiagaraVariableBase& InTargetParameter, const TArray<uint8>& InCurrentValueData, TArray<uint8>& OutAnimatedValueData) const { };
	
	// Specifies a list of alternate types which are supported by this parameter section which can be used if the original parameter isn't found.
	virtual TArrayView<FNiagaraTypeDefinition> GetAlternateParameterTypes() const { return TArrayView<FNiagaraTypeDefinition>(); }

private:
	UPROPERTY()
	FNiagaraVariable Parameter;
};