// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/Parameters/MovieSceneNiagaraParameterTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraParameterTrack)

const FNiagaraVariable& UMovieSceneNiagaraParameterTrack::GetParameter() const
{
	return Parameter;
}

void UMovieSceneNiagaraParameterTrack::SetParameter(FNiagaraVariable InParameter)
{
	Parameter = InParameter;
}
