// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneNiagaraVectorParameterSectionTemplate.h"
#include "NiagaraTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraVectorParameterSectionTemplate)

FMovieSceneNiagaraVectorParameterSectionTemplate::FMovieSceneNiagaraVectorParameterSectionTemplate() : ChannelsUsed(0)
{
}

FMovieSceneNiagaraVectorParameterSectionTemplate::FMovieSceneNiagaraVectorParameterSectionTemplate(FNiagaraVariable InParameter, TArray<FMovieSceneFloatChannel>&& InVectorChannels, int32 InChannelsUsed)
	: FMovieSceneNiagaraParameterSectionTemplate(InParameter)
	, ChannelsUsed(InChannelsUsed)
{
	for (int32 i = 0; i < ChannelsUsed; i++)
	{
		VectorChannels[i] = InVectorChannels[i];
	}
}

void FMovieSceneNiagaraVectorParameterSectionTemplate::GetAnimatedParameterValue(FFrameTime InTime, const FNiagaraVariableBase& InTargetParameter, const TArray<uint8>& InCurrentValueData, TArray<uint8>& OutAnimatedValueData) const
{
	if (ChannelsUsed == 2)
	{
		FVector2f AnimatedValue;
		FMemory::Memcpy(&AnimatedValue, InCurrentValueData.GetData(), sizeof(FVector2f));

		VectorChannels[0].Evaluate(InTime, AnimatedValue.X);
		VectorChannels[1].Evaluate(InTime, AnimatedValue.Y);

		OutAnimatedValueData.AddUninitialized(sizeof(FVector2f));
		FMemory::Memcpy(OutAnimatedValueData.GetData(), (uint8*)&AnimatedValue, sizeof(FVector2f));
	}
	else if (ChannelsUsed == 3)
	{
		FVector3f AnimatedValue;
		FMemory::Memcpy(&AnimatedValue, InCurrentValueData.GetData(), sizeof(FVector3f));

		VectorChannels[0].Evaluate(InTime, AnimatedValue.X);
		VectorChannels[1].Evaluate(InTime, AnimatedValue.Y);
		VectorChannels[2].Evaluate(InTime, AnimatedValue.Z);

		if (InTargetParameter.GetType() != FNiagaraTypeDefinition::GetPositionDef())
		{
			OutAnimatedValueData.AddUninitialized(sizeof(FVector3f));
			FMemory::Memcpy(OutAnimatedValueData.GetData(), (uint8*)&AnimatedValue, sizeof(FVector3f));
		}
		else
		{
			OutAnimatedValueData.AddUninitialized(sizeof(FVector));
			FVector* PositionVector = (FVector*)OutAnimatedValueData.GetData();
			PositionVector->X = AnimatedValue.X;
			PositionVector->Y = AnimatedValue.Y;
			PositionVector->Z = AnimatedValue.Z;
		}
	}
	else if (ChannelsUsed == 4)
	{
		FVector4f AnimatedValue;
		FMemory::Memcpy(&AnimatedValue, InCurrentValueData.GetData(), sizeof(FVector4f));

		VectorChannels[0].Evaluate(InTime, AnimatedValue.X);
		VectorChannels[1].Evaluate(InTime, AnimatedValue.Y);
		VectorChannels[2].Evaluate(InTime, AnimatedValue.Z);
		VectorChannels[3].Evaluate(InTime, AnimatedValue.W);

		OutAnimatedValueData.AddUninitialized(sizeof(FVector4f));
		FMemory::Memcpy(OutAnimatedValueData.GetData(), (uint8*)&AnimatedValue, sizeof(FVector4f));
	}
}

TArrayView<FNiagaraTypeDefinition> FMovieSceneNiagaraVectorParameterSectionTemplate::GetAlternateParameterTypes() const
{
	if (ChannelsUsed == 3)
	{
		static TArray<FNiagaraTypeDefinition> AlternateParameterTypes = { FNiagaraTypeDefinition::GetPositionDef() };
		return TArrayView<FNiagaraTypeDefinition>(AlternateParameterTypes);
	}
	else
	{
		return TArrayView<FNiagaraTypeDefinition>();
	}
}
