// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene/Parameters/MovieSceneNiagaraParameterSectionTemplate.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneNiagaraVectorParameterSectionTemplate.generated.h"

struct FNiagaraVariable;
struct FNiagaraVariableBase;

USTRUCT()
struct FMovieSceneNiagaraVectorParameterSectionTemplate : public FMovieSceneNiagaraParameterSectionTemplate
{
	GENERATED_BODY()

public:
	FMovieSceneNiagaraVectorParameterSectionTemplate();

	FMovieSceneNiagaraVectorParameterSectionTemplate(FNiagaraVariable InParameter, TArray<FMovieSceneFloatChannel>&& InVectorChannels, int32 InChannelsUsed);

private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }

protected:
	virtual void GetAnimatedParameterValue(FFrameTime InTime, const FNiagaraVariableBase& InTargetParameter, const TArray<uint8>& InCurrentValueData, TArray<uint8>& OutAnimatedValueData) const override;
	virtual TArrayView<FNiagaraTypeDefinition> GetAlternateParameterTypes() const override;

private:
	UPROPERTY()
	FMovieSceneFloatChannel VectorChannels[4];

	UPROPERTY()
	int32 ChannelsUsed;
};
