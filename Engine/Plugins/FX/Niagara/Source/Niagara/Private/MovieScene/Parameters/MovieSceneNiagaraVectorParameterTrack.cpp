// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/Parameters/MovieSceneNiagaraVectorParameterTrack.h"
#include "MovieScene/Parameters/MovieSceneNiagaraVectorParameterSectionTemplate.h"
#include "Sections/MovieSceneVectorSection.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Channels/MovieSceneChannelProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraVectorParameterTrack)

bool UMovieSceneNiagaraVectorParameterTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneFloatVectorSection::StaticClass();
}

UMovieSceneSection* UMovieSceneNiagaraVectorParameterTrack::CreateNewSection()
{
	UMovieSceneFloatVectorSection* VectorSection = NewObject<UMovieSceneFloatVectorSection>(this, NAME_None, RF_Transactional);
	VectorSection->SetChannelsUsed(ChannelsUsed);
	return VectorSection;
}

void UMovieSceneNiagaraVectorParameterTrack::SetSectionChannelDefaults(UMovieSceneSection* Section, const TArray<uint8>& DefaultValueData) const
{
	UMovieSceneFloatVectorSection* VectorSection = Cast<UMovieSceneFloatVectorSection>(Section);
	if (ensureMsgf(VectorSection != nullptr, TEXT("Section must be a color section.")) && ensureMsgf(DefaultValueData.Num() == sizeof(float) * VectorSection->GetChannelsUsed(), TEXT("DefaultValueData must be the correct vector type.")))
	{
		FMovieSceneChannelProxy& VectorChannelProxy = VectorSection->GetChannelProxy();

		if (VectorSection->GetChannelsUsed() == 2 && ensureMsgf(DefaultValueData.Num() == sizeof(FVector2f), TEXT("DefaultValueData must be a FVector2f when channels used is 2")))
		{
			FVector2f DefaultValue;
			FMemory::Memcpy(&DefaultValue, DefaultValueData.GetData(), sizeof(FVector2f));

			SetChannelDefault(VectorChannelProxy, VectorSection->GetChannel(0), DefaultValue.X);
			SetChannelDefault(VectorChannelProxy, VectorSection->GetChannel(1), DefaultValue.Y);
		}
		else if (VectorSection->GetChannelsUsed() == 3 && ensureMsgf(DefaultValueData.Num() == sizeof(FVector3f), TEXT("DefaultValueData must be a FVector3f when channels used is 3")))
		{
			FVector3f DefaultValue;
			FMemory::Memcpy(&DefaultValue, DefaultValueData.GetData(), sizeof(FVector3f));

			SetChannelDefault(VectorChannelProxy, VectorSection->GetChannel(0), DefaultValue.X);
			SetChannelDefault(VectorChannelProxy, VectorSection->GetChannel(1), DefaultValue.Y);
			SetChannelDefault(VectorChannelProxy, VectorSection->GetChannel(2), DefaultValue.Z);
		}
		else if (VectorSection->GetChannelsUsed() == 4 && ensureMsgf(DefaultValueData.Num() == sizeof(FVector4f), TEXT("DefaultValueData must be a FVector4f when channels used is 4")))
		{
			FVector4f DefaultValue;
			FMemory::Memcpy(&DefaultValue, DefaultValueData.GetData(), sizeof(FVector4f));

			SetChannelDefault(VectorChannelProxy, VectorSection->GetChannel(0), DefaultValue.X);
			SetChannelDefault(VectorChannelProxy, VectorSection->GetChannel(1), DefaultValue.Y);
			SetChannelDefault(VectorChannelProxy, VectorSection->GetChannel(2), DefaultValue.Z);
			SetChannelDefault(VectorChannelProxy, VectorSection->GetChannel(3), DefaultValue.W);
		}
	}
}

FMovieSceneEvalTemplatePtr UMovieSceneNiagaraVectorParameterTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	const UMovieSceneFloatVectorSection* VectorSection = Cast<UMovieSceneFloatVectorSection>(&InSection);
	if (VectorSection != nullptr)
	{
		TArray<FMovieSceneFloatChannel> ComponentChannels;
		for (int32 i = 0; i < VectorSection->GetChannelsUsed(); i++)
		{
			ComponentChannels.Add(VectorSection->GetChannel(i));
		}
		return FMovieSceneNiagaraVectorParameterSectionTemplate(GetParameter(), MoveTemp(ComponentChannels), VectorSection->GetChannelsUsed());
	}
	return FMovieSceneEvalTemplatePtr();
}

int32 UMovieSceneNiagaraVectorParameterTrack::GetChannelsUsed() const
{
	return ChannelsUsed;
}

void UMovieSceneNiagaraVectorParameterTrack::SetChannelsUsed(int32 InChannelsUsed)
{
	ChannelsUsed = InChannelsUsed;
}

