// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "SkeletalMeshTypes.h"
#include "NiagaraStats.h"
#include "NDISkeletalMeshCommon.h"
#include "Templates/IntegralConstant.h"

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceSkeletalMesh_BoneSampling"

DECLARE_CYCLE_STAT(TEXT("Skel Mesh Skeleton Sampling"), STAT_NiagaraSkel_Bone_Sample, STATGROUP_Niagara);

//Final binders for all static mesh interface functions.
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetSkinnedBoneData)
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, IsValidBone)

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredSocketBoneAt)
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredSocketTransform)

const FName FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataName("GetSkinnedBoneData");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSName("GetSkinnedBoneDataWS");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataInterpolatedName("GetSkinnedBoneDataInterpolated");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSInterpolatedName("GetSkinnedBoneDataWSInterpolated");

const FName FSkeletalMeshInterfaceHelper::IsValidBoneName("IsValidBone");
const FName FSkeletalMeshInterfaceHelper::RandomBoneName("RandomBone");
const FName FSkeletalMeshInterfaceHelper::GetBoneCountName("GetBoneCount");
const FName FSkeletalMeshInterfaceHelper::GetParentBoneName("GetParentBone");

const FName FSkeletalMeshInterfaceHelper::RandomFilteredBoneName("RandomFilteredBone");
const FName FSkeletalMeshInterfaceHelper::GetFilteredBoneCountName("GetFilteredBoneCount");
const FName FSkeletalMeshInterfaceHelper::GetFilteredBoneAtName("GetFilteredBone");

const FName FSkeletalMeshInterfaceHelper::RandomUnfilteredBoneName("RandomUnfilteredBone");
const FName FSkeletalMeshInterfaceHelper::GetUnfilteredBoneCountName("GetUnfilteredBoneCount");
const FName FSkeletalMeshInterfaceHelper::GetUnfilteredBoneAtName("GetUnfilteredBone");

const FName FSkeletalMeshInterfaceHelper::RandomFilteredSocketName("RandomFilteredSocket");
const FName FSkeletalMeshInterfaceHelper::GetFilteredSocketCountName("GetFilteredSocketCount");
const FName FSkeletalMeshInterfaceHelper::GetFilteredSocketTransformName("GetFilteredSocketTransform");
const FName FSkeletalMeshInterfaceHelper::GetFilteredSocketBoneAtName("GetFilteredSocket");

const FName FSkeletalMeshInterfaceHelper::RandomFilteredSocketOrBoneName("RandomFilteredSocketOrBone");
const FName FSkeletalMeshInterfaceHelper::GetFilteredSocketOrBoneCountName("GetFilteredSocketOrBoneCount");
const FName FSkeletalMeshInterfaceHelper::GetFilteredSocketOrBoneAtName("GetFilteredSocketOrBone");

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSkeletalMesh::GetSkeletonSamplingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature DefaultSig;
	DefaultSig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
	DefaultSig.bMemberFunction = true;
	DefaultSig.bRequiresContext = false;

	//////////////////////////////////////////////////////////////////////////
	// Bone functions.
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.SetDescription(LOCTEXT("GetOptionalSkinnedBoneDataDesc", "Returns skinning dependant data for the bone in local space. All outputs are optional and you will incur zero to minimal cost if they are not connected."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.SetDescription(LOCTEXT("GetOptionalSkinnedBoneDataWSDesc", "Returns skinning dependant data for the bone in world space. All outputs are optional and you will incur zero to minimal cost if they are not connected."));
	}


	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataInterpolatedName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interpolation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.SetDescription(LOCTEXT("GetSkinnedBoneDataDesc", "Returns skinning dependant data for the bone in local space. Interpolated between this frame and the previous based on passed interpolation factor. All outputs are optional and you will incur zero to minimal cost if they are not connected."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSInterpolatedName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interpolation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.SetDescription(LOCTEXT("GetSkinnedBoneDataWSDesc", "Returns skinning dependant data for the bone in world space. Interpolated between this frame and the previous based on passed interpolation factor. All outputs are optional and you will incur zero to minimal cost if they are not connected."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::IsValidBoneName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
		Sig.SetDescription(LOCTEXT("IsValidBoneDesc", "Determine if this bone index is valid for this mesh's skeleton."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::RandomBoneName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetBoneCountName;
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));
		Sig.SetDescription(LOCTEXT("GetBoneCountDesc", "Returns the number of bones in the skeletal mesh."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetParentBoneName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("BoneIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParentIndex")));
		Sig.SetDescription(LOCTEXT("GetParentBoneDesc", "Returns the parent bone index for the given bone, -1 if no parent exists."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::RandomFilteredBoneName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetFilteredBoneCountName;
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));
		Sig.SetDescription(LOCTEXT("GetFilteredBoneCountDesc", "Returns the number of filtered bones in the DI list."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetFilteredBoneAtName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
		Sig.SetDescription(LOCTEXT("GetFilteredBoneAtDesc", "Gets the bone at the passed index in the DI's filter bones list."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::RandomUnfilteredBoneName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetUnfilteredBoneCountName;
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));
		Sig.SetDescription(LOCTEXT("GetUnfilteredBoneCountDesc", "Returns the number of unfiltered bones (i.e. the exclusion of filtered bones) in the DI list."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetUnfilteredBoneAtName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
		Sig.SetDescription(LOCTEXT("GetUnfilteredBoneAtDesc", "Gets the bone at the passed index from the exlusion of the DI's filter bones list."));
	}

	//////////////////////////////////////////////////////////////////////////
	//Socket functions
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::RandomFilteredSocketName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Socket Bone")));
		Sig.SetDescription(LOCTEXT("RandomFilteredSocketDesc", "Gets the bone for a random socket in the DI's filtered socket list."));
	}
	
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetFilteredSocketCountName;
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));
		Sig.SetDescription(LOCTEXT("GetFilteredSocketCountDesc", "Returns the number of filtered Sockets in the DI list."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetFilteredSocketBoneAtName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Socket Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Socket Bone")));
		Sig.SetDescription(LOCTEXT("GetFilteredSocketBoneAtDesc", "Gets the bone for the socket at the passed index in the DI's filtered socket list."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetFilteredSocketTransformName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Socket Index")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Apply World Transform")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Socket Translation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Socket Rotation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Socket Scale")));
		Sig.SetDescription(LOCTEXT("GetFilteredSocketTransformDesc", "Gets the transform for the socket at the passed index in the DI's filtered socket list. If the Source component is set it will respect the Relative Transform Space as well.."));
	}

	//////////////////////////////////////////////////////////////////////////
	// Misc Functions
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::RandomFilteredSocketOrBoneName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
		Sig.SetDescription(LOCTEXT("RandomFilteredSocketOrBoneDesc", "Gets the bone for a random filtered socket or bone from the DI's list."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetFilteredSocketOrBoneCountName;
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));
		Sig.SetDescription(LOCTEXT("GetFilteredSocketOrBoneCountDesc", "Gets the total filtered socket and bone count from the DI's list."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = FSkeletalMeshInterfaceHelper::GetFilteredSocketOrBoneAtName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Socket Or Bone Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Bone")));
		Sig.SetDescription(LOCTEXT("GetFilteredSocketOrBoneAtDesc", "Gets a filtered socket or bone count from the DI's list."));
	}
}
#endif

void UNiagaraDataInterfaceSkeletalMesh::BindSkeletonSamplingFunction(const FVMExternalFunctionBindingInfo& BindingInfo, FNDISkeletalMesh_InstanceData* InstanceData, FVMExternalFunction &OutFunc)
{
	using TInterpOff = TIntegralConstant<bool, false>;
	using TInterpOn = TIntegralConstant<bool, true>;

	//Bone Functions
	if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataName)
	{
		ensure(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 13);
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TNDIExplicitBinder<TInterpOff, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetSkinnedBoneData)>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSName)
	{
		ensure(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 13);
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandler, TNDIExplicitBinder<TInterpOff, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetSkinnedBoneData)>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataInterpolatedName)
	{
		ensure(BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 13);
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TNDIExplicitBinder<TInterpOn, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetSkinnedBoneData)>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSInterpolatedName)
	{
		ensure(BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 13);
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandler, TNDIExplicitBinder<TInterpOn, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetSkinnedBoneData)>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::IsValidBoneName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, IsValidBone)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::RandomBoneName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::RandomBone);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetBoneCountName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetBoneCount);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetParentBoneName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetParentBone);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::RandomFilteredBoneName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::RandomFilteredBone);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetFilteredBoneCountName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetFilteredBoneCount);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetFilteredBoneAtName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetFilteredBoneAt);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::RandomUnfilteredBoneName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::RandomUnfilteredBone);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetUnfilteredBoneCountName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetUnfilteredBoneCount);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetUnfilteredBoneAtName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetUnfilteredBoneAt);
	}
	//Socket Functions
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::RandomFilteredSocketName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::RandomFilteredSocket);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetFilteredSocketCountName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetFilteredSocketCount);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetFilteredSocketBoneAtName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredSocketBoneAt)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetFilteredSocketTransformName)
	{
		check(BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 10);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredSocketTransform)::Bind(this, OutFunc);
	}
	// Misc functions
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::RandomFilteredSocketOrBoneName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::RandomFilteredSocketOrBone);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetFilteredSocketOrBoneCountName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetFilteredSocketOrBoneCount);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetFilteredSocketOrBoneAtName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetFilteredSocketOrBoneBoneAt);
	}
}

//////////////////////////////////////////////////////////////////////////
// Direct sampling from listed sockets and bones.

void UNiagaraDataInterfaceSkeletalMesh::GetFilteredBoneCount(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Bone_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);

	FNDIOutputParam<int32> OutCount(Context);

	const int32 Num = InstData->NumFilteredBones;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCount.SetAndAdvance(Num);
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetFilteredBoneAt(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> BoneParam(Context);

	FNDIOutputParam<int32> OutBone(Context);

	const int32 Max = InstData->NumFilteredBones - 1;
	if (Max >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 BoneIndex = FMath::Clamp(BoneParam.GetAndAdvance(), 0, Max);
			OutBone.SetAndAdvance(InstData->FilteredAndUnfilteredBones[BoneIndex]);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutBone.SetAndAdvance(-1);
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::RandomFilteredBone(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIRandomHelper RandHelper(Context);

	FNDIOutputParam<int32> OutBone(Context);

	const int32 Max = InstData->NumFilteredBones - 1;
	if (Max >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			const int32 BoneIndex = RandHelper.RandRange(i, 0, Max);
			OutBone.SetAndAdvance(InstData->FilteredAndUnfilteredBones[BoneIndex]);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutBone.SetAndAdvance(-1);
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetUnfilteredBoneCount(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Bone_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);

	FNDIOutputParam<int32> OutCount(Context);

	const int32 Num = InstData->NumUnfilteredBones;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCount.SetAndAdvance(Num);
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetUnfilteredBoneAt(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> BoneParam(Context);

	FNDIOutputParam<int32> OutBone(Context);

	const int32 Max = InstData->NumUnfilteredBones - 1;
	if (Max >= 0)
	{
		if (InstData->NumFilteredBones == 0)
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				const int32 BoneIndex = FMath::Clamp(BoneParam.GetAndAdvance(), 0, Max);
				OutBone.SetAndAdvance(BoneIndex);
			}
		}
		else
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				const int32 BoneIndex = FMath::Clamp(BoneParam.GetAndAdvance(), 0, Max);
				OutBone.SetAndAdvance(InstData->FilteredAndUnfilteredBones[BoneIndex + InstData->NumFilteredBones]);
			}
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutBone.SetAndAdvance(-1);
		}
	}
}
void UNiagaraDataInterfaceSkeletalMesh::RandomUnfilteredBone(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIRandomHelper RandHelper(Context);

	FNDIOutputParam<int32> OutBone(Context);

	const int32 UnfilteredMax = InstData->NumUnfilteredBones - 1;
	if (UnfilteredMax >= 0)
	{
		if (InstData->NumFilteredBones == 0)
		{
			const int32 ExcludedBoneIndex = InstData->ExcludedBoneIndex;
			const int32 NumBones = InstData->NumUnfilteredBones - (ExcludedBoneIndex >= 0 ? 2 : 1);
			if (NumBones >= 0)
			{
				for (int32 i = 0; i < Context.GetNumInstances(); ++i)
				{
					RandHelper.GetAndAdvance();
					const int32 BoneIndex = RandHelper.RandRange(i, 0, NumBones);
					OutBone.SetAndAdvance(BoneIndex != ExcludedBoneIndex ? BoneIndex : BoneIndex + 1);
				}
			}
			else
			{
				for (int32 i = 0; i < Context.GetNumInstances(); ++i)
				{
					OutBone.SetAndAdvance(-1);
				}
			}
		}
		else
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				RandHelper.GetAndAdvance();
				const int32 BoneIndex = RandHelper.RandRange(i, 0, UnfilteredMax);
				OutBone.SetAndAdvance(InstData->FilteredAndUnfilteredBones[BoneIndex + InstData->NumFilteredBones]);
			}
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutBone.SetAndAdvance(-1);
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::IsValidBone(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> BoneParam(Context);

	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());

	FNDIOutputParam<FNiagaraBool> OutValid(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstData);	

	if (MeshAccessor.AreBonesAccessible())
	{
		const FReferenceSkeleton& RefSkeleton = MeshAccessor.Mesh->GetRefSkeleton();
		int32 NumBones = RefSkeleton.GetNum();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			int32 RequestedIndex = BoneParam.GetAndAdvance();
			OutValid.SetAndAdvance(RequestedIndex >= 0 && RequestedIndex < NumBones);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutValid.SetAndAdvance(false);
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::RandomBone(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIRandomHelper RandHelper(Context);
	FNDIOutputParam<int32> OutBone(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<TIntegralConstant<int32, 0>, TIntegralConstant<int32, 0>>(InstData);

	const int32 ExcludedBoneIndex = InstData->ExcludedBoneIndex;
	int32 NumBones = 0;
	if (MeshAccessor.AreBonesAccessible())
	{
		const FReferenceSkeleton& RefSkeleton = MeshAccessor.Mesh->GetRefSkeleton();
		NumBones = RefSkeleton.GetNum() - (ExcludedBoneIndex >= 0 ? 2 : 1);
	}

	if (NumBones >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			const int32 BoneIndex = RandHelper.RandRange(i, 0, NumBones);
			OutBone.SetAndAdvance(BoneIndex != ExcludedBoneIndex ? BoneIndex : BoneIndex + 1);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutBone.SetAndAdvance(-1);
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetBoneCount(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Bone_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIOutputParam<int32> OutCount(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<TIntegralConstant<int32, 0>, TIntegralConstant<int32, 0>>(InstData);
	
	int32 NumBones = 0;
	if (MeshAccessor.AreBonesAccessible())
	{
		const FReferenceSkeleton& RefSkeleton = MeshAccessor.Mesh->GetRefSkeleton();
		NumBones = RefSkeleton.GetNum();
	}

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCount.SetAndAdvance(NumBones);
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetParentBone(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Bone_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> InBoneIndex(Context);
	FNDIOutputParam<int32> OutParentIndex(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<TIntegralConstant<int32, 0>, TIntegralConstant<int32, 0>>(InstData);

	if (MeshAccessor.AreBonesAccessible())
	{
		const FReferenceSkeleton& RefSkeleton = MeshAccessor.Mesh->GetRefSkeleton();
		const int32 NumBones = RefSkeleton.GetNum();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 BoneIndex = InBoneIndex.GetAndAdvance();
			const int32 ParentIndex = BoneIndex >= 0 && BoneIndex < NumBones ? RefSkeleton.GetParentIndex(BoneIndex) : INDEX_NONE;
			OutParentIndex.SetAndAdvance(ParentIndex);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutParentIndex.SetAndAdvance(INDEX_NONE);
		}
	}
}

struct FBoneSocketSkinnedDataOutputHandler
{
	FBoneSocketSkinnedDataOutputHandler(FVectorVMExternalFunctionContext& Context)
		: Position(Context)
		, Rotation(Context)
		, Scale(Context)
		, Velocity(Context)
		, bNeedsPosition(Position.IsValid())
		, bNeedsRotation(Rotation.IsValid())
		, bNeedsScale(Scale.IsValid())
		, bNeedsVelocity(Velocity.IsValid())
	{
	}

	FNDIOutputParam<FVector3f> Position;
	FNDIOutputParam<FQuat4f> Rotation;
	FNDIOutputParam<FVector3f> Scale;
	FNDIOutputParam<FVector3f> Velocity;

	//TODO: Rotation + Scale too? Use quats so we can get proper interpolation between bone and parent.

	const bool bNeedsPosition;
	const bool bNeedsRotation;
	const bool bNeedsScale;
	const bool bNeedsVelocity;
};

template<typename SkinningHandlerType, typename TransformHandlerType, typename bInterpolated>
void UNiagaraDataInterfaceSkeletalMesh::GetSkinnedBoneData(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);

	SkinningHandlerType SkinningHandler;
	TransformHandlerType TransformHandler;
	FNDIInputParam<int32> BoneParam(Context);
	VectorVM::FExternalFuncInputHandler<float> InterpParam;

	if (bInterpolated::Value)
	{
		InterpParam.Init(Context);
	}

	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());

	FBoneSocketSkinnedDataOutputHandler Output(Context);

	//TODO: Replace this by storing off FTransforms and doing a proper lerp to get a final transform.
	//Also need to pull in a per particle interpolation factor.
	const FMatrix44f InstanceTransform(InstData->Transform);		// LWC_TODO: Precision loss
	const FMatrix44f PrevInstanceTransform(InstData->PrevTransform);
	const FQuat4f InstanceRotation = Output.bNeedsRotation ? FQuat4f(InstanceTransform.GetMatrixWithoutScale().ToQuat()) : FQuat4f::Identity;
	const FQuat4f PrevInstanceRotation = Output.bNeedsRotation ? FQuat4f(PrevInstanceTransform.GetMatrixWithoutScale().ToQuat()) : FQuat4f::Identity;

	FSkeletalMeshAccessorHelper Accessor;
	Accessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstData);
	if (Accessor.AreBonesAccessible())
	{
		const int32 BoneCount = SkinningHandler.GetBoneCount(Accessor, bInterpolated::Value);
		const int32 BoneAndSocketCount = BoneCount + InstData->FilteredSocketInfo.Num();
		float InvDt = 1.0f / InstData->DeltaSeconds;

		const TArray<FTransform3f>& FilteredSocketCurrTransforms = InstData->GetFilteredSocketsCurrBuffer();
		const TArray<FTransform3f>& FilteredSocketPrevTransforms = InstData->GetFilteredSocketsPrevBuffer();

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const float Interp = bInterpolated::Value ? InterpParam.GetAndAdvance() : 1.0f;

			// Determine bone or socket
			const int32 Bone = BoneParam.GetAndAdvance();
			const bool bIsSocket = Bone >= BoneCount;
			const int32 Socket = Bone - BoneCount;

			FVector3f Pos;
			FVector3f Prev;

			// Handle invalid bone indices first
			if (Bone < 0 || Bone >= BoneAndSocketCount)
			{
				Pos = FVector3f::ZeroVector;
				TransformHandler.TransformPosition(Pos, InstanceTransform);

				if (Output.bNeedsVelocity || bInterpolated::Value)
				{
					Prev = FVector3f::ZeroVector;
					TransformHandler.TransformPosition(Prev, PrevInstanceTransform);
				}
				if (Output.bNeedsRotation)
				{
					Output.Rotation.SetAndAdvance(FQuat4f::Identity);
				}
				if (Output.bNeedsScale)
				{
					Output.Scale.SetAndAdvance(FVector3f::ZeroVector);
				}
			}
			else if (bIsSocket)
			{
				FTransform3f CurrSocketTransform = FilteredSocketCurrTransforms[Socket];
				FTransform3f PrevSocketTransform = FilteredSocketPrevTransforms[Socket];

				Pos = CurrSocketTransform.GetLocation();
				TransformHandler.TransformPosition(Pos, InstanceTransform);

				if (Output.bNeedsVelocity || bInterpolated::Value)
				{
					Prev = PrevSocketTransform.GetLocation();
					TransformHandler.TransformPosition(Prev, PrevInstanceTransform);
				}

				if (Output.bNeedsRotation)
				{
					FQuat4f Rotation = CurrSocketTransform.GetRotation();
					TransformHandler.TransformRotation(Rotation, InstanceRotation);
					if (bInterpolated::Value)
					{
						FQuat4f PrevRotation = PrevSocketTransform.GetRotation();
						TransformHandler.TransformRotation(PrevRotation, PrevInstanceRotation);
						Rotation = FQuat4f::Slerp(PrevRotation, Rotation, Interp);
					}

					Output.Rotation.SetAndAdvance(Rotation);
				}
				if (Output.bNeedsScale)
				{
					FVector3f Scale = CurrSocketTransform.GetScale3D();
					TransformHandler.TransformNotUnitVector(Scale, InstanceTransform);
					if (bInterpolated::Value)
					{
						FVector3f PrevScale = PrevSocketTransform.GetScale3D();
						TransformHandler.TransformNotUnitVector(PrevScale, PrevInstanceTransform);
						Scale = FMath::Lerp(PrevScale, Scale, Interp);
					}
					Output.Scale.SetAndAdvance(Scale);
				}
			}
			// Bone
			else
			{
				Pos = SkinningHandler.GetSkinnedBonePosition(Accessor, Bone);
				TransformHandler.TransformPosition(Pos, InstanceTransform);

				if (Output.bNeedsVelocity || bInterpolated::Value)
				{
					Prev = SkinningHandler.GetSkinnedBonePreviousPosition(Accessor, Bone);
					TransformHandler.TransformPosition(Prev, PrevInstanceTransform);
				}

				if (Output.bNeedsRotation)
				{
					FQuat4f Rotation = SkinningHandler.GetSkinnedBoneRotation(Accessor, Bone);
					TransformHandler.TransformRotation(Rotation, InstanceRotation);
					if (bInterpolated::Value)
					{
						FQuat4f PrevRotation = SkinningHandler.GetSkinnedBonePreviousRotation(Accessor, Bone);
						TransformHandler.TransformRotation(PrevRotation, PrevInstanceRotation);
						Rotation = FQuat4f::Slerp(PrevRotation, Rotation, Interp);
					}

					Output.Rotation.SetAndAdvance(Rotation);
				}

				if (Output.bNeedsScale)
				{
					FVector3f Scale = SkinningHandler.GetSkinnedBoneScale(Accessor, Bone);
					TransformHandler.TransformNotUnitVector(Scale, InstanceTransform);
					if (bInterpolated::Value)
					{
						FVector3f PrevScale = SkinningHandler.GetSkinnedBonePreviousScale(Accessor, Bone);
						TransformHandler.TransformNotUnitVector(PrevScale, PrevInstanceTransform);
						Scale = FMath::Lerp(PrevScale, Scale, Interp);
					}
					Output.Scale.SetAndAdvance(Scale);
				}
			}

			if (Output.bNeedsVelocity || bInterpolated::Value)
			{
				Pos = FMath::Lerp(Prev, Pos, Interp);
			}

			if (Output.bNeedsPosition)
			{
				Output.Position.SetAndAdvance(Pos);
			}

			if(Output.bNeedsVelocity)
			{
				//Don't have enough information to get a better interpolated velocity.
				FVector3f Velocity = (Pos - Prev) * InvDt;
				Output.Velocity.SetAndAdvance(Velocity);
			}
		}
	}
	else
	{
		const float InvDt = 1.0f / InstData->DeltaSeconds;

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const float Interp = bInterpolated::Value ? InterpParam.GetAndAdvance() : 1.0f;

			FVector3f Prev = FVector3f::ZeroVector;
			FVector3f Pos = FVector3f::ZeroVector;
			TransformHandler.TransformPosition(Pos, InstanceTransform);

			if (Output.bNeedsVelocity || bInterpolated::Value)
			{
				TransformHandler.TransformPosition(Prev, PrevInstanceTransform);
			}

			if (Output.bNeedsRotation)
			{
				Output.Rotation.SetAndAdvance(FQuat4f::Identity);
			}

			if (Output.bNeedsVelocity || bInterpolated::Value)
			{
				Pos = FMath::Lerp(Prev, Pos, Interp);
			}

			if (Output.bNeedsPosition)
			{
				Output.Position.SetAndAdvance(Pos);
			}

			if (Output.bNeedsVelocity)
			{
				FVector3f Velocity = (Pos - Prev) * InvDt;
				Output.Velocity.SetAndAdvance(Velocity);
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// Sockets

void UNiagaraDataInterfaceSkeletalMesh::GetFilteredSocketCount(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);

	FNDIOutputParam<int32> OutCount(Context);

	const int32 Num = InstData->FilteredSocketInfo.Num();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCount.SetAndAdvance(Num);
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetFilteredSocketBoneAt(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> SocketParam(Context);

	FNDIOutputParam<int32> OutSocketBone(Context);
	const int32 FilteredSocketBoneOffset = InstData->FilteredSocketBoneOffset;
	const int32 Max = FilteredSockets.Num() - 1;

	if (Max != INDEX_NONE)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 SocketIndex = FMath::Clamp(SocketParam.GetAndAdvance(), 0, Max);
			OutSocketBone.SetAndAdvance(FilteredSocketBoneOffset + SocketIndex);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutSocketBone.SetAndAdvance(-1);
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetFilteredSocketTransform(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> SocketParam(Context);
	FNDIInputParam<FNiagaraBool> ApplyWorldTransform(Context);

	FNDIOutputParam<FVector3f> OutSocketTranslate(Context);
	FNDIOutputParam<FQuat4f> OutSocketRotation(Context);
	FNDIOutputParam<FVector3f> OutSocketScale(Context);

	const TArray<FTransform3f>& CurrentFilteredSockets = InstData->GetFilteredSocketsCurrBuffer();
	const int32 SocketMax = CurrentFilteredSockets.Num() - 1;
	if (SocketMax >= 0)
	{
		const bool bNeedsRotation = OutSocketRotation.IsValid();
		const FMatrix InstanceTransform = InstData->Transform;
		const FQuat4f InstanceRotation = bNeedsRotation ? FQuat4f(InstanceTransform.GetMatrixWithoutScale().ToQuat()) : FQuat4f::Identity;

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 SocketIndex = FMath::Clamp(SocketParam.GetAndAdvance(), 0, SocketMax);
			FVector SocketTranslation = (FVector)CurrentFilteredSockets[SocketIndex].GetTranslation();
			FQuat4f SocketRotation = CurrentFilteredSockets[SocketIndex].GetRotation();
			FVector SocketScale = (FVector)CurrentFilteredSockets[SocketIndex].GetScale3D();

			const bool bApplyTransform = ApplyWorldTransform.GetAndAdvance();
			if (bApplyTransform)
			{
				SocketTranslation = InstanceTransform.TransformPosition(SocketTranslation);
				SocketRotation = InstanceRotation * SocketRotation;
				SocketScale = InstanceTransform.TransformVector(SocketScale);
			}

			OutSocketTranslate.SetAndAdvance((FVector3f)SocketTranslation);
			OutSocketRotation.SetAndAdvance(SocketRotation);
			OutSocketScale.SetAndAdvance((FVector3f)SocketScale);
		}
	}
	else
	{
		for (int32 i=0; i < Context.GetNumInstances(); ++i)
		{
			OutSocketTranslate.SetAndAdvance(FVector3f::ZeroVector);
			OutSocketRotation.SetAndAdvance(FQuat4f::Identity);
			OutSocketScale.SetAndAdvance(FVector3f::ZeroVector);
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::RandomFilteredSocket(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIRandomHelper RandHelper(Context);

	FNDIOutputParam<int32> OutSocketBone(Context);
	const int32 FilteredSocketBoneOffset = InstData->FilteredSocketBoneOffset;

	int32 Max = FilteredSockets.Num() - 1;
	if (Max != INDEX_NONE)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			const int32 SocketIndex = RandHelper.RandRange(i, 0, Max);
			OutSocketBone.SetAndAdvance(FilteredSocketBoneOffset + SocketIndex);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutSocketBone.SetAndAdvance(-1);
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::RandomFilteredSocketOrBone(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIRandomHelper RandHelper(Context);

	FNDIOutputParam<int32> OutBoneIndex(Context);

	const int32 Max = FilteredSockets.Num() + InstData->NumFilteredBones - 1;
	if (Max >= 0)
	{
		const int32 NumFilteredBones = InstData->NumFilteredBones;
		const int32 FilteredSocketBoneOffset = InstData->FilteredSocketBoneOffset;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			const int32 FilteredIndex = RandHelper.RandRange(i, 0, Max);
			if (FilteredIndex < NumFilteredBones)
			{
				OutBoneIndex.SetAndAdvance(InstData->FilteredAndUnfilteredBones[FilteredIndex]);
			}
			else
			{
				OutBoneIndex.SetAndAdvance(FilteredSocketBoneOffset + FilteredIndex - NumFilteredBones);
			}
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutBoneIndex.SetAndAdvance(-1);
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetFilteredSocketOrBoneCount(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);

	FNDIOutputParam<int32> OutCount(Context);
	const int32 Count = FilteredSockets.Num() + InstData->NumFilteredBones;
	for (int32 i=0; i < Context.GetNumInstances(); ++i)
	{
		OutCount.SetAndAdvance(Count);
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetFilteredSocketOrBoneBoneAt(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> IndexParam(Context);
	FNDIOutputParam<int32> OutBoneIndex(Context);

	const int32 Max = FilteredSockets.Num() + InstData->NumFilteredBones - 1;
	if (Max >= 0)
	{
		const int32 NumFilteredBones = InstData->NumFilteredBones;
		const int32 FilteredSocketBoneOffset = InstData->FilteredSocketBoneOffset;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 FilteredIndex = IndexParam.GetAndAdvance();
			if (FilteredIndex < 0)
			{
				OutBoneIndex.SetAndAdvance(-1);
			}
			else if (FilteredIndex < NumFilteredBones)
			{
				OutBoneIndex.SetAndAdvance(InstData->FilteredAndUnfilteredBones[FilteredIndex]);
			}
			else
			{
				OutBoneIndex.SetAndAdvance(FilteredSocketBoneOffset + FilteredIndex - NumFilteredBones);
			}
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutBoneIndex.SetAndAdvance(-1);
		}
	}
}

#undef LOCTEXT_NAMESPACE
