// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSkeletalMesh.h"

#include "Components/SkeletalMeshComponent.h"
#include "NDISkeletalMeshCommon.h"
#include "Experimental/NiagaraDataInterfaceSkeletalMeshUvMapping.h"
#include "NiagaraDataInterfaceMeshCommon.h"
#include "NiagaraStats.h"
#include "SkeletalMeshTypes.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/IntegralConstant.h"

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceSkeletalMesh_TriangleSampling"

DECLARE_CYCLE_STAT(TEXT("Skel Mesh Sampling"), STAT_NiagaraSkel_Sample, STATGROUP_Niagara);

//Final binders for all static mesh interface functions.
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, RandomTriCoord);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordSkinnedData);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordSkinnedDataFallback);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, VMGetSkinnedTriangleVertexData);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordColor);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordColorFallback);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordUV);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, IsValidTriCoord);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredTriangleCount);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredTriangleAt);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriangleCoordAtUV);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriangleCoordInAabb);

const FName FSkeletalMeshInterfaceHelper::RandomTriCoordName("RandomTriCoord");
const FName FSkeletalMeshInterfaceHelper::IsValidTriCoordName("IsValidTriCoord");
const FName FSkeletalMeshInterfaceHelper::GetTriangleDataName("GetTriangleData");
const FName FSkeletalMeshInterfaceHelper::GetTriangleIndicesName("GetTriangleIndices");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataName("GetSkinnedTriangleData");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataWSName("GetSkinnedTriangleDataWS");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataInterpName("GetSkinnedTriangleDataInterpolated");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataWSInterpName("GetSkinnedTriangleDataWSInterpolated");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataName("GetSkinnedTriangleVertexData");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataWSName("GetSkinnedTriangleVertexDataWS");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataInterpName("GetSkinnedTriangleVertexDataInterpolated");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataWSInterpName("GetSkinnedTriangleVertexDataWSInterpolated");
const FName FSkeletalMeshInterfaceHelper::GetTriColorName("GetTriColor");
const FName FSkeletalMeshInterfaceHelper::GetTriUVName("GetTriUV");
const FName FSkeletalMeshInterfaceHelper::GetTriCoordVerticesName("GetTriCoordVertices");
const FName FSkeletalMeshInterfaceHelper::RandomTriangleName("RandomTriangle");
const FName FSkeletalMeshInterfaceHelper::GetTriangleCountName("GetTriangleCount");
const FName FSkeletalMeshInterfaceHelper::RandomFilteredTriangleName("RandomFilteredTriangle");
const FName FSkeletalMeshInterfaceHelper::GetFilteredTriangleCountName("GetFilteredTriangleCount");
const FName FSkeletalMeshInterfaceHelper::GetFilteredTriangleAtName("GetFilteredTriangle");
const FName FSkeletalMeshInterfaceHelper::GetTriangleCoordAtUVName("GetTriangleCoordAtUV");
const FName FSkeletalMeshInterfaceHelper::GetTriangleCoordInAabbName("GetTriangleCoordInAabb");
const FName FSkeletalMeshInterfaceHelper::GetAdjacentTriangleIndexName("GetAdjacentTriangleIndex");
const FName FSkeletalMeshInterfaceHelper::GetTriangleNeighborName("GetTriangleNeighbor");

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSkeletalMesh::GetTriangleSamplingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	//-TODO: Remove / deprecate this function!
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::RandomTriCoordName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::IsValidTriCoordName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("IsValidDesc", "Determine if this tri coordinate's triangle index is valid for this mesh. Note that this only checks the mesh index buffer size and does not include any filtering settings.");
	}
	
	{
		FNiagaraFunctionSignature & Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetTriangleDataName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetTriangleDataDesc", "Returns bind pose triangle data.");
	}

	{
		FNiagaraFunctionSignature & Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetTriangleIndicesName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index0"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index1"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index2"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetTriangleDataDesc", "Returns bind pose triangle data.");
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetOptionalSkinnedDataDesc", "Returns skinning dependant data for the pased MeshTriCoord in local space. All outputs are optional and you will incur zerp minimal cost if they are not connected.");
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataWSName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetOptionalSkinnedDataWSDesc", "Returns skinning dependant data for the pased MeshTriCoord in world space. All outputs are optional and you will incur zerp minimal cost if they are not connected.");
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataInterpName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interp")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetSkinnedDataDesc", "Returns skinning dependant data for the pased MeshTriCoord in local space. Interpolates between previous and current frame. All outputs are optional and you will incur zerp minimal cost if they are not connected.");
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataWSInterpName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interp")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetSkinnedDataWSDesc", "Returns skinning dependant data for the pased MeshTriCoord in world space. Interpolates between previous and current frame. All outputs are optional and you will incur zerp minimal cost if they are not connected.");
	}

	{
		FNiagaraFunctionSignature BaseSig;
		BaseSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh"));
		BaseSig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position 0"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity 0"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal 0"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal 0"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent 0"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position 1"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity 1"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal 1"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal 1"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent 1"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position 2"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity 2"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal 2"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal 2"));
		BaseSig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent 2"));
		BaseSig.bMemberFunction = true;
		BaseSig.bRequiresContext = false;

		OutFunctions.Add_GetRef(BaseSig).Name = FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataName;
		OutFunctions.Add_GetRef(BaseSig).Name = FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataWSName;

		BaseSig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interp"));
		OutFunctions.Add_GetRef(BaseSig).Name = FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataInterpName;
		OutFunctions.Add_GetRef(BaseSig).Name = FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataWSInterpName;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetTriColorName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetTriUVName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("UV Set")));

		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetTriCoordVerticesName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("TriangleIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex 0")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex 1")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex 2")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetTriCoordVetsName", "Takes the TriangleIndex from a MeshTriCoord and returns the vertices for that triangle.");
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::RandomTriangleName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetTriangleCountName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::RandomFilteredTriangleName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetFilteredTriangleCountName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetFilteredTriangleAtName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaryCoord"))).SetValue(FVector3f(1.0f / 3.0f));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetTriangleCoordAtUVName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));

		FNiagaraVariable EnabledVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled"));
		EnabledVariable.SetValue(true);
		Sig.Inputs.Add(EnabledVariable);

		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV")));

		FNiagaraVariable ToleranceVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Tolerance"));
		ToleranceVariable.SetValue(KINDA_SMALL_NUMBER);
		Sig.Inputs.Add(ToleranceVariable);

		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetTriangleCoordInAabbName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));

		FNiagaraVariable EnabledVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled"));
		EnabledVariable.SetValue(true);
		Sig.Inputs.Add(EnabledVariable);

		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UvMin")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UvMax")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetAdjacentTriangleIndexName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex ID")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Adjacency Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = FSkeletalMeshInterfaceHelper::GetTriangleNeighborName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Triangle Index")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Edge Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Neighbor Triangle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Neighbor Edge Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
	}
}
#endif

void UNiagaraDataInterfaceSkeletalMesh::BindTriangleSamplingFunction(const FVMExternalFunctionBindingInfo& BindingInfo, FNDISkeletalMesh_InstanceData* InstanceData, FVMExternalFunction &OutFunc)
{
	using TInterpOff = TIntegralConstant<bool, false>;
	using TInterpOn = TIntegralConstant<bool, true>;

	if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::RandomTriCoordName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 4);
		TFilterModeBinder<TAreaWeightingModeBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, RandomTriCoord)>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::IsValidTriCoordName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 1);
		TFilterModeBinder<TAreaWeightingModeBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, IsValidTriCoord)>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetTriangleDataName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 12);
		OutFunc = FVMExternalFunction::CreateLambda([this](FVectorVMExternalFunctionContext& Context) {this->GetTriangleData(Context);});
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetTriangleIndicesName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetTriangleIndices);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 15);
		if (InstanceData->bAllowCPUMeshDataAccess)
		{
			TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TVertexAccessorBinder<TNDIExplicitBinder<TInterpOff, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordSkinnedData)>>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
		}
		else
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordSkinnedDataFallback)::Bind<FNDITransformHandlerNoop, TInterpOff>(this, BindingInfo, InstanceData, OutFunc);
		}
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataWSName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 15);
		if (InstanceData->bAllowCPUMeshDataAccess)
		{
			TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandler, TVertexAccessorBinder<TNDIExplicitBinder<TIntegralConstant<bool, false>, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordSkinnedData)>>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
		}
		else
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordSkinnedDataFallback)::Bind<FNDITransformHandler, TInterpOff>(this, BindingInfo, InstanceData, OutFunc);
		}
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataInterpName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 15);
		if (InstanceData->bAllowCPUMeshDataAccess)
		{
			TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TVertexAccessorBinder<TNDIExplicitBinder<TInterpOn, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordSkinnedData)>>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
		}
		else
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordSkinnedDataFallback)::Bind<FNDITransformHandlerNoop, TInterpOn>(this, BindingInfo, InstanceData, OutFunc);
		}
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataWSInterpName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 15);
		if (InstanceData->bAllowCPUMeshDataAccess)
		{
			TSkinningModeBinder<TNDIExplicitBinder< FNDITransformHandler, TVertexAccessorBinder<TNDIExplicitBinder<TIntegralConstant<bool, true>, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordSkinnedData)>>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
		}
		else
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordSkinnedDataFallback)::Bind<FNDITransformHandler, TInterpOn>(this, BindingInfo, InstanceData, OutFunc);
		}
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataName)
	{
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TVertexAccessorBinder<TNDIExplicitBinder<TIntegralConstant<bool, false>, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, VMGetSkinnedTriangleVertexData)>>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataWSName)
	{
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandler, TVertexAccessorBinder<TNDIExplicitBinder<TIntegralConstant<bool, false>, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, VMGetSkinnedTriangleVertexData)>>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataInterpName)
	{
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TVertexAccessorBinder<TNDIExplicitBinder<TIntegralConstant<bool, true>, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, VMGetSkinnedTriangleVertexData)>>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataWSInterpName)
	{
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandler, TVertexAccessorBinder<TNDIExplicitBinder<TIntegralConstant<bool, true>, NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, VMGetSkinnedTriangleVertexData)>>>>::BindIgnoreCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetTriColorName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 4);
		if (InstanceData->HasColorData())
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordColor)::Bind(this, OutFunc);
		}
		else
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordColorFallback)::Bind(this, OutFunc);
		}
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetTriUVName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 2);
		TVertexAccessorBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriCoordUV)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetTriCoordVerticesName)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetTriangleIndices);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::RandomTriangleName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 4);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::RandomTriangle);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetTriangleCountName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetTriangleCount);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::RandomFilteredTriangleName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 4);
		TFilterModeBinder<TAreaWeightingModeBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, RandomTriCoord)>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetFilteredTriangleCountName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		TFilterModeBinder<TAreaWeightingModeBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredTriangleCount)>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetFilteredTriangleAtName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 4);
		TFilterModeBinder<TAreaWeightingModeBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredTriangleAt)>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetTriangleCoordAtUVName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 5);
		TVertexAccessorBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriangleCoordAtUV)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetTriangleCoordInAabbName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 5);
		TVertexAccessorBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetTriangleCoordInAabb)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
}

template<typename FilterMode, typename AreaWeightingMode>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::RandomTriIndex(FNDIRandomHelper& RandHelper, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 InstanceIndex)
{
	checkf(false, TEXT("Invalid template call for RandomTriIndex. Bug in Filter binding or Area Weighting binding. Contact code team."));
	return 0;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::RandomTriIndex<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>
	(FNDIRandomHelper& RandHelper, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 InstanceIndex)
{
	int32 Tri = -1;
	const int NumSections = Accessor.LODData->RenderSections.Num();
	if ( NumSections > 0 )
	{
		const int32 SectionIndex = RandHelper.RandRange(InstanceIndex, 0, NumSections - 1);
		const FSkelMeshRenderSection& Section = Accessor.LODData->RenderSections[SectionIndex];
		Tri = (Section.BaseIndex / 3) + RandHelper.RandRange(InstanceIndex, 0, Section.NumTriangles - 1);
	}
	return Tri;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::RandomTriIndex<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOn>
	(FNDIRandomHelper& RandHelper, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 InstanceIndex)
{
	check(Accessor.Mesh);
	int32 Tri = -1;
	const FSkeletalMeshSamplingInfo& SamplingInfo = Accessor.Mesh->GetSamplingInfo();
	const FSkeletalMeshSamplingLODBuiltData& WholeMeshBuiltData = SamplingInfo.GetWholeMeshLODBuiltData(InstData->GetLODIndex());
	if (WholeMeshBuiltData.AreaWeightedTriangleSampler.GetNumEntries() > 0 )
	{
		Tri = WholeMeshBuiltData.AreaWeightedTriangleSampler.GetEntryIndex(RandHelper.Rand(InstanceIndex), RandHelper.Rand(InstanceIndex));
	}
	return Tri;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::RandomTriIndex<TNDISkelMesh_FilterModeSingle, TNDISkelMesh_AreaWeightingOff>
	(FNDIRandomHelper& RandHelper, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 InstanceIndex)
{
	int32 Tri = -1;
	const int32 SectionNumTriangles = Accessor.SamplingRegionBuiltData->TriangleIndices.Num();
	if (SectionNumTriangles > 0)
	{
		const int32 Idx = RandHelper.RandRange(InstanceIndex, 0, SectionNumTriangles - 1);
		Tri = Accessor.SamplingRegionBuiltData->TriangleIndices[Idx] / 3;
	}
	return Tri;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::RandomTriIndex<TNDISkelMesh_FilterModeSingle, TNDISkelMesh_AreaWeightingOn>
	(FNDIRandomHelper& RandHelper, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 InstanceIndex)
{
	int32 Tri = -1;
	if (Accessor.SamplingRegionBuiltData->AreaWeightedSampler.GetNumEntries() > 0)
	{
		const int32 Idx = Accessor.SamplingRegionBuiltData->AreaWeightedSampler.GetEntryIndex(RandHelper.Rand(InstanceIndex), RandHelper.Rand(InstanceIndex));
		Tri = Accessor.SamplingRegionBuiltData->TriangleIndices[Idx] / 3;
	}
	return Tri;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::RandomTriIndex<TNDISkelMesh_FilterModeMulti, TNDISkelMesh_AreaWeightingOff>
	(FNDIRandomHelper& RandHelper, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 InstanceIndex)
{
	check(Accessor.Mesh);
	int32 Tri = -1;
	const int32 NumRegions = InstData->SamplingRegionIndices.Num();
	if (NumRegions > 0)
	{
		const int32 RegionIdx = RandHelper.RandRange(InstanceIndex, 0, NumRegions - 1);
		const FSkeletalMeshSamplingInfo& SamplingInfo = Accessor.Mesh->GetSamplingInfo();
		const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(InstData->SamplingRegionIndices[RegionIdx]);
		const FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = SamplingInfo.GetRegionBuiltData(InstData->SamplingRegionIndices[RegionIdx]);
		if (RegionBuiltData.TriangleIndices.Num() > 0)
		{
			const int32 Idx = RandHelper.RandRange(InstanceIndex, 0, RegionBuiltData.TriangleIndices.Num() - 1);
			Tri = RegionBuiltData.TriangleIndices[Idx] / 3;
		}
	}
	return Tri;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::RandomTriIndex<TNDISkelMesh_FilterModeMulti, TNDISkelMesh_AreaWeightingOn>
	(FNDIRandomHelper& RandHelper, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 InstanceIndex)
{
	check(Accessor.Mesh);
	int32 Tri = -1;
	if ( InstData->SamplingRegionAreaWeightedSampler.GetNumEntries() > 0 )
	{
		const int32 RegionIdx = InstData->SamplingRegionAreaWeightedSampler.GetEntryIndex(RandHelper.Rand(InstanceIndex), RandHelper.Rand(InstanceIndex));
		const FSkeletalMeshSamplingInfo& SamplingInfo = Accessor.Mesh->GetSamplingInfo();
		const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(InstData->SamplingRegionIndices[RegionIdx]);
		const FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = SamplingInfo.GetRegionBuiltData(InstData->SamplingRegionIndices[RegionIdx]);
		if (RegionBuiltData.AreaWeightedSampler.GetNumEntries() > 0)
		{
			const int32 Idx = RegionBuiltData.AreaWeightedSampler.GetEntryIndex(RandHelper.Rand(InstanceIndex), RandHelper.Rand(InstanceIndex));
			Tri = RegionBuiltData.TriangleIndices[Idx] / 3;
		}
	}
	return Tri;
}

template<typename FilterMode, typename AreaWeightingMode>
void UNiagaraDataInterfaceSkeletalMesh::RandomTriCoord(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	//FNDIParameter<FNiagaraRandInfo> RandInfoParam(Context);
	FNDIRandomHelper RandHelper(Context);

	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());

	FNDIOutputParam<int32> OutTri(Context);
	FNDIOutputParam<FVector3f> OutBary(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<FilterMode, AreaWeightingMode>(InstData);

	if (MeshAccessor.IsLODAccessible())
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();//We grab the rand info to a local value first so it can be used for multiple rand calls from the helper.
			OutTri.SetAndAdvance(RandomTriIndex<FilterMode, AreaWeightingMode>(RandHelper, MeshAccessor, InstData, i));
			OutBary.SetAndAdvance(RandomBarycentricCoord(Context.GetRandStream()));
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTri.SetAndAdvance(-1);
			OutBary.SetAndAdvance(FVector3f::ZeroVector);
		}
	}
}

template<typename FilterMode, typename AreaWeightingMode>
void UNiagaraDataInterfaceSkeletalMesh::IsValidTriCoord(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);

	VectorVM::FExternalFuncInputHandler<int32> TriParam(Context);
	VectorVM::FExternalFuncInputHandler<float> BaryXParam(Context);
	VectorVM::FExternalFuncInputHandler<float> BaryYParam(Context);
	VectorVM::FExternalFuncInputHandler<float> BaryZParam(Context);

	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());

	FNDIOutputParam<FNiagaraBool> OutValid(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<FilterMode, AreaWeightingMode>(InstData);

	if (MeshAccessor.IsLODAccessible())
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 RequestedIndex = (TriParam.GetAndAdvance() * 3) + 2; // Get the last triangle index of the set
			OutValid.SetAndAdvance(MeshAccessor.IndexBuffer != nullptr && MeshAccessor.IndexBuffer->Num() > RequestedIndex);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutValid.SetAndAdvance(false);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraDataInterfaceSkeletalMesh::RandomTriangle(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIRandomHelper RandHelper(Context);
	FNDIOutputParam<int32> OutTri(Context);
	FNDIOutputParam<FVector3f> OutBary(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<TIntegralConstant<int32, 0>, TIntegralConstant<int32, 0>>(InstData);

	if (!MeshAccessor.IsLODAccessible())
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTri.SetAndAdvance(-1);
			OutBary.SetAndAdvance(FVector3f::ZeroVector);
		}
		return;
	}

	if (USkeletalMesh* SkelMesh = MeshAccessor.Mesh)
	{
		const int32 LODIndex = InstData->GetLODIndex();
		const bool bAreaWeighted = SkelMesh->GetLODInfo(LODIndex)->bSupportUniformlyDistributedSampling;

		if (bAreaWeighted)
		{
			const FSkeletalMeshSamplingInfo& SamplingInfo = SkelMesh->GetSamplingInfo();
			const FSkeletalMeshSamplingBuiltData& SamplingInfoBuiltData = SamplingInfo.GetBuiltData();
			if (SamplingInfoBuiltData.WholeMeshBuiltData.IsValidIndex(LODIndex))
			{
				//-TODO: Consider adding a way to warn the user we failed to get the area weighted data
				const FSkeletalMeshSamplingLODBuiltData& WholeMeshBuiltData = SamplingInfoBuiltData.WholeMeshBuiltData[LODIndex];
				if (WholeMeshBuiltData.AreaWeightedTriangleSampler.GetNumEntries() > 0)
				{
					for (int32 i = 0; i < Context.GetNumInstances(); ++i)
					{
						RandHelper.GetAndAdvance();
						OutTri.SetAndAdvance(WholeMeshBuiltData.AreaWeightedTriangleSampler.GetEntryIndex(RandHelper.Rand(i), RandHelper.Rand(i)));
						OutBary.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
					}
					return;
				}
			}
		}
	}

	const int32 MaxTriangle = (MeshAccessor.IndexBuffer->Num() / 3) - 1;
	if (MaxTriangle >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutTri.SetAndAdvance(RandHelper.RandRange(i, 0, MaxTriangle));
			OutBary.SetAndAdvance(RandHelper.RandomBarycentricCoord(i));
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTri.SetAndAdvance(-1);
			OutBary.SetAndAdvance(FVector3f::ZeroVector);
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetTriangleCount(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIOutputParam<int32> OutCount(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<TIntegralConstant<int32, 0>, TIntegralConstant<int32, 0>>(InstData);
	
	const int32 NumTriangles = MeshAccessor.IsLODAccessible() ? MeshAccessor.IndexBuffer->Num() / 3 : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCount.SetAndAdvance(NumTriangles);
	}
}

template<typename FilterMode, typename AreaWeightingMode>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleCount(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	checkf(false, TEXT("Invalid template call for GetFilteredTriangleCount. Bug in Filter binding or Area Weighting binding. Contact code team."));
	return 0;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleCount<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	int32 NumTris = 0;
	for (int32 i = 0; i < Accessor.LODData->RenderSections.Num(); i++)
	{
		NumTris += Accessor.LODData->RenderSections[i].NumTriangles;
	}
	return NumTris;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleCount<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOn>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	check(Accessor.Mesh);
	const FSkeletalMeshSamplingInfo& SamplingInfo = Accessor.Mesh->GetSamplingInfo();
	const FSkeletalMeshSamplingLODBuiltData& WholeMeshBuiltData = SamplingInfo.GetWholeMeshLODBuiltData(InstData->GetLODIndex());
	return WholeMeshBuiltData.AreaWeightedTriangleSampler.GetNumEntries();
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleCount<TNDISkelMesh_FilterModeSingle, TNDISkelMesh_AreaWeightingOff>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	return Accessor.SamplingRegionBuiltData->TriangleIndices.Num();
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleCount<TNDISkelMesh_FilterModeSingle, TNDISkelMesh_AreaWeightingOn>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	return Accessor.SamplingRegionBuiltData->AreaWeightedSampler.GetNumEntries();
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleCount<TNDISkelMesh_FilterModeMulti, TNDISkelMesh_AreaWeightingOff>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	USkeletalMesh* SkelMesh = Accessor.Mesh;
	check(SkelMesh);
	int32 NumTris = 0;

	for (int32 RegionIdx = 0; RegionIdx < InstData->SamplingRegionIndices.Num(); RegionIdx++)
	{
		const FSkeletalMeshSamplingInfo& SamplingInfo = SkelMesh->GetSamplingInfo();
		const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(InstData->SamplingRegionIndices[RegionIdx]);
		const FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = SamplingInfo.GetRegionBuiltData(InstData->SamplingRegionIndices[RegionIdx]);
		NumTris += RegionBuiltData.TriangleIndices.Num();
	}
	return NumTris;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleCount<TNDISkelMesh_FilterModeMulti, TNDISkelMesh_AreaWeightingOn>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	USkeletalMesh* SkelMesh = Accessor.Mesh;
	check(SkelMesh);
	int32 NumTris = 0;

	for (int32 RegionIdx = 0; RegionIdx < InstData->SamplingRegionIndices.Num(); RegionIdx++)
	{
		const FSkeletalMeshSamplingInfo& SamplingInfo = SkelMesh->GetSamplingInfo();
		const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(InstData->SamplingRegionIndices[RegionIdx]);
		const FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = SamplingInfo.GetRegionBuiltData(InstData->SamplingRegionIndices[RegionIdx]);
		NumTris += RegionBuiltData.TriangleIndices.Num();
	}
	return NumTris;
}

template<typename FilterMode, typename AreaWeightingMode>
void UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleCount(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());

	FNDIOutputParam<int32> OutTri(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<FilterMode, AreaWeightingMode>(InstData);

	int32 Count = MeshAccessor.IsLODAccessible() ? GetFilteredTriangleCount<FilterMode, AreaWeightingMode>(MeshAccessor, InstData) : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutTri.SetAndAdvance(Count);
	}
}

//////////////////////////////////////////////////////////////////////////

template<typename FilterMode, typename AreaWeightingMode>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleAt(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIndex)
{
	checkf(false, TEXT("Invalid template call for GetFilteredTriangleAt. Bug in Filter binding or Area Weighting binding. Contact code team."));
	return 0;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleAt<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIndex)
{
	for (int32 i = 0; i < Accessor.LODData->RenderSections.Num(); i++)
	{
		if (Accessor.LODData->RenderSections[i].NumTriangles > (uint32)FilteredIndex)
		{
			const FSkelMeshRenderSection& Sec = Accessor.LODData->RenderSections[i];
			return Sec.BaseIndex + FilteredIndex;
		}
		FilteredIndex -= Accessor.LODData->RenderSections[i].NumTriangles;
	}
	return 0;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleAt<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOn>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIndex)
{
	int32 TriIdx = FilteredIndex;
	return TriIdx;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleAt<TNDISkelMesh_FilterModeSingle, TNDISkelMesh_AreaWeightingOff>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIndex)
{
	int32 MaxIdx = Accessor.SamplingRegionBuiltData->TriangleIndices.Num() - 1;
	FilteredIndex = FMath::Min(FilteredIndex, MaxIdx);
	return Accessor.SamplingRegionBuiltData->TriangleIndices[FilteredIndex] / 3;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleAt<TNDISkelMesh_FilterModeSingle, TNDISkelMesh_AreaWeightingOn>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIndex)
{
	int32 Idx = FilteredIndex;
	int32 MaxIdx = Accessor.SamplingRegionBuiltData->TriangleIndices.Num() - 1;
	Idx = FMath::Min(Idx, MaxIdx);

	return Accessor.SamplingRegionBuiltData->TriangleIndices[Idx] / 3;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleAt<TNDISkelMesh_FilterModeMulti, TNDISkelMesh_AreaWeightingOff>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIndex)
{
	USkeletalMesh* SkelMesh = Accessor.Mesh;
	check(SkelMesh);

	for (int32 RegionIdx = 0; RegionIdx < InstData->SamplingRegionIndices.Num(); RegionIdx++)
	{
		const FSkeletalMeshSamplingInfo& SamplingInfo = SkelMesh->GetSamplingInfo();
		const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(InstData->SamplingRegionIndices[RegionIdx]);
		const FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = SamplingInfo.GetRegionBuiltData(InstData->SamplingRegionIndices[RegionIdx]);
		if (FilteredIndex < RegionBuiltData.TriangleIndices.Num())
		{
			return RegionBuiltData.TriangleIndices[FilteredIndex] / 3;
		}

		FilteredIndex -= RegionBuiltData.TriangleIndices.Num();
	}
	return 0;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleAt<TNDISkelMesh_FilterModeMulti, TNDISkelMesh_AreaWeightingOn>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIndex)
{
	USkeletalMesh* SkelMesh = Accessor.Mesh;
	check(SkelMesh);

	for (int32 RegionIdx = 0; RegionIdx < InstData->SamplingRegionIndices.Num(); RegionIdx++)
	{
		const FSkeletalMeshSamplingInfo& SamplingInfo = SkelMesh->GetSamplingInfo();
		const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(InstData->SamplingRegionIndices[RegionIdx]);
		const FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = SamplingInfo.GetRegionBuiltData(InstData->SamplingRegionIndices[RegionIdx]);
		if (FilteredIndex < RegionBuiltData.TriangleIndices.Num())
		{
			return RegionBuiltData.TriangleIndices[FilteredIndex] / 3;
		}
		FilteredIndex -= RegionBuiltData.TriangleIndices.Num();
	}
	return 0;
}

template<typename FilterMode, typename AreaWeightingMode>
void UNiagaraDataInterfaceSkeletalMesh::GetFilteredTriangleAt(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	VectorVM::FExternalFuncInputHandler<int32> TriParam(Context);
	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());

	FNDIInputParam<FVector3f> InBary(Context);
	FNDIOutputParam<int32> OutTri(Context);
	FNDIOutputParam<FVector3f> OutBary(Context);

	FSkeletalMeshAccessorHelper Accessor;
	Accessor.Init<FilterMode, AreaWeightingMode>(InstData);

	if (Accessor.IsLODAccessible())
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			int32 Tri = TriParam.GetAndAdvance();
			int32 RealIdx = 0;
			RealIdx = GetFilteredTriangleAt<FilterMode, AreaWeightingMode>(Accessor, InstData, Tri);

			const int32 TriMax = (Accessor.IndexBuffer->Num() / 3) - 1;
			RealIdx = FMath::Clamp(RealIdx, 0, TriMax);

			OutTri.SetAndAdvance(RealIdx);
			OutBary.SetAndAdvance(InBary.GetAndAdvance());
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTri.SetAndAdvance(-1);
			OutBary.SetAndAdvance(InBary.GetAndAdvance());
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetTriCoordColor(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> TriParam(Context);
	FNDIInputParam<FVector3f> BaryParam(Context);

	FNDIOutputParam<FLinearColor> OutColor(Context);

	if ( const FSkeletalMeshLODRenderData* LODData = InstData->CachedLODData )
	{
		const FColorVertexBuffer& Colors = LODData->StaticVertexBuffers.ColorVertexBuffer;

		const FMultiSizeIndexContainer& Indices = LODData->MultiSizeIndexContainer;
		const FRawStaticIndexBuffer16or32Interface* IndexBuffer = Indices.GetIndexBuffer();
		const int32 TriMax = (IndexBuffer->Num() / 3) - 1;
		if (TriMax >= 0 && (Colors.GetNumVertices() > 0) && (Colors.GetVertexData() != nullptr) && Colors.GetAllowCPUAccess() )
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				const int32 Tri = FMath::Clamp(TriParam.GetAndAdvance(), 0, TriMax) * 3;
				const int32 Idx0 = IndexBuffer->Get(Tri);
				const int32 Idx1 = IndexBuffer->Get(Tri + 1);
				const int32 Idx2 = IndexBuffer->Get(Tri + 2);

				FLinearColor Color = BarycentricInterpolate(BaryParam.GetAndAdvance(), Colors.VertexColor(Idx0).ReinterpretAsLinear(), Colors.VertexColor(Idx1).ReinterpretAsLinear(), Colors.VertexColor(Idx2).ReinterpretAsLinear());
				OutColor.SetAndAdvance(Color);
			}
			// Early out as we are done
			return;
		}
	}

	// Error fall-through
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutColor.SetAndAdvance(FLinearColor::White);
	}
}

// Where we determine we are sampling a skeletal mesh without tri color we bind to this fallback method 
void UNiagaraDataInterfaceSkeletalMesh::GetTriCoordColorFallback(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> TriParam(Context);
	FNDIInputParam<FVector3f> BaryParam(Context);

	FNDIOutputParam<FLinearColor> OutColor(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutColor.SetAndAdvance(FLinearColor::White);
	}
}

template<typename VertexAccessorType>
void UNiagaraDataInterfaceSkeletalMesh::GetTriCoordUV(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	VertexAccessorType VertAccessor;
	FNDIInputParam<int32> TriParam(Context);
	FNDIInputParam<FVector3f> BaryParam(Context);
	FNDIInputParam<int32> UVSetParam(Context);

	checkf(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());
	checkf(InstData->bMeshValid, TEXT("Skeletal Mesh Interface has invalid mesh. %s"), *GetPathName());

	FNDIOutputParam<FVector2f> OutUV(Context);

	if ( const FSkeletalMeshLODRenderData* LODData = InstData->CachedLODData )
	{
		const FMultiSizeIndexContainer& Indices = LODData->MultiSizeIndexContainer;
		const FRawStaticIndexBuffer16or32Interface* IndexBuffer = Indices.GetIndexBuffer();
		const int32 TriMax = (IndexBuffer->Num() / 3) - 1;
		if (TriMax >= 0)
		{
			const int32 UVSetMax = LODData->StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() - 1;
			const float InvDt = 1.0f / InstData->DeltaSeconds;
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				const int32 Tri = FMath::Clamp(TriParam.GetAndAdvance(), 0, TriMax) * 3;
				const int32 Idx0 = IndexBuffer->Get(Tri);
				const int32 Idx1 = IndexBuffer->Get(Tri + 1);
				const int32 Idx2 = IndexBuffer->Get(Tri + 2);
				const int32 UVSet = FMath::Clamp(UVSetParam.GetAndAdvance(), 0, UVSetMax);
				const FVector2f UV0 = FVector2f(VertAccessor.GetVertexUV(LODData, Idx0, UVSet));	// LWC_TODO: Precision loss
				const FVector2f UV1 = FVector2f(VertAccessor.GetVertexUV(LODData, Idx1, UVSet));	// LWC_TODO: Precision loss
				const FVector2f UV2 = FVector2f(VertAccessor.GetVertexUV(LODData, Idx2, UVSet));	// LWC_TODO: Precision loss

				FVector2f UV = BarycentricInterpolate(BaryParam.GetAndAdvance(), UV0, UV1, UV2);
				OutUV.SetAndAdvance(UV);
			}

			// Early out as we are done
			return;
		}
	}

	// Fall-though for failure conditions
	for (int32 i=0; i < Context.GetNumInstances(); ++i)
	{
		OutUV.SetAndAdvance(FVector2f::ZeroVector);
	}
}

// Stub specialization for no valid mesh data on the data interface
template<>
void UNiagaraDataInterfaceSkeletalMesh::GetTriCoordUV<FSkelMeshVertexAccessorNoop>(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> TriParam(Context);
	FNDIInputParam<FVector3f> BaryParam(Context);
	FNDIInputParam<int32> UVSetParam(Context);

	FNDIOutputParam<FVector2f> OutUV(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutUV.SetAndAdvance(FVector2f::ZeroVector);
	}
}

template<typename VertexAccessorType>
void UNiagaraDataInterfaceSkeletalMesh::GetTriangleCoordAtUV(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<bool> InEnabled(Context);
	FNDIInputParam<FVector2f> InUV(Context);
	FNDIInputParam<float> InTolerance(Context);

	FNDIOutputParam<int32> OutTriangleIndex(Context);
	FNDIOutputParam<FVector3f> OutBaryCoord(Context);
	FNDIOutputParam<FNiagaraBool> OutIsValid(Context);

	checkf(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());
	checkf(InstData->bMeshValid, TEXT("Skeletal Mesh Interface has invalid mesh. %s"), *GetPathName());

	const FSkeletalMeshUvMapping* UvMapping = InstData->UvMapping ? InstData->UvMapping.GetMappingData() : nullptr;

	if (UvMapping)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool Enabled = InEnabled.GetAndAdvance();
			const FVector2D SourceUv = FVector2D(InUV.GetAndAdvance());
			const float Tolerance = InTolerance.GetAndAdvance();

			FVector3f BaryCoord(ForceInitToZero);
			int32 TriangleIndex = INDEX_NONE;

			if (Enabled)
			{
				FVector BaryCoordD;
				TriangleIndex = UvMapping->FindFirstTriangle(SourceUv, Tolerance, BaryCoordD);
				BaryCoord = FVector3f(BaryCoordD);
			}

			OutTriangleIndex.SetAndAdvance(TriangleIndex);
			OutBaryCoord.SetAndAdvance(BaryCoord);
			OutIsValid.SetAndAdvance(TriangleIndex != INDEX_NONE);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTriangleIndex.SetAndAdvance(INDEX_NONE);
			OutBaryCoord.SetAndAdvance(FVector3f::ZeroVector);
			OutIsValid.SetAndAdvance(false);
		}
	}
}

// Stub specialization for no valid mesh data on the data interface
template<>
void UNiagaraDataInterfaceSkeletalMesh::GetTriangleCoordAtUV<FSkelMeshVertexAccessorNoop>(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<bool> InEnabled(Context);
	FNDIInputParam<FVector2f> InUV(Context);
	FNDIInputParam<float> InTolerance(Context);

	FNDIOutputParam<int32> OutTriangleIndex(Context);
	FNDIOutputParam<FVector3f> OutBaryCoord(Context);
	FNDIOutputParam<bool> OutIsValid(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutTriangleIndex.SetAndAdvance(INDEX_NONE);
		OutBaryCoord.SetAndAdvance(FVector3f::ZeroVector);
		OutIsValid.SetAndAdvance(false);
	}
}


template<typename VertexAccessorType>
void UNiagaraDataInterfaceSkeletalMesh::GetTriangleCoordInAabb(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<bool> InEnabled(Context);
	FNDIInputParam<FVector2f> InMinExtent(Context);
	FNDIInputParam<FVector2f> InMaxExtent(Context);

	FNDIOutputParam<int32> OutTriangleIndex(Context);
	FNDIOutputParam<FVector3f> OutBaryCoord(Context);
	FNDIOutputParam<FNiagaraBool> OutIsValid(Context);

	checkf(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());
	checkf(InstData->bMeshValid, TEXT("Skeletal Mesh Interface has invalid mesh. %s"), *GetPathName());

	const FSkeletalMeshUvMapping* UvMapping = InstData->UvMapping ? InstData->UvMapping.GetMappingData() : nullptr;

	if (UvMapping)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool Enabled = InEnabled.GetAndAdvance();
			const FVector2D MinExtent = FVector2D(InMinExtent.GetAndAdvance());
			const FVector2D MaxExtent = FVector2D(InMaxExtent.GetAndAdvance());

			FVector3f BaryCoord(ForceInitToZero);
			int32 TriangleIndex = INDEX_NONE;
			if (Enabled)
			{
				FVector BaryCoordD;
				TriangleIndex = UvMapping->FindFirstTriangle(FBox2D(MinExtent, MaxExtent), BaryCoordD);
				BaryCoord = FVector3f(BaryCoordD);
			}

			OutTriangleIndex.SetAndAdvance(TriangleIndex);
			OutBaryCoord.SetAndAdvance(BaryCoord);
			OutIsValid.SetAndAdvance(TriangleIndex != INDEX_NONE);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutTriangleIndex.SetAndAdvance(INDEX_NONE);
			OutBaryCoord.SetAndAdvance(FVector3f::ZeroVector);
			OutIsValid.SetAndAdvance(false);
		}
	}
}

template<>
void UNiagaraDataInterfaceSkeletalMesh::GetTriangleCoordInAabb<FSkelMeshVertexAccessorNoop>(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<bool> InEnabled(Context);
	FNDIInputParam<FVector2f> InMinExtent(Context);
	FNDIInputParam<FVector2f> InMaxExtent(Context);

	FNDIOutputParam<int32> OutTriangleIndex(Context);
	FNDIOutputParam<FVector3f> OutBaryCoord(Context);
	FNDIOutputParam<bool> OutIsValid(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutTriangleIndex.SetAndAdvance(INDEX_NONE);
		OutBaryCoord.SetAndAdvance(FVector3f::ZeroVector);
		OutIsValid.SetAndAdvance(false);
	}
}

struct FGetTriCoordSkinnedDataOutputHandler
{
	FGetTriCoordSkinnedDataOutputHandler(FVectorVMExternalFunctionContext& Context)
		: Position(Context)
		, Velocity(Context)
		, Normal(Context)
		, Binormal(Context)
		, Tangent(Context)
		, bNeedsPosition(Position.IsValid())
		, bNeedsVelocity(Velocity.IsValid())
		, bNeedsNorm(Normal.IsValid())
		, bNeedsBinorm(Binormal.IsValid())
		, bNeedsTangent(Tangent.IsValid())
	{
	}

	FNDIOutputParam<FVector3f> Position;
	FNDIOutputParam<FVector3f> Velocity;
	FNDIOutputParam<FVector3f> Normal;
	FNDIOutputParam<FVector3f> Binormal;
	FNDIOutputParam<FVector3f> Tangent;

	const bool bNeedsPosition;
	const bool bNeedsVelocity;
	const bool bNeedsNorm;
	const bool bNeedsBinorm;
	const bool bNeedsTangent;
};

void UNiagaraDataInterfaceSkeletalMesh::GetTriangleData(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstanceData(Context);
	FNDIInputParam<int32> TriParam(Context);
	FNDIInputParam<FVector3f> BaryParam(Context);

	FNDIOutputParam<FVector3f> OutPositionParam(Context);
	FNDIOutputParam<FVector3f> OutTangentZParam(Context);
	FNDIOutputParam<FVector3f> OutTangentYParam(Context);
	FNDIOutputParam<FVector3f> OutTangentXParam(Context);

	FSkeletalMeshAccessorHelper Accessor;
	Accessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstanceData);

	// Is the data valid?
	if (!InstanceData->bAllowCPUMeshDataAccess || !Accessor.IsLODAccessible())
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutPositionParam.SetAndAdvance(FVector3f::ZeroVector);
			OutTangentZParam.SetAndAdvance(FVector3f::ZAxisVector);
			OutTangentYParam.SetAndAdvance(FVector3f::YAxisVector);
			OutTangentXParam.SetAndAdvance(FVector3f::XAxisVector);
		}
		return;
	}

	// Data should be considered valid here
	const FSkeletalMeshLODRenderData* LODData = Accessor.LODData;
	const int32 TriMax = (Accessor.IndexBuffer->Num() / 3) - 1;
	FSkinnedPositionAccessorHelper<TNDISkelMesh_SkinningModeNone> SkinningHandler;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int Triangle = FMath::Clamp(TriParam.GetAndAdvance(), 0, TriMax);
		const FVector3f BaryCoord = BaryParam.GetAndAdvance();

		int32 Indices[3];
		SkinningHandler.GetTriangleIndices(Accessor, Triangle, Indices[0], Indices[1], Indices[2]);

		FVector3f Positions[3];
		SkinningHandler.GetSkinnedTrianglePositions(Accessor, Indices[0], Indices[1], Indices[2], Positions[0], Positions[1], Positions[2]);

		FVector3f TangentsX[3];
		FVector3f TangentsY[3];
		FVector3f TangentsZ[3];
		SkinningHandler.GetSkinnedTangentBasis(Accessor, Indices[0], TangentsX[0], TangentsY[0], TangentsZ[0]);
		SkinningHandler.GetSkinnedTangentBasis(Accessor, Indices[1], TangentsX[1], TangentsY[1], TangentsZ[1]);
		SkinningHandler.GetSkinnedTangentBasis(Accessor, Indices[2], TangentsX[2], TangentsY[2], TangentsZ[2]);

		OutPositionParam.SetAndAdvance(BarycentricInterpolate(BaryCoord, Positions[0], Positions[1], Positions[2]));
		OutTangentZParam.SetAndAdvance(BarycentricInterpolate(BaryCoord, TangentsX[0], TangentsX[1], TangentsX[2]));
		OutTangentYParam.SetAndAdvance(BarycentricInterpolate(BaryCoord, TangentsY[0], TangentsY[1], TangentsY[2]));
		OutTangentXParam.SetAndAdvance(BarycentricInterpolate(BaryCoord, TangentsZ[0], TangentsZ[1], TangentsZ[2]));
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetTriangleIndices(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstanceData(Context);
	FNDIInputParam<int32> TriParam(Context);
	FNDIOutputParam<int32> OutIndex0Param(Context);
	FNDIOutputParam<int32> OutIndex1Param(Context);
	FNDIOutputParam<int32> OutIndex2Param(Context);

	FSkeletalMeshAccessorHelper Accessor;
	Accessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstanceData);

	// Is the data valid?
	if (!InstanceData->bAllowCPUMeshDataAccess || !Accessor.IsLODAccessible())
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutIndex0Param.SetAndAdvance(0);
			OutIndex1Param.SetAndAdvance(0);
			OutIndex2Param.SetAndAdvance(0);
		}
		return;
	}

	// Data should be considered valid here
	const FSkeletalMeshLODRenderData* LODData = Accessor.LODData;
	const int32 TriMax = (Accessor.IndexBuffer->Num() / 3) - 1;
	FSkinnedPositionAccessorHelper<TNDISkelMesh_SkinningModeNone> SkinningHandler;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int Triangle = FMath::Clamp(TriParam.GetAndAdvance(), 0, TriMax);

		int32 Indices[3];
		SkinningHandler.GetTriangleIndices(Accessor, Triangle, Indices[0], Indices[1], Indices[2]);

		OutIndex0Param.SetAndAdvance(Indices[0]);
		OutIndex1Param.SetAndAdvance(Indices[1]);
		OutIndex2Param.SetAndAdvance(Indices[2]);
	}
}

template<typename SkinningHandlerType, typename TransformHandlerType, typename VertexAccessorType, typename bInterpolated>
void UNiagaraDataInterfaceSkeletalMesh::GetTriCoordSkinnedData(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);

	SkinningHandlerType SkinningHandler;
	TransformHandlerType TransformHandler;
	FNDIInputParam<int32> TriParam(Context);
	FNDIInputParam<FVector3f> BaryParam(Context);
	VectorVM::FExternalFuncInputHandler<float> InterpParam;

 	if(bInterpolated::Value)
 	{
 		InterpParam.Init(Context);
 	}	

	checkf(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());
	checkf(InstData->bMeshValid, TEXT("Skeletal Mesh Interface has invalid mesh. %s"), *GetPathName());

	//TODO: Replace this by storing off FTransforms and doing a proper lerp to get a final transform.
	//Also need to pull in a per particle interpolation factor.
	const FMatrix44f Transform(InstData->Transform);				// LWC_TODO: Precision loss
	const FMatrix44f PrevTransform(InstData->PrevTransform);

	FGetTriCoordSkinnedDataOutputHandler Output(Context);

	FSkeletalMeshAccessorHelper Accessor;
	Accessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstData);

	const bool bNeedsCurr = bInterpolated::Value || Output.bNeedsPosition || Output.bNeedsVelocity || Output.bNeedsNorm || Output.bNeedsBinorm || Output.bNeedsTangent;
	const bool bNeedsPrev = bInterpolated::Value || Output.bNeedsVelocity;
	const bool bNeedsTangentBasis = Output.bNeedsNorm || Output.bNeedsBinorm || Output.bNeedsTangent;

	const float InvDt = 1.0f / InstData->DeltaSeconds;

	// If LOD isn't accessible push out failure data
	if ( !Accessor.IsLODAccessible() )
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const float Interp = bInterpolated::Value ? InterpParam.GetAndAdvance() : 1.0f;
			FVector3f PrevPosition = FVector3f::ZeroVector;
			if (bNeedsPrev)
			{
				TransformHandler.TransformPosition(PrevPosition, PrevTransform);
			}
			FVector3f CurrPosition =  FVector3f::ZeroVector;
			if (Output.bNeedsPosition || Output.bNeedsVelocity)
			{
				TransformHandler.TransformPosition(CurrPosition, Transform);
			}
			const FVector3f LerpPosition = bInterpolated::Value ? FMath::Lerp(PrevPosition, CurrPosition, Interp) : CurrPosition;

			Output.Position.SetAndAdvance(LerpPosition);
			Output.Velocity.SetAndAdvance((LerpPosition - PrevPosition) * InvDt);
			Output.Normal.SetAndAdvance(FVector3f::ZAxisVector);
			Output.Binormal.SetAndAdvance(FVector3f::YAxisVector);
			Output.Tangent.SetAndAdvance(FVector3f::XAxisVector);
		}
		// Handled missing / failed data
		return;
	}

	const FSkeletalMeshLODRenderData* LODData = Accessor.LODData;
	const int32 TriMax = (Accessor.IndexBuffer->Num() / 3) - 1;

	FVector3f Pos0 = FVector3f::ZeroVector;		FVector3f Pos1 = FVector3f::ZeroVector;		FVector3f Pos2 = FVector3f::ZeroVector;
	FVector3f Prev0;	FVector3f Prev1;	FVector3f Prev2;
	int32 Idx0; int32 Idx1; int32 Idx2;
	FVector3f Pos;
	FVector3f Prev;
	FVector3f Velocity;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		FMeshTriCoordinate MeshTriCoord(TriParam.GetAndAdvance(), BaryParam.GetAndAdvance());

		float Interp = 1.0f;
		if (bInterpolated::Value)
		{
			Interp = InterpParam.GetAndAdvance();
		}

		if(MeshTriCoord.Tri < 0 || MeshTriCoord.Tri > TriMax)
		{
			MeshTriCoord = FMeshTriCoordinate(0, FVector3f(1.0f, 0.0f, 0.0f));
		}

		SkinningHandler.GetTriangleIndices(Accessor, MeshTriCoord.Tri, Idx0, Idx1, Idx2);

		if (bNeedsCurr)
		{
			SkinningHandler.GetSkinnedTrianglePositions(Accessor, Idx0, Idx1, Idx2, Pos0, Pos1, Pos2);
		}

		if (bNeedsPrev)
		{
			SkinningHandler.GetSkinnedTrianglePreviousPositions(Accessor, Idx0, Idx1, Idx2, Prev0, Prev1, Prev2);
			Prev = BarycentricInterpolate(MeshTriCoord.BaryCoord, Prev0, Prev1, Prev2);
			TransformHandler.TransformPosition(Prev, PrevTransform);
		}

		if (Output.bNeedsPosition || Output.bNeedsVelocity)
		{
			Pos = BarycentricInterpolate(MeshTriCoord.BaryCoord, Pos0, Pos1, Pos2);
			TransformHandler.TransformPosition(Pos, Transform);
			
			if (bInterpolated::Value)
			{
				Pos = FMath::Lerp(Prev, Pos, Interp);
			}

			Output.Position.SetAndAdvance(Pos);
		}

		if (Output.bNeedsVelocity)
		{
			Velocity = (Pos - Prev) * InvDt;

			//No need to handle velocity wrt interpolation as it's based on the prev position anyway
			Output.Velocity.SetAndAdvance(Velocity);
		}

		// Do we need the tangent basis?
		if (bNeedsTangentBasis)
		{
			FVector3f VertexTangentX[3];
			FVector3f VertexTangentY[3];
			FVector3f VertexTangentZ[3];
			SkinningHandler.GetSkinnedTangentBasis(Accessor, Idx0, VertexTangentX[0], VertexTangentY[0], VertexTangentZ[0]);
			SkinningHandler.GetSkinnedTangentBasis(Accessor, Idx1, VertexTangentX[1], VertexTangentY[1], VertexTangentZ[1]);
			SkinningHandler.GetSkinnedTangentBasis(Accessor, Idx2, VertexTangentX[2], VertexTangentY[2], VertexTangentZ[2]);

			FVector3f TangentX = BarycentricInterpolate(MeshTriCoord.BaryCoord, VertexTangentX[0], VertexTangentX[1], VertexTangentX[2]);
			FVector3f TangentY = BarycentricInterpolate(MeshTriCoord.BaryCoord, VertexTangentY[0], VertexTangentY[1], VertexTangentY[2]);
			FVector3f TangentZ = BarycentricInterpolate(MeshTriCoord.BaryCoord, VertexTangentZ[0], VertexTangentZ[1], VertexTangentZ[2]);

			if (bInterpolated::Value)
			{
				FVector3f PrevVertexTangentX[3];
				FVector3f PrevVertexTangentY[3];
				FVector3f PrevVertexTangentZ[3];
				SkinningHandler.GetSkinnedPreviousTangentBasis(Accessor, Idx0, PrevVertexTangentX[0], PrevVertexTangentY[0], PrevVertexTangentZ[0]);
				SkinningHandler.GetSkinnedPreviousTangentBasis(Accessor, Idx1, PrevVertexTangentX[1], PrevVertexTangentY[1], PrevVertexTangentZ[1]);
				SkinningHandler.GetSkinnedPreviousTangentBasis(Accessor, Idx2, PrevVertexTangentX[2], PrevVertexTangentY[2], PrevVertexTangentZ[2]);

				FVector3f PrevTangentX = BarycentricInterpolate(MeshTriCoord.BaryCoord, PrevVertexTangentX[0], PrevVertexTangentX[1], PrevVertexTangentX[2]);
				FVector3f PrevTangentY = BarycentricInterpolate(MeshTriCoord.BaryCoord, PrevVertexTangentY[0], PrevVertexTangentY[1], PrevVertexTangentY[2]);
				FVector3f PrevTangentZ = BarycentricInterpolate(MeshTriCoord.BaryCoord, PrevVertexTangentZ[0], PrevVertexTangentZ[1], PrevVertexTangentZ[2]);

				TangentX = FMath::Lerp(PrevTangentX, TangentX, Interp);
				TangentY = FMath::Lerp(PrevTangentY, TangentY, Interp);
				TangentZ = FMath::Lerp(PrevTangentZ, TangentZ, Interp);
			}

			if (Output.bNeedsNorm)
			{
				TransformHandler.TransformUnitVector(TangentZ, Transform);
				Output.Normal.SetAndAdvance(TangentZ.GetSafeNormal());
			}

			if (Output.bNeedsBinorm)
			{
				TransformHandler.TransformUnitVector(TangentY, Transform);
				Output.Binormal.SetAndAdvance(TangentY.GetSafeNormal());
			}

			if (Output.bNeedsTangent)
			{
				TransformHandler.TransformUnitVector(TangentX, Transform);
				Output.Tangent.SetAndAdvance(TangentX.GetSafeNormal());
			}
		}
	}
}

// Fallback sampling function for no valid mesh on the interface
template<typename TransformHandlerType, typename bInterpolated>
void UNiagaraDataInterfaceSkeletalMesh::GetTriCoordSkinnedDataFallback(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);
	TransformHandlerType TransformHandler;

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);	
	FNDIInputParam<int32> TriParam(Context);
	FNDIInputParam<FVector3f> BaryParam(Context);
	VectorVM::FExternalFuncInputHandler<float> InterpParam;

	if (bInterpolated::Value)
	{
		InterpParam.Init(Context);
	}

	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());

	//TODO: Replace this by storing off FTransforms and doing a proper lerp to get a final transform.
	//Also need to pull in a per particle interpolation factor.
	const FMatrix& Transform = InstData->Transform;
	const FMatrix& PrevTransform = InstData->PrevTransform;

	FGetTriCoordSkinnedDataOutputHandler Output(Context);
	bool bNeedsPrev = bInterpolated::Value || Output.bNeedsVelocity;

	const float InvDt = 1.0f / InstData->DeltaSeconds;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		float Interp = 1.0f;
		if (bInterpolated::Value)
		{
			Interp = InterpParam.GetAndAdvance();
		}

		FVector3f Prev(0.0f);
		FVector3f Pos(0.0f);
		if (bNeedsPrev)
		{
			TransformHandler.TransformPosition(Prev, FMatrix44f(PrevTransform));		// LWC_TODO: Precision loss
		}

		if (Output.bNeedsPosition || Output.bNeedsVelocity)
		{
			TransformHandler.TransformPosition(Pos, FMatrix44f(Transform));		// LWC_TODO: Precision loss

			if (bInterpolated::Value)
			{
				Pos = FMath::Lerp(Prev, Pos, Interp);
			}

			Output.Position.SetAndAdvance(Pos);
		}

		if (Output.bNeedsVelocity)
		{			
			FVector3f Velocity = (Pos - Prev) * InvDt;
			Output.Velocity.SetAndAdvance(Velocity);
		}

		if (Output.bNeedsNorm)
		{
			Output.Normal.SetAndAdvance(FVector3f(0.0f, 0.0f, 1.0f));
		}

		if (Output.bNeedsBinorm)
		{
			Output.Binormal.SetAndAdvance(FVector3f(0.0f, 1.0f, 0.0f));
		}
		
		if (Output.bNeedsTangent)
		{
			Output.Tangent.SetAndAdvance(FVector3f(1.0f, 0.0f, 0.0f));
		}		
	}
}

template<typename SkinningHandlerType, typename TransformHandlerType, typename VertexAccessorType, typename bInterpolated>
void UNiagaraDataInterfaceSkeletalMesh::VMGetSkinnedTriangleVertexData(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> TriangleParam(Context);
	FNDIInputParam<float> InterpParam;
	if constexpr (bInterpolated::Value)
	{
		InterpParam.Init(Context);
	}

	struct FVertexOutput
	{
		explicit FVertexOutput(FVectorVMExternalFunctionContext& Context)
			: OutPosition(Context)
			, OutVelocity(Context)
			, OutNormal(Context)
			, OutBinormal(Context)
			, OutTangent(Context)
		{
		}

		bool WritesPosition() const { return OutPosition.IsValid(); }
		bool WritesVelocity() const { return OutVelocity.IsValid(); }
		bool WritesTangents() const { return OutNormal.IsValid() || OutBinormal.IsValid() || OutTangent.IsValid(); }

		FNDIOutputParam<FVector3f>	OutPosition;
		FNDIOutputParam<FVector3f>	OutVelocity;
		FNDIOutputParam<FVector3f>	OutNormal;
		FNDIOutputParam<FVector3f>	OutBinormal;
		FNDIOutputParam<FVector3f>	OutTangent;
	};
	FVertexOutput VertexOutput[] = {FVertexOutput(Context), FVertexOutput(Context), FVertexOutput(Context)};

	const bool bWritesCurrPos = VertexOutput[0].WritesPosition() || VertexOutput[1].WritesPosition() || VertexOutput[2].WritesPosition();
	const bool bWritesVelocity = VertexOutput[0].WritesVelocity() || VertexOutput[1].WritesVelocity() || VertexOutput[2].WritesVelocity();
	const bool bWritesTangentBasis = VertexOutput[0].WritesTangents() || VertexOutput[1].WritesTangents() || VertexOutput[2].WritesTangents();
	const bool bNeedsPrevPos = bInterpolated::Value || bWritesVelocity;
	const float InvDt = InstData->DeltaSeconds > 0.0f ? 1.0f / InstData->DeltaSeconds : 0.0f;

	SkinningHandlerType SkinningHandler;
	TransformHandlerType TransformHandler;
	const FMatrix44f Transform(InstData->Transform);			// LWC_TODO: Precision loss
	const FMatrix44f PrevTransform(InstData->PrevTransform);

	FSkeletalMeshAccessorHelper Accessor;
	Accessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstData);

	// Invalid of missing data path
	if (!Accessor.IsLODAccessible())
	{
		for (int32 i=0; i < Context.GetNumInstances(); ++i)
		{
			const int32 Triangle = TriangleParam.GetAndAdvance();
			const float Interp = bInterpolated::Value ? InterpParam.GetAndAdvance() : 1.0f;

			for (int32 Vertex=0; Vertex < 3; ++Vertex)
			{
				FVector3f CurrPosition = FVector3f::ZeroVector;
				if (bWritesCurrPos || bWritesVelocity)
				{
					TransformHandler.TransformPosition(CurrPosition, Transform);
				}

				FVector3f PrevPosition = FVector3f::ZeroVector;
				if (bNeedsPrevPos)
				{
					TransformHandler.TransformPosition(PrevPosition, PrevTransform);
				}

				if (bWritesCurrPos)
				{
					if constexpr (bInterpolated::Value)
					{
						VertexOutput[Vertex].OutPosition.SetAndAdvance(FMath::Lerp(PrevPosition, CurrPosition, Interp));
					}
					else
					{
						VertexOutput[Vertex].OutPosition.SetAndAdvance(CurrPosition);
					}
				}
				if (bWritesVelocity)
				{
					VertexOutput[Vertex].OutVelocity.SetAndAdvance((CurrPosition - PrevPosition)* InvDt);
				}
				if (bWritesTangentBasis)
				{
					VertexOutput[Vertex].OutNormal.SetAndAdvance(FVector3f::ZAxisVector);
					VertexOutput[Vertex].OutBinormal.SetAndAdvance(FVector3f::YAxisVector);
					VertexOutput[Vertex].OutTangent.SetAndAdvance(FVector3f::XAxisVector);
				}
			}
		}
		return;
	}

	// Do the sampling
	for (int32 i=0; i < Context.GetNumInstances(); ++i)
	{
		const int32 Triangle = TriangleParam.GetAndAdvance();
		const float Interp = bInterpolated::Value ? InterpParam.GetAndAdvance() : 1.0f;

		int32 Indices[3];
		SkinningHandler.GetTriangleIndices(Accessor, Triangle, Indices[0], Indices[1], Indices[2]);

		for (int32 Index=0; Index < 3; ++Index)
		{
			const int32 Vertex = Indices[Index];

			FVector3f CurrPosition = FVector3f::ZeroVector;
			if (bWritesCurrPos || bWritesVelocity)
			{
				CurrPosition = SkinningHandler.GetSkinnedVertexPosition(Accessor, Vertex);
				TransformHandler.TransformPosition(CurrPosition, Transform);
			}

			FVector3f PrevPosition = FVector3f::ZeroVector;
			if (bNeedsPrevPos)
			{
				PrevPosition = SkinningHandler.GetSkinnedVertexPreviousPosition(Accessor, Vertex);
				TransformHandler.TransformPosition(PrevPosition, PrevTransform);
			}

			if (bWritesCurrPos)
			{
				const FVector3f InterpPosition = bInterpolated::Value ? FMath::Lerp(PrevPosition, CurrPosition, Interp) : CurrPosition;
				VertexOutput[Index].OutPosition.SetAndAdvance(InterpPosition);
			}
			if (bWritesVelocity)
			{
				VertexOutput[Index].OutVelocity.SetAndAdvance((CurrPosition - PrevPosition)* InvDt);
			}
			if (bWritesTangentBasis)
			{
				FVector3f TangentX = FVector3f::ZeroVector;
				FVector3f TangentY = FVector3f::ZeroVector;
				FVector3f TangentZ = FVector3f::ZeroVector;
				SkinningHandler.GetSkinnedTangentBasis(Accessor, Vertex, TangentX, TangentY, TangentZ);

				TransformHandler.TransformUnitVector(TangentX, Transform);
				TransformHandler.TransformUnitVector(TangentY, Transform);
				TransformHandler.TransformUnitVector(TangentZ, Transform);
				VertexOutput[Index].OutNormal.SetAndAdvance(TangentX);
				VertexOutput[Index].OutBinormal.SetAndAdvance(TangentY);
				VertexOutput[Index].OutTangent.SetAndAdvance(TangentZ);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
