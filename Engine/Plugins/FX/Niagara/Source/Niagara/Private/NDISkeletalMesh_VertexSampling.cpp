// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "SkeletalMeshTypes.h"
#include "NiagaraStats.h"
#include "Templates/AlignmentTemplates.h"
#include "NDISkeletalMeshCommon.h"

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceSkeletalMesh_VertexSampling"

DECLARE_CYCLE_STAT(TEXT("Skel Mesh Vertex Sampling"), STAT_NiagaraSkel_Vertex_Sample, STATGROUP_Niagara);

//Final binders for all static mesh interface functions.
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexData);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexSkinnedData);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexColor);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexColorFallback);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexUV);

DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, IsValidFilteredVertex);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, RandomFilteredVertex);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredVertexCount);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredVertexAt);

const FName FSkeletalMeshInterfaceHelper::GetVertexDataName("GetVertexData");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataName("GetSkinnedVertexData");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataWSName("GetSkinnedVertexDataWS");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataInterpolatedName("GetSkinnedVertexDataInterpolated");
const FName FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataInterpolatedWSName("GetSkinnedVertexDataInterpolatedWS");
const FName FSkeletalMeshInterfaceHelper::GetVertexColorName("GetVertexColor");
const FName FSkeletalMeshInterfaceHelper::GetVertexUVName("GetVertexUV");

const FName FSkeletalMeshInterfaceHelper::IsValidVertexName("IsValidVertex");
const FName FSkeletalMeshInterfaceHelper::RandomVertexName("RandomVertex");
const FName FSkeletalMeshInterfaceHelper::GetVertexCountName("GetVertexCount");

const FName FSkeletalMeshInterfaceHelper::IsValidFilteredVertexName("IsValidFilteredVertex");
const FName FSkeletalMeshInterfaceHelper::RandomFilteredVertexName("RandomFilteredVertex");
const FName FSkeletalMeshInterfaceHelper::GetFilteredVertexCountName("GetFilteredVertexCount");
const FName FSkeletalMeshInterfaceHelper::GetFilteredVertexAtName("GetFilteredVertex");

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSkeletalMesh::GetVertexSamplingFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::GetVertexDataName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetVertexDataDesc", "Returns bind pose for the vertex");
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetSkinnedDataDesc", "Returns skinning dependant data for the pased vertex in local space. All outputs are optional and you will incur zero to minimal cost if they are not connected.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataWSName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetSkinnedDataWSDesc", "Returns skinning dependant data for the pased vertex in world space. All outputs are optional and you will incur zero to minimal cost if they are not connected.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataInterpolatedName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interp")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetSkinnedDataDesc", "Returns skinning dependant data for the pased vertex in local space. All outputs are optional and you will incur zero to minimal cost if they are not connected.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataInterpolatedWSName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interp")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = LOCTEXT("GetSkinnedDataWSDesc", "Returns skinning dependant data for the pased vertex in world space. All outputs are optional and you will incur zero to minimal cost if they are not connected.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::GetVertexColorName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::GetVertexUVName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("UV Set")));

		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::IsValidVertexName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::RandomVertexName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::GetVertexCountName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::IsValidFilteredVertexName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::RandomFilteredVertexName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::GetFilteredVertexCountName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FSkeletalMeshInterfaceHelper::GetFilteredVertexAtName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Filtered Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}
}
#endif

void UNiagaraDataInterfaceSkeletalMesh::BindVertexSamplingFunction(const FVMExternalFunctionBindingInfo& BindingInfo, FNDISkeletalMesh_InstanceData* InstanceData, FVMExternalFunction &OutFunc)
{
	using TInterpOff = TIntegralConstant<bool, false>;
	using TInterpOn = TIntegralConstant<bool, true>;

	//////////////////////////////////////////////////////////////////////////
	if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetVertexDataName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 12);
		TVertexAccessorBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexData)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 15);
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TNDIExplicitBinder<TInterpOff, TVertexAccessorBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexSkinnedData)>>>>::BindCheckCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataWSName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 15);
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandler, TNDIExplicitBinder<TInterpOff, TVertexAccessorBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexSkinnedData)>>>>::BindCheckCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataInterpolatedName)
	{
		check(BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 15);
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TNDIExplicitBinder<TInterpOn, TVertexAccessorBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexSkinnedData)>>>>::BindCheckCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataInterpolatedWSName)
	{
		check(BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 15);
		TSkinningModeBinder<TNDIExplicitBinder<FNDITransformHandler, TNDIExplicitBinder<TInterpOn, TVertexAccessorBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexSkinnedData)>>>>::BindCheckCPUAccess(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetVertexColorName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 4);
		if (InstanceData->HasColorData())
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexColor)::Bind(this, OutFunc);
		}
		else
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexColorFallback)::Bind(this, OutFunc);
		}
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetVertexUVName)
	{
		check(BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 2);
		TVertexAccessorBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetVertexUV)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	//////////////////////////////////////////////////////////////////////////
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::IsValidVertexName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::IsValidVertex);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::RandomVertexName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::RandomVertex);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetVertexCountName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::GetVertexCount);
	}
	//////////////////////////////////////////////////////////////////////////
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::IsValidFilteredVertexName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		TFilterModeBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, IsValidFilteredVertex)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::RandomFilteredVertexName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		TFilterModeBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, RandomFilteredVertex)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetFilteredVertexCountName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		TFilterModeBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredVertexCount)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == FSkeletalMeshInterfaceHelper::GetFilteredVertexAtName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		TFilterModeBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSkeletalMesh, GetFilteredVertexAt)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraDataInterfaceSkeletalMesh::IsValidVertex(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Vertex_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> VertexParam(Context);
	FNDIOutputParam<FNiagaraBool> OutValid(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstData);

	const int32 MaxVertex = MeshAccessor.IsLODAccessible() ? MeshAccessor.LODData->GetNumVertices() : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 VertexIndex = VertexParam.GetAndAdvance();
		OutValid.SetAndAdvance(VertexIndex < MaxVertex);
	}
}

void UNiagaraDataInterfaceSkeletalMesh::RandomVertex(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Vertex_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIRandomHelper RandHelper(Context);
	FNDIOutputParam<int32> OutVertex(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstData);

	const int32 MaxVertex = MeshAccessor.IsLODAccessible() ? MeshAccessor.LODData->GetNumVertices() - 1 : -1;
	if (MaxVertex >= 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutVertex.SetAndAdvance(RandHelper.RandRange(i, 0, MaxVertex));
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutVertex.SetAndAdvance(-1);
		}
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetVertexCount(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Vertex_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIOutputParam<int32> OutVertexCount(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstData);

	const int32 MaxVertex = MeshAccessor.IsLODAccessible() ? MeshAccessor.LODData->GetNumVertices() : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutVertexCount.SetAndAdvance(MaxVertex);
	}
}

//////////////////////////////////////////////////////////////////////////
template<typename FilterMode>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::RandomFilteredVertIndex(FNDIRandomHelper& RandHelper, int32 Instance, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	checkf(false, TEXT("Invalid template call for RandomVertIndex. Bug in Filter binding or Area Weighting binding. Contact code team."));
	return 0;
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::RandomFilteredVertIndex<TIntegralConstant<ENDISkeletalMesh_FilterMode, ENDISkeletalMesh_FilterMode::None>>
	(FNDIRandomHelper& RandHelper, int32 Instance, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	return RandHelper.RandRange(Instance, 0, Accessor.LODData->GetNumVertices() - 1);
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::RandomFilteredVertIndex<TIntegralConstant<ENDISkeletalMesh_FilterMode, ENDISkeletalMesh_FilterMode::SingleRegion>>
	(FNDIRandomHelper& RandHelper, int32 Instance, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	int32 Idx = RandHelper.RandRange(Instance, 0, Accessor.SamplingRegionBuiltData->Vertices.Num() - 1);
	return Accessor.SamplingRegionBuiltData->Vertices[Idx];
}

template<>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::RandomFilteredVertIndex<TIntegralConstant<ENDISkeletalMesh_FilterMode, ENDISkeletalMesh_FilterMode::MultiRegion>>
	(FNDIRandomHelper& RandHelper, int32 Instance, FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	USkeletalMesh* SkelMesh = Accessor.Mesh;
	check(SkelMesh);

	int32 RegionIdx = RandHelper.RandRange(Instance, 0, InstData->SamplingRegionIndices.Num() - 1);
	const FSkeletalMeshSamplingInfo& SamplingInfo = SkelMesh->GetSamplingInfo();
	const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(InstData->SamplingRegionIndices[RegionIdx]);
	const FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = SamplingInfo.GetRegionBuiltData(InstData->SamplingRegionIndices[RegionIdx]);
	int32 Idx = RandHelper.RandRange(Instance, 0, RegionBuiltData.Vertices.Num() - 1);
	return RegionBuiltData.Vertices[Idx];
}

template<typename FilterMode>
void UNiagaraDataInterfaceSkeletalMesh::RandomFilteredVertex(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Vertex_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIRandomHelper RandHelper(Context);
	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());	

	FNDIOutputParam<int32> OutVert(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<FilterMode, TIntegralConstant<ENDISkelMesh_AreaWeightingMode, ENDISkelMesh_AreaWeightingMode::None>>(InstData);

	if (MeshAccessor.IsLODAccessible())
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			RandHelper.GetAndAdvance();
			OutVert.SetAndAdvance(RandomFilteredVertIndex<FilterMode>(RandHelper, i, MeshAccessor, InstData));
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutVert.SetAndAdvance(-1);
		}
	}
}

template<typename FilterMode>
void UNiagaraDataInterfaceSkeletalMesh::IsValidFilteredVertex(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Vertex_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> VertexParam(Context);

	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());

	FNDIOutputParam<FNiagaraBool> OutValid(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<FilterMode, TNDISkelMesh_AreaWeightingOff>(InstData);

	const int32 MaxVertex = MeshAccessor.IsLODAccessible() ? MeshAccessor.LODData->GetNumVertices() : 0;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 RequestedIndex = VertexParam.GetAndAdvance();
		OutValid.SetAndAdvance(RequestedIndex < MaxVertex);
	}
}

//////////////////////////////////////////////////////////////////////////

template<typename FilterMode>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredVertexCount(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	checkf(false, TEXT("Invalid template call for GetFilteredVertexCount. Bug in Filter binding or Area Weighting binding. Contact code team."));
	return 0;
}

template<>
FORCEINLINE_DEBUGGABLE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredVertexCount<TNDISkelMesh_FilterModeNone>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	return  Accessor.LODData->GetNumVertices();;
}

template<>
FORCEINLINE_DEBUGGABLE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredVertexCount<TNDISkelMesh_FilterModeSingle>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	return Accessor.SamplingRegionBuiltData->Vertices.Num();
}

template<>
FORCEINLINE_DEBUGGABLE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredVertexCount<TNDISkelMesh_FilterModeMulti>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData)
{
	USkeletalMesh* SkelMesh = Accessor.Mesh;
	check(SkelMesh);

	int32 NumVerts = 0;
	for (int32 RegionIdx = 0; RegionIdx < InstData->SamplingRegionIndices.Num(); RegionIdx++)
	{
		const FSkeletalMeshSamplingInfo& SamplingInfo = SkelMesh->GetSamplingInfo();
		const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(InstData->SamplingRegionIndices[RegionIdx]);
		const FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = SamplingInfo.GetRegionBuiltData(InstData->SamplingRegionIndices[RegionIdx]);
		NumVerts += RegionBuiltData.Vertices.Num();
	}
	return NumVerts;
}

template<typename FilterMode>
void UNiagaraDataInterfaceSkeletalMesh::GetFilteredVertexCount(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Vertex_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());

	FNDIOutputParam<int32> OutVert(Context);

	FSkeletalMeshAccessorHelper MeshAccessor;
	MeshAccessor.Init<FilterMode, TNDISkelMesh_AreaWeightingOff>(InstData);

	const int32 Count = MeshAccessor.IsLODAccessible() ? GetFilteredVertexCount<FilterMode>(MeshAccessor, InstData) : 0;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutVert.SetAndAdvance(Count);
	}
}

//////////////////////////////////////////////////////////////////////////

template<typename FilterMode>
FORCEINLINE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredVertexAt(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIndex)
{
	checkf(false, TEXT("Invalid template call for GetFilteredVertexAt. Bug in Filter binding. Contact code team."));
	return 0;
}

template<>
FORCEINLINE_DEBUGGABLE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredVertexAt<TNDISkelMesh_FilterModeNone>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIndex)
{
	return FilteredIndex;
}

template<>
FORCEINLINE_DEBUGGABLE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredVertexAt<TNDISkelMesh_FilterModeSingle>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIndex)
{
	int32 MaxIdx = Accessor.SamplingRegionBuiltData->Vertices.Num() - 1;
	FilteredIndex = FMath::Min(FilteredIndex, MaxIdx);
	return Accessor.SamplingRegionBuiltData->Vertices[FilteredIndex];
}

template<>
FORCEINLINE_DEBUGGABLE int32 UNiagaraDataInterfaceSkeletalMesh::GetFilteredVertexAt<TNDISkelMesh_FilterModeMulti>
	(FSkeletalMeshAccessorHelper& Accessor, FNDISkeletalMesh_InstanceData* InstData, int32 FilteredIndex)
{
	USkeletalMesh* SkelMesh = Accessor.Mesh;
	check(SkelMesh);

	for (int32 RegionIdx = 0; RegionIdx < InstData->SamplingRegionIndices.Num(); RegionIdx++)
	{
		const FSkeletalMeshSamplingInfo& SamplingInfo = SkelMesh->GetSamplingInfo();
		const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(InstData->SamplingRegionIndices[RegionIdx]);
		const FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = SamplingInfo.GetRegionBuiltData(InstData->SamplingRegionIndices[RegionIdx]);
		if (FilteredIndex < RegionBuiltData.Vertices.Num())
		{
			return RegionBuiltData.Vertices[FilteredIndex];
		}

		FilteredIndex -= RegionBuiltData.Vertices.Num();
	}
	return 0;
}

template<typename FilterMode>
void UNiagaraDataInterfaceSkeletalMesh::GetFilteredVertexAt(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Vertex_Sample);

	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> FilteredVertexParam(Context);
	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());

	FNDIOutputParam<int32> OutVert(Context);

	FSkeletalMeshAccessorHelper Accessor;
	Accessor.Init<FilterMode, TNDISkelMesh_AreaWeightingOff>(InstData);

	if (Accessor.IsLODAccessible())
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const int32 FilteredVert = FilteredVertexParam.GetAndAdvance();
			const int32 RealIdx = GetFilteredVertexAt<FilterMode>(Accessor, InstData, FilteredVert);
			OutVert.SetAndAdvance(RealIdx);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutVert.SetAndAdvance(-1);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraDataInterfaceSkeletalMesh::GetVertexColor(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Vertex_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> VertParam(Context);

	FNDIOutputParam<FLinearColor> OutColor(Context);

	if ( const FSkeletalMeshLODRenderData* LODData = InstData->CachedLODData )
	{
		const FColorVertexBuffer& Colors = LODData->StaticVertexBuffers.ColorVertexBuffer;
		checkfSlow(Colors.GetNumVertices() != 0, TEXT("Trying to access vertex colors from mesh without any."));

		const FMultiSizeIndexContainer& Indices = LODData->MultiSizeIndexContainer;
		const FRawStaticIndexBuffer16or32Interface* IndexBuffer = Indices.GetIndexBuffer();
		const int32 VertMax = LODData->GetNumVertices() - 1;
		if ( VertMax >= 0 )
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				int32 Vertex = VertParam.GetAndAdvance();
				Vertex = FMath::Clamp(VertMax, 0, Vertex);

				OutColor.SetAndAdvance(Colors.VertexColor(Vertex).ReinterpretAsLinear());
			}
			// We are done
			return;
		}
	}

	// Fall though for bad data
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutColor.SetAndAdvance(FLinearColor::White);
	}
}

void UNiagaraDataInterfaceSkeletalMesh::GetVertexColorFallback(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> VertParam(Context);

	FNDIOutputParam<FLinearColor> OutColor(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutColor.SetAndAdvance(FLinearColor::White);
	}
}

template<typename VertexAccessorType>
void UNiagaraDataInterfaceSkeletalMesh::GetVertexUV(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Vertex_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	VertexAccessorType VertAccessor;
	FNDIInputParam<int32> VertParam(Context);
	FNDIInputParam<int32> UVSetParam(Context);

	checkf(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());
	checkf(InstData->bMeshValid, TEXT("Skeletal Mesh Interface has invalid mesh. %s"), *GetPathName());

	FNDIOutputParam<FVector2f> OutUV(Context);

	if ( const FSkeletalMeshLODRenderData* LODData = InstData->CachedLODData )
	{
		const FMultiSizeIndexContainer& Indices = LODData->MultiSizeIndexContainer;
		const FRawStaticIndexBuffer16or32Interface* IndexBuffer = Indices.GetIndexBuffer();
		const int32 VertMax = LODData->GetNumVertices() - 1;
		if (VertMax >= 0)
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				const int32 Vert = FMath::Clamp(VertParam.GetAndAdvance(), 0, VertMax);
				const int32 UVSet = UVSetParam.GetAndAdvance();
				const FVector2D UV = VertAccessor.GetVertexUV(LODData, Vert, UVSet);
				OutUV.SetAndAdvance(FVector2f(UV));	// LWC_TODO: Precision loss
			}
			// We are done
			return;
		}
	}

	// Fall though for bad data
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutUV.SetAndAdvance(FVector2f::ZeroVector);
	}
}

// Stub specialization for no valid mesh data on the data interface
template<>
void UNiagaraDataInterfaceSkeletalMesh::GetVertexUV<FSkelMeshVertexAccessorNoop>(FVectorVMExternalFunctionContext& Context)
{
	FNDIInputParam<int32> VertParam(Context);
	FNDIInputParam<int32> UVSetParam(Context);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);

	FNDIOutputParam<FVector2f> OutUV(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutUV.SetAndAdvance(FVector2f::ZeroVector);
	}
}

struct FGetVertexSkinnedDataOutputHandler
{
	FGetVertexSkinnedDataOutputHandler(FVectorVMExternalFunctionContext& Context)
		: Position(Context)
		, Velocity(Context)
		, TangentZ(Context)
		, TangentY(Context)
		, TangentX(Context)
		, bNeedsPosition(Position.IsValid())
		, bNeedsVelocity(Velocity.IsValid())
		, bNeedsTangentX(TangentX.IsValid())
		, bNeedsTangentY(TangentY.IsValid())
		, bNeedsTangentZ(TangentZ.IsValid())
	{
	}
	FNDIOutputParam<FVector3f> Position;
	FNDIOutputParam<FVector3f> Velocity;
	FNDIOutputParam<FVector3f> TangentZ;
	FNDIOutputParam<FVector3f> TangentY;
	FNDIOutputParam<FVector3f> TangentX;

	const bool bNeedsPosition;
	const bool bNeedsVelocity;
	const bool bNeedsTangentX;
	const bool bNeedsTangentY;
	const bool bNeedsTangentZ;
};

template<typename VertexAccessorType>
void UNiagaraDataInterfaceSkeletalMesh::GetVertexData(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Vertex_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIInputParam<int32> VertParam(Context);

	FNDIOutputParam<FVector3f> OutPositionParam(Context);
	FNDIOutputParam<FVector3f> OutTangentZParam(Context);
	FNDIOutputParam<FVector3f> OutTangentYParam(Context);
	FNDIOutputParam<FVector3f> OutTangentXParam(Context);

	FSkeletalMeshAccessorHelper Accessor;
	Accessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstData);
	if (Accessor.IsLODAccessible())
	{
		FSkinnedPositionAccessorHelper<TNDISkelMesh_SkinningModeNone> SkinningHandler;

		const int32 VertMax = Accessor.LODData->GetNumVertices() - 1;
		if (VertMax >= 0)
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				const int32 Vertex = FMath::Clamp(VertParam.GetAndAdvance(), 0, VertMax);

				FVector3f Position = SkinningHandler.GetSkinnedVertexPosition(Accessor, Vertex);
				FVector3f TangentX = FVector3f::ZeroVector;
				FVector3f TangentY = FVector3f::ZeroVector;
				FVector3f TangentZ = FVector3f::ZeroVector;
				SkinningHandler.GetSkinnedTangentBasis(Accessor, Vertex, TangentX, TangentY, TangentZ);

				OutPositionParam.SetAndAdvance(Position);
				OutTangentXParam.SetAndAdvance(TangentX);
				OutTangentYParam.SetAndAdvance(TangentY);
				OutTangentZParam.SetAndAdvance(TangentZ);
			}

			// We are done
			return;
		}
	}

	// Fall though for bad data
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutPositionParam.SetAndAdvance(FVector3f::ZeroVector);
		OutTangentXParam.SetAndAdvance(FVector3f::ZeroVector);
		OutTangentYParam.SetAndAdvance(FVector3f::ZeroVector);
		OutTangentZParam.SetAndAdvance(FVector3f::ZeroVector);
	}
}

template<typename SkinningHandlerType, typename TransformHandlerType, typename Interpolated, typename VertexAccessorType>
void UNiagaraDataInterfaceSkeletalMesh::GetVertexSkinnedData(FVectorVMExternalFunctionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Vertex_Sample);
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);

	SkinningHandlerType SkinningHandler;
	TransformHandlerType TransformHandler;
	FNDIInputParam<int32> VertParam(Context);
	FNDIInputParam<float> InterpParam;

	if constexpr (Interpolated::Value)
	{
		InterpParam.Init(Context);
	}

	checkfSlow(InstData.Get(), TEXT("Skeletal Mesh Interface has invalid instance data. %s"), *GetPathName());

	const FMatrix44f Transform(InstData->Transform);			// LWC_TODO: Precision loss
	const FMatrix44f PrevTransform(InstData->PrevTransform);

	FGetVertexSkinnedDataOutputHandler Output(Context);

	FSkeletalMeshAccessorHelper Accessor;
	Accessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstData);
	if ( Accessor.IsLODAccessible() )
	{
		const int32 VertMax = Accessor.LODData->GetNumVertices() - 1;
		if ( VertMax >= 0 )
		{
			const float InvDt = 1.0f / InstData->DeltaSeconds;
			const bool bNeedsTangentBasis = Output.bNeedsTangentX || Output.bNeedsTangentY || Output.bNeedsTangentZ;

			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				const int32 Vertex = FMath::Clamp(VertParam.GetAndAdvance(), 0, VertMax);
				float Interp = 1.0f;
				if constexpr (Interpolated::Value)
				{
					Interp = InterpParam.GetAndAdvance();
				}

				FVector3f CurrPos = FVector3f::ZeroVector;
				FVector3f PrevPos = FVector3f::ZeroVector;
				if (Output.bNeedsPosition || Output.bNeedsVelocity)
				{
					CurrPos = SkinningHandler.GetSkinnedVertexPosition(Accessor, Vertex);
					TransformHandler.TransformPosition(CurrPos, Transform);
					if (Output.bNeedsVelocity || Interpolated::Value)
					{
						PrevPos = SkinningHandler.GetSkinnedVertexPreviousPosition(Accessor, Vertex);
						TransformHandler.TransformPosition(PrevPos, PrevTransform);
					}

					if constexpr (Interpolated::Value)
					{
						Output.Position.SetAndAdvance(FMath::Lerp(PrevPos, CurrPos, Interp));
					}
					else
					{
						Output.Position.SetAndAdvance(CurrPos);
					}
				}

				if (Output.bNeedsVelocity)
				{
					const FVector3f Velocity = (CurrPos - PrevPos) * InvDt;
					Output.Velocity.SetAndAdvance(Velocity);
				}

				if (bNeedsTangentBasis)
				{
					FVector3f TangentX = FVector3f::ZeroVector;
					FVector3f TangentY = FVector3f::ZeroVector;
					FVector3f TangentZ = FVector3f::ZeroVector;
					SkinningHandler.GetSkinnedTangentBasis(Accessor, Vertex, TangentX, TangentY, TangentZ);

					if (Output.bNeedsTangentX)
					{
						TransformHandler.TransformUnitVector(TangentX, Transform);
						Output.TangentX.SetAndAdvance(TangentX);
					}

					if (Output.bNeedsTangentY)
					{
						TransformHandler.TransformUnitVector(TangentY, Transform);
						Output.TangentY.SetAndAdvance(TangentY);
					}

					if (Output.bNeedsTangentZ)
					{
						TransformHandler.TransformUnitVector(TangentZ, Transform);
						Output.TangentZ.SetAndAdvance(TangentZ);
					}
				}
			}
			// We are done
			return;
		}
	}

	// Fall though for bad data
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		FVector3f Position = FVector3f::ZeroVector;
		if (Output.bNeedsPosition || Output.bNeedsVelocity)
		{
			TransformHandler.TransformPosition(Position, Transform);
		}
		Output.Position.SetAndAdvance(Position);
		Output.Velocity.SetAndAdvance(FVector3f::ZeroVector);
		Output.TangentX.SetAndAdvance(FVector3f::XAxisVector);
		Output.TangentY.SetAndAdvance(FVector3f::YAxisVector);
		Output.TangentZ.SetAndAdvance(FVector3f::ZAxisVector);
	}
}

#undef LOCTEXT_NAMESPACE
