// Copyright Epic Games, Inc. All Rights Reserved.


#include "NiagaraActor.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Texture2D.h"
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "NiagaraComponent.h"
#include "NiagaraSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraActor)

ANiagaraActor::ANiagaraActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NiagaraComponent = CreateDefaultSubobject<UNiagaraComponent>(TEXT("NiagaraComponent0"));

	RootComponent = NiagaraComponent;

#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("ArrowComponent0"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTextureObject;
			FName ID_Effects;
			FText NAME_Effects;
			FConstructorStatics()
				: SpriteTextureObject(TEXT("/Niagara/Icons/S_ParticleSystem"))
				, ID_Effects(TEXT("Effects"))
				, NAME_Effects(NSLOCTEXT("SpriteCategory", "Effects", "Effects"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (SpriteComponent)
		{
			SpriteComponent->Sprite = ConstructorStatics.SpriteTextureObject.Get();
			SpriteComponent->SetRelativeScale3D_Direct(FVector(0.5f, 0.5f, 0.5f));
			SpriteComponent->bHiddenInGame = true;
			SpriteComponent->bIsScreenSizeScaled = true;
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Effects;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Effects;
			SpriteComponent->SetupAttachment(NiagaraComponent);
			SpriteComponent->bReceivesDecals = false;
		}

		if (ArrowComponent)
		{
			ArrowComponent->ArrowColor = FColor(0, 255, 128);

			ArrowComponent->ArrowSize = 1.5f;
			ArrowComponent->bTreatAsASprite = true;
			ArrowComponent->bIsScreenSizeScaled = true;
			ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Effects;
			ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Effects;
			ArrowComponent->SetupAttachment(NiagaraComponent);
			ArrowComponent->SetUsingAbsoluteScale(true);
		#if WITH_EDITOR
			ArrowComponent->SetIgnoreBoundsForEditorFocus(true);
		#endif
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void ANiagaraActor::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	// Set Notification Delegate
	if (NiagaraComponent)
	{
		NiagaraComponent->OnSystemFinished.AddUniqueDynamic(this, &ANiagaraActor::OnNiagaraSystemFinished);
	}
}

void ANiagaraActor::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();

	// Clear Notification Delegate
	if (NiagaraComponent)
	{
		NiagaraComponent->OnSystemFinished.RemoveAll(this);
	}
}

void ANiagaraActor::SetDestroyOnSystemFinish(bool bShouldDestroyOnSystemFinish)
{
	bDestroyOnSystemFinish = bShouldDestroyOnSystemFinish ? 1 : 0;  
};

void ANiagaraActor::OnNiagaraSystemFinished(UNiagaraComponent* FinishedComponent)
{
	if (bDestroyOnSystemFinish)
	{
		SetLifeSpan(0.0001f);
	}
}

#if WITH_EDITOR
bool ANiagaraActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (NiagaraComponent)
	{
		if (UNiagaraSystem* System = NiagaraComponent->GetAsset())
		{
			Objects.Add(System);
		}
	}

	return true;
}

void ANiagaraActor::ResetInLevel()
{
	if (NiagaraComponent)
	{
		NiagaraComponent->Activate(true);
		NiagaraComponent->ReregisterComponent();
	}
}
#endif // WITH_EDITOR

