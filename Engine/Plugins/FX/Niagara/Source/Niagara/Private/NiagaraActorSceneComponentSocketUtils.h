// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraSceneComponentSocketUtils.h"
#include "NiagaraComponent.h"
#include "NiagaraSystemInstance.h"

#include "Components/SceneComponent.h"
#include "Engine/Canvas.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSocket.h"

class FNiagaraActorSceneComponentSocketUtils final : public FNiagaraSceneComponentSocketUtils
{
public:
	explicit FNiagaraActorSceneComponentSocketUtils(TWeakObjectPtr<UObject> InWeakOwnerComponent)
		: FNiagaraSceneComponentSocketUtils(InWeakOwnerComponent)
	{
	}

	virtual FTransform GetLocalToWorldTransform() const override
	{
		USceneComponent* SceneComponent = Cast<USceneComponent>(WeakResolvedObject.Get());
		if (SceneComponent == nullptr)
		{
			SceneComponent = Cast<USceneComponent>(WeakOwnerComponent.Get());
		}
		return SceneComponent ? SceneComponent->GetComponentToWorld() : FTransform::Identity;
	}

	virtual UObject* ResolveAttachParentInternal() const override
	{
		USceneComponent* AttachComponent = Cast<USceneComponent>(WeakOwnerComponent.Get());
		if (AttachComponent && AttachComponent->IsA<UNiagaraComponent>())
		{
			AttachComponent = AttachComponent->GetAttachParent();
		}

		// Look for the first viable component that satisfies the class and tag
		UClass* AttachComponentClass = WeakAttachComponentClass.Get();
		if (AttachComponentClass || !AttachComponentTag.IsNone())
		{
			while (AttachComponent)
			{
				if ((!AttachComponentClass || AttachComponent->IsA(AttachComponentClass)) &&
					(AttachComponentTag.IsNone() || AttachComponent->ComponentHasTag(AttachComponentTag)))
				{
					break;
				}
				AttachComponent = AttachComponent->GetAttachParent();
			}
		}

		return AttachComponent;
	}

	virtual UObject* ResolveSourceInternal() const override
	{
		UObject* ResolvedObject = SourceActor.Get();
		if (ResolvedObject)
		{
			if (AActor* AsActor = Cast<AActor>(ResolvedObject))
			{
				ResolvedObject = AsActor->GetRootComponent();
			}
		}
		if (!ResolvedObject)
		{
			ResolvedObject = SourceAsset.Get();
		}
		return ResolvedObject;
	}

	virtual bool CacheSocketNamesInternal(UObject* ResolvedObject) override
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(ResolvedObject))
		{
			SocketNames = SceneComponent->GetAllSocketNames();
			return true;
		}
		return false;
	}

	virtual bool GetSocketTransformsInternal(UObject* ResolvedObject, TArray<FTransform3f>&OutSocketTransforms, TOptional<TConstArrayView<int32>> SocketIndices) const override
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(ResolvedObject))
		{
			const bool bIndirect = SocketIndices.IsSet();
			const int32 NumIndicies = bIndirect ? SocketIndices.GetValue().Num() : SocketNames.Num();
			for (int iIndex = 0; iIndex < NumIndicies; ++iIndex)
			{
				const int32 iSocketIndex = bIndirect ? SocketIndices.GetValue()[iIndex] : iIndex;
				if (SocketNames.IsValidIndex(iSocketIndex))
				{
					const FTransform SocketTransform = SceneComponent->GetSocketTransform(SocketNames[iSocketIndex], ERelativeTransformSpace::RTS_Component);
					OutSocketTransforms[iSocketIndex] = FTransform3f(SocketTransform);
				}
			}

			return true;
		}
		return false;
	}
};
