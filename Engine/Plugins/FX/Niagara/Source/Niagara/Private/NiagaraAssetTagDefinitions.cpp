// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraAssetTagDefinitions.h"

#include "GeneralProjectSettings.h"
#include "NiagaraEmitter.h"
#include "NiagaraSystem.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Interfaces/IProjectManager.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraAssetTagDefinitions)

FNiagaraAssetTagDefinition::FNiagaraAssetTagDefinition(FText InAssetTag, int32 InAssetFlags, FText InDescription, ENiagaraAssetTagDefinitionImportance InDisplayType, FLinearColor InColor, FGuid InTagGuid)
	: AssetTag(InAssetTag), AssetFlags(InAssetFlags), Description(InDescription), DisplayType(InDisplayType), Color(InColor), TagGuid(InTagGuid)
{
}

bool FNiagaraAssetTagDefinition::IsValid() const
{
	return TagGuid.IsValid() && AssetTag.IsEmpty() == false;
}

bool FNiagaraAssetTagDefinition::SupportsEmitters() const
{
	return (AssetFlags & (int32) ENiagaraAssetLibraryAssetTypes::Emitters) != 0;
}

bool FNiagaraAssetTagDefinition::SupportsSystems() const
{
	return (AssetFlags & (int32) ENiagaraAssetLibraryAssetTypes::Systems) != 0;
}

bool FNiagaraAssetTagDefinition::SupportsScripts() const
{
	return (AssetFlags & (int32) ENiagaraAssetLibraryAssetTypes::Scripts) != 0;
}

TArray<UClass*> FNiagaraAssetTagDefinition::GetSupportedClasses() const
{
	TArray<UClass*> Result;
	
	if(SupportsEmitters())
	{
		Result.Add(UNiagaraEmitter::StaticClass());
	}

	if(SupportsSystems())
	{
		Result.Add(UNiagaraSystem::StaticClass());
	}

	if(SupportsScripts())
	{
		Result.Add(UNiagaraScript::StaticClass());
	}

	return Result;
}

bool FNiagaraAssetTagDefinition::DoesAssetDataContainTag(const FAssetData& AssetData) const
{
	return AssetData.FindTag(FName(GetGuidAsString()));
}

FString FNiagaraAssetTagDefinition::GetGuidAsString() const
{
	return TagGuid.ToString(EGuidFormats::DigitsWithHyphens);
}

FNiagaraAssetTagDefinitionReference::FNiagaraAssetTagDefinitionReference(const FNiagaraAssetTagDefinition& InTagDefinition)
{
	SetTagDefinitionReference(InTagDefinition);
}

FString FNiagaraAssetTagDefinitionReference::GetGuidAsString() const
{
	return AssetTagDefinitionGuid.ToString(EGuidFormats::DigitsWithHyphens);
}

void FNiagaraAssetTagDefinitionReference::AddTagToAssetRegistryTags(FAssetRegistryTagsContext& Context) const
{
	UObject::FAssetRegistryTag* FoundAssetTag = Context.FindTag(FName(GetGuidAsString()));

	if(FoundAssetTag)
	{
		return;
	}
	else
	{
		// Value has to be >1 of data to be considered non-empty to not get thrown away
		Context.AddTag(UObject::FAssetRegistryTag(FName(GetGuidAsString()), "  ", UObject::FAssetRegistryTag::TT_Alphabetical));
	}
}

FText UDEPRECATED_NiagaraAssetTagDefinitions::GetDisplayName() const
{
	if(DisplayName.IsEmptyOrWhitespace())
	{
		return FText::FromString(GetName());
	}

	return DisplayName;
}

FText UDEPRECATED_NiagaraAssetTagDefinitions::GetDescription() const
{
	return Description;
}

bool UDEPRECATED_NiagaraAssetTagDefinitions::DoesAssetDataContainAnyTag(const FAssetData& AssetData) const
{
	for(const FNiagaraAssetTagDefinition& AssetTagDefinition : TagDefinitions)
	{
		if(AssetTagDefinition.DoesAssetDataContainTag(AssetData))
		{
			return true;
		}
	}

	return false;
}
