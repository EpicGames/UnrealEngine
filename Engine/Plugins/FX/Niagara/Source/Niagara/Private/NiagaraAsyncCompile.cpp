// Copyright Epic Games, Inc. All Rights Reserved.


#include "NiagaraAsyncCompile.h"

#include "Algo/Copy.h"
#include "Modules/ModuleManager.h"
#include "NiagaraConstants.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraModule.h"
#include "NiagaraPrecompileContainer.h"
#include "NiagaraScriptSourceBase.h"
#include "NiagaraStats.h"
#include "NiagaraSystem.h"
#include "UObject/Package.h"
#include "VectorVM.h"
#include "VectorVMBridge.h"
#include "VectorVMOptimizer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraAsyncCompile)

#define LOCTEXT_NAMESPACE "NiagaraAsyncCompile"

#if WITH_EDITOR
#include "DerivedDataCacheInterface.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Niagara - System - Precompile"), STAT_Niagara_System_Precompile, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Niagara - System - CompileScriptTaskGT"), STAT_Niagara_System_CompileScriptTaskGT, STATGROUP_Niagara);

static float GNiagaraCompileDDCWaitTimeout = 10.0f;
static FAutoConsoleVariableRef CVarNiagaraCompileDDCWaitTimeout(
	TEXT("fx.Niagara.CompileDDCWaitTimeout"),
	GNiagaraCompileDDCWaitTimeout,
	TEXT("During script compilation, how long do we wait for the ddc to answer in seconds before starting shader compilation?"),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////

#if WITH_EDITORONLY_DATA
FNiagaraLazyPrecompileReference::FNiagaraLazyPrecompileReference(UNiagaraSystem* InSystem, TConstArrayView<UNiagaraScript*> InScripts)
	: System(InSystem)
	, Scripts(InScripts)
{
	GenerateEmitterScriptIndexMap(EmitterScriptIndex);
}

TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> FNiagaraLazyPrecompileReference::GetPrecompileData(UNiagaraScript* ForScript)
{
	if (SystemPrecompiledData == nullptr)
	{
		UNiagaraPrecompileContainer* Container = NewObject<UNiagaraPrecompileContainer>(GetTransientPackage());
		Container->System = System;
		Container->Scripts = Scripts;
		INiagaraModule& NiagaraModule = FModuleManager::Get().LoadModuleChecked<INiagaraModule>(TEXT("Niagara"));
		SystemPrecompiledData = NiagaraModule.Precompile(Container, FGuid());

		const TArray<FNiagaraEmitterHandle>& EmitterHandles = System->GetEmitterHandles();
		for (int32 i = 0; i < EmitterHandles.Num(); i++)
		{
			const FNiagaraEmitterHandle& Handle = EmitterHandles[i];
			if (Handle.GetEmitterData() && Handle.GetIsEnabled())
			{
				TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> EmitterPrecompiledData = SystemPrecompiledData->GetDependentRequest(i);
				TArray<UNiagaraScript*> EmitterScripts;
				Handle.GetEmitterData()->GetScripts(EmitterScripts, false, true);
				check(EmitterScripts.Num() > 0);
				for (UNiagaraScript* EmitterScript : EmitterScripts)
				{
					EmitterMapping.Add(EmitterScript, EmitterPrecompiledData);
				}
			}
		}
	}
	if (SystemPrecompiledData.IsValid() == false)
	{
		UE_LOGF(LogNiagara, Error, "Failed to precompile %ls.  This is due to unexpected invalid or broken data.  Additional details should be in the log.", *System->GetPathName());
		return nullptr;
	}

	if (ForScript->IsSystemSpawnScript() || ForScript->IsSystemUpdateScript())
	{
		return SystemPrecompiledData;
	}
	TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe>* EmitterPrecompileData = EmitterMapping.Find(ForScript);
	if (EmitterPrecompileData == nullptr)
	{
		UE_LOGF(LogNiagara, Log, "Failed to precompile %ls. This is probably because the system was modified between the original compile request and now.", *System->GetPathName());
		return nullptr;
	}
	return *EmitterPrecompileData;
}

TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> FNiagaraLazyPrecompileReference::GetPrecompileDuplicateData(UNiagaraEmitter* OwningEmitter, UNiagaraScript* TargetScript)
{
	TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> PrecompileDuplicateData;
	for (TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> ExistingPrecompileDuplicateData : PrecompileDuplicateDatas)
	{
		if (ExistingPrecompileDuplicateData->IsDuplicateDataFor(System, OwningEmitter, TargetScript))
		{
			PrecompileDuplicateData = ExistingPrecompileDuplicateData;
		}
	}

	if (PrecompileDuplicateData.IsValid() == false)
	{
		INiagaraModule& NiagaraModule = FModuleManager::Get().LoadModuleChecked<INiagaraModule>(TEXT("Niagara"));

		PrecompileDuplicateData = NiagaraModule.PrecompileDuplicate(SystemPrecompiledData.Get(), System, OwningEmitter, TargetScript, FGuid());
		PrecompileDuplicateDatas.Add(PrecompileDuplicateData);

		PrecompileDuplicateData->GetDuplicatedObjects(MutableView(CompilationRootObjects));
		for (int32 i = 0; i < PrecompileDuplicateData->GetDependentRequestCount(); i++)
		{
			PrecompileDuplicateData->GetDependentRequest(i)->GetDuplicatedObjects(MutableView(CompilationRootObjects));
		}
	}

	return PrecompileDuplicateData;
}

// annoyingly because we're driving some of the compilation process on live data (UNiagaraSystem/UNiagaraEmitter) which
// might be changing in the scenario of a user continuing to modify their effect in the system editor, we need to at
// least put some effort into validating that the system/emitters are still going to match up for the work we do in the
// precompile.  So, we again generate the EmitterScriptIndexMap and make sure that they are the same.  This should handle
// the common case of emitters being added, removed or reordered.
bool FNiagaraLazyPrecompileReference::IsValidForPrecompile() const
{
	if (System)
	{
		FEmitterScriptIndexMap CurrentScriptIndexMap;
		GenerateEmitterScriptIndexMap(CurrentScriptIndexMap);
		
		if (!CurrentScriptIndexMap.OrderIndependentCompareEqual(EmitterScriptIndex))
		{
			return false;
		}
	}

	return true;
}

void FNiagaraLazyPrecompileReference::GenerateEmitterScriptIndexMap(FEmitterScriptIndexMap& OutIndexMap) const
{
	OutIndexMap.Reset();

	if (System)
	{
		const TArray<FNiagaraEmitterHandle>& EmitterHandles = System->GetEmitterHandles();
		for (int32 i = 0; i < EmitterHandles.Num(); i++)
		{
			FNiagaraEmitterHandle Handle = EmitterHandles[i];
			if (Handle.GetIsEnabled() && Handle.GetEmitterData())
			{
				TArray<UNiagaraScript*> EmitterScripts;
				Handle.GetEmitterData()->GetScripts(EmitterScripts, false, true);
				check(EmitterScripts.Num() > 0);
				for (UNiagaraScript* EmitterScript : EmitterScripts)
				{
					OutIndexMap.Add(EmitterScript, i);
				}
			}
		}
	}
}

FNiagaraAsyncCompileTask::FNiagaraAsyncCompileTask(UNiagaraSystem* InOwningSystem, FString InAssetPath, const FEmitterCompiledScriptPair& InScriptPair)
{
	OwningSystem = InOwningSystem;
	AssetPath = InAssetPath;
	ScriptPair = InScriptPair;

	bCompilableScript = InScriptPair.CompiledScript ? InScriptPair.CompiledScript->IsCompilable() : false;

	CurrentState = ENiagaraCompilationState::CheckDDC;
}

void FNiagaraAsyncCompileTask::ProcessCurrentState()
{
	SCOPE_CYCLE_COUNTER(STAT_Niagara_System_CompileScriptTaskGT);
	if (IsDone())
	{
		return;
	}

	if (CurrentState == ENiagaraCompilationState::CheckDDC)
	{
		// if FNiagaraActiveCompilationDefault is going to handle evaluating the CheckDDC state, then skip doing anything here
		if (bCheckDDCEnabled)
		{
			// check if the DDC data is ready
			CheckDDCResult();
		}
	}
	else if (CurrentState == ENiagaraCompilationState::Precompile)
	{
		// gather precompile data
		StartCompileTime = FPlatformTime::Seconds();
		PrecompileData();
		MoveToState(ENiagaraCompilationState::StartCompileJob);
	}
	else if (CurrentState == ENiagaraCompilationState::StartCompileJob)
	{
		// start the async compile job
		if (bCompilableScript)
		{
			StartCompileJob();
			MoveToState(ENiagaraCompilationState::AwaitResult);
		}
		else
		{
			ProcessNonCompilableScript();
			MoveToState(ENiagaraCompilationState::PutToDDC);
		}
	}
	else if (CurrentState == ENiagaraCompilationState::AwaitResult)
	{
		if (AwaitResult())
		{
			MoveToState(ENiagaraCompilationState::OptimizeByteCode);
		}
	}
	else if (CurrentState == ENiagaraCompilationState::OptimizeByteCode)
	{
		OptimizeByteCode();
		MoveToState(ENiagaraCompilationState::ProcessResult);
	}
	else if (CurrentState == ENiagaraCompilationState::ProcessResult)
	{
		// save the result from the compile job
		ProcessResult();
		MoveToState(ENiagaraCompilationState::PutToDDC);
	}
	else if (CurrentState == ENiagaraCompilationState::PutToDDC)
	{
		// put the result from the compile job into the ddc
		PutToDDC();
		MoveToState(ENiagaraCompilationState::Finished);
	}
	else
	{
		check(false);
	}
}

void FNiagaraAsyncCompileTask::MoveToState(ENiagaraCompilationState NewState)
{
	if (CurrentState == ENiagaraCompilationState::Finished || CurrentState == ENiagaraCompilationState::Aborted)
	{
		// we are already in an end state, so nothing to do
		return;
	}

	// check state machine integrity
	check(NewState != ENiagaraCompilationState::CheckDDC);
	if (NewState == ENiagaraCompilationState::Precompile)
	{
		check(CurrentState == ENiagaraCompilationState::CheckDDC);
	}
	if (NewState == ENiagaraCompilationState::StartCompileJob)
	{
		check(CurrentState == ENiagaraCompilationState::Precompile);
	}
	if (NewState == ENiagaraCompilationState::AwaitResult)
	{
		check(CurrentState == ENiagaraCompilationState::StartCompileJob);
	}
	if (NewState == ENiagaraCompilationState::OptimizeByteCode)
	{
		check(CurrentState == ENiagaraCompilationState::AwaitResult);
	}
	if (NewState == ENiagaraCompilationState::ProcessResult)
	{
		check(CurrentState == ENiagaraCompilationState::OptimizeByteCode);
	}
	if (NewState == ENiagaraCompilationState::PutToDDC)
	{
		check(CurrentState == ENiagaraCompilationState::ProcessResult || CurrentState == ENiagaraCompilationState::StartCompileJob);
	}
	if (NewState == ENiagaraCompilationState::Finished)
	{
		CompileMetrics.TaskWallTime = (float) (FPlatformTime::Seconds() - StartTaskTime);
		check(CurrentState == ENiagaraCompilationState::CheckDDC || CurrentState == ENiagaraCompilationState::PutToDDC);
	}

	UE_LOGF(LogNiagara, Verbose, "Changing state %i -> %i for for %ls!", int(CurrentState), int(NewState), *AssetPath);
	CurrentState = NewState;
}

bool FNiagaraAsyncCompileTask::IsDone() const
{
	return CurrentState == ENiagaraCompilationState::Finished || CurrentState == ENiagaraCompilationState::Aborted;
}

void LogVMID(const FNiagaraVMExecutableDataId& ID, FString Name)
{
	// some debug logging to find out why the ddc key changed
	UE_LOGF(LogNiagara, Display, "FNiagaraVMExecutableDataId %ls:", *Name);
	UE_LOGF(LogNiagara, Display, "  ScriptUsageType: %i", int(ID.ScriptUsageType));
	UE_LOGF(LogNiagara, Display, "  ScriptUsageTypeID: %ls", *ID.ScriptUsageTypeID.ToString());
	UE_LOGF(LogNiagara, Display, "  CompilerVersionID: %ls", *ID.CompilerVersionID.ToString());
	UE_LOGF(LogNiagara, Display, "  BaseScriptCompileHash: %ls", *ID.BaseScriptCompileHash.ToString());
	UE_LOGF(LogNiagara, Display, "  bUsesRapidIterationParams: %i", ID.bUsesRapidIterationParams);
	UE_LOGF(LogNiagara, Display, "  AdditionalDefines:");
	for (int32 Idx = 0; Idx < ID.AdditionalDefines.Num(); Idx++)
	{
		UE_LOGF(LogNiagara, Display, "     %ls", *ID.AdditionalDefines[Idx]);
	}
	UE_LOGF(LogNiagara, Display, "  AdditionalVariables:");
	for (int32 Idx = 0; Idx < ID.AdditionalVariables.Num(); Idx++)
	{
		UE_LOGF(LogNiagara, Display, "     %ls / %ls", *ID.AdditionalVariables[Idx].GetName().ToString(), *ID.AdditionalVariables[Idx].GetType().GetName());
	}
	UE_LOGF(LogNiagara, Display, "  ReferencedCompileHashes:");
	for (int32 HashIndex = 0; HashIndex < ID.ReferencedCompileHashes.Num(); HashIndex++)
	{
		UE_LOGF(LogNiagara, Display, "     %ls", *ID.ReferencedCompileHashes[HashIndex].ToString());
	}
}

void FNiagaraAsyncCompileTask::PrecompileData()
{
	SCOPE_CYCLE_COUNTER(STAT_Niagara_System_Precompile);
	UE_LOGF(LogNiagara, Verbose, "Getting precompile data for %ls!", *AssetPath);

	// we do one more check here to see if the source data has been changed out from under us, if it has then we abort the compilation attempt
	if (!CompilationIdMatchesRequest())
	{
		AbortTask();
		return;
	}

	// destructive changes (like emitters being deleted) can disrupt the precompile so we need to validate that we can continue.
	if (!PrecompileReference->IsValidForPrecompile())
	{
		UE_LOGF(LogNiagara, Warning, "Unable to proceed with compilation due to changes to asset %ls.  Aborting.", *AssetPath);
		AbortTask();
		return;
	}

	ComputedPrecompileData = PrecompileReference->GetPrecompileData(ScriptPair.CompiledScript);
	TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> SystemPrecompileDuplicateData = PrecompileReference->GetPrecompileDuplicateData(ScriptPair.VersionedEmitter.Emitter, ScriptPair.CompiledScript);

	if (ComputedPrecompileData == nullptr || SystemPrecompileDuplicateData == nullptr)
	{
		AbortTask();
		return;
	}

	// Grab the list of user variables that were actually encountered so that we can add to them later.
	ComputedPrecompileData->GatherPreCompiledVariables(TEXT("User"), EncounteredExposedVars);
	for(int32 i = 0; i < ComputedPrecompileData->GetDependentRequestCount(); i++)
	{
		ComputedPrecompileData->GetDependentRequest(i)->GatherPreCompiledVariables(TEXT("User"), EncounteredExposedVars);
	}

	// Assign the correct duplicate data for this request.
	if(ScriptPair.CompiledScript->GetUsage() == ENiagaraScriptUsage::SystemSpawnScript || ScriptPair.CompiledScript->GetUsage() == ENiagaraScriptUsage::SystemUpdateScript)
	{
		ComputedPrecompileDuplicateData = SystemPrecompileDuplicateData;
	}
	else
	{
		UNiagaraEmitter* OwningEmitter = ScriptPair.VersionedEmitter.Emitter;
		int32 EmitterIndex = OwningSystem->GetEmitterHandles().IndexOfByPredicate([OwningEmitter](const FNiagaraEmitterHandle& EmitterHandle) { return EmitterHandle.GetInstance().Emitter == OwningEmitter; });
		if(EmitterIndex != INDEX_NONE)
		{
			ComputedPrecompileDuplicateData = SystemPrecompileDuplicateData->GetDependentRequest(EmitterIndex);
		}
	}

	// check that we have valid PrecompileDuplicateData, if we don't we'll abort the task.  This is likely due to
	// the system being edited, like emitters being deleted, within the window of the compile being requested and
	// the precompile being issued
	if (ComputedPrecompileDuplicateData == nullptr)
	{
		AbortTask();
		return;
	}
}

void FNiagaraAsyncCompileTask::StartCompileJob()
{
	UE_LOGF(LogNiagara, Verbose, "Starting compilation for %ls!", *AssetPath);
	
	// Fire off the compile job
	if (ScriptPair.CompiledScript->RequestExternallyManagedAsyncCompile(ComputedPrecompileData, ComputedPrecompileDuplicateData, ScriptPair.CompileId, ScriptPair.PendingJobID))
	{
		UE_LOGF(LogNiagara, Verbose, "Designated compilation ID %i for compilation of %ls!", ScriptPair.PendingJobID, *AssetPath);
		bUsedShaderCompilerWorker = true;
	}
	else
	{
		UE_LOGF(LogNiagara, Warning, "For some reason we are reporting that %ls is in sync even though AreScriptAndSourceSynchronized returned false!", *ScriptPair.CompiledScript->GetPathName())
		AbortTask();
	}
}

bool FNiagaraAsyncCompileTask::AwaitResult()
{
	UE_LOGF(LogNiagara, Verbose, "Waiting on compilation to finish for %ls!", *AssetPath);

	INiagaraModule& NiagaraModule = FModuleManager::Get().LoadModuleChecked<INiagaraModule>(TEXT("Niagara"));

	if (!ExeData.IsValid())
	{
		ExeData = NiagaraModule.GetCompileJobResult(ScriptPair.PendingJobID, bWaitForCompileJob, CompileMetrics);
	}

	return ExeData.IsValid();
}

void FNiagaraAsyncCompileTask::OptimizeByteCode()
{
	check(ExeData);

	if (const uint8* ByteCode = ExeData->ByteCode.GetDataPtr())
	{
		const double OptimizeStartTime = FPlatformTime::Seconds();

		//this is just necessary because VectorVM doesn't know about FVMExternalFunctionBindingInfo
		const int32 NumExtFns = ExeData->CalledVMExternalFunctions.Num();

		TArray<FVectorVMExtFunctionData> ExtFnTable;
		ExtFnTable.SetNumZeroed(NumExtFns);
		for (int i = 0; i < NumExtFns; ++i)
		{
			ExtFnTable[i].NumInputs = ExeData->CalledVMExternalFunctions[i].GetNumInputs();
			ExtFnTable[i].NumOutputs = ExeData->CalledVMExternalFunctions[i].GetNumOutputs();
		}

		{
			const uint64 AssetPathHash = CityHash64((char*)AssetPath.GetCharArray().GetData(), AssetPath.GetCharArray().Num());

			VectorVM::Optimizer::FVectorVMOptimizerContext* OptimizerContext = VectorVM::Optimizer::AllocOptimizerContext();
			VectorVM::Optimizer::OptimizeVectorVMScript(ByteCode, ExeData->ByteCode.GetLength(), ExtFnTable.GetData(), ExtFnTable.Num(), OptimizerContext, AssetPathHash, VectorVM::VVMFlag_OptOmitStats | VectorVM::VVMFlag_OptSaveIntermediateState);

			// extract a human readable version of the script
			VectorVM::Optimizer::GenerateHumanReadableScript(*OptimizerContext, ExeData->LastExperimentalAssemblyScript);

			// freeze the OptimizeContext
			VectorVM::Bridge::FreezeOptimizerContext(*OptimizerContext, ExeData->ExperimentalContextData);

			VectorVM::Optimizer::FreeOptimizerContext(OptimizerContext);
		}

		CompileMetrics.ByteCodeOptimizeTime = (float) (FPlatformTime::Seconds() - OptimizeStartTime);
	}
}

void FNiagaraAsyncCompileTask::ProcessNonCompilableScript()
{
	ExeData = MakeShared<FNiagaraVMExecutableData>();
	ScriptPair.CompileResults = ExeData;
	ScriptPair.bResultsReady = true;
	ExeData->BakedRapidIterationParameters = BakedRapidIterationParameters;
	ScriptPair.CompiledScript->ExecToBinaryData(ScriptPair.CompiledScript, DDCOutData, *ExeData);
}

void FNiagaraAsyncCompileTask::ProcessResult()
{
	check(ExeData);
	UE_LOGF(LogNiagara, Verbose, "Processing compilation results for %ls!", *AssetPath);

	// convert results to be saved in the ddc
	ScriptPair.CompileResults = ExeData;
	ScriptPair.bResultsReady = true;
	ScriptPair.CompileTime = float(FPlatformTime::Seconds() - StartCompileTime);
	ExeData->BakedRapidIterationParameters = BakedRapidIterationParameters;
	ScriptPair.CompiledScript->ExecToBinaryData(ScriptPair.CompiledScript, DDCOutData, *ExeData);
	UE_LOGF(LogNiagara, Verbose, "Got %i bytes in ddc data for %ls", DDCOutData.Num(), *AssetPath);
}

void FNiagaraAsyncCompileTask::WaitAndResolveResult()
{
	check(IsInGameThread());
	
	bWaitForCompileJob = true;
	while (!IsDone())
	{
		ProcessCurrentState();
	}
}

void FNiagaraAsyncCompileTask::AbortTask()
{
	MoveToState(ENiagaraCompilationState::Aborted);
}

void FNiagaraAsyncCompileTask::CheckDDCResult()
{
	FDerivedDataCacheInterface* DDC = GetDerivedDataCache();
	if (bWaitForCompileJob)
	{
		DDC->WaitAsynchronousCompletion(TaskHandle);
	}
	if (DDC->PollAsynchronousCompletion(TaskHandle))
	{
		bool bHasResults = DDC->GetAsynchronousResults(TaskHandle, DDCOutData);
		if (bHasResults && DDCOutData.Num() > 0)
		{
			ExeData = MakeShared<FNiagaraVMExecutableData>();
			if (ScriptPair.CompiledScript->BinaryToExecData(ScriptPair.CompiledScript, DDCOutData, *ExeData))
			{
				ScriptPair.CompileResults = ExeData;
				ScriptPair.bResultsReady = true;
				bResultsFromDDC = true;
				MoveToState(ENiagaraCompilationState::Finished);
				UE_LOGF(LogNiagara, Verbose, "Compilation data for %ls could be pulled from the ddc.", *AssetPath);
			}
			else
			{
				UE_LOGF(LogNiagara, Warning, "Unable to create exec data from ddc data for script %ls, going to recompile it from scratch. DDC might be corrupted or there is a problem with script serialization.", *AssetPath);
				ExeData.Reset();
				MoveToState(ENiagaraCompilationState::Precompile);
			}
		}
		else
		{
			MoveToState(ENiagaraCompilationState::Precompile);
			UE_LOGF(LogNiagara, Verbose, "No compilation data for %ls found in the ddc.", *AssetPath);
		}
		CompileMetrics.DDCFetchTime = (float) (FPlatformTime::Seconds() - StartTaskTime);
		UE_LOGF(LogNiagara, VeryVerbose, "Compile task for %ls took %f seconds to fetch data from ddc.", *AssetPath, CompileMetrics.DDCFetchTime);
	}
	else if ((FPlatformTime::Seconds() - StartTaskTime) > GNiagaraCompileDDCWaitTimeout)
	{
		MoveToState(ENiagaraCompilationState::Precompile);
		UE_LOGF(LogNiagara, Log, "DDC timed out after %i seconds, starting compilation for %ls.", (int)GNiagaraCompileDDCWaitTimeout, *AssetPath);
	}
}

void FNiagaraAsyncCompileTask::PutToDDC()
{
	if (DDCOutData.Num() > 0)
	{
		UE_LOGF(LogNiagara, Verbose, "Writing data for %ls to the ddc", *AssetPath);
		GetDerivedDataCache()->Put(*DDCKey, DDCOutData, *AssetPath, true);

		if (AlternateDDCKey.IsSet())
		{
			GetDerivedDataCache()->Put(*(*AlternateDDCKey), DDCOutData, *AssetPath, true);
		}
	}
}

void FNiagaraAsyncCompileTask::AssignInitialCompilationId(const FNiagaraVMExecutableDataId& InitialCompilationId)
{
	ScriptPair.CompileId = InitialCompilationId;
}

void FNiagaraAsyncCompileTask::UpdateCompilationId(const FNiagaraVMExecutableDataId& UpdatedCompilationId)
{
	// we need to update our CompileId and add a new DDC key where we'll push the data to
	ScriptPair.CompileId = UpdatedCompilationId;
	AlternateDDCKey = UNiagaraScript::BuildNiagaraDDCKeyString(UpdatedCompilationId, AssetPath);
}

bool FNiagaraAsyncCompileTask::CompilationIdMatchesRequest() const
{
	FNiagaraVMExecutableDataId CompilationId;
	ScriptPair.CompiledScript->ComputeVMCompilationId(CompilationId, FGuid());

	const bool IdMatches = CompilationId == ScriptPair.CompileId;
	if (!IdMatches)
	{
		if (LogNiagara.GetVerbosity() >= ELogVerbosity::Verbose)
		{
			UE_LOGF(LogNiagara, Warning, "Compile ID for %ls changed during the compilation task.", *AssetPath);
			FNiagaraVMExecutableDataId OldID = ScriptPair.CompileId;
			LogVMID(ScriptPair.CompileId, "RequestedID");
			LogVMID(ScriptPair.CompiledScript->GetComputedVMCompilationId(), "CurrentID");
		}
	}

	return IdMatches;
}

FNiagaraActiveCompilationDefault::~FNiagaraActiveCompilationDefault()
{
	// make sure that we release all of the compilation copies that have been created for the tasks
	Reset();
}

void FNiagaraActiveCompilationDefault::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddStableReferenceSet(&RootObjects);
}

void FNiagaraActiveCompilationDefault::Abort()
{
	for (auto& AsyncTask : Tasks)
	{
		AsyncTask->AbortTask();
	}
}

bool FNiagaraActiveCompilationDefault::ValidateConsistentResults(const FNiagaraQueryCompilationOptions& Options) const
{
	// for now the only thing we're concerned about is if we've got results for SystemSpawn and SystemUpdate scripts
	// then we need to make sure that they agree in terms of the dataset attributes
	const FAsyncTaskPtr* SpawnScriptRequest = FindTask(Options.System->GetSystemSpawnScript());
	const FAsyncTaskPtr* UpdateScriptRequest = FindTask(Options.System->GetSystemUpdateScript());

	const bool SpawnScriptValid = SpawnScriptRequest
		&& SpawnScriptRequest->Get()->GetScriptPair().CompileResults.IsValid()
		&& SpawnScriptRequest->Get()->GetScriptPair().CompileResults->LastCompileStatus != ENiagaraScriptCompileStatus::NCS_Error;

	const bool UpdateScriptValid = UpdateScriptRequest
		&& UpdateScriptRequest->Get()->GetScriptPair().CompileResults.IsValid()
		&& UpdateScriptRequest->Get()->GetScriptPair().CompileResults->LastCompileStatus != ENiagaraScriptCompileStatus::NCS_Error;

	if (SpawnScriptValid && UpdateScriptValid)
	{
		const FEmitterCompiledScriptPair& SpawnScriptPair = SpawnScriptRequest->Get()->GetScriptPair();
		const FEmitterCompiledScriptPair& UpdateScriptPair = UpdateScriptRequest->Get()->GetScriptPair();
		if (SpawnScriptPair.CompileResults->Attributes != UpdateScriptPair.CompileResults->Attributes)
		{
			// if we had requested a full rebuild, then we've got a case where the generated scripts are not compatible.  This indicates
			// a significant issue where we're allowing graphs to generate invalid collections of scripts.  One known example is using
			// the Script.Context static switch that isn't fully processed in all scripts, leading to attributes differing between the
			// SystemSpawnScript and the SystemUpdateScript
			if (bForced)
			{
				FString MissingAttributes;
				FString AdditionalAttributes;

				for (const auto& SpawnAttrib : SpawnScriptPair.CompileResults->Attributes)
				{
					if (!UpdateScriptPair.CompileResults->Attributes.Contains(SpawnAttrib))
					{
						MissingAttributes.Appendf(TEXT("%s%s"), MissingAttributes.Len() ? TEXT(", ") : TEXT(""), *SpawnAttrib.GetName().ToString());
					}
				}

				for (const auto& UpdateAttrib : UpdateScriptPair.CompileResults->Attributes)
				{
					if (!SpawnScriptPair.CompileResults->Attributes.Contains(UpdateAttrib))
					{
						AdditionalAttributes.Appendf(TEXT("%s%s"), AdditionalAttributes.Len() ? TEXT(", ") : TEXT(""), *UpdateAttrib.GetName().ToString());
					}
				}

				FNiagaraCompileEvent AttributeMismatchEvent(
					FNiagaraCompileEventSeverity::Error,
					FText::Format(LOCTEXT("SystemScriptAttributeMismatchError", "System Spawn/Update scripts have attributes which don't match!\n\tMissing update attributes: {0}\n\tAdditional update attributes: {1}"),
						FText::FromString(MissingAttributes),
						FText::FromString(AdditionalAttributes))
					.ToString());

				SpawnScriptPair.CompileResults->LastCompileStatus = ENiagaraScriptCompileStatus::NCS_Error;
				SpawnScriptPair.CompileResults->LastCompileEvents.Add(AttributeMismatchEvent);
			}
			else
			{
				UE_LOGF(LogNiagara, Log, "Failed to generate consistent results for System spawn and update scripts for system %ls.", *Options.System->GetFullName());
			}

			return false;
		}
	}
	return true;
}

FNiagaraActiveCompilationDefault::FAsyncTaskPtr* FNiagaraActiveCompilationDefault::FindTask(const UNiagaraScript* Script)
{
	return Tasks.FindByPredicate([Script](const FAsyncTaskPtr& TaskPtr) -> bool
	{
		return TaskPtr->GetScriptPair().CompiledScript == Script;
	});
}

const FNiagaraActiveCompilationDefault::FAsyncTaskPtr* FNiagaraActiveCompilationDefault::FindTask(const UNiagaraScript* Script) const
{
	return Tasks.FindByPredicate([Script](const FAsyncTaskPtr& TaskPtr) -> bool
	{
		return TaskPtr->GetScriptPair().CompiledScript == Script;
	});
}

bool FNiagaraActiveCompilationDefault::QueryCompileComplete(const FNiagaraQueryCompilationOptions& Options)
{
	const double QueryStartTime = FPlatformTime::Seconds();

	if (Options.bWait)
	{
		for (FAsyncTaskPtr& AsyncTask : Tasks)
		{
			AsyncTask->bWaitForCompileJob = true;
		}
	}

	if (bEvaluateParametersPending)
	{
		if (!bDDCGetCompleted)
		{
			bDDCGetCompleted = true;
			bool bAllTasksFinalized = true;
			{
				for (FAsyncTaskPtr& AsyncTask : Tasks)
				{
					if (AsyncTask->CurrentState == ENiagaraCompilationState::CheckDDC)
					{
						AsyncTask->CheckDDCResult();
						if (AsyncTask->CurrentState == ENiagaraCompilationState::CheckDDC)
						{
							bDDCGetCompleted = false;
						}
					}

					if (AsyncTask->CurrentState != ENiagaraCompilationState::Finished)
					{
						bAllTasksFinalized = false;
					}
				}
			}

			if (bDDCGetCompleted && bEvaluateParametersPending && !bAllTasksFinalized)
			{
				// run through the active scripts and make sure that they are still up to date before we try to do
				// any additional processing on them
				for (FAsyncTaskPtr& AsyncTask : Tasks)
				{
					if (AsyncTask->CurrentState == ENiagaraCompilationState::Finished)
					{
						continue;
					}

					if (!AsyncTask->CompilationIdMatchesRequest())
					{
						AsyncTask->AbortTask();
					}
				}

				// note - by having this deferred (after we've checked DDC) we do risk that the prepared RI parameters could be
				// out of sync with what was there at the time the compile was requested.  So, we'll only take this path if we're
				// not in edit mode.  In edit mode we'll prepare the RI parameters during UNiagaraSystem::RequestCompile()
				Options.System->PrepareRapidIterationParametersForCompilation();

				// go through our tasks and evaluate the rapid iteration parameters.  We'll need to store them in the ExeData
				// so that subsequent builds can take advantage of the work done in PrepareRapidIterationParamtersForCompilation
				// by storing it in the ExeData.  Additionally if this evaluation has changed the rapid iteration parameters the
				// DDC key will be different, and we'll want to push the data both to the old key and to the new key.
				const int32 TaskCount = Tasks.Num();
				for (int32 TaskIt = 0; TaskIt < TaskCount; ++TaskIt)
				{
					FAsyncTaskPtr& AsyncTask = Tasks[TaskIt];
					if (AsyncTask->CurrentState == ENiagaraCompilationState::Finished)
					{
						continue;
					}

					const FEmitterCompiledScriptPair& ScriptPair = AsyncTask->GetScriptPair();
					if (UNiagaraScript* CompiledScript = ScriptPair.CompiledScript)
					{
						TConstArrayView<FNiagaraVariableWithOffset> RapidIterationParameters = CompiledScript->RapidIterationParameters.ReadParameterVariables();

						AsyncTask->BakedRapidIterationParameters.Reserve(RapidIterationParameters.Num());
						for (const FNiagaraVariableWithOffset& ParamWithOffset : RapidIterationParameters)
						{
							FNiagaraVariable& Parameter = AsyncTask->BakedRapidIterationParameters.Add_GetRef(ParamWithOffset);
							Parameter.SetData(CompiledScript->RapidIterationParameters.GetParameterData(ParamWithOffset));
						}

						// if preparing the rapid iteration parameters has adjusted the compile id, then we want to be sure
						// to populate both entries in the DDC with the results (so subsequent loads don't need to go through
						// this same code path, as the serialized compileID will find the DDC results)
						FNiagaraVMExecutableDataId PostRapidIterationParameterID;
						CompiledScript->ComputeVMCompilationId(PostRapidIterationParameterID, FGuid());

						if (PostRapidIterationParameterID != ScriptPair.CompileId)
						{
							AsyncTask->UpdateCompilationId(PostRapidIterationParameterID);
						}
					}
				}
			}
		}

		if (!bDDCGetCompleted)
		{
			return false;
		}
	}

	if (Options.bWait)
	{
		for (FAsyncTaskPtr& AsyncTask : Tasks)
		{
			// before we start to wait for the compile results, we start the compilation of all remaining tasks
			while (!AsyncTask->IsDone() && AsyncTask->CurrentState < ENiagaraCompilationState::AwaitResult)
			{
				AsyncTask->ProcessCurrentState();
			}
		}
	}

	// Check to see if ALL of the sub-requests have resolved.
	for (auto& AsyncTask : Tasks)
	{
		// if a task is very expensive we bail and continue on the next frame  
		if (FPlatformTime::Seconds() - QueryStartTime < Options.MaxWaitDuration)
		{
			AsyncTask->ProcessCurrentState();
		}

		if (AsyncTask->IsDone())
		{
			continue;
		}

		if (!AsyncTask->bFetchedGCObjects && AsyncTask->CurrentState > ENiagaraCompilationState::Precompile && AsyncTask->CurrentState <= ENiagaraCompilationState::ProcessResult)
		{
			// note that PrecompileReference is shared across all tasks and so the contents within are going to
			// be added multiple times (once per task).  Should pull this out of the task processing.
			RootObjects.Append(AsyncTask->PrecompileReference->CompilationRootObjects);
			AsyncTask->bFetchedGCObjects = true;
		}

		if (Options.bWait)
		{
			AsyncTask->WaitAndResolveResult();
		}
		else
		{
			return false;
		}
	}

	return true;
}

void FNiagaraActiveCompilationDefault::Apply(const FNiagaraQueryCompilationOptions& Options)
{
	auto ShouldApplyTask = [](const FAsyncTaskPtr& AsyncTask) -> bool
	{
		if (AsyncTask->CurrentState == ENiagaraCompilationState::Aborted || !AsyncTask->ScriptPair.bResultsReady)
		{
			return false;
		}

		// for now also check some common failure points (the ComputedPrecompileDuplicateData being cleaned up/incomplete)
		if (AsyncTask->ComputedPrecompileDuplicateData.IsValid())
		{
			if (AsyncTask->ComputedPrecompileDuplicateData->GetScriptSource() == nullptr)
			{
				return false;
			}
		}

		return true;
	};

	if (bEvaluateParametersPending)
	{
		// run a first pass to apply the rapid iteration parameters across all of the tasks with results
		for (FAsyncTaskPtr& AsyncTask : Tasks)
		{
			if (ShouldApplyTask(AsyncTask))
			{
				FEmitterCompiledScriptPair& EmitterCompiledScriptPair = AsyncTask->ScriptPair;
				if (TSharedPtr<FNiagaraVMExecutableData> ExeData = EmitterCompiledScriptPair.CompileResults)
				{
					if (UNiagaraScript* CompiledScript = EmitterCompiledScriptPair.CompiledScript)
					{
						CompiledScript->AssignRapidIterationParameters(*ExeData);
					}
				}
			}
		}
	}

	for (FAsyncTaskPtr& AsyncTask : Tasks)
	{
		if (ShouldApplyTask(AsyncTask))
		{
			FEmitterCompiledScriptPair& EmitterCompiledScriptPair = AsyncTask->ScriptPair;
			TSharedPtr<FNiagaraVMExecutableData> ExeData = EmitterCompiledScriptPair.CompileResults;
			UNiagaraScript* CompiledScript = EmitterCompiledScriptPair.CompiledScript;

			// generate the ObjectNameMap from the source (from the duplicated data if available).
			TMap<FName, UNiagaraDataInterface*> ObjectNameMap;
			if (AsyncTask->ComputedPrecompileDuplicateData.IsValid())
			{
				if (const UNiagaraScriptSourceBase* ScriptSource = AsyncTask->ComputedPrecompileDuplicateData->GetScriptSource())
				{
					ObjectNameMap = ScriptSource->ComputeObjectNameMap(*Options.System, CompiledScript->GetUsage(), CompiledScript->GetUsageId(), AsyncTask->UniqueEmitterName);
				}
				else
				{
					checkf(false, TEXT("Failed to get ScriptSource from ComputedPrecompileDuplicateData"));
				}

				// we need to replace the DI in the above generated map with the duplicates that we've created as a part of the duplication process
				TMap<FName, UNiagaraDataInterface*> DuplicatedObjectNameMap = AsyncTask->ComputedPrecompileDuplicateData->GetObjectNameMap();
				for (auto ObjectMapIt = ObjectNameMap.CreateIterator(); ObjectMapIt; ++ObjectMapIt)
				{
					if (UNiagaraDataInterface** Replacement = DuplicatedObjectNameMap.Find(ObjectMapIt.Key()))
					{
						ObjectMapIt.Value() = *Replacement;
					}
					else
					{
						ObjectMapIt.RemoveCurrent();
					}
				}
			}
			else
			{
				const UNiagaraScriptSourceBase* ScriptSource = CompiledScript->GetLatestSource();
				ObjectNameMap = ScriptSource->ComputeObjectNameMap(*Options.System, CompiledScript->GetUsage(), CompiledScript->GetUsageId(), AsyncTask->UniqueEmitterName);
			}

			EmitterCompiledScriptPair.CompiledScript->SetVMCompilationResults(EmitterCompiledScriptPair.CompileId, *ExeData, AsyncTask->UniqueEmitterName, ObjectNameMap, bEvaluateParametersPending);

			// Synchronize the variables that we actually encountered during precompile so that we can expose them to the end user.
			FNiagaraUserRedirectionParameterStore& ExposedParameters = Options.System->GetExposedParameters();

			auto AddUniqueExposedVariables = [&ExposedParameters](TConstArrayView<FNiagaraVariable> InVariables) -> void
			{
				for (const FNiagaraVariable& InVariable : InVariables)
				{
					if (!ExposedParameters.ReadParameterVariables().Contains(InVariable))
					{
						// Just in case it wasn't added previously..
						ExposedParameters.AddParameter(InVariable, true, false);
					}
				}
			};

			AddUniqueExposedVariables(AsyncTask->EncounteredExposedVars);

			// we also grab from the data contained in the ExeData, since that might be all we have if grabbed results from the DDC rather
			// than actually issuing a precompile
			if (AsyncTask->bResultsFromDDC)
			{
				auto IsValidUserVariable = [](const FNiagaraVariableBase& InVariable) -> bool
				{
					if (!InVariable.IsInNameSpace(FStringView(PARAM_MAP_USER_STR)))
					{
						return false;
					}

					// check if this is a struct type, if it is make sure that we ignore the SWC version
					if (const UStruct* Struct = InVariable.GetType().GetStruct())
					{
						if (FNiagaraTypeHelper::IsConvertedSWCStructure(Struct))
						{
							return false;
						}
					}
					return true;
				};

				TArray<FNiagaraVariable> ScriptExposedVariables;
				for (const FNiagaraVariable& Parameter : ExeData->Parameters.Parameters)
				{
					if (!Parameter.IsInNameSpace(FStringView(PARAM_MAP_USER_STR)))
					{
						continue;
					}

					// we need to also deal with SWC -> LWC conversions
					if (const UStruct* Struct = Parameter.GetType().GetStruct())
					{
						if (FNiagaraTypeHelper::IsConvertedSWCStructure(Struct))
						{
							FNiagaraVariable ConvertedVariable(FNiagaraTypeHelper::GetLWCType(Parameter.GetType()), Parameter.GetName());
							ConvertedVariable.AllocateData();

							if (Parameter.IsDataAllocated())
							{
								FNiagaraLwcStructConverter DataConverter = FNiagaraTypeRegistry::GetStructConverter(ConvertedVariable.GetType());
								DataConverter.ConvertDataFromSimulation(ConvertedVariable.GetData(), Parameter.GetData());
							}
							AddUniqueExposedVariables({ConvertedVariable});
						}
					}
					else
					{
						AddUniqueExposedVariables({Parameter});
					}
				}

				AddUniqueExposedVariables(ScriptExposedVariables);
			}
		}
	}
}

void FNiagaraActiveCompilationDefault::Reset()
{
	// clean up the precompile data
	for (FAsyncTaskPtr& AsyncTask : Tasks)
	{
		AsyncTask->ComputedPrecompileData.Reset();
		if (AsyncTask->ComputedPrecompileDuplicateData.IsValid())
		{
			AsyncTask->ComputedPrecompileDuplicateData->ReleaseCompilationCopies();
			AsyncTask->ComputedPrecompileDuplicateData.Reset();
		}
	}
}

bool FNiagaraActiveCompilationDefault::Launch(const FNiagaraCompilationOptions& Options)
{
	bForced = Options.bForced;
	StartTime = FPlatformTime::Seconds();

	check(Options.System->GetSystemSpawnScript()->GetLatestSource() == Options.System->GetSystemUpdateScript()->GetLatestSource());

	// when not compiling for edit mode we will defer preparing rapid iteration parameters till after we get the
	// results from the DDC search (FNiagaraPrepareCompileTask::Tick())
	const bool UseRapidIterationParameters = Options.System->ShouldUseRapidIterationParameters();

	if (UseRapidIterationParameters)
	{
		Options.System->PrepareRapidIterationParametersForCompilation();
	}

	TArray<UNiagaraScript*> ScriptsNeedingCompile;
	bool bAnyCompiled = false;
	bool bAnyUnsynchronized = false;

	// Pass one... determine if any need to be compiled.
	{
		const TArray<FNiagaraEmitterHandle>& EmitterHandles = Options.System->GetEmitterHandles();
		for (int32 i = 0; i < EmitterHandles.Num(); i++)
		{
			FNiagaraEmitterHandle Handle = EmitterHandles[i];
			FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData();
			if (EmitterData && Handle.GetIsEnabled())
			{
				TArray<UNiagaraScript*> EmitterScripts;
				EmitterData->GetScripts(EmitterScripts, false, true);
				check(EmitterScripts.Num() > 0);
				for (UNiagaraScript* EmitterScript : EmitterScripts)
				{
					FEmitterCompiledScriptPair Pair;
					Pair.VersionedEmitter = Handle.GetInstance();
					Pair.CompiledScript = EmitterScript;

					TSharedPtr<FNiagaraAsyncCompileTask, ESPMode::ThreadSafe> AsyncTask = MakeShared<FNiagaraAsyncCompileTask, ESPMode::ThreadSafe>(Options.System, EmitterScript->GetPathName(), Pair);
					if (!EmitterScript->AreScriptAndSourceSynchronized())
					{
						// we need to compute the vmID here to check later in the ddc task before doing the precompile if anything has changed in the meantime.
						// the compilation id was just calculated during AreScriptAndSourceSynchronized() so we reuse it here
						AsyncTask->AssignInitialCompilationId(EmitterScript->GetComputedVMCompilationId());

						ScriptsNeedingCompile.Add(EmitterScript);
						bAnyUnsynchronized = true;
					}
					else
					{
						AsyncTask->CurrentState = ENiagaraCompilationState::Finished;
					}
					AsyncTask->UniqueEmitterName = Handle.GetInstance().Emitter->GetUniqueEmitterName();
					Tasks.Add(AsyncTask);
				}
			}
		}

		bAnyCompiled = bAnyUnsynchronized || Options.bForced;

		// Now add the system scripts for compilation...
		{
			FEmitterCompiledScriptPair Pair;
			Pair.VersionedEmitter = FVersionedNiagaraEmitter();
			Pair.CompiledScript = Options.System->GetSystemSpawnScript();

			TSharedPtr<FNiagaraAsyncCompileTask, ESPMode::ThreadSafe> AsyncTask = MakeShared<FNiagaraAsyncCompileTask, ESPMode::ThreadSafe>(Options.System, Pair.CompiledScript->GetPathName(), Pair);
			if (!Pair.CompiledScript->AreScriptAndSourceSynchronized())
			{
				// we need to compute the vmID here to check later in the ddc task before doing the precompile if anything has changed in the meantime.
				// the compilation id was just calculated during AreScriptAndSourceSynchronized() so we reuse it here
				AsyncTask->AssignInitialCompilationId(Pair.CompiledScript->GetComputedVMCompilationId());

				ScriptsNeedingCompile.Add(Pair.CompiledScript);
				bAnyCompiled = true;
			}
			else
			{
				AsyncTask->CurrentState = ENiagaraCompilationState::Finished;
			}
			Tasks.Add(AsyncTask);
		}

		{
			FEmitterCompiledScriptPair Pair;
			Pair.VersionedEmitter = FVersionedNiagaraEmitter();
			Pair.CompiledScript = Options.System->GetSystemUpdateScript();

			TSharedPtr<FNiagaraAsyncCompileTask, ESPMode::ThreadSafe> AsyncTask = MakeShared<FNiagaraAsyncCompileTask, ESPMode::ThreadSafe>(Options.System, Pair.CompiledScript->GetPathName(), Pair);
			if (!Pair.CompiledScript->AreScriptAndSourceSynchronized())
			{
				// we need to compute the vmID here to check later in the ddc task before doing the precompile if anything has changed in the meantime.
				// the compilation id was just calculated during AreScriptAndSourceSynchronized() so we reuse it here
				AsyncTask->AssignInitialCompilationId(Pair.CompiledScript->GetComputedVMCompilationId());

				ScriptsNeedingCompile.Add(Pair.CompiledScript);
				bAnyCompiled = true;
			}
			else
			{
				AsyncTask->CurrentState = ENiagaraCompilationState::Finished;
			}
			Tasks.Add(AsyncTask);
		}
	}

	{
		bEvaluateParametersPending = !UseRapidIterationParameters;

		// prepare data for any precompile the ddc tasks need to do
		TSharedPtr<FNiagaraLazyPrecompileReference, ESPMode::ThreadSafe> PrecompileReference = MakeShared<FNiagaraLazyPrecompileReference, ESPMode::ThreadSafe>(Options.System, ScriptsNeedingCompile);

		for (UNiagaraScript* Script : ScriptsNeedingCompile)
		{
			if (FAsyncTaskPtr* AsyncTaskPtr = FindTask(Script))
			{
				if (ensure(*AsyncTaskPtr))
				{
					FNiagaraAsyncCompileTask& CompileTask = *(*AsyncTaskPtr);
					CompileTask.DDCKey = UNiagaraScript::BuildNiagaraDDCKeyString(CompileTask.GetScriptPair().CompileId, CompileTask.AssetPath);
					CompileTask.PrecompileReference = PrecompileReference;
					CompileTask.StartTaskTime = FPlatformTime::Seconds();
					CompileTask.bCheckDDCEnabled = !bEvaluateParametersPending;

					// fire off all the ddc tasks, which will trigger the compilation if the data is not in the ddc
					UE_LOGF(LogNiagara, Verbose, "Scheduling async get task for %ls", *CompileTask.AssetPath);
					CompileTask.TaskHandle = GetDerivedDataCacheRef().GetAsynchronous(*CompileTask.DDCKey, CompileTask.AssetPath);
				}
			}
		}
	}

	bAllScriptsSynchronized = !bAnyUnsynchronized;
	return bAnyCompiled;
}

void FNiagaraActiveCompilationDefault::ReportResults(const FNiagaraQueryCompilationOptions& Options) const
{
	const float ElapsedWallTime = (float)(FPlatformTime::Seconds() - StartTime);
	bool bHasCompiledJobs = false;
	float CombinedCompileTime = 0.0f;

	for (const FAsyncTaskPtr& AsyncTask : Tasks)
	{
		if (AsyncTask->bUsedShaderCompilerWorker)
		{
			bHasCompiledJobs = true;
		}

		CombinedCompileTime += AsyncTask->ScriptPair.CompileTime;
	}

	if (bHasCompiledJobs)
	{
		UE_LOGF(LogNiagara, Log, "Compiling System %ls took %f sec (time since issued), %f sec (combined shader worker time).",
			*Options.System->GetFullName(), ElapsedWallTime, CombinedCompileTime);
	}
	else if (bAllScriptsSynchronized == false)
	{
		UE_LOGF(LogNiagara, Verbose, "Retrieving %ls from DDC took %f sec.", *Options.System->GetFullName(), ElapsedWallTime);
	}

	if (Options.bGenerateTimingsFile)
	{
		FNiagaraSystemCompileMetrics SystemMetrics;
		SystemMetrics.SystemCompileWallTime = ElapsedWallTime;

		for (const FAsyncTaskPtr& AsyncTask : Tasks)
		{
			if (AsyncTask->ScriptPair.bResultsReady)
			{
				SystemMetrics.ScriptMetrics.Add(AsyncTask->ScriptPair.CompiledScript, AsyncTask->CompileMetrics);
			}
		}

		WriteTimingsEntry(TEXT("Default Compilation"), Options, SystemMetrics);
	}
}

bool FNiagaraActiveCompilationDefault::BlocksBeginCacheForCooked() const
{
	// With the current setup we will timeout DDC fill jobs while polling for the work to be complete.
	// Not entirely sure why, but my guess is that we get too many systems in the process of being
	// compiled and we iteratively block the gamethread advancing each of them starving progress
	return true;
}

#endif // WITH_EDITORONLY_DATA

#undef LOCTEXT_NAMESPACE // NiagaraAsyncCompile
