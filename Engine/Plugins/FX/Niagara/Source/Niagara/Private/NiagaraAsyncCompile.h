// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCompilationTypes.h"
#include "NiagaraScript.h"

#include "NiagaraAsyncCompile.generated.h"

class UNiagaraEmitter;
class UNiagaraSystem;

USTRUCT()
struct FEmitterCompiledScriptPair
{
	GENERATED_USTRUCT_BODY()
	
	bool bResultsReady = false;
	FVersionedNiagaraEmitter VersionedEmitter;
	UNiagaraScript* CompiledScript = nullptr;
	uint32 PendingJobID = INDEX_NONE; // this is the ID for any active shader compiler worker job
	FNiagaraVMExecutableDataId CompileId;
	TSharedPtr<FNiagaraVMExecutableData> CompileResults;
	float CompileTime = 0.0f;
};

UENUM()
enum class ENiagaraCompilationState : uint8
{
	CheckDDC,
	Precompile,
	StartCompileJob,
	AwaitResult,
	OptimizeByteCode,
	ProcessResult,
	PutToDDC,
	Finished,
	Aborted
};

struct FNiagaraLazyPrecompileReference
{
	FNiagaraLazyPrecompileReference(UNiagaraSystem* InSystem, TConstArrayView<UNiagaraScript*> InScripts);
	FNiagaraLazyPrecompileReference() = delete;

	TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> GetPrecompileData(UNiagaraScript* ForScript);
	TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> GetPrecompileDuplicateData(UNiagaraEmitter* OwningEmitter, UNiagaraScript* TargetScript);

	bool IsValidForPrecompile() const;
	
	UNiagaraSystem* System = nullptr;
	TArray<UNiagaraScript*> Scripts;

	using FEmitterScriptIndexMap = TMap<UNiagaraScript*, int32>;
	FEmitterScriptIndexMap EmitterScriptIndex;
	TArray<TObjectPtr<UObject>> CompilationRootObjects;

private:
	void GenerateEmitterScriptIndexMap(FEmitterScriptIndexMap& OutIndexMap) const;

	TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> SystemPrecompiledData;
	TMap<UNiagaraScript*, TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe>> EmitterMapping;
	TArray<TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe>> PrecompileDuplicateDatas;
};

class FNiagaraAsyncCompileTask
{
#if WITH_EDITORONLY_DATA
public:
	FString DDCKey;
	TOptional<FString> AlternateDDCKey;
	FString AssetPath;
	FString UniqueEmitterName;
	uint32 TaskHandle = 0;

	double StartTaskTime = 0;
	double StartCompileTime = 0;
	bool bWaitForCompileJob = false;
	bool bUsedShaderCompilerWorker = false;
	bool bFetchedGCObjects = false;
	bool bCompilableScript = false;
	bool bResultsFromDDC = false;

	// in order to coordinate between tasks associated with a system CheckDDC can be handled external to the task.
	// In that case we disable this flag
	bool bCheckDDCEnabled = true;

	UNiagaraSystem* OwningSystem;
	FEmitterCompiledScriptPair ScriptPair;
	TArray<FNiagaraVariable> EncounteredExposedVars;
	TArray<FNiagaraVariable> BakedRapidIterationParameters;
	FNiagaraScriptCompileMetrics CompileMetrics;

	ENiagaraCompilationState CurrentState;

	TSharedPtr<FNiagaraVMExecutableData> ExeData;
	TArray<uint8> DDCOutData;

	// this data is shared between the ddc thread and the game thread that starts the compilation
	TSharedPtr<FNiagaraLazyPrecompileReference, ESPMode::ThreadSafe> PrecompileReference;
	TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> ComputedPrecompileData;
	TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> ComputedPrecompileDuplicateData;

	FNiagaraAsyncCompileTask(UNiagaraSystem* InOwningSystem, FString InAssetPath, const FEmitterCompiledScriptPair& InScriptPair);

	const FEmitterCompiledScriptPair& GetScriptPair() const { return ScriptPair; };
	void WaitAndResolveResult();
	void AbortTask();

	void CheckDDCResult();
	void PutToDDC();
	void ProcessCurrentState();
	void MoveToState(ENiagaraCompilationState NewState);
	bool IsDone() const;

	void PrecompileData();
	void StartCompileJob();
	bool AwaitResult();
	void ProcessResult();
	void OptimizeByteCode();
	void ProcessNonCompilableScript();

	void AssignInitialCompilationId(const FNiagaraVMExecutableDataId& InitialCompilationId);
	void UpdateCompilationId(const FNiagaraVMExecutableDataId& UpdatedCompilationId);
	bool CompilationIdMatchesRequest() const;
#endif
};

class FNiagaraActiveCompilationDefault : public FNiagaraActiveCompilation
{
public:
	virtual ~FNiagaraActiveCompilationDefault();
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("NiagaraActiveCompilationDefault");
	}

	virtual bool Launch(const FNiagaraCompilationOptions& Options) override;
	virtual void Abort() override;
	virtual bool QueryCompileComplete(const FNiagaraQueryCompilationOptions& Options) override;
	virtual bool ValidateConsistentResults(const FNiagaraQueryCompilationOptions& Options) const override;
	virtual void Apply(const FNiagaraQueryCompilationOptions& Options) override;
	virtual void ReportResults(const FNiagaraQueryCompilationOptions& Options) const override;
	virtual bool BlocksBeginCacheForCooked() const override;

	double StartTime = 0.0;

	TSet<TObjectPtr<UObject>> RootObjects;

	using FAsyncTaskPtr = TSharedPtr<FNiagaraAsyncCompileTask, ESPMode::ThreadSafe>;

	FAsyncTaskPtr* FindTask(const UNiagaraScript* Script);
	const FAsyncTaskPtr* FindTask(const UNiagaraScript* Script) const;
	void Reset();

	bool bAllScriptsSynchronized = false;
	bool bEvaluateParametersPending = false;

	TArray<FAsyncTaskPtr> Tasks;

private:
	bool bDDCGetCompleted = false;
};

