// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraAsyncGpuTraceHelper.h"

#include "Components/PrimitiveComponent.h"
#include "NiagaraDataInterface.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraStats.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "RenderingThread.h"

int32 GNiagaraAsyncTraceScratchBucketSize = 1024;
static FAutoConsoleVariableRef CVarNiagaraAsyncTraceScratchBucketSize(
	TEXT("fx.Niagara.AsyncTrace.ScratchPadBucketSize"),
	GNiagaraAsyncTraceScratchBucketSize,
	TEXT("Size (in elements) for async gpu traces scratch buffer buckets. \n"),
	ECVF_Default
);

int32 GNiagaraAsyncTraceCountsScratchBucketSize = 3 * 256;
static FAutoConsoleVariableRef CVarNiagaraAsyncTrceCountsScratchBucketSize(
	TEXT("fx.Niagara.AsyncTrace.CountsScratchPadBucketSize"),
	GNiagaraAsyncTraceCountsScratchBucketSize,
	TEXT("Scratch bucket size for the async gpu trace counts buffer. This buffer requires 4. \n"),
	ECVF_Default
);

FNiagaraAsyncGpuTraceHelper::FNiagaraAsyncGpuTraceHelper(EShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel, FNiagaraGpuComputeDispatchInterface* Dispatcher)
	: ShaderPlatform(InShaderPlatform)
	, FeatureLevel(InFeatureLevel)
	, TraceRequests(GNiagaraAsyncTraceScratchBucketSize, TEXT("NiagaraRayRequests"))
	, TraceResults(GNiagaraAsyncTraceScratchBucketSize, TEXT("NiagaraRayTraceIntersections"))
	, TraceCounts(GNiagaraAsyncTraceCountsScratchBucketSize, TEXT("NiagaraRayTraceCounts"), BUF_Static | BUF_DrawIndirect)
{
	InitProviders(Dispatcher);

#if NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS
	// The owner of the trace helper is constructed on the GameThread so we need to enqueue a command to set the delegates
	ENQUEUE_RENDER_COMMAND(NiagaraAsyncGpuTraceHelperSetupGPUSceneDelegates)
	(
		[RT_Helper=this](FRHICommandListImmediate&)
		{
			FPrimitiveSceneInfo::OnGPUSceneInstancesAllocated.AddRaw(RT_Helper, &FNiagaraAsyncGpuTraceHelper::OnPrimitiveGPUSceneInstancesDirtied);
			FPrimitiveSceneInfo::OnGPUSceneInstancesFreed.AddRaw(RT_Helper, &FNiagaraAsyncGpuTraceHelper::OnPrimitiveGPUSceneInstancesDirtied);
		}
	);
#endif
}

FNiagaraAsyncGpuTraceHelper::~FNiagaraAsyncGpuTraceHelper()
{
#if NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS
	FPrimitiveSceneInfo::OnGPUSceneInstancesAllocated.RemoveAll(this);
	FPrimitiveSceneInfo::OnGPUSceneInstancesFreed.RemoveAll(this);
#endif

	Reset();
	Dispatches.Reset();

	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceRequests.AllocatedBytes());
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceResults.AllocatedBytes());
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceCounts.AllocatedBytes());

	TraceRequests.Release();
	TraceResults.Release();
	TraceCounts.Release();
}

void FNiagaraAsyncGpuTraceHelper::Reset()
{
	for (const auto& TraceProvider : TraceProviders)
	{
		TraceProvider->Reset();
	}
}

void FNiagaraAsyncGpuTraceHelper::InitProviders(FNiagaraGpuComputeDispatchInterface* Dispatcher)
{
	const auto& ProviderPriorities = GetDefault<UNiagaraSettings>()->NDICollisionQuery_AsyncGpuTraceProviderOrder;

	TraceProviders = FNiagaraAsyncGpuTraceProvider::CreateSupportedProviders(ShaderPlatform, Dispatcher, ProviderPriorities);
}

void FNiagaraAsyncGpuTraceHelper::BeginFrame(FRHICommandList& RHICmdList, FNiagaraGpuComputeDispatchInterface* Dispatcher)
{
	//Store off last frames dispatches so we can still access the previous results buffers for reading in this frame's simulations.
	PreviousFrameDispatches = MoveTemp(Dispatches);
	Dispatches.Reset();

	//Ensure we're each buffer is definitely in the right access state.
	TraceRequests.Transition(RHICmdList, ERHIAccess::Unknown, ERHIAccess::UAVCompute);//Simulations will write new ray trace requests.
	TraceResults.Transition(RHICmdList, ERHIAccess::Unknown, ERHIAccess::SRVCompute);//Simulations will read from the previous frames results.
	TraceCounts.Transition(RHICmdList, ERHIAccess::Unknown, ERHIAccess::UAVCompute);//Simulations will accumulate the number of traces requests here.

	//Reset the allocations.
	TraceRequests.Reset();

	//Note this doesn't change any buffers or data themselves and the ray intersections buffer are still going to be read as SRVs in the up coming simulations dispatches.
	//We're just clearing existing allocations here.
	TraceResults.Reset();

	//We have to also clear the counts/indirect args buffer.
	TraceCounts.Reset(RHICmdList, true);

	// Clear the dummy buffer allocations.
	DummyDispatch.Reset();

	// figure out the provider type that should be used for the 'default' case based on the provided scene
	ResolvedDefaultProviderType = ENDICollisionQuery_AsyncGpuTraceProvider::None;

	for (const auto& TraceProvider : TraceProviders)
	{
		if (TraceProvider->IsAvailable())
		{
			ResolvedDefaultProviderType = TraceProvider->GetType();
			break;
		}
	}
}

void FNiagaraAsyncGpuTraceHelper::PostRenderOpaque(FRHICommandList& RHICmdList, FNiagaraGpuComputeDispatchInterface* Dispatcher, TConstStridedView<FSceneView> Views, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI)
{
#if NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS
	if (bCollisionGroupMapDirty)
	{
		FNiagaraAsyncGpuTraceProvider::BuildCollisionGroupHashMap(RHICmdList, FeatureLevel, Dispatcher->GetSceneInterface(), CollisionGroupMap, CollisionGroupHashMapBuffer);
		bCollisionGroupMapDirty = false;
	}
#endif

	FNiagaraAsyncGpuTraceProvider::FCollisionGroupHashMap* CollisionHashMap = GetCollisionGroupHashMap();

	for (const TUniquePtr<FNiagaraAsyncGpuTraceProvider>& TraceProvider : TraceProviders)
	{
		if (TraceProvider->IsAvailable())
		{
			TraceProvider->PostRenderOpaque(RHICmdList, Views, SceneUniformBufferRHI, CollisionHashMap);
		}
	}
}

FNiagaraAsyncGpuTraceProvider* FNiagaraAsyncGpuTraceHelper::GetTraceProvider(ENDICollisionQuery_AsyncGpuTraceProvider::Type ProviderType)
{
	auto* FoundProvider = TraceProviders.FindByPredicate([&](const TUniquePtr<FNiagaraAsyncGpuTraceProvider>& TraceProvider)
	{
		return TraceProvider->GetType() == ProviderType
			|| ((ProviderType == ENDICollisionQuery_AsyncGpuTraceProvider::Default)
				&& (TraceProvider->GetType() == ResolvedDefaultProviderType));
	});

	if (FoundProvider && (*FoundProvider)->IsAvailable())
	{
		return FoundProvider->Get();
	}

	return nullptr;
}

void FNiagaraAsyncGpuTraceHelper::EndFrame(FRHICommandList& RHICmdList, FNiagaraGpuComputeDispatchInterface* Dispatcher, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI)
{
	if (!Dispatches.Num())
	{
		return;
	}

	TraceRequests.Transition(RHICmdList, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute);//Ray trace dispatches will read the ray requests.
	TraceResults.Transition(RHICmdList, ERHIAccess::SRVCompute, ERHIAccess::UAVCompute);//Ray trace dispatches will write new results.
	TraceCounts.Transition(RHICmdList, ERHIAccess::UAVCompute, ERHIAccess::IndirectArgs | ERHIAccess::SRVCompute);//Ray trace dispatches read these counts and use them for indirect dispatches.

	for (const auto& DispatchInfoPair : Dispatches)
	{
		const FNiagaraAsyncGpuTraceDispatchInfo& DispatchInfo = DispatchInfoPair.Value;
		if (DispatchInfo.MaxTraces)
		{
			FNiagaraAsyncGpuTraceProvider::FDispatchRequest Request;
			Request.TracesBuffer = DispatchInfo.TraceRequests.Buffer;
			Request.ResultsBuffer = DispatchInfo.TraceResults.Buffer;
			Request.TraceCountsBuffer = DispatchInfo.TraceCounts.Buffer;
			Request.MaxTraceCount = DispatchInfo.MaxTraces;
			Request.TracesOffset = DispatchInfo.TraceRequests.Offset;
			Request.ResultsOffset = DispatchInfo.TraceResults.Offset;
			Request.TraceCountsOffset = DispatchInfo.TraceCounts.Offset;
			Request.MaxRetraceCount = DispatchInfo.MaxRetraces;

			FNiagaraAsyncGpuTraceProvider::FCollisionGroupHashMap* CollisionHashMap = GetCollisionGroupHashMap();

			if (FNiagaraAsyncGpuTraceProvider* TraceProvider = GetTraceProvider(DispatchInfo.ProviderType))
			{
				TraceProvider->IssueTraces(RHICmdList, Request, SceneUniformBufferRHI, CollisionHashMap);
			}
			else
			{
				// Clear the results
				FNiagaraAsyncGpuTraceProvider::ClearResults(RHICmdList, ShaderPlatform, Request);
			}
		}
	}

	TraceRequests.Transition(RHICmdList, ERHIAccess::SRVCompute, ERHIAccess::UAVCompute);//Next frame, simulations will write new ray trace requests.
	TraceResults.Transition(RHICmdList, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute);//Next frame these results will be read by simulations shaders.
	TraceCounts.Transition(RHICmdList, ERHIAccess::IndirectArgs | ERHIAccess::SRVCompute, ERHIAccess::UAVCompute);//Next frame these counts will be written by simulation shaders.
}

void FNiagaraAsyncGpuTraceHelper::AddToDispatch(FNiagaraDataInterfaceProxy* DispatchKey, uint32 MaxRays, int32 MaxRetraces, ENDICollisionQuery_AsyncGpuTraceProvider::Type ProviderType)
{
	FNiagaraAsyncGpuTraceDispatchInfo& Dispatch = Dispatches.FindOrAdd(DispatchKey);
	Dispatch.MaxTraces += MaxRays;
	Dispatch.MaxRetraces = MaxRetraces;
	Dispatch.ProviderType = ProviderType;
}

void FNiagaraAsyncGpuTraceHelper::BuildDispatch(FRHICommandList& RHICmdList, FNiagaraDataInterfaceProxy* DispatchKey)
{
	FNiagaraAsyncGpuTraceDispatchInfo& Dispatch = Dispatches.FindChecked(DispatchKey);

	//Finalize allocations first time we access the dispatch.
	if (Dispatch.TraceRequests.IsValid() == false)
	{
 		DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceRequests.AllocatedBytes());
 		DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceResults.AllocatedBytes());
 		DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceCounts.AllocatedBytes());

		Dispatch.TraceRequests = TraceRequests.Alloc(RHICmdList, Dispatch.MaxTraces);
		Dispatch.TraceResults = TraceResults.Alloc(RHICmdList, Dispatch.MaxTraces);
		Dispatch.TraceCounts = TraceCounts.Alloc<uint32>(3, RHICmdList, true);

 		INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceRequests.AllocatedBytes());
 		INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceResults.AllocatedBytes());
 		INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceCounts.AllocatedBytes());

		//Find last frame's results buffer if there was one. Ideally we can do this without the map but it will need a bit of wrangling.
		FNiagaraAsyncGpuTraceDispatchInfo* PrevDispatch = PreviousFrameDispatches.Find(DispatchKey);
		if (PrevDispatch && PrevDispatch->TraceResults.IsValid())
		{
			Dispatch.LastFrameTraceResults = PrevDispatch->TraceResults;
		}
		else
		{
			//Set the current frame results just so this is a valid buffer.
			//If we didn't have a results buffer last frame then this won't actually be read.
			Dispatch.LastFrameTraceResults = Dispatch.TraceResults;
		}
	}
}

void FNiagaraAsyncGpuTraceHelper::BuildDummyDispatch(FRHICommandList& RHICmdList)
{
	if (DummyDispatch.IsValid() == false)
	{
 		DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceRequests.AllocatedBytes());
 		DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceResults.AllocatedBytes());
 		DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceCounts.AllocatedBytes());

		DummyDispatch.TraceRequests = TraceRequests.Alloc(RHICmdList, 1);
		DummyDispatch.TraceResults = TraceResults.Alloc(RHICmdList, 1);
		DummyDispatch.LastFrameTraceResults = DummyDispatch.TraceResults;
		DummyDispatch.TraceCounts = TraceCounts.Alloc<uint32>(3, RHICmdList, true);
		
 		INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceRequests.AllocatedBytes());
 		INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceResults.AllocatedBytes());
 		INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TraceCounts.AllocatedBytes());

		DummyDispatch.MaxTraces = 0;
		DummyDispatch.MaxRetraces = 0;
	}
}

const FNiagaraAsyncGpuTraceDispatchInfo& FNiagaraAsyncGpuTraceHelper::GetDispatch(FNiagaraDataInterfaceProxy* DispatchKey) const
{
	const FNiagaraAsyncGpuTraceDispatchInfo& Dispatch = Dispatches.FindChecked(DispatchKey);

	// make sure that the buffer is valid (that BuildDispatch has been called)
	check(Dispatch.TraceRequests.IsValid());

	return Dispatch;
}

const FNiagaraAsyncGpuTraceDispatchInfo& FNiagaraAsyncGpuTraceHelper::GetDummyDispatch() const
{
	// make sure that the buffers are valid (that BuildDummyDispatch has been called)
	check(DummyDispatch.TraceRequests.IsValid());

	return DummyDispatch;
}

ENDICollisionQuery_AsyncGpuTraceProvider::Type FNiagaraAsyncGpuTraceHelper::ResolveSupportedType(ENDICollisionQuery_AsyncGpuTraceProvider::Type InType)
{
	return FNiagaraAsyncGpuTraceProvider::ResolveSupportedType(InType, GetDefault<UNiagaraSettings>()->NDICollisionQuery_AsyncGpuTraceProviderOrder);
}

bool FNiagaraAsyncGpuTraceHelper::RequiresGlobalDistanceField(ENDICollisionQuery_AsyncGpuTraceProvider::Type InType)
{
	return FNiagaraAsyncGpuTraceProvider::RequiresGlobalDistanceField(InType, GetDefault<UNiagaraSettings>()->NDICollisionQuery_AsyncGpuTraceProviderOrder);
}

bool FNiagaraAsyncGpuTraceHelper::RequiresRayTracingScene(ENDICollisionQuery_AsyncGpuTraceProvider::Type InType)
{
	return FNiagaraAsyncGpuTraceProvider::RequiresRayTracingScene(InType, GetDefault<UNiagaraSettings>()->NDICollisionQuery_AsyncGpuTraceProviderOrder);
}

FNiagaraAsyncGpuTraceProvider::FCollisionGroupHashMap* FNiagaraAsyncGpuTraceHelper::GetCollisionGroupHashMap()
{
#if NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS
	return CollisionGroupMap.Num() > 0 ? &CollisionGroupHashMapBuffer : nullptr;
#else
	return nullptr;
#endif
}

#if NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS
void FNiagaraAsyncGpuTraceHelper::SetPrimitiveCollisionGroup(FPrimitiveSceneInfo& Primitive, uint32 CollisionGroup)
{
	CollisionGroupMap.FindOrAdd(Primitive.PrimitiveComponentId) = CollisionGroup;
	bCollisionGroupMapDirty = true;
}

void FNiagaraAsyncGpuTraceHelper::SetPrimitiveRayTracingCollisionGroup_GT(UPrimitiveComponent* Primitive, uint32 Group)
{
	check(IsInGameThread());
	if (Primitive == nullptr || Primitive->SceneProxy == nullptr)
	{
		return;
	}

	ENQUEUE_RENDER_COMMAND(NiagaraSetPrimHWRTCollisionGroup)(
		[RT_Helper = this, Proxy = Primitive->SceneProxy, CollisionGroup = Group](FRHICommandListImmediate& RHICmdList)
	{
		check(Proxy);
		RT_Helper->SetPrimitiveCollisionGroup(*Proxy->GetPrimitiveSceneInfo(), CollisionGroup);
	});
}

int32 FNiagaraAsyncGpuTraceHelper::AcquireGPURayTracedCollisionGroup_GT()
{
	check(IsInGameThread());//one of the few batcher functions that should be called from the GT
	if (FreeGPURayTracedCollisionGroups.Num() > 0)
	{
		return FreeGPURayTracedCollisionGroups.Pop();
	}

	return NumGPURayTracedCollisionGroups++;
}

void FNiagaraAsyncGpuTraceHelper::ReleaseGPURayTracedCollisionGroup_GT(int32 CollisionGroup)
{
	check(IsInGameThread());//one of the few batcher functions that should be called from the GT
	FreeGPURayTracedCollisionGroups.Add(CollisionGroup);
}

void FNiagaraAsyncGpuTraceHelper::OnPrimitiveGPUSceneInstancesDirtied()
{
	bCollisionGroupMapDirty = true;
}
#endif
