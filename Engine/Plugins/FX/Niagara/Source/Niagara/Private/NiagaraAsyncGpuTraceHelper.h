// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraAsyncGpuTraceProvider.h"
#include "NiagaraGpuScratchPad.h"
#include "NiagaraSettings.h"
#include "PrimitiveComponentId.h"
#include "RHIDefinitions.h"
#include "RHIUtilities.h"

struct FNiagaraDataInterfaceProxy;
class FNiagaraGpuComputeDispatchInterface;
class FPrimitiveSceneInfo;
class FRayTracingPipelineState;
class FRHICommandList;
class FRHIRayTracingScene;
class FRHIShaderResourceView;
class FRHIUniformBuffer;
class FRHIUnorderedAccessView;
struct FRWBuffer;
class FScene;
class FSceneView;

// we currently support assigning, from blueprint, primitives into collision groups which can be excluded
// from ray traces.  Currently only supported with HWRT
#define NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS RHI_RAYTRACING

/** Holds all information on a single RayTracing dispatch. */
struct FNiagaraAsyncGpuTraceDispatchInfo
{
	/** Buffer allocation for ray requests to trace. */
	FNiagaraGpuScratchPadStructured<FNiagaraAsyncGpuTrace>::FAllocation TraceRequests;
	/** Buffer allocation for writing trace results into. */
	FNiagaraGpuScratchPadStructured<FNiagaraAsyncGpuTraceResult>::FAllocation TraceResults;
	/** Buffer allocation for last frames trace results to read from in this frame's simulation. */
	FNiagaraGpuScratchPadStructured<FNiagaraAsyncGpuTraceResult>::FAllocation LastFrameTraceResults;
	/** Buffer allocation for ray trace counts. Accumulated in simulations shaders and used as dispatch args for ray tracing shaders. */
	FNiagaraGpuScratchPad::FAllocation TraceCounts;
	/** Total possible rays to trace. This may be significantly higher than the actual rays request we accumulate into RayCounts. */
	uint32 MaxTraces = 0;
	/** Max number or times rays can be re-traced in a shader if they hit something that is invalid/filtered. */
	uint32 MaxRetraces = 0;

	ENDICollisionQuery_AsyncGpuTraceProvider::Type ProviderType = ENDICollisionQuery_AsyncGpuTraceProvider::Default;

	FORCEINLINE bool IsValid() const { return MaxTraces > 0; }
	FORCEINLINE void Reset()
	{
		TraceRequests = FNiagaraGpuScratchPadStructured<FNiagaraAsyncGpuTrace>::FAllocation();
		TraceResults = FNiagaraGpuScratchPadStructured<FNiagaraAsyncGpuTraceResult>::FAllocation();
		TraceCounts = FNiagaraGpuScratchPad::FAllocation();
		MaxTraces = 0;
		MaxRetraces = 0;
		ProviderType = ENDICollisionQuery_AsyncGpuTraceProvider::Default;
	}
};

class FNiagaraAsyncGpuTraceHelper
{
public:
	FNiagaraAsyncGpuTraceHelper(EShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type FeatureLevel, FNiagaraGpuComputeDispatchInterface* Dispatcher);
	FNiagaraAsyncGpuTraceHelper() = delete;
	~FNiagaraAsyncGpuTraceHelper();

	void Reset();

	void BeginFrame(FRHICommandList& RHICmdList, FNiagaraGpuComputeDispatchInterface* Dispatcher);
	void PostRenderOpaque(FRHICommandList& RHICmdList, FNiagaraGpuComputeDispatchInterface* Dispatcher, TConstStridedView<FSceneView> Views, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI);
	void EndFrame(FRHICommandList& RHICmdList, FNiagaraGpuComputeDispatchInterface* Dispatcher, TUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRHI);

	/** Accumulates ray requests from user DIs into single dispatches per DI. */
	void AddToDispatch(FNiagaraDataInterfaceProxy* DispatchKey, uint32 MaxRays, int32 MaxRetraces, ENDICollisionQuery_AsyncGpuTraceProvider::Type ProviderType);

	/** Ensures that the final buffers for the provided proxy have been allocated. */
	void BuildDispatch(FRHICommandList& RHICmdList, FNiagaraDataInterfaceProxy* DispatchKey);

	/** Ensures that buffers for a dummy buffer have been allocated. */
	void BuildDummyDispatch(FRHICommandList& RHICmdList);

	/** Returns the final buffers for each DI for use in simulations and RT dispatches. */
	const FNiagaraAsyncGpuTraceDispatchInfo& GetDispatch(FNiagaraDataInterfaceProxy* DispatchKey) const;

	/** Returns a dummy dispatch to allow shaders to still bind to valid buffers. */
	const FNiagaraAsyncGpuTraceDispatchInfo& GetDummyDispatch() const;

	FNiagaraGpuScratchPadStructured<FNiagaraAsyncGpuTrace>& GetRayRequests(){ return TraceRequests; }
	FNiagaraGpuScratchPadStructured<FNiagaraAsyncGpuTraceResult> GetRayTraceIntersections() { return TraceResults; }
	FNiagaraGpuScratchPad GetTraceCounts() { return TraceCounts; }

	static ENDICollisionQuery_AsyncGpuTraceProvider::Type ResolveSupportedType(ENDICollisionQuery_AsyncGpuTraceProvider::Type InType);
	static bool RequiresGlobalDistanceField(ENDICollisionQuery_AsyncGpuTraceProvider::Type InType);
	static bool RequiresRayTracingScene(ENDICollisionQuery_AsyncGpuTraceProvider::Type InType);

#if NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS
	/** Adds a primitive to a collision group. This data is sent to the GPU on the start of the next frame. */
	void SetPrimitiveCollisionGroup(FPrimitiveSceneInfo& Primitive, uint32 CollisionGroup);

	/** Pushes cpu side copy of the collision group map to the GPU. */
	void UpdateCollisionGroupMap(FRHICommandList& RHICmdList, FScene* Scene, ERHIFeatureLevel::Type FeatureLevel);

	// Set of functions for managing, from the game thread, the creation and assignment of collision groups; currently
	// only supported with ray traced collisions
	void SetPrimitiveRayTracingCollisionGroup_GT(UPrimitiveComponent* Primitive, uint32 Group);
	int32 AcquireGPURayTracedCollisionGroup_GT();
	void ReleaseGPURayTracedCollisionGroup_GT(int32 CollisionGroup);

	void OnPrimitiveGPUSceneInstancesDirtied();
#endif

private:
	const EShaderPlatform ShaderPlatform;
	const ERHIFeatureLevel::Type FeatureLevel;

	/** Scratch buffer holding all ray trace requests for all simulations in the scene. */
	FNiagaraGpuScratchPadStructured<FNiagaraAsyncGpuTrace> TraceRequests;
	/** Scratch buffer holding all ray trace results for all simulations in the scene. */
	FNiagaraGpuScratchPadStructured<FNiagaraAsyncGpuTraceResult> TraceResults;
	/** Scratch buffer holding all accumulated ray request counts for all simulations in the scene. */
	FNiagaraGpuScratchPad TraceCounts;

	/** Map of DI Proxy to Dispatch Info. Each DI will have a single dispatch for all it's instances. */
	TMap<FNiagaraDataInterfaceProxy*, FNiagaraAsyncGpuTraceDispatchInfo> Dispatches;

	/** Last frame's DI proxy map. Allows use to retrieve the buffer allocations to read last frames trace results. TODO: Improve. Ideally don't need a map copy here. */
	TMap<FNiagaraDataInterfaceProxy*, FNiagaraAsyncGpuTraceDispatchInfo> PreviousFrameDispatches;

	/** In places where ray tracing is disabled etc we use a dummy dispatch allocation set so the simulations can bind to valid buffers. */
	FNiagaraAsyncGpuTraceDispatchInfo DummyDispatch;

	TArray<TUniquePtr<FNiagaraAsyncGpuTraceProvider>> TraceProviders;

	ENDICollisionQuery_AsyncGpuTraceProvider::Type ResolvedDefaultProviderType = ENDICollisionQuery_AsyncGpuTraceProvider::None;

	FNiagaraAsyncGpuTraceProvider* GetTraceProvider(ENDICollisionQuery_AsyncGpuTraceProvider::Type ProviderType);
	void InitProviders(FNiagaraGpuComputeDispatchInterface* Dispatcher);

	FNiagaraAsyncGpuTraceProvider::FCollisionGroupHashMap* GetCollisionGroupHashMap();

#if NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS
	/**
	CPU side copy of the PrimID to Collision Group Map.
	This is uploaded to the GPU and used in Niagara RG shader to filter self collisions between objects of the same group.
	*/
	TMap<FPrimitiveComponentId, uint32> CollisionGroupMap;

	/**
	GPU side of the collision group hash map
	*/
	FNiagaraAsyncGpuTraceProvider::FCollisionGroupHashMap CollisionGroupHashMapBuffer;

	/** Pool of free GPU ray traced collision groups. */
	TArray<int32> FreeGPURayTracedCollisionGroups;
	int32 NumGPURayTracedCollisionGroups = 0;
	bool bCollisionGroupMapDirty = true;
#endif
};
