// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerOutput.h"
#include "Misc/Paths.h"
#include "NiagaraSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraBakerOutput)

#if WITH_EDITOR
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Factories/Factory.h"
#include "IAssetTools.h"
#include "Misc/MessageDialog.h"
#include "AssetToolsModule.h"
#endif //WITH_EDITOR

FString UNiagaraBakerOutput::MakeOutputName() const
{
	return SanitizeOutputName(GetFName().ToString());
}

FString UNiagaraBakerOutput::SanitizeOutputName(FString Name)
{
	FString OutName;
	OutName.Reserve(Name.Len());
	for ( TCHAR ch : Name )
	{
		switch (ch)
		{
			case TCHAR(' '):
			case TCHAR(';'):
			case TCHAR(':'):
			case TCHAR(','):
				ch = TCHAR('_');
				break;
			default:
				break;
		}
		OutName.AppendChar(ch);
	}

	return OutName;
}

#if WITH_EDITOR
void UNiagaraBakerOutput::FindWarnings(TArray<FText>& OutWarnings) const
{
}

FString UNiagaraBakerOutput::GetAssetPath(FString PathFormat, int32 FrameIndex) const
{
	UNiagaraSystem* NiagaraSystem = GetTypedOuter<UNiagaraSystem>();
	check(NiagaraSystem);

	const TMap<FString, FStringFormatArg> PathFormatArgs =
	{
		{TEXT("AssetFolder"),	FString(FPathViews::GetPath(NiagaraSystem->GetPathName()))},
		{TEXT("AssetName"),		NiagaraSystem->GetName()},
		{TEXT("OutputName"),	SanitizeOutputName(OutputName)},
		{TEXT("FrameIndex"),	FString::Printf(TEXT("%03d"), FrameIndex)},
	};
	FString AssetPath = FString::Format(*PathFormat, PathFormatArgs);
	AssetPath.ReplaceInline(TEXT("//"), TEXT("/"));
	return AssetPath;
}

FString UNiagaraBakerOutput::GetAssetFolder(FString PathFormat, int32 FrameIndex) const
{
	const FString AssetPath = GetAssetPath(PathFormat, FrameIndex);
	return FString(FPathViews::GetPath(AssetPath));
}

FString UNiagaraBakerOutput::GetExportPath(FString PathFormat, int32 FrameIndex) const
{
	UNiagaraSystem* NiagaraSystem = GetTypedOuter<UNiagaraSystem>();
	check(NiagaraSystem);

	const TMap<FString, FStringFormatArg> PathFormatArgs =
	{
		{TEXT("SavedDir"),		FPaths::ProjectSavedDir()},
		{TEXT("ProjectDir"),	FPaths::GetProjectFilePath()},
		{TEXT("AssetName"),		NiagaraSystem->GetName()},
		{TEXT("OutputName"),	SanitizeOutputName(OutputName)},
		{TEXT("FrameIndex"),	FString::Printf(TEXT("%03d"), FrameIndex)},
	};
	FString ExportPath = FString::Format(*PathFormat, PathFormatArgs);
	ExportPath.ReplaceInline(TEXT("//"), TEXT("/"));
	return FPaths::ConvertRelativePathToFull(ExportPath);
}

FString UNiagaraBakerOutput::GetExportFolder(FString PathFormat, int32 FrameIndex) const
{
	const FString AssetPath = GetExportPath(PathFormat, FrameIndex);
	return FString(FPathViews::GetPath(AssetPath));
}

UObject* UNiagaraBakerOutput::GetOrCreateAsset(const FString& PackagePath, UClass* ObjectClass, UClass* FactoryClass)
{
	// Find existing
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	TArray<FAssetData> FoundAssets;
	if (AssetRegistry->GetAssetsByPackageName(FName(PackagePath), FoundAssets))
	{
		if (FoundAssets.Num() > 0)
		{
			if (UObject* ExistingOject = StaticLoadObject(ObjectClass, nullptr, *PackagePath))
			{
				return ExistingOject;
			}

			// If the above failed then the asset is not the right type, warn the user
			FText ErrorMessage = FText::Format(
				NSLOCTEXT("NiagarBaker", "GetOrCreateAsset_PackageExistsOfWrongType", "Could not bake asset '{0}' as package exists but is not a {1}.\nPlease delete the asset or the output to a different location."),
				FText::FromString(PackagePath),
				FText::FromName(ObjectClass->GetFName())
			);
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
			return nullptr;
		}
	}

	// Create new
	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	UFactory* NewFactory = NewObject<UFactory>(GetTransientPackage(), FactoryClass);
	return AssetTools.CreateAsset(FString(FPathViews::GetCleanFilename(PackagePath)), FString(FPathViews::GetPath(PackagePath)), ObjectClass, NewFactory);
}
#endif

void UNiagaraBakerOutput::PostInitProperties()
{
	Super::PostInitProperties();

	OutputName = MakeOutputName();
}

