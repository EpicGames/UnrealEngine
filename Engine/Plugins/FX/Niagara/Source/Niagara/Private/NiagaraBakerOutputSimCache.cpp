// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerOutputSimCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraBakerOutputSimCache)

bool UNiagaraBakerOutputSimCache::Equals(const UNiagaraBakerOutput& OtherBase) const
{
	const UNiagaraBakerOutputSimCache& Other = *CastChecked<UNiagaraBakerOutputSimCache>(&OtherBase);
	return
		Super::Equals(Other);
}

#if WITH_EDITOR
FString UNiagaraBakerOutputSimCache::MakeOutputName() const
{
	return FString::Printf(TEXT("SimCache_%d"), GetFName().GetNumber());
}
#endif

#if WITH_EDITORONLY_DATA
void UNiagaraBakerOutputSimCache::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

}
#endif

