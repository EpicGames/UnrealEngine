// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerOutputSparseVolumeTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraBakerOutputSparseVolumeTexture)

bool UNiagaraBakerOutputSparseVolumeTexture::Equals(const UNiagaraBakerOutput& OtherBase) const
{
	const UNiagaraBakerOutputSparseVolumeTexture& Other = *CastChecked<UNiagaraBakerOutputSparseVolumeTexture>(&OtherBase);
	return
		Super::Equals(Other) &&
		SourceBinding.SourceName == Other.SourceBinding.SourceName && 
		VolumeWorldSpaceSizeBinding == Other.VolumeWorldSpaceSizeBinding &&
		SparseVolumeTextureAssetPathFormat == Other.SparseVolumeTextureAssetPathFormat &&
		bEnableLoopedOutput == Other.bEnableLoopedOutput &&
		LoopedSparseVolumeTextureAssetPathFormat == Other.LoopedSparseVolumeTextureAssetPathFormat &&
		StartTime == Other.StartTime &&
		BlendDuration == Other.BlendDuration;
}

#if WITH_EDITOR
FString UNiagaraBakerOutputSparseVolumeTexture::MakeOutputName() const
{
	return FString::Printf(TEXT("SparseVolume_%d"), GetFName().GetNumber());
}
#endif

#if WITH_EDITORONLY_DATA
void UNiagaraBakerOutputSparseVolumeTexture::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

}
#endif

