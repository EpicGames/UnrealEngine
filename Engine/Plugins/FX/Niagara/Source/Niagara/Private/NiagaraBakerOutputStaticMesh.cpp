// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerOutputStaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraBakerOutputStaticMesh)

bool UNiagaraBakerOutputStaticMesh::Equals(const UNiagaraBakerOutput& OtherBase) const
{
	const UNiagaraBakerOutputStaticMesh& Other = *CastChecked<UNiagaraBakerOutputStaticMesh>(&OtherBase);
	return
		Super::Equals(Other);
}

#if WITH_EDITOR
FString UNiagaraBakerOutputStaticMesh::MakeOutputName() const
{
	return FString::Printf(TEXT("StaticMesh_%d"), GetFName().GetNumber());
}
#endif

#if WITH_EDITORONLY_DATA
void UNiagaraBakerOutputStaticMesh::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

}
#endif

