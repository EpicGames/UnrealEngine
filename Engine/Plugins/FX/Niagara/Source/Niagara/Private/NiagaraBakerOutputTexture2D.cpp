// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerOutputTexture2D.h"
#include "NiagaraBakerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraBakerOutputTexture2D)

#define LOCTEXT_NAMESPACE "NiagaraBakerOutputTexture2D"

bool UNiagaraBakerOutputTexture2D::Equals(const UNiagaraBakerOutput& OtherBase) const
{
	const UNiagaraBakerOutputTexture2D& Other = *CastChecked<UNiagaraBakerOutputTexture2D>(&OtherBase);
	return
		Super::Equals(Other) &&
		SourceBinding.SourceName == Other.SourceBinding.SourceName &&
		bGenerateAtlas == Other.bGenerateAtlas &&
		bGenerateFrames == Other.bGenerateFrames &&
		bExportFrames == Other.bExportFrames &&
		FrameSize == Other.FrameSize &&
		AtlasTextureSize == Other.AtlasTextureSize;
}

void UNiagaraBakerOutputTexture2D::PostInitProperties()
{
	Super::PostInitProperties();

	if ( UNiagaraBakerSettings* BakerSettings = GetTypedOuter<UNiagaraBakerSettings>() )
	{
		AtlasTextureSize.X = FrameSize.X * BakerSettings->FramesPerDimension.X;
		AtlasTextureSize.Y = FrameSize.Y * BakerSettings->FramesPerDimension.Y;
	}
}

#if WITH_EDITOR
FString UNiagaraBakerOutputTexture2D::MakeOutputName() const
{
	return FString::Printf(TEXT("Texture2D_%d"), GetFName().GetNumber());
}

void UNiagaraBakerOutputTexture2D::FindWarnings(TArray<FText>& OutWarnings) const
{
	if ( bGenerateAtlas )
	{
		if (const UNiagaraBakerSettings* BakerSettings = GetTypedOuter<UNiagaraBakerSettings>())
		{
			const FIntPoint ExpectedAtlasSize(FrameSize.X * BakerSettings->FramesPerDimension.X, FrameSize.Y * BakerSettings->FramesPerDimension.Y);
			if ( (AtlasTextureSize.X != ExpectedAtlasSize.X) || (AtlasTextureSize.Y != ExpectedAtlasSize.Y) )
			{
				OutWarnings.Emplace(
					FText::Format(LOCTEXT("TextureSizeWarningFormat", "Output '{0}' expect atlas size ({1} x {2}) does not match atlas size ({3} x {4}), this can result in jitter on the flipbook if not taken into account"),
					FText::FromString(OutputName),
					ExpectedAtlasSize.X, ExpectedAtlasSize.Y,
					AtlasTextureSize.X, AtlasTextureSize.Y
				));
			}
		}
	}
}
#endif

#if WITH_EDITORONLY_DATA
void UNiagaraBakerOutputTexture2D::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UNiagaraBakerSettings* Settings = GetTypedOuter<UNiagaraBakerSettings>();
	if ( !ensure(Settings) )
	{
		return;
	}

	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	const FName MemberPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if ( MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerOutputTexture2D, FrameSize) )
	{
		FrameSize.X = FMath::Max(FrameSize.X, 1);
		FrameSize.Y = FMath::Max(FrameSize.Y, 1);
		AtlasTextureSize.X = FrameSize.X * Settings->FramesPerDimension.X;
		AtlasTextureSize.Y = FrameSize.Y * Settings->FramesPerDimension.Y;
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerOutputTexture2D, AtlasTextureSize))
	{
		AtlasTextureSize.X = FMath::Max(AtlasTextureSize.X, 1);
		AtlasTextureSize.Y = FMath::Max(AtlasTextureSize.Y, 1);
		FrameSize.X = FMath::DivideAndRoundDown(AtlasTextureSize.X, Settings->FramesPerDimension.X);
		FrameSize.Y = FMath::DivideAndRoundDown(AtlasTextureSize.Y, Settings->FramesPerDimension.Y);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerSettings, FramesPerDimension))
	{
		FrameSize.X = FMath::DivideAndRoundDown(AtlasTextureSize.X, Settings->FramesPerDimension.X);
		FrameSize.Y = FMath::DivideAndRoundDown(AtlasTextureSize.Y, Settings->FramesPerDimension.Y);
	}
}
#endif

#undef LOCTEXT_NAMESPACE

