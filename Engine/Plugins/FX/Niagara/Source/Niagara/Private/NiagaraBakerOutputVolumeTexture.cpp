// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerOutputVolumeTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraBakerOutputVolumeTexture)

bool UNiagaraBakerOutputVolumeTexture::Equals(const UNiagaraBakerOutput& OtherBase) const
{
	const UNiagaraBakerOutputVolumeTexture& Other = *CastChecked<UNiagaraBakerOutputVolumeTexture>(&OtherBase);
	return
		Super::Equals(Other) &&
		SourceBinding.SourceName == Other.SourceBinding.SourceName;
		// As these don't impact the generation process we don't consider for Equals
		//bGenerateAssets == Other.bGenerateAssets &&
		//AssetPathFormat == Other.AssetPathFormat &&
		//bExportGenerated == Other.bExportGenerated &&
		//ExportPathFormat == Other.ExportPathFormat;
}

#if WITH_EDITOR
FString UNiagaraBakerOutputVolumeTexture::MakeOutputName() const
{
	return FString::Printf(TEXT("Volume_%d"), GetFName().GetNumber());
}
#endif

#if WITH_EDITORONLY_DATA
void UNiagaraBakerOutputVolumeTexture::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

}
#endif

