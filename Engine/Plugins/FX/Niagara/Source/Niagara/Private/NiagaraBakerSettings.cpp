// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerSettings.h"
#include "EngineDefines.h"
#include "NiagaraBakerOutputTexture2D.h"
#include "Math/InverseRotationMatrix.h"
#include "NiagaraCustomVersion.h"
#include "Engine/Texture2D.h"
#include "Math/OrthoMatrix.h"
#include "Misc/PathViews.h"
#include "Math/PerspectiveMatrix.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraBakerSettings)

UNiagaraBakerSettings::UNiagaraBakerSettings(const FObjectInitializer& Init)
	: Super(Init)
{
	bLockToSimulationFrameRate = false;
	bPreviewLooping = true;
	bRenderComponentOnly = true;
	bPreviewLoopedOutput = true;
	Outputs.Add(Init.CreateDefaultSubobject<UNiagaraBakerOutputTexture2D>(this, "DefaultOutput"));

	// Fill in default camera settings
	CameraSettings.AddDefaulted(int(ENiagaraBakerViewMode::Num));
	for (int i=0; i < int(ENiagaraBakerViewMode::Num); ++i)
	{
		CameraSettings[i].ViewMode = ENiagaraBakerViewMode(i);
		CameraSettings[i].ResetToDefault();
	}

	// Deprecate properties
	for ( int i=0; i < (int)ENiagaraBakerViewMode::Num; ++i )
	{
		CameraViewportLocation_DEPRECATED[i] = FVector::ZeroVector;
		CameraViewportRotation_DEPRECATED[i] = FRotator::ZeroRotator;
	}
	CameraViewportLocation_DEPRECATED[(int)ENiagaraBakerViewMode::Perspective] = FVector(0.0f, -200.0f, 0.0f);
	CameraViewportRotation_DEPRECATED[(int)ENiagaraBakerViewMode::Perspective] = FRotator(180.0f, 0.0f, 90.0f);
}

bool UNiagaraBakerSettings::Equals(const UNiagaraBakerSettings& Other) const
{
	auto OutputsEquals =
		[&]() -> bool
		{
			if (Outputs.Num() != Other.Outputs.Num())
			{
				return false;
			}

			for (int i = 0; i < Outputs.Num(); ++i)
			{
				if ((Outputs[i]->GetClass() != Other.Outputs[i]->GetClass()) ||
					(Outputs[i]->Equals(*Other.Outputs[i]) == false) )
				{
					return false;
				}
			}
			return true;
		};

	return
		CameraSettings == Other.CameraSettings &&
		CurrentCameraIndex == Other.CurrentCameraIndex &&
		OutputsEquals() &&
		FMath::IsNearlyEqual(StartSeconds, Other.StartSeconds) &&
		FMath::IsNearlyEqual(DurationSeconds, Other.DurationSeconds) &&
		FramesPerSecond == Other.FramesPerSecond &&
		FramesPerDimension == Other.FramesPerDimension &&
		bPreviewLooping == Other.bPreviewLooping &&
		bRenderComponentOnly == Other.bRenderComponentOnly && 
		bPreviewLoopedOutput == Other.bPreviewLoopedOutput;
}

FVector UNiagaraBakerSettings::GetCameraLocation() const
{
	const FNiagaraBakerCameraSettings& CurrentCamera = GetCurrentCamera();
	if (CurrentCamera.IsPerspective())
	{
		const FVector OrbitOffset = CurrentCamera.ViewportRotation.RotateVector(FVector(0.0f, 0.0f, CurrentCamera.OrbitDistance));
		return CurrentCamera.ViewportLocation - OrbitOffset;
	}
	else
	{
		return CurrentCamera.ViewportLocation;
	}
}

FRotator UNiagaraBakerSettings::GetCameraRotation() const
{
	return GetCurrentCamera().ViewportRotation;
}

FMatrix UNiagaraBakerSettings::GetViewportMatrix() const
{
	switch (GetCurrentCamera().ViewMode)
	{
		case ENiagaraBakerViewMode::OrthoFront:		return FMatrix(-FVector::ZAxisVector, -FVector::XAxisVector,  FVector::YAxisVector, FVector::ZeroVector);
		case ENiagaraBakerViewMode::OrthoBack:		return FMatrix( FVector::ZAxisVector,  FVector::XAxisVector,  FVector::YAxisVector, FVector::ZeroVector);
		case ENiagaraBakerViewMode::OrthoLeft:		return FMatrix(-FVector::XAxisVector,  FVector::ZAxisVector,  FVector::YAxisVector, FVector::ZeroVector);
		case ENiagaraBakerViewMode::OrthoRight:		return FMatrix( FVector::XAxisVector, -FVector::ZAxisVector,  FVector::YAxisVector, FVector::ZeroVector);
		case ENiagaraBakerViewMode::OrthoTop:		return FMatrix( FVector::XAxisVector, -FVector::YAxisVector, -FVector::ZAxisVector, FVector::ZeroVector);
		case ENiagaraBakerViewMode::OrthoBottom:	return FMatrix(-FVector::XAxisVector, -FVector::YAxisVector,  FVector::ZAxisVector, FVector::ZeroVector);

		default: return FMatrix::Identity;
	}
}

FMatrix UNiagaraBakerSettings::GetViewMatrix() const
{
	return FInverseRotationMatrix(GetCameraRotation()) * GetViewportMatrix();
}

FMatrix UNiagaraBakerSettings::GetProjectionMatrix() const
{
	const FNiagaraBakerCameraSettings& CurrentCamera = GetCurrentCamera();
	const float AspectRatioY = CurrentCamera.bUseAspectRatio ? CurrentCamera.AspectRatio : 1.0f;
	if (CurrentCamera.ViewMode == ENiagaraBakerViewMode::Perspective)
	{
		const float HalfXFOV = FMath::DegreesToRadians(CurrentCamera.FOV) * 0.5f;
		const float HalfYFOV = FMath::Atan(FMath::Tan(HalfXFOV) / AspectRatioY);
		return FReversedZPerspectiveMatrix(HalfXFOV, HalfYFOV, 1.0f, 1.0, GNearClippingPlane, GNearClippingPlane);
	}
	else
	{
		const FMatrix::FReal ZRange = UE_OLD_WORLD_MAX;
		return FReversedZOrthoMatrix(CurrentCamera.OrthoWidth / 2.0f, CurrentCamera.OrthoWidth * AspectRatioY / 2.0f, 0.5f / ZRange, ZRange);
	}
}

int UNiagaraBakerSettings::GetOutputNumFrames(UNiagaraBakerOutput* BakerOutput) const
{
	return FramesPerDimension.X * FramesPerDimension.Y;
}

FNiagaraBakerOutputFrameIndices UNiagaraBakerSettings::GetOutputFrameIndices(UNiagaraBakerOutput* BakerOutput, float RelativeTime) const
{
	FNiagaraBakerOutputFrameIndices DisplayInfo;
	DisplayInfo.NumFrames = GetOutputNumFrames(BakerOutput);
	DisplayInfo.NormalizedTime = FMath::Max(RelativeTime / DurationSeconds, 0.0f);
	DisplayInfo.NormalizedTime = bPreviewLooping ? FMath::Fractional(DisplayInfo.NormalizedTime) : FMath::Min(DisplayInfo.NormalizedTime, 0.9999f);

	const float FrameTime = DisplayInfo.NormalizedTime * float(DisplayInfo.NumFrames);
	DisplayInfo.FrameIndexA = FMath::FloorToInt(FrameTime);
	DisplayInfo.FrameIndexB = bPreviewLooping ? (DisplayInfo.FrameIndexA + 1) % DisplayInfo.NumFrames : FMath::Min(DisplayInfo.FrameIndexA + 1, DisplayInfo.NumFrames - 1);
	DisplayInfo.Interp = FrameTime - float(DisplayInfo.FrameIndexA);

	return DisplayInfo;
}

int UNiagaraBakerSettings::GetOutputNumFrames(int OutputIndex) const
{
	return Outputs.IsValidIndex(OutputIndex) ? GetOutputNumFrames(Outputs[OutputIndex]) : 0;
}

FNiagaraBakerOutputFrameIndices UNiagaraBakerSettings::GetOutputFrameIndices(int OutputIndex, float RelativeTime) const
{
	return Outputs.IsValidIndex(OutputIndex) ? GetOutputFrameIndices(Outputs[OutputIndex], RelativeTime) : FNiagaraBakerOutputFrameIndices();
}

void UNiagaraBakerSettings::PostLoad()
{
	Super::PostLoad();

	const int32 NiagaraVersion = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);

	if ( OutputTextures_DEPRECATED.Num() > 0 )
	{
		Outputs.Empty();
		for (FNiagaraBakerTextureSettings& Texture : OutputTextures_DEPRECATED)
		{
			UNiagaraBakerOutputTexture2D* NewOutput = NewObject<UNiagaraBakerOutputTexture2D>(this);
			NewOutput->SourceBinding		= Texture.SourceBinding;

			if ( Texture.bUseFrameSize )
			{
				NewOutput->FrameSize		= Texture.FrameSize;
				NewOutput->AtlasTextureSize	= FIntPoint(Texture.FrameSize.X * FramesPerDimension.X, Texture.FrameSize.Y * FramesPerDimension.Y);
			}
			else
			{
				NewOutput->FrameSize		= FIntPoint(Texture.TextureSize.X / FramesPerDimension.X, Texture.TextureSize.Y / FramesPerDimension.Y);
				NewOutput->AtlasTextureSize	= Texture.TextureSize;
			}

			if (!Texture.OutputName.IsNone())
			{
				const FString OutputName = Texture.OutputName.ToString();
				if (OutputName.Len() > 0)
				{
					NewOutput->OutputName = OutputName;
				}
			}

			if ( Texture.GeneratedTexture )
			{
				NewOutput->AtlasAssetPathFormat = Texture.GeneratedTexture->GetPackage()->GetPathName();
			}

			Outputs.Add(NewOutput);
		}

		OutputTextures_DEPRECATED.Empty();
	}

	if (NiagaraVersion < FNiagaraCustomVersion::AddBakerCameraBookmarks)
	{
		for ( int32 i=0; i < int(ENiagaraBakerViewMode::Num); ++i )
		{
			CameraSettings[i].ViewMode = ENiagaraBakerViewMode(i);
			CameraSettings[i].ViewportLocation = CameraViewportLocation_DEPRECATED[i];
			CameraSettings[i].ViewportRotation = CameraViewportRotation_DEPRECATED[i];
			CameraSettings[i].OrbitDistance = CameraOrbitDistance_DEPRECATED;
			CameraSettings[i].FOV = CameraFOV_DEPRECATED;
			CameraSettings[i].OrthoWidth = CameraOrthoWidth_DEPRECATED;
			CameraSettings[i].bUseAspectRatio = bUseCameraAspectRatio_DEPRECATED;
			CameraSettings[i].AspectRatio = CameraAspectRatio_DEPRECATED;
		}
		CurrentCameraIndex = int(CameraViewportMode_DEPRECATED);
	}

	// Sanitize outputs are not nullptr
	// This should not occur but an asset has been seen to get into this state
	Outputs.Remove(nullptr);
}

#if WITH_EDITORONLY_DATA
void UNiagaraBakerSettings::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraBakerOutputTexture2D::StaticClass()));
}
#endif

#if WITH_EDITORONLY_DATA
void UNiagaraBakerSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	for ( UNiagaraBakerOutput* Output : Outputs )
	{
		if (Output)
		{
			Output->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
}
#endif

