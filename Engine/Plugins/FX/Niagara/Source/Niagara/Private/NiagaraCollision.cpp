// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCollision.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "NiagaraStats.h"
#include "Misc/ScopeRWLock.h"
#include "PhysicalMaterials/PhysicalMaterial.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraCollision)

DECLARE_CYCLE_STAT(TEXT("Event Emission"), STAT_NiagaraEventWrite, STATGROUP_Niagara);

const FName FNiagaraDICollisionQueryBatch::CollisionTagName = FName("Niagara");
const FName FNiagaraDICollisionQueryBatch::TraceTagName = FName("NiagaraCollision");

static int32 GNiagaraCollisionCPUEnabled = 1;
static FAutoConsoleVariableRef CVarNiagaraCollisionEnabled(
	TEXT("fx.Niagara.Collision.CPUEnabled"),
	GNiagaraCollisionCPUEnabled,
	TEXT("Controls if CPU collisions are enabled or not."),
	ECVF_Default
);

void FNiagaraDICollisionQueryBatch::DispatchQueries()
{
	check(IsInGameThread());
	CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_Collision);

	// swap the buffers, and use our new read buffer for issuing all of our accumulated queries now that we're on the main thread
	FlipBuffers();

	// locks are not used here because we assume that the per instance ticking is done from the main thread
	if (CollisionWorld)
	{
		const int32 ReadBufferIdx = GetReadBufferIdx();
		const int32 TraceCount = CollisionTraces[ReadBufferIdx].Num();
		if (TraceCount > 0)
		{
			SCOPE_CYCLE_COUNTER(STAT_NiagaraCollision);

			for (int32 TraceIt = 0; TraceIt < TraceCount; ++TraceIt)
			{
				FNiagaraCollisionTrace& CollisionTrace = CollisionTraces[ReadBufferIdx][TraceIt];

				CollisionTrace.CollisionTraceHandle = CollisionWorld->AsyncLineTraceByChannel(
					EAsyncTraceType::Single,
					CollisionTrace.StartPos,
					CollisionTrace.EndPos,
					CollisionTrace.Channel,
					CollisionTrace.CollisionQueryParams,
					FCollisionResponseParams::DefaultResponseParam,
					nullptr,
					TraceIt);
			}
		}
	}
}

void FNiagaraDICollisionQueryBatch::CollectResults()
{
	check(IsInGameThread());
	CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_Collision);

	// locks are not used here because we assume that the per instance ticking is done from the main thread
	if (CollisionWorld)
	{
		const int32 ReadBufferIdx = GetReadBufferIdx();
		const int32 TraceCount = CollisionTraces[ReadBufferIdx].Num();
		if (TraceCount > 0 )
		{
			SCOPE_CYCLE_COUNTER(STAT_NiagaraCollision);

			CollisionResults.Reset(TraceCount);

			for (int32 TraceIt = 0; TraceIt < TraceCount; ++TraceIt)
			{
				FNiagaraCollisionTrace& CollisionTrace = CollisionTraces[ReadBufferIdx][TraceIt];

				FTraceDatum TraceResult;
				const bool TraceReady = CollisionWorld->QueryTraceData(CollisionTrace.CollisionTraceHandle, TraceResult);

				if (TraceReady && TraceResult.OutHits.Num())
				{
					FHitResult* Hit = FHitResult::GetFirstBlockingHit(TraceResult.OutHits);
					if (Hit && Hit->bBlockingHit)
					{
						CollisionTrace.HitIndex = CollisionResults.AddUninitialized();
						FNiagaraDICollsionQueryResult& Result = CollisionResults[CollisionTrace.HitIndex];

						Result.IsInsideMesh = Hit->bStartPenetrating;
						Result.CollisionPos = Hit->ImpactPoint;// -NormVel*(CurTrace.CollisionSize / 2);
						Result.CollisionNormal = Hit->ImpactNormal;
						if (Hit->PhysMaterial.IsValid())
						{
							Result.PhysicalMaterialIdx = Hit->PhysMaterial->SurfaceType.GetValue();
							Result.Friction = Hit->PhysMaterial->Friction;
							Result.Restitution = Hit->PhysMaterial->Restitution;
						}
						else
						{
							Result.PhysicalMaterialIdx = 0;
							Result.Friction = 0.0f;
							Result.Restitution = 0.0f;
						}
					}
				}
			}
		}
	}
}

int32 FNiagaraDICollisionQueryBatch::SubmitQuery(FVector StartPos, FVector EndPos, ECollisionChannel TraceChannel, bool bTraceComplex /*= false*/)
{
	if (!GNiagaraCollisionCPUEnabled)
	{
		return INDEX_NONE;
	}

	FCollisionQueryParams CollisionQueryParams(TraceTagName, StatId, false);
	CollisionQueryParams.OwnerTag = CollisionTagName;
	CollisionQueryParams.bFindInitialOverlaps = false;
	CollisionQueryParams.bReturnFaceIndex = false;
	CollisionQueryParams.bReturnPhysicalMaterial = true;
	CollisionQueryParams.bIgnoreTouches = true;
	CollisionQueryParams.bTraceComplex = bTraceComplex;

	int32 TraceIdx = INDEX_NONE;

	if ((EndPos - StartPos).SizeSquared() > SMALL_NUMBER)
	{
		FRWScopeLock WriteScope(CollisionTraceLock, SLT_Write);

		const int32 WriteBufferIdx = GetWriteBufferIdx();

		TraceIdx = CollisionTraces[WriteBufferIdx].Emplace(StartPos, EndPos, TraceChannel, CollisionQueryParams);
	}

	return TraceIdx;
}

// Work has been done on the collision world side of things to support synchronous queries from multiple threads
bool FNiagaraDICollisionQueryBatch::PerformQuery(FVector StartPos, FVector EndPos, FNiagaraDICollsionQueryResult &Result, ECollisionChannel TraceChannel, bool bTraceComplex /*= false*/)
{
	if (!CollisionWorld || !GNiagaraCollisionCPUEnabled)
	{
		return false;
	}

	SCOPE_CYCLE_COUNTER(STAT_NiagaraCollision);
	CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_Collision);

	FCollisionQueryParams QueryParams(TraceTagName, StatId, false);
	QueryParams.OwnerTag = CollisionTagName;
	QueryParams.bFindInitialOverlaps = false;
	QueryParams.bReturnFaceIndex = false;
	QueryParams.bReturnPhysicalMaterial = true;
	QueryParams.bIgnoreTouches = true;
	QueryParams.bTraceComplex = bTraceComplex;
	FHitResult TraceResult;
	bool ValidHit = CollisionWorld->LineTraceSingleByChannel(TraceResult, StartPos, EndPos, TraceChannel, QueryParams);
	if (ValidHit)
	{
		Result.IsInsideMesh = TraceResult.bStartPenetrating;
		Result.CollisionPos = TraceResult.ImpactPoint;
		Result.CollisionNormal = TraceResult.ImpactNormal;
		if (TraceResult.PhysMaterial.IsValid())
		{
			Result.PhysicalMaterialIdx = TraceResult.PhysMaterial->SurfaceType.GetValue();
			Result.Friction = TraceResult.PhysMaterial->Friction;
			Result.Restitution = TraceResult.PhysMaterial->Restitution;
		}
		else
		{
			Result.PhysicalMaterialIdx = 0;
			Result.Friction = 0.0f;
			Result.Restitution = 0.0f;
		}
	}

	return ValidHit;
}

bool FNiagaraDICollisionQueryBatch::GetQueryResult(uint32 InTraceID, FNiagaraDICollsionQueryResult &Result)
{
	const int32 ReadBufferIdx = GetReadBufferIdx();

	if (CollisionTraces[ReadBufferIdx].IsValidIndex(InTraceID))
	{
		const int32 ResultIndex = CollisionTraces[ReadBufferIdx][InTraceID].HitIndex;

		if (CollisionResults.IsValidIndex(ResultIndex))
		{
			Result = CollisionResults[ResultIndex];
			return true;
		}
	}

	return false;
}
