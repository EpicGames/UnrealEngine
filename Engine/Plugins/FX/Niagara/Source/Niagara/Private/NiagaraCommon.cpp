// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCommon.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "RenderUtils.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/StringBuilder.h"
#include "NiagaraComponent.h"
#include "NiagaraComponentSettings.h"
#include "NiagaraConstants.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraEmitter.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraParameterDefinitionsBase.h"
#include "NiagaraScriptSourceBase.h"
#include "NiagaraSettings.h"
#include "NiagaraStats.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraWorldManager.h"
#include "String/ParseTokens.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "FXSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraCommon)

DECLARE_CYCLE_STAT(TEXT("Niagara - Utilities - PrepareRapidIterationParameters"), STAT_Niagara_Utilities_PrepareRapidIterationParameters, STATGROUP_Niagara);

//////////////////////////////////////////////////////////////////////////

int32 GNiagaraAllowComputeShaders = 1;
FAutoConsoleVariableRef CVarAllowComputeShaders(
	TEXT("fx.NiagaraAllowComputeShaders"),
	GNiagaraAllowComputeShaders,
	TEXT("If true, allow the usage compute shaders within Niagara."),
	FConsoleVariableDelegate::CreateStatic(FNiagaraComponentSettings::RequestUpdateSettings),
	ECVF_Default);

int32 GNiagaraAllowGPUParticles = 1;
FAutoConsoleVariableRef CVarAllowGPUParticles(
	TEXT("fx.NiagaraAllowGPUParticles"),
	GNiagaraAllowGPUParticles,
	TEXT("If true, allow the usage of GPU particles for Niagara."),
	FConsoleVariableDelegate::CreateStatic(FNiagaraComponentSettings::RequestUpdateSettings),
	ECVF_Scalability | ECVF_Default);

int32 GNiagaraGPUCulling = 1;
FAutoConsoleVariableRef CVarNiagaraGPUCulling(
	TEXT("Niagara.GPUCulling"),
	GNiagaraGPUCulling,
	TEXT("Whether to frustum and camera distance cull particles on the GPU"),
	ECVF_Default
);

int32 GNiagaraMaxStatInstanceReports = 20;
FAutoConsoleVariableRef CVarMaxStatInstanceReportss(
    TEXT("fx.NiagaraMaxStatInstanceReports"),
    GNiagaraMaxStatInstanceReports,
    TEXT("The max number of different instances from which stat reports are aggregated."),
    ECVF_Default);

static int32 GbMaxStatRecordedFrames = 30;
static FAutoConsoleVariableRef CVarMaxStatRecordedFrames(
    TEXT("fx.Niagara.MaxStatRecordedFrames"),
    GbMaxStatRecordedFrames,
    TEXT("The number of frames recorded for the stat performance display of niagara cpu and gpu scripts. \n"),
    ECVF_Default
);

static int32 GNiagaraLogVerboseWarnings = WITH_EDITOR ? 1 : 0;
static FAutoConsoleVariableRef CVarNiagaraLogVerboseWarnings(
	TEXT("fx.Niagara.LogVerboseWarnings"),
	GNiagaraLogVerboseWarnings,
	TEXT("Enable to output more verbose warnings to the log file, these are considered dismissable warnings but may provide information when debugging.\n")
	TEXT("Default is enabled in editor builds and disabled in non editor builds.\n"),
	ECVF_Default
);

FNiagaraSystemUpdateContext::~FNiagaraSystemUpdateContext()
{
	// IncrementalCook: Add a ResetContext scope so that dereferences of the Components we gathered do not
	// record build dependencies from our caller to those components. We are providing an update event
	// to the components we gathered; we are not using them for any modifications of our caller.
	UE_TRACK_REFERENCING_OPNAME_SCOPED(PackageAccessTrackingOps::NAME_ResetContext);
	CommitUpdate();
}

void FNiagaraSystemUpdateContext::CommitUpdate()
{
	auto DestroyInstanceNotComponent =
		[this](UNiagaraComponent* Component)
		{
			// Destroy the instance not the component and if we are going to recache the simulation make sure to kill the render state
			// Not killing the render state can result in crashes if it's recached and we tick before flushing EOF updates
			Component->DestroyInstanceNotComponent();
			if (Component->IsRenderStateCreated() && SystemSimsToRecache.Contains(Component->GetAsset()))
			{
				UPrimitiveComponent* PrimitiveComponent = static_cast<UPrimitiveComponent*>(Component);
				PrimitiveComponent->MarkRenderStateDirty();
				PrimitiveComponent->DestroyRenderState_Concurrent();
			}
		};

	if (SystemSimsToDestroy.Num() > 0)
	{
		for (UNiagaraSystem* NiagaraSystem : SystemSimsToDestroy)
		{
			if (NiagaraSystem)
			{
				FNiagaraWorldManager::DestroyAllSystemSimulations(NiagaraSystem);
			}
		}

		for (TObjectIterator<UNiagaraComponent> It; It; ++It)
		{
			UNiagaraComponent* Comp = *It;

			if ( ::IsValid(Comp) && SystemSimsToDestroy.Contains(Comp->GetAsset()) )
			{
				FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = Comp->GetSystemInstanceController();
				if (SystemInstanceController && SystemInstanceController->HasValidSimulation() && SystemInstanceController->IsSolo())
				{
					FNiagaraSystemSimulationPtr SoloSimulation = SystemInstanceController->GetSoloSystemSimulation();
					if (SoloSimulation.IsValid())
					{
						DestroyInstanceNotComponent(Comp);
						SoloSimulation->Destroy();
					}
				}
			}
		}

		SystemSimsToDestroy.Empty();
	}

	for (UNiagaraComponent* Comp : ComponentsToDestroyInstance)
	{
		if (Comp)
		{
			DestroyInstanceNotComponent(Comp);
			PostWork.ExecuteIfBound(Comp);
		}
	}
	ComponentsToDestroyInstance.Empty();

	bool bNeedsWaitOnGpu = true;
	for (UNiagaraSystem* NiagaraSystem : SystemSimsToRecache)
	{
		if (NiagaraSystem && NiagaraSystem->IsReadyToRun())
		{
			if (bNeedsWaitOnGpu == true && NiagaraSystem->HasAnyGPUEmitters())
			{
				bNeedsWaitOnGpu = false;
				FlushRenderingCommands();
			}

			NiagaraSystem->ComputeEmittersExecutionOrder();
			NiagaraSystem->ComputeRenderersDrawOrder();
			NiagaraSystem->CacheFromCompiledData();
		}
	}
	SystemSimsToRecache.Empty();
	
	for (UNiagaraComponent* Comp : ComponentsToReInit)
	{
		if (Comp)
		{
			Comp->ReinitializeSystem();
			Comp->EndUpdateContextReset();
			PostWork.ExecuteIfBound(Comp);
		}
	}
	ComponentsToReInit.Empty();

	for (UNiagaraComponent* Comp : ComponentsToReset)
	{
		if (Comp)
		{
			Comp->ResetSystem();
			Comp->EndUpdateContextReset();
			PostWork.ExecuteIfBound(Comp);
		}
	}
	ComponentsToReset.Empty();

	for (UNiagaraComponent* Comp : ComponentsToNotifySimDestroy)
	{
		if (Comp)
		{
			if (FNiagaraSystemInstanceControllerPtr SystemInstanceController = Comp->GetSystemInstanceController())
			{
				SystemInstanceController->OnSimulationDestroyed();
			}
			Comp->EndUpdateContextReset();
			PostWork.ExecuteIfBound(Comp);
		}
	}
	ComponentsToNotifySimDestroy.Empty();
}

void FNiagaraSystemUpdateContext::AddAll(bool bReInit)
{
	for (TObjectIterator<UNiagaraComponent> It; It; ++It)
	{
		UNiagaraComponent* Comp = *It;
		check(Comp);
		bool bAllowDestroySystemSim = true;
		AddInternal(Comp, bReInit, bAllowDestroySystemSim);
	}
}

void FNiagaraSystemUpdateContext::AddSoloComponent(UNiagaraComponent* Component, bool bReInit)
{
	check(Component);
	if (ensureMsgf(Component->GetForceSolo(), TEXT("A component must have a solo system simulation when used with an update context.")))
	{
		bool bAllowDestroySystemSim = false;
		AddInternal(Component, bReInit, bAllowDestroySystemSim);
	}
}

void FNiagaraSystemUpdateContext::Add(const UNiagaraSystem* System, bool bReInit)
{
	AddComponentsWithMatchingSystem(bReInit,
		[System](const UNiagaraSystem* SystemOnComponent)
		{
			return SystemOnComponent == System;
		});
}
#if WITH_EDITORONLY_DATA

void FNiagaraSystemUpdateContext::Add(const FVersionedNiagaraEmitterBase& Emitter, bool bReInit)
{
	AddComponentsWithMatchingSystem(bReInit,
		[&Emitter](const UNiagaraSystem* System)
		{
			// only worry about systems that are fully loaded (and don't force it to be fully loaded just because it shows up in the iterator
			return System->bFullyLoaded && System->UsesEmitter(Emitter);
		});
}

void FNiagaraSystemUpdateContext::Add(const FVersionedNiagaraEmitter& Emitter, bool bReInit)
{
	AddComponentsWithMatchingSystem(bReInit,
		[&Emitter](const UNiagaraSystem* System)
		{
			// only worry about systems that are fully loaded (and don't force it to be fully loaded just because it shows up in the iterator
			return System->bFullyLoaded && System->UsesEmitter(Emitter);
		});
}

void FNiagaraSystemUpdateContext::Add(const UNiagaraScript* Script, bool bReInit)
{
	AddComponentsWithMatchingSystem(bReInit,
		[Script](const UNiagaraSystem* System)
		{
			// only worry about systems that are fully loaded (and don't force it to be fully loaded just because it shows up in the iterator)
			return System->bFullyLoaded && System->UsesScript(Script);
		});
}

void FNiagaraSystemUpdateContext::Add(const UNiagaraParameterCollection* Collection, bool bReInit)
{
	AddComponentsWithMatchingSystem(bReInit,
		[Collection](const UNiagaraSystem* System)
		{
			// only worry about systems that are fully loaded (and don't force it to be fully loaded just because it shows up in the iterator)
			return System->bFullyLoaded && System->UsesCollection(Collection);
		});
}

#endif

template <typename SystemMatchesFunc>
void FNiagaraSystemUpdateContext::AddComponentsWithMatchingSystem(bool bReInit, SystemMatchesFunc&& SystemMatches)
{

	// IncrementalCook: Add a ResetContext scope so that dereferences of the Components we gather do not
	// record build dependencies from our caller to those components. We are providing an update event
	// to the components we gather; we are not using them for any modifications of our caller.
	UE_TRACK_REFERENCING_OPNAME_SCOPED(PackageAccessTrackingOps::NAME_ResetContext);
	for (TObjectIterator<UNiagaraComponent> It; It; ++It)
	{
		UNiagaraComponent* Comp = *It;
		check(Comp);
		UNiagaraSystem* System = Comp->GetAsset();
		if (System && SystemMatches(System))
		{
			bool bAllowDestroySystemSim = true;
			AddInternal(Comp, bReInit, bAllowDestroySystemSim);
		}
	}
}

void FNiagaraSystemUpdateContext::AddInternal(UNiagaraComponent* Comp, bool bReInit, bool bAllowDestroySystemSim)
{
	PreWork.ExecuteIfBound(Comp);

	Comp->BeginUpdateContextReset();

	// Ensure we wait for any concurrent work to complete
	if (FNiagaraSystemInstanceControllerPtr SystemInstanceController = Comp->GetSystemInstanceController())
	{
		SystemInstanceController->WaitForConcurrentTickAndFinalize();
		if (!IsValidChecked(Comp))
		{
			return;
		}
	}

	if (bAllowDestroySystemSim)
	{
		if (bReInit || bDestroySystemSim)
		{
			SystemSimsToRecache.AddUnique(Comp->GetAsset());
		}

		if (bReInit && bDestroySystemSim)
		{
			SystemSimsToDestroy.AddUnique(Comp->GetAsset());
		}
	}

	const bool bIsActive = Comp->IsActiveForUpdateContext();

	if (bDestroyOnAdd)
	{
		Comp->DeactivateImmediate();
	}

	if (bIsActive || (bOnlyActive == false && Comp->bAutoActivate))
	{
		if (bReInit)
		{
			ComponentsToReInit.AddUnique(Comp);
		}
		else
		{
			ComponentsToReset.AddUnique(Comp);
		}
		return;
	}
	else if (bReInit)
	{
		// Inactive components that have references to the simulations we're about to destroy need to clear them out in case they get reactivated.
		// Otherwise, they will hold reference and bind or remain bound to a system simulation that has been abandoned by the world manager
		if (FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = Comp->GetSystemInstanceController())
		{
			if (SystemInstanceController->HasValidSimulation())
			{
				if (!SystemInstanceController->IsSolo())
				{
					ComponentsToNotifySimDestroy.AddUnique(Comp);
				}
				// solo systems still need to be reinitialized because we don't want them to try to use stale data either (like if a compilation
				// has changed the script data)
				else
				{
					ComponentsToDestroyInstance.AddUnique(Comp);
				}
				return;
			}
		}
	}

	// If we got here, we didn't add the component to any list, so end the reset immediately
	Comp->EndUpdateContextReset();
}

FNiagaraScriptDataKey::FNiagaraScriptDataKey(const FNiagaraEmitterHandle* EmitterHandle, const UNiagaraScript& Script)
	: EmitterHandleId(EmitterHandle != nullptr ? EmitterHandle->GetId() : FGuid())
	, ScriptUsage(Script.GetUsage())
	, ScriptUsageId(Script.GetUsageId())
{
}

//////////////////////////////////////////////////////////////////////////

#if STATS
FStatExecutionTimer::FStatExecutionTimer()
{
	CapturedTimings.Reserve(GbMaxStatRecordedFrames);
}

void FStatExecutionTimer::AddTiming(float NewTiming)
{
	if (CapturedTimings.Num() < GbMaxStatRecordedFrames)
	{
		CapturedTimings.Add(NewTiming);
	}
	else if (CapturedTimings.IsValidIndex(CurrentIndex))
	{
		CapturedTimings[CurrentIndex] = NewTiming;
		CurrentIndex = (CurrentIndex + 1) % GbMaxStatRecordedFrames;
	}
}

void FNiagaraStatDatabase::AddStatCapture(FStatReportKey ReportKey, TMap<TStatIdData const*, float> CapturedData)
{
	if (CapturedData.Num() == 0)
	{
		return;
	}
	FScopeLock Lock(GetCriticalSection());
	if (StatCaptures.Num() > GNiagaraMaxStatInstanceReports)
	{
		// we don't need data from too many emitter instances. If we already have enough, delete an old data point.
		TArray<FStatReportKey> Keys;
		StatCaptures.GetKeys(Keys);
		StatCaptures.Remove(Keys[FMath::RandHelper(Keys.Num())]);
	}

	TMap<TStatIdData const*, FStatExecutionTimer>& InstanceData = StatCaptures.FindOrAdd(ReportKey);
	for (const auto& Entry : CapturedData)
	{
		InstanceData.FindOrAdd(Entry.Key).AddTiming(Entry.Value);
	}
}

void FNiagaraStatDatabase::ClearStatCaptures()
{
	FScopeLock Lock(GetCriticalSection());
	StatCaptures.Empty();
}

float FNiagaraStatDatabase::GetRuntimeStat(FName StatName, ENiagaraScriptUsage Usage, ENiagaraStatEvaluationType EvaluationType)
{
	FScopeLock Lock(GetCriticalSection());
	int32 ValueCount = 0;
	float Sum = 0;
	float Max = 0;
	for (const auto& EmitterEntry : StatCaptures)
	{
		if (Usage != EmitterEntry.Key.Value)
		{
			continue;
		}
		for (const TTuple<TStatIdData const*, FStatExecutionTimer>& StatEntry : EmitterEntry.Value)
		{
			if (MinimalNameToName(StatEntry.Key->Name) == StatName)
			{
				ValueCount += StatEntry.Value.CapturedTimings.Num();
				for (const float& Value : StatEntry.Value.CapturedTimings)
				{
					Max = FMath::Max(Max, Value);
					Sum += Value;
				}
				break;
			}
		}
	}
	if (EvaluationType == ENiagaraStatEvaluationType::Maximum)
	{
		return Max;
	}
	return ValueCount == 0 ? 0 : Sum / ValueCount;
}

float FNiagaraStatDatabase::GetRuntimeStat(ENiagaraScriptUsage Usage, ENiagaraStatEvaluationType EvaluationType)
{
	FScopeLock Lock(GetCriticalSection());
	int32 ValueCount = 0;
	float Sum = 0;
	float Max = 0;
	for (const auto& EmitterEntry : StatCaptures)
	{
		if (Usage != EmitterEntry.Key.Value)
		{
			continue;
		}
		for (const TTuple<TStatIdData const*, FStatExecutionTimer>& StatEntry : EmitterEntry.Value)
		{
			for (int i = 0; i < StatEntry.Value.CapturedTimings.Num(); i++)
			{
				float Value = StatEntry.Value.CapturedTimings[i];
				Max = FMath::Max(Max, Value);
				Sum += Value;
				ValueCount++;
			}
		}
	}
	if (EvaluationType == ENiagaraStatEvaluationType::Maximum)
	{
		return Max;
	}
	return ValueCount == 0 ? 0 : Sum / ValueCount;
}

TMap<ENiagaraScriptUsage, TSet<FName>> FNiagaraStatDatabase::GetAvailableStatNames()
{
	FScopeLock Lock(GetCriticalSection());
	TMap<ENiagaraScriptUsage, TSet<FName>> Result;
	for (const auto& EmitterEntry : StatCaptures)
	{
		for (const auto& StatEntry : EmitterEntry.Value)
		{
			ENiagaraScriptUsage Usage = EmitterEntry.Key.Value;
			Result.FindOrAdd(Usage).Add(MinimalNameToName(StatEntry.Key->Name));
		}
	}
	return Result;
}

void FNiagaraStatDatabase::Init()
{
	if (CriticalSection.IsValid() == false)
	{
		CriticalSection = MakeShared<FCriticalSection>();
	}
}

FCriticalSection* FNiagaraStatDatabase::GetCriticalSection() const
{
	ensure(CriticalSection.IsValid());
	return CriticalSection.Get();
}
#endif

void FNiagaraVariableAttributeBinding::SetValue(const FName& InValue, const FVersionedNiagaraEmitterBase& InVersionedEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	UNiagaraEmitterBase* Emitter = InVersionedEmitter.Emitter;
	RootName = InValue;

	FNiagaraVariableBase RootVariable = GetRootVariable();

	const bool bIsRootParticleValue = RootVariable.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString);
	const bool bIsRootUnaliasedEmitterValue = RootVariable.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString);
	const bool bIsAliasedEmitterValue = Emitter ? RootVariable.IsInNameSpace(Emitter->GetUniqueEmitterName()) : false;
	const bool bIsRootSystemValue = RootVariable.IsInNameSpace(FNiagaraConstants::SystemNamespaceString);
	const bool bIsRootUserValue = RootVariable.IsInNameSpace(FNiagaraConstants::UserNamespaceString);
	const bool bIsStackContextValue = RootVariable.IsInNameSpace(FNiagaraConstants::StackContextNamespaceString);

	// We clear out the namespace for the sourcemode so that we can keep the values up-to-date if you change the source mode.
	if ((bIsStackContextValue || bIsRootParticleValue) && InSourceMode == ENiagaraRendererSourceDataMode::Particles)
	{
		RootVariable.SetName(FNiagaraConstants::GetAttributeAsParticleDataSetKey(RootVariable).GetName());
		BindingSourceMode = ENiagaraBindingSource::ImplicitFromSource;
	}
	else if ((bIsStackContextValue || bIsRootUnaliasedEmitterValue) && InSourceMode == ENiagaraRendererSourceDataMode::Emitter)
	{
		RootVariable.SetName(FNiagaraConstants::GetAttributeAsEmitterDataSetKey(RootVariable).GetName());
		BindingSourceMode = ENiagaraBindingSource::ImplicitFromSource;
	}
	else if (bIsAliasedEmitterValue && InSourceMode == ENiagaraRendererSourceDataMode::Emitter)
	{
		// First, replace unaliased emitter namespace with "Emitter" namespace
		RootVariable = FNiagaraUtilities::ResolveAliases(RootVariable, FNiagaraAliasContext()
			.ChangeEmitterNameToEmitter(Emitter->GetUniqueEmitterName()));

		// Now strip out "Emitter"
		RootVariable.SetName(FNiagaraConstants::GetAttributeAsEmitterDataSetKey(RootVariable).GetName());
		BindingSourceMode = ENiagaraBindingSource::ImplicitFromSource;
	}
	else if (bIsRootParticleValue)
	{
		RootVariable.SetName(FNiagaraConstants::GetAttributeAsParticleDataSetKey(RootVariable).GetName());
		BindingSourceMode = ENiagaraBindingSource::ExplicitParticles;
	}
	else if (bIsRootUnaliasedEmitterValue || bIsAliasedEmitterValue)
	{
		// First, replace unaliased emitter namespace with "Emitter" namespace
		if (Emitter != nullptr)
		{
			RootVariable = FNiagaraUtilities::ResolveAliases(RootVariable, FNiagaraAliasContext()
				.ChangeEmitterNameToEmitter(Emitter->GetUniqueEmitterName()));
		}

		// Now strip out "Emitter"
		RootVariable.SetName(FNiagaraConstants::GetAttributeAsEmitterDataSetKey(RootVariable).GetName());
		BindingSourceMode = ENiagaraBindingSource::ExplicitEmitter;
	}
	else if (bIsRootSystemValue)
	{
		BindingSourceMode = ENiagaraBindingSource::ExplicitSystem;
	}
	else if (bIsRootUserValue)
	{
		BindingSourceMode = ENiagaraBindingSource::ExplicitUser;
	}
	else if (bIsStackContextValue)
	{
		ensureMsgf(!bIsStackContextValue, TEXT("Should not get to this point! Should be covered by first two branch expresssions."));
	}
	RootName = RootVariable.GetName();

	CacheValues(InVersionedEmitter, InSourceMode);
}

void FNiagaraVariableAttributeBinding::SetAsPreviousValue(const FNiagaraVariableBase& Src, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	static const FString PreviousNamespace = FNiagaraConstants::PreviousNamespace.ToString();

	ParamMapVariable = Src;
	RootName = Src.GetName();
	DataSetName = Src.GetName();

	// Split out the name and it's namespace
	TArray<FString> SplitName;
	Src.GetName().ToString().ParseIntoArray(SplitName, TEXT("."));

	// If the name already contains a "Previous" in the name, just go with that
	bool bIsPrev = false;
	for (const FString& Split : SplitName)
	{
		if (Split.Equals(PreviousNamespace, ESearchCase::IgnoreCase))
		{
			bIsPrev = true;
			break;
		}
	}

	if (bIsPrev)
	{
		SetValue(Src.GetName(), InEmitter, InSourceMode);
	}
	else
	{
		// insert "Previous" into the name, after the first namespace. Or the beginning, if it has none
		const int32 Location = SplitName.Num() > 1 ? 1 : 0;
		SplitName.Insert(PreviousNamespace, Location);

		FString PrevName = FString::Join(SplitName, TEXT("."));
		SetValue(*PrevName, InEmitter, InSourceMode);
	}
}

void FNiagaraVariableAttributeBinding::SetAsPreviousValue(const FNiagaraVariableAttributeBinding& Src, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	static const FString PreviousNamespace = FNiagaraConstants::PreviousNamespace.ToString();

	const FNiagaraVariableBase SrcRootVariable = Src.GetRootVariable();

	ParamMapVariable = SrcRootVariable;
	RootName = SrcRootVariable.GetName();
	DataSetName = SrcRootVariable.GetName();

	// Split out the name and it's namespace
	TStringBuilder<128> VarName;
	TArray<FStringView, TInlineAllocator<16>> SplitName;
	SrcRootVariable.GetName().ToString(VarName);
	UE::String::ParseTokens(VarName, TEXT('.'), [&SplitName](FStringView Token) { SplitName.Emplace(Token); });

	// If the name already contains a "Previous" in the name, just go with that
	bool bIsPrev = false;
	for (const FStringView& Split : SplitName)
	{
		if (Split.Equals(PreviousNamespace, ESearchCase::IgnoreCase))
		{
			bIsPrev = true;
			break;
		}
	}

	if (bIsPrev)
	{
		SetValue(SrcRootVariable.GetName(), InEmitter, InSourceMode);
	}
	else
	{
		TStringBuilder<128> PreviousVarName;
		if ( SplitName.Num() > 1 )
		{
			PreviousVarName.Append(SplitName[0]);
			PreviousVarName.Append(TEXT("."));
			PreviousVarName.Append(PreviousNamespace);
			for ( int i=1; i < SplitName.Num(); ++i )
			{
				PreviousVarName.Append(TEXT("."));
				PreviousVarName.Append(SplitName[i]);
			}
		}
		else
		{
			PreviousVarName.Append(PreviousNamespace);
			PreviousVarName.Append(TEXT("."));
			PreviousVarName.Append(SplitName[0]);
		}
		SetValue(PreviousVarName.ToString(), InEmitter, InSourceMode);
	}
}

void FNiagaraVariableAttributeBinding::Setup(const FNiagaraVariableBase& InRootVar, const FNiagaraVariable& InDefaultValue, ENiagaraRendererSourceDataMode InSourceMode)
{
	RootName = InRootVar.GetName();
	ParamMapVariable.SetType(InRootVar.GetType());
#if WITH_EDITORONLY_DATA
	DefaultValue = InDefaultValue;
	BoundVariable = FNiagaraVariable();
#endif
	SetValue(InRootVar.GetName(), FVersionedNiagaraEmitterBase(), InSourceMode);
}

#if WITH_EDITORONLY_DATA
FString FNiagaraVariableAttributeBinding::GetDefaultValueString() const
{
	FString DefaultValueStr;
	if (DefaultValue.IsValid())
	{
		if (DefaultValue.IsDataAllocated())
		{
			DefaultValueStr = DefaultValue.GetType().ToString(DefaultValue.GetData());
			DefaultValueStr.TrimEndInline();
		}
		else
		{
			DefaultValueStr = DefaultValue.GetName().ToString();
		}
	}
	return DefaultValueStr;
}
#endif

void FNiagaraVariableAttributeBinding::PostLoad(ENiagaraRendererSourceDataMode InSourceMode)
{
#if WITH_EDITORONLY_DATA
	if (!RootVariable_DEPRECATED.GetName().IsNone())
	{
		RootName = RootVariable_DEPRECATED.GetName();
	}
	if (BoundVariable.IsValid())
	{
		SetValue(BoundVariable.GetName(), FVersionedNiagaraEmitterBase(), InSourceMode);
		BoundVariable = FNiagaraVariable();
	}
	if (!DataSetVariable_DEPRECATED.GetName().IsNone())
	{
		DataSetName = DataSetVariable_DEPRECATED.GetName();
	}
#endif

}

void FNiagaraVariableAttributeBinding::Dump() const
{
	UE_LOGF(LogNiagara, Log, "PostLoad for FNiagaraVariableAttributeBinding....");
	UE_LOGF(LogNiagara, Log, "ParamMapVariable: %ls %ls", *ParamMapVariable.GetName().ToString(), *ParamMapVariable.GetType().GetName());
	UE_LOGF(LogNiagara, Log, "DataSetName: %ls", *DataSetName.ToString());
	UE_LOGF(LogNiagara, Log, "RootVariable: %ls", *RootName.ToString());
#if WITH_EDITORONLY_DATA
	UE_LOGF(LogNiagara, Log, "BoundVariable: %ls %ls", *BoundVariable.GetName().ToString(), *BoundVariable.GetType().GetName());
	UE_LOGF(LogNiagara, Log, "CachedDisplayName: %ls", *CachedDisplayName.ToString());
#endif
	UE_LOGF(LogNiagara, Log, "BindingSourceMode: %d     bBindingExistsOnSource: %d     bIsCachedParticleValue: %d", (int32)BindingSourceMode.GetValue(),
		bBindingExistsOnSource ? 1 : 0, bIsCachedParticleValue ? 1 : 0 );
}

void FNiagaraVariableAttributeBinding::ResetToDefault(const FNiagaraVariableAttributeBinding& InOther, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	if (InOther.BindingSourceMode == ImplicitFromSource || InOther.BindingSourceMode == ENiagaraBindingSource::ExplicitEmitter || InOther.BindingSourceMode == ENiagaraBindingSource::ExplicitParticles)
	{
		// The default may have been set with a different source mode, so we can't copy values over directly. Instead, we need to copy the implicit values over.
		FNiagaraVariableBase TempVar = InOther.GetRootVariable();
		if ((InSourceMode == ENiagaraRendererSourceDataMode::Emitter && InOther.BindingSourceMode == ENiagaraBindingSource::ImplicitFromSource) ||
			InOther.BindingSourceMode == ENiagaraBindingSource::ExplicitEmitter)
		{
			ensure(!InOther.GetDataSetBindableVariable().IsInNameSpace(FNiagaraConstants::EmitterNamespaceString));
			TempVar.SetNamespacedName(FNiagaraConstants::EmitterNamespaceString, InOther.DataSetName);
		}
		else if ((InSourceMode == ENiagaraRendererSourceDataMode::Particles && InOther.BindingSourceMode == ENiagaraBindingSource::ImplicitFromSource) ||
			InOther.BindingSourceMode == ENiagaraBindingSource::ExplicitParticles)
		{
			ensure(!InOther.GetDataSetBindableVariable().IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString));
			TempVar.SetNamespacedName(FNiagaraConstants::ParticleAttributeNamespaceString, InOther.DataSetName);
		}

		SetValue(TempVar.GetName(), FVersionedNiagaraEmitterBase(), InSourceMode);
	}
	else
	{
		SetValue(InOther.RootName, InEmitter, InSourceMode);
	}
}

bool FNiagaraVariableAttributeBinding::MatchesDefault(const FNiagaraVariableAttributeBinding& InOther, ENiagaraRendererSourceDataMode InSourceMode) const
{
	if (DataSetName != InOther.DataSetName)
		return false;
	if (RootName != InOther.RootName)
		return false;
	return true;
}


bool FNiagaraVariableAttributeBinding::RenameVariableIfMatching(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	// First try a namespace mangling - free match.
	if (OldVariable.GetName() == ParamMapVariable.GetName() && OldVariable.GetType() == ParamMapVariable.GetType())
	{
		SetValue(NewVariable.GetName(), InEmitter, InSourceMode);
		return true;
	}

	// Now we need to deal with any aliased emitter namespaces for the match. If so resolve the aliases then try the match.
	FNiagaraVariable OldVarAliased = OldVariable;
	if (OldVariable.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString))
	{
		// First, resolve any aliases
		OldVarAliased = FNiagaraUtilities::ResolveAliases(OldVariable, FNiagaraAliasContext()
			.ChangeEmitterToEmitterName(InEmitter.Emitter->GetUniqueEmitterName()));
	}
	if (OldVarAliased.GetName() == ParamMapVariable.GetName() && OldVarAliased.GetType() == ParamMapVariable.GetType())
	{
		SetValue(NewVariable.GetName(), InEmitter, InSourceMode);
		return true;
	}
	return false;
}

bool FNiagaraVariableAttributeBinding::Matches(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode) const
{
	// First try a namespace mangling - free match.
	if (OldVariable.GetName() == ParamMapVariable.GetName() && OldVariable.GetType() == ParamMapVariable.GetType())
	{
		return true;
	}

	// Now we need to deal with any aliased emitter namespaces for the match. If so resolve the aliases then try the match.
	FNiagaraVariable OldVarAliased = OldVariable;
	if (InEmitter.Emitter && OldVariable.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString))
	{
		// First, resolve any aliases
		OldVarAliased = FNiagaraUtilities::ResolveAliases(OldVariable, FNiagaraAliasContext()
			.ChangeEmitterToEmitterName(InEmitter.Emitter->GetUniqueEmitterName()));
	}
	if (OldVarAliased.GetName() == ParamMapVariable.GetName() && OldVarAliased.GetType() == ParamMapVariable.GetType())
	{
		return true;
	}
	return false;
}

void FNiagaraVariableAttributeBinding::CacheValues(const FVersionedNiagaraEmitterBase& InVersionedEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	// Some older values may have had the root with the emitter unique name as the namespace, fix this up
	// to meet the new assumptions.
	UNiagaraEmitterBase* Emitter = InVersionedEmitter.Emitter;
	FNiagaraVariableBase RootVariable = GetRootVariable();
	if (Emitter && RootVariable.IsInNameSpace(Emitter->GetUniqueEmitterName()))
	{
		// First, replace unaliased emitter namespace with "Emitter" namespace
		RootVariable = FNiagaraUtilities::ResolveAliases(RootVariable, FNiagaraAliasContext()
			.ChangeEmitterNameToEmitter(Emitter->GetUniqueEmitterName()));

		// Now strip out "Emitter"
		RootVariable.SetName(FNiagaraConstants::GetAttributeAsEmitterDataSetKey(RootVariable).GetName());
		RootName = RootVariable.GetName();
	}

	ParamMapVariable = RootVariable;
	DataSetName = ParamMapVariable.GetName();
	bBindingExistsOnSource = false;

	// Decide if this is going to be bound to a particle attribute (needed for use by the renderers, for instance)
	if (BindingSourceMode == ENiagaraBindingSource::ExplicitParticles || (InSourceMode == ENiagaraRendererSourceDataMode::Particles && BindingSourceMode == ENiagaraBindingSource::ImplicitFromSource))
	{
		bIsCachedParticleValue = true;
	}
	else
	{
		bIsCachedParticleValue = false;
	}

	// If this is one of the possible namespaces that is implicitly defined, go ahead and expand the full namespace. RootVariable should be non-namespaced at this point.
	if (DataSetName.IsNone())
	{
		ParamMapVariable.SetName(NAME_None);
	}
	else if ((InSourceMode == ENiagaraRendererSourceDataMode::Emitter && BindingSourceMode == ENiagaraBindingSource::ImplicitFromSource) ||
		BindingSourceMode == ENiagaraBindingSource::ExplicitEmitter)
	{
		ensure(!GetDataSetBindableVariable().IsInNameSpace(FNiagaraConstants::EmitterNamespaceString));
		ParamMapVariable.SetNamespacedName(FNiagaraConstants::EmitterNamespaceString, DataSetName);
	}
	else if ((InSourceMode == ENiagaraRendererSourceDataMode::Particles && BindingSourceMode == ENiagaraBindingSource::ImplicitFromSource) ||
		BindingSourceMode == ENiagaraBindingSource::ExplicitParticles)
	{
		ensure(!GetDataSetBindableVariable().IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString));
		ParamMapVariable.SetNamespacedName(FNiagaraConstants::ParticleAttributeNamespaceString, DataSetName);
	}

#if WITH_EDITORONLY_DATA
	CachedDisplayName = ParamMapVariable.GetName();
#endif

	// Now resolve if this variable actually exists.
	if (Emitter)
	{
		if (BindingSourceMode == ENiagaraBindingSource::ExplicitEmitter || (InSourceMode == ENiagaraRendererSourceDataMode::Emitter && BindingSourceMode == ENiagaraBindingSource::ImplicitFromSource))
		{
			// Replace  "Emitter" namespace with unaliased emitter namespace
			FNiagaraAliasContext ResolveAliasesContext(FNiagaraAliasContext::ERapidIterationParameterMode::EmitterOrParticleScript);
			ResolveAliasesContext.ChangeEmitterToEmitterName(Emitter->GetUniqueEmitterName());
			ParamMapVariable = FNiagaraUtilities::ResolveAliases(ParamMapVariable, ResolveAliasesContext);
			DataSetName = FNiagaraUtilities::ResolveAliases(GetDataSetBindableVariable(), ResolveAliasesContext).GetName();
		}

		FNiagaraTypeDefinition BoundVarType = ParamMapVariable.GetType();
		if (BindingSourceMode == ENiagaraBindingSource::ExplicitParticles || (InSourceMode == ENiagaraRendererSourceDataMode::Particles && BindingSourceMode == ENiagaraBindingSource::ImplicitFromSource))
		{
			bBindingExistsOnSource = Emitter->CanObtainParticleAttribute(GetDataSetBindableVariable(), InVersionedEmitter.Version, BoundVarType);
		}
		else if (BindingSourceMode == ENiagaraBindingSource::ExplicitEmitter || (InSourceMode == ENiagaraRendererSourceDataMode::Emitter && BindingSourceMode == ENiagaraBindingSource::ImplicitFromSource))
		{
			bBindingExistsOnSource = Emitter->CanObtainEmitterAttribute(ParamMapVariable, BoundVarType);
		}
		else if (BindingSourceMode == ENiagaraBindingSource::ExplicitSystem)
		{
			bBindingExistsOnSource = Emitter->CanObtainSystemAttribute(ParamMapVariable, BoundVarType);
		}
		else if (BindingSourceMode == ENiagaraBindingSource::ExplicitUser)
		{
			bBindingExistsOnSource = Emitter->CanObtainUserVariable(ParamMapVariable);
		}

		if (bBindingExistsOnSource && BoundVarType != ParamMapVariable.GetType())
		{
			ParamMapVariable.SetType(BoundVarType);
		}
	}
}

void FNiagaraVariableAttributeBinding::SetValue(const FName& InValue, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	SetValue(InValue, InEmitter.ToBase(), InSourceMode);
}

void FNiagaraVariableAttributeBinding::SetAsPreviousValue(const FNiagaraVariableBase& Src, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	SetAsPreviousValue(Src, InEmitter.ToBase(), InSourceMode);
}

void FNiagaraVariableAttributeBinding::SetAsPreviousValue(const FNiagaraVariableAttributeBinding& Src, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	SetAsPreviousValue(Src, InEmitter.ToBase(), InSourceMode);
}

void FNiagaraVariableAttributeBinding::CacheValues(const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	CacheValues(InEmitter.ToBase(), InSourceMode);
}

bool FNiagaraVariableAttributeBinding::RenameVariableIfMatching(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	return RenameVariableIfMatching(OldVariable, NewVariable, InEmitter.ToBase(), InSourceMode);
}

bool FNiagaraVariableAttributeBinding::Matches(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode) const
{
	return Matches(OldVariable, InEmitter.ToBase(), InSourceMode);
}

void FNiagaraVariableAttributeBinding::ResetToDefault(const FNiagaraVariableAttributeBinding& InOther, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	ResetToDefault(InOther, InEmitter.ToBase(), InSourceMode);
}

//////////////////////////////////////////////////////////////////////////

const FNiagaraVariableBase& FNiagaraMaterialAttributeBinding::GetParamMapBindableVariable() const
{
	return ResolvedNiagaraVariable;
}

bool FNiagaraMaterialAttributeBinding::RenameVariableIfMatching(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const UNiagaraEmitterBase* InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	// First try a namespace mangling - free match.
	if (OldVariable.GetName() == NiagaraVariable.GetName() && OldVariable.GetType() == NiagaraVariable.GetType())
	{
		NiagaraVariable = NewVariable;
		CacheValues(InEmitter);
		return true;
	}

	// Now we need to deal with any aliased emitter namespaces for the match. If so resolve the aliases then try the match.
	FNiagaraVariable OldVarAliased = OldVariable;
	if (OldVariable.IsInNameSpace(InEmitter->GetUniqueEmitterName()))
	{
		// First, resolve any aliases
		OldVarAliased = FNiagaraUtilities::ResolveAliases(OldVariable, FNiagaraAliasContext()
			.ChangeEmitterNameToEmitter(InEmitter->GetUniqueEmitterName()));
	}
	if (OldVarAliased.GetName() == NiagaraVariable.GetName() && OldVarAliased.GetType() == NiagaraVariable.GetType())
	{
		NiagaraVariable = NewVariable;
		CacheValues(InEmitter);
		return true;
	}
	return false;
}

bool FNiagaraMaterialAttributeBinding::Matches(const FNiagaraVariableBase& OldVariable, const UNiagaraEmitterBase* InEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	// First try a namespace mangling - free match.
	if (OldVariable.GetName() == NiagaraVariable.GetName() && OldVariable.GetType() == NiagaraVariable.GetType())
	{
		return true;
	}

	// Now we need to deal with any aliased emitter namespaces for the match. If so resolve the aliases then try the match.
	FNiagaraVariable OldVarAliased = OldVariable;
	if (OldVariable.IsInNameSpace(InEmitter->GetUniqueEmitterName()))
	{
		// First, resolve any aliases
		OldVarAliased = FNiagaraUtilities::ResolveAliases(OldVariable, FNiagaraAliasContext()
			.ChangeEmitterNameToEmitter(InEmitter->GetUniqueEmitterName()));
	}
	if (OldVarAliased.GetName() == NiagaraVariable.GetName() && OldVarAliased.GetType() == NiagaraVariable.GetType())
	{
		return true;
	}
	return false;
}

void FNiagaraMaterialAttributeBinding::CacheValues(const UNiagaraEmitterBase* InEmitter)
{
	if (InEmitter != nullptr)
	{
		// Resolve emitter alias.
		ResolvedNiagaraVariable = FNiagaraUtilities::ResolveAliases(NiagaraVariable, 
			FNiagaraAliasContext(FNiagaraAliasContext::ERapidIterationParameterMode::EmitterOrParticleScript)
			.ChangeEmitterToEmitterName(InEmitter->GetUniqueEmitterName()));
		
		// Resolve DI bindings.
		if (ResolvedNiagaraVariable.IsDataInterface())
		{
			ResolvedNiagaraVariable = InEmitter->GetResolvedDIBinding(ResolvedNiagaraVariable);
		}
	}
	else
	{
		ResolvedNiagaraVariable = NiagaraVariable;
	}
}

//////////////////////////////////////////////////////////////////////////

#if NIAGARA_VERBOSE_LOGGING_ENABLED
bool FNiagaraUtilities::LogVerboseWarnings()
{
	return GNiagaraLogVerboseWarnings != 0;
}
#endif

bool FNiagaraUtilities::AllowGPUParticles()
{
	return GNiagaraAllowGPUParticles && GNiagaraAllowComputeShaders && GRHISupportsDrawIndirect;
}

bool FNiagaraUtilities::AllowComputeShaders()
{
	return GNiagaraAllowComputeShaders && GRHISupportsDrawIndirect;
}

bool FNiagaraUtilities::AllowGPUSorting()
{
	return FXConsoleVariables::bAllowGPUSorting != 0;
}

bool FNiagaraUtilities::AllowGPUCulling()
{
	return GNiagaraGPUCulling && AllowGPUSorting() && AllowComputeShaders();
}

bool FNiagaraUtilities::AreBufferSRVsAlwaysCreated(EShaderPlatform ShaderPlatform)
{
	return AreBufferSRVsAlwaysCreatedByDefault(ShaderPlatform);
}

ENiagaraCompileUsageStaticSwitch FNiagaraUtilities::ConvertScriptUsageToStaticSwitchUsage(ENiagaraScriptUsage ScriptUsage)
{
	if (ScriptUsage == ENiagaraScriptUsage::ParticleEventScript)
	{
		return ENiagaraCompileUsageStaticSwitch::Event;
	}
	if (ScriptUsage == ENiagaraScriptUsage::ParticleSimulationStageScript)
	{
		return ENiagaraCompileUsageStaticSwitch::SimulationStage;
	}
	if (ScriptUsage == ENiagaraScriptUsage::EmitterSpawnScript || ScriptUsage == ENiagaraScriptUsage::SystemSpawnScript || ScriptUsage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated || ScriptUsage == ENiagaraScriptUsage::ParticleSpawnScript)
	{
		return ENiagaraCompileUsageStaticSwitch::Spawn;
	}
	if (ScriptUsage == ENiagaraScriptUsage::EmitterUpdateScript || ScriptUsage == ENiagaraScriptUsage::SystemUpdateScript || ScriptUsage == ENiagaraScriptUsage::ParticleUpdateScript)
	{
		return ENiagaraCompileUsageStaticSwitch::Update;
	}
	return ENiagaraCompileUsageStaticSwitch::Default;
}

ENiagaraScriptContextStaticSwitch FNiagaraUtilities::ConvertScriptUsageToStaticSwitchContext(ENiagaraScriptUsage ScriptUsage)
{
	if (ScriptUsage == ENiagaraScriptUsage::SystemSpawnScript || ScriptUsage == ENiagaraScriptUsage::SystemUpdateScript)
	{
		return ENiagaraScriptContextStaticSwitch::System;
	}
	if (ScriptUsage == ENiagaraScriptUsage::EmitterSpawnScript || ScriptUsage == ENiagaraScriptUsage::EmitterUpdateScript)
	{
		return ENiagaraScriptContextStaticSwitch::Emitter;
	}
	return ENiagaraScriptContextStaticSwitch::Particle;
}

FName NIAGARA_API FNiagaraUtilities::GetUniqueName(FName CandidateName, const TSet<FName>& ExistingNames)
{
	// This utility function needs to generate a unique name while only considering the text portion of the name and
	// not the index, so generate names with 0 indices before using them for comparison.
	TSet<FName> ExistingNamesWithIndexZero;
	for (FName ExistingName : ExistingNames)
	{
		ExistingNamesWithIndexZero.Add(FName(ExistingName, 0));
	}
	FName CandidateNameWithIndexZero = FName(CandidateName, 0);

	if (ExistingNamesWithIndexZero.Contains(CandidateNameWithIndexZero) == false)
	{
		return CandidateName;
	}

	FString CandidateNameString = CandidateNameWithIndexZero.ToString();
	FString BaseNameString = CandidateNameString;
	if (CandidateNameString.Len() >= 3 && CandidateNameString.Right(3).IsNumeric())
	{
		BaseNameString = CandidateNameString.Left(CandidateNameString.Len() - 3);
	}

	FName UniqueName = FName(*BaseNameString);
	int32 NameIndex = 1;
	while (ExistingNamesWithIndexZero.Contains(UniqueName))
	{
		UniqueName = FName(*FString::Printf(TEXT("%s%03i"), *BaseNameString, NameIndex));
		NameIndex++;
	}

	return UniqueName;
}

FString FNiagaraUtilities::CreateRapidIterationConstantName(FName InVariableName, const TCHAR* InEmitterName, ENiagaraScriptUsage InUsage)
{
	FNameBuilder ConstantName;

	ConstantName.Append(PARAM_MAP_RAPID_ITERATION_BASE_STR);
	ConstantName.AppendChar(TCHAR('.'));

	if (InEmitterName != nullptr)
	{
		FNameBuilder VariableSource(InVariableName);
		FStringView VariableView(VariableSource);

		constexpr TCHAR EmitterNamespace[] = TEXT("Emitter.");
		constexpr int32 EmitterNamespaceLength = UE_ARRAY_COUNT(EmitterNamespace) - 1;

		const int32 EmitterLocation = VariableView.Find(EmitterNamespace);
		const bool HasEmitterNamespace = EmitterLocation != INDEX_NONE
			&& (EmitterLocation == 0 || VariableView[EmitterLocation - 1] == TCHAR('.'))
			&& (VariableView.Len() > (EmitterLocation + EmitterNamespaceLength));

		ConstantName.Append(InEmitterName);
		ConstantName.AppendChar(TCHAR('.'));

		if (HasEmitterNamespace)
		{
			ConstantName.Append(VariableView.Left(EmitterLocation));
			ConstantName.Append(InEmitterName);
			ConstantName.AppendChar(TCHAR('.'));
			ConstantName.Append(VariableView.RightChop(EmitterLocation + EmitterNamespaceLength));
		}
		else
		{
			ConstantName.Append(VariableView);
		}
	}
	else
	{
		InVariableName.AppendString(ConstantName);
	}

	return ConstantName.ToString();
}

FNiagaraVariable FNiagaraUtilities::ConvertVariableToRapidIterationConstantName(FNiagaraVariable InVar, const TCHAR* InEmitterName, ENiagaraScriptUsage InUsage)
{
	FNiagaraVariable Var = InVar;

	Var.SetName(*CreateRapidIterationConstantName(Var.GetName(), InEmitterName, InUsage));

	return Var;
}

bool FNiagaraScriptDataInterfaceInfo::IsUserDataInterface() const
{
	TStringBuilder<128> NameBuilder;
	Name.ToString(NameBuilder);
	return FCString::Strnicmp(NameBuilder.ToString(), PARAM_MAP_USER_STR, 5) == 0;
}

bool FNiagaraScriptResolvedDataInterfaceInfo::NeedsPerInstanceBinding() const
{
	FNameBuilder NameBuilder;
	Name.ToString(NameBuilder);
	return NameBuilder.ToView().StartsWith(PARAM_MAP_USER_STR) 
		|| (ResolvedDataInterface != nullptr && ResolvedDataInterface->PerInstanceDataSize() > 0);
}

bool FNiagaraScriptDataInterfaceCompileInfo::CanExecuteOnTarget(ENiagaraSimTarget SimTarget) const
{
	// Note that this can be called on non-game threads. We ensure that the data interface CDO object is already in existence at application init time.
	UNiagaraDataInterface* Obj = GetDefaultDataInterface();
	if (Obj)
	{
		return Obj->CanExecuteOnTarget(SimTarget);
	}
	UE_LOGF(LogNiagara, Error, "Failed to call CanExecuteOnTarget for DataInterface \"%ls\". Perhaps missing a plugin for your project?", *Name.ToString());
	return false;
}

UNiagaraDataInterface* FNiagaraScriptDataInterfaceCompileInfo::GetDefaultDataInterface() const
{
	// Note that this can be called on non-game threads. We ensure that the data interface CDO object is already in existence at application init time, so we don't allow this to be auto-created.
	if (Type.IsDataInterface())
	{
		const UClass* TargetClass = const_cast<UClass*>(Type.GetClass());
		if (TargetClass)
		{
			UNiagaraDataInterface* Obj = Cast<UNiagaraDataInterface>(TargetClass->GetDefaultObject(false));
			if (Obj)
				return Obj;

			UE_LOGF(LogNiagara, Error, "Failed to create default object for class \"%ls\". Perhaps missing a plugin for your project?", *TargetClass->GetName());
			return nullptr;
		}

	}
	UE_LOGF(LogNiagara, Error, "Failed to create default object for compiled variable \"%ls\". Perhaps missing a plugin for your project?", *this->Name.ToString());
	return nullptr;
}

bool FNiagaraScriptDataInterfaceCompileInfo::NeedsPerInstanceBinding()const
{
	FNameBuilder NameBuilder;
	Name.ToString(NameBuilder);
	if (NameBuilder.ToView().StartsWith(TEXT("User.")))
	{
		return true;
	}

	UNiagaraDataInterface* Obj = GetDefaultDataInterface();
	if (Obj && Obj->PerInstanceDataSize() > 0)
	{
		return true;
	}
	return false;
}

bool FNiagaraScriptDataInterfaceCompileInfo::MatchesClass(const UClass* InClass) const
{
	UNiagaraDataInterface* Obj = GetDefaultDataInterface();
	if (Obj && Obj->GetClass() == InClass)
		return true;
	return false;
}


void FNiagaraUtilities::DumpHLSLText(const FString& SourceCode, const FString& DebugName)
{
	UE_LOGF(LogNiagara, Display, "Compile output as text: %ls", *DebugName);
	UE_LOGF(LogNiagara, Display, "===================================================================================");
	TArray<FString> OutputByLines;
	SourceCode.ParseIntoArrayLines(OutputByLines, false);
	for (int32 i = 0; i < OutputByLines.Num(); i++)
	{
		UE_LOGF(LogNiagara, Display, "/*%04d*/\t\t%ls", i + 1, *OutputByLines[i]);
	}
	UE_LOGF(LogNiagara, Display, "===================================================================================");
}

FString FNiagaraUtilities::SystemInstanceIDToString(FNiagaraSystemInstanceID ID)
{
	TCHAR Buffer[17];
	uint64 Value = ID;
	for (int i = 15; i >= 0; --i)
	{
		TCHAR ch = Value & 0xf;
		Value >>= 4;
		Buffer[i] = (ch >= 10 ? TCHAR('A' - 10) : TCHAR('0')) + ch;
	}
	Buffer[16] = 0;

	return FString(Buffer);
}

TOptional<EPixelFormat> FNiagaraUtilities::BufferFormatToPixelFormat(ENiagaraGpuBufferFormat NiagaraFormat, EPixelFormatCapabilities RequiredCapabilities, int NumberOfChannels, UObject* RequesterForDebugging)
{
	if (!ensure(NumberOfChannels >= 1 && NumberOfChannels <= 4))
	{
		UE_LOGF(LogNiagara, Error, "Invalid number of channels %d", NumberOfChannels);
		return TOptional<EPixelFormat>();
	}

	static_assert(int(ENiagaraGpuBufferFormat::Float) == 0);
	static_assert(int(ENiagaraGpuBufferFormat::HalfFloat) == 1);
	static_assert(int(ENiagaraGpuBufferFormat::UnsignedNormalizedByte) == 2);
	static const EPixelFormat FormatTable[4][int(ENiagaraGpuBufferFormat::Max)] =
	{
		{EPixelFormat::PF_R32_FLOAT,		EPixelFormat::PF_R16F,		EPixelFormat::PF_R8},
		{EPixelFormat::PF_G32R32F,			EPixelFormat::PF_G16R16F,	EPixelFormat::PF_R8G8},
		{EPixelFormat::PF_A32B32G32R32F,	EPixelFormat::PF_FloatRGBA,	EPixelFormat::PF_R8G8B8A8},
		{EPixelFormat::PF_A32B32G32R32F,	EPixelFormat::PF_FloatRGBA,	EPixelFormat::PF_R8G8B8A8},
	};

	static const TPair<EPixelFormat, EPixelFormat> FormatFallbacks[] =
	{
		TPairInitializer(EPixelFormat::PF_R8,			EPixelFormat::PF_R16F),
		TPairInitializer(EPixelFormat::PF_R16F,			EPixelFormat::PF_R32_FLOAT),
		TPairInitializer(EPixelFormat::PF_R32_FLOAT,	EPixelFormat::PF_G32R32F),

		TPairInitializer(EPixelFormat::PF_R8G8,			EPixelFormat::PF_G16R16F),
		TPairInitializer(EPixelFormat::PF_G16R16F,		EPixelFormat::PF_G32R32F),
		TPairInitializer(EPixelFormat::PF_G32R32F,		EPixelFormat::PF_A32B32G32R32F),

		TPairInitializer(EPixelFormat::PF_R8G8B8A8,		EPixelFormat::PF_FloatRGBA),
		TPairInitializer(EPixelFormat::PF_FloatRGBA,	EPixelFormat::PF_A32B32G32R32F),
	};

	EPixelFormat CurrentFormat = FormatTable[NumberOfChannels - 1][int(NiagaraFormat)];
	while (true)
	{
		if (UE::PixelFormat::HasCapabilities(CurrentFormat, RequiredCapabilities))
		{
			return CurrentFormat;
		}

		bool bKeepTrying = false;
		for (const TPair<EPixelFormat, EPixelFormat>& Fallback : FormatFallbacks)
		{
			if (Fallback.Key == CurrentFormat)
			{
				CurrentFormat = Fallback.Value;
				bKeepTrying = true;
				break;
			}
		}
		if (bKeepTrying == false)
		{
			break;
		}
	};

	UE_LOGF(LogNiagara, Error, "Could not find a valid EPixelFormat for NiagaraFormat(%ls) Channels(%d) Caps(%08x) for (%ls)", *StaticEnum<ENiagaraGpuBufferFormat>()->GetNameStringByValue(int64(NiagaraFormat)), NumberOfChannels, int(RequiredCapabilities), *GetFullNameSafe(RequesterForDebugging));
	return TOptional<EPixelFormat>();
}

TOptional<ETextureRenderTargetFormat> FNiagaraUtilities::BufferFormatToRenderTargetFormat(ENiagaraGpuBufferFormat NiagaraFormat, EPixelFormatCapabilities RequiredCapabilities, UObject* RequesterForDebugging)
{
	switch (NiagaraFormat)
	{
		case ENiagaraGpuBufferFormat::UnsignedNormalizedByte:
			if (UE::PixelFormat::HasCapabilities(EPixelFormat::PF_R8, RequiredCapabilities))
			{
				return ETextureRenderTargetFormat::RTF_R8;
			}
			// Intentional fall through to next format to try

		case ENiagaraGpuBufferFormat::HalfFloat:
			if (UE::PixelFormat::HasCapabilities(EPixelFormat::PF_R16F, RequiredCapabilities))
			{
				return ETextureRenderTargetFormat::RTF_R16f;
			}
			// Intentional fall through to next format to try

		case ENiagaraGpuBufferFormat::Float:
			if (UE::PixelFormat::HasCapabilities(EPixelFormat::PF_R32_FLOAT, RequiredCapabilities))
			{
				return ETextureRenderTargetFormat::RTF_R32f;
			}
			// Intentional fall through to failure

		default:
			UE_LOGF(LogNiagara, Error, "Could not find a valid ETextureRenderTargetFormat for NiagaraFormat(%ls) Caps(%08x) for (%ls)", *StaticEnum<ENiagaraGpuBufferFormat>()->GetNameStringByValue(int64(NiagaraFormat)), int(RequiredCapabilities), *GetFullNameSafe(RequesterForDebugging));
			return TOptional<ETextureRenderTargetFormat>();
	}
}

TOptional<EPixelFormat> FNiagaraUtilities::BufferFormatToPixelFormat(ENiagaraGpuBufferFormat NiagaraFormat, EPixelFormatCapabilities RequiredCapabilities, int NumberOfChannels)
{
	return BufferFormatToPixelFormat(NiagaraFormat, RequiredCapabilities, NumberOfChannels, nullptr);
}

TOptional<ETextureRenderTargetFormat> FNiagaraUtilities::BufferFormatToRenderTargetFormat(ENiagaraGpuBufferFormat NiagaraFormat, EPixelFormatCapabilities RequiredCapabilities)
{
	return BufferFormatToRenderTargetFormat(NiagaraFormat, RequiredCapabilities, nullptr);
}

FString FNiagaraUtilities::SanitizeNameForObjectsAndPackages(const FString& InName)
{
	FString SanitizedName = InName;

	const TCHAR* InvalidObjectChar = INVALID_OBJECTNAME_CHARACTERS;
	while (*InvalidObjectChar)
	{
		SanitizedName.ReplaceCharInline(*InvalidObjectChar, TCHAR('_'), ESearchCase::CaseSensitive);
		++InvalidObjectChar;
	}

	const TCHAR* InvalidPackageChar = INVALID_LONGPACKAGE_CHARACTERS;
	while (*InvalidPackageChar)
	{
		SanitizedName.ReplaceCharInline(*InvalidPackageChar, TCHAR('_'), ESearchCase::CaseSensitive);
		++InvalidPackageChar;
	}

	return SanitizedName;
}

FNiagaraAliasContext& FNiagaraAliasContext::ChangeEmitterToEmitterName(const FString& InEmitterName)
{
	EmitterMapping = TPair<FString, FString>(FNiagaraConstants::EmitterNamespaceString, InEmitterName);
	EmitterName = InEmitterName;
	return *this;
}

FNiagaraAliasContext& FNiagaraAliasContext::ChangeEmitterNameToEmitter(const FString& InEmitterName)
{
	EmitterMapping = TPair<FString, FString>(InEmitterName, FNiagaraConstants::EmitterNamespaceString);
	EmitterName = InEmitterName;
	return *this;
}

FNiagaraAliasContext& FNiagaraAliasContext::ChangeEmitterName(const FString& InOldEmitterName, const FString& InNewEmitterName)
{
	EmitterMapping = TPair<FString, FString>(InOldEmitterName, InNewEmitterName);
	EmitterName = InNewEmitterName;
	return *this;
}

FNiagaraAliasContext& FNiagaraAliasContext::ChangeModuleToModuleName(const FString& InModuleName)
{
	ModuleMapping = TPair<FString, FString>(FNiagaraConstants::ModuleNamespaceString, InModuleName);
	ModuleName = InModuleName;
	return *this;
}

FNiagaraAliasContext& FNiagaraAliasContext::ChangeModuleNameToModule(const FString& InModuleName)
{
	ModuleMapping = TPair<FString, FString>(InModuleName, FNiagaraConstants::ModuleNamespaceString);
	ModuleName = InModuleName;
	return *this;
}

FNiagaraAliasContext& FNiagaraAliasContext::ChangeModuleName(const FString& InOldModuleName, const FString& InNewModuleName)
{
	ModuleMapping = TPair<FString, FString>(InOldModuleName, InNewModuleName);
	ModuleName = InNewModuleName;
	return *this;
}

FNiagaraAliasContext& FNiagaraAliasContext::ChangeStackContext(const FString& InStackContextName)
{
	StackContextMapping = TPair<FString, FString>(FNiagaraConstants::StackContextNamespaceString, InStackContextName);
	StackContextName = InStackContextName;
	return *this;
}

FNiagaraAliasContext& FNiagaraAliasContext::ChangeRapidIterationParameterMode(ERapidIterationParameterMode InMode)
{
	RapidIterationParameterMode = InMode;
	return *this;
}

void AliasRapidIterationConstant(const FNiagaraAliasContext& InContext, TArray<FStringView, TInlineAllocator<16>>& InOutSplitName, int32& OutAssignmentNamespaceIndex)
{
	if(ensureMsgf(InContext.GetRapidIterationParameterMode() != FNiagaraAliasContext::ERapidIterationParameterMode::None, TEXT("Can not resolve a rapid iteration variable without specifying the mode in the context.")))
	{
		// Rapid iteration parameters are in the following format:
		//     Constants.[Emitter Name - Optional - Only in non-system scripts].[Module Name].[Assignment Namespace - Optional].[Value Name]
		int32 MinParts;
		int32 EmitterNameIndex;
		int32 ModuleNameIndex;

		const FNiagaraAliasContext::ERapidIterationParameterMode Mode = InContext.GetRapidIterationParameterMode();

		// static variables break the regular conventions here because they can include both system scoped variables as well as particle/emitter scoped
		// variables, so we try to account for this here.  This really needs to be sorted when we can clean up how we manage our static variables and
		// have them no longer being considered rapid iteration parameters.
		const bool bTreatAsSystemVariable = (Mode == FNiagaraAliasContext::ERapidIterationParameterMode::SystemScript)
			|| (Mode == FNiagaraAliasContext::ERapidIterationParameterMode::StaticVariables && InOutSplitName.Num() == 3);

		if (bTreatAsSystemVariable)
		{
			MinParts = 3;
			EmitterNameIndex = INDEX_NONE;
			ModuleNameIndex = 1;
		}
		else
		{
			MinParts = 4;
			EmitterNameIndex = 1;
			ModuleNameIndex = 2;
		}

		if (ensureAlwaysMsgf(InOutSplitName.Num() >= MinParts, TEXT("Can not resolve malformed rapid iteration parameter '%s' we expect %d parts"), *FString::Join(InOutSplitName, TEXT(".")), MinParts))
		{
			const TOptional<TPair<FString, FString>>& EmitterMapping = InContext.GetEmitterMapping();
			const TOptional<TPair<FString, FString>>& ModuleMapping = InContext.GetModuleMapping();
			if (EmitterNameIndex != INDEX_NONE && EmitterMapping.IsSet() &&
				InOutSplitName[EmitterNameIndex].Equals(EmitterMapping.GetValue().Key))
			{
				InOutSplitName[EmitterNameIndex] = EmitterMapping.GetValue().Value;
			}
			if (ModuleMapping.IsSet() &&
				InOutSplitName[ModuleNameIndex].Equals(ModuleMapping.GetValue().Key))
			{
				InOutSplitName[ModuleNameIndex] = ModuleMapping.GetValue().Value;
			}

			OutAssignmentNamespaceIndex = InOutSplitName[ModuleNameIndex].StartsWith(FNiagaraConstants::AssignmentNodePrefixString) ? ModuleNameIndex + 1 : INDEX_NONE;
		}
	}
}

void AliasEngineSuppliedEmitterValue(const FNiagaraAliasContext& InContext, TArray<FStringView, TInlineAllocator<16>>& InOutSplitName)
{
	// Certain engine supplied values must be aliased per emitter.  Format:
	//     Engine.[Emitter Name - Optional].[Value Name]
	const TOptional<TPair<FString, FString>>& EmitterMapping = InContext.GetEmitterMapping();
	if (EmitterMapping.IsSet() && InOutSplitName.Num() > 2 && InOutSplitName[1].Equals(EmitterMapping.GetValue().Key))
	{
		InOutSplitName[1] = EmitterMapping.GetValue().Value;
	}
}

void AliasStandardParameter(const FNiagaraAliasContext& InContext, TArray<FStringView, TInlineAllocator<16>>& InOutSplitName, int32& OutAssignmentNamespaceIndex)
{
	// Standard parameter format:
	//     [Namespace - dataset, transient, or module].[Assignment Namespace - Optional].[Value Name]
	const TOptional<TPair<FString, FString>>& EmitterMapping = InContext.GetEmitterMapping();
	const TOptional<TPair<FString, FString>>& ModuleMapping = InContext.GetModuleMapping();
	const TOptional<TPair<FString, FString>>& StackContextMapping = InContext.GetStackContextMapping();

	// First alias the stack context mapping since it might map to emitter which would need to be further aliased.
	if (StackContextMapping.IsSet() &&
		InOutSplitName[0].Equals(StackContextMapping.GetValue().Key))
	{
		InOutSplitName[0] = StackContextMapping.GetValue().Value;
	}

	// Alias the emitter mapping next, and if that was not aliased, handle the module mapping.
	if (EmitterMapping.IsSet() &&
		InOutSplitName[0] == EmitterMapping.GetValue().Key)
	{
		InOutSplitName[0] = EmitterMapping.GetValue().Value;
	}
	else if (ModuleMapping.IsSet() &&
		InOutSplitName[0] == ModuleMapping.GetValue().Key)
	{
		InOutSplitName[0] = ModuleMapping.GetValue().Value;
	}

	// If there are more than 2 parts in the parameter, and it's not an assignment node, then
	// it may be a module specific dataset value, so the 2nd position must be checked for the
	// module mapping.
	// Examples to match: Particles.Module.CustomOutput, Transient.Module.PhysicsVar
	// Examples *not* to match: Module.SpawnRate, where SpawnRate is also the name of the module.
	if (InOutSplitName.Num() > 2 && ModuleMapping.IsSet() &&
		InOutSplitName[1] == ModuleMapping.GetValue().Key)
	{
		InOutSplitName[1] = ModuleMapping.GetValue().Value;
	}

	OutAssignmentNamespaceIndex = InOutSplitName[0].StartsWith(FNiagaraConstants::AssignmentNodePrefixString) ? 1 : INDEX_NONE;
}

void AliasAssignmentInputNamespace(const FNiagaraAliasContext& InContext, int32& InAssignmentNamespaceIndex, TArray<FStringView, TInlineAllocator<16>>& InOutSplitName)
{
	if(InAssignmentNamespaceIndex < InOutSplitName.Num())
	{
		const TOptional<TPair<FString, FString>>& EmitterMapping = InContext.GetEmitterMapping();
		const TOptional<TPair<FString, FString>>& StackContextMapping = InContext.GetStackContextMapping();

		if (StackContextMapping.IsSet() &&
			InOutSplitName[InAssignmentNamespaceIndex].Equals(StackContextMapping.GetValue().Key))
		{
			InOutSplitName[InAssignmentNamespaceIndex] = StackContextMapping.GetValue().Value;
		}

		if (EmitterMapping.IsSet() &&
			InOutSplitName[InAssignmentNamespaceIndex].Equals(EmitterMapping.GetValue().Key))
		{
			InOutSplitName[InAssignmentNamespaceIndex] = EmitterMapping.GetValue().Value;
		}
	}
}

FNiagaraVariable FNiagaraUtilities::ResolveAliases(const FNiagaraVariable& InVar, const FNiagaraAliasContext& InContext)
{
	FNiagaraVariable OutVar = InVar;

	TStringBuilder<128> VarName;
	InVar.GetName().ToString(VarName);
	TArray<FStringView, TInlineAllocator<16>> SplitName;
	UE::String::ParseTokens(VarName, TEXT('.'), [&SplitName](FStringView Token) { SplitName.Add(Token); });

	int32 AssignmentNamespaceIndex = INDEX_NONE;
	if (SplitName[0].Equals(FNiagaraConstants::RapidIterationParametersNamespaceString))
	{
		AliasRapidIterationConstant(InContext, SplitName, AssignmentNamespaceIndex);
	}
	else if (SplitName[0].Equals(FNiagaraConstants::EngineNamespaceString))
	{
		AliasEngineSuppliedEmitterValue(InContext, SplitName);
	}
	else
	{
		AliasStandardParameter(InContext, SplitName, AssignmentNamespaceIndex);
	}
	if(AssignmentNamespaceIndex != INDEX_NONE)
	{
		AliasAssignmentInputNamespace(InContext, AssignmentNamespaceIndex, SplitName);
	}

	TStringBuilder<128> OutVarStrName;
	OutVarStrName.Join(SplitName, TEXT("."));

	OutVar.SetName(OutVarStrName.ToString());
	return OutVar;
}

#if WITH_EDITORONLY_DATA
void FNiagaraUtilities::PrepareRapidIterationParameters(const TArray<UNiagaraScript*>& Scripts, const TMap<UNiagaraScript*, UNiagaraScript*>& ScriptDependencyMap, const TMap<UNiagaraScript*, FVersionedNiagaraEmitter>& ScriptToEmitterMap)
{
	SCOPE_CYCLE_COUNTER(STAT_Niagara_Utilities_PrepareRapidIterationParameters);

	TMap<UNiagaraScript*, FNiagaraParameterStore> ScriptToPreparedParameterStoreMap;

	// Remove old and initialize new parameters.
	for (UNiagaraScript* Script : Scripts)
	{
		FNiagaraParameterStore& ParameterStoreToPrepare = ScriptToPreparedParameterStoreMap.FindOrAdd(Script);
		Script->RapidIterationParameters.CopyParametersTo(ParameterStoreToPrepare, false, FNiagaraParameterStore::EDataInterfaceCopyMethod::None);
		ParameterStoreToPrepare.ParameterGuidMapping = Script->RapidIterationParameters.ParameterGuidMapping;
		checkf(ScriptToEmitterMap.Find(Script) != nullptr, TEXT("Script to emitter name map must have an entry for each script to be processed."));
		if (const FVersionedNiagaraEmitter* Emitter = ScriptToEmitterMap.Find(Script))
		{
			if (Script->GetLatestSource())
			{
				Script->GetLatestSource()->CleanUpOldAndInitializeNewRapidIterationParameters(*Emitter, Script->GetUsage(), Script->GetUsageId(), ParameterStoreToPrepare);
			}
		}
	}

	// Copy parameters for dependencies.
	for (auto It = ScriptToPreparedParameterStoreMap.CreateIterator(); It; ++It)
	{
		UNiagaraScript* Script = It.Key();
		FNiagaraParameterStore& PreparedParameterStore = It.Value();
		UNiagaraScript*const* DependentScriptPtr = ScriptDependencyMap.Find(Script);
		if (DependentScriptPtr != nullptr)
		{
			UNiagaraScript* DependentScript = *DependentScriptPtr;
			FNiagaraParameterStore* DependentPreparedParameterStore = ScriptToPreparedParameterStoreMap.Find(DependentScript);
			checkf(DependentPreparedParameterStore != nullptr, TEXT("Dependent scripts must be one of the scripts being processed."));
			PreparedParameterStore.CopyParametersTo(*DependentPreparedParameterStore, false, FNiagaraParameterStore::EDataInterfaceCopyMethod::None);
		}
	}

	// Resolve prepared parameters with the source parameters.
	for (auto It = ScriptToPreparedParameterStoreMap.CreateIterator(); It; ++It)
	{
		UNiagaraScript* Script = It.Key();
		FNiagaraParameterStore& PreparedParameterStore = It.Value();

		auto RapidIterationParameters = Script->RapidIterationParameters.ReadParameterVariables();

		bool bOverwriteParameters = false;
		if (RapidIterationParameters.Num() != PreparedParameterStore.ReadParameterVariables().Num())
		{
			bOverwriteParameters = true;
		}
		else
		{
			for (const FNiagaraVariableWithOffset& ParamWithOffset : RapidIterationParameters)
			{
				const FNiagaraVariable& SourceParameter = ParamWithOffset;
				const int32 SourceOffset = ParamWithOffset.Offset;

				int32 PreparedOffset = PreparedParameterStore.IndexOf(SourceParameter);
				if (PreparedOffset == INDEX_NONE)
				{
					bOverwriteParameters = true;
					break;
				}
				else
				{
					if (FMemory::Memcmp(
						Script->RapidIterationParameters.GetParameterData(SourceOffset, SourceParameter.GetType()),
						PreparedParameterStore.GetParameterData(PreparedOffset, SourceParameter.GetType()),
						SourceParameter.GetSizeInBytes()) != 0)
					{
						bOverwriteParameters = true;
						break;
					}
				}
			}
		}

		if (bOverwriteParameters)
		{
			Script->RapidIterationParameters = PreparedParameterStore;
		}
	}
}

bool FNiagaraUtilities::AreTypesAssignable(const FNiagaraTypeDefinition& FromType, const FNiagaraTypeDefinition& ToType)
{
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	bool bStrictAssignable = (FromType == ToType) || (FromType.IsStatic() && !ToType.IsStatic() && FromType == ToType.ToStaticDef());
	if (Settings->bEnforceStrictStackTypes)
	{
		return bStrictAssignable;
	}
	return bStrictAssignable
		|| (FromType == FNiagaraTypeDefinition::GetPositionDef() && ToType == FNiagaraTypeDefinition::GetVec3Def())
		|| (ToType == FNiagaraTypeDefinition::GetPositionDef() && FromType == FNiagaraTypeDefinition::GetVec3Def());
}

#endif

//////////////////////////////////////////////////////////////////////////

FNiagaraUserParameterBinding::FNiagaraUserParameterBinding()
	: Parameter(FNiagaraTypeDefinition::GetUObjectDef(), NAME_None)
	{

	}

FNiagaraUserParameterBinding::FNiagaraUserParameterBinding(const FNiagaraTypeDefinition& InMaterialDef)
	: Parameter(InMaterialDef, NAME_None)
{
}

//////////////////////////////////////////////////////////////////////////

bool FVMExternalFunctionBindingInfo::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);

	if (Ar.IsLoading() || Ar.IsSaving())
	{
		UScriptStruct* Struct = FVMExternalFunctionBindingInfo::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);
	}

#if WITH_EDITORONLY_DATA
	const int32 NiagaraVersion = Ar.CustomVer(FNiagaraCustomVersion::GUID);

	if (NiagaraVersion < FNiagaraCustomVersion::MemorySaving)
	{
		for (auto it = Specifiers_DEPRECATED.CreateConstIterator(); it; ++it)
		{
			FunctionSpecifiers.Emplace(it->Key, it->Value);
		}
	}
#endif

	return true;
}

const FString FNiagaraCompileOptions::CpuScriptDefine = TEXT("CPUSim");
const FString FNiagaraCompileOptions::GpuScriptDefine = TEXT("GPUComputeSim");
const FString FNiagaraCompileOptions::EventSpawnDefine = TEXT("EventSpawn");
const FString FNiagaraCompileOptions::EventSpawnInitialAttribWritesDefine = TEXT("EventSpawnInitialAttribWrites");
const FString FNiagaraCompileOptions::AccurateQuatInterpolation = TEXT("AccurateQuatInterpolation");

FSynchronizeWithParameterDefinitionsArgs::FSynchronizeWithParameterDefinitionsArgs()
	: SpecificDefinitionsUniqueIds(TArray<FGuid>())
	, SpecificDestScriptVarIds(TArray<FGuid>())
	, bForceGatherDefinitions(false)
	, bForceSynchronizeParameters(false)
	, bSubscribeAllNameMatchParameters(false)
	, AdditionalOldToNewNames()
{
}


ENCPoolMethod ToNiagaraPooling(EPSCPoolMethod PoolingMethod)
{
	if (PoolingMethod == EPSCPoolMethod::AutoRelease)
	{
		return ENCPoolMethod::AutoRelease;
	}
	if (PoolingMethod == EPSCPoolMethod::ManualRelease)
	{
		return ENCPoolMethod::ManualRelease;
	}
	return ENCPoolMethod::None;
}

EPSCPoolMethod ToPSCPoolMethod(ENCPoolMethod PoolingMethod)
{
	if (PoolingMethod == ENCPoolMethod::AutoRelease)
	{
		return EPSCPoolMethod::AutoRelease;
	}
	if (PoolingMethod == ENCPoolMethod::ManualRelease)
	{
		return EPSCPoolMethod::ManualRelease;
	}
	return EPSCPoolMethod::None;
}

//////////////////////////////////////////////////////////////////////////


void FNiagaraFunctionSignature::GetVariadicInputs(TArray<FNiagaraVariableBase>& OutVariadicInputs)const
{
	if(!VariadicInput())
	{
		return;
	}

	static const FNiagaraVariableBase InstDataVar(FNiagaraTypeDefinition::GetIntDef(), TEXT("InstanceData"));
	OutVariadicInputs.Reset(NumOptionalInputs());
	int32 NumInputs = 0;
	for (const FNiagaraVariableBase& Param : Inputs)
	{
		if (Param == InstDataVar)
		{
			continue;
		}

		if (NumInputs++ < VariadicInputStartIndex())
		{
			continue;
		}

		OutVariadicInputs.Emplace(Param);
	}
}

void FNiagaraFunctionSignature::GetVariadicOutputs(TArray<FNiagaraVariableBase>& OutVariadicOutputs)const
{
	if(!VariadicOutput())
	{
		return;
	}

	OutVariadicOutputs.Reset(NumOptionalOutputs());
	int32 NumOutputs = 0;
	for (const FNiagaraVariableBase& Param : Outputs)
	{
		if (NumOutputs++ < VariadicOutputStartIndex())
		{
			continue;
		}

		OutVariadicOutputs.Emplace(Param);
	}
}
