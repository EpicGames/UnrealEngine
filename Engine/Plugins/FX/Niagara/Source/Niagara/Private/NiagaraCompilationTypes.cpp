// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCompilationTypes.h"

#include "Algo/RemoveIf.h"
#include "Internationalization/Regex.h"
#include "Modules/ModuleManager.h"
#include "NiagaraAsyncCompile.h"
#include "NiagaraModule.h"
#include "NiagaraScriptSourceBase.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"
#include "HAL/FileManager.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraCompilationTypes)

#define LOCTEXT_NAMESPACE "NiagaraSystem"

#if WITH_EDITORONLY_DATA

enum class ENiagaraCompilationValidateMode : int32
{
	Ignore,
	Warn,
	Ensure,
	Assert
};

static bool GNiagaraCompileDumpTimings = false;
static FAutoConsoleVariableRef CVarNiagaraCompileDumpTimings(
	TEXT("fx.Niagara.CompileDumpTimings"),
	GNiagaraCompileDumpTimings,
	TEXT("If enabled a file containing compile metrics for the different compiled scripts will be dumped to the log folder"),
	ECVF_ReadOnly
);

static int32 GNiagaraCompileValidateMode = (int32) ENiagaraCompilationValidateMode::Warn;
static FAutoConsoleVariableRef CVarGNiagaraCompileValidateMode(
	TEXT("fx.Niagara.CompileValidateMode"),
	GNiagaraCompileValidateMode,
	TEXT("Controls how the validate compile mode will report differences it encounters when comparing default/async compiles"),
	ECVF_Default
);

// Collection of operators to help validate data between two ExeData (see CompareExeData below)
bool operator==(const FNiagaraVMExecutableByteCode& Lhs, const FNiagaraVMExecutableByteCode& Rhs)
{
	return Lhs.GetData() == Rhs.GetData();
}

bool operator==(const FNiagaraParameters& Lhs, const FNiagaraParameters& Rhs)
{
	return Lhs.Parameters == Rhs.Parameters;
}

bool operator==(const FNiagaraScriptDataUsageInfo& Lhs, const FNiagaraScriptDataUsageInfo& Rhs)
{
	return Lhs.bReadsAttributeData == Rhs.bReadsAttributeData;
}


bool operator==(const FShaderParametersMetadata& Lhs, const FShaderParametersMetadata& Rhs)
{
	return true;
}
bool operator==(const FNiagaraDataInterfaceGPUParamInfo& Lhs, const FNiagaraDataInterfaceGPUParamInfo& Rhs)
{
	return Lhs.DataInterfaceHLSLSymbol == Rhs.DataInterfaceHLSLSymbol
		&& Lhs.DIClassName == Rhs.DIClassName
		&& Lhs.ShaderParametersOffset == Rhs.ShaderParametersOffset
		&& Lhs.GeneratedFunctions == Rhs.GeneratedFunctions;
}

bool operator==(const FVMFunctionSpecifier& Lhs, const FVMFunctionSpecifier& Rhs)
{
	return Lhs.Key == Rhs.Key
		&& Lhs.Value == Rhs.Value;
}

bool operator==(const FVMExternalFunctionBindingInfo& Lhs, const FVMExternalFunctionBindingInfo& Rhs)
{
	return Lhs.Name == Rhs.Name
		&& Lhs.OwnerName == Rhs.OwnerName
		&& Lhs.InputParamLocations == Rhs.InputParamLocations
		&& Lhs.NumOutputs == Rhs.NumOutputs
		&& Lhs.FunctionSpecifiers == Rhs.FunctionSpecifiers;
}

bool operator==(const FNiagaraScriptDataInterfaceCompileInfo& Lhs, const FNiagaraScriptDataInterfaceCompileInfo& Rhs)
{
	return Lhs.Name == Rhs.Name
		&& Lhs.UserPtrIdx == Rhs.UserPtrIdx
		&& Lhs.Type == Rhs.Type
		&& Lhs.RegisteredFunctions == Rhs.RegisteredFunctions
		&& Lhs.RegisteredParameterMapRead == Rhs.RegisteredParameterMapRead
		&& Lhs.RegisteredParameterMapWrite == Rhs.RegisteredParameterMapWrite
		&& Lhs.bIsPlaceholder == Rhs.bIsPlaceholder;
}

bool operator==(const FNiagaraShaderScriptParametersMetadata& Lhs, const FNiagaraShaderScriptParametersMetadata& Rhs)
{
	return Lhs.DataInterfaceParamInfo == Rhs.DataInterfaceParamInfo
		&& Lhs.LooseMetadataNames == Rhs.LooseMetadataNames
		&& Lhs.StructIncludeInfos == Rhs.StructIncludeInfos
		&& Lhs.ShaderParametersMetadata == Rhs.ShaderParametersMetadata;
}

bool operator==(const FNiagaraCompileEvent& Lhs, const FNiagaraCompileEvent& Rhs)
{
	return Lhs.Severity ==  Rhs.Severity
		&& Lhs.Message == Rhs.Message
		&& Lhs.ShortDescription == Rhs.ShortDescription
		&& Lhs.NodeGuid == Rhs.NodeGuid
		&& Lhs.PinGuid == Rhs.PinGuid
		&& Lhs.StackGuids == Rhs.StackGuids
		&& Lhs.Source == Rhs.Source;
}

bool operator==(const FSimulationStageMetaData& Lhs, const FSimulationStageMetaData& Rhs)
{
	return Lhs.SimulationStageName == Rhs.SimulationStageName
		&& Lhs.EnabledBinding == Rhs.EnabledBinding
		&& Lhs.ElementCount == Rhs.ElementCount
		&& Lhs.ElementCountXBinding == Rhs.ElementCountXBinding
		&& Lhs.ElementCountYBinding == Rhs.ElementCountYBinding
		&& Lhs.ElementCountZBinding == Rhs.ElementCountZBinding
		&& Lhs.IterationSourceType == Rhs.IterationSourceType
		&& Lhs.IterationDataInterface == Rhs.IterationDataInterface
		&& Lhs.IterationDirectBinding == Rhs.IterationDirectBinding
		&& Lhs.ExecuteBehavior == Rhs.ExecuteBehavior
		&& Lhs.bWritesParticles == Rhs.bWritesParticles
		&& Lhs.bPartialParticleUpdate == Rhs.bPartialParticleUpdate
		&& Lhs.bParticleIterationStateEnabled == Rhs.bParticleIterationStateEnabled
		&& Lhs.bGpuIndirectDispatch == Rhs.bGpuIndirectDispatch
		&& Lhs.ParticleIterationStateBinding == Rhs.ParticleIterationStateBinding
		&& Lhs.ParticleIterationStateComponentIndex == Rhs.ParticleIterationStateComponentIndex
		&& Lhs.ParticleIterationStateRange == Rhs.ParticleIterationStateRange
		&& Lhs.OutputDestinations == Rhs.OutputDestinations
		&& Lhs.InputDataInterfaces == Rhs.InputDataInterfaces
		&& Lhs.NumIterations == Rhs.NumIterations
		&& Lhs.NumIterationsBinding == Rhs.NumIterationsBinding
		&& Lhs.GpuDispatchType == Rhs.GpuDispatchType
		&& Lhs.GpuDirectDispatchElementType == Rhs.GpuDirectDispatchElementType
		&& Lhs.GpuDispatchNumThreads == Rhs.GpuDispatchNumThreads;
}

bool operator==(const FNiagaraDataSetProperties& Lhs, const FNiagaraDataSetProperties& Rhs)
{
	return Lhs.ID == Rhs.ID
		&& Lhs.Variables == Rhs.Variables;
}

bool operator==(const FNiagaraCompilerTag& Lhs, const FNiagaraCompilerTag& Rhs)
{
	return Lhs.Variable == Rhs.Variable
		&& Lhs.StringValue == Rhs.StringValue;
}

bool operator==(const FNiagaraDataInterfaceStructIncludeInfo& Lhs, const FNiagaraDataInterfaceStructIncludeInfo& Rhs)
{
	return Lhs.StructMetadata == Rhs.StructMetadata
		&& Lhs.ParamterOffset == Rhs.ParamterOffset;
}

bool operator==(const FNiagaraShaderScriptExternalConstant& Lhs, const FNiagaraShaderScriptExternalConstant& Rhs)
{
	return Lhs.Type == Rhs.Type
		&& Lhs.Name == Rhs.Name;
}

class FNiagaraActiveCompilationAsyncTask : public FNiagaraActiveCompilation
{
public:
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNiagaraActiveCompilationAsyncTask");
	}

	virtual bool Launch(const FNiagaraCompilationOptions& Options) override
	{
		TaskStartTime = FPlatformTime::Seconds();
		bForced = Options.bForced;

		INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");

		CompileRequestHandle = NiagaraModule.RequestCompileSystem(Options.System, bForced, Options.TargetPlatform);

		return CompileRequestHandle != INDEX_NONE;
	}

	virtual void Abort() override
	{
		INiagaraModule& NiagaraModule = FModuleManager::Get().LoadModuleChecked<INiagaraModule>(TEXT("Niagara"));
		NiagaraModule.AbortSystemCompile(CompileRequestHandle);

		bShouldApply = false;
	}

	virtual bool ValidateConsistentResults(const FNiagaraQueryCompilationOptions& Options) const override
	{
		const UNiagaraScript* SystemSpawnScript = Options.System->GetSystemSpawnScript();
		const UNiagaraScript* SystemUpdateScript = Options.System->GetSystemUpdateScript();

		// for now the only thing we're concerned about is if we've got results for SystemSpawn and SystemUpdate scripts
		// then we need to make sure that they agree in terms of the dataset attributes
		const FNiagaraScriptAsyncCompileData* SpawnScriptRequest = CompileResults.CompileResultMap.Find(SystemSpawnScript);
		const FNiagaraScriptAsyncCompileData* UpdateScriptRequest = CompileResults.CompileResultMap.Find(SystemUpdateScript);

		const bool SpawnScriptValid = SpawnScriptRequest
			&& SpawnScriptRequest->ExeData.IsValid()
			&& SpawnScriptRequest->ExeData->LastCompileStatus != ENiagaraScriptCompileStatus::NCS_Error;

		const bool UpdateScriptValid = UpdateScriptRequest
			&& UpdateScriptRequest->ExeData.IsValid()
			&& UpdateScriptRequest->ExeData->LastCompileStatus != ENiagaraScriptCompileStatus::NCS_Error;

		if (SpawnScriptValid && UpdateScriptValid)
		{
			if (SpawnScriptRequest->ExeData->Attributes != UpdateScriptRequest->ExeData->Attributes)
			{
				// if we had requested a full rebuild, then we've got a case where the generated scripts are not compatible.  This indicates
				// a significant issue where we're allowing graphs to generate invalid collections of scripts.  One known example is using
				// the Script.Context static switch that isn't fully processed in all scripts, leading to attributes differing between the
				// SystemSpawnScript and the SystemUpdateScript
				if (bForced)
				{
					FString MissingAttributes;
					FString AdditionalAttributes;

					for (const auto& SpawnAttrib : SpawnScriptRequest->ExeData->Attributes)
					{
						if (!UpdateScriptRequest->ExeData->Attributes.Contains(SpawnAttrib))
						{
							MissingAttributes.Appendf(TEXT("%s%s"), MissingAttributes.Len() ? TEXT(", ") : TEXT(""), *SpawnAttrib.GetName().ToString());
						}
					}

					for (const auto& UpdateAttrib : UpdateScriptRequest->ExeData->Attributes)
					{
						if (!SpawnScriptRequest->ExeData->Attributes.Contains(UpdateAttrib))
						{
							AdditionalAttributes.Appendf(TEXT("%s%s"), AdditionalAttributes.Len() ? TEXT(", ") : TEXT(""), *UpdateAttrib.GetName().ToString());
						}
					}

					FNiagaraCompileEvent AttributeMismatchEvent(
						FNiagaraCompileEventSeverity::Error,
						FText::Format(LOCTEXT("SystemScriptAttributeMismatchError", "System Spawn/Update scripts have attributes which don't match!\n\tMissing update attributes: {0}\n\tAdditional update attributes: {1}"),
							FText::FromString(MissingAttributes),
							FText::FromString(AdditionalAttributes))
						.ToString());

					SpawnScriptRequest->ExeData->LastCompileStatus = ENiagaraScriptCompileStatus::NCS_Error;
					SpawnScriptRequest->ExeData->LastCompileEvents.Add(AttributeMismatchEvent);
				}
				else
				{
					UE_LOGF(LogNiagara, Log, "Failed to generate consistent results for System spawn and update scripts for system %ls.", *Options.System->GetFullName());
				}

				return false;
			}
		}

		return true;
	}

	virtual bool QueryCompileComplete(const FNiagaraQueryCompilationOptions& Options) override
	{
		if (CompileRequestHandle == INDEX_NONE)
		{
			return true;
		}

		check(IsInGameThread());
		INiagaraModule& NiagaraModule = FModuleManager::Get().LoadModuleChecked<INiagaraModule>(TEXT("Niagara"));

		constexpr bool bPeek = false;
		if (NiagaraModule.PollSystemCompile(CompileRequestHandle, CompileResults, Options.bWait, bPeek))
		{
			CompileRequestHandle = INDEX_NONE;
			return true;
		}
		return false;
	}

	virtual void Apply(const FNiagaraQueryCompilationOptions& Options) override
	{
		// we need to do this apply in multiple passes.  Because the script's VMCompilationId has dependencies
		// on the rapid iteration parameters of all the related scripts then we need to be sure to update the
		// RI parameters with the data that we've collected before we actually try to set the results.
		// Some additional caveats with this path:
		// -our digested graph will not include disabled modules which means that their inputs, which could
		//	resolve to rapid iteration parameters, and so we'll be missing those in our list.  We can't remove
		//	parameters just because things are disabled because the editor will forget the settings.
		constexpr bool bAllowParameterRemoval = false;

		auto GetValidTargetScript = [](UNiagaraScript* Script) -> UNiagaraScript*
		{
			return ::IsValid(Script) ? Script : nullptr;
		};

		for (FNiagaraSystemAsyncCompileResults::FCompileResultMap::TConstIterator ResultIt(CompileResults.CompileResultMap);
			ResultIt;
			++ResultIt)
		{
			const FNiagaraScriptAsyncCompileData& ScriptCompileData = ResultIt->Value;

			if (UNiagaraScript* TargetScript = GetValidTargetScript(ResultIt->Key))
			{
				TargetScript->ApplyRapidIterationParameters(ScriptCompileData.RapidIterationParameters, bAllowParameterRemoval);
			}
		}

		// Now that the above code says they are all complete, go ahead and resolve them all at once.
		for (FNiagaraSystemAsyncCompileResults::FCompileResultMap::TConstIterator ResultIt(CompileResults.CompileResultMap);
			ResultIt;
			++ResultIt)
		{
			if (UNiagaraScript* TargetScript = GetValidTargetScript(ResultIt->Key))
			{
				const FNiagaraScriptAsyncCompileData& ScriptCompileData = ResultIt->Value;

				if (ScriptCompileData.ExeData.IsValid())
				{
					// because our compilation process includes the generation of rapid iteration parameters and static
					// variables we need to generate the ExecutableDataId
					// if we dirtied any RI parameters then we need to regenerate our CompilationId
					FNiagaraVMExecutableDataId UpdatedCompileId;
					TargetScript->ComputeVMCompilationId(UpdatedCompileId, FGuid());

					TMap<FName, UNiagaraDataInterface*> ObjectNameMap;

					// The original implementation would generate DI references from the compilation data (unless things were pulled from
					// the DDC).  We will always pull the data from the target scripts so that we can avoid any weird caching issues with
					// our digested graphs, but it also should ensure more consistent behavior (the compilation should only depend on the
					// aspects of DI that impact compilation, other changes that could have been made by the user shouldn't be erased
					// when the compilation results are applied.
					if (const UNiagaraScriptSourceBase* ScriptSource = TargetScript->GetLatestSource())
					{
						ObjectNameMap = ScriptSource->ComputeObjectNameMap(*Options.System, TargetScript->GetUsage(), TargetScript->GetUsageId(), ScriptCompileData.UniqueEmitterName);
					}

					constexpr bool bApplyRapidIterationParameters = false;
					TargetScript->SetVMCompilationResults(
						UpdatedCompileId,
						*ScriptCompileData.ExeData,
						ScriptCompileData.UniqueEmitterName,
						ObjectNameMap,
						bApplyRapidIterationParameters);

					if (!ScriptCompileData.CompiledShaders.IsEmpty())
					{
						for (const FNiagaraCompiledShaderInfo& ShaderMapInfo : ScriptCompileData.CompiledShaders)
						{
							TargetScript->SetComputeCompilationResults(
								ShaderMapInfo.TargetPlatform,
								ShaderMapInfo.ShaderPlatform,
								ShaderMapInfo.FeatureLevel,
								ScriptCompileData.ExeData->ShaderScriptParametersMetadata,
								ShaderMapInfo.CompiledShader,
								ShaderMapInfo.CompilationErrors);
						}
					}
				}
			}
		}

		// Synchronize the variables that we actually encountered during precompile so that we can expose them to the end user.
		{
			FNiagaraUserRedirectionParameterStore& ExposedParameters = Options.System->GetExposedParameters();

			for (const FNiagaraVariable& ExposedVariable : CompileResults.ExposedVariables)
			{
				if (!ExposedParameters.FindParameterOffset(ExposedVariable))
				{
					ExposedParameters.AddParameter(ExposedVariable, true, false);
				}
			}
		}
	}

	virtual void ReportResults(const FNiagaraQueryCompilationOptions& Options) const override
	{
		const float ElapsedWallTime = (float)(FPlatformTime::Seconds() - TaskStartTime);

		UE_LOGF(LogNiagara, Log, "Compiling System %ls took %f sec (time since issued).",
			*Options.System->GetFullName(), ElapsedWallTime);

		if (Options.bGenerateTimingsFile)
		{
			FNiagaraSystemCompileMetrics SystemMetrics;
			SystemMetrics.SystemCompileWallTime = ElapsedWallTime;
			SystemMetrics.ScriptMetrics.Reserve(CompileResults.CompileResultMap.Num());

			for (FNiagaraSystemAsyncCompileResults::FCompileResultMap::TConstIterator ResultIt(CompileResults.CompileResultMap);
				ResultIt;
				++ResultIt)
			{
				SystemMetrics.ScriptMetrics.Add(ResultIt.Key(), ResultIt.Value().CompileMetrics);
			}

			WriteTimingsEntry(TEXT("AsyncTask Compilation"), Options, SystemMetrics);
		}
	}

	virtual bool BlocksGarbageCollection() const override
	{
		return false;
	}

	FNiagaraCompilationTaskHandle CompileRequestHandle;
	FNiagaraSystemAsyncCompileResults CompileResults;
};

class FNiagaraActiveCompilationVerify : public FNiagaraActiveCompilation
{
public:
	FNiagaraActiveCompilationVerify()
	{
		AsyncTaskRequest = MakeUnique<FNiagaraActiveCompilationAsyncTask>();
		DefaultRequest = MakeUnique<FNiagaraActiveCompilationDefault>();
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		AsyncTaskRequest->AddReferencedObjects(Collector);
		DefaultRequest->AddReferencedObjects(Collector);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNiagaraActiveCompilationVerify");
	}

	virtual bool Launch(const FNiagaraCompilationOptions& Options) override
	{
		bForced = Options.bForced;

		const bool AsyncLaunched = AsyncTaskRequest->Launch(Options);
		const bool DefaultLaunched = DefaultRequest->Launch(Options);

		check(AsyncLaunched == DefaultLaunched);

		return AsyncLaunched || DefaultLaunched;
	}

	virtual void Abort() override
	{
		AsyncTaskRequest->Abort();
		DefaultRequest->Abort();
	}

	virtual bool QueryCompileComplete(const FNiagaraQueryCompilationOptions& Options) override
	{
		const bool AsyncComplete = AsyncTaskRequest->QueryCompileComplete(Options);
		const bool DefaultComplete = DefaultRequest->QueryCompileComplete(Options);

		return AsyncComplete && DefaultComplete;
	}

	virtual bool ValidateConsistentResults(const FNiagaraQueryCompilationOptions& Options) const override
	{
		bool DefaultValid = DefaultRequest->ValidateConsistentResults(Options);
		bool AsyncTaskValid = AsyncTaskRequest->ValidateConsistentResults(Options);

		// for this mode to be valid, both child modes must be valid
		return DefaultValid && AsyncTaskValid;
	}

	virtual void Apply(const FNiagaraQueryCompilationOptions& Options) override
	{
		// for now we'll just apply the default results
		DefaultRequest->Apply(Options);
	}

	virtual void ReportResults(const FNiagaraQueryCompilationOptions& Options) const override
	{
		auto VariableLess = [](const FNiagaraVariableBase& Lhs, const FNiagaraVariableBase& Rhs) -> bool
		{
			return Lhs.GetName().LexicalLess(Rhs.GetName());
		};

		// exposed parameters
		{
			TArray<FNiagaraVariable> DefaultExposedVariables;
			{
				TSet<FNiagaraVariable> UniqueDefaultExposedVariables;
				for (FNiagaraActiveCompilationDefault::FAsyncTaskPtr TaskPtr : DefaultRequest->Tasks)
				{
					UniqueDefaultExposedVariables.Append(TaskPtr->EncounteredExposedVars);
				}

				DefaultExposedVariables = UniqueDefaultExposedVariables.Array();
				DefaultExposedVariables.Sort(VariableLess);
			}

			TArray<FNiagaraVariable> AsyncTaskExposedVariables;
			{
				TSet<FNiagaraVariable> UniqueAsyncTaskExposedVariables(AsyncTaskRequest->CompileResults.ExposedVariables);

				AsyncTaskExposedVariables = UniqueAsyncTaskExposedVariables.Array();
				AsyncTaskExposedVariables.Sort(VariableLess);
			}

			EvaluateTest(Options, nullptr, DefaultExposedVariables == AsyncTaskExposedVariables, TEXT("Comparing ExposedVariables"));
		}

		// FNiagaraVMExecutableData
		{
			using FExeDataPair = TTuple<TSharedPtr<FNiagaraVMExecutableData>, TSharedPtr<FNiagaraVMExecutableData>>;
			TMap<UNiagaraScript*, FExeDataPair> MappedExeData;

			for (FNiagaraActiveCompilationDefault::FAsyncTaskPtr TaskPtr : DefaultRequest->Tasks)
			{
				FExeDataPair& DataPair = MappedExeData.FindOrAdd(TaskPtr->ScriptPair.CompiledScript);
				DataPair.Key = TaskPtr->ExeData;
			}

			for (const auto& ResultIt : AsyncTaskRequest->CompileResults.CompileResultMap)
			{
				FExeDataPair& DataPair = MappedExeData.FindOrAdd(ResultIt.Key);
				DataPair.Value = ResultIt.Value.ExeData;
			}

			for (const auto& ResultIt : MappedExeData)
			{
				CompareExeData(Options, ResultIt.Key, ResultIt.Value.Key, ResultIt.Value.Value);
			}
		}
	}

	virtual bool BlocksBeginCacheForCooked() const override
	{
		return DefaultRequest->BlocksBeginCacheForCooked() || AsyncTaskRequest->BlocksBeginCacheForCooked();
	}

private:
	bool EvaluateTest(const FNiagaraQueryCompilationOptions& Options, UNiagaraScript* Script, bool bTestSuccess, const TCHAR* TestDescription) const
	{
		if (bTestSuccess || GNiagaraCompileValidateMode == (int32) ENiagaraCompilationValidateMode::Ignore)
		{
			return bTestSuccess;
		}

		auto GetSystemScriptName = [](UNiagaraSystem* InSystem, UNiagaraScript* InScript) -> FString
		{
			FString SystemScriptName = GetPathNameSafe(InSystem);
			if (InScript && InSystem)
			{
				SystemScriptName.Append(TEXT(":"));
				SystemScriptName.Append(InScript->GetPathName(InSystem));
			}

			return SystemScriptName;
		};

		switch ((ENiagaraCompilationValidateMode)GNiagaraCompileValidateMode)
		{
			case ENiagaraCompilationValidateMode::Warn:
				UE_LOGF(LogNiagara, Warning, "%ls - Failed NiagaraCompile validation %ls", *GetSystemScriptName(Options.System, Script), TestDescription);
				break;

			case ENiagaraCompilationValidateMode::Ensure:
				ensureMsgf(false, TEXT("%s - Failed NiagaraCompile validation %s"), *GetSystemScriptName(Options.System, Script), TestDescription);
				break;

			case ENiagaraCompilationValidateMode::Assert:
				checkf(false, TEXT("%s - Failed NiagaraCompile validation %s"), *GetSystemScriptName(Options.System, Script), TestDescription);
				break;
		}

		return bTestSuccess;
	}

	void CompareExeData(const FNiagaraQueryCompilationOptions& Options, UNiagaraScript* Script, TSharedPtr<FNiagaraVMExecutableData> DefaultExeData, TSharedPtr<FNiagaraVMExecutableData> AsyncTaskExeData) const
	{
		const bool HasDefaultData = DefaultExeData.IsValid();
		const bool HasAsyncTaskData = AsyncTaskExeData.IsValid();

		if (!EvaluateTest(Options, Script, HasDefaultData == HasAsyncTaskData, TEXT("Missing script compilation result")))
		{
			return;
		}
		if (!HasDefaultData && !HasAsyncTaskData)
		{
			return;
		}

		const bool ByteCodeMatches = DefaultExeData->ByteCode == AsyncTaskExeData->ByteCode;
		if (!EvaluateTest(Options, Script, ByteCodeMatches, TEXT("Comparing ByteCode")))
		{
			bool bSignificantSourceDiff = false;

			if (DefaultExeData->LastHlslTranslation != AsyncTaskExeData->LastHlslTranslation)
			{
				// see if it's just the two common lines that are different
				TArray<FString> ExeDataTextLines;
				DefaultExeData->LastHlslTranslation.ParseIntoArrayLines(ExeDataTextLines);
				TArray<FString> ExeData2TextLines;
				AsyncTaskExeData->LastHlslTranslation.ParseIntoArrayLines(ExeData2TextLines);

				auto IsAComment = [](const FString& TextLine) -> bool
				{
					return TextLine.TrimStart().StartsWith(TEXT("//"));
				};

				ExeDataTextLines.SetNum(Algo::StableRemoveIf(ExeDataTextLines, IsAComment));
				ExeData2TextLines.SetNum(Algo::StableRemoveIf(ExeData2TextLines, IsAComment));

				const int32 LineCount = ExeDataTextLines.Num();
				if (LineCount == ExeData2TextLines.Num())
				{
					for (int32 LineIt = 0; LineIt < LineCount; ++LineIt)
					{
						const FString& ExeDataLine = ExeDataTextLines[LineIt];
						const FString& ExeData2Line = ExeData2TextLines[LineIt];

						if (ExeDataLine == ExeData2Line)
						{
							continue;
						}
						else
						{
							// the following are various hacks put in place to let some specific code
							// differences be ignored.  In particular bCompleteOnInactive0, when talking about
							// the system spawn script is a common false positive error - the flag is initialized
							// to true in the system update script, but is basically undefined in the system
							// spawn script.  With the new digest pattern the default value from the update
							// script is identified and used rather than leaving it undefined (and getting a
							// default value of false).
							{
								FRegexPattern Pattern(TEXT("\\s*bool System_bCompleteOnInactive0 = (.*?);"));
								FRegexMatcher ExeDataMatcher(Pattern, ExeDataLine);
								FRegexMatcher ExeData2Matcher(Pattern, ExeData2Line);
								if (ExeDataMatcher.FindNext() && ExeData2Matcher.FindNext())
								{
									if (ExeDataMatcher.GetCaptureGroup(1) == TEXT("false")
										&& ExeData2Matcher.GetCaptureGroup(1) == TEXT("true"))
									{
										continue;
									}
								}
							}

							// this follows the above false positive with bCompleteOnInactive, where we will
							// sometimes get the code of the following pattern:
							// bool Constant4 = true;
							// System_bCompleteOnInactive = Constant4;
							// This pattern will evaluate the Constant declaration and see if it is immediately
							// being applied to bCompleteOnInactive, if so we can safely ignore the difference
							{
								FRegexPattern Pattern(TEXT("\\s*bool Constant(\\d+) = (.*?);"));
								FRegexMatcher ExeDataMatcher(Pattern, ExeDataLine);
								FRegexMatcher ExeData2Matcher(Pattern, ExeData2Line);
								if (ExeDataMatcher.FindNext() && ExeData2Matcher.FindNext())
								{
									const FString& ExeDataConstantIndex = ExeDataMatcher.GetCaptureGroup(1);
									const FString& ExeData2ConstantIndex = ExeData2Matcher.GetCaptureGroup(1);
									if (ExeDataConstantIndex == ExeData2ConstantIndex)
									{
										// look ahead to the next line to see if this is in relation to bCompleteOnInactive
										// if it is then we'll ignore this difference
										const FString& NextExeDataLine = ExeDataTextLines[LineIt + 1];
										const FString& NextExeData2Line = ExeData2TextLines[LineIt + 1];
										if (NextExeDataLine == NextExeData2Line && NextExeDataLine.Contains(TEXT("bCompleteOnInactive")))
										{
											continue;
										}
									}
								}
							}

							// another common false positive in comparing the two versions of the hlsl source
							// is the difference between:
							// float MyVariable = 0.0f;
							// and
							// float MyVariable = (0);
							{
								FRegexPattern Pattern(TEXT("\t*float .*? = \\(?(.*?)\\)?;"));
								FRegexMatcher ExeDataMatcher(Pattern, ExeDataLine);
								FRegexMatcher ExeData2Matcher(Pattern, ExeData2Line);
								if (ExeDataMatcher.FindNext() && ExeData2Matcher.FindNext())
								{
									const FString& ExeDataValue = ExeDataMatcher.GetCaptureGroup(1);
									const FString& ExeData2Value = ExeData2Matcher.GetCaptureGroup(1);
									if (ExeDataValue == ExeData2Value)
									{
										continue;
									}
									float ExeDataFloat = -1.0f;
									float ExeData2Float = -1.0f;
									LexFromString(ExeDataFloat, *ExeDataValue);
									LexFromString(ExeData2Float, *ExeData2Value);

									if (ExeDataFloat == ExeData2Float)
									{
										continue;
									}
								}
							}

							bSignificantSourceDiff = true;
							break;
						}
					}
				}
				else
				{
					bSignificantSourceDiff = true;
				}
			}

			EvaluateTest(Options, Script, !bSignificantSourceDiff, TEXT("Comparing HLSL source code"));
		}

		EvaluateTest(Options, Script, DefaultExeData->NumTempRegisters == AsyncTaskExeData->NumTempRegisters, TEXT("NumTempRegisters"));
		EvaluateTest(Options, Script, DefaultExeData->NumUserPtrs == AsyncTaskExeData->NumUserPtrs, TEXT("NumUserPtrs"));
		EvaluateTest(Options, Script, DefaultExeData->Parameters == AsyncTaskExeData->Parameters, TEXT("Parameters"));
		EvaluateTest(Options, Script, DefaultExeData->InternalParameters == AsyncTaskExeData->InternalParameters, TEXT("InternalParameters"));
		EvaluateTest(Options, Script, DefaultExeData->ExternalDependencies == AsyncTaskExeData->ExternalDependencies, TEXT("ExternalDependencies"));
		EvaluateTest(Options, Script, DefaultExeData->CompileTags == AsyncTaskExeData->CompileTags, TEXT("CompileTags"));
		EvaluateTest(Options, Script, DefaultExeData->ScriptLiterals == AsyncTaskExeData->ScriptLiterals, TEXT("ScriptLiterals"));
		EvaluateTest(Options, Script, DefaultExeData->Attributes == AsyncTaskExeData->Attributes, TEXT("Attributes"));
		EvaluateTest(Options, Script, DefaultExeData->DataUsage == AsyncTaskExeData->DataUsage, TEXT("DataUsage"));

		// known false positive for DataSetToParameters - when compiling the new graph results from static switches
		// can be propagated forward as we cull the static switch on instantiating the graph.  This can lead to
		// reduced nodes visited during compilation, which can have the side effect of not encountering constants;
		// Emitter.RandomSeed is an example of a constant that is skipped with CalculateRangedInt.  The new
		// traversal is more correct, as the constant is not actually referenced by the system.
		EvaluateTest(Options, Script, DefaultExeData->DataSetToParameters.OrderIndependentCompareEqual(AsyncTaskExeData->DataSetToParameters), TEXT("DataSetToParameters"));
		EvaluateTest(Options, Script, DefaultExeData->AdditionalExternalFunctions == AsyncTaskExeData->AdditionalExternalFunctions, TEXT("AdditionalExternalFunctions"));
		EvaluateTest(Options, Script, DefaultExeData->DataInterfaceInfo == AsyncTaskExeData->DataInterfaceInfo, TEXT("DataInterfaceInfo"));
		EvaluateTest(Options, Script, DefaultExeData->CalledVMExternalFunctions == AsyncTaskExeData->CalledVMExternalFunctions, TEXT("CalledVMExternalFunctions"));
		EvaluateTest(Options, Script, DefaultExeData->ReadDataSets == AsyncTaskExeData->ReadDataSets, TEXT("ReadDataSets"));
		EvaluateTest(Options, Script, DefaultExeData->WriteDataSets == AsyncTaskExeData->WriteDataSets, TEXT("WriteDataSets"));
		EvaluateTest(Options, Script, DefaultExeData->StatScopes == AsyncTaskExeData->StatScopes, TEXT("StatScopes"));
		EvaluateTest(Options, Script, DefaultExeData->LastOpCount == AsyncTaskExeData->LastOpCount, TEXT("LastOpCount"));
		EvaluateTest(Options, Script, DefaultExeData->ShaderScriptParametersMetadata == AsyncTaskExeData->ShaderScriptParametersMetadata, TEXT("ShaderScirptParameterMetaData"));
		EvaluateTest(Options, Script, DefaultExeData->ParameterCollectionPaths == AsyncTaskExeData->ParameterCollectionPaths, TEXT("ParameterCollectionPaths"));
		EvaluateTest(Options, Script, DefaultExeData->SimulationStageMetaData == AsyncTaskExeData->SimulationStageMetaData, TEXT("SimulationStageMetaData"));
		EvaluateTest(Options, Script, DefaultExeData->bReadsAttributeData == AsyncTaskExeData->bReadsAttributeData, TEXT("bReadsAttributeData"));
		EvaluateTest(Options, Script, DefaultExeData->AttributesWritten == AsyncTaskExeData->AttributesWritten, TEXT("AttributesWritten"));
		EvaluateTest(Options, Script, DefaultExeData->StaticVariablesWritten == AsyncTaskExeData->StaticVariablesWritten, TEXT("StaticVaraiblesWritten"));
		EvaluateTest(Options, Script, DefaultExeData->ErrorMsg == AsyncTaskExeData->ErrorMsg, TEXT("ErrorMsg"));
		EvaluateTest(Options, Script, DefaultExeData->ExperimentalContextData == AsyncTaskExeData->ExperimentalContextData, TEXT("ExperimentalContextData"));
		EvaluateTest(Options, Script, DefaultExeData->bReadsSignificanceIndex == AsyncTaskExeData->bReadsSignificanceIndex, TEXT("bReadsSignificanceIndex"));
		EvaluateTest(Options, Script, DefaultExeData->bNeedsGPUContextInit == AsyncTaskExeData->bNeedsGPUContextInit, TEXT("bNeedsGPUContextInit"));

		// because we can't fully trust the rapid iteration parameters we generate with the asyncTask method (because we ignore the
		// entries from disabled modules).  We can't just do a straight compare.  Instead we'll see if any entries from the default
		// are missing from the asynctask results
		for (const FNiagaraVariable& DefaultParameter : DefaultExeData->BakedRapidIterationParameters)
		{
			const int32 AsyncTaskIndex = AsyncTaskExeData->BakedRapidIterationParameters.IndexOfByKey(DefaultParameter);
			EvaluateTest(Options, Script, AsyncTaskIndex != INDEX_NONE, TEXT("Missing BakedRapidIterationParameter"));
			if (AsyncTaskIndex != INDEX_NONE)
			{
				const FNiagaraVariable& AsyncTaskParameter = AsyncTaskExeData->BakedRapidIterationParameters[AsyncTaskIndex];
				EvaluateTest(Options, Script, AsyncTaskParameter.HoldsSameData(DefaultParameter), TEXT("Mismatched BakedRapidIterationParmaeter data value"));
			}
		}
	}

	TUniquePtr<FNiagaraActiveCompilationDefault> DefaultRequest;
	TUniquePtr<FNiagaraActiveCompilationAsyncTask> AsyncTaskRequest;
};

#endif // WITH_EDITORONLY_DATA

TUniquePtr<FNiagaraActiveCompilation> FNiagaraActiveCompilation::CreateCompilation()
{
#if WITH_EDITORONLY_DATA
	const ENiagaraCompilationMode CompilationMode = GetDefault<UNiagaraSettings>()->CompilationMode;

	if (CompilationMode == ENiagaraCompilationMode::AsyncTasks)
	{
		return MakeUnique<FNiagaraActiveCompilationAsyncTask>();
	}
	else if (CompilationMode == ENiagaraCompilationMode::Verify)
	{
		return MakeUnique<FNiagaraActiveCompilationVerify>();
	}

	return MakeUnique<FNiagaraActiveCompilationDefault>();
#else // WITH_EDITORONLY_DATA
	return nullptr;
#endif
}

void FNiagaraActiveCompilation::WriteTimingsEntry(const TCHAR* TimingsSource, const FNiagaraQueryCompilationOptions& Options, const FNiagaraSystemCompileMetrics& SystemMetrics) const
{
#if WITH_EDITORONLY_DATA && ALLOW_DEBUG_FILES
	static bool HasWrittenFile = false;
	static FCriticalSection TimingLock;
	static FString TimingFile;

	FScopeLock Lock(&TimingLock);

	uint32 WriteFlags = FILEWRITE_Append;
	if (!HasWrittenFile)
	{
		WriteFlags = FILEWRITE_None;
		HasWrittenFile = true;
		TimingFile = FPaths::ProjectSavedDir() / TEXT("Logs") / TEXT("NiagaraTimings_") + FDateTime::Now().ToString() + TEXT(".log");
	}

	TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*TimingFile, WriteFlags));

	const int32 ScriptCount = SystemMetrics.ScriptMetrics.Num();
	TArray<TObjectKey<UNiagaraScript>> SortedScripts;
	SystemMetrics.ScriptMetrics.GenerateKeyArray(SortedScripts);

	SortedScripts.Sort([](const TObjectKey<UNiagaraScript>& LhsScriptKey, const TObjectKey<UNiagaraScript>& RhsScriptKey) -> bool
	{
		return GetNameSafe(LhsScriptKey.ResolveObjectPtr()) < GetNameSafe(RhsScriptKey.ResolveObjectPtr());
	});

	FileArchive->Logf(TEXT("============================================================================"));
	FileArchive->Logf(TEXT("==  Compiling %s using %s"), *Options.System->GetFullName(), TimingsSource);
	FileArchive->Logf(TEXT("============================================================================"));
	FileArchive->Logf(TEXT("== Task count: %d -> %fs for full task"), SortedScripts.Num(), SystemMetrics.SystemCompileWallTime);

	if (!SortedScripts.IsEmpty())
	{
		FileArchive->Logf(TEXT("Script, Task Wall Time, DDC Fetch, Translate, Compile (Wall Time), Compile (Preprocess), Compile (Worker), Byte Code Optimize"));
		for (const TObjectKey<UNiagaraScript> ScriptKey : SortedScripts)
		{
			const FNiagaraScriptCompileMetrics& ScriptMetric = SystemMetrics.ScriptMetrics.FindRef(ScriptKey);

			FileArchive->Logf(TEXT("%s, %f, %f, %f, %f, %f, %f, %f"),
				*GetPathNameSafe(ScriptKey.ResolveObjectPtr()),
				ScriptMetric.TaskWallTime,
				ScriptMetric.DDCFetchTime,
				ScriptMetric.TranslateTime,
				ScriptMetric.CompilerWallTime,
				ScriptMetric.CompilerPreprocessTime,
				ScriptMetric.CompilerWorkerTime,
				ScriptMetric.ByteCodeOptimizeTime);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

FNiagaraQueryCompilationOptions::FNiagaraQueryCompilationOptions()
#if WITH_EDITORONLY_DATA && ALLOW_DEBUG_FILES
	: bGenerateTimingsFile(GNiagaraCompileDumpTimings)
#else
	: bGenerateTimingsFile(false)
#endif
{

}

#undef LOCTEXT_NAMESPACE // NiagaraSystem
