// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCompileHashVisitor.h"
#include "ShaderCompilerCore.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraCompileHashVisitor)

namespace NiagaraCompileHashVisitorInternal
{
	// Recursively collects all shader files reachable from VirtualFilePath by following
	// #include "..." directives.  Compared with the engine's GetShaderIncludes this version:
	//   - Does NOT call ReplaceVirtualFilePathForShaderPlatform  (no platform path substitution)
	//   - Does NOT call ReplaceVirtualFilePathForShaderAutogen    (no autogen mapping)
	//   - Does NOT call GShaderHashCache.ShouldIgnoreInclude      (no platform-based filtering)
	//   - Silently ignores files that cannot be loaded
	// The result is a platform-invariant include closure: it only changes when source files change,
	// not when the target shader platform changes.
	static void CollectIncludesPlatformInvariant(const TCHAR* VirtualFilePath, TArray<FString>& InOutVisited, uint32 DepthLimit)
	{
		if (DepthLimit == 0 || InOutVisited.Contains(FString(VirtualFilePath)))
		{
			return;
		}

		InOutVisited.Add(VirtualFilePath);

		// Load raw file content.  SP_PCD3D_SM5 is passed purely to satisfy the API — for Niagara's
		// own .ush files LoadCachedShaderSourceFile applies no platform-specific path substitutions,
		// so the returned content is identical regardless of which platform token is passed.
		FShaderSharedStringPtr FileContents;
		if (!LoadCachedShaderSourceFile(VirtualFilePath, EShaderPlatform::SP_PCD3D_SM5, &FileContents, nullptr) || !FileContents.IsValid())
		{
			// Silently skip — graceful handling of missing files was explicitly requested.
			return;
		}

		const FString& Source = *FileContents;
		const TCHAR*   Pos    = *Source;

		// Scan for every #include "..." directive in the file.
		// Replicates the engine's FindFirstInclude scanning loop, without platform-specific filtering.
		while (const TCHAR* HashPos = FCString::Strstr(Pos, TEXT("#")))
		{
			const TCHAR* ParseHead = HashPos + 1;

			// Skip whitespace between '#' and the directive keyword
			while (*ParseHead == TEXT(' ') || *ParseHead == TEXT('\t'))
			{
				++ParseHead;
			}

			// Match the "include" keyword (case-insensitive)
			const bool bIsInclude = FCString::Strnicmp(ParseHead, TEXT("include"), 7) == 0;
			if (bIsInclude)
			{
				ParseHead += 7;
			}

			// Require trailing whitespace to confirm this is a valid #include directive
			if (bIsInclude && (*ParseHead == TEXT(' ') || *ParseHead == TEXT('\t')))
			{
				// Advance to the opening double-quote, staying on the same line
				while (*ParseHead && *ParseHead != TEXT('"') && *ParseHead != TEXT('\n'))
				{
					++ParseHead;
				}

				if (*ParseHead == TEXT('"'))
				{
					const TCHAR* QuoteStart = ParseHead + 1;
					const TCHAR* QuoteEnd   = QuoteStart;
					while (*QuoteEnd && *QuoteEnd != TEXT('"') && *QuoteEnd != TEXT('\n'))
					{
						++QuoteEnd;
					}

					if (*QuoteEnd == TEXT('"'))
					{
						FString IncludePath = FString::ConstructFromPtrSize(QuoteStart, (int32)(QuoteEnd - QuoteStart));

						// Resolve relative paths relative to the including file's directory
						if (!IncludePath.StartsWith(TEXT("/")))
						{
							IncludePath = FPaths::GetPath(VirtualFilePath) / IncludePath;
							FPaths::CollapseRelativeDirectories(IncludePath);
						}

						// Skip /Engine/Generated/ paths — they are generated in memory at runtime
						// and do not correspond to a stable source file that can be hashed.
						// Skip /Platform/ paths — they are platform-specific virtual paths that
						// LoadCachedShaderSourceFile cannot resolve without a fully mounted platform,
						// causing spurious error logs when we traverse them here.
						if (!IncludePath.StartsWith(TEXT("/Engine/Generated/")) &&
							!IncludePath.StartsWith(TEXT("/Platform/")))
						{
							CollectIncludesPlatformInvariant(*IncludePath, InOutVisited, DepthLimit - 1);
						}
					}
				}
			}

			// Advance past the '#' we just examined
			Pos = (*HashPos != TEXT('\0')) ? HashPos + 1 : HashPos;
		}
	}

	// Session-scoped cache.  Shader source files are treated as immutable after startup
	// (consistent with the engine's own GShaderHashCache behaviour), so we never need to
	// invalidate within a cook or editor session.
	static FRWLock           GCacheLock;
	static TMap<FString, FSHAHash> GCache;

	static FSHAHash GetShaderFileHashPlatformInvariant(const TCHAR* VirtualFilePath)
	{
		// Fast read path
		{
			FRWScopeLock ReadLock(GCacheLock, SLT_ReadOnly);
			if (const FSHAHash* Cached = GCache.Find(VirtualFilePath))
			{
				return *Cached;
			}
		}

		// Slow write path — compute then cache
		FRWScopeLock WriteLock(GCacheLock, SLT_Write);

		// Re-check under write lock; another thread may have computed this while we waited
		if (const FSHAHash* Cached = GCache.Find(VirtualFilePath))
		{
			return *Cached;
		}

		// Build the platform-invariant include closure starting from VirtualFilePath
		TArray<FString> AllFiles;
		CollectIncludesPlatformInvariant(VirtualFilePath, AllFiles, /*DepthLimit=*/100);

		// Hash each reachable file's content in traversal order; skip any that fail to load
		FSHA1 HashState;
		for (const FString& FilePath : AllFiles)
		{
			FShaderSharedStringPtr FileContents;
			if (LoadCachedShaderSourceFile(*FilePath, EShaderPlatform::SP_PCD3D_SM5, &FileContents, nullptr)
				&& FileContents.IsValid())
			{
				const FString& Source = *FileContents;
				HashState.UpdateWithString(*Source, Source.Len());
			}
		}
		HashState.Final();

		FSHAHash Result;
		HashState.GetHash(Result.Hash);

		GCache.Emplace(VirtualFilePath, Result);
		return Result;
	}
} // namespace NiagaraCompileHashVisitorInternal

bool FNiagaraCompileHashVisitor::UpdateShaderFile(const TCHAR* ShaderFilePath)
{
	const FSHAHash Hash = NiagaraCompileHashVisitorInternal::GetShaderFileHashPlatformInvariant(ShaderFilePath);
	FString SanitizedShaderName(ShaderFilePath);
	SanitizedShaderName.ReplaceCharInline('/', '_');
	SanitizedShaderName.ReplaceCharInline('.', '_');
	return UpdateString(*SanitizedShaderName, Hash.ToString());
}

bool FNiagaraCompileHashVisitor::UpdateString(const TCHAR* InDebugName, FStringView InData)
{
	HashState.Update((const uint8*)InData.GetData(), sizeof(TCHAR) * InData.Len());
#if WITH_EDITORONLY_DATA
	if (LogCompileIdGeneration != 0)
	{
		Values.Top().PropertyKeys.Push(InDebugName);
		Values.Top().PropertyValues.Push(FString(InData));
	}
#endif
	return true;
}
