// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Level.h"
#include "Engine/TextureRenderTarget.h"
#include "NiagaraCommon.h"
#include "NiagaraComponentPool.h"
#include "NiagaraComponentSettings.h"
#include "NiagaraConstants.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraDataInterface.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraParameterCollectionResolver.h"
#include "NiagaraRenderer.h"
#include "NiagaraSceneComponentUtils.h"
#include "NiagaraSceneProxy.h"
#include "NiagaraSimCache.h"
#include "NiagaraStats.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraTrace.h"
#include "NiagaraWorldManager.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveViewRelevance.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "UObject/NameTypes.h"
#include "Engine/StaticMesh.h"
#include "NiagaraCullProxyComponent.h"
#include "SceneInterface.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectThreadContext.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "NiagaraActor.h"
#include "PSOPrecacheSettings.h"

#if WITH_EDITOR
#include "INiagaraEditorOnlyDataUtlities.h"
#include "NiagaraModule.h"
#include "Modules/ModuleManager.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraComponent)

DECLARE_CYCLE_STAT(TEXT("Sceneproxy create (GT)"), STAT_NiagaraCreateSceneProxy, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Component Tick (GT)"), STAT_NiagaraComponentTick, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Activate (GT)"), STAT_NiagaraComponentActivate, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Deactivate (GT)"), STAT_NiagaraComponentDeactivate, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Send Render Data (GT)"), STAT_NiagaraComponentSendRenderData, STATGROUP_Niagara);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Set Dynamic Data (RT)"), STAT_NiagaraSetDynamicData, STATGROUP_Niagara, EStatFlags::Verbose);
DECLARE_CYCLE_STAT(TEXT("Get Dynamic Mesh Elements (RT)"), STAT_NiagaraComponentGetDynamicMeshElements, STATGROUP_Niagara);

DEFINE_LOG_CATEGORY(LogNiagara);

DEFINE_RENDER_COMMAND_PIPE(NiagaraDynamicData, ERenderCommandPipeFlags::None);

namespace NiagaraComponentPrivate
{
	bool bDestroyAllComponentsThatCantRender = false;
	static FAutoConsoleVariableRef CVarRollback_DestroyAllComponentsThatCantRender(
		TEXT("fx.Niagara.Rollback.DestroyAllComponentsThatCantRender"),
		bDestroyAllComponentsThatCantRender,
		TEXT("When true any component that can not render will be destroyed not just ones that are bAutoDestroy."),
		ECVF_Default
	);

	bool bAllowReactivateInPostApplyToComponent = true;
	static FAutoConsoleVariableRef CVarRollback_AllowReactivateInPostApplyToComponent(
		TEXT("fx.Niagara.Rollback.AllowReactivateInPostApplyToComponent"),
		bAllowReactivateInPostApplyToComponent,
		TEXT("When enabled activate & registered components will be reactivated at the end of PostApplyToComponent."),
		ECVF_Default
	);

	static bool GParallelGDMEEnabled = true;
	static FAutoConsoleVariableRef CVarNiagaraParallelGDME(
		TEXT("fx.Niagara.ParallelGDME"),
		GParallelGDMEEnabled,
		TEXT("Allow Niagara to run parallel in GDME."),
		ECVF_Default
	);

	static bool GbComponentsEnabled = true;
	static FAutoConsoleVariableRef CVarNiagaraComponentsEnabled(
		TEXT("fx.NiagaraComponentsEnabled"),
		GbComponentsEnabled,
		TEXT("If false Niagara particle components will not be activated."),
		ECVF_Default
	);

	static int32 GWarnNullAsset = 0;
	static FAutoConsoleVariableRef CVarNiagaraComponentWarnNullAsset(
		TEXT("fx.Niagara.ComponentWarnNullAsset"),
		GWarnNullAsset,
		TEXT("When enabled we will warn if a NiagaraComponent is activate with a null asset.  This is sometimes useful for tracking down components that can be removed."),
		ECVF_Default
	);

	static int32 GWarnAsleepCullReaction = 1;
	static FAutoConsoleVariableRef CVarNiagaraComponentWarnAsleepCullReaction(
		TEXT("fx.Niagara.ComponentWarnAsleepCullReaction"),
		GWarnAsleepCullReaction,
		TEXT("When enabled we will warn if a NiagaraComponent completes naturally but has Asleep mode set for cullreaction."),
		ECVF_Default
	);

	static int GForceWaitForCompilationOnActivate = 0;
	static FAutoConsoleVariableRef CVarNiagaraForceWaitForCompilationOnActivate(
		TEXT("fx.Niagara.ForceWaitForCompilationOnActivate"),
		GForceWaitForCompilationOnActivate,
		TEXT("When a component is activated it will stall waiting for any pending shader compilation."),
		ECVF_Default
	);

	static bool GBoostPSOPrecacheHighest = false;
	static FAutoConsoleVariableRef CVarNiagaraComponentBoostPSOPrecacheHighest(
		TEXT("r.PSOPrecache.NiagaraComponentPSOPrecachePriority"),
		GBoostPSOPrecacheHighest,
		TEXT("Niagara component PSO precache priority level.\n")
		TEXT(" 0. Niagara component's PSO precache requests are set to high priority (default)\n")
		TEXT(" 1. Niagara component's PSO precache requests are set to highest priority"),
		ECVF_Default
	);

	static int GCanPreventScalabilityCullingOnPlayerFX = 1;
	static FAutoConsoleVariableRef CVarNiagaraCanPreventScalabilityCullingOnPlayerFX(
		TEXT("fx.Niagara.Scalability.CanPreventCullingOnPlayerFX"),
		GCanPreventScalabilityCullingOnPlayerFX,
		TEXT("When enabled Niagara can optionally prevent scalability culling on FX linked to the player."),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* CVar)
			{
				FNiagaraWorldManager::OnRefreshOwnerAllowsScalability();
			}),
		ECVF_Default
	);

	FAutoConsoleCommandWithWorldAndArgs DumpNiagaraComponentsCommand(
		TEXT("fx.Niagara.DumpComponents"),
		TEXT("Dump Information about all Niagara Components"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
			[](const TArray<FString>& Args, UWorld* World)
			{
				const bool bFullDump = Args.Contains(TEXT("full"));
				const bool bAllWorlds = (World == nullptr) || Args.Contains(TEXT("allworlds"));
				const FString SystemFilter = 
					[&]() -> FString
					{
						static const FString FilterPrefix(TEXT("filter="));
						for ( const FString& Arg : Args )
						{
							if ( Arg.StartsWith(FilterPrefix) )
							{
								return Arg.Mid(FilterPrefix.Len());
							}
						}
						return FString();
					}();

				UE_LOGF(LogNiagara, Log, "=========================== Begin Niagara Dump ===========================");
				for (TObjectIterator<UNiagaraComponent> It; It; ++It)
				{
					UNiagaraComponent* Component = *It;
					if ( !IsValid(Component) )
					{
						continue;
					}

					// Filter by world
					if (!bAllWorlds && (Component->GetWorld() != World) )
					{
						continue;
					}

					// Filter by asset name, we allow null assets to continue so we can identify components who are attached but have no valid asset associated with them
					UNiagaraSystem* NiagaraSystem = Component->GetAsset();
					if ( !SystemFilter.IsEmpty() )
					{
						if (!NiagaraSystem || !NiagaraSystem->GetName().Contains(SystemFilter) )
						{
							continue;
						}
					}

					if (Component->PoolingMethod == ENCPoolMethod::FreeInPool)
					{
						UE_LOGF(LogNiagara, Log, "Component '%ls' Asset '%ls' Free In Component Pool", *GetNameSafe(Component), *GetNameSafe(NiagaraSystem));
					}
					else
					{
						UE_LOGF(LogNiagara, Log, "Component '%ls' Asset '%ls' Actor '%ls' is %ls", *GetNameSafe(Component), *GetNameSafe(NiagaraSystem), *GetNameSafe(Component->GetTypedOuter<AActor>()), Component->IsActive() ? TEXT("Active") : TEXT("Inactive"));
						if (FNiagaraSystemInstanceControllerPtr SystemInstanceController = Component->GetSystemInstanceController())
						{
							SystemInstanceController->DebugDump(bFullDump);
						}
					}
				}
				UE_LOGF(LogNiagara, Log, "=========================== End Niagara Dump ===========================");
			}
		)
	);

	static void ForEachComponentInWorld(const TArray<FString>& InArgs, UWorld* World, TFunction<void(UNiagaraComponent*)> Callback)
	{
		const FString Command = FString::Join(InArgs, TEXT(" "));

		FString SystemFilter;
		FParse::Value(*Command, TEXT("filter="), SystemFilter);

		for (TObjectIterator<UNiagaraComponent> It; It; ++It)
		{
			UNiagaraComponent* Component = *It;
			if (!IsValid(Component) || Component->GetWorld() != World || !Component->IsActive() || !Component->GetAsset())
			{
				continue;
			}

			UNiagaraSystem* NiagaraSystem = Component->GetAsset();
			if (!SystemFilter.IsEmpty() && !NiagaraSystem->GetName().MatchesWildcard(SystemFilter))
			{
				continue;
			}

			Callback(Component);
		}
	}

	FAutoConsoleCommandWithWorldAndArgs ReinitializeActiveCommand(
		TEXT("fx.Niagara.Component.ReinitializeActive"),
		TEXT("Iterates all activate components and reinitializes them."),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
			[](const TArray<FString>& InArgs, UWorld* World)
			{
				ForEachComponentInWorld(InArgs, World, [](UNiagaraComponent* Component) { Component->ReinitializeSystem(); });
			}
		)
	);

	FAutoConsoleCommandWithWorldAndArgs DeactivateActiveCommand(
		TEXT("fx.Niagara.Component.DeactivateActive"),
		TEXT("Iterates all activate components and deactivates them."),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
			[](const TArray<FString>& InArgs, UWorld* World)
			{
				ForEachComponentInWorld(InArgs, World, [](UNiagaraComponent* Component) { Component->DeactivateImmediate(); });
			}
		)
	);
}

bool FitsIntoFloat(FVector InValue)
{
	// copied from PlayerCameraManager,use HALF_WORLD_MAX for float precision vectors
	return (InValue.X < UE_OLD_HALF_WORLD_MAX  && InValue.X > -UE_OLD_HALF_WORLD_MAX &&
			InValue.Y < UE_OLD_HALF_WORLD_MAX && InValue.Y > -UE_OLD_HALF_WORLD_MAX &&
			InValue.Z < UE_OLD_HALF_WORLD_MAX && InValue.Z > -UE_OLD_HALF_WORLD_MAX);
}

FNiagaraSceneProxyDesc::FNiagaraSceneProxyDesc()
{
	SystemAsset = nullptr;
	OcclusionQueryMode = ENiagaraOcclusionQueryMode::Default;
}

FNiagaraSceneProxyDesc::FNiagaraSceneProxyDesc(UNiagaraComponent& InComponent)
:	FPrimitiveSceneProxyDesc(&InComponent)
{
	SystemAsset = InComponent.GetAsset();
	OcclusionQueryMode = InComponent.GetOcclusionQueryMode();
	SystemInstanceController = InComponent.GetSystemInstanceController();
}

FNiagaraSceneProxyDesc::~FNiagaraSceneProxyDesc() = default;

FNiagaraSceneProxy::FNiagaraSceneProxy(const FNiagaraSceneProxyDesc& Desc)
	: FPrimitiveSceneProxy(Desc, Desc.SystemAsset ? Desc.SystemAsset->GetFName() : NAME_None)
	, OcclusionQueryMode(Desc.OcclusionQueryMode)
{
	SetWireframeColor(FLinearColor(3.0f, 0.0f, 0.0f));

	// Prevent continuous VSM invalidation from the bounds of the primitive.
	bHasDeformableMesh = false;

	// Optionally enable GDME for Niagara, any caveats are covered below
	bSupportsParallelGDME = NiagaraComponentPrivate::GParallelGDMEEnabled;

	// Both GDME and GDRTI call methods such as InitializeVertexBuffersResources(...)
	// which means GDME and GDRTI can't be called in parallel.
	// Disable bSupportsParallelGDRTI so that bSupportsParallelGDME can run in parallel
	bSupportsParallelGDRTI = false;

	FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = Desc.SystemInstanceController;
	UNiagaraSystem* NiagaraSystem = Desc.SystemAsset;
	if (SystemInstanceController && NiagaraSystem)
	{
		LLM_SCOPE(ELLMTag::Niagara);

		RenderData = SystemInstanceController->CreateSystemRenderData(GetScene().GetFeatureLevel());
		ComputeDispatchInterface = SystemInstanceController->GetComputeDispatchInterface();

		bAlwaysHasVelocity = RenderData->HasAnyMotionBlurEnabled();
		bIsHeterogeneousVolume = RenderData->HasAnyHeterogeneousVolumesEnabled();

		SystemStatID = NiagaraSystem->GetStatID(false, false);
#if NIAGARAPROXY_EVENTS_ENABLED
		SystemStatString = NiagaraSystem->GetFName().ToString();
#endif

		// Any GPU simulations that tick in PostInitView can not run parallel in GDME as GDME runs async over the top of this call
		//-OPT: Would it be better to run these simulations in PreRender which is after we join from parallel GDME?
		const ENiagaraGpuComputeTickStage::Type GpuComputeTickStage = SystemInstanceController->GetGpuComputeTickStage();
		if (GpuComputeTickStage == ENiagaraGpuComputeTickStage::PostInitViews )
		{
			bSupportsParallelGDME = false;
		}
	}
}

SIZE_T FNiagaraSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FNiagaraSceneProxy::DestroyRenderState_Concurrent()
{
	if (RenderData)
	{
		RenderData->DestroyRenderState_Concurrent();
	}
}

FNiagaraSceneProxy::~FNiagaraSceneProxy()
{
	if (RenderData)
	{
		delete RenderData;
		RenderData = nullptr;
	}

	ReleaseUniformBuffers(true);
}

void FNiagaraSceneProxy::ReleaseRenderThreadResources()
{
	if (RenderData)
	{
		RenderData->ReleaseRenderThreadResources();
	}

	ReleaseUniformBuffers(true);
}

// FPrimitiveSceneProxy interface.
void FNiagaraSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	if (RenderData)
	{
		LLM_SCOPE(ELLMTag::Niagara);
		RenderData->CreateRenderThreadResources(RHICmdList);
	}
}

void FNiagaraSceneProxy::OnTransformChanged(FRHICommandListBase& RHICmdList)
{
	FPrimitiveSceneProxy::OnTransformChanged(RHICmdList);

	LocalToWorldInverse = GetLocalToWorld().Inverse();

	ReleaseUniformBuffers(false);
}

FPrimitiveViewRelevance FNiagaraSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Relevance;

	if (!RenderData || !RenderData->IsRenderingEnabled_RT() || !FNiagaraUtilities::SupportsNiagaraRendering(View->GetFeatureLevel()))
	{
		return Relevance;
	}

	// Read Relevance from the material / renderer first as FPrimitiveViewRelevance() will turn on certain flags which we want to control (i.e. bRenderInMainPass)
	Relevance							= RenderData->GetViewRelevance(*View, *this);
	Relevance.bDynamicRelevance			= true;
	Relevance.bDrawRelevance			= IsShown(View) && View->Family->EngineShowFlags.Particles && View->Family->EngineShowFlags.Niagara;
	Relevance.bRenderCustomDepth		= ShouldRenderCustomDepth();
	Relevance.bShadowRelevance			= IsShadowCast(View);
	Relevance.bRenderInMainPass			= ShouldRenderInMainPass();
	Relevance.bUsesLightingChannels		= GetLightingChannelMask() != GetDefaultLightingChannelMask();
	Relevance.bTranslucentSelfShadow	= bCastVolumetricTranslucentShadow;
	Relevance.bVelocityRelevance		= DrawsVelocity() && Relevance.bOpaque && Relevance.bRenderInMainPass;

	return Relevance;
}

void FNiagaraSceneProxy::ReleaseUniformBuffers(bool bEmpty)
{
	for (auto& CustomUB : CustomUniformBuffers)
	{
		if (CustomUB.Value != nullptr)
		{
			CustomUB.Value->ReleaseResource();
			delete CustomUB.Value;
			CustomUB.Value = nullptr;
		}
	}

	const int32 ExpectedRendererCount = RenderData ? RenderData->GetNumRenderers() : 0;
	const int32 CurrentUBCount = CustomUniformBuffers.Num();
	CustomUniformBuffers.Empty(bEmpty ? 0 : FMath::Min(ExpectedRendererCount, CurrentUBCount));
}

TUniformBuffer<FPrimitiveUniformShaderParameters>* FNiagaraSceneProxy::GetCustomUniformBufferResource(FRHICommandListBase& RHICmdList, bool bHasVelocity, const FBox& InstanceBounds) const
{
	// Use a hash to determine if we can re-use any uniform buffer
	uint32 KeyHash = HashCombine(bHasVelocity, InstanceBounds.IsValid);

	bool bHasPrecomputedVolumetricLightmap;
	FMatrix PreviousLocalToWorld;
	int32 SingleCaptureIndex;
	bool bOutputVelocity;
	FPrimitiveSceneInfo* LocalPrimitiveSceneInfo = GetPrimitiveSceneInfo();
	GetScene().GetPrimitiveUniformShaderParameters_RenderThread(LocalPrimitiveSceneInfo, bHasPrecomputedVolumetricLightmap, PreviousLocalToWorld, SingleCaptureIndex, bOutputVelocity);
	bOutputVelocity |= AlwaysHasVelocity();

	// we need to incorporate the transform of the PreviousLocalToWorld because it stopping moving is relevant
	// and we need to be able to generate a new UB for that case
	KeyHash = HashCombine(KeyHash, UE::Math::GetTypeHash(PreviousLocalToWorld.GetScaledAxis(EAxis::X)));
	KeyHash = HashCombine(KeyHash, UE::Math::GetTypeHash(PreviousLocalToWorld.GetScaledAxis(EAxis::Y)));
	KeyHash = HashCombine(KeyHash, UE::Math::GetTypeHash(PreviousLocalToWorld.GetScaledAxis(EAxis::Z)));
	KeyHash = HashCombine(KeyHash, UE::Math::GetTypeHash(PreviousLocalToWorld.GetOrigin()));

	if ( InstanceBounds.IsValid )
	{
		KeyHash = HashCombine(KeyHash, UE::Math::GetTypeHash(InstanceBounds.Min));
		KeyHash = HashCombine(KeyHash, UE::Math::GetTypeHash(InstanceBounds.Max));
	}

	const FCustomPrimitiveData* LocalCustomPrimitiveData = GetCustomPrimitiveData();
	if (LocalCustomPrimitiveData && LocalCustomPrimitiveData->Data.Num())
	{
		const uint32 CustomFloatHash = FCrc::MemCrc32(LocalCustomPrimitiveData->Data.GetData(), LocalCustomPrimitiveData->Data.Num() * LocalCustomPrimitiveData->Data.GetTypeSize());
		KeyHash = HashCombine(KeyHash, CustomFloatHash);
	}

	UE::TScopeLock LockGuard(CustomUniformBuffersGuard);
	TUniformBuffer<FPrimitiveUniformShaderParameters>*& CustomUBRef = CustomUniformBuffers.FindOrAdd(KeyHash);
	if (CustomUBRef == nullptr)
	{
#if RHI_RAYTRACING
		const bool bHasRayTracingRepresentation = IsRayTracingEnabled()
			? HasRayTracingRepresentation() && IsVisibleInRayTracing()  
			: HasDistanceFieldRepresentation();
#else
		const bool bHasRayTracingRepresentation = HasDistanceFieldRepresentation();
#endif

		FPrimitiveUniformShaderParametersBuilder UBBuilder = FPrimitiveUniformShaderParametersBuilder()
			.Defaults()
				.LocalToWorld(GetLocalToWorld())
				.PreviousLocalToWorld(PreviousLocalToWorld)
				.ActorWorldPosition(GetActorPosition())
				.WorldBounds(GetBounds())
				.LocalBounds(GetLocalBounds())
				.CustomPrimitiveData(LocalCustomPrimitiveData)
				.LightingChannelMask(GetLightingChannelMask())
				.LightmapDataIndex(LocalPrimitiveSceneInfo ? LocalPrimitiveSceneInfo->GetLightmapDataOffset() : 0)
				.LightmapUVIndex(GetLightMapCoordinateIndex())
				.SingleCaptureIndex(SingleCaptureIndex)
				.InstanceSceneDataOffset(LocalPrimitiveSceneInfo ? LocalPrimitiveSceneInfo->GetInstanceSceneDataOffset() : INDEX_NONE)
				.NumInstanceSceneDataEntries(LocalPrimitiveSceneInfo ? LocalPrimitiveSceneInfo->GetNumInstanceSceneDataEntries() : 0)
				.InstancePayloadDataOffset(LocalPrimitiveSceneInfo ? LocalPrimitiveSceneInfo->GetInstancePayloadDataOffset() : INDEX_NONE)
				.InstancePayloadDataStride(LocalPrimitiveSceneInfo ? LocalPrimitiveSceneInfo->GetInstancePayloadDataStride() : 0)
				.ReceivesDecals(ReceivesDecals())
				.OutputVelocity(bOutputVelocity)
				.CastContactShadow(CastsContactShadow())
				.CastShadow(CastsDynamicShadow())
				.Holdout(Holdout())
				.HasRayTracingRepresentation(bHasRayTracingRepresentation)
				.HasCapsuleRepresentation(HasDynamicIndirectShadowCasterRepresentation())
				.UseVolumetricLightmap(bHasPrecomputedVolumetricLightmap)
				.UseSingleSampleShadowFromStationaryLights(UseSingleSampleShadowFromStationaryLights())
				.HasPixelAnimation(AnyMaterialHasPixelAnimation())
				.IsFirstPerson(IsFirstPerson());
		if ( InstanceBounds.IsValid )
		{
			UBBuilder.InstanceLocalBounds(InstanceBounds);

			// TODO: Is this correct anymore? With instance bounds, seems not. Left in for legacy
			UBBuilder.PreSkinnedLocalBounds(InstanceBounds);
		}

		CustomUBRef = new TUniformBuffer<FPrimitiveUniformShaderParameters>();
		CustomUBRef->SetContents(RHICmdList, UBBuilder.Build());
		CustomUBRef->InitResource(RHICmdList);
	}

	return CustomUBRef;
}

FRHIUniformBuffer* FNiagaraSceneProxy::GetCustomUniformBuffer(FRHICommandListBase& RHICmdList, bool bHasVelocity, const FBox& InstanceBounds) const
{
	// Default UB we create for the primitive
	if (bHasVelocity && !InstanceBounds.IsValid)
	{
		return GetUniformBuffer();
	}

	return GetCustomUniformBufferResource(RHICmdList, bHasVelocity, InstanceBounds)->GetUniformBufferRHI();
}

uint32 FNiagaraSceneProxy::GetMemoryFootprint() const
{ 
	return (sizeof(*this) + GetAllocatedSize()); 
}

uint32 FNiagaraSceneProxy::GetAllocatedSize() const
{ 
	uint32 Size = uint32(FPrimitiveSceneProxy::GetAllocatedSize());
	if (RenderData)
	{
		Size += RenderData->GetDynamicDataSize();
	}
	return Size;
}

void FNiagaraSceneProxy::SetRenderingEnabled_GT(bool bInRenderingEnabled)
{
	if (RenderData)
	{
		RenderData->SetRenderingEnabled_GT(bInRenderingEnabled);
	}
}

bool FNiagaraSceneProxy::CanBeOccluded() const
{
	switch (OcclusionQueryMode)
	{
		case ENiagaraOcclusionQueryMode::AlwaysEnabled:		return true;
		case ENiagaraOcclusionQueryMode::AlwaysDisabled:	return false;

		// TODO account for MaterialRelevance.bDisableDepthTest and MaterialRelevance.bPostMotionBlurTranslucency as well
		//case ENiagaraOcclusionQueryMode::Default:
		default:											return !ShouldRenderCustomDepth();
	}
}

void FNiagaraSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraOverview_RT);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraComponentGetDynamicMeshElements);

	FScopeCycleCounter SystemStatCounter(SystemStatID);

#if NIAGARAPROXY_EVENTS_ENABLED
	bool bEndPlatformEvent = false;
#if CPUPROFILERTRACE_ENABLED
	bool bEndTraceEvent = false;
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
	{
		bEndTraceEvent = true;
		FCpuProfilerTrace::OutputBeginDynamicEvent(GetResourceName());
	}
#endif //CPUPROFILERTRACE_ENABLED
#if ENABLE_STATNAMEDEVENTS
#if CPUPROFILERTRACE_ENABLED
	else
#endif
	{
		bEndPlatformEvent = true;
		FPlatformMisc::BeginNamedEvent(FColor(0), *SystemStatString);
	}
#endif //ENABLE_STATNAMEDEVENTS
	ON_SCOPE_EXIT
	{
#if CPUPROFILERTRACE_ENABLED
		if (bEndTraceEvent)
		{
			FCpuProfilerTrace::OutputEndEvent();
		}
#endif //CPUPROFILERTRACE_ENABLED
#if ENABLE_STATNAMEDEVENTS
		if (bEndPlatformEvent)
		{
			FPlatformMisc::EndNamedEvent();
		}
#endif //ENABLE_STATNAMEDEVENTS
	};
#endif

	if (RenderData)
	{
		// Renderers often rely upon executing GPU code, such as sorting / culling, if the FX system is invalid then the count manager, etc,
		// will not run but the mesh batch building will assume they do.  We will early out here for consistency across all Niagara renderers.
		if (ViewFamily.Scene->GetFXSystem() != nullptr)
		{
			RenderData->GetDynamicMeshElements(Views, ViewFamily, VisibilityMap, Collector, *this);
		}
	}

	if (ViewFamily.EngineShowFlags.Bounds && ViewFamily.EngineShowFlags.Particles && ViewFamily.EngineShowFlags.Niagara)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
				if (HasCustomOcclusionBounds())
				{
					RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetCustomOcclusionBounds(), IsSelected());
				}
			}
		}
	}
}

#if RHI_RAYTRACING
void FNiagaraSceneProxy::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector)
{
if (RenderData)
	{
		RenderData->GetDynamicRayTracingInstances(Collector, *this);
	}
}
#endif

void FNiagaraSceneProxy::GatherSimpleLights(const FSceneViewFamily& ViewFamily, FSimpleLightArray& OutParticleLights) const
{
	if (RenderData)
	{
		RenderData->GatherSimpleLights(OutParticleLights);
	}
}

//////////////////////////////////////////////////////////////////////////

UNiagaraComponent::UNiagaraComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bForceSolo(false)
	, bOverrideWarmupSettings(false)
	, WarmupTickDelta(1.0f / 15.0f)
	, AgeUpdateMode(ENiagaraAgeUpdateMode::TickDeltaTime)
	, DesiredAge(0.0f)
	, LastHandledDesiredAge(0.0f)
	, bCanRenderWhileSeeking(true)
	, SeekDelta(1 / 30.0f)
	, bLockDesiredAgeDeltaTimeToSeekDelta(true)
	, MaxSimTime(33.0f / 1000.0f)
	, bIsSeeking(false)
	, bAutoDestroy(false)
	, MaxTimeBeforeForceUpdateTransform(5.0f)
	, PreviewLODDistance(0.0f)
	, PreviewMaxDistance(1.0f)
	, bEnablePreviewLODDistance(false)
#if WITH_EDITORONLY_DATA
	, bWaitForCompilationOnActivate(false)
#endif
	, bAwaitingActivationDueToNotReady(false)
	, bActivateShouldResetWhenReady(false)
	, bDidAutoAttach(false)
	, bAllowScalability(true)
	, bForceLocalPlayerEffect(false)
	, bIsCulledByScalability(false)
	, bIsFullyComplete(true)
	, bDuringUpdateContextReset(false)
	, bDesiredPauseState(false)
	, bRecachePSOs(false)
	//, bIsChangingAutoAttachment(false)
	, ScalabilityManagerHandle(INDEX_NONE)
	, ForceUpdateTransformTime(0.0f)
	, CurrLocalBounds(ForceInit)
	, SystemFixedBounds(ForceInit)
{
	OverrideParameters.SetOwner(this);

	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.EndTickGroup = TG_LastDemotable;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;
	PrimaryComponentTick.SetTickFunctionEnable(false);
	bTickInEditor = true;
	bAutoActivate = true;
	bRenderingEnabled = true;
	SavedAutoAttachRelativeScale3D = FVector(1.f, 1.f, 1.f);

	SetGenerateOverlapEvents(false);
	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	// NIAGARA_WIP - BEGIN: Needs to be disable via version to avoid breaking existing content
	// Disable receiving decals by default.
	bReceivesDecals = false;
	// NIAGARA_WIP - END
}

/********* UFXSystemComponent *********/
void UNiagaraComponent::SetBoolParameter(FName Parametername, bool Param)
{
	SetVariableBool(Parametername, Param);
}

void UNiagaraComponent::SetIntParameter(FName ParameterName, int Param)
{
	SetVariableInt(ParameterName, Param);
}

void UNiagaraComponent::SetFloatParameter(FName ParameterName, float Param)
{
	SetVariableFloat(ParameterName, Param);
}

void UNiagaraComponent::SetVectorParameter(FName ParameterName, FVector Param)
{
	SetVariableVec3(ParameterName, Param);
}

void UNiagaraComponent::SetColorParameter(FName ParameterName, FLinearColor Param)
{
	SetVariableLinearColor(ParameterName, Param);
}

void UNiagaraComponent::SetActorParameter(FName ParameterName, class AActor* Param)
{
	SetVariableActor(ParameterName, Param);
}

UFXSystemAsset* UNiagaraComponent::GetFXSystemAsset() const
{
	return Asset;
}

void UNiagaraComponent::InitForPerformanceBaseline()
{
	bNeverDistanceCull = true;
	SetAllowScalability(false);
	SetPreviewLODDistance(true, 1.0f, 10000.0f);
}

void UNiagaraComponent::SetLODDistance(float InLODDistance, float InMaxLODDistance)
{
	if (!bEnablePreviewLODDistance && SystemInstanceController)
	{
		SystemInstanceController->SetLODDistance(InLODDistance, InMaxLODDistance, false);
	}
}

void UNiagaraComponent::SetEmitterEnable(FName EmitterName, bool bNewEnableState)
{
	FEmitterOverrideInfo& EmitterOverrideInfo = FindOrAddEmitterOverrideInfo(EmitterName);
	EmitterOverrideInfo.bEnabled = bNewEnableState;

	if (SystemInstanceController.IsValid() && !SystemInstanceController->IsComplete())
	{
		SystemInstanceController->SetEmitterEnable(EmitterName, bNewEnableState);
	}
}

void UNiagaraComponent::SetSystemFixedBounds(FBox LocalBounds)
{
	SystemFixedBounds = LocalBounds;
	if ( SystemInstanceController.IsValid() )
	{
		SystemInstanceController->SetSystemFixedBounds(LocalBounds);
	}
}

FBox UNiagaraComponent::GetSystemFixedBounds() const
{
	return SystemFixedBounds;
}

void UNiagaraComponent::ClearSystemFixedBounds()
{
	SystemFixedBounds.Init();
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetSystemFixedBounds(FBox(EForceInit::ForceInit));
	}
}

void UNiagaraComponent::SetEmitterFixedBounds(FName EmitterName, FBox LocalBounds)
{
	FEmitterOverrideInfo& EmitterOverrideInfo = FindOrAddEmitterOverrideInfo(EmitterName);
	EmitterOverrideInfo.FixedBounds = LocalBounds;

	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetEmitterFixedBounds(EmitterName, LocalBounds);
	}
}

FBox UNiagaraComponent::GetEmitterFixedBounds(FName EmitterName) const
{
	if ( const FEmitterOverrideInfo* EmitterOverrideInfo = FindEmitterOverrideInfo(EmitterName) )
	{
		return EmitterOverrideInfo->FixedBounds;
	}
	return FBox(EForceInit::ForceInit);
}

void UNiagaraComponent::ClearEmitterFixedBounds(FName EmitterName)
{
	if (FEmitterOverrideInfo* EmitterOverrideInfo = FindEmitterOverrideInfo(EmitterName))
	{
		EmitterOverrideInfo->FixedBounds = FBox(EForceInit::ForceInit);
	}

	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetEmitterFixedBounds(EmitterName, FBox(EForceInit::ForceInit));
	}
}

void UNiagaraComponent::ReleaseToPool()
{
	// A component may be marked pending kill before the owner has it's reference set to null.
	// In that case there's a window where it can be released back into the pool incorrectly, so we just skip releasing as we know it will be deleted shortly
	if ( !IsValidChecked(this) || IsUnreachable() )
	{
		return;
	}

	if (PoolingMethod != ENCPoolMethod::ManualRelease)
	{		
		// Only emit this warning if pooling is enabled. If it's not, all components will have PoolingMethod none.
		if (UNiagaraComponentPool::Enabled() && FNiagaraUtilities::LogVerboseWarnings())
		{	
			UE_LOGF(LogNiagara, Warning, "ReleaseToPool called but PoolingMethod is (%ls) when it should be ENCPoolMethod::ManualRelease. Asset=%ls Component=%ls", *StaticEnum<ENCPoolMethod>()->GetNameStringByValue(int64(PoolingMethod)), *GetPathNameSafe(Asset), *GetPathName());
		}
		return;
	}

	if (!IsActive())
	{
		UnregisterWithScalabilityManager();

		//If we're already complete then release to the pool straight away.
		UWorld* World = GetWorld();
		check(World);
		if( FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World))
		{
			WorldMan->GetComponentPool()->ReclaimWorldParticleSystem(this);
		}
		else
		{
			DestroyComponent();
		}
	}
	else
	{
		//If we haven't completed, deactivate and defer release to pool.
		PoolingMethod = ENCPoolMethod::ManualRelease_OnComplete;
		Deactivate();
	}
}

uint32 UNiagaraComponent::GetApproxMemoryUsage() const
{
	uint32 MemoryBytes = sizeof(UNiagaraComponent);
	MemoryBytes += static_cast<uint32>(OverrideParameters.GetResourceSize());
	if (SystemInstanceController)
	{
		MemoryBytes += static_cast<uint32>(SystemInstanceController->GetTotalBytesUsed());
	}
	return MemoryBytes;
}

void UNiagaraComponent::ActivateSystem(bool bFlagAsJustAttached)
{
	// Attachment is handled different in niagara so the bFlagAsJustAttached is ignored here.
	if (IsActive())
	{
		// If the system is already active then activate with reset to reset the system simulation but
		// leave the emitter simulations active.
		bool bResetSystem = true;
		bool bIsFromScalability = false;
		ActivateInternal(bResetSystem, bIsFromScalability);
	}
	else
	{
		// Otherwise just follow the standard activate path.
		Activate();
	}
}

/********* UFXSystemComponent *********/

FName UNiagaraComponent::GetFNameForStatID() const
{
	return Asset ? Asset->GetFNameForStatID() : Super::GetFNameForStatID();
}

bool UNiagaraComponent::RequiresGameThreadEndOfFrameRecreate() const
{
#if WITH_EDITORONLY_DATA
	// Niagara is capable of running async but in the editor.  However various UObjects we might use during a create are not.
	// For example, resolving a UStaticMesh will query the mesh compiler which will propagate changes and mark render state dirty which is not allowed during EOF updates.
	return true;
#else
	return false;
#endif
}

void UNiagaraComponent::TickComponent(float DeltaSeconds, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	LLM_SCOPE(ELLMTag::Niagara);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraComponentTick);

	if (!::IsValid(Asset))
	{
		//-TODO: If our Asset is null or has become Garbage should we force deactive the component / ticking?
		return;
	}

	FScopeCycleCounter SystemStatCounter(Asset->GetStatID(true, false));

	if (bAwaitingActivationDueToNotReady)
	{
		UpdateInstanceSoloMode();
		Activate(bActivateShouldResetWhenReady);
		return;
	}

	if (!SystemInstanceController)
	{
		return;
	}

	// The component tick function should only be execute when in solo mode
	// When in batched mode the world manager will ensure the system ticks
	// The replay net driver can sometimes enable our ticks via SetComponentTickEnabled so disable here
	// Ideally we would intercept SetComponentTickEnabled and have a concept of if we are managing the tick or not
	if (SystemInstanceController->IsSolo() == false)
	{
		SetComponentTickEnabled(false);
		return;
	}

	if (Asset->HasFixedTickDelta() && Asset->GetFixedTickDeltaTime() <= 0)
	{
		return;
	}

	if (!IsActive() && bAutoActivate && SystemInstanceController->GetAreDataInterfacesInitialized())
	{
		Activate();
	}

	if (IsActive() && SystemInstanceController.Get() && !SystemInstanceController->IsComplete())
	{
		check(Asset != nullptr);
		Asset->AddToInstanceCountStat(1, true);
		INC_DWORD_STAT_BY(STAT_TotalNiagaraSystemInstances, 1);
		INC_DWORD_STAT_BY(STAT_TotalNiagaraSystemInstancesSolo, 1);

		// Support for pausing / playback rate from the debugger
		if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(GetWorld()))
		{
			if (WorldManager->GetDebugPlaybackMode() == ENiagaraDebugPlaybackMode::Paused)
			{
				return;
			}

			DeltaSeconds *= WorldManager->GetDebugPlaybackRate();
		}

		// If we have a sim cache attached then use that
		if ( SimCache != nullptr )
		{
			float Age = SystemInstanceController->GetAge();

			if ((AgeUpdateMode == ENiagaraAgeUpdateMode::DesiredAge) || (AgeUpdateMode == ENiagaraAgeUpdateMode::DesiredAgeNoSeek))
			{
				if ( DesiredAge < Age )
				{
					SystemInstanceController->Reset(FNiagaraSystemInstance::EResetMode::ResetAll);
				}
				Age = DesiredAge;
			}
			else
			{
				Age += DeltaSeconds;
			}

			SystemInstanceController->SimCacheTick_GameThread(SimCache, Age, DeltaSeconds, (ThisTickFunction && ThisTickFunction->IsCompletionHandleValid()) ? ThisTickFunction->GetCompletionHandle() : nullptr);
		}
		// Otherwise we need to manually tick
		else if (AgeUpdateMode == ENiagaraAgeUpdateMode::TickDeltaTime)
		{
			SystemInstanceController->ManualTick(DeltaSeconds * CustomTimeDilation, (ThisTickFunction && ThisTickFunction->IsCompletionHandleValid()) ? ThisTickFunction->GetCompletionHandle() : nullptr);
		}
		else if(AgeUpdateMode == ENiagaraAgeUpdateMode::DesiredAge)
		{
			float AgeDiff = FMath::Max(DesiredAge, 0.0f) - SystemInstanceController->GetAge();
			bIsSeeking = false;

			if (FMath::Abs(AgeDiff) >= UE_KINDA_SMALL_NUMBER)
			{
				if (AgeDiff < 0.0f)
				{
					SystemInstanceController->Reset(FNiagaraSystemInstance::EResetMode::ResetAll);
					AgeDiff = DesiredAge - SystemInstanceController->GetAge();
				}

				if (SystemInstanceController->GetSoloSystemSimulation().IsValid())
				{
					FGraphEventRef CompletionEvent = (ThisTickFunction && ThisTickFunction->IsCompletionHandleValid()) ? ThisTickFunction->GetCompletionHandle() : nullptr;

					const bool bFixedTickDelta = Asset->HasFixedTickDelta();
					const float TickDelta = bFixedTickDelta ? Asset->GetFixedTickDeltaTime() : SeekDelta;
					if (bLockDesiredAgeDeltaTimeToSeekDelta || AgeDiff > TickDelta)
					{
						// If we're locking the delta time to the seek delta, or we need to seek more than a frame, tick the simulation by the seek delta.
						const int32 TicksToProcess = FMath::FloorToInt(AgeDiff / TickDelta);
						CompletionEvent = TicksToProcess == 1 ? CompletionEvent : nullptr;

						bIsSeeking = SystemInstanceController->MultiTick(TicksToProcess, TickDelta, MaxSimTime, CompletionEvent);
					}
					else
					{
						// Otherwise just tick by the age difference.
						if (!bFixedTickDelta || AgeDiff >= TickDelta)
						{
							SystemInstanceController->ManualTick(AgeDiff, CompletionEvent);
						}
					}
				}
			}
		}
		else if (AgeUpdateMode == ENiagaraAgeUpdateMode::DesiredAgeNoSeek)
		{
			FGraphEventRef CompletionEvent = (ThisTickFunction && ThisTickFunction->IsCompletionHandleValid()) ? ThisTickFunction->GetCompletionHandle() : nullptr;

			int32 MaxForwardFrames = 5; // HACK - for some reason sequencer jumps forwards by multiple frames on pause, so this is being added to allow for FX to stay alive when being controlled by sequencer in the editor.  This should be lowered once that issue is fixed.
			float AgeDiff = DesiredAge - LastHandledDesiredAge;
			float TickDelta = Asset->HasFixedTickDelta() ? Asset->GetFixedTickDeltaTime() : SeekDelta;
			if (AgeDiff < 0)
			{
				if (FMath::Abs(AgeDiff) >= TickDelta)
				{
					// When going back in time for a frame or more, reset and simulate a single frame.  We ignore small negative changes to delta
					// time which can happen when controlling time with the timeline and the time snaps to a previous time when paused.
					SystemInstanceController->Reset(FNiagaraSystemInstance::EResetMode::ResetAll);
					SystemInstanceController->ManualTick(TickDelta, CompletionEvent);
				}
			}
			else if (AgeDiff < MaxForwardFrames * TickDelta)
			{
				// Allow ticks between 0 and MaxForwardFrames, but don't ever send more then 2 x the seek delta.
				SystemInstanceController->ManualTick(FMath::Min(AgeDiff, 2 * TickDelta), CompletionEvent);
			}
			else
			{
				// When going forward in time for more than MaxForwardFrames, reset and simulate a single frame.
				SystemInstanceController->Reset(FNiagaraSystemInstance::EResetMode::ResetAll);
				SystemInstanceController->ManualTick(TickDelta, CompletionEvent);
			}
			LastHandledDesiredAge = DesiredAge;
		}

		if (SceneProxy != nullptr)
		{
			FNiagaraSceneProxy* NiagaraProxy = static_cast<FNiagaraSceneProxy*>(SceneProxy);
			NiagaraProxy->SetRenderingEnabled_GT(bRenderingEnabled && (bCanRenderWhileSeeking || bIsSeeking == false));
		}
	}
}

const UObject* UNiagaraComponent::AdditionalStatObject() const
{
	return Asset;
}

void UNiagaraComponent::ResetSystem()
{
	Activate(true);
}

void UNiagaraComponent::ReinitializeSystem()
{
	DestroyInstanceNotComponent();
	Activate(true);
}

bool UNiagaraComponent::GetRenderingEnabled() const
{
	return bRenderingEnabled;
}

void UNiagaraComponent::SetRenderingEnabled(bool bInRenderingEnabled)
{
	bRenderingEnabled = bInRenderingEnabled;
}

void UNiagaraComponent::AdvanceSimulation(int32 TickCount, float TickDeltaSeconds)
{
	if (SystemInstanceController.IsValid() && TickDeltaSeconds > SMALL_NUMBER)
	{
		SystemInstanceController->AdvanceSimulation(TickCount, TickDeltaSeconds);
	}
}

void UNiagaraComponent::AdvanceSimulationByTime(float SimulateTime, float TickDeltaSeconds)
{
	if (SystemInstanceController.IsValid() && TickDeltaSeconds > SMALL_NUMBER)
	{
		const int32 TickCount = FMath::FloorToInt(SimulateTime / TickDeltaSeconds);
		SystemInstanceController->AdvanceSimulation(TickCount, TickDeltaSeconds);
	}
}

void UNiagaraComponent::SetPaused(bool bInPaused)
{
	bDesiredPauseState = bInPaused;

	bool bIsActuallyActive = SystemInstanceController.IsValid() &&
		SystemInstanceController->GetRequestedExecutionState() == ENiagaraExecutionState::Active &&
		SystemInstanceController->GetActualExecutionState() == ENiagaraExecutionState::Active;

	if(!bIsActuallyActive)
	{
		//Don't allow pausing when the system is not active.
		//Activate/Deactivate should clear the pause state.
		return;
	}

	if(bIsActuallyActive && !bInPaused)
	{
		//If we're active and we unpause, try to re-register with the scalability manager
		RegisterWithScalabilityManager();
	}
	else 
	{
		//If we're being paused externally, unregister from scalability.
		UnregisterWithScalabilityManager();
	}
	
	SetPausedInternal(bDesiredPauseState, false);
}

bool UNiagaraComponent::IsPaused()const
{
	//check the system instance is actually in the right state. 
	//In cases where we don't have a system instance or we're culled by scalability, the instance can have a differnt internal state to the user's desired state.
	bool bIsInstanceValid = SystemInstanceController.IsValid();
	bool bIsInstanceRunning = bIsInstanceValid && SystemInstanceController->GetRequestedExecutionState() == ENiagaraExecutionState::Active && SystemInstanceController->GetActualExecutionState() == ENiagaraExecutionState::Active;
	ensure(bIsInstanceRunning == false || bIsCulledByScalability || SystemInstanceController->IsPaused() == bDesiredPauseState);
	return bDesiredPauseState;
}

void UNiagaraComponent::SetPausedInternal(bool bInPaused, bool bIsScalabilityCull)
{
	if(bIsScalabilityCull)
	{
		bIsCulledByScalability = bInPaused;
	}
	
	if (SystemInstanceController.IsValid())
	{
		bool bShouldPauseFromScalability = false;
		check(GetAsset());
		if(UNiagaraEffectType* FXType = GetAsset()->GetEffectType())
		{
			bShouldPauseFromScalability = FXType->CullReaction == ENiagaraCullReaction::PauseResume && bIsCulledByScalability;
		}
		SystemInstanceController->SetPaused(bShouldPauseFromScalability || bDesiredPauseState);
	}
}

UNiagaraDataInterface* UNiagaraComponent::GetDataInterface(const FString& Name)
{
	return UNiagaraFunctionLibrary::GetDataInterface(UNiagaraDataInterface::StaticClass(), this, *Name);
}

void UNiagaraComponent::SetSystemSignificanceIndex(int32 InIndex)
{
	if(SystemInstanceController) SystemInstanceController->SetSystemSignificanceIndex(InIndex);
}

bool UNiagaraComponent::IsWorldReadyToRun() const
{
	// The Niagara system instance assumes that a ComputeDispatchInterface exists when it is created. We need to wait until this has happened before successfully activating this system.
	UWorld* World = GetWorld();
	if ( World == nullptr)
	{
		return false;
	}

	if ( FNiagaraGpuComputeDispatchInterface::Get(World) == nullptr )
	{
		return false;
	}

	if (FNiagaraWorldManager::Get(World) == nullptr)
	{
		return false;
	}
	return true;
}

bool UNiagaraComponent::InitializeSystem()
{
	if (SystemInstanceController.IsValid() == false)
	{
		LLM_SCOPE(ELLMTag::Niagara);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
		
		UWorld* World = GetWorld();
		check(World);
		check(Asset);

		const bool bPooled = PoolingMethod != ENCPoolMethod::None;
		OverrideParameters.MarkParametersDirty(); // new system instance means new lwc tile, so any position user params need to be re-evaluated

		SystemInstanceController = MakeShared<FNiagaraSystemInstanceController, ESPMode::ThreadSafe>();		
		SystemInstanceController->Initialize(*World, *Asset, &OverrideParameters, this, TickBehavior, bPooled, RandomSeedOffset, RequiresSoloMode(), bOverrideWarmupSettings ? WarmupTickCount : -1, WarmupTickDelta);
		SystemInstanceController->SetOnPostTick(FNiagaraSystemInstance::FOnPostTick::CreateUObject(this, &UNiagaraComponent::PostSystemTick_GameThread));
		SystemInstanceController->SetOnComplete(FNiagaraSystemInstance::FOnComplete::CreateUObject(this, &UNiagaraComponent::OnSystemComplete));

		// NOTE: For now, we're guaranteed this will be called on the game thread when setting material variables.
		SystemInstanceController->SetOnMaterialsUpdated(FNiagaraSystemInstanceController::FOnMaterialsUpdated::CreateUObject(this, &UNiagaraComponent::MarkRenderStateDirty));

		if (bEnableGpuComputeDebug)
		{
			SystemInstanceController->SetGpuComputeDebug(bEnableGpuComputeDebug != 0);
		}
		
		if (bEnablePreviewLODDistance)
		{
			SystemInstanceController->SetLODDistance(PreviewLODDistance, PreviewMaxDistance, true);
		}

		if (SystemFixedBounds.IsValid)
		{
			SystemInstanceController->SetSystemFixedBounds(SystemFixedBounds);
		}
		for ( const FEmitterOverrideInfo& EmitterOverrideInfo : EmitterOverrideInfos)
		{
			if (!EmitterOverrideInfo.bEnabled)
			{
				SystemInstanceController->SetEmitterEnable(EmitterOverrideInfo.EmitterName, EmitterOverrideInfo.bEnabled);
			}
			if (EmitterOverrideInfo.FixedBounds.IsValid)
			{
				SystemInstanceController->SetEmitterFixedBounds(EmitterOverrideInfo.EmitterName, EmitterOverrideInfo.FixedBounds);
			}
		}

		const FNiagaraReadinessSettings& ReadinessSettings = GetReadinessSettings();
		if (ReadinessSettings.PSOPrecacheSettings.PrecachingTime == ENiagaraPSOPrecachingTime::OnInitialize || ReadinessSettings.PSOPrecacheSettings.PrecachingTime == ENiagaraPSOPrecachingTime::OnActivated)
		{
			// this calls into CollectPSOPrecacheData() and needs a valid system instance controller to correctly precache the PSOs
			PrecachePSOs();
		}

#if WITH_EDITORONLY_DATA
		OnSystemInstanceChangedDelegate.Broadcast();
#endif

		MarkRenderStateDirty();

		if (GIsAutomationTesting && GetAsset() != nullptr)
		{
			UE_LOGF(LogNiagara, Log, "UNiagaraComponent> System %ls initialized.", *GetAsset()->GetPathName());
		}
		return true;
	}
	return false;
}

void UNiagaraComponent::Activate(bool bReset /* = false */)
{
	ActivateInternal(bReset, false);
}

void UNiagaraComponent::ActivateInternal(bool bReset /* = false */, bool bIsScalabilityCull)
{
	// Handle component ticking correct
	struct FScopedComponentTickEnabled
	{
		FScopedComponentTickEnabled(UNiagaraComponent* InComponent) : Component(InComponent) {}
		~FScopedComponentTickEnabled() { Component->PrimaryComponentTick.bStartWithTickEnabled &= bTickEnabled; Component->SetComponentTickEnabled(bTickEnabled); }
		UNiagaraComponent* Component = nullptr;
		bool bTickEnabled = false;
	};
	FScopedComponentTickEnabled ScopedComponentTickEnabled(this);

	ON_SCOPE_EXIT { FNiagaraInsights::NotifyComponentActivate(this, bReset, bIsScalabilityCull, bAwaitingActivationDueToNotReady); };

	bAwaitingActivationDueToNotReady = false;

	// Reset our local bounds on reset
	if (bReset)
	{
		CurrLocalBounds.Init();
	}

	if (!NiagaraComponentPrivate::GbComponentsEnabled)
	{
		UnregisterWithScalabilityManager();
		OnSystemComplete(true);
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_NiagaraComponentActivate);
	PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetWorld(), GetAsset(), this), Activation);

	if (Asset == nullptr)
	{
		DestroyInstance();
		if (NiagaraComponentPrivate::GWarnNullAsset && !HasAnyFlags(RF_DefaultSubObject | RF_ArchetypeObject | RF_ClassDefaultObject))
		{
			UE_LOGF(LogNiagara, Warning, "Failed to activate Niagara Component due to missing or invalid asset! (%ls)", *GetFullName());
		}
		return;
	}

	UWorld* World = GetWorld();

	// If the particle system can't ever render (ie on dedicated server or in a commandlet) than do not activate...
	if (!FApp::CanEverRender() || !World || World->IsNetMode(NM_DedicatedServer) || World->bIsTearingDown)
	{
		// For bAutoDestroy components kill them immediately otherwise they will never complete
		// For none bAutoDestroy we can destroy them only if they are registered, killing other types can result in killing a component that may activate / be used later on
		if (bAutoDestroy || (NiagaraComponentPrivate::bDestroyAllComponentsThatCantRender && IsRegistered()))
		{
			DestroyComponent();
		}
		return;
	}

	if (!IsRegistered())
	{
		return;
	}

	// Is the system allowed to run or not?
	if (FNiagaraComponentSettings::IsSystemAllowedToRun(Asset) == false)
	{
		return;
	}


	// On the off chance that the user changed the asset, we need to clear out the existing data.
	if (SystemInstanceController.IsValid() && SystemInstanceController->GetSystem() != Asset)
	{
		UnregisterWithScalabilityManager();
		OnSystemComplete(true);
	}

#if WITH_EDITOR
	// In case we're not yet ready to run due to compilation requests, go ahead and keep polling there..
	if (Asset->HasOutstandingCompilationRequests(true))
	{
		if (bWaitForCompilationOnActivate || NiagaraComponentPrivate::GForceWaitForCompilationOnActivate || GIsAutomationTesting)
		{
			Asset->WaitForCompilationComplete(true);
		}
		Asset->PollForCompilationComplete();
	}
#endif
	
	bool bIsReady = Asset->IsReadyToRun() && IsWorldReadyToRun();
	bIsReady &= CheckPSOCacheIsReadyToRun();

	if (!bIsReady)
	{
		bAwaitingActivationDueToNotReady = true;
		bActivateShouldResetWhenReady = bReset;
		ScopedComponentTickEnabled.bTickEnabled = true;
		UpdateInstanceSoloMode();
		return;
	}

	if (bReset)
	{
		UnregisterWithScalabilityManager();
	}

	if (!bIsScalabilityCull && bIsCulledByScalability && ScalabilityManagerHandle != INDEX_NONE)
	{
		//If this is a non scalability activate call and we're still registered with the manager.
		//If we reach this point then we must have been previously culled by scalability so bail here.
		return;
	}

	if (!Asset->IsAllowedByScalability())
	{
		OnSystemComplete(true);
		return;
	}

	bIsFullyComplete = false;
	bIsCulledByScalability = false;
	
	//Ensure we're not paused. Non-scalability activate calls should override pause state.
	if(bIsScalabilityCull == false)
	{
		bDesiredPauseState = false;
		SetPausedInternal(false, false);
	}

	//Update our owner scalability state but don't register with the manager.
	ResolveOwnerAllowsScalability(false);
	
	//We reset last render time to the current time so that any visibility culling on a delay will function correctly.
	//Leaving as the default of -1000 causes the visibility code to always assume this should be culled until it's first rendered and initialized by the RT.
	//Set the component last render time *before* preculling otherwise there are cases where we can pre cull incorrectly.
	SetLastRenderTime(GetWorld()->GetTimeSeconds());

	// Auto attach if requested
	const bool bWasAutoAttached = bDidAutoAttach;
	bDidAutoAttach = false;
	if (bAutoManageAttachment)
	{
		USceneComponent* NewParent = AutoAttachParent.Get();
		if (NewParent)
		{
			const bool bAlreadyAttached = GetAttachParent() && (GetAttachParent() == NewParent) && (GetAttachSocketName() == AutoAttachSocketName) && GetAttachParent()->GetAttachChildren().Contains(this);
			if (!bAlreadyAttached)
			{
				bDidAutoAttach = bWasAutoAttached;
				CancelAutoAttachment(true);
				SavedAutoAttachRelativeLocation = GetRelativeLocation();
				SavedAutoAttachRelativeRotation = GetRelativeRotation();
				SavedAutoAttachRelativeScale3D = GetRelativeScale3D();
				//bIsChangingAutoAttachment = true;
				AttachToComponent(NewParent, FAttachmentTransformRules(AutoAttachLocationRule, AutoAttachRotationRule, AutoAttachScaleRule, bAutoAttachWeldSimulatedBodies), AutoAttachSocketName);
				//bIsChangingAutoAttachment = false;
			}

			bDidAutoAttach = true;
			//bFlagAsJustAttached = true;
		}
		else
		{
			CancelAutoAttachment(true);
		}
	}

	if (ShouldPreCull())
	{
		#if WITH_PARTICLE_PERF_CSV_STATS

		if (FCsvProfiler* CSVProfiler = FCsvProfiler::Get())
		{
			if (CSVProfiler->IsCapturing() && FParticlePerfStats::GetCSVStatsEnabled())
			{
				static FName PreCullStatName(TEXT("NiagaraCulled/PreCull"));
				CSVProfiler->RecordCustomStat(PreCullStatName, CSV_CATEGORY_INDEX(Particles), 1, ECsvCustomStatOp::Accumulate);
			}
		}
		#endif

		//We have decided to pre cull the system.
		OnSystemComplete(true);
		return;
	}


	// Early out if we're not forcing a reset, and both the component and system instance are already active.
	if (bReset == false &&
		IsActive() &&
		SystemInstanceController != nullptr &&
		SystemInstanceController->GetRequestedExecutionState() == ENiagaraExecutionState::Active &&
		SystemInstanceController->GetActualExecutionState() == ENiagaraExecutionState::Active)
	{
		// It would be incorrect for the component tick to be enabled and not be in solo mode but we will verify this
		ensure(SystemInstanceController->IsSolo() == IsComponentTickEnabled());
		ScopedComponentTickEnabled.bTickEnabled = IsComponentTickEnabled();
		return;
	}
	
	//Unpause on activate if needed.
	checkf(!(bIsCulledByScalability && IsPaused()), TEXT("We should never be paused when being re-activated by scalability. We unregister from the manager when paused so we should never get here."));
	SetPausedInternal(false, false);

	DestroyCullProxy();

	// We can't call 'Super::Activate(bReset);' as this will enable the component tick
	if (bReset || ShouldActivate() == true)
	{
		SetActiveFlag(true);
		OnComponentActivated.Broadcast(this, bReset);
	}
	
#if WITH_EDITOR
	//TODO: Do this else where. I get needing to ensure params are correct from the component but these are stomping over runtime changes to the params in editor builds.
	//For now we can bypass the worst of the impact by disallowing in game worlds.
	if(!World->IsGameWorld())
	{
		ApplyOverridesToParameterStore();
	}
#endif

	FNiagaraSystemInstance::EResetMode ResetMode = FNiagaraSystemInstance::EResetMode::ResetSystem;
	if (InitializeSystem())
	{
		ResetMode = FNiagaraSystemInstance::EResetMode::None;//Already done a reinit set
		
		// creating the system instance controller can change the PSOs we need to render
		if (!CheckPSOCacheIsReadyToRun())
		{
			bAwaitingActivationDueToNotReady = true;
			bActivateShouldResetWhenReady = bReset;
			ScopedComponentTickEnabled.bTickEnabled = true;
			UpdateInstanceSoloMode();
			return;
		}
	}

	if (!SystemInstanceController)
	{
		OnSystemComplete(true);
		return;
	}

	//We reset last render time to the current time so that any visibility culling on a delay will function correctly.
	//Leaving as the default of -1000 causes the visibility code to always assume this should be culled until it's first rendered and initialized by the RT.
	SystemInstanceController->SetLastRenderTime(GetLastRenderTime());

	RegisterWithScalabilityManager();

	// If we have no render state create one
	if (!IsRenderStateCreated())
	{
		CreateRenderState_Concurrent(nullptr);
	}

	// NOTE: This call can cause SystemInstance itself to get destroyed with auto destroy systems
	SystemInstanceController->Activate(ResetMode);

	if (SystemInstanceController.IsValid())
	{
		if (SystemInstanceController->IsComplete())
		{
			OnSystemComplete(true);
		}
		else if (SystemInstanceController->IsSolo())
		{
			const ETickingGroup SoloTickGroup = SystemInstanceController->CalculateTickGroup();
			PrimaryComponentTick.TickGroup = FMath::Max(TG_PrePhysics, SoloTickGroup);
			PrimaryComponentTick.EndTickGroup = TG_LastDemotable;

			// Solo instances require the component tick to be enabled
			ScopedComponentTickEnabled.bTickEnabled = true;
		}
	}

	if(bIsScalabilityCull)
	{
		//If this is an activate call from a scalability cull we may need to begin in a paused state if someone paused us while we were culled from scalability.
		SetPausedInternal(bDesiredPauseState, false);
	}
}

void UNiagaraComponent::Deactivate()
{
	DeactivateInternal(false);
}

void UNiagaraComponent::DeactivateInternal(bool bIsScalabilityCull /* = false */)
{
	FNiagaraInsights::NotifyComponentDeactivate(this, false, bIsScalabilityCull);

	bool bWasCulledByScalabiltiy = bIsCulledByScalability;
	bAwaitingActivationDueToNotReady = false;

	if (bIsScalabilityCull)
	{
		bIsCulledByScalability = true;

		if (!Asset->NeedsSortedSignificanceCull())
		{
			//FX with significance handling have their cull proxies managed by the significance manager.
			CreateCullProxy();
		}
	}
	else
	{
		// Unregister with the scalability manager if this is a genuine deactivation from outside.
		// The scalability manager itself can call this function when culling systems.
		UnregisterWithScalabilityManager();
	
		//Ensure we're not paused.
		bDesiredPauseState = false;
		SetPausedInternal(false, false);
	}

	if (IsActive() && SystemInstanceController)
	{
		SCOPE_CYCLE_COUNTER(STAT_NiagaraComponentDeactivate);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);

		//UE_LOGF(LogNiagara, Log, "Deactivate: %p - %ls - %ls", this, *Asset->GetName(), bIsScalabilityCull ? TEXT("Scalability") : TEXT(""));

		// Don't deactivate in solo mode as we are not ticked by the world but rather the component
		// Deactivating will cause the system to never Complete
		if (SystemInstanceController->IsSolo() == false)
		{
			Super::Deactivate();
		}

		SystemInstanceController->Deactivate();

		// We are considered active until we are complete
		// Note: Deactivate call can finalize -> complete the system -> release to pool -> unregister which will result in a nullptr for the SystemInstance
		SetActiveFlag(SystemInstanceController ? !SystemInstanceController->IsComplete() : false);
	}
	else
	{
		Super::Deactivate();

		if(bWasCulledByScalabiltiy && !bIsCulledByScalability)//We were culled by scalability but no longer, ensure we've handled completion correctly. E.g. returned to the pool etc.
		{
			OnSystemComplete(true);
		}
		SetActiveFlag(false);
	}
}

void UNiagaraComponent::DeactivateImmediate()
{
	DeactivateImmediateInternal(false);
}

ENiagaraExecutionState UNiagaraComponent::GetRequestedExecutionState() const
{
	return SystemInstanceController ? SystemInstanceController->GetRequestedExecutionState() : ENiagaraExecutionState::Complete;
}

ENiagaraExecutionState UNiagaraComponent::GetExecutionState() const
{
	return SystemInstanceController ? SystemInstanceController->GetActualExecutionState() : ENiagaraExecutionState::Complete;
}

bool UNiagaraComponent::IsComplete() const
{
	return SystemInstanceController ? SystemInstanceController->IsComplete() : true;
}

void UNiagaraComponent::DeactivateImmediateInternal(bool bIsScalabilityCull)
{
	FNiagaraInsights::NotifyComponentDeactivate(this, true, bIsScalabilityCull);

	SCOPE_CYCLE_COUNTER(STAT_NiagaraComponentDeactivate);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);

	Super::Deactivate();

	//UE_LOGF(LogNiagara, Log, "DeactivateImmediate: %p - %ls - %ls", this, *Asset->GetName(), bIsScalabilityCull ? TEXT("Scalability") : TEXT(""));

	//UE_LOGF(LogNiagara, Log, "Deactivate %ls", *GetName());

	//Unregister with the scalability manager if this is a genuine deactivation from outside.
	//The scalability manager itself can call this function when culling systems.
	if (bIsScalabilityCull)
	{
		bIsCulledByScalability = true;
	}
	else
	{
		UnregisterWithScalabilityManager();
	}

	SetActiveFlag(false);

	if (SystemInstanceController)
	{
		SystemInstanceController->Deactivate(true);
	}
	
	if (bIsFullyComplete == false)//In some flows we can get here but not yet be fully complete.
	{
		OnSystemComplete(true);
	}
}

bool UNiagaraComponent::ShouldPreCull()
{
	if (bAllowScalability && bOwnerAllowsScalabiltiy)
	{
		if (UNiagaraSystem* System = GetAsset())
		{
			if (UNiagaraEffectType* EffectType = System->GetEffectType())
			{
				if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(GetWorld()))
				{
					if (EffectType->UpdateFrequency == ENiagaraScalabilityUpdateFrequency::SpawnOnly)
					{
						//If we're just set to check on spawn then check for precull here.
						return WorldMan->ShouldPreCull(GetAsset(), this);
					}
				}
			}
		}
	}

	return false;
}

void UNiagaraComponent::RegisterWithScalabilityManager()
{
	if (ScalabilityManagerHandle == INDEX_NONE && bAllowScalability && bOwnerAllowsScalabiltiy)
	{
		if (UNiagaraSystem* System = GetAsset())
		{
			if (UNiagaraEffectType* EffectType = System->GetEffectType())
			{
				if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(GetWorld()))
				{
					ScalabilityEffectType = EffectType;
					WorldMan->RegisterWithScalabilityManager(this, ScalabilityEffectType);
				}
			}
		}
	}
}

void UNiagaraComponent::UnregisterWithScalabilityManager()
{
	if (ScalabilityManagerHandle != INDEX_NONE)
	{
		if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(GetWorld()))
		{
			WorldMan->UnregisterWithScalabilityManager(this, ScalabilityEffectType);
		}
	}
	bIsCulledByScalability = false;
	ScalabilityEffectType = nullptr;
	ScalabilityManagerHandle = INDEX_NONE;//Just to be sure our state is unregistered.
}

void UNiagaraComponent::PostSystemTick_GameThread()
{
	check(SystemInstanceController.IsValid()); // sanity

	if (FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		// return early when we're in the middle of another object's postload phase, as ticking data interfaces and renderers might lead to failed assertions
#if WITH_EDITOR || DO_CHECK
		UE_LOGF(LogNiagara, Warning, "Niagara component %ls PostSystemTick_GameThread was called during PostLoad. This should not happen, as it can lead to dangling RT pointers.", *GetPathName());
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
#endif
		return;
	}

#if WITH_EDITOR
	if (SystemInstanceController->HandleNeedsUIResync())
	{
		for (UNiagaraDataInterface* OverrideDataInterface : OverrideParameters.GetDataInterfaces())
		{
			if (OverrideDataInterface != nullptr)
			{
				OverrideDataInterface->RefreshErrors();
			}
		}
	}
#endif

	// NOTE: Since this is happening before scene visibility calculation, it's likely going to be off by a frame
	SystemInstanceController->SetLastRenderTime(GetLastRenderTime());

	MarkRenderDynamicDataDirty();

	// Check to force update our transform based on a timer or bounds expanding beyond their previous local boundaries
	const FBox NewLocalBounds = SystemInstanceController->GetLocalBounds();
	ForceUpdateTransformTime += GetWorld()->GetDeltaSeconds();
	if (!CurrLocalBounds.IsValid ||
		!CurrLocalBounds.IsInsideOrOn(NewLocalBounds.Min) ||
		!CurrLocalBounds.IsInsideOrOn(NewLocalBounds.Max) ||
		(ForceUpdateTransformTime > MaxTimeBeforeForceUpdateTransform))
	{
		CurrLocalBounds = NewLocalBounds;
		ForceUpdateTransformTime = 0.0f;
		UpdateComponentToWorld();
	}

	// Give renderers a chance to do some processing PostTick
	if (auto NiagaraProxy = static_cast<FNiagaraSceneProxy*>(SceneProxy))
	{
		if (auto RenderData = NiagaraProxy->GetSystemRenderData())
		{
			SystemInstanceController->PostTickRenderers(*RenderData);
		}
	}

	if (bRecachePSOs)
	{
		const FNiagaraReadinessSettings& ReadinessSettings = GetReadinessSettings();
		if (ReadinessSettings.PSOPrecacheSettings.PrecachingTime != ENiagaraPSOPrecachingTime::None && UPSOPrecacheSettingsManager::GetSettings().ParticlePrecachingTime > 1)
		{
			PrecachePSOs();
		}
		bRecachePSOs = false;
	}
}

void UNiagaraComponent::OnSystemComplete(bool bExternalCompletion)
{
	FNiagaraInsights::NotifyComponentComplete(this, bExternalCompletion);

	// Debug feature, if we have loop enabled all world FX will loop unless deactivate by scalability
#if !UE_BUILD_SHIPPING
	if ( !bExternalCompletion && !bIsCulledByScalability )
	{
		if ( UWorld* World = GetWorld() )
		{
			FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(GetWorld());
			if ( WorldManager->GetDebugPlaybackMode() == ENiagaraDebugPlaybackMode::Loop )
			{
				// If we have a loop time set the WorldManager will force a reset, so we will just ignore the completion event in that case
				static const auto CVarGlobalLoopTime = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.Niagara.Debug.GlobalLoopTime"));
				if (CVarGlobalLoopTime && (CVarGlobalLoopTime->GetFloat() <= 0.0f))
				{
					Activate(true);
				}
				return;
			}
		}
	}
#endif

	//UE_LOGF(LogNiagara, Log, "OnSystemComplete: %p - %ls", SystemInstance.Get(), *Asset->GetName());
	SetComponentTickEnabled(false);
	SetActiveFlag(false);

	// Give renderers a chance to handle completion
	if (SystemInstanceController)
	{
		if (auto NiagaraProxy = static_cast<FNiagaraSceneProxy*>(SceneProxy))
		{
			if (auto RenderData = NiagaraProxy->GetSystemRenderData())
			{
				SystemInstanceController->NotifyRenderersComplete(*RenderData);
			}
		}
	}

	// Mark our render data dirty as we want to create an empty set of render data in the case of scalability deactivating
	MarkRenderDynamicDataDirty();

	//Don't really complete if we're being culled by scalability.
	//We want to stop ticking but not be reclaimed by the pools etc.
	//We also want to skip this work if we're destroying during and update context reset.
	if (bIsCulledByScalability == false && bDuringUpdateContextReset == false)
	{
		bIsFullyComplete = true;
		DestroyCullProxy();

		//UE_LOGF(LogNiagara, Log, "OnSystemFinished.Broadcast(this);: { %p -  %p - %ls", this, SystemInstance.Get(), *Asset->GetName());
#if WITH_EDITOR
		// When in the editor do not invoke the system finished delegate if we are reconstrucing blueprint instances as they can have side effects.
		// For example the finished delegate could destroy the actor, attach / remove components / etc
		if (!GIsReconstructingBlueprintInstances)
#endif
		{
			OnSystemFinished.Broadcast(this);
		}
		//UE_LOGF(LogNiagara, Log, "OnSystemFinished.Broadcast(this);: } %p - %ls", SystemInstance.Get(), *Asset->GetName());

		if (PoolingMethod == ENCPoolMethod::AutoRelease)//Don't release back to the pool if we're completing due to scalability culling.
		{
			if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(GetWorld()))
			{
				WorldManager->GetComponentPool()->ReclaimWorldParticleSystem(this);
			}
		}
		else if (PoolingMethod == ENCPoolMethod::ManualRelease_OnComplete)
		{
			PoolingMethod = ENCPoolMethod::ManualRelease;
			if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(GetWorld()))
			{
				WorldManager->GetComponentPool()->ReclaimWorldParticleSystem(this);
			}
		}
		else if (bAutoDestroy)
		{
			//UE_LOGF(LogNiagara, Log, "OnSystemComplete DestroyComponent();: { %p - %ls", SystemInstance.Get(), *Asset->GetName());
			DestroyComponent();
			//UE_LOGF(LogNiagara, Log, "OnSystemComplete DestroyComponent();: } %p - %ls", SystemInstance.Get(), *Asset->GetName());
		}
		else
		{
			// For components who do not auto destroy or auto release back into the pool ensure we remove the render state
			// Without doing this we will 'leak' proxies until the user manually destroys or releases.
			if (IsRenderStateCreated())
			{
				DestroyRenderState_Concurrent();
			}
		}
		
		if (!bAutoDestroy)
		{
			if (bAutoManageAttachment)
			{
				CancelAutoAttachment(/*bDetachFromParent=*/ true);
			}
		}

		if (IsRegisteredWithScalabilityManager())
		{
			if (bExternalCompletion == false)
			{
				//Can we be sure this isn't going to spam erroneously?
				if (UNiagaraEffectType* EffectType = GetAsset()->GetEffectType())
				{
					//Only trigger warning if we're not being deactivated/completed from the outside and this is a natural completion by the system itself.
					if (EffectType->CullReaction == ENiagaraCullReaction::DeactivateImmediateResume || EffectType->CullReaction == ENiagaraCullReaction::DeactivateResume)
					{
						if (NiagaraComponentPrivate::GWarnAsleepCullReaction == 1 && GetAsset())
						{
							//If we're completing naturally, i.e. we're a burst/non-looping system then we shouldn't be using a mode reactivates the effect.
							UE_LOGF(LogNiagara, Warning, "Niagara Effect has completed naturally but has an effect type with the \"Asleep\" cull reaction. If an effect like this is culled before it can complete then it could leak into the scalability manager and be reactivated incorrectly. Please verify this is using the correct EffectType.\nComponent:%ls\nSystem:%ls"
								, *GetFullName(), *GetFullNameSafe(GetAsset()));
						}
					}
				}
			}
			//We've completed naturally so unregister with the scalability manager.
			UnregisterWithScalabilityManager();
		}
	}

	if (GIsAutomationTesting && GetAsset() != nullptr)
	{
		UE_LOGF(LogNiagara, Log, "UNiagaraComponent> System %ls completed.", *GetAsset()->GetPathName());
	}
}

void UNiagaraComponent::DestroyInstance()
{
	//UE_LOGF(LogNiagara, Log, "UNiagaraComponent::DestroyInstance: %p - %p  %ls\n", this, SystemInstance.Get(), *GetAsset()->GetFullName());
	//UE_LOGF(LogNiagara, Log, "DestroyInstance: %p - %ls", this, *Asset->GetName());

	SetActiveFlag(false);
	UnregisterWithScalabilityManager();

	if (SystemInstanceController)
	{
		SystemInstanceController->Deactivate(true);

		if (SystemInstanceController)
		{
			SystemInstanceController->Release();
			SystemInstanceController = nullptr;
		}
	}

#if WITH_EDITORONLY_DATA
	OnSystemInstanceChangedDelegate.Broadcast();
#endif
	MarkRenderStateDirty();
}

void UNiagaraComponent::DestroyInstanceNotComponent()
{
	const bool bCachedAutoDestroy = bAutoDestroy;
	const ENCPoolMethod CachedPoolMethod = PoolingMethod;
	bAutoDestroy = false;
	PoolingMethod = ENCPoolMethod::None;

	DestroyInstance();

	bAutoDestroy = bCachedAutoDestroy;
	PoolingMethod = CachedPoolMethod;
}

void UNiagaraComponent::OnPooledReuse(UWorld* NewWorld)
{
	check(IsValidChecked(this));

	ParameterCollectionOverrides.Reset();

	SetUserParametersToDefaultValues();

	//Need to reset the component's visibility in case it's returned to the pool while marked invisible.
	SetVisibility(true);
	SetHiddenInGame(false);

	if (GetWorld() != NewWorld)
	{
		// Rename the NC to move it into the current PersistentLevel - it may have been spawned in one
		// level but is now needed in another level.
		Rename(nullptr, NewWorld);
	}

	//We reset last render time to the current time so that any visibility culling on a delay will function correctly.
	//Leaving as the default of -1000 causes the visibility code to always assume this should be culled until it's first rendered and initialized by the RT.
	SetLastRenderTime(GetWorld()->GetTimeSeconds());

	ForceUpdateTransformTime = 0.0f;
	CurrLocalBounds.Init();
	SystemFixedBounds.Init();
	EmitterOverrideInfos.Empty();

	PrimaryComponentTick.SetTickFunctionEnable(false);

	if (SystemInstanceController != nullptr)
	{
		UpdateInstanceSoloMode();
		SystemInstanceController->OnPooledReuse(*NewWorld);
		SystemInstanceController->SetLastRenderTime(GetLastRenderTime());
	}
}

bool UNiagaraComponent::IsActiveForUpdateContext() const
{
	return
		// If we are activate and the execution state is also active (anything else means we were winding down / disabled)
		(IsActive() && GetRequestedExecutionState() == ENiagaraExecutionState::Active) ||
		// We were waiting on compilation / DDC request to come back
		(bAwaitingActivationDueToNotReady && PrimaryComponentTick.IsTickFunctionEnabled()) || 
		// We are registered with scalabilty, in which case allow it to take control
		IsRegisteredWithScalabilityManager();
}

void UNiagaraComponent::OnRegister()
{
	if (IsActive() && SystemInstanceController.IsValid() == false)
	{
		// If we're active but don't have an active system instance clear the active flag so that the component
		// gets activated.
		SetActiveFlag(false);
	}

	if (bAutoManageAttachment && !IsActive())
	{
		// Detach from current parent, we are supposed to wait for activation.
		if (GetAttachParent())
		{
			// If no auto attach parent override, use the current parent when we activate
			if (!AutoAttachParent.IsValid())
			{
				AutoAttachParent = GetAttachParent();
			}
			// If no auto attach socket override, use current socket when we activate
			if (AutoAttachSocketName == NAME_None)
			{
				AutoAttachSocketName = GetAttachSocketName();
			}

			// Prevent attachment before Super::OnRegister() tries to attach us, since we only attach when activated.
			if (GetAttachParent()->GetAttachChildren().Contains(this))
			{
				// Only detach if we are not about to auto attach to the same target, that would be wasteful.
				if (!bAutoActivate || (AutoAttachLocationRule != EAttachmentRule::KeepRelative && AutoAttachRotationRule != EAttachmentRule::KeepRelative && AutoAttachScaleRule != EAttachmentRule::KeepRelative) || (AutoAttachSocketName != GetAttachSocketName()) || (AutoAttachParent != GetAttachParent()))
				{
					DetachFromComponent(FDetachmentTransformRules(EDetachmentRule::KeepRelative, /*bCallModify=*/ false));
				}
			}
			else
			{
				SetupAttachment(nullptr, NAME_None);
			}
		}

		SavedAutoAttachRelativeLocation = GetRelativeLocation();
		SavedAutoAttachRelativeRotation = GetRelativeRotation();
		SavedAutoAttachRelativeScale3D = GetRelativeScale3D();
	}

#if WITH_EDITOR
	if (Asset && !AssetExposedParametersChangedHandle.IsValid())
	{
		AssetExposedParametersChangedHandle = Asset->GetExposedParameters().AddOnChangedHandler(
			FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraComponent::AssetExposedParametersChanged));
	}
#endif

	Super::OnRegister();

	// Deal with the case where the particle component is attached to an actor in a hidden sublevel. Without this, the component will be visible instead of being hidden as well.
	if (CachedLevelCollection == nullptr && GetOwner() == nullptr && IsValid(GetAttachParent()))
	{
		const ULevel* const AttachParentLevel = GetAttachParent()->GetComponentLevel();
		CachedLevelCollection = AttachParentLevel ? AttachParentLevel->GetCachedLevelCollection() : nullptr;
	}
}

bool UNiagaraComponent::IsReadyForOwnerToAutoDestroy() const
{
	return !IsActive();
}

void UNiagaraComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	//UE_LOGF(LogNiagara, Log, "OnComponentDestroyed %p %p", this, SystemInstanceController.Get());
	//DestroyInstance();//Can't do this here as we can call this from inside the system instance currently during completion 

	if (PoolingMethod != ENCPoolMethod::None)
	{
		if (UWorld* World = GetWorld())
		{
			if (FNiagaraUtilities::LogVerboseWarnings())
			{
				UE_LOGF(LogNiagara, Warning, "UNiagaraComponent::OnComponentDestroyed: Component (%p - %ls) Asset (%ls) is still pooled (%d) while destroying!", this, *GetFullNameSafe(this), *GetFullNameSafe(Asset), int(PoolingMethod));
			}
			if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World))
			{
				if (UNiagaraComponentPool* ComponentPool = WorldManager->GetComponentPool())
				{
					ComponentPool->PooledComponentDestroyed(this);
				}
			}
		}
		else
		{
			if (FNiagaraUtilities::LogVerboseWarnings())
			{
				UE_LOGF(LogNiagara, Warning, "UNiagaraComponent::OnComponentDestroyed: Component (%p - %ls) Asset (%ls) is still pooled (%d) while destroying and world it nullptr!", this, *GetFullNameSafe(this), *GetFullNameSafe(Asset), int(PoolingMethod));
			}
		}

		// Set pooling method to none as we are destroyed and can not go into the pool after this point
		PoolingMethod = ENCPoolMethod::None;
	}

	UnregisterWithScalabilityManager();

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

void UNiagaraComponent::OnComponentCreated()
{
	Super::OnComponentCreated();
#if WITH_EDITOR
	// When component's properties are initialized on a component on an actor spawned into the world using a template, neither post load or serialize is
	// called on the component when it's properties are up to date so this delegate isn't bound correctly.  the asset property will be valid when this
	// callback is called so we bind the delegate here if it's not already bound.  This fixes an issue where components in the level don't save changes 
	// made to user parameters if the user parameter list is changed. 
	if (Asset != nullptr && AssetExposedParametersChangedHandle.IsValid() == false)
	{
		AssetExposedParametersChangedHandle = Asset->GetExposedParameters().AddOnChangedHandler(
			FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraComponent::AssetExposedParametersChanged));
	}
#endif
}

// When creating niagara components from blueprint templates the constructions scripts are rerun which returns the instance properties to the template
// defaults.  The instance properties are then reapplied to that template, but in the case of Niagara components, the OverrideParameters are not saved
// in the instance properties so the template and instance overrides need to be reapplied here.  In addition to this, when instancing from blueprint 
// we can get into situations where the instancing code will get confused with objects stored in the different parameter override variables
// (TemplateParameterOverrides, InstanceParameterOverrides, OverrideParameters), so we make sure that the outers are properly fixed up here as well.
void UNiagaraComponent::PostApplyToComponent()
{
	Super::PostApplyToComponent();

#if WITH_EDITOR
	const bool bNeedsReactivate = IsRegistered() && IsActive() && NiagaraComponentPrivate::bAllowReactivateInPostApplyToComponent;

	// Changes to the component can be broad ranging and include things like manipulating the instance parameters.  In particular if the
	// data interfaces are changed out, we may need to rebind VM functions, so be sure to recreate the instance.
	DestroyInstanceNotComponent();

	// Ensure update the override parameters as they may be incomplete
	// Without this calls to set override data interface values can fail as the DI may not be present in the override parameter store
	if (Asset != nullptr)
	{
		CopyParametersFromAsset(false);
	}
	ApplyOverridesToParameterStore();
	FixDataInterfaceOuters();

	if (bNeedsReactivate)
	{
		Activate();
	}
#endif
}

void UNiagaraComponent::OnUnregister()
{
	Super::OnUnregister();

	SetActiveFlag(false);

	UnregisterWithScalabilityManager();

	if (SystemInstanceController)
	{
		//UE_LOGF(LogNiagara, Log, "UNiagaraComponent::OnUnregister: %p  %ls\n", SystemInstanceController.Get(), *GetAsset()->GetFullName());

		SystemInstanceController->Deactivate(true);

		if (PoolingMethod == ENCPoolMethod::None)
		{
			SystemInstanceController->Release();
			SystemInstanceController = nullptr;
#if WITH_EDITORONLY_DATA
			OnSystemInstanceChangedDelegate.Broadcast();
#endif
		}
	}
}

void UNiagaraComponent::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	Super::ApplyWorldOffset(InOffset, bWorldShift);

	// If this system is already active, assume that we want to dump the existing data and reset 
	if (IsActive())
	{
	
		Activate(true);
	}
}


void UNiagaraComponent::BeginDestroy()
{
	//UE_LOGF(LogNiagara, Log, "UNiagaraComponent::BeginDestroy(): %p - %d - %ls\n", this, ScalabilityManagerHandle, *GetAsset()->GetFullName());

	if (PoolingMethod != ENCPoolMethod::None)
	{
		if (UWorld* World = GetWorld())
		{
			if (FNiagaraUtilities::LogVerboseWarnings())
			{
				UE_LOGF(LogNiagara, Warning, "UNiagaraComponent::BeginDestroy: Component (%p - %ls) Asset (%ls) is still pooled (%d) while destroying!", this, *GetFullNameSafe(this), *GetFullNameSafe(Asset), int(PoolingMethod));
			}

			if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World))
			{
				if (UNiagaraComponentPool* ComponentPool = WorldManager->GetComponentPool())
				{
					ComponentPool->PooledComponentDestroyed(this);
				}
			}
		}
		else
		{
			if (FNiagaraUtilities::LogVerboseWarnings())
			{
				UE_LOGF(LogNiagara, Warning, "UNiagaraComponent::BeginDestroy: Component (%p - %ls) Asset (%ls) is still pooled (%d) while destroying and world is nullptr!", this, *GetFullNameSafe(this), *GetFullNameSafe(Asset), int(PoolingMethod));
			}
		}

		// Set pooling method to none as we are destroyed and can not go into the pool after this point
		PoolingMethod = ENCPoolMethod::None;
	}

	//In most cases we have already unregistered by now but there are few edge cases where we need to unregister here.
	UnregisterWithScalabilityManager();

	DestroyInstance();

	Super::BeginDestroy();
}

void UNiagaraComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	if (SystemInstanceController)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(SystemInstanceController->GetTotalBytesUsed());
	}
}

const UNiagaraComponent::FEmitterOverrideInfo* UNiagaraComponent::FindEmitterOverrideInfo(FName EmitterName) const
{
	for (const FEmitterOverrideInfo& Info : EmitterOverrideInfos)
	{
		if (Info.EmitterName == EmitterName)
		{
			return &Info;
		}
	}
	return nullptr;
}

UNiagaraComponent::FEmitterOverrideInfo* UNiagaraComponent::FindEmitterOverrideInfo(FName EmitterName)
{
	for (FEmitterOverrideInfo& Info : EmitterOverrideInfos)
	{
		if (Info.EmitterName == EmitterName)
		{
			return &Info;
		}
	}
	return nullptr;
}

UNiagaraComponent::FEmitterOverrideInfo& UNiagaraComponent::FindOrAddEmitterOverrideInfo(FName EmitterName)
{
	for (FEmitterOverrideInfo& Info : EmitterOverrideInfos)
	{
		if (Info.EmitterName == EmitterName)
		{
			return Info;
		}
	}
	return EmitterOverrideInfos.Emplace_GetRef(EmitterName);
}

void UNiagaraComponent::CreateCullProxy(bool bForce)
{
	if (Asset->GetCullProxyMode() != ENiagaraCullProxyMode::None && CullProxy == nullptr)
	{
		FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(GetWorld());
		check(WorldMan);
		
		if ( UNiagaraCullProxyComponent* SystemCullProxy = WorldMan->GetCullProxy(this) )
		{
			if (SystemCullProxy->RegisterCulledComponent(this, bForce))
			{
				CullProxy = SystemCullProxy;
			}
		}
	}
}

void UNiagaraComponent::DestroyCullProxy()
{
	//If we created a cull proxy then remove it now that we are activating.
	if (CullProxy)
	{
		CullProxy->UnregisterCulledComponent(this);
		CullProxy = nullptr;
	}
}

FParticlePerfStatsContext UNiagaraComponent::GetPerfStatsContext()
{
	return FParticlePerfStatsContext(GetWorld(), Asset, this);
}

TSharedPtr<FNiagaraSystemSimulation, ESPMode::ThreadSafe> UNiagaraComponent::GetSystemSimulation()
{
	if (SystemInstanceController)
	{
		return SystemInstanceController->GetSystemSimulation_Unsafe();
	}

	return nullptr;
}

void UNiagaraComponent::OnEndOfFrameUpdateDuringTick()
{
	Super::OnEndOfFrameUpdateDuringTick();
	if (SystemInstanceController)
	{
		SystemInstanceController->WaitForConcurrentTickAndFinalize();
	}
}

void UNiagaraComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);

	// The emitter instance may not tick again next frame so we send the dynamic data here so that the current state
	// renders.  This can happen when while editing, or any time the age update mode is set to desired age.
	FRegisterComponentContext::SendRenderDynamicData(Context, this);
}

void UNiagaraComponent::DestroyRenderState_Concurrent()
{
	if (SceneProxy)
	{
		static_cast<FNiagaraSceneProxy*>(SceneProxy)->DestroyRenderState_Concurrent();
	}

	Super::DestroyRenderState_Concurrent();
}

void UNiagaraComponent::SendRenderDynamicData_Concurrent()
{
	LLM_SCOPE(ELLMTag::Niagara);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_NiagaraSendRenderDynamicData);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraComponentSendRenderData);
	PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetWorld(), GetAsset(), this), EndOfFrame);

	Super::SendRenderDynamicData_Concurrent();

	if (FNiagaraSceneProxy* NiagaraProxy = static_cast<FNiagaraSceneProxy*>(SceneProxy))
	{
		if (FNiagaraSystemRenderData* RenderData = NiagaraProxy->GetSystemRenderData())
		{
			FNiagaraSceneProxy::FDynamicData NewProxyDynamicData;
			NewProxyDynamicData.LWCRenderTile = SystemInstanceController->GetSystemInstance_Unsafe()->GetLWCTile();
			NewProxyDynamicData.LODDistanceOverride = GetPreviewLODDistance();
#if WITH_PARTICLE_PERF_STATS
			NewProxyDynamicData.PerfStatsContext = GetPerfStatsContext();
#endif

			FNiagaraSystemRenderData::FSetDynamicDataCommandList SetDataCommands;

			if (bIsCulledByScalability && CullProxy)
			{
				//Render the cull proxy at this location instead.
				NewProxyDynamicData.bUseCullProxy = true;

				//Some renderers may need this transform on the GT so override here.
				FNiagaraSystemInstanceControllerPtr CullProxyController = CullProxy->GetSystemInstanceController();
				if ( ensureMsgf(CullProxyController.IsValid(), TEXT("CullProxy is missing the system instance controller for System '%s' Component '%s'"), *GetNameSafe(Asset), *GetFullNameSafe(this)) )
				{
					CullProxyController->GetSystemInstance_Unsafe()->SetWorldTransform(SystemInstanceController->GetSystemInstance_Unsafe()->GetWorldTransform());
					CullProxyController->GenerateSetDynamicDataCommands(SetDataCommands, *RenderData, *NiagaraProxy);
					CullProxyController->GetSystemInstance_Unsafe()->SetWorldTransform(FTransform::Identity);
				}
			}
			else
			{
				SystemInstanceController->GenerateSetDynamicDataCommands(SetDataCommands, *RenderData, *NiagaraProxy);
			}

			ENQUEUE_RENDER_COMMAND(NiagaraSetDynamicData)(UE::RenderCommandPipe::NiagaraDynamicData,
				[NiagaraProxy, CommandsRT = MoveTemp(SetDataCommands), NewProxyDynamicData]
				{
					SCOPE_CYCLE_COUNTER(STAT_NiagaraSetDynamicData);
					PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_RT(NewProxyDynamicData.PerfStatsContext, RenderUpdate, 1);

					FNiagaraSystemRenderData::ExecuteDynamicDataCommands_RenderThread(CommandsRT);

					NiagaraProxy->SetProxyDynamicData(NewProxyDynamicData);
				}
			);
		}
	}
}

int32 UNiagaraComponent::GetNumMaterials() const
{
	// we don't really care for the materials used by the component here
	// technically should be the number of materials used by each enabled renderer of each enabled emitter
	return 0;
}

FBoxSphereBounds UNiagaraComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FScopeCycleCounter SystemStatCounter(Asset ? Asset->GetStatID(true, false) : TStatId());

	// When inactive and using auto attachments do not include our bounds as they will be in an invalid location
	// While active it's more complicated as we could become detatched and wish to play the remainder of the effect so we must include them
	const USceneComponent* UseAutoParent = (bAutoManageAttachment && GetAttachParent() == nullptr) ? AutoAttachParent.Get() : nullptr;
	if (UseAutoParent && !IsActive())
	{
		// We use auto attachment but have detached, don't use our own bogus bounds (we're off near 0,0,0), use the usual parent's bounds.
		return UseAutoParent->Bounds;
	}

	if (bIsCulledByScalability && CullProxy)
	{
		check(CullProxy->GetComponentLocation() == FVector::ZeroVector);
		return CullProxy->CalcBounds(LocalToWorld);
	}

	FBoxSphereBounds SystemBounds;
	if (CurrLocalBounds.IsValid)
	{
		ensureMsgf(!CurrLocalBounds.Min.ContainsNaN() && !CurrLocalBounds.Max.ContainsNaN(), TEXT("CurrLocalBounds contains NaN for System(%s)"), *GetNameSafe(Asset));

		SystemBounds = CurrLocalBounds;
		SystemBounds.BoxExtent *= BoundsScale;
		SystemBounds.SphereRadius *= BoundsScale;
	}
	else
	{
		if (SystemFixedBounds.IsValid)
		{
			ensureMsgf(!SystemFixedBounds.Min.ContainsNaN() && !SystemFixedBounds.Max.ContainsNaN(), TEXT("SystemFixedBounds contains NaN for System(%s)"), *GetNameSafe(Asset));
			SystemBounds = FBoxSphereBounds(SystemFixedBounds);
		}
		else if ( Asset && Asset->bFixedBounds )
		{
			const FBox AssetFixedBounds = Asset->GetFixedBounds();
			ensureMsgf(!AssetFixedBounds.Min.ContainsNaN() && !AssetFixedBounds.Max.ContainsNaN(), TEXT("AssetFixedBounds contains NaN for System(%s)"), *GetNameSafe(Asset));
			SystemBounds = FBoxSphereBounds(AssetFixedBounds);
		}
		else
		{
			SystemBounds = FBoxSphereBounds(FVector::ZeroVector, FVector(1.0f), 1.0f);
		}
	}

	return SystemBounds.TransformBy(LocalToWorld);
}

FPrimitiveSceneProxy* UNiagaraComponent::CreateSceneProxy()
{
	LLM_SCOPE(ELLMTag::Niagara);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraCreateSceneProxy);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraOverview_GT);
    
#if UE_WITH_PSO_PRECACHING
	if (Asset != nullptr)
	{
		// If we are relying on proxy creation PSOs only then call precache PSOs here because asset doesn't have collected PSO precache data
		const FNiagaraReadinessSettings& ReadinessSettings = GetReadinessSettings();
		if (ReadinessSettings.PSOPrecacheSettings.PrecachingTime == ENiagaraPSOPrecachingTime::OnRenderStateCreation && !IsPSOPrecaching())
		{
			// Make sure the pso is being precached on the GT as we're likely not creating the scene proxy here, so we can wait for next frame
			TWeakObjectPtr<UNiagaraComponent> WeakThis(this);
			AsyncTask(ENamedThreads::GameThread, [WeakThis]()
			{
				if (WeakThis.IsValid())
				{
					WeakThis->PrecachePSOs();
				}
			});
		}

		if (IsPSOPrecaching() && SystemInstanceController.IsValid())
		{
			// we should only boost priority if we're sure that we actually have some PSOs still in flight that are not just asset-PSOs
			EPSOPrecachePriority PSOPrecachePriority = ReadinessSettings.PSOPrecacheSettings.GetPriorityBoost();
			if (CheckPSOPrecachingAndBoostPriority(PSOPrecachePriority) && GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate)
			{
				UE_LOGF(LogNiagara, Verbose, "Skipping CreateSceneProxy for UNiagaraComponent %ls (UNiagaraSystem PSOs are still compiling)", *GetFullName());
				return nullptr;
			}
		}
	}
#endif // UE_WITH_PSO_PRECACHING

	if (Asset == nullptr || !SystemInstanceController.IsValid() || !IsActive() || GetExecutionState() >= ENiagaraExecutionState::Complete)
	{
		// don't create a scene proxy if there is nothing to render
		return nullptr;
	}
	
	// The constructor will set up the System renderers from the component.
	FNiagaraSceneProxy* Proxy = new FNiagaraSceneProxy(FNiagaraSceneProxyDesc(*this));
	return Proxy;
}

void UNiagaraComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->CollectPSOPrecacheData(BasePrecachePSOParams, OutParams);
		
		if (bRenderCustomDepth)
		{
			for (FMaterialInterfacePSOPrecacheParams& PrecacheParams : OutParams)
			{
				PrecacheParams.PSOPrecacheParams.bRenderCustomDepth = bRenderCustomDepth;
			}
		}
	}
}

void UNiagaraComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	if (CullProxy)
	{
		CullProxy->GetUsedMaterials(OutMaterials, bGetDebugMaterials);
	}
	else if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->GetUsedMaterials(OutMaterials);
	}
}

void UNiagaraComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	if (SystemInstanceController.IsValid())
	{
		FNiagaraMaterialAndScaleArray MaterialAndScales;
		SystemInstanceController->GetMaterialStreamingInfo(MaterialAndScales);

		LevelContext.BindBuildData(nullptr);

		// Build the streaming bounds
		FBoxSphereBounds StreamingBounds = Bounds;
		if (Asset)
		{
			const FBox InitialStreamingBounds = Asset->GetInitialStreamingBounds();
			if (InitialStreamingBounds.IsValid)
			{
				const FTransform& LocalToWorld = GetComponentTransform();
				StreamingBounds = StreamingBounds + InitialStreamingBounds.TransformBy(LocalToWorld);
			}
		}

		if (MaterialAndScales.Num() > 0)
		{
			static const FMeshUVChannelInfo UVChannelData(1.f);

			FPrimitiveMaterialInfo MaterialData;
			MaterialData.PackedRelativeBox = PackedRelativeBox_Identity;
			MaterialData.UVChannelData = &UVChannelData;

			for (const FNiagaraMaterialAndScale& MaterialAndScale : MaterialAndScales)
			{
				check(MaterialAndScale.Material != nullptr);
				MaterialData.Material = MaterialAndScale.Material;
				LevelContext.ProcessMaterial(StreamingBounds, MaterialData, StreamingBounds.SphereRadius * MaterialAndScale.Scale, OutStreamingRenderAssets, bIsValidTextureStreamingBuiltData);
			}
		}

		SystemInstanceController->GetStreamingMeshInfo(StreamingBounds, LevelContext, OutStreamingRenderAssets);
	}
}

bool UNiagaraComponent::ResolveOwnerAllowsScalability(bool bRegister)
{
	bOwnerAllowsScalabiltiy = true;
	
	if(NiagaraComponentPrivate::GCanPreventScalabilityCullingOnPlayerFX && Asset && Asset->AllowScalabilityForLocalPlayerFX() == false)
	{
		bOwnerAllowsScalabiltiy = IsLocalPlayerEffect() == false;
	}	

	//Update manager registration if needed.
	if (bRegister)
	{
		if (bOwnerAllowsScalabiltiy)
		{
			if ( IsActive() )
			{
				RegisterWithScalabilityManager();
			}
		}
		else
		{
			if (bIsCulledByScalability)
			{
				ActivateInternal(false, true);
			}

			UnregisterWithScalabilityManager();
		}
	}
	return bOwnerAllowsScalabiltiy;
}

void UNiagaraComponent::OnAttachmentChanged()
{
	// Uncertain about this. 
	// 	if (bIsActive && !bIsChangingAutoAttachment && !GetOwner()->IsPendingKillPending())
	// 	{
	// 		ResetSystem();
	// 	}

	ResolveOwnerAllowsScalability();
	Super::OnAttachmentChanged();
}

void UNiagaraComponent::OnChildAttached(USceneComponent* ChildComponent)
{
	Super::OnChildAttached(ChildComponent);

}

void UNiagaraComponent::OnChildDetached(USceneComponent* ChildComponent)
{
	Super::OnChildDetached(ChildComponent);

}

bool UNiagaraComponent::IsVisible() const
{
	// Pooled components are owned by the persistent level, so we need to check their current attached parent's level visibility, otherwise they ignore sublevel visibility
	// Note we do not test the attached parents visibility as that doesn't match how Cascade works, and can cause visibility issues with gameplay cues who often attach to hidden components
	if (PoolingMethod != ENCPoolMethod::None)
	{
		if (USceneComponent* OwnerComponent = GetAttachParent())
		{
			if (OwnerComponent->CachedLevelCollection && !OwnerComponent->CachedLevelCollection->IsVisible())
			{
				return false;
			}
		}
	}

	return Super::IsVisible();
}

void UNiagaraComponent::SetTickBehavior(ENiagaraTickBehavior NewTickBehavior)
{
	TickBehavior = NewTickBehavior;
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetTickBehavior(TickBehavior);
	}
}

void UNiagaraComponent::SetRandomSeedOffset(int32 NewRandomSeedOffset)
{
	RandomSeedOffset = NewRandomSeedOffset;
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetRandomSeedOffset(RandomSeedOffset);
	}
}

void UNiagaraComponent::SetVariableLinearColor(FName InVariableName, const FLinearColor& InValue)
{
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetColorDef(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetVariable_Deferred(InVariableName, InValue);
	}
	else
	{
		OverrideParameters.SetParameterValue(InValue, VariableDesc, true);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(&InValue, sizeof(FLinearColor)));
#endif
}

void UNiagaraComponent::SetNiagaraVariableLinearColor(const FString& InVariableName, const FLinearColor& InValue)
{
	SetVariableLinearColor(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetVariableQuat(FName InVariableName, const FQuat& InValue)
{
	FQuat4f AsFloat = (FQuat4f)InValue;
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetQuatDef(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetVariable_Deferred(InVariableName, AsFloat);
	}
	else
	{
		OverrideParameters.SetParameterValue(AsFloat, VariableDesc, true);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(&AsFloat, sizeof(FQuat4f)));
#endif
}

void UNiagaraComponent::SetNiagaraVariableQuat(const FString& InVariableName, const FQuat& InValue)
{
	SetVariableQuat(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetNiagaraVariableMatrix(const FString& InVariableName, const FMatrix& InValue)
{
	SetVariableMatrix(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetVariableMatrix(FName InVariableName, const FMatrix& InValue)
{
	FMatrix44f AsFloat = (FMatrix44f)InValue;	//-TODO: LWC Precision loss
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetMatrix4Def(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetVariable_Deferred(InVariableName, AsFloat);
	}
	else
	{
		OverrideParameters.SetParameterValue(AsFloat, VariableDesc, true);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(&AsFloat, sizeof(FMatrix44f)));
#endif
}

void UNiagaraComponent::SetVariableVec4(FName InVariableName, const FVector4& InValue)
{
	FVector4f AsFloat = (FVector4f)InValue;
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetVec4Def(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetVariable_Deferred(InVariableName, AsFloat);
	}
	else
	{
		OverrideParameters.SetParameterValue(AsFloat, VariableDesc, true);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(&AsFloat, sizeof(FVector4f)));
#endif
}

void UNiagaraComponent::SetNiagaraVariableVec4(const FString& InVariableName, const FVector4& InValue)
{
	SetVariableVec4(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetVariableVec3(FName InVariableName, FVector InValue)
{
	// If the user parameter is of position type we use the position function instead, so it gets converted correctly 
	if (OverrideParameters.GetPositionParameterValue(InVariableName)) 
	{
		SetVariablePosition(InVariableName, InValue);
		return;
	}
	
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (!FitsIntoFloat(InValue))
	{
		UE_LOGF(LogNiagara, Warning, "Setting variable %ls on asset %ls with value (%f, %f, %f), which does not fit into a FVector3f without losing precision! Consider using a position type instead if the input is a location.", *InVariableName.ToString(), *GetPathName(), InValue.X, InValue.Y, InValue.Z);
	}
#endif
	FVector3f AsFloat(InValue);
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetVec3Def(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetVariable_Deferred(InVariableName, AsFloat);
	}
	else
	{
		OverrideParameters.SetParameterValue(AsFloat, VariableDesc, true);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(&AsFloat, sizeof(FVector3f)));
#endif
}

void UNiagaraComponent::SetNiagaraVariableVec3(const FString& InVariableName, FVector InValue)
{
	SetVariableVec3(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetVariablePosition(FName InVariableName, FVector InValue)
{
	// If the user parameter is not of position type we try to set it as vector instead, as users might confuse the two 
	if (!OverrideParameters.GetPositionParameterValue(InVariableName)) 
	{
		SetVariableVec3(InVariableName, InValue);
		return;
	}
	
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetPositionDef(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetVariable_Deferred(InVariableName, InValue);
	}
	else
	{
		OverrideParameters.SetPositionParameterValue(InValue, InVariableName, true);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(&InValue, sizeof(FVector)));
#endif
}

void UNiagaraComponent::SetNiagaraVariablePosition(const FString& InVariableName, FVector InValue)
{
	SetVariablePosition(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetVariableVec2(FName InVariableName, FVector2D InValue)
{
	FVector2f AsFloat = (FVector2f)InValue;
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetVec2Def(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetVariable_Deferred(InVariableName, AsFloat);
	}
	else
	{
		OverrideParameters.SetParameterValue(AsFloat, VariableDesc, true);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(&AsFloat, sizeof(FVector2f)));
#endif
}

void UNiagaraComponent::SetNiagaraVariableVec2(const FString& InVariableName, FVector2D InValue)
{
	SetVariableVec2(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetVariableFloat(FName InVariableName, float InValue)
{
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetFloatDef(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetVariable_Deferred(InVariableName, InValue);
	}
	else
	{
		OverrideParameters.SetParameterValue(InValue, VariableDesc, true);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(&InValue, sizeof(float)));
#endif
}

void UNiagaraComponent::SetNiagaraVariableFloat(const FString& InVariableName, float InValue)
{
	SetVariableFloat(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetVariableInt(FName InVariableName, int32 InValue)
{
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetIntDef(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetVariable_Deferred(InVariableName, InValue);
	}
	else
	{
		OverrideParameters.SetParameterValue(InValue, VariableDesc, true);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(&InValue, sizeof(int32)));
#endif
}

void UNiagaraComponent::SetNiagaraVariableInt(const FString& InVariableName, int32 InValue)
{
	SetVariableInt(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetVariableBool(FName InVariableName, bool InValue)
{
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetBoolDef(), InVariableName);
	const FNiagaraBool BoolValue(InValue);
	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetVariable_Deferred(InVariableName, InValue);
	}
	else
	{
		OverrideParameters.SetParameterValue(BoolValue, VariableDesc, true);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(&BoolValue, sizeof(FNiagaraBool)));
#endif
}

void UNiagaraComponent::SetNiagaraVariableBool(const FString& InVariableName, bool InValue)
{
	SetVariableBool(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetVariableActor(FName InVariableName, AActor* InValue)
{
	SetVariableObject(InVariableName, InValue);
}

void UNiagaraComponent::SetNiagaraVariableActor(const FString& InVariableName, AActor* InValue)
{
	SetVariableObject(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetVariableObject(FName InVariableName, UObject* InValue)
{
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetUObjectDef(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		TWeakObjectPtr<UObject> ObjPtr = InValue;
		SystemInstanceController->SetVariable_Deferred(InVariableName, ObjPtr);
	}
	else
	{
		OverrideParameters.SetUObject(InValue, VariableDesc);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(InValue));
#endif
}

void UNiagaraComponent::SetNiagaraVariableObject(const FString& InVariableName, UObject* InValue)
{
	SetVariableObject(FName(*InVariableName), InValue);
}

void UNiagaraComponent::SetVariableMaterial(FName InVariableName, UMaterialInterface* InValue)
{
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetUMaterialDef(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		TWeakObjectPtr<UMaterialInterface> ObjPtr = InValue;
		SystemInstanceController->SetVariable_Deferred(InVariableName, ObjPtr);
	}
	else
	{
		OverrideParameters.SetUObject(InValue, VariableDesc);
	}

	bRecachePSOs = true;

#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(InValue));
#endif
}

void UNiagaraComponent::SetVariableStaticMesh(FName InVariableName, UStaticMesh* InValue)
{
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetUStaticMeshDef(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		TWeakObjectPtr<UStaticMesh> ObjPtr = InValue;
		SystemInstanceController->SetVariable_Deferred(InVariableName, ObjPtr);
	}
	else
	{
		OverrideParameters.SetUObject(InValue, VariableDesc);
	}

	bRecachePSOs = true;

#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(InValue));
#endif
}

void UNiagaraComponent::SetVariableTexture(FName InVariableName, UTexture* Texture)
{
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetUTextureDef(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		TWeakObjectPtr<UTexture> ObjPtr = Texture;
		SystemInstanceController->SetVariable_Deferred(InVariableName, ObjPtr);
	}
	else
	{
		OverrideParameters.SetUObject(Texture, VariableDesc);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(Texture));
#endif
}

void UNiagaraComponent::SetVariableTextureRenderTarget(FName InVariableName, UTextureRenderTarget* TextureRenderTarget)
{
	const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetUTextureRenderTargetDef(), InVariableName);
	if (SystemInstanceController.IsValid())
	{
		TWeakObjectPtr<UTextureRenderTarget> ObjPtr = TextureRenderTarget;
		SystemInstanceController->SetVariable_Deferred(InVariableName, ObjPtr);
	}
	else
	{
		OverrideParameters.SetUObject(TextureRenderTarget, VariableDesc);
	}
#if WITH_EDITOR
	SetParameterOverride(VariableDesc, FNiagaraVariant(TextureRenderTarget));
#endif
}

void UNiagaraComponent::SetVariable_InternalUseOnly(const FNiagaraVariableBase& Variable, const FNiagaraVariant& Variant)
{
	if (!Variant.IsValid())
	{
		return;
	}

	if (SystemInstanceController.IsValid())
	{
		SystemInstanceController->SetVariable_InternalUseOnly_Deferred(Variable, Variant);
	}
	else
	{
		OverrideParameters.SetParameter_InternalUseOnly(Variable, Variant, true);
	}
#if WITH_EDITOR
	SetParameterOverride(Variable, Variant);
#endif
}

void UNiagaraComponent::GetVariable_InternalUseOnly(const FNiagaraVariableBase& Variable, FNiagaraVariant& Variant) const
{
	Variant.Reset();
	OverrideParameters.GetParameter_InternalUseOnly(Variable, Variant);
}

bool UNiagaraComponent::GetVariableBool(FName InVariableName, bool& bIsValid) const
{
	const FNiagaraVariableBase Variable(FNiagaraTypeDefinition::GetBoolDef(), InVariableName);
	TOptional<FNiagaraBool> Value = OverrideParameters.GetParameterOptionalValue<FNiagaraBool>(Variable);
	bIsValid = Value.IsSet();
	return Value.Get(FNiagaraBool::False).GetValue();
}

int32 UNiagaraComponent::GetVariableInt(FName InVariableName, bool& bIsValid) const
{
	const FNiagaraVariableBase Variable(FNiagaraTypeDefinition::GetIntDef(), InVariableName);
	TOptional<int32> Value = OverrideParameters.GetParameterOptionalValue<int32>(Variable);
	bIsValid = Value.IsSet();
	return Value.Get(0);
}

float UNiagaraComponent::GetVariableFloat(FName InVariableName, bool& bIsValid) const
{
	const FNiagaraVariableBase Variable(FNiagaraTypeDefinition::GetFloatDef(), InVariableName);
	TOptional<float> Value = OverrideParameters.GetParameterOptionalValue<float>(Variable);
	bIsValid = Value.IsSet();
	return Value.Get(0.0f);
}

FVector2D UNiagaraComponent::GetVariableVec2(FName InVariableName, bool& bIsValid) const
{
	const FNiagaraVariableBase Variable(FNiagaraTypeDefinition::GetVec2Def(), InVariableName);
	TOptional<FVector2f> Value = OverrideParameters.GetParameterOptionalValue<FVector2f>(Variable);
	bIsValid = Value.IsSet();
	return FVector2D(Value.Get(FVector2f::ZeroVector));
}

FVector UNiagaraComponent::GetVariableVec3(FName InVariableName, bool& bIsValid) const
{
	const FNiagaraVariableBase Variable(FNiagaraTypeDefinition::GetVec3Def(), InVariableName);
	TOptional<FVector3f> Value = OverrideParameters.GetParameterOptionalValue<FVector3f>(Variable);
	bIsValid = Value.IsSet();
	return FVector(Value.Get(FVector3f::ZeroVector));
}

FVector4 UNiagaraComponent::GetVariableVec4(FName InVariableName, bool& bIsValid) const
{
	const FNiagaraVariableBase Variable(FNiagaraTypeDefinition::GetVec4Def(), InVariableName);
	TOptional<FVector4f> Value = OverrideParameters.GetParameterOptionalValue<FVector4f>(Variable);
	bIsValid = Value.IsSet();
	return FVector4(Value.Get(FVector4f::Zero()));
}

FVector UNiagaraComponent::GetVariablePosition(FName InVariableName, bool& bIsValid) const
{
	const FVector* Position = OverrideParameters.GetPositionParameterValue(InVariableName);
	bIsValid = Position != nullptr;
	return Position ? *Position : FVector::ZeroVector;
}

FLinearColor UNiagaraComponent::GetVariableColor(FName InVariableName, bool& bIsValid) const
{
	const FNiagaraVariableBase Variable(FNiagaraTypeDefinition::GetColorDef(), InVariableName);
	TOptional<FLinearColor> Value = OverrideParameters.GetParameterOptionalValue<FLinearColor>(Variable);
	bIsValid = Value.IsSet();
	return Value.Get(FLinearColor::White);
}

FQuat UNiagaraComponent::GetVariableQuat(FName InVariableName, bool& bIsValid) const
{
	const FNiagaraVariableBase Variable(FNiagaraTypeDefinition::GetQuatDef(), InVariableName);
	TOptional<FQuat4f> Value = OverrideParameters.GetParameterOptionalValue<FQuat4f>(Variable);
	bIsValid = Value.IsSet();
	return FQuat(Value.Get(FQuat4f::Identity));
}

FMatrix UNiagaraComponent::GetVariableMatrix(FName InVariableName, bool& bIsValid) const
{
	const FNiagaraVariableBase Variable(FNiagaraTypeDefinition::GetMatrix4Def(), InVariableName);
	TOptional<FMatrix44f> Value = OverrideParameters.GetParameterOptionalValue<FMatrix44f>(Variable);
	bIsValid = Value.IsSet();
	return FMatrix(Value.Get(FMatrix44f::Identity));
}

void UNiagaraComponent::SetParameterCollectionOverride(UNiagaraParameterCollectionInstance* OverrideCollection)
{
	if (OverrideCollection == nullptr)
	{
		return;
	}

	UNiagaraParameterCollection* OverrideCollectionParent = OverrideCollection->GetParent();
	UNiagaraParameterCollectionInstance* ExistingCollection = nullptr;
	if (SystemInstanceController)
	{
		// Find existing collection, if the same we can early out
		const FNiagaraParameterCollectionResolver ParameterCollectionResolver = SystemInstanceController->CreateParameterCollectionResolver();
		ExistingCollection = ParameterCollectionResolver.ResolveInstance(OverrideCollectionParent);
		if (ExistingCollection == OverrideCollection)
		{
			return;
		}
	}

	// Add collection, replacing a one that shared the same parent if we have one
	const int32 ExistingIndex = ParameterCollectionOverrides.IndexOfByPredicate([OverrideCollectionParent](UNiagaraParameterCollectionInstance* Existing) { return Existing && Existing->GetParent() == OverrideCollectionParent; });
	if (ExistingIndex == INDEX_NONE)
	{
		ParameterCollectionOverrides.Add(OverrideCollection);
	}
	else
	{
		ParameterCollectionOverrides[ExistingIndex] = OverrideCollection;
	}

	// Ensure we are in solo mode
	// Note: This must be done BEFORE we modify any bindings
	UpdateInstanceSoloMode();

	// Bind the override collection
	if (SystemInstanceController)
	{
		SystemInstanceController->RebindParameterCollection(ExistingCollection, OverrideCollection);
	}
}

void UNiagaraComponent::RemoveParameterCollectionOverride(UNiagaraParameterCollectionInstance* OverrideCollection)
{
	if (OverrideCollection == nullptr)
	{
		return;
	}

	if (ParameterCollectionOverrides.RemoveSwap(OverrideCollection) == 0)
	{
		return;
	}

	if (SystemInstanceController)
	{
		const FNiagaraParameterCollectionResolver ParameterCollectionResolver = SystemInstanceController->CreateParameterCollectionResolver();

		UNiagaraParameterCollection* OverrideCollectionParent = OverrideCollection->GetParent();
		UNiagaraParameterCollectionInstance* ReplacementCollection = ParameterCollectionResolver.ResolveInstance(OverrideCollectionParent);
		SystemInstanceController->RebindParameterCollection(OverrideCollection, ReplacementCollection);
	}

	// Update solo mode
	// Note: This must be done AFTER we have potentially modified the bindings
	UpdateInstanceSoloMode();
}

void UNiagaraComponent::ClearParameterCollectionOverrides()
{
	while (ParameterCollectionOverrides.Num() > 0)
	{
		RemoveParameterCollectionOverride(ParameterCollectionOverrides.Last());
	}
}

UNiagaraParameterCollectionInstance* UNiagaraComponent::GetParameterCollectionOverride(UNiagaraParameterCollection* ParameterCollection) const
{
	const int32 ExistingIndex = ParameterCollectionOverrides.IndexOfByPredicate(
		[Parent = ParameterCollection](const UNiagaraParameterCollectionInstance* Instance)
		{
			return Instance && Instance->GetParent() == Parent;
		}
	);

	return ExistingIndex == INDEX_NONE ? nullptr : ParameterCollectionOverrides[ExistingIndex];
}

void FixInvalidUserParameters(FNiagaraUserRedirectionParameterStore& ParameterStore)
{
	static const FString UserPrefix = FNiagaraConstants::UserNamespace.ToString() + TEXT(".");

	TArray<FNiagaraVariable> Parameters;
	ParameterStore.GetParameters(Parameters);
	TArray<FNiagaraVariable> IncorrectlyNamedParameters;
	for (FNiagaraVariable& Parameter : Parameters)
	{
		if (Parameter.GetName().ToString().StartsWith(UserPrefix) == false)
		{
			IncorrectlyNamedParameters.Add(Parameter);
		}
	}

	bool bParameterRenamed = false;
	for(FNiagaraVariable& IncorrectlyNamedParameter : IncorrectlyNamedParameters)
	{
		FString FixedNameString = UserPrefix + IncorrectlyNamedParameter.GetName().ToString();
		FName FixedName = *FixedNameString;
		FNiagaraVariable FixedParameter(IncorrectlyNamedParameter.GetType(), FixedName);
		if (Parameters.Contains(FixedParameter))
		{
			// If the correctly named parameter is also in the collection then both parameters need to be removed and the 
			// correct one re-added.  First we need to cache the value of the parameter so that it's not lost on removal.
			UNiagaraDataInterface* DataInterfaceValue = nullptr;
			TObjectPtr<UObject> ObjectValue = nullptr;
			TArray<uint8> DataValue;
			if (IncorrectlyNamedParameter.IsDataInterface())
			{
				DataInterfaceValue = ParameterStore.GetDataInterface(IncorrectlyNamedParameter);
			}
			else if (IncorrectlyNamedParameter.IsUObject())
			{
				ObjectValue = ParameterStore.GetUObject(IncorrectlyNamedParameter);
			}
			else
			{
				const uint8* DataValuePtr = ParameterStore.GetParameterData(IncorrectlyNamedParameter);
				if (DataValuePtr != nullptr)
				{
					DataValue.AddUninitialized(IncorrectlyNamedParameter.GetSizeInBytes());
					ParameterStore.CopyParameterData(IncorrectlyNamedParameter, DataValue.GetData());
				}
			}

			// Next we remove the parameter twice because the first removal of the incorrect parameter will actually remove
			// the correct version due to the user redirection table.
			ParameterStore.RemoveParameter(IncorrectlyNamedParameter);
			ParameterStore.RemoveParameter(IncorrectlyNamedParameter);

			// Last we add back the fixed parameter and set the value.
			ParameterStore.AddParameter(FixedParameter, false);
			if (DataInterfaceValue != nullptr)
			{
				ParameterStore.SetDataInterface(DataInterfaceValue, FixedParameter);
			}
			else if (ObjectValue != nullptr)
			{
				ParameterStore.SetUObject(ObjectValue, FixedParameter);
			}
			else
			{
				if (DataValue.Num() == FixedParameter.GetSizeInBytes())
				{
					ParameterStore.SetParameterData(DataValue.GetData(), FixedParameter);
				}
			}
		}
		else 
		{
			// If the correctly named parameter was not in the collection already we can just rename the incorrect parameter
			// to the correct one.
			ParameterStore.RenameParameter(IncorrectlyNamedParameter, FixedName);
			bParameterRenamed = true;
		}
	}

	if (bParameterRenamed)
	{
		ParameterStore.RecreateRedirections();
	}
}

void UNiagaraComponent::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);
#if WITH_EDITOR
	if (Record.GetUnderlyingArchive().IsLoading())
	{
		FixDataInterfaceOuters();
		FixDataInterfaceObjectFlags();

		// Check the the event handler here as well as in post load since post load isn't always called due to how components are managed when spawning
		// actors with templates.
		// When called from the loader, we know postload is going to be called, so avoid race conditions on the delegate by avoiding registering to it while in the loader.
		if (!IsInAsyncLoadingThread() && !IsInParallelLoadingThread())
		{
			if (Asset != nullptr && AssetExposedParametersChangedHandle.IsValid() == false)
			{
				AssetExposedParametersChangedHandle = Asset->GetExposedParameters().AddOnChangedHandler(
					FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraComponent::AssetExposedParametersChanged));
			}
		}
	}
#endif
}

void UNiagaraComponent::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	auto PostLoadVariantDIs = [](auto& VariantMap)
	{
		for (const auto& Pair : VariantMap)
		{
			if (Pair.Value.GetMode() == ENiagaraVariantMode::DataInterface)
			{
				if (UNiagaraDataInterface* DI = Pair.Value.GetDataInterface())
				{
					DI->ConditionalPostLoad();
				}
			}
		}
	};
	PostLoadVariantDIs(TemplateParameterOverrides);
	PostLoadVariantDIs(InstanceParameterOverrides);
#endif

	OverrideParameters.PostLoad(this);
	OverrideParameters.SanityCheckData();

#if WITH_EDITOR
	if (Asset != nullptr)
	{
		Asset->ConditionalPostLoad();

		FixInvalidUserParameters(OverrideParameters);		
		UpgradeDeprecatedParameterOverrides();

#if WITH_EDITORONLY_DATA
		const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
		if (NiagaraVer < FNiagaraCustomVersion::ComponentsOnlyHaveUserVariables)
		{
			{
				TArray<FNiagaraVariableBase> ToRemoveNonUser;
				TArray<FNiagaraVariableBase> ToAddNonUser;

				auto InstanceKeyIter = InstanceParameterOverrides.CreateConstIterator();
				while (InstanceKeyIter)
				{
					FName KeyName = InstanceKeyIter.Key().GetName();
					FNiagaraVariableBase VarBase = InstanceKeyIter.Key();
					FNiagaraUserRedirectionParameterStore::MakeUserVariable(VarBase);
					FName UserKeyName = VarBase.GetName();
					if (KeyName != UserKeyName)
					{
						UE_LOGF(LogNiagara, Log, "InstanceParameterOverrides for %ls has non-user keys in it! %ls. Updating in PostLoad to User key.", *GetPathName(), *KeyName.ToString());
						const FNiagaraVariant* FoundVar = InstanceParameterOverrides.Find(VarBase);
						if (FoundVar != nullptr)
						{
							UE_LOGF(LogNiagara, Warning, "InstanceParameterOverrides for %ls has values for both keys in it! %ls and %ls. PostLoad keeping User version.", *GetPathName(), *KeyName.ToString(), *UserKeyName.ToString());
						}
						else
						{
							ToAddNonUser.Add(InstanceKeyIter.Key());
						}
						ToRemoveNonUser.Add(InstanceKeyIter.Key());
					}
					++InstanceKeyIter;
				}

				for (const FNiagaraVariableBase& Var : ToAddNonUser)
				{
					const FNiagaraVariant FoundVar = InstanceParameterOverrides.FindRef(Var);
					FNiagaraVariableBase UserVar = Var;
					FNiagaraUserRedirectionParameterStore::MakeUserVariable(UserVar);
					InstanceParameterOverrides.Emplace(UserVar, FoundVar);
				}

				for (const FNiagaraVariableBase& Var : ToRemoveNonUser)
				{
					InstanceParameterOverrides.Remove(Var);
				}
			}

			{
				TArray<FNiagaraVariableBase> ToRemoveNonUser;
				TArray<FNiagaraVariableBase> ToAddNonUser;

				auto TemplateKeyIter = TemplateParameterOverrides.CreateConstIterator();
				while (TemplateKeyIter)
				{
					FName KeyName = TemplateKeyIter.Key().GetName();
					FNiagaraVariableBase VarBase = TemplateKeyIter.Key();
					FNiagaraUserRedirectionParameterStore::MakeUserVariable(VarBase);
					FName UserKeyName = VarBase.GetName();
					if (KeyName != UserKeyName)
					{
						UE_LOGF(LogNiagara, Log, "TemplateParameterOverrides for %ls has non-user keys in it! %ls. Updating in PostLoad to User key.", *GetPathName(), *KeyName.ToString());
						const FNiagaraVariant* FoundVar = TemplateParameterOverrides.Find(VarBase);
						if (FoundVar != nullptr)
						{
							UE_LOGF(LogNiagara, Warning, "TemplateParameterOverrides for %ls has values for both keys in it! %ls and %ls.  PostLoad keeping User version.", *GetPathName(), *KeyName.ToString(), *UserKeyName.ToString());
						}
						else
						{
							ToAddNonUser.Add(TemplateKeyIter.Key());
						}
						ToRemoveNonUser.Add(TemplateKeyIter.Key());
					}
					++TemplateKeyIter;
				}

				for (const FNiagaraVariableBase& Var : ToAddNonUser)
				{
					const FNiagaraVariant FoundVar = TemplateParameterOverrides.FindRef(Var);
					FNiagaraVariableBase UserVar = Var;
					FNiagaraUserRedirectionParameterStore::MakeUserVariable(UserVar);
					TemplateParameterOverrides.Emplace(UserVar, FoundVar);
				}

				for (const FNiagaraVariableBase& Var : ToRemoveNonUser)
				{
					TemplateParameterOverrides.Remove(Var);
				}
			}
		}
#endif

		FixInvalidUserParameterOverrideData();
		SynchronizeWithSourceSystem();
		FixDataInterfaceOuters();
		OverrideParameters.SanityCheckData();

		if(AssetExposedParametersChangedHandle.IsValid() == false)
		{
			AssetExposedParametersChangedHandle = Asset->GetExposedParameters().AddOnChangedHandler(
				FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraComponent::AssetExposedParametersChanged));
		}
	}
#endif
}

#if WITH_EDITORONLY_DATA
void UNiagaraComponent::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	TArray<UClass*> DataInterfaceClasses;
	GetDerivedClasses(UNiagaraDataInterface::StaticClass(), DataInterfaceClasses);
	for (UClass* DataInterfaceClass : DataInterfaceClasses)
	{
		OutConstructClasses.Add(FTopLevelAssetPath(DataInterfaceClass));
	}
}
#endif


static FNiagaraVariant GetParameterValueFromStore(const FNiagaraVariableBase& Var, const FNiagaraParameterStore& Store)
{
	if (Var.IsDataInterface())
	{
		const int32 Index = Store.IndexOf(Var);
		if (Index != INDEX_NONE)
		{
			return FNiagaraVariant(Store.GetDataInterfaces()[Index]);
		}
	}
	else if (Var.IsUObject())
	{
		const int32 Index = Store.IndexOf(Var);
		if (Index != INDEX_NONE)
		{
			return FNiagaraVariant(Store.GetUObjects()[Index]);
		}
	}

	const uint8* ParameterData = Store.GetParameterData(Var);
	if (ParameterData == nullptr)
	{
		return FNiagaraVariant();
	}

	TArray<uint8> DataValue;
	DataValue.AddUninitialized(Var.GetSizeInBytes());
	if ( ensureMsgf(Store.CopyParameterData(Var, DataValue.GetData()), TEXT("Failed to copy parameter data for '%s'"), *Var.GetName().ToString()) )
	{
		return FNiagaraVariant(DataValue);
	}
	return FNiagaraVariant();
}

#if WITH_EDITOR
bool UNiagaraComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		// We don't support changing this property as it is fully controlled by Niagara in both batached and solo mode
		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FTickFunction, bStartWithTickEnabled))
		{
			return false;
		}
	}

	return Super::CanEditChange(InProperty);
}

void UNiagaraComponent::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (PropertyAboutToChange != nullptr && 
		PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraComponent, Asset) && 
		Asset != nullptr)
	{
		Asset->GetExposedParameters().RemoveOnChangedHandler(AssetExposedParametersChangedHandle);
		AssetExposedParametersChangedHandle.Reset();
		DestroyInstance();
	}
}

void UNiagaraComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName;
	if (PropertyChangedEvent.Property)
	{
		PropertyName = PropertyChangedEvent.Property->GetFName();
	}

	bool bNeedsReinitialize = true;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraComponent, Asset))
	{
		SynchronizeWithSourceSystem();
		if (Asset != nullptr)
		{
			AssetExposedParametersChangedHandle = Asset->GetExposedParameters().AddOnChangedHandler(
				FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraComponent::AssetExposedParametersChanged));
		}
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraComponent, OverrideParameters))
	{
		SynchronizeWithSourceSystem();

		INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");
		bNeedsReinitialize = NiagaraModule.GetEditorOnlyDataUtilities().GetResetSimulationOnParameterChange();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraComponent, TemplateParameterOverrides) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraComponent, InstanceParameterOverrides))
	{
		ApplyOverridesToParameterStore();

		INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");
		bNeedsReinitialize = NiagaraModule.GetEditorOnlyDataUtilities().GetResetSimulationOnParameterChange();
	}
	
	if (bNeedsReinitialize)
	{
		ReinitializeSystem();
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

void UNiagaraComponent::SetUserParametersToDefaultValues()
{
#if WITH_EDITORONLY_DATA
	EditorOverridesValue_DEPRECATED.Empty();
	TemplateParameterOverrides.Empty();
	InstanceParameterOverrides.Empty();
#endif
	
	if (Asset == nullptr)
	{
		OverrideParameters.Empty(false);
		return;
	}

	const FNiagaraUserRedirectionParameterStore& SourceUserParameterStore = Asset->GetExposedParameters();
	TArrayView<const FNiagaraVariableWithOffset> DestParameters = OverrideParameters.ReadParameterVariables();

	TArray<FNiagaraVariableBase, TInlineAllocator<8>> ParametersToRemove;
	bool bInterfacesChanged = false;

	for (int32 i=0; i < DestParameters.Num(); ++i)
	{
		const FNiagaraVariableWithOffset& DestParameter = DestParameters[i];
		const int32 DestIndex = DestParameter.Offset;
		const int32 SourceIndex = SourceUserParameterStore.IndexOf(DestParameter);
		if (SourceIndex != INDEX_NONE)
		{
			if (DestParameter.IsDataInterface())
			{
				check(OverrideParameters.GetDataInterface(DestIndex) != nullptr);
				SourceUserParameterStore.GetDataInterface(SourceIndex)->CopyTo(OverrideParameters.GetDataInterface(DestIndex));
				bInterfacesChanged = true;
			}
			else if (DestParameter.IsUObject())
			{
				OverrideParameters.SetUObject(SourceUserParameterStore.GetUObject(SourceIndex), DestIndex);
			}
			else
			{
				OverrideParameters.SetParameterData(SourceUserParameterStore.GetParameterData(SourceIndex, DestParameter.GetType()), DestIndex, DestParameter.GetSizeInBytes());
			}
		}
		else
		{
			ParametersToRemove.Add(DestParameter);
		}
	}

	for (const FNiagaraVariableBase& ParameterToRemove : ParametersToRemove)
	{
		OverrideParameters.RemoveParameter(ParameterToRemove);
	}

	if (bInterfacesChanged)
	{
		OverrideParameters.OnInterfaceChange();
	}

	OverrideParameters.Rebind();
}

bool UNiagaraComponent::IsLocalPlayerEffect()const
{
	return GetForceLocalPlayerEffect() || FNiagaraWorldManager::IsComponentLocalPlayerLinked(this);
}

#if WITH_EDITOR

void UNiagaraComponent::UpgradeDeprecatedParameterOverrides()
{
	if (EditorOverridesValue_DEPRECATED.Num() == 0)
	{
		return;
	}

	OverrideParameters.SanityCheckData();
	PostLoadNormalizeOverrideNames();
	
	TArray<FNiagaraVariable> UserParameters;
	OverrideParameters.GetUserParameters(UserParameters);

	for (const TPair<FName, bool>& Pair : EditorOverridesValue_DEPRECATED)
	{
		FNiagaraVariable* Found = UserParameters.FindByPredicate([Pair](const FNiagaraVariable& Var)
		{
			return Var.GetName() == Pair.Key;
		});

		if (Found != nullptr)
		{
			FNiagaraVariant StoredValue = GetParameterValueFromStore(*Found, OverrideParameters);

			if (StoredValue.IsValid())
			{
				SetParameterOverride(*Found, StoredValue);
			}
		}
	}

	EditorOverridesValue_DEPRECATED.Empty();
}

void UNiagaraComponent::EnsureOverrideParametersConsistent() const
{
	if (Asset == nullptr)
	{
		return;
	}
	
	TArray<FNiagaraVariable> UserParameters;
	Asset->GetExposedParameters().GetUserParameters(UserParameters);
	
	for (const FNiagaraVariable& Key : UserParameters)
	{
		FNiagaraVariant OverrideValue = FindParameterOverride(Key);
		if (OverrideValue.IsValid())
		{
			if (Key.IsDataInterface())
			{
				const UNiagaraDataInterface* ActualDI = OverrideParameters.GetDataInterface(Key);
				const UNiagaraDataInterface* VariantDI = OverrideValue.GetDataInterface();
				if (ActualDI != nullptr && VariantDI != nullptr)
				{
					// Null DIs are logged elsewhere so we don't need to raise the issue here also
					ensureAlways(VariantDI->Equals(ActualDI));
				}
			}
			else if (Key.IsUObject())
			{
				TObjectPtr<const UObject> ActualObj = OverrideParameters.GetUObject(Key);
				if (ActualObj != nullptr)
				{
					ensureAlways(OverrideValue.GetUObject() == ActualObj);
				}
			}
			else if (Key.GetType() == FNiagaraTypeDefinition::GetPositionDef())
			{
				const FVector* ActualData = OverrideParameters.GetPositionParameterValue(Key.GetName());
				if (ActualData != nullptr)
				{
					ensureAlways(FMemory::Memcmp(ActualData, OverrideValue.GetBytes(), sizeof(FVector)) == 0);
				}
			}
			else
			{
				TArray<uint8> DataValue;
				DataValue.AddUninitialized(Key.GetSizeInBytes());
				if (OverrideParameters.CopyParameterData(Key, DataValue.GetData()))
				{
					ensureAlways(FMemory::Memcmp(DataValue.GetData(), OverrideValue.GetBytes(), Key.GetSizeInBytes()) == 0);
				}
			}
		}
	}
}

void UNiagaraComponent::ApplyOverridesToParameterStore()
{
	if (!IsTemplate())
	{
		if (UNiagaraComponent* Archetype = Cast<UNiagaraComponent>(GetArchetype()))
		{
			TemplateParameterOverrides = Archetype->TemplateParameterOverrides;
		}
	}

	for (const auto& Pair : TemplateParameterOverrides)
	{ 
		if (!FNiagaraUserRedirectionParameterStore::IsUserParameter(Pair.Key))
		{
			continue;
		}

		const int32* ExistingParam = OverrideParameters.FindParameterOffset(Pair.Key);
		if (ExistingParam != nullptr)
		{
			SetOverrideParameterStoreValue(Pair.Key, Pair.Value);
		}
	}

	if (!IsTemplate())
	{
		for (const auto& Pair : InstanceParameterOverrides)
		{ 
			if (!FNiagaraUserRedirectionParameterStore::IsUserParameter(Pair.Key))
			{
				continue;
			}

			const int32* ExistingParam = OverrideParameters.FindParameterOffset(Pair.Key);
			if (ExistingParam != nullptr)
			{
				SetOverrideParameterStoreValue(Pair.Key, Pair.Value);
			}
		}
	}

	EnsureOverrideParametersConsistent();
}

void UNiagaraComponent::FixDataInterfaceOuters()
{
	const EObjectFlags DataInterfaceObjectFlags = UNiagaraDataInterface::BuildObjectFlagsForOwner(this, RF_Transactional);

	auto FixParameterOverrides = [this, DataInterfaceObjectFlags](TMap<FNiagaraVariableBase, FNiagaraVariant>& ParameterOverrides, TMap<FNiagaraVariableBase, UNiagaraDataInterface*>& OutFixedDataInterfaces)
	{
		for (TPair<FNiagaraVariableBase, FNiagaraVariant>& VariableValuePair : ParameterOverrides)
		{
			if (VariableValuePair.Key.IsDataInterface() && VariableValuePair.Value.GetDataInterface() != nullptr)
			{
				if (VariableValuePair.Value.GetDataInterface()->GetOuter() != this)
				{
					UNiagaraDataInterface* FixedDataInterface = NewObject<UNiagaraDataInterface>(this, VariableValuePair.Value.GetDataInterface()->GetClass(), NAME_None, DataInterfaceObjectFlags);
					VariableValuePair.Value.GetDataInterface()->CopyTo(FixedDataInterface);
					VariableValuePair.Value.SetDataInterface(FixedDataInterface);
					OutFixedDataInterfaces.Add(VariableValuePair.Key, FixedDataInterface);
				}
			}
		}
	};

	// Fix data interfaces in the template and instance overrides and collect the replacements.
	TMap<FNiagaraVariableBase, UNiagaraDataInterface*> VariableToFixedDataInterfaces;
	FixParameterOverrides(TemplateParameterOverrides, VariableToFixedDataInterfaces);
	FixParameterOverrides(InstanceParameterOverrides, VariableToFixedDataInterfaces);

	// Update data interfaces in use in the override parameters using the replacement map.
	// Note: the IndexOf for the data interface can return an index OOB, this will be fixed during SanityCheckData in PostLoad so we must guard against reading OOB here
	for (TPair<FNiagaraVariableBase, UNiagaraDataInterface*>& VariableFixedDataInterfacePair : VariableToFixedDataInterfaces)
	{
		int32 OverrideParametersIndex = OverrideParameters.IndexOf(VariableFixedDataInterfacePair.Key);
		if (OverrideParameters.GetDataInterfaces().IsValidIndex(OverrideParametersIndex))
		{
			OverrideParameters.SetDataInterface(VariableFixedDataInterfacePair.Value, OverrideParametersIndex);
		}
	}

	// Fix any additional data interfaces in the override parameters which are still incorrect.
	for (int32 i = 0; i < OverrideParameters.Num(); i++)
	{
		UNiagaraDataInterface* OverrideParameterDataInterface = OverrideParameters.GetDataInterface(i);
		if (OverrideParameterDataInterface != nullptr && OverrideParameterDataInterface->GetOuter() != this)
		{
			UNiagaraDataInterface* FixedDataInterface = NewObject<UNiagaraDataInterface>(this, OverrideParameterDataInterface->GetClass(), NAME_None, DataInterfaceObjectFlags);
			OverrideParameterDataInterface->CopyTo(FixedDataInterface);
			OverrideParameters.SetDataInterface(FixedDataInterface, i);
		}
	}
}

namespace NiagaraComponentImpl
{

void SetDataInterfaceFlags(UNiagaraDataInterface* DataInterface, bool bPublicDataInterfaces)
{
	if (DataInterface != nullptr)
	{
		if (DataInterface->HasAnyFlags(RF_Public) != bPublicDataInterfaces)
		{
			if (bPublicDataInterfaces)
			{
				DataInterface->SetFlags(RF_Public);
			}
			else
			{
				DataInterface->ClearFlags(RF_Public);
			}
		}
	}
}

};

void UNiagaraComponent::FixDataInterfaceObjectFlags()
{
	const bool bPublicDataInterfaces = !!(UNiagaraDataInterface::BuildObjectFlagsForOwner(this) & RF_Public);

	auto FixParameterOverrides = [](TMap<FNiagaraVariableBase, FNiagaraVariant>& ParameterOverrides, bool bPublicDataInterfaces)
	{
		for (TPair<FNiagaraVariableBase, FNiagaraVariant>& VariableValuePair : ParameterOverrides)
		{
			if (VariableValuePair.Key.IsDataInterface())
			{
				NiagaraComponentImpl::SetDataInterfaceFlags(VariableValuePair.Value.GetDataInterface(), bPublicDataInterfaces);
			}
		}
	};

	// Fix data interfaces in the template and instance overrides.
	FixParameterOverrides(TemplateParameterOverrides, bPublicDataInterfaces);
	FixParameterOverrides(InstanceParameterOverrides, bPublicDataInterfaces);

	// Fix any additional data interfaces in the override parameters which are still incorrect.
	for (int32 i = 0; i < OverrideParameters.Num(); i++)
	{
		NiagaraComponentImpl::SetDataInterfaceFlags(OverrideParameters.GetDataInterface(i), bPublicDataInterfaces);
	}
}

#endif

void UNiagaraComponent::CopyParametersFromAsset(bool bResetExistingOverrideParameters)
{
	// Copy Asset Override Parameters into Component Overrides
	const FNiagaraUserRedirectionParameterStore& AssetExposedParameters = Asset->GetExposedParameters();

	FNiagaraUserRedirectionParameterStore ExistingOverrideParameters;
	Swap(ExistingOverrideParameters, OverrideParameters);
	OverrideParameters = AssetExposedParameters;
	OverrideParameters.SetOwner(this);

	// Make sure to transfer bindings over
	ExistingOverrideParameters.TransferBindings(OverrideParameters);

	// Copy Existing Variables into new parameter data
	if (bResetExistingOverrideParameters == false)
	{
		for (const FNiagaraVariableWithOffset& ExistingOverrideVar : ExistingOverrideParameters.ReadParameterVariables())
		{
			OverrideParameters.AddParameter(ExistingOverrideVar, true);
			if (ExistingOverrideVar.IsDataInterface())
			{
				UNiagaraDataInterface* ExistingDataInterface = ExistingOverrideParameters.GetDataInterface(ExistingOverrideVar.Offset);
				OverrideParameters.SetDataInterface(ExistingDataInterface, ExistingOverrideVar);
			}
			else if (ExistingOverrideVar.IsUObject())
			{
				TObjectPtr<UObject> ExistingUObject = ExistingOverrideParameters.GetUObject(ExistingOverrideVar.Offset);
				OverrideParameters.SetUObject(ExistingUObject, ExistingOverrideVar);
			}
			else
			{
				ExistingOverrideParameters.CopyParameterData(OverrideParameters, ExistingOverrideVar);
			}
		}
	}

	// DataInterfaces are unique per component so we need to make new instances if they are not already
	const EObjectFlags DataInterfaceObjectFlags = UNiagaraDataInterface::BuildObjectFlagsForOwner(this, RF_Transactional);

	for (int32 i = 0; i < AssetExposedParameters.GetDataInterfaces().Num(); ++i)
	{
		UNiagaraDataInterface* AssetDataInterface = AssetExposedParameters.GetDataInterface(i);
		UNiagaraDataInterface* OverrideDataInterface = OverrideParameters.GetDataInterface(i);
		if (!AssetDataInterface || !OverrideDataInterface)
		{
			const FNiagaraVariableBase* DIVariable = AssetExposedParameters.FindVariableFromDataInterfaceIndex(i);
			UE_LOGF(LogNiagara, Error, "null data interface found for Variable(%ls) System(%ls) will not run.", DIVariable ? *DIVariable->GetName().ToString() : TEXT("Unknown"), *GetNameSafe(Asset));
			continue;
		}

		if (AssetDataInterface != OverrideDataInterface)
		{
			// We must copy the data regardless as there is an expectation that data interfaces are always reset to the original state
			AssetDataInterface->CopyTo(OverrideDataInterface);
		}
		else
		{
			OverrideDataInterface = NewObject<UNiagaraDataInterface>(this, AssetDataInterface->GetClass(), NAME_None, DataInterfaceObjectFlags);
			AssetDataInterface->CopyTo(OverrideDataInterface);
			OverrideParameters.SetDataInterface(OverrideDataInterface, i);
		}
	}
}

void UNiagaraComponent::SynchronizeWithSourceSystem()
{
	// Synchronizing parameters will create new data interface objects and if the old data
	// interface objects are currently being used by a simulation they may be destroyed due to garbage
	// collection, so preemptively kill the instance here.
	DestroyInstanceNotComponent();

	//TODO: Look through params in system in "Owner" namespace and add to our parameters.
	if (Asset == nullptr)
	{
#if WITH_EDITORONLY_DATA
		OverrideParameters.Empty();
		EditorOverridesValue_DEPRECATED.Empty();
		CacheAndClearEditorParameterOverrides();

		OnSynchronizedWithAssetParametersDelegate.Broadcast();
#endif
		return;
	}

#if WITH_EDITOR
	CacheAndClearEditorParameterOverrides();
	CopyParametersFromAsset();
	CopyCachedEditorParameterOverrides();
	ApplyOverridesToParameterStore();
#endif

	OverrideParameters.Rebind();

#if WITH_EDITORONLY_DATA
	OnSynchronizedWithAssetParametersDelegate.Broadcast();
#endif
}

#if WITH_EDITORONLY_DATA
void FixInvalidDataInterfaceOverrides(TMap<FNiagaraVariableBase, FNiagaraVariant>& ParameterOverrides, const FString& OverrideSource, UNiagaraComponent* OwningComponent)
{
	const EObjectFlags DataInterfaceObjectFlags = UNiagaraDataInterface::BuildObjectFlagsForOwner(OwningComponent, RF_Transactional);

	for (TPair<FNiagaraVariableBase, FNiagaraVariant>& VariableValuePair : ParameterOverrides)
	{
		FNiagaraVariableBase& Variable = VariableValuePair.Key;
		FNiagaraVariant& Value = VariableValuePair.Value;
		if (Variable.IsDataInterface())
		{
			UNiagaraDataInterface* DataInterface = Value.GetDataInterface();
			if (DataInterface == nullptr)
			{
				UE_LOGF(LogNiagara, Warning, "Replaced null user parameter data interface with it's default.  Component: %ls Override Source: %ls Parameter Name: %ls.", *OwningComponent->GetPathName(), *OverrideSource, *Variable.GetName().ToString());
			}
			else if (DataInterface->GetClass() != Variable.GetType().GetClass())
			{
				UE_LOGF(LogNiagara, Warning, "Replaced invalid class '%ls' user parameter data interface '%ls' with it's default.  Component: %ls Override Source: %ls Parameter Name: %ls.", *DataInterface->GetClass()->GetName(), *Variable.GetType().GetClass()->GetName(), *OwningComponent->GetPathName(), *OverrideSource, *Variable.GetName().ToString());
				DataInterface = nullptr;
			}
			if (DataInterface == nullptr)
			{
				DataInterface = NewObject<UNiagaraDataInterface>(OwningComponent, Variable.GetType().GetClass(), NAME_None, DataInterfaceObjectFlags);
				if (UNiagaraSystem* NiagaraSystem = OwningComponent->GetAsset())
				{
					const int32* DataInterfaceOffset = NiagaraSystem->GetExposedParameters().FindParameterOffset(Variable);
					UNiagaraDataInterface* SourceDataInterface = DataInterfaceOffset ? NiagaraSystem->GetExposedParameters().GetDataInterface(*DataInterfaceOffset) : nullptr;
					if (DataInterfaceOffset != nullptr)
					{
						SourceDataInterface->CopyTo(DataInterface);
					}
				}
				Value.SetDataInterface(DataInterface);
			}
		}
	}
}

void UNiagaraComponent::FixInvalidUserParameterOverrideData()
{
	const EObjectFlags DataInterfaceObjectFlags = UNiagaraDataInterface::BuildObjectFlagsForOwner(this, RF_Transactional);

	FixInvalidDataInterfaceOverrides(TemplateParameterOverrides, TEXT("Template"), this);
	FixInvalidDataInterfaceOverrides(InstanceParameterOverrides, TEXT("Instance"), this);

	TArray<FNiagaraVariable> ExistingVars;
	TArray<FNiagaraVariable> SourceVars;
	OverrideParameters.GetParameters(ExistingVars);
	Asset->GetExposedParameters().GetParameters(SourceVars);

	// check if one of the vector parameters was changed to a position type
	for (FNiagaraVariable& ExistingVar : ExistingVars)
	{
		if (ExistingVar.GetType() == FNiagaraTypeDefinition::GetVec3Def() && !SourceVars.Contains(ExistingVar))
		{
			FNiagaraVariable PositionVar(FNiagaraTypeDefinition::GetPositionDef(), ExistingVar.GetName());
			if (SourceVars.Contains(PositionVar))
			{
				OverrideParameters.ChangeParameterType(ExistingVar, FNiagaraTypeDefinition::GetPositionDef());

				auto ConvertOverrideToPosition = [&ExistingVar, &PositionVar](auto & Params)
				{
					FNiagaraVariant VariantData;
					if (Params.RemoveAndCopyValue(ExistingVar, VariantData))
					{
						// Convert the old float data to doubles.
						check(VariantData.GetNumBytes() == sizeof(FVector3f));
						FVector3f ExistingData = *((FVector3f*)(VariantData.GetBytes()));						
						FVector NewData(ExistingData);
						VariantData.SetBytes((uint8*)&NewData, sizeof(NewData));
						Params.Add(PositionVar, VariantData);
					}
				};

				ConvertOverrideToPosition(InstanceParameterOverrides);
				ConvertOverrideToPosition(TemplateParameterOverrides);
				
				ExistingVar = PositionVar;
			}
		}
		// Check for invalid or missing data interfaces
		else if (ExistingVar.IsDataInterface())
		{
			UNiagaraDataInterface* DataInterface = OverrideParameters.GetDataInterface(ExistingVar);
			if (DataInterface == nullptr)
			{
				UE_LOGF(LogNiagara, Log, "ParameterStore has null data interface '%ls' for variable '%ls' asset '%ls', making a default.", *ExistingVar.GetType().GetClass()->GetName(), *ExistingVar.GetName().ToString(), *GetNameSafe(Asset));
			}
			else if (DataInterface->GetClass() != ExistingVar.GetType().GetClass())
			{
				UE_LOGF(LogNiagara, Log, "ParameterStore has invalid type '%ls' data interface '%ls' for variable '%ls' asset '%ls', making a default.", *DataInterface->GetClass()->GetName(), *ExistingVar.GetType().GetClass()->GetName(), *ExistingVar.GetName().ToString(), *GetNameSafe(Asset));
				DataInterface = nullptr;
			}

			if (DataInterface == nullptr)
			{
				DataInterface = NewObject<UNiagaraDataInterface>(this, ExistingVar.GetType().GetClass(), NAME_None, DataInterfaceObjectFlags);
				if ( SourceVars.Contains(ExistingVar) )
				{
					if ( UNiagaraDataInterface* SourceDataInterface = Asset->GetExposedParameters().GetDataInterface(ExistingVar) )
					{
						SourceDataInterface->CopyTo(DataInterface);
					}
				}
				OverrideParameters.SetDataInterface(DataInterface, ExistingVar);
			}
		}
	}
}
#endif

void UNiagaraComponent::AssetExposedParametersChanged()
{
	if (HasAnyFlags(RF_NeedPostLoad))
	{
		// Since we register the change delegate in multiple places, it's possible that this is called before the component was able to go through PostLoad().
		// In that case we skip the sync with the system, as that happens in PostLoad anyways.
		return;
	}

#if WITH_EDITOR
	// Some owner actors might have destroy on finish enabled, if we were to tweak parameters this would trigger completion and destroy the actor
	// Lens effects for example set this making it impossible to tweak them without storing and restoring this information
	TWeakObjectPtr<ANiagaraActor> WeakNiagaraActor = Cast<ANiagaraActor>(GetOwner());
	bool bDestroyOnSystemFinish = false;
	if (ANiagaraActor* NiagaraActor = WeakNiagaraActor.Get())
	{
		if (NiagaraActor->GetDestroyOnSystemFinish())
		{
			bDestroyOnSystemFinish = true;
			NiagaraActor->SetDestroyOnSystemFinish(false);
		}
	}
#endif

	const bool bCurrentlyActive = IsActive();

	SynchronizeWithSourceSystem();
#if WITH_EDITOR
	if ( !GIsTransacting )
#endif
	{
		if (bCurrentlyActive)
		{
			// only reactivate systems that were running before
			ReinitializeSystem();
		}

#if WITH_EDITOR
		if (bDestroyOnSystemFinish)
		{
			if (ANiagaraActor* NiagaraActor = WeakNiagaraActor.Get())
			{
				NiagaraActor->SetDestroyOnSystemFinish(true);
			}
		}
#endif
	}
}

#if WITH_EDITOR

bool UNiagaraComponent::HasParameterOverride(const FNiagaraVariableBase& InKey) const 
{
	FNiagaraVariableBase UserVariable = InKey;

	if (Asset)
	{
		if (!Asset->GetExposedParameters().RedirectUserVariable(UserVariable))
		{
			return false;
		}
	}
	else if (!FNiagaraUserRedirectionParameterStore::IsUserParameter(UserVariable))
	{
		return false;
	}

	if (IsTemplate())
	{
		const FNiagaraVariant* ThisValue = TemplateParameterOverrides.Find(UserVariable);

		const FNiagaraVariant* ArchetypeValue = nullptr;
		if (const UNiagaraComponent* Archetype = Cast<UNiagaraComponent>(GetArchetype()))
		{
			ArchetypeValue = Archetype->TemplateParameterOverrides.Find(UserVariable);
		}

		if (ThisValue != nullptr && ArchetypeValue != nullptr)
		{
			// exists in both, check values
			return *ThisValue != *ArchetypeValue;
		}
		else if (ThisValue != nullptr || ArchetypeValue != nullptr)
		{
			// either added or removed in this
			return true;
		}
	}
	else
	{
		if (InstanceParameterOverrides.Contains(UserVariable))
		{
			return true;
		}
	}

	return false;
}

FNiagaraVariant UNiagaraComponent::FindParameterOverride(const FNiagaraVariableBase& InKey) const
{
	if (Asset == nullptr)
	{
		return FNiagaraVariant();
	}

	FNiagaraVariableBase UserVariable = InKey;

	const FNiagaraUserRedirectionParameterStore& ParameterStore = Asset->GetExposedParameters();

	if (!ParameterStore.RedirectUserVariable(UserVariable))
	{
		return FNiagaraVariant();
	}

	if (ParameterStore.FindParameterOffset(UserVariable) == nullptr)
	{
		return FNiagaraVariant();
	}

	if (!IsTemplate())
	{
		const FNiagaraVariant* Value = InstanceParameterOverrides.Find(UserVariable);
		if (Value != nullptr)
		{
			return *Value;
		}
	}

	{
		const FNiagaraVariant* Value = TemplateParameterOverrides.Find(UserVariable);
		if (Value != nullptr)
		{
			return *Value;
		}
	}

	return FNiagaraVariant();
}

FNiagaraVariant UNiagaraComponent::GetCurrentParameterValue(const FNiagaraVariableBase& InKey) const
{
	FNiagaraVariableBase UserVariable = InKey;
	if (OverrideParameters.RedirectUserVariable(UserVariable) == false)
	{
		return FNiagaraVariant();
	}

	int32 ParameterOffset = OverrideParameters.IndexOf(UserVariable);
	if (ParameterOffset == INDEX_NONE)
	{
		return FNiagaraVariant();
	}

	if (InKey.GetType().IsDataInterface())
	{
		return FNiagaraVariant(OverrideParameters.GetDataInterface(ParameterOffset));
	}
	if (InKey.GetType().IsUObject())
	{
		return FNiagaraVariant(OverrideParameters.GetUObject(ParameterOffset));
	}
	if (InKey.GetType() == FNiagaraTypeDefinition::GetPositionDef())
	{
		FNiagaraVariable RedirectedVar = InKey;
		OverrideParameters.RedirectUserVariable(RedirectedVar);
		const FVector* Value = OverrideParameters.GetPositionParameterValue(RedirectedVar.GetName());
		return Value == nullptr ? FNiagaraVariant() : FNiagaraVariant(Value, sizeof(FVector));
	}

	TArray<uint8> DataValue;
	DataValue.AddUninitialized(UserVariable.GetSizeInBytes());
	OverrideParameters.CopyParameterData(UserVariable, DataValue.GetData());
	return FNiagaraVariant(DataValue);
}

void UNiagaraComponent::CacheAndClearEditorParameterOverrides()
{
	auto CacheOverrides = [](const TMap<FNiagaraVariableBase, FNiagaraVariant>& SourceOverrides, TMap<FNiagaraVariableBase, FNiagaraVariant>& CacheOverrides)
	{
		for (const TPair<FNiagaraVariableBase, FNiagaraVariant>& SourcePair : SourceOverrides)
		{
			CacheOverrides.FindOrAdd(SourcePair.Key) = SourcePair.Value;
		}
	};
	if (IsTemplate())
	{
		CacheOverrides(TemplateParameterOverrides, TemplateParameterOverridesCache);
		TemplateParameterOverrides.Empty();
	}
	else
	{
		CacheOverrides(InstanceParameterOverrides, InstanceParameterOverridesCache);
		InstanceParameterOverrides.Empty();
	}
}

void UNiagaraComponent::CopyCachedEditorParameterOverrides()
{
	auto UpdateOverridesFromCache = [](const TArrayView<const FNiagaraVariableWithOffset>& OverrideParameterVariables, const TMap<FNiagaraVariableBase, FNiagaraVariant>& CacheOverrides, TMap<FNiagaraVariableBase, FNiagaraVariant>& TargetOverrides)
	{
		for (const FNiagaraVariableWithOffset& OverrideParameterVariable : OverrideParameterVariables)
		{
			const FNiagaraVariant* CachedOverrideValue = CacheOverrides.Find(OverrideParameterVariable);
			if (CachedOverrideValue != nullptr)
			{
				if (TargetOverrides.Contains(OverrideParameterVariable) == false)
				{
					TargetOverrides.Add(OverrideParameterVariable, *CachedOverrideValue);
				}
			}
		}
	};
	TArrayView<const FNiagaraVariableWithOffset> OverrideParameterVariables = OverrideParameters.ReadParameterVariables();
	if (IsTemplate())
	{
		UpdateOverridesFromCache(OverrideParameterVariables, TemplateParameterOverridesCache, TemplateParameterOverrides);
	}
	else
	{
		UpdateOverridesFromCache(OverrideParameterVariables, InstanceParameterOverridesCache, InstanceParameterOverrides);
	}
}

void UNiagaraComponent::SetOverrideParameterStoreValue(const FNiagaraVariableBase& InKey, const FNiagaraVariant& InValue)
{
	if (InKey.IsDataInterface())
	{
		UNiagaraDataInterface* OriginalDI = InValue.GetDataInterface();
		if (OriginalDI == nullptr)
		{
			// There is an edge case where the data interface inside FNiagaraVariant has been garbage collected, it's unclear on a consistent repro right now.
			// Rather than crashing make a default and log a warning for investigation (see UE-314893).
			UClass* DataInterfaceClass = InKey.GetType().GetClass();
			check(DataInterfaceClass);
			UE_LOGF(LogNiagara, Log, "SetOverrideParameterStoreValue has null data interface '%ls' for variable '%ls' asset '%ls', making a default.", *DataInterfaceClass->GetName(), *InKey.GetName().ToString(), *GetNameSafe(Asset));

			const EObjectFlags DataInterfaceObjectFlags = UNiagaraDataInterface::BuildObjectFlagsForOwner(this, RF_Transactional);
			OriginalDI = NewObject<UNiagaraDataInterface>(this, DataInterfaceClass, NAME_None, DataInterfaceObjectFlags);
		}
		UNiagaraDataInterface* DuplicatedDI = DuplicateObject(OriginalDI, this);
		OverrideParameters.SetDataInterface(DuplicatedDI, InKey);
		DuplicatedDI->SetUsedWithCPUScript(OriginalDI->IsUsedWithCPUScript());
		DuplicatedDI->SetUsedWithGPUScript(OriginalDI->IsUsedWithGPUScript());
	}
	else if (InKey.IsUObject())
	{
		OverrideParameters.SetUObject(InValue.GetUObject(), InKey);
	}
	else if (InKey.GetType() == FNiagaraTypeDefinition::GetPositionDef())
	{
		if (InValue.GetNumBytes() == sizeof(FVector))
		{
			FVector WorldPos;
			FMemory::Memcpy(&WorldPos, InValue.GetBytes(), InValue.GetNumBytes());
			OverrideParameters.SetPositionParameterValue(WorldPos, InKey.GetName(), true);
		}
		else if (InValue.GetNumBytes() == sizeof(FVector3f))
		{
			FVector3f WorldPos;
			FMemory::Memcpy(&WorldPos, InValue.GetBytes(), InValue.GetNumBytes());
			OverrideParameters.SetPositionParameterValue((FVector)WorldPos, InKey.GetName(), true);
		}
		else
		{
			checkf(false, TEXT("Invalid number of bytes for position data: %i"), InValue.GetNumBytes());
		}
	}
	else
	{
		OverrideParameters.SetParameterData(InValue.GetBytes(), InKey, true);
	}
}

void UNiagaraComponent::SetParameterOverride(const FNiagaraVariableBase& InKey, const FNiagaraVariant& InValue)
{
	if (!ensure(InValue.IsValid()))
	{
		return;
	}

	// we want to be sure we're storing data based on the fully qualified key name (i.e. taking the user redirection into account)
	FNiagaraVariableBase UserVariable = InKey;
	if (!OverrideParameters.RedirectUserVariable(UserVariable))
	{
		return;
	}

	if (IsTemplate())
	{
		TemplateParameterOverrides.Add(UserVariable, InValue);
	}
	else
	{
		InstanceParameterOverrides.Add(UserVariable, InValue);
	}

	SetOverrideParameterStoreValue(UserVariable, InValue);
}

void UNiagaraComponent::RemoveParameterOverride(const FNiagaraVariableBase& InKey)
{
	// we want to be sure we're storing data based on the fully qualified key name (i.e. taking the user redirection into account)
	FNiagaraVariableBase UserVariable = InKey;
	if (!OverrideParameters.RedirectUserVariable(UserVariable))
	{
		return;
	}

	if (!IsTemplate())
	{
		InstanceParameterOverrides.Remove(UserVariable);
		InstanceParameterOverridesCache.Remove(UserVariable);
	}
	else
	{
		TemplateParameterOverrides.Remove(UserVariable);
		TemplateParameterOverridesCache.Remove(UserVariable);

		// we are an archetype, but check if we have an archetype and inherit the value from there
		const UNiagaraComponent* Archetype = Cast<UNiagaraComponent>(GetArchetype());
		if (Archetype != nullptr)
		{
			FNiagaraVariant ArchetypeValue = Archetype->FindParameterOverride(UserVariable);
			if (ArchetypeValue.IsValid())
			{
				// defined in archetype, reset value to that
				if (UserVariable.IsDataInterface())
				{
					UNiagaraDataInterface* DataInterface = DuplicateObject(ArchetypeValue.GetDataInterface(), this);
					TemplateParameterOverrides.Add(UserVariable, FNiagaraVariant(DataInterface));
				}
				else
				{
					TemplateParameterOverrides.Add(UserVariable, ArchetypeValue);
				}
			}
		}
	}

	SynchronizeWithSourceSystem();
}

#endif

ENiagaraAgeUpdateMode UNiagaraComponent::GetAgeUpdateMode() const
{
	return AgeUpdateMode;
}

void UNiagaraComponent::SetAgeUpdateMode(ENiagaraAgeUpdateMode InAgeUpdateMode)
{
	AgeUpdateMode = InAgeUpdateMode;
}

float UNiagaraComponent::GetDesiredAge() const
{
	return DesiredAge;
}

void UNiagaraComponent::SetDesiredAge(float InDesiredAge)
{
	DesiredAge = InDesiredAge;
	bIsSeeking = false;
}

void UNiagaraComponent::SeekToDesiredAge(float InDesiredAge)
{
	DesiredAge = InDesiredAge;
	bIsSeeking = true;
}

void UNiagaraComponent::SetCanRenderWhileSeeking(bool bInCanRenderWhileSeeking)
{
	bCanRenderWhileSeeking = bInCanRenderWhileSeeking;
}

float UNiagaraComponent::GetSeekDelta() const
{
	return SeekDelta;
}

void UNiagaraComponent::SetSeekDelta(float InSeekDelta)
{
	SeekDelta = InSeekDelta;
}

bool UNiagaraComponent::GetLockDesiredAgeDeltaTimeToSeekDelta() const
{
	return bLockDesiredAgeDeltaTimeToSeekDelta;
}

void UNiagaraComponent::SetLockDesiredAgeDeltaTimeToSeekDelta(bool bLock)
{
	bLockDesiredAgeDeltaTimeToSeekDelta = bLock;
}

float UNiagaraComponent::GetMaxSimTime() const
{
	return MaxSimTime;
}

void UNiagaraComponent::SetMaxSimTime(float InMaxTime)
{
	MaxSimTime = InMaxTime;
}

void UNiagaraComponent::SetAutoDestroy(bool bInAutoDestroy)
{
	if (ensureMsgf(!bInAutoDestroy || (PoolingMethod == ENCPoolMethod::None), TEXT("Attempting to set AutoDestroy on a pooled component!  Component(%s) Asset(%s)"), *GetFullName(), GetAsset() != nullptr ? *GetAsset()->GetPathName() : TEXT("None")))
	{
		bAutoDestroy = bInAutoDestroy;
	}
}

FNiagaraSystemInstance* UNiagaraComponent::GetSystemInstance() const
{
	return SystemInstanceController ? SystemInstanceController->GetSystemInstance_Unsafe() : nullptr;
}

void UNiagaraComponent::SetPreviewLODDistance(bool bInEnablePreviewLODDistance, float InPreviewLODDistance, float InPreviewMaxDistance)
{
	bEnablePreviewLODDistance = bInEnablePreviewLODDistance;
	PreviewLODDistance = InPreviewLODDistance;

	if(SystemInstanceController)
	{
		if (bEnablePreviewLODDistance)
		{
			SystemInstanceController->SetLODDistance(PreviewLODDistance, InPreviewMaxDistance, true);
		}
		else
		{
			SystemInstanceController->ClearLODDistance();
		}
	}
}

void UNiagaraComponent::SetSimCache(UNiagaraSimCache* InSimCache, bool bResetSystem)
{
	const bool bForceReset = IsActive() && SimCache ? SimCache->GetAttributeCaptureMode() != ENiagaraSimCacheAttributeCaptureMode::All : false;

	SimCache = InSimCache;
	UpdateInstanceSoloMode();

	if (bResetSystem || bForceReset)
	{
		ReinitializeSystem();
	}
}

UNiagaraSimCache* UNiagaraComponent::GetSimCache() const
{
	return SimCache;
}

INiagaraSceneComponentUtils* UNiagaraComponent::CreateSceneComponentUtils()
{
	if (CreateSceneComponentUtilsFunction != nullptr)
	{
		return CreateSceneComponentUtilsFunction();
	}
	return new FNiagaraActorSceneComponentUtils(this);
}

void UNiagaraComponent::SetAllowScalability(bool bAllow)
{
	bAllowScalability = bAllow; 
	if (bAllow)
	{
		if ( IsActive() )
		{
			RegisterWithScalabilityManager();
		}
	}
	else
	{
		UnregisterWithScalabilityManager();
	}
}

bool UNiagaraComponent::GetAllowScalability()const
{
	return bAllowScalability;
}


void UNiagaraComponent::SetForceLocalPlayerEffect(bool bIsPlayerEffect)
{
	bForceLocalPlayerEffect = bIsPlayerEffect;
	ResolveOwnerAllowsScalability();
}

bool UNiagaraComponent::GetForceLocalPlayerEffect()const
{
	return bForceLocalPlayerEffect;
}

#if WITH_EDITOR

void UNiagaraComponent::PostLoadNormalizeOverrideNames()
{
	TMap<FName, bool> ValueMap;
	for (TPair<FName, bool> Pair : EditorOverridesValue_DEPRECATED)
	{
		FString ValueNameString = Pair.Key.ToString();
		if (ValueNameString.StartsWith(TEXT("User.")))
		{
			 ValueNameString = ValueNameString.RightChop(5);
		}

		ValueMap.Add(FName(*ValueNameString), Pair.Value);
	}
	EditorOverridesValue_DEPRECATED = ValueMap;
}

#endif // WITH_EDITOR

void UNiagaraComponent::SetAsset(UNiagaraSystem* InAsset, bool bResetExistingOverrideParameters)
{
	if (Asset == InAsset)
	{
		return;
	}

	if (FNiagaraUtilities::LogVerboseWarnings())
	{
		if ( PoolingMethod != ENCPoolMethod::None )
		{
			UE_LOGF(LogNiagara, Warning, "SetAsset called on pooled component '%ls' Before '%ls' New '%ls', pleased fix calling code to not do this.", *GetFullNameSafe(this), *GetFullNameSafe(Asset), * GetFullNameSafe(InAsset));
		}
	}

#if WITH_EDITOR
	if (Asset != nullptr && AssetExposedParametersChangedHandle.IsValid())
	{
		Asset->GetExposedParameters().RemoveOnChangedHandler(AssetExposedParametersChangedHandle);
		AssetExposedParametersChangedHandle.Reset();
	}
#endif

	DestroyCullProxy();
	UnregisterWithScalabilityManager();

	const bool bWasActive = SystemInstanceController && SystemInstanceController->GetRequestedExecutionState() == ENiagaraExecutionState::Active;

	DestroyInstanceNotComponent();

	// Set new asset, update parameters and reactivate it it was already active
	Asset = InAsset;

#if WITH_EDITOR
	if (Asset != nullptr && bResetExistingOverrideParameters)
	{
		// To mirror the cooked path we reset the parameters is the asset is valid
		OverrideParameters.Empty();
		InstanceParameterOverrides.Empty();
	}

	SynchronizeWithSourceSystem();
	if (Asset != nullptr)
	{
		AssetExposedParametersChangedHandle = Asset->GetExposedParameters().AddOnChangedHandler(
			FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraComponent::AssetExposedParametersChanged));
	}
#else
	if (Asset != nullptr)
	{
		CopyParametersFromAsset(bResetExistingOverrideParameters);
		OverrideParameters.Rebind();
	}
#endif

	if ( Asset )
	{
		// Apply system overrides to the component
		CastShadow = Asset->bOverrideCastShadow ? Asset->bCastShadow : CastShadow;
		bReceivesDecals = Asset->bOverrideReceivesDecals ? Asset->bReceivesDecals : bReceivesDecals;

		bRenderCustomDepth = Asset->bOverrideRenderCustomDepth ? Asset->bRenderCustomDepth : bRenderCustomDepth;
		CustomDepthStencilWriteMask = Asset->bOverrideCustomDepthStencilValue ? Asset->CustomDepthStencilWriteMask : CustomDepthStencilWriteMask;
		CustomDepthStencilValue = Asset->bOverrideCustomDepthStencilWriteMask ? Asset->CustomDepthStencilValue : CustomDepthStencilValue;

		TranslucencySortPriority = Asset->bOverrideTranslucencySortPriority ? Asset->TranslucencySortPriority : TranslucencySortPriority;
		TranslucencySortDistanceOffset = Asset->bOverrideTranslucencySortDistanceOffset ? Asset->TranslucencySortDistanceOffset : TranslucencySortDistanceOffset;

		// Activate
		if ( IsRegistered() )
		{
			if (bAutoActivate || bWasActive)
			{
				Activate();
			}
		}
	}
}

void UNiagaraComponent::SetOcclusionQueryMode(ENiagaraOcclusionQueryMode Mode)
{
	if (Mode != OcclusionQueryMode)
	{
		OcclusionQueryMode = Mode;
		MarkRenderStateDirty();
	}
}

void UNiagaraComponent::SetForceSolo(bool bInForceSolo)
{ 
	if (bForceSolo != bInForceSolo)
	{
		bForceSolo = bInForceSolo;
		UpdateInstanceSoloMode();
	}
}

bool UNiagaraComponent::RequiresSoloMode() const
{
	//-OPT: If parameter collection overrides are only bound into the particle scripts we can avoid forcing solo mode
	return bForceSolo || SimCache != nullptr || !FMath::IsNearlyEqual(CustomTimeDilation, 1.0f) || (Asset && Asset->HasFixedTickDelta()) || (ParameterCollectionOverrides.Num() > 0) || bAwaitingActivationDueToNotReady;
}

void UNiagaraComponent::UpdateInstanceSoloMode()
{
	const bool bRequireSoloMode = RequiresSoloMode();
	if (SystemInstanceController)
	{
		SystemInstanceController->SetForceSolo(bRequireSoloMode);
	}

	if (IsActive())
	{
		SetComponentTickEnabled(bRequireSoloMode);
	}
}

const FNiagaraReadinessSettings& UNiagaraComponent::GetReadinessSettings() const
{
	if (UNiagaraSystem* Sys = GetAsset())
	{
		return Sys->GetReadinessSettings();
	}

	static FNiagaraReadinessSettings Dummy;
	return Dummy;
}

ENiagaraEmitterImportance UNiagaraComponent::GetHighestEmitterImportance() const
{
	if (UNiagaraSystem* Sys = GetAsset())
	{
		return Sys->GetHighestEmitterImportance();
	}
	return ENiagaraEmitterImportance::Low;
}

bool UNiagaraComponent::CheckEmitterPSOStillRunning() const
{
	if (UNiagaraSystem* Sys = GetAsset())
	{
		FMaterialInterfacePSOPrecacheParamsList PSOPrecacheList;
		for (const FNiagaraEmitterHandle& EmitterHandle : Sys->GetEmitterHandles())
		{
			if (EmitterHandle.GetIsEnabled())
			{
				if (FVersionedNiagaraEmitterData* Data = EmitterHandle.GetEmitterData())
				{
					if (Data->Importance == ENiagaraEmitterImportance::Critical)
					{
						EmitterHandle.ForEachEnabledRendererWithIndex([&PSOPrecacheList](const UNiagaraRendererProperties* Properties, int32)
						{
							Properties->CollectPSOPrecacheData(nullptr, PSOPrecacheList);
						});
					}
				}
			}
		}
		for (FMaterialInterfacePSOPrecacheParams& PSOPrecacheParams : PSOPrecacheList)
		{
			if (Sys->IsLoadingPSOCache(PSOPrecacheParams.MaterialInterface))
			{
				return true;
			}
		}
	}
	return false;
}

bool UNiagaraComponent::CheckPSOCacheIsReadyToRun()
{
#if UE_WITH_PSO_PRECACHING
	const FNiagaraReadinessSettings& ReadinessSettings = GetReadinessSettings();
	ENiagaraEmitterImportance EmitterImportance = GetHighestEmitterImportance();
	bool bDelayIfPSONotReady = ReadinessSettings.PSOPrecacheSettings.DelayMode == ENiagaraSystemPSOPrecachingDelayMode::DelayActivation;
	bool bPrecachePSOs = bDelayIfPSONotReady || ReadinessSettings.PSOPrecacheSettings.PrecachingTime == ENiagaraPSOPrecachingTime::OnActivated || EmitterImportance == ENiagaraEmitterImportance::Critical;	
	if (bPrecachePSOs)
	{
		PrecachePSOs();
	
		// check if the system as a whole needs to have all pso jobs finished before it can be activated
		if (bDelayIfPSONotReady && IsPSOPrecaching())
		{
			EPSOPrecachePriority PSOPrecachePriorityBoost = ReadinessSettings.PSOPrecacheSettings.GetPriorityBoost();
			if (GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate && CheckPSOPrecachingAndBoostPriority(PSOPrecachePriorityBoost))
			{
				UE_LOGF(LogNiagara, Verbose, "Skipping Niagara Activation for UNiagaraComponent %ls (%ls) (UNiagaraSystem PSOs are still compiling)", *GetFullName(), *GetFullNameSafe(Asset));
				return false;
			}
		}
		// check if we have one or more critical emitters waiting to be activated
		else if (EmitterImportance == ENiagaraEmitterImportance::Critical)
		{
			if (GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate && CheckEmitterPSOStillRunning())
			{
				UE_LOGF(LogNiagara, Verbose, "Skipping Niagara Activation for UNiagaraComponent %ls (%ls) (PSOs are still compiling for critical emitters)", *GetFullName(), *GetFullNameSafe(Asset));
				return false;
			}
		}
	}
#endif
	
	return true;
}

void UNiagaraComponent::PrecachePSOs()
{
	// the asset pso cache should only run once, but gives the particle system the chance to do some setup for the regular precaching
	PrecacheAssetPSOs(Asset);
	Super::PrecachePSOs();
}

void UNiagaraComponent::SetGpuComputeDebug(bool bEnableDebug)
{
	if (bEnableGpuComputeDebug != bEnableDebug)
	{
		bEnableGpuComputeDebug = bEnableDebug;
		if (SystemInstanceController.IsValid())
		{
			SystemInstanceController->SetGpuComputeDebug(bEnableGpuComputeDebug != 0);
		}
	}
}

void UNiagaraComponent::SetCustomTimeDilation(float Dilation)
{
	if ( !FMath::IsNearlyEqual(CustomTimeDilation, Dilation) )
	{
		CustomTimeDilation = Dilation;
		UpdateInstanceSoloMode();
	}
}

void UNiagaraComponent::SetAutoAttachmentParameters(USceneComponent* Parent, FName SocketName, EAttachmentRule LocationRule, EAttachmentRule RotationRule, EAttachmentRule ScaleRule)
{
	AutoAttachParent = Parent;
	AutoAttachSocketName = SocketName;
	AutoAttachLocationRule = LocationRule;
	AutoAttachRotationRule = RotationRule;
	AutoAttachScaleRule = ScaleRule;
}


void UNiagaraComponent::CancelAutoAttachment(bool bDetachFromParent)
{
	if (bAutoManageAttachment)
	{
		if (bDidAutoAttach)
		{
			// Restore relative transform from before attachment. Actual transform will be updated as part of DetachFromParent().
			SetRelativeLocation_Direct(SavedAutoAttachRelativeLocation);
			SetRelativeRotation_Direct(SavedAutoAttachRelativeRotation);
			SetRelativeScale3D_Direct(SavedAutoAttachRelativeScale3D);
			bDidAutoAttach = false;
		}

		if (bDetachFromParent)
		{
			UWorld* World = GetWorld();
			if (!World || World->IsGameWorld())
			{
				DetachFromComponent(FDetachmentTransformRules(EDetachmentRule::KeepRelative, /*bCallModify=*/ false));
			}
		}
	}
}

