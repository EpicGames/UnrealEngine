// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraComponentLeakDetector.h"
#include "NiagaraDebugHud.h"
#include "NiagaraEffectType.h"
#include "NiagaraSystem.h"
#include "NiagaraWorldManager.h"
#include "UObject/UObjectIterator.h"

#if WITH_NIAGARA_LEAK_DETECTOR

DEFINE_LOG_CATEGORY_STATIC(LogNiagaraLeakDetector, Log, All);

namespace NiagaraComponentLeakDetectorPrivate
{
	enum class EReportLeakType
	{
		Never = 0,
		Immediate,
		PostGC,
	};

	static bool		GEnabled = false;
	static int32	GReportActiveLeaks = (int32)EReportLeakType::Immediate;
	static int32	GReportTotalLeaks = (int32)EReportLeakType::PostGC;
	static int32	GReportScalabilityIssues = (int32)EReportLeakType::Immediate;
	static int32	GGCReport = 1;
	static float	GTickDeltaSeconds = 1.0f;
	static int32	GGrowthCountThreshold = 16;
	static float	GDebugMessageTime = 5.0f;

	static FAutoConsoleVariableRef CVarEnabled(
		TEXT("fx.Niagara.LeakDetector.Enabled"),
		GEnabled,
		TEXT("Enables or disables the leak detector."),
		ECVF_Default
	);

	static FAutoConsoleVariableRef CVarTickDeltaSeconds(
		TEXT("fx.Niagara.LeakDetector.TickDeltaSeconds"),
		GTickDeltaSeconds,
		TEXT("The time in seconds that must pass before we sample the component information."),
		ECVF_Default
	);

	static FAutoConsoleVariableRef CVarGrowthCountThreshold(
		TEXT("fx.Niagara.LeakDetector.GrowthCountThreshold"),
		GGrowthCountThreshold,
		TEXT("We need to see growth this many times without a drop in count before we consider it a leak."),
		ECVF_Default
	);

	static FAutoConsoleVariableRef CVarReportActiveLeaks(
		TEXT("fx.Niagara.LeakDetector.ReportActiveLeaks"),
		GReportActiveLeaks,
		TEXT("How do we report active components leaks?")
		TEXT("0 - Never report.")
		TEXT("1 - Report immediately. (default)")
		TEXT("2 - Report on GC.\n"),
		ECVF_Default
	);

	static FAutoConsoleVariableRef CVarReportTotalLeaks(
		TEXT("fx.Niagara.LeakDetector.ReportTotalLeaks"),
		GReportTotalLeaks,
		TEXT("How do we report total components leaks?")
		TEXT("0 - Never report.")
		TEXT("1 - Report immediately.")
		TEXT("2 - Report on GC. (default)\n"),
		ECVF_Default
	);

	static FAutoConsoleVariableRef CVarDebugMessageTime(
		TEXT("fx.Niagara.LeakDetector.DebugMessageTime"),
		GDebugMessageTime,
		TEXT("Time we display the debug message for on screen."),
		ECVF_Default
	);

	void AddLeakWarning(UWorld* World, FName MessageKey, const FString& Message)
	{
		UE_LOGF(LogNiagaraLeakDetector, Warning, "%ls", *Message);
#if WITH_NIAGARA_DEBUGGER
		if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World))
		{
			if (FNiagaraDebugHud* DebugHud = WorldManager->GetNiagaraDebugHud())
			{
				DebugHud->AddMessage(MessageKey, FNiagaraDebugMessage(ENiagaraDebugMessageType::Warning, Message, GDebugMessageTime));
			}
		}
#endif
	}

	void ReportLeak(EReportLeakType ReportLeakType, UWorld* World, FName SystemName, FNiagaraComponentLeakDetector::FSystemData& SystemData)
	{
		using namespace NiagaraComponentLeakDetectorPrivate;

		if (int32(ReportLeakType) == GReportTotalLeaks)
		{
			if (!SystemData.bTotalHasWarned)
			{
				if (SystemData.TotalShrinkCounter == 0 && SystemData.TotalGrowthCounter >= GGrowthCountThreshold)
				{
					SystemData.bTotalHasWarned = true;
					AddLeakWarning(World, SystemName, FString::Printf(TEXT("Potential componment leak System(%s) (Total:%d), please investigate."), *SystemName.ToString(), SystemData.TotalPrevCount));
				}
			}
			else if (!SystemData.bTotalFalseWarning && (SystemData.TotalShrinkCounter > 0))
			{
				SystemData.bTotalFalseWarning = true;
				AddLeakWarning(World, SystemName, FString::Printf(TEXT("Potential invalid component leak reported for System(%s) (Total:%d)."), *SystemName.ToString(), SystemData.TotalPrevCount));
			}
		}

		if (int32(ReportLeakType) == GReportActiveLeaks)
		{
			if (!SystemData.bActiveHasWarned)
			{
				if (SystemData.ActiveShrinkCounter == 0 && SystemData.ActiveGrowthCounter >= GGrowthCountThreshold)
				{
					SystemData.bActiveHasWarned = true;
					AddLeakWarning(World, SystemName, FString::Printf(TEXT("Potential Active Component leak System(%s) (Active:%d), please investigate."), *SystemName.ToString(), SystemData.ActivePrevCount));
				}
			}
			else if (!SystemData.bActiveFalseWarning && (SystemData.ActiveShrinkCounter > 0))
			{
				SystemData.bActiveFalseWarning = true;
				AddLeakWarning(World, SystemName, FString::Printf(TEXT("Potential invalid active component leak reported for System(%s) (Active: %d)."), *SystemName.ToString(), SystemData.ActivePrevCount));
			}
		}

		if ( int32(ReportLeakType) == GReportScalabilityIssues )
		{
			if (!SystemData.bScalabilityHasWarned && SystemData.ScalabilityAllowedActiveExecState > 0 && SystemData.ScalabilityMaxActiveExecState > SystemData.ScalabilityAllowedActiveExecState )
			{
				SystemData.bScalabilityHasWarned = true;
				AddLeakWarning(World, SystemName, FString::Printf(TEXT("Scalability instance count limit blown for System(%s) (Active: %d) (Limit: %d)."), *SystemName.ToString(), SystemData.ScalabilityMaxActiveExecState, SystemData.ScalabilityAllowedActiveExecState));
			}
		}
	}
}

void FNiagaraComponentLeakDetector::Tick(UWorld* World)
{
	using namespace NiagaraComponentLeakDetectorPrivate;

	if (!GEnabled)
	{
		return;
	}

	const double CurrentTime = FPlatformTime::Seconds();
	if (CurrentTime < NextUpdateTime)
	{
		return;
	}
	NextUpdateTime = CurrentTime + double(GTickDeltaSeconds);

	for (TObjectIterator<UNiagaraComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		UNiagaraComponent* Component = *ComponentIt;
		UNiagaraSystem* System = Component ? Component->GetAsset() : nullptr;
		if (!System)
		{
			continue;
		}

		if ( Component->GetWorld() != World || Component->PoolingMethod == ENCPoolMethod::FreeInPool)
		{
			continue;
		}

		const FName SystemName = System->GetFName();
		FSystemData* SystemData = PerSystemData.Find(SystemName);
		if (!SystemData)
		{
			UNiagaraEffectType* EffectType = System->GetEffectType();

			SystemData = &PerSystemData.Add(SystemName);
			SystemData->ScalabilityAllowedActiveExecState = EffectType ? EffectType->GetActiveSystemScalabilitySettings().MaxSystemInstances : 0;
		}

		const bool bIsComponentActive = Component->IsActive();
		const bool bIsExecutionActive = bIsComponentActive && (Component->GetExecutionState() == ENiagaraExecutionState::Active);

		SystemData->TotalCurrCount += 1;
		SystemData->ActiveCurrCount += bIsComponentActive ? 1 : 0;
		SystemData->ScalabilityCurrActiveExecState += bIsExecutionActive ? 1 : 0;
	}

	for (auto SystemDataIt=PerSystemData.CreateIterator(); SystemDataIt; ++SystemDataIt)
	{
		const FName SystemName = SystemDataIt.Key();
		FSystemData& SystemData = SystemDataIt.Value();

		// Monitor total count
		SystemData.TotalGrowthCounter += SystemData.TotalCurrCount > SystemData.TotalPrevCount ? 1 : 0;
		SystemData.TotalShrinkCounter += SystemData.TotalCurrCount < SystemData.TotalPrevCount ? 1 : 0;
		SystemData.TotalPrevCount = SystemData.TotalCurrCount;
		SystemData.TotalCurrCount = 0;

		// Monitor active count
		SystemData.ActiveGrowthCounter += SystemData.ActiveCurrCount > SystemData.ActivePrevCount ? 1 : 0;
		SystemData.ActiveShrinkCounter += SystemData.ActiveCurrCount < SystemData.ActivePrevCount ? 1 : 0;
		SystemData.ActivePrevCount = SystemData.ActiveCurrCount;
		SystemData.ActiveCurrCount = 0;

		// Monitor for scalability issues
		SystemData.ScalabilityMaxActiveExecState = FMath::Max(SystemData.ScalabilityMaxActiveExecState, SystemData.ScalabilityCurrActiveExecState);
		SystemData.ScalabilityCurrActiveExecState = 0;

		// Report any leaks
		ReportLeak(EReportLeakType::Immediate, World, SystemName, SystemData);
	}
}

void FNiagaraComponentLeakDetector::ReportLeaks(UWorld* World)
{
	using namespace NiagaraComponentLeakDetectorPrivate;

	if (!GEnabled || GGCReport == 0)
	{
		return;
	}

	for (auto SystemDataIt=PerSystemData.CreateIterator(); SystemDataIt; ++SystemDataIt)
	{
		const FName SystemName = SystemDataIt.Key();
		FSystemData& SystemData = SystemDataIt.Value();
		ReportLeak(EReportLeakType::PostGC, World, SystemName, SystemData);
	}
}

#endif //WITH_NIAGARA_LEAK_DETECTOR
