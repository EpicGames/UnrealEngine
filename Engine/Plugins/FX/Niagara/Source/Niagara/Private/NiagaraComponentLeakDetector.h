// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraComponent.h"

#if WITH_NIAGARA_LEAK_DETECTOR

// Very simple leak detector, will be revised over time.
class FNiagaraComponentLeakDetector
{
public:
	struct FSystemData
	{
		static constexpr uint32 HistoryLength = 4;

		bool	bTotalHasWarned = false;
		bool	bTotalFalseWarning = false;
		int32	TotalGrowthCounter = 0;
		int32	TotalShrinkCounter = 0;
		uint32	TotalCurrCount = 0;
		uint32	TotalPrevCount = 0;

		bool	bActiveHasWarned = false;
		bool	bActiveFalseWarning = false;
		int32	ActiveGrowthCounter = 0;
		int32	ActiveShrinkCounter = 0;
		uint32	ActiveCurrCount = 0;
		uint32	ActivePrevCount = 0;
		
		bool	bScalabilityHasWarned = false;
		int32	ScalabilityCurrActiveExecState = 0;
		int32	ScalabilityMaxActiveExecState = 0;
		int32	ScalabilityAllowedActiveExecState = 0;
	};

public:
	void Tick(UWorld* World);
	void ReportLeaks(UWorld* World);

private:
	double						NextUpdateTime = 0.0;
	TMap<FName, FSystemData>	PerSystemData;
};

#endif //WITH_NIAGARA_LEAK_DETECTOR
