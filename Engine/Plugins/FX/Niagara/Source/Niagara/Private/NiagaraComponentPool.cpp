// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraComponentPool.h"
#include "HAL/IConsoleManager.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "NiagaraSystem.h"
#include "SceneInterface.h"

#include "NiagaraComponent.h"
#include "NiagaraWorldManager.h"
#include "NiagaraCrashReporterHandler.h"
#include "NiagaraGpuComputeDispatchInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraComponentPool)

static float GNiagaraSystemPoolKillUnusedTime = 180.0f;
static FAutoConsoleVariableRef NiagaraSystemPoolKillUnusedTime(
	TEXT("FX.NiagaraComponentPool.KillUnusedTime"),
	GNiagaraSystemPoolKillUnusedTime,
	TEXT("How long a pooled particle component needs to be unused for before it is destroyed.")
);

static int32 GbEnableNiagaraSystemPooling = 1;
static FAutoConsoleVariableRef bEnableNiagaraSystemPooling(
	TEXT("FX.NiagaraComponentPool.Enable"),
	GbEnableNiagaraSystemPooling,
	TEXT("How many Particle System Components to preallocate when creating new ones for the pool.")
);

static int32 GbEnableNiagaraSystemPoolValidation = 0;
static FAutoConsoleVariableRef CVarGbEnableNiagaraSystemPoolValidation(
	TEXT("FX.NiagaraComponentPool.Validation"),
	GbEnableNiagaraSystemPoolValidation,
	TEXT("Enables pooling validation.")
);

static float GNiagaraSystemPoolingCleanTime = 30.0f;
static FAutoConsoleVariableRef NiagaraSystemPoolingCleanTime(
	TEXT("FX.NiagaraComponentPool.CleanTime"),
	GNiagaraSystemPoolingCleanTime,
	TEXT("How often should the pool be cleaned (in seconds).")
);

static int32 GNiagaraKeepPooledComponentsRegistered = 1;
static FAutoConsoleVariableRef NiagaraKeepPooledComponentsRegistered(
	TEXT("FX.NiagaraComponentPool.KeepComponentsRegistered"),
	GNiagaraKeepPooledComponentsRegistered,
	TEXT("If non-zero, components returend to the pool are kept registered with the world but set invisible. This will reduce the cost of pushing/popping components int.")
);

void DumpPooledWorldNiagaraNiagaraSystemInfo(UWorld* World)
{
	check(World);
	if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World))
	{
		WorldManager->GetComponentPool()->Dump();
	}
}

FAutoConsoleCommandWithWorld DumpNCPoolInfoCommand(
	TEXT("FX.DumpNCPoolInfo"),
	TEXT("Dump Niagara System Pooling Info"),
	FConsoleCommandWithWorldDelegate::CreateStatic(&DumpPooledWorldNiagaraNiagaraSystemInfo)
);

void FNCPool::Cleanup()
{
	for (FNCPoolElement& Elem : FreeElements)
	{
		if (Elem.Component)
		{
			Elem.Component->PoolingMethod = ENCPoolMethod::None;//Reset so we don't trigger warnings about destroying pooled NCs.
			Elem.Component->DestroyComponent();
		}
		else
		{
			UE_LOGF(LogNiagara, Error, "Free element in the NiagaraComponentPool was null. Someone must be keeping a reference to a NC that has been freed to the pool and then are manually destroying it.");
		}
	}
	FreeElements.Empty();
}

UNiagaraComponent* FNCPool::Acquire(UWorld* World, UNiagaraSystem* Template, ENCPoolMethod PoolingMethod, bool bForceNew)
{
	check(GbEnableNiagaraSystemPooling);
	check(PoolingMethod != ENCPoolMethod::None);

	FNCPoolElement RetElem;
	while (FreeElements.Num() && !bForceNew)//Loop until we pop a valid free element or we're empty.
	{
		RetElem = FreeElements.Pop(EAllowShrinking::No);
		if (!RetElem.Component || !IsValidChecked(RetElem.Component.Get()))
		{			
			// Possible someone still has a reference to our NC and destroyed it while it was sat in the pool. Or possibly a teardown edgecase path that is GCing components from the pool be
			UE_LOGF(LogNiagara, Warning, "Pooled NC has been destroyed or is pending kill! Possibly via a DestroyComponent() call. You should not destroy pooled components manually. \nJust deactivate them and allow them to destroy themselves or be reclaimed by the pool. | NC: %p |\t System: %ls", RetElem.Component.Get(), *Template->GetFullName());
			RetElem = FNCPoolElement();
		}
		else
		{
			check(RetElem.Component->GetAsset() == Template);
			RetElem.Component->OnPooledReuse(World);
			break;
		}
	}

	if(RetElem.Component == nullptr)
	{
		// None in the pool so create a new one.
		AActor* OuterActor = World->GetWorldSettings();
		UObject* OuterObject = OuterActor ? static_cast<UObject*>(OuterActor) : static_cast<UObject*>(World);

		RetElem.Component.Reset(NewObject<UNiagaraComponent>(OuterObject));
		RetElem.Component->SetAutoDestroy(false);// we don't auto destroy, just periodically clear up the pool.
		RetElem.Component->bAutoActivate = false;
		RetElem.Component->SetVisibleInRayTracing(false);
		RetElem.Component->SetAsset(Template);
	}

	RetElem.Component->PoolingMethod = PoolingMethod;

	return RetElem.Component.Get();
}

void FNCPool::Reclaim(UNiagaraComponent* Component, const double CurrentTimeSeconds)
{
	check(Component);
	check(Component->GetAsset());

	//Don't add back to the pool if we're no longer pooling or we've hit our max resident pool size. Also if the component's world is in the process of tearing down.
	if (GbEnableNiagaraSystemPooling != 0 && FreeElements.Num() < (int32)Component->GetAsset()->MaxPoolSize && Component->GetWorld()->bIsTearingDown == false)
	{
		Component->DeactivateImmediate();

		Component->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform); // When detaching, maintain world position for optimization purposes
		Component->SetRelativeScale3D(FVector(1.f)); // Reset scale to avoid future uses of this NC having incorrect scale
		Component->SetAbsolute(); // Clear out Absolute settings to defaults
		Component->SetCastShadow(false);

		if(GNiagaraKeepPooledComponentsRegistered)
		{	
			//Keep components registered to avoid register/unregister cost.
			Component->SetVisibility(false);		
		}
		else
		{
			Component->UnregisterComponent();
		}

		//TODO: reset the delegates here once they are working
		
		//Ensure a small cull distance doesn't linger to future users.
		Component->SetCullDistance(FLT_MAX);

		if (!IsValidChecked(Component) || Component->IsUnreachable())
		{
			UE_LOGF(LogNiagara, Warning, "Component is pending kill or unreachable when reclaimed Component(%p %ls)", Component, *Component->GetFullName());
			return;
		}

		Component->PoolingMethod = ENCPoolMethod::FreeInPool;
		FreeElements.Push(FNCPoolElement(Component, CurrentTimeSeconds));
	}
	else
	{
		//We've stopped pooling while some effects were in flight so ensure they're destroyed now.
		Component->PoolingMethod = ENCPoolMethod::None;//Reset so we don't trigger warnings about destroying pooled NCs.
		Component->DestroyComponent();
	}
}

bool FNCPool::RemoveComponent(UNiagaraComponent* Component)
{
	int32 i = 0;
	while (i < FreeElements.Num())
	{
		if (FreeElements[i].Component.Get() == Component)
		{
			FreeElements.RemoveAtSwap(i, EAllowShrinking::No);
			return true;
		}
		++i;
	}

	return false;
}

extern int32 GNigaraAllowPrimedPools;
void FNCPool::KillUnusedComponents(double KillTime, UNiagaraSystem* Template)
{
	int32 i = 0;
	int32 PrimedSize = GNigaraAllowPrimedPools != 0 && Template ? Template->PoolPrimeSize : 0;
	while (i < FreeElements.Num() && FreeElements.Num() > PrimedSize)//Don't free below the primed size
	{
		if (FreeElements[i].LastUsedTime < KillTime)
		{
			if (UNiagaraComponent* Component = FreeElements[i].Component.Get())
			{
				Component->PoolingMethod = ENCPoolMethod::None; // Reset so we don't trigger warnings about destroying pooled NCs.
				Component->DestroyComponent();
			}

			FreeElements.RemoveAtSwap(i, EAllowShrinking::No);
		}
		else
		{
			++i;
		}
	}
	FreeElements.Shrink();
}

//////////////////////////////////////////////////////////////////////////

bool UNiagaraComponentPool::Enabled()
{
	return GbEnableNiagaraSystemPooling != 0;
}

UNiagaraComponentPool::UNiagaraComponentPool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	LastParticleSytemPoolCleanTime = 0.0;
}

UNiagaraComponentPool::~UNiagaraComponentPool()
{
	Cleanup(nullptr);
}

void UNiagaraComponentPool::Cleanup(UWorld* World)
{
	for (auto& Pool : WorldParticleSystemPools)
	{
		FNiagaraCrashReporterScope CRScope(Pool.Key.ResolveObjectPtr());//In practice this may be null by now :(
		Pool.Value.Cleanup();
	}

	WorldParticleSystemPools.Empty();

	// If we passed in a world make sure we cleanup any pooled components for that world
	// This is generally used on world cleanup to ensure all pooled components are cleaned up as they can not be returned back to the component pool.
	if ( World != nullptr )
	{
		if (AWorldSettings* WorldSettings = World->GetWorldSettings(false, false))
		{
			TArray<UNiagaraComponent*> ActiveComponents;
			WorldSettings->GetComponents(ActiveComponents);
			for (UNiagaraComponent* Comp : ActiveComponents)
			{
				if ( (Comp->PoolingMethod == ENCPoolMethod::AutoRelease) || (Comp->PoolingMethod == ENCPoolMethod::ManualRelease) )
				{
					Comp->PoolingMethod = ENCPoolMethod::None;
					Comp->DestroyComponent();
				}
			}
		}
	}
}

void UNiagaraComponentPool::ClearPool(UNiagaraSystem* System)
{
	if (FNCPool* NCPool = WorldParticleSystemPools.Find(System))
	{
		NCPool->Cleanup();
	}
}

void UNiagaraComponentPool::PrimePool(UNiagaraSystem* Template, UWorld* World)
{
	check(IsInGameThread());
	check(World);

	if (!Template)
	{
		UE_LOGF(LogNiagara, Warning, "Attempted UNiagaraComponentPool::PrimePool() with a NULL Template!");
		return;
	}

	if (!Template->bFullyLoaded)
	{
		return;
	}

	const uint32 PrimeCount = FMath::Min(Template->PoolPrimeSize, Template->MaxPoolSize);
	if (PrimeCount == 0)
	{
		return;
	}

	if (World->bIsTearingDown)
	{
		UE_LOGF(LogNiagara, Warning, "Failed to prime particle pool as we are tearing the world down.");
		return;
	}

	if (World->Scene == nullptr)
	{
		UE_LOGF(LogNiagara, Verbose, "Failed to prime particle pool as the world does not have a scene.");
		return;
	}	

#if WITH_EDITORONLY_DATA
	if (Template->NeedsRequestCompile())
	{
		UE_LOGF(LogNiagara, Verbose, "Failed to prime particle pool as the Niagara System needs to compile.");
		return;
	}
#endif

	if (Template->IsReadyToRun() == false)
	{
		UE_LOGF(LogNiagara, Verbose, "Failed to prime particle pool as the Niagara System is not ready to run.");
		return;
	}

	FFXSystemInterface* FXSystemInterface = World->Scene->GetFXSystem();
	if (FXSystemInterface)
	{
		if (FNiagaraGpuComputeDispatchInterface::Get(FXSystemInterface) == nullptr)
		{
			UE_LOGF(LogNiagara, Verbose, "Failed to prime particle pool as the world does not have a FNiagaraGpuComputeDispatchInterface.");
			return;
		}
	}
	else
	{
		UE_LOGF(LogNiagara, Verbose, "Failed to prime particle pool as the world does not have an FFXSystem.");
		return;
	}

	if (!World->IsGameWorld())
	{
		return;
	}

	FNiagaraCrashReporterScope CRScope(Template);

	if(PrimeCount > 0)
	{
		FNCPool& Pool = WorldParticleSystemPools.FindOrAdd(Template);

		uint32 ExistingComponents = Pool.NumComponents();
		if (ExistingComponents < PrimeCount)
		{
			TArray<UNiagaraComponent*> NewComps;
			NewComps.SetNum(PrimeCount - ExistingComponents);
			for (auto& Comp : NewComps)
			{
				Comp = Pool.Acquire(World, Template, ENCPoolMethod::ManualRelease, true);//Force the pool to create a new system.
#if ENABLE_NC_POOL_DEBUGGING
				InUseComponents_Manual.Add(Comp);
#endif
				Comp->InitializeSystem();
	
				if(GNiagaraKeepPooledComponentsRegistered)
				{
					Comp->RegisterComponentWithWorld(World);
				}
			}
			for (auto& Comp : NewComps)
			{
				Comp->ReleaseToPool();
			}
		}
	}
}

UNiagaraComponent* UNiagaraComponentPool::CreateWorldParticleSystem(UNiagaraSystem* Template, UWorld* World, ENCPoolMethod PoolingMethod)
{
	check(IsInGameThread());
	check(World);
	if (!Template)
	{
		UE_LOGF(LogNiagara, Warning, "Attempted CreateWorldParticleSystem() with a NULL Template!");
		return nullptr;
	}

	if (World->bIsTearingDown)
	{
		UE_LOGF(LogNiagara, Warning, "Failed to create pooled particle system as we are tearing the world down.");
		return nullptr;
	}

	FNiagaraCrashReporterScope CRScope(Template);

	UNiagaraComponent* Component = nullptr;
	if (GbEnableNiagaraSystemPooling != 0)
	{
		if (Template->MaxPoolSize > 0)
		{
			FNCPool& Pool = WorldParticleSystemPools.FindOrAdd(Template);
			Component = Pool.Acquire(World, Template, PoolingMethod);
		#if ENABLE_NC_POOL_DEBUGGING
			if (Component != nullptr)
			{
				if (PoolingMethod == ENCPoolMethod::AutoRelease)
				{
					InUseComponents_Auto.Add(Component);
				}
				else if (ensure(PoolingMethod == ENCPoolMethod::ManualRelease))
				{
					InUseComponents_Manual.Add(Component);
				}
			}
		#endif
		}
	}
	else
	{
		WorldParticleSystemPools.Empty();//Ensure the pools are cleared out if we've just switched to not pooling.
	}

	if(Component == nullptr)
	{
		// Create a new component as a fallback if we're not pooling
		Component = NewObject<UNiagaraComponent>(World);
		Component->SetAutoDestroy(true);
		Component->bAutoActivate = false;
		Component->SetAsset(Template);

		// even though we're not actually using the pooling system we need to ensure that the PoolingMethod
		// is preserved so that the component can be properly cleaned up (see UNiagaraComponent::ReleaseToPool()
		// and UNiagaraComponent::OnSystemComplete()).
		Component->PoolingMethod = PoolingMethod;
	}

	check(Component);
	return Component;
}

/** Called when an in-use particle component is finished and wishes to be returned to the pool. */
void UNiagaraComponentPool::ReclaimWorldParticleSystem(UNiagaraComponent* Component)
{
	check(IsInGameThread());

	UNiagaraSystem* Asset = Component->GetAsset();
	FNiagaraCrashReporterScope CRScope(Asset);
	
	//If this component has been already destroyed we don't add it back to the pool. Just warn so users can fix it.
	if (!IsValid(Component))
	{
		UE_LOGF(LogNiagara, Log, "Pooled NC has been destroyed! Possibly via a DestroyComponent() call. You should not destroy components set to auto destroy manually. \nJust deactivate them and allow them to destroy themselves or be reclaimed by the pool if pooling is enabled. | NC: %p |\t System: %ls", Component, Asset ? *Asset->GetFullName() : TEXT("(nullptr)"));
		return;
	}

	// WorldParticleSystemPools is empty after world cleanup, so destroy components coming in to be reclaimed instead
	if (GbEnableNiagaraSystemPooling && Asset != nullptr && WorldParticleSystemPools.Num() > 0)
	{
		const double CurrentTime = Component->GetWorld()->GetTimeSeconds();

		//Periodically clear up the pools.
		if (CurrentTime - LastParticleSytemPoolCleanTime > GNiagaraSystemPoolingCleanTime)
		{
			LastParticleSytemPoolCleanTime = CurrentTime;
			for (auto& Pair : WorldParticleSystemPools)
			{
				UNiagaraSystem* Template = Pair.Key.ResolveObjectPtr();
				Pair.Value.KillUnusedComponents(CurrentTime - GNiagaraSystemPoolKillUnusedTime, Template);
			}
		}

#if ENABLE_NC_POOL_DEBUGGING
		bool bWasInList = false;
		if (Component->PoolingMethod == ENCPoolMethod::AutoRelease)
		{
			bWasInList = InUseComponents_Auto.RemoveSingleSwap(Component) > 0;
		}
		else if (ensure(Component->PoolingMethod == ENCPoolMethod::ManualRelease))
		{
			bWasInList = InUseComponents_Manual.RemoveSingleSwap(Component) > 0;
		}

		if (!bWasInList)
		{
			UE_LOGF(LogNiagara, Error, "World Niagara System Pool is reclaiming a component that is not in it's InUse list!");
		}
#else
		constexpr bool bWasInList = false;
#endif

		FNCPool* NCPool = WorldParticleSystemPools.Find(Asset);
		if (!NCPool)
		{
			// We only need to warn if it was in the in use list
			// When in the editor update context's can clear out the existing pools so this will fire incorrectly.
			if (!bWasInList)
			{
				UE_LOGF(LogNiagara, Warning, "WorldNC Pool trying to reclaim a system for which it doesn't have a pool! Likely because SetAsset() has been called on this NC. | World: %p | NC: %p | Sys: %ls", Component->GetWorld(), Component, *Component->GetAsset()->GetFullName());
			}

			//Just add the new pool and reclaim to that one.
			NCPool = &WorldParticleSystemPools.Add(Asset);
		}

		NCPool->Reclaim(Component, CurrentTime);
	}
	else
	{
		// We've stopped pooling while some effects were in flight so ensure they're destroyed now.
		Component->PoolingMethod = ENCPoolMethod::None;
		Component->DestroyComponent();
	}
}

void UNiagaraComponentPool::PooledComponentDestroyed(UNiagaraComponent* Component)
{
	check(IsInGameThread());

	if (GbEnableNiagaraSystemPooling == false)
	{
		return;
	}

	switch (Component->PoolingMethod)
	{
		// We are inside a pool, clear out the entry
		case ENCPoolMethod::FreeInPool:
		{
			if (UNiagaraSystem* NiagaraSystem = Component->GetAsset())
			{
				if (FNCPool* NCPool = WorldParticleSystemPools.Find(NiagaraSystem))
				{
					if (!NCPool->RemoveComponent(Component))
					{
						// Suppress excessive logging when not debugging the component pool - no easy way to tell if this is actually a problem
#if ENABLE_NC_POOL_DEBUGGING
						UE_LOGF(LogNiagara, Warning, "UNiagaraComponentPool::PooledComponentDestroyed: Component is marked as FreeInPool but does not exist");
#endif
					}
				}
			}
			break;
		}

		// In all of these cases we are being force destroyed so we don't need to do anything
		case ENCPoolMethod::None:
		case ENCPoolMethod::AutoRelease:
		case ENCPoolMethod::ManualRelease:
		case ENCPoolMethod::ManualRelease_OnComplete:
			break;

		// We should never get here
		default:
			UE_LOGF(LogNiagara, Warning, "UNiagaraComponentPool::PooledComponentDestroyed: Invalid pooling mode?");
			break;
	}

	// Additional validation that the component doesn't appear in another pool somewhere
	if (GbEnableNiagaraSystemPoolValidation)
	{
		for (auto it=WorldParticleSystemPools.CreateIterator(); it; ++it)
		{
			if (it.Value().RemoveComponent(Component))
			{
				UE_LOGF(LogNiagara, Warning, "UNiagaraComponentPool::PooledComponentDestroyed: Component existed in a pool that it should not be in?");
			}
		}
	}

	Component->PoolingMethod = ENCPoolMethod::None;
}

void UNiagaraComponentPool::RemoveComponentsBySystem(UNiagaraSystem* System)
{
	if ( FNCPool* NCPool = WorldParticleSystemPools.Find(System) )
	{
		NCPool->Cleanup();
		WorldParticleSystemPools.Remove(System);
	}
}

void UNiagaraComponentPool::Dump()
{
#if ENABLE_NC_POOL_DEBUGGING
	FString DumpStr;

	uint32 TotalMemUsage = 0;
	for (auto& Pair : WorldParticleSystemPools)
	{
		UNiagaraSystem* System = Pair.Key.ResolveObjectPtr();
		FNCPool& Pool = Pair.Value;		
		uint32 FreeMemUsage = 0;
		for (FNCPoolElement& Elem : Pool.FreeElements)
		{
			if (ensureAlways(Elem.Component))
			{
				FreeMemUsage += Elem.Component->GetApproxMemoryUsage();
			}
		}
		uint32 InUseMemUsage = 0;
		int32 NumInUseAutoComponents = 0;
		int32 NumInUseManualComponents = 0;
		for (auto WeakComponent : InUseComponents_Auto)
		{
			UNiagaraComponent* Component = WeakComponent.Get();
			if (ensureAlways(Component) && (Component->GetAsset() == System))
			{
				InUseMemUsage += Component->GetApproxMemoryUsage();				
				++NumInUseAutoComponents;
			}
		}
		for (auto WeakComponent : InUseComponents_Manual)
		{
			UNiagaraComponent* Component = WeakComponent.Get();
			if (ensureAlways(Component) && (Component->GetAsset() == System))
			{
				InUseMemUsage += Component->GetApproxMemoryUsage();
				++NumInUseManualComponents;
			}
		}

		TotalMemUsage += FreeMemUsage;
		TotalMemUsage += InUseMemUsage;

		DumpStr += FString::Printf(TEXT("Free: %d (%uB) \t|\t Used(Auto - Manual): %d - %d (%uB) \t|\t System: %s\n"), Pool.FreeElements.Num(), FreeMemUsage, NumInUseAutoComponents, NumInUseManualComponents, InUseMemUsage, *System->GetFullName());
	}

	UE_LOGF(LogNiagara, Log, "***************************************");
	UE_LOGF(LogNiagara, Log, "*Particle System Pool Info - Total Mem = %.2fMB*", TotalMemUsage / 1024.0f / 1024.0f);
	UE_LOGF(LogNiagara, Log, "***************************************");
	UE_LOGF(LogNiagara, Log, "%ls", *DumpStr);
	UE_LOGF(LogNiagara, Log, "***************************************");

#endif
}

