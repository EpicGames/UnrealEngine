// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraComponentRendererProperties.h"
#include "Misc/PackageName.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraModule.h"
#include "NiagaraRenderer.h"
#include "NiagaraConstants.h"
#include "NiagaraRendererComponents.h"
#include "NiagaraComponent.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#if WITH_EDITOR
#include "Widgets/Images/SImage.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "UObject/UObjectIterator.h"
#endif
#include "NiagaraCustomVersion.h"
#include "NiagaraSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraComponentRendererProperties)

static float GNiagaraComponentRenderComponentCountWarning = 50;
static FAutoConsoleVariableRef CVarNiagaraComponentRenderComponentCountWarning(
	TEXT("fx.Niagara.ComponentRenderComponentCountWarning"),
	GNiagaraComponentRenderComponentCountWarning,
	TEXT("The max number of allowed components before a ui warning is shown in the component renderer."),
	ECVF_Default
	);

#define LOCTEXT_NAMESPACE "UNiagaraComponentRendererProperties"

bool UNiagaraComponentRendererProperties::IsConvertible(const FNiagaraTypeDefinition& SourceType, const FNiagaraTypeDefinition& TargetType)
{
	if (SourceType == TargetType)
	{
		return true;
	}

	if ((SourceType == FNiagaraTypeDefinition::GetColorDef() && TargetType.GetStruct() == GetFColorDef().GetStruct()) ||
		(SourceType == FNiagaraTypeDefinition::GetVec3Def() && TargetType.GetStruct() == GetFColorDef().GetStruct()) ||
		(SourceType == FNiagaraTypeDefinition::GetVec3Def() && TargetType.GetStruct() == GetFRotatorDef().GetStruct()) ||
		(SourceType == FNiagaraTypeDefinition::GetVec2Def() && TargetType.GetStruct() == GetFVector2DDef().GetStruct()) ||
		(SourceType == FNiagaraTypeDefinition::GetVec3Def() && TargetType.GetStruct() == GetFVectorDef().GetStruct()) ||
		(SourceType == FNiagaraTypeDefinition::GetVec4Def() && TargetType.GetStruct() == GetFVector4Def().GetStruct()) ||
		(SourceType == FNiagaraTypeDefinition::GetVec4Def() && TargetType.GetStruct() == GetFColorDef().GetStruct()) ||
		(SourceType == FNiagaraTypeDefinition::GetQuatDef() && TargetType.GetStruct() == GetFQuatDef().GetStruct()) ||
		(SourceType == FNiagaraTypeDefinition::GetPositionDef() && TargetType.GetStruct() == GetFVectorDef().GetStruct()) ||
		(SourceType == FNiagaraTypeDefinition::GetPositionDef() && TargetType.GetStruct() == GetFVector3fDef().GetStruct()) ||
		(SourceType == FNiagaraTypeDefinition::GetFloatDef() && TargetType.GetStruct() == GetDoubleDef().GetStruct()) ||
		(SourceType == FNiagaraTypeDefinition::GetQuatDef() && TargetType.GetStruct() == GetFRotatorDef().GetStruct()))
	{
		return true;
	}
	return false;
}

NIAGARA_API FNiagaraTypeDefinition UNiagaraComponentRendererProperties::ToNiagaraType(FProperty* Property)
{
	const FFieldClass* FieldClass = Property->GetClass();
	if (FieldClass->IsChildOf(FBoolProperty::StaticClass()))
	{
		return FNiagaraTypeDefinition::GetBoolDef();
	}
	if (FieldClass->IsChildOf(FIntProperty::StaticClass()))
	{
		return FNiagaraTypeDefinition::GetIntDef();
	}
	if (FieldClass->IsChildOf(FFloatProperty::StaticClass()))
	{
		return FNiagaraTypeDefinition::GetFloatDef();
	}
	if (FieldClass->IsChildOf(FDoubleProperty::StaticClass()))
	{
		return GetDoubleDef();
	}
	if (FieldClass->IsChildOf(FStructProperty::StaticClass()))
	{
		FStructProperty* StructProperty = (FStructProperty*)Property;
		if (StructProperty->Struct)
		{
			if (StructProperty->Struct->GetFName() == NAME_Vector2D || StructProperty->Struct->GetFName() == NAME_Vector2d) 
			{
				return GetFVector2DDef();
			}

			if (StructProperty->Struct->GetFName() == NAME_Vector || StructProperty->Struct->GetFName() == NAME_Vector3d)
			{
				return GetFVectorDef();
			}

			if (StructProperty->Struct->GetFName() == NAME_Vector4 || StructProperty->Struct->GetFName() == NAME_Vector4d)
			{
				return GetFVector4Def();
			}

			if (StructProperty->Struct->GetFName() == NAME_Quat || StructProperty->Struct->GetFName() == NAME_Quat4d)
			{
				return GetFQuatDef();
			}

			if (StructProperty->Struct->GetFName() == FName("NiagaraPosition"))
			{
				return FNiagaraTypeDefinition::GetPositionStruct();
			}

			return FNiagaraTypeDefinition(StructProperty->Struct);
		}
	}

	// we currently don't support reading arbitrary enum or object data from the simulation data
	/*
	if (FieldClass->IsChildOf(FObjectProperty::StaticClass()))
	{
		return FNiagaraTypeDefinition::GetUObjectDef();
	}
	if (FieldClass->IsChildOf(FEnumProperty::StaticClass()))
	{
		FEnumProperty* EnumProperty = (FEnumProperty*)PropertyHandle->GetProperty();
		if (UEnum* EnumDef = EnumProperty->GetEnum())
		{
			return FNiagaraTypeDefinition(EnumDef);
		}
	}
	 */

	return FNiagaraTypeDefinition();
}


FNiagaraTypeDefinition UNiagaraComponentRendererProperties::GetFColorDef()
{
	static UPackage* CoreUObjectPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/CoreUObject"));
	static UScriptStruct* ColorStruct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Color"));
	return FNiagaraTypeDefinition(ColorStruct);
}

FNiagaraTypeDefinition UNiagaraComponentRendererProperties::GetFRotatorDef()
{
	static UPackage* CoreUObjectPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/CoreUObject"));
	static UScriptStruct* RotatorStruct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Rotator"));
	return FNiagaraTypeDefinition(RotatorStruct);
}

FNiagaraTypeDefinition UNiagaraComponentRendererProperties::GetFVector2DDef()
{
	return FNiagaraTypeHelper::GetVector2DDef();
}

FNiagaraTypeDefinition UNiagaraComponentRendererProperties::GetFVectorDef()
{
	return FNiagaraTypeHelper::GetVectorDef();
}

FNiagaraTypeDefinition UNiagaraComponentRendererProperties::GetFVector4Def()
{
	return FNiagaraTypeHelper::GetVector4Def();
}

FNiagaraTypeDefinition UNiagaraComponentRendererProperties::GetFVector3fDef()
{
	return FNiagaraTypeDefinition::GetVec3Def();
}

FNiagaraTypeDefinition UNiagaraComponentRendererProperties::GetFQuatDef()
{
	return FNiagaraTypeHelper::GetQuatDef();
}

FNiagaraTypeDefinition UNiagaraComponentRendererProperties::GetDoubleDef()
{
	return FNiagaraTypeHelper::GetDoubleDef();
}

TArray<TWeakObjectPtr<UNiagaraComponentRendererProperties>> UNiagaraComponentRendererProperties::ComponentRendererPropertiesToDeferredInit;

UNiagaraComponentRendererProperties::UNiagaraComponentRendererProperties()
	: ComponentCountLimit(15)
	, bAssignComponentsOnParticleID(true)
	, bOnlyActivateNewlyAquiredComponents(true)
#if WITH_EDITORONLY_DATA
	, bVisualizeComponents(true)
	, bOnlyCreateComponentsOnParticleSpawn_DEPRECATED(true)
#endif
	, RendererVisibility(0)
	, TemplateComponent(nullptr)
{
	AttributeBindings =
	{
		&EnabledBinding,
		&RendererVisibilityTagBinding,
	};
	IsSetterMappingDirty.store(true);
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UNiagaraComponentRendererProperties::OnObjectsReplacedCallback);
	}
#endif
}

UNiagaraComponentRendererProperties::~UNiagaraComponentRendererProperties()
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	}
#endif
}

void UNiagaraComponentRendererProperties::PostLoad()
{
	Super::PostLoad();
	ENiagaraRendererSourceDataMode InSourceMode = ENiagaraRendererSourceDataMode::Particles;

#if WITH_EDITORONLY_DATA
	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
	if (NiagaraVer < FNiagaraCustomVersion::ComponentRendererSpawnProperty)
	{
		bCreateComponentFirstParticleFrame = bOnlyCreateComponentsOnParticleSpawn_DEPRECATED;
	}
#endif
	
	const TArray<FNiagaraVariable>& OldTypes = FNiagaraConstants::GetOldPositionTypeVariables();
	for (FNiagaraComponentPropertyBinding& Binding : PropertyBindings)
	{
		Binding.AttributeBinding.PostLoad(InSourceMode);

		// Move old bindings over to new position type
		for (FNiagaraVariable OldVarType : OldTypes)
		{
			if (Binding.AttributeBinding.GetParamMapBindableVariable() == (const FNiagaraVariableBase&)OldVarType)
			{
				FNiagaraVariable NewVarType(FNiagaraTypeDefinition::GetPositionDef(), OldVarType.GetName());
				Binding.AttributeBinding.Setup(NewVarType, NewVarType, InSourceMode);
				Binding.PropertyType = GetFVectorDef();
				break;
			}
		}

#if WITH_EDITOR
		// check if any of the bound properties was changed (e.g. to a lwc type) and regenerate the bindings if necessary
		if (!TemplateComponent)
		{
			continue;
		}
		for (TFieldIterator<FProperty> PropertyIt(TemplateComponent->GetClass()); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			if (Property && Property->GetFName() == Binding.PropertyName)
			{
				Binding.PropertyType = ToNiagaraType(Property);
				break;
			}
		}
#endif
	}
	EnabledBinding.PostLoad(InSourceMode);
	RendererVisibilityTagBinding.PostLoad(InSourceMode);

	PostLoadBindings(ENiagaraRendererSourceDataMode::Particles);
}


void UNiagaraComponentRendererProperties::UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit)
{
	FVersionedNiagaraEmitterBase SrcEmitter = GetOuterEmitterBase();
	if (SrcEmitter.Emitter)
	{
		EnabledBinding.CacheValues(SrcEmitter, InSourceMode);
		RendererVisibilityTagBinding.CacheValues(SrcEmitter, InSourceMode);
		for (FNiagaraComponentPropertyBinding& Binding : PropertyBindings)
		{
			Binding.AttributeBinding.CacheValues(SrcEmitter, InSourceMode);
		}
	}

	Super::UpdateSourceModeDerivates(InSourceMode);
}

void UNiagaraComponentRendererProperties::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		// We can end up hitting PostInitProperties before the Niagara Module has initialized bindings this needs, mark this object for deferred init and early out.
		if (FModuleManager::Get().IsModuleLoaded("Niagara") == false)
		{
			ComponentRendererPropertiesToDeferredInit.Add(this);
			return;
		}
		else
		{
			if (!EnabledBinding.IsValid())
			{
				EnabledBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_COMPONENTS_ENABLED);
			}
			if (!RendererVisibilityTagBinding.IsValid())
			{
				RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
// Function definition copied from UEdGraphSchema_K2
bool FindFunctionParameterDefaultValue(const UFunction* Function, const FProperty* Param, FString& OutString)
{
	bool bHasAutomaticValue = false;

	const FString& MetadataDefaultValue = Function->GetMetaData(*Param->GetName());
	if (!MetadataDefaultValue.IsEmpty())
	{
		// Specified default value in the metadata
		OutString = MetadataDefaultValue;
		bHasAutomaticValue = true;

		// If the parameter is a class then try and get the full name as the metadata might just be the short name
		if (Param->IsA<FClassProperty>() && !FPackageName::IsValidObjectPath(OutString))
		{
			if (UClass* DefaultClass = UClass::TryFindTypeSlow<UClass>(OutString, EFindFirstObjectOptions::ExactClass))
			{
				OutString = DefaultClass->GetPathName();
			}
		}
	}
	else
	{
		const FName MetadataCppDefaultValueKey(*(FString(TEXT("CPP_Default_")) + Param->GetName()));
		const FString& MetadataCppDefaultValue = Function->GetMetaData(MetadataCppDefaultValueKey);
		if (!MetadataCppDefaultValue.IsEmpty())
		{
			OutString = MetadataCppDefaultValue;
			bHasAutomaticValue = true;
		}
	}

	return bHasAutomaticValue;
}
#endif

void UNiagaraComponentRendererProperties::CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData)
{
	UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode::Particles);
	UpdateTemplateCoordinateSpace();
}

#if WITH_EDITORONLY_DATA
bool UNiagaraComponentRendererProperties::IsSupportedVariableForBinding(const FNiagaraVariableBase& InSourceForBinding, const FName& InTargetBindingName) const
{
	if (InTargetBindingName == GET_MEMBER_NAME_CHECKED(UNiagaraRendererProperties, RendererEnabledBinding))
	{
		return
			InSourceForBinding.IsInNameSpace(FNiagaraConstants::UserNamespace) ||
			InSourceForBinding.IsInNameSpace(FNiagaraConstants::SystemNamespace) ||
			InSourceForBinding.IsInNameSpace(FNiagaraConstants::EmitterNamespace);
	}

	return false;
}
#endif

void UNiagaraComponentRendererProperties::UpdateSetterFunctions()
{
	SetterFunctionMapping.Empty();
	for (FNiagaraComponentPropertyBinding& PropertyBinding : PropertyBindings)
	{
		if (!TemplateComponent || SetterFunctionMapping.Contains(PropertyBinding.PropertyName))
		{
			continue;
		}
		UFunction* SetterFunction = nullptr;

		// we first check if the property has some metadata that explicitly mentions the setter to use
		if (!PropertyBinding.MetadataSetterName.IsNone())
		{
			SetterFunction = TemplateComponent->FindFunction(PropertyBinding.MetadataSetterName);
		}

		if (!SetterFunction)
		{
			// the setter was not specified, so we try to find one that fits the name
			FString PropertyName = PropertyBinding.PropertyName.ToString();
			if (PropertyBinding.PropertyType == FNiagaraTypeDefinition::GetBoolDef())
			{
				PropertyName.RemoveFromStart("b", ESearchCase::CaseSensitive);
			}

			static const TArray<FString> SetterPrefixes = {
				FString("Set"),
				FString("K2_Set")
			};

			for (const FString& Prefix : SetterPrefixes)
			{
				FName SetterFunctionName = FName(Prefix + PropertyName, FNAME_Find);
				if (!SetterFunctionName.IsNone())
				{
					SetterFunction = TemplateComponent->FindFunction(SetterFunctionName);
					if (SetterFunction)
					{
						break;
					}
				}
			}
		}

		FNiagaraPropertySetter Setter;
		Setter.Function = SetterFunction;

		// Okay, so there is a special case where the *property* of an object has one type, but the *setter* has another type
		// that either doesn't need to be converted (e.g. the color property on a light component) or doesn't fit the converted value.
		// If we detect such a case we adapt the binding to either ignore the conversion or we discard the setter completely.
		if (SetterFunction)
		{
			bool bFirstProperty = true;
			for (FProperty* Property = SetterFunction->PropertyLink; Property; Property = Property->PropertyLinkNext)
			{
				if (Property->IsInContainer(SetterFunction->ParmsSize) && Property->HasAnyPropertyFlags(CPF_Parm) && !Property->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					if (bFirstProperty)
					{
						// the first property is our bound value, so we check for the correct type
						FNiagaraTypeDefinition FieldType = ToNiagaraType(Property);
						if (FieldType != PropertyBinding.PropertyType && FieldType == PropertyBinding.AttributeBinding.GetType())
						{
							// we can use the original Niagara value with the setter instead of converting it.
							Setter.bIgnoreConversion = true;
						}
						else if (FieldType != PropertyBinding.PropertyType)
						{
							// setter is completely unusable
							Setter.Function = nullptr;
						}
						bFirstProperty = false;
					}
#if WITH_EDITORONLY_DATA
					else
					{
						// the other values are just function parameters, so we check if they have custom default values defined in the metadata
						FString DefaultValue;
						if (FindFunctionParameterDefaultValue(SetterFunction, Property, DefaultValue))
						{
							// Store property setter parameter defaults, as this is kept in metadata which is not available at runtime
							PropertyBinding.PropertySetterParameterDefaults.Add(Property->GetName(), DefaultValue);
						}
						else
						{
							PropertyBinding.PropertySetterParameterDefaults.Remove(Property->GetName());
						}
					}
#endif
				}
			}
		}
		SetterFunctionMapping.Add(PropertyBinding.PropertyName, Setter);
	}
}

void UNiagaraComponentRendererProperties::PostDuplicate(bool bDuplicateForPIE)
{
	// sharing the same template component would mean changes in one emitter would be reflected in the other emitter,
	// so we create a new template object instead
	TemplateComponent = DuplicateObject(TemplateComponent, this);
}

void UNiagaraComponentRendererProperties::InitCDOPropertiesAfterModuleStartup()
{
	UNiagaraComponentRendererProperties* CDO = CastChecked<UNiagaraComponentRendererProperties>(UNiagaraComponentRendererProperties::StaticClass()->GetDefaultObject());
	CDO->InitRendererBaseBindings();
	CDO->EnabledBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_COMPONENTS_ENABLED);
	CDO->RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);

	for (TWeakObjectPtr<UNiagaraComponentRendererProperties>& WeakComponentRendererProperties : ComponentRendererPropertiesToDeferredInit)
	{
		if (UNiagaraComponentRendererProperties* RendererProperties = WeakComponentRendererProperties.Get())
		{
			RendererProperties->InitRendererBaseBindings();
			if (!RendererProperties->EnabledBinding.IsValid())
			{
				RendererProperties->EnabledBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_COMPONENTS_ENABLED);
			}
			if (!RendererProperties->RendererVisibilityTagBinding.IsValid())
			{
				RendererProperties->RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
			}
		}
	}
}

FNiagaraRenderer* UNiagaraComponentRendererProperties::CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
	if (IsSetterMappingDirty.exchange(false))
	{
		UpdateSetterFunctions();
	}

	FNiagaraRenderer* NewRenderer = new FNiagaraRendererComponents(FeatureLevel, this, Emitter);
	NewRenderer->Initialize(this, Emitter, InController);
	return NewRenderer;
}

void UNiagaraComponentRendererProperties::CreateTemplateComponent()
{
	TemplateComponent = NewObject<USceneComponent>(this, ComponentType, NAME_None, RF_ArchetypeObject);
	TemplateComponent->SetVisibility(false);
	TemplateComponent->SetAutoActivate(false);
	TemplateComponent->SetComponentTickEnabled(false);
	UpdateTemplateCoordinateSpace();
}

void UNiagaraComponentRendererProperties::UpdateTemplateCoordinateSpace()
{
	if (TemplateComponent)
	{
		FVersionedNiagaraEmitterData* EmitterData = GetOuterEmitter().GetEmitterData();
		const bool IsWorldSpace = EmitterData ? !EmitterData->bLocalSpace : true;
		TemplateComponent->SetAbsolute(IsWorldSpace, IsWorldSpace, IsWorldSpace);
	}
}

#if WITH_EDITOR

void UNiagaraComponentRendererProperties::OnObjectsReplacedCallback(const TMap<UObject*, UObject*>& ReplacementsMap)
{
	// this callback is only registered on the CDO, so we inform all the regular instances like this
	for (TObjectIterator<UNiagaraComponentRendererProperties> Itt; Itt; ++Itt)
	{
		UNiagaraComponentRendererProperties* PropertiesObj = *Itt;
		if (PropertiesObj && PropertiesObj != this)
		{
			PropertiesObj->OnObjectsReplacedInternal(ReplacementsMap);
		}
	}
}

void UNiagaraComponentRendererProperties::OnObjectsReplacedInternal(const TMap<UObject*, UObject*>& ReplacementsMap)
{
	// When a custom component class is recompiled in the editor, we need to switch to the new template component object
	if (TemplateComponent)
	{
		if (UObject* const* Replacement = ReplacementsMap.Find(TemplateComponent))
		{
			TemplateComponent = Cast<USceneComponent>(*Replacement);
			UpdateSetterFunctions();
		}
	}
}

#endif

bool UNiagaraComponentRendererProperties::HasPropertyBinding(FName PropertyName) const
{
	for (const FNiagaraComponentPropertyBinding& Binding : PropertyBindings)
	{
		if (Binding.PropertyName == PropertyName)
		{
			return true;
		}
	}
	return false;
}

#if WITH_EDITORONLY_DATA

void UNiagaraComponentRendererProperties::PostEditChangeProperty(struct FPropertyChangedEvent& e)
{
	FName PropertyName = (e.Property != nullptr) ? e.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraComponentRendererProperties, ComponentType))
	{
		PropertyBindings.Empty();
		if (TemplateComponent)
		{
			TemplateComponent->DestroyComponent();
		}
		if (ComponentType && UNiagaraComponent::StaticClass()->IsChildOf(ComponentType->ClassWithin))
		{
			CreateTemplateComponent();

			FNiagaraComponentPropertyBinding PositionBinding;
			PositionBinding.AttributeBinding.Setup(SYS_PARAM_PARTICLES_POSITION, SYS_PARAM_PARTICLES_POSITION);
			PositionBinding.PropertyName = FName("RelativeLocation");
			PositionBinding.PropertyType = GetFVectorDef();
			PropertyBindings.Add(PositionBinding);

			FNiagaraComponentPropertyBinding ScaleBinding;
			ScaleBinding.AttributeBinding.Setup(SYS_PARAM_PARTICLES_SCALE, SYS_PARAM_PARTICLES_SCALE);
			ScaleBinding.PropertyName = FName("RelativeScale3D");
			PropertyBindings.Add(ScaleBinding);
		}
		else
		{
			TemplateComponent = nullptr;
		}
	}
	IsSetterMappingDirty = true; // to refresh the default values for the setter parameters
	Super::PostEditChangeProperty(e);
}

void UNiagaraComponentRendererProperties::GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	TSharedRef<SWidget> Tooltip = SNew(STextBlock)
		.Text(FText::Format(LOCTEXT("ComponentRendererTooltip", "Component Renderer ({0})"), TemplateComponent ?
			TemplateComponent->GetClass()->GetDisplayNameText() :
			FText::FromString("No type selected")));
	OutWidgets.Add(Tooltip);
}

void UNiagaraComponentRendererProperties::GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const
{
	OutInfo.Add(FNiagaraRendererFeedback(FText::FromString(TEXT("The component renderer is still a very experimental feature that offers great flexibility, \nbut is *not* optimized for performance or safety. \nWith great power comes great responsibility."))));

	if (ComponentType && !UNiagaraComponent::StaticClass()->IsChildOf(ComponentType->ClassWithin))
	{
		FText ErrorDescription = FText::Format(LOCTEXT("NiagaraClassWithinComponentError", "The selected component type is not valid because it can only be attached to an object of type {0}."), FText::FromString(ComponentType->ClassWithin->GetName()));
		FText ErrorSummary = LOCTEXT("NiagaraClassWithinComponentErrorSummary", "Invalid component type selected!");
		OutErrors.Add(FNiagaraRendererFeedback(ErrorDescription, ErrorSummary));
	}

	FVersionedNiagaraEmitterData* EmitterData = InEmitter.GetEmitterData();
	if (InEmitter.Emitter && TemplateComponent && EmitterData)
	{
		if (const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>())
		{
			for (const TPair<FString, FText>& Pair : Settings->ComponentRendererWarningsPerClass)
			{
				FString ClassName = TemplateComponent->GetClass()->GetName();
				if (ClassName == Pair.Key)
				{
					OutWarnings.Add(Pair.Value);
				}
			}
		}

		bool IsWorldSpace = !EmitterData->bLocalSpace;
		FNiagaraRendererFeedbackFix LocalspaceFix = FNiagaraRendererFeedbackFix::CreateLambda([EmitterData]() {	EmitterData->bLocalSpace = !EmitterData->bLocalSpace; });
		if (TemplateComponent->IsUsingAbsoluteLocation() != IsWorldSpace && !HasPropertyBinding(FName("bAbsoluteLocation")))
		{
			FText ErrorDescription = LOCTEXT("NiagaraComponentLocalspaceLocationWarning", "The component location is configured to use a different localspace setting than the emitter.");
			FText ErrorSummary = LOCTEXT("NiagaraComponentLocalspaceLocationWarningSummary", "Component location and emitter localspace different!");
			FText FixText = LOCTEXT("NiagaraComponentLocalspaceLocationWarningFix", "Change emitter localspace setting");
			OutWarnings.Add(FNiagaraRendererFeedback(ErrorDescription, ErrorSummary, FixText, LocalspaceFix, true));
		}
		if (TemplateComponent->IsUsingAbsoluteRotation() != IsWorldSpace && !HasPropertyBinding(FName("bAbsoluteRotation")))
		{
			FText ErrorDescription = LOCTEXT("NiagaraComponentLocalspaceRotationWarning", "The component rotation is configured to use a different localspace setting than the emitter.");
			FText ErrorSummary = LOCTEXT("NiagaraComponentLocalspaceRotationWarningSummary", "Component rotation and emitter localspace different!");
			FText FixText = LOCTEXT("NiagaraComponentLocalspaceRotationWarningFix", "Change emitter localspace setting");
			OutWarnings.Add(FNiagaraRendererFeedback(ErrorDescription, ErrorSummary, FixText, LocalspaceFix, true));
		}
		if (TemplateComponent->IsUsingAbsoluteScale() != IsWorldSpace && !HasPropertyBinding(FName("bAbsoluteScale")))
		{
			FText ErrorDescription = LOCTEXT("NiagaraComponentLocalspaceScaleWarning", "The component scale is configured to use a different localspace setting than the emitter.");
			FText ErrorSummary = LOCTEXT("NiagaraComponentLocalspaceScaleWarningSummary", "Component scale and emitter localspace different!");
			FText FixText = LOCTEXT("NiagaraComponentLocalspaceScaleWarningFix", "Change emitter localspace setting");
			OutWarnings.Add(FNiagaraRendererFeedback(ErrorDescription, ErrorSummary, FixText, LocalspaceFix, true));
		}
	}

	if (ComponentCountLimit > GNiagaraComponentRenderComponentCountWarning)
	{
		OutWarnings.Add(FText::FromString(TEXT("Creating and updating many components each tick will have a serious impact on performance.")));
	}
}

const FSlateBrush* UNiagaraComponentRendererProperties::GetStackIcon() const
{
	return FSlateIconFinder::FindIconBrushForClass(TemplateComponent ? TemplateComponent->GetClass() : GetClass());
}

FText UNiagaraComponentRendererProperties::GetWidgetDisplayName() const
{
	return TemplateComponent ? FText::Format(FText::FromString("{0} Renderer"), TemplateComponent->GetClass()->GetDisplayNameText()) : Super::GetWidgetDisplayName();
}

TArray<FNiagaraVariable> UNiagaraComponentRendererProperties::GetBoundAttributes() const
{
	TArray<FNiagaraVariable> BoundAttributes = Super::GetBoundAttributes();
	BoundAttributes.Reserve(BoundAttributes.Num() + PropertyBindings.Num() + (bAssignComponentsOnParticleID ? 2 : 1));

	BoundAttributes.Add(SYS_PARAM_PARTICLES_COMPONENTS_ENABLED);
	if (bAssignComponentsOnParticleID)
	{
		BoundAttributes.Add(SYS_PARAM_PARTICLES_UNIQUE_ID);
	}
	for (const FNiagaraComponentPropertyBinding& PropertyBinding : PropertyBindings)
	{
		if (PropertyBinding.AttributeBinding.IsValid())
		{
			BoundAttributes.Add(PropertyBinding.AttributeBinding.GetParamMapBindableVariable());
		}
	}
	return BoundAttributes;
}

const TArray<FNiagaraVariable>& UNiagaraComponentRendererProperties::GetOptionalAttributes()
{
	static TArray<FNiagaraVariable> Attrs;
	if (Attrs.Num() == 0)
	{
		Attrs.Add(SYS_PARAM_PARTICLES_COMPONENTS_ENABLED);
	}
	return Attrs;
}

#endif // WITH_EDITORONLY_DATA

#undef LOCTEXT_NAMESPACE
