// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraComponentSettings.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraScript.h"
#include "NiagaraScriptRuntimeData.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstance.h"
#include "UObject/UObjectIterator.h"

namespace FNiagaraComponentSettings
{
	struct FNiagaraEmitterNameSettingsRef
	{
		FName SystemName;
		FName EmitterName;

		FNiagaraEmitterNameSettingsRef() = default;
		explicit FNiagaraEmitterNameSettingsRef(const FStringView& Pair)
		{
			int32 SepIndex;
			Pair.FindChar(':', SepIndex);
			if (SepIndex != INDEX_NONE && Pair.Len() > SepIndex + 1)
			{
				SystemName = FName(Pair.Mid(0, SepIndex));
				EmitterName = FName(Pair.Mid(SepIndex + 1));
			}
		}

		bool operator==(const FNiagaraEmitterNameSettingsRef& Other) const { return SystemName == Other.SystemName && EmitterName == Other.EmitterName; }
		bool operator!=(const FNiagaraEmitterNameSettingsRef& Other) const { return !(*this == Other); }
		bool IsValid() const { return !SystemName.IsNone() && !EmitterName.IsNone(); }
	};
	
	uint32 GetTypeHash(const FNiagaraEmitterNameSettingsRef& Var)
	{
		return HashCombine(GetTypeHash(Var.SystemName), GetTypeHash(Var.EmitterName));
	}

	bool									bNeedsSettingsUpdate = true;

	bool									bUseSystemDenyList = false;
	bool									bUseEmitterDenyList = false;
	bool									bAllowGpuEmitters = true;
	bool									bAllowGpuComputeFloat16 = true;

	TSet<FName>								SystemDenyList;
	TSet<FNiagaraEmitterNameSettingsRef>	EmitterDenyList;
	TSet<FNiagaraEmitterNameSettingsRef>	GpuEmitterDenyList;
	TSet<FName>								GpuDataInterfaceDenyList;

	FString									SystemDenyListString;
	FString									EmitterDenyListString;
	FString									GpuEmitterDenyListString;
	FString									GpuDataInterfaceDenyListString;
	FString									GpuRHIDenyListString;
	FString									GpuRHIAdapterDenyListString;
	FString									GpuDenyListString;

	static bool ParseIntoSet(const FString& StringList, TSet<FName>& OutSet)
	{
		TArray<FString> Names;
		StringList.ParseIntoArray(Names, TEXT(","));

		TSet<FName> ExistingSet;
		Swap(ExistingSet, OutSet);
		for (const FString& Name : Names)
		{
			OutSet.Emplace(Name);
		}
		return ExistingSet.Difference(OutSet).Num() > 0;
	}

	static bool ParseIntoSet(const FString& StringList, TSet<FNiagaraEmitterNameSettingsRef>& OutSet)
	{
		TArray<FString> Pairs;
		StringList.ParseIntoArray(Pairs, TEXT(","));

		TSet<FNiagaraEmitterNameSettingsRef> ExistingSet;
		Swap(ExistingSet, OutSet);
		for (const FString& Pair : Pairs)
		{
			FNiagaraEmitterNameSettingsRef EmitterRef(Pair);
			if (EmitterRef.IsValid())
			{
				OutSet.Emplace(EmitterRef);
			}
		}
		return ExistingSet.Difference(OutSet).Num() > 0;
	}

	static void UpdateScalabilityForSystems()
	{
		for (TObjectIterator<UNiagaraSystem> It; It; ++It)
		{
			if (UNiagaraSystem* System = *It)
			{
				System->UpdateScalability();
			}
		}
	}

	static void UpdateSystemDenyList(IConsoleVariable*)
	{
		const bool bWasChanged = ParseIntoSet(SystemDenyListString, SystemDenyList);
		bUseSystemDenyList = SystemDenyList.Num() > 0;
		if (bWasChanged)
		{
			UpdateScalabilityForSystems();
		}
	}

	static void UpdateEmitterDenyList(IConsoleVariable*)
	{
		bool bWasChanged = ParseIntoSet(EmitterDenyListString, EmitterDenyList);
		bWasChanged |= ParseIntoSet(GpuEmitterDenyListString, GpuEmitterDenyList);
		bWasChanged |= ParseIntoSet(GpuDataInterfaceDenyListString, GpuDataInterfaceDenyList);
		bUseEmitterDenyList = EmitterDenyList.Num() > 0 || GpuEmitterDenyList.Num() > 0 || GpuDataInterfaceDenyList.Num() > 0;
		if (bWasChanged)
		{
			UpdateScalabilityForSystems();
		}
	}

	void RequestUpdateSettings(IConsoleVariable*)
	{
		bNeedsSettingsUpdate = true;
	}

	static FAutoConsoleVariableRef CVarNiagaraSetSystemDenyList(
		TEXT("fx.Niagara.SetSystemDenyList"),
		SystemDenyListString,
		TEXT("Set the system deny List to use. (i.e. NS_SystemA,NS_SystemB)"),
		FConsoleVariableDelegate::CreateStatic(UpdateSystemDenyList), 
		ECVF_Scalability | ECVF_Default
	);
	static FAutoConsoleVariableRef CVarNiagaraSetEmitterDenyList(
		TEXT("fx.Niagara.SetEmitterDenyList"),
		EmitterDenyListString,
		TEXT("Set the emitter deny list to use. (i.e. NS_SystemA:EmitterA,NS_SystemB:EmitterA)"),
		FConsoleVariableDelegate::CreateStatic(UpdateEmitterDenyList),
		ECVF_Scalability | ECVF_Default
	);
	static FAutoConsoleVariableRef CVarNiagaraSetGpuEmitterDenyList(
		TEXT("fx.Niagara.SetGpuEmitterDenyList"),
		GpuEmitterDenyListString,
		TEXT("Set the Gpu emitter deny list to use. (i.e. NS_SystemA:EmitterA,NS_SystemB:EmitterA)"),
		FConsoleVariableDelegate::CreateStatic(UpdateEmitterDenyList),
		ECVF_Scalability | ECVF_Default
	);
	static FAutoConsoleVariableRef CVarNiagaraSetGpuDataInterfaceDenyList(
		TEXT("fx.Niagara.SetGpuDataInterfaceDenyList"),
		GpuDataInterfaceDenyListString,
		TEXT("Set the Gpu data interface deny list to use.  (i.e. UMyDataInteraceA,UMyDataInteraceB)"),
		FConsoleVariableDelegate::CreateStatic(UpdateEmitterDenyList),
		ECVF_Scalability | ECVF_Default
	);
	static FAutoConsoleVariableRef CVarNiagaraGpuRHIDenyList(
		TEXT("fx.Niagara.SetGpuRHIDenyList"),
		GpuRHIDenyListString,
		TEXT("Set Gpu RHI deny list to use, comma separated and uses wildcards, i.e. (*MyRHI*) would exclude anything that contains MyRHI"),
		FConsoleVariableDelegate::CreateStatic(RequestUpdateSettings),
		ECVF_Scalability | ECVF_Default
	);

	static FAutoConsoleVariableRef CVarNiagaraGpuRHIAdapterDenyList(
		TEXT("fx.Niagara.SetGpuRHIAdapterDenyList"),
		GpuRHIAdapterDenyListString,
		TEXT("Set Gpu RHI Adapter deny list to use, comma separated and uses wildcards, i.e. (*MyGpu*) would exclude anything that contains MyGpu"),
		FConsoleVariableDelegate::CreateStatic(RequestUpdateSettings),
		ECVF_Scalability | ECVF_Default
	);

	static FAutoConsoleVariableRef CVarNiagaraGpuDenyList(
		TEXT("fx.Niagara.SetGpuDenyList"),
		GpuDenyListString,
		TEXT("Set Gpu deny list to use, more targetted than to allow comparing OS,OSVersion,CPU,GPU.\n")
		TEXT("Format is OSLabel,OSVersion,CPU,GPU| blank entries are assumed to auto pass matching.\n")
		TEXT("For example, =\",,MyCpu,MyGpu+MyOS,,,\" would match MyCpu & MyGpu or MyOS."),
		FConsoleVariableDelegate::CreateStatic(RequestUpdateSettings),
		ECVF_Scalability | ECVF_Default
	);

	//-TODO: We can remove this once we confirm all RHIs report the correct values
	static bool GGpuEmitterCheckFloat16Support = true;
	static FAutoConsoleVariableRef CVarNiagaraGpuEmitterCheckFloat16Support(
		TEXT("fx.Niagara.GpuEmitterCheckFloat16Support"),
		GGpuEmitterCheckFloat16Support,
		TEXT("When enabled we check to see if the RHI has support for Float16 UAV read / write, if it doesn't GPU emitters that use Float16 are banned from running."),
		FConsoleVariableDelegate::CreateStatic(RequestUpdateSettings),
		ECVF_Scalability | ECVF_Default
	);

	void UpdateSettings()
	{
		if (!bNeedsSettingsUpdate || GDynamicRHI == nullptr)
		{
			return;
		}
		bNeedsSettingsUpdate = false;

		bool bShouldAllowGpuEmitters = FNiagaraUtilities::AllowGPUParticles();
		if (bShouldAllowGpuEmitters && GpuRHIDenyListString.Len() > 0)
		{
			TArray<FString> BanNames;
			GpuRHIDenyListString.ParseIntoArray(BanNames, TEXT(","));

			const FString RHIName = GDynamicRHI->GetName();
			if (BanNames.ContainsByPredicate([&RHIName](const FString& BanName) { return RHIName.MatchesWildcard(BanName); }))
			{
				bShouldAllowGpuEmitters = false;
			}
		}

		if (bShouldAllowGpuEmitters && GpuRHIAdapterDenyListString.Len() > 0)
		{
			TArray<FString> BanNames;
			GpuRHIAdapterDenyListString.ParseIntoArray(BanNames, TEXT(","));

			if (BanNames.ContainsByPredicate([](const FString& BanName) { return GRHIAdapterName.MatchesWildcard(BanName); }))
			{
				bShouldAllowGpuEmitters = false;
			}
		}

		if (bShouldAllowGpuEmitters && GpuDenyListString.Len() > 0)
		{
			TArray<FString> BanList;
			GpuDenyListString.ParseIntoArray(BanList, TEXT("+"));

			FString CategoryData[4];
			FPlatformMisc::GetOSVersions(CategoryData[0], CategoryData[1]);
			CategoryData[2] = FPlatformMisc::GetCPUBrand();
			CategoryData[3] = FPlatformMisc::GetPrimaryGPUBrand();

			TArray<FString> BanCategoryStrings;
			for ( const FString& Ban : BanList )
			{
				Ban.ParseIntoArray(BanCategoryStrings, TEXT(","), false);
				if (BanCategoryStrings.Num() == UE_ARRAY_COUNT(CategoryData))
				{
					bool bMatches = true;
					for (int32 i=0; i < UE_ARRAY_COUNT(CategoryData); ++i)
					{
						bMatches = BanCategoryStrings[i].IsEmpty() || CategoryData[i].MatchesWildcard(BanCategoryStrings[i]);
						if (bMatches == false)
						{
							break;
						}
					}
					if (bMatches)
					{
						bShouldAllowGpuEmitters = false;
						break;
					}
				}
			}
		}

		bool bNeedsScalabilityUpdate = false;
		if (bAllowGpuEmitters != bShouldAllowGpuEmitters)
		{
			bNeedsScalabilityUpdate = true;
			bAllowGpuEmitters = bShouldAllowGpuEmitters;
			if (bAllowGpuEmitters == false)
			{
				UE_LOGF(LogNiagara, Log,
					"GPU emitters are disabled for RHI(%ls) Adapter(%ls).  RHIDeny(%ls) AdapterDeny(%ls) GpuDenyList(%ls).",
					GDynamicRHI->GetName(), *GRHIAdapterName,
					*GpuRHIDenyListString, *GpuRHIAdapterDenyListString, *GpuDenyListString
				);
			}
			else
			{
				UE_LOGF(LogNiagara, Log, "GPU emitters are enabled for RHI(%ls) Adapter(%ls).", GDynamicRHI->GetName(), *GRHIAdapterName);
			}
		}

		const bool Float16UAVSupported = GGpuEmitterCheckFloat16Support ? UE::PixelFormat::HasCapabilities(EPixelFormat::PF_R16F, EPixelFormatCapabilities::TypedUAVLoad | EPixelFormatCapabilities::TypedUAVStore) : true;
		if (bAllowGpuComputeFloat16 != Float16UAVSupported)
		{
			bNeedsScalabilityUpdate = true;
			bAllowGpuComputeFloat16 = Float16UAVSupported;
			if (bAllowGpuComputeFloat16 == false)
			{
				UE_LOGF(LogNiagara, Log, "UAV Float16 is not supported, compressed GPU emitters will be disabled.");
			}
		}

		if (bNeedsScalabilityUpdate)
		{
			UpdateScalabilityForSystems();
		}
	}

	bool IsSystemAllowedToRun(const UNiagaraSystem* System)
	{
		check(System != nullptr);
		return bUseSystemDenyList ? !SystemDenyList.Contains(System->GetFName()) : true;
	}

	bool IsEmitterAllowedToRun(const FVersionedNiagaraEmitterData& EmitterData, const UNiagaraEmitter& NiagaraEmitter, const FGuid& EmitterHandleId)
	{
		if (bUseEmitterDenyList)
		{
			const UNiagaraSystem* NiagaraSystem = NiagaraEmitter.GetTypedOuter<UNiagaraSystem>();

			FNiagaraEmitterNameSettingsRef EmitterRef;
			EmitterRef.SystemName = NiagaraSystem ? NiagaraSystem->GetFName() : NAME_None;
			EmitterRef.EmitterName = FName(*NiagaraEmitter.GetUniqueEmitterName());

			if (EmitterDenyList.Contains(EmitterRef))
			{
				return false;
			}

			if (EmitterData.SimTarget == ENiagaraSimTarget::GPUComputeSim)
			{
				if (GpuEmitterDenyList.Contains(EmitterRef))
				{
					return false;
				}

				if (GpuDataInterfaceDenyList.Num() > 0 && NiagaraSystem != nullptr && EmitterData.GetGPUComputeScript() != nullptr)
				{
					const UNiagaraScript* GPUComputeScript = EmitterData.GetGPUComputeScript();
					TSharedPtr<const FNiagaraScriptRuntimeData> GPUComputeScriptRuntimeData = NiagaraSystem->GetScriptRuntimeData(EmitterHandleId, GPUComputeScript->GetUsage(), GPUComputeScript->GetUsageId());
					if (GPUComputeScriptRuntimeData.IsValid())
					{
						for (const FNiagaraScriptResolvedDataInterfaceInfo& DefaultDIInfo : GPUComputeScriptRuntimeData->GetResolvedDataInterfaces())
						{
							if (GpuDataInterfaceDenyList.Contains(DefaultDIInfo.ResolvedVariable.GetType().GetFName()))
							{
								return false;
							}
						}
					}
				}
			}
		}

		if (bAllowGpuEmitters == false)
		{
			return EmitterData.SimTarget != ENiagaraSimTarget::GPUComputeSim;
		}

		if (bAllowGpuComputeFloat16 == false && (EmitterData.SimTarget == ENiagaraSimTarget::GPUComputeSim))
		{
			// We have to lookup in the system to find if we used compressed attributes on this emitter or not
			if (UNiagaraSystem* OwnerSystem = NiagaraEmitter.GetTypedOuter<UNiagaraSystem>())
			{
				const TArray<FNiagaraEmitterHandle>& EmitterHandles = OwnerSystem->GetEmitterHandles();
				for (int32 iEmitter=0; iEmitter < EmitterHandles.Num(); ++iEmitter)
				{
					if (EmitterHandles[iEmitter].GetInstance().Emitter != &NiagaraEmitter)
					{
						continue;
					}

					const TArray<TSharedRef<const FNiagaraEmitterCompiledData>>& EmitterCompiledData = OwnerSystem->GetEmitterCompiledData();
					if (!EmitterCompiledData.IsValidIndex(iEmitter))
					{
						continue;
					}

					return EmitterCompiledData[iEmitter]->DataSetCompiledData.TotalHalfComponents == 0;
				}
			}
		}

		return true;
	}
}
