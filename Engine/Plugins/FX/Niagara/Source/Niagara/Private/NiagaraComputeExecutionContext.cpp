// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraComputeExecutionContext.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraDataInterface.h"
#include "NiagaraScript.h"
#include "NiagaraScriptRuntimeData.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraGPUInstanceCountManager.h"
#include "NiagaraGPUSystemTick.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraShader.h"

FNiagaraComputeExecutionContext::FNiagaraComputeExecutionContext()
	: MainDataSet(nullptr)
	, GPUScript(nullptr)
	, GPUScript_RT(nullptr)
{
}

FNiagaraComputeExecutionContext::~FNiagaraComputeExecutionContext()
{
	// EmitterInstanceReadback.GPUCountOffset should be INDEX_NONE at this point to ensure the index is reused.
	// When the ComputeDispatchInterface is being destroyed though, we don't free the index, but this would not be leaking.
	// check(EmitterInstanceReadback.GPUCountOffset == INDEX_NONE);
	SetDataToRender(nullptr);

	ExternalCBufferLayout = nullptr;
}

void FNiagaraComputeExecutionContext::Reset(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface)
{
	FNiagaraGpuComputeDispatchInterface* RT_ComputeDispatchInterface = ComputeDispatchInterface && !ComputeDispatchInterface->IsPendingKill() ? ComputeDispatchInterface : nullptr;
	ENQUEUE_RENDER_COMMAND(ResetRT)(
		[RT_ComputeDispatchInterface, RT_Context=this](FRHICommandListImmediate& RHICmdList)
		{
			RT_Context->ResetInternal(RT_ComputeDispatchInterface);
		}
	);
}

void FNiagaraComputeExecutionContext::InitParams(UNiagaraScript* InGPUComputeScript, TSharedPtr<const FNiagaraScriptRuntimeData> InScriptRuntimeData, const FNiagaraSimStageExecutionDataPtr& InSimStageExecData)
{
	if (InGPUComputeScript == nullptr || InScriptRuntimeData.IsValid() == false)
	{
		return;
	}

	GPUScript = InGPUComputeScript;
	ScriptRuntimeData = InScriptRuntimeData;
	SimStageExecData = InSimStageExecData;
	CombinedParamStore.InitFromOwningContext(InGPUComputeScript, ScriptRuntimeData.Get(), true);
	
	HasInterpolationParameters = GPUScript && GPUScript->GetComputedVMCompilationId().HasInterpolatedParameters();

#if DO_CHECK
	// DI Parameters are the same between all shader permutations so we can just get the first one
	FNiagaraShaderRef Shader = InGPUComputeScript->GetRenderThreadScript()->GetShaderGameThread(0);
	if (Shader.IsValid())
	{
		DIClassNames.Empty(Shader->GetDIParameters().Num());
		for (const FNiagaraDataInterfaceParamRef& DIParams : Shader->GetDIParameters())
		{
			DIClassNames.Add(DIParams.DIType.Get(Shader.GetPointerTable().DITypes)->GetClass()->GetFName());
		}
	}
	else
	{
		TSharedRef<FNiagaraShaderScriptParametersMetadata> ScriptParametersMetadata = InGPUComputeScript->GetRenderThreadScript()->GetScriptParametersMetadata();
		DIClassNames.Empty(ScriptParametersMetadata->DataInterfaceParamInfo.Num());
		for (const FNiagaraDataInterfaceGPUParamInfo& DIParams : ScriptParametersMetadata->DataInterfaceParamInfo)
		{
			DIClassNames.Emplace(DIParams.DIClassName);
		}
	}
#endif
}

bool FNiagaraComputeExecutionContext::IsOutputStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 SimulationStageIndex) const
{
	if (DIProxy && !DIProxy->GetSourceDIName().IsNone())
	{		
		return SimStageExecData->SimStageMetaData[SimulationStageIndex].OutputDestinations.Contains(DIProxy->GetSourceDIName());
	}
	return false;
}

bool FNiagaraComputeExecutionContext::IsInputStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 SimulationStageIndex) const
{
	if (DIProxy && !DIProxy->GetSourceDIName().IsNone())
	{
		return SimStageExecData->SimStageMetaData[SimulationStageIndex].InputDataInterfaces.Contains(DIProxy->GetSourceDIName());
	}
	return false;
}

bool FNiagaraComputeExecutionContext::IsIterationStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 SimulationStageIndex) const
{
	if (DIProxy && !DIProxy->GetSourceDIName().IsNone())
	{
		return SimStageExecData->SimStageMetaData[SimulationStageIndex].IterationSourceType == ENiagaraIterationSource::DataInterface && (SimStageExecData->SimStageMetaData[SimulationStageIndex].IterationDataInterface == DIProxy->GetSourceDIName());
	}
	return false;
}

FNiagaraDataInterfaceProxyRW* FNiagaraComputeExecutionContext::FindIterationInterface(const TArray<FNiagaraDataInterfaceProxyRW*>& InProxies, uint32 SimulationStageIndex) const
{
	// Particle stage
	if (SimStageExecData->SimStageMetaData[SimulationStageIndex].IterationSourceType != ENiagaraIterationSource::DataInterface )
	{
		return nullptr;
	}

	for (FNiagaraDataInterfaceProxyRW* Proxy : InProxies)
	{
		if (Proxy->GetSourceDIName() == SimStageExecData->SimStageMetaData[SimulationStageIndex].IterationDataInterface)
		{
			return Proxy;
		}
	}

	UE_LOGF(LogNiagara, Verbose, "FNiagaraComputeExecutionContext::FindIterationInterface could not find IterationInterface %ls", *SimStageExecData->SimStageMetaData[SimulationStageIndex].IterationDataInterface.ToString());

	return nullptr;
}

void FNiagaraComputeExecutionContext::DirtyDataInterfaces()
{
	CombinedParamStore.MarkInterfacesDirty();
}

bool FNiagaraComputeExecutionContext::Tick(FNiagaraSystemInstance* ParentSystemInstance)
{
	check(ParentSystemInstance);
	if (CombinedParamStore.GetInterfacesDirty())
	{
#if DO_CHECK
		// We must make sure that the data interfaces match up between the original script values and our overrides...
		const TArray<UNiagaraDataInterface*>& DataInterfaces = CombinedParamStore.GetDataInterfaces();
		bool bHasMismatchedDIs = DIClassNames.Num() != DataInterfaces.Num();
		if (!bHasMismatchedDIs)
		{
			for (int32 i = 0; i < DIClassNames.Num(); ++i)
			{
				const UNiagaraDataInterface* UsedDataInterface = DataInterfaces[i];
				const FName UsedClassName = UsedDataInterface ? UsedDataInterface->GetClass()->GetFName() : NAME_None;
				bHasMismatchedDIs |= DIClassNames[i] != UsedClassName;
			}
		}

		if (bHasMismatchedDIs)
		{
			UE_LOGF(LogNiagara, Error, "Niagara GPU Execution Context Mismatch with DataInterfaces. System (%ls) will not run", *GetNameSafe(ParentSystemInstance->GetSystem()));

			const int32 MaxDataInterfaces = FMath::Max(DIClassNames.Num(), DataInterfaces.Num());
			for (int32 i = 0; i < MaxDataInterfaces; ++i)
			{
				const FName SourceType = DataInterfaces.IsValidIndex(i) ? DataInterfaces[i]->GetClass()->GetFName() : NAME_None;
				const FName ExpectedType = DIClassNames.IsValidIndex(i) ? DIClassNames[i] : NAME_None;
				const TCHAR* ErrorType = SourceType.IsNone() || ExpectedType.IsNone() ? TEXT("Invalid Array") : TEXT("");
				if (SourceType != ExpectedType)
				{
					ErrorType = TEXT("Mismatched");
				}
				UE_LOGF(LogNiagara, Error, " - DI(%d) SourceType(%ls) ExpectedType(%ls) %ls", i, *SourceType.ToString(), *ExpectedType.ToString(), ErrorType);
			}

			return false;
		}

#endif
		if (CombinedParamStore.GetPositionDataDirty())
		{
			CombinedParamStore.ResolvePositions(ParentSystemInstance->GetLWCConverter());
		}
		CombinedParamStore.Tick();
	}

	return true;
}

bool FNiagaraComputeExecutionContext::OptionalContexInit(FNiagaraSystemInstance* ParentSystemInstance)
{
	if (GPUScript && ScriptRuntimeData.IsValid())
	{
		FNiagaraVMExecutableData& VMData = GPUScript->GetVMExecutableData();
		TConstArrayView<FNiagaraScriptResolvedDataInterfaceInfo> ScriptDataInterfaceInfos = ScriptRuntimeData->GetResolvedDataInterfaces();

		if (VMData.IsValid() && VMData.bNeedsGPUContextInit)
		{
			const TArray<UNiagaraDataInterface*>& DataInterfaces = CombinedParamStore.GetDataInterfaces();
			for (int32 i = 0; i < DataInterfaces.Num(); i++)
			{
				UNiagaraDataInterface* Interface = DataInterfaces[i];

				int32 UserPtrIdx = ScriptDataInterfaceInfos[i].UserPtrIdx;
				if (UserPtrIdx != INDEX_NONE)
				{
					void* InstData = ParentSystemInstance->FindDataInterfaceInstanceData(Interface);
					if (Interface->NeedsGPUContextInit())
					{
						Interface->GPUContextInit(ScriptDataInterfaceInfos[i], VMData.CalledVMExternalFunctions, InstData, ParentSystemInstance);
					}
				}
			}
		}
	}
	return true;
}

void FNiagaraComputeExecutionContext::PostTick()
{
	//If we're for interpolated spawn, copy over the previous frame's parameters into the Prev parameters.
	if (HasInterpolationParameters)
	{
		CombinedParamStore.CopyCurrToPrev();
	}
}

void FNiagaraComputeExecutionContext::ResetInternal(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface)
{
	checkf(IsInRenderingThread(), TEXT("Can only reset the gpu context from the render thread"));

	if (ComputeDispatchInterface)
	{
		ComputeDispatchInterface->GetGPUInstanceCounterManager().FreeEntry(CountOffset_RT);
	}

	SetCurrentNumInstances(0);
	CountOffset_RT = INDEX_NONE;
	EmitterInstanceReadback.GPUCountOffset = INDEX_NONE;

	SetDataToRender(nullptr);
}

void FNiagaraComputeExecutionContext::SetDataToRender(FNiagaraDataBuffer* InDataToRender)
{
	DataToRender = InDataToRender;

	// This call the DataToRender should be equal to the TranslucentDataToRender so we can release the read ref
	if (TranslucentDataToRender)
	{
		ensure((DataToRender == nullptr) || (DataToRender == TranslucentDataToRender));
		TranslucentDataToRender = nullptr;
	}
}

void FNiagaraComputeExecutionContext::SetTranslucentDataToRender(FNiagaraDataBuffer* InTranslucentDataToRender)
{
	TranslucentDataToRender = InTranslucentDataToRender;
}

#if WITH_MGPU
void FNiagaraComputeExecutionContext::SetMultiViewPreviousDataToRender(FNiagaraDataBuffer* InMultiViewPreviousDataToRender)
{
	MultiViewPreviousDataToRender = InMultiViewPreviousDataToRender;
}
#endif //WITH_MGPU

int32 FNiagaraComputeExecutionContext::GetConstantBufferSize() const
{
	return Align(CombinedParamStore.GetExternalParameterSize(), SHADER_PARAMETER_STRUCT_ALIGNMENT);
}

uint8* FNiagaraComputeExecutionContext::WriteConstantBufferInstanceData(uint8* InTargetBuffer, FNiagaraComputeInstanceData& InstanceData) const
{
	const int32 InterpFactor = HasInterpolationParameters ? 2 : 1;

	InstanceData.ExternalParamDataSize = InterpFactor * GetConstantBufferSize();
	InstanceData.ExternalParamData = InTargetBuffer;

	const TArray<uint8>& ParameterDataArray = CombinedParamStore.GetParameterDataArray();
	const int32 SourceDataSize = CombinedParamStore.GetExternalParameterSize();
	const int32 ConstantBufferSize = GetConstantBufferSize();

	check(SourceDataSize <= ParameterDataArray.Num());
	FMemory::Memcpy(InstanceData.ExternalParamData, ParameterDataArray.GetData(), SourceDataSize);

	if (HasInterpolationParameters)
	{
		check(SourceDataSize + SourceDataSize <= ParameterDataArray.Num());
		FMemory::Memcpy(InstanceData.ExternalParamData + ConstantBufferSize, ParameterDataArray.GetData() + SourceDataSize, SourceDataSize);
	}

	return InTargetBuffer + InstanceData.ExternalParamDataSize;
}

bool FNiagaraComputeInstanceData::IsOutputStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 SimulationStageIndex) const
{
	return Context->IsOutputStage(DIProxy, SimulationStageIndex);
}

bool FNiagaraComputeInstanceData::IsInputStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 SimulationStageIndex) const
{
	return Context->IsInputStage(DIProxy, SimulationStageIndex);
}

bool FNiagaraComputeInstanceData::IsIterationStage(FNiagaraDataInterfaceProxy* DIProxy, uint32 SimulationStageIndex) const
{
	return Context->IsIterationStage(DIProxy, SimulationStageIndex);
}

FNiagaraDataInterfaceProxyRW* FNiagaraComputeInstanceData::FindIterationInterface(uint32 SimulationStageIndex) const
{
	return Context->FindIterationInterface(IterationDataInterfaceProxies, SimulationStageIndex);
}

