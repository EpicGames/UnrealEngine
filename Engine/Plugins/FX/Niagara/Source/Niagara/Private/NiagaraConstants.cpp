// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraConstants.h"
#include "NiagaraCommon.h"
#include "NiagaraModule.h"
#include "NiagaraVariableMetaData.h"

#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "FNiagaraConstants"

TArray<FNiagaraVariable> FNiagaraConstants::SystemParameters;
TArray<FNiagaraVariable> FNiagaraConstants::TranslatorParameters;
TArray<FNiagaraVariable> FNiagaraConstants::SwitchParameters;
TArray<FNiagaraVariable> FNiagaraConstants::OldPositionTypes;
TMap<FName, FNiagaraVariable> FNiagaraConstants::UpdatedSystemParameters;
TMap<FNiagaraVariable, FText> FNiagaraConstants::SystemStrMap;
TArray<FNiagaraVariable> FNiagaraConstants::Attributes;
TMap<FNiagaraVariableBase, FString> FNiagaraConstants::AttrDefaultsStrMap;
TMap<FNiagaraVariableBase, FText> FNiagaraConstants::AttrDescStrMap;
TMap<FNiagaraVariableBase, FNiagaraVariableMetaData> FNiagaraConstants::AttrMetaData;
TMap<FNiagaraVariableBase, FNiagaraVariable> FNiagaraConstants::AttrDefaultsValueMap;
TMap<FNiagaraVariableBase, FNiagaraVariable> FNiagaraConstants::AttrDataSetKeyMap;
TArray<FNiagaraVariable> FNiagaraConstants::EngineManagedAttributes;

const FName FNiagaraConstants::InputPinName("InputPin");
const FName FNiagaraConstants::OutputPinName("OutputPin");

const FName FNiagaraConstants::UserNamespace(TEXT("User"));
const FName FNiagaraConstants::EngineNamespace(TEXT("Engine"));
const FName FNiagaraConstants::SystemNamespace(TEXT("System"));
const FName FNiagaraConstants::EmitterNamespace(TEXT("Emitter"));
const FName FNiagaraConstants::ParticleAttributeNamespace(TEXT("Particles"));
const FName FNiagaraConstants::ModuleNamespace(TEXT("Module"));
const FName FNiagaraConstants::OutputNamespace(TEXT("Output"));
const FName FNiagaraConstants::TransientNamespace(TEXT("Transient"));
const FName FNiagaraConstants::StackContextNamespace(TEXT("StackContext"));
const FName FNiagaraConstants::DataInstanceNamespace(TEXT("DataInstance"));
const FName FNiagaraConstants::StaticSwitchNamespace(TEXT("StaticSwitch"));
const FName FNiagaraConstants::ArrayNamespace(TEXT("Array"));
const FName FNiagaraConstants::ParameterCollectionNamespace(TEXT("NPC"));
const FString FNiagaraConstants::InitialPrefix(TEXT("Initial"));
const FName FNiagaraConstants::LocalNamespace(TEXT("Local"));
const FName FNiagaraConstants::InitialNamespace(TEXT("Initial"));
const FName FNiagaraConstants::PreviousNamespace(TEXT("Previous"));
const FName FNiagaraConstants::OwnerNamespace(TEXT("Owner"));
const FName FNiagaraConstants::SharedNamespace(TEXT("Shared"));

const FName FNiagaraConstants::EngineOwnerScopeName(TEXT("EngineOwner"));
const FName FNiagaraConstants::EngineSystemScopeName(TEXT("EngineSystem"));
const FName FNiagaraConstants::EngineEmitterScopeName(TEXT("EngineEmitter"));
const FName FNiagaraConstants::CustomScopeName(TEXT("Custom"));

const FString FNiagaraConstants::AssignmentNodePrefixString(TRANSLATOR_SET_VARIABLES_STR);
const FString FNiagaraConstants::EngineNamespaceString(TEXT("Engine"));
const FString FNiagaraConstants::EmitterNamespaceString(TEXT("Emitter"));
const FString FNiagaraConstants::OutputNamespaceString(TEXT("Output"));
const FString FNiagaraConstants::ModuleNamespaceString(TEXT("Module"));
const FString FNiagaraConstants::ParameterCollectionNamespaceString(TEXT("NPC"));
const FString FNiagaraConstants::ParticleAttributeNamespaceString(TEXT("Particles"));
const FString FNiagaraConstants::RapidIterationParametersNamespaceString(TEXT("Constants"));
const FString FNiagaraConstants::StackContextNamespaceString(TEXT("StackContext"));
const FString FNiagaraConstants::SystemNamespaceString(TEXT("System"));
const FString FNiagaraConstants::UserNamespaceString(TEXT("User"));
const FString FNiagaraConstants::InternalNamespaceString(TEXT("__INTERNAL__"));
const FString FNiagaraConstants::PreviousNamespaceString(TEXT("Previous"));

const FName FNiagaraConstants::InputScopeName(TEXT("Input"));
const FName FNiagaraConstants::OutputScopeName(TEXT("Output"));
const FName FNiagaraConstants::UniqueOutputScopeName(TEXT("OutputModule"));
const FName FNiagaraConstants::ScriptTransientScopeName(TEXT("ScriptTransient"));
const FName FNiagaraConstants::ScriptPersistentScopeName(TEXT("ScriptPersistent"));

const int32 FNiagaraConstants::MaxCategoryNameLength(128);
const int32 FNiagaraConstants::MaxParameterLength(256);
const int32 FNiagaraConstants::MaxScriptNameLength(256);

void FNiagaraConstants::Init()
{
	if (SystemParameters.Num() == 0)
	{
		SystemParameters.Add(SYS_PARAM_ENGINE_WORLD_DELTA_TIME);
		SystemParameters.Add(SYS_PARAM_ENGINE_DELTA_TIME);
		SystemParameters.Add(SYS_PARAM_ENGINE_INV_DELTA_TIME);
		SystemParameters.Add(SYS_PARAM_ENGINE_TIME);
		SystemParameters.Add(SYS_PARAM_ENGINE_REAL_TIME);
		SystemParameters.Add(SYS_PARAM_ENGINE_QUALITY_LEVEL);

		SystemParameters.Add(SYS_PARAM_ENGINE_POSITION);
		SystemParameters.Add(SYS_PARAM_ENGINE_SCALE);
		SystemParameters.Add(SYS_PARAM_ENGINE_VELOCITY);
		SystemParameters.Add(SYS_PARAM_ENGINE_X_AXIS);
		SystemParameters.Add(SYS_PARAM_ENGINE_Y_AXIS);
		SystemParameters.Add(SYS_PARAM_ENGINE_Z_AXIS);
		SystemParameters.Add(SYS_PARAM_ENGINE_LWC_TILE);
		SystemParameters.Add(SYS_PARAM_ENGINE_EXEC_INDEX);
		SystemParameters.Add(SYS_PARAM_ENGINE_EMITTER_ID);

		SystemParameters.Add(SYS_PARAM_ENGINE_ROTATION);

		SystemParameters.Add(SYS_PARAM_ENGINE_LOCAL_TO_WORLD);
		SystemParameters.Add(SYS_PARAM_ENGINE_WORLD_TO_LOCAL);
		SystemParameters.Add(SYS_PARAM_ENGINE_LOCAL_TO_WORLD_TRANSPOSED);
		SystemParameters.Add(SYS_PARAM_ENGINE_WORLD_TO_LOCAL_TRANSPOSED);
		SystemParameters.Add(SYS_PARAM_ENGINE_LOCAL_TO_WORLD_NO_SCALE);
		SystemParameters.Add(SYS_PARAM_ENGINE_WORLD_TO_LOCAL_NO_SCALE);

		SystemParameters.Add(SYS_PARAM_ENGINE_LOD_DISTANCE);
		SystemParameters.Add(SYS_PARAM_ENGINE_LOD_DISTANCE_FRACTION);
		SystemParameters.Add(SYS_PARAM_ENGINE_TIME_SINCE_RENDERED);

		SystemParameters.Add(SYS_PARAM_ENGINE_EXECUTION_STATE);

		SystemParameters.Add(SYS_PARAM_ENGINE_EXEC_COUNT);
		SystemParameters.Add(SYS_PARAM_ENGINE_EMITTER_NUM_PARTICLES);
		SystemParameters.Add(SYS_PARAM_ENGINE_EMITTER_SIMULATION_POSITION);
		SystemParameters.Add(SYS_PARAM_ENGINE_EMITTER_TOTAL_SPAWNED_PARTICLES);
		SystemParameters.Add(SYS_PARAM_ENGINE_EMITTER_SPAWN_COUNT_SCALE);
		SystemParameters.Add(SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS_ALIVE);
		SystemParameters.Add(SYS_PARAM_ENGINE_SYSTEM_SIGNIFICANCE_INDEX);
		SystemParameters.Add(SYS_PARAM_ENGINE_SYSTEM_RANDOM_SEED);
		SystemParameters.Add(SYS_PARAM_ENGINE_SYSTEM_CURRENT_TIME_STEP);
		SystemParameters.Add(SYS_PARAM_ENGINE_SYSTEM_NUM_TIME_STEPS);
		SystemParameters.Add(SYS_PARAM_ENGINE_SYSTEM_TIME_STEP_FRACTION);
		SystemParameters.Add(SYS_PARAM_ENGINE_SYSTEM_NUM_PARTICLES);
		SystemParameters.Add(SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS);
		SystemParameters.Add(SYS_PARAM_ENGINE_NUM_SYSTEM_INSTANCES);
		SystemParameters.Add(SYS_PARAM_ENGINE_GLOBAL_SPAWN_COUNT_SCALE);
		SystemParameters.Add(SYS_PARAM_ENGINE_GLOBAL_SYSTEM_COUNT_SCALE);
		SystemParameters.Add(SYS_PARAM_ENGINE_SYSTEM_AGE);
		SystemParameters.Add(SYS_PARAM_ENGINE_SYSTEM_TICK_COUNT);
		SystemParameters.Add(SYS_PARAM_EMITTER_AGE);
		SystemParameters.Add(SYS_PARAM_EMITTER_LOCALSPACE);
		SystemParameters.Add(SYS_PARAM_EMITTER_DETERMINISM);
		SystemParameters.Add(SYS_PARAM_EMITTER_OVERRIDE_GLOBAL_SPAWN_COUNT_SCALE);
		SystemParameters.Add(SYS_PARAM_EMITTER_RANDOM_SEED);
		SystemParameters.Add(SYS_PARAM_ENGINE_EMITTER_INSTANCE_SEED);
		SystemParameters.Add(SYS_PARAM_EMITTER_SPAWN_GROUP);
	}

	if (TranslatorParameters.Num() == 0)
	{
		TranslatorParameters.Add(TRANSLATOR_PARAM_BEGIN_DEFAULTS);
		TranslatorParameters.Add(TRANSLATOR_PARAM_CALL_ID);
	}

	if (SwitchParameters.Num() == 0)
	{
		SwitchParameters.Add(SYS_PARAM_EMITTER_LOCALSPACE);
		SwitchParameters.Add(SYS_PARAM_EMITTER_DETERMINISM);
		SwitchParameters.Add(SYS_PARAM_EMITTER_OVERRIDE_GLOBAL_SPAWN_COUNT_SCALE);
		SwitchParameters.Add(SYS_PARAM_EMITTER_SIMULATION_TARGET);
		SwitchParameters.Add(SYS_PARAM_EMITTER_INTERP_SPAWN);
		SwitchParameters.Add(SYS_PARAM_SCRIPT_USAGE);
		SwitchParameters.Add(SYS_PARAM_SCRIPT_CONTEXT);
		SwitchParameters.Add(SYS_PARAM_FUNCTION_DEBUG_STATE);
	}

	if (OldPositionTypes.Num() == 0)
	{
		OldPositionTypes.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(),  SYS_PARAM_PARTICLES_POSITION.GetName()));
		OldPositionTypes.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), FName("Particles.Initial.Position")));
		OldPositionTypes.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), FName("Particles.Previous.Position")));
		OldPositionTypes.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), FName(StackContextNamespaceString + ".Position")));
		OldPositionTypes.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), FName(StackContextNamespaceString + ".Initial.Position")));
		OldPositionTypes.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), FName(StackContextNamespaceString + ".Previous.Position")));
		OldPositionTypes.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(),  SYS_PARAM_ENGINE_POSITION.GetName()));
		OldPositionTypes.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(),  SYS_PARAM_ENGINE_EMITTER_SIMULATION_POSITION.GetName()));
	}

	if (UpdatedSystemParameters.Num() == 0)
	{
		UpdatedSystemParameters.Add(FName(TEXT("System World Delta Time")), SYS_PARAM_ENGINE_WORLD_DELTA_TIME);
		UpdatedSystemParameters.Add(FName(TEXT("System Delta Time")), SYS_PARAM_ENGINE_DELTA_TIME);
		UpdatedSystemParameters.Add(FName(TEXT("System Inv Delta Time")), SYS_PARAM_ENGINE_INV_DELTA_TIME);
		UpdatedSystemParameters.Add(FName(TEXT("System Position")), SYS_PARAM_ENGINE_POSITION);
		UpdatedSystemParameters.Add(FName(TEXT("System Velocity")), SYS_PARAM_ENGINE_VELOCITY);
		UpdatedSystemParameters.Add(FName(TEXT("System X Axis")), SYS_PARAM_ENGINE_X_AXIS);
		UpdatedSystemParameters.Add(FName(TEXT("System Y Axis")), SYS_PARAM_ENGINE_Y_AXIS);
		UpdatedSystemParameters.Add(FName(TEXT("System Z Axis")), SYS_PARAM_ENGINE_Z_AXIS);
		UpdatedSystemParameters.Add(FName(TEXT("System Rotation")), SYS_PARAM_ENGINE_ROTATION);

		UpdatedSystemParameters.Add(FName(TEXT("System Local To World")), SYS_PARAM_ENGINE_LOCAL_TO_WORLD);
		UpdatedSystemParameters.Add(FName(TEXT("System World To Local")), SYS_PARAM_ENGINE_WORLD_TO_LOCAL);
		UpdatedSystemParameters.Add(FName(TEXT("System Local To World Transposed")), SYS_PARAM_ENGINE_LOCAL_TO_WORLD_TRANSPOSED);
		UpdatedSystemParameters.Add(FName(TEXT("System World To Local Transposed")), SYS_PARAM_ENGINE_WORLD_TO_LOCAL_TRANSPOSED);
		UpdatedSystemParameters.Add(FName(TEXT("System Local To World No Scale")), SYS_PARAM_ENGINE_LOCAL_TO_WORLD_NO_SCALE);
		UpdatedSystemParameters.Add(FName(TEXT("System World To Local No Scale")), SYS_PARAM_ENGINE_WORLD_TO_LOCAL_NO_SCALE);

		UpdatedSystemParameters.Add(FName(TEXT("Emitter Execution Count")), SYS_PARAM_ENGINE_EXEC_COUNT);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter Spawn Rate")), SYS_PARAM_EMITTER_SPAWNRATE);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter Spawn Interval")), SYS_PARAM_EMITTER_SPAWN_INTERVAL);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter Interp Spawn Start Dt")), SYS_PARAM_EMITTER_INTERP_SPAWN_START_DT);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter Spawn Group")), SYS_PARAM_EMITTER_SPAWN_GROUP);

		UpdatedSystemParameters.Add(FName(TEXT("World Delta Time")), SYS_PARAM_ENGINE_WORLD_DELTA_TIME);
		UpdatedSystemParameters.Add(FName(TEXT("Delta Time")), SYS_PARAM_ENGINE_DELTA_TIME);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter Age")), SYS_PARAM_EMITTER_AGE);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter Local Space")), SYS_PARAM_EMITTER_LOCALSPACE);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter Random Seed")), SYS_PARAM_EMITTER_RANDOM_SEED);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter Instance Seed")), SYS_PARAM_ENGINE_EMITTER_INSTANCE_SEED);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter Determinism")), SYS_PARAM_EMITTER_DETERMINISM);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter Interpolated Spawn")), SYS_PARAM_EMITTER_INTERP_SPAWN);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter Override Global Spawn Count Scale")), SYS_PARAM_EMITTER_OVERRIDE_GLOBAL_SPAWN_COUNT_SCALE);
		UpdatedSystemParameters.Add(FName(TEXT("Effect Position")), SYS_PARAM_ENGINE_POSITION);
		UpdatedSystemParameters.Add(FName(TEXT("Effect Velocity")), SYS_PARAM_ENGINE_VELOCITY);
		UpdatedSystemParameters.Add(FName(TEXT("Effect X Axis")), SYS_PARAM_ENGINE_X_AXIS);
		UpdatedSystemParameters.Add(FName(TEXT("Effect Y Axis")), SYS_PARAM_ENGINE_Y_AXIS);
		UpdatedSystemParameters.Add(FName(TEXT("Effect Z Axis")), SYS_PARAM_ENGINE_Z_AXIS);
		UpdatedSystemParameters.Add(FName(TEXT("Effect Rotation")), SYS_PARAM_ENGINE_ROTATION);

		UpdatedSystemParameters.Add(FName(TEXT("Effect Local To World")), SYS_PARAM_ENGINE_LOCAL_TO_WORLD);
		UpdatedSystemParameters.Add(FName(TEXT("Effect World To Local")), SYS_PARAM_ENGINE_WORLD_TO_LOCAL);
		UpdatedSystemParameters.Add(FName(TEXT("Effect Local To World Transposed")), SYS_PARAM_ENGINE_LOCAL_TO_WORLD_TRANSPOSED);
		UpdatedSystemParameters.Add(FName(TEXT("Effect World To Local Transposed")), SYS_PARAM_ENGINE_WORLD_TO_LOCAL_TRANSPOSED);
		UpdatedSystemParameters.Add(FName(TEXT("Execution Count")), SYS_PARAM_ENGINE_EXEC_COUNT);
		UpdatedSystemParameters.Add(FName(TEXT("Spawn Rate")), SYS_PARAM_EMITTER_SPAWNRATE);
		UpdatedSystemParameters.Add(FName(TEXT("Spawn Interval")), SYS_PARAM_EMITTER_SPAWN_INTERVAL);
		UpdatedSystemParameters.Add(FName(TEXT("Interp Spawn Start Dt")), SYS_PARAM_EMITTER_INTERP_SPAWN_START_DT);
		UpdatedSystemParameters.Add(FName(TEXT("Spawn Group")), SYS_PARAM_EMITTER_SPAWN_GROUP);
		UpdatedSystemParameters.Add(FName(TEXT("Inv Delta Time")), SYS_PARAM_ENGINE_INV_DELTA_TIME);

		UpdatedSystemParameters.Add(FName(TEXT("Exec Index")), SYS_PARAM_ENGINE_EXEC_INDEX);
		UpdatedSystemParameters.Add(FName(TEXT("Emitter ID")), SYS_PARAM_ENGINE_EMITTER_ID);
	}

	if (SystemStrMap.Num() == 0)
	{
		SystemStrMap.Add(SYS_PARAM_ENGINE_WORLD_DELTA_TIME, LOCTEXT("EngineWorldDeltaTimeDesc", "Time in seconds since the last tick of the engine."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_DELTA_TIME, LOCTEXT("EngineDeltaTimeDesc", "Time in seconds since the last simulation tick."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_INV_DELTA_TIME, LOCTEXT("EngineInvDeltaTimeDesc", "One over Engine.DeltaTime"));
		SystemStrMap.Add(SYS_PARAM_ENGINE_TIME, LOCTEXT("EngineTimeDesc", "Time in seconds since level began play, but IS paused when the game is paused, and IS dilated/clamped."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_REAL_TIME, LOCTEXT("EngineRealTimeDesc", "Time in seconds since level began play, but IS NOT paused when the game is paused, and IS NOT dilated/clamped."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_QUALITY_LEVEL, LOCTEXT("EngineQualityLevelDesc", "The current value of fx.Niagara.QualityLevel."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_SYSTEM_AGE, LOCTEXT("EngineSystemTimeDesc", "Time in seconds since the system was first created. Managed by the NiagaraSystemInstance in code."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_SYSTEM_TICK_COUNT, LOCTEXT("EngineSystemTickCount", "The current tick of this system simulation."));

		SystemStrMap.Add(SYS_PARAM_ENGINE_POSITION, LOCTEXT("EnginePositionDesc", "The owning component's position in world space."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_SCALE, LOCTEXT("EngineScaleDesc", "The owning component's scale in world space."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_VELOCITY, LOCTEXT("EngineVelocityDesc", "The owning component's velocity in world space."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_X_AXIS, LOCTEXT("XAxisDesc", "The X-axis of the owning component."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_Y_AXIS, LOCTEXT("YAxisDesc", "The Y-axis of the owning component."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_Z_AXIS, LOCTEXT("ZAxisDesc", "The Z-axis of the owning component."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_ROTATION, LOCTEXT("EngineRotationDesc", "The owning component's rotation in world space."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_LWC_TILE, LOCTEXT("EngineLWCTileDesc", "Due to large world coordinates, the simulation position can be shifted from the actual world position to allow for more accuracy. This is the tile the system is shifted by, so (SimulationPosition + Tile * TileSize) gives the original world position. The x,y,z components of this vector are the tile and the w component is the tile size."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_EXEC_INDEX, LOCTEXT("EngineExecIndexDesc", "Index of this particle in the current execution. For example, in a spawn script this gives the index of the particle being spawned which can be used to interpolate it's position."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_EMITTER_ID, LOCTEXT("EngineEmitterIDDesc", "ID of the currently executing Emitter. This can be used to access or otherwise target a specific emitter in certain Data Interface functions. This is only valid for Emitter and Particle scripts."));

		SystemStrMap.Add(SYS_PARAM_ENGINE_LOCAL_TO_WORLD, LOCTEXT("LocalToWorldDesc", "Owning component's local space to world space transform matrix."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_WORLD_TO_LOCAL, LOCTEXT("WorldToLocalDesc", "Owning component's world space to local space transform matrix."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_LOCAL_TO_WORLD_TRANSPOSED, LOCTEXT("LocalToWorldTransposeDesc", "Owning component's local space to world space transform matrix transposed."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_WORLD_TO_LOCAL_TRANSPOSED, LOCTEXT("WorldToLocalTransposeDesc", "Owning component's world space to local space transform matrix transposed."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_LOCAL_TO_WORLD_NO_SCALE, LOCTEXT("LocalToWorldNoScaleDesc", "Owning component's local space to world space transform matrix with scaling removed."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_WORLD_TO_LOCAL_NO_SCALE, LOCTEXT("WorldToLocalNoScaleDesc", "Owning component's world space to local space transform matrix with scaling removed."));

		SystemStrMap.Add(SYS_PARAM_ENGINE_TIME_SINCE_RENDERED, LOCTEXT("TimeSinceRendered", "The time in seconds that have passed since this system was last rendered."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_LOD_DISTANCE, LOCTEXT("LODDistance", "The distance to use in LOD scaling/culling. Typically, the distance to the nearest camera."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_LOD_DISTANCE_FRACTION, LOCTEXT("LODDistanceFraction", "The distance fraction between this system and it's max culling distance defined in it's EffectType scalabiltiy settings."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_EXECUTION_STATE, LOCTEXT("ExecutionState", "The execution state of the systems owner. Takes precedence over the systems internal execution state."));

		SystemStrMap.Add(SYS_PARAM_ENGINE_EXEC_COUNT, LOCTEXT("ExecCountDesc", "Total number of items we are iterating over in this script invocation."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_EMITTER_NUM_PARTICLES, LOCTEXT("EmitterNumParticles", "The number of particles for this emitter at the beginning of simulation. Should only be used in Emitter scripts."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_EMITTER_SIMULATION_POSITION, LOCTEXT("EngineEmitterSimulationPosition", "The frame of reference space which the Emitter simulation is relative to. Is 0,0,0 for an Emitter in local space, and is Engine.Owner.Position for an Emitter in world space."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_EMITTER_TOTAL_SPAWNED_PARTICLES, LOCTEXT("EmitterTotalSpawnedParticles", "The total number of particles spawned for this emitter instance since creation.  The value is cleared when the simulation is reset or reinitialized."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_EMITTER_SPAWN_COUNT_SCALE, LOCTEXT("EmitterSpawnCountScale", "The global spawn count scale, based on fx.NiagaraGlobalSpawnCountScale."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_EMITTER_INSTANCE_SEED, LOCTEXT("EmitterInstanceSeed", "A random seed that changes with every execution of the emitter instance."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS_ALIVE, LOCTEXT("SystemNumEmittersAlive", "The number of emitters still alive attached to this system. Should only be used in System scripts."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_SYSTEM_SIGNIFICANCE_INDEX, LOCTEXT("SystemSignificanceIndex", "Index denoting how significant this system instance is in relation to others of the same system in this scene. e.g. 0 is the most significanct instance."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_SYSTEM_RANDOM_SEED, LOCTEXT("SystemRandomSeed", "A random seed controlled used for generating system random numbers."));
		
		SystemStrMap.Add(SYS_PARAM_ENGINE_SYSTEM_CURRENT_TIME_STEP, LOCTEXT("CurrentTimeStep", "The current time step."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_SYSTEM_NUM_TIME_STEPS, LOCTEXT("NumTimeSteps", "Number of steps used, can be > 1 when using fixed delta time."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_SYSTEM_TIME_STEP_FRACTION, LOCTEXT("TimeStepFraction", "The fraction of the world delta time integrated for the current sub step."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_SYSTEM_NUM_PARTICLES, LOCTEXT("NumParticles", "The total active particles for all emitters in this system. Note that counts for GPU emitters will be latent and may not be exactly accurate/up to date for a give frame."));

		SystemStrMap.Add(SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS, LOCTEXT("SystemNumEmitters", "The number of emitters attached to this system. Should only be used in System scripts."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_NUM_SYSTEM_INSTANCES, LOCTEXT("SystemNumInstances", "The number of instances of this system currently ticking. Should only be used in System scripts."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_GLOBAL_SPAWN_COUNT_SCALE, LOCTEXT("GlobalSpawnCountScale", "Global Spawn Count Scale. Should only be used in System scripts."));
		SystemStrMap.Add(SYS_PARAM_ENGINE_GLOBAL_SYSTEM_COUNT_SCALE, LOCTEXT("GlobalSystemCountScale", "Global System Count Scale. Should only be used in System scripts."));
	}

	if (Attributes.Num() == 0)
	{
		Attributes.Add(SYS_PARAM_PARTICLES_UNIQUE_ID);
		Attributes.Add(SYS_PARAM_PARTICLES_ID);
		Attributes.Add(SYS_PARAM_PARTICLES_POSITION);
		Attributes.Add(SYS_PARAM_PARTICLES_VELOCITY);
		Attributes.Add(SYS_PARAM_PARTICLES_COLOR);
		Attributes.Add(SYS_PARAM_PARTICLES_SPRITE_ROTATION);
		Attributes.Add(SYS_PARAM_PARTICLES_NORMALIZED_AGE);
		Attributes.Add(SYS_PARAM_PARTICLES_SPRITE_SIZE);
		Attributes.Add(SYS_PARAM_PARTICLES_SPRITE_FACING);
		Attributes.Add(SYS_PARAM_PARTICLES_SPRITE_ALIGNMENT);
		Attributes.Add(SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX);
		Attributes.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM);
		Attributes.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1);
		Attributes.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2);
		Attributes.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3);
		Attributes.Add(SYS_PARAM_PARTICLES_SCALE);
		Attributes.Add(SYS_PARAM_PARTICLES_LIFETIME);
		Attributes.Add(SYS_PARAM_PARTICLES_MESH_ORIENTATION);
		Attributes.Add(SYS_PARAM_PARTICLES_MESH_INDEX);
		Attributes.Add(SYS_PARAM_PARTICLES_CAMERA_OFFSET);
		Attributes.Add(SYS_PARAM_PARTICLES_UV_SCALE);
		Attributes.Add(SYS_PARAM_PARTICLES_PIVOT_OFFSET);
		Attributes.Add(SYS_PARAM_PARTICLES_MATERIAL_RANDOM);
		Attributes.Add(SYS_PARAM_PARTICLES_LIGHT_RADIUS);
		Attributes.Add(SYS_PARAM_PARTICLES_LIGHT_ENABLED);
		Attributes.Add(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
		Attributes.Add(SYS_PARAM_PARTICLES_COMPONENTS_ENABLED);
		Attributes.Add(SYS_PARAM_PARTICLES_RIBBONID);
		Attributes.Add(SYS_PARAM_PARTICLES_RIBBONWIDTH);
		Attributes.Add(SYS_PARAM_PARTICLES_RIBBONTWIST);
		Attributes.Add(SYS_PARAM_PARTICLES_RIBBONFACING);
		Attributes.Add(SYS_PARAM_PARTICLES_RIBBONLINKORDER);
		Attributes.Add(RIBBONUVDISTANCE);
		Attributes.Add(SYS_PARAM_PARTICLES_RIBBONU0OVERRIDE);
		Attributes.Add(SYS_PARAM_PARTICLES_RIBBONV0RANGEOVERRIDE);
		Attributes.Add(SYS_PARAM_PARTICLES_RIBBONU1OVERRIDE);
		Attributes.Add(SYS_PARAM_PARTICLES_RIBBONV1RANGEOVERRIDE);


		Attributes.Add(SYS_PARAM_INSTANCE_ALIVE);
	}

	if (AttrDataSetKeyMap.Num() == 0)
	{
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_POSITION, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_POSITION));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_VELOCITY, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_VELOCITY));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_COLOR, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_COLOR));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_SPRITE_ROTATION, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_SPRITE_ROTATION));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_NORMALIZED_AGE, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_NORMALIZED_AGE));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_SPRITE_SIZE, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_SPRITE_SIZE));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_SPRITE_FACING, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_SPRITE_FACING));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_SPRITE_ALIGNMENT, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_SPRITE_ALIGNMENT));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_SCALE, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_SCALE));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_LIFETIME, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_LIFETIME));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_MESH_ORIENTATION, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_MESH_ORIENTATION));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_MESH_INDEX, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_MESH_INDEX));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_CAMERA_OFFSET, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_CAMERA_OFFSET));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_UV_SCALE, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_UV_SCALE));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_PIVOT_OFFSET, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_PIVOT_OFFSET));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_MATERIAL_RANDOM, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_MATERIAL_RANDOM));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_LIGHT_ENABLED, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_LIGHT_ENABLED));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_LIGHT_RADIUS, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_LIGHT_RADIUS));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_LIGHT_EXPONENT, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_LIGHT_EXPONENT));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_VISIBILITY_TAG, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_VISIBILITY_TAG));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_COMPONENTS_ENABLED, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_COMPONENTS_ENABLED));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_RIBBONID, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_RIBBONID));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_RIBBONWIDTH, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_RIBBONWIDTH));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_RIBBONTWIST, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_RIBBONTWIST));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_RIBBONFACING, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_RIBBONFACING));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_RIBBONLINKORDER, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_RIBBONLINKORDER));
		AttrDataSetKeyMap.Add(RIBBONUVDISTANCE, GetAttributeAsParticleDataSetKey(RIBBONUVDISTANCE));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_RIBBONU0OVERRIDE, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_RIBBONU0OVERRIDE));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_RIBBONV0RANGEOVERRIDE, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_RIBBONV0RANGEOVERRIDE));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_RIBBONU1OVERRIDE, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_RIBBONU1OVERRIDE));
		AttrDataSetKeyMap.Add(SYS_PARAM_PARTICLES_RIBBONV1RANGEOVERRIDE, GetAttributeAsParticleDataSetKey(SYS_PARAM_PARTICLES_RIBBONV1RANGEOVERRIDE));
	}

	if (AttrDefaultsStrMap.Num() == 0)
	{
		FNiagaraVariable Var;
		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_POSITION, SYS_PARAM_ENGINE_EMITTER_SIMULATION_POSITION.GetName().ToString());
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_POSITION, SYS_PARAM_ENGINE_EMITTER_SIMULATION_POSITION);
		
		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_VELOCITY, TEXT("0.0,0.0,0.0"));
		Var = SYS_PARAM_PARTICLES_VELOCITY;
		Var.SetValue<FVector3f>(FVector3f(0.0f, 0.0f, 0.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_VELOCITY, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_COLOR, FLinearColor(1.0f, 1.0f, 1.0f, 1.0f).ToString());
		Var = SYS_PARAM_PARTICLES_COLOR;
		Var.SetValue<FLinearColor>(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_COLOR, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_SPRITE_ROTATION, TEXT("0.0"));
		Var = SYS_PARAM_PARTICLES_SPRITE_ROTATION;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_SPRITE_ROTATION, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_NORMALIZED_AGE, TEXT("0.0"));
		Var = SYS_PARAM_PARTICLES_NORMALIZED_AGE;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_NORMALIZED_AGE, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_SPRITE_SIZE, TEXT("X=50.0 Y=50.0"));
		Var = SYS_PARAM_PARTICLES_SPRITE_SIZE;
		Var.SetValue<FVector2f>(FVector2f(50.0f, 50.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_SPRITE_SIZE, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_SPRITE_FACING, TEXT("1.0,0.0,0.0"));
		Var = SYS_PARAM_PARTICLES_SPRITE_FACING;
		Var.SetValue<FVector3f>(FVector3f(1.0f, 0.0f, 0.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_SPRITE_FACING, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_SPRITE_ALIGNMENT, TEXT("1.0,0.0,0.0"));
		Var = SYS_PARAM_PARTICLES_SPRITE_ALIGNMENT;
		Var.SetValue<FVector3f>(FVector3f(1.0f, 0.0f, 0.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_SPRITE_ALIGNMENT, Var);
		
		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX, TEXT("0.0"));
		Var = SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM, TEXT("1.0,1.0,1.0,1.0"));
		Var = SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM;
		Var.SetValue<FVector4f>(FVector4f(1.0f, 1.0f, 1.0f, 1.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1, TEXT("1.0,1.0,1.0,1.0"));
		Var = SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1;
		Var.SetValue<FVector4f>(FVector4f(1.0f, 1.0f, 1.0f, 1.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2, TEXT("1.0,1.0,1.0,1.0"));
		Var = SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2;
		Var.SetValue<FVector4f>(FVector4f(1.0f, 1.0f, 1.0f, 1.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3, TEXT("1.0,1.0,1.0,1.0"));
		Var = SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3;
		Var.SetValue<FVector4f>(FVector4f(1.0f, 1.0f, 1.0f, 1.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_SCALE, TEXT("1.0,1.0,1.0"));
		Var = SYS_PARAM_PARTICLES_SCALE;
		Var.SetValue<FVector3f>(FVector3f(1.0f, 1.0f, 1.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_SCALE, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_LIFETIME, TEXT("5.0"));
		Var = SYS_PARAM_PARTICLES_LIFETIME;
		Var.SetValue<float>(5.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_LIFETIME, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_MESH_ORIENTATION, TEXT("0.0,0.0,0.0,1.0"));
		Var = SYS_PARAM_PARTICLES_MESH_ORIENTATION;
		Var.SetValue<FQuat4f>(FQuat4f::Identity);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_MESH_ORIENTATION, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_MESH_INDEX, TEXT("0"));
		Var = SYS_PARAM_PARTICLES_MESH_INDEX;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_MESH_INDEX, Var);
		
		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_CAMERA_OFFSET, TEXT("0.0"));
		Var = SYS_PARAM_PARTICLES_CAMERA_OFFSET;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_CAMERA_OFFSET, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_UV_SCALE, TEXT("X=1.0 Y=1.0"));
		Var = SYS_PARAM_PARTICLES_UV_SCALE;
		Var.SetValue<FVector2f>(FVector2f(1.0f, 1.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_UV_SCALE, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_PIVOT_OFFSET, TEXT("X=0.5 Y=0.5"));
		Var = SYS_PARAM_PARTICLES_PIVOT_OFFSET;
		Var.SetValue<FVector2f>(FVector2f(0.5f, 0.5f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_PIVOT_OFFSET, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_MATERIAL_RANDOM, TEXT("0.0"));
		Var = SYS_PARAM_PARTICLES_MATERIAL_RANDOM;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_MATERIAL_RANDOM, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_LIGHT_RADIUS, TEXT("100.0"));
		Var = SYS_PARAM_PARTICLES_LIGHT_RADIUS;
		Var.SetValue<float>(100.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_LIGHT_RADIUS, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_LIGHT_ENABLED, TEXT("true"));
		Var = SYS_PARAM_PARTICLES_LIGHT_ENABLED;
		Var.SetValue<bool>(true);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_LIGHT_ENABLED, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_LIGHT_EXPONENT, TEXT("10.0"));
		Var = SYS_PARAM_PARTICLES_LIGHT_EXPONENT;
		Var.SetValue<float>(10.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_LIGHT_EXPONENT, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING, TEXT("0.0"));
		Var = SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE, TEXT("1.0"));
		Var = SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE;
		Var.SetValue<float>(1.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_VISIBILITY_TAG, TEXT("0"));
		Var = SYS_PARAM_PARTICLES_VISIBILITY_TAG;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_VISIBILITY_TAG, Var);		

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_COMPONENTS_ENABLED, TEXT("true"));
		Var = SYS_PARAM_PARTICLES_COMPONENTS_ENABLED;
		Var.SetValue<bool>(true);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_COMPONENTS_ENABLED, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_RIBBONID, TEXT("0"));
		Var = SYS_PARAM_PARTICLES_RIBBONID;
		Var.SetValue<FNiagaraID>(FNiagaraID());
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_RIBBONID, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_RIBBONWIDTH, TEXT("1.0"));
		Var = SYS_PARAM_PARTICLES_RIBBONWIDTH;
		Var.SetValue<float>(1.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_RIBBONWIDTH, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_RIBBONTWIST, TEXT("0.0"));
		Var = SYS_PARAM_PARTICLES_RIBBONTWIST;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_RIBBONTWIST, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_RIBBONFACING, TEXT("0.0, 0.0, 1.0"));
		Var = SYS_PARAM_PARTICLES_RIBBONFACING;
		Var.SetValue<FVector3f>(FVector3f(0.0f, 0.0f, 1.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_RIBBONFACING, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_RIBBONLINKORDER, TEXT("0"));
		Var = SYS_PARAM_PARTICLES_RIBBONLINKORDER;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_RIBBONLINKORDER, Var);

		AttrDefaultsStrMap.Add(RIBBONUVDISTANCE, TEXT("0"));
		Var = RIBBONUVDISTANCE;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(RIBBONUVDISTANCE, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_RIBBONU0OVERRIDE, TEXT("0"));
		Var = SYS_PARAM_PARTICLES_RIBBONU0OVERRIDE;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_RIBBONU0OVERRIDE, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_RIBBONV0RANGEOVERRIDE, TEXT("0.0, 1.0"));
		Var = SYS_PARAM_PARTICLES_RIBBONV0RANGEOVERRIDE;
		Var.SetValue<FVector2f>(FVector2f(0.0f, 1.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_RIBBONV0RANGEOVERRIDE, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_RIBBONU1OVERRIDE, TEXT("0"));
		Var = SYS_PARAM_PARTICLES_RIBBONU1OVERRIDE;
		Var.SetValue<float>(0.0f);
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_RIBBONU1OVERRIDE, Var);

		AttrDefaultsStrMap.Add(SYS_PARAM_PARTICLES_RIBBONV1RANGEOVERRIDE, TEXT("0.0, 1.0"));
		Var = SYS_PARAM_PARTICLES_RIBBONV1RANGEOVERRIDE;
		Var.SetValue<FVector2f>(FVector2f(0.0f, 1.0f));
		AttrDefaultsValueMap.Add(SYS_PARAM_PARTICLES_RIBBONV1RANGEOVERRIDE, Var);
	}

	if (AttrDescStrMap.Num() == 0)
	{
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_POSITION, LOCTEXT("PositionDesc", "The position of the particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_VELOCITY, LOCTEXT("VelocityDesc", "The velocity in cm/s of the particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_COLOR, LOCTEXT("ColorDesc", "The color of the particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_SPRITE_ROTATION, LOCTEXT("SpriteRotDesc", "The screen aligned roll of the particle in degrees."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_NORMALIZED_AGE, LOCTEXT("NormalizedAgeDesc", "The age in seconds divided by lifetime in seconds. Useful for animation as the value is between 0 and 1."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_SPRITE_SIZE, LOCTEXT("SpriteSizeDesc", "The size of the sprite quad."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_SPRITE_FACING, LOCTEXT("FacingDesc", "Makes the surface of the sprite face towards a custom vector. Must be used with the SpriteRenderer's CustomFacingVector FacingMode option."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_SPRITE_ALIGNMENT, LOCTEXT("AlignmentDesc", "Imagine the texture having an arrow pointing up, this attribute makes the arrow point towards the alignment axis. Must be used with the SpriteRenderer's CustomAlignment Alignment option."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX, LOCTEXT("SubImageIndexDesc", "A value from 0 to the number of entries in the table of SubUV images."));
		FText DynParamText = LOCTEXT("DynamicMaterialParameterDesc", "The 4-float vector used to send custom data to renderer.");
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM, DynParamText);
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1, DynParamText);
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2, DynParamText);
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3, DynParamText);
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_SCALE, LOCTEXT("ScaleParamDesc", "The XYZ scale of the non-sprite based particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_LIFETIME, LOCTEXT("LifetimeParamDesc", "The lifetime of a particle in seconds."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_MESH_ORIENTATION, LOCTEXT("MeshOrientParamDesc", "The axis-angle rotation to be applied to the mesh particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_MESH_INDEX, LOCTEXT("MeshIndexParamDesc", "The index of the mesh to use to render this particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_CAMERA_OFFSET, LOCTEXT("CamOffsetParamDesc", "Used to offset position in the direction of the camera. The value is multiplied by the direction vector from the camera to the particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_UV_SCALE, LOCTEXT("UVScalerParamDesc", "Used to multiply the generated UVs for Sprite renderers."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_PIVOT_OFFSET, LOCTEXT("PivotOffsetParamDesc", "The central pivot point of the sprite in UV space"));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_MATERIAL_RANDOM, LOCTEXT("MaterialRandomParamDesc", "Used to drive the Particle Random node in the Material Editor. Without this set, any Particle Randoms will get 0.0."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_LIGHT_RADIUS, LOCTEXT("LightRadiusParamDesc", "Used to drive the radius of the light when using a Light renderer."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_LIGHT_ENABLED, LOCTEXT("LightEnabledParamDesc", "This can be used with the Light renderer to dynamically control if a light exists on a per-particle basis."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_LIGHT_EXPONENT, LOCTEXT("LightExponentParamDesc", "Used to drive the attenuation of the light when using a Light renderer without inverse squared falloff enabled."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING, LOCTEXT("LightVolumetricScatteringParamDesc", "Used to drive the volumetric scattering intensity of the light when using a Light renderer."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE, LOCTEXT("LightSpecularScaleParamDesc", "Used to drive the specular scale of the light when using a Light renderer."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_LIGHT_DIFFUSE_SCALE, LOCTEXT("LightDiffuseScaleParamDesc", "Used to drive the diffuse scale of the light when using a Light renderer."));
		AttrDescStrMap.Add(SYS_PARAM_INSTANCE_ALIVE, LOCTEXT("AliveParamDesc", "Used to determine whether or not this particle instance is still valid or if it can be deleted."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_VISIBILITY_TAG, LOCTEXT("VisibilityTag", "Used for selecting renderers to use when rendering this particle. Without this, the particle will render in all renderers"));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_COMPONENTS_ENABLED, LOCTEXT("ComponentRenderEnabledParamDesc", "Used to check if component rendering should be enabled on a per-particle basis. Without this, the each particle will spawn a component."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_RIBBONID, LOCTEXT("RibbonIDDesc", "Sets the ribbon id for a particle. Particles with the same ribbon id will be connected into a ribbon."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_RIBBONWIDTH, LOCTEXT("RibbonWidthDesc", "Sets the ribbon width for a particle, in Unreal units."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_RIBBONTWIST, LOCTEXT("RibbonTwistDesc", "Sets the ribbon twist for a particle, in degrees."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_RIBBONFACING, LOCTEXT("RibbonFacingDesc", "Sets the facing vector of the ribbon at the particle position, or the side vector the ribbon's width is extended along, depending on the selected facing mode."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_RIBBONLINKORDER, LOCTEXT("RibbonLinkOrderDesc", "Explicit order for linking particles within a ribbon. Particles of the same ribbon id will be connected into a ribbon in incrementing order of this attribute value."));
		AttrDescStrMap.Add(RIBBONUVDISTANCE, LOCTEXT("RibbonUVDistanceDesc", "Distance from start of ribbon that the particle lies."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_RIBBONU0OVERRIDE, LOCTEXT("RibbonU0OverrideDesc", "Overrides the U component of the UV0 texture coordinate of a ribbon particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_RIBBONV0RANGEOVERRIDE, LOCTEXT("RibbonV0RangeOverrideDesc", "Overrides the V range across the width of a ribbon for the UV0 texture coordinate of a particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_RIBBONU1OVERRIDE, LOCTEXT("RibbonU1OverrideDesc", "Overrides the U component of the UV1 texture coordinate of a ribbon particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_RIBBONV1RANGEOVERRIDE, LOCTEXT("RibbonV1RangeOverrideDesc", "Overrides the V range across the width of a ribbon for the UV1 texture coordinate of a particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_ID, LOCTEXT("IDDesc", "Engine managed particle attribute that is a persistent ID for each particle."));
		AttrDescStrMap.Add(SYS_PARAM_PARTICLES_UNIQUE_ID, LOCTEXT("UniqueIDDesc", "Engine managed particle attribute that is a unique ID for each particle. The ID is incremented for each new particle spawned.")); 
	}

	if (AttrMetaData.Num() == 0)
	{
		// Add the engine default attributes..
		{
			for (FNiagaraVariable Var : Attributes)
			{
				FNiagaraVariableMetaData MetaData;
				MetaData.Description = GetAttributeDescription(Var);
				AttrMetaData.Add(Var, MetaData);
			}
		}

		// Add the engine constants..
		{
			for (FNiagaraVariable Var : SystemParameters)
			{
				FNiagaraVariableMetaData MetaData;
				MetaData.Description = GetEngineConstantDescription(Var);
				AttrMetaData.Add(Var, MetaData);
			}
		}

	}

	if (EngineManagedAttributes.Num() == 0)
	{
		EngineManagedAttributes.Add(SYS_PARAM_PARTICLES_ID);
		// NOTE(mv): UniqueID needs to be distinct from ID, as the latter is not guaranteed to be contiguous and will reuse labels
		//           It is unique and sequential, never resetting until the simulation is reset. 
		//           It needs to be engine managed, otherwise the scripts cannot write to it when it isn't referenced in any scripts. 
		EngineManagedAttributes.Add(SYS_PARAM_PARTICLES_UNIQUE_ID); 
	}
}

const TArray<FNiagaraVariable>& FNiagaraConstants::GetEngineConstants()
{
	check(SystemParameters.Num() != 0);
	return SystemParameters;
}

const TArray<FNiagaraVariable>& FNiagaraConstants::GetTranslatorConstants()
{
	check(TranslatorParameters.Num() != 0);
	return TranslatorParameters;
}

const TArray<FNiagaraVariable>& FNiagaraConstants::GetStaticSwitchConstants()
{
	check(SwitchParameters.Num() != 0);
	return SwitchParameters;
}

bool FNiagaraConstants::IsEngineManagedAttribute(const FNiagaraVariableBase& Var)
{
	return EngineManagedAttributes.Contains(Var);
}

FNiagaraVariable FNiagaraConstants::UpdateEngineConstant(const FNiagaraVariable& InVar)
{
	const FNiagaraVariable* FoundSystemVar = FindEngineConstant(InVar);
	if (nullptr != FoundSystemVar)
	{
		return *FoundSystemVar;
	}
	else
	{
		check(UpdatedSystemParameters.Num() != 0);
		const FNiagaraVariable* FoundSystemVarUpdate = UpdatedSystemParameters.Find(InVar.GetName());
		if (FoundSystemVarUpdate != nullptr)
		{
			return *FoundSystemVarUpdate;
		}
	}
	return InVar;

}

const FNiagaraVariable* FNiagaraConstants::FindEngineConstant(const FNiagaraVariable& InVar)
{
	return GetEngineConstants().FindByPredicate([&](const FNiagaraVariable& Var)
	{
		return Var.GetName() == InVar.GetName();
	});
}

const FNiagaraVariable * FNiagaraConstants::FindStaticSwitchConstant(const FName& InName)
{
	return GetStaticSwitchConstants().FindByPredicate([&](const FNiagaraVariable& Var)
	{
		return Var.GetName() == InName;
	});
}

FText FNiagaraConstants::GetEngineConstantDescription(const FNiagaraVariable& InAttribute)
{
	check(SystemStrMap.Num() != 0);
	FText* FoundStr = SystemStrMap.Find(InAttribute);
	if (FoundStr != nullptr && !FoundStr->IsEmpty())
	{
		return *FoundStr;
	}
	return FText();
}

const TArray<FNiagaraVariable>& FNiagaraConstants::GetOldPositionTypeVariables()
{
	check(OldPositionTypes.Num() != 0);
	return OldPositionTypes;
}

const TArray<FNiagaraVariable>& FNiagaraConstants::GetCommonParticleAttributes()
{
	check(Attributes.Num() != 0);
	return Attributes;
}

FString FNiagaraConstants::GetAttributeDefaultValue(const FNiagaraVariableBase& InAttribute)
{
	check(AttrDefaultsStrMap.Num() != 0);
	FString* FoundStr = AttrDefaultsStrMap.Find(InAttribute);
	if (FoundStr != nullptr && !FoundStr->IsEmpty())
	{
		return *FoundStr;
	}
	return FString();
}

FText FNiagaraConstants::GetAttributeDescription(const FNiagaraVariableBase& InAttribute)
{
	check(AttrDescStrMap.Num() != 0);
	FText* FoundStr = AttrDescStrMap.Find(InAttribute);
	if (FoundStr != nullptr && !FoundStr->IsEmpty())
	{
		return *FoundStr;
	}
	return FText();
}

bool FNiagaraConstants::IsNiagaraConstant(const FNiagaraVariableBase& InVar)
{
	if (GetConstantMetaData(InVar) != nullptr)
	{
		return true;
	}
	return false;
}

const FNiagaraVariableMetaData* FNiagaraConstants::GetConstantMetaData(const FNiagaraVariableBase& InVar)
{
	check(AttrMetaData.Num() != 0);
	return AttrMetaData.Find(InVar);
}

FNiagaraVariable FNiagaraConstants::GetAttributeWithDefaultValue(const FNiagaraVariableBase& InAttribute)
{
	check(AttrDefaultsValueMap.Num() != 0);
	FNiagaraVariable* FoundValue = AttrDefaultsValueMap.Find(InAttribute);
	if (FoundValue != nullptr)
	{
		return *FoundValue;
	}
	return FNiagaraVariable();
}

FNiagaraVariableBase FNiagaraConstants::GetAttributeAsParticleDataSetKey(const FNiagaraVariableBase& InVar)
{
	static FString ParticlesString = TEXT("Particles.");
	static FString StackContextString = TEXT("StackContext.");

	FNiagaraVariableBase OutVar = InVar;

	TStringBuilder<128> DataSetName;
	InVar.GetName().ToString(DataSetName);

	FStringView DataSetNameView(DataSetName);
	if (DataSetNameView.StartsWith(ParticlesString))
	{
		DataSetNameView.RemovePrefix(ParticlesString.Len());
	}
	else if (DataSetNameView.StartsWith(StackContextString))
	{
		DataSetNameView.RemovePrefix(StackContextString.Len());
	}
	OutVar.SetName(FName(DataSetNameView));
	return OutVar;
}

FNiagaraVariableBase FNiagaraConstants::GetAttributeAsEmitterDataSetKey(const FNiagaraVariableBase& InVar)
{
	static FString EmitterString = TEXT("Emitter.");
	static FString StackContextString = TEXT("StackContext.");

	FNiagaraVariableBase OutVar = InVar;

	TStringBuilder<128> DataSetName;
	InVar.GetName().ToString(DataSetName);

	FStringView DataSetNameView(DataSetName);
	if ( DataSetNameView.StartsWith(EmitterString) )
	{
		DataSetNameView.RemovePrefix(EmitterString.Len());
	}
	else if (DataSetNameView.StartsWith(StackContextString))
	{
		DataSetNameView.RemovePrefix(StackContextString.Len());
	}
	OutVar.SetName(FName(DataSetNameView));
	return OutVar;
}
FNiagaraVariableAttributeBinding FNiagaraConstants::GetAttributeDefaultBinding(const FNiagaraVariableBase& InVar)
{
	if (AttrDefaultsValueMap.Num() == 0)
	{
		Init();
	}

	FNiagaraVariableAttributeBinding Binding;
	Binding.Setup(InVar, GetAttributeWithDefaultValue(InVar));
	return Binding;
}

const FNiagaraVariable* FNiagaraConstants::GetKnownConstant(const FName& InName, bool bAllowPartialNameMatch)
{
	return GetKnownConstantInfo(InName, bAllowPartialNameMatch).ConstantVar;
}

const FNiagaraKnownConstantInfo FNiagaraConstants::GetKnownConstantInfo(const FName& InName, bool bAllowPartialNameMatch)
{
	const TArray<FNiagaraVariable>& EngineConstants = GetEngineConstants();

	if (!bAllowPartialNameMatch)
	{
		const FNiagaraVariable* FoundSystemVar = EngineConstants.FindByPredicate([&](const FNiagaraVariable& Var)
		{
			return Var.GetName() == InName;
		});

		if (FoundSystemVar)
		{
			return FNiagaraKnownConstantInfo(FoundSystemVar, ENiagaraKnownConstantType::Other);
		}
	}
	else
	{
		int32 FoundIdx = FNiagaraVariable::SearchArrayForPartialNameMatch(EngineConstants, InName);
		if (FoundIdx != INDEX_NONE)
		{
			return FNiagaraKnownConstantInfo(&EngineConstants[FoundIdx], ENiagaraKnownConstantType::Other);
		}
	}

	const TArray<FNiagaraVariable>& LocalAttributes = GetCommonParticleAttributes();
	if (!bAllowPartialNameMatch)
	{
		const FNiagaraVariable* FoundAttribVar = LocalAttributes.FindByPredicate([&](const FNiagaraVariable& Var)
		{
			return Var.GetName() == InName;
		});

		if (FoundAttribVar)
		{
			return FNiagaraKnownConstantInfo(FoundAttribVar, ENiagaraKnownConstantType::Attribute);
		}
	}
	else
	{
		int32 FoundIdx = FNiagaraVariable::SearchArrayForPartialNameMatch(Attributes, InName);
		if (FoundIdx != INDEX_NONE)
		{
			return FNiagaraKnownConstantInfo(&Attributes[FoundIdx], ENiagaraKnownConstantType::Attribute);
		}
	}

	return FNiagaraKnownConstantInfo(nullptr, ENiagaraKnownConstantType::Other);
}

#undef LOCTEXT_NAMESPACE
