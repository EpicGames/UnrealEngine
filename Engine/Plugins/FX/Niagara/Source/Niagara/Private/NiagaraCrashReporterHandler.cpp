// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCrashReporterHandler.h"

#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Misc/ScopeLock.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemSimulation.h"
#include "NiagaraSystem.h"

#if WITH_NIAGARA_CRASHREPORTER

static int32 GbEnableNiagaraCRHandler = 0;
static FAutoConsoleVariableRef CVarEnableNiagaraCRHandler(
	TEXT("fx.EnableNiagaraCRHandler"),
	GbEnableNiagaraCRHandler,
	TEXT("If > 0 Niagara will push some state into the crash reporter. This is not free so should not be used unless actively tracking a crash in the wild. Even then it should only be enabled on the platforms needed etc. \n"),
	ECVF_Default);

class FNiagaraCrashReporterHandler
{
private:
	enum class EThread { Game, Render, Other };

	struct FThreadScopeInfo
	{
		EThread Thread;
		TArray<FString> Stack;

		FThreadScopeInfo()
			: Thread(GetThreadType())
		{}

		static EThread GetThreadType()
		{
			if (IsInGameThread())
			{
				return EThread::Game;
			}

			if (IsInActualRenderingThread())
			{
				return EThread::Render;
			}

			return EThread::Other;
		}
	};

public:
	FNiagaraCrashReporterHandler()
	{
		NullString = TEXT("nullptr");
	}

	~FNiagaraCrashReporterHandler()
	{
	}

	static FNiagaraCrashReporterHandler& Get()
	{
		static TUniquePtr<FNiagaraCrashReporterHandler> Instance = MakeUnique<FNiagaraCrashReporterHandler>();
		return *Instance.Get();
	}

	void PushInfo(const FString& Info)
	{
		const uint32 ThreadID = FPlatformTLS::GetCurrentThreadId();

		FScopeLock LockGuard(&RWGuard);
		ThreadScopeInfoStack.FindOrAdd(ThreadID).Stack.Push(Info);
		UpdateInfo();
	}

	void PushInfo(const FNiagaraSystemInstance* Inst)
	{
		PushInfo(Inst ? Inst->GetCrashReporterTag() : NullString);
	}

	void PushInfo(const FNiagaraSystemSimulation* SystemSim)
	{
		PushInfo(SystemSim ? SystemSim->GetCrashReporterTag() : NullString);
	}

	void PushInfo(const UNiagaraSystem* System)
	{
		PushInfo(System ? System->GetCrashReporterTag() : NullString);
	}

	void PopInfo()
	{
		const uint32 ThreadID = FPlatformTLS::GetCurrentThreadId();

		FScopeLock LockGuard(&RWGuard);
		ThreadScopeInfoStack[ThreadID].Stack.Pop();
		UpdateInfo();
	}

private:
	void UpdateInfo()
	{
		static const FString CrashReportKey = TEXT("NiagaraCRInfo");
		static const FString GameThreadString = TEXT("GameThread");
		static const FString RenderThreadString = TEXT("RenderThread");
		static const FString OtherThreadString = TEXT("OtherThread");

		CurrentInfo.Empty();
		for (auto it = ThreadScopeInfoStack.CreateIterator(); it; ++it)
		{
			if (it->Value.Stack.Num() == 0)
			{
				continue;
			}

			if (it->Value.Thread == EThread::Game)
			{
				CurrentInfo.Append(GameThreadString);
			}
			else if (it->Value.Thread == EThread::Render)
			{
				CurrentInfo.Append(RenderThreadString);
			}
			else
			{
				CurrentInfo.Append(OtherThreadString);
			}
			CurrentInfo.AppendChar('(');
			CurrentInfo.AppendInt(it->Key);
			CurrentInfo.AppendChars(") ", 2);
			CurrentInfo.Append(it->Value.Stack.Last());
			CurrentInfo.AppendChar('\n');
		}

		FGenericCrashContext::SetEngineData(CrashReportKey, CurrentInfo);
	}

private:
	FCriticalSection				RWGuard;
	TMap<uint32, FThreadScopeInfo>	ThreadScopeInfoStack;
	FString							CurrentInfo;
	FString							NullString;
};

FNiagaraCrashReporterScope::FNiagaraCrashReporterScope(const FNiagaraSystemInstance* Inst)
{
	bWasEnabled = GbEnableNiagaraCRHandler != 0;
	if (bWasEnabled)
	{
		FNiagaraCrashReporterHandler::Get().PushInfo(Inst);
	}
}

FNiagaraCrashReporterScope::FNiagaraCrashReporterScope(const FNiagaraSystemSimulation* Sim)
{
	bWasEnabled = GbEnableNiagaraCRHandler != 0;
	if (bWasEnabled)
	{
		FNiagaraCrashReporterHandler::Get().PushInfo(Sim);
	}
}

FNiagaraCrashReporterScope::FNiagaraCrashReporterScope(const UNiagaraSystem* System)
{
	bWasEnabled = GbEnableNiagaraCRHandler != 0;
	if (bWasEnabled)
	{
		FNiagaraCrashReporterHandler::Get().PushInfo(System);
	}
}

FNiagaraCrashReporterScope::~FNiagaraCrashReporterScope()
{
	if(bWasEnabled)
	{
		FNiagaraCrashReporterHandler::Get().PopInfo();
	}
}

#endif //WITH_NIAGARA_CRASHREPORTER
