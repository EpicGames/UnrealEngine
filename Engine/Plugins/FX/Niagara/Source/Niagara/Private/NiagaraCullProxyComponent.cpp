// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCullProxyComponent.h"
#include "NiagaraEffectType.h"
#include "NiagaraSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraCullProxyComponent)

//////////////////////////////////////////////////////////////////////////

const FName UNiagaraCullProxyComponent::TotalCullProxiesName(TEXT("TotalCullProxies"));

UNiagaraCullProxyComponent::UNiagaraCullProxyComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetAllowScalability(false);//No scalability culling for this system.
}

FBoxSphereBounds UNiagaraCullProxyComponent::CalcBounds(const FTransform& LocalToWorld)const
{
	return Super::CalcBounds(LocalToWorld);
}

FPrimitiveSceneProxy* UNiagaraCullProxyComponent::CreateSceneProxy()
{
	//This component does not render for itself. Rather it's rendered via the scene proxies of it's instances.
	//Though in future we could possible have a mode in which all rendering was done from a special case render proxy.
	//Eg. this proxy could submit all instances as impostor bill board meshes.
	return nullptr;
}

void UNiagaraCullProxyComponent::TickCullProxy()
{
	for (auto it = Instances.CreateIterator(); it; ++it)
	{
		FNiagaraCulledComponentInfo& Info = *it;
		if (UNiagaraComponent* Instance = Info.WeakComponent.Get())
		{
			Instance->MarkRenderDynamicDataDirty();
		}
		else
		{
			it.RemoveCurrent();
		}
	}

#if WITH_PARTICLE_PERF_CSV_STATS
	if (FCsvProfiler* CSVProfiler = FCsvProfiler::Get())
	{
		if (Instances.Num() && FParticlePerfStats::GetCSVStatsEnabled())
		{
			check(CSVStat_NumCullProxies != NAME_None);
			CSVProfiler->RecordCustomStat(TotalCullProxiesName, CSV_CATEGORY_INDEX(Particles), Instances.Num(), ECsvCustomStatOp::Accumulate);
			CSVProfiler->RecordCustomStat(CSVStat_NumCullProxies, CSV_CATEGORY_INDEX(Particles), Instances.Num(), ECsvCustomStatOp::Accumulate);
		}
	}
#endif
}

bool UNiagaraCullProxyComponent::RegisterCulledComponent(UNiagaraComponent* Component, bool bForce)
{
	check(Component);
	check(Component->GetAsset() == GetAsset());
	checkSlow(!Instances.ContainsByPredicate([&Component](FNiagaraCulledComponentInfo& Info) { return Info.WeakComponent.Get() == Component; }));

#if WITH_PARTICLE_PERF_CSV_STATS
	if (CSVStat_NumCullProxies == NAME_None)
	{
		CSVStat_NumCullProxies = *FString::Printf(TEXT("NumCullProxies/%s"), *Component->GetAsset()->GetFName().ToString());
	}
#endif

	if (bForce || Instances.Num() < GetAsset()->GetScalabilitySettings().MaxSystemProxies)
	{
		FNiagaraCulledComponentInfo& NewInfo = Instances.AddDefaulted_GetRef();
		NewInfo.WeakComponent = Component;

		SetPaused(false);

		Component->MarkRenderStateDirty();

		return true;
	}

	return false;
}

void UNiagaraCullProxyComponent::UnregisterCulledComponent(UNiagaraComponent* Component)
{
	int32 Index = Instances.IndexOfByPredicate([Component](const FNiagaraCulledComponentInfo& Info) { return Component == Info.WeakComponent.GetEvenIfUnreachable(); });
	if (Index != INDEX_NONE)
	{
		Instances.RemoveAtSwap(Index);
	}

	if (Instances.Num() == 0)
	{
		SetPaused(true);
	}

	Component->MarkRenderStateDirty();
}

//////////////////////////////////////////////////////////////////////////
