// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannel.h"

#include "NiagaraWorldManager.h"
#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelHandler.h"
#include "NiagaraDataChannelManager.h"
#include "NiagaraDataChannelGameData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannel)

#define LOCTEXT_NAMESPACE "NiagaraDataChannels"

void UNiagaraDataChannel::PostInitProperties()
{
	Super::PostInitProperties();
	INiagaraModule::RequestRefreshDataChannels();
}

void AddVarToHash(const FNiagaraVariable& Var, FBlake3& Builder)
{
	FNameBuilder VarName(Var.GetName());
	FStringView Name = VarName.ToView();
	uint32 ClassHash = GetTypeHash(Var.GetType().ClassStructOrEnum);
	Builder.Update(Name.GetData(), Name.Len() * sizeof(TCHAR));
	Builder.Update(&ClassHash, sizeof(uint32));
	Builder.Update(&Var.GetType().UnderlyingType, sizeof(uint16));
}

void UNiagaraDataChannel::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	static FGuid BaseVersion(TEXT("182b8dd3-f963-477f-a57d-70a449d922d8"));
	for (const FNiagaraVariable& Var : Variables_DEPRECATED)
	{
		FNiagaraDataChannelVariable ChannelVar;
		ChannelVar.SetName(Var.GetName());
		ChannelVar.SetType(FNiagaraDataChannelVariable::ToDataChannelType(Var.GetType()));

		FBlake3 VarHashBuilder;
		VarHashBuilder.Update(&BaseVersion, sizeof(FGuid));
		AddVarToHash(Var, VarHashBuilder);
		FBlake3Hash VarHash = VarHashBuilder.Finalize();
		ChannelVar.Version = FGuid::NewGuidFromHash(VarHash);
		
		ChannelVariables.Add(ChannelVar);
	}
	Variables_DEPRECATED.Empty();
	
	if (!VersionGuid.IsValid())
	{
		// If we don't have a guid yet we create one by hashing the existing variables to get a deterministic start guid
		FBlake3 Builder;
		Builder.Update(&BaseVersion, sizeof(FGuid));
		for (const FNiagaraDataChannelVariable& Var : ChannelVariables)
		{
			AddVarToHash(Var, Builder);
		}

		FBlake3Hash Hash = Builder.Finalize();
		VersionGuid = FGuid::NewGuidFromHash(Hash);
	}
#endif

	GetLayoutInfo();

	TransientAccessContext.Init(GetAccessContextType());

	INiagaraModule::RequestRefreshDataChannels();
}

void UNiagaraDataChannel::BeginDestroy()
{
	Super::BeginDestroy();
	INiagaraModule::RequestRefreshDataChannels();

	RTFence.BeginFence();
}

bool UNiagaraDataChannel::IsReadyForFinishDestroy()
{
	return RTFence.IsFenceComplete() && Super::IsReadyForFinishDestroy();
}

#if WITH_EDITOR

void UNiagaraDataChannel::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	FNiagaraWorldManager::ForAllWorldManagers(
		[DataChannel = this](FNiagaraWorldManager& WorldMan)
		{
			WorldMan.RemoveDataChannel(DataChannel);
		});
}

void UNiagaraDataChannel::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FName VariablesMemberName = GET_MEMBER_NAME_CHECKED(UNiagaraDataChannel, ChannelVariables);
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd && PropertyChangedEvent.GetPropertyName() == VariablesMemberName)
	{
		int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(VariablesMemberName.ToString());
		if(ChannelVariables.IsValidIndex(ArrayIndex))
		{
			TSet<FName> ExistingNames;
			for (const FNiagaraDataChannelVariable& Var : ChannelVariables)
			{
				ExistingNames.Add(Var.GetName());
			}
			FName UniqueName = FNiagaraUtilities::GetUniqueName(FName("MyNewVar"), ExistingNames);
			ChannelVariables[ArrayIndex].SetName(UniqueName);
		}
	}
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate && PropertyChangedEvent.GetPropertyName() == VariablesMemberName)
	{
		int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(VariablesMemberName.ToString());
		if (ChannelVariables.IsValidIndex(ArrayIndex + 1))
		{
			TSet<FName> ExistingNames;
			for (const FNiagaraDataChannelVariable& Var : ChannelVariables)
			{
				ExistingNames.Add(Var.GetName());
			}
			FNiagaraDataChannelVariable& NewEntry = ChannelVariables[ArrayIndex + 1];
			FName UniqueName = FNiagaraUtilities::GetUniqueName(NewEntry.GetName(), ExistingNames);
			NewEntry.SetName(UniqueName);
			NewEntry.Version = FGuid::NewGuid();
		}
	}
	if (PropertyChangedEvent.GetPropertyName() == VariablesMemberName || PropertyChangedEvent.GetMemberPropertyName() == VariablesMemberName)
	{
		VersionGuid = FGuid::NewGuid();
		// the guid of the variable is updated by the details customization, as we don't want to change it when just renaming a variable
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	check(IsInGameThread());

	//Refresh compiled data
	LayoutInfo = nullptr;
	GetLayoutInfo();

	TransientAccessContext.Init(GetAccessContextType());

	INiagaraModule::RequestRefreshDataChannels();
}

#endif//WITH_EDITOR

const FNiagaraDataChannelLayoutInfoPtr UNiagaraDataChannel::GetLayoutInfo()const
{
	if(LayoutInfo == nullptr)
	{
		LayoutInfo =  MakeShared<FNiagaraDataChannelLayoutInfo>(this);
	}
	return LayoutInfo;
}

FNiagaraDataChannelGameDataPtr UNiagaraDataChannel::CreateGameData()const
{
	return MakeShared<FNiagaraDataChannelGameData>(GetLayoutInfo());
}

bool UNiagaraDataChannel::IsValid()const
{
	return LayoutInfo && ChannelVariables.Num() > 0 && LayoutInfo->GetDataSetCompiledData().Variables.Num() == ChannelVariables.Num() && LayoutInfo->GetDataSetCompiledDataGPU().Variables.Num() == ChannelVariables.Num();
}

#undef LOCTEXT_NAMESPACE