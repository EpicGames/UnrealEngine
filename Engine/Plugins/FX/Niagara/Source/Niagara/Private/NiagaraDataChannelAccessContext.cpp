// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelAccessContext.h"

#include "Components/SceneComponent.h"
#include "NiagaraComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannelAccessContext)

FNDCSpawnedSystemRef::FNDCSpawnedSystemRef(UNiagaraComponent* Component)
	: SpawnedSystem(Component)
{
}

FVector FNDCAccessContext::GetLocation()const
{
	return OwningComponent && !bOverrideLocation ? OwningComponent->GetComponentLocation() : Location;
}

FVector FNiagaraDataChannelSearchParameters::GetLocation()const
{
	return OwningComponent && !bOverrideLocation ? OwningComponent->GetComponentLocation() : Location;
}


