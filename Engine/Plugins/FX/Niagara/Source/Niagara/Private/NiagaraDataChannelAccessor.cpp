// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelAccessor.h"

#include "Engine/Engine.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelHandler.h"
#include "NiagaraDataChannelManager.h"
#include "NiagaraDataChannelData.h"

#include "NiagaraTrace.h"
#include "NiagaraWorldManager.h"
#include "NiagaraDataChannelAccessorImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannelAccessor)

namespace NDCCVars
{
#if DEBUG_NDC_ACCESS
	int32 MissingNDCAccessorMode = 1;
	static FAutoConsoleVariableRef CVarMissingNDCAccessorMode(
		TEXT("fx.Niagara.DataChannels.MissingNDCAccessorMode"), 
		MissingNDCAccessorMode, 
		TEXT("Controls validation behavior for missing NDC Variabls being accessed from code. 0=Silent, 1=Log Warning, 2=Log Warning and ensure once, 3=Log Warning and ensure always."),
		ECVF_Default);
#else
	constexpr int32 MissingNDCAccessorMode = 0;
#endif
};

//////////////////////////////////////////////////////////////////////////

void UNiagaraDataChannelReader::Cleanup()
{
	Data = nullptr;
}

bool UNiagaraDataChannelReader::InitAccess(FNiagaraDataChannelSearchParameters SearchParams, bool bReadPreviousFrameData)
{
	Data = nullptr;
	bReadingPreviousFrame = bReadPreviousFrameData;
	check(Owner);

	Data = Owner->FindData(SearchParams, ENiagaraResourceAccess::ReadOnly);
	return Data.IsValid();
}

bool UNiagaraDataChannelReader::BeginRead(FNDCAccessContextInst& AccessContext, bool bReadPreviousFrameData)
{
	Data = nullptr;
	bReadingPreviousFrame = bReadPreviousFrameData;
	check(Owner);

	Data = Owner->FindData(AccessContext, ENiagaraResourceAccess::ReadOnly);
	return Data.IsValid();
}

int32 UNiagaraDataChannelReader::Num()const
{
	if (Data.IsValid())
	{
		return bReadingPreviousFrame ? Data->GetGameData()->PrevNum() : Data->GetGameData()->Num();
	}
	return 0;
}

template<typename T>
bool UNiagaraDataChannelReader::ReadData(const FNiagaraVariableBase& Var, int32 Index, T& OutData)const
{
	if (ensure(Data.IsValid()))
	{
		if (FNiagaraDataChannelVariableBuffer* VarBuffer = Data->GetGameData()->FindVariableBuffer(Var))
		{
			return VarBuffer->Read<T>(Index, OutData, bReadingPreviousFrame);
		}
	}
	return false;
}

double UNiagaraDataChannelReader::ReadFloat(FName VarName, int32 Index, bool& IsValid)const
{
	double RetVal = 0.0f;
	IsValid = ReadData<double>(FNiagaraVariableBase(FNiagaraTypeHelper::GetDoubleDef(), VarName), Index, RetVal);
	return RetVal;
}

FVector2D UNiagaraDataChannelReader::ReadVector2D(FName VarName, int32 Index, bool& IsValid)const
{
	FVector2D RetVal = FVector2D::ZeroVector;
	IsValid = ReadData<FVector2D>(FNiagaraVariableBase(FNiagaraTypeHelper::GetVector2DDef(), VarName), Index, RetVal);
	return RetVal;
}

FVector UNiagaraDataChannelReader::ReadVector(FName VarName, int32 Index, bool& IsValid)const
{
	FVector RetVal = FVector::ZeroVector;
	IsValid = ReadData<FVector>(FNiagaraVariableBase(FNiagaraTypeHelper::GetVectorDef(), VarName), Index, RetVal);
	return RetVal;
}

FVector4 UNiagaraDataChannelReader::ReadVector4(FName VarName, int32 Index, bool& IsValid)const
{
	FVector4 RetVal = FVector4(0.0f);
	IsValid = ReadData<FVector4>(FNiagaraVariableBase(FNiagaraTypeHelper::GetVector4Def(), VarName), Index, RetVal);
	return RetVal;
}

FQuat UNiagaraDataChannelReader::ReadQuat(FName VarName, int32 Index, bool& IsValid)const
{
	FQuat RetVal = FQuat::Identity;
	IsValid = ReadData<FQuat>(FNiagaraVariableBase(FNiagaraTypeHelper::GetQuatDef(), VarName), Index, RetVal);
	return RetVal;
}

FLinearColor UNiagaraDataChannelReader::ReadLinearColor(FName VarName, int32 Index, bool& IsValid)const
{
	FLinearColor RetVal = FLinearColor::White;
	IsValid = ReadData<FLinearColor>(FNiagaraVariableBase(FNiagaraTypeDefinition::GetColorDef(), VarName), Index, RetVal);
	return RetVal;
}

int32 UNiagaraDataChannelReader::ReadInt(FName VarName, int32 Index, bool& IsValid)const
{
	int32 RetVal = 0;
	IsValid = ReadData<int32>(FNiagaraVariableBase(FNiagaraTypeDefinition::GetIntDef(), VarName), Index, RetVal);
	return RetVal;
}

uint8 UNiagaraDataChannelReader::ReadEnum(FName VarName, int32 Index, bool& IsValid) const
{
	return static_cast<uint8>(ReadInt(VarName, Index, IsValid));
}

bool UNiagaraDataChannelReader::ReadBool(FName VarName, int32 Index, bool& IsValid)const
{
	FNiagaraBool RetVal(false);
	IsValid = ReadData<FNiagaraBool>(FNiagaraVariableBase(FNiagaraTypeDefinition::GetBoolDef(), VarName), Index, RetVal);
	return RetVal;
}

FVector UNiagaraDataChannelReader::ReadPosition(FName VarName, int32 Index, bool& IsValid)const
{
	FVector RetVal = FVector::ZeroVector;
	IsValid = ReadData<FVector>(FNiagaraVariableBase(FNiagaraTypeDefinition::GetPositionDef(), VarName), Index, RetVal);
	return RetVal;
}

FNiagaraID UNiagaraDataChannelReader::ReadID(FName VarName, int32 Index, bool& IsValid) const
{
	FNiagaraID RetVal;
	IsValid = ReadData<FNiagaraID>(FNiagaraVariableBase(FNiagaraTypeDefinition::GetIDDef(), VarName), Index, RetVal);
	return RetVal;
}

FNiagaraSpawnInfo UNiagaraDataChannelReader::ReadSpawnInfo(FName VarName, int32 Index, bool& IsValid) const
{
	FNiagaraSpawnInfo RetVal;
	IsValid = ReadData<FNiagaraSpawnInfo>(FNiagaraVariableBase(FNiagaraTypeDefinition(FNiagaraSpawnInfo::StaticStruct()), VarName), Index, RetVal);
	return RetVal;
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraDataChannelWriter::Cleanup()
{
	Data = nullptr;
}

bool UNiagaraDataChannelWriter::InitWrite(FNiagaraDataChannelSearchParameters SearchParams, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU, const FString& DebugSource)
{
	check(IsInGameThread());

	if (Count == 0)
	{
		UE_LOGF(LogNiagara, Warning, "Call to UNiagaraDataChannelWriter::InitWrite with Count == 0. Ignored.");
		return false;
	}

	check(Owner);

	if(FNiagaraDataChannelDataPtr DestData = Owner->FindData(SearchParams, ENiagaraResourceAccess::WriteOnly))
	{
		//Attempt to use an existing cached dest data.
		FString SourceString;
#if !UE_BUILD_SHIPPING
		SourceString = DebugSource.IsEmpty() ? GetPathName() : DebugSource;  
#endif
		Data = DestData->GetGameDataForWriteGT(Count, bVisibleToGame, bVisibleToCPU, bVisibleToGPU, SourceString);	
		StartIndex = Data->Num();
		Data->SetNum(StartIndex + Count);

		return true;
	}

	return false;
}

bool UNiagaraDataChannelWriter::BeginWrite(FNDCAccessContextInst& AccessContext, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU, const FString& DebugSource)
{
	check(IsInGameThread());

	FNiagaraInsights::NotifyDataChannelWrite(Owner ? Owner->GetDataChannel() : nullptr, *DebugSource, Count, bVisibleToGame, bVisibleToCPU, bVisibleToGPU);

	if (Count == 0)
	{
		UE_LOGF(LogNiagara, Warning, "Call to UNiagaraDataChannelWriter::InitWrite with Count == 0. Ignored.");
		return false;
	}

	check(Owner);

	if (FNiagaraDataChannelDataPtr DestData = Owner->FindData(AccessContext, ENiagaraResourceAccess::WriteOnly))
	{
		FString SourceString;
#if !UE_BUILD_SHIPPING
		SourceString = DebugSource.IsEmpty() ? GetPathName() : DebugSource;
#endif

		//Attempt to use an existing cached dest data.
		Data = DestData->GetGameDataForWriteGT(Count, bVisibleToGame, bVisibleToCPU, bVisibleToGPU, SourceString);
		StartIndex = Data->Num();
		Data->SetNum(StartIndex + Count);

		return true;
	}

	return false;
}

int32 UNiagaraDataChannelWriter::Num()const
{
	if (Data.IsValid())
	{
		return Data->Num();
	}
	return 0;
}

void UNiagaraDataChannelWriter::WriteFloat(FName VarName, int32 Index, double InData)
{
	WriteData(FNiagaraVariableBase(FNiagaraTypeHelper::GetDoubleDef(), VarName), StartIndex + Index, InData);
}

void UNiagaraDataChannelWriter::WriteVector2D(FName VarName, int32 Index, FVector2D InData)
{
	WriteData(FNiagaraVariableBase(FNiagaraTypeHelper::GetVector2DDef(), VarName), StartIndex + Index, InData);
}

void UNiagaraDataChannelWriter::WriteVector(FName VarName, int32 Index, FVector InData)
{
	WriteData(FNiagaraVariableBase(FNiagaraTypeHelper::GetVectorDef(), VarName), StartIndex + Index, InData);
}

void UNiagaraDataChannelWriter::WriteVector4(FName VarName, int32 Index, FVector4 InData)
{
	WriteData(FNiagaraVariableBase(FNiagaraTypeHelper::GetVector4Def(), VarName), StartIndex + Index, InData);
}

void UNiagaraDataChannelWriter::WriteQuat(FName VarName, int32 Index, FQuat InData)
{
	WriteData(FNiagaraVariableBase(FNiagaraTypeHelper::GetQuatDef(), VarName), StartIndex + Index, InData);
}

void UNiagaraDataChannelWriter::WriteLinearColor(FName VarName, int32 Index, FLinearColor InData)
{
	WriteData(FNiagaraVariableBase(FNiagaraTypeDefinition::GetColorDef(), VarName), StartIndex + Index, InData);
}

void UNiagaraDataChannelWriter::WriteInt(FName VarName, int32 Index, int32 InData)
{
	WriteData(FNiagaraVariableBase(FNiagaraTypeDefinition::GetIntDef(), VarName), StartIndex + Index, InData);
}

void UNiagaraDataChannelWriter::WriteEnum(FName VarName, int32 Index, uint8 InData)
{
	WriteInt(VarName, Index, InData);
}

void UNiagaraDataChannelWriter::WriteBool(FName VarName, int32 Index, bool InData)
{
	WriteData(FNiagaraVariableBase(FNiagaraTypeDefinition::GetBoolDef(), VarName), StartIndex + Index, FNiagaraBool(InData));
}

void UNiagaraDataChannelWriter::WriteSpawnInfo(FName VarName, int32 Index, FNiagaraSpawnInfo InData)
{
	WriteData(FNiagaraVariableBase(FNiagaraTypeDefinition(FNiagaraSpawnInfo::StaticStruct()), VarName), StartIndex + Index, InData);
}

void UNiagaraDataChannelWriter::WritePosition(FName VarName, int32 Index, FVector InData)
{
	WriteData(FNiagaraVariableBase(FNiagaraTypeDefinition::GetPositionDef(), VarName), StartIndex + Index, InData);
}

void UNiagaraDataChannelWriter::WriteID(FName VarName, int32 Index, FNiagaraID InData)
{
	WriteData(FNiagaraVariableBase(FNiagaraTypeDefinition::GetIDDef(), VarName), StartIndex + Index, InData);
}


//////////////////////////////////////////////////////////////////////////
//FNDCVarAccessorBase - Base class for all NDC accessor helpers. See full description in header.

FNDCVarAccessorBase::FNDCVarAccessorBase(FNDCAccessorBase& Owner, FNiagaraVariableBase InVariable, bool bInIsRequired)
	: Variable(InVariable)
	, VarOffset(INDEX_NONE)
	, bIsRequired(bInIsRequired)
{
	Owner.VariableAccessors.Add(this);
}

void FNDCVarAccessorBase::Init(const UNiagaraDataChannel* DataChannel)
{
	VarOffset = INDEX_NONE;
	if (const FNiagaraDataChannelLayoutInfoPtr LayoutInfo = DataChannel->GetLayoutInfo())
	{
		const FNiagaraDataChannelGameDataLayout& GameDataLayout = LayoutInfo->GetGameDataLayout();

#if DEBUG_NDC_ACCESS
		DebugCachedLayout = LayoutInfo;
		WeakNDC = DataChannel;
#endif
	
		if (const int32* Index = GameDataLayout.VariableIndices.Find(Variable))
		{
			VarOffset = *Index;
		}
		else if(Variable.GetType().IsEnum())
		{
			//In the case of enums we allow for using raw int32s internally to Niagara also.
			FNiagaraVariableBase IntVar(FNiagaraTypeDefinition::GetIntDef(), Variable.GetName());
			if (const int32* IntIndex = GameDataLayout.VariableIndices.Find(IntVar))
			{
				VarOffset = *IntIndex;
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////
//FNDCAccessorBase - Base class for all user code NDC Accessor classes. See full description in header.

void FNDCAccessorBase::Init(const UNiagaraDataChannel* DataChannel)
{
	check(DataChannel);
	
	CachedLayout = DataChannel->GetLayoutInfo();

	TArray<FNiagaraVariableBase> MissingVars;
	for (FNDCVarAccessorBase* Var : VariableAccessors)
	{
		Var->Init(DataChannel);

		if(NDCCVars::MissingNDCAccessorMode > 0)
		{
			if (Var->VarOffset == INDEX_NONE && Var->bIsRequired)
			{
				MissingVars.Reserve(DataChannel->GetVariables().Num());
				MissingVars.Add(Var->Variable);
			}
		}
	}

	if(MissingVars.Num() > 0 && NDCCVars::MissingNDCAccessorMode > 0)
	{
		//This variable is not present in the NDC. If it was required then warn and/or ensure if appropriate.
		if (NDCCVars::MissingNDCAccessorMode >= 1)
		{
			TStringBuilder<0> ErrorBuilder;
			ErrorBuilder.Appendf(TEXT("Niagara Data Channel being accessed from C++ is missing expected required variables.\nNDC:%s\nVariables:\n"), *GetNameSafe(DataChannel->GetAsset()));
			for(FNiagaraVariableBase& MissingVar : MissingVars)
			{
				ErrorBuilder << *MissingVar.GetType().GetName() << TEXT(" ") << MissingVar.GetName() << TEXT("\n");
			}
			ErrorBuilder.Appendf(TEXT("\nVariables in NDC: %s\n"), *GetNameSafe(DataChannel->GetAsset()));
			for(const FNiagaraDataChannelVariable& NDCVar : DataChannel->GetVariables())
			{
				ErrorBuilder << *NDCVar.GetType().GetName() << TEXT(" ") << NDCVar.GetName() << TEXT("\n");
			}
			UE_LOGF(LogNiagara, Warning, "%ls", *ErrorBuilder);
		}
		
		if (NDCCVars::MissingNDCAccessorMode == 2)
		{
			ensureMsgf(false, TEXT("NDC Variable is missing but required."));
		}
		else if (NDCCVars::MissingNDCAccessorMode == 3)
		{
			ensureAlwaysMsgf(false, TEXT("NDC Variable is missing but required."));
		}
	}
}

//////////////////////////////////////////////////////////////////////////
//FNDCWriterBase - Base class for user code NDCWriters. See full description in header.

void FNDCWriterBase::EndWrite()
{
	Data = nullptr;
	StartIndex = INDEX_NONE;
	Count = INDEX_NONE;
}

bool FNDCWriterBase::BeginWrite(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, FNDCAccessContextInst& AccessContext, int32 InCount, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU)
{
	FNiagaraInsights::NotifyDataChannelWrite(DataChannel, *DebugSource, InCount, bVisibleToGame, bVisibleToCPU, bVisibleToGPU);

	return BeginWrite_Internal(WorldContextObject, DataChannel, AccessContext, InCount, bVisibleToGame, bVisibleToCPU, bVisibleToGPU);
}

bool FNDCWriterBase::BeginWrite(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, FNiagaraDataChannelSearchParameters SearchParams, int32 InCount, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU)
{
	FNiagaraInsights::NotifyDataChannelWrite(DataChannel, *DebugSource, InCount, bVisibleToGame, bVisibleToCPU, bVisibleToGPU);

	return BeginWrite_Internal(WorldContextObject, DataChannel, SearchParams, InCount, bVisibleToGame, bVisibleToCPU, bVisibleToGPU);
}

//////////////////////////////////////////////////////////////////////////
//NDCReaderBase - Base class for user code NDCReaders. See full description in header.

bool FNDCReaderBase::BeginRead(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, FNDCAccessContextInst& AccessContext, bool bInPreviousFrame)
{
	return BeginRead_Internal(WorldContextObject, DataChannel, AccessContext, bInPreviousFrame);
}

bool FNDCReaderBase::BeginRead(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, FNiagaraDataChannelSearchParameters SearchParams, bool bInPreviousFrame)
{
	return BeginRead_Internal(WorldContextObject, DataChannel, SearchParams, bInPreviousFrame);
}

void FNDCReaderBase::EndRead()
{
	Data = nullptr;
	bPreviousFrame = false;
}

int32 FNDCReaderBase::Num()const
{
	if(Data)
	{
		return bPreviousFrame ? Data->PrevNum() : Data->Num();
	}
	return INDEX_NONE;
}