// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelAsset.h"
#include "NiagaraDataChannel.h"

#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannelAsset)

#if WITH_EDITOR

void UNiagaraDataChannelAsset::PreEditChange(class FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if(PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataChannelAsset, DataChannel) && DataChannel)
	{
		CachedPreChangeDataChannel = DataChannel;
	}
}

void UNiagaraDataChannelAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataChannelAsset, DataChannel))
	{
		if(CachedPreChangeDataChannel && DataChannel)
		{
			UEngine::FCopyPropertiesForUnrelatedObjectsParams Params;
			UEngine::CopyPropertiesForUnrelatedObjects(CachedPreChangeDataChannel, DataChannel, Params);
			CachedPreChangeDataChannel = nullptr;
		}
	}
}

#endif
