// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelData.h"
#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelHandler.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelGameData.h"

#include "NiagaraDataChannelAccessor.h"
#include "NiagaraDataChannelDebugUtilities.h"
#include "NiagaraGpuComputeDispatchInterface.h"

#include "NiagaraDataSetReadback.h"
#include "NiagaraTrace.h"

#include "RenderGraphUtils.h"
#include "Engine/World.h"

namespace NDCCVars
{
	bool bEmitWarningsOnLateNDCWrites = !UE_BUILD_SHIPPING;
	static FAutoConsoleVariableRef CVarEmitWarningsOnLateNDCWrites(TEXT("fx.Niagara.DataChannels.WarnOnLateWrites"), bEmitWarningsOnLateNDCWrites, TEXT("If true, late writes to NDCs will generate warnings. Late meaning after their final allowed tick group."), ECVF_Default);

	bool AutoUploadGPUSpawnData = true;
	static FAutoConsoleVariableRef CVarAutoUploadGPUSpawnData(TEXT("fx.Niagara.DataChannels.AutoUploadGPUSpawnData"), AutoUploadGPUSpawnData, TEXT("When true we will automatically upload any CPU NDC data to the GPU if it has been used to spawn GPU particles."), ECVF_Default);

	bool EnableAccumulatedWrites = true;
	static FAutoConsoleVariableRef CVarEnableAccumulatedWrites(TEXT("fx.Niagara.DataChannels.EnableAccumulatedWrites"), EnableAccumulatedWrites, TEXT("True if we attempt to accumulate writes into larger shared buffers."), ECVF_Default);
};

namespace NiagaraRollbackCVars
{
	namespace NDC
	{
		bool bRetainGPUDataIfPendingCpuUpdate = true;
		static FAutoConsoleVariableRef CVarRetainGPUFrameDataIfPendingCPUUpdate(
		TEXT("fx.Niagara.Rollback.DataChannels.RetainGPUDataIfPendingCpuUpdate"), 
		bRetainGPUDataIfPendingCpuUpdate, 
		TEXT("If true, NDC data for the GPU will be kept if there has been no update from the CPU. Helps manager CPU/GPU sync issues where NDC writers on the CPU may not be updating at the same rate as the renderer."),
		ECVF_Default | ECVF_RenderThreadSafe);
	};
};

FNiagaraDataChannelData::FNiagaraDataChannelData()
{

}

FNiagaraDataChannelData::~FNiagaraDataChannelData()
{
	GameData.Reset();

	/** We defer the deletion of the dataset to the RT to be sure all in-flight RT commands have finished using it.*/
	ENQUEUE_RENDER_COMMAND(FDeleteContextCommand)(
		[CPUDataChannelDataSet = CPUSimData, GameDataStagingDataSet = GameDataStaging](FRHICommandListImmediate& RHICmdList)
	{
		if (CPUDataChannelDataSet != nullptr)
		{
			delete CPUDataChannelDataSet;
		}
		if (GameDataStagingDataSet != nullptr)
		{
			delete GameDataStagingDataSet;
		}
	}
	);
	GameDataStaging = nullptr;
	CPUSimData = nullptr;
	RTProxy = nullptr;
}

void FNiagaraDataChannelData::Init(UNiagaraDataChannelHandler* Owner)
{
	check(Owner);
	const UNiagaraDataChannel* DataChannel = Owner->GetDataChannel();
	check(DataChannel);
	
	LayoutInfo = DataChannel->GetLayoutInfo();
	
	GameData = DataChannel->CreateGameData();

	CPUSimData = new FNiagaraDataSet();

	CPUSimData->Init(&LayoutInfo->GetDataSetCompiledData(), 1);

	//TODO: Send game data to GPU direct without staging.
	GameDataStaging = new FNiagaraDataSet();
	GameDataStaging->Init(&LayoutInfo->GetDataSetCompiledData());

	UWorld* OwnerWorld = Owner->GetWorld();
	WeakOwnerWorld = OwnerWorld;

	CreateRenderThreadProxy(Owner);
}

void FNiagaraDataChannelData::Reset()
{
	FScopeLock Lock(&PublishCritSec);
	PublishRequests.Reset();
	PublishRequestsFromGPU.Reset();
	PublishRequestsForGPU.Reset();
	
	PrevCPUSimData = nullptr;	

	GameData->Empty();
	CPUSimData->ResetBuffers();
	GameDataStaging->ResetBuffers();
	ENQUEUE_RENDER_COMMAND(FResetProxyCommand)(
		[RT_Proxy = RTProxy](FRHICommandListImmediate& RHICmdList)
	{
		if (RT_Proxy)
		{
			RT_Proxy->Reset();
		}
	}
	);
}

void FNiagaraDataChannelData::BeginFrame(UNiagaraDataChannelHandler* Owner)
{
	//Lazy re-init our proxy data in cases where we've had to recreate our render state while running due to an FScene change etc.
	if(RTProxy == nullptr)
	{
		CreateRenderThreadProxy(Owner);
	}

	GameData->BeginFrame();

	bool bRequirePreviousData = Owner->GetDataChannel()->KeepPreviousFrameData();
	if (bRequirePreviousData)
	{
		PrevCPUSimData = CPUSimData->GetCurrentData();
	}

	//Grab a new buffer to store the CPU data.
	CPUSimData->BeginSimulate();
	CPUSimData->EndSimulate();

	//Pull in our publish requests from the GPU to be processed in the first tick group.
	PublishRequests.Append(PublishRequestsFromGPU);
	PublishRequestsFromGPU.Reset();
}

void FNiagaraDataChannelData::EndFrame(UNiagaraDataChannelHandler* Owner)
{
	//We must do one final tick to process any final items generated by the last things to tick this frame.
	ConsumePublishRequests(Owner, TG_LastDemotable);

	if(RTProxy)
	{
		int32 TotalGPUInstances = 0;
		//Pass over any data bound for the GPU to the render thread.
		TArray<FNiagaraDataBufferRef> BuffersForGPU;
		BuffersForGPU.Reserve(PublishRequestsForGPU.Num());
		
		//If we've used our CPU data to spawn into a GPU sim this frame then we need to pass over all our CPU data regardless of if it was marked for GPU or not.		
		if(NumGPUSpawningReaders > 0 && NDCCVars::AutoUploadGPUSpawnData)
		{
			if(CPUSimData->GetCurrentData() && CPUSimData->GetCurrentData()->GetNumInstances() > 0)
			{
				//First add our entire CPU sim data buffer.
				BuffersForGPU.Emplace(CPUSimData->GetCurrentData());
				TotalGPUInstances += CPUSimData->GetCurrentData()->GetNumInstances();
			}

			//Next add existing buffers we've kept around to pass to the GPU but only those that don't already exist in the main CPU buffers.
			for(auto It = PublishRequestsForGPU.CreateIterator() ; It; ++It)
			{
				FNiagaraDataChannelPublishRequest& PublishRequest = *It;
				check(PublishRequest.bVisibleToGPUSims);
				if(!PublishRequest.bVisibleToCPUSims)
				{
					if(PublishRequest.Data->GetNumInstances() > 0)
					{
						BuffersForGPU.Emplace(PublishRequest.Data);
						TotalGPUInstances += PublishRequest.Data->GetNumInstances();
					}
				}
			}
		}
		else
		{
			//Only add data that was explicitly marked for GPU.
			for (auto It = PublishRequestsForGPU.CreateIterator(); It; ++It)
			{
				FNiagaraDataChannelPublishRequest& PublishRequest = *It;
				check(PublishRequest.bVisibleToGPUSims);
				if (PublishRequest.Data->GetNumInstances() > 0)
				{
					BuffersForGPU.Emplace(PublishRequest.Data);
					TotalGPUInstances += PublishRequest.Data->GetNumInstances();
				}
			}
		}

		if(NiagaraRollbackCVars::NDC::bRetainGPUDataIfPendingCpuUpdate || TotalGPUInstances > 0)
		{
			ENQUEUE_RENDER_COMMAND(FDataChannelProxyEndFrame) (
				[RT_Proxy = RTProxy, RT_BuffersForGPU = MoveTemp(BuffersForGPU)](FRHICommandListImmediate& CmdList)
				{

					RT_Proxy->AddBuffersFromCPU(RT_BuffersForGPU);
				});
		}		
	}
	PublishRequestsForGPU.Reset();
}

int32 FNiagaraDataChannelData::ConsumePublishRequests(UNiagaraDataChannelHandler* Owner, const ETickingGroup& TickGroup)
{
	check(IsValid(Owner));

	FlushAllPendingGameData();

	//There should be no access on other threads at this point anyway but lock just to be safe.
	FScopeLock Lock(&PublishCritSec);

	const UNiagaraDataChannel* DataChannel = Owner->GetDataChannel();
	if(PublishRequests.Num() == 0 || DataChannel == nullptr)
	{
		return 0;
	}
	
	if(NDCCVars::bEmitWarningsOnLateNDCWrites && DataChannel->ShouldEnforceTickGroupReadWriteOrder())
	{
		ETickingGroup PublishSourceTG = static_cast<ETickingGroup>(FMath::Clamp(static_cast<int32>(Owner->GetCurrentTickGroup()) - 1, 0, static_cast<int32>(ETickingGroup::TG_MAX) - 1));//We're consuming from the previous TG.
		ETickingGroup FinalWriteTG = DataChannel->GetFinalWriteTickGroup();

		//TODO: Possibly allow late writes to be deferred to the next frame?
		if(PublishSourceTG > FinalWriteTG)
		{
			static UEnum* TGEnum = StaticEnum<ETickingGroup>();
			UE_LOGF(LogNiagara, Warning, "Data Channel %ls is being written to in Tick Group %ls which is after it's final write tick group %ls. This may cause incorrect read / write ordering and missed data."
			, *DataChannel->GetAsset()->GetName()
			, *TGEnum->GetDisplayNameTextByValue((int32)PublishSourceTG).ToString()
			, *TGEnum->GetDisplayNameTextByValue((int32)FinalWriteTG).ToString());
		}
	}

	UWorld* World = Owner->GetWorld();
	check(IsValid(World));

	int32 GameDataOrigSize = GameData->Num();
	int32 CPUDataOrigSize = CPUSimData->GetCurrentData()->GetNumInstances();
	int32 NewGameDataChannel = GameDataOrigSize;
	int32 NewCPUDataChannel = CPUDataOrigSize;

	//Do a pass to gather the new total size for our DataChannel data.

	//Each DI that generates DataChannel can control whether it's pushed to Game/CPU/GPU.
	int32 RequestCount = PublishRequests.Num();
	PublishRequestsForGPU.Reserve(PublishRequestsForGPU.Num() + RequestCount);

	for (FNiagaraDataChannelPublishRequest& PublishRequest : PublishRequests)
	{
		uint32 NumInsts = 0;
		//Don't bother uploading to the GPU separately if we already know we're going to be sending it via the CPU data as a whole.
		bool bAutoUploadToGPU = NDCCVars::AutoUploadGPUSpawnData && NumGPUSpawningReaders > 0 && PublishRequest.bVisibleToCPUSims;
		bool bShouldSendToGPU = PublishRequest.bVisibleToGPUSims && !bAutoUploadToGPU;

		if (FNiagaraDataChannelGameData* RequestGameData = PublishRequest.GameData.Get())
		{
			NumInsts = RequestGameData->Num();
			if (RequestGameData->Num() > 0 && bShouldSendToGPU)
			{
				//We stage the game data into a data set to facilitate easier copy over to the GPU. TODO: Send and copy the game data directly.	
				FNiagaraDataBuffer* StagingBuf = &GameDataStaging->BeginSimulate();
				StagingBuf->Allocate(NumInsts);
				RequestGameData->WriteToDataSet(StagingBuf, 0, LwcTile);
				GameDataStaging->EndSimulate();
				FNiagaraDataChannelPublishRequest& NewGPUReq = PublishRequestsForGPU.AddDefaulted_GetRef();
				NewGPUReq.Data = StagingBuf;
				NewGPUReq.bVisibleToCPUSims = PublishRequest.bVisibleToCPUSims;
				NewGPUReq.bVisibleToGPUSims = PublishRequest.bVisibleToGPUSims;
				NewGPUReq.LwcTile = PublishRequest.LwcTile;
			}
		}
		else if (ensure(PublishRequest.Data))
		{
			NumInsts = PublishRequest.Data->GetNumInstances();

			if (NumInsts > 0 && bShouldSendToGPU)
			{
				FNiagaraDataChannelPublishRequest& NewGPUReq = PublishRequestsForGPU.AddDefaulted_GetRef();
				NewGPUReq.Data = PublishRequest.Data;
				NewGPUReq.bVisibleToCPUSims = PublishRequest.bVisibleToCPUSims;
				NewGPUReq.bVisibleToGPUSims = PublishRequest.bVisibleToGPUSims;
				NewGPUReq.LwcTile = PublishRequest.LwcTile;
			}
		}

		if (PublishRequest.bVisibleToGame)
		{
			NewGameDataChannel += NumInsts;
		}
		if (PublishRequest.bVisibleToCPUSims)
		{
			NewCPUDataChannel += NumInsts;
		}
	}

	//Allocate Sim buffers ready for gather.
	CPUSimData->BeginSimulate();
	CPUSimData->Allocate(NewCPUDataChannel, true);

	//Now do the actual data collection.

	GameData->Reserve(NewGameDataChannel);

	for (auto It = PublishRequests.CreateIterator(); It ; ++It)
	{
		FNiagaraDataChannelPublishRequest& PublishRequest = *It;
		if (FNiagaraDataChannelGameData* RequestGameData = PublishRequest.GameData.Get())
		{
			if (RequestGameData->Num())
			{
				if(PublishRequest.bVisibleToGame)
				{
					GameData->AppendFromGameData(*RequestGameData);
				}
				if(PublishRequest.bVisibleToCPUSims)
				{
					RequestGameData->WriteToDataSet(CPUSimData->GetDestinationData(), CPUSimData->GetDestinationDataChecked().GetNumInstances(), LwcTile);
				}
			}
		}

		if (PublishRequest.Data)
		{
			if (PublishRequest.bVisibleToGame)
			{
				GameData->AppendFromDataSet(PublishRequest.Data, PublishRequest.LwcTile);
			}
			if (PublishRequest.bVisibleToCPUSims)
			{
				PublishRequest.Data->CopyToUnrelated(CPUSimData->GetDestinationDataChecked(), 0, CPUSimData->GetDestinationDataChecked().GetNumInstances(), PublishRequest.Data->GetNumInstances());
			}
		}
#if WITH_NIAGARA_DEBUGGER
		FNiagaraDataChannelDebugUtilities::LogWrite(PublishRequest, Owner->GetDataChannel(), TickGroup);
#endif
	}

	PublishRequests.Reset();

	CPUSimData->EndSimulate();

#if WITH_NIAGARA_DEBUGGER
	if(Owner->GetDataChannel()->GetVerboseLogging())
	{
		FString Label = FString::Printf(TEXT("Data Channel %s - CURR"), *Owner->GetDataChannel()->GetName());
		CPUSimData->GetCurrentData()->Dump(0, CPUSimData->GetCurrentData()->GetNumInstances(), Label);

		if(PrevCPUSimData)
		{
			FString LabelPrev = FString::Printf(TEXT("Data Channel %s - PREV"), *Owner->GetDataChannel()->GetName());
			PrevCPUSimData->Dump(0, PrevCPUSimData->GetNumInstances(), LabelPrev);
		}
	}
#endif

	Owner->NotifySubscribers(this, GameDataOrigSize, NewGameDataChannel - GameDataOrigSize);
	return RequestCount;
}

FNiagaraDataBufferRef FNiagaraDataChannelData::GetCPUData(bool bPreviousFrame)
{
	if(bPreviousFrame)
	{
		return PrevCPUSimData;
	}
	else
	{
		return CPUSimData->GetCurrentData();
	}
}

FNiagaraDataChannelGameData* FNiagaraDataChannelData::GetGameData()
{
	return GameData.Get();
}

void FNiagaraDataChannelData::Publish(const FNiagaraDataChannelPublishRequest& Request)
{
	FNiagaraInsights::NotifyDataChannelPublish(Request, false);

	FScopeLock Lock(&PublishCritSec);
	PublishRequests.Add(Request);
}

void FNiagaraDataChannelData::PublishFromGPU(const FNiagaraDataChannelPublishRequest& Request)
{
	FNiagaraInsights::NotifyDataChannelPublish(Request, true);

	check(IsInGameThread());
	PublishRequestsFromGPU.Add(Request);
}

const FNiagaraDataSetCompiledData& FNiagaraDataChannelData::GetCompiledData(ENiagaraSimTarget SimTarget)
{
	return SimTarget == ENiagaraSimTarget::CPUSim ? LayoutInfo->GetDataSetCompiledData() : LayoutInfo->GetDataSetCompiledDataGPU();
}

FNiagaraDataBuffer* FNiagaraDataChannelData::GetBufferForCPUWrite()
{
	check(IsInGameThread());
	if(CPUSimData)
	{
		return &CPUSimData->AllocateBuffer();
	}
	return nullptr;
}

bool FNiagaraDataChannelData::IsLayoutValid(UNiagaraDataChannelHandler* Owner)const
{
	if(Owner && LayoutInfo)
	{
		return Owner->GetDataChannel()->GetLayoutInfo() == LayoutInfo;//If our layout has been modified then this NDC data is no longer valid.
	}
	return false;
}

bool FNiagaraDataChannelData::HasData()const
{
	return GameData->Num() > 0 || (CPUSimData && CPUSimData->GetCurrentData()->GetNumInstances() > 0) || PublishRequestsForGPU.Num() > 0;
}

FNiagaraDataChannelGameDataPtr FNiagaraDataChannelData::GetGameDataForWriteGT(int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU, const FString& DebugSource)
{
	//We keep an array of cached data ptrs, one for each of the most recent combinations of visible flags.
	//This dynamic choice of sending data to Game/CPU/GPU is honestly annoying. I think I might look at removing and having only as a system wide thing on the NDC	
	auto GetPendingBufferIdx = [](bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU)
	{
		uint32 PendingBufferIndex = bVisibleToGame ? 1 : 0;
		PendingBufferIndex += (bVisibleToCPU ? 1 : 0) << 1;
		PendingBufferIndex += (bVisibleToGPU ? 1 : 0) << 2;
		return PendingBufferIndex;
	};

	if(PendingDestGameData.Num() == 0)
	{
		PendingDestGameData.SetNum(GetPendingBufferIdx(true,true,true) + 1);
	}

	//See if we have a pending game data that can accommodate the new write.
	
	int32 PendingIndex = GetPendingBufferIdx(bVisibleToGame, bVisibleToCPU, bVisibleToGPU);
	FNiagaraDataChannelGameDataPtr& CachedData = PendingDestGameData[PendingIndex];
	if(NDCCVars::EnableAccumulatedWrites)
	{
		if(CachedData.IsValid() && CachedData->Max() >= CachedData->Num() + Count)
		{
			return CachedData;
		}
		else
		{
			int32 AllocCount = FMath::Max(Count, 128);
			FlushPendingGameData(PendingIndex);
			PendingDestGameData[PendingIndex] = MakeShared<FNiagaraDataChannelGameData>(LayoutInfo);
			PendingDestGameData[PendingIndex]->Reserve(AllocCount);
			return PendingDestGameData[PendingIndex];	
		}
	}
	else
	{
		FNiagaraDataChannelPublishRequest PublishRequest;
		PublishRequest.bVisibleToGame = bVisibleToGame;
		PublishRequest.bVisibleToCPUSims = bVisibleToCPU;
		PublishRequest.bVisibleToGPUSims = bVisibleToGPU;
		PublishRequest.GameData = MakeShared<FNiagaraDataChannelGameData>(LayoutInfo);
#if !UE_BUILD_SHIPPING
		PublishRequest.DebugSource = DebugSource;
#endif
		Publish(PublishRequest); 
		return PublishRequest.GameData;
	}
}

void FNiagaraDataChannelData::FlushPendingGameData(int32 Index)
{
	if(PendingDestGameData[Index].IsValid() && PendingDestGameData[Index]->Num() > 0)
	{
		FNiagaraDataChannelPublishRequest PublishRequest;
		PublishRequest.bVisibleToGame = Index & 1;
		PublishRequest.bVisibleToCPUSims = Index & 2;
		PublishRequest.bVisibleToGPUSims = Index & 4;
		PublishRequest.GameData = PendingDestGameData[Index];
#if !UE_BUILD_SHIPPING
		PublishRequest.DebugSource = TEXT("Accumulated Pending Game Data");
#endif
		Publish(PublishRequest);
	}

	PendingDestGameData[Index] = nullptr;
}

void FNiagaraDataChannelData::FlushAllPendingGameData()
{
	for(int32 i=0; i < PendingDestGameData.Num(); ++i)
	{
		FlushPendingGameData(i);
	}
}

void FNiagaraDataChannelData::DestroyRenderThreadProxy(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface)
{
	//We can leave the proxy to be destroyed by the dispatcher ( or in some edge cases perhaps a DI proxy ).
	RTProxy = nullptr;
}

void FNiagaraDataChannelData::CreateRenderThreadProxy(UNiagaraDataChannelHandler* Owner)
{
	check(Owner);
	UWorld* OwnerWorld = Owner->GetWorld();	
	const UNiagaraDataChannel* DataChannel = Owner->GetDataChannel();
	check(DataChannel);

	if (FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = FNiagaraGpuComputeDispatchInterface::Get(OwnerWorld))
	{
		RTProxy = MakeShared<FNiagaraDataChannelDataProxy>();
		RTProxy->GPUDataSet = new FNiagaraDataSet();
		RTProxy->GPUDataSet->Init(&LayoutInfo->GetDataSetCompiledDataGPU());
		RTProxy->bNeedsPrevFrameData = DataChannel->KeepPreviousFrameData();
		RTProxy->Owner = this->AsWeak();
		RTProxy->LayoutInfo = LayoutInfo;//Keep our own ref to the layout info as the NDCData may die on the GT before the proxy.
#if !UE_BUILD_SHIPPING
		RTProxy->DebugName = FString::Printf(TEXT("%s__GPUData"), *GetNameSafe(DataChannel->GetAsset()));
		RTProxy->DispatchInterfaceForDebuggingOnly = ComputeDispatchInterface;
#endif
		ENQUEUE_RENDER_COMMAND(FNiagaraDataChannelDataProxyInit) (
			[ComputeDispatchInterface, RT_Proxy = RTProxy](FRHICommandListImmediate& CmdList)
			{
				ComputeDispatchInterface->AddNDCDataProxy(RT_Proxy);
			}
		);
	}
}

//////////////////////////////////////////////////////////////////////////
//FNiagaraDataChannelDataProxy - RT Proxy for Data Channel Data.

FNiagaraDataChannelDataProxy::~FNiagaraDataChannelDataProxy()
{
	if(GPUDataSet)
	{
		delete GPUDataSet;
	}
	GPUDataSet = nullptr;

	Owner = nullptr;
	CurrFrameData = nullptr;
	PrevFrameData = nullptr;
	bNeedsPrevFrameData = false;
	PendingCPUBuffers.Empty();
	CurrentCPUBuffers.Empty();
	PendingGPUAllocations = 0;
	CurrentGPUAllocation = 0;
}


void FNiagaraDataChannelDataProxy::OnAddedToDispatcher(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface)
{
#if !UE_BUILD_SHIPPING
	DispatchInterfaceForDebuggingOnly = ComputeDispatchInterface;
#endif
}

void FNiagaraDataChannelDataProxy::OnRemovedFromDispatcher(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface)
{
#if !UE_BUILD_SHIPPING
	check(DispatchInterfaceForDebuggingOnly == ComputeDispatchInterface);
#endif

	FNiagaraGPUInstanceCountManager& InstCountManager = ComputeDispatchInterface->GetGPUInstanceCounterManager();

	if (CurrFrameData)
	{
		uint32 InstanceCountOffset = CurrFrameData->GetGPUInstanceCountBufferOffset();
		InstCountManager.FreeEntry(InstanceCountOffset);
		CurrFrameData->SetGPUInstanceCountBufferOffset(INDEX_NONE);
	}

	if (PrevFrameData)
	{
		uint32 InstanceCountOffset = PrevFrameData->GetGPUInstanceCountBufferOffset();
		InstCountManager.FreeEntry(InstanceCountOffset);
		PrevFrameData->SetGPUInstanceCountBufferOffset(INDEX_NONE);
	}

	for (FNDCGpuReadbackInfo& ReadbackInfo : PendingGPUReadbackBuffers)
	{
		check(ReadbackInfo.Buffer);
		uint32 Offset = ReadbackInfo.Buffer->GetGPUInstanceCountBufferOffset();
		ComputeDispatchInterface->GetGPUInstanceCounterManager().FreeEntry(Offset);
		ReadbackInfo.Buffer->SetGPUInstanceCountBufferOffset(INDEX_NONE);
	}
	PendingGPUReadbackBuffers.Reset();
}

void FNiagaraDataChannelDataProxy::Reset()
{
	PrevFrameData = nullptr;
}

void FNiagaraDataChannelDataProxy::BeginFrame(FNiagaraGpuComputeDispatchInterface* DispatchInterface, FRHICommandListImmediate& RHICmdList)
{
	check(GPUDataSet);
	check(GPUDataSet->GetSimTarget() == ENiagaraSimTarget::GPUComputeSim);
#if !UE_BUILD_SHIPPING
	check(DispatchInterfaceForDebuggingOnly == DispatchInterface);
#endif

	FNiagaraGPUInstanceCountManager& InstCountManager = DispatchInterface->GetGPUInstanceCounterManager();	

	for (FNDCGpuReadbackInfo& ReadbackInfo : PendingGPUReadbackBuffers)
	{
		uint32 Offset = ReadbackInfo.Buffer->GetGPUInstanceCountBufferOffset();
		DispatchInterface->GetGPUInstanceCounterManager().FreeEntry(Offset);
		ReadbackInfo.Buffer->SetGPUInstanceCountBufferOffset(INDEX_NONE);
	}
	PendingGPUReadbackBuffers.Reset();

	//We hold onto our most recent CPU buffers and reapply them each frame to help deal with mis-matched tick rate from the GT and RT.
	if(NiagaraRollbackCVars::NDC::bRetainGPUDataIfPendingCpuUpdate)
	{	
		//If we have no update from the CPU and there have been (and will be) no writes from the GPU, we can just leave the frame data as is and wait for an update from the CPU.
		if(!bHasPendingCpuUpdate && CurrentGPUAllocation == 0 && PendingGPUAllocations == 0)
		{
			return;
		}

		if(bHasPendingCpuUpdate)
		{
			CurrentCPUBuffers = MoveTemp(PendingCPUBuffers);		
		}
	}
	else
	{
		CurrentCPUBuffers = MoveTemp(PendingCPUBuffers);
	}

	bHasPendingCpuUpdate = false;

	if(this->bNeedsPrevFrameData)
	{
		Swap(PrevFrameData, CurrFrameData);
	}

	//The base num instances for writing from the GPU.
	uint32 NumInstanceFromCPU = 0;
	for (auto& Buffer : CurrentCPUBuffers)
	{
		NumInstanceFromCPU += Buffer->GetNumInstances();
	}

	//Allocate our GPU buffers we'll write into. Combine accumulated counts from writing DIs and the data coming in from the CPU.
	uint32 InstancesToAllocate = NumInstanceFromCPU + PendingGPUAllocations;
	CurrentGPUAllocation = PendingGPUAllocations;
	PendingGPUAllocations = 0;

	if(InstancesToAllocate == 0)
	{
		CurrFrameData = nullptr;
		return;
	}

	if(CurrFrameData == nullptr)
	{		
		FNiagaraDataBuffer& NewBuffer = GPUDataSet->AllocateBuffer();
		CurrFrameData = NewBuffer.UnlockForRead();
	}

	uint32 InstanceCountOffset = CurrFrameData->GetGPUInstanceCountBufferOffset();
	InstCountManager.FreeEntry(InstanceCountOffset);

	CurrFrameData->AllocateGPU(RHICmdList, InstancesToAllocate, DispatchInterface->GetFeatureLevel(), GetDebugName());
	CurrFrameData->PushCPUBuffersToGPU(CurrentCPUBuffers, true, RHICmdList, DispatchInterface->GetFeatureLevel(), GetDebugName(), false);
	CurrFrameData->SetNumInstances(InstancesToAllocate);
	
	if(!NiagaraRollbackCVars::NDC::bRetainGPUDataIfPendingCpuUpdate)
	{
		CurrentCPUBuffers.Reset();		
	}

	InstanceCountOffset = InstCountManager.AcquireOrAllocateEntry(RHICmdList);	
	CurrFrameData->SetGPUInstanceCountBufferOffset(InstanceCountOffset);

	//Init the instance count to the value from the CPU before it is added to by the GPU writes.
	if(NumInstanceFromCPU > 0)
	{
		InstCountManager.AddInstanceCountInitTask(InstanceCountOffset, NumInstanceFromCPU);
	}
}

void FNiagaraDataChannelDataProxy::EndFrame(FNiagaraGpuComputeDispatchInterface* DispatchInterface, FRHICommandListImmediate& RHICmdList)
{
	check(GPUDataSet);
	check(GPUDataSet->GetSimTarget() == ENiagaraSimTarget::GPUComputeSim);

#if !UE_BUILD_SHIPPING
	check(DispatchInterfaceForDebuggingOnly == DispatchInterface);
#endif

	check(CurrBufferAccessCounts == 0);

	//Handle pending readbacks for GPU->CPU data.
	{
		for (FNDCGpuReadbackInfo& ReadbackInfo : PendingGPUReadbackBuffers)
		{
			//We enqueue a readback for the data if we're wanting to publish it to the CPU or Game
			check(ReadbackInfo.bPublishToCPU || ReadbackInfo.bPublishToGame);//We should be doing one or the other or we should not get here.

			auto PublishOnCPU = [
				LocalOwner = Owner,
				PASS_PublishCPU = ReadbackInfo.bPublishToCPU,
				PASS_PublishGame = ReadbackInfo.bPublishToGame,
				PASS_LWCTile = ReadbackInfo.LWCTile,
				PASS_DataSetLayoutHash = LayoutInfo->GetDataSetCompiledData().GetLayoutHash()](TSharedRef<FNiagaraDataBufferReadback> Readback)
				{
					if (FNiagaraDataChannelData* NDCData = LocalOwner.Pin().Get())
					{
						const FNiagaraDataSetCompiledData& CompiledData = NDCData->GetCompiledData(ENiagaraSimTarget::CPUSim);
						if(PASS_DataSetLayoutHash == CompiledData.GetLayoutHash())
						{
							//Only consume the readback it the layouts match.
							//It's possible the layout has been changed in the time it took for the readback to get here.
							FNiagaraDataChannelPublishRequest PublishRequest;
							PublishRequest.bVisibleToGame = PASS_PublishGame;
							PublishRequest.bVisibleToCPUSims = PASS_PublishCPU;
							PublishRequest.bVisibleToGPUSims = false;//Don't want to ping pong back to the GPU
							PublishRequest.LwcTile = PASS_LWCTile;

							FNiagaraDataBuffer* ReadbackBuffer = NDCData->GetBufferForCPUWrite();
							Readback->ReadResultsToDataBuffer(ReadbackBuffer);
							PublishRequest.Data = ReadbackBuffer->UnlockForRead();

#if WITH_NIAGARA_DEBUGGER
							PublishRequest.DebugSource = TEXT("NDC GPU Readback");//TODO: Feed in real source names also
#endif
							NDCData->PublishFromGPU(PublishRequest);
						}
					}
				};

			TSharedPtr<FNiagaraDataBufferReadback, ESPMode::ThreadSafe> NewReadback = MakeShared<FNiagaraDataBufferReadback, ESPMode::ThreadSafe>();
			NewReadback->GetOnReadbackComplete().BindLambda(PublishOnCPU);
			NewReadback->EnqueueReadback(RHICmdList, ReadbackInfo.Buffer, DispatchInterface->GetGpuReadbackManager(), DispatchInterface->GetGPUInstanceCounterManager());

			//Defer the release of this until next frame to allow other users of this instance count
// 			uint32 Offset = ReadbackInfo.Buffer->GetGPUInstanceCountBufferOffset();
// 			DispatchInterface->GetGPUInstanceCounterManager().FreeEntry(Offset);
// 			ReadbackInfo.Buffer->SetGPUInstanceCountBufferOffset(INDEX_NONE);
		}
		PendingGPUReadbackBuffers.Reset();
	}
}

FNiagaraDataBufferRef FNiagaraDataChannelDataProxy::PrepareForWriteAccess(FRDGBuilder& GraphBuilder)
{
	check(IsInRenderingThread());

	if(CurrFrameData)
	{
		if (CurrBufferAccessCounts > 0)
		{
#if !UE_BUILD_SHIPPING
			if(!bWarnedAboutSameStageRW)
			{
				bWarnedAboutSameStageRW = true;
				UE_LOG(LogNiagara, Warning, TEXT("Attempting to write to a Niagara Data Channel in the same stage in which it's being read. {%s}\n\
				This is most often caused by an NDC reading the current frame data and the same NDC being written to in PostRenderOpaque.\n\
				A possible fix is to read the previous frames data rather than the current frame.\n\
				Another is to ensure writes to the NDC are done earlier. Things such as Depth/GBuffer reads can make the writing system tick late in the frame."), *DebugName);
			}
#endif
			return nullptr;
		}

		//If we're prepping the first writer, transition to UAV
		if (CurrBufferAccessCounts == 0)
		{
			AddTransition(GraphBuilder, ERHIAccess::SRVMask, ERHIAccess::UAVCompute, CurrFrameData.GetReference());
		}

		--CurrBufferAccessCounts;
	}

	return CurrFrameData;
}

void FNiagaraDataChannelDataProxy::EndWriteAccess(FRDGBuilder& GraphBuilder)
{
	check(IsInRenderingThread());

	check(CurrBufferAccessCounts < 0);

	++CurrBufferAccessCounts;

	//If we reach 0 writers, transition back to SRV.
	if (CurrBufferAccessCounts == 0)
	{
		AddTransition(GraphBuilder, ERHIAccess::UAVCompute, ERHIAccess::SRVMask, CurrFrameData.GetReference());
	}
}

FNiagaraDataBufferRef FNiagaraDataChannelDataProxy::PrepareForReadAccess(FRDGBuilder& GraphBuilder, bool bCurrentFrame)
{
	check(IsInRenderingThread());

	if(bCurrentFrame)
	{
		if(CurrFrameData)
		{
			if (CurrBufferAccessCounts < 0)
			{
#if !UE_BUILD_SHIPPING
				if (!bWarnedAboutSameStageRW)
				{
					bWarnedAboutSameStageRW = true;
					UE_LOG(LogNiagara, Warning, TEXT("Attempting to read from a Niagara Data Channel in the same stage in which it's being written. {%s}\n\
					This is most often caused by an NDC reading the current frame data and the same NDC being written to in PostRenderOpaque.\n\
					A possible fix is to read the previous frame data rather than the current frame.\n\
					Another is to ensure writes to the NDC are done earlier. Things such as Depth/GBuffer reads can make the writing system tick late in the frame."), *DebugName);
				}
#endif
				return nullptr;
			}

			++CurrBufferAccessCounts;

			return CurrFrameData;
		}
		return nullptr;
	}
	else
	{
		return PrevFrameData;
	}
}

void FNiagaraDataChannelDataProxy::EndReadAccess(FRDGBuilder& GraphBuilder, bool bCurrentFrame)
{
	check(IsInRenderingThread());
	if(bCurrentFrame)
	{
		check(CurrBufferAccessCounts > 0);
		--CurrBufferAccessCounts;
	}
}

void FNiagaraDataChannelDataProxy::AddTransition(FRDGBuilder& GraphBuilder, ERHIAccess AccessBefore, ERHIAccess AccessAfter, FNiagaraDataBuffer* Buffer)
{
	if (Buffer)
	{
		TArray<FRHITransitionInfo, TInlineAllocator<3>> Transitions;
		TArray<FRHIUnorderedAccessView*, TInlineAllocator<2>> UAVsToOverlap;
		if (Buffer->GetGPUBufferFloat().UAV)
		{
			Transitions.Emplace(Buffer->GetGPUBufferFloat().UAV, AccessBefore, AccessAfter);
			UAVsToOverlap.Emplace(Buffer->GetGPUBufferFloat().UAV);
		}
		if (Buffer->GetGPUBufferInt().UAV)
		{
			Transitions.Emplace(Buffer->GetGPUBufferInt().UAV, AccessBefore, AccessAfter);
			UAVsToOverlap.Emplace(Buffer->GetGPUBufferInt().UAV);
		}
		//TODO: Half Support

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("NDC Proxy - Transition Buffers"),
		ERDGPassFlags::None,
		[Transitions, AccessAfter, UAVsToOverlap](FRHICommandListImmediate& RHICmdList)
		{	
			RHICmdList.Transition(Transitions);		
			
			//We may have multiple overlapping dispatches accessing the same NDC buffers.
			if(AccessAfter == ERHIAccess::UAVCompute)
			{
				RHICmdList.BeginUAVOverlap(UAVsToOverlap);
			}
			else
			{
				RHICmdList.EndUAVOverlap(UAVsToOverlap);
			}
		});
	}
}

FNiagaraDataBufferRef FNiagaraDataChannelDataProxy::AllocateBufferForCPU(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, int32 AllocationSize, bool bPublishToGame, bool bPublishToCPU, FVector3f LWCTile)
{
	FNiagaraDataBuffer* Buffer = &GPUDataSet->AllocateBuffer();
	Buffer->AllocateGPU(GraphBuilder.RHICmdList, AllocationSize, FeatureLevel, TEXT("NDC GPU Buffers for CPU"));

	FNiagaraDataBufferRef Ret = Buffer->UnlockForRead();

	if(bPublishToCPU || bPublishToGame)
	{
		FNDCGpuReadbackInfo& NewReadback = PendingGPUReadbackBuffers.AddDefaulted_GetRef();
		NewReadback.Buffer = Ret;
		NewReadback.bPublishToCPU = bPublishToCPU;
		NewReadback.bPublishToGame = bPublishToGame;
		NewReadback.LWCTile = LWCTile;
	}

	AddTransition(GraphBuilder, ERHIAccess::SRVMask, ERHIAccess::UAVCompute, Buffer);

	return Ret;
}

void FNiagaraDataChannelDataProxy::AddBuffersFromCPU(const TArray<FNiagaraDataBufferRef>& BuffersFromCPU)
{
	PendingCPUBuffers.Append(BuffersFromCPU);
	bHasPendingCpuUpdate = true;
}

void FNiagaraDataChannelDataProxy::AddGPUAllocationForNextTick(int32 AllocationCount)
{
	PendingGPUAllocations += AllocationCount;
}
