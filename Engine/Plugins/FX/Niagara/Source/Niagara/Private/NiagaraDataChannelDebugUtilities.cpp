// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelDebugUtilities.h"

#include "Engine/Engine.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/LazySingleton.h"

#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelGameData.h"
#include "NiagaraWorldManager.h"
#include "NiagaraDataChannelManager.h"
#include "NiagaraDataChannelPublishRequest.h"

#if WITH_NIAGARA_DEBUGGER
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Dom/JsonObject.h"
#endif

namespace NDCCVars
{
	int32 LogWritesToOutputLog = 0;
	static FAutoConsoleVariableRef CVarLogWritesToOutputLog(TEXT("fx.Niagara.DataChannels.LogWritesToOutputLog"), LogWritesToOutputLog, TEXT("0=Disabled, 1=Log write summary, 2=Also write data; If >0, the NDC debugger will print all data channel writes to the output log."), ECVF_Default);

	int32 FrameDataToCapture = 0;
	static FAutoConsoleVariableRef CVarFrameDataToCapture(TEXT("fx.Niagara.DataChannels.FrameDataToCapture"), FrameDataToCapture, TEXT("The number of frames the debugger will capture for write requests."), ECVF_Default);

#if WITH_NIAGARA_DEBUGGER
	FAutoConsoleCommand CmdDumpWriteLog(
		TEXT("fx.Niagara.DataChannels.DumpWriteLog"),
		TEXT("Dump all the currently stored writes to the log (see fx.Niagara.DataChannels.FrameDataToCapture on how many frames are captured)"),
		FConsoleCommandDelegate::CreateStatic(FNiagaraDataChannelDebugUtilities::DumpAllWritesToLog)
	);
#endif
};

#if WITH_NIAGARA_DEBUGGER
void FNiagaraDataChannelDebugUtilities::BeginFrame(FNiagaraWorldManager* WorldMan, float DeltaSeconds)
{
	check(WorldMan);
	WorldMan->GetDataChannelManager().BeginFrame(DeltaSeconds);
}

void FNiagaraDataChannelDebugUtilities::EndFrame(FNiagaraWorldManager* WorldMan, float DeltaSeconds)
{
	check(WorldMan);
	WorldMan->GetDataChannelManager().EndFrame(DeltaSeconds);
}

void FNiagaraDataChannelDebugUtilities::Tick(FNiagaraWorldManager* WorldMan, float DeltaSeconds, ETickingGroup TickGroup)
{
	check(WorldMan);
	WorldMan->GetDataChannelManager().Tick(DeltaSeconds, TickGroup);
}

UNiagaraDataChannelHandler* FNiagaraDataChannelDebugUtilities::FindDataChannelHandler(FNiagaraWorldManager* WorldMan, UNiagaraDataChannel* DataChannel)
{
	check(WorldMan);
	return WorldMan->GetDataChannelManager().FindDataChannelHandler(DataChannel);
}

void FNiagaraDataChannelDebugUtilities::LogWrite(const FNiagaraDataChannelPublishRequest& WriteRequest, const UNiagaraDataChannel* DataChannel, const ETickingGroup& TickGroup)
{
	FNiagaraDataChannelDebugUtilities& Debugger = Get();
	if (NDCCVars::FrameDataToCapture > 0)
	{
		if (Debugger.FrameData.Num() > NDCCVars::FrameDataToCapture)
		{
			Debugger.FrameData.SetNum(NDCCVars::FrameDataToCapture);
		}
		FFrameDebugData* Data;
		if (Debugger.FrameData.Num() && Debugger.FrameData.Last().FrameNumber == GFrameCounter)
		{
			Data = &Debugger.FrameData.Last();
		} 
		else
		{
			if (Debugger.FrameData.Num() == NDCCVars::FrameDataToCapture)
			{
				Debugger.FrameData.RemoveAt(0);
			}
			Data = &Debugger.FrameData.AddDefaulted_GetRef();
			Data->FrameNumber = GFrameCounter;
		}
		FChannelWriteRequest DebugData;
		DebugData.Channel = DataChannel;
		DebugData.DebugSource = WriteRequest.DebugSource;
		DebugData.bVisibleToGame = WriteRequest.bVisibleToGame;
		DebugData.bVisibleToCPUSims = WriteRequest.bVisibleToCPUSims;
		DebugData.bVisibleToGPUSims = WriteRequest.bVisibleToGPUSims;
		DebugData.TickGroup = TickGroup;
		if (WriteRequest.GameData.IsValid())
		{
			DebugData.Data = WriteRequest.GameData;
		}
		else if (ensure(WriteRequest.Data))
		{
			DebugData.Data = MakeShared<FNiagaraDataChannelGameData>(DataChannel->GetLayoutInfo());
			DebugData.Data->AppendFromDataSet(WriteRequest.Data, WriteRequest.LwcTile);
		}
		Data->WriteRequests.Add(DebugData);
	}
	else
	{
		if (Debugger.FrameData.Num())
		{
			Debugger.FrameData.Empty();
		}
	}
	
	if (NDCCVars::LogWritesToOutputLog > 0)
	{
		FString DataString;
		uint32 NumInsts = 0;
		if (FNiagaraDataChannelGameData* RequestGameData = WriteRequest.GameData.Get())
		{
			NumInsts = RequestGameData->Num();
			if (NDCCVars::LogWritesToOutputLog > 1)
			{
				DataString = ToJson(RequestGameData);
			}
		}
		else if (ensure(WriteRequest.Data))
		{
			NumInsts = WriteRequest.Data->GetNumInstances();
			if (NDCCVars::LogWritesToOutputLog > 1)
			{
				FNiagaraDataChannelGameData TempData(DataChannel->GetLayoutInfo());
				TempData.AppendFromDataSet(WriteRequest.Data, WriteRequest.LwcTile);
				DataString = ToJson(&TempData);
			}
		}
		
		UE_LOGF(LogNiagara, Log, "Frame %llu, TG %ls, NDC write by %ls (BP[%ls]/CPU[%ls]/GPU[%ls]): %i entries to data channel %ls %ls%ls"
			, GFrameCounter
			, *TickGroupToString(TickGroup)
			, *WriteRequest.DebugSource
			, WriteRequest.bVisibleToGame ? TEXT("X") : TEXT(" ")
			, WriteRequest.bVisibleToCPUSims ? TEXT("X") : TEXT(" ")
			, WriteRequest.bVisibleToGPUSims ? TEXT("X") : TEXT(" ")
			, NumInsts
			, *GetPathNameSafe(DataChannel)
			, DataString.IsEmpty() ? TEXT("") : TEXT("\n")
			, *DataString);
	}
}

void FNiagaraDataChannelDebugUtilities::DumpAllWritesToLog()
{
	FNiagaraDataChannelDebugUtilities& Debugger = Get();
	if (Debugger.FrameData.Num() == 0)
	{
		UE_LOGF(LogNiagara, Log, "No writes are currently stored in the log. fx.Niagara.DataChannels.FrameDataToCapture = %i", NDCCVars::FrameDataToCapture);
		return;
	}
	UE_LOGF(LogNiagara, Log, "Current Frame is %llu, logging data from oldest to newest:", GFrameCounter);
	for (const FFrameDebugData& FrameData : Debugger.FrameData)
	{
		UE_LOGF(LogNiagara, Log, "Frame %llu: %i entries", FrameData.FrameNumber, FrameData.WriteRequests.Num());
		for (const FChannelWriteRequest& Request : FrameData.WriteRequests)
		{
			FString DataString = ToJson(Request.Data.Get());
			UE_LOGF(LogNiagara, Log, "Write by %ls (BP[%ls]/CPU[%ls]/GPU[%ls], TG %ls): %i entries to data channel %ls \n%ls"
			, *Request.DebugSource
			, Request.bVisibleToGame ? TEXT("X") : TEXT(" ")
			, Request.bVisibleToCPUSims ? TEXT("X") : TEXT(" ")
			, Request.bVisibleToGPUSims ? TEXT("X") : TEXT(" ")
			, *TickGroupToString(Request.TickGroup)
			, Request.Data ? Request.Data->Num() : -1
			, *GetPathNameSafe(Request.Channel.Get())
			, *DataString);
		}
		UE_LOGF(LogNiagara, Log, "----------------------------------------------");
	} 
}

FNiagaraDataChannelDebugUtilities& FNiagaraDataChannelDebugUtilities::Get()
{
	return TLazySingleton<FNiagaraDataChannelDebugUtilities>::Get();
}

void FNiagaraDataChannelDebugUtilities::TearDown()
{
	return TLazySingleton<FNiagaraDataChannelDebugUtilities>::TearDown();
}

FString FNiagaraDataChannelDebugUtilities::ToJson(FNiagaraDataChannelGameData* Data)
{
	if(Data == nullptr)
	{
		return TEXT("{}");
	}

	FString JsonString;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonString);
	JsonWriter->WriteArrayStart();

	TConstArrayView<FNiagaraDataChannelVariableBuffer> VaraibleBuffers = Data->GetVariableBuffers();

	const FNiagaraDataChannelGameDataLayout& GameDataLayout = Data->GetLayoutInfo()->GetGameDataLayout();
	for (int32 i = 0; i < Data->Num(); i++)
	{
		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		for (auto& LayoutVarPair : GameDataLayout.VariableIndices)
		{
			const FNiagaraVariableBase& Var = LayoutVarPair.Key;
			int32 VarIndex = LayoutVarPair.Value;

			FString VarName = Var.GetName().ToString();
			const FNiagaraDataChannelVariableBuffer* Buffer = nullptr;
			
			if(VaraibleBuffers.IsValidIndex(VarIndex))
			{
				Buffer = &VaraibleBuffers[VarIndex];
			}
			
			if (Var.GetType() == FNiagaraTypeHelper::GetDoubleDef() && Buffer)
			{
				double Value;
				Buffer->Read<double>(i, Value, false);
				JsonObject->SetNumberField(VarName, Value);
			}
			else if (Var.GetType() == FNiagaraTypeDefinition::GetBoolDef() && Buffer)
			{
				FNiagaraBool Value;
				Buffer->Read<FNiagaraBool>(i, Value, false);
				JsonObject->SetBoolField(VarName, Value);
			}
			else if (Var.GetType() == FNiagaraTypeDefinition::GetIntDef() && Buffer)
			{
				int32 Value;
				Buffer->Read<int32>(i, Value, false);
				JsonObject->SetNumberField(VarName, Value);
			}
			else if ((Var.GetType() == FNiagaraTypeHelper::GetVectorDef() || Var.GetType() == FNiagaraTypeDefinition::GetPositionDef()) && Buffer)
			{
				FVector Value;
				Buffer->Read<FVector>(i, Value, false);
				JsonObject->SetStringField(VarName, Value.ToString());
			}
			else if (Var.GetType() == FNiagaraTypeHelper::GetVector2DDef() && Buffer)
			{
				FVector2D Value;
				Buffer->Read<FVector2D>(i, Value, false);
				JsonObject->SetStringField(VarName, Value.ToString());
			}
			else if (Var.GetType() == FNiagaraTypeHelper::GetVector4Def() && Buffer)
			{
				FVector4 Value;
				Buffer->Read<FVector4>(i, Value, false);
				JsonObject->SetStringField(VarName, Value.ToString());
			}
			else if (Var.GetType() == FNiagaraTypeHelper::GetQuatDef() && Buffer)
			{
				FQuat Value;
				Buffer->Read<FQuat>(i, Value, false);
				JsonObject->SetStringField(VarName, Value.ToString());
			}
			else if (Var.GetType() == FNiagaraTypeDefinition::GetColorDef() && Buffer)
			{
				FLinearColor Value;
				Buffer->Read<FLinearColor>(i, Value, false);
				JsonObject->SetStringField(VarName, Value.ToString());
			}
			else if (Var.GetType() == FNiagaraTypeDefinition::GetIDDef() && Buffer)
			{
				FNiagaraID Value;
				Buffer->Read<FNiagaraID>(i, Value, false);
				JsonObject->SetStringField(VarName, FString::FromInt(Value.Index) + "/" + FString::FromInt(Value.AcquireTag));
			}
			else if (Var.GetType().IsEnum() && Buffer)
			{
				int32 Value;
				Buffer->Read<int32>(i, Value, false);
				JsonObject->SetStringField(VarName, Var.GetType().GetEnum()->GetNameByValue(Value).ToString());
			}
			else
			{
				JsonObject->SetStringField(VarName, "???");
			}
		}
		FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter);
	}
	JsonWriter->WriteArrayEnd();
	JsonWriter->Close();
	return JsonString;
}

FString FNiagaraDataChannelDebugUtilities::TickGroupToString(const ETickingGroup& TickGroup)
{
	static UEnum* TGEnum = StaticEnum<ETickingGroup>();
	return TGEnum->GetDisplayNameTextByValue(TickGroup).ToString();
}

#endif //WITH_NIAGARA_DEBUGGER