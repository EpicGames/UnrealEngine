// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelFunctionLibrary.h"

#include "NiagaraDataChannelAccessor.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelHandler.h"
#include "NiagaraDataChannelManager.h"
#include "NiagaraWorldManager.h"
#include "NiagaraDataChannelReference.h"

#include "Engine/Engine.h"
#include "Blueprint/BlueprintExceptionInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannelFunctionLibrary)

#define LOCTEXT_NAMESPACE "NiagaraDataChannels"

UNiagaraDataChannelLibrary::UNiagaraDataChannelLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UNiagaraDataChannelHandler* UNiagaraDataChannelLibrary::GetNiagaraDataChannel(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel)
{
	return FindDataChannelHandler(WorldContextObject, Channel->Get());
}

UNiagaraDataChannelWriter* UNiagaraDataChannelLibrary::WriteToNiagaraDataChannel(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNiagaraDataChannelSearchParameters SearchParams, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU, const FString& DebugSource)
{
	return CreateDataChannelWriter(WorldContextObject, Channel->Get(), SearchParams, Count, bVisibleToGame, bVisibleToCPU, bVisibleToGPU, DebugSource);
}

UNiagaraDataChannelReader* UNiagaraDataChannelLibrary::ReadFromNiagaraDataChannel(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNiagaraDataChannelSearchParameters SearchParams, bool bReadPreviousFrame)
{
	return CreateDataChannelReader(WorldContextObject, Channel->Get(), SearchParams, bReadPreviousFrame);
}

int32 UNiagaraDataChannelLibrary::GetDataChannelElementCount(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNiagaraDataChannelSearchParameters SearchParams, bool bReadPreviousFrame)
{
	if (Channel && Channel->Get())
	{
		if (UNiagaraDataChannelReader* Reader = CreateDataChannelReader(WorldContextObject, Channel->Get(), SearchParams, bReadPreviousFrame))
		{
			return Reader->Num();
		}
	}
	return 0;
}

void UNiagaraDataChannelLibrary::ReadFromNiagaraDataChannelSingle(const UObject*, const UNiagaraDataChannelAsset*, int32, FNiagaraDataChannelSearchParameters, bool, ENiagartaDataChannelReadResult&)
{
	// this function is just a placeholder and calls into CreateDataChannelReader and its individual read functions from the BP node
}

void UNiagaraDataChannelLibrary::WriteToNiagaraDataChannelSingle(const UObject*, const UNiagaraDataChannelAsset*, FNiagaraDataChannelSearchParameters, bool, bool, bool)
{
	// this function is just a placeholder and calls into CreateDataChannelWriter and its individual write functions from the BP node
}

//////////////////////////////////////////////////////////////////////////
// Access Context based functions

UNiagaraDataChannelWriter* UNiagaraDataChannelLibrary::WriteToNiagaraDataChannel_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNDCAccessContextInst& AccessContext, int32 Count, bool bVisibleToBlueprint, bool bVisibleToNiagaraCPU, bool bVisibleToNiagaraGPU, const FString& DebugSource)
{
	return CreateDataChannelWriter_WithContext(WorldContextObject, Channel->Get(), AccessContext, Count, bVisibleToBlueprint, bVisibleToNiagaraCPU, bVisibleToNiagaraGPU, DebugSource);
}

UNiagaraDataChannelReader* UNiagaraDataChannelLibrary::ReadFromNiagaraDataChannel_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNDCAccessContextInst& AccessContext, bool bReadPreviousFrame)
{
	return CreateDataChannelReader_WithContext(WorldContextObject, Channel->Get(), AccessContext, bReadPreviousFrame);
}

int32 UNiagaraDataChannelLibrary::GetDataChannelElementCount_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNDCAccessContextInst& AccessContext, bool bReadPreviousFrame)
{
	if (Channel && Channel->Get())
	{
		if (UNiagaraDataChannelReader* Reader = CreateDataChannelReader_WithContext(WorldContextObject, Channel->Get(), AccessContext, bReadPreviousFrame))
		{
			return Reader->Num();
		}
	}
	return 0;
}

void UNiagaraDataChannelLibrary::ReadFromNiagaraDataChannelSingle_WithContext(const UObject*, const UNiagaraDataChannelAsset*, int32, FNDCAccessContextInst&, bool, ENiagartaDataChannelReadResult&)
{
	// this function is just a placeholder and calls into CreateDataChannelReader_WithContext and its individual read functions from the BP node
}

void UNiagaraDataChannelLibrary::WriteToNiagaraDataChannelSingle_WithContext(const UObject*, const UNiagaraDataChannelAsset*, FNDCAccessContextInst& AccessContext, bool, bool, bool)
{
	// this function is just a placeholder and calls into CreateDataChannelWriter_WithContext and its individual write functions from the BP node
}

//////////////////////////////////////////////////////////////////////////
//New Access context nodes

FNDCAccessContextInst UNiagaraDataChannelLibrary::MakeNDCAccessContextInstance(UScriptStruct* ContextStruct)
{
	// We should never hit this! stubs to avoid NoExport on the class.
	checkNoEntry();
	return FNDCAccessContextInst();
}

void UNiagaraDataChannelLibrary::SetMembersInNDCAccessContextInstance(FNDCAccessContextInst& AccessContext, UScriptStruct* ContextStruct)
{
	// We should never hit this! stubs to avoid NoExport on the class.
	checkNoEntry();
}

void UNiagaraDataChannelLibrary::GetMembersInNDCAccessContextInstance(const FNDCAccessContextInst& AccessContext, UScriptStruct* ContextStruct)
{
	// We should never hit this! stubs to avoid NoExport on the class.
	checkNoEntry();
}

void UNiagaraDataChannelLibrary::GetSinglePropertyInNDCAccessContextInstance(const FNDCAccessContextInst& AccessContext, UScriptStruct* ContextStruct, FName PropertyName, int32& Value)
{
	// We should never hit this! stubs to avoid NoExport on the class.
	checkNoEntry();
}

void UNiagaraDataChannelLibrary::SetSinglePropertyInNDCAccessContextInstance(FNDCAccessContextInst& AccessContext, UScriptStruct* ContextStruct, FName PropertyName, const int32& Value)
{
	// We should never hit this! stubs to avoid NoExport on the class.
	checkNoEntry();
}

DEFINE_FUNCTION(UNiagaraDataChannelLibrary::execMakeNDCAccessContextInstance)
{
	P_GET_OBJECT(UScriptStruct, ContextStruct);
	P_FINISH

	if(ContextStruct && ContextStruct->IsChildOf(FNDCAccessContextBase::StaticStruct()))
	{
		P_NATIVE_BEGIN;
		(*(FNDCAccessContextInst*)RESULT_PARAM).Init(TNDCAccessContextType(ContextStruct));		
		P_NATIVE_END;
	}
	else
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("NDC_MakeAccessContext_InvalidTypeWarning", "Invalid Value type for Make Niagara Data Channel Access Context. You must use a child of FNDCAccessContextBase"));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
}

DEFINE_FUNCTION(UNiagaraDataChannelLibrary::execGetSinglePropertyInNDCAccessContextInstance)
{
	P_GET_STRUCT_REF(FNDCAccessContextInst, AccessContext);
	P_GET_OBJECT(UScriptStruct, ContextStruct);
	P_GET_STRUCT(FName, PropertyName);

	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FProperty>(nullptr);

	P_FINISH

	const FProperty* ValueProp = CastField<FProperty>(Stack.MostRecentProperty);
	void* ValuePtr = Stack.MostRecentPropertyAddress;

	const UScriptStruct* ActualContextStruct = AccessContext.GetScriptStruct();
	const uint8* SrcPtr = AccessContext.AccessContext.GetMemory();
	
	if (ActualContextStruct && ActualContextStruct->IsChildOf(FNDCAccessContextBase::StaticStruct()) && ActualContextStruct->IsChildOf(ContextStruct))
	{
		P_NATIVE_BEGIN
		if(FProperty* SrcProperty = ActualContextStruct->FindPropertyByName(PropertyName))
		{
			SrcProperty->CopyCompleteValue(ValuePtr, SrcProperty->ContainerPtrToValuePtr<uint8>(SrcPtr));
		}
		else
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::AbortExecution,
				FText::Format(LOCTEXT("NDC_AccessContext_MissingSourceProperty_Fmt", "Cannot read {0} from {1}. Property does not exist on this struct type."), FText::FromName(PropertyName), FText::FromString(ActualContextStruct->GetName())));
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}
		P_NATIVE_END
	}
	else
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::NonFatalError,
			LOCTEXT("NDC_GetAccessContext_InvalidTypeWarning", "Invalid Value type for Get Members In Niagara Data Channel Access Context. You must use a child of FNDCAccessContextBase. Possible that calls with Incompatible types have been mixed."));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
}

DEFINE_FUNCTION(UNiagaraDataChannelLibrary::execSetSinglePropertyInNDCAccessContextInstance)
{
	P_GET_STRUCT_REF(FNDCAccessContextInst, AccessContext);
	P_GET_OBJECT(UScriptStruct, ContextStruct);
	P_GET_STRUCT(FName, PropertyName);

	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FProperty>(nullptr);

	P_FINISH

	const FProperty* ValueProp = CastField<FProperty>(Stack.MostRecentProperty);
	void* ValuePtr = Stack.MostRecentPropertyAddress;

	const UScriptStruct* ActualContextStruct = AccessContext.GetScriptStruct();
	uint8* DestPtr = AccessContext.AccessContext.GetMutableMemory();

	if (ActualContextStruct && ActualContextStruct->IsChildOf(FNDCAccessContextBase::StaticStruct()) && ActualContextStruct->IsChildOf(ContextStruct))
	{
		P_NATIVE_BEGIN
			if (FProperty* DestProperty = ActualContextStruct->FindPropertyByName(PropertyName))
			{
				//Need special case handling for bools coming from bit fields.
				if (FBoolProperty* BoolDestProperty = CastField<FBoolProperty>(DestProperty))
				{
					if(const FBoolProperty* BoolSrcProperty = CastField<FBoolProperty>(ValueProp))
					{
						bool SrcValue = BoolSrcProperty->GetPropertyValue(ValuePtr);
						uint8* DstBool = BoolDestProperty->ContainerPtrToValuePtr<uint8>(DestPtr);
						BoolDestProperty->SetPropertyValue(DstBool, SrcValue);
					}
					else
					{					
						FBlueprintExceptionInfo ExceptionInfo(
							EBlueprintExceptionType::AbortExecution,
							FText::Format(LOCTEXT("NDC_AccessContext_MismatchedPropertyTypes_Fmt", "Cannot write {0} to {1}. Property types do not match."), FText::FromName(PropertyName), FText::FromString(DestProperty->GetName())));
						FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
					}
				}
				else
				{
					DestProperty->CopyCompleteValue(DestProperty->ContainerPtrToValuePtr<uint8>(DestPtr), ValuePtr);
				}
			}
			else
			{
				FBlueprintExceptionInfo ExceptionInfo(
					EBlueprintExceptionType::AbortExecution,
					FText::Format(LOCTEXT("NDC_AccessContext_MissingDestProperty_Fmt", "Cannot write {0} to {1}. Property does not exist on this struct type."), FText::FromName(PropertyName), FText::FromString(ActualContextStruct->GetName())));
				FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
			}
		P_NATIVE_END
	}
	else
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("NDC_SetAccessContext_InvalidTypeWarning", "Invalid Value type for Set Niagara Data Channel Access Context Members. You must use a child of FNDCAccessContextBase"));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
}

//////////////////////////////////////////////////////////////////////////
// 
void UNiagaraDataChannelLibrary::SubscribeToNiagaraDataChannel(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNiagaraDataChannelSearchParameters SearchParams, const FOnNewNiagaraDataChannelPublish& UpdateDelegate, int32& UnsubscribeToken)
{
	if (UNiagaraDataChannelHandler* NiagaraDataChannelHandler = GetNiagaraDataChannel(WorldContextObject, Channel))
	{
		NiagaraDataChannelHandler->SubscribeToDataChannelUpdates(UpdateDelegate, SearchParams, UnsubscribeToken);
	}
}

void UNiagaraDataChannelLibrary::SubscribeToNiagaraDataChannel_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNDCAccessContextInst& AccessContext, const FOnNewNiagaraDataChannelPublish& UpdateDelegate, int32& UnsubscribeToken)
{
	if (UNiagaraDataChannelHandler* NiagaraDataChannelHandler = GetNiagaraDataChannel(WorldContextObject, Channel))
	{
		NiagaraDataChannelHandler->SubscribeToDataChannelUpdates_WithContext(UpdateDelegate, AccessContext, UnsubscribeToken);
	}
}

void UNiagaraDataChannelLibrary::UnsubscribeFromNiagaraDataChannel(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, const int32& UnsubscribeToken)
{
	if (UNiagaraDataChannelHandler* NiagaraDataChannelHandler = GetNiagaraDataChannel(WorldContextObject, Channel))
    {
    	NiagaraDataChannelHandler->UnsubscribeFromDataChannelUpdates(UnsubscribeToken);
    }
}

UNiagaraDataChannelHandler* UNiagaraDataChannelLibrary::FindDataChannelHandler(const UObject* WorldContextObject, const UNiagaraDataChannel* Channel)
{
	UWorld* World = (WorldContextObject != nullptr) ? GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) : nullptr;
	if (Channel && World)
	{
		if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World))
		{
			return WorldMan->GetDataChannelManager().FindDataChannelHandler(Channel);
		}
	}
	return nullptr;
}

UNiagaraDataChannelWriter* UNiagaraDataChannelLibrary::CreateDataChannelWriter(const UObject* WorldContextObject, const UNiagaraDataChannel* Channel, FNiagaraDataChannelSearchParameters SearchParams, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU, const FString& DebugSource)
{
	check(IsInGameThread());
	UWorld* World = (WorldContextObject != nullptr) ? GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) : nullptr;
	if (Channel && World && Count > 0)
	{
		if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World))
		{
			if(UNiagaraDataChannelHandler* Handler = WorldMan->GetDataChannelManager().FindDataChannelHandler(Channel))
			{
				if(UNiagaraDataChannelWriter* Writer = Handler->GetDataChannelWriter())
				{
					if(Writer->InitWrite(SearchParams, Count, bVisibleToGame, bVisibleToCPU, bVisibleToGPU, DebugSource))
					{
						return Writer;
					}
				}
			}
		}
	}
	return nullptr;
}

UNiagaraDataChannelReader* UNiagaraDataChannelLibrary::CreateDataChannelReader(const UObject* WorldContextObject, const UNiagaraDataChannel* Channel, FNiagaraDataChannelSearchParameters SearchParams, bool bReadPreviousFrame)
{
	check(IsInGameThread());
	UWorld* World = (WorldContextObject != nullptr) ? GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) : nullptr;
	if (Channel && World)
	{
		if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World))
		{
			if (UNiagaraDataChannelHandler* Handler = WorldMan->GetDataChannelManager().FindDataChannelHandler(Channel))
			{
				if (UNiagaraDataChannelReader* Reader = Handler->GetDataChannelReader())
				{
					if(Reader->InitAccess(SearchParams, bReadPreviousFrame))
					{
						return Reader;
					}
				}
			}
		}
	}
	return nullptr;
}

UNiagaraDataChannelWriter* UNiagaraDataChannelLibrary::CreateDataChannelWriter_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannel* Channel, FNDCAccessContextInst& AccessContext, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU, const FString& DebugSource)
{
	check(IsInGameThread());
	UWorld* World = (WorldContextObject != nullptr) ? GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) : nullptr;
	if (Channel && World && Count > 0 && AccessContext.IsValid())
	{
		if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World))
		{
			if (UNiagaraDataChannelHandler* Handler = WorldMan->GetDataChannelManager().FindDataChannelHandler(Channel))
			{
				if (UNiagaraDataChannelWriter* Writer = Handler->GetDataChannelWriter())
				{					
					if (Writer->BeginWrite(AccessContext, Count, bVisibleToGame, bVisibleToCPU, bVisibleToGPU, DebugSource))
					{
						return Writer;
					}
				}
			}
		}
	}
	return nullptr;	
}

UNiagaraDataChannelReader* UNiagaraDataChannelLibrary::CreateDataChannelReader_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannel* Channel, FNDCAccessContextInst& AccessContext, bool bReadPreviousFrame)
{
	check(IsInGameThread());
	UWorld* World = (WorldContextObject != nullptr) ? GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) : nullptr;
	if (Channel && World && AccessContext.IsValid())
	{
		if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World))
		{
			if (UNiagaraDataChannelHandler* Handler = WorldMan->GetDataChannelManager().FindDataChannelHandler(Channel))
			{
				if (UNiagaraDataChannelReader* Reader = Handler->GetDataChannelReader())
				{
					if (Reader->BeginRead(AccessContext, bReadPreviousFrame))
					{
						return Reader;
					}
				}
			}
		}
	}
	return nullptr;
}

FNDCAccessContextInst& UNiagaraDataChannelLibrary::GetUsableAccessContextFromNDC(const UNiagaraDataChannelAsset* DataChannel)
{
	if(DataChannel && DataChannel->Get())
	{
		return DataChannel->Get()->GetTransientAccessContext();
	}
	static FNDCAccessContextInst Dummy;
	return Dummy;
}

FNDCAccessContextInst& UNiagaraDataChannelLibrary::GetUsableAccessContextFromNDCRef(const FNiagaraDataChannelReference& NDCRef)
{
	return NDCRef.GetUsableAccessContext();
}

FNDCAccessContextInst& UNiagaraDataChannelLibrary::PrepareAccessContextFromNDCRef(FNiagaraDataChannelReference& NDCRef)
{
	check(0);//Dummy function replaced by BP compiler. We should not get here.
	static FNDCAccessContextInst Dummy;
	return Dummy;
}

#undef LOCTEXT_NAMESPACE
