// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelGameData.h"

#include "Misc/LargeWorldRenderPosition.h"
#include "NiagaraDataChannelLayoutInfo.h"
#include "NiagaraDataSet.h"

FNiagaraDataChannelGameData::FNiagaraDataChannelGameData(const FNiagaraDataChannelLayoutInfoPtr& InLayoutInfo)
{
	Init(InLayoutInfo);
}

void FNiagaraDataChannelGameData::Init(const FNiagaraDataChannelLayoutInfoPtr& InLayoutInfo)
{
	LayoutInfo = InLayoutInfo;

	const FNiagaraDataChannelGameDataLayout& Layout = LayoutInfo->GetGameDataLayout();
	VariableData.Empty();
	VariableData.SetNum(Layout.VariableIndices.Num());
	for(const auto& Pair : Layout.VariableIndices)
	{
		int32 Index = Pair.Value;
		VariableData[Index].Init(Pair.Key);
	}
}

void FNiagaraDataChannelGameData::Empty()
{
	NumElements = 0;
	PrevNumElements = 0;
	MaxElements = 0;
	for (FNiagaraDataChannelVariableBuffer& VarData : VariableData)
	{
		VarData.Empty();
	}
}

void FNiagaraDataChannelGameData::Reset()
{
	NumElements = 0;
	PrevNumElements = 0;
	for (FNiagaraDataChannelVariableBuffer& VarData : VariableData)
	{
		VarData.Reset();
	}
}

void FNiagaraDataChannelGameData::BeginFrame()
{
	bool bKeepPrevious = LayoutInfo->KeepPreviousFrameData();
	PrevNumElements = bKeepPrevious ? NumElements : 0;
	NumElements = 0;
	for (FNiagaraDataChannelVariableBuffer& VarData : VariableData)
	{
		VarData.BeginFrame(bKeepPrevious);
	}
}

void FNiagaraDataChannelGameData::SetNum(int32 NewNum)
{
	NumElements = NewNum;
	MaxElements = FMath::Max(NewNum, MaxElements);
	for (auto& Buffer : VariableData)
	{
		Buffer.SetNum(NewNum);
	}
}

void FNiagaraDataChannelGameData::Reserve(int32 NewNum)
{
	MaxElements = FMath::Max(NewNum, MaxElements);
	for (auto& Buffer : VariableData)
	{
		Buffer.Reserve(NewNum);
	}
}

FNiagaraDataChannelVariableBuffer* FNiagaraDataChannelGameData::FindVariableBuffer(const FNiagaraVariableBase& Var)
{
	const FNiagaraDataChannelGameDataLayout& Layout = LayoutInfo->GetGameDataLayout();
	const FNiagaraTypeDefinition& VarType = Var.GetType();
	for(auto& Pair : Layout.VariableIndices)
	{
		const FNiagaraVariableBase& LayoutVar = Pair.Key;
		const FNiagaraTypeDefinition& LayoutVarType = LayoutVar.GetType();
		int32 Index = Pair.Value;
		
		if(Var.GetName() == LayoutVar.GetName())
		{
			//For enum variables we'll hack things a little so that correctly named ints also match. This gets around some limitations in calling code not being able to provide the correct enum types.
			if(VarType == LayoutVarType || (LayoutVarType.IsEnum() && VarType == FNiagaraTypeDefinition::GetIntDef()))
			{
				return &VariableData[Index];
			}
		}
	}
	return nullptr;
}

void FNiagaraDataChannelGameData::WriteToDataSet(FNiagaraDataBuffer* DestBuffer, int32 DestStartIdx, FVector3f SimulationLwcTile)
{
	const FNiagaraDataSetCompiledData& CompiledData = DestBuffer->GetOwner()->GetCompiledData();

	static TArray<uint8> LWCConversionBuffer;

	int32 NumInstances = NumElements;

	if (NumInstances == 0)
	{
		return;
	}
	
	DestBuffer->SetNumInstances(DestStartIdx + NumInstances);

	const FNiagaraDataChannelGameDataLayout& Layout = LayoutInfo->GetGameDataLayout();

	for (const TPair<FNiagaraVariableBase, int32>& VarIndexPair : Layout.VariableIndices)
	{
		FNiagaraVariableBase Var = VarIndexPair.Key;
		int32 VarIndex = VarIndexPair.Value;
		FNiagaraDataChannelVariableBuffer& VarBuffer = VariableData[VarIndex];
		uint8* SrcDataBase = VarBuffer.Data.GetData();
		
		FNiagaraVariableBase SimVar = Var;

		//Convert from LWC types to Niagara Simulation Types where required.
		if (FNiagaraTypeHelper::IsLWCType(Var.GetType()))
		{
			SimVar.SetType(FNiagaraTypeDefinition(FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(CastChecked<UScriptStruct>(Var.GetType().GetStruct()), ENiagaraStructConversion::Simulation)));
		}

		//Niagara Positions are a special case where we're actually storing them as FVectors in the game level data and must convert down to actual Positions/FVector3f in the sim data.
		if(Var.GetType() == FNiagaraTypeDefinition::GetPositionDef())
		{
			Var.SetType(FNiagaraTypeHelper::GetVectorDef());
		}

		int32 SimVarIndex = CompiledData.Variables.IndexOfByKey(SimVar);
		if (SimVarIndex == INDEX_NONE)
		{
			continue; //Did not find this variable in the dataset. Warn?
		}

		int32 SrcVarSize = Var.GetSizeInBytes();
		int32 DestVarSize = SimVar.GetSizeInBytes();
		const FNiagaraVariableLayoutInfo& SimLayout = CompiledData.VariableLayouts[SimVarIndex];

		int32 FloatCompIdx = SimLayout.GetFloatComponentStart();
		int32 IntCompIdx = SimLayout.GetInt32ComponentStart();
		int32 HalfCompIdx = SimLayout.GetHalfComponentStart();

		TFunction<void(UScriptStruct*, UScriptStruct*, uint8*)> WriteData;
		WriteData = [&](UScriptStruct* SrcStruct, UScriptStruct* DestStruct, uint8* SrcPropertyBase)
		{
			//TODO: why doesn't this just use FNiagaraLwcStructConverter?
			
			//Write all data from the data channel into the data set. Converting into from LWC into the local LWC Tile space as we go.

			uint8* SrcData = SrcPropertyBase;

			//Positions are a special case that are stored as FVectors in game data but converted to an LWCTile local FVector3f in simulation data.
			if (DestStruct == FNiagaraTypeDefinition::GetPositionStruct())
			{	
				float* DestX = DestBuffer->GetInstancePtrFloat(FloatCompIdx++, DestStartIdx);
				float* DestY = DestBuffer->GetInstancePtrFloat(FloatCompIdx++, DestStartIdx);
				float* DestZ = DestBuffer->GetInstancePtrFloat(FloatCompIdx++, DestStartIdx);
								
				for (int32 i = 0; i < NumInstances; ++i)
				{
					FVector* Src = reinterpret_cast<FVector*>((SrcData + i * SrcVarSize));

					FVector3f SimLocalSWC = FVector3f((*Src) - FVector(SimulationLwcTile) * FLargeWorldRenderScalar::GetTileSize());
					*DestX++ = SimLocalSWC.X;
					*DestY++ = SimLocalSWC.Y;
					*DestZ++ = SimLocalSWC.Z;
				}
			}
			else
			{
				TFieldIterator<FProperty> SrcPropertyIt(SrcStruct, EFieldIteratorFlags::IncludeSuper);
				TFieldIterator<FProperty> DestPropertyIt(DestStruct, EFieldIteratorFlags::IncludeSuper);
				for (; SrcPropertyIt; ++SrcPropertyIt, ++DestPropertyIt)
				{
					FProperty* SrcProperty = *SrcPropertyIt;
					FProperty* DestProperty = *DestPropertyIt;
					SrcData = SrcPropertyBase + SrcProperty->GetOffset_ForInternal();

					//Convert any LWC doubles to floats. //TODO: Insert LWCTile... probably need to explicitly check for vectors etc.
					if (SrcProperty->IsA(FDoubleProperty::StaticClass()))
					{
						check(DestProperty->IsA(FFloatProperty::StaticClass()));
						float* Dest = DestBuffer->GetInstancePtrFloat(FloatCompIdx++, DestStartIdx);
						//Write all instances for this component.
						for (int32 i = 0; i < NumInstances; ++i)
						{
							double* Src = reinterpret_cast<double*>((SrcData + i * SrcVarSize));
							*Dest++ = static_cast<float>(*Src);
						}
					}
					else if (SrcProperty->IsA(FFloatProperty::StaticClass()))
					{
						float* Dest = DestBuffer->GetInstancePtrFloat(FloatCompIdx++, DestStartIdx);

						//Write all instances for this component.
						for (int32 i = 0; i < NumInstances; ++i)
						{
							float* Src = reinterpret_cast<float*>((SrcData + i * SrcVarSize));
							*Dest++ = *Src;
						}
					}
					else if (SrcProperty->IsA(FUInt16Property::StaticClass()))
					{
						FFloat16* Dest = DestBuffer->GetInstancePtrHalf(HalfCompIdx++, DestStartIdx);

						//Write all instances for this component.
						for (int32 i = 0; i < NumInstances; ++i)
						{
							FFloat16* Src = reinterpret_cast<FFloat16*>((SrcData + i * SrcVarSize));
							*Dest++ = *Src;
						}
					}
					else if (SrcProperty->IsA(FIntProperty::StaticClass()) || SrcProperty->IsA(FBoolProperty::StaticClass()))
					{
						int32* Dest = DestBuffer->GetInstancePtrInt32(IntCompIdx++, DestStartIdx);

						//Write all instances for this component.
						for (int32 i = 0; i < NumInstances; ++i)
						{
							int32* Src = reinterpret_cast<int32*>((SrcData + i * SrcVarSize));
							*Dest++ = *Src;
						}
					}
					//Should be able to support double easily enough
					else if (FStructProperty* StructProp = CastField<FStructProperty>(SrcProperty))
					{
						FStructProperty* DestStructProp = CastField<FStructProperty>(DestProperty);
						WriteData(StructProp->Struct, DestStructProp->Struct, SrcData);
					}
					else
					{
						checkf(false, TEXT("Property(%s) Class(%s) is not a supported type"), *SrcProperty->GetName(), *SrcProperty->GetClass()->GetName());
					}
				}
			}
		};
		WriteData(Var.GetType().GetScriptStruct(), SimVar.GetType().GetScriptStruct(), SrcDataBase);
	}
}
void FNiagaraDataChannelGameData::AppendFromGameData(const FNiagaraDataChannelGameData& GameData)
{
	check(GetLayoutInfo() == GameData.GetLayoutInfo());

	NumElements += GameData.Num();
	TConstArrayView<FNiagaraDataChannelVariableBuffer> SrcBuffers = GameData.GetVariableBuffers();
	for(int32 i=0; i < SrcBuffers.Num(); ++i)
	{
		const FNiagaraDataChannelVariableBuffer& SrcBuffer = SrcBuffers[i];
		FNiagaraDataChannelVariableBuffer& DestBuffer = VariableData[i];
		DestBuffer.Data.Append(SrcBuffer.Data);
	}
}

void FNiagaraDataChannelGameData::AppendFromDataSet(const FNiagaraDataBuffer* SrcBuffer, FVector3f SimulationLwcTile)
{
	const FNiagaraDataSetCompiledData& CompiledData = SrcBuffer->GetOwner()->GetCompiledData();

	static TArray<uint8> LWCConversionBuffer;

	int32 NumInstances = SrcBuffer->GetNumInstances();
	int32 StartIndex = NumElements;
	NumElements += NumInstances;

	const FNiagaraDataChannelGameDataLayout& Layout = LayoutInfo->GetGameDataLayout();
	for (const TPair<FNiagaraVariableBase, int32>& VarIndexPair : Layout.VariableIndices)
	{
		FNiagaraVariableBase Var = VarIndexPair.Key;
		int32 VarIndex = VarIndexPair.Value;
		FNiagaraDataChannelVariableBuffer& VarBuffer = VariableData[VarIndex];

		VarBuffer.SetNum(NumElements);
		
		FNiagaraVariableBase SimVar = Var;
		if (FNiagaraTypeHelper::IsLWCType(Var.GetType()))
		{
			SimVar.SetType(FNiagaraTypeDefinition(FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(Var.GetType().GetScriptStruct(), ENiagaraStructConversion::Simulation)));
		}

		//Niagara Positions are a special case where we're actually storing them as FVectors in the game level data and must convert down to actual Positions/FVector3f in the sim data and visa versa
		if(Var.GetType() == FNiagaraTypeDefinition::GetPositionDef())
		{
			Var.SetType(FNiagaraTypeHelper::GetVectorDef());
		}
		int32 VarSize = Var.GetSizeInBytes();

		uint8* DestDataBase = VarBuffer.Data.GetData() + StartIndex * VarSize;
		
		int32 SimVarIndex = CompiledData.Variables.IndexOfByKey(SimVar);
		if (SimVarIndex == INDEX_NONE)
		{
			continue; //Did not find this variable in the dataset. Warn?
		}

		const FNiagaraVariableLayoutInfo& SimLayout = CompiledData.VariableLayouts[SimVarIndex];
		
		int32 FloatCompIdx = SimLayout.GetFloatComponentStart();
		int32 IntCompIdx = SimLayout.GetInt32ComponentStart();
		int32 HalfCompIdx = SimLayout.GetHalfComponentStart();
		
		TFunction<void(UScriptStruct*, UScriptStruct*, uint8*)> ReadData;
		ReadData = [&](UScriptStruct* SrcStruct, UScriptStruct* DestStruct, uint8* DestDataBase)
		{
			//Read all data from the simulation and place in the game level buffers. Converting from LWC local tile space where needed.

			//Special case for writing to Niagara Positions. We must offset the data into the simulation local LWC space.
			if (SrcStruct == FNiagaraTypeDefinition::GetPositionStruct())
			{
				const float* SrcX = reinterpret_cast<const float*>(SrcBuffer->GetComponentPtrFloat(FloatCompIdx++));
				const float* SrcY = reinterpret_cast<const float*>(SrcBuffer->GetComponentPtrFloat(FloatCompIdx++));
				const float* SrcZ = reinterpret_cast<const float*>(SrcBuffer->GetComponentPtrFloat(FloatCompIdx++));

				for (int32 i = 0; i < NumInstances; ++i)
				{
					FVector* Dest = reinterpret_cast<FVector*>(DestDataBase + VarSize * i);
					*Dest = FVector(*SrcX++, *SrcY++, *SrcZ++) + FVector(SimulationLwcTile) * FLargeWorldRenderScalar::GetTileSize();
				}
			}
			else
			{
				TFieldIterator<FProperty> SrcPropertyIt(SrcStruct, EFieldIteratorFlags::IncludeSuper);
				TFieldIterator<FProperty> DestPropertyIt(DestStruct, EFieldIteratorFlags::IncludeSuper);
				for (; SrcPropertyIt; ++SrcPropertyIt, ++DestPropertyIt)
				{
					FProperty* SrcProperty = *SrcPropertyIt;
					FProperty* DestProperty = *DestPropertyIt;
					int32 DestOffset = DestProperty->GetOffset_ForInternal();
					uint8* DestData = DestDataBase + DestOffset;
					if (DestPropertyIt->IsA(FDoubleProperty::StaticClass()))
					{
						double* Dest = reinterpret_cast<double*>(DestData);
						const float* Src = reinterpret_cast<const float*>(SrcBuffer->GetComponentPtrFloat(FloatCompIdx++));

						//Write all instances for this component.
						for (int32 i = 0; i < NumInstances; ++i)
						{
							Dest = reinterpret_cast<double*>(DestData + VarSize * i);
							*Dest = *Src++;
						}
					}
					else if (DestPropertyIt->IsA(FFloatProperty::StaticClass()))
					{
						float* Dest = reinterpret_cast<float*>(DestData);
						const float* Src = reinterpret_cast<const float*>(SrcBuffer->GetComponentPtrFloat(FloatCompIdx++));

						//Write all instances for this component.
						for (int32 i = 0; i < NumInstances; ++i)
						{
							Dest = reinterpret_cast<float*>(DestData + VarSize * i);
							*Dest = *Src++;
						}
					}
					else if (DestPropertyIt->IsA(FUInt16Property::StaticClass()))
					{
						FFloat16* Dest = reinterpret_cast<FFloat16*>(DestData);
						const FFloat16* Src = reinterpret_cast<const FFloat16*>(SrcBuffer->GetComponentPtrHalf(HalfCompIdx++));

						//Write all instances for this component.
						for (int32 i = 0; i < NumInstances; ++i)
						{
							Dest = reinterpret_cast<FFloat16*>(DestData + VarSize * i);
							*Dest = *Src++;
						}
					}
					else if (DestPropertyIt->IsA(FIntProperty::StaticClass()) || DestPropertyIt->IsA(FBoolProperty::StaticClass()))
					{
						int32* Dest = reinterpret_cast<int32*>(DestData);
						const int32* Src = reinterpret_cast<const int32*>(SrcBuffer->GetComponentPtrInt32(IntCompIdx++));

						//Write all instances for this component.
						for (int32 i = 0; i < NumInstances; ++i)
						{
							Dest = reinterpret_cast<int32*>(DestData + VarSize * i);
							*Dest = *Src++;
						}
					}
					else if (FStructProperty* SrcStructProp = CastField<FStructProperty>(SrcProperty))
					{
						FStructProperty* DestStructProp = CastField<FStructProperty>(DestProperty);
						ReadData(SrcStructProp->Struct, DestStructProp->Struct, DestData);
					}
					else
					{
						checkf(false, TEXT("Property(%s) Class(%s) is not a supported type"), *SrcProperty->GetName(), *SrcProperty->GetClass()->GetName());
					}
				}
			}
		};
		ReadData(SimVar.GetType().GetScriptStruct(), Var.GetType().GetScriptStruct(), DestDataBase);
	}
}

void FNiagaraDataChannelGameData::SetFromSimCache(const FNiagaraVariableBase& SourceVar, TConstArrayView<uint8> Data, int32 Size)
{
	const FNiagaraDataChannelGameDataLayout& Layout = LayoutInfo->GetGameDataLayout();
	if (const int* Index = Layout.VariableIndices.Find(SourceVar))
	{
		if (VariableData[*Index].Size == Size)
		{
			VariableData[*Index].Data = Data;
		}
	}
}