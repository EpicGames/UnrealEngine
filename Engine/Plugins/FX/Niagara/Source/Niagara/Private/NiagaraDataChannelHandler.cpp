// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelHandler.h"
#include "NiagaraDataChannelAccessor.h"
#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelData.h"
#include "Logging/StructuredLog.h"
#include "NiagaraGpuComputeDispatchInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannelHandler)

void UNiagaraDataChannelHandler::BeginDestroy()
{
	Super::BeginDestroy();
	Cleanup();
	
	RTFence.BeginFence();
}

bool UNiagaraDataChannelHandler::IsReadyForFinishDestroy()
{
	return RTFence.IsFenceComplete() && Super::IsReadyForFinishDestroy();
}

void UNiagaraDataChannelHandler::Init(const UNiagaraDataChannel* InChannel)
{
	check(InChannel);
	DataChannel = InChannel;
	AccessContextType = InChannel->GetAccessContextType();

	CachedLayoutInfo = DataChannel->GetLayoutInfo();
}

void UNiagaraDataChannelHandler::Cleanup()
{
	if(Reader)
	{
		Reader->Cleanup();
		Reader = nullptr;
	}
	
	if(Writer)
	{
		Writer->Cleanup();
		Writer = nullptr;
	}

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		//Mark this handler as garbage so any reading DIs will know to stop using it.
		MarkAsGarbage();
	}
}

void UNiagaraDataChannelHandler::BeginFrame(float DeltaTime)
{
	CurrentTG = TG_PrePhysics;

	for(auto It = WeakDataArray.CreateIterator(); It; ++It)
	{
		if(It->IsValid() == false)
		{
			It.RemoveCurrentSwap();
		}
	}
}

void UNiagaraDataChannelHandler::EndFrame(float DeltaTime)
{

}

void UNiagaraDataChannelHandler::Tick(float DeltaTime, ETickingGroup TickGroup)
{
	CurrentTG = TickGroup;
}

void UNiagaraDataChannelHandler::OnDataChannelLayoutChanged()
{
	if(const UNiagaraDataChannel* NDC = DataChannel.Get())
	{
		CachedLayoutInfo = NDC->GetLayoutInfo();
	}
	else
	{
		CachedLayoutInfo.Reset();
	}
}

UNiagaraDataChannelWriter* UNiagaraDataChannelHandler::GetDataChannelWriter()
{
	if(Writer == nullptr)
	{
		Writer =  NewObject<UNiagaraDataChannelWriter>();
		Writer->Owner = this;
	}
	return Writer;
}

UNiagaraDataChannelReader* UNiagaraDataChannelHandler::GetDataChannelReader()
{
	if (Reader == nullptr)
	{
		Reader = NewObject<UNiagaraDataChannelReader>();
		Reader->Owner = this;
	}
	return Reader;
}

void UNiagaraDataChannelHandler::SubscribeToDataChannelUpdates_WithContext(FOnNewNiagaraDataChannelPublish UpdateDelegate, FNDCAccessContextInst& AccessContext, int32& UnsubscribeToken)
{
	if (!UpdateDelegate.IsBound())
	{
		UnsubscribeToken = -1;
		return;
	}

	SubscriberTokens++;
	UnsubscribeToken = SubscriberTokens;

	FChannelSubscription& ChannelSubscription = ChannelSubscriptions.AddDefaulted_GetRef();
	ChannelSubscription.SubscriptionToken = SubscriberTokens;
	ChannelSubscription.OnPublishDelegate = UpdateDelegate;
	ChannelSubscription.AccessContext = AccessContext;
}

void UNiagaraDataChannelHandler::SubscribeToDataChannelUpdates(FOnNewNiagaraDataChannelPublish UpdateDelegate, FNiagaraDataChannelSearchParameters SearchParams, int32& UnsubscribeToken)
{
	if (!UpdateDelegate.IsBound())
	{
		UnsubscribeToken = -1;
		return;
	}

	SubscriberTokens++;
	UnsubscribeToken = SubscriberTokens;
	
	FChannelSubscription& ChannelSubscription = ChannelSubscriptions.AddDefaulted_GetRef();
	ChannelSubscription.SubscriptionToken = SubscriberTokens;
	ChannelSubscription.OnPublishDelegate = UpdateDelegate;
	ChannelSubscription.SearchParams = SearchParams;
}

void UNiagaraDataChannelHandler::UnsubscribeFromDataChannelUpdates(const int32& UnsubscribeToken)
{
	for (int i = 0; i < ChannelSubscriptions.Num(); i++)
	{
		if (ChannelSubscriptions[i].SubscriptionToken == UnsubscribeToken)
		{
			ChannelSubscriptions.RemoveAtSwap(i);
			break;
		}
	}
}

FNiagaraDataChannelDataPtr UNiagaraDataChannelHandler::CreateData()
{
	FNiagaraDataChannelDataPtr Ret = MakeShared<FNiagaraDataChannelData>();
	WeakDataArray.Add(Ret);
	Ret->Init(this);
	return Ret;
}

void UNiagaraDataChannelHandler::NotifySubscribers(FNiagaraDataChannelData* Source, int32 StartIndex, int32 NumNewElements)
{
	if (NumNewElements == 0 || ChannelSubscriptions.IsEmpty() || StartIndex < 0)
	{
		return;
	}

	FNiagaraDataChannelUpdateContext UpdateContext;
	UpdateContext.Reader = GetDataChannelReader();
	UpdateContext.FirstNewDataIndex = StartIndex;
	UpdateContext.LastNewDataIndex = StartIndex + NumNewElements - 1;
	UpdateContext.NewElementCount = NumNewElements;
	for (int i = ChannelSubscriptions.Num() - 1; i >= 0; i--)
	{
		FChannelSubscription& ChannelSubscription = ChannelSubscriptions[i];
		if (ChannelSubscription.OnPublishDelegate.IsCompactable())
		{
			ChannelSubscriptions.RemoveAt(i);
			continue;
		}

		FNiagaraDataChannelDataPtr ChannelData = nullptr;
		if(ChannelSubscription.AccessContext.IsValid())
		{
			ChannelData = FindData(ChannelSubscription.AccessContext, ENiagaraResourceAccess::ReadOnly);
		}
		else
		{
			ChannelData = FindData(ChannelSubscription.SearchParams, ENiagaraResourceAccess::ReadOnly);
		}

		if (ChannelData.Get() == Source)
		{
			UpdateContext.Reader->Data = ChannelData;
			ChannelSubscription.OnPublishDelegate.Execute(UpdateContext);
		}
	}
}

void UNiagaraDataChannelHandler::OnComputeDispatchInterfaceDestroyed(FNiagaraGpuComputeDispatchInterface* InComputeDispatchInterface)
{
	//Destroy all RT proxies when the dispatcher is destroyed.
	//In cases where this is done on a running world, we'll do a lazy reinit next frame.
	ForEachNDCData([InComputeDispatchInterface](FNiagaraDataChannelDataPtr& NDCData)
	{
		check(NDCData);
		NDCData->DestroyRenderThreadProxy(InComputeDispatchInterface);
	});
}


void UNiagaraDataChannelHandler::VerifyAccessContext(FNDCAccessContextInst& AccessContext)
{
#if !UE_BUILD_SHIPPING
	if (const UNiagaraDataChannel* Channel = DataChannel.Get())
	{
		const UScriptStruct* Expected = Channel->GetAccessContextType().Get();
		const UScriptStruct* Actual = AccessContext.GetScriptStruct();
		if (Actual == nullptr || Actual->IsChildOf(Expected) == false)
		{
			UE_LOGFMT(LogNiagara, Warning, "Attempting to find Niagara Data Channel with incorrect access context type.\nData Channel: {DataChannel}\nExpected Type:{Expected}\nActual:{ActualType}", 
				GetNameSafe(DataChannel->GetAsset()), GetNameSafe(Expected), GetNameSafe(Actual));
		}
	}
#endif
}

FNiagaraDataChannelDataPtr UNiagaraDataChannelHandler::FindData(FNiagaraDataChannelSearchParameters SearchParams, ENiagaraResourceAccess AccessType)
{
	UE_LOGFMT(LogNiagara, Error, 
	"Using legacy SearchParams API when accessing Niagara Data Channel that does not support the legacy path.\nPlease update access code to use the newer Access Context based access path.\nData Channel: {DataChannel}",
	DataChannel.IsValid() && DataChannel->GetAsset() ? DataChannel->GetAsset()->GetPathName() : TEXT(""));
	return nullptr;
}

FNiagaraDataChannelDataPtr UNiagaraDataChannelHandler::FindData(FNDCAccessContextInst& AccessContext, ENiagaraResourceAccess AccessType)
{
	if(AccessContext.IsValid() == false)
	{
		return nullptr;
	}

	VerifyAccessContext(AccessContext);
	//Base behavior is to just convert to legacy params and pass through.
	//We'll only need this for NDCs that do not directly support the new API but people still call in with the new API.
	//We only support accesses using or derived from FNDCAccessContext for legacy NDC types.
	if(const FNDCAccessContextLegacy* AccessCtx = AccessContext.Get<FNDCAccessContextLegacy>())
	{
		FNiagaraDataChannelSearchParameters SearchParams;
		SearchParams.OwningComponent = AccessCtx->GetOwner();
		SearchParams.Location = AccessCtx->Location;
		SearchParams.bOverrideLocation = AccessCtx->bOverrideLocation;
		return FindData(SearchParams, AccessType);
	}	
	return nullptr;
}
