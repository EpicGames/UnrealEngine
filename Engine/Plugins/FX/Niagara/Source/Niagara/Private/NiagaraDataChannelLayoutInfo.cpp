// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelLayoutInfo.h"
#include "NiagaraDataChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannelLayoutInfo)

void FNiagaraDataChannelGameDataLayout::Init(TConstArrayView<FNiagaraDataChannelVariable> Variables)
{
	VariableIndices.Reset();
	LwcConverters.Reset(Variables.Num());
	for (const FNiagaraDataChannelVariable& Var : Variables)
	{
		//Sigh.
		//We must convert from the variable stored var in the data channels definition as we currently cannot serialize/store actual LWC types in FNiagaraTypeDefinitions.
		FNiagaraTypeDefinition LWCType = FNiagaraTypeHelper::GetLWCType(Var.GetType());
		FNiagaraVariableBase LWCVar(LWCType, Var.GetName());

		int32& VarIdx = VariableIndices.Add(LWCVar);
		VarIdx = VariableIndices.Num() - 1;

		FNiagaraLwcStructConverter& Converter = LwcConverters.AddDefaulted_GetRef();
		Converter = FNiagaraTypeRegistry::GetStructConverter(LWCType);
	}
}

//////////////////////////////////////////////////////////////////////////

FNiagaraDataChannelLayoutInfo::FNiagaraDataChannelLayoutInfo(const UNiagaraDataChannel* DataChannel)
{
	check(DataChannel);
	GameDataLayout.Init(DataChannel->GetVariables());

	bKeepPreviousFrameData = DataChannel->KeepPreviousFrameData();

	CompiledData.SimTarget = ENiagaraSimTarget::CPUSim;
	CompiledDataGPU.SimTarget = ENiagaraSimTarget::GPUComputeSim;
	for (const FNiagaraDataChannelVariable& NDCVar : DataChannel->GetVariables())
	{
		FNiagaraVariableBase Var = NDCVar;
		if (Var.GetType().IsEnum() == false)
		{
			Var.SetType(FNiagaraTypeDefinition(FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(Var.GetType().GetScriptStruct(), ENiagaraStructConversion::Simulation)));
		}
		CompiledData.Variables.Add(Var);
		CompiledDataGPU.Variables.Add(Var);
	}
	CompiledData.BuildLayout();
	CompiledDataGPU.BuildLayout();
}

FNiagaraDataChannelLayoutInfo::~FNiagaraDataChannelLayoutInfo()
{
	CompiledData.Empty();
	CompiledDataGPU.Empty();
}
