// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelManager.h"

#include "NiagaraModule.h"
#include "NiagaraWorldManager.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelHandler.h"
#include "NiagaraGpuComputeDispatchInterface.h"

DECLARE_CYCLE_STAT(TEXT("FNiagaraDataChannelManager::BeginFrame"), STAT_DataChannelManager_BeginFrame, STATGROUP_NiagaraDataChannels);
DECLARE_CYCLE_STAT(TEXT("FNiagaraDataChannelManager::EndFrame"), STAT_DataChannelManager_EndFrame, STATGROUP_NiagaraDataChannels);
DECLARE_CYCLE_STAT(TEXT("FNiagaraDataChannelManager::Tick"), STAT_DataChannelManager_Tick, STATGROUP_NiagaraDataChannels);


static int GNDCAllowLazyHandlerInit = 1;
static FAutoConsoleVariableRef CVarNDCAllowLazyHandlerInit(
	TEXT("fx.Niagara.DataChannels.AllowLazyHandlerInit"),
	GNDCAllowLazyHandlerInit,
	TEXT("True if we allow lazy initialization of NDC handlers."),
	ECVF_Default
);

FNiagaraDataChannelManager::FNiagaraDataChannelManager(FNiagaraWorldManager* InWorldMan)
	: WorldMan(InWorldMan)
{
}

void FNiagaraDataChannelManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(Channels);
}

void FNiagaraDataChannelManager::RefreshDataChannels()
{
	if (bIsCleanedUp)
	{
		return;
	}

	if(GNDCAllowLazyHandlerInit == 0)
	{
		UNiagaraDataChannel::ForEachDataChannel([&](UNiagaraDataChannel* DataChannel)
		{
			InitDataChannel(DataChannel, false);
		});
	}
}

void FNiagaraDataChannelManager::Init()
{
	bIsInitialized = true;
	bIsCleanedUp = false;
	
	if(GNDCAllowLazyHandlerInit == 0)
	{
		//Initialize any existing data channels, more may be initialized later as they are loaded.
		UNiagaraDataChannel::ForEachDataChannel([&](UNiagaraDataChannel* DataChannel)
		{
			InitDataChannel(DataChannel, true);
		});
	}
}

void FNiagaraDataChannelManager::Cleanup()
{
	for (auto& ChannelPair : Channels)
	{
		if(UNiagaraDataChannelHandler* ChannelHandler = ChannelPair.Value.Get())
		{
			ChannelHandler->Cleanup();
		}
	}
	Channels.Empty();
	bIsCleanedUp = true;
	bIsInitialized = false;
}

void FNiagaraDataChannelManager::BeginFrame(float DeltaSeconds)
{
	if (INiagaraModule::DataChannelsEnabled())
	{
		check(IsInGameThread());
		SCOPE_CYCLE_COUNTER(STAT_DataChannelManager_BeginFrame);
		CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_NiagaraDataChannelManager);

		if(NeedsInit())
		{
			Init();
		}

		//Tick all DataChannel channel handlers.
		for (auto It = Channels.CreateIterator(); It; ++It)
		{
			if(const UNiagaraDataChannel* Channel = It.Key().Get())
			{
				It.Value()->BeginFrame(DeltaSeconds);
			}
			else
			{
				It.Value()->Cleanup();
				It.RemoveCurrent();
			}
		}
	}
}

void FNiagaraDataChannelManager::EndFrame(float DeltaSeconds)
{
	if (INiagaraModule::DataChannelsEnabled())
	{
		check(IsInGameThread());
		SCOPE_CYCLE_COUNTER(STAT_DataChannelManager_EndFrame);
		CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_NiagaraDataChannelManager);

		//Tick all DataChannel channel handlers.
		for (auto It = Channels.CreateIterator(); It; ++It)
		{
			if (const UNiagaraDataChannel* Channel = It.Key().Get())
			{
				It.Value()->EndFrame(DeltaSeconds);
			}
			else
			{
				It.Value()->Cleanup();
				It.RemoveCurrent();
			}
		}
	}
}

void FNiagaraDataChannelManager::Tick(float DeltaSeconds, ETickingGroup TickGroup)
{
	if(INiagaraModule::DataChannelsEnabled())
	{
		check(IsInGameThread());
		SCOPE_CYCLE_COUNTER(STAT_DataChannelManager_Tick);
		CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_NiagaraDataChannelManager);

		//Tick all DataChannel channel handlers.
		for (auto& ChannelPair : Channels)
		{
			ChannelPair.Value->Tick(DeltaSeconds, TickGroup);
		}
	}
	else
	{
		for (auto& ChannelPair : Channels)
		{
			ChannelPair.Value->Cleanup();
		}
		Channels.Empty();
	}
}

UNiagaraDataChannelHandler* FNiagaraDataChannelManager::FindDataChannelHandler(const UNiagaraDataChannel* Channel)
{
	check(IsInGameThread());
	if(Channel == nullptr)
	{
		return nullptr;
	}

	if(GNDCAllowLazyHandlerInit != 0)
	{
		//Try to lazy init the channel.
		//It's possible we've just loaded the channel and it's pending init.
		return InitDataChannel(Channel, false);
	}
	else
	{
		if (TObjectPtr<UNiagaraDataChannelHandler>* Found = Channels.Find(Channel))
		{
			return (*Found).Get();
		}
		return nullptr;
	}
}

UNiagaraDataChannelHandler* FNiagaraDataChannelManager::InitDataChannel(const UNiagaraDataChannel* InChannel, bool bForce)
{
	check(IsInGameThread());
	UWorld* World = GetWorld();
	if(INiagaraModule::DataChannelsEnabled() && World && !World->IsNetMode(NM_DedicatedServer) && InChannel->IsValid())
	{
		if(NeedsInit())
		{
			Init();
		}

		TObjectPtr<UNiagaraDataChannelHandler>& Handler = Channels.FindOrAdd(InChannel);

		if (bForce || Handler == nullptr)
		{
			if(Handler)
			{
				Handler->Cleanup();
			}
			Handler = InChannel->CreateHandler(WorldMan->GetWorld());
			Handler->SetOwningWorld(WorldMan);
		}
		return Handler;
	}
	return nullptr;
}

void FNiagaraDataChannelManager::RemoveDataChannel(const UNiagaraDataChannel* InChannel)
{
	check(IsInGameThread());

	TObjectPtr<UNiagaraDataChannelHandler> RemovedHandler;
	if(Channels.RemoveAndCopyValue(InChannel, RemovedHandler))
	{
		if(RemovedHandler)
		{
			RemovedHandler->Cleanup();
		}
	}
}

UWorld* FNiagaraDataChannelManager::GetWorld()const
{
	return WorldMan->GetWorld();
}

void FNiagaraDataChannelManager::OnComputeDispatchInterfaceDestroyed(FNiagaraGpuComputeDispatchInterface* InComputeDispatchInterface)
{
	check(IsInGameThread());
	for (auto& ChannelPair : Channels)
	{
		ChannelPair.Value->OnComputeDispatchInterfaceDestroyed(InComputeDispatchInterface);
	}
}