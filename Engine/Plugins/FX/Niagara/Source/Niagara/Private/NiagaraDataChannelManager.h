// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"
#include "NiagaraDataChannelAsset.h"

class FNiagaraWorldManager;
class UNiagaraDataChannel;
class UNiagaraDataChannelHandler;
class FNiagaraGpuComputeDispatchInterface;
enum ETickingGroup : int;

/**
Manager class for all Niagara DataChannels at the System/World level.
*/
class FNiagaraDataChannelManager
{
public:

	explicit FNiagaraDataChannelManager(FNiagaraWorldManager* InWorldMan);

	void AddReferencedObjects(FReferenceCollector& Collector);

	void Init();
	void Cleanup();

	UNiagaraDataChannelHandler* InitDataChannel(const UNiagaraDataChannel* Channel, bool bForce);
	void RemoveDataChannel(const UNiagaraDataChannel* Channel);

	void BeginFrame(float DeltaSeconds);
	void EndFrame(float DeltaSeconds);

	void Tick(float DeltaSeconds, ETickingGroup TickGroup);

	void RefreshDataChannels();
	
	/**
	Return the DataChannel handler for the given channel.
	*/
	UNiagaraDataChannelHandler* FindDataChannelHandler(const UNiagaraDataChannel* Channel);
	UNiagaraDataChannelHandler* FindDataChannelHandler(const UNiagaraDataChannelAsset* Channel) { return Channel ? FindDataChannelHandler(Channel->Get()) : nullptr; }

	UWorld* GetWorld()const;
	FNiagaraWorldManager* GetWorldMan()const{ return WorldMan; }

	void OnComputeDispatchInterfaceDestroyed(FNiagaraGpuComputeDispatchInterface* InComputeDispatchInterface);

private:
	FNiagaraWorldManager* WorldMan = nullptr;

	bool NeedsInit()const { return !bIsCleanedUp && !bIsInitialized; }

	/** Runtime handlers for each DataChannel channel. */
	TMap<TWeakObjectPtr<const UNiagaraDataChannel>, TObjectPtr<UNiagaraDataChannelHandler>> Channels;
	bool bIsCleanedUp = false;
	bool bIsInitialized = false;
};
