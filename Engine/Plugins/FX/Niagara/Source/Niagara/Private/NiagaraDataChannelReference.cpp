// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelReference.h"
#include "NiagaraDataChannelAsset.h"
#include "NiagaraDataChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannelReference)

const FNDCAccessContextInst& FNiagaraDataChannelReference::GetAccessContext()const
{
	InitAccessContext();
	return AccessContext;
}

FNDCAccessContextInst& FNiagaraDataChannelReference::GetUsableAccessContext()const
{
	if (DataChannel)
	{
		if (UNiagaraDataChannel* NDC = DataChannel->Get())
		{
			InitAccessContext();
			FNDCAccessContextInst& TransientContext = NDC->GetTransientAccessContext();
			if(bCustomAccessContext && AccessContext.IsValid())
			{
				TransientContext = AccessContext;
			}
			return TransientContext;
		}
	}
	static FNDCAccessContextInst Dummy;
	return Dummy;
}

void FNiagaraDataChannelReference::InitAccessContext(bool bTransferExisting/* =true */)const
{
	//We need to verify that the access context has the correct type.
	//We can get bad types if someone changes the NDC type for example.
	if(DataChannel)
	{
		if (UNiagaraDataChannel* NDC = DataChannel->Get())
		{
			const UScriptStruct* ExpectedType = NDC->GetAccessContextType().Get();
			if(bCustomAccessContext && ExpectedType != FNDCAccessContextLegacy::StaticStruct())//Special case legacy type indicating we should not create a context in most places.
			{
				if (AccessContext.IsValid() && bTransferExisting)
				{
					const UScriptStruct* ActualType = AccessContext.GetScriptStruct();
					if (ActualType != ExpectedType)
					{
						//If we do have the wrong type. 
						//Initialize to the correct type but try to pull any matching properties from the old type.
						FInstancedStruct OldContext = AccessContext.AccessContext;
						AccessContext.Init(NDC->GetAccessContextType());

						const uint8* Src = OldContext.GetMemory();
						uint8* Dst = AccessContext.AccessContext.GetMutableMemory();
						for (TFieldIterator<FProperty> It(ExpectedType); It; ++It)
						{
							FProperty* DestProp = *It;
							FProperty* SrcProp = ActualType->FindPropertyByName(DestProp->GetFName());
							if (SrcProp && SrcProp->SameType(DestProp))
							{
								const uint8* PropSrcPtr = SrcProp->ContainerPtrToValuePtr<const uint8>(Src);
								uint8* PropDstPtr = DestProp->ContainerPtrToValuePtr<uint8>(Dst);
								DestProp->CopyCompleteValue(PropDstPtr, PropSrcPtr);
							}
						}
					}
				}
				else
				{
					AccessContext.Init(NDC->GetAccessContextType());
				}
				return;
			}
		}
	}

	//If we fail to init properly then clear the access context to avoid memory bloat.
	AccessContext.Reset();
}

void FNiagaraDataChannelReference::SetDataChannel(UNiagaraDataChannelAsset* InDataChannel)
{
	if(DataChannel != InDataChannel)
	{
		DataChannel = InDataChannel;
		OnValueChanged();
	}
}

void FNiagaraDataChannelReference::OnValueChanged()
{
	if(DataChannel)
	{
		InitAccessContext();
	}
	else
	{
		AccessContext.Reset();
	}
}
