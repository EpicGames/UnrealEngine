// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelVariable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannelVariable)

bool FNiagaraDataChannelVariable::Serialize(FArchive& Ar)
{
	if (FNiagaraVariableBase::Serialize(Ar))
	{
		if (Ar.IsLoading())
		{
			// fix up variables serialized with wrong type
			// this happens because we only save swc types
			SetType(ToDataChannelType(GetType()));
		}
		return true;
	}
	return false;
}

#if WITH_EDITORONLY_DATA
bool FNiagaraDataChannelVariable::IsAllowedType(const FNiagaraTypeDefinition& Type)
{
	return !(Type.IsDataInterface() || Type.GetClass() || Type == FNiagaraTypeDefinition::GetParameterMapDef() || Type == FNiagaraTypeDefinition::GetGenericNumericDef() || Type == FNiagaraTypeDefinition::GetHalfDef() || Type == FNiagaraTypeDefinition::GetMatrix4Def());
}
#endif

FNiagaraTypeDefinition FNiagaraDataChannelVariable::ToDataChannelType(const FNiagaraTypeDefinition& Type)
{
	if (Type == FNiagaraTypeDefinition::GetVec3Def())
	{
		return FNiagaraTypeHelper::GetVectorDef();
	}
	if (Type == FNiagaraTypeDefinition::GetFloatDef())
	{
		return FNiagaraTypeHelper::GetDoubleDef();
	}
	if (Type == FNiagaraTypeDefinition::GetQuatDef())
	{
		return FNiagaraTypeHelper::GetQuatDef();
	}
	if (Type == FNiagaraTypeDefinition::GetVec2Def())
	{
		return FNiagaraTypeHelper::GetVector2DDef();
	}
	if (Type == FNiagaraTypeDefinition::GetVec4Def())
	{
		return FNiagaraTypeHelper::GetVector4Def();
	}
	return Type;
}
