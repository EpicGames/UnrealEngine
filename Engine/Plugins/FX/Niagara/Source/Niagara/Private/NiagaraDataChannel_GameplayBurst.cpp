// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannel_GameplayBurst.h"
#include "NiagaraCommon.h"
#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelData.h"
#include "NiagaraComponent.h"

#include "BlueprintGameplayTagLibrary.h"
#include "GameFramework/Character.h"
#include "Misc/LargeWorldRenderPosition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannel_GameplayBurst)

namespace NDCGameplayBurst_Local
{
	[[nodiscard]] inline int64 GetCellCoordAsInt64(double CellCoord)
	{
		ensure(FMath::IsFinite(CellCoord));
		ensure(CellCoord == FMath::FloorToDouble(CellCoord));
		return static_cast<int64>(CellCoord);
	}

	//Returns the grid cell for the given location and cell size.
	//Cell size is stored as double which is accurate up to ~9.5ly assuming a 10m cell size.
	[[nodiscard]] inline FVector GetCellCoords(FVector Location, FVector InvCellSize)
	{
		return FVector(FMath::FloorToDouble(Location.X * InvCellSize.X),
			FMath::FloorToDouble(Location.Y * InvCellSize.Y),
			FMath::FloorToDouble(Location.Z * InvCellSize.Z));
	}
};
 
//////////////////////////////////////////////////////////////////////////

UNiagaraDataChannelHandler* UNiagaraDataChannel_GameplayBurst::CreateHandler(UWorld* OwningWorld)const
{
	UNiagaraDataChannelHandler* NewHandler = NewObject<UNiagaraDataChannelHandler_GameplayBurst>(OwningWorld);
	NewHandler->Init(this);
	return NewHandler;
}

//////////////////////////////////////////////////////////////////////////

void FNDCMapEntry_GameplayBurst::Init(FNDCAccessContextInst& AccessContext, UNiagaraDataChannelHandler_MapBase* InOwner, const FNDCMapKey& Key)
{
	const UNiagaraDataChannelHandler_GameplayBurst* TypedOwner = CastChecked<UNiagaraDataChannelHandler_GameplayBurst>(InOwner);
	FNDCAccessContext_GameplayBurst& TypedContext = AccessContext.GetChecked<FNDCAccessContext_GameplayBurst>();

	Super::Init(AccessContext, InOwner, Key);

	UObject* SysToSpawn = TypedOwner->GetSystemToSpawn(AccessContext);

	FVector BoundsPadding = TypedContext.bOverrideBoundsPadding ? TypedContext.SystemBoundsPadding : TypedOwner->DefaultSystemBoundsPadding;
	bool bIsAttached = TypedContext.bAttachedToOwningComponent;
	if (bIsAttached)
	{
		AttachComponent = TypedContext.GetOwner();

		USceneComponent* Comp = TypedContext.GetOwner();
		check(Comp);

		FVector Location = Comp->GetComponentLocation();

		check(Data.IsValid());
		Data->SetLwcTile(FLargeWorldRenderScalar::GetTileFor(Location));

		if (SysToSpawn)
		{
			FBoxSphereBounds Bounds = Comp->CalcBounds(Comp->GetComponentToWorld());
			FVector Extents = Bounds.BoxExtent + BoundsPadding;
			if (UNiagaraComponent* NewComp = SpawnSystem(AccessContext.GetChecked<FNDCAccessContext_MapBase>(), SysToSpawn, Comp, Location, Extents, true))
			{
				//Add the attached state tag so we can be sure of how to handle FindData calls from it in future without having to re-gen attachment settings.
				NewComp->ComponentTags.Add(UNiagaraDataChannelHandler_GameplayBurst::AttachedComponentTag);
			}
		}
	}
	else
	{
		FVector Location = TypedContext.GetLocation();

		check(Data.IsValid());
		Data->SetLwcTile(FLargeWorldRenderScalar::GetTileFor(Location));

		if (SysToSpawn)
		{
			FVector CellSize = TypedContext.bOverrideCellSize ? TypedContext.CellSizeOverride : TypedOwner->DefaultCellSize;
			FVector CellExtents = FVector(CellSize * 0.5);
			FVector CellCoord = NDCGameplayBurst_Local::GetCellCoords(Location, CellSize.Reciprocal());
			FVector CellCentre = CellCoord * CellSize + CellExtents;

			CellExtents += BoundsPadding;
			SpawnSystem(AccessContext.GetChecked<FNDCAccessContext_MapBase>(), SysToSpawn, nullptr, CellCentre, CellExtents, true);
		}
	}
}

void FNDCMapEntry_GameplayBurst::Reset(const FNDCMapKey& Key)
{
	if (!AttachComponent.IsExplicitlyNull())
	{
		const UNiagaraDataChannelHandler_GameplayBurst* TypedOwner = CastChecked<UNiagaraDataChannelHandler_GameplayBurst>(Owner);
		for (TObjectPtr<UNiagaraComponent>& Comp : SpawnedComponents)
		{
			Comp->ComponentTags.Remove(UNiagaraDataChannelHandler_GameplayBurst::AttachedComponentTag);
		}
	}

	AttachComponent.Reset();

	Super::Reset(Key);
}

bool FNDCMapEntry_GameplayBurst::BeginFrame(float DeltaTime, FNiagaraWorldManager* OwningWorld, const FNDCMapKey& Key)
{
	//We also want to have our entry clean up if we're attached and the attach component is no longer valid.
	bool bAttachmentCompValid = AttachComponent.IsExplicitlyNull() || AttachComponent.IsValid();

	return Super::BeginFrame(DeltaTime, OwningWorld, Key) && bAttachmentCompValid;
}

//////////////////////////////////////////////////////////////////////////
//UNiagaraDataChannelHandler_GameplayBurst
//
// NDC handler specifically geared towards game play Burst FX.

UNiagaraDataChannelHandler_GameplayBurst::UNiagaraDataChannelHandler_GameplayBurst(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNiagaraDataChannelHandler_GameplayBurst::Init(const UNiagaraDataChannel* InChannel)
{
	Super::Init(InChannel);
	const UNiagaraDataChannel_GameplayBurst* NDCTyped = CastChecked<UNiagaraDataChannel_GameplayBurst>(InChannel);

	//Cache various data we want to calculate once or modify at runtime.
	DefaultCellSize = NDCTyped->GetGridCellSize();
	DefaultSystemBoundsPadding = NDCTyped->GetSystemBoundsPadding();
	AttachmentSettings = NDCTyped->GetAttachmentSettings();
}

bool FNDCGameplayBurstAttachmentSettings::UseAttachedPathForComponent(const USceneComponent* Component, bool bForce)const
{
	if (!Component)
	{
		return false;
	}

	//For attached NiagaraComponents we deal with their attach parent.
	if (const UNiagaraComponent* NiagaraComp = Cast<const UNiagaraComponent>(Component))
	{
		if (NiagaraComp->GetAttachParent())
		{
			Component = NiagaraComp->GetAttachParent();
		}
	}

	if (bForce)
	{
		return true;
	}

	//Is this component a specified type to use attached path?
	if (ComponentTypes.Contains(Component->GetClass()))
	{
		return true;
	}

	AActor* AttachParent = Component->GetAttachParentActor();
	if (AttachParent == nullptr)
	{
		AttachParent = Component->GetOwner();
	}

	if (AttachParent && GameplayTags.Num() > 0)
	{
		//Does the actor have any gameplay tags that would trigger the attached path?
		if (UBlueprintGameplayTagLibrary::HasAnyMatchingGameplayTags(AttachParent, GameplayTags))
		{
			return true;
		}
	}

	//Check the owners speed to check to auto attach to moving components/actors.
	if (SpeedThreshold >= 0.0f)
	{
		const UPrimitiveComponent* ConstPrimComp = Cast<const UPrimitiveComponent>(Component);
		UPrimitiveComponent* PrimComp = const_cast<UPrimitiveComponent*>(ConstPrimComp);
		const FMovementBaseInterfaceData MovementBaseData(PrimComp);
		//If the searching component is not a Niagara component then we can bail out immediately for static objects without ever searching our PerComponentData map.
		if (MovementBaseUtility::IsDynamicBase(&MovementBaseData))
		{
			FVector BaseVel = MovementBaseUtility::GetMovementBaseVelocity(&MovementBaseData, NAME_None);
			if (BaseVel.SquaredLength() >= (SpeedThreshold * SpeedThreshold))
			{
				return true;
			}
		}

		// In the case of the niagara system during a read, we must check the attach parent
		if (AttachParent)
		{
			// Check the attached parent for movement.
			if (UPrimitiveComponent* PrimitiveComponentBase = Cast<UPrimitiveComponent>(AttachParent->GetRootComponent()))
			{
				const FMovementBaseInterfaceData AttachParentMovementBaseData(PrimitiveComponentBase);
				FVector BaseVel = MovementBaseUtility::GetMovementBaseVelocity(&AttachParentMovementBaseData, NAME_None);
				if (BaseVel.SquaredLength() >= (SpeedThreshold * SpeedThreshold))
				{
					return true;
				}
			}
		}
	}

	return false;
}

const FName UNiagaraDataChannelHandler_GameplayBurst::AttachedComponentTag = FName(TEXT("NDCGameplayBurst_AttachedFX"));

void UNiagaraDataChannelHandler_GameplayBurst::GenerateKey(FNDCAccessContextInst& AccessContext, FNDCMapKeyWriter& KeyWriter)const
{
	FNDCAccessContext_GameplayBurst& TypedContext = AccessContext.GetChecked<FNDCAccessContext_GameplayBurst>();

	//First check whether we should use the attached path and route data to a per-component bucket or add to the regular world grid cell buckets.
	USceneComponent* Comp = TypedContext.GetOwner();

	bool bUseAttachedPath = false;

	if (Comp)
	{
		//If we are a spawned attached component need to use the attach parent.
		if (Comp->ComponentHasTag(AttachedComponentTag))
		{
			check(Comp->IsA<UNiagaraComponent>());
			Comp = Comp->GetAttachParent();

			// Comp can potentially be nullptr now as we moved to the attach parent, update bUseAttachedPath accordingly to avoid a null deref
			bUseAttachedPath = Comp != nullptr;
		}
		else
		{
			bUseAttachedPath = AttachmentSettings.UseAttachedPathForComponent(Comp, TypedContext.bForceAttachToOwningComponent);
		}
	}

	//Inform any interested calling code that we are using the attached path so dest data and all spawned FX are specific to the given owner component.
	TypedContext.bAttachedToOwningComponent = bUseAttachedPath;

	uint8 Flags = bUseAttachedPath ? 1 : 0;

	if (bUseAttachedPath)
	{
		KeyWriter << Flags;

		//In the attached path we use the component ptr as key.
		uint32 ObjID = Comp->GetUniqueID();
		KeyWriter << ObjID;
	}
	else
	{
		Flags |= TypedContext.bOverrideCellSize ?  (1 << 1): 0;

		KeyWriter << Flags;

		//We're not attached so write our world grid cell to the key.
		FVector Location = TypedContext.GetLocation();
		FVector CellSize = TypedContext.bOverrideCellSize ? TypedContext.CellSizeOverride : DefaultCellSize;
		FVector CellCoord = NDCGameplayBurst_Local::GetCellCoords(Location, CellSize.Reciprocal());

		KeyWriter << NDCGameplayBurst_Local::GetCellCoordAsInt64(CellCoord.X);
		KeyWriter << NDCGameplayBurst_Local::GetCellCoordAsInt64(CellCoord.Y);
		KeyWriter << NDCGameplayBurst_Local::GetCellCoordAsInt64(CellCoord.Z);

		if(TypedContext.bOverrideCellSize)
		{
			KeyWriter << TypedContext.CellSizeOverride;
		}
		
		KeyWriter << TypedContext.GameplayTag.GetTagName();
	}

	Super::GenerateKey(AccessContext, KeyWriter);
}
