// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannel_Global.h"
#include "NiagaraWorldManager.h"
#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannel_Global)


int32 GbDebugDumpDumpHandlerTick = 0;
static FAutoConsoleVariableRef CVarDumpHandlerTick(
	TEXT("fx.Niagara.DataChannels.DumpHandlerTick"),
	GbDebugDumpDumpHandlerTick,
	TEXT(" \n"),
	ECVF_Default
);

DECLARE_CYCLE_STAT(TEXT("UNiagaraDataChannelHandler_Global::Tick"), STAT_DataChannelHandler_Global_Tick, STATGROUP_NiagaraDataChannels);

//////////////////////////////////////////////////////////////////////////

UNiagaraDataChannelHandler* UNiagaraDataChannel_Global::CreateHandler(UWorld* OwningWorld)const
{
	UNiagaraDataChannelHandler* NewHandler = NewObject<UNiagaraDataChannelHandler_Global>(OwningWorld);
	NewHandler->Init(this);
	return NewHandler;
}

//TODO: Other channel types.
// Octree etc.


//////////////////////////////////////////////////////////////////////////


UNiagaraDataChannelHandler_Global::UNiagaraDataChannelHandler_Global(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNiagaraDataChannelHandler_Global::Init(const UNiagaraDataChannel* InChannel)
{
	Super::Init(InChannel);
}

void UNiagaraDataChannelHandler_Global::Cleanup()
{
	Data.Reset();
	Super::Cleanup();
}

void UNiagaraDataChannelHandler_Global::BeginFrame(float DeltaTime)
{
	Super::BeginFrame(DeltaTime);
	if(Data)
	{
		Data->BeginFrame(this);
	}
}

void UNiagaraDataChannelHandler_Global::EndFrame(float DeltaTime)
{
	Super::EndFrame(DeltaTime);
	if (Data)
	{
		Data->EndFrame(this);
	}
}

void UNiagaraDataChannelHandler_Global::Tick(float DeltaSeconds, ETickingGroup TickGroup)
{
	SCOPE_CYCLE_COUNTER(STAT_DataChannelHandler_Global_Tick);
	Super::Tick(DeltaSeconds, TickGroup);
	if (Data)
	{
		Data->ConsumePublishRequests(this, TickGroup);
	}
}

FNiagaraDataChannelDataPtr UNiagaraDataChannelHandler_Global::FindData(FNiagaraDataChannelSearchParameters SearchParams, ENiagaraResourceAccess AccessType)
{	
	//For more complicated channels we could check the location + bounds of the system instance etc to return some spatially localized data.
	if(Data.IsValid() == false)
	{
		Data = CreateData();
	}
	return Data;
}
