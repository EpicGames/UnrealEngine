// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannel_Islands.h"

#include "Engine/AssetManager.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "UObject/UObjectGlobals.h"

#include "NiagaraFunctionLibrary.h"
#include "NiagaraComponent.h"
#include "NiagaraWorldManager.h"
#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelData.h"

#include "DrawDebugHelpers.h"

//////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannel_Islands)

namespace NDCIslands_Local
{
	int32 AllowAsyncLoad = 1;
	static FAutoConsoleVariableRef CVarAllowAsyncLoad(TEXT("fx.Niagara.DataChannels.AllowAsyncLoad"), AllowAsyncLoad, TEXT("True if we should attempt to load systems etc asynchronously."), ECVF_Default);

	int32 BlockAsyncLoadOnUse = 1;
	static FAutoConsoleVariableRef CVarBlockAsyncLoadOnUse(TEXT("fx.Niagara.DataChannels.BlockAsyncLoadOnUse"), BlockAsyncLoadOnUse, TEXT("True if we should block on any pending async loads when those assets are used."), ECVF_Default);
}

void UNiagaraDataChannel_Islands::PostLoad()
{
	Super::PostLoad();
	
	//For now immediately load handler systems but in future we should look at deferring the load until a channel is used.
	//Sub classes may want to load differently so we'll keep this a soft ptr.
	AsyncLoadSystems();
}

TConstArrayView<TObjectPtr<UNiagaraSystem>> UNiagaraDataChannel_Islands::GetSystems()const
{
	check(IsInGameThread());
	if(NDCIslands_Local::BlockAsyncLoadOnUse && AsyncLoadHandle && AsyncLoadHandle->IsActive())
	{
		AsyncLoadHandle->WaitUntilComplete();
		PostLoadSystems();
	}

	return SystemsInternal;
}

void UNiagaraDataChannel_Islands::AsyncLoadSystems()const
{
	if(INiagaraModule::DataChannelsEnabled() && !IsRunningDedicatedServer())
	{
		TArray<FSoftObjectPath> Requests;
		for (const TSoftObjectPtr<UNiagaraSystem>& SoftSys : Systems)
		{
			if (SoftSys.IsPending())
			{
				Requests.Add(SoftSys.ToSoftObjectPath());
			}
		}

		if (Requests.Num() > 0)
		{
			if (NDCIslands_Local::AllowAsyncLoad && UAssetManager::IsInitialized())
			{
				AsyncLoadHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(Requests
					, FStreamableDelegate::CreateUObject(this, &UNiagaraDataChannel_Islands::PostLoadSystems)
					, FStreamableManager::AsyncLoadHighPriority
					, false
					, false
					, TEXT("NiagaraDataChannelIsland_HandlerSystems"));
			}
			else
			{
				for (FSoftObjectPath& Path : Requests)
				{
					Path.TryLoad();
				}
				PostLoadSystems();
			}
		}
		else
		{
			PostLoadSystems();
		}
	}
}

void UNiagaraDataChannel_Islands::PostLoadSystems()const
{
	check(IsInGameThread());
	SystemsInternal.Reset(Systems.Num());
	for(auto& SoftSys : Systems)
	{
		SystemsInternal.Add(SoftSys.Get());
	}

	AsyncLoadHandle.Reset();
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataChannelHandler* UNiagaraDataChannel_Islands::CreateHandler(UWorld* OwningWorld)const
{
	UNiagaraDataChannelHandler* NewHandler = NewObject<UNiagaraDataChannelHandler_Islands>(OwningWorld);
	NewHandler->Init(this);
	return NewHandler;
}

UNiagaraDataChannelHandler_Islands::UNiagaraDataChannelHandler_Islands(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNiagaraDataChannelHandler_Islands::Init(const UNiagaraDataChannel* InChannel)
{
	Super::Init(InChannel);
	if (const UNiagaraDataChannel_Islands* IslandChannel = CastChecked<UNiagaraDataChannel_Islands>(InChannel))
	{
		IslandChannel->AsyncLoadSystems();

		int32 InitialPoolCount = IslandChannel->GetIslandPoolSize();
		ActiveIslands.Reserve(InitialPoolCount);
		FreeIslands.Reserve(InitialPoolCount);
		IslandPool.SetNum(InitialPoolCount);
		for (int32 i = 0; i < InitialPoolCount; ++i)
		{
			IslandPool[i].Init(this);
			FreeIslands.Emplace(i);
		}
	}
}

void UNiagaraDataChannelHandler_Islands::Cleanup()
{
	ActiveIslands.Empty();
	FreeIslands.Empty();
	IslandPool.Empty();

	Super::Cleanup();
}

void UNiagaraDataChannelHandler_Islands::BeginFrame(float DeltaTime)
{
	Super::BeginFrame(DeltaTime);
	for (auto It = ActiveIslands.CreateIterator(); It; ++It)
	{
		FNDCIsland& Island = IslandPool[*It];
		Island.BeginFrame();
	}
}

void UNiagaraDataChannelHandler_Islands::EndFrame(float DeltaTime)
{
	Super::EndFrame(DeltaTime);
	const UNiagaraDataChannel_Islands* IslandChannel = GetChannelTyped<const UNiagaraDataChannel_Islands>();
	for (auto It = ActiveIslands.CreateIterator(); It; ++It)
	{
		FNDCIsland& Island = IslandPool[*It];
		Island.EndFrame();

		if (IslandChannel->GetDebugDrawSettings().ShowBounds())
		{
			Island.DebugDrawBounds();
		}

		//Free up islands that are not being used.
		if (Island.IsBeingUsed() == false && Island.GetData()->HasData() == false)
		{
			FreeIslands.Emplace(*It);
			Island.OnReleased();
			It.RemoveCurrentSwap();
		}
	}
}

void UNiagaraDataChannelHandler_Islands::Tick(float DeltaTime, ETickingGroup TickGroup)
{
	Super::Tick(DeltaTime, TickGroup);

	//static UEnum* TGEnum = StaticEnum<ETickingGroup>();
	//UE_LOGF(LogNiagara,Warning, "UNiagaraDataChannelHandler_Islands::Tick - %ls", *TGEnum->GetDisplayNameTextByValue((int32)TickGroup).ToString());
	const UNiagaraDataChannel_Islands* IslandChannel = GetChannelTyped<const UNiagaraDataChannel_Islands>();
	for (auto It = ActiveIslands.CreateIterator(); It; ++It)
	{
		FNDCIsland& Island = IslandPool[*It];
		Island.Tick(TickGroup);
	}
}

FNiagaraDataChannelDataPtr UNiagaraDataChannelHandler_Islands::FindData(FNiagaraDataChannelSearchParameters SearchParams, ENiagaraResourceAccess AccessType)
{
	if (FNDCIsland* Island = FindOrCreateIsland(SearchParams, AccessType))
	{
		return Island->GetData();
	}
	return nullptr;
}

FNDCIsland* UNiagaraDataChannelHandler_Islands::FindOrCreateIsland(const FNiagaraDataChannelSearchParameters& SearchParams, ENiagaraResourceAccess AccessType)
{
	const UNiagaraDataChannel_Islands* IslandChannel = CastChecked<const UNiagaraDataChannel_Islands>(DataChannel);

	FVector Location = SearchParams.GetLocation();
	if (AccessType == ENiagaraResourceAccess::ReadOnly)
	{
		//First we see if this is an islands handler system.
		if(SearchParams.GetOwner())
		{
			for (int32 i : ActiveIslands)
			{
				FNDCIsland& Island = IslandPool[i];
				if (Island.IsHandlerSystem(SearchParams.GetOwner()))
				{
					return &Island;
				}
			}
		}
	}

	//Find the first island that could contain this point.
	//For now we do a linear search of active islands.
	//Assuming that overall active island count will be low.
	//If this is not the case then we'll want to add an acceleration structure to speed up this search.
	FVector MaxExtents = IslandChannel->GetMaxExtents();
	FVector PerElementExtents = IslandChannel->GetPerElementExtents();

	if(IslandChannel->GetMode() == ENiagraDataChannel_IslandMode::AlignedStatic)
	{
		//for static islands we can just find the first one that contains our point.
		for (int32 i : ActiveIslands)
		{
			FNDCIsland& Island = IslandPool[i];
			if(Island.Contains(Location))
			{
				return &Island;
			}
		}
	}
	else if(IslandChannel->GetMode() == ENiagraDataChannel_IslandMode::Dynamic)
	{
		//for dynamic islands we find the first island that can grow to accommodate this point.
		for (int32 i : ActiveIslands)
		{
			FNDCIsland& Island = IslandPool[i];
			if(Island.TryGrow(Location, PerElementExtents, MaxExtents))
			{
				return &Island;
			}
		}
	}

	//Failing that, get and init a new island from the pool.
	FNDCIsland* IslandToUse = nullptr;
	int32 NewIslandIndex = ActivateNewIsland(Location);
	if (ensure(NewIslandIndex != INDEX_NONE))
	{
		IslandToUse = &IslandPool[NewIslandIndex];
	}

	if (IslandToUse)
	{
		IslandToUse->OnAcquired(Location);
	}

	return IslandToUse;
}

int32 UNiagaraDataChannelHandler_Islands::ActivateNewIsland(FVector Location)
{
	int32 NewIndex = INDEX_NONE;
	FNDCIsland* NewIsland = nullptr;
	if (FreeIslands.Num() > 0)
	{
		NewIndex = FreeIslands.Pop(EAllowShrinking::No);
		NewIsland = &IslandPool[NewIndex];
	}

	if (NewIsland == nullptr)
	{
		NewIndex = IslandPool.Num();
		NewIsland = &IslandPool.AddDefaulted_GetRef();
	}

	if (NewIndex != INDEX_NONE)
	{
		check(NewIsland);
		ActiveIslands.Add(NewIndex);
		NewIsland->Init(this);
	}
	else
	{
		UE_LOGF(LogNiagara, Error, "Failed to allocate new island for Island Data Channel %ls", *GetDataChannel()->GetName());
	}

	return NewIndex;
}

//////////////////////////////////////////////////////////////////////////

FNDCIsland::FNDCIsland()
{
}

FNDCIsland::~FNDCIsland()
{
	for (UNiagaraComponent* Comp : SpawnedComponents)
	{
		if (IsValid(Comp))
		{
			Comp->ReleaseToPool();
		}
	}
	SpawnedComponents.Empty();
}

void FNDCIsland::Init(UNiagaraDataChannelHandler_Islands* InOwner)
{
	Owner = InOwner;

	Data = Owner->CreateData();

	const UNiagaraDataChannel_Islands* Channel = CastChecked<UNiagaraDataChannel_Islands>(Owner->GetDataChannel());

	Bounds.Origin = FVector::ZeroVector;
	Bounds.BoxExtent = Channel->GetInitialExtents();
}

void FNDCIsland::OnAcquired(FVector Location)
{
	const UNiagaraDataChannel_Islands* Channel = CastChecked<UNiagaraDataChannel_Islands>(Owner->GetDataChannel());

	if(Channel->GetMode() == ENiagraDataChannel_IslandMode::AlignedStatic)
	{
		//Align our location down to the nearest lower multiple of the max island size to avoid overlapping islands.
		//TODO: make our padding extents come from the owning component or the handler systems?
		FVector ActualMaxSize = Channel->GetMaxExtents() + Channel->GetPerElementExtents();
		FVector BoxSize = Channel->GetMaxExtents() * 2;
		FVector BoxN = Location / BoxSize;
		BoxN.X = FMath::RoundToNegativeInfinity(BoxN.X);
		BoxN.Y = FMath::RoundToNegativeInfinity(BoxN.Y);
		BoxN.Z = FMath::RoundToNegativeInfinity(BoxN.Z);
		Bounds.Origin = BoxN * BoxSize + Channel->GetMaxExtents();
		Bounds.BoxExtent = ActualMaxSize;
	}
	else
	{
		Bounds.Origin = Location;
		Bounds.BoxExtent = Channel->GetInitialExtents();
	}

	Data->SetLwcTile(FLargeWorldRenderScalar::GetTileFor(Bounds.Origin));
	//Spawn our handler systems.
	FBox HandlerSystemBounds(-Bounds.BoxExtent, Bounds.BoxExtent);
	for (const TObjectPtr<UNiagaraSystem>& Sys : Channel->GetSystems())
	{
		if(Sys)
		{
			FFXSystemSpawnParameters SpawnParams;
			SpawnParams.bAutoActivate = false;//We must activate AFTER adding this component to our handler system array.
			SpawnParams.bAutoDestroy = false;
			SpawnParams.Location = Bounds.Origin;
			SpawnParams.PoolingMethod = EPSCPoolMethod::ManualRelease;
			SpawnParams.bPreCullCheck = false;
			SpawnParams.SystemTemplate = Sys;
			SpawnParams.WorldContextObject = Owner->GetWorld();
			if(UNiagaraComponent* NewComp = UNiagaraFunctionLibrary::SpawnSystemAtLocationWithParams(SpawnParams))
			{
				SpawnedComponents.Add(NewComp);
				NewComp->SetSystemFixedBounds(HandlerSystemBounds);
				NewComp->Activate();
			}
		}
	}

	if (Channel->GetMode() == ENiagraDataChannel_IslandMode::Dynamic)
	{
		ensure(TryGrow(Location, Channel->GetPerElementExtents(), Channel->GetMaxExtents()));
	}
}

void FNDCIsland::OnReleased()
{
	Data = nullptr;
	for (UNiagaraComponent* Comp : SpawnedComponents)
	{
		if (::IsValid(Comp))
		{
			Comp->ReleaseToPool();
		}
	}
	SpawnedComponents.Reset();
}

void FNDCIsland::BeginFrame()
{
	if (Data)
	{
		Data->BeginFrame(Owner);
	}
}

void FNDCIsland::EndFrame()
{
	if (Data)
	{
		Data->EndFrame(Owner);
	}
}

void FNDCIsland::Tick(const ETickingGroup& TickGroup)
{
	if (Data == nullptr)
	{
		return;
	}

	int32 AddedData = Data->ConsumePublishRequests(Owner, TickGroup);
	if (AddedData > 0)
	{
		for (UNiagaraComponent* Comp : SpawnedComponents)
		{
			if (::IsValid(Comp) && Comp->IsComplete())
			{
				Comp->Activate();
			}
		}
	}
}

bool FNDCIsland::Contains(FVector Point)
{
	if (DistanceToPoint(Point) > 0)
	{
		return false;
	}

	return true;
}

double FNDCIsland::DistanceToPoint(FVector Point)
{
	return Bounds.ComputeSquaredDistanceFromBoxToPoint(Point);
}

bool FNDCIsland::Intersects(const FBoxSphereBounds& CheckBounds)
{
	return FBoxSphereBounds::BoxesIntersect(Bounds, CheckBounds);
}

bool FNDCIsland::IsHandlerSystem(USceneComponent* Component)
{
	return SpawnedComponents.Contains(Component);
}

bool FNDCIsland::IsBeingUsed()const
{
	const bool bSystemsIncomplete = SpawnedComponents.ContainsByPredicate([](const TObjectPtr<UNiagaraComponent>& Comp) { return IsValid(Comp) && !Comp->IsComplete(); });
	return Data.IsValid() && (!Data.IsUnique() || bSystemsIncomplete);
}

bool FNDCIsland::TryGrow(FVector Point, FVector PerElementExtents, FVector MaxIslandExtents)
{
	FVector CurrMax = Bounds.Origin + Bounds.BoxExtent;
	FVector CurrMin = Bounds.Origin - Bounds.BoxExtent;

	FVector ElementMax = Point + PerElementExtents;
	FVector ElementMin = Point - PerElementExtents;

	//The max size of the island, we pad these out with the per element extents to ensure bounds cover anything right at the boundary.
	FVector Max = Bounds.Origin + MaxIslandExtents + PerElementExtents;
	FVector Min = Bounds.Origin - MaxIslandExtents - PerElementExtents;

	//Can this island contain this element at all?

	//Calculate the require bounds to encompass the point + the per element bounds.
	FVector NewMax(FMath::Max(ElementMax.X, CurrMax.X), FMath::Max(ElementMax.Y, CurrMax.Y), FMath::Max(ElementMax.Z, CurrMax.Z));
	FVector NewMin(FMath::Min(ElementMin.X, CurrMin.X), FMath::Min(ElementMin.Y, CurrMin.Y), FMath::Min(ElementMin.Z, CurrMin.Z));

	//Return false if the new bounds are outside the max allowed for this island.
	if (NewMax.X > Max.X || NewMax.Y > Max.Y || NewMax.Z > Max.Z || NewMin.X < Min.X || NewMin.Y < Min.Y || NewMin.Z < Min.Z)
	{
		return false;
	}

	FVector OldOrigin = Bounds.Origin;
	FBox LocalBounds(NewMin - Bounds.Origin, NewMax - Bounds.Origin);
	Bounds = LocalBounds;
	Bounds.Origin += OldOrigin;

	//If new bounds are valid, grow our bounds and update the system bounds
	FBox HandlerSystemBounds(-Bounds.BoxExtent, Bounds.BoxExtent);
	for (auto It = SpawnedComponents.CreateIterator(); It; ++It)
	{
		UNiagaraComponent* Comp = *It;
		if (IsValid(Comp))
		{
			Comp->SetSystemFixedBounds(LocalBounds);
			Comp->SetWorldLocation(Bounds.Origin);
		}
		else
		{
			It.RemoveCurrentSwap();
		}
	}

	return true;
}

void FNDCIsland::DebugDrawBounds()
{
	if (Owner && Owner->GetWorld())
	{
		DrawDebugBox(Owner->GetWorld(), Bounds.Origin, Bounds.BoxExtent, FQuat::Identity, FColor::Red);
	}
}
