// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannel_Map.h"

#include "HAL/IConsoleManager.h"
#include "NiagaraDataChannelCommon.h"
#include "NiagaraWorldManager.h"
#include "NiagaraComponent.h"
#include "NiagaraDataChannelData.h"
#include "NiagaraFunctionLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannel_Map)

namespace NDCMapLocal
{
	int32 EntryPoolInitialSize = 0;
	static FAutoConsoleVariableRef CVarEntryPoolInitialSize(TEXT("fx.Niagara.DataChannels.NDCMapBase.InitialSize"), EntryPoolInitialSize, TEXT("The initial size of the entry pool for NDC Map types."), ECVF_Default);

	float EntryPoolUnusedCleanTime = 60.0f;
	static FAutoConsoleVariableRef CVarEntryPoolUnusedCleanTime(TEXT("fx.Niagara.DataChannels.NDCMapBase.FreeUnusedTime"), EntryPoolUnusedCleanTime, TEXT("Time in seconds we keep unused entries in the pool."), ECVF_Default);
}

//////////////////////////////////////////////////////////////////////////
// UNiagaraDataChannelHandler_MapBase 
//
// Base class for NDC handler types sub divide their data internally based on a customizable map key.
//

UNiagaraDataChannelHandler_MapBase::UNiagaraDataChannelHandler_MapBase(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNiagaraDataChannelHandler_MapBase::Init(const UNiagaraDataChannel* InChannel)
{
	Super::Init(InChannel);

	const UNiagaraDataChannel_MapBase* NDCTyped = CastChecked<UNiagaraDataChannel_MapBase>(InChannel);
	DefaultSystemToSpawn = NDCTyped->GetDefaultSystemToSpawn();
}

void UNiagaraDataChannelHandler_MapBase::Cleanup()
{
	Super::Cleanup();

	for (auto It = ActiveEntries.CreateIterator(); It; ++It)
	{
		FNDCMapEntry& Entry = EntryPool[It.Value()];
		Entry.Get().Reset(It.Key());
		Entry.Get().Cleanup();
	}
	for (auto It = FreeEntries.CreateIterator(); It; ++It)
	{
		FNDCMapEntry& Entry = EntryPool[*It];
		Entry.Get().Cleanup();
	}

	ActiveEntries.Empty();
	FreeEntries.Empty();
	EntryPool.Empty();
	SpawnedComponentsToActiveEntry.Empty();

	DefaultSystemToSpawn = nullptr;
}

void UNiagaraDataChannelHandler_MapBase::BeginFrame(float DeltaTime)
{
	Super::BeginFrame(DeltaTime);

	for(auto It = ActiveEntries.CreateIterator(); It; ++It)
	{
		FNDCMapEntry& Entry = EntryPool[It.Value()];
		if (Entry.Get().BeginFrame(DeltaTime, OwningWorld, It.Key()) == false)
 		{
 			//This entry is done with so reset it and return to the free entries.
 
 			// First clear our it's spawned components.
 			for(TObjectPtr<UNiagaraComponent> SpawnedSystem : Entry.Get().SpawnedComponents)
 			{
 				if(SpawnedSystem)
 				{
 					SpawnedComponentsToActiveEntry.Remove(SpawnedSystem.Get());
 				}
 			}
 
 			Entry.Get().Reset(It.Key());
 			FreeEntries.Emplace(It.Value());
 			It.RemoveCurrent();
 		}
	}

	CleanFreeEntriesTimer += DeltaTime;
	if (CleanFreeEntriesTimer >= NDCMapLocal::EntryPoolUnusedCleanTime)
	{
		CleanFreeEntriesTimer = 0.0f;
		float RealTime = OwningWorld->GetWorld()->GetRealTimeSeconds();
		
		//Sort the free list into reverse order periodically so we tend to use free items earlier in the entry pool and we can more often free items from the end.
		FreeEntries.Sort([](int32 A, int32 B){ return A > B;});

		for(auto It = FreeEntries.CreateIterator(); It; ++It)
		{
			int32 FreeEntryIdx = *It;
			FNDCMapEntryBase& BaseEntry = EntryPool[FreeEntryIdx].Get();

			//Pop entries off the back of the pool if they've been free for a long time.
			float UnusedTime = RealTime - BaseEntry.GetLastUsedTime();
			if(FreeEntryIdx == EntryPool.Num() - 1 && UnusedTime >= NDCMapLocal::EntryPoolUnusedCleanTime)
			{
				BaseEntry.Cleanup();
				EntryPool.Pop(EAllowShrinking::No);
				It.RemoveCurrent();
			}
			else
			{
				break;
			}
		}

		FreeEntries.Shrink();
		EntryPool.Shrink();
	}
}

void UNiagaraDataChannelHandler_MapBase::EndFrame(float DeltaTime)
{
	Super::EndFrame(DeltaTime);
	for (auto It = ActiveEntries.CreateIterator(); It; ++It)
	{
		FNDCMapEntry& Entry = EntryPool[It.Value()];
		Entry.Get().EndFrame(DeltaTime, OwningWorld, It.Key());
	}
}

void UNiagaraDataChannelHandler_MapBase::Tick(float DeltaTime, ETickingGroup TickGroup)
{
	Super::Tick(DeltaTime, TickGroup);
	for (auto It = ActiveEntries.CreateIterator(); It; ++It)
	{
		FNDCMapEntry& Entry = EntryPool[It.Value()];
		Entry.Get().Tick(DeltaTime, TickGroup, OwningWorld, It.Key());
	}
}

FNiagaraDataChannelDataPtr UNiagaraDataChannelHandler_MapBase::FindData(FNDCAccessContextInst& AccessContext, ENiagaraResourceAccess AccessType)
{
	FNDCMapEntry& Entry = FindOrAddEntry(AccessContext);

	if(Entry.IsValid())
	{
		return Entry.Get().GetData();
	}
	return nullptr;
}

UObject* UNiagaraDataChannelHandler_MapBase::GetSystemToSpawn(FNDCAccessContextInst& AccessContext)const
{
	const FNDCAccessContext_MapBase& TypedContext = AccessContext.GetChecked<FNDCAccessContext_MapBase>();
	if(TypedContext.bOverrideSystemToSpawn)
	{
		return TypedContext.SystemToSpawn;
	}
	return DefaultSystemToSpawn;
}

void UNiagaraDataChannelHandler_MapBase::GenerateKey(FNDCAccessContextInst& AccessContext, FNDCMapKeyWriter& KeyWriter)const
{
	const FNDCAccessContext_MapBase& TypedContext = AccessContext.GetChecked<FNDCAccessContext_MapBase>();

	// We write our systems into the key so that all users can use a single NDC that has internally distinct buckets for different handler systems.
	const UObject* SystemToSpawn = GetSystemToSpawn(AccessContext);
	if(IsValid(SystemToSpawn))
	{
		KeyWriter << SystemToSpawn->GetFName();
	}
}

FNDCMapEntry& UNiagaraDataChannelHandler_MapBase::FindOrAddEntry(FNDCAccessContextInst& AccessContext)
{
	FNDCAccessContext_MapBase& TypedContext = AccessContext.GetChecked<FNDCAccessContext_MapBase>();

	//Clear out any transient output data from our context.
	//TODO; Should possibly use virtual or iterate over properties to clear out all transient output properties here.
	TypedContext.SpawnedSystems.Reset();
	FNDCMapEntry* Ret = nullptr;
	int32 NumSpawnedSystems = 0;
	if (TypedContext.bIsAutoLinkingSystem)
	{
		//Only Niagara Components spawned by this NDC should be setting this flag. It is purely to allow auto-linking back to the originating NDC data without the need for generating a key.
		if (UNiagaraComponent* OwnerComp = Cast<UNiagaraComponent>(TypedContext.GetOwner()))
		{
			if (CurrentlyInitializingEntry)
			{
				if (CurrentlyInitializingEntry->Get().SpawnedComponents.Contains(OwnerComp))
				{
					//The currently initializing entry spawned this component so just return this entry.
					Ret = CurrentlyInitializingEntry;
				}
			}
			else
			{
				//Even if we're not currently activating an entry, this might be a handler system.
				//Use the reverse mapping to find it without needing to do a full key gen.
				if (int32* FoundEntryIdx = SpawnedComponentsToActiveEntry.Find(OwnerComp))
				{
					check(EntryPool[*FoundEntryIdx].Get().SpawnedComponents.Contains(OwnerComp));
					Ret = &EntryPool[*FoundEntryIdx];
				}
			}
		}
	}
	else
	{
		//Generate the key for this Access Context. Each child class will implement this as needed.
		FNDCMapKey Key;
		FNDCMapKeyWriter KeyWriter(Key);
		GenerateKey(AccessContext, KeyWriter);

		int32* ActiveEntry = ActiveEntries.Find(Key);
		if (ActiveEntry)
		{					
			Ret = &EntryPool[*ActiveEntry];
		}
		else if (!TypedContext.bIsAutoLinkingSystem)//Don't create new entries for auto linking Niagara systems.
		{
			int32 NewEntryIdx = INDEX_NONE;
			if (FreeEntries.Num() > 0)
			{
				NewEntryIdx = FreeEntries.Pop();
			}
			else
			{
				NewEntryIdx = EntryPool.Num();
				EntryPool.Emplace(GetMapEntryType(), NewEntryIdx);
			}

			check(NewEntryIdx != INDEX_NONE);
			FNDCMapEntry* RetEntry = &EntryPool[NewEntryIdx];
			check(RetEntry);
			ActiveEntries.Add(Key) = NewEntryIdx;

			//Init the new entry.
			//Cache off the currently initializing entry so we can retreive it easily should the entry spawn any systems that re-enter FindData() and need to quickly find their spawning NDC data.
			CurrentlyInitializingEntry = RetEntry;

			RetEntry->Get().Init(AccessContext, this, Key);

			NumSpawnedSystems = TypedContext.SpawnedSystems.Num();
			CurrentlyInitializingEntry = nullptr;

			//Add any spawned components to the reverse lookup map.
			for (const FNDCSpawnedSystemRef& WeakSpawned : TypedContext.SpawnedSystems)
			{
				if (UNiagaraComponent* Spawned = WeakSpawned.Get())
				{
					SpawnedComponentsToActiveEntry.Add(Spawned) = NewEntryIdx;
				}
			}
			Ret = RetEntry;
		}
	}
	
	if(Ret)
	{
		//Return any existing systems if needed.
		FNDCMapEntryBase& TypedEntry = Ret->Get<FNDCMapEntryBase>();
		if(TypedContext.bReturnExistingSystems)
		{
			//Any newly spawned systems are added in entry init. If we need all systems, add the rest now.
			const TArray<TObjectPtr<UNiagaraComponent>>& EntrySystems = TypedEntry.GetSpawnedSystems();
			for(int32 i=0; i < EntrySystems.Num() - NumSpawnedSystems; ++i)
			{
				TypedContext.SpawnedSystems.Add(EntrySystems[i].Get());
			}			
		}
		return *Ret;
	}

	static FNDCMapEntry Dummy;
	return Dummy;
}

//////////////////////////////////////////////////////////////////////////
// FNDCMapEntryBase

void FNDCMapEntryBase::Init(FNDCAccessContextInst& AccessContext, UNiagaraDataChannelHandler_MapBase* InOwner, const FNDCMapKey& Key)
{
	Owner = InOwner;
	Data = Owner->CreateData();
	LastUsedTime = 0.0f;
}

void FNDCMapEntryBase::Reset(const FNDCMapKey& Key)
{
	if (Data)
	{
		Data->Reset();
	}

	for (TObjectPtr<UNiagaraComponent>& Comp : SpawnedComponents)
	{
		if(Comp)
		{
			Comp->ReleaseToPool();
		}
	}
	SpawnedComponents.Empty();
}

void FNDCMapEntryBase::Cleanup()
{
	Owner = nullptr;
	Data = nullptr;
	LastUsedTime = 0.0f;
	check(SpawnedComponents.Num() == 0);//We should already hit a reset call to clear up our components before this.
}

bool FNDCMapEntryBase::BeginFrame(float DeltaTime, FNiagaraWorldManager* OwningWorld, const FNDCMapKey& Key)
{
	check(OwningWorld && OwningWorld->GetWorld());
	LastUsedTime = OwningWorld->GetWorld()->GetRealTimeSeconds();
	bool bInUse = false;
	if (Data)
	{
		Data->BeginFrame(Owner);

		//Return unused if our data is unique. Child types may add additional definitions of "in use" such as waiting for spawned fx to complete.
		bInUse = Data.IsUnique() == false;
	}

	if (!bInUse)
	{
		for (TObjectPtr<UNiagaraComponent> Comp : SpawnedComponents)
		{
			if (Comp && !Comp->IsComplete())
			{
				bInUse = true;//Still have a running system so return that we are being used.
				break;
			}
		}
	}

	return bInUse;
}

void FNDCMapEntryBase::EndFrame(float DeltaTime, FNiagaraWorldManager* OwningWorld, const FNDCMapKey& Key)
{
	if (Data)
	{
		Data->EndFrame(Owner);
	}
}

void FNDCMapEntryBase::Tick(float DeltaTime, ETickingGroup TickGroup, FNiagaraWorldManager* OwningWorld, const FNDCMapKey& Key)
{
	if (Data)
	{
		Data->ConsumePublishRequests(Owner, TickGroup);
	}
}

UNiagaraComponent* FNDCMapEntryBase::SpawnSystem(FNDCAccessContext_MapBase& AccessContext, UObject* System, USceneComponent* AttachParent, FVector Location, FVector BoundExtents, bool bActivate)
{
	UNiagaraComponent* NewComp = nullptr;
	auto SpawnSys = [&](UNiagaraSystem* Sys)
	{
		if (Sys)
		{
			FFXSystemSpawnParameters SpawnParams;
			SpawnParams.bAutoActivate = false;//We must activate AFTER adding this component to our entry system array.
			SpawnParams.bAutoDestroy = false;
			SpawnParams.PoolingMethod = EPSCPoolMethod::ManualRelease;
			SpawnParams.bPreCullCheck = false;
			SpawnParams.SystemTemplate = Sys;
			SpawnParams.WorldContextObject = Owner->GetWorld();

			if (AttachParent)
			{
				SpawnParams.AttachToComponent = AttachParent;
				SpawnParams.LocationType = EAttachLocation::SnapToTarget;
				NewComp = UNiagaraFunctionLibrary::SpawnSystemAttachedWithParams(SpawnParams);
			}
			else
			{
				SpawnParams.Location = Location;
				NewComp = UNiagaraFunctionLibrary::SpawnSystemAtLocationWithParams(SpawnParams);
			}

			if (NewComp)
			{
				//Track our spawned Systems. This MUST be done before we activate the system to enable auto linking for NDC DIs inside the spawned system.
				SpawnedComponents.Add(NewComp);
				//Also pass the newly spawned component to the access context so the caller can set parameters etc on the spawned system.
				AccessContext.SpawnedSystems.Add(NewComp);

				FBox HandlerSystemBounds(-BoundExtents, BoundExtents);
				NewComp->SetSystemFixedBounds(HandlerSystemBounds);
				if (bActivate)
				{
					NewComp->Activate();
				}
			}
		}
	};

	if(UNiagaraSystem* NiagaraSystem = Cast<UNiagaraSystem>(System))
	{
		SpawnSys(NiagaraSystem);
	}
	else if(UNiagaraSystemCollection* SystemCollection = Cast<UNiagaraSystemCollection>(System))
	{
		for(UNiagaraSystem* Sys : SystemCollection->GetSystems())
		{
			SpawnSys(Sys);
		}
	}
	return NewComp;
}
