// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterface.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraTypes.h"
#include "NiagaraGPUSystemTick.h"
#include "NiagaraGpuComputeDispatch.h"
#include "NiagaraShader.h"
#include "NiagaraSystem.h"
#include "NiagaraComponent.h"
#include "NiagaraSystemGpuComputeProxy.h"
#include "ShaderCompilerCore.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraScriptRuntimeData.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterface)

#define LOCTEXT_NAMESPACE "NiagaraDataInterface"

void FNiagaraDataInterfaceProxy::SetSourceDIName(FName InName)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (SourceDIName == InName)
	{
		return;
	}
	SourceDIName = InName;

	ENQUEUE_RENDER_COMMAND(SetProxySourceDIName)
	(
		[this, InName](FRHICommandListImmediate& RHICmdList) mutable
		{
			SourceDIName_RT = InName;
		}
	);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FNDIStageTickHandler::Init(ENiagaraScriptUsage ScriptUsage, const FNiagaraScriptRuntimeData* ScriptRuntimeData, FNiagaraSystemInstance* Instance)
{
	check(ScriptRuntimeData);
	check(Instance);
	Usage = ScriptUsage;
	PreStageTickList.Empty();
	PostStageTickList.Empty();

	// Resolved interfaces can point to duplicate instances so we need to make sure that we're only adding
	// each data interface instance to the tick lists once.
	TSet<UNiagaraDataInterface*> SeenDataInterfaces;
	for (const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDIData : ScriptRuntimeData->GetResolvedDataInterfaces())
	{
		if (ResolvedDIData.ResolvedDataInterface && SeenDataInterfaces.Contains(ResolvedDIData.ResolvedDataInterface) == false)
		{
			SeenDataInterfaces.Add(ResolvedDIData.ResolvedDataInterface);
			bool bPreStage = ResolvedDIData.ResolvedDataInterface->HasPreStageTick(Usage);
			bool bPostStage = ResolvedDIData.ResolvedDataInterface->HasPostStageTick(Usage);
			if (bPreStage || bPostStage)
			{
				int32 InstDataIdx = Instance->FindDataInterfaceInstanceDataIndex(ResolvedDIData.ResolvedDataInterface);
				if (InstDataIdx != INDEX_NONE)
				{
					if (bPreStage)
					{
						PreStageTickList.Add(InstDataIdx);
					}
					if (bPostStage)
					{
						PostStageTickList.Add(InstDataIdx);
					}
				}
			}
		}
	}
}

void FNDIStageTickHandler::PreStageTick(FNiagaraSystemInstance* Instance, float DeltaSeconds)
{
	FNDICpuPreStageContext Context;
	Context.SystemInstance = Instance;
	Context.DeltaSeconds = DeltaSeconds;
	Context.Usage = Usage;
	for (int32 DIInstDataIndex : PreStageTickList)
	{
		UNiagaraDataInterface* Interface = nullptr;
		Instance->GetDataInterfaceInstanceDataInfo(DIInstDataIndex, Interface, Context.PerInstanceData);
		if (Interface && Context.PerInstanceData)
		{			
			Interface->PreStageTick(Context);
		}
	}
}

void FNDIStageTickHandler::PostStageTick(FNiagaraSystemInstance* Instance, float DeltaSeconds)
{
	FNDICpuPostStageContext Context;
	Context.SystemInstance = Instance;
	Context.DeltaSeconds = DeltaSeconds;
	Context.Usage = Usage;
	for (int32 DIInstDataIndex : PostStageTickList)
	{
		UNiagaraDataInterface* Interface = nullptr;
		Instance->GetDataInterfaceInstanceDataInfo(DIInstDataIndex, Interface, Context.PerInstanceData);
		if (Interface && Context.PerInstanceData)
		{
			Interface->PostStageTick(Context);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterface::UNiagaraDataInterface(FObjectInitializer const& ObjectInitializer)
{
	bRenderDataDirty = false;
	bUsedWithCPUScript = false;
	bUsedWithGPUScript = false;
}

UNiagaraDataInterface::~UNiagaraDataInterface()
{
	if ( FNiagaraDataInterfaceProxy* ReleasedProxy = Proxy.Release() )
	{
		ENQUEUE_RENDER_COMMAND(FDeleteProxyRT) (
			[RT_Proxy= ReleasedProxy](FRHICommandListImmediate& CmdList) mutable
			{
				delete RT_Proxy;
			}
		);
		check(Proxy.IsValid() == false);
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterface::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	//-TODO: Currently applied to all, but we only need to hash this in for the iteration source
	const UNiagaraDataInterface* BaseDataInterface = GetDefault<UNiagaraDataInterface>();
	const FString DataInterfaceName = GetClass()->GetName();
	if (BaseDataInterface->GetGpuDispatchType() != GetGpuDispatchType())
	{
		InVisitor->UpdatePOD(*FString::Printf(TEXT("%s_GpuDispatchType"), *DataInterfaceName), (int32)GetGpuDispatchType());
		InVisitor->UpdateString(*FString::Printf(TEXT("%s_GpuDispatchNumThreads"), *DataInterfaceName), *FString::Printf(TEXT("%dx%dx%d"), GetGpuDispatchNumThreads().X, GetGpuDispatchNumThreads().Y, GetGpuDispatchNumThreads().Z));
	}
	InVisitor->UpdatePOD(*FString::Printf(TEXT("%s_GpuUseIndirectDispatch"), *DataInterfaceName), (int32)GetGpuUseIndirectDispatch());
	
	return true;
}
#endif

#if WITH_EDITOR
void UNiagaraDataInterface::ModifyCompilationEnvironment(EShaderPlatform ShaderPlatform, struct FShaderCompilerEnvironment& OutEnvironment) const
{
	if (FDataDrivenShaderPlatformInfo::GetSupportsDxc(ShaderPlatform))
	{
		// Always enable DXC to avoid compile errors caused by RWBuffer/Buffer in structs. Example NiagaraDataInterfaceHairStrands.ush struct FDIHairStrandsContext
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}
}
#endif

void UNiagaraDataInterface::GetAssetTagsForContext(const UObject* InAsset,  FGuid AssetVersion, const TArray<const UNiagaraDataInterface*>& InProperties, TMap<FName, uint32>& NumericKeys, TMap<FName, FString>& StringKeys) const
{
	UClass* Class = GetClass();

	// Default count up how many instances there are of this class and report to content browser
	if (Class)
	{
		uint32 NumInstances = 0;
		for (const UNiagaraDataInterface* Prop : InProperties)
		{
			if (Prop && Prop->IsA(Class))
			{
				NumInstances++;
			}
		}
		
		// Note that in order for these tags to be registered, we always have to put them in place for the CDO of the object, but 
		// for readability's sake, we leave them out of non-CDO assets.
		if (NumInstances > 0 || (InAsset && InAsset->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject)))
		{
			FString Key = Class->GetName();
			Key.ReplaceInline(TEXT("NiagaraDataInterface"), TEXT(""));
			NumericKeys.Add(*Key) = NumInstances;
		}
	}
}

#if WITH_EDITOR
void UNiagaraDataInterface::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	RefreshErrors();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UNiagaraDataInterface::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITORONLY_DATA
	if (BuildObjectFlagsForOwner(GetOuter()) & RF_Public)
	{
		SetFlags(RF_Public);
	}
	else
	{
		ClearFlags(RF_Public);
	}
#endif
}

bool UNiagaraDataInterface::CopyTo(UNiagaraDataInterface* Destination) const 
{
	bool result = CopyToInternal(Destination);
#if WITH_EDITOR
	Destination->OnChanged().Broadcast();
#endif
	return result;
}

bool UNiagaraDataInterface::Equals(const UNiagaraDataInterface* Other) const
{
	if (Other == nullptr || Other->GetClass() != GetClass())
	{
		return false;
	}
	return true;
}

bool UNiagaraDataInterface::IsDataInterfaceType(const FNiagaraTypeDefinition& TypeDef)
{
	const UClass* Class = TypeDef.GetClass();
	if (Class && Class->IsChildOf(UNiagaraDataInterface::StaticClass()))
	{
		return true;
	}
	return false;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterface::AppendTemplateHLSL(FString& OutHLSL, const TCHAR* TemplateShaderFile, const TMap<FString, FStringFormatArg>& TemplateArgs) const
{
	FString TemplateFile;
	LoadShaderSourceFile(TemplateShaderFile, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL.Append(FString::Format(*TemplateFile, TemplateArgs));
	OutHLSL.AppendChar('\n');
}
#endif

bool UNiagaraDataInterface::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (Destination == nullptr || Destination->GetClass() != GetClass())
	{
		return false;
	}
	return true;
}

#if WITH_EDITOR

void UNiagaraDataInterface::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL)
{
	//Base implementation will call into the legacy functions so that all DIs need not be updated immediately.
	//Do not implement both current and legacy functions in any DI or it's parent classes.
	GetParameterDefinitionHLSL(HlslGenContext.ParameterInfo, OutHLSL);
}

bool UNiagaraDataInterface::GetFunctionHLSL(const FNiagaraDataInterfaceHlslGenerationContext& HlslGenContext, FString& OutHLSL)
{
	//Base implementation will call into the legacy functions so that all DIs need not be updated immediately.
	//Do not implement both current and legacy functions in any DI or it's parent classes.
	return GetFunctionHLSL(HlslGenContext.ParameterInfo, HlslGenContext.ParameterInfo.GeneratedFunctions[HlslGenContext.FunctionInstanceIndex], HlslGenContext.FunctionInstanceIndex, OutHLSL);
}

void UNiagaraDataInterface::GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	OutErrors = GetErrors();
	OutWarnings.Empty();
	OutInfo.Empty();
}

void UNiagaraDataInterface::GetFeedback(UNiagaraDataInterface* DataInterface, TArray<FNiagaraDataInterfaceError>& Errors, TArray<FNiagaraDataInterfaceFeedback>& Warnings,
	TArray<FNiagaraDataInterfaceFeedback>& Info)
{
	if (!DataInterface)
		return;

	UNiagaraSystem* Asset = nullptr;
	UNiagaraComponent* Component = nullptr;

	// Walk the hierarchy to attempt to get the system and/or component
	UObject* Curr = DataInterface->GetOuter();
	while (Curr)
	{
		Asset = Cast<UNiagaraSystem>(Curr);
		if (Asset)
		{
			break;
		}

		Component = Cast<UNiagaraComponent>(Curr);
		if (Component)
		{
			Asset = Component->GetAsset();
			break;
		}

		Curr = Curr->GetOuter();
	}

	DataInterface->GetFeedback(Asset, Component, Errors, Warnings, Info);
}

void UNiagaraDataInterface::GetFunctionSignatures(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
#if WITH_EDITORONLY_DATA
	// only cache things if we're dealing with the CDO
	if (HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
		{
			UE::TReadScopeLock ReadLock(CachedFunctionSignatureLock);
			if (!CachedFunctionSignatures.IsEmpty())
			{
				OutFunctions = CachedFunctionSignatures;
				return;
			}
		}

		{
			UE::TWriteScopeLock WriteLock(CachedFunctionSignatureLock);

			if (CachedFunctionSignatures.IsEmpty())
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS // to be removed when GetFunctions is removed
				const_cast<UNiagaraDataInterface*>(this)->GetFunctions(CachedFunctionSignatures);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS // to be removed when GetFunctions is removed
			}

			OutFunctions = CachedFunctionSignatures;
		}
	}
	else
	{
		return GetClass()->GetDefaultObject<UNiagaraDataInterface>()->GetFunctionSignatures(OutFunctions);
	}
#endif
}

void UNiagaraDataInterface::ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors)
{
	TArray<FNiagaraFunctionSignature> DIFuncs;
	GetFunctionSignatures(DIFuncs);

	if (!DIFuncs.ContainsByPredicate([&](const FNiagaraFunctionSignature& Sig) { return Sig.EqualsIgnoringSpecifiers(Function); }))
	{
		//We couldn't find this signature in the list of available functions.
		//Lets try to find one with the same name whose parameters may have changed.
		int32 ExistingSigIdx = DIFuncs.IndexOfByPredicate([&](const FNiagaraFunctionSignature& Sig) { return Sig.Name == Function.Name; });;
		if (ExistingSigIdx != INDEX_NONE)
		{
			OutValidationErrors.Add(FText::Format(LOCTEXT("DI Function Parameter Mismatch!", "Data Interface function called but it's parameters do not match any available function!\nThe API for this data interface function has likely changed and you need to update your graphs.\nInterface: {0}\nFunction: {1}\n"), FText::FromString(GetClass()->GetName()), FText::FromName(Function.Name)));
		}
		else
		{
			OutValidationErrors.Add(FText::Format(LOCTEXT("Unknown DI Function", "Unknown Data Interface function called!\nThe API for this data interface has likely changed and you need to update your graphs.\nInterface: {0}\nFunction: {1}\n"), FText::FromString(GetClass()->GetName()), FText::FromName(Function.Name)));
		}
	}
}

void UNiagaraDataInterface::RefreshErrors()
{
	OnErrorsRefreshedDelegate.Broadcast();
}

FSimpleMulticastDelegate& UNiagaraDataInterface::OnErrorsRefreshed()
{
	return OnErrorsRefreshedDelegate;
}

#endif

// DataInterfaces owned by public objects or templates will be made public.  We might be able to further
// refine this, but for now it provides a significant reduction in public DI which ensures that DI created
// in the editor that are unnecessary in game will be stripped during the cook process and reduce our static
// memory footprint.
EObjectFlags UNiagaraDataInterface::BuildObjectFlagsForOwner(UObject* OwnerObject, EObjectFlags DefaultFlags)
{
	if (OwnerObject && (OwnerObject->HasAnyFlags(RF_Public) || OwnerObject->IsTemplate()))
	{
		return DefaultFlags | RF_Public;
	}

	return DefaultFlags;
}

//////////////////////////////////////////////////////////////////////////
//FNDIGpuComputeContext

FRDGExternalAccessQueue& FNDIGpuComputeContext::GetRDGExternalAccessQueue() const
{
	return static_cast<const FNiagaraGpuComputeDispatch&>(ComputeDispatchInterface).GetCurrentPassExternalAccessQueue();
}

//////////////////////////////////////////////////////////////////////////
//FNDIGpuComputePrePostStageContext

FNiagaraSystemInstanceID FNDIGpuComputePrePostStageContext::GetSystemInstanceID() const
{
	return SystemTick.SystemInstanceID;
}

FVector3f FNDIGpuComputePrePostStageContext::GetSystemLWCTile() const
{
	return SystemTick.SystemGpuComputeProxy->GetSystemLWCTile();
}

bool FNDIGpuComputePrePostStageContext::IsOutputStage() const
{
	check(DataInterfaceProxy);
	return ComputeInstanceData.IsOutputStage(DataInterfaceProxy, SimStageData.StageIndex);
}

bool FNDIGpuComputePrePostStageContext::IsInputStage() const
{
	check(DataInterfaceProxy);
	return ComputeInstanceData.IsInputStage(DataInterfaceProxy, SimStageData.StageIndex);
}

bool FNDIGpuComputePrePostStageContext::IsIterationStage() const
{
	check(DataInterfaceProxy);
	return ComputeInstanceData.IsIterationStage(DataInterfaceProxy, SimStageData.StageIndex);
}

FNiagaraGPUInstanceCountManager& FNDIGpuComputePrePostStageContext::GetInstanceCountManager() const
{
	return ComputeDispatchInterface.GetGPUInstanceCounterManager();
}

//////////////////////////////////////////////////////////////////////////
//FNDIGpuComputePostSimulateContext

FNiagaraGPUInstanceCountManager& FNDIGpuComputePostSimulateContext::GetInstanceCountManager() const
{
	return ComputeDispatchInterface.GetGPUInstanceCounterManager();
}

//////////////////////////////////////////////////////////////////////////

FRDGExternalAccessQueue& FNiagaraDataInterfaceSetShaderParametersContext::GetRDGExternalAccessQueue() const
{
	return static_cast<const FNiagaraGpuComputeDispatch&>(ComputeDispatchInterface).GetCurrentPassExternalAccessQueue();
}

FNiagaraSystemInstanceID FNiagaraDataInterfaceSetShaderParametersContext::GetSystemInstanceID() const
{
	return SystemTick.SystemInstanceID;
}

FVector3f FNiagaraDataInterfaceSetShaderParametersContext::GetSystemLWCTile() const
{
	return SystemTick.SystemGpuComputeProxy->GetSystemLWCTile();
}

bool FNiagaraDataInterfaceSetShaderParametersContext::IsResourceBound(const void* ResourceAddress) const
{
	const uint16 ByteOffset = uint16(uintptr_t(ResourceAddress) - uintptr_t(BaseParameters));

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	for (const FShaderParameterBindings::FBindlessResourceParameter& BindlessResourceParameter : ShaderRef->Bindings.BindlessResourceParameters)
	{
		if (BindlessResourceParameter.ByteOffset == ByteOffset)
		{
			return true;
		}
	}
#endif

	for (const FShaderParameterBindings::FResourceParameter& ResourceParameter : ShaderRef->Bindings.ResourceParameters)
	{
		if ( ResourceParameter.ByteOffset == ByteOffset )
		{
			return true;
		}
	}
	return false;
}

bool FNiagaraDataInterfaceSetShaderParametersContext::IsParameterBound(const void* ParameterAddress) const
{
	const uint16 ByteOffset = uint16(uintptr_t(ParameterAddress) - uintptr_t(BaseParameters));
	for (const FShaderParameterBindings::FParameter& Parameter : ShaderRef->Bindings.Parameters)
	{
		if (Parameter.ByteOffset > ByteOffset)
		{
			return false;
		}

		const uint16 Delta = ByteOffset - Parameter.ByteOffset;
		if (Delta < Parameter.ByteSize)
		{
			return true;
		}
	}
	return false;
}

bool FNiagaraDataInterfaceSetShaderParametersContext::IsStructBoundInternal(const void* StructAddress, uint32 StructSize) const
{
	const uint16 ByteStart = uint16(uintptr_t(StructAddress) - uintptr_t(BaseParameters));
	const uint16 ByteEnd = ByteStart + uint16(StructSize);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	// Loop over bindless resources
	for (const FShaderParameterBindings::FBindlessResourceParameter& BindlessResourceParameter : ShaderRef->Bindings.BindlessResourceParameters)
	{
		if (BindlessResourceParameter.ByteOffset >= ByteEnd)
		{
			break;
		}
		if (BindlessResourceParameter.ByteOffset >= ByteStart)
		{
			return true;
		}
	}
#endif

	// Loop over resources
	for (const FShaderParameterBindings::FResourceParameter& ResourceParameter : ShaderRef->Bindings.ResourceParameters)
	{
		if (ResourceParameter.ByteOffset >= ByteEnd)
		{
			break;
		}
		if (ResourceParameter.ByteOffset >= ByteStart)
		{
			return true;
		}
	}

	// Loop over parameters
	for (const FShaderParameterBindings::FParameter& Parameter : ShaderRef->Bindings.Parameters)
	{
		if (Parameter.ByteOffset >= ByteEnd)
		{
			break;
		}

		if (Parameter.ByteOffset >= ByteStart)
		{
			return true;
		}
	}

	return false;
}

uint16 FNiagaraDataInterfaceSetShaderParametersContext::GetParameterIncludedStructInternal(const FShaderParametersMetadata* StructMetadata) const
{
	for (const FNiagaraDataInterfaceStructIncludeInfo& StructIncludeInfo : ShaderParametersMetadata.StructIncludeInfos)
	{
		if (StructIncludeInfo.StructMetadata == StructMetadata)
		{
			ensure(StructIncludeInfo.ParamterOffset <= TNumericLimits<uint16>::Max());
			return uint16(StructIncludeInfo.ParamterOffset);
		}
	}

	UE_LOGF(LogNiagara, Fatal, "Failed to find Included Parameter Struct '%ls' in parameters", StructMetadata->GetStructTypeName());
	return 0;
}

bool FNiagaraDataInterfaceSetShaderParametersContext::IsOutputStage() const
{
	return ComputeInstanceData.IsOutputStage(DataInterfaceProxy, SimStageData.StageIndex);
}

bool FNiagaraDataInterfaceSetShaderParametersContext::IsIterationStage() const
{
	return ComputeInstanceData.IsIterationStage(DataInterfaceProxy, SimStageData.StageIndex);
}

#if WITH_NIAGARA_DEBUG_EMITTER_NAME
FString FNiagaraDataInterfaceSetShaderParametersContext::GetDebugString() const
{
	TStringBuilder<128> Builder;
	Builder.Append(ComputeInstanceData.Context ? ComputeInstanceData.Context->GetDebugSimName() : TEXT("Unknown"));
	if (SimStageData.StageMetaData)
	{
		Builder.AppendChar(TEXT(','));
		SimStageData.StageMetaData->SimulationStageName.AppendString(Builder);
	}
	return Builder.ToString();
}
#endif

#undef LOCTEXT_NAMESPACE

