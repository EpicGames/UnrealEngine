// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterface2DArrayTexture.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"

#include "Engine/Texture2DArray.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "RenderGraphBuilder.h"
#include "RHIStaticStates.h"
#include "RenderGraphUtils.h"
#include "ShaderCompilerCore.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterface2DArrayTexture)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterface2DArrayTexture"

const TCHAR* UNiagaraDataInterface2DArrayTexture::TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceTexture2DArrayTemplate.ush");
const FName UNiagaraDataInterface2DArrayTexture::LoadTextureName(TEXT("LoadTexture"));
const FName UNiagaraDataInterface2DArrayTexture::GatherRedTextureName(TEXT("GatherRedTexture"));
const FName UNiagaraDataInterface2DArrayTexture::SampleTextureName(TEXT("SampleTexture"));
const FName UNiagaraDataInterface2DArrayTexture::TextureDimsName(TEXT("TextureDimensions"));

struct FNDITexture2DArrayInstanceData_GameThread
{
	TWeakObjectPtr<UTexture> CurrentTexture = nullptr;
	FIntVector CurrentTextureSize = FIntVector::ZeroValue;
	FNiagaraParameterDirectBinding<UObject*> UserParamBinding;
};

struct FNDITexture2DArrayInstanceData_RenderThread
{
	FSamplerStateRHIRef		SamplerStateRHI;
	FTextureReferenceRHIRef	TextureReferenceRHI;
	FVector3f				TextureSize;

	FRDGTextureRef			TransientRDGTexture = nullptr;
};

struct FNiagaraDataInterfaceProxyTexture2DArray : public FNiagaraDataInterfaceProxy
{
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { check(false); }
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override
	{
		if (Context.IsFinalPostSimulate())
		{
			if (FNDITexture2DArrayInstanceData_RenderThread* InstanceData = InstanceData_RT.Find(Context.GetSystemInstanceID()))
			{
				InstanceData->TransientRDGTexture = nullptr;
			}
		}
	}

	TMap<FNiagaraSystemInstanceID, FNDITexture2DArrayInstanceData_RenderThread> InstanceData_RT;
};

UNiagaraDataInterface2DArrayTexture::UNiagaraDataInterface2DArrayTexture(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, Texture(nullptr)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyTexture2DArray());

	FNiagaraTypeDefinition Def(UTexture::StaticClass());
	TextureUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterface2DArrayTexture::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterface2DArrayTexture::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterface2DArrayTexture* DestinationTexture = CastChecked<UNiagaraDataInterface2DArrayTexture>(Destination);
	DestinationTexture->Texture = Texture;
	DestinationTexture->TextureUserParameter = TextureUserParameter;

	return true;
}

bool UNiagaraDataInterface2DArrayTexture::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterface2DArrayTexture* OtherTexture = CastChecked<const UNiagaraDataInterface2DArrayTexture>(Other);
	return
		OtherTexture->Texture == Texture &&
		OtherTexture->TextureUserParameter == TextureUserParameter;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterface2DArrayTexture::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	OutFunctions.Reserve(OutFunctions.Num() + 3);

	FNiagaraFunctionSignature DefaultSig;
	DefaultSig.bMemberFunction = true;
	DefaultSig.bRequiresContext = false;
	DefaultSig.bSupportsCPU = false;
	DefaultSig.bSupportsGPU = true;
	DefaultSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Texture"));

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = LoadTextureName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelZ"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("TextureLoad2DArrayTextureDesc", "Read a texel from the provided location & mip without any filtering or sampling."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = GatherRedTextureName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UVW"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("TextureGatherTexture2ArrayDDesc", "Gather the 4 samples (Red only) that would be used in bilinear interpolation."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SampleTextureName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UVW"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("TextureSample2DArrayTextureDesc", "Sample the specified mip level of the input texture at the specified UVW coordinates. Where W is the slice to sample (0,1,2, etc) and UV are the coordinates into the slice."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = TextureDimsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Texture"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Dimensions"));
		Sig.SetDescription(LOCTEXT("TextureDimsDesc", "Get the dimensions of mip 0 of the texture."));
	}
}
#endif

void UNiagaraDataInterface2DArrayTexture::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == TextureDimsName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterface2DArrayTexture::GetTextureDimensions);
	}
}

int32 UNiagaraDataInterface2DArrayTexture::PerInstanceDataSize() const
{
	return sizeof(FNDITexture2DArrayInstanceData_GameThread);
}

bool UNiagaraDataInterface2DArrayTexture::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDITexture2DArrayInstanceData_GameThread* InstanceData = new (PerInstanceData) FNDITexture2DArrayInstanceData_GameThread();
	InstanceData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), TextureUserParameter.Parameter);
	return true;
}

void UNiagaraDataInterface2DArrayTexture::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDITexture2DArrayInstanceData_GameThread* InstanceData = static_cast<FNDITexture2DArrayInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FNDITexture2DArrayInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(NDITexture_RemoveInstance)
	(
		[RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyTexture2DArray>(), RT_InstanceID = SystemInstance->GetId()](FRHICommandListImmediate&)
		{
			RT_Proxy->InstanceData_RT.Remove(RT_InstanceID);
		}
	);
}

bool UNiagaraDataInterface2DArrayTexture::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FNDITexture2DArrayInstanceData_GameThread* InstanceData = static_cast<FNDITexture2DArrayInstanceData_GameThread*>(PerInstanceData);

	UTexture* CurrentTexture = InstanceData->UserParamBinding.GetValueOrDefault<UTexture>(Texture);
	if ( InstanceData->CurrentTexture != CurrentTexture )
	{
		UTexture2DArray* CurrentTextureArray = Cast<UTexture2DArray>(CurrentTexture);
		UTextureRenderTarget2DArray* CurrentTextureRT = Cast<UTextureRenderTarget2DArray>(CurrentTexture);
		if (CurrentTextureArray || CurrentTextureRT)
		{
			const FIntVector CurrentTextureSize = CurrentTextureArray ?
				FIntVector(CurrentTextureArray->GetSizeX(), CurrentTextureArray->GetSizeY(), CurrentTextureArray->GetArraySize()) :
				FIntVector(CurrentTextureRT->SizeX, CurrentTextureRT->SizeY, CurrentTextureRT->Slices);

			InstanceData->CurrentTexture = CurrentTexture;
			InstanceData->CurrentTextureSize = CurrentTextureSize;

			ENQUEUE_RENDER_COMMAND(NDITexture_UpdateInstance)
			(
				[RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxyTexture2DArray>(), RT_InstanceID=SystemInstance->GetId(), RT_Texture=CurrentTexture, RT_TextureSize=CurrentTextureSize](FRHICommandListImmediate&)
				{
					FNDITexture2DArrayInstanceData_RenderThread& InstanceData = RT_Proxy->InstanceData_RT.FindOrAdd(RT_InstanceID);
					if (RT_Texture)
					{
						InstanceData.TextureReferenceRHI = RT_Texture->TextureReference.TextureReferenceRHI;
						InstanceData.SamplerStateRHI = RT_Texture->GetResource() ? RT_Texture->GetResource()->SamplerStateRHI.GetReference() : TStaticSamplerState<SF_Point>::GetRHI();
						InstanceData.TextureSize.X = float(RT_TextureSize.X);
						InstanceData.TextureSize.Y = float(RT_TextureSize.Y);
						InstanceData.TextureSize.Z = float(RT_TextureSize.Z);
					}
					else
					{
						InstanceData.TextureReferenceRHI = nullptr;
						InstanceData.SamplerStateRHI = nullptr;
						InstanceData.TextureSize = FVector3f::ZeroVector;
					}
				}
			);
		}
	}
	return false;
}

void UNiagaraDataInterface2DArrayTexture::GetTextureDimensions(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDITexture2DArrayInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<FVector3f> DimensionsOut(Context);

	const FVector FloatTextureSize(InstData->CurrentTextureSize.X, InstData->CurrentTextureSize.Y, InstData->CurrentTextureSize.Z);
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		DimensionsOut.SetAndAdvance((FVector3f)FloatTextureSize);
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterface2DArrayTexture::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterface2DArrayTexture::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterface2DArrayTexture::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	static const TSet<FName> ValidGpuFunctions =
	{
		LoadTextureName,
		GatherRedTextureName,
		SampleTextureName,
		TextureDimsName,
	};
	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}
#endif

void UNiagaraDataInterface2DArrayTexture::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterface2DArrayTexture::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyTexture2DArray& TextureProxy = Context.GetProxy<FNiagaraDataInterfaceProxyTexture2DArray>();
	FNDITexture2DArrayInstanceData_RenderThread* InstanceData = TextureProxy.InstanceData_RT.Find(Context.GetSystemInstanceID());
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	FShaderParameters* Parameters = Context.GetParameterNestedStruct<FShaderParameters>();
	if (InstanceData && InstanceData->TextureReferenceRHI.IsValid())
	{
		Parameters->TextureSize = InstanceData->TextureSize;
		Parameters->TextureSampler = InstanceData->SamplerStateRHI;
		if (Context.IsResourceBound(&Parameters->Texture))
		{
			FRDGTextureRef RDGTexture = InstanceData ? InstanceData->TransientRDGTexture : nullptr;
			if (InstanceData && RDGTexture == nullptr)
			{
				FTextureRHIRef ResolvedTextureRHI = InstanceData->TextureReferenceRHI->GetReferencedTexture();
				if (ResolvedTextureRHI.IsValid())
				{
					InstanceData->TransientRDGTexture = GraphBuilder.FindExternalTexture(ResolvedTextureRHI);
					if (InstanceData->TransientRDGTexture == nullptr)
					{
						InstanceData->TransientRDGTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(ResolvedTextureRHI, TEXT("NiagaraTexture2DArray")));

						GraphBuilder.UseInternalAccessMode(InstanceData->TransientRDGTexture);
						Context.GetRDGExternalAccessQueue().Add(InstanceData->TransientRDGTexture);
					}
					RDGTexture = InstanceData->TransientRDGTexture;
				}
			}

			Parameters->Texture = RDGTexture ? RDGTexture : Context.GetComputeDispatchInterface().GetBlackTexture(GraphBuilder, ETextureDimension::Texture2DArray);
		}
	}
	else
	{
		Parameters->TextureSize = FVector3f::ZeroVector;
		Parameters->TextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		Parameters->Texture = Context.GetComputeDispatchInterface().GetBlackTexture(GraphBuilder, ETextureDimension::Texture2DArray);
	}
}

void UNiagaraDataInterface2DArrayTexture::SetTexture(UTexture* InTexture)
{
	if (InTexture)
	{
		Texture = InTexture;
	}
}

#undef LOCTEXT_NAMESPACE

