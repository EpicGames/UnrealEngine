// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceArray.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraComponent.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"

#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceArray)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterfaceArray"

UNiagaraDataInterfaceArray::UNiagaraDataInterfaceArray(FObjectInitializer const& ObjectInitializer)
{
}

void UNiagaraDataInterfaceArray::PostInitProperties()
{
	Super::PostInitProperties();
	
	if (HasAnyFlags(RF_ClassDefaultObject) && (GetClass() != UNiagaraDataInterfaceArray::StaticClass()))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		MarkRenderDataDirty();
	}
}

void UNiagaraDataInterfaceArray::PostLoad()
{
	Super::PostLoad();
	MarkRenderDataDirty();
}

#if WITH_EDITOR
void UNiagaraDataInterfaceArray::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

bool UNiagaraDataInterfaceArray::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceArray* OtherTyped = CastChecked<UNiagaraDataInterfaceArray>(Destination);
	OtherTyped->MaxElements = MaxElements;
	OtherTyped->GpuSyncMode = GpuSyncMode;
	return GetProxyAs<INDIArrayProxyBase>()->CopyToInternal(OtherTyped->GetProxyAs<INDIArrayProxyBase>());
}

bool UNiagaraDataInterfaceArray::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfaceArray* OtherTyped = CastChecked<UNiagaraDataInterfaceArray>(Other);
	if ((OtherTyped->MaxElements != MaxElements) ||
		(OtherTyped->GpuSyncMode != GpuSyncMode))
	{
		return false;
	}

	return GetProxyAs<INDIArrayProxyBase>()->Equals(OtherTyped->GetProxyAs<INDIArrayProxyBase>());
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceArray::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderParameters<INDIArrayProxyBase::FShaderParameters>();
	bSuccess &= GetProxyAs<INDIArrayProxyBase>()->AppendCompileHash(InVisitor);
	return bSuccess;
}
#endif

void UNiagaraDataInterfaceArray::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<INDIArrayProxyBase::FShaderParameters>();
}

void UNiagaraDataInterfaceArray::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	INDIArrayProxyBase& ArrayProxy = Context.GetProxy<INDIArrayProxyBase>();
	INDIArrayProxyBase::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<INDIArrayProxyBase::FShaderParameters>();

	ArrayProxy.SetShaderParameters(ShaderParameters, Context.GetSystemInstanceID());
}

UObject* UNiagaraDataInterfaceArray::SimCacheBeginWrite(UObject* SimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	return OptionalPerInstanceData ? NewObject<UNDIArraySimCacheData>(SimCache) : nullptr;
}

bool UNiagaraDataInterfaceArray::SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	check(OptionalPerInstanceData && StorageObject);
	UNDIArraySimCacheData* CacheData = CastChecked<UNDIArraySimCacheData>(StorageObject);

	const INDIArrayProxyBase* ArrayProxy = GetProxyAs<INDIArrayProxyBase>();
	return ArrayProxy->SimCacheWriteFrame(CacheData, FrameIndex, SystemInstance);
}

bool UNiagaraDataInterfaceArray::SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext)
{
	INDIArrayProxyBase* ArrayProxy = GetProxyAs<INDIArrayProxyBase>();
	return ArrayProxy->SimCacheReadFrame(ReadContext.GetStorageObject<UNDIArraySimCacheData>(), ReadContext.GetFrameIndexA(), ReadContext.GetSystemInstance());
}

bool UNiagaraDataInterfaceArray::SimCacheCompareFrame(const UObject* LhsStorageObject, const UObject* RhsStorageObject, int FrameIndex, TOptional<float> InTolerance, FString& OutErrors) const
{
	const UNDIArraySimCacheData* LhsCacheData = CastChecked<const UNDIArraySimCacheData>(LhsStorageObject);
	const UNDIArraySimCacheData* RhsCacheData = CastChecked<const UNDIArraySimCacheData>(RhsStorageObject);
	const bool bHasCpuData = LhsCacheData->CpuFrameData.IsValidIndex(FrameIndex) || RhsCacheData->CpuFrameData.IsValidIndex(FrameIndex);
	const bool bHasGpuData = LhsCacheData->GpuFrameData.IsValidIndex(FrameIndex) || RhsCacheData->GpuFrameData.IsValidIndex(FrameIndex);
	const bool bIsCpuFrameIndexValid = LhsCacheData->CpuFrameData.IsValidIndex(FrameIndex) && RhsCacheData->CpuFrameData.IsValidIndex(FrameIndex);
	const bool bIsGpuFrameIndexValid = !bHasGpuData || (LhsCacheData->GpuFrameData.IsValidIndex(FrameIndex) && RhsCacheData->GpuFrameData.IsValidIndex(FrameIndex));

	if (!bIsCpuFrameIndexValid || !bIsGpuFrameIndexValid )
	{
		OutErrors = TEXT("FrameIndex was not valid");
		return false;
	}

	const INDIArrayProxyBase* ArrayProxy = GetProxyAs<INDIArrayProxyBase>();
	const float Tolerance = InTolerance.Get(UE_SMALL_NUMBER);

	auto CompareFrames =
		[&](const FNDIArraySimCacheDataFrame& LhsFrame, const FNDIArraySimCacheDataFrame& RhsFrame, const TCHAR* SimType)
		{
			if (LhsFrame.NumElements != RhsFrame.NumElements)
			{
				OutErrors = FString::Printf(TEXT("Element Count Mismatch (%d -> %d) for %s data"), LhsFrame.NumElements, RhsFrame.NumElements, SimType);
				return false;
			}

			const uint8* LhsArrayData = LhsCacheData->BufferData.GetData() + LhsFrame.DataOffset;
			const uint8* RhsArrayData = RhsCacheData->BufferData.GetData() + RhsFrame.DataOffset;
			for (int32 i=0; i < LhsFrame.NumElements; ++i)
			{
				if (!ArrayProxy->SimCacheCompareElement(LhsArrayData, RhsArrayData, i, Tolerance))
				{
					const FString LhsValue = ArrayProxy->SimCacheVisualizerRead(LhsCacheData, LhsFrame, i);
					const FString RhsValue = ArrayProxy->SimCacheVisualizerRead(RhsCacheData, RhsFrame, i);
					OutErrors = FString::Printf(TEXT("Element %d Mismatch (%s -> %s) for %s data"), i, *LhsValue, *RhsValue, SimType);
					return false;
				}
			}
			return true;
		};

	return
		(!bHasCpuData || CompareFrames(LhsCacheData->CpuFrameData[FrameIndex], RhsCacheData->CpuFrameData[FrameIndex], TEXT("CPU"))) &&
		(!bHasGpuData || CompareFrames(LhsCacheData->GpuFrameData[FrameIndex], RhsCacheData->GpuFrameData[FrameIndex], TEXT("GPU")));
}

FString UNiagaraDataInterfaceArray::SimCacheVisualizerRead(const UNDIArraySimCacheData* CacheData, const FNDIArraySimCacheDataFrame& FrameData, int Element) const
{
	const INDIArrayProxyBase* ArrayProxy = GetProxyAs<INDIArrayProxyBase>();
	return ArrayProxy->SimCacheVisualizerRead(CacheData, FrameData, Element);
}

#if WITH_EDITOR

void UNiagaraDataInterfaceArray::CreateAndSetVariant(UNiagaraComponent* NiagaraComponent, FName OverrideName, TFunction<void(UNiagaraDataInterfaceArray*)> SetDataDelegate)
{
	// We only need to do this for editor instances of the component as we are storing instance data on them
	// For runtime instances they already have a unique copy of the data interface that we are modifying
	UWorld* World = NiagaraComponent ? NiagaraComponent->GetWorld() : nullptr;
	if (World && !World->IsGameWorld())
	{
		NiagaraComponent->Modify();

		UNiagaraDataInterfaceArray* VariantDI = CastChecked<UNiagaraDataInterfaceArray>(DuplicateObject(this, NiagaraComponent));
		CopyTo(VariantDI);
		SetDataDelegate(VariantDI);
		NiagaraComponent->SetParameterOverride(FNiagaraVariableBase(FNiagaraTypeDefinition(GetClass()), OverrideName), FNiagaraVariant(VariantDI));
	
		// We reinitialize the system as we need to flush out any queue'ed up GPU ticks as the DI could be GCed.
		// This would normally occur naturally on the next system tick, but we may not tick the system before rendering
		if (NiagaraComponent->IsActive())
		{
			NiagaraComponent->ReinitializeSystem();
		}
	}
}
#endif

int32 UNDIArraySimCacheData::FindOrAddData(TConstArrayView<uint8> ArrayData)
{
	UE::TScopeLock ScopeLock(FindOrAddGuard);

	const int32 Last = BufferData.Num() - ArrayData.Num();
	for (int32 i = 0; i <= Last; ++i)
	{
		if (FMemory::Memcmp(BufferData.GetData() + i, ArrayData.GetData(), ArrayData.Num()) == 0)
		{
			return i;
		}
	}

	int32 NewOffset = BufferData.Num();
	BufferData.AddUninitialized(ArrayData.Num());
	FMemory::Memcpy(BufferData.GetData() + NewOffset, ArrayData.GetData(), ArrayData.Num());
	return NewOffset;
}

#undef LOCTEXT_NAMESPACE

