// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceArrayFloat.h"
#include "NiagaraSystemInstance.h"

#include "Engine/Canvas.h"
#include "DrawDebugHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceArrayFloat)

#if WITH_NIAGARA_DEBUGGER

static float GNDIArrayPositionDebugRadius = 32.0f;
static FAutoConsoleVariableRef CVarNDIArrayPositionDebugRadius(
	TEXT("fx.Niagara.Array.PositionDebugRadius"),
	GNDIArrayPositionDebugRadius,
	TEXT("When using the Niagara Debugger the radius of position array debug display"),
	ECVF_Default
);

void UNiagaraDataInterfaceArrayPosition::DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const
{
	Super::DrawDebugHud(DebugHudContext);

	if (DebugHudContext.IsVerbose())
	{
		UCanvas* Canvas = DebugHudContext.GetCanvas();
		const FNiagaraLWCConverter LWCConverter = DebugHudContext.GetSystemInstance()->GetLWCConverter();
		const float Radius = GNDIArrayPositionDebugRadius;
		for (const FNiagaraPosition& Position : PositionData)
		{
			const FVector WorldPosition = LWCConverter.ConvertSimulationPositionToWorld(Position);
			const FVector ScreenPosition = Canvas->Project(WorldPosition, false);
			if (ScreenPosition.Z <= 0.0f)
			{
				continue;
			}

			DrawDebugCanvasWireSphere(Canvas, WorldPosition, FColor::Red, Radius, 16);
		}
	}
}
#endif
