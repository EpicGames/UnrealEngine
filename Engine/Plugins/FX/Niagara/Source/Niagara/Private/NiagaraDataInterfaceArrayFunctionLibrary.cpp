// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceArrayFunctionLibrary.h"

#include "Engine/World.h"
#include "NiagaraComponent.h"
#include "NiagaraDataInterfaceArrayFloat.h"
#include "NiagaraDataInterfaceArrayInt.h"
#include "NiagaraDataInterfaceArrayImpl.h"
#include "NiagaraDataInterfaceArrayImplHelpers.h"
#include "NiagaraDataInterfaceArrayNiagaraID.h"
#include "NiagaraComponent.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraFunctionLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceArrayFunctionLibrary)

NDIARRAY_GENERATE_IMPL(UNiagaraDataInterfaceArrayFloat, float, FloatData)
NDIARRAY_GENERATE_IMPL_LWC(UNiagaraDataInterfaceArrayFloat2, FVector2f, FloatData)
NDIARRAY_GENERATE_IMPL_LWC(UNiagaraDataInterfaceArrayFloat3, FVector3f, FloatData)
NDIARRAY_GENERATE_IMPL(UNiagaraDataInterfaceArrayPosition, FNiagaraPosition, PositionData)
NDIARRAY_GENERATE_IMPL_LWC(UNiagaraDataInterfaceArrayFloat4, FVector4f, FloatData)
NDIARRAY_GENERATE_IMPL(UNiagaraDataInterfaceArrayColor, FLinearColor, ColorData)
NDIARRAY_GENERATE_IMPL_LWC(UNiagaraDataInterfaceArrayQuat, FQuat4f, QuatData)
NDIARRAY_GENERATE_IMPL_LWC(UNiagaraDataInterfaceArrayMatrix, FMatrix44f, MatrixData)
NDIARRAY_GENERATE_IMPL(UNiagaraDataInterfaceArrayInt32, int32, IntData)
NDIARRAY_GENERATE_IMPL_LWC(UNiagaraDataInterfaceArrayUInt8, uint8, IntData)
NDIARRAY_GENERATE_IMPL(UNiagaraDataInterfaceArrayBool, bool, BoolData)
NDIARRAY_GENERATE_IMPL(UNiagaraDataInterfaceArrayNiagaraID, FNiagaraID, IntData)

// If / when we share user parameter UObjects we will need to make this per instance which introduces some tricky things about allocating before the instance is active
template<typename TArrayType, typename TDataInterace>
void SetNiagaraArray(UNiagaraComponent* NiagaraComponent, FName OverrideName, TConstArrayView<TArrayType> InArray)
{
	if (TDataInterace* ArrayDI = UNiagaraFunctionLibrary::GetDataInterface<TDataInterace>(NiagaraComponent, OverrideName))
	{
		auto* ArrayProxy = static_cast<typename TDataInterace::FProxyType*>(ArrayDI->GetProxy());
		ArrayProxy->SetArrayData(InArray);
	#if WITH_EDITOR
		ArrayDI->CreateAndSetVariant(
			NiagaraComponent,
			OverrideName,
			[InArray](UNiagaraDataInterfaceArray* VariantDI)
			{
				CastChecked<TDataInterace>(VariantDI)->SetVariantArrayData(InArray);
			}
		);
	#endif
	}
}

template<typename TArrayType, typename TDataInterace>
TArray<TArrayType> GetNiagaraArray(UNiagaraComponent* NiagaraComponent, FName OverrideName)
{
	if (TDataInterace* ArrayDI = UNiagaraFunctionLibrary::GetDataInterface<TDataInterace>(NiagaraComponent, OverrideName))
	{
		auto* ArrayProxy = static_cast<typename TDataInterace::FProxyType*>(ArrayDI->GetProxy());
		return ArrayProxy->template GetArrayDataCopy<TArrayType>();
	}
	return TArray<TArrayType>();
}

template<typename TArrayType, typename TDataInterace>
void SetNiagaraArrayValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, const TArrayType& Value, bool bSizeToFit)
{
	if (TDataInterace* ArrayDI = UNiagaraFunctionLibrary::GetDataInterface<TDataInterace>(NiagaraComponent, OverrideName))
	{
		auto* ArrayProxy = static_cast<typename TDataInterace::FProxyType*>(ArrayDI->GetProxy());
		ArrayProxy->SetArrayValue(Index, Value, bSizeToFit);
#if WITH_EDITOR
		// We only need to do this for editor instances of the component as we are storing instance data on them
		// For runtime instances they already have a unique copy of the data interface that we are modifying
		UWorld* World = NiagaraComponent->GetWorld();
		if (World && !World->IsGameWorld())
		{
			TDataInterace* VariantDI = CastChecked<TDataInterace>(DuplicateObject(ArrayDI, NiagaraComponent));
			ArrayDI->CopyTo(VariantDI);
			VariantDI->SetVariantArrayValue(Index, Value, bSizeToFit);
			NiagaraComponent->SetParameterOverride(FNiagaraVariableBase(FNiagaraTypeDefinition(TDataInterace::StaticClass()), OverrideName), FNiagaraVariant(VariantDI));

			// We reinitialize the system as we need to flush out any queue'ed up GPU ticks as the DI could be GCed.
			// This would normally occur naturally on the next system tick, but we may not tick the system before rendering
			if (NiagaraComponent->IsActive())
			{
				NiagaraComponent->ReinitializeSystem();
			}
		}
#endif
	}
}

template<typename TArrayType, typename TDataInterace>
TArrayType GetNiagaraArrayValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index)
{
	if (TDataInterace* ArrayDI = UNiagaraFunctionLibrary::GetDataInterface<TDataInterace>(NiagaraComponent, OverrideName))
	{
		auto* ArrayProxy = static_cast<typename TDataInterace::FProxyType*>(ArrayDI->GetProxy());
		return ArrayProxy->template GetArrayValue<TArrayType>(Index);
	}
	//-TODO: Should be DefaultValue
	return TArrayType();
}

FNiagaraLWCConverter GetLWCConverter(UNiagaraComponent* NiagaraComponent)
{
	if (NiagaraComponent == nullptr)
	{
		return FNiagaraLWCConverter();
	}

	FNiagaraSystemInstanceControllerPtr SystemInstanceController = NiagaraComponent->GetSystemInstanceController();
	if (!SystemInstanceController.IsValid())
	{
		// the instance controller can be invalid if there is no simulation, for example when fx.SuppressNiagaraSystems is set 
		return FNiagaraLWCConverter();
	}
	FNiagaraSystemInstance* SystemInstance = SystemInstanceController->GetSystemInstance_Unsafe();
	return SystemInstance->GetLWCConverter();
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayFloat(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<float>& ArrayData)
{
	SetNiagaraArray<float, UNiagaraDataInterfaceArrayFloat>(NiagaraComponent, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayVector2D(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<FVector2D>& ArrayData)
{
	SetNiagaraArray<FVector2D, UNiagaraDataInterfaceArrayFloat2>(NiagaraComponent, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayVector(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<FVector>& ArrayData)
{
	SetNiagaraArray<FVector, UNiagaraDataInterfaceArrayFloat3>(NiagaraComponent, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayPosition(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<FVector>& ArrayData)
{
	FNiagaraLWCConverter LwcConverter = GetLWCConverter(NiagaraComponent);
	TArray<FNiagaraPosition> ConvertedData;
	ConvertedData.SetNumUninitialized(ArrayData.Num());
	for (int i = 0; i < ArrayData.Num(); i++)
	{
		ConvertedData[i] = LwcConverter.ConvertWorldToSimulationPosition(ArrayData[i]);
	}
	SetNiagaraArray<FNiagaraPosition, UNiagaraDataInterfaceArrayPosition>(NiagaraComponent, OverrideName, ConvertedData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayVector4(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<FVector4>& ArrayData)
{
	SetNiagaraArray<FVector4, UNiagaraDataInterfaceArrayFloat4>(NiagaraComponent, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayColor(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<FLinearColor>& ArrayData)
{
	SetNiagaraArray<FLinearColor, UNiagaraDataInterfaceArrayColor>(NiagaraComponent, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayQuat(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<FQuat>& ArrayData)
{
	SetNiagaraArray<FQuat, UNiagaraDataInterfaceArrayQuat>(NiagaraComponent, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayMatrix(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<FMatrix>& ArrayData, bool bApplyLWCRebase)
{
	if (bApplyLWCRebase)
	{
		const FNiagaraLWCConverter LwcConverter = GetLWCConverter(NiagaraComponent);

		TArray<FMatrix> RebasedArrayData;
		RebasedArrayData.SetNumUninitialized(ArrayData.Num());
		for (int32 i=0; i < ArrayData.Num(); ++i)
		{
			RebasedArrayData[i] = LwcConverter.ConvertWorldToSimulationMatrix(ArrayData[i]);
		}
		SetNiagaraArray<FMatrix, UNiagaraDataInterfaceArrayMatrix>(NiagaraComponent, OverrideName, RebasedArrayData);
	}
	else
	{
		SetNiagaraArray<FMatrix, UNiagaraDataInterfaceArrayMatrix>(NiagaraComponent, OverrideName, ArrayData);
	}
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayInt32(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<int32>& ArrayData)
{
	SetNiagaraArray<int32, UNiagaraDataInterfaceArrayInt32>(NiagaraComponent, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayUInt8(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<int32>& ArrayData)
{
	SetNiagaraArray<int32, UNiagaraDataInterfaceArrayUInt8>(NiagaraComponent, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayBool(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<bool>& ArrayData)
{
	SetNiagaraArray<bool, UNiagaraDataInterfaceArrayBool>(NiagaraComponent, OverrideName, ArrayData);
}

//////////////////////////////////////////////////////////////////////////

TArray<float> UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayFloat(UNiagaraComponent* NiagaraComponent, FName OverrideName)
{
	return GetNiagaraArray<float, UNiagaraDataInterfaceArrayFloat>(NiagaraComponent, OverrideName);
}

TArray<FVector2D> UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayVector2D(UNiagaraComponent* NiagaraComponent, FName OverrideName)
{
	return GetNiagaraArray<FVector2D, UNiagaraDataInterfaceArrayFloat2>(NiagaraComponent, OverrideName);
}

TArray<FVector> UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayVector(UNiagaraComponent* NiagaraComponent, FName OverrideName)
{
	return GetNiagaraArray<FVector, UNiagaraDataInterfaceArrayFloat3>(NiagaraComponent, OverrideName);
}

TArray<FVector> UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayPosition(UNiagaraComponent* NiagaraComponent, FName OverrideName)
{
	const TArray<FNiagaraPosition> SimData = GetNiagaraArray<FNiagaraPosition, UNiagaraDataInterfaceArrayPosition>(NiagaraComponent, OverrideName);
	
	FNiagaraLWCConverter LwcConverter = GetLWCConverter(NiagaraComponent);
	TArray<FVector> ConvertedData;
	ConvertedData.SetNumUninitialized(SimData.Num());
	for (int i = 0; i < SimData.Num(); i++)
	{
		ConvertedData[i] = LwcConverter.ConvertSimulationPositionToWorld(SimData[i]);
	}
	
	return ConvertedData;
}

TArray<FVector4> UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayVector4(UNiagaraComponent* NiagaraComponent, FName OverrideName)
{
	return GetNiagaraArray<FVector4, UNiagaraDataInterfaceArrayFloat4>(NiagaraComponent, OverrideName);
}

TArray<FLinearColor> UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayColor(UNiagaraComponent* NiagaraComponent, FName OverrideName)
{
	return GetNiagaraArray<FLinearColor, UNiagaraDataInterfaceArrayColor>(NiagaraComponent, OverrideName);
}

TArray<FQuat> UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayQuat(UNiagaraComponent* NiagaraComponent, FName OverrideName)
{
	return GetNiagaraArray<FQuat, UNiagaraDataInterfaceArrayQuat>(NiagaraComponent, OverrideName);
}

TArray<FMatrix> UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayMatrix(UNiagaraComponent* NiagaraComponent, FName OverrideName, bool bApplyLWCRebase)
{
	TArray<FMatrix> Results = GetNiagaraArray<FMatrix, UNiagaraDataInterfaceArrayMatrix>(NiagaraComponent, OverrideName);
	if (bApplyLWCRebase)
	{
		const FNiagaraLWCConverter LwcConverter = GetLWCConverter(NiagaraComponent);
		for (FMatrix& Result : Results)
		{
			Result = LwcConverter.ConvertSimulationToWorldMatrix(Result);
		}
	}
	return Results;
}

TArray<int32> UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayInt32(UNiagaraComponent* NiagaraComponent, FName OverrideName)
{
	return GetNiagaraArray<int32, UNiagaraDataInterfaceArrayInt32>(NiagaraComponent, OverrideName);
}

TArray<int32> UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayUInt8(UNiagaraComponent* NiagaraComponent, FName OverrideName)
{
	return GetNiagaraArray<int32, UNiagaraDataInterfaceArrayUInt8>(NiagaraComponent, OverrideName);
}

TArray<bool> UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayBool(UNiagaraComponent* NiagaraComponent, FName OverrideName)
{
	return GetNiagaraArray<bool, UNiagaraDataInterfaceArrayBool>(NiagaraComponent, OverrideName);
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayFloatValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, float Value, bool bSizeToFit)
{
	SetNiagaraArrayValue<float, UNiagaraDataInterfaceArrayFloat>(NiagaraComponent, OverrideName, Index, Value, bSizeToFit);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayVector2DValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, const FVector2D& Value, bool bSizeToFit)
{
	SetNiagaraArrayValue<FVector2D, UNiagaraDataInterfaceArrayFloat2>(NiagaraComponent, OverrideName, Index, Value, bSizeToFit);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayVectorValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, const FVector& Value, bool bSizeToFit)
{
	SetNiagaraArrayValue<FVector, UNiagaraDataInterfaceArrayFloat3>(NiagaraComponent, OverrideName, Index, Value, bSizeToFit);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayPositionValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, const FVector& Value, bool bSizeToFit)
{
	const FNiagaraLWCConverter LwcConverter = GetLWCConverter(NiagaraComponent);
	const FNiagaraPosition SimulationPosition = LwcConverter.ConvertWorldToSimulationPosition(Value);
	SetNiagaraArrayValue<FNiagaraPosition, UNiagaraDataInterfaceArrayPosition>(NiagaraComponent, OverrideName, Index, SimulationPosition, bSizeToFit);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayVector4Value(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, const FVector4& Value, bool bSizeToFit)
{
	SetNiagaraArrayValue<FVector4, UNiagaraDataInterfaceArrayFloat4>(NiagaraComponent, OverrideName, Index, Value, bSizeToFit);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayColorValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, const FLinearColor& Value, bool bSizeToFit)
{
	SetNiagaraArrayValue<FLinearColor, UNiagaraDataInterfaceArrayColor>(NiagaraComponent, OverrideName, Index, Value, bSizeToFit);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayQuatValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, const FQuat& Value, bool bSizeToFit)
{
	SetNiagaraArrayValue<FQuat, UNiagaraDataInterfaceArrayQuat>(NiagaraComponent, OverrideName, Index, Value, bSizeToFit);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayMatrixValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, const FMatrix& Value, bool bSizeToFit, bool bApplyLWCRebase)
{
	if (bApplyLWCRebase)
	{
		const FNiagaraLWCConverter LwcConverter = GetLWCConverter(NiagaraComponent);
		SetNiagaraArrayValue<FMatrix, UNiagaraDataInterfaceArrayMatrix>(NiagaraComponent, OverrideName, Index, LwcConverter.ConvertWorldToSimulationMatrix(Value), bSizeToFit);
	}
	else
	{
		SetNiagaraArrayValue<FMatrix, UNiagaraDataInterfaceArrayMatrix>(NiagaraComponent, OverrideName, Index, Value, bSizeToFit);
	}
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayInt32Value(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, int32 Value, bool bSizeToFit)
{
	SetNiagaraArrayValue<int32, UNiagaraDataInterfaceArrayInt32>(NiagaraComponent, OverrideName, Index, Value, bSizeToFit);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayUInt8Value(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, int32 Value, bool bSizeToFit)
{
	SetNiagaraArrayValue<int32, UNiagaraDataInterfaceArrayUInt8>(NiagaraComponent, OverrideName, Index, Value, bSizeToFit);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayBoolValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, const bool& Value, bool bSizeToFit)
{
	SetNiagaraArrayValue<bool, UNiagaraDataInterfaceArrayBool>(NiagaraComponent, OverrideName, Index, Value, bSizeToFit);
}

//////////////////////////////////////////////////////////////////////////

float UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayFloatValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index)
{
	return GetNiagaraArrayValue<float, UNiagaraDataInterfaceArrayFloat>(NiagaraComponent, OverrideName, Index);
}

FVector2D UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayVector2DValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index)
{
	return GetNiagaraArrayValue<FVector2D, UNiagaraDataInterfaceArrayFloat2>(NiagaraComponent, OverrideName, Index);
}

FVector UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayVectorValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index)
{
	return GetNiagaraArrayValue<FVector, UNiagaraDataInterfaceArrayFloat3>(NiagaraComponent, OverrideName, Index);
}

FVector UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayPositionValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index)
{
	FNiagaraPosition SimPosition = GetNiagaraArrayValue<FNiagaraPosition, UNiagaraDataInterfaceArrayPosition>(NiagaraComponent, OverrideName, Index);
	FNiagaraLWCConverter LwcConverter = GetLWCConverter(NiagaraComponent);
	return LwcConverter.ConvertSimulationPositionToWorld(SimPosition);
}

FVector4 UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayVector4Value(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index)
{
	return GetNiagaraArrayValue<FVector4, UNiagaraDataInterfaceArrayFloat4>(NiagaraComponent, OverrideName, Index);
}

FLinearColor UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayColorValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index)
{
	return GetNiagaraArrayValue<FLinearColor, UNiagaraDataInterfaceArrayColor>(NiagaraComponent, OverrideName, Index);
}

FQuat UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayQuatValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index)
{
	return GetNiagaraArrayValue<FQuat, UNiagaraDataInterfaceArrayQuat>(NiagaraComponent, OverrideName, Index);
}

FMatrix UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayMatrixValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index, bool bApplyLWCRebase)
{
	FMatrix Result = GetNiagaraArrayValue<FMatrix, UNiagaraDataInterfaceArrayMatrix>(NiagaraComponent, OverrideName, Index);
	if (bApplyLWCRebase)
	{
		const FNiagaraLWCConverter LwcConverter = GetLWCConverter(NiagaraComponent);
		Result = LwcConverter.ConvertSimulationToWorldMatrix(Result);
	}

	return Result;
}

int32 UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayInt32Value(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index)
{
	return GetNiagaraArrayValue<int32, UNiagaraDataInterfaceArrayInt32>(NiagaraComponent, OverrideName, Index);
}

int32 UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayUInt8Value(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index)
{
	return GetNiagaraArrayValue<int32, UNiagaraDataInterfaceArrayUInt8>(NiagaraComponent, OverrideName, Index);
}

bool UNiagaraDataInterfaceArrayFunctionLibrary::GetNiagaraArrayBoolValue(UNiagaraComponent* NiagaraComponent, FName OverrideName, int Index)
{
	return GetNiagaraArrayValue<bool, UNiagaraDataInterfaceArrayBool>(NiagaraComponent, OverrideName, Index);
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// None BP compatable set functions
void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayInt32(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<int64> ArrayData)
{
	SetNiagaraArray<int64, UNiagaraDataInterfaceArrayInt32>(NiagaraSystem, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayFloat(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<double> ArrayData)
{
	SetNiagaraArray<double, UNiagaraDataInterfaceArrayFloat>(NiagaraSystem, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayVector2D(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<FVector2f> ArrayData)
{
	SetNiagaraArray<FVector2f, UNiagaraDataInterfaceArrayFloat2>(NiagaraSystem, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayVector(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<FVector3f> ArrayData)
{
	SetNiagaraArray<FVector3f, UNiagaraDataInterfaceArrayFloat3>(NiagaraSystem, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayVector4(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<FVector4f> ArrayData)
{
	SetNiagaraArray<FVector4f, UNiagaraDataInterfaceArrayFloat4>(NiagaraSystem, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayQuat(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<FQuat4f> ArrayData)
{
	SetNiagaraArray<FQuat4f, UNiagaraDataInterfaceArrayQuat>(NiagaraSystem, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayMatrix(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<FMatrix44f> ArrayData)
{
	SetNiagaraArray<FMatrix44f, UNiagaraDataInterfaceArrayMatrix>(NiagaraSystem, OverrideName, ArrayData);
}

void UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayUInt8(UNiagaraComponent* NiagaraSystem, FName OverrideName, TConstArrayView<uint8> ArrayData)
{
	SetNiagaraArray<uint8, UNiagaraDataInterfaceArrayUInt8>(NiagaraSystem, OverrideName, ArrayData);
}
