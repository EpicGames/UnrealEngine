// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceArrayImpl.h"
#include "NiagaraGpuComputeDispatch.h"
#include "NiagaraStats.h"

const TCHAR* FNiagaraDataInterfaceArrayImplInternal::HLSLReadTemplateFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceArrayTemplate.ush");
const TCHAR* FNiagaraDataInterfaceArrayImplInternal::HLSLReadWriteTemplateFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceArrayRWTemplate.ush");

const FName FNiagaraDataInterfaceArrayImplInternal::Function_LengthName(TEXT("Length"));
const FName FNiagaraDataInterfaceArrayImplInternal::Function_IsValidIndexName(TEXT("IsValidIndex"));
const FName FNiagaraDataInterfaceArrayImplInternal::Function_LastIndexName(TEXT("LastIndex"));
const FName FNiagaraDataInterfaceArrayImplInternal::Function_GetName(TEXT("Get"));

const FName FNiagaraDataInterfaceArrayImplInternal::Function_ClearName(TEXT("Clear"));
const FName FNiagaraDataInterfaceArrayImplInternal::Function_ResizeName(TEXT("Resize"));
const FName FNiagaraDataInterfaceArrayImplInternal::Function_ResizeAndFillName(TEXT("ResizeAndFill"));
const FName FNiagaraDataInterfaceArrayImplInternal::Function_SetArrayElemName(TEXT("SetArrayElem"));
const FName FNiagaraDataInterfaceArrayImplInternal::Function_AddName(TEXT("Add"));
const FName FNiagaraDataInterfaceArrayImplInternal::Function_RemoveLastElemName(TEXT("RemoveLastElem"));

const FName FNiagaraDataInterfaceArrayImplInternal::Function_AtomicAddName("AtomicAdd");
const FName FNiagaraDataInterfaceArrayImplInternal::Function_AtomicMinName("AtomicMin");
const FName FNiagaraDataInterfaceArrayImplInternal::Function_AtomicMaxName("AtomicMax");

#if WITH_EDITORONLY_DATA
void FNiagaraDataInterfaceArrayImplInternal::GetFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, UClass* DIClass, FNiagaraTypeDefinition ValueTypeDef, bool bSupportsCPU, bool bSupportsGPU, bool bSupportsAtomicOps)
{
	OutFunctions.Reserve(OutFunctions.Num() + 9);

	// Immutable functions
	FNiagaraFunctionSignature DefaultImmutableSig;
	DefaultImmutableSig.bMemberFunction = true;
	DefaultImmutableSig.bRequiresContext = false;
	DefaultImmutableSig.bSupportsCPU = bSupportsCPU;
	DefaultImmutableSig.bSupportsGPU = bSupportsGPU;
	DefaultImmutableSig.Inputs.Emplace(FNiagaraTypeDefinition(DIClass), TEXT("Array interface"));
	DefaultImmutableSig.FunctionVersion = FNiagaraDataInterfaceArrayImplInternal::FFunctionVersion::LatestVersion;

	GetImmutableFunctions(OutFunctions, DefaultImmutableSig, ValueTypeDef);

	// Mutable functions
	FNiagaraFunctionSignature DefaultMutableSig = DefaultImmutableSig;
	DefaultMutableSig.bSupportsGPU = bSupportsGPU;
	DefaultMutableSig.bRequiresExecPin = true;

	GetMutableFunctions(OutFunctions, DefaultMutableSig, ValueTypeDef);
	if (bSupportsAtomicOps)
	{
		GetAtomicOpFunctions(OutFunctions, DefaultMutableSig, ValueTypeDef);
	}
}

void FNiagaraDataInterfaceArrayImplInternal::GetImmutableFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& DefaultImmutableSig, FNiagaraTypeDefinition ValueTypeDef)
{
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultImmutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_LengthName;
		Sig.bSupportsCPU = true;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num"));
		Sig.Description = NSLOCTEXT("Niagara", "Array_LengthDesc", "Gets the number of elements in the array.");
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultImmutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_IsValidIndexName;
		Sig.bSupportsCPU = true;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid"));
		Sig.Description = NSLOCTEXT("Niagara", "Array_IsValidIndexDesc", "Tests to see if the index is valid and exists in the array.");
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultImmutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_LastIndexName;
		Sig.bSupportsCPU = true;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Description = NSLOCTEXT("Niagara", "Array_LastIndexDesc", "Returns the last valid index in the array, will be -1 if no elements.");
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultImmutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_GetName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Outputs.Emplace(ValueTypeDef, TEXT("Value"));
		Sig.Description = NSLOCTEXT("Niagara", "Array_GetDesc", "Gets the value from the array at the given zero based index.");
	}
}

void FNiagaraDataInterfaceArrayImplInternal::GetMutableFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& DefaultMutableSig, FNiagaraTypeDefinition ValueTypeDef)
{
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultMutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_ClearName;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::System | ENiagaraScriptUsageMask::Emitter;
		Sig.Description = NSLOCTEXT("Niagara", "Array_ClearDesc", "Clears the array, removing all elements");
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultMutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_ResizeName;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::System | ENiagaraScriptUsageMask::Emitter;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num"));
		Sig.Description = NSLOCTEXT("Niagara", "Array_ResizeDesc", "Resizes the array to the specified size, initializing new elements with the default value.");
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultMutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_ResizeAndFillName;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::System | ENiagaraScriptUsageMask::Emitter;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("FillNewOnly"));
		Sig.Inputs.Emplace(ValueTypeDef, TEXT("Value"));
		Sig.Description = NSLOCTEXT("Niagara", "Array_ResizeAndFillDesc", "Resizes the array to the specified size.  Will initialize all elements or only the new elements depending on Fill New Only.");
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultMutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_SetArrayElemName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("SkipSet"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Inputs.Emplace(ValueTypeDef, TEXT("Value"));
		Sig.Description = NSLOCTEXT("Niagara", "Array_SetArrayElemDesc", "Sets the value at the given zero based index (i.e the first element is 0).");
		Sig.InputDescriptions.Add(Sig.Inputs[1], NSLOCTEXT("Niagara", "Array_SetArrayElemDesc_SkipSet", "When enabled will not set the array value."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultMutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_AddName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("SkipAdd"));
		Sig.Inputs.Emplace(ValueTypeDef, TEXT("Value"));
		Sig.Description = NSLOCTEXT("Niagara", "Array_AddDesc", "Optionally add a value onto the end of the array.");
		Sig.InputDescriptions.Add(Sig.Inputs[1], NSLOCTEXT("Niagara", "Array_AddDesc_SkipAdd", "When enabled we will not add an element to the array."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultMutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_RemoveLastElemName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("SkipRemove"));
		Sig.Outputs.Emplace(ValueTypeDef, TEXT("Value"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
		Sig.Description = NSLOCTEXT("Niagara", "Array_RemoveLastElemDesc", "Optionally remove the last element from the array.  Returns the default value if no elements are in the array or you skip the remove.");
		Sig.InputDescriptions.Add(Sig.Inputs[1], NSLOCTEXT("Niagara", "Array_RemoveLastElemDesc_SkipRemove", "When enabled will not remove a value from the array, the return value will therefore be invalid."));
		Sig.OutputDescriptions.Add(Sig.Outputs[1], NSLOCTEXT("Niagara", "Array_RemoveLastElemDesc_IsValid", "True if we removed a value from the array, False if no entries or we skipped the remove."));
	}
}

void FNiagaraDataInterfaceArrayImplInternal::GetAtomicOpFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions, const FNiagaraFunctionSignature& DefaultMutableSig, FNiagaraTypeDefinition ValueTypeDef)
{
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultMutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_AtomicAddName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("SkipAdd"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Inputs.Emplace(ValueTypeDef, TEXT("Value"));
		Sig.Outputs.Emplace(ValueTypeDef, TEXT("PreviousValue"));
		Sig.Outputs.Emplace(ValueTypeDef, TEXT("CurrentValue"));
		Sig.SetDescription(NSLOCTEXT("Niagara", "Array_AtomicAddDesc", "Optionally perform an atomic add on the array element."));
		Sig.SetInputDescription(Sig.Inputs[1], NSLOCTEXT("Niagara", "Array_AtomicAdd_SkipAdd", "When enabled will not perform the add operation, the return values will therefore be invalid."));
		Sig.SetOutputDescription(Sig.Inputs[0], NSLOCTEXT("Niagara", "Array_AtomicAdd_PrevValue", "The value before the operation was performed."));
		Sig.SetOutputDescription(Sig.Inputs[1], NSLOCTEXT("Niagara", "Array_AtomicAdd_CurrValue", "The value after the operation was performed."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultMutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_AtomicMinName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("SkipMin"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Inputs.Emplace(ValueTypeDef, TEXT("Value"));
		Sig.Outputs.Emplace(ValueTypeDef, TEXT("PreviousValue"));
		Sig.Outputs.Emplace(ValueTypeDef, TEXT("CurrentValue"));
		Sig.SetDescription(NSLOCTEXT("Niagara", "Array_AtomicMinDesc", "Optionally perform an atomic min on the array element."));
		Sig.SetInputDescription(Sig.Inputs[1], NSLOCTEXT("Niagara", "Array_AtomicMin_SkipMin", "When enabled will not perform the min operation, the return values will therefore be invalid."));
		Sig.SetOutputDescription(Sig.Inputs[0], NSLOCTEXT("Niagara", "Array_AtomicMin_PrevValue", "The value before the operation was performed."));
		Sig.SetOutputDescription(Sig.Inputs[1], NSLOCTEXT("Niagara", "Array_AtomicMin_CurrValue", "The value after the operation was performed."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultMutableSig);
		Sig.Name = FNiagaraDataInterfaceArrayImplInternal::Function_AtomicMaxName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("SkipMax"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index"));
		Sig.Inputs.Emplace(ValueTypeDef, TEXT("Value"));
		Sig.Outputs.Emplace(ValueTypeDef, TEXT("PreviousValue"));
		Sig.Outputs.Emplace(ValueTypeDef, TEXT("CurrentValue"));
		Sig.SetDescription(NSLOCTEXT("Niagara", "Array_AtomicMaxDesc", "Optionally perform an atomic max on the array element."));
		Sig.SetInputDescription(Sig.Inputs[1], NSLOCTEXT("Niagara", "Array_AtomicMax_SkipMax", "When enabled will not perform the max operation, the return values will therefore be invalid."));
		Sig.SetOutputDescription(Sig.Inputs[0], NSLOCTEXT("Niagara", "Array_AtomicMax_PrevValue", "The value before the operation was performed."));
		Sig.SetOutputDescription(Sig.Inputs[1], NSLOCTEXT("Niagara", "Array_AtomicMax_CurrValue", "The value after the operation was performed."));
	}
}

bool FNiagaraDataInterfaceArrayImplInternal::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	// Early out, nothing to do here
	if ( FunctionSignature.FunctionVersion == FFunctionVersion::LatestVersion )
	{
		return false;
	}

	if ( FunctionSignature.FunctionVersion < FFunctionVersion::AddOptionalExecuteToSet )
	{
		static const TPair<FName, FName> NodeRenames[] =
		{
			MakeTuple(FName("GetNum"),			FNiagaraDataInterfaceArrayImplInternal::Function_LengthName),
			MakeTuple(FName("GetValue"),		FNiagaraDataInterfaceArrayImplInternal::Function_GetName),
			MakeTuple(FName("Reset"),			FNiagaraDataInterfaceArrayImplInternal::Function_ClearName),
			MakeTuple(FName("SetNum"),			FNiagaraDataInterfaceArrayImplInternal::Function_ResizeName),
			MakeTuple(FName("SetValue"),		FNiagaraDataInterfaceArrayImplInternal::Function_SetArrayElemName),
			MakeTuple(FName("PushValue"),		FNiagaraDataInterfaceArrayImplInternal::Function_AddName),
			MakeTuple(FName("PopValue"),		FNiagaraDataInterfaceArrayImplInternal::Function_RemoveLastElemName),
		};

		for (const auto& Pair : NodeRenames)
		{
			if (Pair.Key == FunctionSignature.Name)
			{
				FunctionSignature.Name = Pair.Value;
				break;
			}
		}

		if (FunctionSignature.Name == FNiagaraDataInterfaceArrayImplInternal::Function_SetArrayElemName)
		{
			FunctionSignature.Inputs.EmplaceAt(1, FNiagaraTypeDefinition::GetBoolDef(), TEXT("SkipSet"));
		}
	}

	FunctionSignature.FunctionVersion = FFunctionVersion::LatestVersion;

	return true;
}
#endif

bool FNiagaraDataInterfaceArrayImplInternal::IsRWFunction(const FName FunctionName)
{
	static const TSet<FName> RWFunctions =
	{
		Function_ClearName,
		Function_ResizeName,
		Function_ResizeAndFillName,
		Function_SetArrayElemName,
		Function_AddName,
		Function_RemoveLastElemName,

		Function_AtomicAddName,
		Function_AtomicMinName,
		Function_AtomicMaxName,
	};
	return RWFunctions.Contains(FunctionName);
}

ERHIAccess FNiagaraDataInterfaceArrayImplInternal::GetCountBufferRHIAccess(const FNiagaraGpuComputeDispatchInterface& InComputeInterface)
{
	const FNiagaraGpuComputeDispatch& ComputeInterface = static_cast<const FNiagaraGpuComputeDispatch&>(InComputeInterface);
	return ComputeInterface.IsExecutingLastDispatchGroup() ? FNiagaraGPUInstanceCountManager::kCountBufferDefaultState : ERHIAccess::UAVCompute;
}

FNDIArrayInstanceData_RenderThreadBase::~FNDIArrayInstanceData_RenderThreadBase()
{
	if (CountOffset != INDEX_NONE)
	{
		ComputeInterface->GetGPUInstanceCounterManager().FreeEntry(CountOffset);
		CountOffset = INDEX_NONE;
	}

	ReleaseData();
}

void FNDIArrayInstanceData_RenderThreadBase::Initialize(FRHICommandListImmediate& RHICmdList, FNiagaraGpuComputeDispatchInterface* InComputeInterface, int32 InDefaultElements, bool bRWGpuArray)
{
	ComputeInterface	= InComputeInterface;
	DefaultElements		= 0;
	NumElements			= INDEX_NONE;
	CountOffset			= INDEX_NONE;

	if (bRWGpuArray)
	{
		DefaultElements = InDefaultElements;
		CountOffset = ComputeInterface->GetGPUInstanceCounterManager().AcquireOrAllocateEntry(RHICmdList);
	}
}

void FNDIArrayInstanceData_RenderThreadBase::UpdateDataInternal(FRHICommandList& RHICmdList, int32 ArrayNum, int32 NewNumElements, uint32 ElementSize, EPixelFormat PixelFormat)
{
	// Do we need to update the backing storage for the buffer
	if (NewNumElements != NumElements)
	{
		// Allocate new data
		NumElements = NewNumElements;
		ArrayNumBytes = (NumElements + 1) * ElementSize;	// Note +1 because we store the default value at the end of the buffer
		INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, ArrayNumBytes);

		const int32 TypeStride = GPixelFormats[PixelFormat].BlockBytes;

		// Create Buffer
		const EBufferUsageFlags BufferUsage = BUF_Static | BUF_ShaderResource | BUF_VertexBuffer | BUF_SourceCopy | (IsReadOnly() ? BUF_None : BUF_UnorderedAccess);
		const ERHIAccess DefaultAccess = IsReadOnly() ? ERHIAccess::SRVCompute : ERHIAccess::UAVCompute;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("NiagaraDataInterfaceArray"), ArrayNumBytes, TypeStride, BufferUsage)
			.SetInitialState(DefaultAccess);

		ArrayBuffer = RHICmdList.CreateBuffer(CreateDesc);

		ArraySRV = RHICmdList.CreateShaderResourceView(
			ArrayBuffer, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PixelFormat));

		if ( !IsReadOnly() )
		{
			ArrayUAV = RHICmdList.CreateUnorderedAccessView(
				ArrayBuffer, 
				FRHIViewDesc::CreateBufferUAV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PixelFormat));
		}
	}

	// Adjust counter value
	if (CountOffset != INDEX_NONE)
	{
		//-OPT: We could push this into the count manager and batch set as part of the clear process
		const FNiagaraGPUInstanceCountManager& CounterManager = ComputeInterface->GetGPUInstanceCounterManager();
		const FRWBuffer& CountBuffer = CounterManager.GetInstanceCountBuffer();

		const TPair<uint32, uint32> DataToClear(CountOffset, ArrayNum);
		RHICmdList.Transition(FRHITransitionInfo(CountBuffer.UAV, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState, ERHIAccess::UAVCompute));
		NiagaraClearCounts::ClearCountsUInt(RHICmdList, CountBuffer.UAV, MakeArrayView(&DataToClear, 1));
		RHICmdList.Transition(FRHITransitionInfo(CountBuffer.UAV, ERHIAccess::UAVCompute, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState));
	}
}

void FNDIArrayInstanceData_RenderThreadBase::ReleaseData()
{
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, ArrayNumBytes);
	ArrayBuffer.SafeRelease();
	ArrayUAV.SafeRelease();
	ArraySRV.SafeRelease();
	ArrayNumBytes = 0;
}

void FNDIArrayInstanceData_RenderThreadBase::SimCacheWriteFrame(FRHICommandListImmediate& RHICmdList, UNDIArraySimCacheData* CacheData, int32 FrameIndex, int32 ArrayTypeSize, void(*CopyGpuToCpuMemory)(void*, const void*, int32)) const
{

	if (ArrayNumBytes == 0)
	{
		return;
	}

	if (CacheData->GpuFrameData.Num() <= FrameIndex)
	{
		CacheData->GpuFrameData.AddDefaulted(FrameIndex + 1 - CacheData->GpuFrameData.Num());
	}
	FNDIArraySimCacheDataFrame& FrameData = CacheData->GpuFrameData[FrameIndex];

	TArray<FNiagaraGpuReadbackManager::FBufferRequest, TInlineAllocator<2>> BufferRequests;
	TArray<FRHITransitionInfo, TInlineAllocator<2>> TransitionsBefore;
	TArray<FRHITransitionInfo, TInlineAllocator<2>> TransitionsAfter;

	const ERHIAccess DefaultAccess = IsReadOnly() ? ERHIAccess::SRVCompute : ERHIAccess::UAVCompute;

	BufferRequests.Emplace(ArrayBuffer, 0, ArrayNumBytes);
	TransitionsBefore.Emplace(ArrayBuffer, DefaultAccess, ERHIAccess::CopySrc);
	TransitionsAfter.Emplace(ArrayBuffer, ERHIAccess::CopySrc, DefaultAccess);

	if (IsReadOnly())
	{
		FrameData.NumElements = NumElements;
	}
	else
	{
		const FNiagaraGPUInstanceCountManager& CountManager = ComputeInterface->GetGPUInstanceCounterManager();
		BufferRequests.Emplace(CountManager.GetInstanceCountBuffer().Buffer, uint32(CountOffset * sizeof(uint32)), sizeof(uint32));
		TransitionsBefore.Emplace(CountManager.GetInstanceCountBuffer().UAV, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState, ERHIAccess::CopySrc);
		TransitionsAfter.Emplace(CountManager.GetInstanceCountBuffer().UAV, ERHIAccess::CopySrc, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState);
	}

	FNiagaraGpuReadbackManager* ReadbackManager = ComputeInterface->GetGpuReadbackManager();
	RHICmdList.Transition(TransitionsBefore);
	ReadbackManager->EnqueueReadbacks(
		RHICmdList,
		BufferRequests,
		[CacheData, &FrameData, bReadOnly=IsReadOnly(), ArrayTypeSize, CopyGpuToCpuMemory](TConstArrayView<TPair<void*, uint32>> ReadbackData)
		{
			if (!bReadOnly)
			{
				FrameData.NumElements = *reinterpret_cast<const uint32*>(ReadbackData[1].Key);
			}
			if ( FrameData.NumElements > 0 )
			{
				TArray<uint8> ArrayData;
				ArrayData.AddUninitialized(FrameData.NumElements * ArrayTypeSize);
				CopyGpuToCpuMemory(ArrayData.GetData(), ReadbackData[0].Key, FrameData.NumElements);

				FrameData.DataOffset = CacheData->FindOrAddData(
					MakeArrayView(
						ArrayData.GetData(),
						ArrayData.Num()
					)
				);
			}
		}
	);
	RHICmdList.Transition(TransitionsAfter);
	ReadbackManager->WaitCompletion(RHICmdList);
}
