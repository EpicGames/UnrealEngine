// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceArrayImpl.h"

template<>
struct FNDIArrayImplHelper<float> : public FNDIArrayImplHelperBase<float>
{
	typedef float TVMArrayType;

	static constexpr TCHAR const* HLSLVariableType = TEXT("float");
	static constexpr EPixelFormat ReadPixelFormat = PF_R32_FLOAT;
	static constexpr TCHAR const* ReadHLSLBufferType = TEXT("float");
	static constexpr TCHAR const* ReadHLSLBufferRead = TEXT("Value = BUFFER_NAME[Index];");
	static constexpr EPixelFormat RWPixelFormat = PF_R32_FLOAT;
	static constexpr TCHAR const* RWHLSLBufferType = TEXT("float");
	static constexpr TCHAR const* RWHLSLBufferRead = TEXT("Value = BUFFER_NAME[Index];");
	static constexpr TCHAR const* RWHLSLBufferWrite = TEXT("BUFFER_NAME[Index] = Value;");

	static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetFloatDef(); }
	static const float GetDefaultValue() { return 0.0f; }

	static void CopyCpuToCpuMemory(float* Dest, const float* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(float));
	}

	static void CopyCpuToCpuMemory(float* Dest, const double* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = float(Src[i]);
		}
	}

	static void AppendValueToString(const float Value, FString& OutString)
	{
		OutString.Appendf(TEXT("%f"), Value);
	}
};

template<>
struct FNDIArrayImplHelper<FVector2f> : public FNDIArrayImplHelperBase<FVector2f>
{
	typedef FVector2f TVMArrayType;

	static constexpr TCHAR const* HLSLVariableType = TEXT("float2");
	static constexpr EPixelFormat ReadPixelFormat = PF_G32R32F;
	static constexpr TCHAR const* ReadHLSLBufferType = TEXT("float2");
	static constexpr TCHAR const* ReadHLSLBufferRead = TEXT("Value = float2(BUFFER_NAME[Index]);");
	static constexpr EPixelFormat RWPixelFormat = PF_R32_FLOAT;
	static constexpr TCHAR const* RWHLSLBufferType = TEXT("float");
	static constexpr TCHAR const* RWHLSLBufferRead = TEXT("Value = float2(BUFFER_NAME[Index * 2 + 0], BUFFER_NAME[Index * 2 + 1]);");
	static constexpr TCHAR const* RWHLSLBufferWrite = TEXT("BUFFER_NAME[Index * 2 + 0] = Value.x, BUFFER_NAME[Index * 2 + 1] = Value.y;");

	static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetVec2Def(); }
	static const FVector2f GetDefaultValue() { return FVector2f::ZeroVector; }

	static void CopyCpuToCpuMemory(FVector2f* Dest, const FVector2f* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(FVector2f));
	}

	static void CopyCpuToCpuMemory(FVector2f* Dest, const FVector2D* Src, int32 NumElements)
	{
		for ( int32 i=0; i < NumElements; ++i )
		{
			Dest[i] = FVector2f(Src[i]);
		}
	}

	static void CopyCpuToCpuMemory(FVector2D* Dest, const FVector2f* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = FVector2D(Src[i]);
		}
	}

	static void AppendValueToString(const FVector2f& Value, FString& OutString)
	{
		OutString.Appendf(TEXT("%f, %f"), Value.X, Value.Y);
	}

	static bool IsNearlyEqual(const FVector2f& Lhs, const FVector2f& Rhs, float Tolerance)
	{
		return
			FMath::IsNearlyEqual(Lhs.X, Rhs.X, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.Y, Rhs.Y, Tolerance);
	}
};

template<>
struct FNDIArrayImplHelper<FVector3f> : public FNDIArrayImplHelperBase<FVector3f>
{
	typedef FVector3f TVMArrayType;

	static constexpr TCHAR const* HLSLVariableType = TEXT("float3");
	static constexpr EPixelFormat ReadPixelFormat = PF_R32_FLOAT;		// Lack of float3 format
	static constexpr TCHAR const* ReadHLSLBufferType = TEXT("float");
	static constexpr TCHAR const* ReadHLSLBufferRead = TEXT("Value = float3(BUFFER_NAME[Index * 3 + 0], BUFFER_NAME[Index * 3 + 1], BUFFER_NAME[Index * 3 + 2]);");
	static constexpr EPixelFormat RWPixelFormat = PF_R32_FLOAT;
	static constexpr TCHAR const* RWHLSLBufferType = TEXT("float");
	static constexpr TCHAR const* RWHLSLBufferRead = TEXT("Value = float3(BUFFER_NAME[Index * 3 + 0], BUFFER_NAME[Index * 3 + 1], BUFFER_NAME[Index * 3 + 2]);");
	static constexpr TCHAR const* RWHLSLBufferWrite = TEXT("BUFFER_NAME[Index * 3 + 0] = Value.x, BUFFER_NAME[Index * 3 + 1] = Value.y, BUFFER_NAME[Index * 3 + 2] = Value.z;");

	static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetVec3Def(); }
	static const FVector3f GetDefaultValue() { return FVector3f::ZeroVector; }

	static void CopyCpuToCpuMemory(FVector3f* Dest, const FVector3f* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(FVector3f));
	}

	static void CopyCpuToCpuMemory(FVector3f* Dest, const FVector* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = FVector3f(Src[i]);
		}
	}

	static void CopyCpuToCpuMemory(FVector* Dest, const FVector3f* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = FVector(Src[i]);
		}
	}

	static void AppendValueToString(const FVector3f& Value, FString& OutString)
	{
		OutString.Appendf(TEXT("%f, %f, %f"), Value.X, Value.Y, Value.Z);
	}

	static bool IsNearlyEqual(const FVector3f& Lhs, const FVector3f& Rhs, float Tolerance)
	{
		return
			FMath::IsNearlyEqual(Lhs.X, Rhs.X, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.Y, Rhs.Y, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.Z, Rhs.Z, Tolerance);
	}
};

template<>
struct FNDIArrayImplHelper<FNiagaraPosition> : public FNDIArrayImplHelper<FVector3f>
{
	typedef FNiagaraPosition TVMArrayType;

	static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetPositionDef(); }
	static const FNiagaraPosition GetDefaultValue() { return FVector3f::ZeroVector; }

	static void CopyCpuToCpuMemory(FVector3f* Dest, const FNiagaraPosition* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = Src[i];
		}
	}

	static void AppendValueToString(const FVector3f& Value, FString& OutString)
	{
		OutString.Appendf(TEXT("%f, %f, %f"), Value.X, Value.Y, Value.Z);
	}

	static bool IsNearlyEqual(const FNiagaraPosition& Lhs, const FNiagaraPosition& Rhs, float Tolerance)
	{
		return
			FMath::IsNearlyEqual(Lhs.X, Rhs.X, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.Y, Rhs.Y, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.Z, Rhs.Z, Tolerance);
	}
};

template<>
struct FNDIArrayImplHelper<FVector4f> : public FNDIArrayImplHelperBase<FVector4f>
{
	typedef FVector4f TVMArrayType;

	static constexpr TCHAR const* HLSLVariableType = TEXT("float4");
	static constexpr EPixelFormat ReadPixelFormat = PF_A32B32G32R32F;
	static constexpr TCHAR const* ReadHLSLBufferType = TEXT("float4");
	static constexpr TCHAR const* ReadHLSLBufferRead = TEXT("Value = BUFFER_NAME[Index];");
	static constexpr EPixelFormat RWPixelFormat = PF_R32_FLOAT;
	static constexpr TCHAR const* RWHLSLBufferType = TEXT("float");
	static constexpr TCHAR const* RWHLSLBufferRead = TEXT("Value = float4(BUFFER_NAME[Index * 4 + 0], BUFFER_NAME[Index * 4 + 1], BUFFER_NAME[Index * 4 + 2], BUFFER_NAME[Index * 4 + 3]);");
	static constexpr TCHAR const* RWHLSLBufferWrite = TEXT("BUFFER_NAME[Index * 4 + 0] = Value.x, BUFFER_NAME[Index * 4 + 1] = Value.y, BUFFER_NAME[Index * 4 + 2] = Value.z, BUFFER_NAME[Index * 4 + 3] = Value.w;");

	static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetVec4Def(); }
	static const FVector4f GetDefaultValue() { return FVector4f(ForceInitToZero); }

	static void CopyCpuToCpuMemory(FVector4f* Dest, const FVector4f* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(FVector4f));
	}

	static void CopyCpuToCpuMemory(FVector4f* Dest, const FVector4* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = FVector4f(Src[i]);
		}
	}

	static void CopyCpuToCpuMemory(FVector4* Dest, const FVector4f* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = FVector4(Src[i]);
		}
	}

	static void AppendValueToString(const FVector4f& Value, FString& OutString)
	{
		OutString.Appendf(TEXT("%f, %f, %f, %f"), Value.X, Value.Y, Value.Z, Value.W);
	}

	static bool IsNearlyEqual(const FVector4f& Lhs, const FVector4f& Rhs, float Tolerance)
	{
		return
			FMath::IsNearlyEqual(Lhs.X, Rhs.X, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.Y, Rhs.Y, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.Z, Rhs.Z, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.W, Rhs.W, Tolerance);
	}
};

template<>
struct FNDIArrayImplHelper<FLinearColor> : public FNDIArrayImplHelperBase<FLinearColor>
{
	typedef FLinearColor TVMArrayType;

	static constexpr TCHAR const* HLSLVariableType = TEXT("float4");
	static constexpr EPixelFormat ReadPixelFormat = PF_A32B32G32R32F;
	static constexpr TCHAR const* ReadHLSLBufferType = TEXT("float4");
	static constexpr TCHAR const* ReadHLSLBufferRead = TEXT("Value = BUFFER_NAME[Index];");
	static constexpr EPixelFormat RWPixelFormat = PF_R32_FLOAT;
	static constexpr TCHAR const* RWHLSLBufferType = TEXT("float");
	static constexpr TCHAR const* RWHLSLBufferRead = TEXT("Value = float4(BUFFER_NAME[Index * 4 + 0], BUFFER_NAME[Index * 4 + 1], BUFFER_NAME[Index * 4 + 2], BUFFER_NAME[Index * 4 + 3]);");
	static constexpr TCHAR const* RWHLSLBufferWrite = TEXT("BUFFER_NAME[Index * 4 + 0] = Value.x, BUFFER_NAME[Index * 4 + 1] = Value.y, BUFFER_NAME[Index * 4 + 2] = Value.z, BUFFER_NAME[Index * 4 + 3] = Value.w;");

	static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetColorDef(); }
	static const FLinearColor GetDefaultValue() { return FLinearColor::White; }

	static void AppendValueToString(const FLinearColor& Value, FString& OutString)
	{
		OutString.Appendf(TEXT("%f, %f, %f, %f"), Value.R, Value.G, Value.B, Value.A);
	}

	static bool IsNearlyEqual(const FLinearColor& Lhs, const FLinearColor& Rhs, float Tolerance)
	{
		return
			FMath::IsNearlyEqual(Lhs.R, Rhs.R, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.G, Rhs.G, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.B, Rhs.B, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.A, Rhs.A, Tolerance);
	}
};

template<>
struct FNDIArrayImplHelper<FQuat4f> : public FNDIArrayImplHelperBase<FQuat4f>
{
	typedef FQuat4f TVMArrayType;

	static constexpr TCHAR const* HLSLVariableType = TEXT("float4");
	static constexpr EPixelFormat ReadPixelFormat = PF_A32B32G32R32F;
	static constexpr TCHAR const* ReadHLSLBufferType = TEXT("float4");
	static constexpr TCHAR const* ReadHLSLBufferRead = TEXT("Value = BUFFER_NAME[Index];");
	static constexpr EPixelFormat RWPixelFormat = PF_R32_FLOAT;
	static constexpr TCHAR const* RWHLSLBufferType = TEXT("float");
	static constexpr TCHAR const* RWHLSLBufferRead = TEXT("Value = float4(BUFFER_NAME[Index * 4 + 0], BUFFER_NAME[Index * 4 + 1], BUFFER_NAME[Index * 4 + 2], BUFFER_NAME[Index * 4 + 3]);");
	static constexpr TCHAR const* RWHLSLBufferWrite = TEXT("BUFFER_NAME[Index * 4 + 0] = Value.x, BUFFER_NAME[Index * 4 + 1] = Value.y, BUFFER_NAME[Index * 4 + 2] = Value.z, BUFFER_NAME[Index * 4 + 3] = Value.w;");

	static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetQuatDef(); }
	static const FQuat4f GetDefaultValue() { return FQuat4f::Identity; }

	static void CopyCpuToCpuMemory(FQuat4f* Dest, const FQuat4f* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(FQuat4f));
	}

	static void CopyCpuToCpuMemory(FQuat4f* Dest, const FQuat* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = FQuat4f(Src[i]);
		}
	}

	static void CopyCpuToCpuMemory(FQuat* Dest, const FQuat4f* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = FQuat(Src[i]);
		}
	}

	static void AppendValueToString(const FQuat4f& Value, FString& OutString)
	{
		OutString.Appendf(TEXT("%f, %f, %f, %f"), Value.X, Value.Y, Value.Z, Value.W);
	}

	static bool IsNearlyEqual(const FQuat4f& Lhs, const FQuat4f& Rhs, float Tolerance)
	{
		return
			FMath::IsNearlyEqual(Lhs.X, Rhs.X, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.Y, Rhs.Y, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.Z, Rhs.Z, Tolerance) &&
			FMath::IsNearlyEqual(Lhs.W, Rhs.W, Tolerance);
	}
};

template<>
struct FNDIArrayImplHelper<FMatrix44f> : public FNDIArrayImplHelperBase<FMatrix44f>
{
	typedef FMatrix44f TVMArrayType;

	static constexpr TCHAR const* HLSLVariableType = TEXT("float4x4");
	static constexpr EPixelFormat ReadPixelFormat = PF_A32B32G32R32F;
	static constexpr TCHAR const* ReadHLSLBufferType = TEXT("float4");
	static constexpr TCHAR const* ReadHLSLBufferRead = TEXT("Value = float4x4(BUFFER_NAME[Index * 4 + 0], BUFFER_NAME[Index * 4 + 1], BUFFER_NAME[Index * 4 + 2], BUFFER_NAME[Index * 4 + 3]);");
	static constexpr EPixelFormat RWPixelFormat = PF_R32_FLOAT;
	static constexpr TCHAR const* RWHLSLBufferType = TEXT("float");
	static constexpr TCHAR const* RWHLSLBufferRead = TEXT(R"(Value = float4x4(
		BUFFER_NAME[Index * 16 +  0], BUFFER_NAME[Index * 16 +  1], BUFFER_NAME[Index * 16 +  2], BUFFER_NAME[Index * 16 +  3],
		BUFFER_NAME[Index * 16 +  4], BUFFER_NAME[Index * 16 +  5], BUFFER_NAME[Index * 16 +  6], BUFFER_NAME[Index * 16 +  7],
		BUFFER_NAME[Index * 16 +  8], BUFFER_NAME[Index * 16 +  9], BUFFER_NAME[Index * 16 + 10], BUFFER_NAME[Index * 16 + 11],
		BUFFER_NAME[Index * 16 + 12], BUFFER_NAME[Index * 16 + 13], BUFFER_NAME[Index * 16 + 14], BUFFER_NAME[Index * 16 + 15]);
	)");
	static constexpr TCHAR const* RWHLSLBufferWrite =
		TEXT("BUFFER_NAME[Index * 16 +  0] = Value[0].x, BUFFER_NAME[Index * 16 +  1] = Value[0].y, BUFFER_NAME[Index * 16 +  2] = Value[0].z, BUFFER_NAME[Index * 16 +  3] = Value[0].w,")
		TEXT("BUFFER_NAME[Index * 16 +  4] = Value[1].x, BUFFER_NAME[Index * 16 +  5] = Value[1].y, BUFFER_NAME[Index * 16 +  6] = Value[1].z, BUFFER_NAME[Index * 16 +  7] = Value[1].w,")
		TEXT("BUFFER_NAME[Index * 16 +  8] = Value[2].x, BUFFER_NAME[Index * 16 +  9] = Value[2].y, BUFFER_NAME[Index * 16 + 10] = Value[2].z, BUFFER_NAME[Index * 16 + 11] = Value[2].w,")
		TEXT("BUFFER_NAME[Index * 16 + 12] = Value[3].x, BUFFER_NAME[Index * 16 + 13] = Value[3].y, BUFFER_NAME[Index * 16 + 14] = Value[3].z, BUFFER_NAME[Index * 16 + 15] = Value[3].w;");

	static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetMatrix4Def(); }
	static const FMatrix44f GetDefaultValue() { return FMatrix44f::Identity; }

	static void CopyCpuToCpuMemory(FMatrix44f* Dest, const FMatrix44f* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(FMatrix44f));
	}

	static void CopyCpuToCpuMemory(FMatrix44f* Dest, const FMatrix* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = FMatrix44f(Src[i]);
		}
	}

	static void CopyCpuToCpuMemory(FMatrix* Dest, const FMatrix44f* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = FMatrix(Src[i]);
		}
	}

	static void AppendValueToString(const FMatrix44f& Value, FString& OutString)
	{
		for (int32 r=0; r < 4; ++r)
		{
			for (int32 c=0; c < 4; ++c)
			{
				if (r + c == 0)
				{
					OutString.Appendf(TEXT("%f"), Value.M[r][c]);
				}
				else
				{
					OutString.Appendf(TEXT(", %f"), Value.M[r][c]);
				}
			}
		}
	}

	static bool IsNearlyEqual(const FMatrix44f& Lhs, const FMatrix44f& Rhs, float Tolerance)
	{
		for (int32 r = 0; r < 4; ++r)
		{
			for (int32 c = 0; c < 4; ++c)
			{
				if (!FMath::IsNearlyEqual(Lhs.M[r][c], Rhs.M[r][c], Tolerance))
				{
					return false;
				}
			}
		}

		return true;
	}
};

template<>
struct FNDIArrayImplHelper<int32> : public FNDIArrayImplHelperBase<int32>
{
	typedef int32 TVMArrayType;

	static constexpr bool bSupportsAtomicOps = true;

	static constexpr TCHAR const* HLSLVariableType = TEXT("int");
	static constexpr EPixelFormat ReadPixelFormat = PF_R32_SINT;
	static constexpr TCHAR const* ReadHLSLBufferType = TEXT("int");
	static constexpr TCHAR const* ReadHLSLBufferRead = TEXT("Value = BUFFER_NAME[Index];");
	static constexpr EPixelFormat RWPixelFormat = PF_R32_SINT;
	static constexpr TCHAR const* RWHLSLBufferType = TEXT("int");
	static constexpr TCHAR const* RWHLSLBufferRead = TEXT("Value = BUFFER_NAME[Index];");
	static constexpr TCHAR const* RWHLSLBufferWrite = TEXT("BUFFER_NAME[Index] = Value;");

	static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetIntDef(); }
	static const int32 GetDefaultValue() { return 0; }

	static void CopyCpuToCpuMemory(int32* Dest, const int32* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(int32));
	}

	static void CopyCpuToCpuMemory(int32* Dest, const int64* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = int32(Src[i]);
		}
	}

	static int32 AtomicAdd(int32* Dest, int32 Value)
	{
		return FPlatformAtomics::InterlockedAdd(Dest, Value);
	}
	static int32 AtomicMin(int32* Dest, int32 Value)
	{
		int32 PrevValue = *Dest;
		while (PrevValue > Value)
		{
			PrevValue = FPlatformAtomics::InterlockedCompareExchange(Dest, Value, PrevValue);
		}
		return PrevValue;
	}
	static int32 AtomicMax(int32* Dest, int32 Value)
	{
		int32 PrevValue = *Dest;
		while (PrevValue < Value)
		{
			PrevValue = FPlatformAtomics::InterlockedCompareExchange(Dest, Value, PrevValue);
		}
		return PrevValue;
	}

	static void AppendValueToString(const int32 Value, FString& OutString)
	{
		OutString.AppendInt(Value);
	}
};

template<>
struct FNDIArrayImplHelper<uint8> : public FNDIArrayImplHelperBase<uint8>
{
	typedef int32 TVMArrayType;

	static constexpr TCHAR const* HLSLVariableType = TEXT("int");
	static constexpr EPixelFormat ReadPixelFormat = PF_R8_UINT;
	static constexpr TCHAR const* ReadHLSLBufferType = TEXT("int");
	static constexpr TCHAR const* ReadHLSLBufferRead = TEXT("Value = BUFFER_NAME[Index];");
	static constexpr EPixelFormat RWPixelFormat = PF_R8_UINT;
	static constexpr TCHAR const* RWHLSLBufferType = TEXT("int");
	static constexpr TCHAR const* RWHLSLBufferRead = TEXT("Value = BUFFER_NAME[Index];");
	static constexpr TCHAR const* RWHLSLBufferWrite = TEXT("BUFFER_NAME[Index] = Value;");

	static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetIntDef(); }
	static const int32 GetDefaultValue() { return 0; }

	static void CopyCpuToCpuMemory(uint8* Dest, const uint8* Src, int32 NumElements)
	{
		FMemory::Memcpy(Dest, Src, NumElements * sizeof(uint8));
	}

	static void CopyCpuToCpuMemory(uint8* Dest, const int32* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = uint8(Src[i]);
		}
	}

	static void CopyCpuToCpuMemory(int32* Dest, const uint8* Src, int32 NumElements)
	{
		for (int32 i = 0; i < NumElements; ++i)
		{
			Dest[i] = int32(Src[i]);
		}
	}

	static void AppendValueToString(const uint8 Value, FString& OutString)
	{
		OutString.AppendInt(Value);
	}
};

template<>
struct FNDIArrayImplHelper<bool> : public FNDIArrayImplHelperBase<bool>
{
	typedef FNiagaraBool TVMArrayType;

	static constexpr TCHAR const* HLSLVariableType = TEXT("bool");
	static constexpr EPixelFormat ReadPixelFormat = PF_R8_UINT;
	static constexpr TCHAR const* ReadHLSLBufferType = TEXT("uint");
	static constexpr TCHAR const* ReadHLSLBufferRead = TEXT("Value = BUFFER_NAME[Index];");
	static constexpr EPixelFormat RWPixelFormat = PF_R8_UINT;
	static constexpr TCHAR const* RWHLSLBufferType = TEXT("uint");
	static constexpr TCHAR const* RWHLSLBufferRead = TEXT("Value = BUFFER_NAME[Index];");
	static constexpr TCHAR const* RWHLSLBufferWrite = TEXT("BUFFER_NAME[Index] = Value;");

	static const FNiagaraTypeDefinition& GetTypeDefinition() { return FNiagaraTypeDefinition::GetBoolDef(); }
	static const FNiagaraBool GetDefaultValue() { return FNiagaraBool::False; }

	static void CopyCpuToGpuMemory(void* Dest, const bool* Src, int32 NumElements)
	{
		uint8* TypedDest = reinterpret_cast<uint8*>(Dest);
		while (NumElements--)
		{
			*TypedDest++ = *Src++ == false ? 0 : 0xff;
		}
	}

	static void CopyGpuToCpuMemory(void* Dest, const void* Src, int32 NumElements)
	{
		FNiagaraBool* TypedDest = reinterpret_cast<FNiagaraBool*>(Dest);
		const uint8* TypedSrc = reinterpret_cast<const uint8*>(Src);
		while (NumElements--)
		{
			*TypedDest++ = *TypedSrc++ == 0 ? false : true;
		}
	}

	static void AppendValueToString(const bool Value, FString& OutString)
	{
		OutString.Append(Value ? TEXT("True") : TEXT("False"));
	}

	static bool IsNearlyEqual(bool Lhs, bool Rhs, float Tolerance)
	{
		return !!Lhs == !!Rhs;
	}
};

template<>
struct FNDIArrayImplHelper<FNiagaraID> : public FNDIArrayImplHelperBase<FNiagaraID>
{
	typedef FNiagaraID TVMArrayType;

	static constexpr TCHAR const* HLSLVariableType = TEXT("NiagaraID");
	static constexpr EPixelFormat ReadPixelFormat = PF_R32_SINT;		// Should be int2 but we don't have it
	static constexpr TCHAR const* ReadHLSLBufferType = TEXT("int");
	static constexpr TCHAR const* ReadHLSLBufferRead = TEXT("Value.Index = BUFFER_NAME[Index * 2 + 0]; Value.AcquireTag = BUFFER_NAME[Index * 2 + 1];");
	static constexpr EPixelFormat RWPixelFormat = PF_R32_SINT;
	static constexpr TCHAR const* RWHLSLBufferType = TEXT("int");
	static constexpr TCHAR const* RWHLSLBufferRead = TEXT("Value.Index = BUFFER_NAME[Index * 2 + 0]; Value.AcquireTag = BUFFER_NAME[Index * 2 + 1];");
	static constexpr TCHAR const* RWHLSLBufferWrite = TEXT("BUFFER_NAME[Index * 2 + 0] = Value.Index; BUFFER_NAME[Index * 2 + 1] = Value.AcquireTag;");

	static const FNiagaraTypeDefinition GetTypeDefinition() { return FNiagaraTypeDefinition(FNiagaraID::StaticStruct()); }
	static const FNiagaraID GetDefaultValue() { return FNiagaraID(); }

	static void AppendValueToString(const FNiagaraID Value, FString& OutString)
	{
		OutString.Appendf(TEXT("%d, %d"), Value.Index, Value.AcquireTag);
	}

	static bool IsNearlyEqual(const FNiagaraID& Lhs, const FNiagaraID& Rhs, float Tolerance)
	{
		return Lhs.AcquireTag == Rhs.AcquireTag && Lhs.Index == Rhs.Index;
	}
};
