// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceArrayInt.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceArrayInt)

static_assert(sizeof(bool) == sizeof(uint8), "Bool != 1 byte this will mean the GPU array does not match in size");
