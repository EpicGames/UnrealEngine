// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceAudioOscilloscope.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraGeneratedDataAudioSampling.h"
#include "NiagaraRenderer.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraTypes.h"
#include "NiagaraWorldManager.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SystemTextures.h"
#include "UnifiedBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceAudioOscilloscope)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceGridAudioOscilloscope"

namespace UE::Niagara::DataInterfaceAudioOscilloscope::Private
{
	// Global VM function names, also used by the shaders code generation methods.
	static const FName SampleAudioBufferFunctionName("SampleAudioBuffer");
	static const FName GetAudioBufferNumChannelsFunctionName("GetAudioBufferNumChannels");

	const TCHAR* CommonShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceAudioCommon.ush");
	const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceAudioOscilloscopeTemplate.ush");

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(int, ChannelCount)
		SHADER_PARAMETER(int, Resolution)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, OscilloscopeBuffer)
	END_SHADER_PARAMETER_STRUCT()

	struct FInstanceData_GameThread
	{
		FNDIAudio_SharedResourceHandle SharedResource;

		TWeakObjectPtr<USoundSubmix> SoundSubmix;
		TArray<float> DownsampledBuffer;
		int32 DownsampledResolution = 0;
		float ScopeInMilliseconds = 0.0f;
		bool bSoundSubmixAssigned = false;
	};

	struct FGameToRenderInstanceData
	{
		TArray<float> DataToUpload;

		int32 ChannelCount = 0;
		int32 ChannelResolution = 0;
	};

	struct FInstanceData_RenderThread
	{
		TArray<float> DataToUpload;
		TRefCountPtr<FRDGPooledBuffer> PooledBuffer;

		int32 ChannelCount = 0;
		int32 ChannelResolution = 0;
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FGameToRenderInstanceData); }

		static void ProvidePerInstanceDataForRenderThread(void* InDataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
		{
			FInstanceData_GameThread* InstanceData = static_cast<FInstanceData_GameThread*>(PerInstanceData);
			FGameToRenderInstanceData* DataForRenderThread = new(InDataForRenderThread) FGameToRenderInstanceData();
			const FNDIAudio_SharedResource& AudioResource = InstanceData->SharedResource.ReadResource();

			DataForRenderThread->DataToUpload = AudioResource.ReadAudioBuffer(
				DataForRenderThread->ChannelCount, DataForRenderThread->ChannelResolution);
		}

		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override
		{
			FGameToRenderInstanceData* InstanceDataFromGT = static_cast<FGameToRenderInstanceData*>(PerInstanceData);
			FInstanceData_RenderThread& InstanceData = SystemInstancesToInstanceData_RT.FindOrAdd(InstanceID);

			InstanceData.DataToUpload = MoveTemp(InstanceDataFromGT->DataToUpload);
			InstanceData.ChannelCount = InstanceDataFromGT->ChannelCount;
			InstanceData.ChannelResolution = InstanceDataFromGT->ChannelResolution;

			InstanceDataFromGT->~FGameToRenderInstanceData();
		}

		virtual void PreStage(const FNDIGpuComputePreStageContext& Context)
		{
			const FNiagaraSystemInstanceID SystemInstanceID = Context.GetSystemInstanceID();
			FInstanceData_RenderThread& InstanceData = SystemInstancesToInstanceData_RT.FindChecked(SystemInstanceID);
			if (InstanceData.DataToUpload.Num() > 0)
			{
				FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

				const void* InitialData = InstanceData.DataToUpload.GetData();
				const uint64 InitialDataSize = InstanceData.DataToUpload.Num() * InstanceData.DataToUpload.GetTypeSize();
				const uint64 BufferSize = Align(InitialDataSize, 16);

				FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateByteAddressDesc(IntCastChecked<uint32>(BufferSize));
				ResizeBufferIfNeeded(GraphBuilder, InstanceData.PooledBuffer, BufferDesc, TEXT("NiagaraAudioOscilloscopeBuffer"));

				GraphBuilder.QueueBufferUpload(
					GraphBuilder.RegisterExternalBuffer(InstanceData.PooledBuffer),
					InitialData,
					InitialDataSize,
					[Data = MoveTemp(InstanceData.DataToUpload)](const void*) {}
				);

				InstanceData.DataToUpload.Empty();
			}
		}

		TMap<FNiagaraSystemInstanceID, FInstanceData_RenderThread> SystemInstancesToInstanceData_RT;
	};

	static float SampleInterleavedAudioBuffer(TConstArrayView<float> Samples, int32 ChannelCount, int32 FrameCount, int32 ChannelIndex, float Position)
	{
		float FrameIndex = FMath::Clamp(Position, 0.0f, 1.0f) * (FrameCount - 1);
		int32 LowerIndex = FMath::Max(0, FMath::FloorToInt(FrameIndex));
		int32 UpperIndex = FMath::Min(LowerIndex + 1, FrameCount - 1);

		float Fraction = FrameIndex - float(LowerIndex);

		float LowerValue = Samples[LowerIndex * ChannelCount + ChannelIndex];
		float UpperValue = Samples[UpperIndex * ChannelCount + ChannelIndex];

		return FMath::Lerp(LowerValue, UpperValue, Fraction);
	}
}

UNiagaraDataInterfaceAudioOscilloscope::UNiagaraDataInterfaceAudioOscilloscope(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new UE::Niagara::DataInterfaceAudioOscilloscope::Private::FNDIProxy());
}

void UNiagaraDataInterfaceAudioOscilloscope::SampleAudio(FVectorVMExternalFunctionContext& Context)
{
	using namespace UE::Niagara::DataInterfaceAudioOscilloscope::Private;

	const int32 InstanceCount = Context.GetNumInstances();

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncInputHandler<float> InNormalizedPos(Context);
	VectorVM::FExternalFuncInputHandler<int32> InChannel(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutAmplitude(Context);

	const FNDIAudio_SharedResource& SharedAudio = InstData->SharedResource.ReadResource();
	{
		SharedAudio.WaitForAudio();
	}
	int32 ChannelCount = 0;
	int32 ChannelResolution = 0;
	TConstArrayView<float> AudioBuffer = SharedAudio.ReadAudioBuffer(ChannelCount, ChannelResolution);

	if (ChannelCount && ChannelResolution)
	{
		for (int32 InstanceIdx = 0; InstanceIdx < InstanceCount; ++InstanceIdx)
		{
			const float Position = InNormalizedPos.GetAndAdvance();
			const int32 ChannelIndex = InChannel.GetAndAdvance();
			*OutAmplitude.GetDestAndAdvance() = SampleInterleavedAudioBuffer(
				AudioBuffer,
				ChannelCount,
				ChannelResolution,
				FMath::Clamp(ChannelIndex, 0, ChannelCount - 1),
				Position);
		}
	}
	else
	{
		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			InNormalizedPos.GetAndAdvance();
			InChannel.GetAndAdvance();
			*OutAmplitude.GetDestAndAdvance() = 0.0f;
		}
	}
}

void UNiagaraDataInterfaceAudioOscilloscope::GetNumChannels(FVectorVMExternalFunctionContext& Context)
{
	const int32 InstanceCount = Context.GetNumInstances();

	VectorVM::FUserPtrHandler<UE::Niagara::DataInterfaceAudioOscilloscope::Private::FInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutChannel(Context);

	const FNDIAudio_SharedResource& SharedAudio = InstData->SharedResource.ReadResource();
	const int32 ChannelCount = SharedAudio.GetChannelCount();

	for (int32 InstanceIdx = 0; InstanceIdx < InstanceCount; ++InstanceIdx)
	{
		*OutChannel.GetDestAndAdvance() = ChannelCount;
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceAudioOscilloscope::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace UE::Niagara::DataInterfaceAudioOscilloscope::Private;
	Super::GetFunctionsInternal(OutFunctions);

	{
		FNiagaraFunctionSignature SampleAudioBufferSignature;
		SampleAudioBufferSignature.Name = SampleAudioBufferFunctionName;
		SampleAudioBufferSignature.Inputs.Add(FNiagaraVariable(GetClass(), TEXT("Oscilloscope")));
		SampleAudioBufferSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("NormalizedPositionInBuffer")));
		SampleAudioBufferSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ChannelIndex")));
		SampleAudioBufferSignature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Amplitude")));

		SampleAudioBufferSignature.bMemberFunction = true;
		SampleAudioBufferSignature.bRequiresContext = false;
		OutFunctions.Add(SampleAudioBufferSignature);
	}

	{
		FNiagaraFunctionSignature NumChannelsSignature;
		NumChannelsSignature.Name = GetAudioBufferNumChannelsFunctionName;
		NumChannelsSignature.Inputs.Add(FNiagaraVariable(GetClass(), TEXT("Oscilloscope")));
		NumChannelsSignature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumChannels")));

		NumChannelsSignature.bMemberFunction = true;
		NumChannelsSignature.bRequiresContext = false;
		OutFunctions.Add(NumChannelsSignature);
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioOscilloscope, SampleAudio);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioOscilloscope, GetNumChannels);

void UNiagaraDataInterfaceAudioOscilloscope::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	using namespace UE::Niagara::DataInterfaceAudioOscilloscope::Private;

	if (BindingInfo.Name == SampleAudioBufferFunctionName)
	{
		//TNDIParamBinder<0, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioOscilloscope, SampleAudio)>::Bind(this, BindingInfo, InstanceData, OutFunc);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioOscilloscope, SampleAudio)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetAudioBufferNumChannelsFunctionName)
	{
		//TNDIParamBinder<0, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioOscilloscope, GetNumChannels)>::Bind(this, BindingInfo, InstanceData, OutFunc);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioOscilloscope, GetNumChannels)::Bind(this, OutFunc);
	}
	else
	{
		ensureMsgf(false, TEXT("Error! Function defined for this class but not bound."));
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceAudioOscilloscope::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace UE::Niagara::DataInterfaceAudioOscilloscope::Private;

	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}
	InVisitor->UpdateShaderFile(CommonShaderFile);
	InVisitor->UpdateShaderFile(TemplateShaderFile);
	InVisitor->UpdateShaderParameters<FShaderParameters>();
	return true;
}

bool UNiagaraDataInterfaceAudioOscilloscope::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace UE::Niagara::DataInterfaceAudioOscilloscope::Private;

	if (Super::GetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, OutHLSL))
	{
		return true;
	}

	if ((FunctionInfo.DefinitionName == SampleAudioBufferFunctionName) ||
		(FunctionInfo.DefinitionName == GetAudioBufferNumChannelsFunctionName))
	{
		return true;
	}

	return false;
}

void UNiagaraDataInterfaceAudioOscilloscope::GetCommonHLSL(FString& OutHLSL)
{
	Super::GetCommonHLSL(OutHLSL);
	OutHLSL.Appendf(TEXT("#include \"%s\"\n"), UE::Niagara::DataInterfaceAudioOscilloscope::Private::CommonShaderFile);
}

void UNiagaraDataInterfaceAudioOscilloscope::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, UE::Niagara::DataInterfaceAudioOscilloscope::Private::TemplateShaderFile, TemplateArgs);
}

#endif

void UNiagaraDataInterfaceAudioOscilloscope::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<UE::Niagara::DataInterfaceAudioOscilloscope::Private::FShaderParameters>();
}

void UNiagaraDataInterfaceAudioOscilloscope::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace UE::Niagara::DataInterfaceAudioOscilloscope::Private;
	
	FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	FInstanceData_RenderThread& InstanceData = DIProxy.SystemInstancesToInstanceData_RT.FindChecked(Context.GetSystemInstanceID());

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->ChannelCount = InstanceData.ChannelCount;
	ShaderParameters->Resolution = InstanceData.ChannelResolution;

	if (InstanceData.PooledBuffer)
	{
		FRDGBufferRef RDGBuffer = GraphBuilder.RegisterExternalBuffer(InstanceData.PooledBuffer);
		ShaderParameters->OscilloscopeBuffer = GraphBuilder.CreateSRV(RDGBuffer, PF_R32_FLOAT);
	}
	else
	{
		FRDGBufferRef RDGBuffer = GSystemTextures.GetDefaultBuffer(GraphBuilder, sizeof(float), 0.0f);
		ShaderParameters->OscilloscopeBuffer = GraphBuilder.CreateSRV(RDGBuffer, PF_R32_FLOAT);
	}
}

bool UNiagaraDataInterfaceAudioOscilloscope::Equals(const UNiagaraDataInterface* Other) const
{
	bool bIsEqual = Super::Equals(Other);

	const UNiagaraDataInterfaceAudioOscilloscope* OtherOscilloscope = CastChecked<const UNiagaraDataInterfaceAudioOscilloscope>(Other);

	bIsEqual &= OtherOscilloscope->Submix == Submix;
	bIsEqual &= OtherOscilloscope->Resolution == Resolution;
	bIsEqual &= OtherOscilloscope->ScopeInMilliseconds == ScopeInMilliseconds;

	return bIsEqual;
}

void UNiagaraDataInterfaceAudioOscilloscope::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceAudioOscilloscope::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace UE::Niagara::DataInterfaceAudioOscilloscope::Private;

	FInstanceData_GameThread* Inst = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);

	// reset our per instance data if the submix has changed/invalidated
	if (Inst->bSoundSubmixAssigned && !Inst->SoundSubmix.IsValid())
	{
		return true;
	}

	if (Inst->SoundSubmix != Submix)
	{
		return true;
	}

	return false;
}

bool UNiagaraDataInterfaceAudioOscilloscope::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace UE::Niagara::DataInterfaceAudioOscilloscope::Private;
	
	FInstanceData_GameThread* Inst = new (PerInstanceData) FInstanceData_GameThread();

	Inst->SoundSubmix = MakeWeakObjectPtr(Submix);
	Inst->ScopeInMilliseconds = ScopeInMilliseconds;
	Inst->DownsampledResolution = Resolution;
	Inst->bSoundSubmixAssigned = Submix != nullptr;

	if (FNiagaraWorldManager* WorldManager = SystemInstance->GetWorldManager())
	{
		FNDIAudio_GeneratedData::FResourceDesc AudioDesc;
		AudioDesc.DeviceId = INDEX_NONE;
		AudioDesc.Submix = Submix;
		AudioDesc.bUseLatestAudio = true;
		AudioDesc.bResampleAudio = true;
		AudioDesc.AudioResamplingResolution = Resolution;
		AudioDesc.SamplingMethod = FNDIAudio_SharedResource::ESamplingWindowMethod::ByTime;
		AudioDesc.SamplingWindowInMilliseconds = ScopeInMilliseconds;

		if (UWorld* World = WorldManager->GetWorld())
		{
			FAudioDeviceHandle AudioDeviceHandle = World->GetAudioDevice();
			if (AudioDeviceHandle.IsValid())
			{
				AudioDesc.DeviceId = AudioDeviceHandle.GetDeviceID();
			}
		}

		FNDIAudio_SharedResourceUsage ResourceUsage;
		{
			bool bInstanceRequiresCpu = false;
			bool bInstanceRequiresGpu = false;
			SystemInstance->EvaluateBoundFunction(SampleAudioBufferFunctionName, bInstanceRequiresCpu, bInstanceRequiresGpu);
			FNDIAudio_SharedResourceUsage::UpdateUsageFlags(bInstanceRequiresCpu, bInstanceRequiresGpu, ResourceUsage.AudioUsage);
		}

		FNDIAudio_GeneratedData& SpectrumGeneratedData = WorldManager->EditGeneratedData<FNDIAudio_GeneratedData>();
		Inst->SharedResource = SpectrumGeneratedData.GetSharedResource(ResourceUsage, AudioDesc);
	}

	return true;
}

void UNiagaraDataInterfaceAudioOscilloscope::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace UE::Niagara::DataInterfaceAudioOscilloscope::Private;

	FInstanceData_GameThread* InstData = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);

	InstData->~FInstanceData_GameThread();
}


int32 UNiagaraDataInterfaceAudioOscilloscope::PerInstanceDataSize() const
{
	return sizeof(UE::Niagara::DataInterfaceAudioOscilloscope::Private::FInstanceData_GameThread);
}

void UNiagaraDataInterfaceAudioOscilloscope::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	UE::Niagara::DataInterfaceAudioOscilloscope::Private::FNDIProxy::ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, SystemInstance);
}

ETickingGroup UNiagaraDataInterfaceAudioOscilloscope::CalculateTickGroup(const void* PerInstanceData) const
{
	using namespace UE::Niagara::DataInterfaceAudioOscilloscope::Private;

	if (const FInstanceData_GameThread* Inst = reinterpret_cast<const FInstanceData_GameThread*>(PerInstanceData))
	{
		if (Inst->SharedResource.Usage.IsValid())
		{
			return static_cast<ETickingGroup>(FNDIAudio_GeneratedData::GeneratedDataTickGroup + 1);
		}
	}

	return NiagaraFirstTickGroup;
}

bool UNiagaraDataInterfaceAudioOscilloscope::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceAudioOscilloscope* CastedDestination = Cast<UNiagaraDataInterfaceAudioOscilloscope>(Destination);

	if (CastedDestination)
	{
		CastedDestination->Submix = Submix;
		CastedDestination->Resolution = Resolution;
		CastedDestination->ScopeInMilliseconds = ScopeInMilliseconds;
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
