// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceAudioPlayer.h"

#include "NiagaraTypes.h"
#include "NiagaraStats.h"
#include "Internationalization/Internationalization.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraWorldManager.h"
#include "Kismet/GameplayStatics.h"
#include "Sound/SoundBase.h"
#include "Components/AudioComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceAudioPlayer)

DECLARE_CYCLE_STAT(TEXT("Audio DI update persistent sound"), STAT_NiagaraAudioDIUpdateSound, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Audio DI create persistent sound"), STAT_NiagaraAudioDICreateSound, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Audio DI stop persistent sound"), STAT_NiagaraAudioDIStopSound, STATGROUP_Niagara);

const FName UNiagaraDataInterfaceAudioPlayer::PlayAudioName(TEXT("PlayAudioAtLocation"));
const FName UNiagaraDataInterfaceAudioPlayer::PlayPersistentAudioName(TEXT("PlayPersistentAudio"));
const FName UNiagaraDataInterfaceAudioPlayer::SetPersistentAudioVolumeName(TEXT("UpdateAudioVolume"));
const FName UNiagaraDataInterfaceAudioPlayer::SetPersistentAudioPitchName(TEXT("UpdateAudioPitch"));
const FName UNiagaraDataInterfaceAudioPlayer::SetPersistentAudioLocationName(TEXT("UpdateAudioLocation"));
const FName UNiagaraDataInterfaceAudioPlayer::SetPersistentAudioRotationName(TEXT("UpdateAudioRotation"));
const FName UNiagaraDataInterfaceAudioPlayer::SetPersistentAudioBoolParamName(TEXT("SetBooleanParameter"));
const FName UNiagaraDataInterfaceAudioPlayer::SetPersistentAudioIntegerParamName(TEXT("SetIntegerParameter"));
const FName UNiagaraDataInterfaceAudioPlayer::SetPersistentAudioFloatParamName(TEXT("SetFloatParameter"));
const FName UNiagaraDataInterfaceAudioPlayer::SetInitialAudioBoolParamName(TEXT("SetInitialBooleanParameter"));
const FName UNiagaraDataInterfaceAudioPlayer::SetInitialAudioIntegerParamName(TEXT("SetInitialIntegerParameter"));
const FName UNiagaraDataInterfaceAudioPlayer::SetInitialAudioFloatParamName(TEXT("SetInitialFloatParameter"));
const FName UNiagaraDataInterfaceAudioPlayer::PausePersistentAudioName(TEXT("SetPaused"));

struct FNiagaraAudioPlayerDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		LWCConversion = 1,
		InitialAudioParameters = 2,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

UNiagaraDataInterfaceAudioPlayer::UNiagaraDataInterfaceAudioPlayer(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SoundToPlay = nullptr;
	Attenuation = nullptr;
	Concurrency = nullptr;
	bLimitPlaysPerTick = true;
	MaxPlaysPerTick = 10;

	FNiagaraTypeDefinition Def(UObject::StaticClass());
	ConfigurationUserParameter.Parameter.SetType(Def);
}

UNiagaraDataInterfaceAudioPlayerSettings::UNiagaraDataInterfaceAudioPlayerSettings()
{
}

#if WITH_EDITORONLY_DATA

bool UNiagaraDataInterfaceAudioPlayer::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	// always upgrade to the latest version
	if (FunctionSignature.FunctionVersion < FNiagaraAudioPlayerDIFunctionVersion::LatestVersion)
	{
		TArray<FNiagaraFunctionSignature> AllFunctions;
		GetFunctionsInternal(AllFunctions);
		for (const FNiagaraFunctionSignature& Sig : AllFunctions)
		{
			if (FunctionSignature.Name == Sig.Name)
			{
				FunctionSignature = Sig;
				return true;
			}
		}
	}

	return false;
}

#endif

void UNiagaraDataInterfaceAudioPlayer::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceAudioPlayer::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FAudioPlayerInterface_InstanceData* PIData = new (PerInstanceData) FAudioPlayerInterface_InstanceData;
	PIData->LWCConverter = SystemInstance->GetLWCConverter();
	if (bLimitPlaysPerTick)
	{
		PIData->MaxPlaysPerTick = MaxPlaysPerTick;
	}
	PIData->bStopWhenComponentIsDestroyed = bStopWhenComponentIsDestroyed;
#if WITH_EDITORONLY_DATA
	PIData->bOnlyActiveDuringGameplay = bOnlyActiveDuringGameplay;
#endif
	return true;
}

void UNiagaraDataInterfaceAudioPlayer::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_Audio);

	FAudioPlayerInterface_InstanceData* InstData = (FAudioPlayerInterface_InstanceData*)PerInstanceData;
	for (const auto& Entry : InstData->PersistentAudioMapping)
	{
		if (Entry.Value.IsValid())
		{
			Entry.Value->Stop();
		}
	}
	InstData->~FAudioPlayerInterface_InstanceData();
}

bool UNiagaraDataInterfaceAudioPlayer::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_Audio);

	FAudioPlayerInterface_InstanceData* PIData = static_cast<FAudioPlayerInterface_InstanceData*>(PerInstanceData);
	if (!PIData)
	{
		return true;
	}
	if (ConfigurationUserParameter.Parameter.IsValid() && PIData->UserParamBinding.GetValue() != PIData->CachedUserParam)
	{
		// Reset if the user object ptr has been changed
		return true;
	}
	
	if (IsValid(SoundToPlay) && SystemInstance)
	{
		if (PIData->SoundToPlay != SoundToPlay)
		{
			PIData->SoundToPlay = SoundToPlay;
			PIData->bSoundToPlayIsLooping = SoundToPlay->IsLooping();
		}

		PIData->Attenuation = Attenuation;
		PIData->Concurrency = Concurrency;
		PIData->bValidOneShotSound = bAllowLoopingOneShotSounds || !PIData->bSoundToPlayIsLooping;

		if (ConfigurationUserParameter.Parameter.IsValid())
		{
			// Initialize the binding and retrieve the object.
			UObject* UserParamObject = PIData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), ConfigurationUserParameter.Parameter);
			PIData->CachedUserParam = Cast<UNiagaraDataInterfaceAudioPlayerSettings>(UserParamObject);
		}
	}
	else
	{
		PIData->SoundToPlay.Reset();
		PIData->Attenuation.Reset();
		PIData->Concurrency.Reset();
		PIData->CachedUserParam.Reset();
		PIData->bValidOneShotSound = false;
		PIData->bSoundToPlayIsLooping = false;
	}

	
	PIData->ParameterNames = ParameterNames;
	return false;
}

FAudioParameter ConvertVariableToAudioParam(const FNiagaraVariable& NiagaraVariable)
{
	FAudioParameter Param;
	Param.ParamName = NiagaraVariable.GetName();
	if (NiagaraVariable.GetType() == FNiagaraTypeDefinition::GetBoolDef() && NiagaraVariable.IsDataAllocated())
	{
		Param.BoolParam = NiagaraVariable.GetValue<bool>();
	}
	else if (NiagaraVariable.GetType() == FNiagaraTypeDefinition::GetIntDef() && NiagaraVariable.IsDataAllocated())
	{
		Param.IntParam = NiagaraVariable.GetValue<int32>();
	}
	else if (NiagaraVariable.GetType() == FNiagaraTypeDefinition::GetFloatDef() && NiagaraVariable.IsDataAllocated())
	{
		Param.FloatParam = NiagaraVariable.GetValue<float>();
	}
	return Param;
}

bool UNiagaraDataInterfaceAudioPlayer::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_Audio);

	FAudioPlayerInterface_InstanceData* PIData = static_cast<FAudioPlayerInterface_InstanceData*>(PerInstanceData);
	UNiagaraSystem* System = SystemInstance->GetSystem();
	UWorld* World = SystemInstance->GetWorldManager()->GetWorld();

#if WITH_EDITORONLY_DATA
	if (World == nullptr || (World->HasBegunPlay() == false && PIData->bOnlyActiveDuringGameplay))
	{
		PIData->PlayAudioQueue.Empty();
		PIData->InitialParamDataQueue.Empty();
		PIData->PersistentAudioMapping.Empty();
		return false;
	}
#endif

	TMap<int32, TArray<FAudioInitialParamData>> PerParticleParams;
	FAudioInitialParamData ParamData;
	if (PIData->PlayAudioQueue.IsEmpty())
	{
		PIData->InitialParamDataQueue.Empty();
	}
	else
	{
		PerParticleParams.Reserve(PIData->ParamCountEstimate);
		PIData->ParamCountEstimate = 0;
	}
	
	while (PIData->InitialParamDataQueue.Dequeue(ParamData))
	{
		PerParticleParams.FindOrAdd(ParamData.ParticleID).Add(ParamData);
	}
	
	if (!PIData->PlayAudioQueue.IsEmpty() && System)
	{
		//Drain the queue into an array here
		TArray<FAudioParticleData> Data;
		FAudioParticleData Value;
		while (PIData->PlayAudioQueue.Dequeue(Value))
		{
			Data.Add(Value);
			if (PIData->MaxPlaysPerTick > 0 && Data.Num() >= PIData->MaxPlaysPerTick)
			{
				// discard the rest of the queue if over the tick limit
				PIData->PlayAudioQueue.Empty();
				break;
			}
		}

		// play the fire-and-forget sounds
		if (World && PIData->SoundToPlay.IsValid())
		{
			AActor* SoundOwner = nullptr;
			if (const USceneComponent* AttachComp = SystemInstance->GetAttachComponent())
			{
				if (const USceneComponent* AttachParent = AttachComp->GetAttachParent())
				{
					SoundOwner = AttachParent->GetOwner();
				}
			}
			for (const FAudioParticleData& ParticleData : Data)
			{
				UInitialActiveSoundParams* InitialParams = nullptr;
				TArray<FAudioInitialParamData>* ParticleAudioParams = PerParticleParams.Find(ParticleData.ParticleID);
				if (PIData->GlobalInitialParameterValues.Num() > 0 || ParticleAudioParams)
				{
					InitialParams = NewObject<UInitialActiveSoundParams>(); // unfortunately the api needs a temp object as wrapper for the data
					for (const FNiagaraVariable& Var : PIData->GlobalInitialParameterValues)
					{
						InitialParams->AudioParams.Add(ConvertVariableToAudioParam(Var));
					}
					if (ParticleAudioParams)
					{
						for (const FAudioInitialParamData& Var : *ParticleAudioParams)
						{
							InitialParams->AudioParams.Add(ConvertVariableToAudioParam(Var.Value));
						}
					}
				}
				UGameplayStatics::PlaySoundAtLocation(World, PIData->SoundToPlay.Get(), ParticleData.Position, ParticleData.Rotation, ParticleData.Volume, ParticleData.Pitch, ParticleData.StartTime, PIData->Attenuation.Get(), PIData->Concurrency.Get(), SoundOwner, InitialParams);
			}
		}
	}

	// process the persistent audio updates
	FPersistentAudioParticleData Value;
	TSet<int32> UpdatedAudioHandles;
	while (PIData->PersistentAudioActionQueue.Dequeue(Value))
	{
		UAudioComponent* AudioComponent = nullptr;
		if (Value.AudioHandle > 0)
		{
			auto MappedValue = PIData->PersistentAudioMapping.Find(Value.AudioHandle);
			if (MappedValue && MappedValue->IsValid())
			{
				AudioComponent = MappedValue->Get();
			}
		}
		
		// since we are in the game thread here, it is safe for the callback to access the audio component
		if (Value.UpdateCallback)
		{
			Value.UpdateCallback(PIData, AudioComponent, SystemInstance);
		}
		UpdatedAudioHandles.Add(Value.AudioHandle);
	}

	// destroy all persistent audio components that were not updated this frame - this prevents looping sounds to go on after a particle has died. It also stops sounds if an emitter is culled by scalability.
	for (auto Iterator = PIData->PersistentAudioMapping.CreateIterator(); Iterator; ++Iterator)
	{
		if (!UpdatedAudioHandles.Contains(Iterator.Key()))
		{
			SCOPE_CYCLE_COUNTER(STAT_NiagaraAudioDIStopSound);
			TWeakObjectPtr<UAudioComponent> WeakComponent = Iterator.Value();
			UAudioComponent* AudioComponent = WeakComponent.IsValid() ? WeakComponent.Get() : nullptr;
			if (AudioComponent && AudioComponent->IsPlaying())
			{
				AudioComponent->Stop();
			}
			Iterator.RemoveCurrent();
		}
	}
	
	return false;
}

bool UNiagaraDataInterfaceAudioPlayer::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfaceAudioPlayer* OtherPlayer = CastChecked<UNiagaraDataInterfaceAudioPlayer>(Other);
	return OtherPlayer->SoundToPlay == SoundToPlay && OtherPlayer->Attenuation == Attenuation && OtherPlayer->Concurrency == Concurrency && OtherPlayer->ConfigurationUserParameter == ConfigurationUserParameter && OtherPlayer->bLimitPlaysPerTick == bLimitPlaysPerTick && OtherPlayer->MaxPlaysPerTick == MaxPlaysPerTick;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceAudioPlayer::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature Sig;
	Sig.Name = PlayAudioName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "PlayAudioDIFunctionDescription", "This function plays a sound at the given location after the simulation has ticked.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio interface")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Play Audio")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("PositionWS")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("RotationWS")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("VolumeFactor")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("PitchFactor")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("StartTime")));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")), NSLOCTEXT("Niagara", "PlayAudioParticleIDDescription", "This ID parameter is only needed when initial parameter values are used (to match the previously set parameters to the particle pülaying the audio)."));
	Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = PlayPersistentAudioName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "PlayPersistentAudioDIFunctionDescription", "This function plays a sound at the given location after the simulation has ticked. The returned handle can be used to control the sound in subsequent ticks.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Play Audio")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Existing Audio Handle")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position WS")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Rotation WS")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Volume Factor")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Pitch Factor")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Start Time")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Fade In Time")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Fade Out Time")));
	Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Audio Handle")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = SetPersistentAudioBoolParamName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "SetPersistentAudioBoolParamFunctionDescription", "If an active audio effect can be found for the given handle then the given sound cue parameter will be set on it.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Audio Handle")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Parameter Name Index")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Parameter Value")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = SetPersistentAudioIntegerParamName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "SetPersistentAudioIntegerParamFunctionDescription", "If an active audio effect can be found for the given handle then the given sound cue parameter will be set on it.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Audio Handle")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Parameter Name Index")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Parameter Value")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = SetPersistentAudioFloatParamName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "SetPersistentAudioFloatParamFunctionDescription", "If an active audio effect can be found for the given handle then the given sound cue parameter will be set on it.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Audio Handle")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Parameter Name Index")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Parameter Value")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = SetInitialAudioBoolParamName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "SetInitialAudioBoolParamFunctionDescription", "Sets a bool parameter for one-shot audio invocations.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Is Global Parameter")), NSLOCTEXT("Niagara", "SetInitialAudioBoolParamIsGlobalTooltip", "If true then this parameter is used for all one-shot sounds played with this data interface."));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle ID")), NSLOCTEXT("Niagara", "SetInitialAudioBoolParamParticleIDTooltip", "Only used when this is not a global parameter."));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Parameter Name Index")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Parameter Value")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = SetInitialAudioIntegerParamName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "SetInitialAudioIntegerParamFunctionDescription", "Sets an int parameter for one-shot audio invocations.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Is Global Parameter")), NSLOCTEXT("Niagara", "SetInitialAudioIntegerParamIsGlobalTooltip", "If true then this parameter is used for all one-shot sounds played with this data interface."));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle ID")), NSLOCTEXT("Niagara", "SetInitialAudioIntegerParamParticleIDTooltip", "Only used when this is not a global parameter."));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Parameter Name Index")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Parameter Value")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = SetInitialAudioFloatParamName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "SetInitialAudioFloatParamFunctionDescription", "Sets a float parameter for one-shot audio invocations.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Is Global Parameter")), NSLOCTEXT("Niagara", "SetInitialAudioFloatParamIsGlobalTooltip", "If true then this parameter is used for all one-shot sounds played with this data interface."));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle ID")), NSLOCTEXT("Niagara", "SetInitialAudioFloatParamParticleIDTooltip", "Only used when this is not a global parameter."));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Parameter Name Index")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Parameter Value")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = SetPersistentAudioVolumeName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "SetPersistentAudioVolumeFunctionDescription", "If an active audio effect can be found for the given handle then the this will adjusts its volume multiplier.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Audio Handle")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Volume Multiplier")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = SetPersistentAudioPitchName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "SetPersistentAudioPitchFunctionDescription", "If an active audio effect can be found for the given handle then the this will adjusts its pitch multiplier.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Audio Handle")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Pitch Multiplier")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = SetPersistentAudioLocationName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "SetPersistentAudioLocationFunctionDescription", "If an active audio effect can be found for the given handle then the this will adjusts its world position.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Audio Handle")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position WS")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = SetPersistentAudioRotationName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "SetPersistentAudioRotationFunctionDescription", "If an active audio effect can be found for the given handle then the this will adjusts its rotation in the world.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Audio Handle")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Rotation WS")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = PausePersistentAudioName;
#if WITH_EDITORONLY_DATA
	Sig.Description = NSLOCTEXT("Niagara", "SetPersistentAudioPausedDescription", "If an active audio effect can be found for the given handle then the this will either pause or unpause the effect.");
	Sig.FunctionVersion = FNiagaraAudioPlayerDIFunctionVersion::LatestVersion;
#endif
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Audio Interface")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Audio Handle")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Pause Audio")));
	OutFunctions.Add(Sig);
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, PlayOneShotAudio);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, PlayPersistentAudio);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetParameterBool);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetParameterInteger);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetParameterFloat);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetInitialParameterBool);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetInitialParameterInteger);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetInitialParameterFloat);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, UpdateVolume);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, UpdatePitch);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, UpdateLocation);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, UpdateRotation);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetPausedState);
void UNiagaraDataInterfaceAudioPlayer::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == PlayAudioName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, PlayOneShotAudio)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == PlayPersistentAudioName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, PlayPersistentAudio)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetPersistentAudioBoolParamName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetParameterBool)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetPersistentAudioIntegerParamName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetParameterInteger)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetPersistentAudioFloatParamName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetParameterFloat)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetInitialAudioBoolParamName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetInitialParameterBool)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetInitialAudioIntegerParamName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetInitialParameterInteger)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetInitialAudioFloatParamName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetInitialParameterFloat)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetPersistentAudioVolumeName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, UpdateVolume)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetPersistentAudioPitchName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, UpdatePitch)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetPersistentAudioLocationName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, UpdateLocation)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetPersistentAudioRotationName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, UpdateRotation)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == PausePersistentAudioName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioPlayer, SetPausedState)::Bind(this, OutFunc);
	}
	else
	{
		UE_LOGF(LogNiagara, Display, "Could not find data interface external function in %ls. Expected Name: %ls  Actual Name: %ls", *GetPathNameSafe(this), *PlayAudioName.ToString(), *BindingInfo.Name.ToString());
	}
}

void UNiagaraDataInterfaceAudioPlayer::SetParameterBool(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<int32> AudioHandleInParam(Context);
	FNDIInputParam<int32> NameIndexParam(Context);
	FNDIInputParam<FNiagaraBool> ValueParam(Context);
	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 Handle = AudioHandleInParam.GetAndAdvance();
		int32 NameIndex = NameIndexParam.GetAndAdvance();
		bool Value = ValueParam.GetAndAdvance();

		if (Handle > 0 && InstData->ParameterNames.IsValidIndex(NameIndex))
		{
			FName ParameterName = InstData->ParameterNames[NameIndex];
			FPersistentAudioParticleData AudioData;
			AudioData.AudioHandle = Handle;			
			AudioData.UpdateCallback = [ParameterName, Value](FAudioPlayerInterface_InstanceData*, UAudioComponent* AudioComponent, FNiagaraSystemInstance*)
			{
				if (AudioComponent && AudioComponent->IsPlaying())
				{
					AudioComponent->SetBoolParameter(ParameterName, Value);
				}
			};
			InstData->PersistentAudioActionQueue.Enqueue(AudioData);
		}
	}
}

void UNiagaraDataInterfaceAudioPlayer::SetParameterInteger(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<int32> AudioHandleInParam(Context);
	FNDIInputParam<int32> NameIndexParam(Context);
	FNDIInputParam<int32> ValueParam(Context);
	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 Handle = AudioHandleInParam.GetAndAdvance();
		int32 NameIndex = NameIndexParam.GetAndAdvance();
		int32 Value = ValueParam.GetAndAdvance();

		if (Handle > 0 && InstData->ParameterNames.IsValidIndex(NameIndex))
		{
			FName ParameterName = InstData->ParameterNames[NameIndex];
			FPersistentAudioParticleData AudioData;
			AudioData.AudioHandle = Handle;			
			AudioData.UpdateCallback = [ParameterName, Value](FAudioPlayerInterface_InstanceData*, UAudioComponent* AudioComponent, FNiagaraSystemInstance*)
			{
				if (AudioComponent && AudioComponent->IsPlaying())
				{
					AudioComponent->SetIntParameter(ParameterName, Value);
				}
			};
			InstData->PersistentAudioActionQueue.Enqueue(AudioData);
		}
	}
}

void UNiagaraDataInterfaceAudioPlayer::SetParameterFloat(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<int32> AudioHandleInParam(Context);
	FNDIInputParam<int32> NameIndexParam(Context);
	FNDIInputParam<float> ValueParam(Context);
	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 Handle = AudioHandleInParam.GetAndAdvance();
		int32 NameIndex = NameIndexParam.GetAndAdvance();
		float Value = ValueParam.GetAndAdvance();

		if (Handle > 0 && InstData->ParameterNames.IsValidIndex(NameIndex))
		{
			FName ParameterName = InstData->ParameterNames[NameIndex];
			FPersistentAudioParticleData AudioData;
			AudioData.AudioHandle = Handle;			
			AudioData.UpdateCallback = [ParameterName, Value](FAudioPlayerInterface_InstanceData*, UAudioComponent* AudioComponent, FNiagaraSystemInstance*)
			{
				if (AudioComponent && AudioComponent->IsPlaying())
				{
					AudioComponent->SetFloatParameter(ParameterName, Value);
				}
			};
			InstData->PersistentAudioActionQueue.Enqueue(AudioData);
		}
	}
}

void UNiagaraDataInterfaceAudioPlayer::SetInitialParameterBool(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<FNiagaraBool> IsGlobalParam(Context);
	FNDIInputParam<int32> ParticleIDParam(Context);
	FNDIInputParam<int32> NameIndexParam(Context);
	FNDIInputParam<FNiagaraBool> ValueParam(Context);
	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		bool IsGlobal = IsGlobalParam.GetAndAdvance();
		int32 ParticleID = ParticleIDParam.GetAndAdvance();
		int32 NameIndex = NameIndexParam.GetAndAdvance();
		bool Value = ValueParam.GetAndAdvance();

		if (InstData->ParameterNames.IsValidIndex(NameIndex))
		{
			FNiagaraVariable Parameter(FNiagaraTypeDefinition::GetBoolDef(), InstData->ParameterNames[NameIndex]);
			Parameter.SetValue(Value);

			if (IsGlobal)
			{
				InstData->GlobalInitialParameterValues.Add(Parameter);
			}
			else if (ParticleID >= 0)
			{
				InstData->InitialParamDataQueue.Enqueue({ ParticleID, Parameter });
			}
		}
	}
	InstData->ParamCountEstimate = Context.GetNumInstances();
}

void UNiagaraDataInterfaceAudioPlayer::SetInitialParameterInteger(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<FNiagaraBool> IsGlobalParam(Context);
	FNDIInputParam<int32> ParticleIDParam(Context);
	FNDIInputParam<int32> NameIndexParam(Context);
	FNDIInputParam<int32> ValueParam(Context);
	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		bool IsGlobal = IsGlobalParam.GetAndAdvance();
		int32 ParticleID = ParticleIDParam.GetAndAdvance();
		int32 NameIndex = NameIndexParam.GetAndAdvance();
		int32 Value = ValueParam.GetAndAdvance();

		if (InstData->ParameterNames.IsValidIndex(NameIndex))
		{
			FNiagaraVariable Parameter(FNiagaraTypeDefinition::GetIntDef(), InstData->ParameterNames[NameIndex]);
			Parameter.SetValue(Value);

			if (IsGlobal)
			{
				InstData->GlobalInitialParameterValues.Add(Parameter);
			}
			else if (ParticleID >= 0)
			{
				InstData->InitialParamDataQueue.Enqueue({ ParticleID, Parameter });
			}
		}
	}
	InstData->ParamCountEstimate = Context.GetNumInstances();
}

void UNiagaraDataInterfaceAudioPlayer::SetInitialParameterFloat(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<FNiagaraBool> IsGlobalParam(Context);
	FNDIInputParam<int32> ParticleIDParam(Context);
	FNDIInputParam<int32> NameIndexParam(Context);
	FNDIInputParam<float> ValueParam(Context);
	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		bool IsGlobal = IsGlobalParam.GetAndAdvance();
		int32 ParticleID = ParticleIDParam.GetAndAdvance();
		int32 NameIndex = NameIndexParam.GetAndAdvance();
		float Value = ValueParam.GetAndAdvance();

		if (InstData->ParameterNames.IsValidIndex(NameIndex))
		{
			FNiagaraVariable Parameter(FNiagaraTypeDefinition::GetFloatDef(), InstData->ParameterNames[NameIndex]);
			Parameter.SetValue(Value);

			if (IsGlobal)
			{
				InstData->GlobalInitialParameterValues.Add(Parameter);
			}
			else if (ParticleID >= 0)
			{
				InstData->InitialParamDataQueue.Enqueue({ ParticleID, Parameter });
			}
		}
	}
	InstData->ParamCountEstimate = Context.GetNumInstances();
}

void UNiagaraDataInterfaceAudioPlayer::UpdateVolume(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<int32> AudioHandleInParam(Context);
	FNDIInputParam<float> VolumeParam(Context);
	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 Handle = AudioHandleInParam.GetAndAdvance();
		float Volume = VolumeParam.GetAndAdvance();

		if (Handle > 0)
		{
			FPersistentAudioParticleData AudioData;
			AudioData.AudioHandle = Handle;			
			AudioData.UpdateCallback = [Volume](FAudioPlayerInterface_InstanceData*, UAudioComponent* AudioComponent, FNiagaraSystemInstance*)
			{
				if (AudioComponent && AudioComponent->IsPlaying())
				{
					AudioComponent->SetVolumeMultiplier(Volume);
				}
			};
			InstData->PersistentAudioActionQueue.Enqueue(AudioData);
		}
	}
}

void UNiagaraDataInterfaceAudioPlayer::UpdatePitch(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<int32> AudioHandleInParam(Context);
	FNDIInputParam<float> PitchParam(Context);
	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 Handle = AudioHandleInParam.GetAndAdvance();
		float Pitch = PitchParam.GetAndAdvance();

		if (Handle > 0)
		{
			FPersistentAudioParticleData AudioData;
			AudioData.AudioHandle = Handle;			
			AudioData.UpdateCallback = [Pitch](FAudioPlayerInterface_InstanceData*, UAudioComponent* AudioComponent, FNiagaraSystemInstance*)
			{
				if (AudioComponent && AudioComponent->IsPlaying())
				{
					AudioComponent->SetPitchMultiplier(Pitch);
				}
			};
			InstData->PersistentAudioActionQueue.Enqueue(AudioData);
		}
	}
}

void UNiagaraDataInterfaceAudioPlayer::UpdateLocation(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<int32> AudioHandleInParam(Context);
	FNDIInputParam<FVector3f> LocationParam(Context);
	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 Handle = AudioHandleInParam.GetAndAdvance();
		FVector Location = InstData->LWCConverter.ConvertSimulationVectorToWorld(LocationParam.GetAndAdvance());

		if (Handle > 0)
		{
			FPersistentAudioParticleData AudioData;
			AudioData.AudioHandle = Handle;			
			AudioData.UpdateCallback = [Location](FAudioPlayerInterface_InstanceData*, UAudioComponent* AudioComponent, FNiagaraSystemInstance*)
			{
				if (AudioComponent && AudioComponent->IsPlaying())
				{
					AudioComponent->SetWorldLocation(Location);
				}
			};
			InstData->PersistentAudioActionQueue.Enqueue(AudioData);
		}
	}
}

void UNiagaraDataInterfaceAudioPlayer::UpdateRotation(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<int32> AudioHandleInParam(Context);
	FNDIInputParam<FVector3f> RotationParam(Context);
	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 Handle = AudioHandleInParam.GetAndAdvance();
		FVector3f Rotation = RotationParam.GetAndAdvance();

		if (Handle > 0)
		{
			FPersistentAudioParticleData AudioData;
			AudioData.AudioHandle = Handle;			
			AudioData.UpdateCallback = [Rotation](FAudioPlayerInterface_InstanceData*, UAudioComponent* AudioComponent, FNiagaraSystemInstance*)
			{
				if (AudioComponent && AudioComponent->IsPlaying())
				{
					FRotator NewRotator(Rotation.X, Rotation.Y, Rotation.Z);
					AudioComponent->SetWorldRotation(NewRotator);
				}
			};
			InstData->PersistentAudioActionQueue.Enqueue(AudioData);
		}
	}
}

void UNiagaraDataInterfaceAudioPlayer::SetPausedState(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<int32> AudioHandleInParam(Context);
	FNDIInputParam<FNiagaraBool> PausedParam(Context);
	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 Handle = AudioHandleInParam.GetAndAdvance();
		bool IsPaused = PausedParam.GetAndAdvance();

		if (Handle > 0)
		{
			FPersistentAudioParticleData AudioData;
			AudioData.AudioHandle = Handle;			
			AudioData.UpdateCallback = [IsPaused](FAudioPlayerInterface_InstanceData*, UAudioComponent* AudioComponent, FNiagaraSystemInstance*)
			{
				if (AudioComponent)
				{
					AudioComponent->SetPaused(IsPaused);
				}
			};
			InstData->PersistentAudioActionQueue.Enqueue(AudioData);
		}
	}
}

void UNiagaraDataInterfaceAudioPlayer::PlayOneShotAudio(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<FNiagaraBool> PlayDataParam(Context);
	FNDIInputParam<FNiagaraPosition> PositionParam(Context);
	
	FNDIInputParam<float> RotationParamX(Context);
	FNDIInputParam<float> RotationParamY(Context);
	FNDIInputParam<float> RotationParamZ(Context);
	
	FNDIInputParam<float> VolumeParam(Context);
	FNDIInputParam<float> PitchParam(Context);
	FNDIInputParam<float> StartTimeParam(Context);
	FNDIInputParam<int32> ParticleIDParam(Context);

	FNDIOutputParam<FNiagaraBool> OutSample(Context);

	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());
	bool ValidSoundData = InstData->SoundToPlay.IsValid() && InstData->bValidOneShotSound;

#if WITH_EDITOR
	if (InstData->SoundToPlay.IsValid() && !InstData->bValidOneShotSound)
	{
		UE_LOGF(LogNiagara, Warning, "Suppressing Niagara one-shot sound %ls, because it's a looping sound. Use the 'play persistent audio' module instead. Source: %ls", *InstData->SoundToPlay->GetPathName(), *GetPathNameSafe(this));
	}
#endif

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		FNiagaraBool ShouldPlay = PlayDataParam.GetAndAdvance();
		FAudioParticleData Data;
		FNiagaraPosition SimulationPosition = PositionParam.GetAndAdvance();
		Data.Position = InstData->LWCConverter.ConvertSimulationPositionToWorld(SimulationPosition);
		Data.Rotation = FRotator(RotationParamX.GetAndAdvance(), RotationParamY.GetAndAdvance(), RotationParamZ.GetAndAdvance());
		Data.Volume = VolumeParam.GetAndAdvance();
		Data.Pitch = PitchParam.GetAndAdvance();
		Data.StartTime = StartTimeParam.GetAndAdvance();
		Data.ParticleID = ParticleIDParam.GetAndAdvance();

		FNiagaraBool Valid;
		if (ValidSoundData && ShouldPlay)
		{
			Valid.SetValue(InstData->PlayAudioQueue.Enqueue(Data));
		}
		OutSample.SetAndAdvance(Valid);
	}
}

void UNiagaraDataInterfaceAudioPlayer::PlayPersistentAudio(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FAudioPlayerInterface_InstanceData> InstData(Context);

	FNDIInputParam<FNiagaraBool> PlayAudioParam(Context);
	FNDIInputParam<int32> AudioHandleInParam(Context);
	FNDIInputParam<FVector3f> PositionParam(Context);
	FNDIInputParam<FVector3f> RotationParam(Context);
	FNDIInputParam<float> VolumeParam(Context);
	FNDIInputParam<float> PitchParam(Context);
	FNDIInputParam<float> StartTimeParam(Context);
	FNDIInputParam<float> FadeInParam(Context);
	FNDIInputParam<float> FadeOutParam(Context);

	FNDIOutputParam<int32> AudioHandleOutParam(Context);

	checkf(InstData.Get(), TEXT("Audio player interface has invalid instance data. %s"), *GetPathName());

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		bool ShouldPlay = PlayAudioParam.GetAndAdvance();
		int32 Handle = AudioHandleInParam.GetAndAdvance();
		FVector Position = InstData->LWCConverter.ConvertSimulationVectorToWorld(PositionParam.GetAndAdvance());
		FVector3f InRot = RotationParam.GetAndAdvance();
		FRotator Rotation = FRotator(InRot.X, InRot.Y, InRot.Z);
		float Volume = VolumeParam.GetAndAdvance();
		float Pitch = PitchParam.GetAndAdvance();
		float StartTime = StartTimeParam.GetAndAdvance();
		float FadeIn = FadeInParam.GetAndAdvance();
		float FadeOut = FadeOutParam.GetAndAdvance();

		FPersistentAudioParticleData AudioData;
		if (ShouldPlay)
		{
			if (Handle <= 0)
			{
				// play a new sound
				Handle = InstData->HandleCount.Increment();
				AudioData.AudioHandle = Handle;
				AudioData.UpdateCallback = [Handle, Position, Rotation, Volume, Pitch, StartTime, FadeIn](FAudioPlayerInterface_InstanceData* InstanceData, UAudioComponent*, FNiagaraSystemInstance* SystemInstance)
				{
					SCOPE_CYCLE_COUNTER(STAT_NiagaraAudioDICreateSound);
					USceneComponent* NiagaraComponent = SystemInstance->GetAttachComponent();
					TWeakObjectPtr<USoundBase> Sound = InstanceData->SoundToPlay;
					if (NiagaraComponent && Sound.IsValid())
					{
						bool bStopWithEffect = InstanceData->bStopWhenComponentIsDestroyed || Sound->IsLooping(); // we don't allow looping effects to outlive us because then they keep playing forever
						USoundConcurrency* SoundConcurrency = InstanceData->Concurrency.Get();
						if (InstanceData->CachedUserParam.IsValid() && InstanceData->CachedUserParam->bOverrideConcurrency)
						{
							SoundConcurrency = InstanceData->CachedUserParam->Concurrency;
						}

						USoundAttenuation* AttenuationSettings = InstanceData->Attenuation.Get();
						UAudioComponent* AudioComponent = UGameplayStatics::SpawnSoundAttached(Sound.Get(), NiagaraComponent, NAME_None, Position, Rotation, EAttachLocation::KeepWorldPosition, bStopWithEffect, Volume, Pitch, StartTime, AttenuationSettings, SoundConcurrency, true);
						if (AudioComponent == nullptr)
						{
							// looks like audio is disabled, so we'll skip adding a mapping
							return;
						}
						if (InstanceData->CachedUserParam.IsValid() && InstanceData->CachedUserParam->bOverrideAttenuationSettings)
						{
							AudioComponent->AdjustAttenuation(InstanceData->CachedUserParam->AttenuationSettings);
							AudioComponent->Play(StartTime);
						}
						if (FadeIn > 0.0)
						{
							AudioComponent->FadeIn(FadeIn, Volume, StartTime);
						}
						InstanceData->PersistentAudioMapping.Add(Handle, AudioComponent);
					}
				};
			}
			else
			{
				// add a dummy entry so the handle keeps playing
				AudioData.AudioHandle = Handle;
			}
			InstData->PersistentAudioActionQueue.Enqueue(AudioData);
			AudioHandleOutParam.SetAndAdvance(Handle);
			continue;
		}

		if (Handle > 0)
		{
			// stop sound
			AudioData.AudioHandle = Handle;
			AudioData.UpdateCallback = [Handle, FadeOut](FAudioPlayerInterface_InstanceData* InstanceData, UAudioComponent* AudioComponent, FNiagaraSystemInstance*)
			{
				SCOPE_CYCLE_COUNTER(STAT_NiagaraAudioDIStopSound);
				if (AudioComponent && AudioComponent->IsPlaying())
				{
					if (FadeOut > 0.0)
					{
						AudioComponent->FadeOut(FadeOut, 0);
					}
					else
					{
						AudioComponent->Stop();
					}
					InstanceData->PersistentAudioMapping.Remove(Handle);
				}
			};
			InstData->PersistentAudioActionQueue.Enqueue(AudioData);
		}
		AudioHandleOutParam.SetAndAdvance(0);
	}
}

bool UNiagaraDataInterfaceAudioPlayer::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceAudioPlayer* OtherTyped = CastChecked<UNiagaraDataInterfaceAudioPlayer>(Destination);
	OtherTyped->SoundToPlay = SoundToPlay;
	OtherTyped->Attenuation = Attenuation;
	OtherTyped->Concurrency = Concurrency;
	OtherTyped->ConfigurationUserParameter = ConfigurationUserParameter;
	OtherTyped->bLimitPlaysPerTick = bLimitPlaysPerTick;
	OtherTyped->MaxPlaysPerTick = MaxPlaysPerTick;
	OtherTyped->ParameterNames = ParameterNames;
	OtherTyped->bStopWhenComponentIsDestroyed = bStopWhenComponentIsDestroyed;
#if WITH_EDITORONLY_DATA
	OtherTyped->bOnlyActiveDuringGameplay = bOnlyActiveDuringGameplay;
#endif
	return true;
}

