// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceAudioSpectrum.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraGeneratedDataAudioSampling.h"
#include "NiagaraRenderer.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraTypes.h"
#include "NiagaraWorldManager.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SystemTextures.h"
#include "UnifiedBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceAudioSpectrum)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceGridAudioSpectrum"

namespace UE::Niagara::DataInterfaceAudioSpectrum::Private
{
	// Global VM function names, also used by the shaders code generation methods.
	static const FName AudioSpectrumFunctionName("AudioSpectrum");
	static const FName GetNumChannelsFunctionName("GetNumChannels");

	const TCHAR* CommonShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceAudioCommon.ush");
	const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceAudioSpectrumTemplate.ush");

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(int, ChannelCount)
		SHADER_PARAMETER(int, Resolution)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, SpectrumBuffer)
	END_SHADER_PARAMETER_STRUCT()

	constexpr int32 DefaultSamplesToBufferCount = 16384;

	struct FInstanceData_GameThread
	{
		FNDIAudio_SharedResourceHandle SharedResource;

		// if null, we should be using the default submix from each of the devices
		TWeakObjectPtr<USoundSubmix> SoundSubmix;
		int32 Resolution = 0;
		float MinimumFrequency = 0.0f;
		float MaximumFrequency = 0.0f;
		float NoiseFloorDb = 0.0f;

		bool bSoundSubmixAssigned = false;
	};

	struct FGameToRenderInstanceData
	{
		TArray<float> DataToUpload;

		int32 ChannelCount = 0;
		int32 ChannelResolution = 0;
	};

	struct FInstanceData_RenderThread
	{
		TArray<float> DataToUpload;
		TRefCountPtr<FRDGPooledBuffer> PooledBuffer;

		int32 ChannelCount = 0;
		int32 ChannelResolution = 0;
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FGameToRenderInstanceData); }

		static void ProvidePerInstanceDataForRenderThread(void* InDataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
		{
			FInstanceData_GameThread* InstanceData = static_cast<FInstanceData_GameThread*>(PerInstanceData);
			FGameToRenderInstanceData* DataForRenderThread = new(InDataForRenderThread) FGameToRenderInstanceData();

			if (InstanceData->SharedResource)
			{
				const FNDIAudio_SharedResource& AudioResource = InstanceData->SharedResource.ReadResource();

				// we need to be sure that the CPU tasks have been finished
				AudioResource.WaitForAudio();

				DataForRenderThread->DataToUpload = AudioResource.ReadSpectrumBuffer(
					DataForRenderThread->ChannelCount, DataForRenderThread->ChannelResolution);
			}
		}

		virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override
		{
			FGameToRenderInstanceData* InstanceDataFromGT = static_cast<FGameToRenderInstanceData*>(PerInstanceData);
			FInstanceData_RenderThread& InstanceData = SystemInstancesToInstanceData_RT.FindOrAdd(InstanceID);

			InstanceData.DataToUpload = MoveTemp(InstanceDataFromGT->DataToUpload);
			InstanceData.ChannelCount = InstanceDataFromGT->ChannelCount;
			InstanceData.ChannelResolution = InstanceDataFromGT->ChannelResolution;

			InstanceDataFromGT->~FGameToRenderInstanceData();
		}

		virtual void PreStage(const FNDIGpuComputePreStageContext& Context)
		{
			const FNiagaraSystemInstanceID SystemInstanceID = Context.GetSystemInstanceID();
			FInstanceData_RenderThread& InstanceData = SystemInstancesToInstanceData_RT.FindChecked(SystemInstanceID);
			if (InstanceData.DataToUpload.Num() > 0)
			{
				FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

				const void* InitialData = InstanceData.DataToUpload.GetData();
				const uint64 InitialDataSize = InstanceData.DataToUpload.Num() * InstanceData.DataToUpload.GetTypeSize();
				const uint64 BufferSize = Align(InitialDataSize, 16);

				FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(float), IntCastChecked<uint32>(BufferSize / sizeof(float)));
				ResizeBufferIfNeeded(GraphBuilder, InstanceData.PooledBuffer, BufferDesc, TEXT("NiagaraAudioSpectrumBuffer"));

				GraphBuilder.QueueBufferUpload(
					GraphBuilder.RegisterExternalBuffer(InstanceData.PooledBuffer),
					InitialData,
					InitialDataSize,
					[Data = MoveTemp(InstanceData.DataToUpload)](const void*) {}
				);

				InstanceData.DataToUpload.Empty();
			}
		}

		TMap<FNiagaraSystemInstanceID, FInstanceData_RenderThread> SystemInstancesToInstanceData_RT;
	};

	static float SamplePlanarAudioBuffer(TConstArrayView<float> Samples, int32 ChannelCount, int32 FrameCount, int32 ChannelIndex, float Position)
	{
		float FrameIndex = FMath::Clamp(Position, 0.0f, 1.0f) * (FrameCount - 1);
		int32 LowerIndex = FMath::Max(0, FMath::FloorToInt(FrameIndex));
		int32 UpperIndex = FMath::Min(LowerIndex + 1, FrameCount - 1);

		float Fraction = FrameIndex - float(LowerIndex);

		float LowerValue = Samples[ChannelIndex * FrameCount + LowerIndex];
		float UpperValue = Samples[ChannelIndex * FrameCount + UpperIndex];

		return FMath::Lerp(LowerValue, UpperValue, Fraction);
	}
}

UNiagaraDataInterfaceAudioSpectrum::UNiagaraDataInterfaceAudioSpectrum(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new UE::Niagara::DataInterfaceAudioSpectrum::Private::FNDIProxy());
}

void UNiagaraDataInterfaceAudioSpectrum::GetSpectrumValue(FVectorVMExternalFunctionContext& Context)
{
	using namespace UE::Niagara::DataInterfaceAudioSpectrum::Private;

	const int32 InstanceCount = Context.GetNumInstances();

	VectorVM::FUserPtrHandler<FInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncInputHandler<float> InNormalizedPos(Context);
	VectorVM::FExternalFuncInputHandler<int32> InChannel(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutValue(Context);

	if (InstData->SharedResource)
	{
		const FNDIAudio_SharedResource& SharedAudio = InstData->SharedResource.ReadResource();

		{
			SharedAudio.WaitForAudio();
		}

		int32 ChannelCount = 0;
		int32 ChannelResolution = 0;
		TConstArrayView<float> SpectrumBuffer = SharedAudio.ReadSpectrumBuffer(ChannelCount, ChannelResolution);

		if (ChannelCount && ChannelResolution)
		{
			for (int32 InstanceIdx = 0; InstanceIdx < InstanceCount; ++InstanceIdx)
			{
				const float Position = InNormalizedPos.GetAndAdvance();
				const int32 ChannelIndex = InChannel.GetAndAdvance();
				*OutValue.GetDestAndAdvance() = SamplePlanarAudioBuffer(
					SpectrumBuffer,
					ChannelCount,
					ChannelResolution,
					FMath::Clamp(ChannelIndex, 0, ChannelCount - 1),
					Position);
			}

			return;
		}
	}

	// fallback defaults
	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		InNormalizedPos.GetAndAdvance();
		InChannel.GetAndAdvance();
		*OutValue.GetDestAndAdvance() = 0.0f;
	}
}

void UNiagaraDataInterfaceAudioSpectrum::GetNumChannels(FVectorVMExternalFunctionContext& Context)
{
	const int32 InstanceCount = Context.GetNumInstances();

	VectorVM::FUserPtrHandler<UE::Niagara::DataInterfaceAudioSpectrum::Private::FInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutChannel(Context);

	const int32 ChannelCount = InstData->SharedResource
		? InstData->SharedResource.ReadResource().GetChannelCount()
		: 0;

	for (int32 InstanceIdx = 0; InstanceIdx < InstanceCount; ++InstanceIdx)
	{
		*OutChannel.GetDestAndAdvance() = ChannelCount;
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceAudioSpectrum::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace UE::Niagara::DataInterfaceAudioSpectrum::Private;
	Super::GetFunctionsInternal(OutFunctions);

	{
		FNiagaraFunctionSignature GetSpectrumSignature;
		GetSpectrumSignature.Name = AudioSpectrumFunctionName;
		GetSpectrumSignature.Inputs.Add(FNiagaraVariable(GetClass(), TEXT("Spectrum")));
		GetSpectrumSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("NormalizedPositionInSpectrum")));
		GetSpectrumSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ChannelIndex")));
		GetSpectrumSignature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Amplitude")));

		GetSpectrumSignature.bMemberFunction = true;
		GetSpectrumSignature.bRequiresContext = false;
		OutFunctions.Add(GetSpectrumSignature);
	}

	{
		FNiagaraFunctionSignature NumChannelsSignature;
		NumChannelsSignature.Name = GetNumChannelsFunctionName;
		NumChannelsSignature.Inputs.Add(FNiagaraVariable(GetClass(), TEXT("Spectrum")));
		NumChannelsSignature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumChannels")));

		NumChannelsSignature.bMemberFunction = true;
		NumChannelsSignature.bRequiresContext = false;
		OutFunctions.Add(NumChannelsSignature);
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioSpectrum, GetSpectrumValue);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceAudioSpectrum, GetNumChannels);

void UNiagaraDataInterfaceAudioSpectrum::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	using namespace UE::Niagara::DataInterfaceAudioSpectrum::Private;

	if (BindingInfo.Name == AudioSpectrumFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioSpectrum, GetSpectrumValue)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetNumChannelsFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceAudioSpectrum, GetNumChannels)::Bind(this, OutFunc);
	}
	else
	{
		ensureMsgf(false, TEXT("Error! Function defined for this class but not bound."));
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceAudioSpectrum::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace UE::Niagara::DataInterfaceAudioSpectrum::Private;

	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}
	InVisitor->UpdateShaderFile(CommonShaderFile);
	InVisitor->UpdateShaderFile(TemplateShaderFile);
	InVisitor->UpdateShaderParameters<FShaderParameters>();
	return true;
}

bool UNiagaraDataInterfaceAudioSpectrum::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace UE::Niagara::DataInterfaceAudioSpectrum::Private;

	if (Super::GetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, OutHLSL))
	{
		return true;
	}

	if ((FunctionInfo.DefinitionName == AudioSpectrumFunctionName) ||
		(FunctionInfo.DefinitionName == GetNumChannelsFunctionName))
	{
		return true;
	}

	return false;
}

void UNiagaraDataInterfaceAudioSpectrum::GetCommonHLSL(FString& OutHLSL)
{
	Super::GetCommonHLSL(OutHLSL);
	OutHLSL.Appendf(TEXT("#include \"%s\"\n"), UE::Niagara::DataInterfaceAudioSpectrum::Private::CommonShaderFile);
}

void UNiagaraDataInterfaceAudioSpectrum::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, UE::Niagara::DataInterfaceAudioSpectrum::Private::TemplateShaderFile, TemplateArgs);
}

#endif

void UNiagaraDataInterfaceAudioSpectrum::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<UE::Niagara::DataInterfaceAudioSpectrum::Private::FShaderParameters>();
}

void UNiagaraDataInterfaceAudioSpectrum::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace UE::Niagara::DataInterfaceAudioSpectrum::Private;

	FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	FInstanceData_RenderThread& InstanceData = DIProxy.SystemInstancesToInstanceData_RT.FindChecked(Context.GetSystemInstanceID());

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->ChannelCount = InstanceData.ChannelCount;
	ShaderParameters->Resolution = InstanceData.ChannelResolution;

	if (InstanceData.PooledBuffer)
	{
		FRDGBufferRef RDGBuffer = GraphBuilder.RegisterExternalBuffer(InstanceData.PooledBuffer);
		ShaderParameters->SpectrumBuffer = GraphBuilder.CreateSRV(RDGBuffer, PF_R32_FLOAT);
	}
	else
	{
		FRDGBufferRef RDGBuffer = GSystemTextures.GetDefaultBuffer(GraphBuilder, sizeof(float), 0.0f);
		ShaderParameters->SpectrumBuffer = GraphBuilder.CreateSRV(RDGBuffer, PF_R32_FLOAT);
	}
}

bool UNiagaraDataInterfaceAudioSpectrum::Equals(const UNiagaraDataInterface* Other) const
{
	bool bIsEqual = Super::Equals(Other);

	const UNiagaraDataInterfaceAudioSpectrum* OtherSpectrum = CastChecked<const UNiagaraDataInterfaceAudioSpectrum>(Other);

	bIsEqual &= OtherSpectrum->Submix == Submix;
	bIsEqual &= OtherSpectrum->Resolution == Resolution;
	bIsEqual &= OtherSpectrum->MinimumFrequency == MinimumFrequency;
	bIsEqual &= OtherSpectrum->MaximumFrequency == MaximumFrequency;
	bIsEqual &= OtherSpectrum->NoiseFloorDb == NoiseFloorDb;

	return bIsEqual;
}

void UNiagaraDataInterfaceAudioSpectrum::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceAudioSpectrum::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace UE::Niagara::DataInterfaceAudioSpectrum::Private;

	FInstanceData_GameThread* Inst = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);

	// reset our per instance data if the submix has changed/invalidated
	if (Inst->bSoundSubmixAssigned && !Inst->SoundSubmix.IsValid())
	{
		return true;
	}

	if (Inst->SoundSubmix != Submix)
	{
		return true;
	}

	return false;
}

bool UNiagaraDataInterfaceAudioSpectrum::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace UE::Niagara::DataInterfaceAudioSpectrum::Private;

	FInstanceData_GameThread* Inst = new (PerInstanceData) FInstanceData_GameThread();

	Inst->SoundSubmix = MakeWeakObjectPtr(Submix);
	Inst->Resolution = Resolution;
	Inst->MinimumFrequency = MinimumFrequency;
	Inst->MaximumFrequency = MaximumFrequency;
	Inst->NoiseFloorDb = NoiseFloorDb;
	Inst->bSoundSubmixAssigned = Submix != nullptr;

	if (FNiagaraWorldManager* WorldManager = SystemInstance->GetWorldManager())
	{
		FNDIAudio_GeneratedData::FResourceDesc AudioDesc;
		AudioDesc.DeviceId = INDEX_NONE;
		AudioDesc.Submix = Submix;
		AudioDesc.bUseLatestAudio = false;
		AudioDesc.bGenerateSpectrum = true;
		AudioDesc.MinimumFrequency = MinimumFrequency;
		AudioDesc.MaximumFrequency = MaximumFrequency;
		AudioDesc.NoiseFloorDb = NoiseFloorDb;
		AudioDesc.SpectrumSamplingResolution = Resolution;
		AudioDesc.SamplingMethod = FNDIAudio_SharedResource::ESamplingWindowMethod::ByCount;
		AudioDesc.SamplingWindowCount = DefaultSamplesToBufferCount;
		AudioDesc.bContinuousSampling = true;

		if (UWorld* World = WorldManager->GetWorld())
		{
			FAudioDeviceHandle AudioDeviceHandle = World->GetAudioDevice();
			if (AudioDeviceHandle.IsValid())
			{
				AudioDesc.DeviceId = AudioDeviceHandle.GetDeviceID();
			}
		}

		FNDIAudio_SharedResourceUsage ResourceUsage;
		{
			bool bInstanceRequiresCpu = false;
			bool bInstanceRequiresGpu = false;
			SystemInstance->EvaluateBoundFunction(AudioSpectrumFunctionName, bInstanceRequiresCpu, bInstanceRequiresGpu);
			FNDIAudio_SharedResourceUsage::UpdateUsageFlags(bInstanceRequiresCpu, bInstanceRequiresGpu, ResourceUsage.SpectrumUsage);
		}

		FNDIAudio_GeneratedData& SpectrumGeneratedData = WorldManager->EditGeneratedData<FNDIAudio_GeneratedData>();
		Inst->SharedResource = SpectrumGeneratedData.GetSharedResource(ResourceUsage, AudioDesc);
	}

	return true;
}

void UNiagaraDataInterfaceAudioSpectrum::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace UE::Niagara::DataInterfaceAudioSpectrum::Private;

	FInstanceData_GameThread* InstData = reinterpret_cast<FInstanceData_GameThread*>(PerInstanceData);

	InstData->~FInstanceData_GameThread();
}

int32 UNiagaraDataInterfaceAudioSpectrum::PerInstanceDataSize() const
{
	return sizeof(UE::Niagara::DataInterfaceAudioSpectrum::Private::FInstanceData_GameThread);
}

void UNiagaraDataInterfaceAudioSpectrum::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	UE::Niagara::DataInterfaceAudioSpectrum::Private::FNDIProxy::ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, SystemInstance);
}

ETickingGroup UNiagaraDataInterfaceAudioSpectrum::CalculateTickGroup(const void* PerInstanceData) const
{
	using namespace UE::Niagara::DataInterfaceAudioSpectrum::Private;

	const FInstanceData_GameThread* Inst = reinterpret_cast<const FInstanceData_GameThread*>(PerInstanceData);
	if (Inst->SharedResource.Usage.IsValid())
	{
		return static_cast<ETickingGroup>(FNDIAudio_GeneratedData::GeneratedDataTickGroup + 1);
	}

	return NiagaraFirstTickGroup;
}

bool UNiagaraDataInterfaceAudioSpectrum::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceAudioSpectrum* CastedDestination = Cast<UNiagaraDataInterfaceAudioSpectrum>(Destination);

	if (CastedDestination)
	{
		CastedDestination->Submix = Submix;
		CastedDestination->Resolution = Resolution;
		CastedDestination->MinimumFrequency = MinimumFrequency;
		CastedDestination->MaximumFrequency = MaximumFrequency;
		CastedDestination->NoiseFloorDb = NoiseFloorDb;
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
