// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceCamera.h"

#include "Engine/LocalPlayer.h"
#include "Engine/GameViewportClient.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraTypes.h"
#include "NiagaraWorldManager.h"
#include "Internationalization/Internationalization.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceCamera)

#if WITH_EDITORONLY_DATA
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"
#endif

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceCamera"

struct FNiagaraCameraDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
        AddedPreViewTranslation = 1,
		AddedBufferSizeInvSizeForGPU = 2,
		LWCConversion = 3,
		AddGPUNearPlane = 4,
		AddGPUFOVData = 5,
		AddCPUViewportAndNearPlane = 6,

        VersionPlusOne,
        LatestVersion = VersionPlusOne - 1
    };
};

const TCHAR* UNiagaraDataInterfaceCamera::TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceCamera.ush");
const FName UNiagaraDataInterfaceCamera::GetViewPropertiesName(TEXT("GetViewPropertiesGPU"));
const FName UNiagaraDataInterfaceCamera::GetClipSpaceTransformsName(TEXT("GetClipSpaceTransformsGPU"));
const FName UNiagaraDataInterfaceCamera::GetViewSpaceTransformsName(TEXT("GetViewSpaceTransformsGPU"));
const FName UNiagaraDataInterfaceCamera::GetCameraPropertiesName(TEXT("GetCameraPropertiesCPU/GPU"));
const FName UNiagaraDataInterfaceCamera::GetFieldOfViewName(TEXT("GetFieldOfView"));
const FName UNiagaraDataInterfaceCamera::CalculateDistancesName(TEXT("CalculateParticleDistancesCPU"));
const FName UNiagaraDataInterfaceCamera::QueryClosestName(TEXT("QueryClosestParticlesCPU"));
const FName UNiagaraDataInterfaceCamera::GetTAAJitterName(TEXT("GetTAAJitter"));
const FName UNiagaraDataInterfaceCamera::GetSplitScreenName(TEXT("GetSplitscreenInfo"));
const FName UNiagaraDataInterfaceCamera::ApplyPreViewTranslationToPosition(TEXT("ApplyPreViewTranslationToPositionGPU"));
const FName UNiagaraDataInterfaceCamera::RemovePreViewTranslationFromPosition(TEXT("RemovePreViewTranslationFromPositionGPU"));

struct FCameraDataInterface_InstanceDataRT
{
	ESplitScreenType::Type SplitscreenMode = ESplitScreenType::Type::None;
};

struct FNiagaraDataInterfaceProxyCameraQuery : FNiagaraDataInterfaceProxy
{
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FCameraDataInterface_InstanceDataRT); }

	static void ProvidePerInstanceDataForRenderThread(void* InDataForRenderThread, void* InDataFromGameThread, const FNiagaraSystemInstanceID& SystemInstance)
	{
		FCameraDataInterface_InstanceDataRT* DataForRenderThread = new (InDataForRenderThread) FCameraDataInterface_InstanceDataRT();
		const FCameraDataInterface_InstanceData* DataFromGameThread = static_cast<FCameraDataInterface_InstanceData*>(InDataFromGameThread);
		
		DataForRenderThread->SplitscreenMode = DataFromGameThread->SplitscreenMode;
	}

	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& InstanceID) override
	{
		FCameraDataInterface_InstanceDataRT* InstanceDataFromGT = static_cast<FCameraDataInterface_InstanceDataRT*>(PerInstanceData);
		FCameraDataInterface_InstanceDataRT& InstanceData = SystemInstancesToInstanceData_RT.FindOrAdd(InstanceID);
		InstanceData = *InstanceDataFromGT;

		// clean up the GT data
		InstanceDataFromGT->~FCameraDataInterface_InstanceDataRT();
	}

	TMap<FNiagaraSystemInstanceID, FCameraDataInterface_InstanceDataRT> SystemInstancesToInstanceData_RT;
};

UNiagaraDataInterfaceCamera::UNiagaraDataInterfaceCamera(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyCameraQuery());
}

void UNiagaraDataInterfaceCamera::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceCamera::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FCameraDataInterface_InstanceData* PIData = new (PerInstanceData) FCameraDataInterface_InstanceData;
	return true;
}

void UNiagaraDataInterfaceCamera::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FCameraDataInterface_InstanceData* PIData = static_cast<FCameraDataInterface_InstanceData*>(PerInstanceData);
	PIData->~FCameraDataInterface_InstanceData();

	ENQUEUE_RENDER_COMMAND(RemoveProxy)
	(
		[RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxyCameraQuery>(), InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->SystemInstancesToInstanceData_RT.Remove(InstanceID);
		}
	);
}

bool UNiagaraDataInterfaceCamera::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FCameraDataInterface_InstanceData* PIData = (FCameraDataInterface_InstanceData*)PerInstanceData;
	if (!PIData)
	{
		return true;
	}

	// calculate the distance for each particle and sort by distance (if required)
	PIData->ParticlesSortedByDistance.Empty();
	FDistanceData DistanceData;
	while (PIData->DistanceSortQueue.Dequeue(DistanceData))
	{
		PIData->ParticlesSortedByDistance.Add(DistanceData);
	}
	PIData->ParticlesSortedByDistance.StableSort([](const FDistanceData& A, const FDistanceData& B) { return A.DistanceSquared < B.DistanceSquared; });

	// grab the current camera data
	FNiagaraLWCConverter LWCConverter = SystemInstance->GetLWCConverter();
	PIData->LWCConverter = LWCConverter;
	UWorld* World = SystemInstance->GetWorldManager()->GetWorld();
	if (World && PlayerControllerIndex < World->GetNumPlayerControllers())
	{
		int32 LocalPlayerIndex = 0;
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController != nullptr && PlayerController->GetLocalPlayer() != nullptr)
			{
				if (LocalPlayerIndex == PlayerControllerIndex)
				{
					if (PlayerController->PlayerCameraManager)
					{
						PIData->CameraLocation = LWCConverter.ConvertWorldToSimulationPosition(PlayerController->PlayerCameraManager->GetCameraLocation());
						PIData->CameraRotation = PlayerController->PlayerCameraManager->GetCameraRotation();
						PIData->CameraFOV = PlayerController->PlayerCameraManager->GetFOVAngle();
						PIData->NearClipPlane = GNearClippingPlane;
						if (ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer())
						{
							PIData->SplitscreenMode = LocalPlayer->ViewportClient ? LocalPlayer->ViewportClient->GetCurrentSplitscreenConfiguration() : ESplitScreenType::Type::None;
							if (LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->Viewport)
							{
								const FIntPoint Size = LocalPlayer->ViewportClient->Viewport->GetSizeXY();
								PIData->ViewportSizeX = static_cast<float>(Size.X);
								PIData->ViewportSizeY = static_cast<float>(Size.Y);
							}
							else
							{
								PIData->ViewportSizeX = 0;
								PIData->ViewportSizeY = 0;
							}
						}
						return false;
					}
					else
					{
						// if the requested local player doesn't have a player camera manager, then just break out of our search and provide defaults
						UE_LOGF(LogNiagara, Warning, "%ls failed to find PlayerCameraManager for LocalPlayer %d", *GetPathNameSafe(this), PlayerControllerIndex);
						break;
					}
				}

				++LocalPlayerIndex;
			}
		}
	}
#if WITH_EDITORONLY_DATA
	if (GCurrentLevelEditingViewportClient)
	{
		const FViewportCameraTransform& ViewTransform = GCurrentLevelEditingViewportClient->GetViewTransform();
		PIData->CameraLocation = LWCConverter.ConvertWorldToSimulationPosition(ViewTransform.GetLocation());
		PIData->CameraRotation = ViewTransform.GetRotation();
		PIData->CameraFOV = GCurrentLevelEditingViewportClient->ViewFOV;
		PIData->NearClipPlane = GNearClippingPlane;
		if (GCurrentLevelEditingViewportClient->Viewport)
		{
			const FIntPoint Size = GCurrentLevelEditingViewportClient->Viewport->GetSizeXY();
			PIData->ViewportSizeX = static_cast<float>(Size.X);
			PIData->ViewportSizeY = static_cast<float>(Size.Y);
		}
		return false;
	}
#endif

	PIData->CameraLocation = FNiagaraPosition(ForceInitToZero);
	PIData->CameraRotation = FRotator(0);
	PIData->CameraFOV = 0;
	
	return false;
}

void UNiagaraDataInterfaceCamera::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	FNiagaraDataInterfaceProxyCameraQuery::ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, SystemInstance);
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceCamera::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature Sig;
	Sig.Name = GetViewPropertiesName;
	Sig.Description = LOCTEXT("GetViewPropertiesDescription", "This function returns the properties of the current view. Only valid for gpu particles.");
	Sig.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;
	Sig.bMemberFunction = true;
	Sig.bRequiresContext = false;
	Sig.bSupportsCPU = false;
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Camera interface")));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("View Position World")), LOCTEXT("ViewPositionWorldDescription", "The camera position in world space."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("View Forward Vector")), LOCTEXT("ViewForwardVectorDescription", "The world space direction that the camera is pointing."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("View Up Vector")), LOCTEXT("ViewUpVectorDescription", "The camera's up direction in world space."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("View Right Vector")), LOCTEXT("ViewRightVectorDescription", "The camera's right direction in world space."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("View Size And Inverse Size")), LOCTEXT("ViewSizeAndInverseSizeDescription", "Returns a vec4 - the x and y values are width and height of the view; the z and w values are the inverse width and height."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Screen To View Space")), LOCTEXT("ScreenToViewSpaceDescription", "Can be used to map standard viewport UV to an unprojected viewpos.\nExample usage:\nViewPos.x =  ViewportUV.x * ScreenToViewSpace.x + ScreenToViewSpace.z;\nViewPos.y =  ViewportUV.y * ScreenToViewSpace.y + ScreenToViewSpace.w;"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Temporal AA Jitter (Current Frame)")), LOCTEXT("TemporalAACurrentDescription", "Returns the movement of the current frame view due to AA jittering"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Temporal AA Jitter (Previous Frame)")), LOCTEXT("TemporalAAPreviousDescription", "Returns the movement of the previous frame view due to AA jittering"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("PreViewTranslation")), LOCTEXT("PreViewTranslationDescription", "Returns the translation to apply for the various 'Translated XX to XX' transforms."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Buffer Size And Inverse Size")), LOCTEXT("BufferSizeAndInverseSizeDescription", "Returns a vec4 - the x and y values are width and height of the buffer; the z and w values are the inverse width and height."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Viewport Offset")), LOCTEXT("ViewportOffsetDescription", "Returns the viewport offset in the buffer."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Near Plane")), LOCTEXT("NearPlaneDescription", "Returns the near plane value."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Field Of View (Current Frame)")), LOCTEXT("FOVCurrentDescription", "Returns the current camera field of view (X & Y) in radians."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Field Of View (Previous Frame)")), LOCTEXT("FOVPreviousDescription", "Returns the previous camera field of view (X & Y) in radians."));
	OutFunctions.Add(Sig);


	Sig = FNiagaraFunctionSignature();
	Sig.Name = GetClipSpaceTransformsName;
	Sig.Description = LOCTEXT("GetClipSpaceTransformsDescription", "This function returns the clip transforms for the current view. Only valid for gpu particles.");
	Sig.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;
	Sig.bMemberFunction = true;
	Sig.bSupportsCPU = false;
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Camera interface")));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("World To Clip Transform")), LOCTEXT("WorldToClipTransformDescription", "Transforms a world space position to clip space"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Translated World To Clip Transform")), LOCTEXT("TranslatedWorldToClipTransformDescription", "Same as world to clip transform, but the camera position is at the origin when camera translation is enabled.\nThis allows for more precision of the transform when dealing with big coordinates.\nTo use this transform, you first need to add PreViewTranslation to your transform target."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Clip To World Transform")), LOCTEXT("ClipToWorldTransformDescription", "Transforms a clip space position to world space"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Clip To View Transform")), LOCTEXT("ClipToViewTransformDescription", "Transforms a clip space position to view space"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Clip To Translated World Transform")), LOCTEXT("ClipToTranslatedWorldTransformDescription", "The inverse of the translated world to clip transform. When camera translation is enabled, the transform is such that the camera position is at the origin.\nThis allows for more precision of the transform when dealing with big coordinates.\nTo get the actual world space position, you need to subtract PreViewTranslation from the transform result."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Screen To World Transform")), LOCTEXT("ScreenToWorldTransformDescription", "Converts a screen space position a world position.\nExample combining the transform with scene depth:\nfloat3 WorldPosition = mul(float4(ScreenPosition * SceneDepth, SceneDepth, 1), ScreenToWorld).xyz;"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Screen To Translated World Transform")), LOCTEXT("ScreenToTranslatedWorldTransformDescription", "Same as the screen to world transform, but with the assumption that the camera is at the origin.\nThis allows for more precision of the transform when dealing with big coordinates.\nTo get the actual world space position, you need to subtract PreViewTranslation from the transform result."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Clip To Previous Clip Transform")), LOCTEXT("ClipToPreviousClipTransformDescription", "Transforms from a current clip space position to last frame's clip space position.\nThis can be used to calculate for example post process velocity."));
	OutFunctions.Add(Sig);


	Sig = FNiagaraFunctionSignature();
	Sig.Name = GetViewSpaceTransformsName;
	Sig.Description = LOCTEXT("GetViewSpaceTransformsDescription", "This function returns the relevant transforms for the current view. Only valid for gpu particles.");
	Sig.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;
	Sig.bMemberFunction = true;
	Sig.bSupportsCPU = false;
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Camera interface")));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Translated World To View Transform")), LOCTEXT("TranslatedWorldToViewTransformDescription", "Transforms a position from world to view, but the view position is assumed to be at the origin.\nThis allows for more precision of the transform when dealing with big coordinates.\nTo use this transform, you first need to add PreViewTranslation to your transform target."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("View To Translated World Transform")), LOCTEXT("ViewToTranslatedWorldTransformDescription", "The inverse of the translated world to view transform. When camera translation is enabled, the transform is such that the camera position is at the origin.\nThis allows for more precision of the transform when dealing with big coordinates.\nTo get the actual world space position, you need to subtract PreViewTranslation from the transform result."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Translated World To Camera View Transform")), LOCTEXT("TranslatedWorldToCameraTransformDescription", "Same as 'Translated World To View', but transforms to the camera position instead of the view position."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Camera View To Translated World Transform")), LOCTEXT("CameraToTranslatedWorldTransformDescription", "Same as 'View To Translated World', but transforms from the camera position instead of the view position."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("View To Clip Transform")), LOCTEXT("ViewToClipTransformDescription", "Transforms a view space position to clip space"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("View To ClipNoAA Transform")), LOCTEXT("ViewToClipNoAATransformDescription", "Transforms a view space position to clip space without the temporal AA jittering"));
	OutFunctions.Add(Sig);

	
	Sig = FNiagaraFunctionSignature();
	Sig.Name = ApplyPreViewTranslationToPosition;
	Sig.Description = LOCTEXT("ApplyPreViewTranslationToPositionDescription", "This function applies the pre view translation to a lwc positions such as Particles.Position. The resulting vector can be used with any of the 'Translated X to Y' transform matrices.");
	Sig.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;
	Sig.bMemberFunction = true;
	Sig.bSupportsCPU = false;
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Camera interface")));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")), LOCTEXT("ApplyPreViewTranslationToPositionInputDescription", "Position input to be transformed."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Translated Position")), LOCTEXT("ApplyPreViewTranslationToPositionOutputDescription", "Translated vector that can be used with the 'Translated X to Y' transforms matrices."));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = RemovePreViewTranslationFromPosition;
	Sig.Description = LOCTEXT("RemovePreViewTranslationFromPositionDescription", "This function removes the pre view translation from a vector to return a lwc position. Use the result of a 'X to Translated Y' transform matrix multiplication as input vector.");
	Sig.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;
	Sig.bMemberFunction = true;
	Sig.bSupportsCPU = false;
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Camera interface")));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Translated Position")), LOCTEXT("RemovePreViewTranslationFromPositionInputDescription", "Translated vector that is a result of a 'X to Translated Y' transforms matrix."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")), LOCTEXT("RemovePreViewTranslationFromPositionOutputDescription", "The transformed position."));
	OutFunctions.Add(Sig);


	Sig = FNiagaraFunctionSignature();
	Sig.Name = GetFieldOfViewName;
	Sig.Description = LOCTEXT("GetNiagaraFOVDescription", "This function returns the field of view angle (in degrees) for the active camera. For gpu particles this returns the x axis fov.");
	Sig.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;
	Sig.bMemberFunction = true;
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Camera interface")));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Field Of View Angle")), LOCTEXT("FieldOfViewAngleDescription", "Return the camera's field of view in degrees."));
	OutFunctions.Add(Sig);


	Sig = FNiagaraFunctionSignature();
	Sig.Name = GetCameraPropertiesName;
	Sig.Description = LOCTEXT("GetCameraPositionDescription", "This function returns the position of the currently active camera.");
	Sig.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;
	Sig.bMemberFunction = true;
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Camera interface")));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Camera Position World")), LOCTEXT("CameraPositionWorldDescription", "The camera position in world space."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Forward Vector World")), LOCTEXT("ForwardVectorWorldDescription", "The world space direction that the camera is pointing."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Up Vector World")), LOCTEXT("UpVectorWorldDescription", "The camera's up direction in world space."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Right Vector World")), LOCTEXT("RightVectorWorldDescription", "The camera's right direction in world space."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Viewport Size")), LOCTEXT("ViewportSizeDescription", "The viewport size in pixels (width, height)."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Near Clip Plane")), LOCTEXT("NearClipPlaneDescription", "The near clip plane distance."));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = QueryClosestName;
	Sig.Description = LOCTEXT("QueryClosestDescription", "This function checks the previously calculated distance of each particle and then returns true for the closest particles and false for the other ones.\nThis function needs to be paired with CalculateParticleDistancesCPU to work correctly.");
	Sig.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Camera interface")));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Max Valid Results")), LOCTEXT("MaxValidResultsDescription", "The max number of particles closest to the camera that this function should return true for.\nFor example, if there are 30 particles and Max Valid Results is set to 10, then this functions returns true for the 10 closest particles."));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Is Closest")), LOCTEXT("IsClosestDescription", "Returns true if the given particle ID is one of the closest n particles based on last frame's calculation."));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = CalculateDistancesName;
	Sig.Description = LOCTEXT("CalculateDistancesDescription", "This function compares the particle position against the camera position and stores the result to be queried in the next frame.\nThe results can then be queried with QueryClosestParticlesCPU.");
	Sig.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;
	Sig.bMemberFunction = true;
	Sig.bSupportsGPU = false;
	Sig.bRequiresExecPin = true;
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Camera interface")));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Particle Position World")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = GetTAAJitterName;
	Sig.Description = NSLOCTEXT("Niagara", "GetTAAJitterDescription", "This function returns the TAA jitter values of the currently active camera.");
	Sig.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;
	Sig.bMemberFunction = true;
	Sig.bSupportsCPU = false;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Camera interface")));
	Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Current TAA Jitter (clip)")));
	Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Previous TAA Jitter (clip)")));
	OutFunctions.Add(Sig);

	Sig = FNiagaraFunctionSignature();
	Sig.Name = GetSplitScreenName;
	Sig.Description = NSLOCTEXT("Niagara", "GetSplitScreenNameDescription", "This function returns the currently active splitscreen mode as ESplitScreenType value.");
	Sig.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;
	Sig.bMemberFunction = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Camera interface")));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition(StaticEnum<ESplitScreenType::Type>()), TEXT("Current Splitscreen Mode")), LOCTEXT("SplitscreenModeOutput", "The splitscreen mode as ESplitScreenType value. A value of 0 means there is no splitscreen active."));
	OutFunctions.Add(Sig);
}

bool UNiagaraDataInterfaceCamera::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if ((FunctionInfo.DefinitionName == GetViewPropertiesName) ||
		(FunctionInfo.DefinitionName == GetFieldOfViewName) ||
		(FunctionInfo.DefinitionName == GetClipSpaceTransformsName) ||
		(FunctionInfo.DefinitionName == GetViewSpaceTransformsName) ||
		(FunctionInfo.DefinitionName == GetCameraPropertiesName) ||
		(FunctionInfo.DefinitionName == ApplyPreViewTranslationToPosition) ||
		(FunctionInfo.DefinitionName == RemovePreViewTranslationFromPosition) ||
		(FunctionInfo.DefinitionName == GetSplitScreenName) ||
		(FunctionInfo.DefinitionName == GetTAAJitterName))
	{
		return true;
	}
	return false;
}

bool UNiagaraDataInterfaceCamera::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	bool bWasChanged = false;

	// Early out for version matching
	if (FunctionSignature.FunctionVersion == FNiagaraCameraDIFunctionVersion::LatestVersion)
	{
		return bWasChanged;
	}

	// Added a new output to the gpu properties
	if (FunctionSignature.FunctionVersion < FNiagaraCameraDIFunctionVersion::AddedPreViewTranslation && FunctionSignature.Name == GetViewPropertiesName && FunctionSignature.Outputs.Num() == 8)
	{
		FunctionSignature.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("PreViewTranslation")), LOCTEXT("PreViewTranslationDescription", "Returns the translation to apply for the various 'Translated XX to XX' transforms."));
		bWasChanged = true;
	}
	
	// Added a few new outputs for GPU properties
	if (FunctionSignature.FunctionVersion < FNiagaraCameraDIFunctionVersion::AddedBufferSizeInvSizeForGPU && FunctionSignature.Name == GetViewPropertiesName)
	{
		ensure(FunctionSignature.Outputs.Num() == 9);
		FunctionSignature.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Buffer Size And Inverse Size")), LOCTEXT("BufferSizeAndInverseSizeDescription", "Returns a vec4 - the x and y values are width and height of the buffer; the z and w values are the inverse width and height."));
		FunctionSignature.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Viewport Offset")), LOCTEXT("ViewportOffsetDescription", "Returns the viewport offset in the buffer."));

		bWasChanged = true;
	}

	// LWCConversion + AddGPUNearPlane + AddGPUFOVData
	if (FunctionSignature.FunctionVersion < FNiagaraCameraDIFunctionVersion::LatestVersion)
	{
		TArray<FNiagaraFunctionSignature> AllFunctions;
		GetFunctionsInternal(AllFunctions);
		for (const FNiagaraFunctionSignature& Sig : AllFunctions)
		{
			if (FunctionSignature.Name == Sig.Name)
			{
				FunctionSignature = Sig;
				return true;
			}
		}
	}

	// Set latest version
    FunctionSignature.FunctionVersion = FNiagaraCameraDIFunctionVersion::LatestVersion;

	return bWasChanged;
}

bool UNiagaraDataInterfaceCamera::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}
	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	InVisitor->UpdateShaderParameters<FShaderParameters>();
	return true;
}

void UNiagaraDataInterfaceCamera::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}
#endif

void UNiagaraDataInterfaceCamera::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceCamera::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyCameraQuery& DataInterfaceProxy = Context.GetProxy<FNiagaraDataInterfaceProxyCameraQuery>();
	FCameraDataInterface_InstanceDataRT& InstanceData = DataInterfaceProxy.SystemInstancesToInstanceData_RT.FindChecked(Context.GetSystemInstanceID());
	
	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->SplitscreenMode = static_cast<int32>(InstanceData.SplitscreenMode);
}

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceCamera, GetClosestParticles);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceCamera, CalculateParticleDistances);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceCamera, GetCameraFOV);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceCamera, GetCameraProperties);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceCamera, GetGetSplitScreenInfo);
void UNiagaraDataInterfaceCamera::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == GetFieldOfViewName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceCamera, GetCameraFOV)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == CalculateDistancesName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceCamera, CalculateParticleDistances)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == QueryClosestName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceCamera, GetClosestParticles)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetCameraPropertiesName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceCamera, GetCameraProperties)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetSplitScreenName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceCamera, GetGetSplitScreenInfo)::Bind(this, OutFunc);
	}
	else
	{
		UE_LOGF(LogNiagara, Display, "Could not find data interface external function in %ls. Received Name: %ls", *GetPathNameSafe(this), *BindingInfo.Name.ToString());
	}
}

void UNiagaraDataInterfaceCamera::GetCameraFOV(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FCameraDataInterface_InstanceData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutFov(Context);

	float Fov = InstData.Get()->CameraFOV;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*OutFov.GetDestAndAdvance() = Fov;
	}
}

void UNiagaraDataInterfaceCamera::GetCameraProperties(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FCameraDataInterface_InstanceData> InstData(Context);

	VectorVM::FExternalFuncRegisterHandler<float> CamPosX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> CamPosY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> CamPosZ(Context);

	VectorVM::FExternalFuncRegisterHandler<float> CamForwardX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> CamForwardY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> CamForwardZ(Context);

	VectorVM::FExternalFuncRegisterHandler<float> CamUpX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> CamUpY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> CamUpZ(Context);

	VectorVM::FExternalFuncRegisterHandler<float> CamRightX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> CamRightY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> CamRightZ(Context);

	VectorVM::FExternalFuncRegisterHandler<float> ViewportX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> ViewportY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> NearClip(Context);

	FCameraDataInterface_InstanceData* CamData = InstData.Get();
	const float XPos = CamData->CameraLocation.X;
	const float YPos = CamData->CameraLocation.Y;
	const float ZPos = CamData->CameraLocation.Z;

	FRotationMatrix RotationMatrix(CamData->CameraRotation);
	const FVector3f Forward = FVector3f(RotationMatrix.GetScaledAxis(EAxis::X));
	const FVector3f Up = FVector3f(RotationMatrix.GetScaledAxis(EAxis::Z));
	const FVector3f Right = FVector3f(RotationMatrix.GetScaledAxis(EAxis::Y));
	const float VPX = CamData->ViewportSizeX;
	const float VPY = CamData->ViewportSizeY;
	const float NCP = CamData->NearClipPlane;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*CamPosX.GetDestAndAdvance() = XPos;
		*CamPosY.GetDestAndAdvance() = YPos;
		*CamPosZ.GetDestAndAdvance() = ZPos;

		*CamForwardX.GetDestAndAdvance() = Forward.X;
		*CamForwardY.GetDestAndAdvance() = Forward.Y;
		*CamForwardZ.GetDestAndAdvance() = Forward.Z;

		*CamUpX.GetDestAndAdvance() = Up.X;
		*CamUpY.GetDestAndAdvance() = Up.Y;
		*CamUpZ.GetDestAndAdvance() = Up.Z;

		*CamRightX.GetDestAndAdvance() = Right.X;
		*CamRightY.GetDestAndAdvance() = Right.Y;
		*CamRightZ.GetDestAndAdvance() = Right.Z;

		*ViewportX.GetDestAndAdvance() = VPX;
		*ViewportY.GetDestAndAdvance() = VPY;
		*NearClip.GetDestAndAdvance() = NCP;
	}
}

void UNiagaraDataInterfaceCamera::GetGetSplitScreenInfo(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FCameraDataInterface_InstanceData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<ESplitScreenType::Type> OutMode(Context);

	ESplitScreenType::Type SplitscreenMode = InstData.Get()->SplitscreenMode;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*OutMode.GetDestAndAdvance() = SplitscreenMode;
	}
}

void UNiagaraDataInterfaceCamera::GetClosestParticles(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FCameraDataInterface_InstanceData> InstData(Context);

	FNDIInputParam<FNiagaraID> ParticleIDParam(Context);
	FNDIInputParam<int32> CountParam(Context);
	FNDIOutputParam<FNiagaraBool> ResultOutParam(Context);

	int32 Count = Context.GetNumInstances() > 0 ? CountParam.GetAndAdvance() : 0;
	if (Count == 0 || InstData->ParticlesSortedByDistance.Num() == 0)
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			ResultOutParam.SetAndAdvance(false);
		}
		return;
	}

	// grab the IDs of the closest n particles
	TSet<FNiagaraID> ClosestParticleIDs;
	for (int32 i = 0; i < Count; ++i)
	{
		ClosestParticleIDs.Add(InstData->ParticlesSortedByDistance[i].ParticleID);
	}

	// Assign each particles their result
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		FNiagaraID ParticleID = ParticleIDParam.GetAndAdvance();
		ResultOutParam.SetAndAdvance(ClosestParticleIDs.Contains(ParticleID));
	}
}

void UNiagaraDataInterfaceCamera::CalculateParticleDistances(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FCameraDataInterface_InstanceData> InstData(Context);

	FNDIInputParam<FNiagaraID> IDParam(Context);
	FNDIInputParam<FNiagaraPosition> ParticlePosParam(Context);

	FNiagaraPosition CameraPos = InstData->CameraLocation;
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		FDistanceData DistanceData;
		FNiagaraPosition ParticlePos = ParticlePosParam.GetAndAdvance();
		DistanceData.ParticleID = IDParam.GetAndAdvance();
		DistanceData.DistanceSquared = (ParticlePos - CameraPos).SizeSquared();
		InstData->DistanceSortQueue.Enqueue(DistanceData);
	}
}

ETickingGroup UNiagaraDataInterfaceCamera::CalculateTickGroup(const void* PerInstanceData) const
{
	return IsUsedWithCPUScript() && bRequireCurrentFrameData ? ETickingGroup::TG_PostUpdateWork : NiagaraFirstTickGroup;
}

#if WITH_EDITOR	
void UNiagaraDataInterfaceCamera::GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component,
	TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& Warnings,
	TArray<FNiagaraDataInterfaceFeedback>& Info)
{
	if (Asset == nullptr)
	{
		return;
	}
}
#endif

bool UNiagaraDataInterfaceCamera::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceCamera* OtherTyped = CastChecked<UNiagaraDataInterfaceCamera>(Destination);
	OtherTyped->PlayerControllerIndex = PlayerControllerIndex;
	OtherTyped->bRequireCurrentFrameData = bRequireCurrentFrameData;
	return true;
}

bool UNiagaraDataInterfaceCamera::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceCamera* OtherTyped = CastChecked<const UNiagaraDataInterfaceCamera>(Other);
	return OtherTyped->PlayerControllerIndex == PlayerControllerIndex &&
		OtherTyped->bRequireCurrentFrameData == bRequireCurrentFrameData ;
}

#undef LOCTEXT_NAMESPACE

