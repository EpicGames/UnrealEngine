// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceCollisionQuery.h"

#include "FXRenderingUtils.h"
#include "GlobalDistanceFieldParameters.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraDistanceFieldHelper.h"
#include "NiagaraEmitter.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGpuComputeDispatch.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystem.h"
#include "NiagaraTypes.h"
#include "NiagaraSystemInstance.h"
#include "Shader.h"
#include "ShaderCore.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceCollisionQuery)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceCollisionQuery"

namespace NDICollisionQueryLocal
{
	static const TCHAR* CommonShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceCollisionQuery.ush");
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceCollisionQueryTemplate.ush");

	static const FName SceneDepthName(TEXT("QuerySceneDepthGPU"));
	static const FName ScenePartialDepthName(TEXT("QueryScenePartialDepthGPU"));
	static const FName CustomDepthName(TEXT("QueryCustomDepthGPU"));
	static const FName DistanceFieldName(TEXT("QueryMeshDistanceFieldGPU"));
	static const FName SyncTraceName(TEXT("PerformCollisionQuerySyncCPU"));
	static const FName AsyncTraceName(TEXT("PerformCollisionQueryAsyncCPU"));

	// DEPRECATE_BEGIN
	static const FName IssueAsyncRayTraceName(TEXT("IssueAsyncRayTraceGpu"));
	static const FName CreateAsyncRayTraceName(TEXT("CreateAsyncRayTraceGpu"));
	static const FName ReserveAsyncRayTraceName(TEXT("ReserveAsyncRayTraceGpu"));
	static const FName ReadAsyncRayTraceName(TEXT("ReadAsyncRayTraceGpu"));
	// DEPRECATE_END

	static const FString SystemLWCTileName(TEXT("SystemLWCTile_"));
}

FCriticalSection UNiagaraDataInterfaceCollisionQuery::CriticalSection;

struct FNiagaraCollisionDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		AddedTraceSkip = 1,
		AddedCustomDepthCollision = 2,
		ReturnCollisionMaterialIdx = 3,
		LargeWorldCoordinates = 4,
		DeprecatedAsyncGpuTrace = 5,
		AddedTraceComplex = 6,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

struct FNiagaraDataIntefaceProxyCollisionQuery : public FNiagaraDataInterfaceProxy
{
	// There's nothing in this proxy. It just reads from scene textures.

	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return 0;
	}
};

UNiagaraDataInterfaceCollisionQuery::UNiagaraDataInterfaceCollisionQuery(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TraceChannelEnum = StaticEnum<ECollisionChannel>();
	Proxy.Reset(new FNiagaraDataIntefaceProxyCollisionQuery());
}

bool UNiagaraDataInterfaceCollisionQuery::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance)
{
	CQDIPerInstanceData *PIData = new (PerInstanceData) CQDIPerInstanceData;
	PIData->SystemInstance = InSystemInstance;
	if (InSystemInstance)
	{
		TStatId CollisionStatId = SCENE_QUERY_STAT_ONLY(NiagaraCollision);

		// If we don't have stats enabled can we pull a valid stat from the asset?
		// This will allow us to trace back to the asset when profiling
		#if !STATS && (ENABLE_STATNAMEDEVENTS && ENABLE_STATNAMEDEVENTS_UOBJECT)
			TStatId AssetStatId = static_cast<const UObjectBaseUtility*>(InSystemInstance->GetSystem())->GetStatID();
			if (AssetStatId.IsValidStat())
			{
				CollisionStatId = AssetStatId;
			}
		#endif
		PIData->CollisionBatch.Init(InSystemInstance->GetId(), InSystemInstance->GetWorld(), CollisionStatId);
	}

	return true;
}

void UNiagaraDataInterfaceCollisionQuery::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance)
{
	CQDIPerInstanceData* InstData = (CQDIPerInstanceData*)PerInstanceData;
	InstData->~CQDIPerInstanceData();
}

void UNiagaraDataInterfaceCollisionQuery::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(TraceChannelEnum), Flags);
	}
}

void UNiagaraDataInterfaceCollisionQuery::GetAssetTagsForContext(const UObject* InAsset, FGuid AssetVersion, const TArray<const UNiagaraDataInterface*>& InProperties, TMap<FName, uint32>& NumericKeys, TMap<FName, FString>& StringKeys) const
{
#if WITH_EDITOR
	const UNiagaraSystem* System = Cast<UNiagaraSystem>(InAsset);
	const UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(InAsset);

	// We need to check if the DI is used to access collisions in a cpu context so that artists can surface potential perf problems
	// through the content browser.

	TArray<const UNiagaraScript*> Scripts;
	if (System)
	{
		Scripts.Add(System->GetSystemSpawnScript());
		Scripts.Add(System->GetSystemUpdateScript());
		for (auto&& EmitterHandle : System->GetEmitterHandles())
		{
			FVersionedNiagaraEmitterData* HandleEmitter = EmitterHandle.GetEmitterData();
			if (HandleEmitter)
			{
				if (HandleEmitter->SimTarget == ENiagaraSimTarget::GPUComputeSim)
				{
					// Ignore gpu emitters
					continue;
				}
				TArray<UNiagaraScript*> OutScripts;
				HandleEmitter->GetScripts(OutScripts, false);
				Scripts.Append(OutScripts);
			}
		}
	}
	if (Emitter)
	{
		const FVersionedNiagaraEmitterData* EmitterData = Emitter->GetEmitterData(AssetVersion);
		if (EmitterData && EmitterData->SimTarget != ENiagaraSimTarget::GPUComputeSim)
		{
			TArray<UNiagaraScript*> OutScripts;
			EmitterData->GetScripts(OutScripts, false);
			Scripts.Append(OutScripts);
		}
	}

	// Check if any CPU script uses Collsion query CPU functions
	//TODO: This is the same as in the skel mesh DI for GetFeedback, it doesn't guarantee that the DI used by these functions are THIS DI.
	// Has a possibility of false positives
	bool bHaCPUQueriesWarning = [this, &Scripts]()
	{
		for (const auto Script : Scripts)
		{
			if (Script)
			{
				for (const auto& Info : Script->GetVMExecutableData().DataInterfaceInfo)
				{
					if (Info.MatchesClass(GetClass()))
					{
						for (const auto& Func : Info.RegisteredFunctions)
						{
							if (Func.Name == NDICollisionQueryLocal::SyncTraceName || Func.Name == NDICollisionQueryLocal::AsyncTraceName)
							{
								return true;
							}
						}
					}
				}
			}
		}
		return false;
	}();

	// Note that in order for these tags to be registered, we always have to put them in place for the CDO of the object, but 
	// for readability's sake, we leave them out of non-CDO assets.
	if (bHaCPUQueriesWarning || (InAsset && InAsset->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject)))
	{
		StringKeys.Add(FName("CPUCollision")) = bHaCPUQueriesWarning ? TEXT("True") : TEXT("False");

	}

#endif
	
	// Make sure and get the base implementation tags
	Super::GetAssetTagsForContext(InAsset, AssetVersion, InProperties, NumericKeys, StringKeys);
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceCollisionQuery::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	{
		FNiagaraFunctionSignature SigDepth;
		SigDepth.Name = NDICollisionQueryLocal::SceneDepthName;
		SigDepth.bMemberFunction = true;
		SigDepth.bSupportsCPU = false;
		SigDepth.FunctionVersion = FNiagaraCollisionDIFunctionVersion::LatestVersion;
		SigDepth.Description = LOCTEXT("SceneDepthSignatureDescription", "Projects a given world position to view space and then queries the depth buffer with that position.");
		const FText DepthSamplePosWorldDescription = LOCTEXT("DepthSamplePosWorldDescription", "The world position where the depth should be queried. The position gets automatically transformed to view space to query the depth buffer.");
		const FText SceneDepthDescription = LOCTEXT("SceneDepthDescription", "If the query was successful this returns the scene depth, otherwise -1.");
		const FText CameraPosWorldDescription = LOCTEXT("CameraPosWorldDescription", "Returns the current camera position in world space.");
		const FText IsInsideViewDescription = LOCTEXT("IsInsideViewDescription", "Returns true if the query position could be projected to valid screen coordinates.");
		const FText SamplePosWorldDescription = LOCTEXT("SamplePosWorldDescription", "If the query was successful, this returns the world position that was recalculated from the scene depth. Otherwise returns (0, 0, 0).");
		const FText SampleWorldNormalDescription = LOCTEXT("SampleWorldNormalDescription", "If the query was successful, this returns the world normal at the sample point. Otherwise returns (0, 0, 1).");

		SigDepth.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("CollisionQuery")));
		SigDepth.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("DepthSamplePosWorld")), DepthSamplePosWorldDescription);
		SigDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("SceneDepth")), SceneDepthDescription);
		SigDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("CameraPosWorld")), CameraPosWorldDescription);
		SigDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsInsideView")), IsInsideViewDescription);
		SigDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("SamplePosWorld")), SamplePosWorldDescription);
		SigDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("SampleWorldNormal")), SampleWorldNormalDescription);
		OutFunctions.Add(SigDepth);


		FNiagaraFunctionSignature SigPartialDepth;
		SigPartialDepth.Name = NDICollisionQueryLocal::ScenePartialDepthName;
		SigPartialDepth.bMemberFunction = true;
		SigPartialDepth.bSupportsCPU = false;
		SigPartialDepth.MiscUsageBitMask = (uint16)ENiagaraScriptMiscUsageMask::UsesPartialDepthCollisionQuery;
		SigPartialDepth.FunctionVersion = FNiagaraCollisionDIFunctionVersion::LatestVersion;
		SigPartialDepth.Description = LOCTEXT("ScenePartialDepthSignatureDescription", "Projects a given world position to view space and then queries the partial depth buffer (opaque emitter using this function are not in this depth buffer) with that position.");
		SigPartialDepth.ExperimentalMessage = LOCTEXT("ScenePartialDepthSignatureEpxMessage", "Careful: using this node involve a scene depth copy that might slow down the base pass on some platform.");
		SigPartialDepth.bExperimental = true;
		SigPartialDepth.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("CollisionQuery")));
		SigPartialDepth.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("DepthSamplePosWorld")), DepthSamplePosWorldDescription);
		SigPartialDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("SceneDepth")), SceneDepthDescription);
		SigPartialDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("CameraPosWorld")), CameraPosWorldDescription);
		SigPartialDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsInsideView")), IsInsideViewDescription);
		SigPartialDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("SamplePosWorld")), SamplePosWorldDescription);
		SigPartialDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("SampleWorldNormal")), SampleWorldNormalDescription);
		OutFunctions.Add(SigPartialDepth);

		
		FNiagaraFunctionSignature SigCustomDepth;
		SigCustomDepth.Name = NDICollisionQueryLocal::CustomDepthName;
		SigCustomDepth.bMemberFunction = true;
		SigCustomDepth.bSupportsCPU = false;
		SigCustomDepth.FunctionVersion = FNiagaraCollisionDIFunctionVersion::LatestVersion;
		SigCustomDepth.Description = LOCTEXT("CustomDepthDescription", "Projects a given world position to view space and then queries the custom depth buffer with that position.");
		SigCustomDepth.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("CollisionQuery")));
		SigCustomDepth.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("DepthSamplePosWorld")), DepthSamplePosWorldDescription);
		SigCustomDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("SceneDepth")), SceneDepthDescription);
		SigCustomDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("CameraPosWorld")), CameraPosWorldDescription);
		SigCustomDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsInsideView")), IsInsideViewDescription);
		SigCustomDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("SamplePosWorld")), SamplePosWorldDescription);
		SigCustomDepth.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("SampleWorldNormal")), SampleWorldNormalDescription);
		OutFunctions.Add(SigCustomDepth);
	}

	{
		FNiagaraFunctionSignature SigMeshField;
		SigMeshField.Name = NDICollisionQueryLocal::DistanceFieldName;
		SigMeshField.bMemberFunction = true;
		SigMeshField.bSupportsCPU = false;
		SigMeshField.FunctionVersion = FNiagaraCollisionDIFunctionVersion::LatestVersion;
		SigMeshField.Description = LOCTEXT("DistanceFieldDescription", "Queries the global distance field for a given world position.\nPlease note that the distance field resolution gets lower the farther away the queried position is from the camera.");
		const FText FieldSamplePosWorldDescription = LOCTEXT("FieldSamplePosWorldDescription", "The world position where the distance field should be queried.");
		const FText DistanceToNearestSurfaceDescription = LOCTEXT("DistanceToNearestSurfaceDescription", "If the query was successful this returns the distance to the nearest surface, otherwise returns 0.");
		const FText FieldGradientDescription = LOCTEXT("FieldGradientDescription", "If the query was successful this returns the non-normalized direction to the nearest surface, otherwise returns (0, 0, 0).");
		const FText IsDistanceFieldValidDescription = LOCTEXT("IsDistanceFieldValidDescription", "Returns true if the global distance field is available and there was a valid value retrieved for the given sample position.");
	
		SigMeshField.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("CollisionQuery")));
		SigMeshField.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("FieldSamplePosWorld")), FieldSamplePosWorldDescription);
		SigMeshField.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("DistanceToNearestSurface")), DistanceToNearestSurfaceDescription);
		SigMeshField.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("FieldGradient")), FieldGradientDescription);
		SigMeshField.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsDistanceFieldValid")), IsDistanceFieldValidDescription);
		OutFunctions.Add(SigMeshField);
	}

	
	const FText TraceStartWorldDescription = LOCTEXT("TraceStartWorldDescription", "Ray starting point in world space");
	const FText TraceEndWorldDescription = LOCTEXT("TraceEndWorldDescription", "Ray end point in world space");

	// DEPRECATE_BEGIN
	{
		const FText AsyncTraceChannelDescription = LOCTEXT("AsyncTraceChannelDescription", "Currently unused, will represent the trace channels for which geometry the trace should test against");
		const FText QueryIDDescription = LOCTEXT("QueryIDDescription", "Unique (for this frame) index of the query being enqueued (used in subsequent frames to retrieve results).  Must be less than MaxRayTraceCount");
		const FText IsQueryValidDescription = LOCTEXT("IsQueryValidDescription", "Returns true if the query was enqueued");
		
		FNiagaraFunctionSignature& IssueRayTrace = OutFunctions.AddDefaulted_GetRef();
		IssueRayTrace.Name = NDICollisionQueryLocal::IssueAsyncRayTraceName;
		IssueRayTrace.bRequiresExecPin = true;
		IssueRayTrace.bMemberFunction = true;
		IssueRayTrace.bSupportsCPU = false;
		IssueRayTrace.bSoftDeprecatedFunction = true;
		IssueRayTrace.FunctionVersion = FNiagaraCollisionDIFunctionVersion::LatestVersion;
		IssueRayTrace.Description = LOCTEXT("IssueAsync_RayTraceDescription", "Enqueues a GPU raytrace with the result being available the following frame");
		IssueRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("CollisionQuery")));
		IssueRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("QueryID")), QueryIDDescription);
		IssueRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceStartWorld")), TraceStartWorldDescription);
		IssueRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceEndWorld")), TraceEndWorldDescription);
		IssueRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("TraceChannel")), AsyncTraceChannelDescription);
		IssueRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsQueryValid")), IsQueryValidDescription);
		
		FNiagaraFunctionSignature& CreateRayTrace = OutFunctions.AddDefaulted_GetRef();
		CreateRayTrace.Name = NDICollisionQueryLocal::CreateAsyncRayTraceName;
		CreateRayTrace.bRequiresExecPin = true;
		CreateRayTrace.bMemberFunction = true;
		CreateRayTrace.bSupportsCPU = false;
		CreateRayTrace.bSoftDeprecatedFunction = true;
		CreateRayTrace.FunctionVersion = FNiagaraCollisionDIFunctionVersion::LatestVersion;
		CreateRayTrace.Description = LOCTEXT("CreateAsync_RayTraceDescription", "Creates a GPU raytrace with the result being available the following frame (index is returned)");
		CreateRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("CollisionQuery")));
		CreateRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceStartWorld")), TraceStartWorldDescription);
		CreateRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceEndWorld")), TraceEndWorldDescription);
		CreateRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("TraceChannel")), AsyncTraceChannelDescription);
		CreateRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("QueryID")), QueryIDDescription);
		CreateRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsQueryValid")), IsQueryValidDescription);
	}

	{
		FNiagaraFunctionSignature& ReserveRayTrace = OutFunctions.AddDefaulted_GetRef();
		ReserveRayTrace.Name = NDICollisionQueryLocal::ReserveAsyncRayTraceName;
		ReserveRayTrace.bRequiresExecPin = true;
		ReserveRayTrace.bMemberFunction = true;
		ReserveRayTrace.bSupportsCPU = false;
		ReserveRayTrace.bSoftDeprecatedFunction = true;
		ReserveRayTrace.FunctionVersion = FNiagaraCollisionDIFunctionVersion::LatestVersion;
		ReserveRayTrace.Description = LOCTEXT("ReserveAsync_RayTraceDescription", "Reserves a number of ray trace request slots");
		ReserveRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("CollisionQuery")));
		ReserveRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("TraceCount")), LOCTEXT("ReserveAsync_QueryIDDescription", "Number of async raytrace requests to be reserved"));
		ReserveRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("FirstQueryID")), LOCTEXT("ReserveAsync_TraceChannelDescription", "The first index in the block reserved through this call"));
		ReserveRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsQueryValid")), LOCTEXT("ReserveAsync_IsQueryValidDescription", "Returns true if the requested indices were reserved"));
	}

	{
		FNiagaraFunctionSignature& ReadRayTrace = OutFunctions.AddDefaulted_GetRef();
		ReadRayTrace.Name = NDICollisionQueryLocal::ReadAsyncRayTraceName;
		ReadRayTrace.bMemberFunction = true;
		ReadRayTrace.bSupportsCPU = false;
		ReadRayTrace.bSoftDeprecatedFunction = true;
		ReadRayTrace.FunctionVersion = FNiagaraCollisionDIFunctionVersion::LatestVersion;
		ReadRayTrace.Description = LOCTEXT("ReadAsync_RayTraceDescription", "Reads the results of a previously enqueued GPU ray trace");
		ReadRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("CollisionQuery")));
		ReadRayTrace.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PreviousFrameQueryID")), LOCTEXT("ReadAsync_PreviousFrameQueryIDDescription", "The index of the results being retrieved"));
		ReadRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("CollisionValid")), LOCTEXT("ReadAsync_CollisionValidDescription", "Returns true if a Hit was encountered"));
		ReadRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CollisionDistance")), LOCTEXT("ReadAsync_CollisionDistanceDescription", "The distance in world space from the ray starting point to the intersection"));
		ReadRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("CollisionPosWorld")), LOCTEXT("ReadAsync_CollisionPosWorldDescription", "The point in world space where the intersection occured"));
		ReadRayTrace.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CollisionNormal")), LOCTEXT("ReadAsync_CollisionNormalDescription", "The surface normal of the world geometry at the point of intersection"));
	}
	// DEPRECATE_END

	{
		const FText TraceChannelDescription = LOCTEXT("TraceChannelDescription", "The trace channel to collide against. Trace channels can be configured in the project settings.");
		const FText SkipTraceDescription = LOCTEXT("SkipTraceDescription", "If true then the trace will be skipped completely.\nThis can be used as a performance optimization, as branch nodes in the graph still execute every path.");
		const FText TraceComplexDescription = LOCTEXT("TraceComplexDescription", "If true then the trace will hit complex collision.");
		const FText CollisionValidDescription = LOCTEXT("CollisionValidDescription", "Returns true if the trace was not skipped and the trace was blocked by some world geometry.");
		const FText IsTraceInsideMeshDescription = LOCTEXT("IsTraceInsideMeshDescription", "If true then the trace started in penetration, i.e. with an initial blocking overlap.");
		const FText CollisionPosWorldDescription = LOCTEXT("CollisionPosWorldDescription", "If the collision is valid, this returns the location of the blocking hit.");
		const FText CollisionNormalDescription = LOCTEXT("CollisionNormalDescription", "If the collision is valid, this returns the normal at the position of the blocking hit.");
		const FText CollisionMaterialFrictionDescription = LOCTEXT("CollisionMaterialFrictionDescription", "Friction value of surface, controls how easily things can slide on this surface (0 is frictionless, higher values increase the amount of friction).");
		const FText CollisionMaterialRestitutionDescription = LOCTEXT("CollisionMaterialRestitutionDescription", "Restitution or 'bounciness' of this surface, between 0 (no bounce) and 1 (outgoing velocity is same as incoming)");
		const FText CollisionMaterialIndexDescription = LOCTEXT("CollisionMaterialIndexDescription", "Returns the index of the surface as defined in the ProjectSettings/Physics/PhysicalSurface section");
		
		FNiagaraFunctionSignature SigCpuSync;
		SigCpuSync.Name = NDICollisionQueryLocal::SyncTraceName;
		SigCpuSync.bMemberFunction = true;
		SigCpuSync.bSupportsGPU = false;
		SigCpuSync.FunctionVersion = FNiagaraCollisionDIFunctionVersion::LatestVersion;
		SigCpuSync.Description = LOCTEXT("SigCpuSyncDescription", "Traces a ray against the world using a specific channel and return the first blocking hit.");
		SigCpuSync.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("CollisionQuery")));
		SigCpuSync.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceStartWorld")), TraceStartWorldDescription);
		SigCpuSync.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceEndWorld")), TraceEndWorldDescription);
		SigCpuSync.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(TraceChannelEnum), TEXT("TraceChannel")), TraceChannelDescription);
		SigCpuSync.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("SkipTrace")), SkipTraceDescription);
		SigCpuSync.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("TraceComplex")), TraceComplexDescription);
		SigCpuSync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("CollisionValid")), CollisionValidDescription);
		SigCpuSync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsTraceInsideMesh")), IsTraceInsideMeshDescription);
		SigCpuSync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("CollisionPosWorld")), CollisionPosWorldDescription);
		SigCpuSync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CollisionNormal")), CollisionNormalDescription);
		SigCpuSync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CollisionMaterialFriction")), CollisionMaterialFrictionDescription);
		SigCpuSync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CollisionMaterialRestitution")), CollisionMaterialRestitutionDescription);
		SigCpuSync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("CollisionMaterialIndex")), CollisionMaterialIndexDescription);
		OutFunctions.Add(SigCpuSync);
		
		FNiagaraFunctionSignature SigCpuAsync;
		SigCpuAsync.Name = NDICollisionQueryLocal::AsyncTraceName;
		SigCpuAsync.bMemberFunction = true;
		SigCpuAsync.bSupportsGPU = false;
		SigCpuAsync.FunctionVersion = FNiagaraCollisionDIFunctionVersion::LatestVersion;
		SigCpuAsync.Description = LOCTEXT("SigCpuAsyncDescription", "Traces a ray against the world using a specific channel and return the first blocking hit the next frame.\nNote that this is the ASYNC version of the trace function, meaning it will not returns the result right away, but with one frame latency.");
		SigCpuAsync.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("CollisionQuery")));
		SigCpuAsync.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PreviousFrameQueryID")), LOCTEXT("TraceAsync_PreviousFrameQueryIDDescription", "The query ID returned from the last frame's async trace call.\nRegardless if it is a valid ID or not this function call with issue a new async line trace, but it will only return results with a valid ID."));
		SigCpuAsync.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceStartWorld")), TraceStartWorldDescription);
		SigCpuAsync.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("TraceEndWorld")), TraceEndWorldDescription);
		SigCpuAsync.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(TraceChannelEnum), TEXT("TraceChannel")), TraceChannelDescription);
		SigCpuAsync.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("SkipTrace")), SkipTraceDescription);
		SigCpuAsync.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("TraceComplex")), TraceComplexDescription);
		SigCpuAsync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NextFrameQueryID")), LOCTEXT("TraceAsync_NextFrameQueryIDDescription", "The query ID to save and use as input to this function in the next frame."));
		SigCpuAsync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("CollisionValid")), CollisionValidDescription);
		SigCpuAsync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsTraceInsideMesh")), IsTraceInsideMeshDescription);
		SigCpuAsync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("CollisionPosWorld")), CollisionPosWorldDescription);
		SigCpuAsync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CollisionNormal")), CollisionNormalDescription);
		SigCpuAsync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CollisionMaterialFriction")), CollisionMaterialFrictionDescription);
		SigCpuAsync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CollisionMaterialRestitution")), CollisionMaterialRestitutionDescription);
		SigCpuAsync.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("CollisionMaterialIndex")), CollisionMaterialIndexDescription);
		OutFunctions.Add(SigCpuAsync);
	}
}

// build the shader function HLSL; function name is passed in, as it's defined per-DI; that way, configuration could change
// the HLSL in the spirit of a static switch
// TODO: need a way to identify each specific function here
bool UNiagaraDataInterfaceCollisionQuery::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if ( (FunctionInfo.DefinitionName == NDICollisionQueryLocal::SceneDepthName) ||
		 (FunctionInfo.DefinitionName == NDICollisionQueryLocal::ScenePartialDepthName) ||
		 (FunctionInfo.DefinitionName == NDICollisionQueryLocal::CustomDepthName) ||
		 (FunctionInfo.DefinitionName == NDICollisionQueryLocal::DistanceFieldName) ||
		// DEPRECATE_BEGIN
		 (FunctionInfo.DefinitionName == NDICollisionQueryLocal::IssueAsyncRayTraceName) ||
		 (FunctionInfo.DefinitionName == NDICollisionQueryLocal::CreateAsyncRayTraceName) ||
		 (FunctionInfo.DefinitionName == NDICollisionQueryLocal::ReserveAsyncRayTraceName) ||
		 (FunctionInfo.DefinitionName == NDICollisionQueryLocal::ReadAsyncRayTraceName) )
		// DEPRECATE_END
	{
		return true;
	}

	return false;
}

bool UNiagaraDataInterfaceCollisionQuery::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	// always upgrade to the latest version
	if (FunctionSignature.FunctionVersion < FNiagaraCollisionDIFunctionVersion::LatestVersion)
	{
		TArray<FNiagaraFunctionSignature> AllFunctions;
		GetFunctionsInternal(AllFunctions);
		for (const FNiagaraFunctionSignature& Sig : AllFunctions)
		{
			if (FunctionSignature.Name == Sig.Name)
			{
				FunctionSignature = Sig;
				return true;
			}
		}
	}

	return false;
}
#endif

bool IsDistanceFieldEnabled()
{
	static const auto* CVarGenerateMeshDistanceFields = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GenerateMeshDistanceFields"));
	return CVarGenerateMeshDistanceFields != nullptr && CVarGenerateMeshDistanceFields->GetValueOnAnyThread() > 0;
}

#if WITH_EDITOR
void UNiagaraDataInterfaceCollisionQuery::ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors)
{
	if (Function.Name == NDICollisionQueryLocal::DistanceFieldName)
	{
		if (!IsDistanceFieldEnabled())
		{
			OutValidationErrors.Add(NSLOCTEXT("NiagaraDataInterfaceCollisionQuery", "NiagaraDistanceFieldNotEnabledMsg", "The mesh distance field generation is currently not enabled, please check the project settings.\nNiagara cannot query the distance field otherwise."));
		}
	}
}
#endif

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceCollisionQuery::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDICollisionQueryLocal::TemplateShaderFile, TemplateArgs);
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceCollisionQuery, PerformQuerySyncCPU);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceCollisionQuery, PerformQueryAsyncCPU);

void UNiagaraDataInterfaceCollisionQuery::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == NDICollisionQueryLocal::SyncTraceName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceCollisionQuery, PerformQuerySyncCPU)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == NDICollisionQueryLocal::AsyncTraceName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceCollisionQuery, PerformQueryAsyncCPU)::Bind(this, OutFunc);
	}
	else
	{
		UE_LOGF(LogNiagara, Display, "Could not find data interface external function in %ls. %ls\n",
			*GetPathNameSafe(this), *BindingInfo.Name.ToString());
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceCollisionQuery::GetCommonHLSL(FString& OutHlsl)
{
	OutHlsl.Appendf(TEXT("#include \"%s\"\n"), NDICollisionQueryLocal::CommonShaderFile);
}

bool UNiagaraDataInterfaceCollisionQuery::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}

	InVisitor->UpdatePOD(TEXT("NiagaraCollisionDI_DistanceField"), IsDistanceFieldEnabled());
	InVisitor->UpdateShaderFile(NDICollisionQueryLocal::CommonShaderFile);
	InVisitor->UpdateShaderFile(NDICollisionQueryLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<FGlobalDistanceFieldParameters2>();

	return true;
}

#endif

void UNiagaraDataInterfaceCollisionQuery::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddIncludedStruct<FGlobalDistanceFieldParameters2>();
}

void UNiagaraDataInterfaceCollisionQuery::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FGlobalDistanceFieldParameters2* ShaderGDFParameters = Context.GetParameterIncludedStruct<FGlobalDistanceFieldParameters2>();
	if (Context.IsStructBound(ShaderGDFParameters))
	{
		FNiagaraDistanceFieldHelper::SetGlobalDistanceFieldParameters(Context.GetComputeDispatchInterface().GetGlobalDistanceFieldData(), *ShaderGDFParameters);
	}
}

void UNiagaraDataInterfaceCollisionQuery::PerformQuerySyncCPU(FVectorVMExternalFunctionContext & Context)
{
	VectorVM::FUserPtrHandler<CQDIPerInstanceData> InstanceData(Context);

	FNDIInputParam<FNiagaraPosition> StartPosParam(Context);
	FNDIInputParam<FNiagaraPosition> EndPosParam(Context);
	FNDIInputParam<ECollisionChannel> TraceChannelParam(Context);
	FNDIInputParam<FNiagaraBool> IsSkipTrace(Context);
	FNDIInputParam<FNiagaraBool> IsTraceComplex(Context);
	
	FNDIOutputParam<FNiagaraBool> OutQueryValid(Context);
	FNDIOutputParam<FNiagaraBool> OutInsideMesh(Context);
	FNDIOutputParam<FNiagaraPosition> OutCollisionPos(Context);
	FNDIOutputParam<FVector3f> OutCollisionNormal(Context);
	FNDIOutputParam<float> OutFriction(Context);
	FNDIOutputParam<float> OutRestitution(Context);
	FNDIOutputParam<int32> OutPhysicalMaterialIdx(Context);

	FNiagaraLWCConverter LWCConverter = InstanceData->SystemInstance->GetLWCConverter();

	FScopeLock ScopeLock(&CriticalSection);
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		FNiagaraPosition TraceStart = StartPosParam.GetAndAdvance();
		FNiagaraPosition TraceEnd = EndPosParam.GetAndAdvance();
		ECollisionChannel TraceChannel = TraceChannelParam.GetAndAdvance();
		bool Skip = IsSkipTrace.GetAndAdvance();
		bool TraceComplex = IsTraceComplex.GetAndAdvance();
		ensure(!TraceStart.ContainsNaN());
		ensure(!TraceEnd.ContainsNaN());
		FNiagaraDICollsionQueryResult Res;

		if (!Skip && InstanceData->CollisionBatch.PerformQuery(LWCConverter.ConvertSimulationPositionToWorld(TraceStart), LWCConverter.ConvertSimulationPositionToWorld(TraceEnd), Res, TraceChannel, TraceComplex))
		{
			OutQueryValid.SetAndAdvance(true);
			OutInsideMesh.SetAndAdvance(Res.IsInsideMesh);
			OutCollisionPos.SetAndAdvance(LWCConverter.ConvertWorldToSimulationPosition(Res.CollisionPos));
			OutCollisionNormal.SetAndAdvance((FVector3f)Res.CollisionNormal);
			OutFriction.SetAndAdvance(Res.Friction);
			OutRestitution.SetAndAdvance(Res.Restitution);
			OutPhysicalMaterialIdx.SetAndAdvance(Res.PhysicalMaterialIdx);
		}
		else
		{			
			OutQueryValid.SetAndAdvance(false);
			OutInsideMesh.SetAndAdvance(false);
			OutCollisionPos.SetAndAdvance(FVector3f::ZeroVector);
			OutCollisionNormal.SetAndAdvance(FVector3f::ZeroVector);
			OutFriction.SetAndAdvance(0);
			OutRestitution.SetAndAdvance(0);
			OutPhysicalMaterialIdx.SetAndAdvance(0);
		}
	}
}

void UNiagaraDataInterfaceCollisionQuery::PerformQueryAsyncCPU(FVectorVMExternalFunctionContext & Context)
{
	VectorVM::FUserPtrHandler<CQDIPerInstanceData> InstanceData(Context);

	FNDIInputParam<int32> InIDParam(Context);
	FNDIInputParam<FNiagaraPosition> StartPosParam(Context);
	FNDIInputParam<FNiagaraPosition> EndPosParam(Context);
	FNDIInputParam<ECollisionChannel> TraceChannelParam(Context);
	FNDIInputParam<FNiagaraBool> IsSkipTrace(Context);
	FNDIInputParam<FNiagaraBool> IsTraceComplex(Context);
	
	FNDIOutputParam<int32> OutQueryID(Context);
	FNDIOutputParam<FNiagaraBool> OutQueryValid(Context);
	FNDIOutputParam<FNiagaraBool> OutInsideMesh(Context);
	FNDIOutputParam<FNiagaraPosition> OutCollisionPos(Context);
	FNDIOutputParam<FVector3f> OutCollisionNormal(Context);
	FNDIOutputParam<float> OutFriction(Context);
	FNDIOutputParam<float> OutRestitution(Context);
	FNDIOutputParam<int32> OutPhysicalMaterialIdx(Context);

	FNiagaraLWCConverter LWCConverter = InstanceData->SystemInstance->GetLWCConverter();

	FScopeLock ScopeLock(&CriticalSection);
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		FNiagaraPosition TraceStart = StartPosParam.GetAndAdvance();
		FNiagaraPosition TraceEnd = EndPosParam.GetAndAdvance();
		ECollisionChannel TraceChannel = TraceChannelParam.GetAndAdvance();
		bool Skip = IsSkipTrace.GetAndAdvance();
		bool TraceComplex = IsTraceComplex.GetAndAdvance();
		ensure(!TraceStart.ContainsNaN());
		ensure(!TraceEnd.ContainsNaN());

		int QueryID = Skip ? INDEX_NONE : InstanceData->CollisionBatch.SubmitQuery(LWCConverter.ConvertSimulationPositionToWorld(TraceStart), LWCConverter.ConvertSimulationPositionToWorld(TraceEnd), TraceChannel, TraceComplex);
		OutQueryID.SetAndAdvance(QueryID + 1);

		// try to retrieve a query with the supplied query ID
		FNiagaraDICollsionQueryResult Res;
		int32 ID = InIDParam.GetAndAdvance();
		if (ID > 0 && InstanceData->CollisionBatch.GetQueryResult(ID - 1, Res))
		{
			OutQueryValid.SetAndAdvance(true);
			OutInsideMesh.SetAndAdvance(Res.IsInsideMesh);
			OutCollisionPos.SetAndAdvance(LWCConverter.ConvertWorldToSimulationPosition(Res.CollisionPos));
			OutCollisionNormal.SetAndAdvance(FVector3f(Res.CollisionNormal));
			OutFriction.SetAndAdvance(Res.Friction);
			OutRestitution.SetAndAdvance(Res.Restitution);
			OutPhysicalMaterialIdx.SetAndAdvance(Res.PhysicalMaterialIdx);
		}
		else
		{
			OutQueryValid.SetAndAdvance(false);
			OutInsideMesh.SetAndAdvance(false);
			OutCollisionPos.SetAndAdvance(FVector3f::ZeroVector);
			OutCollisionNormal.SetAndAdvance(FVector3f::ZeroVector);
			OutFriction.SetAndAdvance(0);
			OutRestitution.SetAndAdvance(0);
			OutPhysicalMaterialIdx.SetAndAdvance(0);
		}
	}
}

bool UNiagaraDataInterfaceCollisionQuery::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance, float DeltaSeconds)
{
	CQDIPerInstanceData* PIData = static_cast<CQDIPerInstanceData*>(PerInstanceData);
	PIData->CollisionBatch.CollectResults();

	return false;
}

bool UNiagaraDataInterfaceCollisionQuery::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance, float DeltaSeconds)
{
	CQDIPerInstanceData* PIData = static_cast<CQDIPerInstanceData*>(PerInstanceData);
	PIData->CollisionBatch.DispatchQueries();
	PIData->CollisionBatch.ClearWrite();
	return false;
}

#undef LOCTEXT_NAMESPACE

