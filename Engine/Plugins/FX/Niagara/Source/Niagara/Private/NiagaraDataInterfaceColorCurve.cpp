// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceColorCurve.h"
#include "NiagaraTypes.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceColorCurve)

#if WITH_EDITORONLY_DATA
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveVector.h"
#include "Curves/CurveFloat.h"
#include "Interfaces/ITargetPlatform.h"
#endif

//////////////////////////////////////////////////////////////////////////
//Color Curve

const FName UNiagaraDataInterfaceColorCurve::SampleCurveName(TEXT("SampleColorCurve"));

UNiagaraDataInterfaceColorCurve::UNiagaraDataInterfaceColorCurve(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExposedName = TEXT("Color Curve");

	SetDefaultLUT();
}

void UNiagaraDataInterfaceColorCurve::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}

#if WITH_EDITORONLY_DATA
	UpdateLUT();
#endif
}

void UNiagaraDataInterfaceColorCurve::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (bUseLUT && Ar.IsCooking() && Ar.CookingTarget()->RequiresCookedData())
	{
		UpdateLUT(true);

		Exchange(RedCurve, RedCurveCookedEditorCache);
		Exchange(GreenCurve, GreenCurveCookedEditorCache);
		Exchange(BlueCurve, BlueCurveCookedEditorCache);
		Exchange(AlphaCurve, AlphaCurveCookedEditorCache);

		Super::Serialize(Ar);

		Exchange(RedCurve, RedCurveCookedEditorCache);
		Exchange(GreenCurve, GreenCurveCookedEditorCache);
		Exchange(BlueCurve, BlueCurveCookedEditorCache);
		Exchange(AlphaCurve, AlphaCurveCookedEditorCache);
	}
	else if (bUseLUT && Ar.IsLoading() && GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		Super::Serialize(Ar);

		Exchange(RedCurve, RedCurveCookedEditorCache);
		Exchange(GreenCurve, GreenCurveCookedEditorCache);
		Exchange(BlueCurve, BlueCurveCookedEditorCache);
		Exchange(AlphaCurve, AlphaCurveCookedEditorCache);
	}
	else
#endif
	{
		Super::Serialize(Ar);
	}
}

void UNiagaraDataInterfaceColorCurve::UpdateTimeRanges()
{
	if ((RedCurve.GetNumKeys() > 0 || GreenCurve.GetNumKeys() > 0 || BlueCurve.GetNumKeys() > 0 || AlphaCurve.GetNumKeys() > 0))
	{
		LUTMinTime = FLT_MAX;
		LUTMinTime = FMath::Min(RedCurve.GetNumKeys() > 0 ? RedCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);
		LUTMinTime = FMath::Min(GreenCurve.GetNumKeys() > 0 ? GreenCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);
		LUTMinTime = FMath::Min(BlueCurve.GetNumKeys() > 0 ? BlueCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);
		LUTMinTime = FMath::Min(AlphaCurve.GetNumKeys() > 0 ? AlphaCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);

		LUTMaxTime = -FLT_MAX;
		LUTMaxTime = FMath::Max(RedCurve.GetNumKeys() > 0 ? RedCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTMaxTime = FMath::Max(GreenCurve.GetNumKeys() > 0 ? GreenCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTMaxTime = FMath::Max(BlueCurve.GetNumKeys() > 0 ? BlueCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTMaxTime = FMath::Max(AlphaCurve.GetNumKeys() > 0 ? AlphaCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTInvTimeRange = 1.0f / (LUTMaxTime - LUTMinTime);
	}
	else
	{
		LUTMinTime = 0.0f;
		LUTMaxTime = 1.0f;
		LUTInvTimeRange = 1.0f;
	}
}

TArray<float> UNiagaraDataInterfaceColorCurve::BuildLUT(int32 NumEntries) const
{
	TArray<float> OutputLUT;
	const float InvEntryCountFactor = (NumEntries > 1) ? (1.0f / float(NumEntries - 1.0f)) : 0.0f;

	OutputLUT.Reserve(NumEntries * 4);
	for (int32 i = 0; i < NumEntries; i++)
	{
		float X = UnnormalizeTime(float(i) * InvEntryCountFactor);
		FLinearColor C(RedCurve.Eval(X), GreenCurve.Eval(X), BlueCurve.Eval(X), AlphaCurve.Eval(X));
		OutputLUT.Add(C.R);
		OutputLUT.Add(C.G);
		OutputLUT.Add(C.B);
		OutputLUT.Add(C.A);
	}
	return OutputLUT;
}

bool UNiagaraDataInterfaceColorCurve::CopyToInternal(UNiagaraDataInterface* Destination) const 
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceColorCurve* DestinationColorCurve = CastChecked<UNiagaraDataInterfaceColorCurve>(Destination);
	DestinationColorCurve->RedCurve = RedCurve;
	DestinationColorCurve->GreenCurve = GreenCurve;
	DestinationColorCurve->BlueCurve = BlueCurve;
	DestinationColorCurve->AlphaCurve = AlphaCurve;
#if WITH_EDITORONLY_DATA
	DestinationColorCurve->UpdateLUT();

	if (!CompareLUTS(DestinationColorCurve->ShaderLUT))
	{
		UE_LOGF(LogNiagara, Log, "CopyToInternal LUT generation is out of sync. Please investigate. %ls to %ls", *GetPathName(), *DestinationColorCurve->GetPathName());
	}
#endif
	return true;
}

bool UNiagaraDataInterfaceColorCurve::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceColorCurve* OtherColoRedCurve = CastChecked<const UNiagaraDataInterfaceColorCurve>(Other);
	return OtherColoRedCurve->RedCurve == RedCurve &&
		OtherColoRedCurve->GreenCurve == GreenCurve &&
		OtherColoRedCurve->BlueCurve == BlueCurve &&
		OtherColoRedCurve->AlphaCurve == AlphaCurve;
}

void UNiagaraDataInterfaceColorCurve::GetCurveData(TArray<FCurveData>& OutCurveData)
{
	OutCurveData.Add(FCurveData(&RedCurve, TEXT("Red"), FLinearColor(1.0f, 0.05f, 0.05f)));
	OutCurveData.Add(FCurveData(&GreenCurve, TEXT("Green"), FLinearColor(0.05f, 1.0f, 0.05f)));
	OutCurveData.Add(FCurveData(&BlueCurve, TEXT("Blue"), FLinearColor(0.1f, 0.2f, 1.0f)));
	OutCurveData.Add(FCurveData(&AlphaCurve, TEXT("Alpha"), FLinearColor(0.2f, 0.2f, 0.2f)));
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceColorCurve::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature Sig;
	Sig.Name = SampleCurveName;
	Sig.bMemberFunction = true;
	Sig.bRequiresContext = false;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("ColorCurve")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("X")));
	Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value")));
	//Sig.Owner = *GetName();

	OutFunctions.Add(Sig);
}

// build the shader function HLSL; function name is passed in, as it's defined per-DI; that way, configuration could change
// the HLSL in the spirit of a static switch
// TODO: need a way to identify each specific function here
// 
void UNiagaraDataInterfaceColorCurve::SyncCurvesToAsset()
{
	if (UCurveLinearColor* ColorCurve = Cast<UCurveLinearColor>(CurveAsset))
	{
		RedCurve = ColorCurve->FloatCurves[0];
		GreenCurve = ColorCurve->FloatCurves[1];
		BlueCurve = ColorCurve->FloatCurves[2];
		AlphaCurve = ColorCurve->FloatCurves[3];
	}
	else if (UCurveVector* VecCurve = Cast<UCurveVector>(CurveAsset))
	{
		RedCurve = VecCurve->FloatCurves[0];
		GreenCurve = VecCurve->FloatCurves[1];
		BlueCurve = VecCurve->FloatCurves[2];
		AlphaCurve.Reset();
		AlphaCurve.AddKey(0, 1);
	}
	else if (UCurveFloat* FloatCurve = Cast<UCurveFloat>(CurveAsset))
	{
		RedCurve = FloatCurve->FloatCurve;
		GreenCurve = FloatCurve->FloatCurve;
		BlueCurve = FloatCurve->FloatCurve;
		AlphaCurve = FloatCurve->FloatCurve;
	}
}

bool UNiagaraDataInterfaceColorCurve::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if (FunctionInfo.DefinitionName == SampleCurveName)
	{
		return true;
	}
	return false;
}
#endif

DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceColorCurve, SampleCurve);
void UNiagaraDataInterfaceColorCurve::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == SampleCurveName && BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 4)
	{
		TCurveUseLUTBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceColorCurve, SampleCurve)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else
	{
		UE_LOGF(LogNiagara, Display, "Could not find data interface external function in %ls.\n\tExpected Name: SampleColorCurve  Actual Name: %ls\n\tExpected Inputs: 1  Actual Inputs: %i\n\tExpected Outputs: 4  Actual Outputs: %i",
			*GetPathNameSafe(this), *BindingInfo.Name.ToString(), BindingInfo.GetNumInputs(), BindingInfo.GetNumOutputs());
		OutFunc = FVMExternalFunction();
	}
}

template<>
FORCEINLINE_DEBUGGABLE FLinearColor UNiagaraDataInterfaceColorCurve::SampleCurveInternal<TIntegralConstant<bool, true>>(float X)
{
	float RemappedX = FMath::Clamp(NormalizeTime(X) * LUTNumSamplesMinusOne, 0.0f, LUTNumSamplesMinusOne);
	float PrevEntry = FMath::TruncToFloat(RemappedX);
	float NextEntry = PrevEntry < LUTNumSamplesMinusOne ? PrevEntry + 1.0f : PrevEntry;
	float Interp = RemappedX - PrevEntry;
	
	int32 AIndex = (int32)(PrevEntry * (float)CurveLUTNumElems);
	int32 BIndex = (int32)(NextEntry * (float)CurveLUTNumElems);
	FLinearColor A = FLinearColor(ShaderLUT[AIndex], ShaderLUT[AIndex + 1], ShaderLUT[AIndex + 2], ShaderLUT[AIndex + 3]);
	FLinearColor B = FLinearColor(ShaderLUT[BIndex], ShaderLUT[BIndex + 1], ShaderLUT[BIndex + 2], ShaderLUT[BIndex + 3]);
	return FMath::Lerp(A, B, Interp);
}

template<>
FORCEINLINE_DEBUGGABLE FLinearColor UNiagaraDataInterfaceColorCurve::SampleCurveInternal<TIntegralConstant<bool, false>>(float X)
{
	return FLinearColor(RedCurve.Eval(X), GreenCurve.Eval(X), BlueCurve.Eval(X), AlphaCurve.Eval(X));
}


//#if VECTORVM_SUPPORTS_EXPERIMENTAL && !VECTORVM_SUPPORTS_LEGACY && PLATFORM_ENABLE_VECTORINTRINSICS && !PLATFORM_ENABLE_VECTORINTRINSICS_NEON
#if 0 //there's a bug in some cases and it's not properly optimized, just a naive implementation... @TODO: fix this properly
template<>
void UNiagaraDataInterfaceColorCurve::SampleCurve<TIntegralConstant<bool, true>>(FVectorVMExternalFunctionContext& Context)
{
	const int32 NumInstances = Context.GetNumInstances();

	if (NumInstances == 1) //could be a per-instance function call, in which can we can't write 4-wide so just use the old method
	{ 
		VectorVM::FExternalFuncInputHandler<float> XParam(Context);
		VectorVM::FExternalFuncRegisterHandler<float> SamplePtrR(Context);
		VectorVM::FExternalFuncRegisterHandler<float> SamplePtrG(Context);
		VectorVM::FExternalFuncRegisterHandler<float> SamplePtrB(Context);
		VectorVM::FExternalFuncRegisterHandler<float> SamplePtrA(Context);

		for (int32 i = 0; i < NumInstances; ++i)
		{
			float X = XParam.GetAndAdvance();
			FLinearColor C = SampleCurveInternal<TIntegralConstant<bool, true>>(X);
			*SamplePtrR.GetDestAndAdvance() = C.R;
			*SamplePtrG.GetDestAndAdvance() = C.G;
			*SamplePtrB.GetDestAndAdvance() = C.B;
			*SamplePtrA.GetDestAndAdvance() = C.A;
		}
	}
	else
	{
		const int32 NumLoops = Context.GetNumLoops<4>();

		float *LUT = ShaderLUT.GetData();
		VectorRegister4f LutNumSamplesMinusOne4 = VectorSetFloat1(LUTNumSamplesMinusOne);
		VectorRegister4f LutMinTime4            = VectorSetFloat1(LUTMinTime);
		VectorRegister4f LutInvTimeRange4       = VectorSetFloat1(LUTInvTimeRange);

		VectorRegister4f *XParam = (VectorRegister4f *)Context.RegisterData[0];
		VectorRegister4f *RReg   = (VectorRegister4f *)Context.RegisterData[1];
		VectorRegister4f *GReg   = (VectorRegister4f *)Context.RegisterData[2];
		VectorRegister4f *BReg   = (VectorRegister4f *)Context.RegisterData[3];
		VectorRegister4f *AReg   = (VectorRegister4f *)Context.RegisterData[4];

		int IdxA[4];
		int IdxB[4];

		for (int i = 0; i < NumLoops; ++i)
		{
			VectorRegister4f NormalizedX = VectorMultiply(VectorSubtract(XParam[i & Context.RegInc[0]], LutMinTime4), LutInvTimeRange4);
			VectorRegister4f RemappedX   = VectorClamp(VectorMultiply(NormalizedX, LutNumSamplesMinusOne4), VectorZeroFloat(), LutNumSamplesMinusOne4);
			VectorRegister4f PrevEntry   = VectorTruncate(RemappedX);
			VectorRegister4f NextEntry   = VectorAdd(PrevEntry, VectorBitwiseAnd(VectorOneFloat(), VectorCompareLT(PrevEntry, LutNumSamplesMinusOne4))); //this could be made faster by duplicating the last entry in the LUT so you can read one past it
			VectorRegister4f Interp      = VectorSubtract(RemappedX, PrevEntry);
			VectorRegister4i IdxA4       = VectorShiftLeftImm(VectorFloatToInt(PrevEntry), 2);
			VectorRegister4i IdxB4       = VectorShiftLeftImm(VectorFloatToInt(NextEntry), 2);

			VectorIntStore(IdxA4, IdxA);
			VectorIntStore(IdxB4, IdxB);

			VectorRegister4f A0 = VectorLoad(LUT + IdxA[0]);
			VectorRegister4f A1 = VectorLoad(LUT + IdxA[1]);
			VectorRegister4f A2 = VectorLoad(LUT + IdxA[2]);
			VectorRegister4f A3 = VectorLoad(LUT + IdxA[3]);

			VectorRegister4f B0 = VectorLoad(LUT + IdxB[0]);
			VectorRegister4f B1 = VectorLoad(LUT + IdxB[1]);
			VectorRegister4f B2 = VectorLoad(LUT + IdxB[2]);
			VectorRegister4f B3 = VectorLoad(LUT + IdxB[3]);

			VectorRegister4f I0 = VectorCastIntToFloat(VectorShuffleImmediate(VectorCastFloatToInt(Interp), 0, 0, 0, 0));
			VectorRegister4f I1 = VectorCastIntToFloat(VectorShuffleImmediate(VectorCastFloatToInt(Interp), 1, 1, 1, 1));
			VectorRegister4f I2 = VectorCastIntToFloat(VectorShuffleImmediate(VectorCastFloatToInt(Interp), 2, 2, 2, 2));
			VectorRegister4f I3 = VectorCastIntToFloat(VectorShuffleImmediate(VectorCastFloatToInt(Interp), 3, 3, 3, 3));

			RReg[i] = VectorMultiplyAdd(B0, I0, VectorMultiply(A0, VectorSubtract(VectorOneFloat(), I0)));
			GReg[i] = VectorMultiplyAdd(B1, I1, VectorMultiply(A1, VectorSubtract(VectorOneFloat(), I1)));
			BReg[i] = VectorMultiplyAdd(B2, I2, VectorMultiply(A2, VectorSubtract(VectorOneFloat(), I2)));
			AReg[i] = VectorMultiplyAdd(B3, I3, VectorMultiply(A3, VectorSubtract(VectorOneFloat(), I3)));

			_MM_TRANSPOSE4_PS(RReg[i], GReg[i], BReg[i], AReg[i]);
		}
	}
}
#endif // VECTORVM_SUPPORTS_EXPERIMENTAL

template<typename UseLUT>
void UNiagaraDataInterfaceColorCurve::SampleCurve(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FExternalFuncInputHandler<float> XParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> SamplePtrR(Context);
	VectorVM::FExternalFuncRegisterHandler<float> SamplePtrG(Context);
	VectorVM::FExternalFuncRegisterHandler<float> SamplePtrB(Context);
	VectorVM::FExternalFuncRegisterHandler<float> SamplePtrA(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		float X = XParam.GetAndAdvance();
		FLinearColor C = SampleCurveInternal<UseLUT>(X);
		*SamplePtrR.GetDestAndAdvance() = C.R;
		*SamplePtrG.GetDestAndAdvance() = C.G;
		*SamplePtrB.GetDestAndAdvance() = C.B;
		*SamplePtrA.GetDestAndAdvance() = C.A;
	}
}

