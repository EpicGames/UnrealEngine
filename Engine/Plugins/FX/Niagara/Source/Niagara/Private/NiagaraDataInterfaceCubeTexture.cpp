// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceCubeTexture.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"

#include "Engine/TextureCube.h"
#include "Engine/TextureRenderTargetCube.h"
#include "RenderGraphBuilder.h"
#include "RHIStaticStates.h"
#include "RenderGraphUtils.h"
#include "ShaderCompilerCore.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceCubeTexture)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterfaceCubeTexture"

const TCHAR* UNiagaraDataInterfaceCubeTexture::TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceCubeTextureTemplate.ush");
const FName UNiagaraDataInterfaceCubeTexture::SampleCubeTextureName(TEXT("SampleCubeTexture"));
const FName UNiagaraDataInterfaceCubeTexture::TextureDimsName(TEXT("TextureDimensions"));

struct FNDICubeTextureInstanceData_GameThread
{
	TWeakObjectPtr<UTexture> CurrentTexture = nullptr;
	FIntPoint CurrentTextureSize = FIntPoint::ZeroValue;
	FNiagaraParameterDirectBinding<UObject*> UserParamBinding;
};

struct FNDICubeTextureInstanceData_RenderThread
{
	FSamplerStateRHIRef		SamplerStateRHI;
	FTextureReferenceRHIRef	TextureReferenceRHI;
	FIntPoint				TextureSize;

	FRDGTextureRef			TransientRDGTexture = nullptr;
};

struct FNiagaraDataInterfaceProxyCubeTexture : public FNiagaraDataInterfaceProxy
{
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { check(false); }
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override
	{
		if (Context.IsFinalPostSimulate())
		{
			if (FNDICubeTextureInstanceData_RenderThread* InstanceData = InstanceData_RT.Find(Context.GetSystemInstanceID()))
			{
				InstanceData->TransientRDGTexture = nullptr;
			}
		}
	}

	TMap<FNiagaraSystemInstanceID, FNDICubeTextureInstanceData_RenderThread> InstanceData_RT;
}; 

UNiagaraDataInterfaceCubeTexture::UNiagaraDataInterfaceCubeTexture(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, Texture(nullptr)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyCubeTexture());

	FNiagaraTypeDefinition Def(UTexture::StaticClass());
	TextureUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceCubeTexture::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceCubeTexture::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceCubeTexture* DestinationTexture = CastChecked<UNiagaraDataInterfaceCubeTexture>(Destination);
	DestinationTexture->Texture = Texture;
	DestinationTexture->TextureUserParameter = TextureUserParameter;

	return true;
}

bool UNiagaraDataInterfaceCubeTexture::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceCubeTexture* OtherTexture = CastChecked<const UNiagaraDataInterfaceCubeTexture>(Other);
	return
		OtherTexture->Texture == Texture &&
		OtherTexture->TextureUserParameter == TextureUserParameter;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceCubeTexture::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = SampleCubeTextureName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Texture")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UVW")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.SetDescription(LOCTEXT("TextureSampleCubeTextureDesc", "Sample the specified mip level of the input cube texture at the specified UVW coordinates"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = TextureDimsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Texture")));
		Sig.SetDescription(LOCTEXT("TextureDimsDesc", "Get the dimensions of mip 0 of the texture."));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Width")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Height")));
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceCubeTexture, SampleCubeTexture)
void UNiagaraDataInterfaceCubeTexture::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == SampleCubeTextureName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 4);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceCubeTexture, SampleCubeTexture)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == TextureDimsName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 2);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceCubeTexture::GetTextureDimensions);
	}
}

int32 UNiagaraDataInterfaceCubeTexture::PerInstanceDataSize() const
{
	return sizeof(FNDICubeTextureInstanceData_GameThread);
}

bool UNiagaraDataInterfaceCubeTexture::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDICubeTextureInstanceData_GameThread* InstanceData = new (PerInstanceData) FNDICubeTextureInstanceData_GameThread();
	InstanceData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), TextureUserParameter.Parameter);
	return true;
}

void UNiagaraDataInterfaceCubeTexture::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDICubeTextureInstanceData_GameThread* InstanceData = static_cast<FNDICubeTextureInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FNDICubeTextureInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(NDITexture_RemoveInstance)
	(
		[RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyCubeTexture>(), RT_InstanceID = SystemInstance->GetId()](FRHICommandListImmediate&)
		{
			RT_Proxy->InstanceData_RT.Remove(RT_InstanceID);
		}
	);
}

bool UNiagaraDataInterfaceCubeTexture::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FNDICubeTextureInstanceData_GameThread* InstanceData = static_cast<FNDICubeTextureInstanceData_GameThread*>(PerInstanceData);

	UTexture* CurrentTexture = InstanceData->UserParamBinding.GetValueOrDefault<UTexture>(Texture);
	if ( InstanceData->CurrentTexture != CurrentTexture )
	{
		UTextureCube* CurrentTextureCube = Cast<UTextureCube>(CurrentTexture);
		UTextureRenderTargetCube* CurrentTextureRT = Cast<UTextureRenderTargetCube>(CurrentTexture);
		if (CurrentTextureCube || CurrentTextureRT)
		{
			const FIntPoint CurrentTextureSize = CurrentTextureCube != nullptr ?
				FIntPoint(CurrentTextureCube->GetSizeX(), CurrentTextureCube->GetSizeY()) :
				FIntPoint(CurrentTextureRT->SizeX, CurrentTextureRT->SizeX);

			InstanceData->CurrentTexture = CurrentTexture;
			InstanceData->CurrentTextureSize = CurrentTextureSize;

			ENQUEUE_RENDER_COMMAND(NDITexture_UpdateInstance)
			(
				[RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxyCubeTexture>(), RT_InstanceID=SystemInstance->GetId(), RT_Texture=CurrentTexture, RT_TextureSize=CurrentTextureSize](FRHICommandListImmediate&)
				{
					FNDICubeTextureInstanceData_RenderThread& InstanceData = RT_Proxy->InstanceData_RT.FindOrAdd(RT_InstanceID);
					if (RT_Texture)
					{
						InstanceData.TextureReferenceRHI = RT_Texture->TextureReference.TextureReferenceRHI;
						InstanceData.SamplerStateRHI = RT_Texture->GetResource() ? RT_Texture->GetResource()->SamplerStateRHI.GetReference() : TStaticSamplerState<SF_Point>::GetRHI();
						InstanceData.TextureSize = RT_TextureSize;
					}
					else
					{
						InstanceData.TextureReferenceRHI = nullptr;
						InstanceData.SamplerStateRHI = nullptr;
						InstanceData.TextureSize = FIntPoint::ZeroValue;
					}
				}
			);
		}
	}
	return false;
}

void UNiagaraDataInterfaceCubeTexture::GetTextureDimensions(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDICubeTextureInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int32> OutWidth(Context);
	FNDIOutputParam<int32> OutHeight(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutWidth.SetAndAdvance(InstData->CurrentTextureSize.X);
		OutHeight.SetAndAdvance(InstData->CurrentTextureSize.Y);
	}
}

void UNiagaraDataInterfaceCubeTexture::SampleCubeTexture(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDICubeTextureInstanceData_GameThread> InstData(Context);
	FNDIInputParam<FVector3f> InCoord(Context);
	FNDIInputParam<float> InMipLevel(Context);
	FNDIOutputParam<FVector4f> OutColor(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutColor.SetAndAdvance(FVector4f(1.0f, 0.0f, 1.0f, 1.0f));
	}

}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceCubeTexture::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceCubeTexture::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceCubeTexture::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if ((FunctionInfo.DefinitionName == SampleCubeTextureName) ||
		(FunctionInfo.DefinitionName == TextureDimsName))
	{
		return true;
	}
	return false;
}
#endif

void UNiagaraDataInterfaceCubeTexture::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceCubeTexture::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyCubeTexture& TextureProxy = Context.GetProxy<FNiagaraDataInterfaceProxyCubeTexture>();
	FNDICubeTextureInstanceData_RenderThread* InstanceData = TextureProxy.InstanceData_RT.Find(Context.GetSystemInstanceID());
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	FShaderParameters* Parameters = Context.GetParameterNestedStruct<FShaderParameters>();
	if (InstanceData && InstanceData->TextureReferenceRHI.IsValid())
	{
		Parameters->TextureSize = InstanceData->TextureSize;
		Parameters->TextureSampler = InstanceData->SamplerStateRHI;
		if (Context.IsResourceBound(&Parameters->Texture))
		{
			FRDGTextureRef RDGTexture = InstanceData ? InstanceData->TransientRDGTexture : nullptr;
			if (InstanceData && RDGTexture == nullptr)
			{
				FTextureRHIRef ResolvedTextureRHI = InstanceData->TextureReferenceRHI->GetReferencedTexture();
				if (ResolvedTextureRHI.IsValid())
				{
					InstanceData->TransientRDGTexture = GraphBuilder.FindExternalTexture(ResolvedTextureRHI);
					if (InstanceData->TransientRDGTexture == nullptr)
					{
						InstanceData->TransientRDGTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(ResolvedTextureRHI, TEXT("NiagaraTextureCube")));

						GraphBuilder.UseInternalAccessMode(InstanceData->TransientRDGTexture);
						Context.GetRDGExternalAccessQueue().Add(InstanceData->TransientRDGTexture);
					}
					RDGTexture = InstanceData->TransientRDGTexture;
				}
			}

			Parameters->Texture = RDGTexture ? RDGTexture : Context.GetComputeDispatchInterface().GetBlackTexture(GraphBuilder, ETextureDimension::TextureCube);
		}
	}
	else
	{
		Parameters->TextureSize = FIntPoint::ZeroValue;
		Parameters->TextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		Parameters->Texture = Context.GetComputeDispatchInterface().GetBlackTexture(GraphBuilder, ETextureDimension::TextureCube);
	}
}

void UNiagaraDataInterfaceCubeTexture::SetTexture(UTexture* InTexture)
{
	if (InTexture)
	{
		Texture = InTexture;
	}
}

#undef LOCTEXT_NAMESPACE

