// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceCurlNoise.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSimplexNoise.h"
#include "Math/IntVector.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceCurlNoise)

static const FName SampleNoiseFieldName(TEXT("SampleNoiseField"));

UNiagaraDataInterfaceCurlNoise::UNiagaraDataInterfaceCurlNoise(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, Seed(0)
{
	OffsetFromSeed = SimplexNoiseOffsetFromSeed(Seed);
	Proxy.Reset(new FNiagaraDataInterfaceProxyCurlNoise(OffsetFromSeed));
}

void UNiagaraDataInterfaceCurlNoise::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceCurlNoise::PostLoad()
{
	Super::PostLoad();
	OffsetFromSeed = SimplexNoiseOffsetFromSeed(Seed);

	MarkRenderDataDirty();
}

#if WITH_EDITOR

void UNiagaraDataInterfaceCurlNoise::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);
	
	// Flush the rendering thread before making any changes to make sure the 
	// data read by the compute shader isn't subject to a race condition.
	// TODO(mv): Solve properly using something like a RT Proxy.
	FlushRenderingCommands();
}
void UNiagaraDataInterfaceCurlNoise::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceCurlNoise, Seed))
	{
		// NOTE: Calculate the offset based on the seed on-change instead of on every invocation for every particle...
		OffsetFromSeed = SimplexNoiseOffsetFromSeed(Seed);
	}

	MarkRenderDataDirty();
}

#endif

bool UNiagaraDataInterfaceCurlNoise::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceCurlNoise* DestinationCurlNoise = CastChecked<UNiagaraDataInterfaceCurlNoise>(Destination);
	DestinationCurlNoise->Seed = Seed;
	DestinationCurlNoise->OffsetFromSeed = OffsetFromSeed;
	DestinationCurlNoise->MarkRenderDataDirty();

	return true;
}

bool UNiagaraDataInterfaceCurlNoise::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceCurlNoise* OtherCurlNoise = CastChecked<const UNiagaraDataInterfaceCurlNoise>(Other);
	return OtherCurlNoise->Seed == Seed;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceCurlNoise::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature Sig;
	Sig.Name = SampleNoiseFieldName;
	Sig.bMemberFunction = true;
	Sig.bRequiresContext = false;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("NoiseField")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("XYZ")));
	Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
	//Sig.Owner = *GetName();

	OutFunctions.Add(Sig);
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceCurlNoise, SampleNoiseField);
void UNiagaraDataInterfaceCurlNoise::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	check(BindingInfo.Name == SampleNoiseFieldName);
	check(BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 3);
	NDI_FUNC_BINDER(UNiagaraDataInterfaceCurlNoise, SampleNoiseField)::Bind(this, OutFunc);
}

void UNiagaraDataInterfaceCurlNoise::SampleNoiseField(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FExternalFuncInputHandler<float> XParam(Context);
	VectorVM::FExternalFuncInputHandler<float> YParam(Context);
	VectorVM::FExternalFuncInputHandler<float> ZParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleZ(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const FVector3f InCoords = FVector3f(XParam.GetAndAdvance(), YParam.GetAndAdvance(), ZParam.GetAndAdvance());

		// See comments to JacobianSimplex_ALU in Random.ush
		FNiagaraMatrix3x4 J = JacobianSimplex_ALU(InCoords + OffsetFromSeed);
		*OutSampleX.GetDestAndAdvance() = J[1][2] - J[2][1];
		*OutSampleY.GetDestAndAdvance() = J[2][0] - J[0][2];
		*OutSampleZ.GetDestAndAdvance() = J[0][1] - J[1][0];
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceCurlNoise::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

bool UNiagaraDataInterfaceCurlNoise::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	static const TCHAR* FormatSample = TEXT(R"(
		void {FunctionName}(float3 In_XYZ, out float3 Out_Value)
		{
			// NOTE(mv): The comments in random.ush claims that the unused part is optimized away, so it only uses 6 out of 12 values in our case.
			float3x4 J = JacobianSimplex_ALU(In_XYZ + {OffsetFromSeedName}, false, 1.0);
			Out_Value = float3(J[1][2]-J[2][1], J[2][0]-J[0][2], J[0][1]-J[1][0]); // See comments to JacobianSimplex_ALU in Random.ush
		}
	)");
	TMap<FString, FStringFormatArg> ArgsSample;
	ArgsSample.Add(TEXT("FunctionName"), FunctionInfo.InstanceName);
	ArgsSample.Add(TEXT("OffsetFromSeedName"), ParamInfo.DataInterfaceHLSLSymbol + TEXT("_OffsetFromSeed"));
	OutHLSL += FString::Format(FormatSample, ArgsSample);
	return true;
}

void UNiagaraDataInterfaceCurlNoise::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	OutHLSL.Appendf(TEXT("float3 %s_OffsetFromSeed;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
}
#endif

void UNiagaraDataInterfaceCurlNoise::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceCurlNoise::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyCurlNoise& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyCurlNoise>();
	
	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->OffsetFromSeed = FVector3f(DIProxy.OffsetFromSeed);
}

void UNiagaraDataInterfaceCurlNoise::PushToRenderThreadImpl()
{
	FNiagaraDataInterfaceProxyCurlNoise* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyCurlNoise>();

	// Push Updates to Proxy.
	ENQUEUE_RENDER_COMMAND(FUpdateDIColorCurve)(
		[RT_Proxy, RT_Offset=OffsetFromSeed](FRHICommandListImmediate& RHICmdList)
	{
		RT_Proxy->OffsetFromSeed = RT_Offset;
	});
}

