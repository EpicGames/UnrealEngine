// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceCurve.h"
#include "NiagaraTypes.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceCurve)

#if WITH_EDITORONLY_DATA
#include "Curves/CurveFloat.h"
#include "Interfaces/ITargetPlatform.h"
#endif

const FName UNiagaraDataInterfaceCurve::SampleCurveName(TEXT("SampleCurve"));

UNiagaraDataInterfaceCurve::UNiagaraDataInterfaceCurve(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExposedName = TEXT("Float Curve");

	SetDefaultLUT();
}

void UNiagaraDataInterfaceCurve::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}

#if WITH_EDITORONLY_DATA
	UpdateLUT();
#endif
}

void UNiagaraDataInterfaceCurve::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (bUseLUT && Ar.IsCooking() && Ar.CookingTarget()->RequiresCookedData())
	{
		UpdateLUT(true);

		Exchange(Curve, CurveCookedEditorCache);

		Super::Serialize(Ar);

		Exchange(Curve, CurveCookedEditorCache);
	}
	else if (bUseLUT && Ar.IsLoading() && GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		Super::Serialize(Ar);

		Exchange(Curve, CurveCookedEditorCache);
	}
	else
#endif
	{
		Super::Serialize(Ar);
	}
}

bool UNiagaraDataInterfaceCurve::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	CastChecked<UNiagaraDataInterfaceCurve>(Destination)->Curve = Curve;
#if WITH_EDITORONLY_DATA
	CastChecked<UNiagaraDataInterfaceCurve>(Destination)->UpdateLUT();
	if (!CompareLUTS(CastChecked<UNiagaraDataInterfaceCurve>(Destination)->ShaderLUT))
	{
		UE_LOGF(LogNiagara, Log, "Post CopyToInternal LUT generation is out of sync. Please investigate. %ls", *GetPathName());
	}
#endif
	return true;
}

bool UNiagaraDataInterfaceCurve::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	return CastChecked<UNiagaraDataInterfaceCurve>(Other)->Curve == Curve;
}

void UNiagaraDataInterfaceCurve::GetCurveData(TArray<FCurveData>& OutCurveData)
{
	OutCurveData.Add(FCurveData(&Curve, NAME_None, FLinearColor(1.0f, 0.05f, 0.05f)));
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceCurve::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature Sig;
	Sig.Name = SampleCurveName;
	Sig.bMemberFunction = true;
	Sig.bRequiresContext = false;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Curve")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("X")));
	Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
//	Sig.Owner = *GetName();

	OutFunctions.Add(Sig);
}
#endif

void UNiagaraDataInterfaceCurve::UpdateTimeRanges()
{
	if (Curve.GetNumKeys() > 0)
	{
		LUTMinTime = Curve.GetFirstKey().Time;
		LUTMaxTime = Curve.GetLastKey().Time;
		LUTInvTimeRange = 1.0f / (LUTMaxTime - LUTMinTime);
	}
	else
	{
		LUTMinTime = 0.0f;
		LUTMaxTime = 1.0f;
		LUTInvTimeRange = 1.0f;
	}
}

TArray<float> UNiagaraDataInterfaceCurve::BuildLUT(int32 NumEntries) const
{
	TArray<float> OutputLUT;
	const float InvEntryCountFactor = (NumEntries > 1) ? (1.0f / float(NumEntries - 1.0f)) : 0.0f;

	OutputLUT.Reserve(NumEntries);
	for (int32 i = 0; i < NumEntries; i++)
	{
		float X = UnnormalizeTime(i * InvEntryCountFactor);
		float C = Curve.Eval(X);
		OutputLUT.Add(C);
	}
	return OutputLUT;
}

// build the shader function HLSL; function name is passed in, as it's defined per-DI; that way, configuration could change
// the HLSL in the spirit of a static switch
// TODO: need a way to identify each specific function here
// 
#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceCurve::SyncCurvesToAsset()
{
	if (UCurveFloat* LoadedCurve = Cast<UCurveFloat>(CurveAsset))
	{
		Curve = LoadedCurve->FloatCurve;
	}
}

bool UNiagaraDataInterfaceCurve::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if (FunctionInfo.DefinitionName == SampleCurveName)
	{
		return true;
	}
	return false;
}
#endif

DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceCurve, SampleCurve);
void UNiagaraDataInterfaceCurve::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == SampleCurveName && BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1)
	{
		TCurveUseLUTBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceCurve, SampleCurve)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else
	{
		UE_LOGF(LogNiagara, Display, "Could not find data interface external function in %ls.\n\tExpected Name: SampleCurve  Actual Name: %ls\n\tExpected Inputs: 1  Actual Inputs: %i\n\tExpected Outputs: 1  Actual Outputs: %i",
			*GetPathNameSafe(this), *BindingInfo.Name.ToString(), BindingInfo.GetNumInputs(), BindingInfo.GetNumOutputs());
	}
}

template<>
FORCEINLINE_DEBUGGABLE float UNiagaraDataInterfaceCurve::SampleCurveInternal<TIntegralConstant<bool, true>>(float X)
{
	float RemappedX = FMath::Clamp(NormalizeTime(X) * LUTNumSamplesMinusOne, 0.0f, LUTNumSamplesMinusOne);
	float PrevEntry = FMath::TruncToFloat(RemappedX);
	float NextEntry = PrevEntry < LUTNumSamplesMinusOne ? PrevEntry + 1.0f : PrevEntry;
	float Interp = RemappedX - PrevEntry;

	int32 AIndex = (int32)(PrevEntry * (float)CurveLUTNumElems);
	int32 BIndex = (int32)(NextEntry * (float)CurveLUTNumElems);
	float A = ShaderLUT[AIndex];
	float B = ShaderLUT[BIndex];
	return FMath::Lerp(A, B, Interp);
}

template<>
FORCEINLINE_DEBUGGABLE float UNiagaraDataInterfaceCurve::SampleCurveInternal<TIntegralConstant<bool, false>>(float X)
{
	return Curve.Eval(X);
}

//#if VECTORVM_SUPPORTS_EXPERIMENTAL && !VECTORVM_SUPPORTS_LEGACY
#if 0 //there's a bug in some cases and it's not properly optimized, just a naive implementation... @TODO: fix this properly
template<>
void UNiagaraDataInterfaceCurve::SampleCurve<TIntegralConstant<bool, true>>(FVectorVMExternalFunctionContext& Context)
{
	const int32 NumInstances = Context.GetNumInstances();

	if (NumInstances == 1)
	{
		VectorVM::FExternalFuncInputHandler<float> XParam(Context);
		VectorVM::FExternalFuncRegisterHandler<float> OutSample(Context);

		for (int32 i = 0; i < NumInstances; ++i)
		{
			*OutSample.GetDest() = SampleCurveInternal<TIntegralConstant<bool, true>>(XParam.Get());
			XParam.Advance();
			OutSample.Advance();
		}
	}
	else
	{
		float *LUT = ShaderLUT.GetData();
		VectorRegister4f LutNumSamplesMinusOne4 = VectorSetFloat1(LUTNumSamplesMinusOne);
		VectorRegister4f LutMinTime4            = VectorSetFloat1(LUTMinTime);
		VectorRegister4f LutInvTimeRange4       = VectorSetFloat1(LUTInvTimeRange);
	
		VectorRegister4f *XParam = (VectorRegister4f *)Context.RegisterData[0];
		VectorRegister4f *OutSample = (VectorRegister4f *)Context.RegisterData[1];

		int IdxA[4];
		int IdxB[4];

		for (int i = 0; i < Context.NumLoops; ++i)
		{
			VectorRegister4f NormalizedTime4 = VectorMultiply(VectorSubtract(XParam[i & Context.RegInc[0]], LutMinTime4), LutInvTimeRange4);
			VectorRegister4f RemappedX4      = VectorMin(VectorMax(VectorMultiply(NormalizedTime4, LutNumSamplesMinusOne4), VectorZeroFloat()), LutNumSamplesMinusOne4);
			VectorRegister4f PrevEntry4      = VectorTruncate(RemappedX4);
			VectorRegister4f NextEntry4      = VectorAdd(PrevEntry4, VectorBitwiseAnd(VectorOneFloat(), VectorCompareLT(PrevEntry4, LutNumSamplesMinusOne4))); //this could be made faster by duplicating the last entry in the LUT so you can read one past it
			VectorRegister4f Interp4         = VectorSubtract(RemappedX4, PrevEntry4);
			VectorRegister4i IdxA4           = VectorFloatToInt(PrevEntry4);
			VectorRegister4i IdxB4           = VectorFloatToInt(NextEntry4);

			VectorIntStore(IdxA4, IdxA);
			VectorIntStore(IdxB4, IdxB);

			VectorRegister4f A4 = MakeVectorRegisterFloat(LUT[IdxA[0]], LUT[IdxA[1]], LUT[IdxA[2]], LUT[IdxA[3]]);
			VectorRegister4f B4 = MakeVectorRegisterFloat(LUT[IdxB[0]], LUT[IdxB[1]], LUT[IdxB[2]], LUT[IdxB[3]]);

			OutSample[i] = VectorMultiplyAdd(B4, Interp4, VectorMultiply(A4, VectorSubtract(VectorOneFloat(), Interp4)));
		}
	}
}
#endif // VECTORVM_SUPPORTS_EXPERIMENTAL

template<typename UseLUT>
void UNiagaraDataInterfaceCurve::SampleCurve(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FExternalFuncInputHandler<float> XParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*OutSample.GetDest() = SampleCurveInternal<UseLUT>(XParam.Get());
		XParam.Advance();
		OutSample.Advance();
	}
}

