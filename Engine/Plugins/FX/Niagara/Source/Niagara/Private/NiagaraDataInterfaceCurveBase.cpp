// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceCurveBase.h"
#include "Engine/Texture2D.h"
#include "Internationalization/Internationalization.h"

#include "NiagaraCompileHashVisitor.h"
#include "NiagaraConstants.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraEmitter.h"
#include "NiagaraRenderer.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraStats.h"
#include "NiagaraSystemStaticBuffers.h"
#include "NiagaraTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceCurveBase)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceCurveBase"

float GNiagaraLUTOptimizeThreshold = UNiagaraDataInterfaceCurveBase::DefaultOptimizeThreshold;
static FAutoConsoleVariableRef CVarNiagaraLUTOptimizeThreshold(
	TEXT("fx.Niagara.LUT.OptimizeThreshold"),
	GNiagaraLUTOptimizeThreshold,
	TEXT("Error Threshold used when optimizing Curve LUTs, setting to 0.0 or below will result in no optimization\n"),
	ECVF_Default
);

int32 GNiagaraLUTVerifyPostLoad = 0;
static FAutoConsoleVariableRef CVarNiagaraLUTVerifyPostLoad(
	TEXT("fx.Niagara.LUT.VerifyPostLoad"),
	GNiagaraLUTVerifyPostLoad,
	TEXT("Enable to verify LUTs match in PostLoad vs the Loaded Data\n"),
	ECVF_Default
);

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraDataInterfaceCurveParameters, )
	SHADER_PARAMETER(float, MinTime)
	SHADER_PARAMETER(float, MaxTime)
	SHADER_PARAMETER(float, InvTimeRange)
	SHADER_PARAMETER(uint32, CurveLUTNumMinusOne)
	SHADER_PARAMETER(uint32, LUTOffset)
	SHADER_PARAMETER_SRV(Buffer<float>, CurveLUT)
END_SHADER_PARAMETER_STRUCT()

static const TCHAR* NDICurve_TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceCurveTemplate.ush");

namespace NiagaraDataInterfaceCurveBaseImpl
{

static bool PassesErrorThreshold(TConstArrayView<float> ShaderLUT, TConstArrayView<float> ResampledLUT, int32 NumElements, float ErrorThreshold)
{
	const int32 CurrNumSamples = ShaderLUT.Num() / NumElements;
	const int32 NewNumSamples = ResampledLUT.Num() / NumElements;

	for (int iSample = 0; iSample < CurrNumSamples; ++iSample)
	{
		const float NormalizedSampleTime = float(iSample) / float(CurrNumSamples - 1);

		const float LhsInterp = FMath::Frac(NormalizedSampleTime * CurrNumSamples);
		const int LhsSampleA = FMath::Min(FMath::FloorToInt(NormalizedSampleTime * CurrNumSamples), CurrNumSamples - 1);
		const int LhsSampleB = FMath::Min(LhsSampleA + 1, CurrNumSamples - 1);

		const float RhsInterp = FMath::Frac(NormalizedSampleTime * NewNumSamples);
		const int RhsSampleA = FMath::Min(FMath::FloorToInt(NormalizedSampleTime * NewNumSamples), NewNumSamples - 1);
		const int RhsSampleB = FMath::Min(RhsSampleA + 1, NewNumSamples - 1);

		for (int iElement = 0; iElement < NumElements; ++iElement)
		{
			const float LhsValue = FMath::Lerp(ShaderLUT[LhsSampleA * NumElements + iElement], ShaderLUT[LhsSampleB * NumElements + iElement], LhsInterp);
			const float RhsValue = FMath::Lerp(ResampledLUT[RhsSampleA * NumElements + iElement], ResampledLUT[RhsSampleB * NumElements + iElement], RhsInterp);
			const float Error = FMath::Abs(LhsValue - RhsValue);
			if (Error > ErrorThreshold)
			{
				return false;
			}
		}
	}

	return true;
}

template<int32 ElementStride>
static bool PassesErrorThresholdVectorized(TConstArrayView<float> ShaderLUT, TConstArrayView<float> ResampledLUT, float ErrorThreshold)
{
	const int32 CurrNumSamples = ShaderLUT.Num() / ElementStride;
	const int32 NewNumSamples = ResampledLUT.Num() / ElementStride;

	const float* ShaderLUTData = ShaderLUT.GetData();
	const float* ResampledLUTData = ResampledLUT.GetData();

	const VectorRegister4Float Threshold = VectorSetFloat1(ErrorThreshold);

	for (int iSample = 0; iSample < CurrNumSamples; ++iSample)
	{
		const float NormalizedSampleTime = float(iSample) / float(CurrNumSamples - 1);

		const int LhsSampleA = FMath::Min(FMath::FloorToInt(NormalizedSampleTime * CurrNumSamples), CurrNumSamples - 1);
		const int LhsSampleB = FMath::Min(LhsSampleA + 1, CurrNumSamples - 1);

		const int RhsSampleA = FMath::Min(FMath::FloorToInt(NormalizedSampleTime * NewNumSamples), NewNumSamples - 1);
		const int RhsSampleB = FMath::Min(RhsSampleA + 1, NewNumSamples - 1);

		VectorRegister4Float LhsInterp = VectorSetFloat1(NormalizedSampleTime * CurrNumSamples);
		LhsInterp = VectorSubtract(LhsInterp, VectorTruncate(LhsInterp));

		VectorRegister4Float RhsInterp = VectorSetFloat1(NormalizedSampleTime * NewNumSamples);
		RhsInterp = VectorSubtract(RhsInterp, VectorTruncate(RhsInterp));

		VectorRegister4Float LhsValue = VectorLerp(VectorLoad(ShaderLUTData + LhsSampleA * ElementStride), VectorLoad(ShaderLUTData + LhsSampleB * ElementStride), LhsInterp);
		VectorRegister4Float RhsValue = VectorLerp(VectorLoad(ResampledLUTData + RhsSampleA * ElementStride), VectorLoad(ResampledLUTData + RhsSampleB * ElementStride), RhsInterp);
		VectorRegister4Float Error = VectorAbs(VectorSubtract(LhsValue, RhsValue));
		if (VectorAnyGreaterThan(Error, Threshold))
		{
			return false;
		}
	}

	return true;
}

}; // NiagaraDataInterfaceCurveBaseImpl

/** Base class for curve data proxy data. */
struct FNiagaraDataInterfaceProxyCurveBase : public FNiagaraDataInterfaceProxy
{
	virtual ~FNiagaraDataInterfaceProxyCurveBase()
	{
		check(IsInRenderingThread());
		DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, CurveLUT.NumBytes);
		CurveLUT.Release();
	}

	float LUTMinTime = 0.0f;
	float LUTMaxTime = 0.0f;
	float LUTInvTimeRange = 0.0f;
	uint32 CurveLUTNumMinusOne = 0;
	uint32 LUTOffset = INDEX_NONE;
	FReadBuffer CurveLUT;

	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return 0;
	}

	// @todo REMOVEME
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { check(false); }
};

UNiagaraDataInterfaceCurveBase::UNiagaraDataInterfaceCurveBase()
	: LUTMinTime(0.0f)
	, LUTMaxTime(1.0f)
	, LUTInvTimeRange(1.0f)
	, bUseLUT(true)
	, bExposeCurve(false)
#if WITH_EDITORONLY_DATA
	, bOptimizeLUT(true)
	, bOverrideOptimizeThreshold(false)
	, HasEditorData(true)
	, OptimizeThreshold(DefaultOptimizeThreshold)
#endif
	, ExposedName(TEXT("Curve"))
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyCurveBase());
}

UNiagaraDataInterfaceCurveBase::UNiagaraDataInterfaceCurveBase(FObjectInitializer const& ObjectInitializer)
	: LUTMinTime(0.0f)
	, LUTMaxTime(1.0f)
	, LUTInvTimeRange(1.0f)
	, bUseLUT(true)
	, bExposeCurve(false)
#if WITH_EDITORONLY_DATA
	, bOptimizeLUT(true)
	, bOverrideOptimizeThreshold(false)
	, HasEditorData(true)
	, OptimizeThreshold(DefaultOptimizeThreshold)
#endif
	, ExposedName(TEXT("Curve"))
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyCurveBase());
}

void UNiagaraDataInterfaceCurveBase::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (HasEditorData)
	{
		const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);

		if (NiagaraVer < FNiagaraCustomVersion::CurveLUTRegen)
		{
			UpdateLUT();
		}
	}
#endif
}

void UNiagaraDataInterfaceCurveBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Push to render thread if loading
	if (Ar.IsLoading())
	{
#if WITH_EDITORONLY_DATA
		HasEditorData = !Ar.IsFilterEditorOnly();
		// Sometimes curves are out of date which needs to be tracked down
		// Temporarily we will make sure they are up to date in editor builds,
		// but not for cooked editor packages since they're already up to date,
		// and the curve data is not valid at this point.
		if (HasEditorData && Ar.IsLoadingFromCookedPackage() == false && GetClass() != UNiagaraDataInterfaceCurveBase::StaticClass())
		{
			UpdateLUT(true);
		}
#endif
		MarkRenderDataDirty();
	}
}

#if WITH_EDITOR
void UNiagaraDataInterfaceCurveBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceCurveBase, bExposeCurve))
	{
		UpdateExposedTexture();
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceCurveBase, CurveAsset))
	{
		UpdateLUT();
	}
}
#endif

bool UNiagaraDataInterfaceCurveBase::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceCurveBase* DestinationTyped = CastChecked<UNiagaraDataInterfaceCurveBase>(Destination);
	DestinationTyped->bUseLUT = bUseLUT;
	DestinationTyped->ShaderLUT = ShaderLUT;
	DestinationTyped->LUTMinTime = LUTMinTime;
	DestinationTyped->LUTMaxTime = LUTMaxTime;
	DestinationTyped->LUTInvTimeRange = LUTInvTimeRange;
	DestinationTyped->LUTNumSamplesMinusOne = LUTNumSamplesMinusOne;
#if WITH_EDITORONLY_DATA
	DestinationTyped->CurveAsset = CurveAsset;
	DestinationTyped->bOptimizeLUT = bOptimizeLUT;
	DestinationTyped->bOverrideOptimizeThreshold = bOverrideOptimizeThreshold;
	DestinationTyped->OptimizeThreshold = OptimizeThreshold;
#endif
	DestinationTyped->bExposeCurve = bExposeCurve;
	DestinationTyped->ExposedName = ExposedName;
	DestinationTyped->MarkRenderDataDirty();

	return true;
}


bool UNiagaraDataInterfaceCurveBase::CompareLUTS(const TArray<float>& OtherLUT) const
{
	if (ShaderLUT.Num() == OtherLUT.Num())
	{
		bool bMatched = true;
		for (int32 i = 0; i < ShaderLUT.Num(); i++)
		{
			if (false == FMath::IsNearlyEqual(ShaderLUT[i], OtherLUT[i], 0.0001f))
			{
				bMatched = false;
				UE_LOGF(LogNiagara, Log, "First LUT mismatch found on comparison - LUT[%d] = %.9f  Other = %.9f \t%.9f", i, ShaderLUT[i], OtherLUT[i], fabsf(ShaderLUT[i] - OtherLUT[i]));
				break;
			}
		}
		return bMatched;
	}
	else
	{
		UE_LOGF(LogNiagara, Log, "Table sizes don't match");
		return false;
	}
}

void UNiagaraDataInterfaceCurveBase::SetDefaultLUT()
{
	ShaderLUT.Empty(GetCurveNumElems());
	ShaderLUT.AddDefaulted(GetCurveNumElems());
	LUTNumSamplesMinusOne = 0;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceCurveBase::UpdateLUT(bool bFromSerialize)
{
	SyncCurvesToAsset();
	UpdateTimeRanges();
	if (bUseLUT)
	{
		ShaderLUT = BuildLUT(CurveLUTDefaultWidth);
		OptimizeLUT();
		LUTNumSamplesMinusOne = float((ShaderLUT.Num() / GetCurveNumElems()) - 1);
	}

	if (!bUseLUT || (ShaderLUT.Num() == 0))
	{
		SetDefaultLUT();
	}

	if (!bFromSerialize)
	{
		UpdateExposedTexture();
	}

	MarkRenderDataDirty();
}

void UNiagaraDataInterfaceCurveBase::OptimizeLUT()
{
	// Do we optimize this LUT?
	if (!bOptimizeLUT)
	{
		return;
	}

	// Check error threshold is valid for us to optimize
	const float ErrorThreshold = bOverrideOptimizeThreshold ? OptimizeThreshold : GNiagaraLUTOptimizeThreshold;
	if (ErrorThreshold <= 0.0f)
	{
		return;
	}

	const int32 NumElements = GetCurveNumElems();
	check((ShaderLUT.Num() % NumElements) == 0);

	const int CurrNumSamples = ShaderLUT.Num() / NumElements;

	if (NumElements == 4)
	{
		for (int32 NewNumSamples = 1; NewNumSamples < CurrNumSamples; ++NewNumSamples)
		{
			TArray<float> ResampledLUT = BuildLUT(NewNumSamples);

			if (NiagaraDataInterfaceCurveBaseImpl::PassesErrorThresholdVectorized<4>(ShaderLUT, ResampledLUT, ErrorThreshold))
			{
				ShaderLUT = MoveTemp(ResampledLUT);
				break;
			}
		}
	}
	else
	{
		for (int32 NewNumSamples = 1; NewNumSamples < CurrNumSamples; ++NewNumSamples)
		{
			TArray<float> ResampledLUT = BuildLUT(NewNumSamples);

			if (NiagaraDataInterfaceCurveBaseImpl::PassesErrorThreshold(ShaderLUT, ResampledLUT, NumElements, ErrorThreshold))
			{
				ShaderLUT = MoveTemp(ResampledLUT);
				break;
			}
		}
	}
}

void UNiagaraDataInterfaceCurveBase::UpdateExposedTexture()
{
	if (bExposeCurve == false)
	{
		//-TODO: Do we need to invalidate the owning system to be safe??
		ExposedTexture = nullptr;
		return;
	}

	constexpr int32 CurveWidth = 256;

	// Build LUT
	TArray<FFloat16Color> Texels;
	{
		const int32 NumElements = GetCurveNumElems();
		const FLinearColor DefaultColors(1.0f, 1.0f, 1.0f, 1.0f);

		TArray<float> TempLUT = BuildLUT(CurveWidth);
		Texels.AddDefaulted(CurveWidth);
		for (int32 i = 0; i < CurveWidth; ++i)
		{
			Texels[i].R = TempLUT[(i * NumElements) + 0];
			Texels[i].G = NumElements >= 2 ? TempLUT[(i * NumElements) + 1] : DefaultColors.G;
			Texels[i].B = NumElements >= 3 ? TempLUT[(i * NumElements) + 2] : DefaultColors.B;
			Texels[i].A = NumElements >= 4 ? TempLUT[(i * NumElements) + 3] : DefaultColors.A;
		}
	}

	if (ExposedTexture == nullptr)
	{
		ExposedTexture = NewObject<UTexture2D>(this);
		ExposedTexture->Source.Init(CurveWidth, 1, 1, 1, ETextureSourceFormat::TSF_RGBA16F, reinterpret_cast<const uint8*>(Texels.GetData()));
		ExposedTexture->SRGB = false;
		ExposedTexture->CompressionNone = true;
		ExposedTexture->MipGenSettings = TMGS_NoMipmaps;
		ExposedTexture->AddressX = TA_Clamp;
		ExposedTexture->AddressY = TA_Clamp;
		ExposedTexture->LODGroup = TEXTUREGROUP_EffectsNotFiltered;
		ExposedTexture->SetDeterministicLightingGuid();

		ExposedTexture->UpdateResource();
	}
	else
	{
		ExposedTexture->PreEditChange(nullptr);
		ExposedTexture->Source.Init(CurveWidth, 1, 1, 1, ETextureSourceFormat::TSF_RGBA16F, reinterpret_cast<const uint8*>(Texels.GetData()));
		ExposedTexture->PostEditChange();

		// PostEditChange() will assign a random GUID to the texture, which leads to non-deterministic builds.
		ExposedTexture->SetDeterministicLightingGuid();
	}
}
#endif

bool UNiagaraDataInterfaceCurveBase::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfaceCurveBase* OtherTyped = CastChecked<UNiagaraDataInterfaceCurveBase>(Other);
	bool bEqual = OtherTyped->bUseLUT == bUseLUT;
#if WITH_EDITORONLY_DATA
	bEqual &= OtherTyped->CurveAsset == CurveAsset;
	bEqual &= OtherTyped->bOptimizeLUT == bOptimizeLUT;
	bEqual &= OtherTyped->bOverrideOptimizeThreshold == bOverrideOptimizeThreshold;
	if (bOverrideOptimizeThreshold)
	{
		bEqual &= OtherTyped->OptimizeThreshold == OptimizeThreshold;
	}
#endif
	bEqual &= OtherTyped->bExposeCurve == bExposeCurve;
	bEqual &= OtherTyped->ExposedName == ExposedName;
	if ( bEqual && bUseLUT )
	{
		bEqual &= OtherTyped->ShaderLUT == ShaderLUT;
	}

	return bEqual;
}

void UNiagaraDataInterfaceCurveBase::CacheStaticBuffers(struct FNiagaraSystemStaticBuffers& StaticBuffers, bool bIsExternalDataInterface, bool bUsedByCPU, bool bUsedByGPU)
{
	LUTOffset = INDEX_NONE;
	if (bUsedByGPU && bIsExternalDataInterface == false)
	{
		LUTOffset = StaticBuffers.AddGpuData(ShaderLUT);
	}
	MarkRenderDataDirty();
}

void UNiagaraDataInterfaceCurveBase::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FNiagaraDataInterfaceCurveParameters>();
}

void UNiagaraDataInterfaceCurveBase::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	const FNiagaraDataInterfaceProxyCurveBase& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyCurveBase>();
	FNiagaraDataInterfaceCurveParameters* Parameters = Context.GetParameterNestedStruct<FNiagaraDataInterfaceCurveParameters>();
	Parameters->MinTime = DIProxy.LUTMinTime;
	Parameters->MaxTime = DIProxy.LUTMaxTime;
	Parameters->InvTimeRange = DIProxy.LUTInvTimeRange;
	Parameters->CurveLUTNumMinusOne = DIProxy.CurveLUTNumMinusOne;
	Parameters->LUTOffset = DIProxy.LUTOffset;
	Parameters->CurveLUT = DIProxy.CurveLUT.SRV.IsValid() ? DIProxy.CurveLUT.SRV.GetReference() : FNiagaraRenderer::GetDummyFloatBuffer();
}

void UNiagaraDataInterfaceCurveBase::PostCompile(const UNiagaraSystem& OwningSystem)
{
#if WITH_EDITORONLY_DATA
	SyncCurvesToAsset();
#endif
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceCurveBase::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(NDICurve_TemplateShaderFile);
	InVisitor->UpdateShaderParameters<FNiagaraDataInterfaceCurveParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceCurveBase::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const int32 NumElements = GetCurveNumElems();
	check(NumElements > 0 && NumElements <= 4);

	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("UseUniqueBuffer"),			ParamInfo.IsExternalParameter() ? TEXT("1") : TEXT("0")},
		{TEXT("NumElements"),				FString::FromInt(NumElements)},
		{TEXT("CurveSampleFunctionName"),	GetCurveSampleFunctionName().ToString()},
	};
	AppendTemplateHLSL(OutHLSL, NDICurve_TemplateShaderFile, TemplateArgs);
}
#endif

void UNiagaraDataInterfaceCurveBase::GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const
{
	if (bExposeCurve)
	{
		OutVariables.Emplace(FNiagaraTypeDefinition(UTexture::StaticClass()), ExposedName);
	}
}

bool UNiagaraDataInterfaceCurveBase::GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const
{
	if (bExposeCurve && ExposedTexture != nullptr)
	{
		*reinterpret_cast<UObject**>(OutData) = ExposedTexture;
		return true;
	}

	return false;
}

void UNiagaraDataInterfaceCurveBase::PushToRenderThreadImpl()
{
	ENQUEUE_RENDER_COMMAND(FUpdateDICurve)(
		[
			RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxyCurveBase>(),
			RT_LUTMinTime=LUTMinTime,
			RT_LUTMaxTime=LUTMaxTime,
			RT_LUTInvTimeRange=LUTInvTimeRange,
			RT_CurveLUTNumMinusOne=LUTNumSamplesMinusOne,
			RT_LUTOffset=LUTOffset,
			RT_ShaderLUT=(LUTOffset == INDEX_NONE) ? ShaderLUT : TArray<float>()
		](FRHICommandListImmediate& RHICmdList)
		{
			RT_Proxy->LUTMinTime = RT_LUTMinTime;
			RT_Proxy->LUTMaxTime = RT_LUTMaxTime;
			RT_Proxy->LUTInvTimeRange = RT_LUTInvTimeRange;
			RT_Proxy->CurveLUTNumMinusOne = uint32(RT_CurveLUTNumMinusOne);
			RT_Proxy->LUTOffset = RT_LUTOffset;

			DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, RT_Proxy->CurveLUT.NumBytes);
			RT_Proxy->CurveLUT.Release();

			if (RT_ShaderLUT.Num() > 0)
			{
				RT_Proxy->CurveLUT.Initialize(RHICmdList, TEXT("CurveLUT"), sizeof(float), RT_ShaderLUT.Num(), EPixelFormat::PF_R32_FLOAT, BUF_Static);
				const uint32 BufferSize = RT_ShaderLUT.Num() * sizeof(float);
				void* BufferData = RHICmdList.LockBuffer(RT_Proxy->CurveLUT.Buffer, 0, BufferSize, EResourceLockMode::RLM_WriteOnly);
				FPlatformMemory::Memcpy(BufferData, RT_ShaderLUT.GetData(), BufferSize);
				RHICmdList.UnlockBuffer(RT_Proxy->CurveLUT.Buffer);
				INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, RT_Proxy->CurveLUT.NumBytes);
			}
		}
	);
}

#if WITH_EDITOR	
/** Refreshes and returns the errors detected with the corresponding data, if any.*/
TArray<FNiagaraDataInterfaceError> UNiagaraDataInterfaceCurveBase::GetErrors()
{
	// Trace down the root emitter (if there is one)
	TArray<FNiagaraDataInterfaceError> Errors;
	UNiagaraEmitter* Emitter = GetTypedOuter<UNiagaraEmitter>();

	// If there is a root emitter, assume that we are in its particle stack and point out that we need bUseLUT true for GPU sims.
	if (Emitter && Emitter->GetLatestEmitterData()->SimTarget == ENiagaraSimTarget::GPUComputeSim && !bUseLUT)
	{
		FNiagaraDataInterfaceError LUTsNeededForGPUSimsError(LOCTEXT("LUTsNeededForGPUSims", "This Data Interface must have bUseLUT set to true for GPU sims."),
			LOCTEXT("LUTsNeededForGPUSimsSummary", "bUseLUT Required"),
			FNiagaraDataInterfaceFix());

		Errors.Add(LUTsNeededForGPUSimsError);
	}
	return Errors;
}
#endif

#undef LOCTEXT_NAMESPACE
