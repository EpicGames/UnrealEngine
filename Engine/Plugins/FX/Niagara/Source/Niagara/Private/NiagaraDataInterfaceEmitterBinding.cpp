// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceEmitterBinding.h"
#include "NiagaraConstants.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraScriptRuntimeData.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceEmitterBinding)

FNiagaraEmitterInstance* FNiagaraDataInterfaceEmitterBinding::Resolve(const FNiagaraSystemInstance* SystemInstance, const UNiagaraDataInterface* DataInterface) const
{
	if (BindingMode == ENiagaraDataInterfaceEmitterBindingMode::Self)
	{
		// If we came from a particle script our outer will be a UNiagaraEmitter
		if (UNiagaraEmitter* OwnerEmitter = DataInterface->GetTypedOuter<UNiagaraEmitter>())
		{
			for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
			{
				if (EmitterInstance->GetEmitter() == OwnerEmitter)
				{
					return &EmitterInstance.Get();
				}
			}
		}
		// If we came from an emitter script we need to look over the default data interfaces to find ourself
		else
		{
			FString SourceEmitterName;
			UNiagaraSystem* NiagaraSystem = SystemInstance->GetSystem();
			for (UNiagaraScript* NiagaraScript : { NiagaraSystem->GetSystemUpdateScript(), NiagaraSystem->GetSystemSpawnScript() })
			{
				if (NiagaraScript != nullptr)
				{
					const FNiagaraScriptRuntimeData* ScriptRuntimeData = NiagaraSystem->GetScriptRuntimeData(nullptr, *NiagaraScript).Get();
					if (ScriptRuntimeData == nullptr)
					{
						continue;
					}
					const FNiagaraScriptResolvedDataInterfaceInfo* MatchingResolvedDataInterface = ScriptRuntimeData->GetResolvedDataInterfaces().FindByPredicate(
						[&DataInterface](const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDataInterface) { return ResolvedDataInterface.ResolvedDataInterface == DataInterface; });
					if (MatchingResolvedDataInterface != nullptr)
					{
						SourceEmitterName = MatchingResolvedDataInterface->ResolvedSourceEmitterName;
						break;
					}
				}
			}

			if (SourceEmitterName.IsEmpty() == false)
			{
				for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
				{
					if (const UNiagaraEmitter* CachedEmitter = EmitterInstance->GetEmitter())
					{
						if (CachedEmitter->GetUniqueEmitterName() == SourceEmitterName)
						{
							return &EmitterInstance.Get();
						}
					}
				}
			}
		}
		if ( FNiagaraUtilities::LogVerboseWarnings() )
		{
			UE_LOGF(LogNiagara, Warning, "EmitterBinding failed to find self emitter");
		}
	}
	else
	{
		if ( EmitterName.IsNone() )
		{
			if ( FNiagaraUtilities::LogVerboseWarnings() )
			{
				UE_LOGF(LogNiagara, Warning, "EmitterName has not been set but we are in Other mode");
			}
			return nullptr;
		}

		FNameBuilder EmitterNameString;
		EmitterName.ToString(EmitterNameString);
		FStringView EmitterNameStringView = EmitterNameString.ToView();

		for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
		{
			if (const UNiagaraEmitter* CachedEmitter = EmitterInstance->GetEmitter())
			{
				//-TODO: UniqueEmitterName should probably be a FName?
				if (EmitterNameStringView.Equals(CachedEmitter->GetUniqueEmitterName(), ESearchCase::IgnoreCase) )
				{
					return &EmitterInstance.Get();
				}
			}
		}

		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "EmitterBinding failed to find emitter '%ls' it might not exist or has been cooked out", *EmitterNameString);
		}
	}
	return nullptr;
}

const FNiagaraEmitterHandle* FNiagaraDataInterfaceEmitterBinding::ResolveHandle(const UNiagaraDataInterface* DataInterface) const
{
	UNiagaraSystem* OwnerSystem = DataInterface->GetTypedOuter<UNiagaraSystem>();
	if (OwnerSystem == nullptr)
	{
		return nullptr;
	}

	if (BindingMode == ENiagaraDataInterfaceEmitterBindingMode::Self)
	{
		// First check if this data interface is owned directly by an emitter.
		if (UNiagaraEmitter* OwnerEmitter = DataInterface->GetTypedOuter<UNiagaraEmitter>())
		{
			for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
			{
				if (OwnerEmitter == EmitterHandle.GetInstance().Emitter)
				{
					return &EmitterHandle;
				}
			}
		}
		else
		{
			// Try to find this data interface in one of the emitter's particle scripts.
			for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
			{
				FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
				if (EmitterHandle.GetIsEnabled() && EmitterData)
				{
					bool bFoundDI = false;
					bool bIsEmitterNamespace = false;
					EmitterData->ForEachScript(
						[OwnerSystem, &EmitterHandle, DataInterface, &bFoundDI, &bIsEmitterNamespace](const UNiagaraScript* Script)
						{
							const FNiagaraScriptRuntimeData* ScriptRuntimeData = OwnerSystem->GetScriptRuntimeData(&EmitterHandle, *Script).Get();
							if (ScriptRuntimeData == nullptr)
							{
								return;
							}
							for (const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDI : ScriptRuntimeData->GetResolvedDataInterfaces())
							{
								if (ResolvedDI.ResolvedDataInterface == DataInterface)
								{
									bFoundDI = true;
									bIsEmitterNamespace = FNiagaraVariableBase::IsInNameSpace(FNiagaraConstants::EmitterNamespaceString, ResolvedDI.CompileName);
									return;
								}
							}
						}
					);

					if (bFoundDI)
					{
						return bIsEmitterNamespace ? &EmitterHandle : nullptr;
					}
				}
			}
			// Check the system scripts since they contain the compiled emitter data.
			TArray<UNiagaraScript*> SystemScripts = { OwnerSystem->GetSystemSpawnScript(), OwnerSystem->GetSystemUpdateScript() };
			for (UNiagaraScript* SystemScript : SystemScripts)
			{
				const FNiagaraScriptRuntimeData* ScriptRuntimeData = OwnerSystem->GetScriptRuntimeData(nullptr, *SystemScript).Get();
				if (ScriptRuntimeData == nullptr)
				{
					continue;
				}
				for (const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDI : ScriptRuntimeData->GetResolvedDataInterfaces())
				{
					if (ResolvedDI.ResolvedDataInterface == DataInterface)
					{
						if (ResolvedDI.ResolvedSourceEmitterName.IsEmpty() == false)
						{
							return OwnerSystem->GetEmitterHandles().FindByPredicate([&ResolvedDI](const FNiagaraEmitterHandle& EmitterHandle)
								{ return EmitterHandle.GetUniqueInstanceName() == ResolvedDI.ResolvedSourceEmitterName; });
						}
					}
				}
			}
		}
	}
	else if (BindingMode == ENiagaraDataInterfaceEmitterBindingMode::Other)
	{
		if (!EmitterName.IsNone())
		{
			FNameBuilder EmitterNameString;
			EmitterName.ToString(EmitterNameString);
			FStringView EmitterNameStringView = EmitterNameString.ToView();

			for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
			{
				if (UNiagaraEmitter* NiagaraEmitter = EmitterHandle.GetInstance().Emitter)
				{
					//-TODO: UniqueEmitterName should probably be a FName?
					if (EmitterNameStringView.Equals(NiagaraEmitter->GetUniqueEmitterName(), ESearchCase::IgnoreCase))
					{
						return &EmitterHandle;
					}
				}
			}
		}
	}
	return nullptr;
}

const FNiagaraEmitterHandle* FNiagaraDataInterfaceEmitterBinding::ResolveHandle(const UNiagaraSystem* OwnerSystem, const FNiagaraEmitterHandle* OwnerEmitter) const
{
	check(OwnerSystem);

	if (BindingMode == ENiagaraDataInterfaceEmitterBindingMode::Self)
	{
		return OwnerEmitter;
	}
	else if (BindingMode == ENiagaraDataInterfaceEmitterBindingMode::Other)
	{
		if (!EmitterName.IsNone())
		{
			FNameBuilder EmitterNameString;
			EmitterName.ToString(EmitterNameString);
			FStringView EmitterNameStringView = EmitterNameString.ToView();

			for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
			{
				if (UNiagaraEmitter* NiagaraEmitter = EmitterHandle.GetInstance().Emitter)
				{
					//-TODO: UniqueEmitterName should probably be a FName?
					if (EmitterNameStringView.Equals(NiagaraEmitter->GetUniqueEmitterName(), ESearchCase::IgnoreCase))
					{
						return &EmitterHandle;
					}
				}
			}
		}
	}
	return nullptr;
}

UNiagaraEmitter* FNiagaraDataInterfaceEmitterBinding::Resolve(const UNiagaraDataInterface* DataInterface) const
{
	const FNiagaraEmitterHandle* EmitterHandle = ResolveHandle(DataInterface);
	return EmitterHandle ? EmitterHandle->GetInstance().Emitter : nullptr;
}

FString FNiagaraDataInterfaceEmitterBinding::ResolveUniqueName(const UNiagaraDataInterface* DataInterface) const
{
	if (UNiagaraEmitter* ResolvedEmitter = Resolve(DataInterface))
	{
		return ResolvedEmitter->GetUniqueEmitterName();
	}
	else
	{
		if (BindingMode == ENiagaraDataInterfaceEmitterBindingMode::Self)
		{
			return FString(TEXT("Self"));
		}
		else if (BindingMode == ENiagaraDataInterfaceEmitterBindingMode::Other)
		{
			return EmitterName.ToString();
		}
	}
	return FString();
}
