// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceExport.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraTypes.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGpuReadbackManager.h"
#include "NiagaraGPUSystemTick.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraWorldManager.h"

#include "Internationalization/Internationalization.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderCompilerCore.h"
#include "Misc/LargeWorldRenderPosition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceExport)

namespace NDIExportLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(uint32,						WriteBufferSize)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>,	RWWriteBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR*		TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceExportTemplate.ush");

	static const FName		StoreDataName_DEPRECATED(TEXT("StoreParticleData"));
	static const FName		ExportDataName(TEXT("ExportParticleData"));

	static constexpr uint32 NumFloatsPerInstance = 7;

	static int GGPUMaxReadbackCount = 1000;
	static FAutoConsoleVariableRef CVarGPUMaxReadbackCount(
		TEXT("fx.Niagara.NDIExport.GPUMaxReadbackCount"),
		GGPUMaxReadbackCount,
		TEXT("Maximum buffer instance count for the GPU readback when in PerParticleMode, where <= 0 means ignore."),
		ECVF_Default
	);

	void EnqueueGameThreadCallback(TWeakObjectPtr<UObject> WeakCallbackHandler, TArray<FBasicParticleData> ParticleData, TWeakObjectPtr<UNiagaraSystem> WeakSystem, FVector3f SystemTileOffset)
	{
		FNiagaraWorldManager::EnqueueGlobalDeferredCallback(
			[WeakCallbackHandler, ParticleData=MoveTemp(ParticleData), WeakSystem, SystemTileOffset]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraExportGTCallback);

				UNiagaraSystem* System = WeakSystem.Get();
				if (System == nullptr)
				{
					UE_LOGF(LogNiagara, Warning, "Invalid system handle in export data interface callback, skipping");
					return;
				}

				UObject* CallbackHandler = WeakCallbackHandler.Get();
				if (CallbackHandler == nullptr)
				{
					UE_LOGF(LogNiagara, Warning, "Invalid CallbackHandler in export data interface callback, skipping");
					return;
				}

				INiagaraParticleCallbackHandler::Execute_ReceiveParticleData(CallbackHandler, ParticleData, System, FVector(SystemTileOffset) * FLargeWorldRenderScalar::GetTileSize());
			}
		);
	}
}

//////////////////////////////////////////////////////////////////////////
// Instance and Proxy Data

struct FNDIExportInstanceData_GameThread
{
	TWeakObjectPtr<UObject> CallbackHandler;

	/** We use a lock-free queue here because multiple threads might try to push data to it at the same time. */
	TQueue<FBasicParticleData, EQueueMode::Mpsc> GatheredData;

	/** A binding to the user ptr we're reading the handler from. */
	FNiagaraParameterDirectBinding<UObject*> UserParamBinding;
};

struct FNDIExportInstanceData_RenderThread
{
	ENDIExport_GPUAllocationMode	AllocationMode = ENDIExport_GPUAllocationMode::FixedSize;
	int								AllocationFixedSize = 0;
	float							AllocationPerParticleSize = 0.0f;

	TWeakObjectPtr<UObject>			WeakCallbackHandler;
	TWeakObjectPtr<UNiagaraSystem>	WeakSystem;

	bool							bWriteBufferUsed = false;
	uint32							WriteBufferInstanceCount = 0;

	FRDGBufferRef					RDGWriteBuffer = nullptr;
	FRDGBufferUAVRef				RDGWriteBufferUAV = nullptr;
};

struct FNDIExportProxy : public FNiagaraDataInterfaceProxy
{
	FNDIExportProxy() {}
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override {}
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return 0;
	}

	virtual void PostStage(const FNDIGpuComputePostStageContext& Context) override
	{
		FNDIExportInstanceData_RenderThread& InstanceData = SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());

		// If we bound the buffer as a UAV the stage may have written to it
		if (InstanceData.bWriteBufferUsed)
		{
			InstanceData.bWriteBufferUsed = false;

			FNiagaraGpuReadbackManager* ReadbackManager = Context.GetComputeDispatchInterface().GetGpuReadbackManager();
			ReadbackManager->EnqueueReadback(
				Context.GetGraphBuilder(),
				InstanceData.RDGWriteBuffer,
				[MaxInstances= InstanceData.WriteBufferInstanceCount, WeakCallbackHandler=InstanceData.WeakCallbackHandler, WeakSystem=InstanceData.WeakSystem, SystemLWCTile= Context.GetSystemLWCTile()](TConstArrayView<TPair<void*, uint32>> Buffers)
				{
					const uint32 ReadbackInstanceCount = *reinterpret_cast<uint32*>(Buffers[0].Key);
					check(ReadbackInstanceCount <= MaxInstances);
					if (ReadbackInstanceCount > 0)
					{
						// Translate float data into Export Data
						TArray<FBasicParticleData> ExportParticleData;
						ExportParticleData.AddUninitialized(ReadbackInstanceCount);

						const float* FloatData = reinterpret_cast<float*>(Buffers[0].Key) + 1;
						for (uint32 i = 0; i < ReadbackInstanceCount; ++i)
						{
							ExportParticleData[i].Position.X = FloatData[0];
							ExportParticleData[i].Position.Y = FloatData[1];
							ExportParticleData[i].Position.Z = FloatData[2];
							ExportParticleData[i].Size = FloatData[3];
							ExportParticleData[i].Velocity.X = FloatData[4];
							ExportParticleData[i].Velocity.Y = FloatData[5];
							ExportParticleData[i].Velocity.Z = FloatData[6];

							FloatData += NDIExportLocal::NumFloatsPerInstance;
						}

						NDIExportLocal::EnqueueGameThreadCallback(WeakCallbackHandler, MoveTemp(ExportParticleData), WeakSystem, SystemLWCTile);
					}
				}
			);
		}
		InstanceData.RDGWriteBuffer = nullptr;
		InstanceData.RDGWriteBufferUAV = nullptr;
	}

	TMap<FNiagaraSystemInstanceID, FNDIExportInstanceData_RenderThread> SystemInstancesToProxyData_RT;
};

//////////////////////////////////////////////////////////////////////////
// Data Interface

UNiagaraDataInterfaceExport::UNiagaraDataInterfaceExport(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNDIExportProxy());

	FNiagaraTypeDefinition Def(UObject::StaticClass());
	CallbackHandlerParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceExport::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceExport::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIExportInstanceData_GameThread* PIData = new (PerInstanceData) FNDIExportInstanceData_GameThread;

	ENQUEUE_RENDER_COMMAND(FNDIExport_AddProxy)
	(
		[RT_Proxy=GetProxyAs<FNDIExportProxy>(), RT_InstanceID=SystemInstance->GetId(), RT_AllocationMode=GPUAllocationMode, RT_AllocationFixedSize=GPUAllocationFixedSize, RT_AllocationPerParticleSize=GPUAllocationPerParticleSize](FRHICommandListImmediate& RHICmdList)
		{
			check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(RT_InstanceID));
			FNDIExportInstanceData_RenderThread* InstanceData = &RT_Proxy->SystemInstancesToProxyData_RT.Add(RT_InstanceID);
			check(InstanceData);

			InstanceData->AllocationMode = RT_AllocationMode;
			InstanceData->AllocationFixedSize = RT_AllocationFixedSize;
			InstanceData->AllocationPerParticleSize = RT_AllocationPerParticleSize;
		}
	);

	return true;
}

void UNiagaraDataInterfaceExport::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIExportInstanceData_GameThread* InstData = (FNDIExportInstanceData_GameThread*)PerInstanceData;
	InstData->~FNDIExportInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(FNDIExport_RemoveProxy)
	(
		[RT_Proxy=GetProxyAs<FNDIExportProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		}
	);
}

int32 UNiagaraDataInterfaceExport::PerInstanceDataSize() const
{
	return sizeof(FNDIExportInstanceData_GameThread);
}


bool UNiagaraDataInterfaceExport::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	check(SystemInstance);
	FNDIExportInstanceData_GameThread* PIData = (FNDIExportInstanceData_GameThread*)PerInstanceData;
	if (!PIData)
	{
		return true;
	}

	TWeakObjectPtr<UObject> PreviousCallbackHandler = PIData->CallbackHandler;
	if (CallbackHandlerParameter.Parameter.IsValid() && SystemInstance)
	{
		//-OPT: Don't need to init this per frame, could be bound once and read per frame
		PIData->CallbackHandler = PIData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), CallbackHandlerParameter.Parameter);
	}
	else
	{
		PIData->CallbackHandler.Reset();
	}

	// If we switched modify the RT data
	if (PreviousCallbackHandler != PIData->CallbackHandler)
	{
		ENQUEUE_RENDER_COMMAND(FNDIExport_UpdateHandler)
		(
			[RT_Proxy=GetProxyAs<FNDIExportProxy>(), RT_InstanceID=SystemInstance->GetId(), RT_WeakHandler=PIData->CallbackHandler, RT_WeakSystem=SystemInstance->GetSystem()](FRHICommandListImmediate& RHICmdList)
			{
				FNDIExportInstanceData_RenderThread* InstanceData = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID);
				if (ensure(InstanceData != nullptr))
				{
					InstanceData->WeakCallbackHandler = RT_WeakHandler;
					InstanceData->WeakSystem = RT_WeakSystem;
				}
			}
		);
	}

	return false;
}

bool UNiagaraDataInterfaceExport::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FNDIExportInstanceData_GameThread* PIData = static_cast<FNDIExportInstanceData_GameThread*>(PerInstanceData);
	if ( !PIData->GatheredData.IsEmpty() )
	{
		UObject* CallbackHandler = PIData->CallbackHandler.Get();
		if ( CallbackHandler && CallbackHandler->GetClass()->ImplementsInterface(UNiagaraParticleCallbackHandler::StaticClass()) )
		{
			//Drain the queue into an array here
			TArray<FBasicParticleData> Data;
			FBasicParticleData Value;
			while (PIData->GatheredData.Dequeue(Value))
			{
				Data.Add(Value);
			}

			NDIExportLocal::EnqueueGameThreadCallback(CallbackHandler, MoveTemp(Data), SystemInstance->GetSystem(), SystemInstance->GetLWCTile());
		}
	}
	return false;
}

bool UNiagaraDataInterfaceExport::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfaceExport* OtherTyped = CastChecked<const UNiagaraDataInterfaceExport>(Other);

	return
		OtherTyped->CallbackHandlerParameter == CallbackHandlerParameter &&
		OtherTyped->GPUAllocationMode == GPUAllocationMode &&
		OtherTyped->GPUAllocationFixedSize == GPUAllocationFixedSize &&
		OtherTyped->GPUAllocationPerParticleSize == GPUAllocationPerParticleSize;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceExport::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature SigOld;
	SigOld.Name = NDIExportLocal::StoreDataName_DEPRECATED;
	SigOld.Description = NSLOCTEXT("Niagara", "ExportDataFunctionDescription", "This function takes the particle data and stores it to be exported to the registered callback handler after the simulation has ticked.");
	SigOld.bMemberFunction = true;
	SigOld.bRequiresContext = false;
	SigOld.bSupportsGPU = true;
	SigOld.bSoftDeprecatedFunction = true;
	SigOld.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Export interface")));
	SigOld.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Store Data")));
	SigOld.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position")));
	SigOld.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Size")));
	SigOld.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
	SigOld.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));
	OutFunctions.Add(SigOld);

	FNiagaraFunctionSignature Sig;
	Sig.Name = NDIExportLocal::ExportDataName;
	Sig.Description = NSLOCTEXT("Niagara", "ExportDataFunctionDescription", "This function takes the particle data and stores it to be exported to the registered callback handler after the simulation has ticked.");
	Sig.bMemberFunction = true;
	Sig.bRequiresContext = false;
	Sig.bSupportsGPU = true;
	Sig.bRequiresExecPin = true;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Export interface")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Store Data")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Size")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
	Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("WasExported")));
	OutFunctions.Add(Sig);
}

void UNiagaraDataInterfaceExport::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIExportLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceExport::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIExportLocal;

	if ( (FunctionInfo.DefinitionName == StoreDataName_DEPRECATED) || (FunctionInfo.DefinitionName == ExportDataName) )
	{
		return true;
	}

	// Invalid function
	return false;
}

bool UNiagaraDataInterfaceExport::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIExportLocal::TemplateShaderFile);
	return bSuccess;
}
#endif

void UNiagaraDataInterfaceExport::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIExportLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceExport::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNDIExportProxy& DIProxy = Context.GetProxy<FNDIExportProxy>();
	FNDIExportInstanceData_RenderThread& InstanceData_RT = DIProxy.SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());

	NDIExportLocal::FShaderParameters* Parameters = Context.GetParameterNestedStruct<NDIExportLocal::FShaderParameters>();
	if ( Context.IsResourceBound(&Parameters->RWWriteBuffer) )
	{
		if ( !InstanceData_RT.WeakCallbackHandler.IsExplicitlyNull() )
		{
			FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
			if (InstanceData_RT.RDGWriteBuffer == nullptr)
			{
				// Ensure our buffer if big enough to hold all the data
				const int32 AllocationRounding = 64;
				const int32 DataBufferNumInstances = Context.GetComputeInstanceData().Context->CurrentMaxInstances_RT;
				int32 NumInstances = 0;
				if (InstanceData_RT.AllocationMode == ENDIExport_GPUAllocationMode::PerParticle)
				{
					NumInstances = int32(float(DataBufferNumInstances) * InstanceData_RT.AllocationPerParticleSize);
					NumInstances = FMath::Max(NumInstances, 0);
					if (NDIExportLocal::GGPUMaxReadbackCount > 0)
					{
						NumInstances = FMath::Min(NumInstances, NDIExportLocal::GGPUMaxReadbackCount);
					}
				}
				else
				{
					NumInstances = InstanceData_RT.AllocationFixedSize;
				}

				InstanceData_RT.WriteBufferInstanceCount = NumInstances;

				const int32 AllocationNumInstances = FMath::DivideAndRoundUp(NumInstances, AllocationRounding) * AllocationRounding;
				const int32 RequiredElements = 1 + (AllocationNumInstances * NDIExportLocal::NumFloatsPerInstance);
				const int32 RequiredBytes = sizeof(float) * RequiredElements;

				FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), RequiredElements);
				BufferDesc.Usage |= BUF_SourceCopy;
				InstanceData_RT.RDGWriteBuffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("FNDIExportProxy_WriteBuffer"));
				InstanceData_RT.RDGWriteBufferUAV = GraphBuilder.CreateUAV(InstanceData_RT.RDGWriteBuffer, PF_R32_UINT);

			}

			//-OPT: We only need to reset the first 4 bytes for the count
			//-OPT: We only need to clear if we actual use the buffer
			AddClearUAVPass(GraphBuilder, InstanceData_RT.RDGWriteBufferUAV, 0);

			InstanceData_RT.bWriteBufferUsed = true;
			Parameters->WriteBufferSize = InstanceData_RT.WriteBufferInstanceCount;
			Parameters->RWWriteBuffer = InstanceData_RT.RDGWriteBufferUAV;
		}
		else
		{
			Parameters->WriteBufferSize = 0;
			Parameters->RWWriteBuffer = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(Context.GetGraphBuilder(), PF_R32_UINT);
		}
	}
	else
	{
		Parameters->WriteBufferSize	= 0;
		Parameters->RWWriteBuffer = nullptr;
	}
}

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceExport, StoreData);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceExport, ExportData);
void UNiagaraDataInterfaceExport::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == NDIExportLocal::StoreDataName_DEPRECATED)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceExport, StoreData)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == NDIExportLocal::ExportDataName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceExport, ExportData)::Bind(this, OutFunc);
	}
	else
	{
		UE_LOGF(LogNiagara, Display, "Could not find data interface external function in %ls. Expected Name: %ls  Actual Name: %ls", *GetPathNameSafe(this), *NDIExportLocal::ExportDataName.ToString(), *BindingInfo.Name.ToString());
	}
}

void UNiagaraDataInterfaceExport::StoreData(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIExportInstanceData_GameThread> InstData(Context);

	FNDIInputParam<FNiagaraBool> StoreDataParam(Context);
	FNDIInputParam<FVector3f> PositionParam(Context);
	FNDIInputParam<float> SizeParam(Context);
	FNDIInputParam<FVector3f> VelocityParam(Context);

	FNDIOutputParam<FNiagaraBool> OutSample(Context);

	checkfSlow(InstData.Get(), TEXT("Export data interface has invalid instance data. %s"), *GetPathName());
	bool ValidHandlerData = InstData->UserParamBinding.BoundVariable.IsValid() && InstData->CallbackHandler.IsValid();

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool ShouldStore = StoreDataParam.GetAndAdvance();

		FBasicParticleData Data;
		Data.Position = (FVector)PositionParam.GetAndAdvance();
		Data.Size = SizeParam.GetAndAdvance();
		Data.Velocity = (FVector)VelocityParam.GetAndAdvance();

		bool Valid = false;
		if (ValidHandlerData && ShouldStore)
		{
			Valid = InstData->GatheredData.Enqueue(Data);
		}
		OutSample.SetAndAdvance(Valid);
	}
}

void UNiagaraDataInterfaceExport::ExportData(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIExportInstanceData_GameThread> InstData(Context);

	FNDIInputParam<FNiagaraBool> StoreDataParam(Context);
	FNDIInputParam<FVector3f> PositionParam(Context);
	FNDIInputParam<float> SizeParam(Context);
	FNDIInputParam<FVector3f> VelocityParam(Context);

	FNDIOutputParam<FNiagaraBool> OutWasExported(Context);

	checkfSlow(InstData.Get(), TEXT("Export data interface has invalid instance data. %s"), *GetPathName());
	bool ValidHandlerData = InstData->UserParamBinding.BoundVariable.IsValid() && InstData->CallbackHandler.IsValid();

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const bool ShouldStore = StoreDataParam.GetAndAdvance();

		FBasicParticleData Data;
		Data.Position = (FVector)PositionParam.GetAndAdvance();
		Data.Size = SizeParam.GetAndAdvance();
		Data.Velocity = (FVector)VelocityParam.GetAndAdvance();

		bool Valid = false;
		if (ValidHandlerData && ShouldStore)
		{
			Valid = InstData->GatheredData.Enqueue(Data);
		}
		OutWasExported.SetAndAdvance(Valid);
	}
}

bool UNiagaraDataInterfaceExport::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceExport* OtherTyped = CastChecked<UNiagaraDataInterfaceExport>(Destination);
	OtherTyped->CallbackHandlerParameter = CallbackHandlerParameter;
	OtherTyped->GPUAllocationMode = GPUAllocationMode;
	OtherTyped->GPUAllocationFixedSize = GPUAllocationFixedSize;
	OtherTyped->GPUAllocationPerParticleSize = GPUAllocationPerParticleSize;
	return true;
}

