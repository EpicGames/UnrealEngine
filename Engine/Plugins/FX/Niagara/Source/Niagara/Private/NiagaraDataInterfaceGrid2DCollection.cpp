// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceGrid2DCollection.h"
#include "CanvasTypes.h"
#include "NiagaraBatchedElements.h"
#include "GlobalRenderResources.h"
#include "NiagaraConstants.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraGpuComputeDebugInterface.h"
#include "NiagaraComponent.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraEmitter.h"
#include "NiagaraRenderer.h"
#include "NiagaraRenderGraphUtils.h"
#include "NiagaraSettings.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSimStageData.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraScriptRuntimeData.h"

#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "CanvasItem.h"
#include "RHIStaticStates.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterMetadataBuilder.h"
#include "TextureResource.h"

#include "NiagaraDataInterfaceGrid2DCollectionReader.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceGrid2DCollection)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceGrid2DCollection"

IMPLEMENT_TYPE_LAYOUT(FNiagaraDataInterfaceParametersCS_Grid2DCollection);

BEGIN_SHADER_PARAMETER_STRUCT(FNDIGrid2DShaderParameters, )
	SHADER_PARAMETER(int,							NumAttributes)
	SHADER_PARAMETER(FVector2f,						UnitToUV)
	SHADER_PARAMETER(FIntPoint,						NumCells)
	SHADER_PARAMETER(FVector2f,						CellSize)
	SHADER_PARAMETER(FVector2f,						WorldBBoxSize)

	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<float>,		Grid)
	SHADER_PARAMETER_SAMPLER(SamplerState,						GridSampler)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float>,	OutputGrid)
END_SHADER_PARAMETER_STRUCT()

static int32 GNiagaraGrid2DCubicInterpMethod = 0;
static FAutoConsoleVariableRef CVarNiagaraGrid2DCubicInterpMethod(
	TEXT("fx.Niagara.Grid2D.CubicInterpMethod"),
	GNiagaraGrid2DCubicInterpMethod,
	TEXT("0=Bridson, 1=monotonic/Fedkiw.\n"),
	ECVF_Default
);

static constexpr EPixelFormatCapabilities GNDIGrid2DFomatCaps = EPixelFormatCapabilities::TypedUAVLoad | EPixelFormatCapabilities::TypedUAVStore | EPixelFormatCapabilities::Texture2D;

const FString UNiagaraDataInterfaceGrid2DCollection::GridName(TEXT("_Grid"));
const FString UNiagaraDataInterfaceGrid2DCollection::OutputGridName(TEXT("_OutputGrid"));
const FString UNiagaraDataInterfaceGrid2DCollection::SamplerName(TEXT("_GridSampler"));

const FName UNiagaraDataInterfaceGrid2DCollection::SetNumCellsFunctionName("SetNumCells");

// Global VM function names, also used by the shaders code generation methods.
const FName UNiagaraDataInterfaceGrid2DCollection::SetValueFunctionName("SetGridValue");
const FName UNiagaraDataInterfaceGrid2DCollection::GetValueFunctionName("GetGridValue");
const FName UNiagaraDataInterfaceGrid2DCollection::SetVector4ValueFunctionName("SetVector4Value");
const FName UNiagaraDataInterfaceGrid2DCollection::GetVector4ValueFunctionName("GetVector4Value");
const FName UNiagaraDataInterfaceGrid2DCollection::SampleGridVector4FunctionName("SampleGridVector4Value");
const FName UNiagaraDataInterfaceGrid2DCollection::SetVector3ValueFunctionName("SetVector3Value");
const FName UNiagaraDataInterfaceGrid2DCollection::GetVector3ValueFunctionName("GetVector3Value");
const FName UNiagaraDataInterfaceGrid2DCollection::SampleGridVector3FunctionName("SampleGridVector3Value");
const FName UNiagaraDataInterfaceGrid2DCollection::SetVector2ValueFunctionName("SetVector2Value");
const FName UNiagaraDataInterfaceGrid2DCollection::GetVector2ValueFunctionName("GetVector2Value");
const FName UNiagaraDataInterfaceGrid2DCollection::SampleGridVector2FunctionName("SampleGridVector2Value");
const FName UNiagaraDataInterfaceGrid2DCollection::SetFloatValueFunctionName("SetFloatValue");
const FName UNiagaraDataInterfaceGrid2DCollection::GetFloatValueFunctionName("GetFloatValue");
const FName UNiagaraDataInterfaceGrid2DCollection::SampleGridFloatFunctionName("SampleGridFloatValue");

const FName UNiagaraDataInterfaceGrid2DCollection::SetValueAtIndexFunctionName("SetValueAtIndex");
const FName UNiagaraDataInterfaceGrid2DCollection::GetPreviousValueAtIndexFunctionName("GetPreviousValueAtIndex");
const FName UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridAtIndexFunctionName("SamplePreviousGridAtIndex");
const FName UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridAtIndexFunctionName("CubicSamplePreviousGridAtIndex");

const FName UNiagaraDataInterfaceGrid2DCollection::GetPreviousVector4ValueFunctionName("GetPreviousVector4Value");
const FName UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVector4FunctionName("SamplePreviousGridVector4Value");
const FName UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridVector4FunctionName("CubicSamplePreviousGridVector4Value");
const FName UNiagaraDataInterfaceGrid2DCollection::SetVectorValueFunctionName("SetVectorValue");
const FName UNiagaraDataInterfaceGrid2DCollection::GetPreviousVectorValueFunctionName("GetPreviousVectorValue");
const FName UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVectorFunctionName("SamplePreviousGridVector3Value");
const FName UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridVectorFunctionName("CubicSamplePreviousGridVector3Value");
const FName UNiagaraDataInterfaceGrid2DCollection::SetVector2DValueFunctionName("SetVector2DValue");
const FName UNiagaraDataInterfaceGrid2DCollection::GetPreviousVector2DValueFunctionName("GetPreviousVector2DValue");
const FName UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVector2DFunctionName("SamplePreviousGridVector2DValue");
const FName UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridVector2DFunctionName("CubicSamplePreviousGridVector2DValue");
const FName UNiagaraDataInterfaceGrid2DCollection::GetPreviousFloatValueFunctionName("GetPreviousFloatValue");
const FName UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridFloatFunctionName("SamplePreviousGridFloatValue");
const FName UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridFloatFunctionName("CubicSamplePreviousGridFloatValue");

const FName UNiagaraDataInterfaceGrid2DCollection::GetVector4AttributeIndexFunctionName("GetVector4AttributeIndex");
const FName UNiagaraDataInterfaceGrid2DCollection::GetVectorAttributeIndexFunctionName("GetVectorAttributeIndex");
const FName UNiagaraDataInterfaceGrid2DCollection::GetVector2DAttributeIndexFunctionName("GetVector2DAttributeIndex");
const FName UNiagaraDataInterfaceGrid2DCollection::GetFloatAttributeIndexFunctionName("GetFloatAttributeIndex");

const FString UNiagaraDataInterfaceGrid2DCollection::AnonymousAttributeString("Attribute At Index");

const FName UNiagaraDataInterfaceGrid2DCollection::ClearCellFunctionName("ClearCell");
const FName UNiagaraDataInterfaceGrid2DCollection::CopyPreviousToCurrentForCellFunctionName("CopyPreviousToCurrentForCell");

const FString UNiagaraDataInterfaceGrid2DCollection::AttributeIndicesBaseName(TEXT("AttributeIndices"));
const TCHAR* UNiagaraDataInterfaceGrid2DCollection::VectorComponentNames[] = { TEXT(".x"), TEXT(".y"), TEXT(".z"), TEXT(".w") };

const FName UNiagaraDataInterfaceGrid2DCollection::SampleGridFunctionName("SampleGrid");

FNiagaraVariableBase UNiagaraDataInterfaceGrid2DCollection::ExposedRTVar;

const TCHAR* UNiagaraDataInterfaceGrid2DCollection::TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceGrid2DCollection.ush");

struct FNiagaraGridCollection2DDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		AddReadMemberToFunctions = 1,
		AddExecuteToSetNumCells = 2,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};


bool UNiagaraDataInterfaceGrid2DCollection::CanCreateVarFromFuncName(const FName& FuncName)
{
	if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector4ValueFunctionName	|| FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVector4ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridVector4FunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousVector4ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVector4FunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridVector4FunctionName)
		return true;
	else if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector3ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVectorValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVector3ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridVector3FunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousVectorValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVectorFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridVectorFunctionName)
		return true;
	else if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector2ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector2DValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVector2ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridVector2FunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousVector2DValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVector2DFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridVector2DFunctionName)
		return true;
	else if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetFloatValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetFloatValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridFloatFunctionName       || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousFloatValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridFloatFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridFloatFunctionName)
		return true;
	return false;
}

FNiagaraTypeDefinition UNiagaraDataInterfaceGrid2DCollection::GetValueTypeFromFuncName(const FName& FuncName)
{
	if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector4ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVector4ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridVector4FunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousVector4ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVector4FunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridVector4FunctionName)
		return FNiagaraTypeDefinition::GetVec4Def();
	else if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector3ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVectorValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVector3ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridVector3FunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousVectorValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVectorFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridVectorFunctionName)
		return FNiagaraTypeDefinition::GetVec3Def();
	else if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector2ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector2DValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVector2ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridVector2FunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousVector2DValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVector2DFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridVector2DFunctionName)
		return FNiagaraTypeDefinition::GetVec2Def();
	else if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetFloatValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetFloatValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridFloatFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousFloatValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridFloatFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::CubicSamplePreviousGridFloatFunctionName)
		return FNiagaraTypeDefinition::GetFloatDef();

	return FNiagaraTypeDefinition();
}

int32 UNiagaraDataInterfaceGrid2DCollection::GetComponentCountFromFuncName(const FName& FuncName)
{
	if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector4ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVector4ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridVector4FunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVector4AttributeIndexFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousVector4ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVector4FunctionName)
		return 4;
	else if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector3ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVectorValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVector3ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridVector3FunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVectorAttributeIndexFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousVectorValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVectorFunctionName)
		return 3;
	else if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector2ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SetVector2DValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVector2ValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridVector2FunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetVector2DAttributeIndexFunctionName|| FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousVector2DValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridVector2DFunctionName)
		return 2;
	else if (FuncName == UNiagaraDataInterfaceGrid2DCollection::SetFloatValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetFloatValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SampleGridFloatFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::GetPreviousFloatValueFunctionName || FuncName == UNiagaraDataInterfaceGrid2DCollection::SamplePreviousGridFloatFunctionName|| FuncName == UNiagaraDataInterfaceGrid2DCollection::GetFloatAttributeIndexFunctionName)
		return 1;

	return INDEX_NONE;
}

static float GNiagaraGrid2DResolutionMultiplier = 1.0f;
static FAutoConsoleVariableRef CVarNiagaraGrid2DResolutionMultiplier(
	TEXT("fx.Niagara.Grid2D.ResolutionMultiplier"),
	GNiagaraGrid2DResolutionMultiplier,
	TEXT("Optional global modifier to grid resolution\n"),
	ECVF_Default
);

static int32 GNiagaraGrid2DOverrideFormat = -1;
static FAutoConsoleVariableRef CVarNiagaraGrid2DOverrideFormat(
	TEXT("fx.Niagara.Grid2D.OverrideFormat"),
	GNiagaraGrid2DOverrideFormat,
	TEXT("Optional override for all grids to use this format.\n"),
	ECVF_Default
);

/*--------------------------------------------------------------------------------------------------------------------------*/
// Helper class to translate between Arrays and 2D textures
struct FNiagaraGrid2DLegacyTiled2DInfo
{
	FNiagaraGrid2DLegacyTiled2DInfo(const FIntPoint& InNumCells, int InNumAttributes)
		: NumAttributes(InNumAttributes)
		, NumCells(InNumCells)
	{
		const int MaxTilesX = FMath::DivideAndRoundDown<int>(GMaxTextureDimensions, NumCells.X);
		const int MaxTilesY = FMath::DivideAndRoundDown<int>(GMaxTextureDimensions, NumCells.Y);
		const int MaxAttributes = MaxTilesX * MaxTilesY;
		if (NumAttributes <= MaxAttributes)
		{
			bIsValid = true;

			NumTiles.X = NumAttributes <= MaxTilesX ? NumAttributes : MaxTilesX;
			NumTiles.Y = FMath::DivideAndRoundUp(NumAttributes, NumTiles.X);

			Size.X = NumCells.X * NumTiles.X;
			Size.Y = NumCells.Y * NumTiles.Y;
		}
	}

	void CopyTo2D(FRDGBuilder& GraphBuilder, FRDGTexture* SourceTexture, FRHITexture* DestinationTextureRHI, const TCHAR* NameIfNotRegistered) const
	{
		const FRHITextureDesc& DestTextureDesc = DestinationTextureRHI->GetDesc();
		if (ensureMsgf(DestTextureDesc.Dimension != ETextureDimension::Texture2D, TEXT("Destination for Grid2D copy is invalid type (%d), copy will fail"), EnumToUnderlyingType(DestTextureDesc.Dimension)))
		{
			return;
		}

		FRDGTexture* DestinationTexture = GraphBuilder.FindExternalTexture(DestinationTextureRHI);
		if (DestinationTexture == nullptr)
		{
			DestinationTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DestinationTextureRHI, NameIfNotRegistered));
		}

		for (int iAttribute = 0; iAttribute < NumAttributes; ++iAttribute)
		{
			FRHICopyTextureInfo CopyInfo;
			CopyInfo.Size = FIntVector(NumCells.X, NumCells.Y, 1);
			CopyInfo.SourceSliceIndex = iAttribute;
			CopyInfo.DestPosition.X = (iAttribute % NumTiles.X) * NumCells.X;
			CopyInfo.DestPosition.Y = (iAttribute / NumTiles.X) * NumCells.Y;
			CopyInfo.DestPosition.Z = 0;
			if (ensureMsgf(
					(CopyInfo.DestPosition.X + CopyInfo.Size.X <= DestTextureDesc.Extent.X) && (CopyInfo.DestPosition.Y + CopyInfo.Size.Y <= DestTextureDesc.Extent.Y),
					TEXT("Destination for Grid2D can not hold all the attributes, copy will fail.")))
			{
				AddCopyTexturePass(GraphBuilder, SourceTexture, DestinationTexture, CopyInfo);
			}
		}
	}

	void CopyTo2D(FRHICommandList& RHICmdList, FRHITexture* Src, FRHITexture* Dst) const
	{
		check(Src != nullptr && Dst != nullptr);

		const FRHITextureDesc& DestTextureDesc = Dst->GetDesc();
		if (ensureMsgf(DestTextureDesc.Dimension != ETextureDimension::Texture2D, TEXT("Destination for Grid2D copy is invalid type (%d), copy will fail"), EnumToUnderlyingType(DestTextureDesc.Dimension)))
		{
			return;
		}

		FRHITransitionInfo TransitionsBefore[] = {
			FRHITransitionInfo(Src, ERHIAccess::SRVMask, ERHIAccess::CopySrc),
			FRHITransitionInfo(Dst, ERHIAccess::SRVMask, ERHIAccess::CopyDest)
		};
		RHICmdList.Transition(MakeArrayView(TransitionsBefore, UE_ARRAY_COUNT(TransitionsBefore)));

		for (int iAttribute = 0; iAttribute < NumAttributes; ++iAttribute)
		{
			FRHICopyTextureInfo CopyInfo;
			CopyInfo.Size = FIntVector(NumCells.X, NumCells.Y, 1);
			CopyInfo.SourceSliceIndex = iAttribute;
			CopyInfo.DestPosition.X = (iAttribute % NumTiles.X) * NumCells.X;
			CopyInfo.DestPosition.Y = (iAttribute / NumTiles.X) * NumCells.Y;
			CopyInfo.DestPosition.Z = 0;
			if (ensureMsgf(
				(CopyInfo.DestPosition.X + CopyInfo.Size.X <= DestTextureDesc.Extent.X) && (CopyInfo.DestPosition.Y + CopyInfo.Size.Y <= DestTextureDesc.Extent.Y),
				TEXT("Destination for Grid2D can not hold all the attributes, copy will fail.")))
			{
				RHICmdList.CopyTexture(Src, Dst, CopyInfo);
			}
		}

		FRHITransitionInfo TransitionsAfter[] = {
			FRHITransitionInfo(Src, ERHIAccess::CopySrc, ERHIAccess::SRVMask),
			FRHITransitionInfo(Dst, ERHIAccess::CopyDest, ERHIAccess::SRVMask)
		};
		RHICmdList.Transition(MakeArrayView(TransitionsAfter, UE_ARRAY_COUNT(TransitionsAfter)));
	}

	bool bIsValid = false;
	int NumAttributes = 0;
	FIntPoint NumCells = FIntPoint::ZeroValue;
	FIntPoint NumTiles = FIntPoint::ZeroValue;
	FIntPoint Size = FIntPoint::ZeroValue;
};

UNiagaraDataInterfaceGrid2DCollection::UNiagaraDataInterfaceGrid2DCollection(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer), OverrideBufferFormat(ENiagaraGpuBufferFormat::Float), bOverrideFormat(false)
#if WITH_EDITORONLY_DATA
	, bPreviewGrid(false)
#endif
	, bUseIndirection(false)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyGrid2DCollectionProxy());

	FNiagaraTypeDefinition Def(UTextureRenderTarget::StaticClass());
	RenderTargetUserParameter.Parameter.SetType(Def);
}


void UNiagaraDataInterfaceGrid2DCollection::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
		UNiagaraDataInterfaceGrid2DCollection::ExposedRTVar = FNiagaraVariableBase(FNiagaraTypeDefinition(UTexture::StaticClass()), TEXT("RenderTarget"));
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceGrid2DCollection::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
		return false;


	InVisitor->UpdatePOD(TEXT("UNiagaraDataInterfaceGrid2DCollectionVersion"), (int32)FNiagaraGridCollection2DDIFunctionVersion::LatestVersion);

	const FGuid FNiagara2DGridCollectionGUID(0x2C8975F3, 0xC8AB4145, 0xA312186C, 0xC9BCBA2A);
	InVisitor->UpdatePOD(TEXT("UNiagaraDataInterfaceGrid2DCollectionGUID"), FNiagara2DGridCollectionGUID);

	InVisitor->UpdateShaderParameters<FNDIGrid2DShaderParameters>();
	return true;
}
#endif

#if WITH_EDITOR
void UNiagaraDataInterfaceGrid2DCollection::GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors,
	TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	Super::GetFeedback(Asset, Component, OutErrors, OutWarnings, OutInfo);
	// Put in placeholder for now.


}
#endif

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceGrid2DCollection::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	if (Super::UpgradeFunctionCall(FunctionSignature))
		return true;

	FName UpgradeName;
	if (FunctionSignature.Name == SetVector3ValueFunctionName)
		UpgradeName = SetVectorValueFunctionName;
	else if (FunctionSignature.Name == GetVector3ValueFunctionName)
		UpgradeName = GetPreviousVectorValueFunctionName;
	else if (FunctionSignature.Name == SampleGridVector3FunctionName)
		UpgradeName = SamplePreviousGridVectorFunctionName;
	else if (FunctionSignature.Name == SetVector2ValueFunctionName)
		UpgradeName = SetVector2DValueFunctionName;
	else if (FunctionSignature.Name == GetVector2ValueFunctionName)
		UpgradeName = GetPreviousVector2DValueFunctionName;
	else if (FunctionSignature.Name == SampleGridVector2FunctionName)
		UpgradeName = SamplePreviousGridVector2DFunctionName;
	else if (FunctionSignature.Name == GetValueFunctionName)
		UpgradeName = GetPreviousValueAtIndexFunctionName;
	else if (FunctionSignature.Name == GetFloatValueFunctionName)
		UpgradeName = GetPreviousFloatValueFunctionName;
	else if (FunctionSignature.Name == SampleGridFloatFunctionName)
		UpgradeName = SamplePreviousGridFloatFunctionName;


	if (UpgradeName != NAME_None)
	{
		TArray<FNiagaraFunctionSignature> Sigs;
		GetFunctionsInternal(Sigs);

		for (const FNiagaraFunctionSignature& Sig : Sigs)
		{
			if (Sig.Name == UpgradeName)
			{
				FNiagaraFunctionSignature Backup = FunctionSignature;
				FunctionSignature = Sig;
				return true;
			}
		}
	}

	if ( FunctionSignature.FunctionVersion < FNiagaraGridCollection2DDIFunctionVersion::AddReadMemberToFunctions )
	{
		TArray<FNiagaraFunctionSignature> Sigs;
		GetFunctionsInternal(Sigs);

		for (const FNiagaraFunctionSignature& Sig : Sigs)
		{
			if (Sig.Name == FunctionSignature.Name)
			{
				FNiagaraFunctionSignature Backup = FunctionSignature;
				FunctionSignature = Sig;
				return true;
			}
		}
	}

	if ( FunctionSignature.FunctionVersion < FNiagaraGridCollection2DDIFunctionVersion::AddExecuteToSetNumCells )
	{
		TArray<FNiagaraFunctionSignature> Sigs;
		GetFunctionsInternal(Sigs);

		for (const FNiagaraFunctionSignature& Sig : Sigs)
		{
			if (Sig.Name == FunctionSignature.Name && FunctionSignature.Name == SetNumCellsFunctionName)
			{
				FunctionSignature.Inputs = Sig.Inputs;
				return true;
			}
		}
	}

	return false;
}
#endif

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceGrid2DCollection::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	Super::GetFunctionsInternal(OutFunctions);

	int32 StartIndex = OutFunctions.Num();

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetNumCellsFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		OutFunctions.Add(Sig);
	}

	{
		// Older, deprecated form
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetValueFunction", "Get the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousValueAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_PreviousValueAtIndexFunction", "Get the value at a specific index.");

		OutFunctions.Add(Sig);
	}

	{
		// Older, deprecated form
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IGNORE")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetValueAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bRequiresExecPin = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSoftDeprecatedFunction = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueAtIndexFunction", "Set the value at a specific index.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ClearCellFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_ClearCellFunction", "Set all attributes for a given cell to be zeroes.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CopyPreviousToCurrentForCellFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_CopyPreviousToCurrentForCell", "Take the previous contents of the cell and copy to the output location for the cell.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetVector4ValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetVector4", "Sets a Vector4 value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousVector4ValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetVector4", "Gets a Vector4 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = GetVector4ValueFunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridVector4FunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SampleVector4", "Sample a Vector4 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = SampleGridVector4FunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridVector4FunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_CubicSampleVector4", "Cubic sample a Vector4 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = SampleGridVector4FunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetVectorValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetVector3", "Sets a Vector3 value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = SetVector3ValueFunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousVectorValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetVector3", "Gets a Vector3 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = GetVector3ValueFunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridVectorFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SampleVector3", "Sample a Vector3 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = SampleGridVector3FunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridVectorFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_CubicSampleVector3", "Cubic sample a Vector3 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = SampleGridVector3FunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetVector2DValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetVector2", "Sets a Vector2 value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = SetVector2ValueFunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousVector2DValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetVector2", "Gets a Vector2 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = GetVector2ValueFunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridVector2DFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SampleVector2", "Sample a Vector2 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = SampleGridVector2FunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridVector2DFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_CubicSampleVector2", "Cubic sample a Vector2 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = SampleGridVector2FunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetFloatValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetFloat", "Sets a float value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousFloatValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetFloat", "Gets a float value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = GetFloatValueFunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridFloatFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SampleFloat", "Sample a float value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = SampleGridFloatFunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridFloatFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_CubicSampleFloat", "Cubic sample a float value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);

		// Add older version for back compat with CustomHLSL.
		Sig.Name = SampleGridFloatFunctionName;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SampleGridFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVector4AttributeIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetVector4AttributeIndex", "Gets a attribute starting index value for Vector4 on the Grid by Attribute name. Returns -1 if not found.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVectorAttributeIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetVector3AttributeIndex", "Gets a attribute starting index value for Vector3 on the Grid by Attribute name. Returns -1 if not found.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVector2DAttributeIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetVector2AttributeIndex", "Gets a attribute starting index value for Vector2 on the Grid by Attribute name. Returns -1 if not found.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetFloatAttributeIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetFloatAttributeIndex", "Gets a attribute starting index value for float on the Grid by Attribute name. Returns -1 if not found.");
		OutFunctions.Add(Sig);
	}

	for (int32 i = StartIndex; i < OutFunctions.Num(); i++)
	{
		FNiagaraFunctionSignature& Function = OutFunctions[i];
		Function.FunctionVersion = FNiagaraGridCollection2DDIFunctionVersion::LatestVersion;
	}
}
#endif

// #todo(dmp): expose more CPU functionality
// #todo(dmp): ideally these would be exposed on the parent class, but we can't bind functions of parent classes but need to work on the interface
// for sharing an instance data object with the super class
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMGetWorldBBoxSize);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMGetCellSize);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMGetNumCells);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMSetNumCells);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceGrid2DCollection, VMGetAttributeIndex);
void UNiagaraDataInterfaceGrid2DCollection::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	Super::GetVMExternalFunction(BindingInfo, InstanceData, OutFunc);

	if (BindingInfo.Name == UNiagaraDataInterfaceRWBase::WorldBBoxSizeFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 2);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMGetWorldBBoxSize)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == UNiagaraDataInterfaceRWBase::CellSizeFunctionName)
	{
		// #todo(dmp): this will override the base class definition for GetCellSize because the data interface instance data computes cell size
		// it would be nice to refactor this so it can be part of the super class
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 2);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMGetCellSize)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == UNiagaraDataInterfaceRWBase::NumCellsFunctionName)
	{
		// #todo(dmp): this will override the base class definition for GetCellSize because the data interface instance data computes cell size
		// it would be nice to refactor this so it can be part of the super class
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 2);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMGetNumCells)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetNumCellsFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMSetNumCells)::Bind(this, OutFunc);
	}

	else if (BindingInfo.Name == GetVector4AttributeIndexFunctionName)
	{
		FName AttributeName = BindingInfo.FindSpecifier(UNiagaraDataInterfaceRWBase::NAME_Attribute)->Value;
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMGetAttributeIndex)::Bind(this, OutFunc, AttributeName, 4);
	}
	else if (BindingInfo.Name == GetVectorAttributeIndexFunctionName)
	{
		FName AttributeName = BindingInfo.FindSpecifier(UNiagaraDataInterfaceRWBase::NAME_Attribute)->Value;
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMGetAttributeIndex)::Bind(this, OutFunc, AttributeName, 3);
	}
	else if (BindingInfo.Name == GetVector2DAttributeIndexFunctionName)
	{
		FName AttributeName = BindingInfo.FindSpecifier(UNiagaraDataInterfaceRWBase::NAME_Attribute)->Value;
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMGetAttributeIndex)::Bind(this, OutFunc, AttributeName, 2);
	}
	else if (BindingInfo.Name == GetFloatAttributeIndexFunctionName)
	{
		FName AttributeName = BindingInfo.FindSpecifier(UNiagaraDataInterfaceRWBase::NAME_Attribute)->Value;
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid2DCollection, VMGetAttributeIndex)::Bind(this, OutFunc, AttributeName, 1);
	}
}

bool UNiagaraDataInterfaceGrid2DCollection::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceGrid2DCollection* OtherTyped = CastChecked<const UNiagaraDataInterfaceGrid2DCollection>(Other);

	return OtherTyped != nullptr &&
#if WITH_EDITOR
		OtherTyped->bPreviewGrid == bPreviewGrid &&
		OtherTyped->PreviewAttribute == PreviewAttribute &&
#endif
		OtherTyped->RenderTargetUserParameter == RenderTargetUserParameter &&
		OtherTyped->OverrideBufferFormat == OverrideBufferFormat &&
		OtherTyped->bOverrideFormat == bOverrideFormat;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceGrid2DCollection::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	static const TCHAR *FormatDeclarations = TEXT(R"(
		Texture2DArray<float> {GridName};
		RWTexture2DArray<float> {OutputGridName};
		SamplerState {SamplerName};
		int4 {AttributeIndicesName}[{AttributeInt4Count}];
		int {NumAttributesName};
	)");

	// If we use an int array for the attribute indices, the shader compiler will actually use int4 due to the packing rules,
	// and leave 3 elements unused. Besides being wasteful, this means that the array we send to the CS would need to be padded,
	// which is a hassle. Instead, use int4 explicitly, and access individual components in the generated code.
	// Note that we have to have at least one here because hlsl doesn't support arrays of size 0.
	const int AttributeInt4Count = FMath::Max(1, FMath::DivideAndRoundUp(ParamInfo.GeneratedFunctions.Num(), 4));

	TMap<FString, FStringFormatArg> ArgsDeclarations = {
		{ TEXT("ParameterName"),		ParamInfo.DataInterfaceHLSLSymbol},
		{ TEXT("GridName"),				ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{ TEXT("Grid"),					ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{ TEXT("SamplerName"),			ParamInfo.DataInterfaceHLSLSymbol + SamplerName},
		{ TEXT("OutputGridName"),		ParamInfo.DataInterfaceHLSLSymbol + OutputGridName},
		{ TEXT("AttributeIndicesName"),	ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + AttributeIndicesBaseName},
		{ TEXT("AttributeInt4Count"),	AttributeInt4Count},
		{ TEXT("NumAttributesName"),	ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
		{ TEXT("NumCellsName"),			ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
	};
	OutHLSL += FString::Format(FormatDeclarations, ArgsDeclarations);

	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, ArgsDeclarations);
}

void UNiagaraDataInterfaceGrid2DCollection::WriteSetHLSL(const FNiagaraDataInterfaceRWAttributeHelper& AttributeHelper, const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL)
{
	FString FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, float{NumChannelsVariableSuffix} In_Value)
			{
				int In_AttributeIndex = {AttributeIndex};

				)");
	if (InNumChannels == 1)
	{
		FormatBounds += FString("	{OutputGrid}[int3(In_IndexX, In_IndexY, In_AttributeIndex)] = In_Value;\n");
	}
	else if (InNumChannels > 1)
	{
		for (int i = 0; i < InNumChannels; ++i)
					{
			FormatBounds += FString("		{OutputGrid}[int3(In_IndexX, In_IndexY, In_AttributeIndex + ").Append(FString::FromInt(i)).Append(")] = In_Value.")
				.Append(FNiagaraDataInterfaceRWAttributeHelper::Channels[i]).Append(";\n");
		}
	}

		FormatBounds += TEXT(R"(
			}
		)");

	FString AttributeIndexPlaceholder = "{AttributeIndex}";
	FString AttributeQueryString = GenerateAttributeIndexHLSL(AttributeHelper, FunctionInfo);
	FormatBounds.ReplaceInline(*AttributeIndexPlaceholder, *AttributeQueryString);

	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("FunctionName"),				FunctionInfo.InstanceName},
		{TEXT("OutputGrid"),				ParamInfo.DataInterfaceHLSLSymbol + OutputGridName},
		{TEXT("Grid"),						ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("AttributeIndicesName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + AttributeIndicesBaseName},
		{TEXT("AttributeIndexGroup"),		FunctionInstanceIndex / 4},
		{TEXT("AttributeIndexComponent"),	VectorComponentNames[FunctionInstanceIndex % 4]},
		{TEXT("NumChannels"),				FString::FromInt(InNumChannels)},
		{TEXT("NumChannelsVariableSuffix"), InNumChannels > 1 ? FString::FromInt(InNumChannels) : TEXT("")},

	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

void UNiagaraDataInterfaceGrid2DCollection::WriteGetHLSL(const FNiagaraDataInterfaceRWAttributeHelper& AttributeHelper, const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL)
{
	FString FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, out float{NumChannelsVariableSuffix} Out_Val)
			{
				int In_AttributeIndex = {AttributeIndex};

					)");
	if (InNumChannels == 1)
	{
		FormatBounds += FString("	Out_Val = {Grid}.Load(int4(In_IndexX, In_IndexY, In_AttributeIndex, 0));\n");
	}
	else if (InNumChannels > 1)
	{
		for (int i = 0; i < InNumChannels; ++i)
					{
			FormatBounds += FString("			Out_Val.").Append(FNiagaraDataInterfaceRWAttributeHelper::Channels[i]).
				Append(" = {Grid}.Load(int4(In_IndexX, In_IndexY, In_AttributeIndex + ").Append(FString::FromInt(i)).Append(", 0)); \n");
	}
	}
		FormatBounds += TEXT(R"(
	}
		)");

	FString AttributeIndexPlaceholder = "{AttributeIndex}";
	FString AttributeQueryString = GenerateAttributeIndexHLSL(AttributeHelper, FunctionInfo);
	FormatBounds.ReplaceInline(*AttributeIndexPlaceholder, *AttributeQueryString);

	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("FunctionName"),				FunctionInfo.InstanceName},
		{TEXT("OutputGrid"),				ParamInfo.DataInterfaceHLSLSymbol + OutputGridName},
		{TEXT("Grid"),						ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("AttributeIndicesName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + AttributeIndicesBaseName},
		{TEXT("AttributeIndexGroup"),		FunctionInstanceIndex / 4},
		{TEXT("AttributeIndexComponent"),	VectorComponentNames[FunctionInstanceIndex % 4]},
		{TEXT("NumChannels"),				FString::FromInt(InNumChannels)},
		{TEXT("NumChannelsVariableSuffix"),	InNumChannels > 1 ? FString::FromInt(InNumChannels) : TEXT("")},

	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

void UNiagaraDataInterfaceGrid2DCollection::WriteSampleHLSL(const FNiagaraDataInterfaceRWAttributeHelper& AttributeHelper, const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, bool IsCubic, FString& OutHLSL)
{
	FString SampleFunctionPlaceholder = "{SampleFunction}";
	FString SampleFunction = "{Grid}.SampleLevel";
	if (IsCubic)
	{
		switch (GNiagaraGrid2DCubicInterpMethod)
		{
			case 0:
		SampleFunction = "SampleBiCubic_{ParameterName}";
				break;
			case 1:
				SampleFunction = "Monotonic2DCubic_{ParameterName}";
				break;
			default:
				SampleFunction = "SampleBiCubic_{ParameterName}";
				break;
	}
	}


	FString FormatBounds = TEXT(R"(
			void {FunctionName}(float2 In_Unit, out float{NumChannelsVariableSuffix} Out_Val)
			{
				int In_AttributeIndex = {AttributeIndex};

					)");
	if (InNumChannels == 1)
	{
		FormatBounds += FString("	Out_Val = {SampleFunction}({SamplerName}, float3(In_Unit, In_AttributeIndex), 0);\n");
	}
	else if (InNumChannels > 1)
	{
		for (int i = 0; i < InNumChannels; ++i)
					{
			FormatBounds += FString("			Out_Val.").Append(FNiagaraDataInterfaceRWAttributeHelper::Channels[i]).
				Append(" = {SampleFunction}({SamplerName}, float3(In_Unit, In_AttributeIndex + ").Append(FString::FromInt(i)).Append("), 0); \n");
		}
	}

	FormatBounds += TEXT(R"(
				}
		)");

	FString AttributeIndexPlaceholder = "{AttributeIndex}";
	FString AttributeQueryString = GenerateAttributeIndexHLSL(AttributeHelper, FunctionInfo);
	FormatBounds.ReplaceInline(*AttributeIndexPlaceholder, *AttributeQueryString);
	FormatBounds.ReplaceInline(*SampleFunctionPlaceholder, *SampleFunction);

	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("FunctionName"),				FunctionInfo.InstanceName},
		{TEXT("Grid"),						ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{TEXT("SamplerName"),				ParamInfo.DataInterfaceHLSLSymbol + SamplerName},
		{TEXT("SampleFunction"),			SampleFunction },
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("NumChannels"),				FString::FromInt(InNumChannels)},
		{TEXT("NumChannelsVariableSuffix"),	InNumChannels > 1 ? FString::FromInt(InNumChannels) : TEXT("")},
		{TEXT("AttributeIndicesName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + AttributeIndicesBaseName},
		{TEXT("AttributeIndexGroup"),		FunctionInstanceIndex / 4},
		{TEXT("AttributeIndexComponent"),	VectorComponentNames[FunctionInstanceIndex % 4]},

	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

void UNiagaraDataInterfaceGrid2DCollection::WriteAttributeGetIndexHLSL(const FNiagaraDataInterfaceRWAttributeHelper& AttributeHelper, const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL)
{
	FString FormatBounds = TEXT(R"(
			void {FunctionName}(out int Out_Val)
			{
				Out_Val = {AttributeIndex};				
			}
	)");

	FString AttributeIndexPlaceholder = "{AttributeIndex}";
	FString AttributeQueryString = GenerateAttributeIndexHLSL(AttributeHelper, FunctionInfo);
	FormatBounds.ReplaceInline(*AttributeIndexPlaceholder, *AttributeQueryString);

	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("FunctionName"), FunctionInfo.InstanceName},
		{TEXT("AttributeIndicesName"), ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + AttributeIndicesBaseName},
		{TEXT("AttributeIndexGroup"), FunctionInstanceIndex / 4},
		{TEXT("AttributeIndexComponent"), VectorComponentNames[FunctionInstanceIndex % 4]},
	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

FString UNiagaraDataInterfaceGrid2DCollection::GenerateAttributeIndexHLSL(const FNiagaraDataInterfaceRWAttributeHelper& AttributeHelper, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo)
{
	FString OutString = "";
	if (bUseIndirection)
	{
		OutString = "{AttributeIndicesName}[{AttributeIndexGroup}]{AttributeIndexComponent}";
	}
	else
	{
		const FName* AttributeName = FunctionInfo.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute);
		if (AttributeName == nullptr)
		{
			return OutString;
		}

		const FNiagaraDataInterfaceRWAttributeHelper::FAttributeInfo* AttributeInfo = AttributeHelper.FindAttributeInfo(*AttributeName);
		if (AttributeInfo == nullptr)
		{
			return OutString;
		}

		OutString = AttributeHelper.GetAttributeIndex(AttributeInfo);
	}

	return OutString;
}

const TCHAR* UNiagaraDataInterfaceGrid2DCollection::TypeDefinitionToHLSLTypeString(const FNiagaraTypeDefinition& InDef) const
{
	if (InDef == FNiagaraTypeDefinition::GetFloatDef())
		return TEXT("float");
	if (InDef == FNiagaraTypeDefinition::GetVec2Def())
		return TEXT("float2");
	if (InDef == FNiagaraTypeDefinition::GetVec3Def())
		return TEXT("float3");
	if (InDef == FNiagaraTypeDefinition::GetVec4Def() || InDef == FNiagaraTypeDefinition::GetColorDef())
		return TEXT("float4");
	return nullptr;
}
FName UNiagaraDataInterfaceGrid2DCollection::TypeDefinitionToGetFunctionName(const FNiagaraTypeDefinition& InDef) const
{
	if (InDef == FNiagaraTypeDefinition::GetFloatDef())
		return GetPreviousFloatValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec2Def())
		return GetPreviousVector2DValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec3Def())
		return GetPreviousVectorValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec4Def() || InDef == FNiagaraTypeDefinition::GetColorDef())
		return GetPreviousVector4ValueFunctionName;
	return NAME_None;;
}
FName UNiagaraDataInterfaceGrid2DCollection::TypeDefinitionToSetFunctionName(const FNiagaraTypeDefinition& InDef) const
{
	if (InDef == FNiagaraTypeDefinition::GetFloatDef())
		return SetFloatValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec2Def())
		return SetVector2DValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec3Def())
		return SetVectorValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec4Def() || InDef == FNiagaraTypeDefinition::GetColorDef())
		return SetVector4ValueFunctionName;
	return NAME_None;;
}

bool UNiagaraDataInterfaceGrid2DCollection::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	FNiagaraDataInterfaceRWAttributeHelper AttributeHelper(ParamInfo);
	AttributeHelper.Init<UNiagaraDataInterfaceGrid2DCollection>();

	bool ParentRet = Super::GetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, OutHLSL);
	if (ParentRet)
	{
		return true;
	}

	TMap<FString, FStringFormatArg> ArgsBounds =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("FunctionName"),	FunctionInfo.InstanceName},
		{TEXT("Grid"),			ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{TEXT("OutputGrid"),	ParamInfo.DataInterfaceHLSLSymbol + OutputGridName},
		{TEXT("NumAttributes"),	ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
		{TEXT("NumCells"),		ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),	ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("SamplerName"),	ParamInfo.DataInterfaceHLSLSymbol + SamplerName},
	};

	if (FunctionInfo.DefinitionName == GetValueFunctionName || FunctionInfo.DefinitionName == GetPreviousValueAtIndexFunctionName)
	{
		static const TCHAR *FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_AttributeIndex, out float Out_Val)
			{
				Out_Val = {Grid}.Load(int4(In_IndexX, In_IndexY, In_AttributeIndex, 0));
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetValueAtIndexFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_AttributeIndex, float In_Value)
			{
				{OutputGrid}[int3(In_IndexX, In_IndexY, In_AttributeIndex)] = In_Value;
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetValueFunctionName)
	{
		static const TCHAR *FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_AttributeIndex, float In_Value, out int val)
			{
				val = 0;
				{OutputGrid}[int3(In_IndexX, In_IndexY, In_AttributeIndex)] = In_Value;
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == CopyPreviousToCurrentForCellFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY)
			{
				for (int AttributeIndex = 0; AttributeIndex < {NumAttributes}.x; AttributeIndex++)
				{
					float Val = {Grid}.Load(int4(In_IndexX, In_IndexY, AttributeIndex, 0));
					{OutputGrid}[int3(In_IndexX, In_IndexY, AttributeIndex)] = Val;
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == ClearCellFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY)
			{
				for (int AttributeIndex = 0; AttributeIndex < {NumAttributes}.x; AttributeIndex++)
				{
					float Val = 0.0f;
					{OutputGrid}[int3(In_IndexX, In_IndexY, AttributeIndex)] = Val;
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetVector4ValueFunctionName)
	{
		WriteSetHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 4, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVector4ValueFunctionName || FunctionInfo.DefinitionName == GetPreviousVector4ValueFunctionName)
	{
		WriteGetHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 4, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SampleGridVector4FunctionName || FunctionInfo.DefinitionName == SamplePreviousGridVector4FunctionName)
	{
		WriteSampleHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 4, false, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridVector4FunctionName)
	{
		WriteSampleHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 4, true, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetVector3ValueFunctionName || FunctionInfo.DefinitionName == SetVectorValueFunctionName)
	{
		WriteSetHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 3, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVector3ValueFunctionName || FunctionInfo.DefinitionName == GetPreviousVectorValueFunctionName)
	{
		WriteGetHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 3, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SampleGridVector3FunctionName || FunctionInfo.DefinitionName == SamplePreviousGridVectorFunctionName)
	{
		WriteSampleHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 3, false, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridVectorFunctionName)
	{
		WriteSampleHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 3, true, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetVector2ValueFunctionName || FunctionInfo.DefinitionName == SetVector2DValueFunctionName)
	{
		WriteSetHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 2, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVector2ValueFunctionName || FunctionInfo.DefinitionName == GetPreviousVector2DValueFunctionName)
	{
		WriteGetHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 2, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SampleGridVector2FunctionName || FunctionInfo.DefinitionName == SamplePreviousGridVector2DFunctionName)
	{
		WriteSampleHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 2, false, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridVector2DFunctionName)
	{
		WriteSampleHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 2, true, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetFloatValueFunctionName)
	{
		WriteSetHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 1, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetFloatValueFunctionName || FunctionInfo.DefinitionName == GetPreviousFloatValueFunctionName)
	{
		WriteGetHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 1, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SampleGridFloatFunctionName || FunctionInfo.DefinitionName == SamplePreviousGridFloatFunctionName)
	{
		WriteSampleHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 1, false, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridFloatFunctionName)
	{
		WriteSampleHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 1, true, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVector4AttributeIndexFunctionName)
	{
		WriteAttributeGetIndexHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 4, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVectorAttributeIndexFunctionName)
	{
		WriteAttributeGetIndexHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 3, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVector2DAttributeIndexFunctionName)
	{
		WriteAttributeGetIndexHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 2, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetFloatAttributeIndexFunctionName)
	{
		WriteAttributeGetIndexHLSL(AttributeHelper, ParamInfo, FunctionInfo, FunctionInstanceIndex, 1, OutHLSL);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SampleGridFunctionName || FunctionInfo.DefinitionName == SamplePreviousGridAtIndexFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
				void {FunctionName}(float In_UnitX, float In_UnitY, int In_AttributeIndex, out float Out_Val)
				{
					float3 UVW = float3(In_UnitX, In_UnitY, In_AttributeIndex);
					Out_Val = {Grid}.SampleLevel({SamplerName}, UVW, 0);
				}
			)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridAtIndexFunctionName)
	{
		FString SampleFunctionPlaceholder = "{SampleFunction}";
		FString SampleFunction = "SampleBiCubic_";

		switch (GNiagaraGrid2DCubicInterpMethod)
		{
		case 0:
			SampleFunction = "SampleBiCubic_";
			break;
		case 1:
			SampleFunction = "Monotonic2DCubic_";
			break;
		default:
			SampleFunction = "SampleBiCubic_";
			break;
		}
		

		FString FormatBounds = TEXT(R"(
				void {FunctionName}(float In_UnitX, float In_UnitY, int In_AttributeIndex, out float Out_Val)
				{
					float3 UVW = float3(In_UnitX, In_UnitY, In_AttributeIndex);
					Out_Val = {SampleFunction}{ParameterName}({SamplerName}, UVW, 0);
				}
			)");

		FormatBounds.ReplaceInline(*SampleFunctionPlaceholder, *SampleFunction);

		OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
		return true;
	}
	return false;
}
#endif

void UNiagaraDataInterfaceGrid2DCollection::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FNDIGrid2DShaderParameters>();

	const int32 IndirectionTableSize = FMath::Max(FMath::DivideAndRoundUp(ShaderParametersBuilder.GetGeneratedFunctions().Num(), 4), 1);
	ShaderParametersBuilder.AddLooseParamArray<FIntVector4>(*UNiagaraDataInterfaceGrid2DCollection::AttributeIndicesBaseName, IndirectionTableSize);
}

void UNiagaraDataInterfaceGrid2DCollection::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyGrid2DCollectionProxy& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyGrid2DCollectionProxy>();
	FGrid2DCollectionRWInstanceData_RenderThread* ProxyData = DIProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	FGrid2DCollectionRWInstanceData_RenderThread* OriginalProxyData = ProxyData;
	check(ProxyData);

	if (ProxyData->OtherProxy != nullptr)
	{
		FNiagaraDataInterfaceProxyGrid2DCollectionProxy* OtherGrid3DProxy = static_cast<FNiagaraDataInterfaceProxyGrid2DCollectionProxy*>(ProxyData->OtherProxy);
		ProxyData = OtherGrid3DProxy->SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
		check(ProxyData);
	}

	const FNiagaraDataInterfaceParametersCS_Grid2DCollection& ShaderStorage = Context.GetShaderStorage<FNiagaraDataInterfaceParametersCS_Grid2DCollection>();
	if (OriginalProxyData->AttributeIndices.Num() == 0 && ShaderStorage.AttributeNames.Num() > 0)
	{
		OriginalProxyData->AttributeIndices.SetNumZeroed(Align(ShaderStorage.AttributeNames.Num(), 4));

		// TODO handle mismatched types!
		for (int32 i = 0; i < ShaderStorage.AttributeNames.Num(); i++)
		{
			const int32 FoundIdx = ProxyData->Vars.Find(ShaderStorage.AttributeNames[i]);
			check(ShaderStorage.AttributeNames.Num() == ShaderStorage.AttributeChannelCount.Num());
			check(ProxyData->Offsets.Num() == ProxyData->VarComponents.Num());
			check(ProxyData->Offsets.Num() == ProxyData->Vars.Num());
			if (ProxyData->Offsets.IsValidIndex(FoundIdx) && ShaderStorage.AttributeChannelCount[i] == ProxyData->VarComponents[FoundIdx])
			{
				OriginalProxyData->AttributeIndices[i] = ProxyData->Offsets[FoundIdx];
			}
			else
			{
				OriginalProxyData->AttributeIndices[i] = -1; // We may need to protect against this in the hlsl as this might underflow an array lookup if used incorrectly.
			}
		}
	}

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	FNDIGrid2DShaderParameters* Parameters = Context.GetParameterNestedStruct<FNDIGrid2DShaderParameters>();
	Parameters->NumAttributes = ProxyData->NumAttributes;
	Parameters->UnitToUV = FVector2f(1.0f) / FVector2f(ProxyData->NumCells);
	Parameters->NumCells = ProxyData->NumCells;
	Parameters->CellSize = FVector2f(ProxyData->CellSize);
	Parameters->WorldBBoxSize = FVector2f(ProxyData->WorldBBoxSize);	// LWC_TODO: Precision loss?
	if (Context.IsResourceBound(&Parameters->Grid))
	{
		if (ProxyData->CurrentData)
		{
			Parameters->Grid = ProxyData->CurrentData->GetOrCreateSRV(GraphBuilder);
		}
		else
		{
			Parameters->Grid = Context.GetComputeDispatchInterface().GetBlackTextureSRV(GraphBuilder, ETextureDimension::Texture2DArray);
		}
	}
	Parameters->GridSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	if (Context.IsResourceBound(&Parameters->OutputGrid))
	{
		if (Context.IsOutputStage() && ProxyData->DestinationData)
		{
			Parameters->OutputGrid = ProxyData->DestinationData->GetOrCreateUAV(GraphBuilder);
		}
		else
		{
			Parameters->OutputGrid = Context.GetComputeDispatchInterface().GetEmptyTextureUAV(GraphBuilder, PF_R32_FLOAT, ETextureDimension::Texture2DArray);
		}
	}

	// Do we need to set the indirection table?
	if (ShaderStorage.AttributeNames.Num() > 0)
	{
		const int NumAttributesVector4s = FMath::Max(FMath::DivideAndRoundUp(ShaderStorage.AttributeNames.Num(), 4), 1);
		TArrayView<FIntVector4> AttributeIndices = Context.GetParameterLooseArray<FIntVector4>(NumAttributesVector4s);
		check(OriginalProxyData->AttributeIndices.Num() * OriginalProxyData->AttributeIndices.GetTypeSize() == AttributeIndices.Num() * AttributeIndices.GetTypeSize());
		FMemory::Memcpy(AttributeIndices.GetData(), OriginalProxyData->AttributeIndices.GetData(), AttributeIndices.Num() * AttributeIndices.GetTypeSize());
	}
}

FNiagaraDataInterfaceParametersCS* UNiagaraDataInterfaceGrid2DCollection::CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const
{
	FNiagaraDataInterfaceParametersCS_Grid2DCollection* ShaderStorage = new FNiagaraDataInterfaceParametersCS_Grid2DCollection();

	// Gather up all the attribute names referenced. Note that there may be multiple in the list of the same name,
	// but we only deal with this by the number of bound methods.
	const int32 NumFuncs = ParameterInfo.GeneratedFunctions.Num();
	ShaderStorage->AttributeNames.Reserve(NumFuncs);
	ShaderStorage->AttributeChannelCount.Reserve(NumFuncs);

	for (int32 FuncIdx=0; FuncIdx < NumFuncs; ++FuncIdx)
	{
		const FNiagaraDataInterfaceGeneratedFunction& Func = ParameterInfo.GeneratedFunctions[FuncIdx];
		
		const FName* AttributeName = Func.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute);
		if (AttributeName != nullptr)
		{
			int32 ComponentCount = UNiagaraDataInterfaceGrid2DCollection::GetComponentCountFromFuncName(Func.DefinitionName);
			ShaderStorage->AttributeNames.Add(*AttributeName);
			ShaderStorage->AttributeChannelCount.Add(ComponentCount);
		}
		else
		{
			ShaderStorage->AttributeNames.Add(FName());
			ShaderStorage->AttributeChannelCount.Add(INDEX_NONE);
		}
	}

	ShaderStorage->AttributeNames.Shrink();
	ShaderStorage->AttributeChannelCount.Shrink();

	return ShaderStorage;
}

const FTypeLayoutDesc* UNiagaraDataInterfaceGrid2DCollection::GetShaderStorageType() const
{
	return &StaticGetTypeLayoutDesc<FNiagaraDataInterfaceParametersCS_Grid2DCollection>();
}

#if WITH_EDITOR
bool UNiagaraDataInterfaceGrid2DCollection::GenerateIterationSourceNamespaceReadAttributesHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, const FNiagaraVariable& IterationSourceVar, TConstArrayView<FNiagaraVariable> InArguments, TConstArrayView<FNiagaraVariable> InAttributes, TConstArrayView<FString> InAttributeHLSLNames, bool bInSetToDefaults, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const
{

	FString DIVarName;
	OutHLSL += TEXT("\t//Generated by UNiagaraDataInterfaceGrid2DCollection::GenerateIterationSourceNamespaceReadAttributesHLSL\n");
	for (int32 i = 0; i < InArguments.Num(); i++)
	{
		OutHLSL += FString::Printf(TEXT("\t// Argument Name \"%s\" Type \"%s\"\n"), *InArguments[i].GetName().ToString(), *InArguments[i].GetType().GetName());
		if (InArguments[i].GetType().GetClass() == GetClass())
		{
			DIVarName = InArguments[i].GetName().ToString();
		}
	}


	if (InAttributes.Num() != InAttributeHLSLNames.Num())
		return false;

	if (InAttributes.Num() > 0)
	{
		OutHLSL += FString::Printf(TEXT("\tint X, Y;\n\t%s.ExecutionIndexToGridIndex(X, Y);\n"), *DIVarName);
	}

	TArray<FString> RootArray;
	IterationSourceVar.GetName().ToString().ParseIntoArray(RootArray, TEXT("."));

	for (int32 i = 0; i < InAttributes.Num(); i++)
	{
		OutHLSL += FString::Printf(TEXT("\t// Variable Name \"%s\" Type \"%s\" Var \"%s\"\n" ), *InAttributes[i].GetName().ToString(), *InAttributes[i].GetType().GetName(), *InAttributeHLSLNames[i]);

		TArray<FString> OutArray;
		if (InAttributes[i].GetName().ToString().ParseIntoArray(OutArray, TEXT(".")) > 0)
		{
			if (TypeDefinitionToSetFunctionName(InAttributes[i].GetType()) == NAME_None)
			{
				FText Error = FText::Format(LOCTEXT("UnknownType", "Unsupported Type {0} , Attribute {1} for custom iteration source"), InAttributes[i].GetType().GetNameText(), FText::FromName(InAttributes[i].GetName()));
				OutErrors.Add(Error);
				continue;
			}

			// Clear out the shared namespace with the root variable...
			FString AttributeName;
			for (int32 NamespaceIdx = 0;  NamespaceIdx < OutArray.Num(); NamespaceIdx++)
			{
				if (NamespaceIdx < RootArray.Num() && RootArray[NamespaceIdx] == OutArray[NamespaceIdx])
					continue;
				if (OutArray[NamespaceIdx] == (FNiagaraConstants::PreviousNamespace.ToString()) || OutArray[NamespaceIdx] == (FNiagaraConstants::InitialNamespace.ToString()))
				{
					FText Error = FText::Format(LOCTEXT("UnknownSubNamespace", "Unsupported NamespaceModifier Attribute {0}"), FText::FromName(InAttributes[i].GetName()));
					OutErrors.Add(Error);
				}
				if (AttributeName.Len() != 0)
					AttributeName += TEXT(".");
				AttributeName += OutArray[NamespaceIdx];
			}
			OutHLSL += FString::Printf(TEXT("\t%s.%s<Attribute=\"%s\">(X, Y, %s);\n"), *DIVarName, *TypeDefinitionToGetFunctionName(InAttributes[i].GetType()).ToString(), *AttributeName, *InAttributeHLSLNames[i]);
		}

	}
	return true;
}
bool UNiagaraDataInterfaceGrid2DCollection::GenerateIterationSourceNamespaceWriteAttributesHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, const FNiagaraVariable& IterationSourceVar, TConstArrayView<FNiagaraVariable> InArguments, TConstArrayView<FNiagaraVariable> InAttributes, TConstArrayView<FString> InAttributeHLSLNames, TConstArrayView<FNiagaraVariable> InAllAttributes, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const
{
	FString DIVarName;
	OutHLSL += TEXT("\t//Generated by UNiagaraDataInterfaceGrid2DCollection::GenerateIterationSourceNamespaceWriteAttributesHLSL\n");
	for (int32 i = 0; i < InArguments.Num(); i++)
	{
		OutHLSL += FString::Printf(TEXT("\t// Argument Name \"%s\" Type \"%s\"\n"), *InArguments[i].GetName().ToString(), *InArguments[i].GetType().GetName());
		if (InArguments[i].GetType().GetClass() == GetClass())
		{
			DIVarName = InArguments[i].GetName().ToString();
		}
	}
	if (InAttributes.Num() != InAttributeHLSLNames.Num())
		return false;

	// First we need to copy all the data over from the input buffer, because we can't assume that this function will know all the attributes held within the grid. Instead, we copy all of them
	// over AND THEN overlay the local changes. Hopefully the optimizer will know enough to fix this up.
	if (InAttributes.Num() > 0)
	{
		OutHLSL += FString::Printf(TEXT("\tint X, Y;\n\t%s.ExecutionIndexToGridIndex(X, Y);\n"), *DIVarName);
	}

	// only copy previous to next if we haven't written to all the attributes
	if (InAttributes.Num() > 0 && InAttributes.Num() < InAllAttributes.Num() && !bSpawnOnly && !bPartialWrites)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			// We need to copy from previous to current first thing, because other functions afterwards may just set values on the local grid.			
			{Grid}.CopyPreviousToCurrentForCell(X,Y);
		)");
		TMap<FString, FStringFormatArg> ArgsBounds = {
			{TEXT("Grid"), DIVarName},
		};
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
	}

	TArray<FString> RootArray;
	IterationSourceVar.GetName().ToString().ParseIntoArray(RootArray, TEXT("."));

	for (int32 i = 0; i < InAttributes.Num(); i++)
	{
		OutHLSL += FString::Printf(TEXT("\t// Name \"%s\" Type \"%s\" Var \"%s\"\n"), *InAttributes[i].GetName().ToString(), *InAttributes[i].GetType().GetName(), *InAttributeHLSLNames[i]);

		TArray<FString> OutArray;
		if (InAttributes[i].GetName().ToString().ParseIntoArray(OutArray, TEXT(".")) > 0)
		{
			if (TypeDefinitionToSetFunctionName(InAttributes[i].GetType()) == NAME_None)
			{
				FText Error = FText::Format(LOCTEXT("UnknownType", "Unsupported Type {0} , Attribute {1} for custom iteration source"), InAttributes[i].GetType().GetNameText(), FText::FromName(InAttributes[i].GetName()));
				OutErrors.Add(Error);
				continue;
			}

			// Clear out the shared namespace with the root variable...
			FString AttributeName;
			for (int32 NamespaceIdx = 0; NamespaceIdx < OutArray.Num(); NamespaceIdx++)
			{
				if (NamespaceIdx < RootArray.Num() && RootArray[NamespaceIdx] == OutArray[NamespaceIdx])
					continue;

				if (OutArray[NamespaceIdx] == (FNiagaraConstants::PreviousNamespace.ToString()) || OutArray[NamespaceIdx] == (FNiagaraConstants::InitialNamespace.ToString()))
				{
					FText Error = FText::Format(LOCTEXT("UnknownSubNamespace", "Unsupported NamespaceModifier Attribute {0}"), FText::FromName(InAttributes[i].GetName()));
					OutErrors.Add(Error);
				}
				if (AttributeName.Len() != 0)
					AttributeName += TEXT(".");
				AttributeName += OutArray[NamespaceIdx];
			}

			OutHLSL += FString::Printf(TEXT("\t%s.%s<Attribute=\"%s\">(X, Y, %s);\n"), *DIVarName, *TypeDefinitionToSetFunctionName(InAttributes[i].GetType()).ToString(), *AttributeName, *InAttributeHLSLNames[i]);
		}
	}
	return true;
}


bool UNiagaraDataInterfaceGrid2DCollection::GenerateSetupHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, TConstArrayView<FNiagaraVariable> InArguments, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const
{
	FString DIVarName;
	OutHLSL += TEXT("\t//Generated by UNiagaraDataInterfaceGrid2DCollection::GenerateSetupHLSL\n");
	for (int32 i = 0; i < InArguments.Num(); i++)
	{
		OutHLSL += FString::Printf(TEXT("\t// Argument Name \"%s\" Type \"%s\"\n"), *InArguments[i].GetName().ToString(), *InArguments[i].GetType().GetName());

		if (InArguments[i].GetType().GetClass() == GetClass())
		{
			DIVarName = InArguments[i].GetName().ToString();
		}
	}

	return true;
}

bool UNiagaraDataInterfaceGrid2DCollection::GenerateTeardownHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, TConstArrayView<FNiagaraVariable> InArguments, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const
{
	OutHLSL += TEXT("\t//Generated by UNiagaraDataInterfaceGrid2DCollection::GenerateTeardownHLSL\n");


	return true;
}
#endif
bool UNiagaraDataInterfaceGrid2DCollection::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceGrid2DCollection* OtherTyped = CastChecked<UNiagaraDataInterfaceGrid2DCollection>(Destination);
	OtherTyped->RenderTargetUserParameter = RenderTargetUserParameter;
	OtherTyped->OverrideBufferFormat = OverrideBufferFormat;
	OtherTyped->bOverrideFormat = bOverrideFormat;
#if WITH_EDITORONLY_DATA
	OtherTyped->bPreviewGrid = bPreviewGrid;
	OtherTyped->PreviewAttribute = PreviewAttribute;
#endif

	return true;
}

bool UNiagaraDataInterfaceGrid2DCollection::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	check(Proxy);

	FGrid2DCollectionRWInstanceData_GameThread* InstanceData = new (PerInstanceData) FGrid2DCollectionRWInstanceData_GameThread();
	SystemInstancesToProxyData_GT.Emplace(SystemInstance->GetId(), InstanceData);

	InstanceData->NumCells.X = NumCellsX;
	InstanceData->NumCells.Y = NumCellsY;

	/* Go through all references to this data interface and build up the attribute list from the function metadata of those referenced.*/

	int32 NumAttribChannelsFound = 0;
	int32 NumNamedAttribChannelsFound = 0;
	FindAttributes(InstanceData->Vars, InstanceData->Offsets, NumNamedAttribChannelsFound);

	// Ensure we never allocate 0 attributes as that would fail to create the texture
	NumAttribChannelsFound = FMath::Max(NumAttributes + NumNamedAttribChannelsFound, 1);
	InstanceData->NumAttributes = NumAttribChannelsFound;

	// give an error if there are any unnamed attributes
	if (NumAttributes > 0 && NumNamedAttribChannelsFound > 0) //-V1051
	{
		UE_LOGF(LogNiagara, Log, "%ls: Unnamed attributes should not be used with named.  This is a deprecated workflow", *Proxy->GetSourceDIName().ToString());
	}

	InstanceData->WorldBBoxSize = WorldBBoxSize;

	ENiagaraGpuBufferFormat BufferFormat = bOverrideFormat ? OverrideBufferFormat : GetDefault<UNiagaraSettings>()->DefaultGridFormat;
	if (GNiagaraGrid2DOverrideFormat >= int32(ENiagaraGpuBufferFormat::Float) && (GNiagaraGrid2DOverrideFormat < int32(ENiagaraGpuBufferFormat::Max)))
	{
		BufferFormat = ENiagaraGpuBufferFormat(GNiagaraGrid2DOverrideFormat);
	}

	InstanceData->PixelFormat = FNiagaraUtilities::BufferFormatToPixelFormat(BufferFormat, GNDIGrid2DFomatCaps, 1, this);

	if (!FMath::IsNearlyEqual(GNiagaraGrid2DResolutionMultiplier, 1.0f))
	{
		InstanceData->NumCells.X = FMath::Max(1, int32(float(InstanceData->NumCells.X) * GNiagaraGrid2DResolutionMultiplier));
		InstanceData->NumCells.Y = FMath::Max(1, int32(float(InstanceData->NumCells.Y) * GNiagaraGrid2DResolutionMultiplier));
	}

	// If we are setting the grid from the voxel size, then recompute NumVoxels and change bbox
	if (SetGridFromMaxAxis)
	{
		FVector2D::FReal CellSize = FMath::Max(WorldBBoxSize.X, WorldBBoxSize.Y) / NumCellsMaxAxis;

		InstanceData->NumCells.X = int32(FMath::FloorToInt(WorldBBoxSize.X / CellSize));
		InstanceData->NumCells.Y = int32(FMath::FloorToInt(WorldBBoxSize.Y / CellSize));

		// Pad grid by 1 voxel if our computed bounding box is too small
		if (WorldBBoxSize.X > WorldBBoxSize.Y && !FMath::IsNearlyEqual(CellSize * InstanceData->NumCells.Y, WorldBBoxSize.Y))
		{
			InstanceData->NumCells.Y++;
		}
		else if (WorldBBoxSize.X < WorldBBoxSize.Y && !FMath::IsNearlyEqual(CellSize * InstanceData->NumCells.X, WorldBBoxSize.X))
		{
			InstanceData->NumCells.X++;
		}

		InstanceData->WorldBBoxSize = FVector2D(InstanceData->NumCells.X, InstanceData->NumCells.Y) * CellSize;
		NumCellsX = InstanceData->NumCells.X;
		NumCellsY = InstanceData->NumCells.Y;
	}

	InstanceData->CellSize = InstanceData->WorldBBoxSize / FVector2D(InstanceData->NumCells.X, InstanceData->NumCells.Y);

	// Initialize target texture
	InstanceData->TargetTexture = nullptr;
	InstanceData->RTUserParamBinding.Init(SystemInstance->GetInstanceParameters(), RenderTargetUserParameter.Parameter);
	InstanceData->UpdateTargetTexture(BufferFormat, this);

#if WITH_EDITORONLY_DATA
	InstanceData->bPreviewGrid = bPreviewGrid;
	InstanceData->PreviewAttribute = FIntVector4(INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE);
	if (bPreviewGrid && !PreviewAttribute.IsNone())
	{
		const int32 VariableIndex = InstanceData->Vars.IndexOfByPredicate([&](const FNiagaraVariableBase& Variable) { return Variable.GetName() == PreviewAttribute; });
		if (VariableIndex != INDEX_NONE)
		{
			const int32 NumComponents = InstanceData->Vars[VariableIndex].GetType().GetSize() / sizeof(float);
			if (ensure(NumComponents > 0 && NumComponents <= 4))
			{
				const int32 ComponentOffset = InstanceData->Offsets[VariableIndex];
				for (int32 i = 0; i < NumComponents; ++i)
				{
					InstanceData->PreviewAttribute[i] = ComponentOffset + i;
				}
			}
		}
		// Look for anonymous attributes
		else if ( NumAttributes > 0 )
		{
			const FString PreviewAttributeString = PreviewAttribute.ToString();
			if (PreviewAttributeString.StartsWith(AnonymousAttributeString))
			{
				InstanceData->PreviewAttribute[0] = FCString::Atoi(&PreviewAttributeString.GetCharArray()[AnonymousAttributeString.Len() + 1]);
			}
		}

		if (InstanceData->PreviewAttribute == FIntVector4(INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE))
		{
			UE_LOGF(LogNiagara, Warning, "Failed to map PreviewAttribute %ls to a grid index", *PreviewAttribute.ToString());
		}
	}
#endif

	InstanceData->ClearBeforeNonIterationStage = ClearBeforeNonIterationStage;

	// Push Updates to Proxy.
	FNiagaraDataInterfaceProxyGrid2DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid2DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Resource=InstanceData->TargetTexture ? InstanceData->TargetTexture->GetResource() : nullptr, RT_Proxy, InstanceID=SystemInstance->GetId(), RT_InstanceData=*InstanceData](FRHICommandListImmediate& RHICmdList)
	{
		check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));
		FGrid2DCollectionRWInstanceData_RenderThread* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.Add(InstanceID);

		TargetData->SourceDIName = RT_Proxy->GetSourceDIName();
		TargetData->ClearBeforeNonIterationStage = RT_InstanceData.ClearBeforeNonIterationStage;

		TargetData->NumCells = RT_InstanceData.NumCells;
		TargetData->NumAttributes = RT_InstanceData.NumAttributes;
		TargetData->CellSize = RT_InstanceData.CellSize;
		TargetData->WorldBBoxSize = RT_InstanceData.WorldBBoxSize;
		TargetData->PixelFormat = RT_InstanceData.PixelFormat;
		TargetData->Offsets = RT_InstanceData.Offsets;
		TargetData->Vars.Reserve(RT_InstanceData.Vars.Num());
		for (int32 i = 0; i < RT_InstanceData.Vars.Num(); i++)
		{
			TargetData->Vars.Emplace(RT_InstanceData.Vars[i].GetName());
			TargetData->VarComponents.Add(RT_InstanceData.Vars[i].GetType().GetSize() / sizeof(float));
		}

#if WITH_EDITORONLY_DATA
		TargetData->bPreviewGrid = RT_InstanceData.bPreviewGrid;
		TargetData->PreviewAttribute = RT_InstanceData.PreviewAttribute;
#endif

		if (RT_Resource && RT_Resource->TextureRHI.IsValid())
		{
			TargetData->RenderTargetToCopyTo = RT_Resource->TextureRHI;
		}
		else
		{
			TargetData->RenderTargetToCopyTo = nullptr;
		}
	});

	return true;
}


void UNiagaraDataInterfaceGrid2DCollection::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	SystemInstancesToProxyData_GT.Remove(SystemInstance->GetId());

	FGrid2DCollectionRWInstanceData_GameThread* InstanceData = static_cast<FGrid2DCollectionRWInstanceData_GameThread*>(PerInstanceData);

	InstanceData->~FGrid2DCollectionRWInstanceData_GameThread();

	FNiagaraDataInterfaceProxyGrid2DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid2DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[RT_Proxy, InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			//check(ThisProxy->SystemInstancesToProxyData.Contains(InstanceID));
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		}
	);

	// Make sure to clear out the reference to the render target if we created one.
	FNiagaraSystemInstanceID SysId = SystemInstance->GetId();
	ManagedRenderTargets.Remove(SysId);
}

bool UNiagaraDataInterfaceGrid2DCollection::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FGrid2DCollectionRWInstanceData_GameThread* InstanceData = SystemInstancesToProxyData_GT.FindRef(SystemInstance->GetId());

	ENiagaraGpuBufferFormat BufferFormat = bOverrideFormat ? OverrideBufferFormat : GetDefault<UNiagaraSettings>()->DefaultGridFormat;
	if (GNiagaraGrid2DOverrideFormat >= int32(ENiagaraGpuBufferFormat::Float) && (GNiagaraGrid2DOverrideFormat < int32(ENiagaraGpuBufferFormat::Max)))
	{
		BufferFormat = ENiagaraGpuBufferFormat(GNiagaraGrid2DOverrideFormat);
	}

	bool NeedsReset = InstanceData->UpdateTargetTexture(BufferFormat, this);

	FNiagaraDataInterfaceProxyGrid2DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid2DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Resource=InstanceData->TargetTexture ? InstanceData->TargetTexture->GetResource() : nullptr, RT_Proxy, InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
	{
		FGrid2DCollectionRWInstanceData_RenderThread* TargetData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID);
		if (RT_Resource && RT_Resource->TextureRHI.IsValid())
		{
			TargetData->RenderTargetToCopyTo = RT_Resource->TextureRHI;
		}
		else
		{
			TargetData->RenderTargetToCopyTo = nullptr;
		}

	});

	return NeedsReset;
}

void UNiagaraDataInterfaceGrid2DCollection::GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const
{
	OutVariables.Emplace(ExposedRTVar);
}

bool UNiagaraDataInterfaceGrid2DCollection::GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const
{
	FGrid2DCollectionRWInstanceData_GameThread* InstanceData = static_cast<FGrid2DCollectionRWInstanceData_GameThread*>(InPerInstanceData);
	if (InVariable.IsValid() && InVariable == ExposedRTVar && InstanceData && InstanceData->TargetTexture)
	{
		UTextureRenderTarget** Var = (UTextureRenderTarget**)OutData;
		*Var = InstanceData->TargetTexture;
		return true;
	}
	return false;
}

void UNiagaraDataInterfaceGrid2DCollection::CollectAttributesForScript(UNiagaraScript* Script, const FNiagaraScriptRuntimeData& ScriptRuntimeData, const UNiagaraDataInterface* DataInterface, TArray<FNiagaraVariableBase>& OutVariables, TArray<uint32>& OutVariableOffsets, int32& TotalAttributes, TArray<FText>* OutWarnings)
{
	if (const FNiagaraScriptExecutionParameterStore* ParameterStore = ScriptRuntimeData.GetExecutionReadyParameterStore())
	{
		int32 IndexOfDataInterface = ParameterStore->GetDataInterfaces().IndexOfByKey(DataInterface);
		TConstArrayView<FNiagaraDataInterfaceGPUParamInfo> ParamInfoArray = Script->GetDataInterfaceGPUParamInfos();
		if (IndexOfDataInterface != INDEX_NONE && IndexOfDataInterface < ParamInfoArray.Num())
		{
			for (const FNiagaraDataInterfaceGeneratedFunction& Func : ParamInfoArray[IndexOfDataInterface].GeneratedFunctions)
			{
				if (const FName* AttributeName = Func.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute))
				{
					FNiagaraVariableBase NewVar(UNiagaraDataInterfaceGrid2DCollection::GetValueTypeFromFuncName(Func.DefinitionName), *AttributeName);
					if (UNiagaraDataInterfaceGrid2DCollection::CanCreateVarFromFuncName(Func.DefinitionName))
					{
						if (!OutVariables.Contains(NewVar))
						{
							const int32 FoundNameMatch = OutVariables.IndexOfByPredicate([&](const FNiagaraVariableBase& Var) { return Var.GetName() == *AttributeName; });
							if (FoundNameMatch == INDEX_NONE)
							{
								OutVariables.Add(NewVar);
								const int32 NumComponents = NewVar.GetSizeInBytes() / sizeof(float);
								OutVariableOffsets.Add(TotalAttributes);
								TotalAttributes += NumComponents;
							}
							else
							{
								if (OutWarnings)
								{
									FText Warning = FText::Format(LOCTEXT("BadType", "Same name, different types! {0} vs {1}, Attribute {2}"), NewVar.GetType().GetNameText(), OutVariables[FoundNameMatch].GetType().GetNameText(), FText::FromName(NewVar.GetName()));
									OutWarnings->Add(Warning);
								}
							}
						}
					}
				}
			}
		}
	}
}

void UNiagaraDataInterfaceGrid2DCollection::FindAttributes(TArray<FNiagaraVariableBase>& OutVariables, TArray<uint32>& OutVariableOffsets, int32& OutNumAttribChannelsFound, TArray<FText>* OutWarnings) const
{
	OutNumAttribChannelsFound = 0;

	UNiagaraSystem* OwnerSystem = GetTypedOuter<UNiagaraSystem>();
	if (OwnerSystem == nullptr)
	{
		return;
	}

	int32 TotalAttributes = NumAttributes;
	for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
	{
		FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
		if (EmitterData && EmitterHandle.GetIsEnabled() && EmitterData->IsValid() && (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim))
		{
			TSharedPtr<const FNiagaraScriptRuntimeData> ScriptRuntimeData = OwnerSystem->GetScriptRuntimeData(EmitterHandle.GetId(), ENiagaraScriptUsage::ParticleGPUComputeScript, FGuid());
			if (ScriptRuntimeData.IsValid())
			{
				CollectAttributesForScript(EmitterData->GetGPUComputeScript(), *ScriptRuntimeData, this, OutVariables, OutVariableOffsets, TotalAttributes, OutWarnings);
			}
		}
	}
	OutNumAttribChannelsFound = TotalAttributes - NumAttributes;
}

void UNiagaraDataInterfaceGrid2DCollection::GetCanvasVariables(TArray<FNiagaraVariableBase>& OutVariables) const
{
	TArray<uint32> VariableOffsets;
	int32 NumAttribChannelsFound;
	FindAttributes(OutVariables, VariableOffsets, NumAttribChannelsFound);

	//-TODO: We could add anonymous attributes in here as well?
}

bool UNiagaraDataInterfaceGrid2DCollection::RenderVariableToCanvas(FNiagaraSystemInstanceID SystemInstanceID, FName VariableName, class FCanvas* Canvas, const FIntRect& DrawRect) const
{
	if (!Canvas)
	{
		return false;
	}

	FGrid2DCollectionRWInstanceData_GameThread* Grid2DInstanceData = SystemInstancesToProxyData_GT.FindRef(SystemInstanceID);
	if (!Grid2DInstanceData)
	{
		return false;
	}

	const int32 VariableIndex = Grid2DInstanceData->Vars.IndexOfByPredicate([&VariableName](const FNiagaraVariableBase& VariableBase) { return VariableBase.GetName() == VariableName; });
	if (VariableIndex == INDEX_NONE)
	{
		return false;
	}

	const int32 FirstSlice = Grid2DInstanceData->Offsets[VariableIndex];
	const int32 NumFloats = Grid2DInstanceData->Vars[VariableIndex].GetType().GetSize() / sizeof(float);

	FIntVector4 AttributeSlices;
	AttributeSlices.X = FirstSlice;
	AttributeSlices.Y = NumFloats > 1 ? FirstSlice + 1 : INDEX_NONE;
	AttributeSlices.Z = NumFloats > 2 ? FirstSlice + 2 : INDEX_NONE;
	AttributeSlices.W = NumFloats > 3 ? FirstSlice + 3 : INDEX_NONE;

	FCanvasTileItem TileItem(FVector2D(DrawRect.Min.X, DrawRect.Min.Y), GWhiteTexture, FVector2D(DrawRect.Width(), DrawRect.Height()), FVector2D(0.0, 1.0f), FVector2D(1.0, 0.0f), FLinearColor::White);
	TileItem.BlendMode = SE_BLEND_Opaque;
	TileItem.BatchedElementParameters = new FBatchedElementNiagara2DArrayAttribute(
		AttributeSlices,
		[RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxyGrid2DCollectionProxy>(), RT_SystemInstanceID=SystemInstanceID](FRHITexture*& OutTexture, FRHISamplerState*& OutSamplerState)
		{
			if ( const FGrid2DCollectionRWInstanceData_RenderThread* RT_InstanceData = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_SystemInstanceID) )
			{
				if ( RT_InstanceData->CurrentData != nullptr )
				{
					OutTexture = RT_InstanceData->CurrentData->GetPooledTexture()->GetRHI();
					OutSamplerState = TStaticSamplerState<SF_Bilinear>::GetRHI();
				}
			}
		}
	);
	Canvas->DrawItem(TileItem);

	return true;
}

bool UNiagaraDataInterfaceGrid2DCollection::FillTexture2D(const UNiagaraComponent *Component, UTextureRenderTarget2D *Dest, int AttributeIndex)
{
	if (!Component || !Dest)
	{
		return false;
	}

	FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = Component->GetSystemInstanceController();
	if (!SystemInstanceController.IsValid())
	{
		return false;
	}

	// check valid attribute index
	if (AttributeIndex < 0 || AttributeIndex >=NumAttributes)
	{
		return false;
	}

	// check dest size and type needs to be float
	// #todo(dmp): don't hardcode float since we might do other stuff in the future
	EPixelFormat RequiredTye = PF_R32_FLOAT;
	if (Dest->SizeX != NumCellsX || Dest->SizeY != NumCellsY || Dest->GetFormat() != RequiredTye)
	{
		return false;
	}

	FNiagaraDataInterfaceProxyGrid2DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid2DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateDIColorCurve)(
		[RT_Proxy, InstanceID=SystemInstanceController->GetSystemInstanceID(), RT_TextureResource=Dest->GetResource(), AttributeIndex](FRHICommandListImmediate& RHICmdList)
	{
		FGrid2DCollectionRWInstanceData_RenderThread* Grid2DInstanceData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID);

		if (RT_TextureResource && RT_TextureResource->TextureRHI.IsValid() && Grid2DInstanceData && Grid2DInstanceData->CurrentData)
		{
			FRHICopyTextureInfo CopyInfo;
			CopyInfo.Size = FIntVector(Grid2DInstanceData->NumCells.X, Grid2DInstanceData->NumCells.Y, 1);
			CopyInfo.SourcePosition = FIntVector(0, 0, AttributeIndex);
			TransitionAndCopyTexture(RHICmdList, Grid2DInstanceData->CurrentData->GetPooledTexture()->GetRHI(), RT_TextureResource->TextureRHI, CopyInfo);
		}
	});

	return true;
}

bool UNiagaraDataInterfaceGrid2DCollection::FillRawTexture2D(const UNiagaraComponent *Component, UTextureRenderTarget2D *Dest, int &TilesX, int &TilesY)
{
	if (!Component)
	{
		TilesX = -1;
		TilesY = -1;
		return false;
	}

	FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = Component->GetSystemInstanceController();
	if (!SystemInstanceController.IsValid())
	{
		TilesX = -1;
		TilesY = -1;
		return false;
	}
	const FNiagaraSystemInstanceID SystemID = SystemInstanceController->GetSystemInstanceID();

	FGrid2DCollectionRWInstanceData_GameThread* Grid2DInstanceData = SystemInstancesToProxyData_GT.FindRef(SystemID);
	if (!Grid2DInstanceData)
	{
		TilesX = -1;
		TilesY = -1;
		return false;
	}

	const FNiagaraGrid2DLegacyTiled2DInfo Tiled2DInfo(Grid2DInstanceData->NumCells, Grid2DInstanceData->NumAttributes);
	TilesX = Tiled2DInfo.NumTiles.X;
	TilesY = Tiled2DInfo.NumTiles.Y;

	// check dest size and type needs to be float
	// #todo(dmp): don't hardcode float since we might do other stuff in the future
	EPixelFormat RequiredTye = PF_R32_FLOAT;
	if (!Dest || Dest->SizeX != Tiled2DInfo.Size.X || Dest->SizeY != Tiled2DInfo.Size.Y || Dest->GetFormat() != RequiredTye)
	{
		return false;
	}

	FNiagaraDataInterfaceProxyGrid2DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid2DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateDIColorCurve)(
		[RT_Proxy, RT_InstanceID=SystemID, RT_TextureResource=Dest->GetResource()](FRHICommandListImmediate& RHICmdList)
	{
		FGrid2DCollectionRWInstanceData_RenderThread* RT_Grid2DInstanceData = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID);
		if (RT_TextureResource && RT_TextureResource->TextureRHI.IsValid() && RT_Grid2DInstanceData && RT_Grid2DInstanceData->CurrentData)
		{
			const FNiagaraGrid2DLegacyTiled2DInfo Tiled2DInfo(RT_Grid2DInstanceData->NumCells, RT_Grid2DInstanceData->NumAttributes);
			Tiled2DInfo.CopyTo2D(RHICmdList, RT_Grid2DInstanceData->CurrentData->GetPooledTexture()->GetRHI(), RT_TextureResource->TextureRHI);
		}
	});

	return true;
}

void UNiagaraDataInterfaceGrid2DCollection::GetRawTextureSize(const UNiagaraComponent *Component, int &SizeX, int &SizeY)
{
	if (!Component)
	{
		SizeX = -1;
		SizeY = -1;
		return;
	}

	FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = Component->GetSystemInstanceController();
	if (!SystemInstanceController.IsValid())
	{
		SizeX = -1;
		SizeY = -1;
		return;
	}
	FNiagaraSystemInstanceID InstanceID = SystemInstanceController->GetSystemInstanceID();

	FGrid2DCollectionRWInstanceData_GameThread* Grid2DInstanceData = SystemInstancesToProxyData_GT.FindRef(InstanceID);
	if (!Grid2DInstanceData)
	{
		SizeX = -1;
		SizeY = -1;
		return;
	}

	const FNiagaraGrid2DLegacyTiled2DInfo Tiled2DInfo(Grid2DInstanceData->NumCells, NumAttributes);
	SizeX = Tiled2DInfo.Size.X;
	SizeY = Tiled2DInfo.Size.Y;
}

void UNiagaraDataInterfaceGrid2DCollection::GetTextureSize(const UNiagaraComponent *Component, int &SizeX, int &SizeY)
{
	if (!Component)
	{
		SizeX = -1;
		SizeY = -1;
		return;
	}

	FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = Component->GetSystemInstanceController();
	if (!SystemInstanceController.IsValid())
	{
		SizeX = -1;
		SizeY = -1;
		return;
	}
	FNiagaraSystemInstanceID InstanceID = SystemInstanceController->GetSystemInstanceID();

	FGrid2DCollectionRWInstanceData_GameThread* Grid2DInstanceData = SystemInstancesToProxyData_GT.FindRef(InstanceID);
	if (!Grid2DInstanceData)
	{
		SizeX = -1;
		SizeY = -1;
		return;
	}

	SizeX = Grid2DInstanceData->NumCells.X;
	SizeY = Grid2DInstanceData->NumCells.Y;
}

void UNiagaraDataInterfaceGrid2DCollection::VMGetWorldBBoxSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FGrid2DCollectionRWInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<FVector2f> OutWorldBounds(Context);
	const FVector2f BBoxSize = FVector2f(InstData->WorldBBoxSize);	// LWC_TODO: Precision loss

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutWorldBounds.SetAndAdvance(BBoxSize);
	}
}


void UNiagaraDataInterfaceGrid2DCollection::VMGetCellSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FGrid2DCollectionRWInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<FVector2f> OutCellSize(Context);
	const FVector2f CellSize = FVector2f(InstData->CellSize);	// LWC_TODO: Precision loss

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutCellSize.SetAndAdvance(CellSize);
	}
}

void UNiagaraDataInterfaceGrid2DCollection::VMGetNumCells(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FGrid2DCollectionRWInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int> OutNumCellsX(Context);
	FNDIOutputParam<int> OutNumCellsY(Context);

	int32 TmpNumCellsX = InstData->NumCells.X;
	int32 TmpNumCellsY = InstData->NumCells.Y;	
	if (InstData->OtherInstanceData != nullptr)
	{
		TmpNumCellsX = InstData->OtherInstanceData->NumCells.X;
		TmpNumCellsY = InstData->OtherInstanceData->NumCells.Y;	
	}

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutNumCellsX.SetAndAdvance(TmpNumCellsX);
		OutNumCellsY.SetAndAdvance(TmpNumCellsY);
	}
}

void UNiagaraDataInterfaceGrid2DCollection::VMSetNumCells(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FGrid2DCollectionRWInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsX(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsY(Context);
	VectorVM::FExternalFuncInputHandler<FNiagaraBool> InExecute(Context);
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		int NewNumCellsX = InNumCellsX.GetAndAdvance();
		int NewNumCellsY = InNumCellsY.GetAndAdvance();
		bool bExecute = InExecute.GetAndAdvance();
		bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1 && NumCellsX >= 0 && NumCellsY >= 0) && bExecute;
		*OutSuccess.GetDestAndAdvance() = bSuccess;
		if (bSuccess)
		{
			FIntPoint OldNumCells = InstData->NumCells;

			InstData->NumCells.X = NewNumCellsX;
			InstData->NumCells.Y = NewNumCellsY;

			if (!FMath::IsNearlyEqual(GNiagaraGrid2DResolutionMultiplier, 1.0f))
			{
				InstData->NumCells.X = FMath::Max(1, int32(float(InstData->NumCells.X) * GNiagaraGrid2DResolutionMultiplier));
				InstData->NumCells.Y = FMath::Max(1, int32(float(InstData->NumCells.Y) * GNiagaraGrid2DResolutionMultiplier));
			}

			InstData->NeedsRealloc = OldNumCells != InstData->NumCells;
		}
	}
}

bool UNiagaraDataInterfaceGrid2DCollection::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FGrid2DCollectionRWInstanceData_GameThread* InstanceData = static_cast<FGrid2DCollectionRWInstanceData_GameThread*>(PerInstanceData);
	bool bNeedsReset = false;

	if (InstanceData->NeedsRealloc && InstanceData->NumCells.X > 0 && InstanceData->NumCells.Y > 0)
	{
		InstanceData->NeedsRealloc = false;

		InstanceData->CellSize = InstanceData->WorldBBoxSize / FVector2D(InstanceData->NumCells.X, InstanceData->NumCells.Y);

		if (InstanceData->TargetTexture)
		{
			ENiagaraGpuBufferFormat BufferFormat = bOverrideFormat ? OverrideBufferFormat : GetDefault<UNiagaraSettings>()->DefaultGridFormat;
			if (GNiagaraGrid2DOverrideFormat >= int32(ENiagaraGpuBufferFormat::Float) && (GNiagaraGrid2DOverrideFormat < int32(ENiagaraGpuBufferFormat::Max)))
			{
				BufferFormat = ENiagaraGpuBufferFormat(GNiagaraGrid2DOverrideFormat);
			}

			bNeedsReset = InstanceData->UpdateTargetTexture(BufferFormat, this);
		}

		// Push Updates to Proxy.
		FNiagaraDataInterfaceProxyGrid2DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid2DCollectionProxy>();
		ENQUEUE_RENDER_COMMAND(FUpdateData)(
			[RT_Resource=InstanceData->TargetTexture ? InstanceData->TargetTexture->GetResource() : nullptr, RT_Proxy, InstanceID=SystemInstance->GetId(), RT_InstanceData=*InstanceData](FRHICommandListImmediate& RHICmdList)
			{
				check(RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));
				FGrid2DCollectionRWInstanceData_RenderThread* TargetData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID);

				TargetData->NumCells = RT_InstanceData.NumCells;
				TargetData->NumAttributes = RT_InstanceData.NumAttributes;
				TargetData->CellSize = RT_InstanceData.CellSize;

				TargetData->Buffers.Empty();
				TargetData->CurrentData = nullptr;
				TargetData->DestinationData = nullptr;

				if (RT_Resource && RT_Resource->TextureRHI.IsValid())
				{
					TargetData->RenderTargetToCopyTo = RT_Resource->TextureRHI;
				}
				else
				{
					TargetData->RenderTargetToCopyTo = nullptr;
				}
			}
		);
	}

	return false;
}


void UNiagaraDataInterfaceGrid2DCollection::VMGetAttributeIndex(FVectorVMExternalFunctionContext& Context, const FName& InName, int32 NumChannels)
{
	VectorVM::FUserPtrHandler<FGrid2DCollectionRWInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int> OutIndex(Context);
	int32 Index = INDEX_NONE;
	if (InstData.Get())
		Index = InstData.Get()->FindAttributeIndexByName(InName, NumChannels);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutIndex.GetDestAndAdvance() = Index;
	}
}

int32 FGrid2DCollectionRWInstanceData_GameThread::FindAttributeIndexByName(const FName& InName, int32 NumChannels)
{
	for (int32 i = 0; i < Vars.Num(); i++)
	{
		const FNiagaraVariableBase& Var = Vars[i];
		if (Var.GetName() == InName)
		{
			if (NumChannels == 1 && Var.GetType() == FNiagaraTypeDefinition::GetFloatDef())
				return Offsets[i];
			else if (NumChannels == 2 && Var.GetType() == FNiagaraTypeDefinition::GetVec2Def())
				return Offsets[i];
			else if (NumChannels == 3 && Var.GetType() == FNiagaraTypeDefinition::GetVec3Def())
				return Offsets[i];
			else if (NumChannels == 4 && Var.GetType() == FNiagaraTypeDefinition::GetVec4Def())
				return Offsets[i];
			else if (NumChannels == 4 && Var.GetType() == FNiagaraTypeDefinition::GetColorDef())
				return Offsets[i];
		}
	}

	return INDEX_NONE;
}

bool FGrid2DCollectionRWInstanceData_GameThread::UpdateTargetTexture(ENiagaraGpuBufferFormat BufferFormat, UObject* Owner)
{
	// Pull value from user parameter
	if (UObject* UserParamObject = RTUserParamBinding.GetValue())
	{
		if (UserParamObject->IsA<UTextureRenderTarget2DArray>() || UserParamObject->IsA<UTextureRenderTarget2D>() )
		{
			TargetTexture = CastChecked<UTextureRenderTarget>(UserParamObject);
		}
		else
		{
			UE_LOGF(LogNiagara, Error, "RenderTarget UserParam is a '%ls' but is expected to be a UTextureRenderTarget2DArray or UTextureRenderTarget2D", *GetNameSafe(UserParamObject->GetClass()));
		}
	}

	// Could be from user parameter of created internally
	if (TargetTexture != nullptr)
	{
		if ( UTextureRenderTarget2DArray* TargetTextureArray = Cast<UTextureRenderTarget2DArray>(TargetTexture) )
		{
			const TOptional<EPixelFormat> RenderTargetFormat = FNiagaraUtilities::BufferFormatToPixelFormat(BufferFormat, GNDIGrid2DFomatCaps, 1, Owner);
			if (RenderTargetFormat.IsSet())
			{
				if (TargetTextureArray->SizeX != NumCells.X || TargetTextureArray->SizeY != NumCells.Y || TargetTextureArray->Slices != NumAttributes || TargetTextureArray->OverrideFormat != RenderTargetFormat.GetValue())
				{
					TargetTextureArray->OverrideFormat = RenderTargetFormat.GetValue();
					TargetTextureArray->ClearColor = FLinearColor(0.5, 0, 0, 0);
					TargetTextureArray->InitAutoFormat(NumCells.X, NumCells.Y, NumAttributes);
					TargetTextureArray->UpdateResourceImmediate(true);
					return true;
				}
			}
		}
		else if (UTextureRenderTarget2D* TargetTexture2D = Cast<UTextureRenderTarget2D>(TargetTexture))
		{
			const int MaxTextureDim = GMaxTextureDimensions;
			const int MaxTilesX = FMath::DivideAndRoundDown<int>(GMaxTextureDimensions, NumCells.X);
			const int MaxTilesY = FMath::DivideAndRoundDown<int>(GMaxTextureDimensions, NumCells.Y);
			const int MaxAttributes = MaxTilesX * MaxTilesY;
			if ( NumAttributes > MaxAttributes )
			{
				TargetTexture = nullptr;
			}
			else
			{
				const FNiagaraGrid2DLegacyTiled2DInfo Tiled2DInfo(NumCells, NumAttributes);

				const TOptional<ETextureRenderTargetFormat> RenderTargetFormat  = FNiagaraUtilities::BufferFormatToRenderTargetFormat(BufferFormat, GNDIGrid2DFomatCaps, Owner);
				if ( RenderTargetFormat.IsSet() )
				{
					if (TargetTexture2D->SizeX != Tiled2DInfo.Size.X || TargetTexture2D->SizeY != Tiled2DInfo.Size.Y || TargetTexture2D->RenderTargetFormat != RenderTargetFormat.GetValue())
					{
						TargetTexture2D->RenderTargetFormat = RenderTargetFormat.GetValue();
						TargetTexture2D->ClearColor = FLinearColor(0.5, 0, 0, 0);
						TargetTexture2D->bAutoGenerateMips = false;
						TargetTexture2D->InitAutoFormat(Tiled2DInfo.Size.X, Tiled2DInfo.Size.Y);
						TargetTexture2D->UpdateResourceImmediate(true);
						return true;
					}
				}
			}
		}
	}

	return false;
}

void FGrid2DCollectionRWInstanceData_RenderThread::BeginSimulate(FRDGBuilder& GraphBuilder, bool RequiresBuffering)
{
	for (TUniquePtr<FGrid2DBuffer>& Buffer : Buffers)
	{
		check(Buffer.IsValid());
		if (Buffer.Get() != CurrentData || !RequiresBuffering)
		{
			DestinationData = Buffer.Get();
			break;
		}
	}

	if (DestinationData == nullptr && PixelFormat.IsSet())
	{
		DestinationData = new FGrid2DBuffer();
		Buffers.Emplace(DestinationData);

		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2DArray(NumCells, PixelFormat.GetValue(), FClearValueBinding::Black, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV, uint16(NumAttributes));

		DestinationData->Initialize(GraphBuilder, TEXT("Grid2D::GridTexture"), TextureDesc);

		// This destination buffer will sometimes have old data in it.  Force it to clear.
		AddClearUAVPass(GraphBuilder, DestinationData->GetOrCreateUAV(GraphBuilder), FVector4f(ForceInitToZero));
	}
}

void FGrid2DCollectionRWInstanceData_RenderThread::EndSimulate()
{
	CurrentData = DestinationData;
	DestinationData = nullptr;
}

void FNiagaraDataInterfaceProxyGrid2DCollectionProxy::ResetData(const FNDIGpuComputeResetContext& Context)
{
	FGrid2DCollectionRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	if (!ProxyData)
	{
		return;
	}

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	for (TUniquePtr<FGrid2DBuffer>& Buffer : ProxyData->Buffers)
	{
		AddClearUAVPass(GraphBuilder, Buffer->GetOrCreateUAV(GraphBuilder), FVector4f(ForceInitToZero));
	}
}

void FNiagaraDataInterfaceProxyGrid2DCollectionProxy::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	// #todo(dmp): Context doesnt need to specify if a stage is output or not since we moved pre/post stage to the DI itself.  Not sure which design is better for the future
	if (Context.IsOutputStage())
	{
		FGrid2DCollectionRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

		FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
		ProxyData->BeginSimulate(GraphBuilder, Context.IsInputStage());

		// If there is an output DI and the stage dispatch count is different from the grid resolution, then we want to consider it for a pre-stage UAV clear.
		// This is to enable the most common use case of scattered writes to temporary buffers		
		const FNiagaraSimStageData& SimStageData = Context.GetSimStageData();
		const FIntVector3& ElementCount = SimStageData.DispatchArgs.ElementCount;

		if (ProxyData->ClearBeforeNonIterationStage && ProxyData->DestinationData && (ElementCount.X != ProxyData->NumCells.X || ElementCount.Y != ProxyData->NumCells.Y))
		{
			AddClearUAVPass(GraphBuilder, ProxyData->DestinationData->GetOrCreateUAV(GraphBuilder), FVector4f(ForceInitToZero));
		}
	}
}

void FNiagaraDataInterfaceProxyGrid2DCollectionProxy::PostStage(const FNDIGpuComputePostStageContext& Context)
{
	if (Context.IsOutputStage())
	{
		FGrid2DCollectionRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
		ProxyData->EndSimulate();
	}
}

void FNiagaraDataInterfaceProxyGrid2DCollectionProxy::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	FGrid2DCollectionRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

	if (ProxyData->RenderTargetToCopyTo != nullptr && ProxyData->CurrentData != nullptr && ProxyData->CurrentData->IsValid())
	{
		FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
		if (ProxyData->RenderTargetToCopyTo->GetTexture2DArray() != nullptr)
		{
			ProxyData->CurrentData->CopyToTexture(GraphBuilder, ProxyData->RenderTargetToCopyTo, TEXT("NiagaraRenderTargetToCopyTo"));
		}
		else if (ensure(ProxyData->RenderTargetToCopyTo->GetTexture2D() != nullptr))
		{
			const FNiagaraGrid2DLegacyTiled2DInfo Tiled2DInfo(ProxyData->NumCells, ProxyData->NumAttributes);
			Tiled2DInfo.CopyTo2D(GraphBuilder, ProxyData->CurrentData->GetOrCreateTexture(GraphBuilder), ProxyData->RenderTargetToCopyTo, TEXT("NiagaraRenderTargetToCopyTo"));
		}
	}

#if NIAGARA_COMPUTEDEBUG_ENABLED && WITH_EDITORONLY_DATA
	if (ProxyData->bPreviewGrid && ProxyData->CurrentData && ProxyData->CurrentData->IsValid())
	{
		FNiagaraGpuComputeDebugInterface GpuComputeDebugInterface = Context.GetComputeDispatchInterface().GetGpuComputeDebugInterface();
		FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
		if (ProxyData->PreviewAttribute[0] != INDEX_NONE)
		{
			GpuComputeDebugInterface.AddAttributeTexture(GraphBuilder, Context.GetSystemInstanceID(), GetSourceDIName(), ProxyData->CurrentData->GetOrCreateTexture(GraphBuilder), FIntPoint::ZeroValue, ProxyData->PreviewAttribute);
		}
		else
		{
			GpuComputeDebugInterface.AddTexture(GraphBuilder, Context.GetSystemInstanceID(), GetSourceDIName(), ProxyData->CurrentData->GetOrCreateTexture(GraphBuilder));
		}
	}
#endif

	// Clear out the transient resource we cached
	if (Context.IsFinalPostSimulate())
	{
		for (TUniquePtr<FGrid2DBuffer>& Buffer : ProxyData->Buffers)
		{
			Buffer->EndGraphUsage();
		}

		// Readers point to data not owned by themselves so can be caching resources on the 'other' proxy
		// Therefore we need to ensure the transient buffers are correctly cleared
		if ( FNiagaraDataInterfaceProxyGrid2DCollectionProxy* OtherGrid3DProxy = static_cast<FNiagaraDataInterfaceProxyGrid2DCollectionProxy*>(ProxyData->OtherProxy) )
		{
			FGrid2DCollectionRWInstanceData_RenderThread& OtherProxyData = OtherGrid3DProxy->SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());
			for (TUniquePtr<FGrid2DBuffer>& Buffer : OtherProxyData.Buffers)
			{
				Buffer->EndGraphUsage();
			}
		}
	}
}

void FNiagaraDataInterfaceProxyGrid2DCollectionProxy::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context)
{
	if ( const FGrid2DCollectionRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()) )
	{
		// support a grid reader acting as an iteration source
		if (ProxyData->OtherProxy != nullptr)
		{
			FNiagaraDataInterfaceProxyGrid2DCollectionProxy* OtherGrid2DProxy = static_cast<FNiagaraDataInterfaceProxyGrid2DCollectionProxy*>(ProxyData->OtherProxy);
			const FGrid2DCollectionRWInstanceData_RenderThread* OtherProxyData = OtherGrid2DProxy->SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
			Context.SetDirect(OtherProxyData->NumCells);
		}
		else
		{
			Context.SetDirect(ProxyData->NumCells);
		}		
	}
}

#undef LOCTEXT_NAMESPACE

