// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceGrid2DCollectionReader.h"
#include "Engine/TextureRenderTarget.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraScriptRuntimeData.h"
#include "NiagaraSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceGrid2DCollectionReader)


#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceGrid2DCollectionReader"

UNiagaraDataInterfaceGrid2DCollectionReader::UNiagaraDataInterfaceGrid2DCollectionReader(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyGrid2DCollectionProxy());

	FNiagaraTypeDefinition Def(UTextureRenderTarget::StaticClass());
	RenderTargetUserParameter.Parameter.SetType(Def);

	bUseIndirection = true;
}

bool UNiagaraDataInterfaceGrid2DCollectionReader::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceGrid2DCollectionReader* OtherTyped = CastChecked<const UNiagaraDataInterfaceGrid2DCollectionReader>(Other);

	return OtherTyped != nullptr && OtherTyped->EmitterName == EmitterName && OtherTyped->DIName == DIName;
}

bool UNiagaraDataInterfaceGrid2DCollectionReader::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceGrid2DCollectionReader* OtherTyped = CastChecked<UNiagaraDataInterfaceGrid2DCollectionReader>(Destination);
	OtherTyped->EmitterName = EmitterName;
	OtherTyped->DIName = DIName;
	return true;
}

bool UNiagaraDataInterfaceGrid2DCollectionReader::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	check(Proxy);

	FGrid2DCollectionRWInstanceData_GameThread* InstanceData = new (PerInstanceData) FGrid2DCollectionRWInstanceData_GameThread();
	SystemInstancesToProxyData_GT.Emplace(SystemInstance->GetId(), InstanceData);

	FNiagaraEmitterInstance* EmitterInstanceToUse = nullptr;
	for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
	{
		const UNiagaraEmitter* Emitter = EmitterInstance->GetEmitter();
		if (Emitter == nullptr)
		{
			continue;
		}

		if (EmitterName == Emitter->GetUniqueEmitterName())
		{
			EmitterInstanceToUse = &EmitterInstance.Get();
			break;
		}
	}

	// Look for proxy we are going to use
	FNiagaraDataInterfaceProxy* ProxyToUse = nullptr;
	UNiagaraDataInterfaceGrid2DCollection* GridInterface = nullptr;

	if (EmitterInstanceToUse != nullptr)
	{
		FNiagaraComputeExecutionContext* ExecContext = EmitterInstanceToUse->GetGPUContext();
		if (ExecContext != nullptr && ExecContext->GPUScript != nullptr && ExecContext->GetScriptRuntimeData() != nullptr)
		{
			TConstArrayView<FNiagaraScriptResolvedDataInterfaceInfo> DataInterfaceInfo = ExecContext->GetScriptRuntimeData()->GetResolvedDataInterfaces();
			const TArray<UNiagaraDataInterface*>& DataInterfaces = ExecContext->CombinedParamStore.GetDataInterfaces();

			FString FullName = FString("Emitter.") + DIName;
			int Index = 0;

			// #todo(dmp): we are looking at the UObjects that define the DIs here 

			for (UNiagaraDataInterface* Interface : DataInterfaces)
			{
				if (DataInterfaceInfo[Index].CompileName.GetPlainNameString() == FullName)
				{
					GridInterface = static_cast<UNiagaraDataInterfaceGrid2DCollection*>(Interface);
					ProxyToUse = Interface->GetProxy();
					break;
				}
				++Index;
			}
		}

		if (GridInterface && ProxyToUse)
		{

			int32 NumAttribChannelsFound = 0;
			int32 NumNamedAttribChannelsFound = 0;
			TArray<FNiagaraVariableBase> Vars;
			TArray<uint32> Offsets;
			FindAttributes(Vars, Offsets, NumNamedAttribChannelsFound, nullptr);

			FGrid2DCollectionRWInstanceData_GameThread* Grid2DInstanceData = GridInterface->GetSystemInstancesToProxyData_GT().FindRef(SystemInstance->GetId());
			InstanceData->OtherDI = GridInterface;
			InstanceData->OtherInstanceData = Grid2DInstanceData;

			// error if the reader is trying to access a grid that has both named and unnamed attributes
			if (InstanceData->OtherInstanceData != nullptr && GridInterface->NumAttributes > 0 && GridInterface->NumAttributes != InstanceData->OtherInstanceData->NumAttributes)
			{
				UE_LOGF(LogNiagara, Warning, "Grid2DCollectionReaders are not compatible with Emitter %ls and DataInterface %ls.  Do not use unnamed and named grids together.", *this->EmitterName, *this->DIName);				
			}
		}
		else
		{
			UE_LOGF(LogNiagara, Warning, "Grid2DCollectionReader %ls refers to an invalid Grid2D Collection DI", *this->DIName);				
		}
	}
	else 
	{
		UE_LOGF(LogNiagara, Warning, "Grid2DCollectionReader %ls has invalid emitter", *this->DIName);
	}

	// Push Updates to Proxy.
	FNiagaraDataInterfaceProxyGrid2DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid2DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Proxy, InstanceID = SystemInstance->GetId(), RT_ProxyToUse = ProxyToUse](FRHICommandListImmediate& RHICmdList)
	{
		check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));				

		FGrid2DCollectionRWInstanceData_RenderThread* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.Add(InstanceID);
		TargetData->OtherProxy = RT_ProxyToUse;
	});

	return true;
}

void UNiagaraDataInterfaceGrid2DCollectionReader::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FGrid2DCollectionRWInstanceData_GameThread* InstanceData = static_cast<FGrid2DCollectionRWInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FGrid2DCollectionRWInstanceData_GameThread();

	SystemInstancesToProxyData_GT.Remove(SystemInstance->GetId());

	FNiagaraDataInterfaceProxyGrid2DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid2DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Proxy, InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
		{
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		});
}

void UNiagaraDataInterfaceGrid2DCollectionReader::GetEmitterDependencies(UNiagaraSystem* Asset, TArray<FVersionedNiagaraEmitter>& Dependencies) const
{
	if (!Asset)
	{
		return;
	}

	for (const FNiagaraEmitterHandle& EmitterHandle : Asset->GetEmitterHandles())
	{
		FVersionedNiagaraEmitter EmitterInstance = EmitterHandle.GetInstance();
		if (EmitterInstance.Emitter && EmitterInstance.Emitter->GetUniqueEmitterName() == EmitterName)
		{
			Dependencies.Add(EmitterInstance);
			return;
		}
	}
}


#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceGrid2DCollectionReader::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	int32 NumFunctionsBefore = OutFunctions.Num();

	Super::GetFunctionsInternal(OutFunctions);
	TSet<FName> FunctionsToRemove =
	{
		ClearCellFunctionName,
		CopyPreviousToCurrentForCellFunctionName,		
		SetValueFunctionName,		
		SetVector4ValueFunctionName,
		SetVector3ValueFunctionName,
		SetVector2ValueFunctionName,
		SetFloatValueFunctionName,
		GetPreviousVector4ValueFunctionName,
		SetVectorValueFunctionName,
		SetVector2DValueFunctionName,
		SetNumCellsFunctionName,
		SetNumCellsFunctionName
	};

	for (int32 ItemIndex = NumFunctionsBefore; ItemIndex < OutFunctions.Num();)
	{
		if (FunctionsToRemove.Find(OutFunctions[ItemIndex].Name) != nullptr)
		{
			OutFunctions.RemoveAtSwap(ItemIndex, 1);
		}
		else
		{
			++ItemIndex;
		}
	}
}
#endif

bool UNiagaraDataInterfaceGrid2DCollectionReader::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FGrid2DCollectionRWInstanceData_GameThread* InstanceData = SystemInstancesToProxyData_GT.FindRef(SystemInstance->GetId());

	// we can run into the case where depending on the ordering of DI initialization, we might have not been able to grab the other grid's InstanceData
	// in InitPerInstanceData.  If this is the case, we ensure it is correct here.
	if (InstanceData->OtherInstanceData == nullptr && InstanceData->OtherDI)
	{
		UNiagaraDataInterfaceGrid2DCollection* OtherGridDI = static_cast<UNiagaraDataInterfaceGrid2DCollection*>(InstanceData->OtherDI);
		
		if (OtherGridDI)
		{
			FGrid2DCollectionRWInstanceData_GameThread* Grid2DInstanceData = OtherGridDI->GetSystemInstancesToProxyData_GT().FindRef(SystemInstance->GetId());
			InstanceData->OtherInstanceData = Grid2DInstanceData;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

