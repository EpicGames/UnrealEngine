// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceGrid3DCollection.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraComponent.h"
#include "NiagaraConstants.h"
#include "NiagaraEmitter.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraRenderGraphUtils.h"
#include "NiagaraSimStageData.h"
#include "NiagaraRenderer.h"
#include "NiagaraSettings.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraGpuComputeDebugInterface.h"
#include "NiagaraDataInterfaceGrid3DCollectionUtils.h"
#include "NiagaraScriptRuntimeData.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/TextureRenderTargetVolume.h"
#include "RHIStaticStates.h"
#include "RenderGraphUtils.h"
#include "TextureResource.h"

#include "NiagaraDataInterfaceGrid3DCollectionReader.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceGrid3DCollection)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceGrid3DCollection"

IMPLEMENT_TYPE_LAYOUT(FNiagaraDataInterfaceParametersCS_Grid3DCollection);

BEGIN_SHADER_PARAMETER_STRUCT(FNDIGrid3DShaderParameters, )
	SHADER_PARAMETER(int,							NumAttributes)
	SHADER_PARAMETER(int,							NumNamedAttributes)
	SHADER_PARAMETER(FVector3f,						UnitToUV)
	SHADER_PARAMETER(FIntVector,					NumCells)
	SHADER_PARAMETER(FVector3f,						CellSize)
	SHADER_PARAMETER(FIntVector,					NumTiles)
	SHADER_PARAMETER(FVector3f,						OneOverNumTiles)
	SHADER_PARAMETER(FVector3f,						UnitClampMin)
	SHADER_PARAMETER(FVector3f,						UnitClampMax)
	SHADER_PARAMETER(FVector3f,						WorldBBoxSize)

	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D<float>,		Grid)
	SHADER_PARAMETER_SAMPLER(SamplerState,					GridSampler)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>,	OutputGrid)
	SHADER_PARAMETER_SRV(Buffer<float4>,					PerAttributeData)
END_SHADER_PARAMETER_STRUCT()

static float GNiagaraGrid3DResolutionMultiplier = 1.0f;
static FAutoConsoleVariableRef CVarNiagaraGrid3DResolutionMultiplier(
	TEXT("fx.Niagara.Grid3D.ResolutionMultiplier"),
	GNiagaraGrid3DResolutionMultiplier,
	TEXT("Optional global modifier to grid resolution\n"),
	ECVF_Default
);

static int32 GNiagaraGrid3DOverrideFormat = -1;
static FAutoConsoleVariableRef CVarNiagaraGrid3DOverrideFormat(
	TEXT("fx.Niagara.Grid3D.OverrideFormat"),
	GNiagaraGrid3DOverrideFormat,
	TEXT("Optional override for all grids to use this format.\n"),
	ECVF_Default
);

static constexpr EPixelFormatCapabilities GNDIGrid3DFomatCaps = EPixelFormatCapabilities::TypedUAVLoad | EPixelFormatCapabilities::TypedUAVStore | EPixelFormatCapabilities::Texture3D;

const FString UNiagaraDataInterfaceGrid3DCollection::NumTilesName(TEXT("_NumTiles"));
const FString UNiagaraDataInterfaceGrid3DCollection::OneOverNumTilesName(TEXT("_OneOverNumTiles"));
const FString UNiagaraDataInterfaceGrid3DCollection::UnitClampMinName(TEXT("_UnitClampMin"));
const FString UNiagaraDataInterfaceGrid3DCollection::UnitClampMaxName(TEXT("_UnitClampMax"));

const FString UNiagaraDataInterfaceGrid3DCollection::GridName(TEXT("_Grid"));
const FString UNiagaraDataInterfaceGrid3DCollection::OutputGridName(TEXT("_OutputGrid"));
const FString UNiagaraDataInterfaceGrid3DCollection::SamplerName(TEXT("_GridSampler"));

const FName UNiagaraDataInterfaceGrid3DCollection::SetNumCellsFunctionName("SetNumCells");

const FName UNiagaraDataInterfaceGrid3DCollection::ClearCellFunctionName("ClearCell");
const FName UNiagaraDataInterfaceGrid3DCollection::CopyPreviousToCurrentForCellFunctionName("CopyPreviousToCurrentForCell");
const FName UNiagaraDataInterfaceGrid3DCollection::CopyMaskedPreviousToCurrentForCellFunctionName("CopyMaskedPreviousToCurrentForCell");

// Global VM function names, also used by the shaders code generation methods.
const FName UNiagaraDataInterfaceGrid3DCollection::SetValueFunctionName("SetGridValue");
const FName UNiagaraDataInterfaceGrid3DCollection::GetValueFunctionName("GetGridValue");

const FName UNiagaraDataInterfaceGrid3DCollection::SetFullGridValueFunctionName("SetFullGridValue");
const FName UNiagaraDataInterfaceGrid3DCollection::GetFullGridPreviousValueFunctionName("GetFullGridPreviousValue");
const FName UNiagaraDataInterfaceGrid3DCollection::SamplePreviousFullGridFunctionName("SamplePreviousFullGrid");
const FName UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousFullGridFunctionName("CubicSamplePreviousFullGrid");

const FName UNiagaraDataInterfaceGrid3DCollection::SetVector4ValueFunctionName("SetVector4Value");
const FName UNiagaraDataInterfaceGrid3DCollection::SetVector3ValueFunctionName("SetVector3Value");
const FName UNiagaraDataInterfaceGrid3DCollection::SetVector2ValueFunctionName("SetVector2Value");
const FName UNiagaraDataInterfaceGrid3DCollection::GetVector2ValueFunctionName("GetVector2Value");
const FName UNiagaraDataInterfaceGrid3DCollection::SetFloatValueFunctionName("SetFloatValue");
const FName UNiagaraDataInterfaceGrid3DCollection::GetPreviousValueAtIndexFunctionName("GetPreviousValueAtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridAtIndexFunctionName("SamplePreviousGridAtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridAtIndexFunctionName("CubicSamplePreviousGridAtIndex");

const FName UNiagaraDataInterfaceGrid3DCollection::SetVector4ValueAtIndexFunctionName("SetVector4ValueAtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::SetVector3ValueAtIndexFunctionName("SetVector3ValueAtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::SetVector2ValueAtIndexFunctionName("SetVector2ValueAtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::SetFloatValueAtIndexFunctionName("SetFloatValueAtIndex");

const FName UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector4ValueAtIndexFunctionName("GetPreviousVector4ValueAtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector3ValueAtIndexFunctionName("GetPreviousVector3ValueAtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector2ValueAtIndexFunctionName("GetPreviousVector2ValueAtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::GetPreviousFloatValueAtIndexFunctionName("GetPreviousFloatValueAtIndex");

const FName UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector4AtIndexFunctionName("SamplePreviousGridVector4AtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector3AtIndexFunctionName("SamplePreviousGridVector3AtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector2AtIndexFunctionName("SamplePreviousGridVector2AtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridFloatAtIndexFunctionName("SamplePreviousGridFloatAtIndex");

const FName UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector4AtIndexFunctionName("CubicSamplePreviousGridVector4AtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector3AtIndexFunctionName("CubicSamplePreviousGridVector3AtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector2AtIndexFunctionName("CubicSamplePreviousGridVector2AtIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridFloatAtIndexFunctionName("CubicSamplePreviousGridFloatAtIndex");

const FName UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector4ValueFunctionName("GetPreviousVector4Value");
const FName UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector4FunctionName("SamplePreviousGridVector4Value");
const FName UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector4FunctionName("CubicSamplePreviousGridVector4Value");
const FName UNiagaraDataInterfaceGrid3DCollection::SetVectorValueFunctionName("SetVectorValue");
const FName UNiagaraDataInterfaceGrid3DCollection::GetPreviousVectorValueFunctionName("GetPreviousVectorValue");
const FName UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVectorFunctionName("SamplePreviousGridVector3Value");
const FName UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVectorFunctionName("CubicSamplePreviousGridVector3Value");
const FName UNiagaraDataInterfaceGrid3DCollection::SetVector2DValueFunctionName("SetVector2DValue");
const FName UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector2DValueFunctionName("GetPreviousVector2DValue");
const FName UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector2DFunctionName("SamplePreviousGridVector2DValue");
const FName UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector2DFunctionName("CubicSamplePreviousGridVector2DValue");
const FName UNiagaraDataInterfaceGrid3DCollection::GetPreviousFloatValueFunctionName("GetPreviousFloatValue");
const FName UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridFloatFunctionName("SamplePreviousGridFloatValue");
const FName UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridFloatFunctionName("CubicSamplePreviousGridFloatValue");

const FName UNiagaraDataInterfaceGrid3DCollection::GetOutputGridFloatValueFunctionName("GetOutputGridFloatValue");
const FName UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVector2ValueFunctionName("GetOutputGridVector2Value");
const FName UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVectorValueFunctionName("GetOutputGridVectorValue");
const FName UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVector4ValueFunctionName("GetOutputGridVector4Value");

const FName UNiagaraDataInterfaceGrid3DCollection::GetVector4AttributeIndexFunctionName("GetVector4AttributeIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::GetVectorAttributeIndexFunctionName("GetVectorAttributeIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::GetVector2DAttributeIndexFunctionName("GetVector2DAttributeIndex");
const FName UNiagaraDataInterfaceGrid3DCollection::GetFloatAttributeIndexFunctionName("GetFloatAttributeIndex");

const FName UNiagaraDataInterfaceGrid3DCollection::SampleGridFunctionName("SampleGrid");
const FName UNiagaraDataInterfaceGrid3DCollection::CubicSampleGridFunctionName("CubicSampleGrid");

const FString UNiagaraDataInterfaceGrid3DCollection::AttributeIndicesBaseName(TEXT("AttributeIndices"));
const FString UNiagaraDataInterfaceGrid3DCollection::PerAttributeDataName(TEXT("PerAttributeData"));

const TCHAR* UNiagaraDataInterfaceGrid3DCollection::VectorComponentNames[] = { TEXT(".x"), TEXT(".y"), TEXT(".z"), TEXT(".w") };

FNiagaraVariableBase UNiagaraDataInterfaceGrid3DCollection::ExposedRTVar;
const FString UNiagaraDataInterfaceGrid3DCollection::AnonymousAttributeString("Attribute At Index");

const TCHAR* UNiagaraDataInterfaceGrid3DCollection::TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceGrid3DCollection.ush");

struct FNiagaraGridCollection3DDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		AddGetOutputGridValue,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};


bool UNiagaraDataInterfaceGrid3DCollection::CanCreateVarFromFuncName(const FName& FuncName)
{
	if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector4ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector4ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector4FunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector4FunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector4ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector4ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector4AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector4AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVector4ValueFunctionName)

		return true;
	else if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector3ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVectorValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVectorValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVectorFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVectorFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector3ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector3ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector3AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector3AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVectorValueFunctionName)
		return true;
	else if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector2ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector2DValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector2DValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector2DFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector2DFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector2ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector2ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector2AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector2AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVector2ValueFunctionName)
		return true;
	else if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetFloatValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousFloatValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridFloatFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridFloatFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetFloatValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousFloatValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridFloatAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridFloatAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridFloatValueFunctionName)
		return true;
	return false;
}

FNiagaraTypeDefinition UNiagaraDataInterfaceGrid3DCollection::GetValueTypeFromFuncName(const FName& FuncName)
{
	if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector4ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector4ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector4FunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector4FunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector4ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector4ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector4AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector4AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVector4ValueFunctionName)
		return FNiagaraTypeDefinition::GetVec4Def();
	else if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector3ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVectorValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVectorValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVectorFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVectorFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector3ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector3ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector3AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector3AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVectorValueFunctionName)
		return FNiagaraTypeDefinition::GetVec3Def();
	else if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector2ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector2DValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector2DValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector2DFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector2DFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector2ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector2ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector2AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector2AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVector2ValueFunctionName)
		return FNiagaraTypeDefinition::GetVec2Def();
	else if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetFloatValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousFloatValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridFloatFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridFloatFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetFloatValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousFloatValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridFloatAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridFloatAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridFloatValueFunctionName)
		return FNiagaraTypeDefinition::GetFloatDef();

	return FNiagaraTypeDefinition();
}

int32 UNiagaraDataInterfaceGrid3DCollection::GetComponentCountFromFuncName(const FName& FuncName)
{
	if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector4ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector4ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector4FunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector4FunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector4ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector4ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector4AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector4AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetVector4AttributeIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVector4ValueFunctionName)
		return 4;
	else if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector3ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVectorValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVectorValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVectorFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVectorFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector3ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector3ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector3AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector3AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetVectorAttributeIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVectorValueFunctionName)
		return 3;
	else if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector2ValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector2DValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector2DValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector2DFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector2DFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetVector2ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousVector2ValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridVector2AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridVector2AtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetVector2DAttributeIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridVector2ValueFunctionName)
		return 2;
	else if (
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetFloatValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousFloatValueFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridFloatFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridFloatFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SetFloatValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetPreviousFloatValueAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::SamplePreviousGridFloatAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::CubicSamplePreviousGridFloatAtIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetFloatAttributeIndexFunctionName ||
		FuncName == UNiagaraDataInterfaceGrid3DCollection::GetOutputGridFloatValueFunctionName)
		return 1;

	return INDEX_NONE;
}

UNiagaraDataInterfaceGrid3DCollection::UNiagaraDataInterfaceGrid3DCollection(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, NumAttributes(0)
	, OverrideBufferFormat(ENiagaraGpuBufferFormat::Float)
	, bOverrideFormat(false)
#if WITH_EDITORONLY_DATA
	, bPreviewGrid(false)
#endif
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyGrid3DCollectionProxy());

	FNiagaraTypeDefinition Def(UTextureRenderTarget::StaticClass());
	RenderTargetUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceGrid3DCollection::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
		ExposedRTVar = FNiagaraVariableBase(FNiagaraTypeDefinition(UTexture::StaticClass()), TEXT("RenderTarget"));
	}
}

#if WITH_EDITOR
bool UNiagaraDataInterfaceGrid3DCollection::ShouldCompile(EShaderPlatform ShaderPlatform) const
{
	return
		RHIVolumeTextureRenderingSupportGuaranteed(ShaderPlatform) &&
		Super::ShouldCompile(ShaderPlatform);
}
#endif

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceGrid3DCollection::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
		return false;


	InVisitor->UpdatePOD(TEXT("UNiagaraDataInterfaceGrid3DCollectionVersion"), (int32)FNiagaraGridCollection3DDIFunctionVersion::LatestVersion);
	InVisitor->UpdatePOD(TEXT("UNiagaraDataInterfaceGrid3DCollectionSupportsRGBAGrid"), FGrid3DCollectionAttributeHlslWriter::SupportsRGBAGrid() ? 1 : 0);
	//InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	InVisitor->UpdateShaderParameters<FNDIGrid3DShaderParameters>();

	return true;
}
#endif

#if WITH_EDITOR
void UNiagaraDataInterfaceGrid3DCollection::GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors,
	TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	Super::GetFeedback(Asset, Component, OutErrors, OutWarnings, OutInfo);
		
	int32 NumAttribChannelsFound = 0;
	int32 NumNamedAttribChannelsFound = 0;
	TArray<FNiagaraVariableBase> Vars;
	TArray<uint32> Offsets;
	FindAttributes(Vars, Offsets, NumNamedAttribChannelsFound);

	// Ensure we never allocate 0 attributes as that would fail to create the texture
	NumAttribChannelsFound = NumAttributes + NumNamedAttribChannelsFound;

	if (NumAttribChannelsFound == 0)
	{
		OutWarnings.Emplace(LOCTEXT("Grid3DZeroAttrs", "Zero attributes defined on grid"),
			LOCTEXT("Grid3DZeroAttrsFmt", "Zero attributes defined on grid"),
			FNiagaraDataInterfaceFix());
	}

}
#endif
#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceGrid3DCollection::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	if (Super::UpgradeFunctionCall(FunctionSignature))
		return true;

	FName UpgradeName;
	if (FunctionSignature.Name == GetValueFunctionName)
		UpgradeName = GetPreviousValueAtIndexFunctionName;

	if (UpgradeName != NAME_None)
	{
		TArray<FNiagaraFunctionSignature> Sigs;
		GetFunctionsInternal(Sigs);

		for (const FNiagaraFunctionSignature& Sig : Sigs)
		{
			if (Sig.Name == UpgradeName)
			{
				FNiagaraFunctionSignature Backup = FunctionSignature;
				FunctionSignature = Sig;
				return true;
			}
		}
	}

	return false;
}

void UNiagaraDataInterfaceGrid3DCollection::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	Super::GetFunctionsInternal(OutFunctions);

	int32 StartIndex = OutFunctions.Num();

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetNumCellsFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		OutFunctions.Add(Sig);
	}

	{
		// Older, deprecated form
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_GetValueFunction", "Get the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousValueAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_PreviousValueAtIndexFunction", "Get the value at a specific index.");

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetFullGridPreviousValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.bHidden = true;
		Sig.bSoftDeprecatedFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_PreviousValueAtIndexFunction", "Get the value at a specific index.");

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IGNORE")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetFullGridValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IGNORE")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bHidden = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ClearCellFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_ClearCellFunction", "Set all attributes for a given cell to be zeroes.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CopyPreviousToCurrentForCellFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_CopyPreviousToCurrentForCell", "Take the previous contents of the cell and copy to the output location for the cell.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CopyMaskedPreviousToCurrentForCellFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumAttributesSet")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeMask")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_CopyMaskedPreviousToCurrentForCell", "Take the previous contents of the cell and copy to the output location for the cell.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetVector4ValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetVector4", "Sets a Vector4 value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousVector4ValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_GetVector4", "Gets a Vector4 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridVector4FunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_SampleVector4", "Sample a Vector4 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridVector4FunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_CubicSampleVector4", "Cubic Sample a Vector4 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetVectorValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_SetVector3", "Sets a Vector3 value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousVectorValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_GetVector3", "Gets a Vector3 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridVectorFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_SampleVector3", "Sample a Vector3 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridVectorFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_CubicSampleVector3", "Cubic Sample a Vector3 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetVector2DValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_SetVector2", "Sets a Vector2 value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousVector2DValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_GetVector2", "Gets a Vector2 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridVector2DFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_SampleVector2", "Sample a Vector2 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridVector2DFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_CubicSampleVector2", "Cubic Sample a Vector2 value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetFloatValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_SetFloat", "Sets a float value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousFloatValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_GetFloat", "Gets a float value on the Grid by Attribute name. Note that this is the value from the previous execution stage.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridFloatFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_SampleFloat", "Sample a float value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);

	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridFloatFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_CubicSampleFloat", "Cubic Sample a float value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);

	}

	// GetOutputGrid* functions - read from OutputGrid UAV (destination buffer) for in-place iterative solvers (e.g. RBGS-SOR).
	// NOTE: bWriteFunction is set to true even though these functions only READ from the OutputGrid. This is intentional:
	// bWriteFunction adds the DI to OutputDestinations, which makes IsOutputStage()=true so PreStage/PostStage fire.
	// Crucially, bReadFunction is NOT set, so the DI is NOT added to InputDataInterfaces, meaning IsInputStage()=false.
	// This causes BeginSimulate to receive RequiresBuffering=false, which sets DestinationData=CurrentData (single buffer).
	// If we used bReadFunction instead, it would force double-buffering, defeating the purpose of these functions.
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetOutputGridFloatValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bWriteFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_GetOutputGridFloat", "Gets a float value from the current OutputGrid (destination buffer) by Attribute name. Use for in-place iterative solvers like Red-Black Gauss-Seidel.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetOutputGridVector2ValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bWriteFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_GetOutputGridVector2", "Gets a Vector2 value from the current OutputGrid (destination buffer) by Attribute name. Use for in-place iterative solvers like Red-Black Gauss-Seidel.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetOutputGridVectorValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bWriteFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_GetOutputGridVector", "Gets a Vector3 value from the current OutputGrid (destination buffer) by Attribute name. Use for in-place iterative solvers like Red-Black Gauss-Seidel.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetOutputGridVector4ValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bWriteFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_GetOutputGridVector4", "Gets a Vector4 value from the current OutputGrid (destination buffer) by Attribute name. Use for in-place iterative solvers like Red-Black Gauss-Seidel.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SampleGridFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSampleGridFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("UnitZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousFullGridFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.bHidden = true;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousFullGridFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.bHidden = true;
		Sig.bSoftDeprecatedFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVector4AttributeIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetVector4AttributeIndex", "Gets a attribute starting index value for Vector4 on the Grid by Attribute name. Returns -1 if not found.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVectorAttributeIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetVector3AttributeIndex", "Gets a attribute starting index value for Vector3 on the Grid by Attribute name. Returns -1 if not found.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVector2DAttributeIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetVector2AttributeIndex", "Gets a attribute starting index value for Vector2 on the Grid by Attribute name. Returns -1 if not found.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetFloatAttributeIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.FunctionSpecifiers.Add(FName(UNiagaraDataInterfaceRWBase::NAME_Attribute));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_GetFloatAttributeIndex", "Gets a attribute starting index value for float on the Grid by Attribute name. Returns -1 if not found.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousFloatValueAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_PreviousValueAtIndexFunction", "Get the value at a specific index.");

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousVector2ValueAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_PreviousValueAtIndexFunction", "Get the value at a specific index.");

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousVector3ValueAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_PreviousValueAtIndexFunction", "Get the value at a specific index.");

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPreviousVector4ValueAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_PreviousValueAtIndexFunction", "Get the value at a specific index.");

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridFloatAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridVector2AtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridVector3AtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePreviousGridVector4AtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridFloatAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridVector2AtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridVector3AtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = CubicSamplePreviousGridVector4AtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetFloatValueAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_SetFloat", "Sets a float value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetVector2ValueAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_SetFloat", "Sets a float value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetVector3ValueAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_SetFloat", "Sets a float value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetVector4ValueAtIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_SetFloat", "Sets a float value on the Grid by Attribute name.");
		OutFunctions.Add(Sig);
	}

	for (int32 i = StartIndex; i < OutFunctions.Num(); i++)
	{
		FNiagaraFunctionSignature& Function = OutFunctions[i];
		Function.FunctionVersion = FNiagaraGridCollection3DDIFunctionVersion::LatestVersion;
	}
}
#endif

// #todo(dmp): expose more CPU functionality
// #todo(dmp): ideally these would be exposed on the parent class, but we can't bind functions of parent classes but need to work on the interface
// for sharing an instance data object with the super class
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMGetWorldBBoxSize);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMGetCellSize);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMSetNumCells);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMGetNumCells);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMUnitToFloatIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceGrid3DCollection, VMGetAttributeIndex);
void UNiagaraDataInterfaceGrid3DCollection::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	Super::GetVMExternalFunction(BindingInfo, InstanceData, OutFunc);

	if (BindingInfo.Name == UNiagaraDataInterfaceRWBase::WorldBBoxSizeFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMGetWorldBBoxSize)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == UNiagaraDataInterfaceRWBase::CellSizeFunctionName)
	{
		// #todo(dmp): this will override the base class definition for GetCellSize because the data interface instance data computes cell size
		// it would be nice to refactor this so it can be part of the super class
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMGetCellSize)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetNumCellsFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMSetNumCells)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == NumCellsFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMGetNumCells)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == UnitToFloatIndexFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMUnitToFloatIndex)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetVector4AttributeIndexFunctionName)
	{
		FName AttributeName = BindingInfo.FindSpecifier(UNiagaraDataInterfaceRWBase::NAME_Attribute)->Value;
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMGetAttributeIndex)::Bind(this, OutFunc, AttributeName, 4);
	}
	else if (BindingInfo.Name == GetVectorAttributeIndexFunctionName)
	{
		FName AttributeName = BindingInfo.FindSpecifier(UNiagaraDataInterfaceRWBase::NAME_Attribute)->Value;
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMGetAttributeIndex)::Bind(this, OutFunc, AttributeName, 3);
	}
	else if (BindingInfo.Name == GetVector2DAttributeIndexFunctionName)
	{
		FName AttributeName = BindingInfo.FindSpecifier(UNiagaraDataInterfaceRWBase::NAME_Attribute)->Value;
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMGetAttributeIndex)::Bind(this, OutFunc, AttributeName, 2);
	}
	else if (BindingInfo.Name == GetFloatAttributeIndexFunctionName)
	{
		FName AttributeName = BindingInfo.FindSpecifier(UNiagaraDataInterfaceRWBase::NAME_Attribute)->Value;
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceGrid3DCollection, VMGetAttributeIndex)::Bind(this, OutFunc, AttributeName, 1);
	}
}

void UNiagaraDataInterfaceGrid3DCollection::VMGetNumCells(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FGrid3DCollectionRWInstanceData_GameThread> InstData(Context);

	FNDIOutputParam<int32> NumCellsX(Context);
	FNDIOutputParam<int32> NumCellsY(Context);
	FNDIOutputParam<int32> NumCellsZ(Context);

	int32 TmpNumCellsX = InstData->NumCells.X;
	int32 TmpNumCellsY = InstData->NumCells.Y;
	int32 TmpNumCellsZ = InstData->NumCells.Z;
	if (InstData->OtherInstanceData != nullptr)
	{
		TmpNumCellsX = InstData->OtherInstanceData->NumCells.X;
		TmpNumCellsY = InstData->OtherInstanceData->NumCells.Y;
		TmpNumCellsZ = InstData->OtherInstanceData->NumCells.Z;
	}

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		NumCellsX.SetAndAdvance(TmpNumCellsX);
		NumCellsY.SetAndAdvance(TmpNumCellsY);
		NumCellsZ.SetAndAdvance(TmpNumCellsZ);
	}
}

bool UNiagaraDataInterfaceGrid3DCollection::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceGrid3DCollection* OtherTyped = CastChecked<const UNiagaraDataInterfaceGrid3DCollection>(Other);

	return OtherTyped != nullptr &&
		OtherTyped->NumAttributes == NumAttributes &&
		OtherTyped->RenderTargetUserParameter == RenderTargetUserParameter &&
		OtherTyped->OverrideBufferFormat == OverrideBufferFormat &&
#if WITH_EDITORONLY_DATA
		OtherTyped->bPreviewGrid == bPreviewGrid &&
		OtherTyped->PreviewAttribute == PreviewAttribute &&
#endif
		OtherTyped->bOverrideFormat == bOverrideFormat;
}

bool UNiagaraDataInterfaceGrid3DCollection::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceGrid3DCollection* OtherTyped = CastChecked<UNiagaraDataInterfaceGrid3DCollection>(Destination);
	OtherTyped->NumAttributes = NumAttributes;
	OtherTyped->RenderTargetUserParameter = RenderTargetUserParameter;
	OtherTyped->OverrideBufferFormat = OverrideBufferFormat;
	OtherTyped->bOverrideFormat = bOverrideFormat;
#if WITH_EDITORONLY_DATA
	OtherTyped->bPreviewGrid = bPreviewGrid;
	OtherTyped->PreviewAttribute = PreviewAttribute;
#endif

	return true;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceGrid3DCollection::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	static const TCHAR* FormatDeclarations = TEXT(R"(
		Texture3D<float{FloatBufferSuffix}> {GridName};
		RWTexture3D<float{FloatBufferSuffix}> {OutputGridName};
		int3 {NumTiles};
		float3 {OneOverNumTiles};
		float3 {UnitClampMin};
		float3 {UnitClampMax};
		SamplerState {SamplerName};
		int4 {AttributeIndicesName}[{AttributeInt4Count}];
		Buffer<float4> {PerAttributeDataName};
		int {NumAttributesName};
		int {NumNamedAttributesName};
	)");

	// If we use an int array for the attribute indices, the shader compiler will actually use int4 due to the packing rules,
	// and leave 3 elements unused. Besides being wasteful, this means that the array we send to the CS would need to be padded,
	// which is a hassle. Instead, use int4 explicitly, and access individual components in the generated code.
	// Note that we have to have at least one here because hlsl doesn't support arrays of size 0.
	const int AttributeCount = ParamInfo.GeneratedFunctions.Num();
	const int AttributeInt4Count = FMath::Max(1, FMath::DivideAndRoundUp(AttributeCount, 4));

	// format of the grid is defined by number of attributes
	FString FloatBufferSuffix = "";
	FGrid3DCollectionAttributeHlslWriter AttributeWriter(ParamInfo);

	int TotalChannels = AttributeWriter.GetAttributeHelper().GetTotalChannels();
	if (AttributeWriter.UseRGBAGrid() && TotalChannels > 1)
	{
		FloatBufferSuffix = FString::FromInt(TotalChannels);
	}

	TMap<FString, FStringFormatArg> ArgsDeclarations = {
		{ TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
		{ TEXT("GridName"),			ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{ TEXT("SamplerName"),		ParamInfo.DataInterfaceHLSLSymbol + SamplerName},
		{ TEXT("OutputGridName"),	ParamInfo.DataInterfaceHLSLSymbol + OutputGridName},
		{ TEXT("FloatBufferSuffix"),FloatBufferSuffix},
		{ TEXT("NumChannels"),		AttributeWriter.UseRGBAGrid() ? TotalChannels : 1}, // float grids only have a single channel for the underlying buffer
		{ TEXT("NumTiles"),			ParamInfo.DataInterfaceHLSLSymbol + NumTilesName},
		{ TEXT("OneOverNumTiles"),	ParamInfo.DataInterfaceHLSLSymbol + OneOverNumTilesName},
		{ TEXT("UnitClampMin"),		ParamInfo.DataInterfaceHLSLSymbol + UnitClampMinName},
		{ TEXT("UnitClampMax"),		ParamInfo.DataInterfaceHLSLSymbol + UnitClampMaxName},
		{ TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{ TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{ TEXT("AttributeIndicesName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + AttributeIndicesBaseName},
		{ TEXT("PerAttributeDataName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + PerAttributeDataName},
		{ TEXT("AttributeInt4Count"),		AttributeInt4Count},
		{ TEXT("NumAttributesName"),		ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
		{ TEXT("NumNamedAttributesName"),	ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumNamedAttributesName},
	};

	OutHLSL += FString::Format(FormatDeclarations, ArgsDeclarations);

	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, ArgsDeclarations);
}
void UNiagaraDataInterfaceGrid3DCollection::WriteSetHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL)
{

	FString FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, float{NumChannelsVariableSuffix} In_Value)
			{
				int In_AttributeIndex = {AttributeIndicesName}[{AttributeIndexGroup}]{AttributeIndexComponent};

			    for (int i = 0; i < {NumChannels}; i++)
				{
				)");
	if (InNumChannels == 1)
	{
		FormatBounds += TEXT(R"(
					float Val = In_Value;
		)");
	}
	else
	{
		FormatBounds += TEXT(R"(
					float Val = In_Value[i];
		)");
	}

	FormatBounds += TEXT(R"(	
					int CurAttributeIndex = In_AttributeIndex + i;
					int3 TileOffset = {PerAttributeDataName}[CurAttributeIndex].xyz;
					{OutputGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)] = Val;
				}
			}
		)");
	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("FunctionName"),				FunctionInfo.InstanceName},
		{TEXT("OutputGrid"),				ParamInfo.DataInterfaceHLSLSymbol + OutputGridName},
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("PerAttributeDataName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + PerAttributeDataName},
		{TEXT("AttributeIndicesName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + AttributeIndicesBaseName},
		{TEXT("AttributeIndexGroup"),		FunctionInstanceIndex / 4},
		{TEXT("AttributeIndexComponent"),	VectorComponentNames[FunctionInstanceIndex % 4]},
		{TEXT("NumChannels"),				FString::FromInt(InNumChannels)},
		{TEXT("NumChannelsVariableSuffix"),	InNumChannels > 1 ? FString::FromInt(InNumChannels) : TEXT("")},
		{TEXT("NumTiles"),					ParamInfo.DataInterfaceHLSLSymbol + NumTilesName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("NumAttributesName"),			ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

// code generation for reading grid attributes at a specific index - only considering tiled grid attributes
void UNiagaraDataInterfaceGrid3DCollection::WriteGetAtIndexHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int32 InNumChannels, FString& OutHLSL)
{

	FString FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int AttributeIndex, out float{NumChannelsVariableSuffix} Out_Val)
			{				
			    for (int i = 0; i < {NumChannels}; i++)
				{
					int CurAttributeIndex = AttributeIndex + i;
					int3 TileOffset = {PerAttributeDataName}[CurAttributeIndex].xyz;
					float Val = {Grid}.Load(int4(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z, 0));
					)");
	if (InNumChannels == 1)
	{
		FormatBounds += TEXT("					Out_Val = Val;\n");
	}
	else
	{
		FormatBounds += TEXT("					Out_Val[i] = Val;\n");
	}
	FormatBounds += TEXT(R"(
				}
			}
		)");
	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("FunctionName"),				FunctionInfo.InstanceName},
		{TEXT("OutputGrid"),				ParamInfo.DataInterfaceHLSLSymbol + OutputGridName},
		{TEXT("Grid"),						ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("PerAttributeDataName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + PerAttributeDataName},
		{TEXT("NumChannels"),				FString::FromInt(InNumChannels)},
		{TEXT("NumChannelsVariableSuffix"),	InNumChannels > 1 ? FString::FromInt(InNumChannels) : TEXT("")},
		{TEXT("NumTiles"),					ParamInfo.DataInterfaceHLSLSymbol + NumTilesName},
		{TEXT("NumAttributesName"),			ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

// code generation for writing grid attributes at a specific index - only considering tiled grid attributes
void UNiagaraDataInterfaceGrid3DCollection::WriteSetAtIndexHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int32 InNumChannels, FString& OutHLSL)
{

	FString FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int AttributeIndex, float{NumChannelsVariableSuffix} In_Value)
			{				
			    for (int i = 0; i < {NumChannels}; i++)
				{
				)");
	if (InNumChannels == 1)
	{
		FormatBounds += TEXT(R"(
					float Val = In_Value;
		)");
	}
	else
	{
		FormatBounds += TEXT(R"(
					float Val = In_Value[i];
		)");
	}

	FormatBounds += TEXT(R"(	
					int CurAttributeIndex = AttributeIndex + i;
					int3 TileOffset = {PerAttributeDataName}[CurAttributeIndex].xyz;
					{OutputGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)] = Val;
				}
			}
		)");
	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("FunctionName"),				FunctionInfo.InstanceName},
		{TEXT("OutputGrid"),				ParamInfo.DataInterfaceHLSLSymbol + OutputGridName},
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("PerAttributeDataName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + PerAttributeDataName},
		{TEXT("NumChannels"),				FString::FromInt(InNumChannels)},
		{TEXT("NumChannelsVariableSuffix"),	InNumChannels > 1 ? FString::FromInt(InNumChannels) : TEXT("")},
		{TEXT("NumTiles"),					ParamInfo.DataInterfaceHLSLSymbol + NumTilesName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("NumAttributesName"),			ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

void UNiagaraDataInterfaceGrid3DCollection::WriteGetHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL)
{

	FString FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, out float{NumChannelsVariableSuffix} Out_Val)
			{
				int In_AttributeIndex = {AttributeIndicesName}[{AttributeIndexGroup}]{AttributeIndexComponent};

			    for (int i = 0; i < {NumChannels}; i++)
				{
					int CurAttributeIndex = In_AttributeIndex + i;
					int3 TileOffset = {PerAttributeDataName}[CurAttributeIndex].xyz;
					float Val = {Grid}.Load(int4(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z, 0));
					)");
	if (InNumChannels == 1)
	{
		FormatBounds += TEXT("					Out_Val = Val;\n");
	}
	else
	{
		FormatBounds += TEXT("					Out_Val[i] = Val;\n");
	}
	FormatBounds += TEXT(R"(
				}
			}
		)");
	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("FunctionName"),				FunctionInfo.InstanceName},
		{TEXT("OutputGrid"),				ParamInfo.DataInterfaceHLSLSymbol + OutputGridName},
		{TEXT("Grid"),						ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("PerAttributeDataName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + PerAttributeDataName},
		{TEXT("AttributeIndicesName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + AttributeIndicesBaseName},
		{TEXT("AttributeIndexGroup"),		FunctionInstanceIndex / 4},
		{TEXT("AttributeIndexComponent"),	VectorComponentNames[FunctionInstanceIndex % 4]},
		{TEXT("NumChannels"),				FString::FromInt(InNumChannels)},
		{TEXT("NumChannelsVariableSuffix"),	InNumChannels > 1 ? FString::FromInt(InNumChannels) : TEXT("")},
		{TEXT("NumTiles"),					ParamInfo.DataInterfaceHLSLSymbol + NumTilesName},
		{TEXT("NumAttributesName"),			ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

void UNiagaraDataInterfaceGrid3DCollection::WriteGetOutputGridHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL)
{
	// Like WriteGetHLSL but reads from OutputGrid UAV instead of Grid SRV
	FString FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, out float{NumChannelsVariableSuffix} Out_Val)
			{
				int In_AttributeIndex = {AttributeIndicesName}[{AttributeIndexGroup}]{AttributeIndexComponent};

			    for (int i = 0; i < {NumChannels}; i++)
				{
					int CurAttributeIndex = In_AttributeIndex + i;
					int3 TileOffset = {PerAttributeDataName}[CurAttributeIndex].xyz;
					float Val = {OutputGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)];
					)");
	if (InNumChannels == 1)
	{
		FormatBounds += TEXT("					Out_Val = Val;\n");
	}
	else
	{
		FormatBounds += TEXT("					Out_Val[i] = Val;\n");
	}
	FormatBounds += TEXT(R"(
				}
			}
		)");
	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("FunctionName"),				FunctionInfo.InstanceName},
		{TEXT("OutputGrid"),				ParamInfo.DataInterfaceHLSLSymbol + OutputGridName},
		{TEXT("Grid"),						ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("PerAttributeDataName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + PerAttributeDataName},
		{TEXT("AttributeIndicesName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + AttributeIndicesBaseName},
		{TEXT("AttributeIndexGroup"),		FunctionInstanceIndex / 4},
		{TEXT("AttributeIndexComponent"),	VectorComponentNames[FunctionInstanceIndex % 4]},
		{TEXT("NumChannels"),				FString::FromInt(InNumChannels)},
		{TEXT("NumChannelsVariableSuffix"),	InNumChannels > 1 ? FString::FromInt(InNumChannels) : TEXT("")},
		{TEXT("NumTiles"),					ParamInfo.DataInterfaceHLSLSymbol + NumTilesName},
		{TEXT("NumAttributesName"),			ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

void UNiagaraDataInterfaceGrid3DCollection::WriteSampleHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString SampleFunction, FString& OutHLSL)
{
	SampleFunction.ReplaceInline(TEXT("{Grid}"), *(ParamInfo.DataInterfaceHLSLSymbol + GridName));
	SampleFunction.ReplaceInline(TEXT("{ParameterName}"), *ParamInfo.DataInterfaceHLSLSymbol);

	FString FormatBounds = TEXT(R"(
			void {FunctionName}(float3 In_Unit, out float{NumChannelsVariableSuffix} Out_Val)
			{
				int In_AttributeIndex = {AttributeIndicesName}[{AttributeIndexGroup}]{AttributeIndexComponent};

				float3 TileUV = clamp(In_Unit, {UnitClampMin}, {UnitClampMax}) * {OneOverNumTiles};
			    for (int i = 0; i < {NumChannels}; i++)
				{
					int CurAttributeIndex = In_AttributeIndex + i;
					float3 UVW = {PerAttributeDataName}[CurAttributeIndex + {NumAttributesName}].xyz + TileUV;

					float Val = {SampleFunction}({SamplerName}, UVW, 0);
					)");
	if (InNumChannels == 1)
	{
		FormatBounds += TEXT("					Out_Val = Val;\n");
	}
	else
	{
		FormatBounds += TEXT("					Out_Val[i] = Val;\n");
	}

	FormatBounds += TEXT(R"(
				}
			}
		)");

	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("FunctionName"),				FunctionInfo.InstanceName},
		{TEXT("Grid"),						ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{TEXT("SamplerName"),				ParamInfo.DataInterfaceHLSLSymbol + SamplerName },
		{TEXT("SampleFunction"),			SampleFunction },
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("NumChannels"),				FString::FromInt(InNumChannels)},
		{TEXT("NumChannelsVariableSuffix"),	InNumChannels > 1 ? FString::FromInt(InNumChannels) : TEXT("")},
		{TEXT("AttributeIndicesName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + AttributeIndicesBaseName},
		{TEXT("PerAttributeDataName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + PerAttributeDataName},
		{TEXT("AttributeIndexGroup"),		FunctionInstanceIndex / 4},
		{TEXT("AttributeIndexComponent"),	VectorComponentNames[FunctionInstanceIndex % 4]},
		{TEXT("NumTiles"),					ParamInfo.DataInterfaceHLSLSymbol + NumTilesName},
		{TEXT("OneOverNumTiles"),			ParamInfo.DataInterfaceHLSLSymbol + OneOverNumTilesName},
		{TEXT("UnitClampMin"),				ParamInfo.DataInterfaceHLSLSymbol + UnitClampMinName},
		{TEXT("UnitClampMax"),				ParamInfo.DataInterfaceHLSLSymbol + UnitClampMaxName},
		{TEXT("NumAttributesName"),			ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

// code generation for sampling grid attributes at a specific index - only considering tiled grid attributes
void UNiagaraDataInterfaceGrid3DCollection::WriteSampleAtIndexHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int32 InNumChannels, FString SampleFunction, FString& OutHLSL)
{
	SampleFunction.ReplaceInline(TEXT("{Grid}"), *(ParamInfo.DataInterfaceHLSLSymbol + GridName));
	SampleFunction.ReplaceInline(TEXT("{ParameterName}"), *ParamInfo.DataInterfaceHLSLSymbol);

	FString FormatBounds = TEXT(R"(
			void {FunctionName}(float3 In_Unit, int AttributeIndex, out float{NumChannelsVariableSuffix} Out_Val)
			{			
				float3 TileUV = clamp(In_Unit, {UnitClampMin}, {UnitClampMax}) * {OneOverNumTiles};
			    for (int i = 0; i < {NumChannels}; i++)
				{
					int CurAttributeIndex = AttributeIndex + i;
					float3 UVW = {PerAttributeDataName}[CurAttributeIndex + {NumAttributesName}].xyz + TileUV;

					float Val = {SampleFunction}({SamplerName}, UVW, 0);
					)");
	if (InNumChannels == 1)
	{
		FormatBounds += TEXT("					Out_Val = Val;\n");
	}
	else
	{
		FormatBounds += TEXT("					Out_Val[i] = Val;\n");
	}

	FormatBounds += TEXT(R"(
				}
			}
		)");

	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("FunctionName"),				FunctionInfo.InstanceName},
		{TEXT("Grid"),						ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{TEXT("SamplerName"),				ParamInfo.DataInterfaceHLSLSymbol + SamplerName },
		{TEXT("SampleFunction"),			SampleFunction },
		{TEXT("NumCellsName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("NumChannels"),				FString::FromInt(InNumChannels)},
		{TEXT("NumChannelsVariableSuffix"),	InNumChannels > 1 ? FString::FromInt(InNumChannels) : TEXT("")},
		{TEXT("PerAttributeDataName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + PerAttributeDataName},
		{TEXT("NumTiles"),					ParamInfo.DataInterfaceHLSLSymbol + NumTilesName},
		{TEXT("OneOverNumTiles"),			ParamInfo.DataInterfaceHLSLSymbol + OneOverNumTilesName},
		{TEXT("UnitClampMin"),				ParamInfo.DataInterfaceHLSLSymbol + UnitClampMinName},
		{TEXT("UnitClampMax"),				ParamInfo.DataInterfaceHLSLSymbol + UnitClampMaxName},
		{TEXT("NumAttributesName"),			ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

void UNiagaraDataInterfaceGrid3DCollection::WriteAttributeGetIndexHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, int32 InNumChannels, FString& OutHLSL)
{
	FString FormatBounds = TEXT(R"(
			void {FunctionName}(out int Out_Val)
			{
				int In_AttributeIndex = {AttributeIndicesName}[{AttributeIndexGroup}]{AttributeIndexComponent};
				Out_Val = In_AttributeIndex;
			}
	)");


	TMap<FString, FStringFormatArg> ArgsBounds = {
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("FunctionName"), FunctionInfo.InstanceName},
		{TEXT("AttributeIndicesName"), ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + AttributeIndicesBaseName},
		{TEXT("AttributeIndexGroup"), FunctionInstanceIndex / 4},
		{TEXT("AttributeIndexComponent"), VectorComponentNames[FunctionInstanceIndex % 4]},
	};
	OutHLSL += FString::Format(*FormatBounds, ArgsBounds);
}

const TCHAR* UNiagaraDataInterfaceGrid3DCollection::TypeDefinitionToHLSLTypeString(const FNiagaraTypeDefinition& InDef) const
{
	if (InDef == FNiagaraTypeDefinition::GetFloatDef())
		return TEXT("float");
	if (InDef == FNiagaraTypeDefinition::GetVec2Def())
		return TEXT("float2");
	if (InDef == FNiagaraTypeDefinition::GetVec3Def())
		return TEXT("float3");
	if (InDef == FNiagaraTypeDefinition::GetVec4Def() || InDef == FNiagaraTypeDefinition::GetColorDef())
		return TEXT("float4");
	return nullptr;
}
FName UNiagaraDataInterfaceGrid3DCollection::TypeDefinitionToGetFunctionName(const FNiagaraTypeDefinition& InDef) const
{
	if (InDef == FNiagaraTypeDefinition::GetFloatDef())
		return GetPreviousFloatValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec2Def())
		return GetPreviousVector2DValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec3Def())
		return GetPreviousVectorValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec4Def() || InDef == FNiagaraTypeDefinition::GetColorDef())
		return GetPreviousVector4ValueFunctionName;
	return NAME_None;;
}
FName UNiagaraDataInterfaceGrid3DCollection::TypeDefinitionToSetFunctionName(const FNiagaraTypeDefinition& InDef) const
{
	if (InDef == FNiagaraTypeDefinition::GetFloatDef())
		return SetFloatValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec2Def())
		return SetVector2DValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec3Def())
		return SetVectorValueFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec4Def() || InDef == FNiagaraTypeDefinition::GetColorDef())
		return SetVector4ValueFunctionName;
	return NAME_None;;
}

FName UNiagaraDataInterfaceGrid3DCollection::TypeDefinitionToAttributeIndexFunctionName(const FNiagaraTypeDefinition& InDef) const
{
	if (InDef == FNiagaraTypeDefinition::GetFloatDef())
		return GetFloatAttributeIndexFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec2Def())
		return GetVector2DAttributeIndexFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec3Def())
		return GetVectorAttributeIndexFunctionName;
	if (InDef == FNiagaraTypeDefinition::GetVec4Def() || InDef == FNiagaraTypeDefinition::GetColorDef())
		return GetVector4AttributeIndexFunctionName;
	return NAME_None;;
}

bool UNiagaraDataInterfaceGrid3DCollection::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	bool ParentRet = Super::GetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, OutHLSL);
	if (ParentRet)
	{
		return true;
	}

	FGrid3DCollectionAttributeHlslWriter AttributeWriter(ParamInfo);
	FString GridChannelString = AttributeWriter.GetAttributeHelper().GetGridChannelString();

	TMap<FString, FStringFormatArg> ArgsBounds =
	{
		{TEXT("ParameterName"),			ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("FunctionName"),			FunctionInfo.InstanceName},
		{TEXT("Grid"),					ParamInfo.DataInterfaceHLSLSymbol + GridName},
		{TEXT("OutputGrid"),			ParamInfo.DataInterfaceHLSLSymbol + OutputGridName},
		{TEXT("NumAttributes"),			ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
		{TEXT("NumNamedAttributes"),	ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumNamedAttributesName},
		{TEXT("NumCells"),				ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),			ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("SamplerName"),			ParamInfo.DataInterfaceHLSLSymbol + SamplerName},
		{TEXT("NumTiles"),				ParamInfo.DataInterfaceHLSLSymbol + NumTilesName},
		{TEXT("OneOverNumTiles"),		ParamInfo.DataInterfaceHLSLSymbol + OneOverNumTilesName},
		{TEXT("UnitClampMin"),			ParamInfo.DataInterfaceHLSLSymbol + UnitClampMinName},
		{TEXT("UnitClampMax"),			ParamInfo.DataInterfaceHLSLSymbol + UnitClampMaxName},
		{TEXT("NumCellsName"),			ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("PerAttributeDataName"),	ParamInfo.DataInterfaceHLSLSymbol + TEXT("_") + PerAttributeDataName},
		{TEXT("NumAttributesName"),		ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
		{TEXT("GridChannels"),			GridChannelString},
	};

	if (FunctionInfo.DefinitionName == GetValueFunctionName || FunctionInfo.DefinitionName == GetPreviousValueAtIndexFunctionName)
	{
		// if we have an rgba grid, then we must query for the rgba value at the current point, and get the specific channel requested
		if (AttributeWriter.UseRGBAGrid())
		{			
			FString RGBAHlsl;
			RGBAHlsl.Appendf(TEXT("void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, out float Out_Val)\n"));
			RGBAHlsl.Appendf(TEXT("{\n"));
			RGBAHlsl.Appendf(TEXT("    Out_Val = 0;\n"));
			RGBAHlsl.Appendf(TEXT("    const float{GridChannels} TmpVal = {Grid}.Load(int4(In_IndexX, In_IndexY, In_IndexZ, 0));\n"));
			RGBAHlsl.Appendf(TEXT("    if (In_AttributeIndex == 0)\n"));
			RGBAHlsl.Appendf(TEXT("    {\n"));
			RGBAHlsl.Appendf(TEXT("        Out_Val = TmpVal.r;\n"));
			RGBAHlsl.Appendf(TEXT("    }\n"));

			for (int i = 0; i < AttributeWriter.GetAttributeHelper().GetTotalChannels() - 1; ++i)
			{
				RGBAHlsl.Appendf(TEXT("    else if (In_AttributeIndex == %d)\n"), i + 1);
				RGBAHlsl.Appendf(TEXT("    {\n"));
				RGBAHlsl.Appendf(TEXT("        Out_Val = TmpVal.%s;\n"), *FGrid3DCollectionAttributeHlslWriter::Channels[i + 1]);
				RGBAHlsl.Appendf(TEXT("    }\n"));
			}

			RGBAHlsl.Appendf(TEXT("}\n"));
			OutHLSL += FString::Format(*RGBAHlsl, ArgsBounds);
		}
		else
		{
			static const TCHAR* FormatBounds = TEXT(R"(
				void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, out float Out_Val)
				{
					Out_Val = 0;
					if ( In_AttributeIndex < {NumAttributesName} )
					{
						int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;
						Out_Val = {Grid}.Load(int4(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z, 0));
					}
				}
			)");
			OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetFullGridPreviousValueFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, out float Out_Val)
			{
				Out_Val = {Grid}.Load(int4(In_IndexX, In_IndexY, In_IndexZ, 0));
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	} 
	else if (FunctionInfo.DefinitionName == SetValueFunctionName)
	{

		// if we are using an rgba grid, then we must query the float4 value from the uav, then set the channel we are requesting
		// note that the dynamic branching seems necessary otherwise it didn't work as expected.  Might be due to code optimization
		if (AttributeWriter.UseRGBAGrid())
		{			
			FString RGBAHlsl;
			RGBAHlsl.Appendf(TEXT("void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, float In_Value, out int val)\n"));
			RGBAHlsl.Appendf(TEXT("{\n"));
			RGBAHlsl.Appendf(TEXT("    val = 0;\n"));
			RGBAHlsl.Appendf(TEXT("    [branch] if (In_AttributeIndex == 0)\n"));
			RGBAHlsl.Appendf(TEXT("    {\n"));
			RGBAHlsl.Appendf(TEXT("		   float{GridChannels} TmpVal = {OutputGrid}[int3(In_IndexX, In_IndexY, In_IndexZ)];\n"));
			RGBAHlsl.Appendf(TEXT("        TmpVal.r = In_Value;\n"));
			RGBAHlsl.Appendf(TEXT("        {OutputGrid}[int3(In_IndexX, In_IndexY, In_IndexZ)] = TmpVal;\n"));
			RGBAHlsl.Appendf(TEXT("    }\n"));

			for (int i = 0; i < AttributeWriter.GetAttributeHelper().GetTotalChannels() - 1; ++i)
			{
				RGBAHlsl.Appendf(TEXT("    else if (In_AttributeIndex == %d)\n"), i + 1);
				RGBAHlsl.Appendf(TEXT("    {\n"));
				RGBAHlsl.Appendf(TEXT("		   float{GridChannels} TmpVal = {OutputGrid}[int3(In_IndexX, In_IndexY, In_IndexZ)];\n"));
				RGBAHlsl.Appendf(TEXT("        TmpVal.%s = In_Value;\n"), *FGrid3DCollectionAttributeHlslWriter::Channels[i + 1]);
				RGBAHlsl.Appendf(TEXT("        {OutputGrid}[int3(In_IndexX, In_IndexY, In_IndexZ)] = TmpVal;\n"));
				RGBAHlsl.Appendf(TEXT("    }\n"));
			}

			RGBAHlsl.Appendf(TEXT("}\n"));
			OutHLSL += FString::Format(*RGBAHlsl, ArgsBounds);
		}
		else
		{
			static const TCHAR* FormatBounds = TEXT(R"(
				void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, float In_Value, out int val)
				{			
					val = 0;
					if ( In_AttributeIndex < {NumAttributesName} )
					{
						int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;
						{OutputGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)] = In_Value;
					}
				}
			)");
			OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		}

		return true;
	}
	else if (FunctionInfo.DefinitionName == SetFullGridValueFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, float In_Value, out int val)
			{			
				val = 0;
				{OutputGrid}[int3(In_IndexX, In_IndexY, In_IndexZ)] = In_Value;
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == CopyPreviousToCurrentForCellFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			static const TCHAR* FormatBounds = TEXT(R"(
				void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ)
				{					
					const int3 Index = int3(In_IndexX, In_IndexY, In_IndexZ);
					{OutputGrid}[Index] = {Grid}.Load(Index, 0));
				}
			)");
			OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		}
		else
		{
			static const TCHAR* FormatBounds = TEXT(R"(
				void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ)
				{
					for (int AttributeIndex = 0; AttributeIndex < {NumNamedAttributes}.x; AttributeIndex++)
					{		
						int3 TileOffset = {PerAttributeDataName}[AttributeIndex].xyz;

						float Val = {Grid}.Load(int4(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z, 0));	
						{OutputGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)] = Val;
					}
				}
			)");
			OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == CopyMaskedPreviousToCurrentForCellFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{			
			static const TCHAR* FormatBounds = TEXT(R"(
				void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int NumAttributesSet, int AttributeMask)
				{		
					const int3 Index = int3(In_IndexX, In_IndexY, In_IndexZ);									
					{OutputGrid}[Index] = {Grid}.Load(int4(Index, 0));					
				}
			)");
			OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		}
		else
		{
			static const TCHAR* FormatBounds = TEXT(R"(
				void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int NumAttributesSet, int AttributeMask)
				{	
					// early out if we set all the attributes that exist on the DI
					[branch]
					if (NumAttributesSet == {NumAttributes}.x)
					{
						return;					
					}

					for (int AttributeIndex = 0; AttributeIndex < {NumAttributes}.x; AttributeIndex++)
					{					
						// check the attribute index in the attribute mask to see if it has been set
						// we automatically pass through attributes higher than 31 since we can only
						// store 32 attributes in the mask
						[branch]
						if ((AttributeMask & (1l << AttributeIndex)) == 0 || AttributeIndex >= 32)
						{						
							int3 TileOffset = {PerAttributeDataName}[AttributeIndex].xyz;

							float Val = {Grid}.Load(int4(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z, 0));	
							{OutputGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)] = Val;
						}
					}

		// alternative implementation - look
		/*
					int AttributeIndex = -1;
					[loop]			
					for (int x = 0; x < {NumAttributes}.x - NumAttributesSet; ++x)
					{											
						do
						{
							AttributeIndex++;																		
						} while(AttributeIndex < 32 && (AttributeMask & (1 << AttributeIndex)) == 1);
					
						int3 TileOffset = {PerAttributeDataName}[AttributeIndex].xyz;
						float Val = {Grid}.Load(int4(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z, 0));	
						{OutputGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)] = Val;
					}
	*/
				}
			)");
			OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == ClearCellFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ)
			{
				for (int AttributeIndex = 0; AttributeIndex < {NumAttributes}.x; AttributeIndex++)
				{
					float Val = 0.0f;

					int3 TileOffset = {PerAttributeDataName}[AttributeIndex].xyz;

					{OutputGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)] = In_Value;
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetVector4ValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSetHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteSetHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 4, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetPreviousVector4ValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteGetHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 4, OutHLSL);
		}

		return true;
	}
	else if (FunctionInfo.DefinitionName == SamplePreviousGridVector4FunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, false, OutHLSL);
		}
		else
		{
			WriteSampleHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 4, "{Grid}.SampleLevel", OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridVector4FunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, true, OutHLSL);
		}
		else
		{
			WriteSampleHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 4, "SampleTriCubicLagrange_{ParameterName}", OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetVector3ValueFunctionName || FunctionInfo.DefinitionName == SetVectorValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSetHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteSetHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 3, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetPreviousVectorValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteGetHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 3, OutHLSL);
		}

		return true;
	}
	else if (FunctionInfo.DefinitionName == SamplePreviousGridVectorFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, false, OutHLSL);
		}
		else
		{
			WriteSampleHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 3, "{Grid}.SampleLevel", OutHLSL);
		}

		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridVectorFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, true, OutHLSL);
		}
		else
		{
			WriteSampleHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 3, "SampleTriCubicLagrange_{ParameterName}", OutHLSL);
		}
	}
	else if (FunctionInfo.DefinitionName == SetVector2ValueFunctionName || FunctionInfo.DefinitionName == SetVector2DValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSetHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteSetHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 2, OutHLSL);
		}

		return true;
	}
	else if (FunctionInfo.DefinitionName == GetPreviousVector2DValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteGetHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 2, OutHLSL);
		}

		return true;
	}
	else if (FunctionInfo.DefinitionName == SamplePreviousGridVector2DFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, false, OutHLSL);
		}
		else
		{
			WriteSampleHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 2, "{Grid}.SampleLevel", OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridVector2DFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, true, OutHLSL);
		}
		else
		{
			WriteSampleHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 2, "SampleTriCubicLagrange_{ParameterName}", OutHLSL);
		}
	}
	else if (FunctionInfo.DefinitionName == SetFloatValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSetHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteSetHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 1, OutHLSL);
		}

		return true;
	}
	else if (FunctionInfo.DefinitionName == GetPreviousFloatValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteGetHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 1, OutHLSL);
		}

		return true;
	}
	else if (FunctionInfo.DefinitionName == GetOutputGridVector4ValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetOutputGridHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteGetOutputGridHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 4, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetOutputGridVectorValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetOutputGridHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteGetOutputGridHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 3, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetOutputGridVector2ValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetOutputGridHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteGetOutputGridHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 2, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetOutputGridFloatValueFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetOutputGridHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteGetOutputGridHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 1, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SamplePreviousGridFloatFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, false, OutHLSL);
		}
		else
		{
			WriteSampleHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 1, "{Grid}.SampleLevel", OutHLSL);
		}

		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridFloatFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, true, OutHLSL);
		}
		else
		{
			WriteSampleHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 1, "SampleTriCubicLagrange_{ParameterName}", OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVector4AttributeIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteAttributeGetIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteAttributeGetIndexHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 4, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVectorAttributeIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteAttributeGetIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteAttributeGetIndexHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 3, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVector2DAttributeIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteAttributeGetIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteAttributeGetIndexHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 2, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetFloatAttributeIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteAttributeGetIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, OutHLSL);
		}
		else
		{
			WriteAttributeGetIndexHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, 1, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SampleGridFunctionName || FunctionInfo.DefinitionName == SamplePreviousGridAtIndexFunctionName)
	{
		
	// if we have an rgba grid, then we must sample for the rgba value at the current point, and get the specific channel requested
		if (AttributeWriter.UseRGBAGrid())
		{			
			FString RGBAHlsl;
			RGBAHlsl.Appendf(TEXT("void {FunctionName}(float In_UnitX, float In_UnitY, float In_UnitZ, int In_AttributeIndex, out float Out_Val)\n"));
			RGBAHlsl.Appendf(TEXT("{\n"));
			RGBAHlsl.Appendf(TEXT("    Out_Val = 0;\n"));
			RGBAHlsl.Appendf(TEXT("    const float{GridChannels} TmpVal = {Grid}.SampleLevel({SamplerName}, float3(In_UnitX, In_UnitY, In_UnitZ), 0);\n"));

			RGBAHlsl.Appendf(TEXT("    [branch] if (In_AttributeIndex == 0)\n"));
			RGBAHlsl.Appendf(TEXT("    {\n"));
			RGBAHlsl.Appendf(TEXT("        Out_Val = TmpVal.r;\n"));
			RGBAHlsl.Appendf(TEXT("    }\n"));

			for (int i = 0; i < AttributeWriter.GetAttributeHelper().GetTotalChannels() - 1; ++i)
			{
				RGBAHlsl.Appendf(TEXT("    else if (In_AttributeIndex == %d)\n"), i + 1);
				RGBAHlsl.Appendf(TEXT("    {\n"));
				RGBAHlsl.Appendf(TEXT("        Out_Val = TmpVal.%s;\n"), *FGrid3DCollectionAttributeHlslWriter::Channels[i + 1]);
				RGBAHlsl.Appendf(TEXT("    }\n"));
			}

			RGBAHlsl.Appendf(TEXT("}\n"));
			OutHLSL += FString::Format(*RGBAHlsl, ArgsBounds);
		}
		else
		{
			static const TCHAR* FormatBounds = TEXT(R"(
					void {FunctionName}(float In_UnitX, float In_UnitY, float In_UnitZ, int In_AttributeIndex, out float Out_Val)
					{
						Out_Val = 0;
						if ( In_AttributeIndex < {NumAttributesName} )
						{
							float3 Unit = float3(In_UnitX, In_UnitY, In_UnitZ);
							Unit = clamp(Unit, {UnitClampMin}, {UnitClampMax}) * {OneOverNumTiles};

							float3 UVW = {PerAttributeDataName}[In_AttributeIndex + {NumAttributesName}].xyz + Unit;
							Out_Val = {Grid}.SampleLevel({SamplerName}, UVW, 0);
						}
					}
				)");
			OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		}

		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSampleGridFunctionName || FunctionInfo.DefinitionName == CubicSamplePreviousGridAtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			//#todo(dmp): maybe fix this at some point or deprecate?
			UE_LOGF(LogNiagara, Error, "RGBA grids are not supported for this function");
			return false;
		}
		else
		{
			static const TCHAR* FormatBounds = TEXT(R"(
					void {FunctionName}(float In_UnitX, float In_UnitY, float In_UnitZ, int In_AttributeIndex, out float Out_Val)
					{
						Out_Val = 0;
						if ( In_AttributeIndex < {NumAttributesName} )
						{
							float3 Unit = float3(In_UnitX, In_UnitY, In_UnitZ);
							Unit = clamp(Unit, {UnitClampMin}, {UnitClampMax}) * {OneOverNumTiles};

							float3 UVW = {PerAttributeDataName}[In_AttributeIndex + {NumAttributesName}].xyz + Unit;
							Out_Val = SampleTriCubicLagrange_{ParameterName}({SamplerName}, UVW, 0);
						}
					}
				)");
			OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SamplePreviousFullGridFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			//#todo(dmp): maybe fix this at some point or deprecate?
			UE_LOGF(LogNiagara, Error, "RGBA grids are not supported for this function");
			return false;
		}
		else
		{
			static const TCHAR* FormatBounds = TEXT(R"(
						void {FunctionName}(float3 In_Unit, out float Out_Val)
						{
							Out_Val = {Grid}.SampleLevel({SamplerName}, In_Unit, 0);
						}
					)");
			OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		}

		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousFullGridFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			//#todo(dmp): maybe fix this at some point or deprecate?
			UE_LOGF(LogNiagara, Error, "RGBA grids are not supported for this function");
			return false;
		}
		else
		{
			static const TCHAR* FormatBounds = TEXT(R"(
						void {FunctionName}(float3 In_Unit, out float Out_Val)
						{
							Out_Val = SampleTriCubicLagrange_{ParameterName}({SamplerName}, In_Unit, 0);
						}
					)");
			OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetVector4ValueAtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSetAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 4, OutHLSL);
		}
		else
		{
			WriteSetAtIndexHLSL(ParamInfo, FunctionInfo, 4, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetVector3ValueAtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSetAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 3, OutHLSL);
		}
		else
		{
			WriteSetAtIndexHLSL(ParamInfo, FunctionInfo, 3, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetVector2ValueAtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSetAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 2, OutHLSL);
		}
		else
		{
			WriteSetAtIndexHLSL(ParamInfo, FunctionInfo, 2, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetFloatValueAtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSetAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 1, OutHLSL);
		}
		else
		{
			WriteSetAtIndexHLSL(ParamInfo, FunctionInfo, 1, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetPreviousVector4ValueAtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 4, OutHLSL);
		}
		else
		{
			WriteGetAtIndexHLSL(ParamInfo, FunctionInfo, 4, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetPreviousVector3ValueAtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 3, OutHLSL);
		}
		else
		{
			WriteGetAtIndexHLSL(ParamInfo, FunctionInfo, 3, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetPreviousVector2ValueAtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 2, OutHLSL);
		}
		else
		{
			WriteGetAtIndexHLSL(ParamInfo, FunctionInfo, 2, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetPreviousFloatValueAtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteGetAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 1, OutHLSL);
		}
		else
		{
			WriteGetAtIndexHLSL(ParamInfo, FunctionInfo, 1, OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SamplePreviousGridVector4AtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 4, false, OutHLSL);
		}
		else
		{
			WriteSampleAtIndexHLSL(ParamInfo, FunctionInfo, 4, "{Grid}.SampleLevel", OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SamplePreviousGridVector3AtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 3, false, OutHLSL);
		}
		else
		{
			WriteSampleAtIndexHLSL(ParamInfo, FunctionInfo, 3, "{Grid}.SampleLevel", OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SamplePreviousGridVector2AtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 2, false, OutHLSL);
		}
		else
		{
			WriteSampleAtIndexHLSL(ParamInfo, FunctionInfo, 2, "{Grid}.SampleLevel", OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == SamplePreviousGridFloatAtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 1, false, OutHLSL);
		}
		else
		{
			WriteSampleAtIndexHLSL(ParamInfo, FunctionInfo, 1, "{Grid}.SampleLevel", OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridVector4AtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 4, true, OutHLSL);
		}
		else
		{
			WriteSampleAtIndexHLSL(ParamInfo, FunctionInfo, 4, "SampleTriCubicLagrange_{ParameterName}", OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridVector3AtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 3, true, OutHLSL);
		}
		else
		{
			WriteSampleAtIndexHLSL(ParamInfo, FunctionInfo, 3, "SampleTriCubicLagrange_{ParameterName}", OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridVector2AtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 2, true, OutHLSL);
		}
		else
		{
			WriteSampleAtIndexHLSL(ParamInfo, FunctionInfo, 2, "SampleTriCubicLagrange_{ParameterName}", OutHLSL);
		}
		return true;
	}
	else if (FunctionInfo.DefinitionName == CubicSamplePreviousGridFloatAtIndexFunctionName)
	{
		if (AttributeWriter.UseRGBAGrid())
		{
			AttributeWriter.WriteSampleAtIndexHLSL(EGridAttributeRetrievalMode::RGBAGrid, FunctionInfo, 1, true, OutHLSL);
		}
		else
		{
			WriteSampleAtIndexHLSL(ParamInfo, FunctionInfo, 1, "SampleTriCubicLagrange_{ParameterName}", OutHLSL);
		}
		return true;
	}

	return false;
}
#endif

void UNiagaraDataInterfaceGrid3DCollection::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FNDIGrid3DShaderParameters>();

	const int32 IndirectionTableSize = FMath::Max(FMath::DivideAndRoundUp(ShaderParametersBuilder.GetGeneratedFunctions().Num(), 4), 1);
	ShaderParametersBuilder.AddLooseParamArray<FIntVector4>(*UNiagaraDataInterfaceGrid3DCollection::AttributeIndicesBaseName, IndirectionTableSize);
}

void UNiagaraDataInterfaceGrid3DCollection::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyGrid3DCollectionProxy& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyGrid3DCollectionProxy>();
	FGrid3DCollectionRWInstanceData_RenderThread* ProxyData = DIProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	FGrid3DCollectionRWInstanceData_RenderThread* OriginalProxyData = ProxyData;
	check(ProxyData);

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	if (ProxyData->OtherProxy != nullptr)
	{
		FNiagaraDataInterfaceProxyGrid3DCollectionProxy* OtherGrid3DProxy = static_cast<FNiagaraDataInterfaceProxyGrid3DCollectionProxy*>(ProxyData->OtherProxy);
		ProxyData = OtherGrid3DProxy->SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
		check(ProxyData);
	}

	const FNiagaraDataInterfaceParametersCS_Grid3DCollection& ShaderStorage = Context.GetShaderStorage<FNiagaraDataInterfaceParametersCS_Grid3DCollection>();
	if (OriginalProxyData->AttributeIndices.Num() == 0 && ShaderStorage.AttributeNames.Num() > 0)
	{
		OriginalProxyData->AttributeIndices.SetNumZeroed(Align(ShaderStorage.AttributeNames.Num(), 4));

		// TODO handle mismatched types!
		for (int32 i = 0; i < ShaderStorage.AttributeNames.Num(); i++)
		{
			const int32 FoundIdx = ProxyData->Vars.Find(ShaderStorage.AttributeNames[i]);
			check(ShaderStorage.AttributeNames.Num() == ShaderStorage.AttributeChannelCount.Num());
			check(ProxyData->Offsets.Num() == ProxyData->VarComponents.Num());
			check(ProxyData->Offsets.Num() == ProxyData->Vars.Num());
			if (ProxyData->Offsets.IsValidIndex(FoundIdx) && ShaderStorage.AttributeChannelCount[i] == ProxyData->VarComponents[FoundIdx])
			{
				OriginalProxyData->AttributeIndices[i] = ProxyData->Offsets[FoundIdx];
			}
			else
			{
				OriginalProxyData->AttributeIndices[i] = -1; // We may need to protect against this in the hlsl as this might underflow an array lookup if used incorrectly.
			}
		}
	}


	// Generate per-attribute data
	if (ProxyData->PerAttributeData.NumBytes == 0)
	{
		TResourceArray<FVector4f> PerAttributeData;
		PerAttributeData.AddUninitialized((ProxyData->TotalNumAttributes * 2) + 1);
		for (int32 iAttribute = 0; iAttribute < ProxyData->TotalNumAttributes; ++iAttribute)
		{
			const FIntVector AttributeTileIndex(iAttribute % ProxyData->NumTiles.X, (iAttribute / ProxyData->NumTiles.X) % ProxyData->NumTiles.Y, iAttribute / (ProxyData->NumTiles.X * ProxyData->NumTiles.Y));
			PerAttributeData[iAttribute] = FVector4f(
				float(AttributeTileIndex.X * ProxyData->NumCells.X),
				float(AttributeTileIndex.Y * ProxyData->NumCells.Y),
				float(AttributeTileIndex.Z * ProxyData->NumCells.Z),
				0
			);
			PerAttributeData[iAttribute + ProxyData->TotalNumAttributes] = FVector4f(
				(1.0f / float(ProxyData->NumTiles.X)) * float(AttributeTileIndex.X),
				(1.0f / float(ProxyData->NumTiles.Y)) * float(AttributeTileIndex.Y),
				(1.0f / float(ProxyData->NumTiles.Z)) * float(AttributeTileIndex.Z),
				0.0f
			);
		}
		PerAttributeData[ProxyData->TotalNumAttributes * 2] = FVector4f(65535, 65535, 65535, 65535);
		ProxyData->PerAttributeData.Initialize(GraphBuilder.RHICmdList, TEXT("Grid3D::PerAttributeData"), sizeof(FVector4f), PerAttributeData.Num(), EPixelFormat::PF_A32B32G32R32F, BUF_Static, &PerAttributeData);
	}

	// Set parameters
	const FVector3f HalfPixelOffset = FVector3f(0.5f / ProxyData->NumCells.X, 0.5f / ProxyData->NumCells.Y, 0.5f / ProxyData->NumCells.Z);

	FNDIGrid3DShaderParameters* Parameters = Context.GetParameterNestedStruct<FNDIGrid3DShaderParameters>();
	Parameters->NumAttributes = ProxyData->TotalNumAttributes;
	Parameters->NumNamedAttributes = ProxyData->TotalNumNamedAttributes;
	Parameters->UnitToUV = FVector3f(1.0f) / FVector3f(ProxyData->NumCells);
	Parameters->NumCells = ProxyData->NumCells;
	Parameters->CellSize = FVector3f(ProxyData->CellSize);
	Parameters->NumTiles = ProxyData->NumTiles;
	Parameters->OneOverNumTiles = FVector3f(1.0f) / FVector3f(ProxyData->NumTiles);
	Parameters->UnitClampMin = HalfPixelOffset;
	Parameters->UnitClampMax = FVector3f::OneVector - HalfPixelOffset;
	Parameters->WorldBBoxSize = FVector3f(ProxyData->WorldBBoxSize);

	if (Context.IsResourceBound(&Parameters->Grid))
	{
		if (ProxyData->CurrentData)
		{
			Parameters->Grid = ProxyData->CurrentData->GetOrCreateSRV(GraphBuilder);
		}
		else
		{
			Parameters->Grid = Context.GetComputeDispatchInterface().GetBlackTextureSRV(GraphBuilder, ETextureDimension::Texture3D);
		}
	}
	Parameters->GridSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	if (Context.IsResourceBound(&Parameters->OutputGrid))
	{
		if (Context.IsOutputStage() && ProxyData->DestinationData)
		{
			Parameters->OutputGrid = ProxyData->DestinationData->GetOrCreateUAV(GraphBuilder);
		}
		else
		{
			Parameters->OutputGrid = Context.GetComputeDispatchInterface().GetEmptyTextureUAV(GraphBuilder, PF_R32_FLOAT, ETextureDimension::Texture3D);
		}
	}
	Parameters->PerAttributeData = ProxyData->PerAttributeData.SRV;

	if (OriginalProxyData->AttributeIndices.Num() > 0)
	{
		const int NumAttributesVector4s = FMath::Max(FMath::DivideAndRoundUp(ShaderStorage.AttributeNames.Num(), 4), 1);
		TArrayView<FIntVector4> AttributeIndices = Context.GetParameterLooseArray<FIntVector4>(NumAttributesVector4s);
		check(OriginalProxyData->AttributeIndices.Num() * OriginalProxyData->AttributeIndices.GetTypeSize() == AttributeIndices.Num() * AttributeIndices.GetTypeSize());
		FMemory::Memcpy(AttributeIndices.GetData(), OriginalProxyData->AttributeIndices.GetData(), AttributeIndices.Num() * AttributeIndices.GetTypeSize());
	}
}

FNiagaraDataInterfaceParametersCS* UNiagaraDataInterfaceGrid3DCollection::CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const
{
	FNiagaraDataInterfaceParametersCS_Grid3DCollection* ShaderStorage = new FNiagaraDataInterfaceParametersCS_Grid3DCollection();

	// Gather up all the attribute names referenced. Note that there may be multiple in the list of the same name,
	// but we only deal with this by the number of bound methods.
	const int32 NumFuncs = ParameterInfo.GeneratedFunctions.Num();
	ShaderStorage->AttributeNames.Reserve(NumFuncs);
	ShaderStorage->AttributeChannelCount.Reserve(NumFuncs);

	for (int32 FuncIdx = 0; FuncIdx < NumFuncs; ++FuncIdx)
	{
		const FNiagaraDataInterfaceGeneratedFunction& Func = ParameterInfo.GeneratedFunctions[FuncIdx];

		const FName* AttributeName = Func.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute);
		if (AttributeName != nullptr)
		{
			int32 ComponentCount = UNiagaraDataInterfaceGrid3DCollection::GetComponentCountFromFuncName(Func.DefinitionName);
			ShaderStorage->AttributeNames.Add(*AttributeName);
			ShaderStorage->AttributeChannelCount.Add(ComponentCount);
		}
		else
		{
			ShaderStorage->AttributeNames.Add(FName());
			ShaderStorage->AttributeChannelCount.Add(INDEX_NONE);
		}
	}

	ShaderStorage->AttributeNames.Shrink();
	ShaderStorage->AttributeChannelCount.Shrink();

	return ShaderStorage;
}

const FTypeLayoutDesc* UNiagaraDataInterfaceGrid3DCollection::GetShaderStorageType() const
{
	return &StaticGetTypeLayoutDesc<FNiagaraDataInterfaceParametersCS_Grid3DCollection>();
}


#if WITH_EDITOR
bool UNiagaraDataInterfaceGrid3DCollection::GenerateIterationSourceNamespaceReadAttributesHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, const FNiagaraVariable& IterationSourceVar, TConstArrayView<FNiagaraVariable> InArguments, TConstArrayView<FNiagaraVariable> InAttributes, TConstArrayView<FString> InAttributeHLSLNames, bool bInSetToDefaults, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const
{

	FString DIVarName;
	OutHLSL += TEXT("\t//Generated by UNiagaraDataInterfaceGrid3DCollection::GenerateIterationSourceNamespaceReadAttributesHLSL\n");
	for (int32 i = 0; i < InArguments.Num(); i++)
	{
		OutHLSL += FString::Printf(TEXT("\t// Argument Name \"%s\" Type \"%s\"\n"), *InArguments[i].GetName().ToString(), *InArguments[i].GetType().GetName());
		if (InArguments[i].GetType().GetClass() == GetClass())
		{
			DIVarName = InArguments[i].GetName().ToString();
		}
	}

	if (InAttributes.Num() != InAttributeHLSLNames.Num())
		return false;

	if (InAttributes.Num() > 0)
	{
		OutHLSL += FString::Printf(TEXT("\tint X, Y, Z;\n\t%s.ExecutionIndexToGridIndex(X, Y, Z);\n"), *DIVarName);
	}

	TArray<FString> RootArray;
	IterationSourceVar.GetName().ToString().ParseIntoArray(RootArray, TEXT("."));

	for (int32 i = 0; i < InAttributes.Num(); i++)
	{
		OutHLSL += FString::Printf(TEXT("\t// Variable Name \"%s\" Type \"%s\" Var \"%s\"\n"), *InAttributes[i].GetName().ToString(), *InAttributes[i].GetType().GetName(), *InAttributeHLSLNames[i]);

		TArray<FString> OutArray;
		if (InAttributes[i].GetName().ToString().ParseIntoArray(OutArray, TEXT(".")) > 0)
		{
			if (TypeDefinitionToSetFunctionName(InAttributes[i].GetType()) == NAME_None)
			{
				FText Error = FText::Format(LOCTEXT("UnknownType", "Unsupported Type {0} , Attribute {1} for custom iteration source"), InAttributes[i].GetType().GetNameText(), FText::FromName(InAttributes[i].GetName()));
				OutErrors.Add(Error);
				continue;
			}

			// Clear out the shared namespace with the root variable...
			FString AttributeName;
			for (int32 NamespaceIdx = 0; NamespaceIdx < OutArray.Num(); NamespaceIdx++)
			{
				if (NamespaceIdx < RootArray.Num() && RootArray[NamespaceIdx] == OutArray[NamespaceIdx])
					continue;
				if (OutArray[NamespaceIdx] == (FNiagaraConstants::PreviousNamespace.ToString()) || OutArray[NamespaceIdx] == (FNiagaraConstants::InitialNamespace.ToString()))
				{
					FText Error = FText::Format(LOCTEXT("UnknownSubNamespace", "Unsupported NamespaceModifier Attribute {0}"), FText::FromName(InAttributes[i].GetName()));
					OutErrors.Add(Error);
				}
				if (AttributeName.Len() != 0)
					AttributeName += TEXT(".");
				AttributeName += OutArray[NamespaceIdx];
			}
			OutHLSL += FString::Printf(TEXT("\t%s.%s<Attribute=\"%s\">(X, Y, Z, %s);\n"), *DIVarName, *TypeDefinitionToGetFunctionName(InAttributes[i].GetType()).ToString(), *AttributeName, *InAttributeHLSLNames[i]);
		}

	}
	return true;
}
bool UNiagaraDataInterfaceGrid3DCollection::GenerateIterationSourceNamespaceWriteAttributesHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, const FNiagaraVariable& IterationSourceVar, TConstArrayView<FNiagaraVariable> InArguments, TConstArrayView<FNiagaraVariable> InAttributes, TConstArrayView<FString> InAttributeHLSLNames, TConstArrayView<FNiagaraVariable> InAllAttributes, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const
{
	FGrid3DCollectionAttributeHlslWriter AttributeWriter(DIInstanceInfo);

	FString DIVarName;
	OutHLSL += TEXT("\t//Generated by UNiagaraDataInterfaceGrid3DCollection::GenerateIterationSourceNamespaceWriteAttributesHLSL\n");
	for (int32 i = 0; i < InArguments.Num(); i++)
	{
		OutHLSL += FString::Printf(TEXT("\t// Argument Name \"%s\" Type \"%s\"\n"), *InArguments[i].GetName().ToString(), *InArguments[i].GetType().GetName());
		if (InArguments[i].GetType().GetClass() == GetClass())
		{
			DIVarName = InArguments[i].GetName().ToString();
		}
	}
	if (InAttributes.Num() != InAttributeHLSLNames.Num())
		return false;

	// determine the total number of channels in the grid
	int TotalChannels = 0;
	for (int32 i = 0; i < InAllAttributes.Num(); i++)
	{
		int ComponentCount = GetComponentCountFromFuncName(TypeDefinitionToSetFunctionName(InAllAttributes[i].GetType()));
		TotalChannels += ComponentCount;
	}
	bool UseRGBATexture = FGrid3DCollectionAttributeHlslWriter::ShouldUseRGBAGrid(TotalChannels, InAttributes.Num());

	TArray<FString> RootArray;
	IterationSourceVar.GetName().ToString().ParseIntoArray(RootArray, TEXT("."));

	// if we aren't using an rgba texture, then we want to maintain the attribute is set mask
	if (!UseRGBATexture)
	{
		OutHLSL += FString("\tint AttributeIsSetMask = 0;\n");
	}

	// generate xyz execution index for the current grid
	OutHLSL += FString("\tint CurrAttributeIndex;\n");
	OutHLSL += FString("\tint X, Y, Z;\n");
	OutHLSL += FString::Printf(TEXT("\t%s.ExecutionIndexToGridIndex(X, Y, Z);\n"), *DIVarName);

	// only copy attributes from prev->next if the number of attributes written is less than the total number of attributes for the DI. 
	if (UseRGBATexture && InAttributes.Num() > 0 && InAttributes.Num() < InAllAttributes.Num() && !bSpawnOnly && !bPartialWrites)
	{
		static const TCHAR* FormatBounds = TEXT(R"(			
			{Grid}.CopyMaskedPreviousToCurrentForCell(X,Y,Z,0,0);
		)");
		TMap<FString, FStringFormatArg> ArgsBounds = {
			{TEXT("Grid"), DIVarName},
		};
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
	}

	// generate code for setting values from parameter map to grid attributes
	int NumAttributesSet = 0;
	for (int32 i = 0; i < InAttributes.Num(); i++)
	{
		OutHLSL += FString::Printf(TEXT("\t// Name \"%s\" Type \"%s\" Var \"%s\"\n"), *InAttributes[i].GetName().ToString(), *InAttributes[i].GetType().GetName(), *InAttributeHLSLNames[i]);
		int ComponentCount = GetComponentCountFromFuncName(TypeDefinitionToSetFunctionName(InAttributes[i].GetType()));

		TArray<FString> OutArray;
		if (InAttributes[i].GetName().ToString().ParseIntoArray(OutArray, TEXT(".")) > 0)
		{
			if (TypeDefinitionToSetFunctionName(InAttributes[i].GetType()) == NAME_None)
			{
				FText Error = FText::Format(LOCTEXT("UnknownType", "Unsupported Type {0} , Attribute {1} for custom iteration source"), InAttributes[i].GetType().GetNameText(), FText::FromName(InAttributes[i].GetName()));
				OutErrors.Add(Error);
				continue;
			}

			// Clear out the shared namespace with the root variable...
			FString AttributeName;
			for (int32 NamespaceIdx = 0; NamespaceIdx < OutArray.Num(); NamespaceIdx++)
			{
				if (NamespaceIdx < RootArray.Num() && RootArray[NamespaceIdx] == OutArray[NamespaceIdx])
					continue;

				if (OutArray[NamespaceIdx] == (FNiagaraConstants::PreviousNamespace.ToString()) || OutArray[NamespaceIdx] == (FNiagaraConstants::InitialNamespace.ToString()))
				{
					FText Error = FText::Format(LOCTEXT("UnknownSubNamespace", "Unsupported NamespaceModifier Attribute {0}"), FText::FromName(InAttributes[i].GetName()));
					OutErrors.Add(Error);
				}
				if (AttributeName.Len() != 0)
					AttributeName += TEXT(".");
				AttributeName += OutArray[NamespaceIdx];
			}

			// maintain the attribute set mask if we are not using an rgba texture
			if (!UseRGBATexture)
			{
				OutHLSL += FString::Printf(TEXT("\t%s.%s<Attribute=\"%s\">(CurrAttributeIndex);\n"), *DIVarName, *TypeDefinitionToAttributeIndexFunctionName(InAttributes[i].GetType()).ToString(), *AttributeName);

				for (int j = 0; j < ComponentCount; ++j)
				{
					OutHLSL += FString::Printf(TEXT("\tAttributeIsSetMask |= 1l << (CurrAttributeIndex+%d);\n"), j);
					NumAttributesSet++;
				}
			}

			OutHLSL += FString::Printf(TEXT("\t%s.%s<Attribute=\"%s\">(X, Y, Z,  %s);\n"), *DIVarName, *TypeDefinitionToSetFunctionName(InAttributes[i].GetType()).ToString(), *AttributeName, *InAttributeHLSLNames[i]);
		}
	}
	
	// if we aren't using an rgba texture then we need to copy the grid attributes that were not set via the parameter map to make sure we don't lose values from previous stages
	if (!UseRGBATexture && InAttributes.Num() > 0 && InAttributes.Num() < InAllAttributes.Num() && !bSpawnOnly && !bPartialWrites)
	{

		static const TCHAR* FormatBounds = TEXT(R"(			
			{Grid}.CopyMaskedPreviousToCurrentForCell(X,Y,Z,{NumAttributesSet},AttributeIsSetMask);
		)");
		TMap<FString, FStringFormatArg> ArgsBounds = {
			{TEXT("Grid"), DIVarName},
			{TEXT("NumAttributesSet"), NumAttributesSet},
		};
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);

	}
	return true;
}


bool UNiagaraDataInterfaceGrid3DCollection::GenerateSetupHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, TConstArrayView<FNiagaraVariable> InArguments, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const
{
	FString DIVarName;
	OutHLSL += TEXT("\t//Generated by UNiagaraDataInterfaceGrid3DCollection::GenerateSetupHLSL\n");
	for (int32 i = 0; i < InArguments.Num(); i++)
	{
		OutHLSL += FString::Printf(TEXT("\t// Argument Name \"%s\" Type \"%s\"\n"), *InArguments[i].GetName().ToString(), *InArguments[i].GetType().GetName());

		if (InArguments[i].GetType().GetClass() == GetClass())
		{
			DIVarName = InArguments[i].GetName().ToString();
		}
	}

	return true;
}

bool UNiagaraDataInterfaceGrid3DCollection::GenerateTeardownHLSL(FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, TConstArrayView<FNiagaraVariable> InArguments, bool bSpawnOnly, bool bPartialWrites, TArray<FText>& OutErrors, FString& OutHLSL) const
{
	OutHLSL += TEXT("\t//Generated by UNiagaraDataInterfaceGrid3DCollection::GenerateTeardownHLSL\n");


	return true;
}
#endif
bool UNiagaraDataInterfaceGrid3DCollection::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	check(Proxy);

	FGrid3DCollectionRWInstanceData_GameThread* InstanceData = new (PerInstanceData) FGrid3DCollectionRWInstanceData_GameThread();
	SystemInstancesToProxyData_GT.Emplace(SystemInstance->GetId(), InstanceData);

	/* Go through all references to this data interface and build up the attribute list from the function metadata of those referenced.*/
	int32 NumAttribChannelsFound = 0;
	int32 NumNamedAttribChannelsFound = 0;
	FindAttributes(InstanceData->Vars, InstanceData->Offsets, NumNamedAttribChannelsFound);

	NumAttribChannelsFound = NumAttributes + NumNamedAttribChannelsFound;
	InstanceData->TotalNumAttributes = NumAttribChannelsFound;
	InstanceData->TotalNumNamedAttributes = NumNamedAttribChannelsFound;

	// give an error if there are any unnamed attributes
	if (NumAttributes > 0 && NumNamedAttribChannelsFound > 0)
	{
		UE_LOGF(LogNiagara, Warning, "%ls: Unnamed attributes should not be used with named.  This is a deprecated workflow", *Proxy->GetSourceDIName().ToString());
	}

	if (SetResolutionMethod == ESetResolutionMethod::Independent)
	{
		InstanceData->NumCells.X = NumCells.X;
		InstanceData->NumCells.Y = NumCells.Y;
		InstanceData->NumCells.Z = NumCells.Z;

		InstanceData->WorldBBoxSize = WorldBBoxSize;
		InstanceData->CellSize = InstanceData->WorldBBoxSize / FVector(InstanceData->NumCells);
	}
	else if (SetResolutionMethod == ESetResolutionMethod::MaxAxis)
	{
		InstanceData->CellSize = FVector(FMath::Max(FMath::Max(WorldBBoxSize.X, WorldBBoxSize.Y), WorldBBoxSize.Z) / double(NumCellsMaxAxis));
	}
	else if (SetResolutionMethod == ESetResolutionMethod::CellSize)
	{
		InstanceData->CellSize = FVector(CellSize);
	}

	// compute world bounds and padding based on cell size
	if (SetResolutionMethod == ESetResolutionMethod::MaxAxis || SetResolutionMethod == ESetResolutionMethod::CellSize)
	{
		InstanceData->NumCells.X = int32(FMath::FloorToInt(WorldBBoxSize.X / InstanceData->CellSize[0]));
		InstanceData->NumCells.Y = int32(FMath::FloorToInt(WorldBBoxSize.Y / InstanceData->CellSize[0]));
		InstanceData->NumCells.Z = int32(FMath::FloorToInt(WorldBBoxSize.Z / InstanceData->CellSize[0]));

		// Pad grid by 1 cell if our computed bounding box is too small
		if (WorldBBoxSize.X > WorldBBoxSize.Y && WorldBBoxSize.X > WorldBBoxSize.Z)
		{
			if (!FMath::IsNearlyEqual(InstanceData->CellSize[0] * InstanceData->NumCells.Y, WorldBBoxSize.Y))
			{
				InstanceData->NumCells.Y++;
			}

			if (!FMath::IsNearlyEqual(InstanceData->CellSize[0] * InstanceData->NumCells.Z, WorldBBoxSize.Z))
			{
				InstanceData->NumCells.Z++;
			}
		}
		else if (WorldBBoxSize.Y > WorldBBoxSize.X && WorldBBoxSize.Y > WorldBBoxSize.Z)
		{
			if (!FMath::IsNearlyEqual(InstanceData->CellSize[0] * InstanceData->NumCells.X, WorldBBoxSize.X))
			{
				InstanceData->NumCells.X++;
			}

			if (!FMath::IsNearlyEqual(InstanceData->CellSize[0] * InstanceData->NumCells.Z, WorldBBoxSize.Z))
			{
				InstanceData->NumCells.Z++;
			}
		}
		else if (WorldBBoxSize.Z > WorldBBoxSize.X && WorldBBoxSize.Z > WorldBBoxSize.Y)
		{
			if (!FMath::IsNearlyEqual(InstanceData->CellSize[0] * InstanceData->NumCells.X, WorldBBoxSize.X))
			{
				InstanceData->NumCells.X++;
			}

			if (!FMath::IsNearlyEqual(InstanceData->CellSize[0] * InstanceData->NumCells.Y, WorldBBoxSize.Y))
			{
				InstanceData->NumCells.Y++;
			}
		}

		InstanceData->WorldBBoxSize = FVector(InstanceData->NumCells.X, InstanceData->NumCells.Y, InstanceData->NumCells.Z) * InstanceData->CellSize[0];
	}

	if (InstanceData->NumCells.X <= 0 ||
		InstanceData->NumCells.Y <= 0 ||
		InstanceData->NumCells.Z <= 0)
	{
		UE_LOGF(LogNiagara, Error, "Zero grid resolution defined on %ls", *Proxy->GetSourceDIName().ToString());
		return false;
	}

	if (!FMath::IsNearlyEqual(GNiagaraGrid3DResolutionMultiplier, 1.0f))
	{
		InstanceData->NumCells.X = FMath::Max(1, int32(float(InstanceData->NumCells.X) * GNiagaraGrid3DResolutionMultiplier));
		InstanceData->NumCells.Y = FMath::Max(1, int32(float(InstanceData->NumCells.Y) * GNiagaraGrid3DResolutionMultiplier));
		InstanceData->NumCells.Z = FMath::Max(1, int32(float(InstanceData->NumCells.Z) * GNiagaraGrid3DResolutionMultiplier));
	}

	if (NumAttribChannelsFound == 0)
	{
		if (!Proxy->GetSourceDIName().IsNone())
		{
			UE_LOGF(LogNiagara, Log, "Zero attributes defined on %ls", *Proxy->GetSourceDIName().ToString());
		}

		// Push Updates to Proxy.
		FNiagaraDataInterfaceProxyGrid3DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid3DCollectionProxy>();
		ENQUEUE_RENDER_COMMAND(FUpdateData)(
			[RT_Proxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
			{
				check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));
				FGrid3DCollectionRWInstanceData_RenderThread* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.Add(InstanceID);

			});

		return true;
	}

	// Initialize target texture
	ENiagaraGpuBufferFormat BufferFormat = bOverrideFormat ? OverrideBufferFormat : GetDefault<UNiagaraSettings>()->DefaultGridFormat;
	if (GNiagaraGrid3DOverrideFormat >= int32(ENiagaraGpuBufferFormat::Float) && (GNiagaraGrid3DOverrideFormat < int32(ENiagaraGpuBufferFormat::Max)))
	{
		BufferFormat = ENiagaraGpuBufferFormat(GNiagaraGrid3DOverrideFormat);
	}

	// if we have 4 or less attributes, pack data into rgba textures rather than a single large float texture with attributes tiled spatially
	bool UseRGBATexture = FGrid3DCollectionAttributeHlslWriter::ShouldUseRGBAGrid(NumAttribChannelsFound, InstanceData->Vars.Num());

	if (UseRGBATexture)
	{

		if (NumAttribChannelsFound - NumNamedAttribChannelsFound > 0)
		{
			UE_LOGF(LogNiagara, Error, "%ls: Unnamed attributes do not work with rgba textures", *Proxy->GetSourceDIName().ToString());
			return false;
		}

		InstanceData->NumTiles.X = 1;
		InstanceData->NumTiles.Y = 1;
		InstanceData->NumTiles.Z = 1;
		
		InstanceData->PixelFormat = FNiagaraUtilities::BufferFormatToPixelFormat(BufferFormat, GNDIGrid3DFomatCaps, NumAttribChannelsFound, this);

		InstanceData->UseRGBATexture = true;
	}
	else
	{
		// Compute number of tiles based on resolution of individual attributes

		// #todo(dmp): refactor
		const float MaxDim = 2048;
		const int32 MaxTilesX = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.X));
		const int32 MaxTilesY = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.Y));
		const int32 MaxTilesZ = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.Z));
		const int32 MaxAttributes = MaxTilesX * MaxTilesY * MaxTilesZ;

		if (InstanceData->NumCells.X > MaxDim || InstanceData->NumCells.Y > MaxDim || InstanceData->NumCells.Z > MaxDim)
		{
			UE_LOGF(LogNiagara, Error, "Resolution is too high on %ls... max is 2048, defined as %d, %d %d", *Proxy->GetSourceDIName().ToString(), InstanceData->NumCells.X, InstanceData->NumCells.Y, InstanceData->NumCells.Z);
			return false;
		}

		if (NumAttribChannelsFound > MaxAttributes && MaxAttributes > 0)
		{
			UE_LOGF(LogNiagara, Error, "Invalid number of attributes defined on %ls... max is %d, num defined is %d", *Proxy->GetSourceDIName().ToString(), MaxAttributes, NumAttribChannelsFound);
			return false;
		}

		// need to determine number of tiles in x and y based on number of attributes and max dimension size
		const int32 NumTilesX = FMath::Min<int32>(MaxTilesX, NumAttribChannelsFound);
		const int32 NumTilesY = FMath::Min<int32>(MaxTilesY, FMath::CeilToInt(1.0f * float(NumAttribChannelsFound) / float(NumTilesX)));
		const int32 NumTilesZ = FMath::Min<int32>(MaxTilesZ, FMath::CeilToInt(1.0f * float(NumAttribChannelsFound) / float(NumTilesX * NumTilesY)));

		InstanceData->NumTiles.X = NumTilesX;
		InstanceData->NumTiles.Y = NumTilesY;
		InstanceData->NumTiles.Z = NumTilesZ;

		check(InstanceData->NumTiles.X > 0);
		check(InstanceData->NumTiles.Y > 0);
		check(InstanceData->NumTiles.Z > 0);

		InstanceData->UseRGBATexture = false;
		InstanceData->PixelFormat = FNiagaraUtilities::BufferFormatToPixelFormat(BufferFormat, GNDIGrid3DFomatCaps, 1, this);
	}

	InstanceData->TargetTexture = nullptr;
	InstanceData->RTUserParamBinding.Init(SystemInstance->GetInstanceParameters(), RenderTargetUserParameter.Parameter);
	InstanceData->UpdateTargetTexture(BufferFormat, this);

#if WITH_EDITOR
	InstanceData->bPreviewGrid = bPreviewGrid;
	InstanceData->PreviewAttribute = FIntVector4(INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE);
	if (bPreviewGrid && !PreviewAttribute.IsNone())
	{
		const int32 VariableIndex = InstanceData->Vars.IndexOfByPredicate([&](const FNiagaraVariableBase& Variable) { return Variable.GetName() == PreviewAttribute; });
		if (VariableIndex != INDEX_NONE)
		{
			const int32 NumComponents = InstanceData->Vars[VariableIndex].GetType().GetSize() / sizeof(float);
			if (ensure(NumComponents > 0 && NumComponents <= 4))
			{
				const int32 ComponentOffset = InstanceData->Offsets[VariableIndex];
				for (int32 i = 0; i < NumComponents; ++i)
				{
					InstanceData->PreviewAttribute[i] = ComponentOffset + i;
				}
			}
		}
		// Look for anonymous attributes
		else if (NumAttributes > 0)
		{
			const FString PreviewAttributeString = PreviewAttribute.ToString();
			if (PreviewAttributeString.StartsWith(AnonymousAttributeString))
			{
				InstanceData->PreviewAttribute[0] = FCString::Atoi(&PreviewAttributeString.GetCharArray()[AnonymousAttributeString.Len() + 1]);
			}
		}

		if (InstanceData->PreviewAttribute == FIntVector4(INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE))
		{
			UE_LOGF(LogNiagara, Warning, "Failed to map PreviewAttribute %ls to a grid index", *PreviewAttribute.ToString());
		}
	}
#endif

	InstanceData->ClearBeforeNonIterationStage = ClearBeforeNonIterationStage;

	// Push Updates to Proxy.
	FNiagaraDataInterfaceProxyGrid3DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid3DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Proxy, RT_Resource = InstanceData->TargetTexture ? InstanceData->TargetTexture->GetResource() : nullptr, InstanceID = SystemInstance->GetId(), RT_InstanceData = *InstanceData](FRHICommandListImmediate& RHICmdList)
		{
			check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));
			FGrid3DCollectionRWInstanceData_RenderThread* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.Add(InstanceID);

			TargetData->SourceDIName = RT_Proxy->GetSourceDIName();

			TargetData->ClearBeforeNonIterationStage = RT_InstanceData.ClearBeforeNonIterationStage;

			TargetData->NumCells = RT_InstanceData.NumCells;
			TargetData->NumTiles = RT_InstanceData.NumTiles;
			TargetData->TotalNumAttributes = RT_InstanceData.TotalNumAttributes;
			TargetData->TotalNumNamedAttributes = RT_InstanceData.TotalNumNamedAttributes;
			TargetData->CellSize = RT_InstanceData.CellSize;
			TargetData->WorldBBoxSize = RT_InstanceData.WorldBBoxSize;
			TargetData->PixelFormat = RT_InstanceData.PixelFormat;
			TargetData->UseRGBATexture = RT_InstanceData.UseRGBATexture;
			TargetData->Offsets = RT_InstanceData.Offsets;
			TargetData->Vars.Reserve(RT_InstanceData.Vars.Num());
			for (int32 i = 0; i < RT_InstanceData.Vars.Num(); i++)
			{
				TargetData->Vars.Emplace(RT_InstanceData.Vars[i].GetName());
				TargetData->VarComponents.Add(RT_InstanceData.Vars[i].GetType().GetSize() / sizeof(float));
			}
#if WITH_EDITOR
			TargetData->bPreviewGrid = RT_InstanceData.bPreviewGrid;
			TargetData->PreviewAttribute = RT_InstanceData.PreviewAttribute;
#endif

			if (RT_Resource && RT_Resource->TextureRHI.IsValid())
			{
				TargetData->RenderTargetToCopyTo = RT_Resource->TextureRHI;
			}
			else
			{
				TargetData->RenderTargetToCopyTo = nullptr;
			}
		});

	return true;
}


void UNiagaraDataInterfaceGrid3DCollection::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	SystemInstancesToProxyData_GT.Remove(SystemInstance->GetId());

	FGrid3DCollectionRWInstanceData_GameThread* InstanceData = static_cast<FGrid3DCollectionRWInstanceData_GameThread*>(PerInstanceData);

	InstanceData->~FGrid3DCollectionRWInstanceData_GameThread();

	FNiagaraDataInterfaceProxyGrid3DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid3DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[RT_Proxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			//check(ThisProxy->SystemInstancesToProxyData.Contains(InstanceID));
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		}
	);
}


bool UNiagaraDataInterfaceGrid3DCollection::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FGrid3DCollectionRWInstanceData_GameThread* InstanceData = SystemInstancesToProxyData_GT.FindRef(SystemInstance->GetId());

	ENiagaraGpuBufferFormat BufferFormat = bOverrideFormat ? OverrideBufferFormat : GetDefault<UNiagaraSettings>()->DefaultGridFormat;
	if (GNiagaraGrid3DOverrideFormat >= int32(ENiagaraGpuBufferFormat::Float) && (GNiagaraGrid3DOverrideFormat < int32(ENiagaraGpuBufferFormat::Max)))
	{
		BufferFormat = ENiagaraGpuBufferFormat(GNiagaraGrid3DOverrideFormat);
	}
	bool NeedsReset = InstanceData->UpdateTargetTexture(BufferFormat, this);

	FNiagaraDataInterfaceProxyGrid3DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid3DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Resource = InstanceData->TargetTexture ? InstanceData->TargetTexture->GetResource() : nullptr, RT_Proxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
		{
			FGrid3DCollectionRWInstanceData_RenderThread* TargetData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID);

			if (RT_Resource && RT_Resource->TextureRHI.IsValid())
			{
				TargetData->RenderTargetToCopyTo = RT_Resource->TextureRHI;
			}
			else
			{
				TargetData->RenderTargetToCopyTo = nullptr;
			}

		});

	return false;
}

void UNiagaraDataInterfaceGrid3DCollection::GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const
{
	OutVariables.Emplace(ExposedRTVar);
}

bool UNiagaraDataInterfaceGrid3DCollection::GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const
{
	FGrid3DCollectionRWInstanceData_GameThread* InstanceData = static_cast<FGrid3DCollectionRWInstanceData_GameThread*>(InPerInstanceData);
	if (InVariable.IsValid() && InVariable == ExposedRTVar && InstanceData && InstanceData->TargetTexture)
	{
		UTextureRenderTarget** Var = (UTextureRenderTarget**)OutData;
		*Var = InstanceData->TargetTexture;
		return true;
	}
	return false;
}

bool UNiagaraDataInterfaceGrid3DCollection::FillVolumeTexture(const UNiagaraComponent* Component, UVolumeTexture* Dest, int AttributeIndex)
{
	/*
	#todo(dmp): we need to get a UVolumeTextureRenderTarget for any of this to work
	if (!Component || !Dest)
	{
		return false;
	}

	FNiagaraSystemInstance *SystemInstance = Component->GetSystemInstance();
	if (!SystemInstance)
	{
		return false;
	}

	// check valid attribute index
	if (AttributeIndex < 0 || AttributeIndex >=NumAttributes)
	{
		return false;
	}

	// check dest size and type needs to be float
	// #todo(dmp): don't hardcode float since we might do other stuff in the future
	EPixelFormat RequiredTye = PF_R32_FLOAT;
	if (!Dest || Dest->GetSizeX() != NumCells.X || Dest->GetSizeY() != NumCells.Y || Dest->GetSizeZ() != NumCells.Z || Dest->GetPixelFormat() != RequiredTye)
	{
		return false;
	}

	FNiagaraDataInterfaceProxyGrid3DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid3DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateDIColorCurve)(
		[RT_Proxy, InstanceID=SystemInstance->GetId(), RT_TextureResource=Dest->Resource, AttributeIndex](FRHICommandListImmediate& RHICmdList)
	{
		FGrid3DCollectionRWInstanceData_RenderThread* Grid3DInstanceData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID);

		if (RT_TextureResource && RT_TextureResource->TextureRHI.IsValid() && Grid3DInstanceData && Grid3DInstanceData->CurrentData)
		{
			FRHICopyTextureInfo CopyInfo;
			CopyInfo.Size = FIntVector(Grid3DInstanceData->NumCells.X, Grid3DInstanceData->NumCells.Y, 1);

			int TileIndexX = AttributeIndex % Grid3DInstanceData->NumTiles.X;
			int TileIndexY = AttributeIndex / Grid3DInstanceData->NumTiles.X;
			int StartX = TileIndexX * Grid3DInstanceData->NumCells.X;
			int StartY = TileIndexY * Grid3DInstanceData->NumCells.Y;
			CopyInfo.SourcePosition = FIntVector(StartX, StartY, 0);

			RHICmdList.TransitionResource(EResourceTransitionAccess::EReadable, Grid3DInstanceData->CurrentData->GridBuffer.Buffer);
			RHICmdList.TransitionResource(EResourceTransitionAccess::EWritable, RT_TextureResource->TextureRHI);

			RHICmdList.CopyTexture(Grid3DInstanceData->CurrentData->GridBuffer.Buffer, RT_TextureResource->TextureRHI, CopyInfo);

			RHICmdList.TransitionResource(EResourceTransitionAccess::EReadable, RT_TextureResource->TextureRHI);
		}
	});

	return true;
	*/
	return false;
}

bool UNiagaraDataInterfaceGrid3DCollection::FillRawVolumeTexture(const UNiagaraComponent* Component, UVolumeTexture* Dest, int& TilesX, int& TilesY, int& TilesZ)
{
	/*
	#todo(dmp): we need to get a UVolumeTextureRenderTarget for any of this to work

	if (!Component)
	{
		TilesX = -1;
		TilesY = -1;
		TilesZ = -1;
		return false;
	}

	FNiagaraSystemInstance* SystemInstance = Component->GetSystemInstance();
	if (!SystemInstance)
	{
		TilesX = -1;
		TilesY = -1;
		TilesZ = -1;
		return false;
	}

	FGrid3DCollectionRWInstanceData_GameThread* Grid3DInstanceData = SystemInstancesToProxyData_GT.FindRef(SystemInstance->GetId());
	if (!Grid3DInstanceData)
	{
		TilesX = -1;
		TilesY = -1;
		TilesZ = -1;
		return false;
	}

	TilesX = Grid3DInstanceData->NumTiles.X;
	TilesY = Grid3DInstanceData->NumTiles.Y;
	TilesZ = Grid3DInstanceData->NumTiles.Z;

	// check dest size and type needs to be float
	// #todo(dmp): don't hardcode float since we might do other stuff in the future
	EPixelFormat RequiredTye = PF_R32_FLOAT;
	if (!Dest || Dest->GetSizeX() != NumCells.X * TilesX || Dest->GetSizeY() != NumCells.Y * TilesY || Dest->GetSizeZ() != NumCells.Z * TilesZ || Dest->GetPixelFormat() != RequiredTye)
	{
		return false;
	}

	FNiagaraDataInterfaceProxyGrid3DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid3DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateDIColorCurve)(
		[RT_Proxy, RT_InstanceID=SystemInstance->GetId(), RT_TextureResource=Dest->Resource](FRHICommandListImmediate& RHICmdList)
	{
		FGrid3DCollectionRWInstanceData_RenderThread* RT_Grid3DInstanceData = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID);

		if (RT_TextureResource && RT_TextureResource->TextureRHI.IsValid() && RT_Grid3DInstanceData && RT_Grid3DInstanceData->CurrentData)
		{
			RHICmdList.TransitionResource(EResourceTransitionAccess::EReadable, RT_Grid3DInstanceData->CurrentData->GridBuffer.Buffer);
			RHICmdList.TransitionResource(EResourceTransitionAccess::EWritable, RT_TextureResource->TextureRHI);

			FRHICopyTextureInfo CopyInfo;
			RHICmdList.CopyTexture(RT_Grid3DInstanceData->CurrentData->GridBuffer.Buffer, RT_TextureResource->TextureRHI, CopyInfo);

			RHICmdList.TransitionResource(EResourceTransitionAccess::EReadable, RT_TextureResource->TextureRHI);
		}

	});

	return true;
	*/

	return false;
}

void UNiagaraDataInterfaceGrid3DCollection::GetRawTextureSize(const UNiagaraComponent* Component, int& SizeX, int& SizeY, int& SizeZ)
{
	if (!Component)
	{
		SizeX = -1;
		SizeY = -1;
		SizeZ = -1;
		return;
	}

	FNiagaraSystemInstanceControllerConstPtr Controller = Component->GetSystemInstanceController();
	if (!Controller)
	{
		SizeX = -1;
		SizeY = -1;
		SizeZ = -1;
		return;
	}
	FNiagaraSystemInstanceID InstanceID = Controller->GetSystemInstanceID();

	FGrid3DCollectionRWInstanceData_GameThread* Grid3DInstanceData = SystemInstancesToProxyData_GT.FindRef(InstanceID);
	if (!Grid3DInstanceData)
	{
		SizeX = -1;
		SizeY = -1;
		SizeZ = -1;
		return;
	}

	SizeX = Grid3DInstanceData->NumCells.X * Grid3DInstanceData->NumTiles.X;
	SizeY = Grid3DInstanceData->NumCells.Y * Grid3DInstanceData->NumTiles.Y;
	SizeZ = Grid3DInstanceData->NumCells.Z * Grid3DInstanceData->NumTiles.Z;
}

void UNiagaraDataInterfaceGrid3DCollection::GetTextureSize(const UNiagaraComponent* Component, int& SizeX, int& SizeY, int& SizeZ)
{
	if (!Component)
	{
		SizeX = -1;
		SizeY = -1;
		SizeZ = -1;
		return;
	}

	FNiagaraSystemInstanceControllerConstPtr Controller = Component->GetSystemInstanceController();
	if (!Controller)
	{
		SizeX = -1;
		SizeY = -1;
		SizeZ = -1;
		return;
	}
	FNiagaraSystemInstanceID InstanceID = Controller->GetSystemInstanceID();

	FGrid3DCollectionRWInstanceData_GameThread* Grid3DInstanceData = SystemInstancesToProxyData_GT.FindRef(InstanceID);
	if (!Grid3DInstanceData)
	{
		SizeX = -1;
		SizeY = -1;
		SizeZ = -1;
		return;
	}

	SizeX = Grid3DInstanceData->NumCells.X;
	SizeY = Grid3DInstanceData->NumCells.Y;
	SizeZ = Grid3DInstanceData->NumCells.Z;
}

void UNiagaraDataInterfaceGrid3DCollection::VMGetWorldBBoxSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FGrid3DCollectionRWInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutWorldBoundsX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutWorldBoundsY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutWorldBoundsZ(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutWorldBoundsX.GetDestAndAdvance() = float(InstData->WorldBBoxSize.X);
		*OutWorldBoundsY.GetDestAndAdvance() = float(InstData->WorldBBoxSize.Y);
		*OutWorldBoundsZ.GetDestAndAdvance() = float(InstData->WorldBBoxSize.Z);
	}
}

void UNiagaraDataInterfaceGrid3DCollection::VMSetNumCells(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FGrid3DCollectionRWInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsX(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsY(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsZ(Context);
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		int NewNumCellsX = InNumCellsX.GetAndAdvance();
		int NewNumCellsY = InNumCellsY.GetAndAdvance();
		int NewNumCellsZ = InNumCellsZ.GetAndAdvance();

		bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1 && NumCells.X >= 0 && NumCells.Y >= 0 && NumCells.Z >= 0);
		*OutSuccess.GetDestAndAdvance() = bSuccess;
		if (bSuccess)
		{
			FIntVector OldNumCells = InstData->NumCells;

			InstData->NumCells.X = FMath::Max(1, NewNumCellsX);
			InstData->NumCells.Y = FMath::Max(1, NewNumCellsY);
			InstData->NumCells.Z = FMath::Max(1, NewNumCellsZ);

			if (!FMath::IsNearlyEqual(GNiagaraGrid3DResolutionMultiplier, 1.0f))
			{
				InstData->NumCells.X = FMath::Max(1, int32(float(InstData->NumCells.X) * GNiagaraGrid3DResolutionMultiplier));
				InstData->NumCells.Y = FMath::Max(1, int32(float(InstData->NumCells.Y) * GNiagaraGrid3DResolutionMultiplier));
				InstData->NumCells.Z = FMath::Max(1, int32(float(InstData->NumCells.Z) * GNiagaraGrid3DResolutionMultiplier));
			}

			InstData->NeedsRealloc = OldNumCells != InstData->NumCells;
		}
	}
}

void UNiagaraDataInterfaceGrid3DCollection::VMUnitToFloatIndex(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FGrid3DCollectionRWInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncInputHandler<float> InUnitX(Context);
	VectorVM::FExternalFuncInputHandler<float> InUnitY(Context);
	VectorVM::FExternalFuncInputHandler<float> InUnitZ(Context);

	VectorVM::FExternalFuncRegisterHandler<float> OutFloatX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutFloatY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutFloatZ(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		FVector3f InUnit;
		InUnit.X = InUnitX.GetAndAdvance();
		InUnit.Y = InUnitY.GetAndAdvance();
		InUnit.Z = InUnitZ.GetAndAdvance();

		FVector3f OutIndex = InUnit * FVector3f(InstData->NumCells) - .5;
		*OutFloatX.GetDestAndAdvance() = OutIndex.X;
		*OutFloatY.GetDestAndAdvance() = OutIndex.Y;
		*OutFloatZ.GetDestAndAdvance() = OutIndex.Z;
	}
}

bool UNiagaraDataInterfaceGrid3DCollection::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FGrid3DCollectionRWInstanceData_GameThread* InstanceData = static_cast<FGrid3DCollectionRWInstanceData_GameThread*>(PerInstanceData);
	bool bNeedsReset = false;

	if (InstanceData && InstanceData->NeedsRealloc && InstanceData->NumCells.X > 0 && InstanceData->NumCells.Y > 0 && InstanceData->NumCells.Z > 0)
	{
		InstanceData->NeedsRealloc = false;

		InstanceData->CellSize = InstanceData->WorldBBoxSize / FVector(InstanceData->NumCells.X, InstanceData->NumCells.Y, InstanceData->NumCells.Z);

		FNiagaraDataInterfaceProxyGrid3DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid3DCollectionProxy>();

		// rgba textures only have a single tile since attributes are packed into r,g,b, and a channels
		if (InstanceData->UseRGBATexture)
		{

			InstanceData->NumTiles.X = 1;
			InstanceData->NumTiles.Y = 1;
			InstanceData->NumTiles.Z = 1;
		}
		else
		{
			// #todo(dmp): refactor
			const float MaxDim = 2048.0f;
			const int32 MaxTilesX = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.X));
			const int32 MaxTilesY = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.Y));
			const int32 MaxTilesZ = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.Z));
			const int32 MaxAttributes = MaxTilesX * MaxTilesY * MaxTilesZ;

			if (InstanceData->NumCells.X > MaxDim || InstanceData->NumCells.Y > MaxDim || InstanceData->NumCells.Z > MaxDim)
			{
				UE_LOGF(LogNiagara, Error, "Resolution is too high on %ls... max is 2048, defined as %d, %d %d", *Proxy->GetSourceDIName().ToString(), InstanceData->NumCells.X, InstanceData->NumCells.Y, InstanceData->NumCells.Z);
				return false;
			}

			if ((InstanceData->TotalNumAttributes > MaxAttributes && MaxAttributes > 0))
			{
				UE_LOGF(LogNiagara, Error, "Invalid number of attributes defined on %ls... max is %d, num defined is %d", *Proxy->GetSourceDIName().ToString(), MaxAttributes, InstanceData->TotalNumAttributes);
				return false;
			}

			if (InstanceData->TotalNumAttributes <= 0)
			{
				UE_LOGF(LogNiagara, Warning, "Invalid number of attributes defined on %ls... max is %d, num defined is %d", *Proxy->GetSourceDIName().ToString(), MaxAttributes, InstanceData->TotalNumAttributes);
				return false;
			}

			// need to determine number of tiles in x and y based on number of attributes and max dimension size
			const int32 NumTilesX = FMath::Min<int32>(MaxTilesX, InstanceData->TotalNumAttributes);
			const int32 NumTilesY = FMath::Min<int32>(MaxTilesY, FMath::CeilToInt(1.0f * float(InstanceData->TotalNumAttributes) / float(NumTilesX)));
			const int32 NumTilesZ = FMath::Min<int32>(MaxTilesZ, FMath::CeilToInt(1.0f * float(InstanceData->TotalNumAttributes) / float(NumTilesX * NumTilesY)));

			InstanceData->NumTiles.X = NumTilesX;
			InstanceData->NumTiles.Y = NumTilesY;
			InstanceData->NumTiles.Z = NumTilesZ;

			check(InstanceData->NumTiles.X > 0);
			check(InstanceData->NumTiles.Y > 0);
			check(InstanceData->NumTiles.Z > 0);
		}
		// #todo(dmp): we should align this method with the implementation in Grid3DCollection.  For now, we are relying on the next call to tick  to reset the User Texture

		// Push Updates to Proxy.
		ENQUEUE_RENDER_COMMAND(FUpdateData)(
			[RT_Proxy, InstanceID = SystemInstance->GetId(), RT_InstanceData = *InstanceData](FRHICommandListImmediate& RHICmdList)
			{
				check(RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));
				FGrid3DCollectionRWInstanceData_RenderThread* TargetData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID);

				TargetData->NumCells = RT_InstanceData.NumCells;
				TargetData->NumTiles = RT_InstanceData.NumTiles;

				TargetData->TotalNumAttributes = RT_InstanceData.TotalNumAttributes;
				TargetData->TotalNumNamedAttributes = RT_InstanceData.TotalNumNamedAttributes;
				TargetData->CellSize = RT_InstanceData.CellSize;

				TargetData->Buffers.Empty();
				TargetData->CurrentData = nullptr;
				TargetData->DestinationData = nullptr;
				TargetData->PerAttributeData.Release();
			});

	}

	return false;
}

void UNiagaraDataInterfaceGrid3DCollection::VMGetCellSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FGrid3DCollectionRWInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutCellSizeX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutCellSizeY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutCellSizeZ(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutCellSizeX.GetDestAndAdvance() = float(InstData->CellSize.X);
		*OutCellSizeY.GetDestAndAdvance() = float(InstData->CellSize.Y);
		*OutCellSizeZ.GetDestAndAdvance() = float(InstData->CellSize.Z);
	}
}

void UNiagaraDataInterfaceGrid3DCollection::VMGetAttributeIndex(FVectorVMExternalFunctionContext& Context, const FName& InName, int32 NumChannels)
{
	VectorVM::FUserPtrHandler<FGrid3DCollectionRWInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int> OutIndex(Context);
	int32 Index = INDEX_NONE;
	if (InstData.Get())
		Index = InstData.Get()->FindAttributeIndexByName(InName, NumChannels);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutIndex.GetDestAndAdvance() = Index;
	}
}

int32 FGrid3DCollectionRWInstanceData_GameThread::FindAttributeIndexByName(const FName& InName, int32 NumChannels)
{
	for (int32 i = 0; i < Vars.Num(); i++)
	{
		const FNiagaraVariableBase& Var = Vars[i];
		if (Var.GetName() == InName)
		{
			if (NumChannels == 1 && Var.GetType() == FNiagaraTypeDefinition::GetFloatDef())
				return Offsets[i];
			else if (NumChannels == 2 && Var.GetType() == FNiagaraTypeDefinition::GetVec2Def())
				return Offsets[i];
			else if (NumChannels == 3 && Var.GetType() == FNiagaraTypeDefinition::GetVec3Def())
				return Offsets[i];
			else if (NumChannels == 4 && Var.GetType() == FNiagaraTypeDefinition::GetVec4Def())
				return Offsets[i];
			else if (NumChannels == 4 && Var.GetType() == FNiagaraTypeDefinition::GetColorDef())
				return Offsets[i];
		}
	}

	return INDEX_NONE;
}

bool FGrid3DCollectionRWInstanceData_GameThread::UpdateTargetTexture(ENiagaraGpuBufferFormat BufferFormat, UObject* Owner)
{
	// Pull value from user parameter
	if (UObject* UserParamObject = RTUserParamBinding.GetValue())
	{
		TargetTexture = Cast<UTextureRenderTargetVolume>(UserParamObject);

		if (TargetTexture == nullptr)
		{
			UE_LOGF(LogNiagara, Error, "RenderTarget UserParam is a '%ls' but is expected to be a UTextureRenderTargetVolume", *GetNameSafe(UserParamObject->GetClass()));
		}
	}

	// Could be from user parameter of created internally
	if (TargetTexture != nullptr)
	{
		const FIntVector RTSize(NumCells.X * NumTiles.X, NumCells.Y * NumTiles.Y, NumCells.Z * NumTiles.Z);
		const TOptional<EPixelFormat> RenderTargetFormat = FNiagaraUtilities::BufferFormatToPixelFormat(BufferFormat, GNDIGrid3DFomatCaps, 1, Owner);
		if (RenderTargetFormat.IsSet())
		{
			if (TargetTexture->SizeX != RTSize.X || TargetTexture->SizeY != RTSize.Y || TargetTexture->SizeZ != RTSize.Z || TargetTexture->OverrideFormat != RenderTargetFormat.GetValue())
			{
				TargetTexture->OverrideFormat = RenderTargetFormat.GetValue();
				TargetTexture->ClearColor = FLinearColor(0, 0, 0, 0);
				TargetTexture->InitAutoFormat(RTSize.X, RTSize.Y, RTSize.Z);
				TargetTexture->UpdateResourceImmediate(true);

				return true;
			}
		}
	}

	return false;
}

void FGrid3DCollectionRWInstanceData_RenderThread::BeginSimulate(FRDGBuilder& GraphBuilder, bool RequiresBuffering)
{
	for (TUniquePtr<FGrid3DBuffer>& Buffer : Buffers)
	{
		check(Buffer.IsValid());
		if (Buffer.Get() != CurrentData || !RequiresBuffering)
		{
			DestinationData = Buffer.Get();
			break;
		}
	}

	if (DestinationData == nullptr && PixelFormat.IsSet())
	{
		DestinationData = new FGrid3DBuffer();
		Buffers.Emplace(DestinationData);

		const FIntVector TextureSize(NumCells.X * NumTiles.X, NumCells.Y * NumTiles.Y, NumCells.Z * NumTiles.Z);
		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create3D(TextureSize, PixelFormat.GetValue(), FClearValueBinding::Black, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);

		DestinationData->Initialize(GraphBuilder, TEXT("Grid3D::GridTexture"), TextureDesc);

		// This destination buffer will sometimes have old data in it.  Force it to clear.
		AddClearUAVPass(GraphBuilder, DestinationData->GetOrCreateUAV(GraphBuilder), FVector4f(ForceInitToZero));
	}
}

void FGrid3DCollectionRWInstanceData_RenderThread::EndSimulate()
{
	CurrentData = DestinationData;
	DestinationData = nullptr;
}

void FNiagaraDataInterfaceProxyGrid3DCollectionProxy::ResetData(const FNDIGpuComputeResetContext& Context)
{
	FGrid3DCollectionRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	if (!ProxyData)
	{
		return;
	}

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	for (TUniquePtr<FGrid3DBuffer>& Buffer : ProxyData->Buffers)
	{
		AddClearUAVPass(GraphBuilder, Buffer->GetOrCreateUAV(GraphBuilder), FVector4f(ForceInitToZero));
	}
}

void FNiagaraDataInterfaceProxyGrid3DCollectionProxy::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	// #todo(dmp): Context doesnt need to specify if a stage is output or not since we moved pre/post stage to the DI itself.  Not sure which design is better for the future
	if (Context.IsOutputStage())
	{
		FGrid3DCollectionRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
		FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

		ProxyData->BeginSimulate(GraphBuilder, Context.IsInputStage());

		// If there is an output DI and the stage dispatch count is different from the grid resolution, then we want to consider it for a pre-stage UAV clear.
		// This is to enable the most common use case of scattered writes to temporary buffers				
		const FNiagaraSimStageData& SimStageData = Context.GetSimStageData();
		const FIntVector3& ElementCount = SimStageData.DispatchArgs.ElementCount;

		if (ProxyData->ClearBeforeNonIterationStage &&
			ProxyData->DestinationData &&
			(ElementCount.X != ProxyData->NumCells.X || ElementCount.Y != ProxyData->NumCells.Y || ElementCount.Z != ProxyData->NumCells.Z))
		{
			// When the grid is in single-buffer mode (DestinationData == CurrentData), clearing
			// the destination would destroy the only copy of the data. This happens when the grid
			// is an output but not an input for a stage (e.g. only GetOutputGrid* + Set* are used,
			// no read functions), so BeginSimulate receives RequiresBuffering=false and reuses the
			// same buffer for both read and write. Skip the clear in this case.
			// #todo(dmp): surface this as a per-instance editor warning in GetFeedback so users
			// know ClearBeforeNonIterationStage has no effect when the grid is in single-buffer mode.
			if (ProxyData->DestinationData != ProxyData->CurrentData)
			{
				AddClearUAVPass(GraphBuilder, ProxyData->DestinationData->GetOrCreateUAV(GraphBuilder), FVector4f(ForceInitToZero));
			}
		}
	}
}

void FNiagaraDataInterfaceProxyGrid3DCollectionProxy::PostStage(const FNDIGpuComputePostStageContext& Context)
{
	if (Context.IsOutputStage())
	{
		FGrid3DCollectionRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
		ProxyData->EndSimulate();
	}
}

void FNiagaraDataInterfaceProxyGrid3DCollectionProxy::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	FGrid3DCollectionRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

	if (ProxyData->RenderTargetToCopyTo != nullptr && ProxyData->CurrentData != nullptr && ProxyData->CurrentData->IsValid())
	{
		ProxyData->CurrentData->CopyToTexture(Context.GetGraphBuilder(), ProxyData->RenderTargetToCopyTo, TEXT("NiagaraRenderTargetToCopyTo"));
	}

#if WITH_EDITOR
	if (ProxyData->bPreviewGrid && ProxyData->CurrentData && ProxyData->CurrentData->IsValid())
	{
		FNiagaraGpuComputeDebugInterface GpuComputeDebugInterface = Context.GetComputeDispatchInterface().GetGpuComputeDebugInterface();
		FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
		if (ProxyData->PreviewAttribute[0] != INDEX_NONE)
		{
			const FIntVector4 TotalNumAttributeVector = FIntVector4(ProxyData->NumTiles.X, ProxyData->NumTiles.Y, ProxyData->NumTiles.Z, 0);
			GpuComputeDebugInterface.AddAttributeTexture(GraphBuilder, Context.GetSystemInstanceID(), GetSourceDIName(), ProxyData->CurrentData->GetOrCreateTexture(GraphBuilder), TotalNumAttributeVector, ProxyData->PreviewAttribute);
		}
		else
		{
			GpuComputeDebugInterface.AddTexture(GraphBuilder, Context.GetSystemInstanceID(), GetSourceDIName(), ProxyData->CurrentData->GetOrCreateTexture(GraphBuilder));
		}
	}
#endif

	// Clear out the transient resource we cached
	if (Context.IsFinalPostSimulate())
	{
		for (TUniquePtr<FGrid3DBuffer>& Buffer : ProxyData->Buffers)
		{
			Buffer->EndGraphUsage();
		}

		// Readers point to data not owned by themselves so can be caching resources on the 'other' proxy
		// Therefore we need to ensure the transient buffers are correctly cleared
		if (FNiagaraDataInterfaceProxyGrid3DCollectionProxy* OtherGrid3DProxy = static_cast<FNiagaraDataInterfaceProxyGrid3DCollectionProxy*>(ProxyData->OtherProxy))
		{
			FGrid3DCollectionRWInstanceData_RenderThread& OtherProxyData = OtherGrid3DProxy->SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());
			for (TUniquePtr<FGrid3DBuffer>& Buffer : OtherProxyData.Buffers)
			{
				Buffer->EndGraphUsage();
			}
		}
	}
}

void FNiagaraDataInterfaceProxyGrid3DCollectionProxy::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context)
{
	if (const FGrid3DCollectionRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()))
	{
		// support a grid reader acting as an iteration source
		if (ProxyData->OtherProxy != nullptr)
		{
			FNiagaraDataInterfaceProxyGrid3DCollectionProxy* OtherGrid3DProxy = static_cast<FNiagaraDataInterfaceProxyGrid3DCollectionProxy*>(ProxyData->OtherProxy);
			const FGrid3DCollectionRWInstanceData_RenderThread* OtherProxyData = OtherGrid3DProxy->SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
			Context.SetDirect(OtherProxyData->NumCells);
		}
		else
		{
			Context.SetDirect(ProxyData->NumCells);
		}
	}
}

void UNiagaraDataInterfaceGrid3DCollection::FindAttributes(TArray<FNiagaraVariableBase>& OutVariables, TArray<uint32>& OutVariableOffsets, int32& OutNumAttribChannelsFound, TArray<FText>* OutWarnings) const
{
	{
		OutNumAttribChannelsFound = 0;

		UNiagaraSystem* OwnerSystem = GetTypedOuter<UNiagaraSystem>();
		if (OwnerSystem == nullptr)
		{
			return;
		}

		int32 TotalAttributes = NumAttributes;
		for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
		{
			FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
			if (EmitterData && EmitterHandle.GetIsEnabled() && EmitterData->IsValid() && (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim))
			{
				TSharedPtr<const FNiagaraScriptRuntimeData> ScriptRuntimeData = OwnerSystem->GetScriptRuntimeData(EmitterHandle.GetId(), ENiagaraScriptUsage::ParticleGPUComputeScript, FGuid());
				if (ScriptRuntimeData.IsValid())
				{
					CollectAttributesForScript(EmitterData->GetGPUComputeScript(), *ScriptRuntimeData, this, OutVariables, OutVariableOffsets, TotalAttributes, OutWarnings);
				}
			}
		}
		OutNumAttribChannelsFound = TotalAttributes - NumAttributes;
	}
}

void UNiagaraDataInterfaceGrid3DCollection::CollectAttributesForScript(UNiagaraScript* Script, const FNiagaraScriptRuntimeData& ScriptRuntimeData, const UNiagaraDataInterface* DataInterface, TArray<FNiagaraVariableBase>& OutVariables, TArray<uint32>& OutVariableOffsets, int32& TotalAttributes, TArray<FText>* OutWarnings)
{
	if (const FNiagaraScriptExecutionParameterStore* ParameterStore = ScriptRuntimeData.GetExecutionReadyParameterStore())
	{
		int32 IndexOfDataInterface = ParameterStore->GetDataInterfaces().IndexOfByKey(DataInterface);
		if (IndexOfDataInterface != INDEX_NONE)
		{
			TConstArrayView<FNiagaraDataInterfaceGPUParamInfo> ParamInfoArray = Script->GetDataInterfaceGPUParamInfos();
			for (const FNiagaraDataInterfaceGeneratedFunction& Func : ParamInfoArray[IndexOfDataInterface].GeneratedFunctions)
			{
				if (const FName* AttributeName = Func.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute))
				{
					FNiagaraVariableBase NewVar(UNiagaraDataInterfaceGrid3DCollection::GetValueTypeFromFuncName(Func.DefinitionName), *AttributeName);
					if (UNiagaraDataInterfaceGrid3DCollection::CanCreateVarFromFuncName(Func.DefinitionName))
					{
						if (!OutVariables.Contains(NewVar))
						{
							const int32 FoundNameMatch = OutVariables.IndexOfByPredicate([&](const FNiagaraVariableBase& Var) { return Var.GetName() == *AttributeName; });
							if (FoundNameMatch == INDEX_NONE)
							{
								OutVariables.Add(NewVar);
								const int32 NumComponents = NewVar.GetSizeInBytes() / sizeof(float);
								OutVariableOffsets.Add(TotalAttributes);
								TotalAttributes += NumComponents;
							}
							else
							{
								if (OutWarnings)
								{
									FText Warning = FText::Format(NSLOCTEXT("NiagaraDataInterfaceGrid3DCollection", "BadType", "Same name, different types! {0} vs {1}, Attribute {2}"), NewVar.GetType().GetNameText(), OutVariables[FoundNameMatch].GetType().GetNameText(), FText::FromName(NewVar.GetName()));
									OutWarnings->Add(Warning);
								}
							}
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
