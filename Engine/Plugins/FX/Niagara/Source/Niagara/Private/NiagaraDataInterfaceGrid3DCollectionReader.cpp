// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceGrid3DCollectionReader.h"
#include "Engine/TextureRenderTarget.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraDataInterfaceGrid3DCollectionUtils.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraScriptRuntimeData.h"
#include "NiagaraSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceGrid3DCollectionReader)


#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceGrid3DCollectionReader"

UNiagaraDataInterfaceGrid3DCollectionReader::UNiagaraDataInterfaceGrid3DCollectionReader(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyGrid3DCollectionProxy());

	FNiagaraTypeDefinition Def(UTextureRenderTarget::StaticClass());
	RenderTargetUserParameter.Parameter.SetType(Def);
}

bool UNiagaraDataInterfaceGrid3DCollectionReader::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceGrid3DCollectionReader* OtherTyped = CastChecked<const UNiagaraDataInterfaceGrid3DCollectionReader>(Other);

	return OtherTyped != nullptr && OtherTyped->EmitterName == EmitterName && OtherTyped->DIName == DIName;
}

bool UNiagaraDataInterfaceGrid3DCollectionReader::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceGrid3DCollectionReader* OtherTyped = CastChecked<UNiagaraDataInterfaceGrid3DCollectionReader>(Destination);
	OtherTyped->EmitterName = EmitterName;
	OtherTyped->DIName = DIName;
	return true;
}

bool UNiagaraDataInterfaceGrid3DCollectionReader::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	check(Proxy);
	
	FGrid3DCollectionRWInstanceData_GameThread* InstanceData = new (PerInstanceData) FGrid3DCollectionRWInstanceData_GameThread();
	SystemInstancesToProxyData_GT.Emplace(SystemInstance->GetId(), InstanceData);

	FNiagaraEmitterInstance* EmitterInstanceToUse = nullptr;
	for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
	{
		const UNiagaraEmitter* Emitter = EmitterInstance->GetEmitter();
		if (Emitter == nullptr)
		{
			continue;
		}

		if (EmitterName == Emitter->GetUniqueEmitterName())
		{
			EmitterInstanceToUse = &EmitterInstance.Get();
			break;
		}
	}

	// Look for proxy we are going to use
	FNiagaraDataInterfaceProxy* ProxyToUse = nullptr;
	UNiagaraDataInterfaceGrid3DCollection* GridInterface = nullptr;

	if (EmitterInstanceToUse != nullptr)
	{
		FNiagaraComputeExecutionContext* ExecContext = EmitterInstanceToUse->GetGPUContext();
		if (ExecContext != nullptr && ExecContext->GPUScript != nullptr && ExecContext->GetScriptRuntimeData() != nullptr)
		{
			TConstArrayView<FNiagaraScriptResolvedDataInterfaceInfo> DataInterfaceInfo = ExecContext->GetScriptRuntimeData()->GetResolvedDataInterfaces();
			const TArray<UNiagaraDataInterface*>& DataInterfaces = ExecContext->CombinedParamStore.GetDataInterfaces();

			FString FullName = FString("Emitter.") + DIName;
			int Index = 0;

			// #todo(dmp): we are looking at the UObjects that define the DIs here 
			
			for (UNiagaraDataInterface* Interface : DataInterfaces)
			{
				if (DataInterfaceInfo[Index].CompileName.GetPlainNameString() == FullName)
				{
					GridInterface = static_cast<UNiagaraDataInterfaceGrid3DCollection*>(Interface);
					ProxyToUse = Interface->GetProxy();
					break;
				}
				++Index;
			}
		}

		if (GridInterface && ProxyToUse)
		{
			// Does the reader think it should be an rgba grid?
			int32 NumAttribChannelsFound = 0;
			int32 NumNamedAttribChannelsFound = 0;
			TArray<FNiagaraVariableBase> Vars;
			TArray<uint32> Offsets;
			FindAttributes(Vars, Offsets, NumNamedAttribChannelsFound, nullptr);

			// #todo(dmp): slight hack here - these aren't always the same, but we generally can't use unnamed attrs very well
			NumAttribChannelsFound = NumNamedAttribChannelsFound;
			bool UseRGBATexture = FGrid3DCollectionAttributeHlslWriter::ShouldUseRGBAGrid(NumAttribChannelsFound, Vars.Num());

			FGrid3DCollectionRWInstanceData_GameThread* Grid3DInstanceData = GridInterface->GetSystemInstancesToProxyData_GT().FindRef(SystemInstance->GetId());
			InstanceData->OtherDI = GridInterface;
			InstanceData->OtherInstanceData = Grid3DInstanceData;

			// error if the reader thinks it should be an rgba grid and the underlying grid collection disagrees.  This is due to the reader not using all attributes defined
			if (NumAttribChannelsFound > 0 && InstanceData->OtherInstanceData != nullptr && UseRGBATexture != InstanceData->OtherInstanceData->UseRGBATexture)
			{
				UE_LOGF(LogNiagara, Error, "RGBA grids and Grid3DCollectionReaders are not compatible with Emitter %ls and DataInterface %ls.  Turn off RGBA grids with fx.Niagara.Grid3D.UseRGBAGrid 0, or split up your grid collections so the one passed to this reader has only one attribute.", *this->EmitterName, *this->DIName);
				return false;
			}

			// Push Updates to Proxy.
			FNiagaraDataInterfaceProxyGrid3DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid3DCollectionProxy>();
			ENQUEUE_RENDER_COMMAND(FUpdateData)(
				[RT_Proxy, InstanceID = SystemInstance->GetId(), RT_ProxyToUse = ProxyToUse](FRHICommandListImmediate& RHICmdList)
			{
				check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));				

				FGrid3DCollectionRWInstanceData_RenderThread* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.Add(InstanceID);
				TargetData->OtherProxy = RT_ProxyToUse;
			});
			return true;
		}
		else
		{
			UE_LOGF(LogNiagara, Error, "Emitter %ls doesn't have Grid3D Collection DI named %ls for DI: %ls", *this->EmitterName, *this->DIName, *Proxy->GetSourceDIName().ToString());
		}
	}
	else
	{
		UE_LOGF(LogNiagara, Error, "Emitter %ls doesn't exist for DI: %ls", *this->EmitterName, *Proxy->GetSourceDIName().ToString());
	}

	return false;
}

void UNiagaraDataInterfaceGrid3DCollectionReader::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FGrid3DCollectionRWInstanceData_GameThread* PIData = static_cast<FGrid3DCollectionRWInstanceData_GameThread*>(PerInstanceData);
	PIData->~FGrid3DCollectionRWInstanceData_GameThread();

	SystemInstancesToProxyData_GT.Remove(SystemInstance->GetId());

	FNiagaraDataInterfaceProxyGrid3DCollectionProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyGrid3DCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Proxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
		{
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		});
}

void UNiagaraDataInterfaceGrid3DCollectionReader::GetEmitterDependencies(UNiagaraSystem* Asset, TArray<FVersionedNiagaraEmitter>& Dependencies) const
{
	if (!Asset)
	{
		return;
	}

	for (const FNiagaraEmitterHandle& EmitterHandle : Asset->GetEmitterHandles())
	{
		UNiagaraEmitter* EmitterInstance = EmitterHandle.GetInstance().Emitter;
		if (EmitterInstance && EmitterInstance->GetUniqueEmitterName() == EmitterName)
		{
			Dependencies.Add(EmitterHandle.GetInstance());
			return;
		}
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceGrid3DCollectionReader::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	int32 NumFunctionsBefore = OutFunctions.Num();

	Super::GetFunctionsInternal(OutFunctions);
	TSet<FName> FunctionsToRemove = 
	{
		ClearCellFunctionName,
		CopyPreviousToCurrentForCellFunctionName,
		CopyMaskedPreviousToCurrentForCellFunctionName,
		SetValueFunctionName,
		SetFullGridValueFunctionName,
		SetVector4ValueFunctionName,
		SetVector3ValueFunctionName,
		SetVector2ValueFunctionName,
		SetFloatValueFunctionName,		
		SetVectorValueFunctionName,
		SetVector2DValueFunctionName,
		SetNumCellsFunctionName,
		SetNumCellsFunctionName
	};
	
	for (int32 ItemIndex = NumFunctionsBefore; ItemIndex < OutFunctions.Num();)
	{
		if (FunctionsToRemove.Find(OutFunctions[ItemIndex].Name) != nullptr)
		{
			OutFunctions.RemoveAtSwap(ItemIndex, 1);
		}
		else
		{
			++ItemIndex;
		}
	}	
}
#endif

bool UNiagaraDataInterfaceGrid3DCollectionReader::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FGrid3DCollectionRWInstanceData_GameThread* InstanceData = SystemInstancesToProxyData_GT.FindRef(SystemInstance->GetId());
	
	// we can run into the case where depending on the ordering of DI initialization, we might have not been able to grab the other grid's InstanceData
	// in InitPerInstanceData.  If this is the case, we ensure it is correct here.
	if (InstanceData->OtherInstanceData == nullptr && InstanceData->OtherDI)
	{
		UNiagaraDataInterfaceGrid3DCollection*  OtherGridDI = static_cast<UNiagaraDataInterfaceGrid3DCollection*>(InstanceData->OtherDI);
		FGrid3DCollectionRWInstanceData_GameThread* Grid3DInstanceData = OtherGridDI->GetSystemInstancesToProxyData_GT().FindRef(SystemInstance->GetId());

		InstanceData->OtherInstanceData = Grid3DInstanceData;

		// Does the reader think it should be an rgba grid?
		int32 NumAttribChannelsFound = 0;
		int32 NumNamedAttribChannelsFound = 0;
		TArray<FNiagaraVariableBase> Vars;
		TArray<uint32> Offsets;
		FindAttributes(Vars, Offsets, NumNamedAttribChannelsFound, nullptr);

		// #todo(dmp): slight hack here - these aren't always the same, but we generally can't use unnamed attrs very well
		NumAttribChannelsFound = NumNamedAttribChannelsFound;
		bool UseRGBATexture = FGrid3DCollectionAttributeHlslWriter::ShouldUseRGBAGrid(NumAttribChannelsFound, Vars.Num());

		if (InstanceData->OtherInstanceData == nullptr)
		{
			UE_LOGF(LogNiagara, Error, "%ls: The DI for Emitter %ls and DataInterface %ls is not initialized.", *Proxy->GetSourceDIName().ToString(), *this->EmitterName, *this->DIName);
			return false;
		}

		// error if the reader thinks it should be an rgba grid and the underlying grid collection disagrees.  This is due to the reader not using all attributes defined
		if (NumAttribChannelsFound > 0 && UseRGBATexture != InstanceData->OtherInstanceData->UseRGBATexture)
		{
			UE_LOGF(LogNiagara, Error, "RGBA grids and Grid3DCollectionReaders are not compatible with Emitter %ls and DataInterface %ls.  Turn off RGBA grids with fx.Niagara.Grid3D.UseRGBAGrid 0, or split up your grid collections so the one passed to this reader has only one attribute.", *this->EmitterName, *this->DIName);
			return false;
		}

	}

	return false;
}

#undef LOCTEXT_NAMESPACE

