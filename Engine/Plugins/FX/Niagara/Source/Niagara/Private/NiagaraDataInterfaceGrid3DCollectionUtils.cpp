// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceGrid3DCollectionUtils.h"
#include "NiagaraDataInterfaceGrid3DCollection.h"

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceGrid3DCollection"

static int32 GNiagaraGrid3DUseRGBAGrid = 1;
static FAutoConsoleVariableRef CVarNiagaraGrid3DUseRGBAGrid(
	TEXT("fx.Niagara.Grid3D.UseRGBAGrid"),
	GNiagaraGrid3DUseRGBAGrid,
	TEXT("Use RGBA textures when possible\n"),
	ECVF_Default
);

TArray<FString> FGrid3DCollectionAttributeHlslWriter::Channels = { "r", "g", "b", "a" };

// static member function
bool FGrid3DCollectionAttributeHlslWriter::ShouldUseRGBAGrid(const int TotalChannels, const int TotalNumAttributes)
{
	return TotalNumAttributes == 1 && TotalChannels <= 4 && GNiagaraGrid3DUseRGBAGrid != 0;
}

bool FGrid3DCollectionAttributeHlslWriter::SupportsRGBAGrid()
{
	return GNiagaraGrid3DUseRGBAGrid == 1;
}

FGrid3DCollectionAttributeHlslWriter::FGrid3DCollectionAttributeHlslWriter(const FNiagaraDataInterfaceGPUParamInfo& InParamInfo, TArray<FText>* OutWarnings /*= nullptr*/) :
	ParamInfo(InParamInfo), AttributeHelper(InParamInfo)
{
	AttributeHelper.Init<UNiagaraDataInterfaceGrid3DCollection>(OutWarnings);
		}

bool FGrid3DCollectionAttributeHlslWriter::UseRGBAGrid()
		{
	return ShouldUseRGBAGrid(AttributeHelper.GetTotalChannels(), AttributeHelper.GetNumAttributes());
}

#if WITH_EDITORONLY_DATA
FString FGrid3DCollectionAttributeHlslWriter::GetPerAttributePixelOffset(const TCHAR* DataInterfaceHLSLSymbol)
{
	return FString::Printf(TEXT("int3(%s_%s[(AttributeIndex * 2) + 0].xyz)"), DataInterfaceHLSLSymbol, *UNiagaraDataInterfaceGrid3DCollection::PerAttributeDataName);
}

FString FGrid3DCollectionAttributeHlslWriter::GetPerAttributePixelOffset() const
{
	return GetPerAttributePixelOffset(*ParamInfo.DataInterfaceHLSLSymbol);
}

FString FGrid3DCollectionAttributeHlslWriter::GetPerAttributeUVWOffset(const TCHAR* DataInterfaceHLSLSymbol)
{
	return FString::Printf(TEXT("%s%s[(AttributeIndex * 2) + 1].xyz"), *UNiagaraDataInterfaceGrid3DCollection::PerAttributeDataName, DataInterfaceHLSLSymbol);
}

FString FGrid3DCollectionAttributeHlslWriter::GetPerAttributeUVWOffset() const
{
	return GetPerAttributeUVWOffset(*ParamInfo.DataInterfaceHLSLSymbol);
}

bool FGrid3DCollectionAttributeHlslWriter::WriteGetHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, FString& OutHLSL)
{
	const FName* AttributeName = FunctionInfo.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute);
	if (AttributeName == nullptr)
	{
		return false;
	}

	const FNiagaraDataInterfaceRWAttributeHelper::FAttributeInfo* AttributeInfo = AttributeHelper.FindAttributeInfo(*AttributeName);
	if (AttributeInfo == nullptr)
	{
		return false;
	}

	const FString GridNameHLSL = ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceGrid3DCollection::GridName;

	if (AttributeStorage == EGridAttributeRetrievalMode::RGBAGrid)
	{

		FString NumChannelsString;
		FString AttrGridChannels;

		AttributeHelper.GetChannelStrings(AttributeInfo, NumChannelsString, AttrGridChannels);


		OutHLSL.Appendf(TEXT("void %s(int IndexX, int IndexY, int IndexZ, out float%s Value)\n"), *FunctionInfo.InstanceName, *NumChannelsString);
		OutHLSL.Appendf(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	Value = %s.Load(int4(IndexX, IndexY, IndexZ, 0)).%s;\n"), *GridNameHLSL, *AttrGridChannels);
		OutHLSL.Appendf(TEXT("}\n"));
	}
	else
	{
		if (AttributeInfo->NumChannels == 1)
		{
			OutHLSL.Appendf(TEXT("void %s(int IndexX, int IndexY, int IndexZ, out float Value)\n"), *FunctionInfo.InstanceName);
			OutHLSL.Appendf(TEXT("{\n"));
			OutHLSL.Appendf(TEXT("	int z %s;\n"), *AttributeHelper.GetAttributeIndex(AttributeInfo));
			OutHLSL.Appendf(TEXT("	int3 PixelOffset = int3(IndexX, IndexY, IndexZ) + %s;\n"), *GetPerAttributePixelOffset());
			OutHLSL.Appendf(TEXT("	Value = %s.Load(int4(PixelOffset, 0));\n"), *GridNameHLSL);
			OutHLSL.Appendf(TEXT("}\n"));
		}
		else
		{
			OutHLSL.Appendf(TEXT("void %s(int IndexX, int IndexY, int IndexZ, out float%d Value)\n"), *FunctionInfo.InstanceName, AttributeInfo->NumChannels);
			OutHLSL.Appendf(TEXT("{\n"));
			for (int32 i = 0; i < AttributeInfo->NumChannels; ++i)
			{
				OutHLSL.Appendf(TEXT("	{\n"));
				OutHLSL.Appendf(TEXT("		int AttributeIndex = %s;\n"), *AttributeHelper.GetAttributeIndex(AttributeInfo, i));
				OutHLSL.Appendf(TEXT("		int3 PixelOffset = int3(IndexX, IndexY, IndexZ) + %s;\n"), *GetPerAttributePixelOffset());
				OutHLSL.Appendf(TEXT("		Value[%d] = %s.Load(int4(PixelOffset, 0));\n"), i, *GridNameHLSL);
				OutHLSL.Appendf(TEXT("	}\n"));
			}
			OutHLSL.Appendf(TEXT("}\n"));
		}
	}

	return true;
}

bool FGrid3DCollectionAttributeHlslWriter::WriteGetOutputGridHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, FString& OutHLSL)
{
	const FName* AttributeName = FunctionInfo.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute);
	if (AttributeName == nullptr)
	{
		return false;
	}

	const FNiagaraDataInterfaceRWAttributeHelper::FAttributeInfo* AttributeInfo = AttributeHelper.FindAttributeInfo(*AttributeName);
	if (AttributeInfo == nullptr)
	{
		return false;
	}

	// Read from OutputGrid UAV instead of Grid SRV
	const FString OutputGridNameHLSL = ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceGrid3DCollection::OutputGridName;

	if (AttributeStorage == EGridAttributeRetrievalMode::RGBAGrid)
	{
		FString NumChannelsString;
		FString AttrGridChannels;

		AttributeHelper.GetChannelStrings(AttributeInfo, NumChannelsString, AttrGridChannels);

		OutHLSL.Appendf(TEXT("void %s(int IndexX, int IndexY, int IndexZ, out float%s Value)\n"), *FunctionInfo.InstanceName, *NumChannelsString);
		OutHLSL.Appendf(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	Value = %s[int3(IndexX, IndexY, IndexZ)].%s;\n"), *OutputGridNameHLSL, *AttrGridChannels);
		OutHLSL.Appendf(TEXT("}\n"));
	}
	else
	{
		if (AttributeInfo->NumChannels == 1)
		{
			OutHLSL.Appendf(TEXT("void %s(int IndexX, int IndexY, int IndexZ, out float Value)\n"), *FunctionInfo.InstanceName);
			OutHLSL.Appendf(TEXT("{\n"));
			OutHLSL.Appendf(TEXT("	int AttributeIndex = %s;\n"), *AttributeHelper.GetAttributeIndex(AttributeInfo));
			OutHLSL.Appendf(TEXT("	int3 PixelOffset = int3(IndexX, IndexY, IndexZ) + %s;\n"), *GetPerAttributePixelOffset());
			OutHLSL.Appendf(TEXT("	Value = %s[PixelOffset];\n"), *OutputGridNameHLSL);
			OutHLSL.Appendf(TEXT("}\n"));
		}
		else
		{
			OutHLSL.Appendf(TEXT("void %s(int IndexX, int IndexY, int IndexZ, out float%d Value)\n"), *FunctionInfo.InstanceName, AttributeInfo->NumChannels);
			OutHLSL.Appendf(TEXT("{\n"));
			for (int32 i = 0; i < AttributeInfo->NumChannels; ++i)
			{
				OutHLSL.Appendf(TEXT("	{\n"));
				OutHLSL.Appendf(TEXT("		int AttributeIndex = %s;\n"), *AttributeHelper.GetAttributeIndex(AttributeInfo, i));
				OutHLSL.Appendf(TEXT("		int3 PixelOffset = int3(IndexX, IndexY, IndexZ) + %s;\n"), *GetPerAttributePixelOffset());
				OutHLSL.Appendf(TEXT("		Value[%d] = %s[PixelOffset];\n"), i, *OutputGridNameHLSL);
				OutHLSL.Appendf(TEXT("	}\n"));
			}
			OutHLSL.Appendf(TEXT("}\n"));
		}
	}

	return true;
}

bool FGrid3DCollectionAttributeHlslWriter::WriteGetAtIndexHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int NumChannels, FString& OutHLSL)
{
	FString NumChannelsString;
	FString AttrGridChannels;

	const FString GridNameHLSL = ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceGrid3DCollection::GridName;

	AttributeHelper.GetChannelStrings(0, NumChannels, NumChannelsString, AttrGridChannels);

	if (AttributeHelper.GetNumAttributes() > 1)
	{
		//#todo(dmp): fill this in
		return false;
	}
	else
	{
		OutHLSL.Appendf(TEXT("void %s(int IndexX, int IndexY, int IndexZ, int AttributeIndex, out float%s Value)\n"), *FunctionInfo.InstanceName, *NumChannelsString);
		OutHLSL.Appendf(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	Value = %s.Load(int4(IndexX, IndexY, IndexZ, 0)).%s;\n"), *GridNameHLSL, *AttrGridChannels);
		OutHLSL.Appendf(TEXT("}\n"));
	}

	return true;
}

bool FGrid3DCollectionAttributeHlslWriter::WriteSetAtIndexHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int NumChannels, FString& OutHLSL)
{
	FString NumChannelsString;
	FString AttrGridChannels;

	const FString OutputGridNameHLSL = ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceGrid3DCollection::OutputGridName;

	AttributeHelper.GetChannelStrings(0, NumChannels, NumChannelsString, AttrGridChannels);

	OutHLSL.Appendf(TEXT("void %s(int IndexX, int IndexY, int IndexZ, int AttributeIndex, float%s Value)\n"), *FunctionInfo.InstanceName, *NumChannelsString);
	OutHLSL.Appendf(TEXT("{\n"));

	// more than 1 attribute, we must read first
	if (AttributeHelper.GetNumAttributes() > 1)
	{
		//#todo(dmp): fill this in
		return false;
	}
	else
	{
		OutHLSL.Appendf(TEXT("	%s[float3(IndexX, IndexY, IndexZ)].%s = Value;\n"), *OutputGridNameHLSL, *AttrGridChannels);
	}
	OutHLSL.Appendf(TEXT("}\n"));

	return true;
}

bool FGrid3DCollectionAttributeHlslWriter::WriteSampleAtIndexHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int NumChannels, bool IsCubic, FString& OutHLSL)
{
	FString NumChannelsString;
	FString AttrGridChannels;

	const FString GridNameHLSL = ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceGrid3DCollection::GridName;
	const FString SamplerNameHLSL = ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceGrid3DCollection::SamplerName;

	AttributeHelper.GetChannelStrings(0, NumChannels, NumChannelsString, AttrGridChannels);

	if (AttributeHelper.GetNumAttributes() > 1)
	{
		//#todo(dmp): fill this in
		return false;
	}
	else
	{
		OutHLSL.Appendf(TEXT("void %s(float3 Unit, int AttributeIndex, out float%s Value)\n"), *FunctionInfo.InstanceName, *NumChannelsString);
		OutHLSL.Appendf(TEXT("{\n"));
		if (IsCubic)
		{
			OutHLSL.Appendf(TEXT("	Value = SampleTriCubicLagrange_%s(%s, Unit, 0).%s;\n"), *ParamInfo.DataInterfaceHLSLSymbol, *SamplerNameHLSL, *AttrGridChannels);
		}
		else
		{
			OutHLSL.Appendf(TEXT("	Value = %s.SampleLevel(%s, Unit, 0).%s;\n"), *GridNameHLSL, *SamplerNameHLSL, *AttrGridChannels);
		}

		OutHLSL.Appendf(TEXT("}\n"));
	}

	return true;
}

bool FGrid3DCollectionAttributeHlslWriter::WriteSetHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, FString& OutHLSL)
{
	const FName* AttributeName = FunctionInfo.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute);
	if (AttributeName == nullptr)
	{
		return false;
	}

	const FNiagaraDataInterfaceRWAttributeHelper::FAttributeInfo* AttributeInfo = AttributeHelper.FindAttributeInfo(*AttributeName);
	if (AttributeInfo == nullptr)
	{
		return false;
	}

	const FString OutputGridNameHLSL = ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceGrid3DCollection::OutputGridName;

	if (AttributeStorage == EGridAttributeRetrievalMode::RGBAGrid)
	{
		FString NumChannelsString;
		FString AttrGridChannels;

		AttributeHelper.GetChannelStrings(AttributeInfo, NumChannelsString, AttrGridChannels);

		OutHLSL.Appendf(TEXT("void %s(int IndexX, int IndexY, int IndexZ, float%s Value)\n"), *FunctionInfo.InstanceName, *NumChannelsString);
		OutHLSL.Appendf(TEXT("{\n"));

		// more than 1 attribute, we must read first
		if (AttributeHelper.GetNumAttributes() > 1)
		{
			FString GridNumChannels = FString::FromInt(AttributeHelper.GetTotalChannels());

			OutHLSL.Appendf(TEXT("	float%s TmpValue = %s.Load(int4(IndexX, IndexY, IndexZ, 0));\n"), *GridNumChannels, *OutputGridNameHLSL);
			OutHLSL.Appendf(TEXT("	TmpValue.%s = Value;\n"), *AttrGridChannels);
			OutHLSL.Appendf(TEXT("	%s[float3(IndexX, IndexY, IndexZ)] = TmpValue;\n"), *OutputGridNameHLSL);
		}
		else
		{
			OutHLSL.Appendf(TEXT("	%s[float3(IndexX, IndexY, IndexZ)].%s = Value;\n"), *OutputGridNameHLSL, *AttrGridChannels);
		}
		OutHLSL.Appendf(TEXT("}\n"));
	}
	else
	{
		if (AttributeInfo->NumChannels == 1)
		{
			OutHLSL.Appendf(TEXT("void %s(int IndexX, int IndexY, int IndexZ, float Value)\n"), *FunctionInfo.InstanceName);
			OutHLSL.Appendf(TEXT("{\n"));
			OutHLSL.Appendf(TEXT("	int AttributeIndex = %s;\n"), *AttributeHelper.GetAttributeIndex(AttributeInfo));
			OutHLSL.Appendf(TEXT("	int3 PixelOffset = int3(IndexX, IndexY, IndexZ) + %s;\n"), *GetPerAttributePixelOffset());
			OutHLSL.Appendf(TEXT("	%s[PixelOffset] = Value;\n"), *OutputGridNameHLSL);
			OutHLSL.Appendf(TEXT("}\n"));
		}
		else
		{
			OutHLSL.Appendf(TEXT("void %s(int IndexX, int IndexY, int IndexZ, float%d Value)\n"), *FunctionInfo.InstanceName, AttributeInfo->NumChannels);
			OutHLSL.Appendf(TEXT("{\n"));
			for (int32 i = 0; i < AttributeInfo->NumChannels; ++i)
			{
				OutHLSL.Appendf(TEXT("	{\n"));
				OutHLSL.Appendf(TEXT("		int AttributeIndex = %s;\n"), *AttributeHelper.GetAttributeIndex(AttributeInfo, i));
				OutHLSL.Appendf(TEXT("		int3 PixelOffset = int3(IndexX, IndexY, IndexZ) + %s;\n"), *GetPerAttributePixelOffset());
				OutHLSL.Appendf(TEXT("		%s[PixelOffset] = Value[%d];\n"), *OutputGridNameHLSL, i);
				OutHLSL.Appendf(TEXT("	}\n"));
			}
			OutHLSL.Appendf(TEXT("}\n"));
		}
	}
	return true;
}

bool FGrid3DCollectionAttributeHlslWriter::WriteSampleHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, bool IsCubic, FString& OutHLSL)
{
	const FName* AttributeName = FunctionInfo.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute);
	if (AttributeName == nullptr)
	{
		return false;
	}

	const FNiagaraDataInterfaceRWAttributeHelper::FAttributeInfo* AttributeInfo = AttributeHelper.FindAttributeInfo(*AttributeName);
	if (AttributeInfo == nullptr)
	{
		return false;
	}

	const FString GridNameHLSL = ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceGrid3DCollection::GridName;
	const FString SamplerNameHLSL = ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceGrid3DCollection::SamplerName;

	if (AttributeStorage == EGridAttributeRetrievalMode::RGBAGrid)
	{
		FString NumChannelsString;
		FString AttrGridChannels;
		AttributeHelper.GetChannelStrings(AttributeInfo, NumChannelsString, AttrGridChannels);

		OutHLSL.Appendf(TEXT("void %s(float3 Unit, out float%s Value)\n"), *FunctionInfo.InstanceName, *NumChannelsString);
		OutHLSL.Appendf(TEXT("{\n"));
		if (IsCubic)
		{
			OutHLSL.Appendf(TEXT("	Value = SampleTriCubicLagrange_%s(%s, Unit, 0).%s;\n"), *ParamInfo.DataInterfaceHLSLSymbol, *SamplerNameHLSL, *AttrGridChannels);
		}
		else
		{
			OutHLSL.Appendf(TEXT("	Value = %s.SampleLevel(%s, Unit, 0).%s;\n"), *GridNameHLSL, *SamplerNameHLSL, *AttrGridChannels);
		}

		OutHLSL.Appendf(TEXT("}\n"));
	}
	else
	{
		const FString NumAttributesNameHLSL = UNiagaraDataInterfaceRWBase::NumAttributesName + ParamInfo.DataInterfaceHLSLSymbol;
		if (AttributeInfo->NumChannels == 1)
		{
			OutHLSL.Appendf(TEXT("void %s(float3 Unit, out float Value)\n"), *FunctionInfo.InstanceName);
			OutHLSL.Appendf(TEXT("{\n"));
			OutHLSL.Appendf(TEXT("	float3 TileUVW = clamp(Unit, %s%s, %s%s) * %s%s;\n"), *UNiagaraDataInterfaceGrid3DCollection::UnitClampMinName, *ParamInfo.DataInterfaceHLSLSymbol, *UNiagaraDataInterfaceGrid3DCollection::UnitClampMaxName, *ParamInfo.DataInterfaceHLSLSymbol, *UNiagaraDataInterfaceGrid3DCollection::OneOverNumTilesName, *ParamInfo.DataInterfaceHLSLSymbol);
			OutHLSL.Appendf(TEXT("	int AttributeIndex = %s;\n"), *AttributeHelper.GetAttributeIndex(AttributeInfo));
			OutHLSL.Appendf(TEXT("	float3 UVW = TileUVW + %s;\n"), *GetPerAttributeUVWOffset());
			if (IsCubic)
			{
				OutHLSL.Appendf(TEXT("	Value = SampleTriCubicLagrange(%s, Unit, 0).%s;\n"), *ParamInfo.DataInterfaceHLSLSymbol, *SamplerNameHLSL);
			}
			else
			{
				OutHLSL.Appendf(TEXT("	Value = %s.SampleLevel(%s, UVW, 0);\n"), *GridNameHLSL, *SamplerNameHLSL);
			}
			OutHLSL.Appendf(TEXT("}\n"));
		}
		else
		{
			OutHLSL.Appendf(TEXT("void %s(float3 Unit, out float%d Value)\n"), *FunctionInfo.InstanceName, AttributeInfo->NumChannels);
			OutHLSL.Appendf(TEXT("{\n"));
			OutHLSL.Appendf(TEXT("	float3 TileUVW = clamp(Unit, %s%s, %s%s) * %s%s;\n"), *UNiagaraDataInterfaceGrid3DCollection::UnitClampMinName, *ParamInfo.DataInterfaceHLSLSymbol, *UNiagaraDataInterfaceGrid3DCollection::UnitClampMaxName, *ParamInfo.DataInterfaceHLSLSymbol, *UNiagaraDataInterfaceGrid3DCollection::OneOverNumTilesName, *ParamInfo.DataInterfaceHLSLSymbol);
			for (int32 i = 0; i < AttributeInfo->NumChannels; ++i)
			{
				OutHLSL.Appendf(TEXT("	{\n"));
				OutHLSL.Appendf(TEXT("		int AttributeIndex = %s;\n"), *AttributeHelper.GetAttributeIndex(AttributeInfo, i));
				OutHLSL.Appendf(TEXT("		float3 UVW = TileUVW + %s;\n"), *GetPerAttributeUVWOffset());
				if (IsCubic)
				{
					OutHLSL.Appendf(TEXT("	Value = SampleTriCubicLagrange(%s, Unit, 0).%s;\n"), *ParamInfo.DataInterfaceHLSLSymbol, *SamplerNameHLSL);
				}
				else
				{
					OutHLSL.Appendf(TEXT("Value[%d] = %s.SampleLevel(%s, UVW, 0);\n"), i, *GridNameHLSL, *SamplerNameHLSL);
				}

				OutHLSL.Appendf(TEXT("	}\n"));
			}
			OutHLSL.Appendf(TEXT("}\n"));
		}
	}
	return true;
}

bool FGrid3DCollectionAttributeHlslWriter::WriteAttributeGetIndexHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, FString& OutHLSL)
{
	const FName* AttributeName = FunctionInfo.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute);
	if (AttributeName == nullptr)
	{
		return false;
	}

	const FNiagaraDataInterfaceRWAttributeHelper::FAttributeInfo* AttributeInfo = AttributeHelper.FindAttributeInfo(*AttributeName);
	if (AttributeInfo == nullptr)
	{
		return false;
	}
	
	// #todo(dmp): for now it is ok to assume rgba grids only store 1 attribute per grid, so
	// attributes are always stored at index 0.  In the future we might have a way to support
	// multiple attributes per rgba grid, and this will have to change
	if (AttributeStorage == EGridAttributeRetrievalMode::RGBAGrid)
	{
		OutHLSL.Appendf(TEXT("void %s(out int Value)\n"), *FunctionInfo.InstanceName);
		OutHLSL.Appendf(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	Value = 0;\n"));
		OutHLSL.Appendf(TEXT("}\n"));
	}
	else
	{	
		OutHLSL.Appendf(TEXT("void %s(out int Value)\n"), *FunctionInfo.InstanceName);
		OutHLSL.Appendf(TEXT("{\n"));
		OutHLSL.Appendf(TEXT("	Value = %s;\n"), *AttributeHelper.GetAttributeIndex(AttributeInfo));
		OutHLSL.Appendf(TEXT("}\n"));
	}

	return true;
}
#endif //WITH_EDITORONLY_DATA

#undef LOCTEXT_NAMESPACE