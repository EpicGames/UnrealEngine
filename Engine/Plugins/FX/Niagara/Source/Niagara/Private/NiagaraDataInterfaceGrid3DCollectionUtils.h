// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraDataInterfaceRWUtils.h"

enum EGridAttributeRetrievalMode
{
	RGBAGrid = 0,
	Indirection,
	NoIndirection
};

struct FGrid3DCollectionAttributeHlslWriter
{

	// only support rgba textures when we have a single attribute that contains up to 4 channels
	static bool ShouldUseRGBAGrid(const int TotalChannels, const int TotalNumAttributes);
	static bool SupportsRGBAGrid();

	static TArray<FString> Channels;

	explicit FGrid3DCollectionAttributeHlslWriter(const FNiagaraDataInterfaceGPUParamInfo& InParamInfo, TArray<FText>* OutWarnings = nullptr);

	const FNiagaraDataInterfaceRWAttributeHelper& GetAttributeHelper() { return AttributeHelper; }

	bool UseRGBAGrid();

#if WITH_EDITORONLY_DATA
	// Returns pixel offset for the channel
	static FString GetPerAttributePixelOffset(const TCHAR* DataInterfaceHLSLSymbol);
	FString GetPerAttributePixelOffset() const;
	
	// Returns UVW offset for the channel
	static FString GetPerAttributeUVWOffset(const TCHAR* DataInterfaceHLSLSymbol);
	FString GetPerAttributeUVWOffset() const;

	bool WriteGetHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, FString& OutHLSL);
	bool WriteGetOutputGridHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, FString& OutHLSL);
	bool WriteGetAtIndexHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int NumChannels, FString& OutHLSL);
	bool WriteSetAtIndexHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int NumChannels, FString& OutHLSL);
	bool WriteSampleAtIndexHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int NumChannels, bool IsCubic, FString& OutHLSL);
	bool WriteSetHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, FString& OutHLSL);
	bool WriteSampleHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, bool IsCubic, FString& OutHLSL);
	bool WriteAttributeGetIndexHLSL(EGridAttributeRetrievalMode AttributeStorage, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, FString& OutHLSL);
#endif //WITH_EDITORONLY_DATA

	const FNiagaraDataInterfaceGPUParamInfo& ParamInfo;
	FNiagaraDataInterfaceRWAttributeHelper				AttributeHelper;	
};
