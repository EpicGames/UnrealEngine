// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceIntRenderTarget2D.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraDataInterfaceRenderTargetCommon.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraGpuComputeDebugInterface.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraStats.h"

#include "TextureResource.h"
#include "ShaderCompilerCore.h"
#include "CanvasTypes.h"
#include "Engine/TextureRenderTarget2D.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceIntRenderTarget2D)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceIntRenderTarget"

namespace NDIIntRenderTarget2DLocal
{
	struct FFunctionVersion
	{
		enum Type
		{
			InitialVersion = 0,
			AddedOptionalExecute = 1,

			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};
	};

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceIntRenderTarget2D.ush");

	static const int32 NumFunctions = 12;
	static const FName GetValueFunctionName("GetValue");
	static const FName SetValueFunctionName("SetValue");
	static const FName AtomicAddFunctionName("AtomicAdd");
	//static const FName AtomicAndFunctionName("AtomicAnd");
	static const FName AtomicCASFunctionName("AtomicCompareAndExchange");
	//static const FName AtomicCSFunctionName("AtomicCompareStore");
	//static const FName AtomicExchangeFunctionName("AtomicExchange");
	static const FName AtomicMaxFunctionName("AtomicMax");
	static const FName AtomicMinFunctionName("AtomicMin");
	//static const FName AtomicOrFunctionName("AtomicOr");
	//static const FName AtomicXorunctionName("AtomicXor");

	static const FName GetSizeFunctionName("GetRenderTargetSize");
	static const FName SetSizeFunctionName("SetRenderTargetSize");

	static const FName LinearToIndexFunctionName("LinearToIndex");
	static const FName LinearToUVFunctionName("LinearToUV");

	static const FName ExecToIndexFunctionName("ExecToIndex");
	static const FName ExecToUVFunctionName("ExecToUV");

	// Shader parameters
	static const FString TextureSizeAndInvSizeName(TEXT("TextureSizeAndInvSize_"));
	static const FString TextureUAVName(TEXT("TextureUAV_"));
}

//////////////////////////////////////////////////////////////////////////

struct FNDIIntRenderTarget2DInstanceData_GameThread
{
	FIntVector2 Size = FIntVector2(EForceInit::ForceInitToZero);
	EPixelFormat Format = EPixelFormat::PF_R32_SINT;
#if WITH_EDITORONLY_DATA
	bool bPreviewRenderTarget = false;
	FVector2D PreviewDisplayRange = FVector2D(0.0f, 255.0f);
#endif

	bool bManagedTexture = false;
	UTextureRenderTarget2D* TargetTexture = nullptr;
	FNiagaraParameterDirectBinding<UObject*> RTUserParamBinding;
};

//////////////////////////////////////////////////////////////////////////

struct FNDIIntRenderTarget2DInstanceData_RenderThread
{
	FIntVector2 Size = FIntVector2(EForceInit::ForceInitToZero);
#if WITH_EDITORONLY_DATA
	bool bPreviewRenderTarget = false;
	FVector2D PreviewDisplayRange = FVector2D(0.0f, 255.0f);
#endif
	bool bWroteThisFrame = false;
	bool bReadThisFrame = false;

	FTextureRHIRef		TextureRHI;

	FRDGTextureRef		TransientRDGTexture = nullptr;
	FRDGTextureUAVRef	TransientRDGUAV = nullptr;

#if STATS
	void UpdateMemoryStats()
	{
		DEC_MEMORY_STAT_BY(STAT_NiagaraRenderTargetMemory, MemorySize);

		MemorySize = 0;
		if (FRHITexture* RHITexture = TextureRHI)
		{
			MemorySize = RHIComputeMemorySize(RHITexture);
		}

		INC_MEMORY_STAT_BY(STAT_NiagaraRenderTargetMemory, MemorySize);
	}
	uint64 MemorySize = 0;
#endif
};

//////////////////////////////////////////////////////////////////////////

struct FNDIIntRenderTarget2DProxy : public FNiagaraDataInterfaceProxyRW
{
	FNDIIntRenderTarget2DProxy() {}
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override {}
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override
	{
		FNDIIntRenderTarget2DInstanceData_RenderThread& InstanceData = SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());

		// We only need to transfer this frame if it was written to.
		// If also read then we need to notify that the texture is important for the simulation
		// We also assume the texture is important for rendering, without discovering renderer bindings we don't really know
		if (InstanceData.bWroteThisFrame)
		{
			Context.GetComputeDispatchInterface().MultiGPUResourceModified(Context.GetGraphBuilder(), InstanceData.TextureRHI, InstanceData.bReadThisFrame, true);
		}

		InstanceData.bWroteThisFrame = true;
		InstanceData.bReadThisFrame = true;

#if NIAGARA_COMPUTEDEBUG_ENABLED && WITH_EDITORONLY_DATA
		if (InstanceData.bPreviewRenderTarget && InstanceData.TransientRDGTexture)
		{
			FNiagaraGpuComputeDebugInterface GpuComputeDebugInterface = Context.GetComputeDispatchInterface().GetGpuComputeDebugInterface();
			GpuComputeDebugInterface.AddTexture(Context.GetGraphBuilder(), Context.GetSystemInstanceID(), GetSourceDIName(), InstanceData.TransientRDGTexture, InstanceData.PreviewDisplayRange);
		}
#endif
		if (Context.IsFinalPostSimulate())
		{
			InstanceData.TransientRDGTexture = nullptr;
			InstanceData.TransientRDGUAV = nullptr;
		}
	}

	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override
	{
		if (const FNDIIntRenderTarget2DInstanceData_RenderThread* InstanceData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()))
		{
			Context.SetDirect(InstanceData->Size);
		}
	}

	TMap<FNiagaraSystemInstanceID, FNDIIntRenderTarget2DInstanceData_RenderThread> SystemInstancesToProxyData_RT;
};

//////////////////////////////////////////////////////////////////////////

FNiagaraVariableBase UNiagaraDataInterfaceIntRenderTarget2D::ExposedRTVar;

UNiagaraDataInterfaceIntRenderTarget2D::UNiagaraDataInterfaceIntRenderTarget2D(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNDIIntRenderTarget2DProxy());

	FNiagaraTypeDefinition Def(UTextureRenderTarget::StaticClass());
	RenderTargetUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceIntRenderTarget2D::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);

		ExposedRTVar = FNiagaraVariableBase(FNiagaraTypeDefinition(UTexture::StaticClass()), TEXT("RenderTarget"));
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceIntRenderTarget2D::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	Super::GetFunctionsInternal(OutFunctions);

	const int32 EmitterSystemOnlyBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
	OutFunctions.Reserve(OutFunctions.Num() + NDIIntRenderTarget2DLocal::NumFunctions);

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::GetValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = false;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = LOCTEXT("GetValueDesc", "Gets the value from the render target at the pixel offset");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::SetValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = LOCTEXT("SetValueDesc", "Sets the value on the render target at the pixel offset.");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::AtomicAddFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Amount")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("CurrentValue")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PreviousValue")));
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = LOCTEXT("AtomicAddDesc", "Atomic min the value to the pixel at the offset, returns the current & previous values.  This opertion is thread safe.");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}
	////static const FName AtomicAndFunctionName("AtomicAnd");
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::AtomicCASFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Comparison Value")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Original Value")));
		Sig.bHidden = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = LOCTEXT("AtomicCASFunctionDesc", "Compares the pixel value against the comparison value, if they are equal the value is replaced.  Original Value is the pixel value before the operation completes.  This opertion is thread safe.");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}
	////static const FName AtomicCSFunctionName("AtomicCompareStore");
	////static const FName AtomicExchangeFunctionName("AtomicExchange");
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::AtomicMaxFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("CurrentValue")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PreviousValue")));
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = LOCTEXT("AtomicMaxDesc", "Atomic max the value to the pixel at the offset, returns the current & previous values.  This opertion is thread safe.");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::AtomicMinFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("CurrentValue")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PreviousValue")));
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = LOCTEXT("AtomicMinDesc", "Atomic min the value to the pixel at the offset, returns the current & previous values.  This opertion is thread safe.");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}
	////static const FName AtomicOrFunctionName("AtomicOr");
	////static const FName AtomicXorunctionName("AtomicXor");

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::GetSizeFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Width")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Height")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = true;
		Sig.Description = LOCTEXT("GetSizeDesc", "Gets the size of the rendertarget");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::SetSizeFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Width")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Height")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));
		Sig.ModuleUsageBitmask = EmitterSystemOnlyBitmask;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		Sig.Description = LOCTEXT("SetSizeDesc", "Sets the size of the rendertarget");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::LinearToIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelX")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelY")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = LOCTEXT("LinearToIndexDesc", "Converts a linear index into a pixel coordinate");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::LinearToUVFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = LOCTEXT("LinearToUVDesc", "Converts a linear index into a UV coordinate");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::ExecToIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelX")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PixelY")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = LOCTEXT("ExecToIndexDesc", "Returns the execution index as a pixel coordinate");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIIntRenderTarget2DLocal::ExecToUVFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = LOCTEXT("ExecToUVDesc", "Returns the execution index as a UV coordinate");
		Sig.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;
	}
}

bool UNiagaraDataInterfaceIntRenderTarget2D::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	bool bWasChanged = false;

	if (FunctionSignature.FunctionVersion < NDIIntRenderTarget2DLocal::FFunctionVersion::AddedOptionalExecute)
	{
		if (FunctionSignature.Name == NDIIntRenderTarget2DLocal::SetValueFunctionName)
		{
			check(FunctionSignature.Inputs.Num() == 4);
			FunctionSignature.Inputs.Insert_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute")), 1).SetValue(true);
			bWasChanged = true;
		}
	}

	// Set latest version
	FunctionSignature.FunctionVersion = NDIIntRenderTarget2DLocal::FFunctionVersion::LatestVersion;

	return bWasChanged;
}
#endif

void UNiagaraDataInterfaceIntRenderTarget2D::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	Super::GetVMExternalFunction(BindingInfo, InstanceData, OutFunc);
	if (BindingInfo.Name == NDIIntRenderTarget2DLocal::GetSizeFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateLambda([this](FVectorVMExternalFunctionContext& Context) { VMGetSize(Context); });
	}
	else if (BindingInfo.Name == NDIIntRenderTarget2DLocal::SetSizeFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateLambda([this](FVectorVMExternalFunctionContext& Context) { VMSetSize(Context); });
	}
}

bool UNiagaraDataInterfaceIntRenderTarget2D::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	
	const UNiagaraDataInterfaceIntRenderTarget2D* OtherTyped = CastChecked<const UNiagaraDataInterfaceIntRenderTarget2D>(Other);
	return
		OtherTyped != nullptr &&
		OtherTyped->Size == Size &&
#if WITH_EDITORONLY_DATA
		OtherTyped->bPreviewRenderTarget == bPreviewRenderTarget &&
		OtherTyped->PreviewDisplayRange == PreviewDisplayRange &&
#endif
		OtherTyped->RenderTargetUserParameter == RenderTargetUserParameter;
}

bool UNiagaraDataInterfaceIntRenderTarget2D::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceIntRenderTarget2D* DestinationTyped = CastChecked<UNiagaraDataInterfaceIntRenderTarget2D>(Destination);
	if (!DestinationTyped)
	{
		return false;
	}

	DestinationTyped->Size = Size;
#if WITH_EDITORONLY_DATA
	DestinationTyped->bPreviewRenderTarget = bPreviewRenderTarget;
	DestinationTyped->PreviewDisplayRange = PreviewDisplayRange;
#endif
	DestinationTyped->RenderTargetUserParameter = RenderTargetUserParameter;
	return true;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceIntRenderTarget2D::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIIntRenderTarget2DLocal::TemplateShaderFile);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceIntRenderTarget2D::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};

	AppendTemplateHLSL(OutHLSL, NDIIntRenderTarget2DLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceIntRenderTarget2D::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if (
		(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::GetValueFunctionName) ||
		(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::SetValueFunctionName) ||
		(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::AtomicAddFunctionName) ||
		//(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::AtomicAndFunctionName) ||
		(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::AtomicCASFunctionName) ||
		//(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::AtomicCSFunctionName) ||
		//(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::AtomicExchangeFunctionName) ||
		(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::AtomicMaxFunctionName) ||
		(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::AtomicMinFunctionName) ||
		//(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::AtomicOrFunctionName) ||
		//(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::AtomicXorunctionName) ||
		(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::GetSizeFunctionName) ||
		//(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::SetSizeFunctionName) ||
		(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::LinearToIndexFunctionName) ||
		(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::LinearToUVFunctionName) ||
		(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::ExecToIndexFunctionName) ||
		(FunctionInfo.DefinitionName == NDIIntRenderTarget2DLocal::ExecToUVFunctionName) )
	{
		return true;
	}
	return false;
}
#endif

void UNiagaraDataInterfaceIntRenderTarget2D::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceIntRenderTarget2D::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	check(IsInRenderingThread());

	FNDIIntRenderTarget2DProxy& DataInterfaceProxy = Context.GetProxy<FNDIIntRenderTarget2DProxy>();
	FNDIIntRenderTarget2DInstanceData_RenderThread& InstanceData = DataInterfaceProxy.SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());

	// Set Parameters
	FShaderParameters* Parameters = Context.GetParameterNestedStruct<FShaderParameters>();
	Parameters->TextureSizeAndInvSize.X = float(InstanceData.Size.X);
	Parameters->TextureSizeAndInvSize.Y = float(InstanceData.Size.Y);
	Parameters->TextureSizeAndInvSize.Z = 1.0f / float(InstanceData.Size.X);
	Parameters->TextureSizeAndInvSize.W = 1.0f / float(InstanceData.Size.Y);

	if (Context.IsResourceBound(&Parameters->RWTextureUAV))
	{
		InstanceData.bWroteThisFrame = true;
		InstanceData.bReadThisFrame = true;

		FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

		// Ensure RDG resources are ready to use
		if (InstanceData.TransientRDGTexture == nullptr && InstanceData.TextureRHI.IsValid())
		{
			InstanceData.TransientRDGTexture = GraphBuilder.FindExternalTexture(InstanceData.TextureRHI);
			if (InstanceData.TransientRDGTexture == nullptr)
			{
				InstanceData.TransientRDGTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InstanceData.TextureRHI, TEXT("NiagaraIntRenderTarget2D")));
			}
			InstanceData.TransientRDGUAV = GraphBuilder.CreateUAV(InstanceData.TransientRDGTexture);

			GraphBuilder.UseInternalAccessMode(InstanceData.TransientRDGTexture);
			Context.GetRDGExternalAccessQueue().Add(InstanceData.TransientRDGTexture);
		}

		if (InstanceData.TransientRDGUAV != nullptr)
		{
			Parameters->RWTextureUAV = InstanceData.TransientRDGUAV;
		}
		else
		{
			Parameters->RWTextureUAV = Context.GetComputeDispatchInterface().GetEmptyTextureUAV(GraphBuilder, EPixelFormat::PF_R32_SINT, ETextureDimension::Texture2D);
		}
	}
}

bool UNiagaraDataInterfaceIntRenderTarget2D::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	check(Proxy);

	FNDIIntRenderTarget2DInstanceData_GameThread* InstanceData = new (PerInstanceData) FNDIIntRenderTarget2DInstanceData_GameThread();

	if (NiagaraDataInterfaceRenderTargetCommon::GIgnoreCookedOut && !IsUsedWithGPUScript())
	{
		return true;
	}

	// Note: If the Format can ever change we will need to replicate this elsewhere also
	if (UE::PixelFormat::HasCapabilities(InstanceData->Format, EPixelFormatCapabilities::TypedUAVStore) == false)
	{
		return false;
	}

	InstanceData->Size.X = FMath::Clamp<int>(int(float(Size.X) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
	InstanceData->Size.Y = FMath::Clamp<int>(int(float(Size.Y) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
#if WITH_EDITORONLY_DATA
	InstanceData->bPreviewRenderTarget = bPreviewRenderTarget;
	InstanceData->PreviewDisplayRange = PreviewDisplayRange;
#endif
	InstanceData->RTUserParamBinding.Init(SystemInstance->GetInstanceParameters(), RenderTargetUserParameter.Parameter);

	UpdateInstanceTexture(SystemInstance, InstanceData);

	// Push Updates to Proxy.
	FNDIIntRenderTarget2DProxy* RT_Proxy = GetProxyAs<FNDIIntRenderTarget2DProxy>();
	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Proxy, RT_InstanceID=SystemInstance->GetId(), RT_InstanceData=*InstanceData, RT_TargetTexture=InstanceData->TargetTexture ? InstanceData->TargetTexture->GameThread_GetRenderTargetResource() : nullptr](FRHICommandListImmediate& RHICmdList)
		{
			check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(RT_InstanceID));

			FNDIIntRenderTarget2DInstanceData_RenderThread* InstanceData = &RT_Proxy->SystemInstancesToProxyData_RT.Add(RT_InstanceID);
			InstanceData->Size = RT_InstanceData.Size;
		#if WITH_EDITORONLY_DATA
			InstanceData->bPreviewRenderTarget = RT_InstanceData.bPreviewRenderTarget;
			InstanceData->PreviewDisplayRange = RT_InstanceData.PreviewDisplayRange;
		#endif
			if (RT_TargetTexture)
			{
				if (FTextureRenderTarget2DResource* Resource2D = RT_TargetTexture->GetTextureRenderTarget2DResource())
				{
					InstanceData->TextureRHI = Resource2D->GetTextureRHI();
				}
			}
		}
	);
	return true;
}


void UNiagaraDataInterfaceIntRenderTarget2D::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIIntRenderTarget2DInstanceData_GameThread* InstanceData = static_cast<FNDIIntRenderTarget2DInstanceData_GameThread*>(PerInstanceData);
	NiagaraDataInterfaceRenderTargetCommon::ReleaseRenderTarget(SystemInstance, InstanceData);
	InstanceData->~FNDIIntRenderTarget2DInstanceData_GameThread();

	FNDIIntRenderTarget2DProxy* RT_Proxy = GetProxyAs<FNDIIntRenderTarget2DProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[RT_Proxy, InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		}
	);
}

void UNiagaraDataInterfaceIntRenderTarget2D::GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const
{
	OutVariables.Emplace(ExposedRTVar);
}

bool UNiagaraDataInterfaceIntRenderTarget2D::GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const
{
	FNDIIntRenderTarget2DInstanceData_GameThread* InstanceData = static_cast<FNDIIntRenderTarget2DInstanceData_GameThread*>(InPerInstanceData);
	if (InVariable.IsValid() && InVariable == ExposedRTVar && InstanceData && InstanceData->TargetTexture)
	{
		UObject** Var = (UObject**)OutData;
		*Var = InstanceData->TargetTexture;
		return true;
	}
	return false;
}

int32 UNiagaraDataInterfaceIntRenderTarget2D::PerInstanceDataSize() const
{
	return sizeof(FNDIIntRenderTarget2DInstanceData_GameThread);
}

bool UNiagaraDataInterfaceIntRenderTarget2D::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	//-TEMP: Until we prune data interface on cook this will avoid consuming memory
	if (NiagaraDataInterfaceRenderTargetCommon::GIgnoreCookedOut && !IsUsedWithGPUScript())
	{
		return false;
	}

	FNDIIntRenderTarget2DInstanceData_GameThread* InstanceData = static_cast<FNDIIntRenderTarget2DInstanceData_GameThread*>(PerInstanceData);

	{
		bool bUpdateRT = true;
		UpdateInstanceTexture(SystemInstance, InstanceData);
		if (bUpdateRT)
		{
			FNDIIntRenderTarget2DProxy* RT_Proxy = GetProxyAs<FNDIIntRenderTarget2DProxy>();
			ENQUEUE_RENDER_COMMAND(FUpdateData)
			(
				[RT_Proxy, RT_InstanceID=SystemInstance->GetId(), RT_InstanceData=*InstanceData, RT_TargetTexture=InstanceData->TargetTexture ? InstanceData->TargetTexture->GameThread_GetRenderTargetResource() : nullptr](FRHICommandListImmediate& RHICmdList)
				{
					FNDIIntRenderTarget2DInstanceData_RenderThread* InstanceData = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID);
					if (ensureMsgf(InstanceData != nullptr, TEXT("InstanceData was not found for %llu"), RT_InstanceID))
					{
						InstanceData->Size = RT_InstanceData.Size;
					#if WITH_EDITORONLY_DATA
						InstanceData->bPreviewRenderTarget = RT_InstanceData.bPreviewRenderTarget;
						InstanceData->PreviewDisplayRange = RT_InstanceData.PreviewDisplayRange;
					#endif
						InstanceData->TextureRHI.SafeRelease();
						if (RT_TargetTexture)
						{
							if (FTextureRenderTarget2DResource* Resource2D = RT_TargetTexture->GetTextureRenderTarget2DResource())
							{
								InstanceData->TextureRHI = Resource2D->GetTextureRHI();
							}
						}
					}
				}
			);
		}
	}

	return false;
}

bool UNiagaraDataInterfaceIntRenderTarget2D::UpdateInstanceTexture(FNiagaraSystemInstance* SystemInstance, FNDIIntRenderTarget2DInstanceData_GameThread* InstanceData)
{
	// Update from user parameter
	bool bIsRenderTargetUserParam = false;
	if (UObject* UserParamObject = InstanceData->RTUserParamBinding.GetValue())
	{
		if (UTextureRenderTarget2D* UserTargetTexture = Cast<UTextureRenderTarget2D>(UserParamObject))
		{
			bIsRenderTargetUserParam = true;

			// If the texture has changed remove the old one and release if it's one we created
			if (InstanceData->TargetTexture != UserTargetTexture)
			{
				NiagaraDataInterfaceRenderTargetCommon::ReleaseRenderTarget(SystemInstance, InstanceData);
				InstanceData->TargetTexture = UserTargetTexture;
			}
		}
		else
		{
			UE_LOGF(LogNiagara, Error, "RenderTarget UserParam is a '%ls' but is expected to be a UTextureRenderTarget2D", *GetNameSafe(UserParamObject->GetClass()));
		}
	}

	// Do we need to create a new texture?
	bool bHasChanged = false;
	if (InstanceData->TargetTexture == nullptr)
	{
		if (NiagaraDataInterfaceRenderTargetCommon::CreateRenderTarget<UTextureRenderTarget2D>(SystemInstance, InstanceData) == false)
		{
			return false;
		}
		check(InstanceData->TargetTexture);
		InstanceData->TargetTexture->bCanCreateUAV = true;
		InstanceData->TargetTexture->bAutoGenerateMips = false;
		InstanceData->TargetTexture->OverrideFormat = InstanceData->Format;
		InstanceData->TargetTexture->ClearColor = FLinearColor(0.0, 0, 0, 0);
		InstanceData->TargetTexture->InitAutoFormat(InstanceData->Size.X, InstanceData->Size.Y);
		InstanceData->TargetTexture->UpdateResourceImmediate(true);

		bHasChanged = true;
	}
	// Do we need to update the existing texture?
	else
	{
		const bool bAutoGenerateMips = false;
		if ((InstanceData->TargetTexture->SizeX != InstanceData->Size.X) || (InstanceData->TargetTexture->SizeY != InstanceData->Size.Y) ||
			(InstanceData->TargetTexture->OverrideFormat != InstanceData->Format) ||
			!InstanceData->TargetTexture->bCanCreateUAV ||
			(InstanceData->TargetTexture->bAutoGenerateMips != bAutoGenerateMips))
		{
			// resize RT to match what we need for the output
			InstanceData->TargetTexture->bCanCreateUAV = true;
			InstanceData->TargetTexture->bAutoGenerateMips = bAutoGenerateMips;
			InstanceData->TargetTexture->OverrideFormat = InstanceData->Format;
			InstanceData->TargetTexture->InitAutoFormat(InstanceData->Size.X, InstanceData->Size.Y);
			InstanceData->TargetTexture->UpdateResourceImmediate(true);

			bHasChanged = true;
		}
	}

	return bHasChanged;
}

void UNiagaraDataInterfaceIntRenderTarget2D::VMGetSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIIntRenderTarget2DInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int32> OutSizeX(Context);
	FNDIOutputParam<int32> OutSizeY(Context);
	
	for (int32 i=0; i < Context.GetNumInstances(); ++i)
	{
		OutSizeX.SetAndAdvance(InstData->Size.X);
		OutSizeY.SetAndAdvance(InstData->Size.Y);
	}
}

void UNiagaraDataInterfaceIntRenderTarget2D::VMSetSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIIntRenderTarget2DInstanceData_GameThread> InstData(Context);
	FNDIInputParam<int32> InSizeX(Context);
	FNDIInputParam<int32> InSizeY(Context);
	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	for (int32 i=0; i < Context.GetNumInstances(); ++i)
	{
		const int SizeX = InSizeX.GetAndAdvance();
		const int SizeY = InSizeY.GetAndAdvance();
		const bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1 && SizeX >= 0 && SizeY >= 0);
		OutSuccess.SetAndAdvance(bSuccess);
		if (bSuccess)
		{
			InstData->Size.X = FMath::Clamp<int>(int(float(SizeX) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
			InstData->Size.Y = FMath::Clamp<int>(int(float(SizeY) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
		}
	}
}

#undef LOCTEXT_NAMESPACE

