// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceLandscape.h"

#include "Algo/RemoveIf.h"
#include "Containers/ResourceArray.h"
#include "EngineModule.h"
#include "EngineUtils.h"
#include "GlobalRenderResources.h"
#include "Landscape.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "LandscapeInfo.h"
#include "LandscapeProxy.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraStats.h"
#include "NiagaraWorldManager.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "RenderUtils.h"
#include "RHIStaticStates.h"
#include "ShaderCompilerCore.h"
#include "VT/RuntimeVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceLandscape)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterfaceLandscape"

//////////////////////////////////////////////////////////////////////////
// remaining features
// -getting the albedo colour at a point
// -support for CPU

class FNDI_Landscape_SharedResource;
struct FNDILandscapeData_GameThread;

namespace NiagaraDataInterfaceLandscape
{
	const FName NAME_GetBaseColor("GetBaseColor");
	const FName NAME_GetHeight("GetHeight");
	const FName NAME_GetWorldNormal("GetWorldNormal");
	const FName NAME_GetPhysicalMaterialIndex("GetPhysicalMaterialIndex");
	const FName NAME_GenerateRvtFeedback("GenerateVirtualTextureFeedback");

	// This controls the maximum number of regions that will be evaluated for capture in FNDI_Landscape_GeneratedData.
	// A value of -1 means no limit.
	static int32 GMaxRegionSearchCount = -1;
	static FAutoConsoleVariableRef CVarMaxRegionSearchCount(
		TEXT("fx.Niagara.Landscape.MaxRegionSearchCount"),
		GMaxRegionSearchCount,
		TEXT("The maximum number of collision components that will be evaluated for capture by the Landscape DI."),
		ECVF_Default
	);

	// This controls the maximum number of regions that will be captured in FNDI_Landscape_GeneratedData
	// A value of -1 means no limit.
	static int32 GMaxRegionCaptureCount = -1;
	static FAutoConsoleVariableRef CVarMaxRegionCaptureCount(
		TEXT("fx.Niagara.Landscape.MaxRegionCaptureCount"),
		GMaxRegionCaptureCount,
		TEXT("The maximum number of collision components that will be captured by the Landscape DI."),
		ECVF_Default
	);

	enum Version
	{
		InitialVersion = 0,
		SupportVirtualTextures = 1,
		LWCPosition = 2,
		MoreLWCPosition = 3,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceLandscape.ush");

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
		SHADER_PARAMETER_SRV(Texture2D,				BaseColorVirtualTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D<uint4>,	BaseColorVirtualTexturePageTable)
		SHADER_PARAMETER_SAMPLER(SamplerState,		BaseColorVirtualTextureSampler)
		SHADER_PARAMETER(FVector3f,					BaseColorVirtualTextureLWCTile)
		SHADER_PARAMETER(FMatrix44f,				BaseColorVirtualTextureWorldToUvTransform)
		SHADER_PARAMETER(uint32,					BaseColorVirtualTextureUnpackSRGB)
		SHADER_PARAMETER(uint32,					BaseColorVirtualTextureUnpackYCoCg)
		SHADER_PARAMETER(uint32,					BaseColorVirtualTextureEnabled)
		SHADER_PARAMETER(FUintVector4,				BaseColorVirtualTexturePackedUniform0)
		SHADER_PARAMETER(FUintVector4,				BaseColorVirtualTexturePackedUniform1)
		SHADER_PARAMETER(FUintVector4,				BaseColorVirtualTextureUniforms)
		SHADER_PARAMETER_SRV(Texture2D,				HeightVirtualTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D<uint4>,	HeightVirtualTexturePageTable)
		SHADER_PARAMETER_SAMPLER(SamplerState,		HeightVirtualTextureSampler)
		SHADER_PARAMETER(FVector3f,					HeightVirtualTextureLWCTile)
		SHADER_PARAMETER(FMatrix44f,				HeightVirtualTextureWorldToUvTransform)
		SHADER_PARAMETER(uint32,					HeightVirtualTextureEnabled)
		SHADER_PARAMETER(FUintVector4,				HeightVirtualTexturePackedUniform0)
		SHADER_PARAMETER(FUintVector4,				HeightVirtualTexturePackedUniform1)
		SHADER_PARAMETER(FUintVector4,				HeightVirtualTextureUniforms)
		SHADER_PARAMETER_SRV(Texture2D,				NormalVirtualTexture0)
		SHADER_PARAMETER_SRV(Texture2D,				NormalVirtualTexture1)
		SHADER_PARAMETER_TEXTURE(Texture2D<uint4>,	NormalVirtualTexturePageTable)
		SHADER_PARAMETER_SAMPLER(SamplerState,		NormalVirtualTexture0Sampler)
		SHADER_PARAMETER_SAMPLER(SamplerState,		NormalVirtualTexture1Sampler)
		SHADER_PARAMETER(FVector3f,					NormalVirtualTextureLWCTile)
		SHADER_PARAMETER(FMatrix44f,				NormalVirtualTextureWorldToUvTransform)
		SHADER_PARAMETER(FUintVector4,				NormalVirtualTexturePackedUniform0)
		SHADER_PARAMETER(FUintVector4,				NormalVirtualTexturePackedUniform1)
		SHADER_PARAMETER(FUintVector4,				NormalVirtualTextureUniforms0)
		SHADER_PARAMETER(FUintVector4,				NormalVirtualTextureUniforms1)
		SHADER_PARAMETER(int32,						NormalVirtualTextureUnpackMode)
		SHADER_PARAMETER(uint32,					NormalVirtualTextureEnabled)
		SHADER_PARAMETER_TEXTURE(Texture2D,			CachedHeightTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,		CachedHeightTextureSampler)
		SHADER_PARAMETER(uint32,					CachedHeightTextureEnabled)
		SHADER_PARAMETER(FVector3f,					CachedHeightTextureLWCTile)
		SHADER_PARAMETER(FMatrix44f,				CachedHeightTextureWorldToUvTransform)
		SHADER_PARAMETER(FMatrix44f,				CachedHeightTextureUvToWorldTransform)
		SHADER_PARAMETER(FVector4f,					CachedHeightTextureUvScaleBias)
		SHADER_PARAMETER(FVector2f,					CachedHeightTextureWorldGridSize)
		SHADER_PARAMETER(FIntPoint,					CachedHeightTextureDimension)
		SHADER_PARAMETER_SAMPLER(SamplerState,		PointClampedSampler)
		SHADER_PARAMETER_TEXTURE(Texture2D<uint>,	CachedPhysMatTexture)
		SHADER_PARAMETER(FIntPoint,					CachedPhysMatTextureDimension)
	END_SHADER_PARAMETER_STRUCT()


	enum class EBaseColorUnpackType : int32
	{
		None = 0,		// no unpacking is required 
		SRGBUnpack,		// base color is manually packed as SRGB
		YCoCgUnpack,	// base color is manually packed as YCoCg
	};
}

// RenderResource used to hold textures generated by this DI, pulled from the collision geometry of the terrain
class FLandscapeTextureResource : public FRenderResource
{
public:
	FLandscapeTextureResource(const FIntPoint& CellCount);
	virtual ~FLandscapeTextureResource() = default;

	FLandscapeTextureResource() = delete;
	FLandscapeTextureResource(const FLandscapeTextureResource&) = delete;
	FLandscapeTextureResource(const FLandscapeTextureResource&&) = delete;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	void ReleaseSourceData();

	FRHITexture* GetHeightTexture() const { return HeightTexture.Buffer; }
	FRHITexture* GetPhysMatTexture() const { return PhysMatTexture.Buffer; }
	FIntPoint GetDimensions() const { return CellCount; }

	TArray<float>& EditHeightValues(int32 SampleCount);
	TArray<uint8>& EditPhysMatValues(int32 SampleCount);

private:
	FTextureReadBuffer2D HeightTexture;
	FTextureReadBuffer2D PhysMatTexture;
	FIntPoint CellCount;

	TArray<float> HeightValues;
	TArray<uint8> PhysMatValues;

#if STATS
	int32 GpuMemoryUsage = 0;
#endif
};

// SharedResource that can be held by multiple system instances which manages the resources created by this DI
class FNDI_Landscape_SharedResource
{
public:
	struct FResourceKey
	{
		TWeakObjectPtr<const ALandscape> Source = nullptr;
		TArray<TWeakObjectPtr<const UPhysicalMaterial>> PhysicalMaterials;
		TArray<FIntPoint> CapturedRegions;
		FIntPoint MinCaptureRegion = FIntPoint(ForceInitToZero);
		FIntPoint MaxCaptureRegion = FIntPoint(ForceInitToZero);
		bool IncludesCachedHeight = false;
		bool IncludesCachedPhysMat = false;
	};

	FNDI_Landscape_SharedResource() = delete;
	FNDI_Landscape_SharedResource(const FNDI_Landscape_SharedResource&) = delete;
	FNDI_Landscape_SharedResource(const FResourceKey& InKey);

	bool IsUsed() const;
	bool CanBeDestroyed() const;
	bool CanRepresent(const FResourceKey& InKey) const;

	void RegisterUser(const FNDI_SharedResourceUsage& Usage, bool bNeedsDataImmediately);
	void UnregisterUser(const FNDI_SharedResourceUsage& Usage);

	void UpdateState(bool& LandscapeRemoved);
	void Release();

	TUniquePtr<FLandscapeTextureResource> LandscapeTextures;
	FVector3f LandscapeLWCTile = FVector3f::ZeroVector;
	FMatrix ActorToWorldTransform = FMatrix::Identity;
	FMatrix WorldToActorTransform = FMatrix::Identity;
	FVector4 UvScaleBias = FVector4(1.0f, 1.0f, 0.0f, 0.0f);
	FIntPoint CellCount = FIntPoint(ForceInitToZero);
	FVector2D TextureWorldGridSize = FVector2D(1.0f, 1.0f);

	TConstArrayView<FIntPoint> ReadCapturedRegions() const { return ResourceKey.CapturedRegions; }

private:
	enum class EResourceState : uint8
	{
		Uninitialized,
		Initialized,
		Released,
	};

	void Initialize();

	FResourceKey ResourceKey;

	std::atomic<int32> ShaderPhysicsDataUserCount{ 0 };

	EResourceState CurrentState = EResourceState::Uninitialized;
	EResourceState NextState = EResourceState::Uninitialized;
};

using FNDI_Landscape_SharedResourceHandle = FNDI_SharedResourceHandle<FNDI_Landscape_SharedResource, FNDI_SharedResourceUsage>;

struct FRegionRequest
{
	FBox2D Region = FBox2D(FVector2D::Zero(), FVector2D::One());
	int32 MipLevel = 0;
	bool bIncludeBaseColor = true;
	bool bIncludeHeight = true;
	bool bIncludeNormal = true;
};

// Landscape data used for the game thread
struct FNDILandscapeData_GameThread
{
	TWeakObjectPtr<ALandscape> Landscape;
	TWeakObjectPtr<ULandscapeHeightfieldCollisionComponent> CollisionComponent;
	FNDI_Landscape_SharedResourceHandle SharedResourceHandle;
	TArray<FRegionRequest> RegionRequests;
	bool BaseColorVirtualTextureSRGB = false;
	NiagaraDataInterfaceLandscape::EBaseColorUnpackType BaseColorVirtualTextureUnpackType = NiagaraDataInterfaceLandscape::EBaseColorUnpackType::None;
	int32 BaseColorVirtualTextureIndex = INDEX_NONE;
	int32 HeightVirtualTextureIndex = INDEX_NONE;
	int32 NormalVirtualTextureIndex = INDEX_NONE;
	ERuntimeVirtualTextureMaterialType NormalVirtualTextureMode = ERuntimeVirtualTextureMaterialType::Count;
	FBox2D RVTLandscapeBounds = FBox2D(EForceInit::ForceInit);
	bool RequiresCollisionCacheCpu = false;
	bool RequiresCollisionCacheGpu = false;
	bool RequiresPhysMatCacheGpu = false;
	bool SystemRequiresBaseColorGpu = false;
	bool SystemRequiresHeightsGpu = false;
	bool SystemRequiresNormalsGpu = false;

	void Reset()
	{
		Landscape = nullptr;
		SharedResourceHandle = FNDI_Landscape_SharedResourceHandle();
		RegionRequests.Empty();
		BaseColorVirtualTextureSRGB = false;
		BaseColorVirtualTextureUnpackType = NiagaraDataInterfaceLandscape::EBaseColorUnpackType::None;
		BaseColorVirtualTextureIndex = INDEX_NONE;
		HeightVirtualTextureIndex = INDEX_NONE;
		NormalVirtualTextureIndex = INDEX_NONE;
		NormalVirtualTextureMode = ERuntimeVirtualTextureMaterialType::Count;
		RVTLandscapeBounds = FBox2D(EForceInit::ForceInit);
		RequiresCollisionCacheCpu = false;
		RequiresCollisionCacheGpu = false;
		RequiresPhysMatCacheGpu = false;
		SystemRequiresBaseColorGpu = false;
		SystemRequiresHeightsGpu = false;
		SystemRequiresNormalsGpu = false;
	}
};

struct FNDILandscapeData_GameToRenderThread
{
	const FLandscapeTextureResource* TextureResources = nullptr;

	FVector3f CachedHeightTextureLWCTile = FVector3f::ZeroVector;
	FMatrix CachedHeightTextureWorldToUvTransform = FMatrix::Identity;
	FMatrix CachedHeightTextureUvToWorldTransform = FMatrix::Identity;
	FVector4 CachedHeightTextureUvScaleBias = FVector4::Zero();
	FVector2D CachedHeightTextureWorldGridSize = FVector2D(1.0f, 1.0f);
	TArray<FRegionRequest> RegionRequests;

	NiagaraDataInterfaceLandscape::EBaseColorUnpackType BaseColorVirtualTextureUnpackType = NiagaraDataInterfaceLandscape::EBaseColorUnpackType::None;
};

// Landscape data used on the render thread
struct FNDILandscapeData_RenderThread
{
	enum class ENormalUnpackType : int32
	{
		None = 0,
		BC3BC3,
		BC5BC1,
		B5G6R5
	};

	struct FVirtualTextureLayer
	{
		bool IsValid() const
		{
			return TextureSRV.IsValid();
		}

		void Reset()
		{
			TextureSRV = nullptr;
			TextureUniforms = FUintVector4(0, 0, 0, 0);
		}

		void Update(const URuntimeVirtualTexture* VirtualTexture, uint32 LayerIndex, bool bSRGB)
		{
			Reset();

			if (VirtualTexture && ::IsValid(VirtualTexture))
			{
				if (IAllocatedVirtualTexture* AllocatedTexture = VirtualTexture->GetAllocatedVirtualTexture())
				{
					if (FRHIShaderResourceView* PhysicalTextureSRV = AllocatedTexture->GetPhysicalTextureSRV(LayerIndex, bSRGB))
					{
						TextureSRV = PhysicalTextureSRV;
						AllocatedTexture->GetPackedUniform(&TextureUniforms, LayerIndex);
					}
				}
			}
		}

		FShaderResourceViewRHIRef TextureSRV;
		FUintVector4 TextureUniforms = FUintVector4(0, 0, 0, 0);
	};

	struct FVirtualTexturePage
	{
		bool IsValid() const
		{
			return PageTableRef.IsValid();
		}

		void Reset()
		{
			PageTableRef = nullptr;
			PageTableUniforms[0] = FUintVector4(0, 0, 0, 0);
			PageTableUniforms[1] = FUintVector4(0, 0, 0, 0);

			WorldToUvParameters[0] = FVector4::Zero();
			WorldToUvParameters[1] = FVector4::Zero();
			WorldToUvParameters[2] = FVector4::Zero();
			WorldToUvParameters[3] = FVector4::Zero();

			AllocatedTexture = nullptr;
			SizeInPages = FUintPoint(0, 0);
		}

		void Update(const URuntimeVirtualTexture* VirtualTexture, uint32 PageTableIndex, bool bIncludeWorldToUv, bool bIncludeHeightUnpack)
		{
			Reset();

			if (VirtualTexture && ::IsValid(VirtualTexture))
			{
				AllocatedTexture = VirtualTexture->GetAllocatedVirtualTexture();
				if (AllocatedTexture)
				{
					PageTableRef = AllocatedTexture->GetPageTableTexture(PageTableIndex);
					if (PageTableRef.IsValid())
					{
						AllocatedTexture->GetPackedPageTableUniform(PageTableUniforms);

						if (bIncludeWorldToUv)
						{
							WorldToUvParameters[0] = VirtualTexture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldToUVTransform0);
							WorldToUvParameters[1] = VirtualTexture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1);
							WorldToUvParameters[2] = VirtualTexture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2);
						}

						if (bIncludeHeightUnpack)
						{
							WorldToUvParameters[3] = VirtualTexture->GetUniformParameter(ERuntimeVirtualTextureShaderUniform_WorldHeightUnpack);
						}
					}

					SizeInPages = FUintPoint(AllocatedTexture->GetWidthInTiles(), AllocatedTexture->GetHeightInTiles());
				}
			}
		}

		FVector2D WorldToUv(const FVector& WorldPosition) const
		{
			const FVector NormalizedPosition = WorldPosition - WorldToUvParameters[0];
			return FVector2D(
				FVector(WorldToUvParameters[1]).Dot(NormalizedPosition),
				FVector(WorldToUvParameters[2]).Dot(NormalizedPosition));
		}

		FTextureRHIRef PageTableRef;
		FUintVector4 PageTableUniforms[2];

		FVector4 WorldToUvParameters[4];

		IAllocatedVirtualTexture* AllocatedTexture = nullptr;
		FUintPoint SizeInPages = FUintPoint(0, 0);
	};

	FVirtualTexturePage BaseColorVirtualPage;
	FVirtualTextureLayer BaseColorVirtualLayer;

	FVirtualTexturePage HeightVirtualPage;
	FVirtualTextureLayer HeightVirtualLayer;

	FVirtualTexturePage NormalVirtualPage;
	FVirtualTextureLayer NormalVirtualLayer0;
	FVirtualTextureLayer NormalVirtualLayer1;
	ENormalUnpackType NormalUnpackMode;

	FNDILandscapeData_GameToRenderThread LandscapeData;

	bool SetBaseColorVirtualTextureParameters(NiagaraDataInterfaceLandscape::FShaderParameters* ShaderParameters) const
	{
		if (!BaseColorVirtualPage.IsValid() || !BaseColorVirtualLayer.IsValid())
		{
			return false;
		}

		ShaderParameters->BaseColorVirtualTexture = BaseColorVirtualLayer.TextureSRV;
		ShaderParameters->BaseColorVirtualTexturePageTable = BaseColorVirtualPage.PageTableRef;

		FLargeWorldRenderPosition BaseColorVirtualTextureOrigin(BaseColorVirtualPage.WorldToUvParameters[0]);
		ShaderParameters->BaseColorVirtualTextureLWCTile = BaseColorVirtualTextureOrigin.GetTile();

		ShaderParameters->BaseColorVirtualTextureWorldToUvTransform = FMatrix44f(
			BaseColorVirtualTextureOrigin.GetOffset(),
			FVector3f((FVector4f)BaseColorVirtualPage.WorldToUvParameters[1]),
			FVector3f((FVector4f)BaseColorVirtualPage.WorldToUvParameters[2]),
			FVector3f(0, 0, 0));
		ShaderParameters->BaseColorVirtualTextureUnpackSRGB = LandscapeData.BaseColorVirtualTextureUnpackType == NiagaraDataInterfaceLandscape::EBaseColorUnpackType::SRGBUnpack;
		ShaderParameters->BaseColorVirtualTextureUnpackYCoCg = LandscapeData.BaseColorVirtualTextureUnpackType == NiagaraDataInterfaceLandscape::EBaseColorUnpackType::YCoCgUnpack;
		ShaderParameters->BaseColorVirtualTextureEnabled = 1;
		ShaderParameters->BaseColorVirtualTexturePackedUniform0 = BaseColorVirtualPage.PageTableUniforms[0];
		ShaderParameters->BaseColorVirtualTexturePackedUniform1 = BaseColorVirtualPage.PageTableUniforms[1];
		ShaderParameters->BaseColorVirtualTextureUniforms = BaseColorVirtualLayer.TextureUniforms;

		return true;
	}

	static void SetBaseColorVirtualTextureParameters_Default(NiagaraDataInterfaceLandscape::FShaderParameters* ShaderParameters)
	{
		const FUintVector4 DummyUint4(ForceInitToZero);

		ShaderParameters->BaseColorVirtualTexture = GBlackTextureWithSRV->ShaderResourceViewRHI;
		ShaderParameters->BaseColorVirtualTexturePageTable = GBlackUintTexture->TextureRHI;
		ShaderParameters->BaseColorVirtualTextureWorldToUvTransform = FMatrix44f::Identity;
		ShaderParameters->BaseColorVirtualTextureUnpackSRGB = 0;
		ShaderParameters->BaseColorVirtualTextureUnpackYCoCg = 0;
		ShaderParameters->BaseColorVirtualTextureEnabled = 0;
		ShaderParameters->BaseColorVirtualTexturePackedUniform0 = DummyUint4;
		ShaderParameters->BaseColorVirtualTexturePackedUniform1 = DummyUint4;
		ShaderParameters->BaseColorVirtualTextureUniforms = DummyUint4;
	}

	bool SetHeightVirtualTextureParameters(NiagaraDataInterfaceLandscape::FShaderParameters* ShaderParameters) const
	{
		if (!HeightVirtualPage.IsValid() || !HeightVirtualLayer.IsValid())
		{
			return false;
		}

		ShaderParameters->HeightVirtualTexture = HeightVirtualLayer.TextureSRV;
		ShaderParameters->HeightVirtualTexturePageTable = HeightVirtualPage.PageTableRef;

		FLargeWorldRenderPosition HeightVirtualTextureOrigin(HeightVirtualPage.WorldToUvParameters[0]);
		ShaderParameters->HeightVirtualTextureLWCTile = HeightVirtualTextureOrigin.GetTile();

		ShaderParameters->HeightVirtualTextureWorldToUvTransform = FMatrix44f(
			HeightVirtualTextureOrigin.GetOffset(),
			FVector3f((FVector4f)HeightVirtualPage.WorldToUvParameters[1]),
			FVector3f((FVector4f)HeightVirtualPage.WorldToUvParameters[2]),
			FVector3f((FVector4f)HeightVirtualPage.WorldToUvParameters[3]));

		ShaderParameters->HeightVirtualTextureEnabled = 1;
		ShaderParameters->HeightVirtualTexturePackedUniform0 = HeightVirtualPage.PageTableUniforms[0];
		ShaderParameters->HeightVirtualTexturePackedUniform1 = HeightVirtualPage.PageTableUniforms[1];
		ShaderParameters->HeightVirtualTextureUniforms = HeightVirtualLayer.TextureUniforms;

		return true;
	}

	static void SetHeightVirtualTextureParameters_Default(NiagaraDataInterfaceLandscape::FShaderParameters* ShaderParameters)
	{
		const FUintVector4 DummyUint4(ForceInitToZero);

		ShaderParameters->HeightVirtualTexture = GBlackTextureWithSRV->ShaderResourceViewRHI;
		ShaderParameters->HeightVirtualTexturePageTable = GBlackUintTexture->TextureRHI;
		ShaderParameters->HeightVirtualTextureWorldToUvTransform = FMatrix44f::Identity;
		ShaderParameters->HeightVirtualTextureEnabled = 0;
		ShaderParameters->HeightVirtualTexturePackedUniform0 = DummyUint4;
		ShaderParameters->HeightVirtualTexturePackedUniform1 = DummyUint4;
		ShaderParameters->HeightVirtualTextureUniforms = DummyUint4;
	}

	bool SetNormalVirtualTextureParameters(NiagaraDataInterfaceLandscape::FShaderParameters* ShaderParameters) const
	{
		if (!NormalVirtualPage.IsValid() || !NormalVirtualLayer0.IsValid() || !NormalVirtualLayer1.IsValid())
		{
			return false;
		}

		ShaderParameters->NormalVirtualTexture0 = NormalVirtualLayer0.TextureSRV;
		ShaderParameters->NormalVirtualTexture1 = NormalVirtualLayer1.TextureSRV;
		ShaderParameters->NormalVirtualTexturePageTable = NormalVirtualPage.PageTableRef;

		FLargeWorldRenderPosition NormalVirtualTextureOrigin(NormalVirtualPage.WorldToUvParameters[0]);
		ShaderParameters->NormalVirtualTextureLWCTile = NormalVirtualTextureOrigin.GetTile();

		ShaderParameters->NormalVirtualTextureWorldToUvTransform = FMatrix44f(
			FPlane4f(NormalVirtualTextureOrigin.GetOffset()),
			FPlane4f((FVector4f)NormalVirtualPage.WorldToUvParameters[1]),
			FPlane4f((FVector4f)NormalVirtualPage.WorldToUvParameters[2]),
			FPlane4f(0.0f, 0.0f, 0.0f, 1.0f));

		ShaderParameters->NormalVirtualTextureEnabled = 1;
		ShaderParameters->NormalVirtualTextureUnpackMode = (int32)NormalUnpackMode;
		ShaderParameters->NormalVirtualTexturePackedUniform0 = NormalVirtualPage.PageTableUniforms[0];
		ShaderParameters->NormalVirtualTexturePackedUniform1 = NormalVirtualPage.PageTableUniforms[1];
		ShaderParameters->NormalVirtualTextureUniforms0 = NormalVirtualLayer0.TextureUniforms;
		ShaderParameters->NormalVirtualTextureUniforms1 = NormalVirtualLayer1.TextureUniforms;

		return true;
	}

	static void SetNormalVirtualTextureParameters_Default(NiagaraDataInterfaceLandscape::FShaderParameters* ShaderParameters)
	{
		const FUintVector4 DummyUint4(ForceInitToZero);

		ShaderParameters->NormalVirtualTexture0 = GBlackTextureWithSRV->ShaderResourceViewRHI;
		ShaderParameters->NormalVirtualTexture1 = GBlackTextureWithSRV->ShaderResourceViewRHI;
		ShaderParameters->NormalVirtualTexturePageTable = GBlackUintTexture->TextureRHI;
		ShaderParameters->NormalVirtualTextureWorldToUvTransform = FMatrix44f::Identity;
		ShaderParameters->NormalVirtualTextureEnabled = 0;
		ShaderParameters->NormalVirtualTextureUnpackMode = (int32)ENormalUnpackType::None;
		ShaderParameters->NormalVirtualTexturePackedUniform0 = DummyUint4;
		ShaderParameters->NormalVirtualTexturePackedUniform1 = DummyUint4;
		ShaderParameters->NormalVirtualTextureUniforms0 = DummyUint4;
		ShaderParameters->NormalVirtualTextureUniforms1 = DummyUint4;
	}

	bool SetCachedHeightTextureParameters(NiagaraDataInterfaceLandscape::FShaderParameters* ShaderParameters) const
	{
		if (LandscapeData.TextureResources)
		{
			FRHITexture* HeightTexture = LandscapeData.TextureResources->GetHeightTexture();
			FRHITexture* PhysMatTexture = LandscapeData.TextureResources->GetPhysMatTexture();

			if (HeightTexture || PhysMatTexture)
			{
				const FIntPoint TextureDimensions(LandscapeData.TextureResources->GetDimensions());

				ShaderParameters->CachedHeightTextureLWCTile = LandscapeData.CachedHeightTextureLWCTile;
				ShaderParameters->CachedHeightTextureWorldToUvTransform = (FMatrix44f)LandscapeData.CachedHeightTextureWorldToUvTransform;
				ShaderParameters->CachedHeightTextureUvToWorldTransform = (FMatrix44f)LandscapeData.CachedHeightTextureUvToWorldTransform;
				ShaderParameters->CachedHeightTextureUvScaleBias = (FVector4f)LandscapeData.CachedHeightTextureUvScaleBias;
				ShaderParameters->CachedHeightTextureWorldGridSize = (FVector2f)LandscapeData.CachedHeightTextureWorldGridSize;

				if (HeightTexture)
				{
					ShaderParameters->CachedHeightTexture = HeightTexture;
					ShaderParameters->CachedHeightTextureEnabled = 1;
					ShaderParameters->CachedHeightTextureDimension = TextureDimensions;
				}
				else
				{
					ShaderParameters->CachedHeightTexture = GBlackTexture->TextureRHI;
					ShaderParameters->CachedHeightTextureEnabled = 0;
					ShaderParameters->CachedHeightTextureDimension = FIntPoint(ForceInitToZero);
				}

				if (PhysMatTexture)
				{
					ShaderParameters->CachedPhysMatTexture = PhysMatTexture;
					ShaderParameters->CachedPhysMatTextureDimension = TextureDimensions;
				}
				else
				{
					const FIntPoint PhysMatDimensions(ForceInitToZero);
					ShaderParameters->CachedPhysMatTexture = GBlackUintTexture->TextureRHI;
					ShaderParameters->CachedPhysMatTextureDimension = FIntPoint(ForceInitToZero);
				}

				return true;
			}
		}

		return false;
	}

	static void SetCachedHeightTextureParameters_Defaults(NiagaraDataInterfaceLandscape::FShaderParameters* ShaderParameters)
	{
		const FVector4f DummyVector4 = FVector4f::Zero();

		ShaderParameters->CachedHeightTexture = GBlackTexture->TextureRHI;
		ShaderParameters->CachedHeightTextureLWCTile = FVector3f::ZeroVector;
		ShaderParameters->CachedHeightTextureWorldToUvTransform = FMatrix44f::Identity;
		ShaderParameters->CachedHeightTextureUvToWorldTransform = FMatrix44f::Identity;
		ShaderParameters->CachedHeightTextureUvScaleBias = DummyVector4;
		ShaderParameters->CachedHeightTextureWorldGridSize = FVector2f::ZeroVector;
		ShaderParameters->CachedHeightTextureEnabled = 0;

		ShaderParameters->CachedPhysMatTexture = GBlackUintTexture->TextureRHI;
	}
};

class FNDI_Landscape_GeneratedData : public FNDI_GeneratedData
{
public:
	virtual ~FNDI_Landscape_GeneratedData();
	virtual void Tick(ETickingGroup TickGroup, float DeltaSeconds) override;

	static TypeHash GetTypeHash();

	FNDI_Landscape_SharedResourceHandle GetLandscapeData(const UNiagaraDataInterfaceLandscape& LandscapeDI, const FNiagaraSystemInstance& SystemInstance, const FNDILandscapeData_GameThread& InstanceData, FNDI_SharedResourceUsage Usage, bool bNeedsDataImmediately);

private:
	FRWLock LandscapeDataGuard;
	TArray<TSharedPtr<FNDI_Landscape_SharedResource>> LandscapeData;
	TArray<TSharedPtr<FNDI_Landscape_SharedResource>> ReleasedLandscapeData;
};

struct FNiagaraDataInterfaceProxyLandscape : public FNiagaraDataInterfaceProxy
{
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override
	{
		const FNDILandscapeData_GameToRenderThread& SourceData = *reinterpret_cast<const FNDILandscapeData_GameToRenderThread*>(PerInstanceData);
		SystemInstancesToProxyData_RT.FindOrAdd(Instance).LandscapeData = SourceData;
	}

	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return sizeof(FNDILandscapeData_GameToRenderThread);
	}

	void UpdateProxy_RT(
		FNiagaraSystemInstanceID ID,
		const URuntimeVirtualTexture* BaseColorVirtualTexture,
		bool bBaseColorTextureSRGB,
		const URuntimeVirtualTexture* HeightVirtualTexture,
		const URuntimeVirtualTexture* NormalVirtualTexture,
		ERuntimeVirtualTextureMaterialType NormalVirtualTextureMode)
	{
		if (FNDILandscapeData_RenderThread* Proxy_RT = SystemInstancesToProxyData_RT.Find(ID))
		{
			// todo - need to figure out a way to confirm that this is in fact the best/only option for the page/layer indices
			constexpr uint32 BaseColorVirtualTextureLayerIndex = 0;
			constexpr uint32 BaseColorVirtualTexturePageIndex = 0;
			Proxy_RT->BaseColorVirtualLayer.Update(BaseColorVirtualTexture, BaseColorVirtualTextureLayerIndex, bBaseColorTextureSRGB);
			Proxy_RT->BaseColorVirtualPage.Update(BaseColorVirtualTexture, BaseColorVirtualTexturePageIndex, true /*bIncludeWorldToUv*/, false /*bIncludeHeightUnpack*/);

			constexpr uint32 HeightVirtualTextureLayerIndex = 0;
			constexpr uint32 HeightVirtualTexturePageIndex = 0;
			Proxy_RT->HeightVirtualLayer.Update(HeightVirtualTexture, HeightVirtualTextureLayerIndex, false /* bSRGB */);
			Proxy_RT->HeightVirtualPage.Update(HeightVirtualTexture, HeightVirtualTexturePageIndex, true /*bIncludeWorldToUv*/, true /*bIncludeHeightUnpack*/);

			constexpr uint32 NormalVirtualTexturePageIndex = 0;
			Proxy_RT->NormalVirtualPage.Update(NormalVirtualTexture, NormalVirtualTexturePageIndex, true /*bIncludeWorldToUv*/, false /*bIncludeHeightUnpack*/);

			switch (NormalVirtualTextureMode)
			{
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
				Proxy_RT->NormalVirtualLayer0.Update(NormalVirtualTexture, 0, false /* bSRGB */);
				Proxy_RT->NormalVirtualLayer1.Update(NormalVirtualTexture, 1, false /* bSRGB */);
				Proxy_RT->NormalUnpackMode = FNDILandscapeData_RenderThread::ENormalUnpackType::B5G6R5;
				break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
				Proxy_RT->NormalVirtualLayer0.Update(NormalVirtualTexture, 0, false /* bSRGB */);
				Proxy_RT->NormalVirtualLayer1.Update(NormalVirtualTexture, 1, false /* bSRGB */);
				Proxy_RT->NormalUnpackMode = FNDILandscapeData_RenderThread::ENormalUnpackType::BC3BC3;
				break;

			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
				Proxy_RT->NormalVirtualLayer0.Update(NormalVirtualTexture, 1, false /* bSRGB */);
				Proxy_RT->NormalVirtualLayer1.Update(NormalVirtualTexture, 2, false /* bSRGB */);
				Proxy_RT->NormalUnpackMode = FNDILandscapeData_RenderThread::ENormalUnpackType::BC5BC1;
				break;

			default:
				Proxy_RT->NormalVirtualPage.Reset();
				Proxy_RT->NormalVirtualLayer0.Reset();
				Proxy_RT->NormalVirtualLayer1.Reset();
				Proxy_RT->NormalUnpackMode = FNDILandscapeData_RenderThread::ENormalUnpackType::None;
				break;
			}

			TArray<FRegionRequest> RegionsToProcess;
			Swap(RegionsToProcess, Proxy_RT->LandscapeData.RegionRequests);

			if (!RegionsToProcess.IsEmpty())
			{
				IRendererModule& RenderModule = GetRendererModule();

				for (const FRegionRequest& RegionRequest : RegionsToProcess)
				{
					const FNDILandscapeData_RenderThread::FVirtualTexturePage* TexturePages[] =
					{
						RegionRequest.bIncludeBaseColor ? &Proxy_RT->BaseColorVirtualPage : nullptr,
						RegionRequest.bIncludeHeight ? &Proxy_RT->HeightVirtualPage : nullptr,
						RegionRequest.bIncludeNormal ? &Proxy_RT->NormalVirtualPage : nullptr,
					};

					for (const FNDILandscapeData_RenderThread::FVirtualTexturePage* TexturePage : TexturePages)
					{
						if (TexturePage && TexturePage->IsValid() && TexturePage->AllocatedTexture)
						{
							const FVector2D UVMin = TexturePage->WorldToUv(FVector(RegionRequest.Region.Min.X, RegionRequest.Region.Min.Y, 0.0));
							const FVector2D UVMax = TexturePage->WorldToUv(FVector(RegionRequest.Region.Max.X, RegionRequest.Region.Max.Y, 0.0));

							const FVector2D ClampedUVMin = FVector2D::Clamp(UVMin, FVector2D::Zero(), FVector2D::One());
							const FVector2D ClampedUVMax = FVector2D::Clamp(UVMax, FVector2D::Zero(), FVector2D::One());

							RenderModule.RequestVirtualTextureTiles(TexturePage->AllocatedTexture, FVector2D::One(), FVector2D::Zero(), FVector2D::One(), ClampedUVMin, ClampedUVMax, RegionRequest.MipLevel);
						}
					}
				}
			}
		}
	}

	TMap<FNiagaraSystemInstanceID, FNDILandscapeData_RenderThread> SystemInstancesToProxyData_RT;
};

FLandscapeTextureResource::FLandscapeTextureResource(const FIntPoint& InCellCount)
: CellCount(InCellCount)
{
}

void FLandscapeTextureResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (HeightValues.Num())
	{
		FResourceBulkDataArrayView BulkData(HeightValues);
		HeightTexture.Initialize(RHICmdList, TEXT("FLandscapeTextureResource_HeightTexture"), sizeof(float), CellCount.X, CellCount.Y, EPixelFormat::PF_R32_FLOAT, FTextureReadBuffer2D::DefaultTextureInitFlag, &BulkData);
	}

	if (PhysMatValues.Num())
	{
		FResourceBulkDataArrayView BulkData(PhysMatValues);
		PhysMatTexture.Initialize(RHICmdList, TEXT("FLandscapeTextureResource_PhysMatTexture"), sizeof(uint8), CellCount.X, CellCount.Y, EPixelFormat::PF_R8_UINT, FTextureReadBuffer2D::DefaultTextureInitFlag, &BulkData);
	}

	ReleaseSourceData();

#if STATS
	check(GpuMemoryUsage == 0);
	GpuMemoryUsage = HeightTexture.NumBytes + PhysMatTexture.NumBytes;
	INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GpuMemoryUsage);
#endif
}

void FLandscapeTextureResource::ReleaseRHI()
{
	HeightTexture.Release();
	PhysMatTexture.Release();

#if STATS
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GpuMemoryUsage);
	GpuMemoryUsage = 0;
#endif
}

void FLandscapeTextureResource::ReleaseSourceData()
{
	HeightValues.Empty();
	PhysMatValues.Empty();
}

TArray<float>& FLandscapeTextureResource::EditHeightValues(int32 SampleCount)
{
	const float DefaultHeight = 0.0f;
	HeightValues.Reset(SampleCount);
	HeightValues.Init(DefaultHeight, SampleCount);

	return HeightValues;
}

TArray<uint8>& FLandscapeTextureResource::EditPhysMatValues(int32 SampleCount)
{
	const uint8 DefaultPhysMat = INDEX_NONE;
	PhysMatValues.Reset(SampleCount);
	PhysMatValues.Init(DefaultPhysMat, SampleCount);

	return PhysMatValues;
}

FNDI_Landscape_SharedResource::FNDI_Landscape_SharedResource(const FResourceKey& InKey)
: LandscapeTextures(nullptr)
, ResourceKey(InKey)
{
}

bool FNDI_Landscape_SharedResource::IsUsed() const
{
	return (ShaderPhysicsDataUserCount > 0) && CurrentState != EResourceState::Released;
}

bool FNDI_Landscape_SharedResource::CanBeDestroyed() const
{
	const bool ReadyForRemoval = !IsUsed();

	if (ReadyForRemoval && LandscapeTextures && LandscapeTextures->IsInitialized())
	{
		UE_LOGF(LogNiagara, Error, "FNDI_Landscape_SharedResource::CanBeDestroyed returning true, but the LandscpaeTextures is still initialized! Source[%ls] MinRegion[%d,%d] MaxRegion[%d,%d]",
			*GetNameSafe(ResourceKey.Source.Get()), ResourceKey.MinCaptureRegion.X, ResourceKey.MinCaptureRegion.Y, ResourceKey.MaxCaptureRegion.X, ResourceKey.MaxCaptureRegion.Y);
	}

	return ReadyForRemoval;
}

bool FNDI_Landscape_SharedResource::CanRepresent(const FResourceKey& RequestKey) const
{
	if (CurrentState == EResourceState::Released)
	{
		return false;
	}

	if (ResourceKey.Source != RequestKey.Source)
	{
		return false;
	}

	if ((RequestKey.IncludesCachedHeight && !ResourceKey.IncludesCachedHeight)
		|| (RequestKey.IncludesCachedPhysMat && !ResourceKey.IncludesCachedPhysMat))
	{
		return false;
	}

	if (ResourceKey.MinCaptureRegion.X > RequestKey.MinCaptureRegion.X
		|| ResourceKey.MaxCaptureRegion.X < RequestKey.MaxCaptureRegion.X
		|| ResourceKey.MinCaptureRegion.Y > RequestKey.MinCaptureRegion.Y
		|| ResourceKey.MaxCaptureRegion.Y < RequestKey.MaxCaptureRegion.Y)
	{
		return false;
	}

	if (ResourceKey.PhysicalMaterials.Num() < RequestKey.PhysicalMaterials.Num())
	{
		return false;
	}

	for (int32 MaterialIt = 0; MaterialIt < RequestKey.PhysicalMaterials.Num(); ++MaterialIt)
	{
		if (ResourceKey.PhysicalMaterials[MaterialIt] != RequestKey.PhysicalMaterials[MaterialIt])
		{
			return false;
		}
	}

	const int32 CapturedRegionCount = ResourceKey.CapturedRegions.Num();
	int32 SearchIndex = 0;
	for (const FIntPoint& RequestRegion : RequestKey.CapturedRegions)
	{
		while (SearchIndex < CapturedRegionCount && ResourceKey.CapturedRegions[SearchIndex] != RequestRegion)
		{
			++SearchIndex;
		}
	}

	return SearchIndex < CapturedRegionCount;
}

void FNDI_Landscape_SharedResource::RegisterUser(const FNDI_SharedResourceUsage& Usage, bool bNeedsDataImmediately)
{
	check(Usage.RequiresCpuAccess == false);

	if (Usage.RequiresGpuAccess)
	{
		if (ShaderPhysicsDataUserCount++ == 0)
		{
			NextState = EResourceState::Initialized;
			if (bNeedsDataImmediately)
			{
				bool bLandscapeRemoved = false;
				UpdateState(bLandscapeRemoved);
			}
		}
	}
}

void FNDI_Landscape_SharedResource::UnregisterUser(const FNDI_SharedResourceUsage& Usage)
{
	check(Usage.RequiresCpuAccess == false);

	if (Usage.RequiresGpuAccess)
	{
		if (--ShaderPhysicsDataUserCount == 0)
		{
			NextState = EResourceState::Released;
		}
	}
}

void FNDI_Landscape_SharedResource::Initialize()
{
	if (const ULandscapeInfo* LandscapeInfo = ResourceKey.Source->GetLandscapeInfo())
	{
		const int32 ComponentQuadCount = ResourceKey.Source->ComponentSizeQuads;
		const FIntPoint RegionSpan = ResourceKey.MaxCaptureRegion - ResourceKey.MinCaptureRegion + FIntPoint(1, 1);
		const FIntPoint CaptureQuadSpan = RegionSpan * ComponentQuadCount;
		const FIntPoint CaptureVertexSpan = CaptureQuadSpan + FIntPoint(1, 1);
		const int32 SampleCount = CaptureVertexSpan.X * CaptureVertexSpan.Y;

		LandscapeTextures = MakeUnique<FLandscapeTextureResource>(CaptureVertexSpan);

		TArray<float>* HeightValues = ResourceKey.IncludesCachedHeight ? &LandscapeTextures->EditHeightValues(SampleCount) : nullptr;
		TArray<uint8>* PhysMatValues = ResourceKey.IncludesCachedPhysMat ? &LandscapeTextures->EditPhysMatValues(SampleCount) : nullptr;

		const FIntPoint RegionVertexBase = ResourceKey.MinCaptureRegion * ComponentQuadCount;

		for (const FIntPoint& Region : ResourceKey.CapturedRegions)
		{
			const ULandscapeHeightfieldCollisionComponent* CollisionComponent = LandscapeInfo->XYtoCollisionComponentMap.FindRef(Region);
			if (ensure(CollisionComponent != nullptr))
			{
				if (HeightValues)
				{
					const FIntPoint SectionBase = (Region - ResourceKey.MinCaptureRegion) * ComponentQuadCount;
					CollisionComponent->FillHeightTile(*HeightValues, SectionBase.X + SectionBase.Y * CaptureVertexSpan.X, CaptureVertexSpan.X);
				}

				if (PhysMatValues)
				{
					const FIntPoint SectionBase = (Region - ResourceKey.MinCaptureRegion) * ComponentQuadCount;
					CollisionComponent->FillMaterialIndexTile(*PhysMatValues, SectionBase.X + SectionBase.Y * CaptureVertexSpan.X, CaptureVertexSpan.X);

					// remap the material index to the list we have on the DI
					TArray<uint8> PhysMatRemap;
					for (const UPhysicalMaterial* ComponentMaterial : CollisionComponent->CookedPhysicalMaterials)
					{
						const int32 RemapIndex = ResourceKey.PhysicalMaterials.IndexOfByKey(ComponentMaterial);
						ensure(RemapIndex <= TNumericLimits<uint8>::Max());
						PhysMatRemap.Emplace(uint8(RemapIndex));
					}

					for (int32 Y = 0; Y < ComponentQuadCount; ++Y)
					{
						for (int32 X = 0; X < ComponentQuadCount; ++X)
						{
							const int32 WriteIndex = SectionBase.X + X + (SectionBase.Y + Y) * CaptureVertexSpan.X;
							uint8& PhysMatIndex = (*PhysMatValues)[WriteIndex];
							PhysMatIndex = PhysMatRemap.IsValidIndex(PhysMatIndex) ? PhysMatRemap[PhysMatIndex] : INDEX_NONE;
						}
					}
				}
			}
		}

		// number of cells that are represented in our heights array
		CellCount = CaptureVertexSpan;

		// mapping to get the UV from 'cell space' which is relative to the entire terrain (not just the captured regions)
		FVector2D UvScale(1.0f / CaptureVertexSpan.X, 1.0f / CaptureVertexSpan.Y);

		UvScaleBias = FVector4(
			UvScale.X,
			UvScale.Y,
			(0.5f - RegionVertexBase.X) * UvScale.X,
			(0.5f - RegionVertexBase.Y) * UvScale.Y);

		FTransform LandscapeTransform = ResourceKey.Source->GetTransform();
		FLargeWorldRenderPosition LandscapeTransformOrigin(LandscapeTransform.GetLocation());

		LandscapeLWCTile = LandscapeTransformOrigin.GetTile();
		LandscapeTransform.SetLocation(FVector(LandscapeTransformOrigin.GetOffset()));

		ActorToWorldTransform = LandscapeTransform.ToMatrixWithScale();
		WorldToActorTransform = ActorToWorldTransform.Inverse();
		TextureWorldGridSize = FVector2D(ResourceKey.Source->GetTransform().GetScale3D());

		BeginInitResource(LandscapeTextures.Get());
	}
}

void FNDI_Landscape_SharedResource::Release()
{
	if (FLandscapeTextureResource* Resource = LandscapeTextures.Release())
	{
		ENQUEUE_RENDER_COMMAND(BeginDestroyCommand)([RT_Resource = Resource](FRHICommandListImmediate& RHICmdList)
		{
			RT_Resource->ReleaseResource();

			// On some RHIs textures will push data on the RHI thread
			// Therefore we are not 'released' until the RHI thread has processed all commands
			RHICmdList.EnqueueLambda([RT_Resource](FRHICommandListImmediate& RHICmdList)
				{
					delete RT_Resource;
				});
		});
	}
}

void FNDI_Landscape_SharedResource::UpdateState(bool& LandscapeReleased)
{
	const EResourceState RequestedState = NextState;

	LandscapeReleased = false;

	if (RequestedState == CurrentState)
	{
		return;
	}

	if (RequestedState == EResourceState::Initialized)
	{
		Initialize();
	}
	else if (RequestedState == EResourceState::Released)
	{
		Release();
		LandscapeReleased = true;
	}

	CurrentState = RequestedState;
}

FNDI_Landscape_GeneratedData::~FNDI_Landscape_GeneratedData()
{
	FRWScopeLock WriteLock(LandscapeDataGuard, SLT_Write);

	for (TSharedPtr<FNDI_Landscape_SharedResource> Landscape : LandscapeData)
	{
		Landscape->Release();
	}
}

void FNDI_Landscape_GeneratedData::Tick(ETickingGroup TickGroup, float DeltaSeconds)
{
	FRWScopeLock WriteLock(LandscapeDataGuard, SLT_Write);

	{ // handle any changes to the generated data
		TArray<int32, TInlineAllocator<32>> LandscapeToRemove;

		const int32 LandscapeCount = LandscapeData.Num();

		for (int32 LandscapeIt = 0; LandscapeIt < LandscapeCount; ++LandscapeIt)
		{
			TSharedPtr<FNDI_Landscape_SharedResource> Landscape = LandscapeData[LandscapeIt];

			bool LandscapeReleased = false;

			Landscape->UpdateState(LandscapeReleased);

			if (LandscapeReleased)
			{
				LandscapeToRemove.Add(LandscapeIt);
			}
		}

		while (LandscapeToRemove.Num())
		{
			const int32 LandscapeIt = LandscapeToRemove.Pop(EAllowShrinking::No);

			TSharedPtr<FNDI_Landscape_SharedResource> Landscape = LandscapeData[LandscapeIt];
			LandscapeData.RemoveAtSwap(LandscapeIt);

			if (!Landscape->CanBeDestroyed())
			{
				ReleasedLandscapeData.Add(Landscape);
			}
		}
	}

	{ // check any shared resources that we've got pending release to see if they can now be destroyed
		ReleasedLandscapeData.SetNum(Algo::RemoveIf(ReleasedLandscapeData, [&](TSharedPtr<FNDI_Landscape_SharedResource>& Landscape)
		{
			return Landscape->CanBeDestroyed();
		}));
	}
}

FNDI_GeneratedData::TypeHash FNDI_Landscape_GeneratedData::GetTypeHash()
{
	static const TypeHash Hash = FCrc::Strihash_DEPRECATED(TEXT("FNDI_Landscape_GeneratedData"));
	return Hash;
}

FNDI_Landscape_SharedResourceHandle FNDI_Landscape_GeneratedData::GetLandscapeData(const UNiagaraDataInterfaceLandscape& LandscapeDI, const FNiagaraSystemInstance& SystemInstance, const FNDILandscapeData_GameThread& InstanceData, FNDI_SharedResourceUsage Usage, bool bNeedsDataImmediately)
{
	using namespace NiagaraDataInterfaceLandscape;

	check(IsInGameThread());

	const ALandscape* Landscape = InstanceData.Landscape.Get();
	const ULandscapeInfo* LandscapeInfo = Landscape ? Landscape->GetLandscapeInfo() : nullptr;

	if (LandscapeInfo == nullptr)
	{
		return FNDI_Landscape_SharedResourceHandle();
	}

	// we want to use the bounds of the system to figure out which cells of the landscape that we need to handle
	const int32 MaxLandscapeRegionCount = LandscapeInfo->XYtoCollisionComponentMap.Num();

	const FVector LWCTileOffset = FVector(SystemInstance.GetLWCTile()) * FLargeWorldRenderScalar::GetTileSize();
	FBox SystemWorldBounds = SystemInstance.GetLocalBounds().TransformBy(SystemInstance.GetWorldTransform());
	SystemWorldBounds.Min += LWCTileOffset;
	SystemWorldBounds.Max += LWCTileOffset;

	const FTransform& LandscapeActorToWorld = Landscape->LandscapeActorToWorld();
	const FVector SystemMinInLandscape = LandscapeActorToWorld.InverseTransformPosition(SystemWorldBounds.Min);
	const FVector SystemMaxInLandscape = LandscapeActorToWorld.InverseTransformPosition(SystemWorldBounds.Max);

	FBox SystemBoundsInLandscape(
		SystemMinInLandscape.ComponentMin(SystemMaxInLandscape),
		SystemMinInLandscape.ComponentMax(SystemMaxInLandscape));

	// transform the above box into a range of integers covering the cells of the landscape
	// first clamp it at 0
	SystemBoundsInLandscape.Min = SystemBoundsInLandscape.Min.ComponentMax(FVector::ZeroVector);
	SystemBoundsInLandscape.Max = SystemBoundsInLandscape.Max.ComponentMax(FVector::ZeroVector);

	// next rescale based on the quad size
	const double QuadSizeScaleFactor = 1.0 / ((double)Landscape->ComponentSizeQuads);
	SystemBoundsInLandscape.Min *= QuadSizeScaleFactor;
	SystemBoundsInLandscape.Max *= QuadSizeScaleFactor;

	// truncate to integers
	const FVector MaxIntValue = FVector(double(TNumericLimits<int32>::Max()));
	SystemBoundsInLandscape.Min = SystemBoundsInLandscape.Min.ComponentMin(MaxIntValue);
	SystemBoundsInLandscape.Max = SystemBoundsInLandscape.Max.ComponentMin(MaxIntValue);

	const FIntRect SystemRect = FIntRect(
		FIntPoint(FMath::FloorToInt(SystemBoundsInLandscape.Min.X), FMath::FloorToInt(SystemBoundsInLandscape.Min.Y)),
		FIntPoint(FMath::CeilToInt(SystemBoundsInLandscape.Max.X), FMath::CeilToInt(SystemBoundsInLandscape.Max.Y)));

	// for obnoxiously large system bounds we need to guard against potential overflow on the number of cells
	const int32 MaxSystemWidth = FMath::Clamp<int32>(SystemRect.Max.X - SystemRect.Min.X, 0, MaxLandscapeRegionCount);
	const int32 MaxSystemHeight = FMath::Clamp<int32>(SystemRect.Max.Y - SystemRect.Min.Y, 0, MaxLandscapeRegionCount);

	const int64 MaxSystemRegionCount64 = int64(MaxSystemWidth) * int64(MaxSystemHeight);
	const int32 MaxSystemRegionCount = int32(FMath::Min<int64>(int64(TNumericLimits<int32>::Max()), MaxSystemRegionCount64));

	const int32 MaxRegionCount = FMath::Min(MaxSystemRegionCount, MaxLandscapeRegionCount);

	FNDI_Landscape_SharedResource::FResourceKey Key;
	Key.Source = Landscape;
	Key.CapturedRegions.Reserve(MaxRegionCount);
	Key.MinCaptureRegion = FIntPoint(TNumericLimits<int32>::Max(), TNumericLimits<int32>::Max());
	Key.MaxCaptureRegion = FIntPoint(TNumericLimits<int32>::Min(), TNumericLimits<int32>::Min());
	Key.IncludesCachedHeight = InstanceData.RequiresCollisionCacheGpu;
	Key.IncludesCachedPhysMat = InstanceData.RequiresPhysMatCacheGpu;
	Key.PhysicalMaterials.Reserve(LandscapeDI.PhysicalMaterials.Num());
	for (const UPhysicalMaterial* Material : LandscapeDI.PhysicalMaterials)
	{
		Key.PhysicalMaterials.Emplace(Material);
	}

	ensureMsgf(GMaxRegionSearchCount < 0 || MaxRegionCount <= GMaxRegionSearchCount, TEXT("FNDI_Landscape_GeneratedData exceeded search count (%d:%d vs %d) for NiagaraSystem %s"),
		MaxSystemRegionCount,
		MaxLandscapeRegionCount,
		GMaxRegionSearchCount,
		*GetNameSafe(SystemInstance.GetSystem()));

	auto CaptureRegion = [&](const FIntPoint& Region) -> bool
	{
		if (GMaxRegionCaptureCount >= 0 && Key.CapturedRegions.Num() >= GMaxRegionCaptureCount)
		{
			return false;
		}

		Key.CapturedRegions.Add(Region);
		Key.MinCaptureRegion = Key.MinCaptureRegion.ComponentMin(Region);
		Key.MaxCaptureRegion = Key.MaxCaptureRegion.ComponentMax(Region);
		return true;
	};

	bool bFailedToCaptureRegion = false;
	int32 RegionSearchCount = 0;

	auto ConditionalCaptureByComponent = [&]() -> void
	{
		for (const auto& LandscapeComponent : LandscapeInfo->XYtoCollisionComponentMap)
		{
			if (GMaxRegionSearchCount >= 0 && RegionSearchCount >= GMaxRegionSearchCount)
			{
				return;
			}

			++RegionSearchCount;
			if (SystemRect.Contains(LandscapeComponent.Key) && LandscapeComponent.Value)
			{
				if (!CaptureRegion(LandscapeComponent.Key))
				{
					bFailedToCaptureRegion = true;
					return;
				}
			}
		}
	};

	auto ConditionalCaptureByRect = [&]() -> void
	{
		for (int32 GridY = SystemRect.Min.Y; GridY < SystemRect.Max.Y; ++GridY)
		{
			for (int32 GridX = SystemRect.Min.X; GridX < SystemRect.Max.X; ++GridX)
			{
				if (GMaxRegionSearchCount >= 0 && RegionSearchCount >= GMaxRegionSearchCount)
				{
					return;
				}

				++RegionSearchCount;
				const FIntPoint CurrentRegion(GridX, GridY);
				if (LandscapeInfo->XYtoCollisionComponentMap.FindRef(CurrentRegion) != nullptr)
				{
					if (!CaptureRegion(CurrentRegion))
					{
						bFailedToCaptureRegion = true;
						return;
					}
				}
			}
		}
	};

	if (MaxSystemRegionCount > MaxLandscapeRegionCount)
	{
		ConditionalCaptureByComponent();
	}
	else
	{
		ConditionalCaptureByRect();
	}

	ensureMsgf(!bFailedToCaptureRegion, TEXT("FNDI_Landscape_GeneratedData exceeded maximum capture count (%d) for NiagaraSystem %s"),
		GMaxRegionCaptureCount,
		*GetNameSafe(SystemInstance.GetSystem()));

	if (!Key.CapturedRegions.Num())
	{
		return FNDI_Landscape_SharedResourceHandle();
	}

	// Attempt to Find data
	{
		FRWScopeLock ReadLock(LandscapeDataGuard, SLT_ReadOnly);
		TSharedPtr<FNDI_Landscape_SharedResource>* Existing = LandscapeData.FindByPredicate([&](const TSharedPtr<FNDI_Landscape_SharedResource>& LandscapeEntry)
		{
			return LandscapeEntry->CanRepresent(Key);
		});

		if (Existing)
		{
			return FNDI_Landscape_SharedResourceHandle(Usage, *Existing, bNeedsDataImmediately);
		}
	}

	// We need to add
	// Note we do not need to check for other threads adding here as it's only every done on the GameThread
	FRWScopeLock WriteLock(LandscapeDataGuard, SLT_Write);
	return FNDI_Landscape_SharedResourceHandle(
		Usage,
		LandscapeData.Add_GetRef(MakeShared<FNDI_Landscape_SharedResource>(Key)),
		bNeedsDataImmediately);
}

UNiagaraDataInterfaceLandscape::UNiagaraDataInterfaceLandscape(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)		
{
	SourceLandscape = nullptr;
	Proxy.Reset(new FNiagaraDataInterfaceProxyLandscape());
}

void UNiagaraDataInterfaceLandscape::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}	
}

bool UNiagaraDataInterfaceLandscape::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceLandscape* DestinationLandscape = CastChecked<UNiagaraDataInterfaceLandscape>(Destination);
	DestinationLandscape->SourceLandscape = SourceLandscape;	
	DestinationLandscape->SourceMode = SourceMode;
	DestinationLandscape->PhysicalMaterials = PhysicalMaterials;
	DestinationLandscape->bVirtualTexturesSupported = bVirtualTexturesSupported;

	return true;
}

bool UNiagaraDataInterfaceLandscape::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceLandscape* OtherLandscape = CastChecked<const UNiagaraDataInterfaceLandscape>(Other);
	return OtherLandscape->SourceLandscape == SourceLandscape
		&& OtherLandscape->SourceMode == SourceMode
		&& OtherLandscape->PhysicalMaterials == PhysicalMaterials
		&& OtherLandscape->bVirtualTexturesSupported == bVirtualTexturesSupported;
}

#if WITH_NIAGARA_DEBUGGER
void UNiagaraDataInterfaceLandscape::DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const
{
	const FNDILandscapeData_GameThread* InstanceData_GT = DebugHudContext.GetSystemInstance()->FindTypedDataInterfaceInstanceData<FNDILandscapeData_GameThread>(this);
	if (InstanceData_GT == nullptr)
	{
		return;
	}

	auto SafeActorLabel = [](const AActor* Actor) -> FString
	{
		if (Actor)
		{
			return Actor->GetActorNameOrLabel();
		}
		else
		{
			return TEXT("none");
		}
	};

	TStringBuilder<256> OutputString;
	OutputString.Appendf(TEXT("Landscape(%s)"),	*SafeActorLabel(InstanceData_GT->Landscape.Get()));
	if (InstanceData_GT->BaseColorVirtualTextureIndex != INDEX_NONE)
	{
		OutputString.Append(TEXT(" BaseColor:RVT"));
	}
	if (InstanceData_GT->HeightVirtualTextureIndex != INDEX_NONE)
	{
		OutputString.Append(TEXT(" Height:RVT"));
	}
	if (InstanceData_GT->NormalVirtualTextureIndex != INDEX_NONE)
	{
		OutputString.Append(TEXT(" Normal:RVT"));
	}
	if (InstanceData_GT->RequiresCollisionCacheGpu || InstanceData_GT->RequiresPhysMatCacheGpu)
	{
		OutputString.Append(TEXT(" HasCollisionCache"));
	}
	if (InstanceData_GT->SystemRequiresBaseColorGpu || InstanceData_GT->SystemRequiresHeightsGpu || InstanceData_GT->SystemRequiresNormalsGpu)
	{
		OutputString.Appendf(TEXT(" GpuReqs(BaseColor:%s, Height:%s, Normal:%s)"),
			InstanceData_GT->SystemRequiresBaseColorGpu ? TEXT("Yes") : TEXT("No"),
			InstanceData_GT->SystemRequiresHeightsGpu ? TEXT("Yes") : TEXT("No"),
			InstanceData_GT->SystemRequiresNormalsGpu ? TEXT("Yes") : TEXT("No"));
	}

	if (DebugHudContext.IsVerbose() && InstanceData_GT->SharedResourceHandle)
	{
		if (ALandscape* LandscapeActor = InstanceData_GT->Landscape.Get())
		{
			if (ULandscapeInfo* LandscapeInfo = LandscapeActor->GetLandscapeInfo())
			{
				// getting the list of publicly facing LandscapeProxies corresponding to the captured regions is a bit of a pain
				TSet<const ULandscapeHeightfieldCollisionComponent*> RelevantCollisionComponents;
				TSet<const ALandscapeProxy*> RelevantLandscapeProxies;

				const FNDI_Landscape_SharedResource& LandscapeData = InstanceData_GT->SharedResourceHandle.ReadResource();
				for (const FIntPoint& CapturedRegion : LandscapeData.ReadCapturedRegions())
				{
					if (const ULandscapeHeightfieldCollisionComponent* RefCollisionComponent = LandscapeInfo->XYtoCollisionComponentMap.FindRef(CapturedRegion))
					{
						RelevantCollisionComponents.Add(RefCollisionComponent);
					}
				}

				LandscapeInfo->ForEachLandscapeProxy([&RelevantCollisionComponents, &RelevantLandscapeProxies](ALandscapeProxy* LandscapeProxy) -> bool
				{
					for (const ULandscapeHeightfieldCollisionComponent* CollisionComponent : LandscapeProxy->CollisionComponents)
					{
						if (RelevantCollisionComponents.Contains(CollisionComponent))
						{
							RelevantLandscapeProxies.Add(LandscapeProxy);
						}
					}
					return true;
				});

				if (!RelevantLandscapeProxies.IsEmpty())
				{
					OutputString.Append(TEXT(" Proxies:"));

					for (const ALandscapeProxy* RelevantProxy : RelevantLandscapeProxies)
					{
						OutputString.Append(TEXT("\n"));
						OutputString.Appendf(TEXT("\t%s"), *SafeActorLabel(RelevantProxy));
					}
				}
			}
		}
	}

	DebugHudContext.GetOutputString().Append(OutputString);
}
#endif

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceLandscape::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NiagaraDataInterfaceLandscape::NAME_GetBaseColor;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Landscape")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("WorldPos")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Color")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NiagaraDataInterfaceLandscape::NAME_GetHeight;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Landscape")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("WorldPos")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NiagaraDataInterfaceLandscape::NAME_GetWorldNormal;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Landscape")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("WorldPos")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NiagaraDataInterfaceLandscape::NAME_GetPhysicalMaterialIndex;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Landscape")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("WorldPos")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NiagaraDataInterfaceLandscape::NAME_GenerateRvtFeedback;
		Sig.bExperimental = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Landscape")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("RegionExtentMin")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("RegionExtentMax")));
		Sig.Inputs.Emplace_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"))).SetValue(0);
		Sig.Inputs.Emplace_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IncludeBaseColor"))).SetValue(true);
		Sig.Inputs.Emplace_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IncludeHeight"))).SetValue(true);
		Sig.Inputs.Emplace_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IncludeNormal"))).SetValue(true);
		Sig.Inputs.Emplace_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled"))).SetValue(true);
		Sig.SetDescription(LOCTEXT("GenerateRVTFeedback", "Generates a request to refine the RVT in the specified region."));

		OutFunctions.Add(Sig);
	}
	for (FNiagaraFunctionSignature& Sig : OutFunctions)
	{
		Sig.FunctionVersion = NiagaraDataInterfaceLandscape::LatestVersion;
	}
}

bool UNiagaraDataInterfaceLandscape::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	// always upgrade to the latest version
	if (FunctionSignature.FunctionVersion < NiagaraDataInterfaceLandscape::LatestVersion)
	{
		TArray<FNiagaraFunctionSignature> AllFunctions;
		GetFunctionsInternal(AllFunctions);
		for (const FNiagaraFunctionSignature& Sig : AllFunctions)
		{
			if (FunctionSignature.Name == Sig.Name)
			{
				FunctionSignature = Sig;
				return true;
			}
		}
	}

	return false;
}
#endif

void UNiagaraDataInterfaceLandscape::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* PerInstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NiagaraDataInterfaceLandscape;

	if (BindingInfo.Name == NiagaraDataInterfaceLandscape::NAME_GenerateRvtFeedback)
	{
		OutFunc = FVMExternalFunction::CreateLambda([this](FVectorVMExternalFunctionContext& Context) { VMGenerateRvtFeedback(Context); });
	}
}

void UNiagaraDataInterfaceLandscape::VMGenerateRvtFeedback(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDILandscapeData_GameThread> InstanceData(Context);
	FNDIInputParam<FVector2f> InExtentMin(Context);
	FNDIInputParam<FVector2f> InExtentMax(Context);
	FNDIInputParam<int32> InMipLevel(Context);
	FNDIInputParam<bool> InIncludeBaseColor(Context);
	FNDIInputParam<bool> InIncludeHeight(Context);
	FNDIInputParam<bool> InIncludeNormal(Context);
	FNDIInputParam<bool> InEnabled(Context);

	const bool bHasValidLandscapeBounds = InstanceData->RVTLandscapeBounds.bIsValid;
	const int32 InstanceCount = Context.GetNumInstances();
	if (InstanceCount)
	{
		if (bHasValidLandscapeBounds)
		{
			InstanceData->RegionRequests.Reserve(InstanceData->RegionRequests.Num() + InstanceCount);
		}

		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			const FVector2f ExtentMin = InExtentMin.GetAndAdvance();
			const FVector2f ExtentMax = InExtentMax.GetAndAdvance();
			const int32 RequestedMipLevel = InMipLevel.GetAndAdvance();
			const bool bIncludeBaseColor = InIncludeBaseColor.GetAndAdvance();
			const bool bIncludeHeight = InIncludeHeight.GetAndAdvance();
			const bool bIncludeNormal = InIncludeNormal.GetAndAdvance();
			const bool bEnabled = InEnabled.GetAndAdvance();

			if (bEnabled && bHasValidLandscapeBounds)
			{
				const FBox2D RequestBounds = InstanceData->RVTLandscapeBounds.Overlap(FBox2D(FVector2D(ExtentMin), FVector2D(ExtentMax)));
				if (RequestBounds.bIsValid)
				{
					FRegionRequest& Request = InstanceData->RegionRequests.AddDefaulted_GetRef();
					Request.Region = RequestBounds;
					Request.MipLevel = RequestedMipLevel;
					Request.bIncludeBaseColor = bIncludeBaseColor;
					Request.bIncludeHeight = bIncludeHeight;
					Request.bIncludeNormal = bIncludeNormal;
				}
			}
		}
	}
}

void UNiagaraDataInterfaceLandscape::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	FNDILandscapeData_GameThread& SourceData = *reinterpret_cast<FNDILandscapeData_GameThread*>(PerInstanceData);
	FNDILandscapeData_GameToRenderThread* TargetData = new(DataForRenderThread) FNDILandscapeData_GameToRenderThread();

	if (SourceData.SharedResourceHandle)
	{
		const FNDI_Landscape_SharedResource& SourceResource = SourceData.SharedResourceHandle.ReadResource();

		TargetData->TextureResources = SourceResource.LandscapeTextures.Get();
		
		TargetData->CachedHeightTextureUvScaleBias = SourceResource.UvScaleBias;
		TargetData->CachedHeightTextureLWCTile = SourceResource.LandscapeLWCTile;
		TargetData->CachedHeightTextureWorldToUvTransform = SourceResource.WorldToActorTransform;
		TargetData->CachedHeightTextureUvToWorldTransform = SourceResource.ActorToWorldTransform;
		TargetData->CachedHeightTextureWorldGridSize = SourceResource.TextureWorldGridSize;
	}

	TargetData->BaseColorVirtualTextureUnpackType = SourceData.BaseColorVirtualTextureUnpackType;
	TargetData->RegionRequests = MoveTemp(SourceData.RegionRequests);
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceLandscape::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}

	InVisitor->UpdateShaderFile(NiagaraDataInterfaceLandscape::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<NiagaraDataInterfaceLandscape::FShaderParameters>();

	return true;

}

void UNiagaraDataInterfaceLandscape::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NiagaraDataInterfaceLandscape::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceLandscape::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if ( (FunctionInfo.DefinitionName == NiagaraDataInterfaceLandscape::NAME_GetBaseColor) ||
		 (FunctionInfo.DefinitionName == NiagaraDataInterfaceLandscape::NAME_GetHeight) ||
		 (FunctionInfo.DefinitionName == NiagaraDataInterfaceLandscape::NAME_GetWorldNormal) ||
		 (FunctionInfo.DefinitionName == NiagaraDataInterfaceLandscape::NAME_GetPhysicalMaterialIndex) )
	{
		return true;
	}

	return false;
}
#endif

void UNiagaraDataInterfaceLandscape::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NiagaraDataInterfaceLandscape::FShaderParameters>();
}

void UNiagaraDataInterfaceLandscape::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyLandscape& RT_Proxy = Context.GetProxy<FNiagaraDataInterfaceProxyLandscape>();
	FNDILandscapeData_RenderThread* ProxyData = RT_Proxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

	NiagaraDataInterfaceLandscape::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NiagaraDataInterfaceLandscape::FShaderParameters>();

	// Set Samplers
	FRHISamplerState* BilinearSamplerState				= TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	FRHISamplerState* PointClampedSampler				= TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	ShaderParameters->BaseColorVirtualTextureSampler	= BilinearSamplerState;
	ShaderParameters->HeightVirtualTextureSampler		= BilinearSamplerState;
	ShaderParameters->NormalVirtualTexture0Sampler		= BilinearSamplerState;
	ShaderParameters->NormalVirtualTexture1Sampler		= BilinearSamplerState;
	ShaderParameters->CachedHeightTextureSampler		= RHIPixelFormatHasCapabilities(EPixelFormat::PF_R32_FLOAT, EPixelFormatCapabilities::TextureFilterable) ? BilinearSamplerState : PointClampedSampler;
	ShaderParameters->PointClampedSampler				= PointClampedSampler;

	// Set Textures
	bool ApplyBaseColorVirtualTextureDefaults = true;
	bool ApplyHeightVirtualTextureDefaults = true;
	bool ApplyNormalVirtualTextureDefaults = true;
	bool ApplyCachedHeightTextureDefaults = true;

	if (ProxyData)
	{
		ApplyBaseColorVirtualTextureDefaults = ProxyData->SetBaseColorVirtualTextureParameters(ShaderParameters) != true;
		ApplyHeightVirtualTextureDefaults = ProxyData->SetHeightVirtualTextureParameters(ShaderParameters) != true;
		ApplyNormalVirtualTextureDefaults = ProxyData->SetNormalVirtualTextureParameters(ShaderParameters) != true;
		ApplyCachedHeightTextureDefaults = ProxyData->SetCachedHeightTextureParameters(ShaderParameters) != true;
	}

	if (ApplyBaseColorVirtualTextureDefaults)
	{
		FNDILandscapeData_RenderThread::SetBaseColorVirtualTextureParameters_Default(ShaderParameters);
	}

	if (ApplyHeightVirtualTextureDefaults)
	{
		FNDILandscapeData_RenderThread::SetHeightVirtualTextureParameters_Default(ShaderParameters);
	}

	if (ApplyNormalVirtualTextureDefaults)
	{
		FNDILandscapeData_RenderThread::SetNormalVirtualTextureParameters_Default(ShaderParameters);
	}

	if (ApplyCachedHeightTextureDefaults)
	{
		FNDILandscapeData_RenderThread::SetCachedHeightTextureParameters_Defaults(ShaderParameters);
	}
}

bool UNiagaraDataInterfaceLandscape::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDILandscapeData_GameThread* InstanceData = new(PerInstanceData) FNDILandscapeData_GameThread();

	bool SystemRequiresHeightsCpu = false;
	bool SystemRequiresHeightsGpu = false;
	SystemInstance->EvaluateBoundFunction(NiagaraDataInterfaceLandscape::NAME_GetHeight, SystemRequiresHeightsCpu, SystemRequiresHeightsGpu);

	bool SystemRequiresNormalsCpu = false;
	bool SystemRequiresNormalsGpu = false;
	SystemInstance->EvaluateBoundFunction(NiagaraDataInterfaceLandscape::NAME_GetWorldNormal, SystemRequiresNormalsCpu, SystemRequiresNormalsGpu);

	bool SystemRequiresPhysMatCpu = false;
	bool SystemRequiresPhysMatGpu = false;
	SystemInstance->EvaluateBoundFunction(NiagaraDataInterfaceLandscape::NAME_GetPhysicalMaterialIndex, SystemRequiresPhysMatCpu, SystemRequiresPhysMatGpu);

	bool SystemRequiresBaseColorCpu = false;
	bool SystemRequiresBaseColorGpu = false;
	SystemInstance->EvaluateBoundFunction(NiagaraDataInterfaceLandscape::NAME_GetBaseColor, SystemRequiresBaseColorCpu, SystemRequiresBaseColorGpu);

	InstanceData->SystemRequiresBaseColorGpu = SystemRequiresBaseColorGpu;
	InstanceData->SystemRequiresHeightsGpu = SystemRequiresHeightsGpu;
	InstanceData->SystemRequiresNormalsGpu = SystemRequiresNormalsGpu;
	InstanceData->RequiresPhysMatCacheGpu = SystemRequiresPhysMatGpu;

	ApplyLandscape(*SystemInstance, *InstanceData);

	FNiagaraDataInterfaceProxyLandscape* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyLandscape>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDICreateProxy) (
		[RT_Proxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
	{
		check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));
		RT_Proxy->SystemInstancesToProxyData_RT.Add(InstanceID);
	});

	return true;
}

void UNiagaraDataInterfaceLandscape::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	if (FNDILandscapeData_GameThread* InstanceData = reinterpret_cast<FNDILandscapeData_GameThread*>(PerInstanceData))
	{
		InstanceData->~FNDILandscapeData_GameThread();

		FNiagaraDataInterfaceProxyLandscape* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyLandscape>();
		ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyProxy) (
			[RT_Proxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		});
	}
}

int32 UNiagaraDataInterfaceLandscape::PerInstanceDataSize() const
{
	return sizeof(FNDILandscapeData_GameThread);
}

bool UNiagaraDataInterfaceLandscape::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	bool ResetInstance = false;

	if (FNDILandscapeData_GameThread* InstanceData = reinterpret_cast<FNDILandscapeData_GameThread*>(PerInstanceData))
	{
		// todo - can we avoid checking this every tick?  currently it is required in case the landscape beneath us changes or new data
		// has streamed in for the landscape and we need to update our capture of the data
		ApplyLandscape(*SystemInstance, *InstanceData);

		if (InstanceData->RequiresCollisionCacheGpu || InstanceData->RequiresPhysMatCacheGpu)
		{
			FNDI_Landscape_GeneratedData& GeneratedData = SystemInstance->GetWorldManager()->EditGeneratedData<FNDI_Landscape_GeneratedData>();
			InstanceData->SharedResourceHandle = GeneratedData.GetLandscapeData(*this, *SystemInstance, *InstanceData, FNDI_SharedResourceUsage(false, true), true);
		}
		else
		{
			InstanceData->SharedResourceHandle = FNDI_Landscape_SharedResourceHandle();
		}

		const URuntimeVirtualTexture* BaseColorVirtualTexture = nullptr;
		const URuntimeVirtualTexture* HeightVirtualTexture = nullptr;
		const URuntimeVirtualTexture* NormalVirtualTexture = nullptr;

		if (ALandscape* SourceDataLandscape = InstanceData->Landscape.Get())
		{
			if (SourceDataLandscape->RuntimeVirtualTextures.IsValidIndex(InstanceData->BaseColorVirtualTextureIndex))
			{
				BaseColorVirtualTexture = SourceDataLandscape->RuntimeVirtualTextures[InstanceData->BaseColorVirtualTextureIndex];
			}
			if (SourceDataLandscape->RuntimeVirtualTextures.IsValidIndex(InstanceData->HeightVirtualTextureIndex))
			{
				HeightVirtualTexture = SourceDataLandscape->RuntimeVirtualTextures[InstanceData->HeightVirtualTextureIndex];
			}
			if (SourceDataLandscape->RuntimeVirtualTextures.IsValidIndex(InstanceData->NormalVirtualTextureIndex))
			{
				NormalVirtualTexture = SourceDataLandscape->RuntimeVirtualTextures[InstanceData->NormalVirtualTextureIndex];
			}
		}

		FNiagaraDataInterfaceProxyLandscape* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyLandscape>();
		ENQUEUE_RENDER_COMMAND(FNiagaraDIUpdateProxy) (
			[RT_Proxy,
			BaseColorVirtualTexture,
			bBaseColorTextureSRGB = InstanceData->BaseColorVirtualTextureSRGB,
			HeightVirtualTexture,
			NormalVirtualTexture,
			NormalVirtualTextureMode = InstanceData->NormalVirtualTextureMode,
			InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->UpdateProxy_RT(InstanceID, BaseColorVirtualTexture, bBaseColorTextureSRGB, HeightVirtualTexture, NormalVirtualTexture, NormalVirtualTextureMode);
		});
	}

	return ResetInstance;
}

void UNiagaraDataInterfaceLandscape::ApplyLandscape(const FNiagaraSystemInstance& SystemInstance, FNDILandscapeData_GameThread& InstanceData) const
{
	ALandscape* Landscape = GetLandscape(SystemInstance, InstanceData);

	// when in editor the contents of the Landscape are volatile and so we'll make sure to
	// refresh our instance properties any time we apply
	#if !WITH_EDITOR
	if (InstanceData.Landscape == Landscape)
	{
		return;
	}
	#endif

	if (!Landscape)
	{
		InstanceData.Reset();
		return;
	}

	InstanceData.Landscape = Landscape;
	InstanceData.BaseColorVirtualTextureIndex = INDEX_NONE;
	InstanceData.HeightVirtualTextureIndex = INDEX_NONE;
	InstanceData.NormalVirtualTextureIndex = INDEX_NONE;
	InstanceData.RVTLandscapeBounds = FBox2D(EForceInit::ForceInit);

	// only worry about virtual textures if our current platform supports them
	if (bVirtualTexturesSupported && UseVirtualTexturing(GetFeatureLevelShaderPlatform(SystemInstance.GetFeatureLevel())))
	{
		const int32 RuntimeVirtualTextureCount = InstanceData.Landscape->RuntimeVirtualTextures.Num();
		for (int32 TextureIt = 0; TextureIt < RuntimeVirtualTextureCount; ++TextureIt)
		{
			if (const URuntimeVirtualTexture* Vt = InstanceData.Landscape->RuntimeVirtualTextures[TextureIt])
			{
				const ERuntimeVirtualTextureMaterialType VirtualMaterialType = Vt->GetMaterialType();

				switch (VirtualMaterialType)
				{
				case ERuntimeVirtualTextureMaterialType::WorldHeight:
					if (InstanceData.HeightVirtualTextureIndex == INDEX_NONE)
					{
						InstanceData.HeightVirtualTextureIndex = TextureIt;
					}
					break;

				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
					if (InstanceData.NormalVirtualTextureIndex == INDEX_NONE)
					{
						InstanceData.NormalVirtualTextureIndex = TextureIt;
						InstanceData.NormalVirtualTextureMode = VirtualMaterialType;
					}
					// intentionally falling through to the BaseColor option
				case ERuntimeVirtualTextureMaterialType::BaseColor:
					if (InstanceData.BaseColorVirtualTextureIndex == INDEX_NONE)
					{
						InstanceData.BaseColorVirtualTextureIndex = TextureIt;
					}
					break;
				}

				if (InstanceData.BaseColorVirtualTextureIndex == TextureIt)
				{
					InstanceData.BaseColorVirtualTextureSRGB = Vt->IsLayerSRGB(0);
					if (Vt->IsLayerYCoCg(0))
					{
						InstanceData.BaseColorVirtualTextureUnpackType = NiagaraDataInterfaceLandscape::EBaseColorUnpackType::YCoCgUnpack;
					}
					else if (VirtualMaterialType == ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness)
					{
						InstanceData.BaseColorVirtualTextureUnpackType = NiagaraDataInterfaceLandscape::EBaseColorUnpackType::SRGBUnpack;
					}
				}
			}
		}

	}

	const bool bHasVirtualTexture = InstanceData.HeightVirtualTextureIndex != INDEX_NONE
		|| InstanceData.NormalVirtualTextureIndex != INDEX_NONE
		|| InstanceData.BaseColorVirtualTextureIndex != INDEX_NONE;

	if (bHasVirtualTexture)
	{
		FVector LandscapeBoundsOrigin;
		FVector LandscapeBoundsExtent;
		Landscape->GetActorBounds(true /*bOnlyCollidingComponents*/, LandscapeBoundsOrigin, LandscapeBoundsExtent);

		InstanceData.RVTLandscapeBounds = FBox2D(FVector2D(LandscapeBoundsOrigin - LandscapeBoundsExtent), FVector2D(LandscapeBoundsOrigin + LandscapeBoundsExtent));
	}

	// we need to create our own copy of the collision geometry if either the heights are needed, and they're not
	// provided by a virtual texture or if the normals are needed and they're not provided by a virtual texture
	InstanceData.RequiresCollisionCacheGpu =
		(InstanceData.SystemRequiresBaseColorGpu && InstanceData.BaseColorVirtualTextureIndex == INDEX_NONE) ||
		(InstanceData.SystemRequiresHeightsGpu && InstanceData.HeightVirtualTextureIndex == INDEX_NONE) ||
		(InstanceData.SystemRequiresNormalsGpu && InstanceData.NormalVirtualTextureIndex == INDEX_NONE);
}


// Users can supply a ALandscape actor
// if none is provided, then we use the World's LandscapeInfoMap to find an appropriate ALandscape actor
ALandscape* UNiagaraDataInterfaceLandscape::GetLandscape(const FNiagaraSystemInstance& SystemInstance, FNDILandscapeData_GameThread& InstanceData) const
{
	if (ALandscape* Landscape = Cast<ALandscape>(SourceLandscape))
	{
		if (SourceMode == ENDILandscape_SourceMode::Source || SourceMode == ENDILandscape_SourceMode::Default)
		{
			return Landscape;
		}
	}

	const FBox WorldBounds = SystemInstance.GetLocalBounds().TransformBy(SystemInstance.GetWorldTransform());

	auto TestLandscape = [&](const ALandscape* InLandscape)
	{
		if (ULandscapeHeightfieldCollisionComponent* CollisionComponent = InstanceData.CollisionComponent.Get())
		{
			if (WorldBounds.IntersectXY(CollisionComponent->Bounds.GetBox()))
			{
				return true;
			}
		}

		if (InLandscape->GetWorld() == SystemInstance.GetWorld())
		{
			if (const ULandscapeInfo* LandscapeInfo = InLandscape->GetLandscapeInfo())
			{
				for (const auto& ComponentIt : LandscapeInfo->XYtoCollisionComponentMap)
				{
					if (WorldBounds.IntersectXY(ComponentIt.Value->Bounds.GetBox()))
					{
						InstanceData.CollisionComponent = ComponentIt.Value;
						return true;
					}
				}
			}
		}

		return false;
	};

	ALandscape* Hint = InstanceData.Landscape.Get();
	if (Hint && TestLandscape(Hint))
	{
		return Hint;
	}

	for (TActorIterator<ALandscape> LandscapeIt(SystemInstance.GetWorld()); LandscapeIt; ++LandscapeIt)
	{
		if (ALandscape* Landscape = *LandscapeIt)
		{
			if (TestLandscape(Landscape))
			{
				return Landscape;
			}
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE

