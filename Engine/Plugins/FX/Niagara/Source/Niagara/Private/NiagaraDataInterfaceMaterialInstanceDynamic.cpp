// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceMaterialInstanceDynamic.h"

#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSystemInstance.h"

#include "Async/Async.h"
#include "UObject/WeakInterfacePtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceMaterialInstanceDynamic)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceMaterialInstanceDynamic"

namespace NDIMIDPrivate
{
	static const FName SetScalarParameterFuncName(TEXT("SetScalarParameter"));
	static const FText SetScalarParameterDescription = LOCTEXT("NiagaraMID_SetScalarParameter_Desc", "Sets a scalar parameter to a Material instance dynamic by name.");
	static const FName SetVector4ParameterFuncName(TEXT("SetVector4Parameter"));
	static const FText SetVector4ParameterDescription = LOCTEXT("NiagaraMID_SetVector4Parameter_Desc", "Sets a Vector4 parameter to a Material instance dynamic by name.");

	struct FInstanceData_GT
	{
		TWeakObjectPtr<UMaterialInstanceDynamic> MID;
	};

	static void VMSetScalarParameter(FVectorVMExternalFunctionContext& Context, FName MaterialScalarParamName)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GT> InstanceData(Context);
		FNDIInputParam<float> InScalarValue(Context);

		TWeakObjectPtr<UMaterialInstanceDynamic> MID = InstanceData->MID;
		const float ScalarParam = InScalarValue.Get();
		if (MID.IsValid())
		{
			AsyncTask(ENamedThreads::GameThread,
				[WeakMID=MID, ParamName=MaterialScalarParamName, ScalarValue=ScalarParam]()
				{
					if (WeakMID.IsValid())
					{
						auto* MID = WeakMID.Get();
						if (MID)
						{
							MID->SetScalarParameterValue(ParamName, ScalarValue);
						}
					}
				}
			);
		}
	}

	static void VMSetVector4Parameter(FVectorVMExternalFunctionContext& Context, FName MaterialVectorParamName)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GT> InstanceData(Context);
		FNDIInputParam<FVector4f> InVectorValue(Context);

		TWeakObjectPtr<UMaterialInstanceDynamic> MID = InstanceData->MID;
		const FVector4f Vec4Param = InVectorValue.Get();
		if (MID.IsValid())
		{
			AsyncTask(ENamedThreads::GameThread,
				[WeakMID=MID, ParamName=MaterialVectorParamName, Vector4Value=Vec4Param]()
				{
					if (WeakMID.IsValid())
					{
						auto* MID = WeakMID.Get();
						if (MID)
						{
							MID->SetVectorParameterValue(ParamName, Vector4Value);
						}
					}
				}
			);
		}
	}

}

UNiagaraDataInterfaceMaterialInstanceDynamic::UNiagaraDataInterfaceMaterialInstanceDynamic(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, DefaultMaterialInst(nullptr)
	, InstancedMaterialParamBinding()
{
}

void UNiagaraDataInterfaceMaterialInstanceDynamic::InitBindingMembers()
{
#if WITH_EDITORONLY_DATA
	InstancedMaterialParamBinding.SetUsage(ENiagaraParameterBindingUsage::User);
	InstancedMaterialParamBinding.SetAllowedObjects({ UObject::StaticClass() });
#endif
}

void UNiagaraDataInterfaceMaterialInstanceDynamic::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		const ENiagaraTypeRegistryFlags Flags =
			ENiagaraTypeRegistryFlags::AllowUserVariable |
			ENiagaraTypeRegistryFlags::AllowEmitterVariable |
			ENiagaraTypeRegistryFlags::AllowSystemVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;

		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);

		InitBindingMembers();
	}
}

bool UNiagaraDataInterfaceMaterialInstanceDynamic::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	NDIMIDPrivate::FInstanceData_GT* InstanceData = new (PerInstanceData) NDIMIDPrivate::FInstanceData_GT();
	InstanceData->MID = GetMaterialInstance(SystemInstance);
	return true;
}

bool UNiagaraDataInterfaceMaterialInstanceDynamic::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	NDIMIDPrivate::FInstanceData_GT* InstanceData = static_cast<NDIMIDPrivate::FInstanceData_GT*>(PerInstanceData);
	InstanceData->MID = GetMaterialInstance(SystemInstance);
	return false;
}

UMaterialInstanceDynamic* UNiagaraDataInterfaceMaterialInstanceDynamic::GetMaterialInstance(FNiagaraSystemInstance* SystemInstance) const
{
	UMaterialInstanceDynamic* MaterialInstance = DefaultMaterialInst;
	if (InstancedMaterialParamBinding.ResolvedParameter.IsValid())
	{
		FNiagaraParameterDirectBinding<UObject*> InstancedMaterialDirectBinding;
		UObject* InstancedMaterialObj = InstancedMaterialDirectBinding.Init(SystemInstance->GetInstanceParameters(), InstancedMaterialParamBinding.ResolvedParameter);
		if (UMaterialInstanceDynamic* InstancedMat = Cast<UMaterialInstanceDynamic>(InstancedMaterialObj))
		{
			MaterialInstance = InstancedMat;
		}
	}

	return MaterialInstance;
}

bool UNiagaraDataInterfaceMaterialInstanceDynamic::CanExecuteOnTarget(ENiagaraSimTarget Target) const
{
	return Target == ENiagaraSimTarget::CPUSim;
}

int32 UNiagaraDataInterfaceMaterialInstanceDynamic::PerInstanceDataSize() const
{
	return sizeof(NDIMIDPrivate::FInstanceData_GT);
}

void UNiagaraDataInterfaceMaterialInstanceDynamic::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	auto* InstanceData = static_cast<NDIMIDPrivate::FInstanceData_GT*>(PerInstanceData);
	InstanceData->~FInstanceData_GT();
}

bool UNiagaraDataInterfaceMaterialInstanceDynamic::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	auto* OtherTyped = CastChecked< UNiagaraDataInterfaceMaterialInstanceDynamic >(Destination);

	OtherTyped->DefaultMaterialInst = DefaultMaterialInst;
	OtherTyped->InstancedMaterialParamBinding = InstancedMaterialParamBinding;

	return true;
}

bool UNiagaraDataInterfaceMaterialInstanceDynamic::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfaceMaterialInstanceDynamic* OtherTyped = CastChecked< const UNiagaraDataInterfaceMaterialInstanceDynamic >(Other);

	return OtherTyped->DefaultMaterialInst == DefaultMaterialInst
		&& OtherTyped->InstancedMaterialParamBinding == InstancedMaterialParamBinding;
}


void UNiagaraDataInterfaceMaterialInstanceDynamic::GetFunctions(TArray< FNiagaraFunctionSignature >& OutFunctions)
{
	FNiagaraFunctionSignature SigTemplate;
	SigTemplate.bMemberFunction = true;
	SigTemplate.bRequiresExecPin = true;
	SigTemplate.bRequiresContext = false;
	SigTemplate.bSupportsCPU = true;
	SigTemplate.bSupportsGPU = false;
	SigTemplate.FunctionSpecifiers.Add(FName("Parameter Name"));
	SigTemplate.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("MID")));

	// Scalar parameters
	{
		FNiagaraFunctionSignature Sig = SigTemplate;
		Sig.Name = NDIMIDPrivate::SetScalarParameterFuncName;
		Sig.SetDescription(NDIMIDPrivate::SetScalarParameterDescription);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		OutFunctions.Add(Sig);
	}

	// Vector4 parameters
	{
		FNiagaraFunctionSignature Sig = SigTemplate;
		Sig.Name = NDIMIDPrivate::SetVector4ParameterFuncName;
		Sig.SetDescription(NDIMIDPrivate::SetVector4ParameterDescription);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		OutFunctions.Add(Sig);
	}
}

void UNiagaraDataInterfaceMaterialInstanceDynamic::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	if (BindingInfo.Name == NDIMIDPrivate::SetScalarParameterFuncName)
	{
		OutFunc = FVMExternalFunction::CreateLambda(
			[ParamName=BindingInfo.FunctionSpecifiers[0].Value](FVectorVMExternalFunctionContext& Context)
			{
				NDIMIDPrivate::VMSetScalarParameter(Context, ParamName);
			}
		);
	}
	else if (BindingInfo.Name == NDIMIDPrivate::SetVector4ParameterFuncName)
	{
		OutFunc = FVMExternalFunction::CreateLambda(
			[ParamName=BindingInfo.FunctionSpecifiers[0].Value](FVectorVMExternalFunctionContext& Context)
			{
				NDIMIDPrivate::VMSetVector4Parameter(Context, ParamName);
			}
		);
	}
}

#undef LOCTEXT_NAMESPACE
