// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialInstanceDynamic.h"
#include "NiagaraCommon.h"
#include "NiagaraDataInterface.h"
#include "NiagaraParameterBinding.h"
#include "NiagaraShared.h"
#include "VectorVM.h"

#include "NiagaraDataInterfaceMaterialInstanceDynamic.generated.h"

UCLASS(EditInlineNew, Category = "Material", CollapseCategories, meta = (DisplayName = "Material Instance Dynamic"), MinimalAPI)
class UNiagaraDataInterfaceMaterialInstanceDynamic : public UNiagaraDataInterface
{
	GENERATED_BODY()

public:
	UNiagaraDataInterfaceMaterialInstanceDynamic(const FObjectInitializer& Initializer);

	// UObject Interface
	virtual void PostInitProperties() override;
	// UObject Interface End

	// UNiagaraDataInterface Interface
	virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override;
	virtual int32 PerInstanceDataSize() const override;
	virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual void GetFunctions(TArray< FNiagaraFunctionSignature >& OutFunctions) override;
	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	virtual bool HasPreSimulateTick() const override { return true; }
	// UNiagaraDataInterface Interface

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInstanceDynamic> DefaultMaterialInst;

	UPROPERTY(EditAnywhere, Category = "Material")
	FNiagaraParameterBinding InstancedMaterialParamBinding;

private:
	void InitBindingMembers();

	UMaterialInstanceDynamic* GetMaterialInstance(FNiagaraSystemInstance* SystemInstance) const;
};