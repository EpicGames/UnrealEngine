// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceMaterialParameterCollection.h"

#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSystemInstance.h"

#include "Async/Async.h"
#include "Engine/World.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "UObject/WeakInterfacePtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceMaterialParameterCollection)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceMaterialParameterCollection"

namespace NDIMPCPrivate
{
	static const FName SetScalarParameterFuncName(TEXT("SetScalarParameter"));
	static const FName SetVector4ParameterFuncName(TEXT("SetVector4Parameter"));

	struct FInstanceData_GT
	{
		FNiagaraParameterDirectBinding<UObject*>				ObjectBinding;
		TWeakObjectPtr<UMaterialParameterCollectionInstance>	MPC;

		void ResolveParameterCollection(FNiagaraSystemInstance* SystemInstance, UMaterialParameterCollection* DefaultCollection)
		{
			UMaterialParameterCollection* BoundCollection = DefaultCollection;
			if (UObject* BoundObject = ObjectBinding.GetValue())
			{
				if (UMaterialParameterCollectionInstance* AsInstance = Cast<UMaterialParameterCollectionInstance>(BoundObject))
				{
					MPC = AsInstance;
					return;
				}
				if (UMaterialParameterCollection* AsCollection = Cast<UMaterialParameterCollection>(BoundObject))
				{
					BoundCollection = AsCollection;
				}
			}

			if (BoundCollection)
			{
				UWorld* World = SystemInstance->GetWorld();
				MPC = World->GetParameterCollectionInstance(BoundCollection);
			}
		}
	};

	static void VMSetScalarParameter(FVectorVMExternalFunctionContext& Context, FName ParameterName)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GT> InstanceData(Context);
		FNDIInputParam<float> InValue(Context);

		TWeakObjectPtr<UMaterialParameterCollectionInstance> MPC = InstanceData->MPC;
		if (MPC.IsValid())
		{
			AsyncTask(
				ENamedThreads::GameThread,
				[WeakMPC=MPC, ParameterName, Value=InValue.Get()]()
				{
					if ( UMaterialParameterCollectionInstance* MPC = WeakMPC.Get() )
					{
						MPC->SetScalarParameterValue(ParameterName, Value);
					}
				}
			);
		}
	}

	static void VMSetVector4Parameter(FVectorVMExternalFunctionContext& Context, FName ParameterName)
	{
		VectorVM::FUserPtrHandler<FInstanceData_GT> InstanceData(Context);
		FNDIInputParam<FVector4f> InValue(Context);

		TWeakObjectPtr<UMaterialParameterCollectionInstance> MPC = InstanceData->MPC;
		if (MPC.IsValid())
		{
			AsyncTask(
				ENamedThreads::GameThread,
				[WeakMPC=MPC, ParameterName, Value=InValue.Get()]()
				{
					if (UMaterialParameterCollectionInstance* MPC = WeakMPC.Get())
					{
						MPC->SetVectorParameterValue(ParameterName, Value);
					}
				}
			);
		}
	}
}

UNiagaraDataInterfaceMaterialParameterCollection::UNiagaraDataInterfaceMaterialParameterCollection(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
}

void UNiagaraDataInterfaceMaterialParameterCollection::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		const ENiagaraTypeRegistryFlags Flags =
			ENiagaraTypeRegistryFlags::AllowUserVariable |
			ENiagaraTypeRegistryFlags::AllowEmitterVariable |
			ENiagaraTypeRegistryFlags::AllowSystemVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;

		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);

#if WITH_EDITORONLY_DATA
		CollectionBinding.SetUsage(ENiagaraParameterBindingUsage::User);
		CollectionBinding.SetAllowedObjects({ UObject::StaticClass() });
#endif
	}
}

bool UNiagaraDataInterfaceMaterialParameterCollection::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	NDIMPCPrivate::FInstanceData_GT* InstanceData = new (PerInstanceData) NDIMPCPrivate::FInstanceData_GT();
	InstanceData->ObjectBinding.Init(SystemInstance->GetInstanceParameters(), CollectionBinding.ResolvedParameter);
	InstanceData->ResolveParameterCollection(SystemInstance, DefaultCollection);
	
	return true;
}

bool UNiagaraDataInterfaceMaterialParameterCollection::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	NDIMPCPrivate::FInstanceData_GT* InstanceData = static_cast<NDIMPCPrivate::FInstanceData_GT*>(PerInstanceData);
	InstanceData->ResolveParameterCollection(SystemInstance, DefaultCollection);
	return false;
}

bool UNiagaraDataInterfaceMaterialParameterCollection::CanExecuteOnTarget(ENiagaraSimTarget Target) const
{
	return Target == ENiagaraSimTarget::CPUSim;
}

int32 UNiagaraDataInterfaceMaterialParameterCollection::PerInstanceDataSize() const
{
	return sizeof(NDIMPCPrivate::FInstanceData_GT);
}

void UNiagaraDataInterfaceMaterialParameterCollection::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	auto* InstanceData = static_cast<NDIMPCPrivate::FInstanceData_GT*>(PerInstanceData);
	InstanceData->~FInstanceData_GT();
}

bool UNiagaraDataInterfaceMaterialParameterCollection::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	auto* OtherTyped = CastChecked< UNiagaraDataInterfaceMaterialParameterCollection >(Destination);

	OtherTyped->DefaultCollection = DefaultCollection;
	OtherTyped->CollectionBinding = CollectionBinding;

	return true;
}

bool UNiagaraDataInterfaceMaterialParameterCollection::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfaceMaterialParameterCollection* OtherTyped = CastChecked<const UNiagaraDataInterfaceMaterialParameterCollection>(Other);

	return OtherTyped->DefaultCollection == DefaultCollection
		&& OtherTyped->CollectionBinding == CollectionBinding;
}


void UNiagaraDataInterfaceMaterialParameterCollection::GetFunctions(TArray< FNiagaraFunctionSignature >& OutFunctions)
{
	FNiagaraFunctionSignature SigTemplate;
	SigTemplate.bMemberFunction = true;
	SigTemplate.bRequiresExecPin = true;
	SigTemplate.bRequiresContext = false;
	SigTemplate.bSupportsCPU = true;
	SigTemplate.bSupportsGPU = false;
	SigTemplate.FunctionSpecifiers.Add(FName("Parameter Name"));
	SigTemplate.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("MPC DI")));

	// Scalar parameters
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(SigTemplate);
		Sig.Name = NDIMPCPrivate::SetScalarParameterFuncName;
		Sig.SetDescription(LOCTEXT("SetScalarParameterFuncDesc", "Set a scalar parameter value on the bound parameter collection"));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
	}

	// Vector4 parameters
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Emplace_GetRef(SigTemplate);
		Sig.Name = NDIMPCPrivate::SetVector4ParameterFuncName;
		Sig.SetDescription(LOCTEXT("SetVector4ParameterFuncDesc", "Set a vector 4 parameter value on the bound parameter collection"));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
	}
}

void UNiagaraDataInterfaceMaterialParameterCollection::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	if (BindingInfo.Name == NDIMPCPrivate::SetScalarParameterFuncName)
	{
		OutFunc = FVMExternalFunction::CreateLambda(
			[ParamName=BindingInfo.FunctionSpecifiers[0].Value](FVectorVMExternalFunctionContext& Context)
			{
				NDIMPCPrivate::VMSetScalarParameter(Context, ParamName);
			}
		);
	}
	else if (BindingInfo.Name == NDIMPCPrivate::SetVector4ParameterFuncName)
	{
		OutFunc = FVMExternalFunction::CreateLambda(
			[ParamName=BindingInfo.FunctionSpecifiers[0].Value](FVectorVMExternalFunctionContext& Context)
			{
				NDIMPCPrivate::VMSetVector4Parameter(Context, ParamName);
			}
		);
	}
}

#undef LOCTEXT_NAMESPACE
