// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialParameterCollection.h"
#include "NiagaraCommon.h"
#include "NiagaraDataInterface.h"
#include "NiagaraParameterBinding.h"
#include "NiagaraShared.h"
#include "VectorVM.h"

#include "NiagaraDataInterfaceMaterialParameterCollection.generated.h"

UCLASS(EditInlineNew, Category = "Material", CollapseCategories, meta = (DisplayName = "Material Parameter Collection"), MinimalAPI)
class UNiagaraDataInterfaceMaterialParameterCollection : public UNiagaraDataInterface
{
	GENERATED_BODY()

public:
	UNiagaraDataInterfaceMaterialParameterCollection(const FObjectInitializer& Initializer);

	// UObject Interface
	virtual void PostInitProperties() override;
	// UObject Interface End

	// UNiagaraDataInterface Interface
	virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override;
	virtual int32 PerInstanceDataSize() const override;
	virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
	virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual void GetFunctions(TArray< FNiagaraFunctionSignature >& OutFunctions) override;
	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	virtual bool HasPreSimulateTick() const override { return true; }
	// UNiagaraDataInterface Interface

	/** Find the parameter collection on the world that matches this type. */
	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialParameterCollection> DefaultCollection;

	/** Bind to either a UMaterialParameterCollection or UMaterialParameterCollectionInstance. */
	UPROPERTY(EditAnywhere, Category = "Material")
	FNiagaraParameterBinding CollectionBinding;
};
