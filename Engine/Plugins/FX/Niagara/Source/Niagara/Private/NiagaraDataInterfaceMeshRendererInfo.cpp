// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceMeshRendererInfo.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraMeshRendererProperties.h"
#include "NiagaraRenderer.h"
#include "NiagaraShaderParametersBuilder.h"

#include "Engine/StaticMesh.h"
#include "RenderingThread.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceMeshRendererInfo)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceMeshRendererInfo"

namespace NDIMeshRendererInfoInternal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
		SHADER_PARAMETER(uint32,			bSubImageBlend)
		SHADER_PARAMETER(FVector2f,			SubImageSize)
		SHADER_PARAMETER(uint32,			NumMeshes)
		SHADER_PARAMETER_SRV(Buffer<float>,	MeshDataBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceMeshRendererInfoTemplate.ush");

	static const FName GetNumMeshesName("GetNumMeshes");
	static const FName GetMeshLocalBoundsName("GetMeshLocalBounds");
	static const FName GetSubUVDetailsName("GetSubUVDetails");
}

enum class ENDIMeshRendererInfoVersion : uint32
{
	InitialVersion = 0,
	AddSizeToMeshLocalBounds,

	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

/** The render thread proxy of the data interface */
struct FNDIMeshRendererInfoProxy : public FNiagaraDataInterfaceProxy
{
	bool		bSubImageBlend = false;
	FVector2f	SubImageSize = FVector2f::ZeroVector;
	uint32		NumMeshes;
	FReadBuffer	GPUMeshData;		//-OPT: Can shared if pointing to same resources

	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceMeshRendererInfo::UNiagaraDataInterfaceMeshRendererInfo(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNDIMeshRendererInfoProxy);
	MarkRenderDataDirty();
}

void UNiagaraDataInterfaceMeshRendererInfo::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		
		// We can't allow user variables of this type because it will cause components to have external reference (the renderer)
		Flags &= ~ENiagaraTypeRegistryFlags::AllowUserVariable;

		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}	
}

void UNiagaraDataInterfaceMeshRendererInfo::PostLoad()
{
	Super::PostLoad();

	if (MeshRenderer)
	{
		MeshRenderer->ConditionalPostLoad();
		OnMeshRendererChanged(MeshRenderer);
	}
}

void UNiagaraDataInterfaceMeshRendererInfo::BeginDestroy()
{
#if WITH_EDITOR
	OnMeshRendererChanged(nullptr);
#endif
	Super::BeginDestroy();
}

#if WITH_EDITOR

void UNiagaraDataInterfaceMeshRendererInfo::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (MeshRenderer && PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceMeshRendererInfo, MeshRenderer))
	{		
		OnMeshRendererChanged(nullptr);
	}
}

void UNiagaraDataInterfaceMeshRendererInfo::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// If coming from undo, property will be nullptr and since we copy the info, we need to reacquire if new.
	if (PropertyChangedEvent.Property == nullptr || (PropertyChangedEvent.Property &&
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceMeshRendererInfo, MeshRenderer)))
	{
		OnMeshRendererChanged(MeshRenderer);
	}
}

#endif // WITH_EDITOR

void UNiagaraDataInterfaceMeshRendererInfo::OnMeshRendererChanged(UNiagaraMeshRendererProperties* NewMeshRenderer)
{
#if WITH_EDITOR
	if (MeshRenderer)
	{
		MeshRenderer->OnChanged().Remove(OnMeshRendererChangedHandle);
		OnMeshRendererChangedHandle.Reset();
	}

	if (NewMeshRenderer)
	{
		OnMeshRendererChangedHandle = NewMeshRenderer->OnChanged().AddLambda([this]() { UpdateCachedData(); });
	}
#endif
	UpdateCachedData();
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceMeshRendererInfo::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.AddDefaulted_GetRef();
		Signature.Name = NDIMeshRendererInfoInternal::GetNumMeshesName;
		Signature.FunctionVersion = (uint32)ENDIMeshRendererInfoVersion::LatestVersion;
		Signature.Description = LOCTEXT("GetNumMeshesInRendererDesc", "Retrieves the number of meshes on the mesh renderer by index, or -1 if the index is invalid.");
		Signature.bMemberFunction = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("MeshRendererInfo")));
		Signature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("OutNumMeshes")));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.AddDefaulted_GetRef();
		Signature.Name = NDIMeshRendererInfoInternal::GetMeshLocalBoundsName;
		Signature.FunctionVersion = (uint32)ENDIMeshRendererInfoVersion::LatestVersion;
		Signature.Description = LOCTEXT("GetMeshLocalBoundsDesc", "Retrieves the local bounds of the specified mesh's vertices.");
		Signature.bMemberFunction = true;
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("MeshRendererInfo")));
		Signature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MeshIndex")));
		Signature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("OutMinBounds")));
		Signature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("OutMaxBounds")));
		Signature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("OutSize")));
	}
	{
		FNiagaraFunctionSignature& Signature = OutFunctions.AddDefaulted_GetRef();
		Signature.Name = NDIMeshRendererInfoInternal::GetSubUVDetailsName;
		Signature.FunctionVersion = (uint32)ENDIMeshRendererInfoVersion::LatestVersion;
		Signature.bMemberFunction = true;
		Signature.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("MeshRendererInfo"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("BlendEnabled"));
		Signature.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("SubImageSize"));
	}
}
#endif

void UNiagaraDataInterfaceMeshRendererInfo::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	if (BindingInfo.Name == NDIMeshRendererInfoInternal::GetNumMeshesName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceMeshRendererInfo::VMGetNumMeshes);
	}
	else if (BindingInfo.Name == NDIMeshRendererInfoInternal::GetMeshLocalBoundsName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceMeshRendererInfo::VMGetMeshLocalBounds);
	}
	else if (BindingInfo.Name == NDIMeshRendererInfoInternal::GetSubUVDetailsName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceMeshRendererInfo::VMGetSubUVDetails);
	}
}

bool UNiagaraDataInterfaceMeshRendererInfo::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	auto OtherTyped = CastChecked<const UNiagaraDataInterfaceMeshRendererInfo>(Other);
	return MeshRenderer == OtherTyped->MeshRenderer;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceMeshRendererInfo::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIMeshRendererInfoInternal::TemplateShaderFile);
	bSuccess &= InVisitor->UpdateShaderParameters<NDIMeshRendererInfoInternal::FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceMeshRendererInfo::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIMeshRendererInfoInternal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceMeshRendererInfo::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	static const TSet<FName> ValidGpuFunction =
	{
		NDIMeshRendererInfoInternal::GetNumMeshesName,
		NDIMeshRendererInfoInternal::GetMeshLocalBoundsName,
		NDIMeshRendererInfoInternal::GetSubUVDetailsName,
	};

	return ValidGpuFunction.Contains(FunctionInfo.DefinitionName);
}
#endif

void UNiagaraDataInterfaceMeshRendererInfo::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIMeshRendererInfoInternal::FShaderParameters>();
}

void UNiagaraDataInterfaceMeshRendererInfo::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIMeshRendererInfoInternal;

	FNDIMeshRendererInfoProxy& DIProxy = Context.GetProxy<FNDIMeshRendererInfoProxy>();
	FShaderParameters* ShaderParameters	= Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->bSubImageBlend	= DIProxy.bSubImageBlend;
	ShaderParameters->SubImageSize		= DIProxy.SubImageSize;
	ShaderParameters->NumMeshes			= DIProxy.NumMeshes;
	ShaderParameters->MeshDataBuffer	= FNiagaraRenderer::GetSrvOrDefaultFloat(DIProxy.GPUMeshData.SRV);
}

#if WITH_EDITOR

bool UNiagaraDataInterfaceMeshRendererInfo::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	bool bWasChanged = false;
	if (FunctionSignature.Name == NDIMeshRendererInfoInternal::GetMeshLocalBoundsName && FunctionSignature.Outputs.Num() == 2)
	{
		FunctionSignature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("OutSize")));
		bWasChanged = true;
	}

	FunctionSignature.FunctionVersion = (uint32)ENDIMeshRendererInfoVersion::LatestVersion;

	return bWasChanged;
}

void UNiagaraDataInterfaceMeshRendererInfo::GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	if (MeshRenderer == nullptr)
	{
		FNiagaraDataInterfaceFeedback NoMeshRendererSelectedWarning(
			LOCTEXT("NoRendererSelectedWarning", "A Mesh Renderer applied to an emitter in this system is expected to be selected here"),
			LOCTEXT("NoRendererSelectedWarningSummary", "No Mesh Renderer selected"),
			FNiagaraDataInterfaceFix()
		);
		OutWarnings.Add(NoMeshRendererSelectedWarning);
	}
	else if (!MeshRenderer->GetIsEnabled())
	{
		FNiagaraDataInterfaceFeedback MeshRendererDisabledWarning(
			LOCTEXT("RendererDisabledWarning", "The selected Mesh Renderer is disabled"),
			LOCTEXT("RendererDisabledWarningSummary", "Mesh Renderer is disabled"),
			FNiagaraDataInterfaceFix::CreateLambda(
				[this]()
				{
					MeshRenderer->SetIsEnabled(true);
					return true;
				}
			)
		);
		OutWarnings.Add(MeshRendererDisabledWarning);
	}
	// Check for the renderer being referenced but no longer referenced by it's parent emitter as that's likely a invalid dangling reference
	else if ( !MeshRenderer->IsContainedInEmitterRendererArray() )
	{
		OutWarnings.Emplace(
			LOCTEXT("OrphanedRendererWarning", "The selected Mesh Renderer is invalid"),
			LOCTEXT("OrphanedRendererWarningSummary", "Mesh Renderer is orphaned and no longer valid"),
			FNiagaraDataInterfaceFix::CreateLambda(
				[this]()
				{
					const FScopedTransaction Transaction(LOCTEXT("ClearMeshRenderer", "Clear Mesh Renderer"));
					Modify();
					MeshRenderer = nullptr;
					return true;
				}
			)
		);
	}
}

#endif // WITH_EDITOR

bool UNiagaraDataInterfaceMeshRendererInfo::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}


	auto OtherTyped = CastChecked<UNiagaraDataInterfaceMeshRendererInfo>(Destination);
	OtherTyped->OnMeshRendererChanged(nullptr);
	OtherTyped->MeshRenderer = MeshRenderer;
	OtherTyped->OnMeshRendererChanged(MeshRenderer);

	return true;
}

void UNiagaraDataInterfaceMeshRendererInfo::PushToRenderThreadImpl()
{
	FNDIMeshRendererInfoProxy* TypedProxy = GetProxyAs<FNDIMeshRendererInfoProxy>();

	const UNiagaraMeshRendererProperties* Properties = MeshRenderer ? MeshRenderer.Get() : GetDefault<UNiagaraMeshRendererProperties>();

	const bool bSubImageBlend_RT	= Properties->bSubImageBlend;
	const FVector2f SubImageSize_RT	= FVector2f(Properties->SubImageSize);
	ENQUEUE_RENDER_COMMAND(FDIMeshRendererInfoPushToRT)
	(
		[TypedProxy, CachedMeshData_RT=CachedMeshData, bSubImageBlend_RT, SubImageSize_RT](FRHICommandList& RHICmdList)
		{
			TypedProxy->bSubImageBlend = bSubImageBlend_RT;
			TypedProxy->SubImageSize = SubImageSize_RT;
			TypedProxy->NumMeshes = CachedMeshData_RT.Num();
			TypedProxy->GPUMeshData.Release();
			if ( TypedProxy->NumMeshes > 0 )
			{
				TypedProxy->GPUMeshData.Initialize(RHICmdList, TEXT("UNiagaraDataInterfaceMeshRendererInfo"), sizeof(float), CachedMeshData_RT.Num() * 6, PF_R32_FLOAT, BUF_Static);
				float* GpuMeshData = reinterpret_cast<float*>(RHICmdList.LockBuffer(TypedProxy->GPUMeshData.Buffer, 0, TypedProxy->GPUMeshData.NumBytes, RLM_WriteOnly));
				for ( const UNiagaraDataInterfaceMeshRendererInfo::FMeshData& MeshData : CachedMeshData_RT )
				{
					*GpuMeshData++ = MeshData.MinLocalBounds.X;
					*GpuMeshData++ = MeshData.MinLocalBounds.Y;
					*GpuMeshData++ = MeshData.MinLocalBounds.Z;

					*GpuMeshData++ = MeshData.MaxLocalBounds.X;
					*GpuMeshData++ = MeshData.MaxLocalBounds.Y;
					*GpuMeshData++ = MeshData.MaxLocalBounds.Z;
				}
				RHICmdList.UnlockBuffer(TypedProxy->GPUMeshData.Buffer);
			}
		}
	);
}

void UNiagaraDataInterfaceMeshRendererInfo::UpdateCachedData()
{
	CachedMeshData.Empty();
	if ( MeshRenderer != nullptr )
	{
		CachedMeshData.AddDefaulted(MeshRenderer->Meshes.Num());
		for (int32 i = 0; i < MeshRenderer->Meshes.Num(); ++i)
		{
			FNiagaraMeshRendererMeshProperties& MeshProperties = MeshRenderer->Meshes[i];
			if (MeshProperties.Mesh == nullptr)
			{
				continue;
			}

			const FBox LocalBounds = MeshProperties.Mesh->GetExtendedBounds().GetBox();
			if (LocalBounds.IsValid)
			{
				FMeshData& CachedMesh = CachedMeshData[i];
				CachedMesh.MinLocalBounds = FVector3f(LocalBounds.Min * MeshProperties.Scale);
				CachedMesh.MaxLocalBounds = FVector3f(LocalBounds.Max * MeshProperties.Scale);
			}
		}
	}
	MarkRenderDataDirty();
}

void UNiagaraDataInterfaceMeshRendererInfo::VMGetNumMeshes(FVectorVMExternalFunctionContext& Context)
{
	FNDIOutputParam<int32> OutNum(Context);

	const int32 NumMeshes = CachedMeshData.Num();
	for (int32 Instance = 0; Instance < Context.GetNumInstances(); ++Instance)
	{
		OutNum.SetAndAdvance(NumMeshes);
	}
}

void UNiagaraDataInterfaceMeshRendererInfo::VMGetMeshLocalBounds(FVectorVMExternalFunctionContext& Context)
{
	FNDIInputParam<int32> InMeshIdx(Context);
	FNDIOutputParam<FVector3f> OutMinBounds(Context);
	FNDIOutputParam<FVector3f> OutMaxBounds(Context);
	FNDIOutputParam<FVector3f> OutSize(Context);

	for (int32 Instance = 0; Instance < Context.GetNumInstances(); ++Instance)
	{
		FVector3f MinLocalBounds(ForceInitToZero);
		FVector3f MaxLocalBounds(ForceInitToZero);
		if (CachedMeshData.Num() > 0)
		{
			const int32 MeshIdx = FMath::Clamp(InMeshIdx.GetAndAdvance(), 0, CachedMeshData.Num() - 1);
			const FMeshData& MeshData = CachedMeshData[MeshIdx];
			MinLocalBounds = MeshData.MinLocalBounds;
			MaxLocalBounds = MeshData.MaxLocalBounds;
		}
		OutMinBounds.SetAndAdvance(MinLocalBounds);
		OutMaxBounds.SetAndAdvance(MaxLocalBounds);
		OutSize.SetAndAdvance(MaxLocalBounds - MinLocalBounds);
	}
}

void UNiagaraDataInterfaceMeshRendererInfo::VMGetSubUVDetails(FVectorVMExternalFunctionContext& Context)
{
	FNDIOutputParam<bool> OutBlendEnabled(Context);
	FNDIOutputParam<FVector2f> OutSubImageSize(Context);

	const UNiagaraMeshRendererProperties* Properties = MeshRenderer ? MeshRenderer.Get() : GetDefault<UNiagaraMeshRendererProperties>();
	const FVector2f SubImageSize(Properties->SubImageSize);
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutBlendEnabled.SetAndAdvance(Properties->bSubImageBlend != 0);
		OutSubImageSize.SetAndAdvance(SubImageSize);
	}
}

#undef LOCTEXT_NAMESPACE
