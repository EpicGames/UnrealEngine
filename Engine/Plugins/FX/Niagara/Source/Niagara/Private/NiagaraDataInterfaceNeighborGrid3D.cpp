// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceNeighborGrid3D.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderGraphUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceNeighborGrid3D)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceNeighborGrid3D"

static const FString MaxNeighborsPerCellName(TEXT("_MaxNeighborsPerCellValue"));
static const FString ParticleNeighborsName(TEXT("_ParticleNeighbors"));
static const FString ParticleNeighborCountName(TEXT("_ParticleNeighborCount"));
static const FString OutputParticleNeighborsName(TEXT("_OutputParticleNeighbors"));
static const FString OutputParticleNeighborCountName(TEXT("_OutputParticleNeighborCount"));

const FName UNiagaraDataInterfaceNeighborGrid3D::SetNumCellsFunctionName("SetNumCells");

// Global VM function names, also used by the shaders code generation methods.

static const FName MaxNeighborsPerCellFunctionName("MaxNeighborsPerCell");
static const FName NeighborGridIndexToLinearFunctionName("NeighborGridIndexToLinear");
static const FName GetParticleNeighborFunctionName("GetParticleNeighbor");
static const FName SetParticleNeighborFunctionName("SetParticleNeighbor");
static const FName GetParticleNeighborCountFunctionName("GetParticleNeighborCount");
static const FName SetParticleNeighborCountFunctionName("SetParticleNeighborCount");
static const FName AddParticleFunctionName("AddParticle");

static int32 GMaxNiagaraNeighborGridCells = (128*128*128*64);
static FAutoConsoleVariableRef CVarMaxNiagaraNeighborGridCells(
	TEXT("fx.MaxNiagaraNeighborGridCells"),
	GMaxNiagaraNeighborGridCells,
	TEXT("The max number of supported grid cells in Niagara. Overflowing this threshold will cause the sim to warn and fail. \n"),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////

void FNDINeighborGrid3DInstanceData_RT::ResizeBuffers(FRDGBuilder& GraphBuilder)
{
	const uint32 NumTotalCells = NumCells.X * NumCells.Y * NumCells.Z;
	const uint32 NumIntsInGridBuffer = NumTotalCells * MaxNeighborsPerCell;

	bNeedsRealloc = false;
	if (NumTotalCells > (uint32)GMaxNiagaraNeighborGridCells)
	{
		return;
	}

	NeighborhoodCountBuffer.Release();
	NeighborhoodBuffer.Release();

	NeighborhoodCountBuffer.Initialize(GraphBuilder, TEXT("NiagaraNeighborGrid3D::NeighborCount"), EPixelFormat::PF_R32_SINT, sizeof(int32), FMath::Max<uint32>(NumTotalCells, 1u), BUF_Static);
	NeighborhoodBuffer.Initialize(GraphBuilder, TEXT("NiagaraNeighborGrid3D::NeighborsGrid"), EPixelFormat::PF_R32_SINT, sizeof(int32), FMath::Max(NumIntsInGridBuffer, 1u), BUF_Static);
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceNeighborGrid3D::UNiagaraDataInterfaceNeighborGrid3D(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, MaxNeighborsPerCell(8)
{
	SetResolutionMethod = ESetResolutionMethod::CellSize;

	Proxy.Reset(new FNiagaraDataInterfaceProxyNeighborGrid3D());	
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceNeighborGrid3D::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	Super::GetFunctionsInternal(OutFunctions);

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetNumCellsFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MaxNeighborsPerCell")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = MaxNeighborsPerCellFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MaxNeighborsPerCell")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NeighborGridIndexToLinearFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Neighbor")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear Index")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetParticleNeighborFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NeighborIndex")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetParticleNeighborFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NeighborIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IGNORE")));

		Sig.bWriteFunction = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetParticleNeighborCountFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NeighborCount")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetParticleNeighborCountFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Increment")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("PrevNeighborCount")));

		Sig.bWriteFunction = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = AddParticleFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));		
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.bWriteFunction = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		OutFunctions.Add(Sig);
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceNeighborGrid3D, SetNumCells);
void UNiagaraDataInterfaceNeighborGrid3D::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	Super::GetVMExternalFunction(BindingInfo, InstanceData, OutFunc);

	// #todo(dmp): this overrides the empty function set by the super class
	if (BindingInfo.Name == UNiagaraDataInterfaceRWBase::WorldBBoxSizeFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceNeighborGrid3D::GetWorldBBoxSize);
	}
	else if (BindingInfo.Name == UNiagaraDataInterfaceRWBase::NumCellsFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceNeighborGrid3D::GetNumCells);
	}
	else if (BindingInfo.Name == MaxNeighborsPerCellFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceNeighborGrid3D::GetMaxNeighborsPerCell);
	}
	else if (BindingInfo.Name == SetNumCellsFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceNeighborGrid3D, SetNumCells)::Bind(this, OutFunc);
	}
	//else if (BindingInfo.Name == NeighborGridIndexToLinearFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == GetParticleNeighborFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == SetParticleNeighborFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == GetParticleNeighborCountFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == SetParticleNeighborCountFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
}

void UNiagaraDataInterfaceNeighborGrid3D::GetWorldBBoxSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDINeighborGrid3DInstanceData_GT> InstData(Context);

	FNDIOutputParam<FVector3f> OutWorldBounds(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutWorldBounds.SetAndAdvance((FVector3f)WorldBBoxSize);	// LWC_TODO: Precision Loss
	}
}

void UNiagaraDataInterfaceNeighborGrid3D::GetNumCells(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDINeighborGrid3DInstanceData_GT> InstData(Context);

	FNDIOutputParam<int32> NumCellsX(Context);
	FNDIOutputParam<int32> NumCellsY(Context);
	FNDIOutputParam<int32> NumCellsZ(Context);

	int32 TmpNumCellsX = InstData->NumCells.X;
	int32 TmpNumCellsY = InstData->NumCells.Y;
	int32 TmpNumCellsZ = InstData->NumCells.Z;

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		NumCellsX.SetAndAdvance(TmpNumCellsX);
		NumCellsY.SetAndAdvance(TmpNumCellsY);
		NumCellsZ.SetAndAdvance(TmpNumCellsZ);
	}
}

void UNiagaraDataInterfaceNeighborGrid3D::GetMaxNeighborsPerCell(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDINeighborGrid3DInstanceData_GT> InstData(Context);

	FNDIOutputParam<int32> OutMaxNeighborsPerCell(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutMaxNeighborsPerCell.SetAndAdvance(InstData->MaxNeighborsPerCell);
	}
}

bool UNiagaraDataInterfaceNeighborGrid3D::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceNeighborGrid3D* OtherTyped = CastChecked<const UNiagaraDataInterfaceNeighborGrid3D>(Other);

	return OtherTyped->MaxNeighborsPerCell == MaxNeighborsPerCell;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceNeighborGrid3D::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceNeighborGrid3D::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	OutHLSL.Appendf(TEXT("int			%s%s;\n"), *ParamInfo.DataInterfaceHLSLSymbol, *MaxNeighborsPerCellName);
	OutHLSL.Appendf(TEXT("Buffer<int>	%s%s;\n"), *ParamInfo.DataInterfaceHLSLSymbol, *ParticleNeighborsName);
	OutHLSL.Appendf(TEXT("Buffer<int>	%s%s;\n"), *ParamInfo.DataInterfaceHLSLSymbol, *ParticleNeighborCountName);
	OutHLSL.Appendf(TEXT("RWBuffer<int>	%s%s;\n"), *ParamInfo.DataInterfaceHLSLSymbol, *OutputParticleNeighborsName);
	OutHLSL.Appendf(TEXT("RWBuffer<int>	%s%s;\n"), *ParamInfo.DataInterfaceHLSLSymbol, *OutputParticleNeighborCountName);
}

bool UNiagaraDataInterfaceNeighborGrid3D::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	bool ParentRet = Super::GetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, OutHLSL);
	if (ParentRet)
	{
		return true;
	}

	TMap<FString, FStringFormatArg> FormatArgs =
	{
		{TEXT("FunctionName"),					FunctionInfo.InstanceName},
		{TEXT("DIName"),						ParamInfo.DataInterfaceHLSLSymbol},
		{TEXT("NumCellsName"),					ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{TEXT("UnitToUVName"),					ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{TEXT("MaxNeighborsPerCellName"),		ParamInfo.DataInterfaceHLSLSymbol + MaxNeighborsPerCellName},
		{TEXT("ParticleNeighbors"),				ParamInfo.DataInterfaceHLSLSymbol + ParticleNeighborsName},
		{TEXT("ParticleNeighborCount"),			ParamInfo.DataInterfaceHLSLSymbol + ParticleNeighborCountName},
		{TEXT("OutputParticleNeighbors"),		ParamInfo.DataInterfaceHLSLSymbol + OutputParticleNeighborsName},
		{TEXT("OutputParticleNeighborCount"),	ParamInfo.DataInterfaceHLSLSymbol + OutputParticleNeighborCountName},
	};

	if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::NumCellsFunctionName)
	{
		static const TCHAR* FormatHLSL = TEXT(R"(
			void {FunctionName}(out int OutNumCellsX, out int OutNumCellsY, out int OutNumCellsZ)
			{
				OutNumCellsX = {NumCellsName}.x;
				OutNumCellsY = {NumCellsName}.y;
				OutNumCellsZ = {NumCellsName}.z;
			}
		)");
		OutHLSL += FString::Format(FormatHLSL, FormatArgs);
	}
	else if (FunctionInfo.DefinitionName == MaxNeighborsPerCellFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(out int Out_MaxNeighborsPerCell)
			{
				Out_MaxNeighborsPerCell = {MaxNeighborsPerCellName};
			}
		)");
		OutHLSL += FString::Format(FormatSample, FormatArgs);
		return true;
	}
	else if (FunctionInfo.DefinitionName == NeighborGridIndexToLinearFunctionName)
	{
		static const TCHAR *FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_Neighbor, out int Out_Linear)
			{
				Out_Linear = In_Neighbor + In_IndexX * {MaxNeighborsPerCellName} + In_IndexY * {MaxNeighborsPerCellName}*{NumCellsName}.x + In_IndexZ * {MaxNeighborsPerCellName}*{NumCellsName}.x*{NumCellsName}.y;
			}
		)");
		OutHLSL += FString::Format(FormatBounds, FormatArgs);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetParticleNeighborFunctionName)
	{
		static const TCHAR *FormatBounds = TEXT(R"(
			void {FunctionName}(int In_Index, out int Out_ParticleNeighborIndex)
			{
				Out_ParticleNeighborIndex = {ParticleNeighbors}[In_Index];				
			}
		)");
		OutHLSL += FString::Format(FormatBounds, FormatArgs);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetParticleNeighborFunctionName)
	{
		static const TCHAR *FormatBounds = TEXT(R"(
			void {FunctionName}(int In_Index, int In_ParticleNeighborIndex, out int Out_Ignore)
			{
				{OutputParticleNeighbors}[In_Index] = In_ParticleNeighborIndex;				
				Out_Ignore = 0;
			}
		)");
		OutHLSL += FString::Format(FormatBounds, FormatArgs);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetParticleNeighborCountFunctionName)
	{
		static const TCHAR *FormatBounds = TEXT(R"(
			void {FunctionName}(int In_Index, out int Out_ParticleNeighborIndex)
			{
				Out_ParticleNeighborIndex = {ParticleNeighborCount}[In_Index];				
			}
		)");
		OutHLSL += FString::Format(FormatBounds, FormatArgs);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetParticleNeighborCountFunctionName)
	{
		static const TCHAR *FormatBounds = TEXT(R"(
			void {FunctionName}(int In_Index, int In_Increment, out int PreviousNeighborCount)
			{				
				InterlockedAdd({OutputParticleNeighborCount}[In_Index], In_Increment, PreviousNeighborCount);				
			}
		)");
		OutHLSL += FString::Format(FormatBounds, FormatArgs);
		return true;
	}
	else if (FunctionInfo.DefinitionName == AddParticleFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int IndexX, int IndexY, int IndexZ, int ParticleIndex, out bool Success)
			{		
				Success = false;
		
				if (IndexX >= 0 && IndexX < {NumCellsName}.x && 
					IndexY >= 0 && IndexY < {NumCellsName}.y && 
					IndexZ >= 0 && IndexZ < {NumCellsName}.z)
				{
					int LinearIndex = IndexX + IndexY * {NumCellsName}.x + IndexZ * {NumCellsName}.x * {NumCellsName}.y;

					int PreviousNeighborCount;
					InterlockedAdd({OutputParticleNeighborCount}[LinearIndex], 1, PreviousNeighborCount);					

					if (PreviousNeighborCount < {MaxNeighborsPerCellName})
					{
						Success = true;

						int NeighborGridLinear = 
							PreviousNeighborCount + IndexX * {MaxNeighborsPerCellName} + 
							IndexY * {MaxNeighborsPerCellName}*{NumCellsName}.x + 
							IndexZ * {MaxNeighborsPerCellName}*{NumCellsName}.x*{NumCellsName}.y;
						
						{OutputParticleNeighbors}[NeighborGridLinear] = ParticleIndex;					
					}		
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, FormatArgs);
		return true;
	}

	return false;
}
#endif

void UNiagaraDataInterfaceNeighborGrid3D::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceNeighborGrid3D::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyNeighborGrid3D& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyNeighborGrid3D>();
	FNDINeighborGrid3DInstanceData_RT* ProxyData = DIProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	if (ProxyData && ProxyData->NeighborhoodBuffer.IsValid())
	{
		ShaderParameters->NumCells		= ProxyData->NumCells;
		ShaderParameters->UnitToUV		= FVector3f(1.0f) / FVector3f(ProxyData->NumCells);
		ShaderParameters->CellSize.X	= float(ProxyData->WorldBBoxSize.X / double(ProxyData->NumCells.X));
		ShaderParameters->CellSize.Y	= float(ProxyData->WorldBBoxSize.Y / double(ProxyData->NumCells.Y));
		ShaderParameters->CellSize.Z	= float(ProxyData->WorldBBoxSize.Z / double(ProxyData->NumCells.Z));
		ShaderParameters->WorldBBoxSize	= FVector3f(ProxyData->WorldBBoxSize);
		ShaderParameters->MaxNeighborsPerCellValue = ProxyData->MaxNeighborsPerCell;

		if (Context.IsOutputStage())
		{
			ShaderParameters->ParticleNeighbors = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
			ShaderParameters->ParticleNeighborCount = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
			ShaderParameters->OutputParticleNeighbors = ProxyData->NeighborhoodBuffer.GetOrCreateUAV(GraphBuilder);
			ShaderParameters->OutputParticleNeighborCount = ProxyData->NeighborhoodCountBuffer.GetOrCreateUAV(GraphBuilder);
		}
		else
		{
			ShaderParameters->ParticleNeighbors = ProxyData->NeighborhoodBuffer.GetOrCreateSRV(GraphBuilder);
			ShaderParameters->ParticleNeighborCount = ProxyData->NeighborhoodCountBuffer.GetOrCreateSRV(GraphBuilder);
			ShaderParameters->OutputParticleNeighbors = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
			ShaderParameters->OutputParticleNeighborCount = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
		}
	}
	else
	{
		ShaderParameters->NumCells		= FIntVector::ZeroValue;
		ShaderParameters->UnitToUV		= FVector3f::ZeroVector;
		ShaderParameters->CellSize		= FVector3f::ZeroVector;
		ShaderParameters->WorldBBoxSize	= FVector3f::ZeroVector;

		ShaderParameters->MaxNeighborsPerCellValue	= 0;
		ShaderParameters->ParticleNeighbors = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
		ShaderParameters->ParticleNeighborCount = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
		ShaderParameters->OutputParticleNeighbors = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
		ShaderParameters->OutputParticleNeighborCount = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
	}
}

bool UNiagaraDataInterfaceNeighborGrid3D::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	if (UE::PixelFormat::HasCapabilities(EPixelFormat::PF_R32_SINT, EPixelFormatCapabilities::TypedUAVLoad | EPixelFormatCapabilities::TypedUAVStore | EPixelFormatCapabilities::BufferAtomics) == false)
	{
		return false;
	}

	FNDINeighborGrid3DInstanceData_GT* InstanceData = new (PerInstanceData) FNDINeighborGrid3DInstanceData_GT();

	FNiagaraDataInterfaceProxyNeighborGrid3D* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyNeighborGrid3D>();

	FIntVector RT_NumCells = NumCells;
	uint32 RT_MaxNeighborsPerCell = MaxNeighborsPerCell;	
	FVector RT_WorldBBoxSize = WorldBBoxSize;

	FVector::FReal TmpCellSize = RT_WorldBBoxSize[0] / RT_NumCells[0];

	if (SetResolutionMethod == ESetResolutionMethod::MaxAxis)
	{
		TmpCellSize = FMath::Max(FMath::Max(WorldBBoxSize.X, WorldBBoxSize.Y), WorldBBoxSize.Z) / NumCellsMaxAxis;
	}
	else if (SetResolutionMethod == ESetResolutionMethod::CellSize)
	{
		TmpCellSize = CellSize;
	}

	// compute world bounds and padding based on cell size
	if (SetResolutionMethod == ESetResolutionMethod::MaxAxis || SetResolutionMethod == ESetResolutionMethod::CellSize)
	{
		RT_NumCells.X = int32(FMath::FloorToInt(WorldBBoxSize.X / TmpCellSize));
		RT_NumCells.Y = int32(FMath::FloorToInt(WorldBBoxSize.Y / TmpCellSize));
		RT_NumCells.Z = int32(FMath::FloorToInt(WorldBBoxSize.Z / TmpCellSize));

		// Pad grid by 1 cell if our computed bounding box is too small
		if (WorldBBoxSize.X > WorldBBoxSize.Y && WorldBBoxSize.X > WorldBBoxSize.Z)
		{
			if (!FMath::IsNearlyEqual(TmpCellSize * RT_NumCells.Y, WorldBBoxSize.Y))
			{
				RT_NumCells.Y++;
			}

			if (!FMath::IsNearlyEqual(TmpCellSize * RT_NumCells.Z, WorldBBoxSize.Z))
			{
				RT_NumCells.Z++;
			}
		}
		else if (WorldBBoxSize.Y > WorldBBoxSize.X && WorldBBoxSize.Y > WorldBBoxSize.Z)
		{
			if (!FMath::IsNearlyEqual(TmpCellSize * RT_NumCells.X, WorldBBoxSize.X))
			{
				RT_NumCells.X++;
			}

			if (!FMath::IsNearlyEqual(TmpCellSize * RT_NumCells.Z, WorldBBoxSize.Z))
			{
				RT_NumCells.Z++;
			}
		}
		else if (WorldBBoxSize.Z > WorldBBoxSize.X && WorldBBoxSize.Z > WorldBBoxSize.Y)
		{
			if (!FMath::IsNearlyEqual(TmpCellSize * RT_NumCells.X, WorldBBoxSize.X))
			{
				RT_NumCells.X++;
			}

			if (!FMath::IsNearlyEqual(TmpCellSize * RT_NumCells.Y, WorldBBoxSize.Y))
			{
				RT_NumCells.Y++;
			}
		}

		RT_WorldBBoxSize = FVector(RT_NumCells.X, RT_NumCells.Y, RT_NumCells.Z) * TmpCellSize;		 
	}
	RT_NumCells.X = FMath::Max(RT_NumCells.X, 1);
	RT_NumCells.Y = FMath::Max(RT_NumCells.Y, 1);
	RT_NumCells.Z = FMath::Max(RT_NumCells.Z, 1);

	InstanceData->CellSize = float(TmpCellSize);
	InstanceData->WorldBBoxSize = RT_WorldBBoxSize;
	InstanceData->MaxNeighborsPerCell = RT_MaxNeighborsPerCell;	
	InstanceData->NumCells = RT_NumCells;

	const uint32 NumTotalCells = RT_NumCells.X * RT_NumCells.Y * RT_NumCells.Z * RT_MaxNeighborsPerCell;
	const uint64 MaxViewDimensionForTypedBuffer = GRHIGlobals.MaxViewDimensionForTypedBuffer;
	if (NumTotalCells == 0 || NumTotalCells > uint64(GMaxNiagaraNeighborGridCells))
	{
		UE_LOGF(LogNiagara, Error, "NiagaraDataInterfaceNeighborGrid3D - Invalid NumCells(%dx%dx%dx%d) with total = %u exceeds the maximum of %d for '%ls'.  Please adjust your NeighborGrid3D or increase fx.MaxNiagaraNeighborGridCells", RT_NumCells.X, RT_NumCells.Y, RT_NumCells.Z, RT_MaxNeighborsPerCell, NumTotalCells, GMaxNiagaraNeighborGridCells, *GetFullNameSafe(this));
		return false;
	}
	else if (NumTotalCells > MaxViewDimensionForTypedBuffer)
	{
		UE_LOGF(LogNiagara, Error, "NiagaraDataInterfaceNeighborGrid3D - Invalid NumCells(%dx%dx%dx%d) with total = %u exceeds the platform maximum of %llu for '%ls'.  Please reduce the memory usage.", RT_NumCells.X, RT_NumCells.Y, RT_NumCells.Z, RT_MaxNeighborsPerCell, NumTotalCells, MaxViewDimensionForTypedBuffer, *GetFullNameSafe(this));
		return false;
	}

	// @todo-threadsafety. This would be a race but I'm taking a ref here. Not ideal in the long term.
	// Push Updates to Proxy.
	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Proxy, RT_NumCells, RT_MaxNeighborsPerCell, RT_WorldBBoxSize, InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
		{
			check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));
			FNDINeighborGrid3DInstanceData_RT* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.Add(InstanceID);

			TargetData->NumCells = RT_NumCells;
			TargetData->MaxNeighborsPerCell = RT_MaxNeighborsPerCell;		
			TargetData->WorldBBoxSize = RT_WorldBBoxSize;
			TargetData->bNeedsRealloc = true;
		}
	);

	return true;
}

void UNiagaraDataInterfaceNeighborGrid3D::SetNumCells(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FNDINeighborGrid3DInstanceData_GT> InstData(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsX(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsY(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsZ(Context);
	VectorVM::FExternalFuncInputHandler<int> InMaxNeighborsPerCell(Context);
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const int NewNumCellsX = InNumCellsX.GetAndAdvance();
		const int NewNumCellsY = InNumCellsY.GetAndAdvance();
		const int NewNumCellsZ = InNumCellsZ.GetAndAdvance();
		const int NewMaxNeighborsPerCell = InMaxNeighborsPerCell.GetAndAdvance();
		bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1 && NumCells.X >= 0 && NumCells.Y >= 0 && NumCells.Z >= 0 && MaxNeighborsPerCell >= 0);

		const uint32 NumTotalCells = NewNumCellsX * NewNumCellsY * NewNumCellsZ * NewMaxNeighborsPerCell;
		const uint64 MaxViewDimensionForTypedBuffer = GRHIGlobals.MaxViewDimensionForTypedBuffer;
		if (NumTotalCells == 0 || NumTotalCells > uint64(GMaxNiagaraNeighborGridCells))
		{
			UE_LOGF(LogNiagara, Error, "NiagaraDataInterfaceNeighborGrid3D - Invalid NumCells(%dx%dx%dx%d) with total = %u exceeds the maximum of %d for '%ls'.  Please adjust your NeighborGrid3D or increase fx.MaxNiagaraNeighborGridCells", NewNumCellsX, NewNumCellsY, NewNumCellsZ, NewMaxNeighborsPerCell, NumTotalCells, GMaxNiagaraNeighborGridCells, *GetFullNameSafe(this));
			bSuccess = false;
		}
		else if (NumTotalCells > MaxViewDimensionForTypedBuffer)
		{
			UE_LOGF(LogNiagara, Error, "NiagaraDataInterfaceNeighborGrid3D - Invalid NumCells(%dx%dx%dx%d) with total = %u exceeds the platform maximum of %lld for '%ls'.  Please reduce the memory usage.", NewNumCellsX, NewNumCellsY, NewNumCellsZ, NewMaxNeighborsPerCell, NumTotalCells, MaxViewDimensionForTypedBuffer, *GetFullNameSafe(this));
			bSuccess = false;
		}

		*OutSuccess.GetDestAndAdvance() = bSuccess;
		if (bSuccess)
		{
			FIntVector OldNumCells = InstData->NumCells;
			int OldMaxNeighborsPerCell = InstData->MaxNeighborsPerCell;
			
			InstData->NumCells.X = FMath::Max(1, NewNumCellsX);
			InstData->NumCells.Y = FMath::Max(1, NewNumCellsY);
			InstData->NumCells.Z = FMath::Max(1, NewNumCellsZ);

			InstData->MaxNeighborsPerCell = NewMaxNeighborsPerCell;
		
			InstData->bNeedsRealloc = OldNumCells != InstData->NumCells || OldMaxNeighborsPerCell != InstData->MaxNeighborsPerCell;
		}
	}
}

bool UNiagaraDataInterfaceNeighborGrid3D::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FNDINeighborGrid3DInstanceData_GT* InstanceData = static_cast<FNDINeighborGrid3DInstanceData_GT*>(PerInstanceData);
	bool bNeedsReset = false;

	if (InstanceData->bNeedsRealloc && InstanceData->NumCells.X > 0 && InstanceData->NumCells.Y > 0 && InstanceData->NumCells.Z > 0 && InstanceData->MaxNeighborsPerCell > 0)
	{
		InstanceData->bNeedsRealloc = false;

		InstanceData->CellSize = float((InstanceData->WorldBBoxSize / FVector(InstanceData->NumCells.X, InstanceData->NumCells.Y, InstanceData->NumCells.Z))[0]);

		FNiagaraDataInterfaceProxyNeighborGrid3D* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyNeighborGrid3D>();
		ENQUEUE_RENDER_COMMAND(FUpdateData)(
			[RT_Proxy, RT_NumCells = InstanceData->NumCells, RT_MaxNeighborsPerCell = InstanceData->MaxNeighborsPerCell, RT_CellSize = InstanceData->CellSize, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
		{
			check(RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));
			FNDINeighborGrid3DInstanceData_RT* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.Add(InstanceID);

			TargetData->NumCells = RT_NumCells;
			TargetData->MaxNeighborsPerCell = RT_MaxNeighborsPerCell;			
			TargetData->CellSize = RT_CellSize;
			TargetData->bNeedsRealloc = true;
		});
	}

	return false;
}


void UNiagaraDataInterfaceNeighborGrid3D::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{		
	FNDINeighborGrid3DInstanceData_GT* InstanceData = static_cast<FNDINeighborGrid3DInstanceData_GT*>(PerInstanceData);
	InstanceData->~FNDINeighborGrid3DInstanceData_GT();

	FNiagaraDataInterfaceProxyNeighborGrid3D* ThisProxy = GetProxyAs<FNiagaraDataInterfaceProxyNeighborGrid3D>();
	if (!ThisProxy)
		return;

	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
	{
		//check(ThisProxy->SystemInstancesToProxyData_RT.Contains(InstanceID));
		ThisProxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
	}
	);
}

void FNiagaraDataInterfaceProxyNeighborGrid3D::ResetData(const FNDIGpuComputeResetContext& Context)
{	
	FNDINeighborGrid3DInstanceData_RT* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	AddClearUAVPass(GraphBuilder, ProxyData->NeighborhoodBuffer.GetOrCreateUAV(GraphBuilder), -1);
	AddClearUAVPass(GraphBuilder, ProxyData->NeighborhoodCountBuffer.GetOrCreateUAV(GraphBuilder), 0);
}

void FNiagaraDataInterfaceProxyNeighborGrid3D::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	FNDINeighborGrid3DInstanceData_RT& ProxyData = SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());
	if (ProxyData.bNeedsRealloc)
	{
		ProxyData.ResizeBuffers(GraphBuilder);
	}

	if (Context.IsOutputStage() && ProxyData.NeighborhoodBuffer.IsValid() && ProxyData.ClearBeforeNonIterationStage)
	{
		RDG_RHI_EVENT_SCOPE(GraphBuilder, NiagaraNeighborGrid3DClearNeighborInfo);
		AddClearUAVPass(GraphBuilder, ProxyData.NeighborhoodBuffer.GetOrCreateUAV(GraphBuilder), -1);
		AddClearUAVPass(GraphBuilder, ProxyData.NeighborhoodCountBuffer.GetOrCreateUAV(GraphBuilder), 0);
	}
}

void FNiagaraDataInterfaceProxyNeighborGrid3D::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	if (Context.IsFinalPostSimulate())
	{
		FNDINeighborGrid3DInstanceData_RT& ProxyData = SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());
		ProxyData.NeighborhoodBuffer.EndGraphUsage();
		ProxyData.NeighborhoodCountBuffer.EndGraphUsage();
	}
}

void FNiagaraDataInterfaceProxyNeighborGrid3D::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context)
{
	if ( const FNDINeighborGrid3DInstanceData_RT* TargetData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()) )
	{
		Context.SetDirect(TargetData->NumCells);
	}
}

bool UNiagaraDataInterfaceNeighborGrid3D::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceNeighborGrid3D* OtherTyped = CastChecked<UNiagaraDataInterfaceNeighborGrid3D>(Destination);


	OtherTyped->MaxNeighborsPerCell = MaxNeighborsPerCell;


	return true;
}

#undef LOCTEXT_NAMESPACE

