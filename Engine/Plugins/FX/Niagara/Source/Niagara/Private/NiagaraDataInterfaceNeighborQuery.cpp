// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceNeighborQuery.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSimStageData.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraEmitter.h"
#include "NiagaraScript.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderGraphUtils.h"
#include "SystemTextures.h"
#include "NiagaraNeighborQuerySort.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceNeighborQuery)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceNeighborQuery"

namespace NDINeighborQueryLocal
{
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceNeighborQuery.ush");

	// Function names — must match the _{ParameterName} suffix functions in the .ush
	static const FName SetNumCellsFunctionName(TEXT("SetNumCells"));
	static const FName MaxCellsPerParticleFunctionName(TEXT("MaxCellsPerParticle"));
	static const FName AddParticleFunctionName(TEXT("AddParticle"));
	static const FName AddParticleWithRadiusFunctionName(TEXT("AddParticleWithRadius"));
	static const FName AddParticleToOverlappedCellsFunctionName(TEXT("AddParticleToOverlappedCells"));
	static const FName AddParticleToNeighborCellsFunctionName(TEXT("AddParticleToNeighborCells"));
	static const FName GetParticleNeighborCountFunctionName(TEXT("GetParticleNeighborCount"));
	static const FName GetParticleNeighborFunctionName(TEXT("GetParticleNeighbor"));
	static const FName UnitToCellCornerFloatIndexFunctionName(TEXT("UnitToCellCornerFloatIndex"));
}



static int32 GMaxNiagaraNeighborQueryCells = (512 * 512 * 512);
static FAutoConsoleVariableRef CVarMaxNiagaraNeighborQueryCells(
	TEXT("fx.MaxNiagaraNeighborQueryCells"),
	GMaxNiagaraNeighborQueryCells,
	TEXT("The max number of supported grid cells for NeighborQuery. Overflowing this threshold will cause the sim to warn and fail.\n"),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////

void FNDINeighborQueryInstanceData_RT::ResizeBuffers(FRDGBuilder& GraphBuilder)
{
	const uint32 NumTotalCells = (uint32)((uint64)NumCells.X * NumCells.Y * NumCells.Z);
	const uint64 NumSlots64 = FMath::Max<uint64>((uint64) NumParticles * MaxCellsPerParticle, 1u);
	const uint32 NumSlots = (uint32)FMath::Min<uint64>(NumSlots64, MAX_uint32);

	// Slot-sized buffers: reallocate when particle count * MaxCellsPerParticle grows
	if (NumSlots > AllocatedNumSlots)
	{
		CellIdBuffer.Release();

		CellIdBuffer.Initialize(GraphBuilder, TEXT("NeighborQuery::CellId"), EPixelFormat::PF_R32_SINT, sizeof(int32), NumSlots, BUF_Static);

		if (!bCountOnly)
		{
			ParticleListBuffer.Release();
			ParticleListBuffer.Initialize(GraphBuilder, TEXT("NeighborQuery::ParticleList"), EPixelFormat::PF_R32_SINT, sizeof(int32), NumSlots, BUF_Static);

			if (bUsePersistentIDs)
			{
				ParticleIdIndexBuffer.Release();
				ParticleIdIndexBuffer.Initialize(GraphBuilder, TEXT("NeighborQuery::ParticleIdIndex"), EPixelFormat::PF_R32_SINT, sizeof(int32), NumSlots, BUF_Static);

				AcquireTagBuffer.Release();
				AcquireTagBuffer.Initialize(GraphBuilder, TEXT("NeighborQuery::AcquireTag"), EPixelFormat::PF_R32_SINT, sizeof(int32), NumSlots, BUF_Static);

				AcquireTagListBuffer.Release();
				AcquireTagListBuffer.Initialize(GraphBuilder, TEXT("NeighborQuery::AcquireTagList"), EPixelFormat::PF_R32_SINT, sizeof(int32), NumSlots, BUF_Static);
			}
		}

		AllocatedNumSlots = NumSlots;
	}

	// Release persistent-ID-only buffers when persistent IDs are disabled or count-only mode is active.
	// In count-only mode only the histogram runs — scatter is skipped, so there is no need to carry
	// particle indices or acquire tags through the sort, regardless of bUsePersistentIDs.
	if (!bUsePersistentIDs || bCountOnly)
	{
		ParticleIdIndexBuffer.Release();
		AcquireTagBuffer.Release();
		AcquireTagListBuffer.Release();
	}

	// Cell-sized buffers: reallocate when grid dimensions change
	if (NumTotalCells != AllocatedNumCells)
	{
		if (NumTotalCells > (uint32)GMaxNiagaraNeighborQueryCells)
		{
			UE_LOGF(LogNiagara, Verbose, "NiagaraDataInterfaceNeighborQuery - Invalid NumCells(%dx%dx%d) with total = %u exceeds the maximum of %d .", NumCells.X, NumCells.Y, NumCells.Z, NumTotalCells, GMaxNiagaraNeighborQueryCells);
			// Do NOT clear bNeedsRealloc — cell buffers were not allocated, so subsequent
			// frames must retry. Slot-sized buffers above may have succeeded, but the system
			// is not functional without cell buffers.
			return;
		}

		CellCountBuffer.Release();

		const uint32 CellBufferSize = FMath::Max<uint32>(NumTotalCells, 1u);
		CellCountBuffer.Initialize(GraphBuilder, TEXT("NeighborQuery::CellCount"), EPixelFormat::PF_R32_SINT, sizeof(int32), CellBufferSize, BUF_Static);

		if (!bCountOnly)
		{
			CellOffsetBuffer.Release();
			CellOffsetBuffer.Initialize(GraphBuilder, TEXT("NeighborQuery::CellOffset"), EPixelFormat::PF_R32_SINT, sizeof(int32), CellBufferSize, BUF_Static);
		}

		AllocatedNumCells = NumTotalCells;
	}

	// Both allocation blocks succeeded — clear the realloc flag.
	bNeedsRealloc = false;
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceNeighborQuery::UNiagaraDataInterfaceNeighborQuery(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, MaxCellsPerParticle(1)
	, bCountOnly(false)
	, bUsePersistentIDs(true)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyNeighborQuery());
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceNeighborQuery::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDINeighborQueryLocal;

	Super::GetFunctionsInternal(OutFunctions);

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetNumCellsFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MaxCellsPerParticle")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		OutFunctions.Add(Sig);
	}

	// MaxCellsPerParticle() -> MaxCellsPerParticle
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = MaxCellsPerParticleFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MaxCellsPerParticle")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	// AddParticle(Enabled, IndexPos, ParticleIndex, AcquireTag) -> Success
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NDINeighborQueryLocal::AddParticleFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("IndexPos")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleIndex")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AcquireTag"))).SetValue(0);
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.bWriteFunction = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		OutFunctions.Add(Sig);
	}

	// AddParticleWithRadius(Enabled, IndexPos, Radius, ParticleIndex, AcquireTag) -> Success
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = AddParticleWithRadiusFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("IndexPos")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Radius")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleIndex")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AcquireTag"))).SetValue(0);
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.bWriteFunction = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		OutFunctions.Add(Sig);
	}

	// AddParticleToOverlappedCells(Enabled, IndexPos, ParticleIndex, AcquireTag) -> Success
	// Writes the particle to all 8 cells in the 2x2x2 positive neighborhood from floor(IndexPos).
	// For asymmetric gather (P2G rasterization with trilinear kernel). Requires MaxCellsPerParticle >= 8.
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = AddParticleToOverlappedCellsFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("IndexPos")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleIndex")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AcquireTag"))).SetValue(0);
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.bWriteFunction = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		OutFunctions.Add(Sig);
	}

	// AddParticleToNeighborCells(Enabled, IndexPos, ParticleIndex, AcquireTag) -> Success
	// Writes the particle to all 27 cells in the 3x3x3 neighborhood centered on round(IndexPos).
	// For symmetric bilateral queries (PBD constraints) with single-cell query. Requires MaxCellsPerParticle >= 27.
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = AddParticleToNeighborCellsFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled"))).SetValue(true);
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("IndexPos")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleIndex")));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AcquireTag"))).SetValue(0);
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.bWriteFunction = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		OutFunctions.Add(Sig);
	}

	// GetParticleNeighborCount(IndexX, IndexY, IndexZ) -> Count
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NDINeighborQueryLocal::GetParticleNeighborCountFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		OutFunctions.Add(Sig);
	}

	// GetParticleNeighbor(IndexX, IndexY, IndexZ, NeighborIdx) -> (ParticleIdx, AcquireTag)
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NDINeighborQueryLocal::GetParticleNeighborFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NeighborIdx")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleIdx")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AcquireTag")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		OutFunctions.Add(Sig);
	}

	// UnitToCellCornerFloatIndex(Unit) -> Index
	// Like UnitToFloatIndex but without the -0.5 center offset.
	// Returns indices where cell boundaries align with integers, suitable for floor()-based cell assignment.
	// Use with AddParticle (which uses floor) for correct -1 to 0 asymmetric gather in P2G rasterization.
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UnitToCellCornerFloatIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Index")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

}
#endif

//////////////////////////////////////////////////////////////////////////
// VM function bindings

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceNeighborQuery, SetNumCells);
void UNiagaraDataInterfaceNeighborQuery::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDINeighborQueryLocal;

	Super::GetVMExternalFunction(BindingInfo, InstanceData, OutFunc);

	if (BindingInfo.Name == UNiagaraDataInterfaceRWBase::NumCellsFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceNeighborQuery::GetNumCells);
	}
	else if (BindingInfo.Name == MaxCellsPerParticleFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceNeighborQuery::GetMaxCellsPerParticle);
	}
	else if (BindingInfo.Name == SetNumCellsFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceNeighborQuery, SetNumCells)::Bind(this, OutFunc);
	}
}

//////////////////////////////////////////////////////////////////////////
// VM function implementations

void UNiagaraDataInterfaceNeighborQuery::GetNumCells(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDINeighborQueryInstanceData_GT> InstData(Context);

	FNDIOutputParam<int32> NumCellsX(Context);
	FNDIOutputParam<int32> NumCellsY(Context);
	FNDIOutputParam<int32> NumCellsZ(Context);

	int32 TmpNumCellsX = InstData->NumCells.X;
	int32 TmpNumCellsY = InstData->NumCells.Y;
	int32 TmpNumCellsZ = InstData->NumCells.Z;

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		NumCellsX.SetAndAdvance(TmpNumCellsX);
		NumCellsY.SetAndAdvance(TmpNumCellsY);
		NumCellsZ.SetAndAdvance(TmpNumCellsZ);
	}
}

void UNiagaraDataInterfaceNeighborQuery::GetMaxCellsPerParticle(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDINeighborQueryInstanceData_GT> InstData(Context);

	FNDIOutputParam<int32> OutMaxCellsPerParticle(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutMaxCellsPerParticle.SetAndAdvance(InstData->MaxCellsPerParticle);
	}
}

bool UNiagaraDataInterfaceNeighborQuery::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceNeighborQuery* OtherTyped = CastChecked<const UNiagaraDataInterfaceNeighborQuery>(Other);

	return OtherTyped->MaxCellsPerParticle == MaxCellsPerParticle
		&& OtherTyped->bCountOnly == bCountOnly
		&& OtherTyped->bUsePersistentIDs == bUsePersistentIDs;
}

//////////////////////////////////////////////////////////////////////////
// HLSL generation — .ush template approach

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceNeighborQuery::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}

	InVisitor->UpdateShaderFile(NDINeighborQueryLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<FShaderParameters>();

	return true;
}

void UNiagaraDataInterfaceNeighborQuery::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	// Inject NQ_DIST_BITS before the template so the .ush picks up the C++ constant
	// instead of its fallback default. The #ifndef guard in the .ush prevents redefinition
	// when multiple NeighborQuery DIs are present in the same shader.
	OutHLSL += FString::Printf(TEXT("#ifndef NQ_DIST_BITS\n#define NQ_DIST_BITS %u\n#endif\n"), NiagaraNeighborQuerySort::NQDistBits);

	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"), ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDINeighborQueryLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceNeighborQuery::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDINeighborQueryLocal;

	bool ParentRet = Super::GetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, OutHLSL);
	if (ParentRet)
	{
		return true;
	}

	// UnitToCellCornerFloatIndex — like UnitToFloatIndex but without the -0.5 center offset.
	// Cell boundaries land on integers so floor() gives the correct cell index.
	if (FunctionInfo.DefinitionName == UnitToCellCornerFloatIndexFunctionName)
	{
		static const TCHAR* FormatSample = TEXT(R"(
			void {FunctionName}(float3 In_Unit, out float3 Out_Index)
			{
				Out_Index = In_Unit * {NumCellsName};
			}
		)");

		TMap<FString, FStringFormatArg> Args = {
			{ TEXT("FunctionName"), FunctionInfo.InstanceName },
			{ TEXT("NumCellsName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName },
		};
		OutHLSL += FString::Format(FormatSample, Args);
		return true;
	}

	// All functions are handled by the .ush template (including _UEImpureCall wrappers for exec-pin functions).
	if (FunctionInfo.DefinitionName == MaxCellsPerParticleFunctionName ||
		FunctionInfo.DefinitionName == NDINeighborQueryLocal::AddParticleFunctionName ||
		FunctionInfo.DefinitionName == AddParticleWithRadiusFunctionName ||
		FunctionInfo.DefinitionName == AddParticleToOverlappedCellsFunctionName ||
		FunctionInfo.DefinitionName == AddParticleToNeighborCellsFunctionName ||
		FunctionInfo.DefinitionName == NDINeighborQueryLocal::GetParticleNeighborCountFunctionName ||
		FunctionInfo.DefinitionName == NDINeighborQueryLocal::GetParticleNeighborFunctionName)
	{
		return true;
	}

	return false;
}
#endif

//////////////////////////////////////////////////////////////////////////
// Shader parameter binding

void UNiagaraDataInterfaceNeighborQuery::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceNeighborQuery::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyNeighborQuery& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyNeighborQuery>();
	FNDINeighborQueryInstanceData_RT* ProxyData = DIProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	if (ProxyData && ProxyData->CellCountBuffer.IsValid())
	{
		ShaderParameters->NumCells = ProxyData->NumCells;
		ShaderParameters->UnitToUV = FVector3f(1.0f) / FVector3f(ProxyData->NumCells);
		ShaderParameters->CellSize = FVector3f::ZeroVector;
		ShaderParameters->WorldBBoxSize = FVector3f::ZeroVector;
		ShaderParameters->MaxCellsPerParticleValue = ProxyData->MaxCellsPerParticle;

		if (Context.IsOutputStage())
		{
			// Output stage: user modules write CellIdBuffer, ParticleIdIndexBuffer, and AcquireTagBuffer; read buffers are empty
			ShaderParameters->CellCount = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
			ShaderParameters->CellOffset = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
			ShaderParameters->ParticleList = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
			ShaderParameters->OutputCellId = ProxyData->CellIdBuffer.GetOrCreateUAV(GraphBuilder);
			ShaderParameters->OutputParticleIdIndex = ProxyData->ParticleIdIndexBuffer.IsValid()
				? ProxyData->ParticleIdIndexBuffer.GetOrCreateUAV(GraphBuilder)
				: Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
			ShaderParameters->OutputAcquireTag = ProxyData->AcquireTagBuffer.IsValid()
				? ProxyData->AcquireTagBuffer.GetOrCreateUAV(GraphBuilder)
				: Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
			ShaderParameters->AcquireTagList = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
		}
		else
		{
			// Read stage: counting sort has completed, read results
			UE_LOGF(LogNiagara, Verbose, "NeighborQuery::SetShaderParameters(READ) — NumCells=(%d,%d,%d) bCountOnly=%d CellOffsetValid=%d ParticleListValid=%d AcquireTagListValid=%d",
				ProxyData->NumCells.X, ProxyData->NumCells.Y, ProxyData->NumCells.Z,
				ProxyData->bCountOnly, ProxyData->CellOffsetBuffer.IsValid(), ProxyData->ParticleListBuffer.IsValid(), ProxyData->AcquireTagListBuffer.IsValid());
			ShaderParameters->CellCount = ProxyData->CellCountBuffer.GetOrCreateSRV(GraphBuilder);
			if (ProxyData->bCountOnly)
			{
				ShaderParameters->CellOffset = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
				ShaderParameters->ParticleList = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
				ShaderParameters->AcquireTagList = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
			}
			else
			{
				ShaderParameters->CellOffset = ProxyData->CellOffsetBuffer.GetOrCreateSRV(GraphBuilder);
				ShaderParameters->ParticleList = ProxyData->ParticleListBuffer.GetOrCreateSRV(GraphBuilder);
				ShaderParameters->AcquireTagList = ProxyData->AcquireTagListBuffer.IsValid()
					? ProxyData->AcquireTagListBuffer.GetOrCreateSRV(GraphBuilder)
					: Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
			}
			ShaderParameters->OutputCellId = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
			ShaderParameters->OutputParticleIdIndex = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
			ShaderParameters->OutputAcquireTag = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
		}
	}
	else
	{
		UE_LOGF(LogNiagara, Warning, "NeighborQuery::SetShaderParameters — FALLBACK: ProxyData=%p CellCountValid=%d. All buffers set to empty.",
			ProxyData, ProxyData ? ProxyData->CellCountBuffer.IsValid() : false);

		ShaderParameters->NumCells = FIntVector::ZeroValue;
		ShaderParameters->UnitToUV = FVector3f::ZeroVector;
		ShaderParameters->CellSize = FVector3f::ZeroVector;
		ShaderParameters->WorldBBoxSize = FVector3f::ZeroVector;
		ShaderParameters->MaxCellsPerParticleValue = 0;

		ShaderParameters->CellCount = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
		ShaderParameters->CellOffset = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
		ShaderParameters->ParticleList = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
		ShaderParameters->OutputCellId = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
		ShaderParameters->OutputParticleIdIndex = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
		ShaderParameters->OutputAcquireTag = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_SINT);
		ShaderParameters->AcquireTagList = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_SINT);
	}
}

//////////////////////////////////////////////////////////////////////////
// Instance data lifecycle

bool UNiagaraDataInterfaceNeighborQuery::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	if (UE::PixelFormat::HasCapabilities(EPixelFormat::PF_R32_SINT, EPixelFormatCapabilities::TypedUAVLoad | EPixelFormatCapabilities::TypedUAVStore) == false)
	{
		return false;
	}

	FNDINeighborQueryInstanceData_GT* InstanceData = new (PerInstanceData) FNDINeighborQueryInstanceData_GT();

	// Defaults — expected to be overridden by SetNumCells from emitter graph
	InstanceData->NumCells = FIntVector(1, 1, 1);
	InstanceData->MaxCellsPerParticle = MaxCellsPerParticle;
	InstanceData->bCountOnly = bCountOnly;
	InstanceData->bUsePersistentIDs = bUsePersistentIDs;

	FNiagaraDataInterfaceProxyNeighborQuery* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyNeighborQuery>();
	const FIntVector RT_NumCells = InstanceData->NumCells;
	const uint32 RT_MaxCellsPerParticle = InstanceData->MaxCellsPerParticle;
	const bool RT_bCountOnly = InstanceData->bCountOnly;
	const bool RT_bUsePersistentIDs = InstanceData->bUsePersistentIDs;

	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Proxy, RT_NumCells, RT_MaxCellsPerParticle, RT_bCountOnly, RT_bUsePersistentIDs, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
		{
			check(!RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));
			FNDINeighborQueryInstanceData_RT* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.Add(InstanceID);

			TargetData->NumCells = RT_NumCells;
			TargetData->MaxCellsPerParticle = RT_MaxCellsPerParticle;
			TargetData->bCountOnly = RT_bCountOnly;
			TargetData->bUsePersistentIDs = RT_bUsePersistentIDs;
			TargetData->bNeedsRealloc = true;
		}
	);

	return true;
}

void UNiagaraDataInterfaceNeighborQuery::SetNumCells(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDINeighborQueryInstanceData_GT> InstData(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsX(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsY(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsZ(Context);
	VectorVM::FExternalFuncInputHandler<int> InMaxCellsPerParticle(Context);
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const int NewNumCellsX = InNumCellsX.GetAndAdvance();
		const int NewNumCellsY = InNumCellsY.GetAndAdvance();
		const int NewNumCellsZ = InNumCellsZ.GetAndAdvance();
		const int NewMaxCellsPerParticle = InMaxCellsPerParticle.GetAndAdvance();
		bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1 && NewNumCellsX >= 1 && NewNumCellsY >= 1 && NewNumCellsZ >= 1 && NewMaxCellsPerParticle >= 1);

		const uint64 NumTotalCells = (uint64) NewNumCellsX * NewNumCellsY * NewNumCellsZ;
		if (NumTotalCells > NiagaraNeighborQuerySort::NQMaxSortCells)
		{
			// The sort key packs CellId into (32 - NQ_DIST_BITS) high bits. If the total
			// cell count exceeds 1 << (32 - NQ_DIST_BITS) the cell ID overflows into the
			// distance bits, silently corrupting the histogram and scatter passes.
			UE_LOGF(LogNiagara, Error,
				"NiagaraDataInterfaceNeighborQuery - NumCells(%dx%dx%d) total %llu exceeds the sort key limit of %llu (NQ_DIST_BITS=%u). "
				"Reduce the grid resolution for '%ls'.",
				NewNumCellsX, NewNumCellsY, NewNumCellsZ, NumTotalCells,
				NiagaraNeighborQuerySort::NQMaxSortCells,
				NiagaraNeighborQuerySort::NQDistBits,
				*GetFullNameSafe(this));
			bSuccess = false;
		}
		else if (NumTotalCells == 0 || NumTotalCells > (uint64)GMaxNiagaraNeighborQueryCells)
		{
			UE_LOGF(LogNiagara, Error,
				"NiagaraDataInterfaceNeighborQuery - Invalid NumCells(%dx%dx%d) with total = %llu exceeds the maximum of %d for '%ls'.",
				NewNumCellsX, NewNumCellsY, NewNumCellsZ, NumTotalCells,
				GMaxNiagaraNeighborQueryCells,
				*GetFullNameSafe(this));
			bSuccess = false;
		}

		*OutSuccess.GetDestAndAdvance() = bSuccess;
		if (bSuccess)
		{
			FIntVector OldNumCells = InstData->NumCells;
			int OldMaxCellsPerParticle = InstData->MaxCellsPerParticle;

			InstData->NumCells.X = FMath::Max(1, NewNumCellsX);
			InstData->NumCells.Y = FMath::Max(1, NewNumCellsY);
			InstData->NumCells.Z = FMath::Max(1, NewNumCellsZ);
			InstData->MaxCellsPerParticle = FMath::Max(1, NewMaxCellsPerParticle);

			InstData->bNeedsRealloc = OldNumCells != InstData->NumCells || OldMaxCellsPerParticle != InstData->MaxCellsPerParticle;
		}
	}
}

bool UNiagaraDataInterfaceNeighborQuery::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FNDINeighborQueryInstanceData_GT* InstanceData = static_cast<FNDINeighborQueryInstanceData_GT*>(PerInstanceData);

	if (InstanceData->bNeedsRealloc && InstanceData->NumCells.X > 0 && InstanceData->NumCells.Y > 0 && InstanceData->NumCells.Z > 0)
	{
		InstanceData->bNeedsRealloc = false;

		FNiagaraDataInterfaceProxyNeighborQuery* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyNeighborQuery>();
		ENQUEUE_RENDER_COMMAND(FUpdateData)(
			[RT_Proxy, RT_NumCells = InstanceData->NumCells, RT_MaxCellsPerParticle = InstanceData->MaxCellsPerParticle, RT_bCountOnly = InstanceData->bCountOnly, RT_bUsePersistentIDs = InstanceData->bUsePersistentIDs, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
		{
			check(RT_Proxy->SystemInstancesToProxyData_RT.Contains(InstanceID));
			FNDINeighborQueryInstanceData_RT* TargetData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID);

			TargetData->NumCells = RT_NumCells;
			TargetData->MaxCellsPerParticle = RT_MaxCellsPerParticle;
			TargetData->bCountOnly = RT_bCountOnly;
			TargetData->bUsePersistentIDs = RT_bUsePersistentIDs;
			TargetData->bNeedsRealloc = true;

			TargetData->NumParticles = 0;
			TargetData->AllocatedNumSlots = 0;
			TargetData->AllocatedNumCells = 0;
		});
	}

	return false;
}

void UNiagaraDataInterfaceNeighborQuery::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDINeighborQueryInstanceData_GT* InstanceData = static_cast<FNDINeighborQueryInstanceData_GT*>(PerInstanceData);
	InstanceData->~FNDINeighborQueryInstanceData_GT();

	FNiagaraDataInterfaceProxyNeighborQuery* ThisProxy = GetProxyAs<FNiagaraDataInterfaceProxyNeighborQuery>();
	if (!ThisProxy)
		return;

	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData)(
		[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
	{
		ThisProxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
	}
	);
}

//////////////////////////////////////////////////////////////////////////
// Proxy methods

void FNiagaraDataInterfaceProxyNeighborQuery::ResetData(const FNDIGpuComputeResetContext& Context)
{
	FNDINeighborQueryInstanceData_RT* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	if (!ProxyData)
	{
		return;
	}

	UE_LOGF(LogNiagara, Verbose, "NeighborQuery::ResetData — NumCells=(%d,%d,%d) NumParticles=%u AllocatedSlots=%u AllocatedCells=%u bUsePersistentIDs=%d",
		ProxyData->NumCells.X, ProxyData->NumCells.Y, ProxyData->NumCells.Z,
		ProxyData->NumParticles, ProxyData->AllocatedNumSlots, ProxyData->AllocatedNumCells, ProxyData->bUsePersistentIDs);

	// Only clear the write-side slot buffers (CellIdBuffer, ParticleIdIndexBuffer, AcquireTagBuffer).
	// These are written by AddParticle in the output stage; the INVALID_CELL sentinel in CellIdBuffer
	// ensures unused slots are skipped by the Histogram pass.
	//
	// Do NOT clear CellCountBuffer or CellOffsetBuffer here. Those are derived outputs of the sort
	// passes (Histogram + PrefixSum) and are correctly cleared in PreStage only for the emitter that
	// owns the write. Clearing them in ResetData is wrong when a read-only emitter has bResetData=true:
	// that would call ResetData and destroy counts just filled by the writer's PostStage sort passes,
	// causing the reader to see 0 particle counts (intermittent flicker on reader-emitter resets).
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	if (ProxyData->CellIdBuffer.IsValid())
	{
		AddClearUAVPass(GraphBuilder, ProxyData->CellIdBuffer.GetOrCreateUAV(GraphBuilder), 0xFFFFFFFF);  // INVALID_CELL sentinel
	}
	if (ProxyData->ParticleIdIndexBuffer.IsValid())
	{
		AddClearUAVPass(GraphBuilder, ProxyData->ParticleIdIndexBuffer.GetOrCreateUAV(GraphBuilder), 0);
	}
	if (ProxyData->AcquireTagBuffer.IsValid())
	{
		AddClearUAVPass(GraphBuilder, ProxyData->AcquireTagBuffer.GetOrCreateUAV(GraphBuilder), 0);
	}
}

void FNiagaraDataInterfaceProxyNeighborQuery::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	FNDINeighborQueryInstanceData_RT& ProxyData = SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());

	if (Context.IsOutputStage())
	{
		// Use the maximum of Source and Destination instance counts.  The GPU dispatch runs on
		// DestinationNumInstances threads, so AddParticle can be called for that many particles.
		// SourceNumInstances may be smaller on frames where new particles are spawned (stage 0
		// sets SourceNumInstances = PrevNumInstances, before adding spawns).
		const uint32 SrcInstances = Context.GetSimStageData().SourceNumInstances;
		const uint32 DstInstances = Context.GetSimStageData().DestinationNumInstances;
		const uint32 NewNumParticles = FMath::Max(SrcInstances, DstInstances);
		const uint64 NeededSlots = (uint64) NewNumParticles * ProxyData.MaxCellsPerParticle;

		UE_LOGF(LogNiagara, Verbose, "NeighborQuery::PreStage — SrcInstances=%u DstInstances=%u → NumParticles=%u  NumSlots=%llu  AllocatedSlots=%u  bNeedsRealloc=%d",
			SrcInstances, DstInstances, NewNumParticles, NeededSlots, ProxyData.AllocatedNumSlots, ProxyData.bNeedsRealloc);

		ProxyData.NumParticles = NewNumParticles;

		// Realloc if grid changed or slot buffers need to grow.
		if (ProxyData.bNeedsRealloc || NeededSlots > ProxyData.AllocatedNumSlots)
		{
			ProxyData.ResizeBuffers(GraphBuilder);
		}

		if (ProxyData.ClearBeforeNonIterationStage)
		{
			RDG_RHI_EVENT_SCOPE(GraphBuilder, NiagaraNeighborQueryClear);

			if (ProxyData.CellIdBuffer.IsValid())
			{
				AddClearUAVPass(GraphBuilder, ProxyData.CellIdBuffer.GetOrCreateUAV(GraphBuilder), 0xFFFFFFFF);  // INVALID_CELL sentinel
			}
			if (ProxyData.ParticleIdIndexBuffer.IsValid())
			{
				AddClearUAVPass(GraphBuilder, ProxyData.ParticleIdIndexBuffer.GetOrCreateUAV(GraphBuilder), 0);
			}
			if (ProxyData.AcquireTagBuffer.IsValid())
			{
				AddClearUAVPass(GraphBuilder, ProxyData.AcquireTagBuffer.GetOrCreateUAV(GraphBuilder), 0);
			}
			if (ProxyData.CellCountBuffer.IsValid())
			{
				AddClearUAVPass(GraphBuilder, ProxyData.CellCountBuffer.GetOrCreateUAV(GraphBuilder), 0);
			}
		}
	}
	else if (ProxyData.bNeedsRealloc)
	{
		ProxyData.ResizeBuffers(GraphBuilder);
	}
}

void FNiagaraDataInterfaceProxyNeighborQuery::PostStage(const FNDIGpuComputePostStageContext& Context)
{
	if (!Context.IsOutputStage())
	{
		return;
	}

	FNDINeighborQueryInstanceData_RT& ProxyData = SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());
	if (!ProxyData.CellIdBuffer.IsValid() || !ProxyData.CellCountBuffer.IsValid())
	{
		UE_LOGF(LogNiagara, Warning, "NeighborQuery::PostStage — skipped: CellIdBuffer.IsValid()=%d CellCountBuffer.IsValid()=%d",
			ProxyData.CellIdBuffer.IsValid(), ProxyData.CellCountBuffer.IsValid());
		return;
	}

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	const uint32 NumTotalCells = (uint32)((uint64)ProxyData.NumCells.X * ProxyData.NumCells.Y * ProxyData.NumCells.Z);
	const uint64 NumSlots64 = (uint64)ProxyData.NumParticles * ProxyData.MaxCellsPerParticle;
	if (NumSlots64 > ProxyData.AllocatedNumSlots)
	{
		UE_LOGF(LogNiagara, Warning, "NeighborQuery::PostStage — NumSlots (%llu) exceeds AllocatedNumSlots (%u), clamping.", NumSlots64, ProxyData.AllocatedNumSlots);
	}
	const uint32 NumSlots = (uint32)FMath::Min<uint64>(NumSlots64, ProxyData.AllocatedNumSlots);

	UE_LOGF(LogNiagara, Verbose, "NeighborQuery::PostStage — NumParticles=%u MaxCellsPerParticle=%u NumSlots=%u NumTotalCells=%u bCountOnly=%d bUsePersistentIDs=%d AcquireTagValid=%d AcquireTagListValid=%d CellOffsetValid=%d ParticleListValid=%d",
		ProxyData.NumParticles, ProxyData.MaxCellsPerParticle, NumSlots, NumTotalCells,
		ProxyData.bCountOnly, ProxyData.bUsePersistentIDs,
		ProxyData.AcquireTagBuffer.IsValid(), ProxyData.AcquireTagListBuffer.IsValid(),
		ProxyData.CellOffsetBuffer.IsValid(), ProxyData.ParticleListBuffer.IsValid());

	FRDGBufferSRVRef CellIdSRV = ProxyData.CellIdBuffer.GetOrCreateSRV(GraphBuilder);
	FRDGBufferUAVRef CellCountUAV = ProxyData.CellCountBuffer.GetOrCreateUAV(GraphBuilder);

	RDG_EVENT_SCOPE(GraphBuilder, "NiagaraNeighborQuerySort Particles(%u) Cells(%dx%dx%d) Slots(%u)%s",
		ProxyData.NumParticles, ProxyData.NumCells.X, ProxyData.NumCells.Y, ProxyData.NumCells.Z, NumSlots,
		ProxyData.bCountOnly ? TEXT(" CountOnly") : TEXT(""));

	// Pass 2: Histogram — count particles per cell
	NiagaraNeighborQuerySort::Histogram(GraphBuilder, CellIdSRV, CellCountUAV, NumSlots);

	if (!ProxyData.bCountOnly)
	{
		const bool bHasPersistentIDBuffers = ProxyData.bUsePersistentIDs && ProxyData.ParticleIdIndexBuffer.IsValid();
		FRDGBufferSRVRef ParticleIdIndexSRV;
		FRDGBufferSRVRef AcquireTagSRV;
		FRDGBufferUAVRef AcquireTagListUAV;
		if (bHasPersistentIDBuffers)
		{
			ParticleIdIndexSRV = ProxyData.ParticleIdIndexBuffer.GetOrCreateSRV(GraphBuilder);
			AcquireTagSRV = ProxyData.AcquireTagBuffer.GetOrCreateSRV(GraphBuilder);
			AcquireTagListUAV = ProxyData.AcquireTagListBuffer.GetOrCreateUAV(GraphBuilder);
		}
		else
		{
			// Use system default buffer for SRVs and a transient buffer for UAV.
			// Cannot use the empty UAV pool here because PostStage is not inside
			// a FNiagaraEmptyRDGUAVPoolScopedAccess scope.
			FRDGBufferRef DefaultBuffer = GSystemTextures.GetDefaultBuffer(GraphBuilder, sizeof(int32), 0u);
			ParticleIdIndexSRV = GraphBuilder.CreateSRV(DefaultBuffer, PF_R32_SINT);
			AcquireTagSRV = GraphBuilder.CreateSRV(DefaultBuffer, PF_R32_SINT);
			FRDGBufferRef DummyUAVBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), 1), TEXT("NiagaraNeighborQueryDummyAcquireTagUAV"));
			AcquireTagListUAV = GraphBuilder.CreateUAV(DummyUAVBuffer, PF_R32_SINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
		}
		FRDGBufferUAVRef CellOffsetUAV = ProxyData.CellOffsetBuffer.GetOrCreateUAV(GraphBuilder);
		FRDGBufferUAVRef ParticleListUAV = ProxyData.ParticleListBuffer.GetOrCreateUAV(GraphBuilder);

		// Pass 3: Prefix Sum — exclusive scan over cell counts
		NiagaraNeighborQuerySort::PrefixSum(GraphBuilder, CellCountUAV, CellOffsetUAV, NumTotalCells);

		// Pass 4: Scatter — write sorted particle ID indices (and AcquireTag) into ParticleListBuffer.
		// NOTE: Scatter atomically increments CellOffsetBuffer, so after this pass
		// CellOffset[i] = original_prefix_sum[i] + CellCount[i]. The read functions
		// in the .ush account for this by subtracting CellCount.
		NiagaraNeighborQuerySort::Scatter(GraphBuilder, CellIdSRV, ParticleIdIndexSRV, CellOffsetUAV, ParticleListUAV, AcquireTagSRV, AcquireTagListUAV, NumSlots, ProxyData.MaxCellsPerParticle, bHasPersistentIDBuffers);
	}
}

void FNiagaraDataInterfaceProxyNeighborQuery::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	if (Context.IsFinalPostSimulate())
	{
		FNDINeighborQueryInstanceData_RT& ProxyData = SystemInstancesToProxyData_RT.FindChecked(Context.GetSystemInstanceID());
		if (ProxyData.CellIdBuffer.IsValid())
		{
			ProxyData.CellIdBuffer.EndGraphUsage();
		}
		if (ProxyData.AcquireTagBuffer.IsValid())
		{
			ProxyData.AcquireTagBuffer.EndGraphUsage();
		}
		if (ProxyData.CellCountBuffer.IsValid())
		{
			ProxyData.CellCountBuffer.EndGraphUsage();
		}
		if (!ProxyData.bCountOnly)
		{
			if (ProxyData.ParticleIdIndexBuffer.IsValid())
			{
				ProxyData.ParticleIdIndexBuffer.EndGraphUsage();
			}
			if (ProxyData.CellOffsetBuffer.IsValid())
			{
				ProxyData.CellOffsetBuffer.EndGraphUsage();
			}
			if (ProxyData.ParticleListBuffer.IsValid())
			{
				ProxyData.ParticleListBuffer.EndGraphUsage();
			}
			if (ProxyData.AcquireTagListBuffer.IsValid())
			{
				ProxyData.AcquireTagListBuffer.EndGraphUsage();
			}
		}
	}
}

void FNiagaraDataInterfaceProxyNeighborQuery::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context)
{
	if (const FNDINeighborQueryInstanceData_RT* TargetData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()))
	{
		Context.SetDirect(TargetData->NumCells);
	}
}

bool UNiagaraDataInterfaceNeighborQuery::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceNeighborQuery* OtherTyped = CastChecked<UNiagaraDataInterfaceNeighborQuery>(Destination);

	OtherTyped->MaxCellsPerParticle = MaxCellsPerParticle;
	OtherTyped->bCountOnly = bCountOnly;
	OtherTyped->bUsePersistentIDs = bUsePersistentIDs;

	return true;
}

#if WITH_EDITOR
void UNiagaraDataInterfaceNeighborQuery::GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	using namespace NDINeighborQueryLocal;

	Super::GetFeedback(InAsset, InComponent, OutErrors, OutWarnings, OutInfo);

	// Inform the user of the sort key cell-count limit so they know to watch for it at runtime.
	// (NumCells is set dynamically via SetNumCells, so we can only check the limit here, not the actual value.)
	if (NiagaraNeighborQuerySort::NQDistBits > 0)
	{
		OutInfo.Add(FNiagaraDataInterfaceFeedback(
			FText::Format(
				LOCTEXT("SortKeyCellLimitInfo",
					"Sort key limit: NQ_DIST_BITS={0} — max grid cells = {1} (approx {2}^3). "
					"Exceeding this will silently corrupt the counting sort. "
					"Reduce the grid resolution to stay within the limit."),
				FText::AsNumber(NiagaraNeighborQuerySort::NQDistBits),
				FText::AsNumber(NiagaraNeighborQuerySort::NQMaxSortCells),
				FText::AsNumber((uint32)FMath::RoundToInt(FMath::Pow((float)NiagaraNeighborQuerySort::NQMaxSortCells, 1.0f / 3.0f)))),
			LOCTEXT("SortKeyCellLimitInfoSummary", "Sort key cell-count limit"),
			FNiagaraDataInterfaceFix()
		));
	}

	if (!InAsset)
	{
		return;
	}

	for (const FNiagaraEmitterHandle& EmitterHandle : InAsset->GetEmitterHandles())
	{
		FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
		if (!EmitterData)
		{
			continue;
		}

		// Find this DI's name and check sim stage metadata in a single pass over scripts
		FName DIName = NAME_None;
		TArray<UNiagaraScript*> Scripts;
		EmitterData->GetScripts(Scripts, false);
		for (const UNiagaraScript* Script : Scripts)
		{
			if (!Script)
			{
				continue;
			}

			// Resolve the DI name if we haven't yet
			if (DIName.IsNone())
			{
				for (const FNiagaraScriptDataInterfaceInfo& DIInfo : Script->GetCachedDefaultDataInterfaces())
				{
					if (DIInfo.DataInterface && DIInfo.DataInterface->GetClass() == GetClass() && DIInfo.DataInterface->Equals(this))
					{
						DIName = DIInfo.Name;
						break;
					}
				}
			}

			// Check sim stage metadata for non-particle-writing stages that output to this DI
			if (!DIName.IsNone())
			{
				TConstArrayView<FSimulationStageMetaData> StageMetaData = Script->GetSimulationStageMetaData();
				for (int32 StageIdx = 1; StageIdx < StageMetaData.Num(); ++StageIdx)
				{
					const FSimulationStageMetaData& StageMeta = StageMetaData[StageIdx];
					if (StageMeta.OutputDestinations.Contains(DIName))
					{
						OutWarnings.Add(FNiagaraDataInterfaceFeedback(
							FText::Format(LOCTEXT("NQNonParticleWriteStage",
								"NeighborQuery '{0}' is output of simulation stage '{1}' which does not write particles. "
								"AddParticle uses ExecIndex() for slot addressing, which requires a particle-writing stage. "
								"Move the AddParticle call into the Particle Update stage or another stage that writes particle attributes."),
								FText::FromName(DIName),
								FText::FromName(StageMeta.SimulationStageName)),
							LOCTEXT("NQNonParticleWriteStageSummary", "NeighborQuery AddParticle should be in a particle-writing stage"),
							FNiagaraDataInterfaceFix()
						));

						// #todo(dmp): duplicate error since it doesn't propagate properly
						UE_LOGF(LogNiagara, Warning, "NiagaraDataInterfaceNeighborQuery - AddParticle should be in a particle-writing stage.  Use at your own risk.'");
					}
				}
			}
		}

		if (DIName.IsNone())
		{
			continue;
		}

		// Check persistent ID requirement
		if (bUsePersistentIDs && !EmitterData->RequiresPersistentIDs())
		{
			FNiagaraDataInterfaceFix FixDelegate;
			FixDelegate.BindLambda([EmitterData]()
			{
				EmitterData->bRequiresPersistentIDs = true;
				return true;
			});

			OutErrors.Add(FNiagaraDataInterfaceError(
				FText::Format(LOCTEXT("NeedsPersistentIDs",
					"Emitter '{0}' must have Persistent IDs enabled for NeighborQuery. "
					"Either enable Persistent IDs on the emitter, or disable 'Use Persistent IDs' on this data interface (not recommended)."),
					FText::FromString(EmitterHandle.GetUniqueInstanceName())),
				LOCTEXT("NeedsPersistentIDsSummary", "Emitter needs Persistent IDs"),
				FixDelegate
			));

			// #todo(dmp): duplicate error since it doesn't propagate properly
			UE_LOGF(LogNiagara, Error, "NiagaraDataInterfaceNeighborQuery - Emitter must have Persistent IDs enabled for NeighborQuery or uncheck 'Use Persistent IDs'");
		}
	}
}
#endif

#undef LOCTEXT_NAMESPACE
