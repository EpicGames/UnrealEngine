// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceOcclusion.h"
#include "Containers/StridedView.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraTypes.h"
#include "NiagaraShaderParametersBuilder.h"

#include "RHIStaticStates.h"
#include "SceneView.h"
#include "Internationalization/Internationalization.h"
#include "SceneManagement.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceOcclusion)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceOcclusion"

namespace NDIOcclusionImpl
{
	static const TCHAR*	TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceOcclusion.ush");
	static const FName	Deprecated_GetCameraOcclusionRectangleName(TEXT("QueryOcclusionFactorWithRectangleGPU"));
	static const FName	GetCameraOcclusionRectangleName(TEXT("QueryOcclusionFactorWithRectangle"));
	static const FName	Deprecated_GetCameraOcclusionCircleName(TEXT("QueryOcclusionFactorWithCircleGPU"));
	static const FName	QueryOcclusionFactorWithCircleName(TEXT("QueryOcclusionFactorWithCircle"));
	static const FName	QueryCloudOcclusionWithCircleName(TEXT("QueryCloudOcclusionWithCircle"));
}

struct FNiagaraOcclusionDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		LWCConversion = 1,
		AddAtmosphereTransmittance = 2,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

UNiagaraDataInterfaceOcclusion::UNiagaraDataInterfaceOcclusion(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataIntefaceProxyOcclusionQuery());
}

void UNiagaraDataInterfaceOcclusion::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceOcclusion::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIOcclusionImpl;

	static const FText VisibilityFractionDescription(LOCTEXT("VisibilityFractionDescription", "Returns a value 0..1 depending on how many of the samples on the screen were occluded.\nFor example, a value of 0.3 means that 70% of visible samples were occluded.\nIf the sample fraction is 0 then this also returns 0."));
	static const FText OnScreenFractionDescription(LOCTEXT("OnScreenFractionDescription", "Returns a valid 0..1 depending on how may of the samples are on screen.\nFor example, a valid of 0.3 means 70% of the samples are off screen and 30% are on screen."));
	static const FText DepthPassFractionDescription(LOCTEXT("DepthPassFractionDescription", "Returns a value 0..1 depending on how many of the samples passed the depth test.\nFor example, a value of 0.3 means 70% of the depth test samples are closer to the screen.\nNote this fraction only includes on screen samples."));
	static const FText SampleFractionDescription(LOCTEXT("SampleFractionDescription", "Returns a value 0..1 depending on how many samples were inside the viewport or outside of it.\nFor example, a value of 0.3 means that 70% of samples were outside the current viewport and therefore not visible."));
	static const FText CircleCenterPosDescription(LOCTEXT("CircleCenterPosDescription", "This world space position where the center of the sample circle should be."));
	static const FText SampleWindowDiameterDescription(LOCTEXT("SampleWindowDiameterDescription", "The world space diameter of the circle to sample.\nIf the particle is a spherical sprite then this is the sprite size."));
	static const FText SamplesPerRingDescription(LOCTEXT("SamplesPerRingDescription", "The number of samples for each ring inside the circle.\nThe total number of samples is NumRings * SamplesPerRing."));
	static const FText NumberOfSampleRingsDescription(LOCTEXT("NumberOfSampleRingsDescription", "This number of concentric rings to sample inside the circle.\nThe total number of samples is NumRings * SamplesPerRing."));
	static const FText IncludeCenterSampleDescription(LOCTEXT("IncludeCenterSampleDescription", "When enabled we sample the center of the circle in addition to the rings."));

	FNiagaraFunctionSignature DefaultSig;
	DefaultSig.bMemberFunction = true;
	DefaultSig.bRequiresContext = false;
	DefaultSig.bSupportsCPU = false;
	DefaultSig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Occlusion interface")));
	DefaultSig.SetFunctionVersion(FNiagaraOcclusionDIFunctionVersion::LatestVersion);

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		FNiagaraFunctionSignature& SigNew = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = Deprecated_GetCameraOcclusionRectangleName;
		Sig.bSoftDeprecatedFunction = true;
		Sig.SetDescription(LOCTEXT("GetCameraOcclusionRectFunctionDescription", "This function returns the occlusion factor of a sprite. It samples the depth buffer in a rectangular grid around the given world position and compares each sample with the camera distance."));
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Sample Center World Position")), LOCTEXT("RectCenterPosDescription", "This world space position where the center of the sample rectangle should be."));
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Sample Window Width World")), LOCTEXT("SampleWindowWidthWorldDescription", "The total width of the sample rectangle in world space.\nIf the particle is a camera-aligned sprite then this is the sprite width."));
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Sample Window Height World")), LOCTEXT("SampleWindowHeightWorldDescription", "The total height of the sample rectangle in world space.\nIf the particle is a camera-aligned sprite then this is the sprite height."));
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Sample Steps Per Line")), LOCTEXT("StepsPerLineDescription", "The number of samples to take horizontally. The total number of samples is this value squared."));
		Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Visibility Fraction")), VisibilityFractionDescription);
		Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Sample Fraction")), SampleFractionDescription);

		SigNew = Sig;
		SigNew.bSoftDeprecatedFunction = false;
		SigNew.Name = GetCameraOcclusionRectangleName;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = Deprecated_GetCameraOcclusionCircleName;
		Sig.bSoftDeprecatedFunction = true;
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Sample Center World Position")), CircleCenterPosDescription);
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Sample Window Diameter World")), SampleWindowDiameterDescription);
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Samples per ring")), SamplesPerRingDescription);
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Number of sample rings")), NumberOfSampleRingsDescription);
		Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Visibility Fraction")), VisibilityFractionDescription);
		Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Sample Fraction")), SampleFractionDescription);
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = QueryOcclusionFactorWithCircleName;
		Sig.SetDescription(LOCTEXT("QueryCameraOcclusionCircleDescription", "This function returns the occlusion factor of a sprite. It samples the depth buffer in a spiral pattern around the given world position and compares each sample with the camera distance."));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("WorldPosition"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("WorldDiameter"));
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IncludeCenterSample")).SetValue(true);
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumberOfRings")).SetValue(1);
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetIntDef(), TEXT("SamplesPerRing")).SetValue(1);
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("VisibilityFraction"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("OnScreenFraction"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("DepthPassFraction"));

		Sig.SetInputDescription(Sig.Inputs[1], CircleCenterPosDescription);
		Sig.SetInputDescription(Sig.Inputs[2], SampleWindowDiameterDescription);
		Sig.SetInputDescription(Sig.Inputs[3], IncludeCenterSampleDescription);
		Sig.SetInputDescription(Sig.Inputs[4], NumberOfSampleRingsDescription);
		Sig.SetInputDescription(Sig.Inputs[5], SamplesPerRingDescription);
		Sig.SetOutputDescription(Sig.Outputs[0], VisibilityFractionDescription);
		Sig.SetOutputDescription(Sig.Outputs[1], OnScreenFractionDescription);
		Sig.SetOutputDescription(Sig.Outputs[2], DepthPassFractionDescription);
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = QueryCloudOcclusionWithCircleName;
		Sig.SetDescription(LOCTEXT("QueryCloudOcclusionWithCircleDescription", "Returns the cloud occlusion factor for the world position. "));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("WorldPosition"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("WorldDiameter"));
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IncludeCenterSample")).SetValue(true);
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumberOfRings")).SetValue(1);
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetIntDef(), TEXT("SamplesPerRing")).SetValue(1);
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("VisibilityFraction"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("SampleFraction"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("AtmosphereTransmittance"));

		Sig.SetInputDescription(Sig.Inputs[1], CircleCenterPosDescription);
		Sig.SetInputDescription(Sig.Inputs[2], SampleWindowDiameterDescription);
		Sig.SetInputDescription(Sig.Inputs[3], IncludeCenterSampleDescription);
		Sig.SetInputDescription(Sig.Inputs[4], NumberOfSampleRingsDescription);
		Sig.SetInputDescription(Sig.Inputs[5], SamplesPerRingDescription);
		Sig.SetOutputDescription(Sig.Outputs[0], VisibilityFractionDescription);
		Sig.SetOutputDescription(Sig.Outputs[1], SampleFractionDescription);
	}
}

bool UNiagaraDataInterfaceOcclusion::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace NDIOcclusionImpl;

	if (!Super::AppendCompileHash(InVisitor))
		return false;

	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	InVisitor->UpdateShaderParameters<FShaderParameters>();
	return true;
}

void UNiagaraDataInterfaceOcclusion::GetCommonHLSL(FString& OutHLSL)
{
	OutHLSL += TEXT("#include \"/Engine/Private/DeferredShadingCommon.ush\"\n");
}

bool UNiagaraDataInterfaceOcclusion::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIOcclusionImpl;

	static const TSet<FName> ValidGpuFunctions =
	{
		GetCameraOcclusionRectangleName,
		QueryOcclusionFactorWithCircleName,
		QueryCloudOcclusionWithCircleName,
		Deprecated_GetCameraOcclusionCircleName,
		Deprecated_GetCameraOcclusionRectangleName,
	};
	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

bool UNiagaraDataInterfaceOcclusion::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	if (FunctionSignature.FunctionVersion < FNiagaraOcclusionDIFunctionVersion::LatestVersion)
	{
		TArray<FNiagaraFunctionSignature> AllFunctions;
		GetFunctionsInternal(AllFunctions);
		for (const FNiagaraFunctionSignature& Sig : AllFunctions)
		{
			if (FunctionSignature.Name == Sig.Name)
			{
				FunctionSignature = Sig;
				return true;
			}
		}
	}

	return false;
}

void UNiagaraDataInterfaceOcclusion::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo,	FString& OutHLSL)
{
	using namespace NDIOcclusionImpl;

	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}
#endif

void UNiagaraDataInterfaceOcclusion::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceOcclusion::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();

	if (Context.IsResourceBound(&ShaderParameters->CloudVolumetricTexture))
	{
		TConstStridedView<FSceneView> SimulationSceneViews = Context.GetComputeDispatchInterface().GetSimulationSceneViews();
		const FSceneView* View = SimulationSceneViews.Num() > 0 ? &SimulationSceneViews[0] : nullptr;
		FRDGTextureRef CloudTexture = View && View->State ? View->State->GetVolumetricCloudTexture(Context.GetGraphBuilder()) : nullptr;
		if (CloudTexture == nullptr)
		{
			CloudTexture = Context.GetComputeDispatchInterface().GetBlackTexture(Context.GetGraphBuilder(), ETextureDimension::Texture2D);
		}
		ShaderParameters->CloudVolumetricTexture = CloudTexture;
		ShaderParameters->CloudVolumetricTextureUVScale = View && View->State ? View->State->GetVolumetricCloudTextureUVScale() : FVector2f(1.0f, 1.0f);
		ShaderParameters->CloudVolumetricTextureUVMax = View && View->State ? View->State->GetVolumetricCloudTextureUVMax() : FVector2f(1.0f, 1.0f);
	}
	ShaderParameters->CloudVolumetricTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
}

#undef LOCTEXT_NAMESPACE

