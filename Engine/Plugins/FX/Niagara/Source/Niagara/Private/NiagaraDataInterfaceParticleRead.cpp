// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceParticleRead.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraDataSetAccessor.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraGPUSystemTick.h"
#include "NiagaraRenderer.h"
#include "NiagaraSimStageData.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraShaderParametersBuilder.h"

#include "NiagaraSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceParticleRead)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceParticleRead"

struct FNiagaraParticleReadDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		RenamedSpawnIndex = 1,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

static const FName GetLocalSpaceName("GetLocalSpace");

static const FName GetNumSpawnedParticlesFunctionName("Get Num Spawned Particles");
static const FName GetIDAtSpawnIndexFunctionName("Get ID At Spawn Index");
static const FName GetNumParticlesFunctionName("Get Num Particles");
static const FName GetParticleIndexFunctionName("Get Particle Index");

static const FName GetIntByIDFunctionName("Get Int By ID");
static const FName GetBoolByIDFunctionName("Get Bool By ID");
static const FName GetFloatByIDFunctionName("Get Float By ID");
static const FName GetVec2ByIDFunctionName("Get Vector 2D By ID");
static const FName GetVec3ByIDFunctionName("Get Vector By ID");
static const FName GetVec4ByIDFunctionName("Get Vector 4 By ID");
static const FName GetColorByIDFunctionName("Get Color By ID");
static const FName GetPositionByIDFunctionName("Get Position By ID");
static const FName GetQuatByIDFunctionName("Get Quaternion By ID");
static const FName GetIDByIDFunctionName("Get ID By ID");

static const FName GetIntByIndexFunctionName("Get Int By Index");
static const FName GetBoolByIndexFunctionName("Get Bool By Index");
static const FName GetFloatByIndexFunctionName("Get Float By Index");
static const FName GetVec2ByIndexFunctionName("Get Vector 2D By Index");
static const FName GetVec3ByIndexFunctionName("Get Vector By Index");
static const FName GetVec4ByIndexFunctionName("Get Vector 4 By Index");
static const FName GetColorByIndexFunctionName("Get Color By Index");
static const FName GetPositionByIndexFunctionName("Get Position By Index");
static const FName GetQuatByIndexFunctionName("Get Quaternion By Index");
static const FName GetIDByIndexFunctionName("Get ID By Index");

static const FName GetParticleIndexFromIDTableName("Get Particle Index From ID Table");

static const FName ParticleReadIDName(TEXT("ID"));

static const TCHAR* IsLocalSpaceName = TEXT("_IsLocalSpace");
static const TCHAR* NumSpawnedParticlesName = TEXT("_NumSpawnedParticles");
static const TCHAR* SpawnedParticlesAcquireTagName = TEXT("_SpawnedParticlesAcquireTag");
static const TCHAR* InstanceCountOffsetName = TEXT("_InstanceCountOffset");
static const TCHAR* SpawnedIDsBufferName = TEXT("_SpawnedIDsBuffer");
static const TCHAR* IDToIndexTableName = TEXT("_IDToIndexTable");
static const TCHAR* InputFloatBufferName = TEXT("_InputFloatBuffer");
static const TCHAR* InputIntBufferName = TEXT("_InputIntBuffer");
static const TCHAR* InputHalfBufferName = TEXT("_InputHalfBuffer");
static const TCHAR* ParticleStrideFloatName = TEXT("_ParticleStrideFloat");
static const TCHAR* ParticleStrideIntName = TEXT("_ParticleStrideInt");
static const TCHAR* ParticleStrideHalfName = TEXT("_ParticleStrideHalf");
static const TCHAR* AcquireTagRegisterIndexName = TEXT("_AcquireTagRegisterIndex");
static const TCHAR* AttributeIndicesName = TEXT("_AttributeIndices");
static const TCHAR* AttributeCompressedName = TEXT("_AttributeCompressed");
static const TCHAR* AttributeIndicesMetadataName = TEXT("AttributeIndices");
static const TCHAR* AttributeCompressedMetadataName = TEXT("AttributeCompressed");

enum class ENiagaraParticleDataComponentType : uint8
{
	Float,
	Int,
	Bool,
	ID
};

enum class ENiagaraParticleDataValueType : uint8
{
	Invalid,
	Int,
	Float,
	Vec2,
	Vec3,
	Vec4,
	Bool,
	Color,
	Quat,
	ID,
	Position
};

//-TODO: This is a workaround for when a read is safe because the dispatches can not overlap.  UE-187062
static bool GNiagaraParticleReadIgnoreUnsafeReads = false;
static FAutoConsoleVariableRef CVarNiagaraParticleReadIgnoreUnsafeReads(
	TEXT("fx.Niagara.ParticleRead.IgnoreUnsafeReads"),
	GNiagaraParticleReadIgnoreUnsafeReads,
	TEXT("When enabled we will allow unsafe reads to compile with a warning, however the read result will be invalid."),
	ECVF_Default
);

DECLARE_INTRINSIC_TYPE_LAYOUT(ENiagaraParticleDataValueType);

struct FNiagaraDataInterfaceParametersCS_ParticleRead : public FNiagaraDataInterfaceParametersCS
{
	DECLARE_TYPE_LAYOUT(FNiagaraDataInterfaceParametersCS_ParticleRead, NonVirtual);

	LAYOUT_FIELD(TMemoryImageArray<FMemoryImageName>, AttributeNames);
	LAYOUT_FIELD(TMemoryImageArray<ENiagaraParticleDataValueType>, AttributeTypes);
};

IMPLEMENT_TYPE_LAYOUT(FNiagaraDataInterfaceParametersCS_ParticleRead);

static const TCHAR* NiagaraParticleDataValueTypeName(ENiagaraParticleDataValueType Type)
{
	switch (Type)
	{
		case ENiagaraParticleDataValueType::Invalid:	return TEXT("INVALID");
		case ENiagaraParticleDataValueType::Int:		return TEXT("int");
		case ENiagaraParticleDataValueType::Float:		return TEXT("float");
		case ENiagaraParticleDataValueType::Vec2:		return TEXT("vec2");
		case ENiagaraParticleDataValueType::Vec3:		return TEXT("vec3");
		case ENiagaraParticleDataValueType::Vec4:		return TEXT("vec4");
		case ENiagaraParticleDataValueType::Bool:		return TEXT("bool");
		case ENiagaraParticleDataValueType::Color:		return TEXT("color");
		case ENiagaraParticleDataValueType::Position:	return TEXT("position");
		case ENiagaraParticleDataValueType::Quat:		return TEXT("quaternion");
		case ENiagaraParticleDataValueType::ID:			return TEXT("ID");
		default:										return TEXT("UNKNOWN");
	}
}

static bool CheckVariableType(const FNiagaraTypeDefinition& VarType, ENiagaraParticleDataValueType AttributeType) 
{
	switch (AttributeType)
	{
		case ENiagaraParticleDataValueType::Int: return VarType == FNiagaraTypeDefinition::GetIntDef();
		case ENiagaraParticleDataValueType::Bool: return VarType == FNiagaraTypeDefinition::GetBoolDef();
		case ENiagaraParticleDataValueType::Float: return VarType == FNiagaraTypeDefinition::GetFloatDef();
		case ENiagaraParticleDataValueType::Vec2: return VarType == FNiagaraTypeDefinition::GetVec2Def();
		case ENiagaraParticleDataValueType::Vec3: return VarType == FNiagaraTypeDefinition::GetVec3Def() || VarType == FNiagaraTypeDefinition::GetPositionDef();
		case ENiagaraParticleDataValueType::Vec4: return VarType == FNiagaraTypeDefinition::GetVec4Def();
		case ENiagaraParticleDataValueType::Color: return VarType == FNiagaraTypeDefinition::GetColorDef();
		case ENiagaraParticleDataValueType::Position: return VarType == FNiagaraTypeDefinition::GetPositionDef();
		case ENiagaraParticleDataValueType::Quat: return VarType == FNiagaraTypeDefinition::GetQuatDef();
		case ENiagaraParticleDataValueType::ID: return VarType == FNiagaraTypeDefinition::GetIDDef();
		default: return false;
	}
}

static bool CheckHalfVariableType(const FNiagaraTypeDefinition& VarType, ENiagaraParticleDataValueType AttributeType)
{
	switch (AttributeType)
	{
		case ENiagaraParticleDataValueType::Float: return VarType == FNiagaraTypeDefinition::GetHalfDef();
		case ENiagaraParticleDataValueType::Vec2: return VarType == FNiagaraTypeDefinition::GetHalfVec2Def();
		case ENiagaraParticleDataValueType::Vec3: return VarType == FNiagaraTypeDefinition::GetHalfVec3Def();
		case ENiagaraParticleDataValueType::Vec4: return VarType == FNiagaraTypeDefinition::GetHalfVec4Def();
		case ENiagaraParticleDataValueType::Color: return VarType == FNiagaraTypeDefinition::GetHalfVec4Def();
		case ENiagaraParticleDataValueType::Position: return VarType == FNiagaraTypeDefinition::GetHalfVec3Def();
		case ENiagaraParticleDataValueType::Quat: return VarType == FNiagaraTypeDefinition::GetHalfVec4Def();
		default: return false;
	}
}

static ENiagaraParticleDataValueType GetValueTypeFromFuncName(const FName& FuncName)
{
	if (FuncName == GetIntByIDFunctionName || FuncName == GetIntByIndexFunctionName) return ENiagaraParticleDataValueType::Int;
	if (FuncName == GetBoolByIDFunctionName || FuncName == GetBoolByIndexFunctionName) return ENiagaraParticleDataValueType::Bool;
	if (FuncName == GetFloatByIDFunctionName || FuncName == GetFloatByIndexFunctionName) return ENiagaraParticleDataValueType::Float;
	if (FuncName == GetVec2ByIDFunctionName || FuncName == GetVec2ByIndexFunctionName) return ENiagaraParticleDataValueType::Vec2;
	if (FuncName == GetVec3ByIDFunctionName || FuncName == GetVec3ByIndexFunctionName) return ENiagaraParticleDataValueType::Vec3;
	if (FuncName == GetVec4ByIDFunctionName || FuncName == GetVec4ByIndexFunctionName) return ENiagaraParticleDataValueType::Vec4;
	if (FuncName == GetColorByIDFunctionName || FuncName == GetColorByIndexFunctionName) return ENiagaraParticleDataValueType::Color;
	if (FuncName == GetPositionByIDFunctionName || FuncName == GetPositionByIndexFunctionName) return ENiagaraParticleDataValueType::Position;
	if (FuncName == GetQuatByIDFunctionName || FuncName == GetQuatByIndexFunctionName) return ENiagaraParticleDataValueType::Quat;
	if (FuncName == GetIDByIDFunctionName || FuncName == GetIDByIndexFunctionName) return ENiagaraParticleDataValueType::ID;
	return ENiagaraParticleDataValueType::Invalid;
}

struct FNDIParticleRead_InstanceData
{
	FNiagaraSystemInstance* SystemInstance;
	FNiagaraEmitterInstance* EmitterInstance;
};

struct FNDIParticleRead_GameToRenderData
{
	FNDIParticleRead_GameToRenderData() : SourceEmitterGPUContext(nullptr) {}

	FNiagaraComputeExecutionContext* SourceEmitterGPUContext;
};

struct FNDIParticleRead_RenderInstanceData
{
	FNDIParticleRead_RenderInstanceData()
	{
		bSourceEmitterNotGPUErrorShown = false;
		bWarnFailedToFindAcquireTag = true;
	}

	FNiagaraComputeExecutionContext* SourceEmitterGPUContext = nullptr;
	FString DebugSourceName;
	const FNiagaraDataSet* CachedDataSet = nullptr;
	TArray<int32> AttributeIndices;
	TArray<int32> AttributeCompressed;
	uint32 IsLocalSpace = 0;
	int32 AcquireTagRegisterIndex = -1;
	uint32 bSourceEmitterNotGPUErrorShown : 1;
	uint32 bWarnFailedToFindAcquireTag : 1;
};

//////////////////////////////////////////////////////////////////////////

struct FNiagaraDataInterfaceProxyParticleRead : public FNiagaraDataInterfaceProxyRW
{
	virtual void ConsumePerInstanceDataFromGameThread(void* Data, const FNiagaraSystemInstanceID& InstanceID) override
	{
		FNDIParticleRead_GameToRenderData* IncomingData = static_cast<FNDIParticleRead_GameToRenderData*>(Data);
		FNDIParticleRead_RenderInstanceData* InstanceData = SystemsRenderData.Find(InstanceID);
		if (ensure(InstanceData))
		{
			if (IncomingData)
			{
				InstanceData->SourceEmitterGPUContext = IncomingData->SourceEmitterGPUContext;
			}
			else
			{
				InstanceData->SourceEmitterGPUContext = nullptr;
			}
		}
		IncomingData->~FNDIParticleRead_GameToRenderData();
	}
	
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return sizeof(FNDIParticleRead_GameToRenderData);
	}

	void CreateRenderThreadSystemData(const FNiagaraSystemInstanceID& InstanceID, bool bIsLocalSpace, FString InDebugSourceName)
	{
		check(IsInRenderingThread());
		check(!SystemsRenderData.Contains(InstanceID));
		FNDIParticleRead_RenderInstanceData& RTInstanceData = SystemsRenderData.Add(InstanceID);
		RTInstanceData.IsLocalSpace = bIsLocalSpace ? 1 : 0;
		RTInstanceData.DebugSourceName = InDebugSourceName;
	}

	void DestroyRenderThreadSystemData(const FNiagaraSystemInstanceID& InstanceID)
	{
		check(IsInRenderingThread());
		SystemsRenderData.Remove(InstanceID);
	}

	FNDIParticleRead_RenderInstanceData* GetRenderDataForSystem(const FNiagaraSystemInstanceID& InstanceID)
	{
		return SystemsRenderData.Find(InstanceID);
	}

	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override
	{
		const FNDIParticleRead_RenderInstanceData* InstanceData = SystemsRenderData.Find(Context.GetSystemInstanceID());
		if (ensure(InstanceData))
		{
			if ( InstanceData->SourceEmitterGPUContext != nullptr )
			{
				if ( FNiagaraDataSet* SourceDataSet = InstanceData->SourceEmitterGPUContext->MainDataSet )
				{
					if ( const FNiagaraDataBuffer* CurrentData = SourceDataSet->GetCurrentData() )
					{
						Context.SetDirect(CurrentData->GetNumInstances(), CurrentData->GetGPUInstanceCountBufferOffset());
					}
				}
			}
		}
	}

private:
	TMap<FNiagaraSystemInstanceID, FNDIParticleRead_RenderInstanceData> SystemsRenderData;
};

//////////////////////////////////////////////////////////////////////////

static void NDIParticleRead_FindAttributeIndices(FNDIParticleRead_RenderInstanceData* InstanceData, const FNiagaraDataSet* SourceDataSet, const FNiagaraDataInterfaceParametersCS_ParticleRead& ShaderStorage)
{
	check(ShaderStorage.AttributeNames.Num() == ShaderStorage.AttributeTypes.Num());

	const int NumAttrIndices = Align(ShaderStorage.AttributeNames.Num(), 4);
	InstanceData->AttributeIndices.SetNumUninitialized(NumAttrIndices);
	InstanceData->AttributeCompressed.SetNumUninitialized(NumAttrIndices);

	// Find the register index for each named attribute in the source emitter.
	const TArray<FNiagaraVariableBase>& SourceEmitterVariables = SourceDataSet->GetVariables();
	const TArray<FNiagaraVariableLayoutInfo>& SourceEmitterVariableLayouts = SourceDataSet->GetVariableLayouts();
	for (int AttrNameIdx = 0; AttrNameIdx < ShaderStorage.AttributeNames.Num(); ++AttrNameIdx)
	{
		const FMemoryImageName& AttrName = ShaderStorage.AttributeNames[AttrNameIdx];
		if (AttrName == NAME_None)
		{
			InstanceData->AttributeIndices[AttrNameIdx] = -1;
			InstanceData->AttributeCompressed[AttrNameIdx] = 0;
			continue;
		}

		bool FoundVariable = false;
		for (int VarIdx = 0; VarIdx < SourceEmitterVariables.Num(); ++VarIdx)
		{
			const FNiagaraVariableBase& Var = SourceEmitterVariables[VarIdx];
			if (Var.GetName() == AttrName)
			{
				ENiagaraParticleDataValueType AttributeType = ShaderStorage.AttributeTypes[AttrNameIdx];
				if (CheckVariableType(Var.GetType(), AttributeType))
				{
					const FNiagaraVariableLayoutInfo& Layout = SourceEmitterVariableLayouts[VarIdx];
					InstanceData->AttributeIndices[AttrNameIdx] =
						(AttributeType == ENiagaraParticleDataValueType::Int || AttributeType == ENiagaraParticleDataValueType::Bool || AttributeType == ENiagaraParticleDataValueType::ID) ? Layout.GetInt32ComponentStart() : Layout.GetFloatComponentStart();
					InstanceData->AttributeCompressed[AttrNameIdx] = 0;
				}
				else if (CheckHalfVariableType(Var.GetType(), AttributeType))
				{
					const FNiagaraVariableLayoutInfo& Layout = SourceEmitterVariableLayouts[VarIdx];
					InstanceData->AttributeIndices[AttrNameIdx] = Layout.GetHalfComponentStart();
					InstanceData->AttributeCompressed[AttrNameIdx] = 1;
				}
				else
				{
					UE_LOGF(LogNiagara, Error, "Variable '%ls' in emitter '%ls' has type '%ls', but particle read DI tried to access it as '%ls'.",
						*Var.GetName().ToString(), *InstanceData->DebugSourceName, *Var.GetType().GetName(), NiagaraParticleDataValueTypeName(AttributeType)
					);
					InstanceData->AttributeIndices[AttrNameIdx] = -1;
					InstanceData->AttributeCompressed[AttrNameIdx] = 0;
				}
				FoundVariable = true;
				break;
			}
		}

		if (!FoundVariable)
		{
			UE_LOGF(LogNiagara, Error, "Particle read DI is trying to access inexistent variable '%ls' in emitter '%ls'.", *AttrName.ToString(), *InstanceData->DebugSourceName);
			InstanceData->AttributeIndices[AttrNameIdx] = -1;
			InstanceData->AttributeCompressed[AttrNameIdx] = 0;
		}
	}

	// Find the register index for the AcquireTag part of the particle ID in the source emitter.
	{
		const FName FName_ID(TEXT("ID"));

		InstanceData->AcquireTagRegisterIndex = -1;
		for (int VarIdx = 0; VarIdx < SourceEmitterVariables.Num(); ++VarIdx)
		{
			const FNiagaraVariable& Var = SourceEmitterVariables[VarIdx];
			if (Var.GetName() == FName_ID)
			{
				InstanceData->AcquireTagRegisterIndex = SourceEmitterVariableLayouts[VarIdx].GetInt32ComponentStart() + 1;
				break;
			}
		}
	}

	// Initialize the buffer padding too, so we don't move garbage around.
	for (int AttrIdx = ShaderStorage.AttributeNames.Num(); AttrIdx < InstanceData->AttributeIndices.Num(); ++AttrIdx)
	{
		InstanceData->AttributeIndices[AttrIdx] = -1;
		InstanceData->AttributeCompressed[AttrIdx] = 0;
	}
}

//////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceParticleRead::UNiagaraDataInterfaceParticleRead(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyParticleRead());
}

void UNiagaraDataInterfaceParticleRead::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		const ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowNotUserVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceParticleRead::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (!EmitterName_DEPRECATED.IsEmpty())
	{
		// We aren't going to search to see if this should be self, as we don't know the user intention and it could be confusing if we switched to this mode
		EmitterBinding.BindingMode = ENiagaraDataInterfaceEmitterBindingMode::Other;
		EmitterBinding.EmitterName = FName(*EmitterName_DEPRECATED);

		EmitterName_DEPRECATED.Empty();
	}
#endif
}

#if WITH_EDITOR
void UNiagaraDataInterfaceParticleRead::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName;
	if (PropertyChangedEvent.Property)
	{
		PropertyName = PropertyChangedEvent.Property->GetFName();
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceParticleRead, EmitterBinding))
	{
		UNiagaraSystem::RecomputeExecutionOrderForDataInterface(this);
	}
}
#endif

bool UNiagaraDataInterfaceParticleRead::HasInternalAttributeReads(const UNiagaraEmitter* OwnerEmitter, const UNiagaraEmitter* Provider) const
{
	return Provider == EmitterBinding.Resolve(this);
}

bool UNiagaraDataInterfaceParticleRead::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIParticleRead_InstanceData* PIData = new (PerInstanceData) FNDIParticleRead_InstanceData;
	PIData->SystemInstance = SystemInstance;
	PIData->EmitterInstance = nullptr;

	if ( UNiagaraEmitter* OwnerEmitter = EmitterBinding.Resolve(this) )
	{
		for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
		{
			if (OwnerEmitter == EmitterInstance->GetEmitter())
			{
				PIData->EmitterInstance = &EmitterInstance.Get();
				break;
			}
		}
	}

	if (PIData->EmitterInstance == nullptr)
	{
		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "Source emitter '%ls' not found. System: %ls", *EmitterBinding.ResolveUniqueName(this), *GetFullNameSafe(SystemInstance->GetSystem()));
		}
	}

	FString DebugSourceName;
	if ( UNiagaraSystem* NiagaraSystem = PIData->SystemInstance->GetSystem() )
	{
		DebugSourceName = FString::Printf(TEXT("%s.%s"), *NiagaraSystem->GetName(), *EmitterBinding.ResolveUniqueName(this));
	}

	FNiagaraDataInterfaceProxyParticleRead* ThisProxy = GetProxyAs<FNiagaraDataInterfaceProxyParticleRead>();
	const bool bIsLocalSpace = PIData->EmitterInstance ? PIData->EmitterInstance->IsLocalSpace() : false;
	ENQUEUE_RENDER_COMMAND(FNDIParticleReadCreateRTInstance)(
		[ThisProxy, InstanceID = SystemInstance->GetId(), bIsLocalSpace, RT_DebugSourceName=MoveTemp(DebugSourceName)](FRHICommandList& CmdList)
		{
			ThisProxy->CreateRenderThreadSystemData(InstanceID, bIsLocalSpace, RT_DebugSourceName);
		}
	);

	return true;
}

void UNiagaraDataInterfaceParticleRead::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIParticleRead_InstanceData* InstData = (FNDIParticleRead_InstanceData*)PerInstanceData;
	InstData->~FNDIParticleRead_InstanceData();

	FNiagaraDataInterfaceProxyParticleRead* ThisProxy = GetProxyAs<FNiagaraDataInterfaceProxyParticleRead>();
	ENQUEUE_RENDER_COMMAND(FNDIParticleReadDestroyRTInstance) (
		[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			ThisProxy->DestroyRenderThreadSystemData(InstanceID);
		}
	);
}

int32 UNiagaraDataInterfaceParticleRead::PerInstanceDataSize() const
{
	return sizeof(FNDIParticleRead_InstanceData);
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceParticleRead::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	// Misc functionality
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = GetLocalSpaceName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsLocalSpace"));
		Sig.SetDescription(LOCTEXT("GetEmitterLocalSpace", "Returns if the emitter is using local space or world space."));
	}

	//
	// Spawn info and particle count
	//
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNumSpawnedParticlesFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Spawned")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetNumSpawnedDesc", "Returns the current number of spawned particles this frame.  Note: This is the request spawn amount, when killing particles during spawn it will not be accurate.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetIDAtSpawnIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Spawn Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("ID")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetSpawnedIDAtIndexDesc", "Returns the Niagara ID for the particle spawned at Index.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNumParticlesFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Particles")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetNumParticlesDesc", "Returns the current number of particles in the buffer.  Note: When reading from self this will be the previous frames data.");
		OutFunctions.Add(Sig);
	}

	GetPersistentIDFunctions(OutFunctions);
	GetIndexFunctions(OutFunctions);

	for (FNiagaraFunctionSignature& Function : OutFunctions)
	{
		Function.FunctionVersion = FNiagaraParticleReadDIFunctionVersion::LatestVersion;
	}
}

void UNiagaraDataInterfaceParticleRead::GetPersistentIDFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetParticleIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetParticleIndex", "Returns the particle index from the NiagaraID or -1 if not valid.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = GetParticleIndexFromIDTableName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ID Table Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle Index")));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetParticleIndexFromIDTable", "Returns the particle index from the ID table, or -1 if that index or particle is not valid.\nThe ID to Index table is guaranteed to give a stable particle index over a range should the source emitter only ever spawn and not kill particles.");
#endif
	}

	//
	// Get attribute by ID
	//
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetIntByIDFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));

		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));

		Sig.FunctionSpecifiers.Add(FName("Attribute"));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetIntByIDDesc", "Returns an Int value from a particle by NiagaraID.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetBoolByIDFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetBoolByIDDesc", "Returns a Bool value from a particle by NiagaraID.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetFloatByIDFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetFloatByIDDesc", "Returns a Float value from a particle by NiagaraID.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVec2ByIDFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetVector2ByIDDesc", "Returns a Vector2 value from a particle by NiagaraID.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVec3ByIDFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetVector3ByIDDesc", "Returns a Vector3 value from a particle by NiagaraID.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPositionByIDFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetPositionByIDDesc", "Returns a Position value from a particle by NiagaraID.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVec4ByIDFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetVector4ByIDDesc", "Returns a Vector4 value from a particle by NiagaraID.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetColorByIDFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetColorByIDDesc", "Returns a Color value from a particle by NiagaraID.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetQuatByIDFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetQuatByIDDesc", "Returns a Quat value from a particle by NiagaraID.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetIDByIDFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particle ID")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetIDByIDDesc", "Returns a NiagaraID value from a particle by NiagaraID.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

}


void UNiagaraDataInterfaceParticleRead::GetIndexFunctions(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{

	//
	// Get attribute by index
	//
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetIntByIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetIntByIndexDesc", "Returns an Int value from a particle by index.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetBoolByIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetBoolByIndexDesc", "Returns a Bool value from a particle by index.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetFloatByIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetFloatByIndexDesc", "Returns a Float value from a particle by index.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVec2ByIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetVector2ByIndexDesc", "Returns a Vector2 value from a particle by index.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVec3ByIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetVector3ByIndexDesc", "Returns a Vector3 value from a particle by index.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVec4ByIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetVector4ByIndexDesc", "Returns a Vector4 value from a particle by index.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPositionByIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetPositionByIndexDesc", "Returns a position value from a particle by index.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetColorByIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetColorByIndexDesc", "Returns a Color value from a particle by index.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetQuatByIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetQuatByIndexDesc", "Returns a Quat value from a particle by index.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetIDByIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Particle Reader")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particle Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Value")));
		Sig.FunctionSpecifiers.Add(FName("Attribute"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
#if WITH_EDITORONLY_DATA
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceParticleRead_GetIDByIndexDesc", "Returns a NiagaraID value from a particle by index.  Note: When reading from self this will be the previous frames data.");
#endif
		OutFunctions.Add(Sig);
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, GetNumSpawnedParticles);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, GetSpawnedIDAtIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, GetNumParticles);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, GetParticleIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, GetParticleIndexFromIDTable);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadInt);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadBool);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadFloat);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadVector2);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadVector3);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadVector4);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadColor);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadPosition);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadQuat);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadID);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadIntByIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadBoolByIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadFloatByIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadVector2ByIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadVector3ByIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadVector4ByIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadColorByIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadPositionByIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadQuatByIndex);
DEFINE_NDI_DIRECT_FUNC_BINDER_WITH_PAYLOAD(UNiagaraDataInterfaceParticleRead, ReadIDByIndex);

static bool HasMatchingVariable(TArrayView<const FNiagaraVariableBase> Variables, FName AttributeName, const FNiagaraTypeDefinition& ValidType)
{
	return Variables.Find(FNiagaraVariableBase(ValidType, AttributeName)) != INDEX_NONE;
}

void UNiagaraDataInterfaceParticleRead::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	// Misc functionality
	if ( BindingInfo.Name == GetLocalSpaceName )
	{	
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceParticleRead::VMGetLocalSpace);
		return;
	}

	//
	// Spawn info and particle count
	//
	if (BindingInfo.Name == GetNumSpawnedParticlesFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, GetNumSpawnedParticles)::Bind(this, OutFunc);
		return;
	}

	if (BindingInfo.Name == GetIDAtSpawnIndexFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, GetSpawnedIDAtIndex)::Bind(this, OutFunc);
		return;
	}

	if (BindingInfo.Name == GetNumParticlesFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, GetNumParticles)::Bind(this, OutFunc);
		return;
	}

	if (BindingInfo.Name == GetParticleIndexFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, GetParticleIndex)::Bind(this, OutFunc);
		return;
	}

	if (BindingInfo.Name == GetParticleIndexFromIDTableName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, GetParticleIndexFromIDTable)::Bind(this, OutFunc);
		return;
	}

	bool bBindSuccessful = false;
	FNDIParticleRead_InstanceData* PIData = static_cast<FNDIParticleRead_InstanceData*>(InstanceData);

	const FVMFunctionSpecifier* FunctionSpecifier = BindingInfo.FindSpecifier(UNiagaraDataInterfaceRWBase::NAME_Attribute);
	if (FunctionSpecifier == nullptr)
	{
		UE_LOGF(LogNiagara, Error, "VMExternalFunction '%ls' does not have a function specifier 'attribute'!", *BindingInfo.Name.ToString());
		return;
	}

	TArrayView<const FNiagaraVariableBase> EmitterVariables;
	if (PIData->EmitterInstance)
	{
		EmitterVariables = PIData->EmitterInstance->GetParticleData().GetVariables();
	}

	const FName AttributeToRead = FunctionSpecifier->Value;

	//
	// Get attribute by ID
	//
	if (BindingInfo.Name == GetIntByIDFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadInt)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetIntDef());
	}
	else if (BindingInfo.Name == GetBoolByIDFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadBool)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetBoolDef());
	}
	else if (BindingInfo.Name == GetFloatByIDFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadFloat)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetFloatDef())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfDef()));
	}
	else if (BindingInfo.Name == GetVec2ByIDFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadVector2)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetVec2Def())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec2Def()));
	}
	else if (BindingInfo.Name == GetVec3ByIDFunctionName)
	{
		// allow the vec3 reader to also read position data
		if (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetPositionDef()))
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadPosition)::Bind(this, OutFunc, AttributeToRead);
			bBindSuccessful = true;
		}
		else
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadVector3)::Bind(this, OutFunc, AttributeToRead);
			bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetVec3Def())
				|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec3Def()));
		}
	}
	else if (BindingInfo.Name == GetVec4ByIDFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadVector4)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetVec4Def())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec4Def()));
	}
	else if (BindingInfo.Name == GetPositionByIDFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadPosition)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetPositionDef())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec3Def()));
	}
	else if (BindingInfo.Name == GetColorByIDFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadColor)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetColorDef())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec4Def()));
	}
	else if (BindingInfo.Name == GetQuatByIDFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadQuat)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetQuatDef())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec4Def()));
	}
	else if (BindingInfo.Name == GetIDByIDFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadID)::Bind(this, OutFunc, AttributeToRead);
		FNiagaraVariable VariableToRead(FNiagaraTypeDefinition::GetIDDef(), AttributeToRead);
		bBindSuccessful = EmitterVariables.Find(VariableToRead) != INDEX_NONE;
	}
	//
	// Get attribute by index
	//
	else if (BindingInfo.Name == GetIntByIndexFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadIntByIndex)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetIntDef());
	}
	else if (BindingInfo.Name == GetBoolByIndexFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadBoolByIndex)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetBoolDef());
	}
	else if (BindingInfo.Name == GetFloatByIndexFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadFloatByIndex)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetFloatDef())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfDef()));
	}
	else if (BindingInfo.Name == GetVec2ByIndexFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadVector2ByIndex)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetVec2Def())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec2Def()));
	}
	else if (BindingInfo.Name == GetVec3ByIndexFunctionName)
	{
		// allow the vec3 reader to also read position data
		if (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetPositionDef()))
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadPositionByIndex)::Bind(this, OutFunc, AttributeToRead);
			bBindSuccessful = true;
		}
		else
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadVector3ByIndex)::Bind(this, OutFunc, AttributeToRead);
			bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetVec3Def())
				|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec3Def()));
		}
	}
	else if (BindingInfo.Name == GetVec4ByIndexFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadVector4ByIndex)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetVec4Def())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec4Def()));
	}
	else if (BindingInfo.Name == GetPositionByIndexFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadPositionByIndex)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetPositionDef())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec3Def()));
	}
	else if (BindingInfo.Name == GetColorByIndexFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadColorByIndex)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetColorDef())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec4Def()));
	}
	else if (BindingInfo.Name == GetQuatByIndexFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadQuatByIndex)::Bind(this, OutFunc, AttributeToRead);
		bBindSuccessful = (HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetQuatDef())
			|| HasMatchingVariable(EmitterVariables, AttributeToRead, FNiagaraTypeDefinition::GetHalfVec4Def()));
	}
	else if (BindingInfo.Name == GetIDByIndexFunctionName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceParticleRead, ReadIDByIndex)::Bind(this, OutFunc, AttributeToRead);
		FNiagaraVariable VariableToRead(FNiagaraTypeDefinition::GetIDDef(), AttributeToRead);
		bBindSuccessful = EmitterVariables.Find(VariableToRead) != INDEX_NONE;
	}

	if (!bBindSuccessful)
	{
		const UNiagaraSystem* NiagaraSystem = PIData->SystemInstance ? PIData->SystemInstance->GetSystem() : nullptr;
		const UNiagaraEmitter* NiagaraEmitter = PIData->EmitterInstance ? PIData->EmitterInstance->GetEmitter() : nullptr;
		UE_LOGF(LogNiagara, Warning, "ParticleRead: Failed to '%ls' attribute '%ls' System '%ls' Emitter '%ls'! Check that the attribute is named correctly.", *BindingInfo.Name.ToString(), *AttributeToRead.ToString(), *GetNameSafe(NiagaraSystem), *EmitterBinding.ResolveUniqueName(this));
	}
}

void UNiagaraDataInterfaceParticleRead::VMGetLocalSpace(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIParticleRead_InstanceData> InstData(Context);
	FNDIOutputParam<bool> OutIsLocalSpace(Context);

	const FNiagaraEmitterInstance* EmitterInstance = InstData.Get()->EmitterInstance;
	const bool bIsLocalSpace = EmitterInstance ? EmitterInstance->IsLocalSpace() : false;

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutIsLocalSpace.SetAndAdvance(bIsLocalSpace);
	}
}

void UNiagaraDataInterfaceParticleRead::GetNumSpawnedParticles(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIParticleRead_InstanceData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutNumSpawned(Context);

	const FNiagaraEmitterInstance* EmitterInstance = InstData.Get()->EmitterInstance;
	const FNiagaraDataBuffer* CurrentData = EmitterInstance ? EmitterInstance->GetParticleData().GetCurrentData() : nullptr;
	const int32 NumSpawned = CurrentData ? CurrentData->GetNumSpawnedInstances() : 0;

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutNumSpawned.GetDestAndAdvance() = NumSpawned;
	}
}

static const int32* GetSpawnedIDs(FVectorVMExternalFunctionContext& Context, FNiagaraEmitterInstance* EmitterInstance, int32& NumSpawned)
{
	if (!EmitterInstance)
	{
		NumSpawned = 0;
		return nullptr;
	}

	const FNiagaraDataSet& DataSet = EmitterInstance->GetParticleData();

	const int32* SpawnedIDs = DataSet.GetFreeIDTable().GetData() + DataSet.GetFreeIDTable().Max();
	NumSpawned = DataSet.NumSpawnedIDs;
	if (SpawnedIDs)
	{
		SpawnedIDs -= NumSpawned;
	}

	return SpawnedIDs;
}

void UNiagaraDataInterfaceParticleRead::GetSpawnedIDAtIndex(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIParticleRead_InstanceData> InstData(Context);
	VectorVM::FExternalFuncInputHandler<int32> InIndex(Context);

	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> OutValid(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutIDIndex(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutIDAcquireTag(Context);

	FNiagaraEmitterInstance* EmitterInstance = InstData.Get()->EmitterInstance;

	int32 NumSpawned = 0;
	const int32* SpawnedIDs = GetSpawnedIDs(Context, EmitterInstance, NumSpawned);

	int32 IDAcquireTag = EmitterInstance ? EmitterInstance->GetParticleData().GetIDAcquireTag() : 0;

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		FNiagaraBool ValidValue;
		FNiagaraID IDValue;

		int32 SpawnIndex = InIndex.GetAndAdvance();
		if (SpawnIndex >= 0 && SpawnIndex < NumSpawned)
		{
			ValidValue.SetValue(true);
			IDValue.Index = SpawnedIDs[SpawnIndex];
			IDValue.AcquireTag = IDAcquireTag;
		}
		else
		{
			ValidValue.SetValue(false);
			IDValue.Index = 0;
			IDValue.AcquireTag = 0;
		}

		*OutValid.GetDestAndAdvance() = ValidValue;
		*OutIDIndex.GetDestAndAdvance() = IDValue.Index;
		*OutIDAcquireTag.GetDestAndAdvance() = IDValue.AcquireTag;
	}
}

void UNiagaraDataInterfaceParticleRead::GetNumParticles(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIParticleRead_InstanceData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutNumParticles (Context);

	const FNiagaraEmitterInstance* EmitterInstance = InstData.Get()->EmitterInstance;
	const FNiagaraDataBuffer* CurrentData = EmitterInstance ? EmitterInstance->GetParticleData().GetCurrentData() : nullptr;
	const int32 NumParticles = CurrentData ? CurrentData->GetNumInstances() : 0;

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutNumParticles.GetDestAndAdvance() = NumParticles;
	}
}

void UNiagaraDataInterfaceParticleRead::GetParticleIndex(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIParticleRead_InstanceData> InstData(Context);
	VectorVM::FExternalFuncInputHandler<int32> ParticleIDIndexParam(Context);
	VectorVM::FExternalFuncInputHandler<int32> ParticleIDAcquireTagParam(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutIndex(Context);

	const FNiagaraEmitterInstance* EmitterInstance = InstData.Get()->EmitterInstance;
	const FNiagaraDataBuffer* CurrentData = EmitterInstance ? EmitterInstance->GetParticleData().GetCurrentData() : nullptr;

	if (!CurrentData)
	{
		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			*OutIndex.GetDestAndAdvance() = -1;
		}
		return;
	}

	const auto IDData = FNiagaraDataSetAccessor<FNiagaraID>::CreateReader(EmitterInstance->GetParticleData(), ParticleReadIDName);
	const TArray<int32>& IDTable = CurrentData->GetIDTable();

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		FNiagaraID ParticleID = { ParticleIDIndexParam.GetAndAdvance(), ParticleIDAcquireTagParam.GetAndAdvance() };

		int32 ParticleIndex = -1;
		if (ParticleID.Index >= 0 && ParticleID.Index < IDTable.Num())
		{
			ParticleIndex = IDTable[ParticleID.Index];
			if (ParticleIndex >= 0)
			{
				FNiagaraID ActualID = IDData.GetSafe(ParticleIndex, NIAGARA_INVALID_ID);
				if (ActualID != ParticleID)
				{
					ParticleIndex = -1;
				}
			}
		}

		*OutIndex.GetDestAndAdvance() = ParticleIndex;
	}
}

void UNiagaraDataInterfaceParticleRead::GetParticleIndexFromIDTable(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIParticleRead_InstanceData> InstData(Context);
	FNDIInputParam<int32> InIDTableIndex(Context);
	FNDIOutputParam<bool> OutValid(Context);
	FNDIOutputParam<int32> OutParticleIndex(Context);

	const FNiagaraEmitterInstance* EmitterInstance = InstData.Get()->EmitterInstance;
	const FNiagaraDataBuffer* CurrentData = EmitterInstance ? EmitterInstance->GetParticleData().GetCurrentData() : nullptr;

	if ( !CurrentData )
	{
		for ( int32 i=0; i < Context.GetNumInstances(); ++i )
		{
			OutValid.SetAndAdvance(false);
			OutParticleIndex.SetAndAdvance(INDEX_NONE);
		}
		return;
	}

	const TArray<int32>& IDTable = CurrentData->GetIDTable();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 IDTableIndex = InIDTableIndex.GetAndAdvance();
		if ( IDTable.IsValidIndex(IDTableIndex) )
		{
			const int32 ParticleIndex = IDTable[IDTableIndex];
			OutValid.SetAndAdvance(ParticleIndex != INDEX_NONE);
			OutParticleIndex.SetAndAdvance(ParticleIndex);
		}
		else
		{
			OutValid.SetAndAdvance(false);
			OutParticleIndex.SetAndAdvance(INDEX_NONE);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

template<typename T>
struct FDirectReadParamHandler
{
	FDirectReadParamHandler(FVectorVMExternalFunctionContext& Context)
		: InstanceData(Context)
		, ParticleIDIndexParam(Context)
		, ParticleIDAcquireTagParam(Context)
		, OutValid(Context)
		, OutValue(Context)
	{
	}

	FORCEINLINE FNiagaraEmitterInstance* GetEmitterInstance() { return InstanceData->EmitterInstance; }
	FORCEINLINE FNiagaraSystemInstance* GetSysteInstance() { return InstanceData->SystemInstance; }

	FORCEINLINE FNiagaraID GetID() { return FNiagaraID(ParticleIDIndexParam.GetAndAdvance(), ParticleIDAcquireTagParam.GetAndAdvance()); }
	FORCEINLINE void SetValid(bool bValid) { OutValid.SetAndAdvance(FNiagaraBool(bValid)); }
	FORCEINLINE void SetValue(T Val) { OutValue.SetAndAdvance(Val); }

	VectorVM::FUserPtrHandler<FNDIParticleRead_InstanceData> InstanceData;
	VectorVM::FExternalFuncInputHandler<int32> ParticleIDIndexParam;
	VectorVM::FExternalFuncInputHandler<int32> ParticleIDAcquireTagParam;
	FNDIOutputParam<FNiagaraBool> OutValid;
	FNDIOutputParam<T> OutValue;
};

template <typename T>
FORCEINLINE void ReadWithCheck(FVectorVMExternalFunctionContext& Context, FName AttributeToRead, T Default)
{
	static_assert(!TIsUECoreVariant<T, double>::Value, "Passing double core variant. Must be float type!");
	FDirectReadParamHandler<T> Params(Context);
	bool bWriteDummyData = true;
	if (FNiagaraEmitterInstance* EmitterInstance = Params.GetEmitterInstance())
	{
		const FNiagaraDataBuffer* CurrentData = EmitterInstance->GetParticleData().GetCurrentData();//TODO: We should really be grabbing this in the instance data tick and adding a read ref to it.
		if (CurrentData && EmitterInstance->GetGPUContext() == nullptr)
		{
			const TArray<int32>& IDTable = CurrentData->GetIDTable();
			int32 NumSourceInstances = CurrentData->GetNumInstances();

			if (IDTable.Num() > 0)
			{
				const auto ValueData = FNiagaraDataSetAccessor<T>::CreateReader(EmitterInstance->GetParticleData(), AttributeToRead);
				const auto IDData = FNiagaraDataSetAccessor<FNiagaraID>::CreateReader(EmitterInstance->GetParticleData(), ParticleReadIDName);

				if (IDData.IsValid() && ValueData.IsValid())
				{
					bWriteDummyData = false;

					for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
					{
						FNiagaraID ParticleID = Params.GetID();
						bool bValid = false;
						T Value = Default;

						if (ParticleID.Index >= 0 && ParticleID.Index < IDTable.Num())
						{
							int32 ParticleIndex = IDTable[ParticleID.Index];
							if (ParticleIndex >= 0 && ParticleIndex < NumSourceInstances)
							{
								FNiagaraID ActualID = IDData.GetSafe(ParticleIndex, NIAGARA_INVALID_ID);
								if (ActualID == ParticleID)
								{
									Value = ValueData.GetSafe(ParticleIndex, Default);
									bValid = true;
								}
							}
						}

						Params.SetValid(bValid);
						Params.SetValue(Value);
					}
				}
			}
		}
	}

	// Do we need to write dummy data due to not being in a valid state?
	if (bWriteDummyData)
	{
		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			Params.SetValid(false);
			Params.SetValue(Default);
		}
	}
}

//TODO: We can remove all this boiler plate and bind directly to ReadWithCheck.
void UNiagaraDataInterfaceParticleRead::ReadInt(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadWithCheck<int32>(Context, AttributeToRead, 0);
}

void UNiagaraDataInterfaceParticleRead::ReadBool(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadWithCheck<FNiagaraBool>(Context, AttributeToRead, FNiagaraBool(false));
}

void UNiagaraDataInterfaceParticleRead::ReadFloat(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadWithCheck<float>(Context, AttributeToRead, 0.0f);
}

void UNiagaraDataInterfaceParticleRead::ReadVector2(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadWithCheck<FVector2f>(Context, AttributeToRead, FVector2f::ZeroVector);
}

void UNiagaraDataInterfaceParticleRead::ReadVector3(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadWithCheck<FVector3f>(Context, AttributeToRead, FVector3f::ZeroVector);
}

void UNiagaraDataInterfaceParticleRead::ReadVector4(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadWithCheck<FVector4f>(Context, AttributeToRead, FVector4f(ForceInit));
}

void UNiagaraDataInterfaceParticleRead::ReadColor(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadWithCheck<FLinearColor>(Context, AttributeToRead, FLinearColor::White);
}

void UNiagaraDataInterfaceParticleRead::ReadPosition(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadWithCheck<FNiagaraPosition>(Context, AttributeToRead, FNiagaraPosition(ForceInit));
}

void UNiagaraDataInterfaceParticleRead::ReadQuat(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadWithCheck<FQuat4f>(Context, AttributeToRead, FQuat4f::Identity);
}

void UNiagaraDataInterfaceParticleRead::ReadID(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadWithCheck<FNiagaraID>(Context, AttributeToRead, FNiagaraID());
}

//////////////////////////////////////////////////////////////////////////

template<typename T>
struct FDirectReadByIndexParamHandler
{
	FDirectReadByIndexParamHandler(FVectorVMExternalFunctionContext& Context)
		: InstanceData(Context)
		, ParticleIndexParam(Context)
		, OutValid(Context)
		, OutValue(Context)
	{
	}

	FORCEINLINE FNiagaraEmitterInstance* GetEmitterInstance() { return InstanceData->EmitterInstance; }
	FORCEINLINE FNiagaraSystemInstance* GetSysteInstance() { return InstanceData->SystemInstance; }

	FORCEINLINE int32 GetIndex() { return ParticleIndexParam.GetAndAdvance(); }
	FORCEINLINE void SetValid(bool bValid) { OutValid.SetAndAdvance(FNiagaraBool(bValid)); }
	FORCEINLINE void SetValue(T Val) { OutValue.SetAndAdvance(Val); }

	VectorVM::FUserPtrHandler<FNDIParticleRead_InstanceData> InstanceData;
	VectorVM::FExternalFuncInputHandler<int32> ParticleIndexParam;
	FNDIOutputParam<FNiagaraBool> OutValid;
	FNDIOutputParam<T> OutValue;
};

template<typename T>
FORCEINLINE void ReadByIndexWithCheck(FVectorVMExternalFunctionContext& Context, FName AttributeToRead, T Default)
{
	static_assert(!TIsUECoreVariant<T, double>::Value, "Passing double core variant. Must be float type!");
	FDirectReadByIndexParamHandler<T> Params(Context);

	bool bWriteDummyData = true;
	if (FNiagaraEmitterInstance* EmitterInstance = Params.GetEmitterInstance())
	{
		const FNiagaraDataBuffer* CurrentData = EmitterInstance->GetParticleData().GetCurrentData();//TODO: We should really be grabbing these during instance data tick and adding a read ref. Releasing that on PostTick.
		if (CurrentData && CurrentData->GetNumInstances() > 0 && EmitterInstance->GetGPUContext() == nullptr)
		{
			int32 NumSourceInstances = (int32)CurrentData->GetNumInstances();

			const auto ValueData = FNiagaraDataSetAccessor<T>::CreateReader(EmitterInstance->GetParticleData(), AttributeToRead);
			if (ValueData.IsValid())
			{
				bWriteDummyData = false;

				for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
				{
					int32 ParticleIndex = Params.GetIndex();

					T Value = Default;
					bool bValid = false;
					if (ParticleIndex >= 0 && ParticleIndex < NumSourceInstances)
					{
						Value = ValueData.GetSafe(ParticleIndex, Default);
						bValid = true;					
					}

					Params.SetValid(bValid);
					Params.SetValue(Value);
				}
			}
		}
	}

	// Do we need to write dummy data due to not being in a valid state?
	if (bWriteDummyData)
	{
		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			Params.SetValid(false);
			Params.SetValue(Default);
		}
	}
}

//TODO: We can remove all this boiler plate and bind directly to ReadByIndexWithCheck.
void UNiagaraDataInterfaceParticleRead::ReadIntByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadByIndexWithCheck<int32>(Context, AttributeToRead, 0);
}

void UNiagaraDataInterfaceParticleRead::ReadBoolByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadByIndexWithCheck<FNiagaraBool>(Context, AttributeToRead, FNiagaraBool(false));
}

void UNiagaraDataInterfaceParticleRead::ReadFloatByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadByIndexWithCheck<float>(Context, AttributeToRead, 0.0f);
}

void UNiagaraDataInterfaceParticleRead::ReadVector2ByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadByIndexWithCheck<FVector2f>(Context, AttributeToRead, FVector2f::ZeroVector);
}

void UNiagaraDataInterfaceParticleRead::ReadVector3ByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadByIndexWithCheck<FVector3f>(Context, AttributeToRead, FVector3f::ZeroVector);
}

void UNiagaraDataInterfaceParticleRead::ReadVector4ByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadByIndexWithCheck<FVector4f>(Context, AttributeToRead, FVector4f(ForceInit));
}

void UNiagaraDataInterfaceParticleRead::ReadPositionByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadByIndexWithCheck<FNiagaraPosition>(Context, AttributeToRead, FNiagaraPosition(ForceInit));
}

void UNiagaraDataInterfaceParticleRead::ReadColorByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadByIndexWithCheck<FLinearColor>(Context, AttributeToRead, FLinearColor::White);
}

void UNiagaraDataInterfaceParticleRead::ReadQuatByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadByIndexWithCheck<FQuat4f>(Context, AttributeToRead, FQuat4f::Identity);
}

void UNiagaraDataInterfaceParticleRead::ReadIDByIndex(FVectorVMExternalFunctionContext& Context, FName AttributeToRead)
{
	ReadByIndexWithCheck<FNiagaraID>(Context, AttributeToRead, FNiagaraID());
}

bool UNiagaraDataInterfaceParticleRead::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	return CastChecked<UNiagaraDataInterfaceParticleRead>(Other)->EmitterBinding == EmitterBinding;
}

bool UNiagaraDataInterfaceParticleRead::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	CastChecked<UNiagaraDataInterfaceParticleRead>(Destination)->EmitterBinding = EmitterBinding;
	return true;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceParticleRead::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceParticleRead::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	// If we use an int array for the attribute indices, the shader compiler will actually use int4 due to the packing rules,
	// and leave 3 elements unused. Besides being wasteful, this means that the array we send to the CS would need to be padded,
	// which is a hassle. Instead, use int4 explicitly, and access individual components in the generated code.
	// Note that we have to have at least one here because hlsl doesn't support arrays of size 0.
	const int AttributeInt4Count = FMath::Max(1, FMath::DivideAndRoundUp(ParamInfo.GeneratedFunctions.Num(), 4));
	OutHLSL.Appendf(TEXT("uint			%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, IsLocalSpaceName);
	OutHLSL.Appendf(TEXT("int			%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, NumSpawnedParticlesName);
	OutHLSL.Appendf(TEXT("int			%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, SpawnedParticlesAcquireTagName);
	OutHLSL.Appendf(TEXT("uint			%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, InstanceCountOffsetName);
	OutHLSL.Appendf(TEXT("uint			%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, ParticleStrideFloatName);
	OutHLSL.Appendf(TEXT("uint			%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, ParticleStrideIntName);
	OutHLSL.Appendf(TEXT("uint			%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, ParticleStrideHalfName);
	OutHLSL.Appendf(TEXT("int			%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, AcquireTagRegisterIndexName);
	OutHLSL.Appendf(TEXT("Buffer<int>	%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, SpawnedIDsBufferName);
	OutHLSL.Appendf(TEXT("Buffer<int>	%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, IDToIndexTableName);
	OutHLSL.Appendf(TEXT("Buffer<float>	%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, InputFloatBufferName);
	OutHLSL.Appendf(TEXT("Buffer<int>	%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, InputIntBufferName);
	OutHLSL.Appendf(TEXT("Buffer<half>	%s%s;"), *ParamInfo.DataInterfaceHLSLSymbol, InputHalfBufferName);
	OutHLSL.Appendf(TEXT("int4			%s%s[%d];"), *ParamInfo.DataInterfaceHLSLSymbol, AttributeIndicesName, AttributeInt4Count);
	OutHLSL.Appendf(TEXT("int4			%s%s[%d];"), *ParamInfo.DataInterfaceHLSLSymbol, AttributeCompressedName, AttributeInt4Count);
}

static FString GenerateFetchValueHLSL(int NumComponents, const TCHAR* ComponentNames[], const TCHAR* ComponentTypeName, const FString& InputBufferName, const FString& ParticleStrideName, bool bExtraIndent)
{
	FString Code;

	for (int ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
	{
		const TCHAR* ComponentName = (NumComponents > 1) ? ComponentNames[ComponentIndex] : TEXT("");
		const FString FetchComponentCode = FString::Printf(TEXT("            Out_Value%s = %s(%s[(RegisterIndex + %d)*%s + ParticleIndex]);\n"), ComponentName, ComponentTypeName, *InputBufferName, ComponentIndex, *ParticleStrideName);
		if (bExtraIndent)
		{
			Code += TEXT("    ");
		}
		Code += FetchComponentCode;
	}

	return Code;
}

static bool GenerateGetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, ENiagaraParticleDataComponentType ComponentType, int NumComponents, bool bByIndex, FString& OutHLSL)
{
	FString FuncTemplate;

	if (bByIndex)
	{
		FuncTemplate = TEXT("void {FunctionName}(int ParticleIndex, out bool Out_Valid, out {ValueType} Out_Value)\n");
	}
	else
	{
		FuncTemplate = TEXT("void {FunctionName}(NiagaraID In_ParticleID, out bool Out_Valid, out {ValueType} Out_Value)\n");
	}

	FuncTemplate += TEXT(
		"{\n"
		"    int RegisterIndex = {AttributeIndicesName}[{AttributeIndexGroup}]{AttributeIndexComponent};\n"
	);

	if (bByIndex)
	{
		FuncTemplate += TEXT(
			"    int NumParticles = {InstanceCountOffsetName} != 0xffffffff ? RWInstanceCounts[{InstanceCountOffsetName}] : 0;\n"
			"    if(RegisterIndex != -1 && ParticleIndex >= 0 && ParticleIndex < NumParticles)\n"
		);
	}
	else
	{
		FuncTemplate += TEXT(
			"    int ParticleIndex = (RegisterIndex != -1) && (In_ParticleID.Index >= 0) ? {IDToIndexTableName}[In_ParticleID.Index] : -1;\n"
			"    int AcquireTag = (ParticleIndex != -1) ? {InputIntBufferName}[{AcquireTagRegisterIndexName}*{ParticleStrideIntName} + ParticleIndex] : 0;\n"
			"    if(ParticleIndex != -1 && In_ParticleID.AcquireTag == AcquireTag)\n"
		);
	}

	static const TCHAR* VectorComponentNames[] = { TEXT(".x"), TEXT(".y"), TEXT(".z"), TEXT(".w") };
	static const TCHAR* IDComponentNames[] = { TEXT(".Index"), TEXT(".AcquireTag") };

	const FString FullParticleStrideFloatName = ParamInfo.DataInterfaceHLSLSymbol + ParticleStrideFloatName;
	const FString FullParticleStrideIntName = ParamInfo.DataInterfaceHLSLSymbol + ParticleStrideIntName;
	const FString FullParticleStrideHalfName = ParamInfo.DataInterfaceHLSLSymbol + ParticleStrideHalfName;
	const FString FullInputFloatBufferName = ParamInfo.DataInterfaceHLSLSymbol + InputFloatBufferName;
	const FString FullInputIntBufferName = ParamInfo.DataInterfaceHLSLSymbol + InputIntBufferName;
	const FString FullInputHalfBufferName = ParamInfo.DataInterfaceHLSLSymbol + InputHalfBufferName;

	const TCHAR* ComponentTypeName;
	FString FetchValueCode;
	if (ComponentType != ENiagaraParticleDataComponentType::Float)
	{
		const TCHAR** ValueComponentNames = VectorComponentNames;
		switch (ComponentType)
		{
			case ENiagaraParticleDataComponentType::Int:
				ComponentTypeName = TEXT("int");
				break;
			case ENiagaraParticleDataComponentType::Bool:
				ComponentTypeName = TEXT("bool");
				break;
			case ENiagaraParticleDataComponentType::ID:
				ComponentTypeName = TEXT("int");
				// IDs are structs with 2 components and specific field names.
				NumComponents = 2;
				ValueComponentNames = IDComponentNames;
				break;
			default:
				UE_LOGF(LogNiagara, Error, "Unknown component type %d while generating function %ls", int(ComponentType), *FunctionInfo.InstanceName);
				return false;
		}

		FetchValueCode = GenerateFetchValueHLSL(NumComponents, ValueComponentNames, ComponentTypeName, FullInputIntBufferName, FullParticleStrideIntName, false);
	}
	else
	{
		ComponentTypeName = TEXT("float");
		// Floats and vectors can be compressed, so we need to add extra code which checks.
		FString FetchFloatCode = GenerateFetchValueHLSL(NumComponents, VectorComponentNames, ComponentTypeName, FullInputFloatBufferName, FullParticleStrideFloatName, true);
		FString FetchHalfCode = GenerateFetchValueHLSL(NumComponents, VectorComponentNames, ComponentTypeName, FullInputHalfBufferName, FullParticleStrideHalfName, true);
		FetchValueCode = FString(
			TEXT(
			"#if NIAGARA_COMPRESSED_ATTRIBUTES_ENABLED\n"
			"        BRANCH\n"
			"        if ({AttributeCompressedName}[{AttributeIndexGroup}]{AttributeIndexComponent})\n"
			"        {\n"
			)) +
			FetchHalfCode + TEXT(
			"        }\n"
			"        else\n"
			"#endif //NIAGARA_COMPRESSED_ATTRIBUTES_ENABLED\n"
			"        {\n"
			) +
			FetchFloatCode + TEXT(
			"        }\n"
			);
	}

	FString ValueTypeName;
	if (ComponentType == ENiagaraParticleDataComponentType::ID)
	{
		ValueTypeName = TEXT("NiagaraID");
	}
	else if (NumComponents == 1)
	{
		ValueTypeName = ComponentTypeName;
	}
	else
	{
		ValueTypeName = FString::Printf(TEXT("%s%d"), ComponentTypeName, NumComponents);
	}

	FuncTemplate += TEXT(
		"    {\n"
		"        Out_Valid = true;\n"
	);

	FuncTemplate += FetchValueCode;

	FuncTemplate += TEXT(
		"    }\n"
		"    else\n"
		"    {\n"
		"        Out_Valid = false;\n"
		"        Out_Value = ({ValueType})0;\n"
		"    }\n"
		"}\n\n"
	);

	TMap<FString, FStringFormatArg> FuncTemplateArgs;
	FuncTemplateArgs.Add(TEXT("FunctionName"), FunctionInfo.InstanceName);
	FuncTemplateArgs.Add(TEXT("ValueType"), ValueTypeName);
	FuncTemplateArgs.Add(TEXT("AttributeIndicesName"), ParamInfo.DataInterfaceHLSLSymbol + AttributeIndicesName);
	FuncTemplateArgs.Add(TEXT("AttributeCompressedName"), ParamInfo.DataInterfaceHLSLSymbol + AttributeCompressedName);
	FuncTemplateArgs.Add(TEXT("AttributeIndexGroup"), FunctionInstanceIndex / 4);
	FuncTemplateArgs.Add(TEXT("AttributeIndexComponent"), VectorComponentNames[FunctionInstanceIndex % 4]);
	FuncTemplateArgs.Add(TEXT("IDToIndexTableName"),  ParamInfo.DataInterfaceHLSLSymbol + IDToIndexTableName);
	FuncTemplateArgs.Add(TEXT("InputIntBufferName"), ParamInfo.DataInterfaceHLSLSymbol + InputIntBufferName);
	FuncTemplateArgs.Add(TEXT("AcquireTagRegisterIndexName"), ParamInfo.DataInterfaceHLSLSymbol + AcquireTagRegisterIndexName);
	FuncTemplateArgs.Add(TEXT("ParticleStrideIntName"), ParamInfo.DataInterfaceHLSLSymbol + ParticleStrideIntName);
	FuncTemplateArgs.Add(TEXT("FetchValueCode"), FetchValueCode);
	FuncTemplateArgs.Add(TEXT("InstanceCountOffsetName"), ParamInfo.DataInterfaceHLSLSymbol + InstanceCountOffsetName);

	OutHLSL += FString::Format(*FuncTemplate, FuncTemplateArgs);

	return true;
}

bool UNiagaraDataInterfaceParticleRead::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if (FunctionInfo.DefinitionName == GetLocalSpaceName)
	{
		static const TCHAR* FuncTemplate = TEXT(
			"void {FunctionName}(out bool Out_LocalSpace)\n"
			"{\n"
			"    Out_LocalSpace = {IsLocalSpace} != 0;\n"
			"}\n\n"
		);

		TMap<FString, FStringFormatArg> FuncTemplateArgs;
		FuncTemplateArgs.Add(TEXT("FunctionName"), FunctionInfo.InstanceName);
		FuncTemplateArgs.Add(TEXT("IsLocalSpace"), ParamInfo.DataInterfaceHLSLSymbol + IsLocalSpaceName);

		OutHLSL += FString::Format(FuncTemplate, FuncTemplateArgs);
		return true;
	}

	//
	// Spawn info and particle count
	//
	if (FunctionInfo.DefinitionName == GetNumSpawnedParticlesFunctionName)
	{
		static const TCHAR* FuncTemplate = TEXT(
			"void {FunctionName}(out int Out_NumSpawned)\n"
			"{\n"
			"    Out_NumSpawned = {NumSpawnedParticlesName};\n"
			"}\n\n"
		);

		TMap<FString, FStringFormatArg> FuncTemplateArgs;
		FuncTemplateArgs.Add(TEXT("FunctionName"), FunctionInfo.InstanceName);
		FuncTemplateArgs.Add(TEXT("NumSpawnedParticlesName"), ParamInfo.DataInterfaceHLSLSymbol + NumSpawnedParticlesName);

		OutHLSL += FString::Format(FuncTemplate, FuncTemplateArgs);
		return true;
	}
	
	if (FunctionInfo.DefinitionName == GetIDAtSpawnIndexFunctionName)
	{
		static const TCHAR* FuncTemplate = TEXT(
			"void {FunctionName}(int In_SpawnIndex, out bool Out_Valid, out NiagaraID Out_ID)\n"
			"{\n"
			"    if(In_SpawnIndex >= 0 && In_SpawnIndex < {NumSpawnedParticlesName})\n"
			"    {\n"
			"        Out_Valid = true;\n"
			"        Out_ID.Index = {SpawnedIDsBufferName}[In_SpawnIndex];\n"
			"        Out_ID.AcquireTag = {SpawnedParticlesAcquireTagName};\n"
			"    }\n"
			"    else\n"
			"    {\n"
			"        Out_Valid = false;\n"
			"        Out_ID.Index = 0;\n"
			"        Out_ID.AcquireTag = 0;\n"
			"    }\n"
			"}\n\n"
		);

		TMap<FString, FStringFormatArg> FuncTemplateArgs;
		FuncTemplateArgs.Add(TEXT("FunctionName"), FunctionInfo.InstanceName);
		FuncTemplateArgs.Add(TEXT("NumSpawnedParticlesName"), ParamInfo.DataInterfaceHLSLSymbol + NumSpawnedParticlesName);
		FuncTemplateArgs.Add(TEXT("SpawnedParticlesAcquireTagName"), ParamInfo.DataInterfaceHLSLSymbol + SpawnedParticlesAcquireTagName);
		FuncTemplateArgs.Add(TEXT("SpawnedIDsBufferName"), ParamInfo.DataInterfaceHLSLSymbol + SpawnedIDsBufferName);

		OutHLSL += FString::Format(FuncTemplate, FuncTemplateArgs);
		return true;
	}

	if (FunctionInfo.DefinitionName == GetNumParticlesFunctionName)
	{
		static const TCHAR* FuncTemplate = TEXT(
			"void {FunctionName}(out int Out_NumParticles)\n"
			"{\n"
			"    if({InstanceCountOffsetName} != 0xffffffff)\n"
			"    {\n"
			"        Out_NumParticles = RWInstanceCounts[{InstanceCountOffsetName}];\n"
			"    }\n"
			"    else\n"
			"    {\n"
			"        Out_NumParticles = 0;\n"
			"    }\n"
			"}\n\n"
		);

		TMap<FString, FStringFormatArg> FuncTemplateArgs;
		FuncTemplateArgs.Add(TEXT("FunctionName"), FunctionInfo.InstanceName);
		FuncTemplateArgs.Add(TEXT("InstanceCountOffsetName"), ParamInfo.DataInterfaceHLSLSymbol + InstanceCountOffsetName);

		OutHLSL += FString::Format(FuncTemplate, FuncTemplateArgs);
		return true;
	}

	if (FunctionInfo.DefinitionName == GetParticleIndexFunctionName)
	{
		static const TCHAR* FuncTemplate = TEXT(
			"void {FunctionName}(NiagaraID In_ParticleID, out int Out_Index)\n"
			"{\n"
			"    Out_Index = (In_ParticleID.Index >= 0) ? {IDToIndexTableName}[In_ParticleID.Index] : -1;\n"
			"    int AcquireTag = (Out_Index != -1) ? {InputIntBufferName}[{AcquireTagRegisterIndexName}*{ParticleStrideIntName} + Out_Index] : 0;\n"
			"    if(In_ParticleID.AcquireTag != AcquireTag)\n"
			"    {\n"
			"        Out_Index = -1;\n"
			"    }\n"
			"}\n\n"
		);

		TMap<FString, FStringFormatArg> FuncTemplateArgs;
		FuncTemplateArgs.Add(TEXT("FunctionName"), FunctionInfo.InstanceName);
		FuncTemplateArgs.Add(TEXT("IDToIndexTableName"), ParamInfo.DataInterfaceHLSLSymbol + IDToIndexTableName);
		FuncTemplateArgs.Add(TEXT("InputIntBufferName"), ParamInfo.DataInterfaceHLSLSymbol + InputIntBufferName);
		FuncTemplateArgs.Add(TEXT("AcquireTagRegisterIndexName"), ParamInfo.DataInterfaceHLSLSymbol + AcquireTagRegisterIndexName);
		FuncTemplateArgs.Add(TEXT("ParticleStrideIntName"), ParamInfo.DataInterfaceHLSLSymbol + ParticleStrideIntName);

		OutHLSL += FString::Format(FuncTemplate, FuncTemplateArgs);
		return true;
	}

	if (FunctionInfo.DefinitionName == GetParticleIndexFromIDTableName)
	{
		static const TCHAR* FuncTemplate = TEXT(
			"void {FunctionName}(int IDTableIndex, out bool bValid, out int ParticleIndex)\n"
			"{\n"
			"    ParticleIndex = (IDTableIndex >= 0) ? {IDToIndexTableName}[IDTableIndex] : -1;\n"
			"    bValid = (IDTableIndex >= 0) && (ParticleIndex != -1);\n"
			"}\n\n"
		);

		TMap<FString, FStringFormatArg> FuncTemplateArgs;
		FuncTemplateArgs.Add(TEXT("FunctionName"), FunctionInfo.InstanceName);
		FuncTemplateArgs.Add(TEXT("IDToIndexTableName"), ParamInfo.DataInterfaceHLSLSymbol + IDToIndexTableName);

		OutHLSL += FString::Format(FuncTemplate, FuncTemplateArgs);
		return true;
	}


	//
	// Get attribute by ID
	//
	if (FunctionInfo.DefinitionName == GetIntByIDFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Int, 1, false, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetBoolByIDFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Bool, 1, false, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetFloatByIDFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 1, false, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetVec2ByIDFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 2, false, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetVec3ByIDFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 3, false, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetVec4ByIDFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 4, false, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetPositionByIDFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 3, false, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetColorByIDFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 4, false, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetQuatByIDFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 4, false, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetIDByIDFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::ID, 1, false, OutHLSL);
	}

	//
	// Get attribute by index
	//
	if (FunctionInfo.DefinitionName == GetIntByIndexFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Int, 1, true, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetBoolByIndexFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Bool, 1, true, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetFloatByIndexFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 1, true, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetVec2ByIndexFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 2, true, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetVec3ByIndexFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 3, true, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetVec4ByIndexFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 4, true, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetPositionByIndexFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 3, true, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetColorByIndexFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 4, true, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetQuatByIndexFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::Float, 4, true, OutHLSL);
	}

	if (FunctionInfo.DefinitionName == GetIDByIndexFunctionName)
	{
		return GenerateGetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, ENiagaraParticleDataComponentType::ID, 1, true, OutHLSL);
	}

	return false;
}
#endif

void UNiagaraDataInterfaceParticleRead::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();

	const int NumAttributesInt4 = FMath::Max(1, FMath::DivideAndRoundUp(ShaderParametersBuilder.GetGeneratedFunctions().Num(), 4));
	ShaderParametersBuilder.AddLooseParamArray<FIntVector4>(AttributeIndicesMetadataName, NumAttributesInt4);
	ShaderParametersBuilder.AddLooseParamArray<FIntVector4>(AttributeCompressedMetadataName, NumAttributesInt4);
}

void UNiagaraDataInterfaceParticleRead::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyParticleRead& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyParticleRead>();

	const FNiagaraDataInterfaceParametersCS_ParticleRead& ShaderStorage = Context.GetShaderStorage<FNiagaraDataInterfaceParametersCS_ParticleRead>();
	const int32 NumAttributes = ShaderStorage.AttributeNames.Num();
	const int32 NumAttributesInt4 = FMath::Max(1, FMath::DivideAndRoundUp(NumAttributes, 4));

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	TArrayView<FIntVector4> AttributeIndices = Context.GetParameterLooseArray<FIntVector4>(NumAttributesInt4);
	TArrayView<FIntVector4> AttributeCompressed = Context.GetParameterLooseArray<FIntVector4>(NumAttributesInt4);

	auto SetErrorParams =
		[&](bool bClearSpawnInfo)
		{
			if (bClearSpawnInfo)
			{
				ShaderParameters->NumSpawnedParticles			= 0;
				ShaderParameters->SpawnedParticlesAcquireTag	= 0;
				ShaderParameters->SpawnedIDsBuffer				= FNiagaraRenderer::GetDummyIntBuffer();
			}
			ShaderParameters->InstanceCountOffset		= INDEX_NONE;
			ShaderParameters->ParticleStrideFloat		= 0;
			ShaderParameters->ParticleStrideInt			= 0;
			ShaderParameters->ParticleStrideHalf		= 0;
			ShaderParameters->AcquireTagRegisterIndex	= INDEX_NONE;
			ShaderParameters->IDToIndexTable			= FNiagaraRenderer::GetDummyIntBuffer();
			ShaderParameters->InputFloatBuffer			= FNiagaraRenderer::GetDummyFloatBuffer();
			ShaderParameters->InputIntBuffer			= FNiagaraRenderer::GetDummyIntBuffer();
			ShaderParameters->InputHalfBuffer			= FNiagaraRenderer::GetDummyHalfBuffer();

			FMemory::Memset(AttributeIndices.GetData(), 0xFF, AttributeIndices.GetTypeSize());
			FMemory::Memset(AttributeCompressed.GetData(), 0x00, AttributeCompressed.GetTypeSize());
		};

	FNDIParticleRead_RenderInstanceData* InstanceData = DIProxy.GetRenderDataForSystem(Context.GetSystemInstanceID());
	if (InstanceData == nullptr)
	{
		ShaderParameters->IsLocalSpace = 0;
		SetErrorParams(true);
		return;
	}

	ShaderParameters->IsLocalSpace = InstanceData->IsLocalSpace;
	if (InstanceData->SourceEmitterGPUContext == nullptr)
	{
		if (!InstanceData->bSourceEmitterNotGPUErrorShown)
		{
			UE_LOGF(LogNiagara, Error, "GPU particle read could not find GPU context for '%ls', emitter either doesn't exist or is not a GPU simulation.", *InstanceData->DebugSourceName);
			InstanceData->bSourceEmitterNotGPUErrorShown = true;
		}
		SetErrorParams(true);
		InstanceData->CachedDataSet = nullptr;
		InstanceData->bWarnFailedToFindAcquireTag = true;
		return;
	}
	InstanceData->bSourceEmitterNotGPUErrorShown = false;

	FNiagaraDataSet* SourceDataSet = InstanceData->SourceEmitterGPUContext->MainDataSet;
	if (!SourceDataSet)
	{
		SetErrorParams(true);
		InstanceData->CachedDataSet = nullptr;
		InstanceData->bWarnFailedToFindAcquireTag = true;
		return;
	}

	FNiagaraDataBuffer* SourceData = nullptr;
	uint32 NumSpawnedInstances = 0;
	uint32 IDAcquireTag = 0;

	// Test for reading ourself
	if (Context.GetComputeInstanceData().Context == InstanceData->SourceEmitterGPUContext)
	{
		// Pull source data from SimStageData as this has the correct information until all stages are dispatched
		// Note: In the case of an in place (aka partial update) stage this does not work as we need to bind the
		//       same buffer as both UAV & SRV.  For the moment the user must disable partial writes.
		//       Another option would be to read from the UAV, but that would equally race.
		//-TODO: Automate this in some way or surface as feedback...
		if (const FSimulationStageMetaData* StageMetaData = Context.GetSimStageData().StageMetaData)
		{
			if (StageMetaData->bPartialParticleUpdate)
			{
				if (GNiagaraParticleReadIgnoreUnsafeReads == false)
				{
					UE_LOGF(LogNiagara, Error, "Particle read DI reading self '%ls' on stage '%ls' is unsafe, please disable partial writes on the stage.", *InstanceData->DebugSourceName, *StageMetaData->SimulationStageName.ToString());
				}
				SetErrorParams(true);
				return;
			}
		}

		SourceData = Context.GetSimStageData().Source;

		// Pull some spawned / acquire tag from destination data as we are reading from ourselves this is what we are currently doing in this stage.
		if (Context.GetSimStageData().Destination)
		{
			ensure(SourceData != nullptr);
			NumSpawnedInstances = Context.GetSimStageData().Destination->GetNumSpawnedInstances();
			IDAcquireTag = Context.GetSimStageData().Destination->GetIDAcquireTag();
		}
		// In cases where we do not have destination data pull from source (i.e. this is an iteration on none particle stages)
		else if (SourceData)
		{
			NumSpawnedInstances = SourceData->GetNumSpawnedInstances();
			IDAcquireTag = SourceData->GetIDAcquireTag();
		}
	}
	else
	{
		SourceData = SourceDataSet->GetCurrentData();
		if (SourceData)
		{
			NumSpawnedInstances = SourceData->GetNumSpawnedInstances();
			IDAcquireTag = SourceData->GetIDAcquireTag();
		}
	}

	ShaderParameters->NumSpawnedParticles = NumSpawnedInstances;
	ShaderParameters->SpawnedParticlesAcquireTag = IDAcquireTag;
	ShaderParameters->SpawnedIDsBuffer = FNiagaraRenderer::GetSrvOrDefaultInt(SourceDataSet->GetGPUFreeIDs());

	if (!SourceData)
	{
		SetErrorParams(false);
		return;
	}

	if (InstanceData->CachedDataSet != SourceDataSet)
	{
		NDIParticleRead_FindAttributeIndices(InstanceData, SourceDataSet, ShaderStorage);
		InstanceData->CachedDataSet = SourceDataSet;
		InstanceData->bWarnFailedToFindAcquireTag = true;
	}

	ShaderParameters->InstanceCountOffset		= SourceData->GetGPUInstanceCountBufferOffset();
	ShaderParameters->ParticleStrideFloat		= SourceData->GetFloatStride() / sizeof(float);
	ShaderParameters->ParticleStrideInt			= SourceData->GetInt32Stride() / sizeof(int32);
	ShaderParameters->ParticleStrideHalf		= SourceData->GetHalfStride() / sizeof(FFloat16);
	ShaderParameters->AcquireTagRegisterIndex	= InstanceData->AcquireTagRegisterIndex;

	// There's no need to transition the input buffers, because the grouping logic inside FNiagaraGpuComputeDispatch ensures that our source emitter has ran before us,
	// and its buffers have been transitioned to readable.
	ShaderParameters->IDToIndexTable	= FNiagaraRenderer::GetSrvOrDefaultInt(SourceData->GetGPUIDToIndexTable());
	ShaderParameters->InputFloatBuffer	= FNiagaraRenderer::GetSrvOrDefaultFloat(SourceData->GetGPUBufferFloat());
	ShaderParameters->InputIntBuffer	= FNiagaraRenderer::GetSrvOrDefaultInt(SourceData->GetGPUBufferInt());
	ShaderParameters->InputHalfBuffer	= FNiagaraRenderer::GetSrvOrDefaultHalf(SourceData->GetGPUBufferHalf());

	if (NumAttributes > 0)
	{
		check(InstanceData->AttributeIndices.Num() * InstanceData->AttributeIndices.GetTypeSize() == AttributeIndices.Num() * AttributeIndices.GetTypeSize());
		check(InstanceData->AttributeCompressed.Num() * InstanceData->AttributeCompressed.GetTypeSize() == AttributeCompressed.Num() * AttributeCompressed.GetTypeSize());

		FMemory::Memcpy(AttributeIndices.GetData(), InstanceData->AttributeIndices.GetData(), AttributeIndices.Num() * AttributeIndices.GetTypeSize());
		FMemory::Memcpy(AttributeCompressed.GetData(), InstanceData->AttributeCompressed.GetData(), AttributeCompressed.Num() * AttributeCompressed.GetTypeSize());
	}

	if (InstanceData->bWarnFailedToFindAcquireTag && (InstanceData->AcquireTagRegisterIndex == -1) && Context.IsParameterBound(&ShaderParameters->AcquireTagRegisterIndex) )
	{
		InstanceData->bWarnFailedToFindAcquireTag = false;
		UE_LOGF(LogNiagara, Error, "Particle read DI cannot find ID variable in emitter '%ls'.", *InstanceData->DebugSourceName);
	}
}

FNiagaraDataInterfaceParametersCS* UNiagaraDataInterfaceParticleRead::CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const
{
	FNiagaraDataInterfaceParametersCS_ParticleRead* ShaderStorage = new FNiagaraDataInterfaceParametersCS_ParticleRead();

	const int32 NumFuncs = ParameterInfo.GeneratedFunctions.Num();
	ShaderStorage->AttributeNames.SetNum(NumFuncs);
	ShaderStorage->AttributeTypes.SetNum(NumFuncs);
	for (int32 FuncIdx = 0; FuncIdx < NumFuncs; ++FuncIdx)
	{
		const FNiagaraDataInterfaceGeneratedFunction& Func = ParameterInfo.GeneratedFunctions[FuncIdx];
		const FName* AttributeName = Func.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute);
		if (AttributeName != nullptr)
		{
			ShaderStorage->AttributeNames[FuncIdx] = *AttributeName;
			ShaderStorage->AttributeTypes[FuncIdx] = GetValueTypeFromFuncName(Func.DefinitionName);
		}
		else
		{
			// This is not an error. GetNumSpawnedParticles and GetIDAtSpawnIndexFunctionName don't use specifiers,
			// but they take up slots in the attribute indices array for simplicity. Just stick NAME_None in here to ignore them.
			ShaderStorage->AttributeNames[FuncIdx] = NAME_None;
			ShaderStorage->AttributeTypes[FuncIdx] = ENiagaraParticleDataValueType::Invalid;
		}
	}

	return ShaderStorage;
}

const FTypeLayoutDesc* UNiagaraDataInterfaceParticleRead::GetShaderStorageType() const
{
	return &StaticGetTypeLayoutDesc<FNiagaraDataInterfaceParametersCS_ParticleRead>();
}

void UNiagaraDataInterfaceParticleRead::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	FNDIParticleRead_GameToRenderData* RTData = new (DataForRenderThread) FNDIParticleRead_GameToRenderData;
	FNDIParticleRead_InstanceData* PIData = static_cast<FNDIParticleRead_InstanceData*>(PerInstanceData);
	if (PIData && PIData->EmitterInstance)
	{
		RTData->SourceEmitterGPUContext = PIData->EmitterInstance->GetGPUContext();
	}
}

#if WITH_EDITORONLY_DATA

bool UNiagaraDataInterfaceParticleRead::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	bool bWasChanged = false;

	// Early out for version matching
	if (FunctionSignature.FunctionVersion == FNiagaraParticleReadDIFunctionVersion::LatestVersion)
	{
		return bWasChanged;
	}

	// Renamed some functions
	if (FunctionSignature.FunctionVersion < FNiagaraParticleReadDIFunctionVersion::RenamedSpawnIndex)
	{
		static const TPair<FName, FName> FunctionRenames[] =
		{
			MakeTuple(FName("Get Spawned ID At Index"), GetIDAtSpawnIndexFunctionName),
		};

		for (const auto& RenamePair : FunctionRenames)
		{
			if (FunctionSignature.Name == RenamePair.Key)
			{
				FunctionSignature.Name = RenamePair.Value;
				bWasChanged = true;
				break;
			}
		}
	}

	// Set latest version
	FunctionSignature.FunctionVersion = FNiagaraParticleReadDIFunctionVersion::LatestVersion;

	return bWasChanged;
}

#endif

#if WITH_EDITOR	

void UNiagaraDataInterfaceParticleRead::GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& Warnings, TArray<FNiagaraDataInterfaceFeedback>& Info)
{
	if (!Asset)
	{
		return;
	}

	FVersionedNiagaraEmitter FoundSourceEmitter;
	if (const FNiagaraEmitterHandle* EmitterHandle = EmitterBinding.ResolveHandle(this))
	{
		FoundSourceEmitter = EmitterHandle->GetInstance();
	}

	if (!FoundSourceEmitter.Emitter)
	{
		Warnings.Emplace(
			LOCTEXT("SourceEmitterNotFound", "Source emitter was not found."),
			FText::Format(LOCTEXT("SourceEmitterNotFoundSummary", "Source emitter '{0}' could not be found"), FText::FromString(EmitterBinding.ResolveUniqueName(this))),
			FNiagaraDataInterfaceFix()
		);
	}

	// Filter through all the relevant CPU scripts
	TArray<UNiagaraScript*> Scripts;
	Scripts.Add(Asset->GetSystemSpawnScript());
	Scripts.Add(Asset->GetSystemUpdateScript());
	for (auto&& EmitterHandle : Asset->GetEmitterHandles())
	{
		if (FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData())
		{
			TArray<UNiagaraScript*> OutScripts;
			EmitterData->GetScripts(OutScripts, false);
			Scripts.Append(OutScripts);
		}
	}

	// Now check if any script uses functions that require persisitent ID access
	TArray<FNiagaraFunctionSignature> PersistentIDFunctions;
	TArray<FNiagaraFunctionSignature> IndexFunctions;
	GetPersistentIDFunctions(PersistentIDFunctions);
	GetIndexFunctions(IndexFunctions);

	bool bHasPersistenIDAccessWarning = [this, &Scripts, &PersistentIDFunctions]()
	{
		for (const auto Script : Scripts)
		{
			for (const auto& DIInfo : Script->GetVMExecutableData().DataInterfaceInfo)
			{
				if (DIInfo.MatchesClass(GetClass()))
				{
					for (const auto& Func : DIInfo.RegisteredFunctions)
					{
						auto Filter = [&Func](const FNiagaraFunctionSignature& CPUSig)
						{
							return CPUSig.Name == Func.Name;
						};
						if (PersistentIDFunctions.FindByPredicate(Filter))
						{
							return true;
						}
					}
				}
			}
		}
		return false;
	}();


	for (const auto Script : Scripts)
	{
		const TArray<FNiagaraScriptDataInterfaceInfo>& CachedDefaultDIs = Script->GetCachedDefaultDataInterfaces();

		for (int32 Idx = 0; Idx < Script->GetVMExecutableData().DataInterfaceInfo.Num(); Idx++)
		{
			const auto& DIInfo = Script->GetVMExecutableData().DataInterfaceInfo[Idx];
			if (!DIInfo.MatchesClass(GetClass()))
			{
				continue;
			}

			if (!CachedDefaultDIs.IsValidIndex(Idx) || CachedDefaultDIs[Idx].DataInterface == nullptr || !CachedDefaultDIs[Idx].DataInterface->Equals(this))
			{
				continue;
			}

			bool bNeedsCPUGPUCheck = false;
			for (const auto& Func : DIInfo.RegisteredFunctions)
			{
				const FName* AttributeName = Func.FunctionSpecifiers.Find(UNiagaraDataInterfaceRWBase::NAME_Attribute);
				ENiagaraParticleDataValueType AttributeType = ENiagaraParticleDataValueType::Invalid;
				if (AttributeName != nullptr)
				{
					AttributeType = GetValueTypeFromFuncName(Func.Name);
				}

				if (!bNeedsCPUGPUCheck)
				{
					bNeedsCPUGPUCheck = 
						PersistentIDFunctions.ContainsByPredicate([Func](const FNiagaraFunctionSignature& Sig) { return Sig.Name == Func.Name;}) ||
						IndexFunctions.ContainsByPredicate([Func](const FNiagaraFunctionSignature& Sig) { return Sig.Name == Func.Name; });
				}

				if (AttributeName && FoundSourceEmitter.GetEmitterData())
				{
					if (AttributeType != ENiagaraParticleDataValueType::Invalid)
					{
						auto AttribFilter = [AttributeName](const FNiagaraVariable& Var)
						{
							return Var.GetName() == *AttributeName;
						};

						TArray<FNiagaraVariableBase> Variables;
						FoundSourceEmitter.GetEmitterData()->GatherCompiledParticleAttributes(Variables);
						const FNiagaraVariableBase* FoundVar = Variables.FindByPredicate(AttribFilter);
						if (FoundVar && !CheckVariableType(FoundVar->GetType(), AttributeType))
						{
							FText Msg = FText::Format(LOCTEXT("SourceEmitterTypeMismatchError", "Source Emitter has attribute named, \"{0}\" but the type isn't compatible with the function \"{1}\", and will not succeed."), FText::FromName(*AttributeName), FText::FromName(Func.Name));
							Info.AddUnique(FNiagaraDataInterfaceFeedback(Msg, Msg, FNiagaraDataInterfaceFix()));
						}
						else if (!FoundVar)
						{
							FText Msg = FText::Format(LOCTEXT("SourceEmitterNameMismatchError", "Source Emitter does not have attribute named, \"{0}\" referenced by function \"{1}\", and will not succeed."), FText::FromName(*AttributeName), FText::FromName(Func.Name));
							Info.AddUnique(FNiagaraDataInterfaceFeedback(Msg, Msg, FNiagaraDataInterfaceFix()));
						}
					}
				}
			}

			// We assume that if the properties match, it's a valid match for us.
			if (bNeedsCPUGPUCheck)
			{
				FVersionedNiagaraEmitter OuterEmitter = Script->GetOuterEmitter();
				if (OuterEmitter.GetEmitterData() && FoundSourceEmitter.GetEmitterData())
				{
					if (OuterEmitter.GetEmitterData()->SimTarget != FoundSourceEmitter.GetEmitterData()->SimTarget)
					{
						FText Msg = FText::Format(LOCTEXT("SourceEmitterSimTypeMismatchError", "Emitter \"{0}\" SimTarget not compatible (CPU vs GPU)!"), FText::FromName(OuterEmitter.Emitter->GetFName()));
						OutErrors.AddUnique(FNiagaraDataInterfaceError(Msg, Msg, FNiagaraDataInterfaceFix()));
					}
				}
				// This is likely from a system script
				else if (FoundSourceEmitter.GetEmitterData() && FoundSourceEmitter.GetEmitterData()->SimTarget == ENiagaraSimTarget::GPUComputeSim)
				{
					FText Msg = LOCTEXT("SourceSystemSimTypeMismatchError", "Sytem script can not read from GPU emitter!");
					OutErrors.AddUnique(FNiagaraDataInterfaceError(Msg, Msg, FNiagaraDataInterfaceFix()));
				}
			}
		}
	}

	// If we found persistent ID functions in use and the target emitter isn't set to expose them, trigger a fixable warning.
	FVersionedNiagaraEmitterData* EmitterData = FoundSourceEmitter.GetEmitterData();
	if (bHasPersistenIDAccessWarning && EmitterData && EmitterData->bRequiresPersistentIDs == false)
	{
		FNiagaraDataInterfaceError SourceEmitterNeedsPersistentIDError(LOCTEXT("SourceEmitterNeedsPersistenIDError", "Source Emitter Needs PersistentIDs set."),
			LOCTEXT("SourceEmitterNeedsPersistenIDErrorSummary", "Source emitter needs persistent id's set."),
			FNiagaraDataInterfaceFix::CreateLambda([=]()
				{
					FName PropertyName = GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, bRequiresPersistentIDs);
					FProperty* FoundProp = nullptr;
					for (TFieldIterator<FProperty> PropertyIt(FVersionedNiagaraEmitterData::StaticStruct()); PropertyIt; ++PropertyIt)
					{
						FProperty* Property = *PropertyIt;
						if (Property && Property->GetFName() == PropertyName)
						{
							FoundProp = Property;
							break;
						}
					}

					FoundSourceEmitter.Emitter->Modify();
					if (FoundProp)
					{
						FPropertyChangedEvent EmptyPropertyUpdateStruct(FoundProp);

						// Go through Pre/Post edit change cycle on these because changing them will invoke a recompile on the target emitter.
						FoundSourceEmitter.Emitter->PreEditChange(FoundProp);
						EmitterData->bRequiresPersistentIDs = true;
						FoundSourceEmitter.Emitter->PostEditChangeProperty(EmptyPropertyUpdateStruct);
					}
					return true;
				}));
		OutErrors.Add(SourceEmitterNeedsPersistentIDError);
	}
}
#endif

void UNiagaraDataInterfaceParticleRead::GetEmitterDependencies(UNiagaraSystem* Asset, TArray<FVersionedNiagaraEmitter>& Dependencies) const
{
	if (!Asset)
	{
		return;
	}

	if (const FNiagaraEmitterHandle* ResolvedEmitterHandle = EmitterBinding.ResolveHandle(this))
	{
		Dependencies.Add(ResolvedEmitterHandle->GetInstance());
	}
}

void UNiagaraDataInterfaceParticleRead::GetEmitterReferencesByName(TArray<FString>& EmitterReferences) const
{
	if (const UNiagaraEmitter* ResolvedEmitter = EmitterBinding.Resolve(this))
	{
		EmitterReferences.AddUnique(ResolvedEmitter->GetUniqueEmitterName());
	}
}

#undef LOCTEXT_NAMESPACE

