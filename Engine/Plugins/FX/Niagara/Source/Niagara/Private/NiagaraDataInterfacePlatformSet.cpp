// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfacePlatformSet.h"
#include "NiagaraTypes.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraShaderParametersBuilder.h"

#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfacePlatformSet)

namespace NDIPlatformSetPrivate
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(int, bIsActive)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfacePlatformSetTemplate.ush");

	static FName IsActiveName(TEXT("IsActive"));

	struct FNDIPlatformSetProxy : public FNiagaraDataInterfaceProxy
	{
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

		bool bIsActive = false;
	};

	void RefreshPlatformSet()
	{
		for (TObjectIterator<UNiagaraDataInterfacePlatformSet> It; It; ++It)
		{
			UNiagaraDataInterfacePlatformSet* DataInterface = *It;
			if (::IsValid(DataInterface) && !DataInterface->HasAnyFlags(RF_ClassDefaultObject))
			{
				static_cast<FNDIPlatformSetProxy*>(DataInterface->GetProxy())->bIsActive = DataInterface->Platforms.IsActive();
			}
		}
	}

	FAutoConsoleVariableSink CVarSyncPlatformSet(FConsoleCommandDelegate::CreateStatic(&RefreshPlatformSet));
}

UNiagaraDataInterfacePlatformSet::UNiagaraDataInterfacePlatformSet(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDIPlatformSetPrivate;
	Proxy.Reset(new FNDIPlatformSetProxy());
}

void UNiagaraDataInterfacePlatformSet::PostInitProperties()
{
	using namespace NDIPlatformSetPrivate;
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfacePlatformSet::PostLoad()
{
	using namespace NDIPlatformSetPrivate;
	Super::PostLoad();
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		GetProxyAs<FNDIPlatformSetProxy>()->bIsActive = Platforms.IsActive();
	}
}

#if WITH_EDITOR
void UNiagaraDataInterfacePlatformSet::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	using namespace NDIPlatformSetPrivate;
	ENQUEUE_RENDER_COMMAND(UpdateProxyState)
		(
			[RT_Proxy = GetProxyAs<FNDIPlatformSetProxy>(), bActive = Platforms.IsActive()](FRHICommandListImmediate& CmdList)
			{
				RT_Proxy->bIsActive = bActive;
			}
		);
}

#endif

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfacePlatformSet::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIPlatformSetPrivate;
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = IsActiveName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("PlatformSet")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Result")));
		//	Sig.Owner = *GetName();

		OutFunctions.Add(Sig);
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePlatformSet, IsActive);
void UNiagaraDataInterfacePlatformSet::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIPlatformSetPrivate;
	if (BindingInfo.Name == IsActiveName && BindingInfo.GetNumInputs() == 0 && BindingInfo.GetNumOutputs() == 1)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfacePlatformSet, IsActive)::Bind(this, OutFunc);
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfacePlatformSet::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace NDIPlatformSetPrivate;
	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}

	InVisitor->UpdateShaderFile(TemplateShaderFile);
	InVisitor->UpdateShaderParameters<FShaderParameters>();
	return true;
}

bool UNiagaraDataInterfacePlatformSet::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIPlatformSetPrivate;
	return FunctionInfo.DefinitionName == IsActiveName;
}

void UNiagaraDataInterfacePlatformSet::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	using namespace NDIPlatformSetPrivate;
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFile, TemplateArgs);
}
#endif

void UNiagaraDataInterfacePlatformSet::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NDIPlatformSetPrivate;
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfacePlatformSet::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIPlatformSetPrivate;
	FNDIPlatformSetProxy& DIProxy = Context.GetProxy<FNDIPlatformSetProxy>();

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->bIsActive = DIProxy.bIsActive ? 1 : 0;
}

bool UNiagaraDataInterfacePlatformSet::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfacePlatformSet* TypedOther = CastChecked<const UNiagaraDataInterfacePlatformSet>(Other);
	return TypedOther->Platforms == Platforms;
}

bool UNiagaraDataInterfacePlatformSet::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfacePlatformSet* DestinationTyped = CastChecked<UNiagaraDataInterfacePlatformSet>(Destination);
	DestinationTyped->Platforms = Platforms;

	return true;
}

void UNiagaraDataInterfacePlatformSet::IsActive(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> OutValue(Context);

	bool bIsActive = Platforms.IsActive();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*OutValue.GetDestAndAdvance() = FNiagaraBool(bIsActive);
	}
}
