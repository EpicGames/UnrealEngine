// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSimStageData.h"
#include "RenderGraph.h"

#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceRW)


#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceRW"

// Global HLSL variable base names, used by HLSL.
const FString UNiagaraDataInterfaceRWBase::NumAttributesName(TEXT("_NumAttributes"));
const FString UNiagaraDataInterfaceRWBase::NumNamedAttributesName(TEXT("_NumNamedAttributes"));
const FString UNiagaraDataInterfaceRWBase::NumCellsName(TEXT("_NumCells"));
const FString UNiagaraDataInterfaceRWBase::UnitToUVName(TEXT("_UnitToUV"));
const FString UNiagaraDataInterfaceRWBase::CellSizeName(TEXT("_CellSize"));
const FString UNiagaraDataInterfaceRWBase::WorldBBoxSizeName(TEXT("_WorldBBoxSize"));

// Attribute names
const FName UNiagaraDataInterfaceRWBase::NAME_Attribute("Attribute");

// Global VM function names, also used by the shaders code generation methods.
const FName UNiagaraDataInterfaceRWBase::NumCellsFunctionName("GetNumCells");
const FName UNiagaraDataInterfaceRWBase::CellSizeFunctionName("GetCellSize");

const FName UNiagaraDataInterfaceRWBase::WorldBBoxSizeFunctionName("GetWorldBBoxSize");

const FName UNiagaraDataInterfaceRWBase::SimulationToUnitFunctionName("SimulationToUnit");
const FName UNiagaraDataInterfaceRWBase::UnitToSimulationFunctionName("UnitToSimulation");
const FName UNiagaraDataInterfaceRWBase::UnitToIndexFunctionName("UnitToIndex");
const FName UNiagaraDataInterfaceRWBase::UnitToFloatIndexFunctionName("UnitToFloatIndex");
const FName UNiagaraDataInterfaceRWBase::IndexToUnitFunctionName("IndexToUnit");
const FName UNiagaraDataInterfaceRWBase::IndexToUnitStaggeredXFunctionName("IndexToUnitStaggeredX");
const FName UNiagaraDataInterfaceRWBase::IndexToUnitStaggeredYFunctionName("IndexToUnitStaggeredY");

const FName UNiagaraDataInterfaceRWBase::IndexToLinearFunctionName("IndexToLinear");
const FName UNiagaraDataInterfaceRWBase::LinearToIndexFunctionName("LinearToIndex");

const FName UNiagaraDataInterfaceRWBase::ExecutionIndexToUnitFunctionName("ExecutionIndexToUnit");
const FName UNiagaraDataInterfaceRWBase::ExecutionIndexToGridIndexFunctionName("ExecutionIndexToGridIndex");

/*--------------------------------------------------------------------------------------------------------------------------*/

void FNDIGpuComputeDispatchArgsGenContext::SetDirect(const FIntVector3& InElementCount, uint32 GpuCountOffset) const
{
	SimStageData->DispatchArgs.ElementCount = InElementCount;
	SimStageData->DispatchArgs.GpuElementCountOffset = GpuCountOffset;
}

void FNDIGpuComputeDispatchArgsGenContext::SetIndirect(FRDGBuffer* InBuffer, uint32 BufferByteOffset) const
{
	SimStageData->DispatchArgs.IndirectBuffer = InBuffer;
	SimStageData->DispatchArgs.IndirectOffset = BufferByteOffset;
}

void FNDIGpuComputeDispatchArgsGenContext::SetIndirect(FRDGBuffer* InBuffer, uint32 BufferByteOffset, FCreateIndirectCallback&& Callback) const
{
	SimStageData->DispatchArgs.IndirectBuffer = InBuffer;
	SimStageData->DispatchArgs.IndirectOffset = BufferByteOffset;

	auto& IndirectCallback = IndirectCallbacks.AddDefaulted_GetRef();
	IndirectCallback.Key = Callback;
	IndirectCallback.Value.Key = InBuffer;
	IndirectCallback.Value.Value = BufferByteOffset;
}

FNDIGpuComputeDispatchArgsGenContext::FIndirectArgs FNDIGpuComputeDispatchArgsGenContext::CreateIndirect(const FUintVector3& InCounterOffsets) const
{
	const int32 BatchSize = FNiagaraDispatchIndirectArgsGenCS::ThreadCount;

	// Do we need to allocate new indirect buffer?
	if (IndirectBuffer == nullptr)
	{
		IndirectCounterGenArgs = GraphBuilder.AllocPODArray<FNiagaraDispatchIndirectInfoCS>(BatchSize);
		NumIndirectCounterGenArgs = 0;

		IndirectBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FNiagaraDispatchIndirectParametersCS>(BatchSize), TEXT("NiagaraIndirectBuffer"));
	}

	SimStageData->DispatchArgs.IndirectBuffer = IndirectBuffer;
	SimStageData->DispatchArgs.IndirectOffset = NumIndirectCounterGenArgs * sizeof(FNiagaraDispatchIndirectParametersCS);

	FIndirectArgs IndirectArgs;
	IndirectArgs.Key = IndirectBuffer;
	IndirectArgs.Value = NumIndirectCounterGenArgs;
	IndirectCounterGenArgs[NumIndirectCounterGenArgs++] = FNiagaraDispatchIndirectInfoCS(InCounterOffsets, SimStageData->StageMetaData->GpuDispatchNumThreads, IndirectArgs.Value);
	if (NumIndirectCounterGenArgs == BatchSize)
	{
		FlushPass();
	}
	return IndirectArgs;
}

void FNDIGpuComputeDispatchArgsGenContext::AddBufferAccess(FRDGBuffer* InBuffer, ERHIAccess InAccess) const
{
	GetPassParameters()->BufferAccessArray.Emplace(InBuffer, InAccess);
}

void FNDIGpuComputeDispatchArgsGenContext::AddTextureAccess(FRDGTexture* InTexture, ERHIAccess InAccess) const
{
	GetPassParameters()->TextureAccessArray.Emplace(InTexture, InAccess);
}

FNDIGpuComputeDispatchArgsGenParameters* FNDIGpuComputeDispatchArgsGenContext::GetPassParameters() const
{
	if (PassParameters == nullptr)
	{
		PassParameters = GraphBuilder.AllocParameters<FNDIGpuComputeDispatchArgsGenParameters>();
	}
	return PassParameters;
}

void FNDIGpuComputeDispatchArgsGenContext::FlushPass() const
{
	if (NumIndirectCounterGenArgs == 0 && IndirectCallbacks.Num() == 0)
	{
		return;
	}

	// Create Args Buffer and Upload into it
	FNiagaraDispatchIndirectArgsGenCS::FParameters* IndirectArgsGenParameters = &GetPassParameters()->IndirectArgsGenParameters;
	TArray<FRHITransitionInfo, TInlineAllocator<1>> TransitionsBefore;
	TArray<FRHITransitionInfo, TInlineAllocator<1>> TransitionsAfter;
	if (NumIndirectCounterGenArgs > 0)
	{
		const uint32 DispatchInfosBufferSize = sizeof(FNiagaraDispatchIndirectInfoCS) * NumIndirectCounterGenArgs;
		FRDGBufferRef DispatchInfosBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), DispatchInfosBufferSize / sizeof(uint32)), TEXT("NiagaraIndirectGenArgs"));
		GraphBuilder.QueueBufferUpload(DispatchInfosBuffer, IndirectCounterGenArgs, DispatchInfosBufferSize, ERDGInitialDataFlags::NoCopy);

		const FRWBuffer& InstanceCountBuffer = ComputeDispatchInterface.GetGPUInstanceCounterManager().GetInstanceCountBuffer();
		TransitionsBefore.Emplace(InstanceCountBuffer.Buffer, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute);
		TransitionsAfter.Emplace(InstanceCountBuffer.Buffer, ERHIAccess::SRVCompute, ERHIAccess::UAVCompute);

		IndirectArgsGenParameters->DispatchInfos = GraphBuilder.CreateSRV(DispatchInfosBuffer, PF_R32_UINT);
		IndirectArgsGenParameters->NumDispatchInfos = NumIndirectCounterGenArgs;
		IndirectArgsGenParameters->MaxGroupsPerDimension = FUintVector3(GRHIMaxDispatchThreadGroupsPerDimension);

		IndirectArgsGenParameters->InstanceCounts = InstanceCountBuffer.SRV;
		IndirectArgsGenParameters->RWDispatchIndirectArgs = GraphBuilder.CreateUAV(IndirectBuffer, PF_R32_UINT);
	}

	const FRWBuffer& CountBuffer = ComputeDispatchInterface.GetGPUInstanceCounterManager().GetInstanceCountBuffer();

	// Kick pass to genreate args
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Niagara::ExecuteTicks::DispatchGroupPre"),
		GetPassParameters(),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		[PassParameters=GetPassParameters(), IndirectArgsGenParameters, CreateCallbacks=MoveTemp(IndirectCallbacks), TransitionsBefore, TransitionsAfter](FRHICommandListImmediate& RHICmdList)
		{
			if (IndirectArgsGenParameters != nullptr)
			{
				RHICmdList.Transition(TransitionsBefore);

				TShaderMapRef<FNiagaraDispatchIndirectArgsGenCS> DispatchIndirectArgsGenCS(GetGlobalShaderMap(GMaxRHIFeatureLevel));
				FComputeShaderUtils::Dispatch(RHICmdList, DispatchIndirectArgsGenCS, *IndirectArgsGenParameters, FIntVector(1, 1, 1));

				RHICmdList.Transition(TransitionsAfter);
			}
			for (const auto& CreateCallback : CreateCallbacks)
			{
				CreateCallback.Key(RHICmdList, CreateCallback.Value.Key, CreateCallback.Value.Value);
			}
		}
	);

	// Clear our data
	IndirectBuffer = nullptr;
	IndirectCounterGenArgs = nullptr;
	NumIndirectCounterGenArgs = 0;
	IndirectCallbacks.Empty();
	PassParameters = nullptr;
}

/*--------------------------------------------------------------------------------------------------------------------------*/

void FNiagaraDataInterfaceProxyRW::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Context.SetDirect(GetElementCount(Context.GetSystemInstanceID()), GetGPUInstanceCountOffset(Context.GetSystemInstanceID()));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

/*--------------------------------------------------------------------------------------------------------------------------*/

UNiagaraDataInterfaceRWBase::UNiagaraDataInterfaceRWBase(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

/*--------------------------------------------------------------------------------------------------------------------------*/

UNiagaraDataInterfaceGrid3D::UNiagaraDataInterfaceGrid3D(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, ClearBeforeNonIterationStage(true)
	, NumCells(3, 3, 3)
	, CellSize(1.)
	, NumCellsMaxAxis(10)
	, SetResolutionMethod(ESetResolutionMethod::Independent)	
	, WorldBBoxSize(100., 100., 100.)
{
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceGrid3D::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::WorldBBoxSizeFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("WorldBBoxSize")));
		Sig.bHidden = true;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::NumCellsFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsZ")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::SimulationToUnitFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Simulation")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("SimulationToUnitTransform")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::UnitToSimulationFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("UnitToSimulationTransform")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Simulation")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::UnitToIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::UnitToFloatIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Index")));	

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::IndexToUnitFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::IndexToLinearFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::LinearToIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::ExecutionIndexToUnitFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::ExecutionIndexToGridIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::CellSizeFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CellSize")));		
		Sig.bHidden = true;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}
}
#endif

void UNiagaraDataInterfaceGrid3D::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	//if (BindingInfo.Name == WorldBBoxSizeFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == NumCellsFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }	
	//else if (BindingInfo.Name == SimulationToUnitFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == UnitToSimulationFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == UnitToIndexFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == UnitToFloatIndexFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == IndexToUnitFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == IndexToLinearFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == LinearToIndexFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == ExecutionIndexToUnitFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == ExecutionIndexToGridIndexFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == CellSizeFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
}

bool UNiagaraDataInterfaceGrid3D::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceGrid3D* OtherTyped = CastChecked<const UNiagaraDataInterfaceGrid3D>(Other);

	return 
		OtherTyped->ClearBeforeNonIterationStage == ClearBeforeNonIterationStage &&
		OtherTyped->NumCells == NumCells &&
		FMath::IsNearlyEqual(OtherTyped->CellSize, CellSize) &&		
		OtherTyped->WorldBBoxSize.Equals(WorldBBoxSize) && 
		OtherTyped->SetResolutionMethod == SetResolutionMethod && 
		OtherTyped->NumCellsMaxAxis == NumCellsMaxAxis;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceGrid3D::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	static const TCHAR *FormatDeclarations = TEXT(R"(
		int3 {NumCellsName};
		float3 {UnitToUVName};
		float3 {CellSizeName};		
		float3 {WorldBBoxSizeName};
	)");
	TMap<FString, FStringFormatArg> ArgsDeclarations = {
		{ TEXT("NumCellsName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName },
		{ TEXT("UnitToUVName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName },
		{ TEXT("CellSizeName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::CellSizeName },
		{ TEXT("WorldBBoxSizeName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::WorldBBoxSizeName },
	};
	OutHLSL += FString::Format(FormatDeclarations, ArgsDeclarations);
}

bool UNiagaraDataInterfaceGrid3D::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	TMap<FString, FStringFormatArg> ArgsDeclarations = {
		{ TEXT("FunctionName"), FunctionInfo.InstanceName},
		{ TEXT("NumCellsName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName },
		{ TEXT("UnitToUVName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName },
		{ TEXT("CellSizeName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::CellSizeName },
		{ TEXT("WorldBBoxSizeName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::WorldBBoxSizeName },
	};

	if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::WorldBBoxSizeFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(out float3 Out_WorldBBox)
			{
				Out_WorldBBox = {WorldBBoxSizeName};				
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::NumCellsFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(out int Out_NumCellsX, out int Out_NumCellsY, out int Out_NumCellsZ)
			{
				Out_NumCellsX = {NumCellsName}.x;
				Out_NumCellsY = {NumCellsName}.y;
				Out_NumCellsZ = {NumCellsName}.z;
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::SimulationToUnitFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(float3 In_Simulation, float4x4 In_SimulationToUnitTransform, out float3 Out_Unit)
			{
				Out_Unit = mul(float4(In_Simulation, 1.0), In_SimulationToUnitTransform).xyz;
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::UnitToSimulationFunctionName)
	{
		static const TCHAR* FormatSample = TEXT(R"(
			void {FunctionName}(float3 In_Unit, float4x4 In_UnitToSimulationTransform, out float3 Out_Simulation)
			{
				Out_Simulation = mul(float4(In_Unit, 1.0), In_UnitToSimulationTransform).xyz;
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::UnitToIndexFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(float3 In_Unit, out int Out_IndexX, out int Out_IndexY, out int Out_IndexZ)
			{
				int3 Out_IndexTmp = round(In_Unit * {NumCellsName} - .5);
				Out_IndexX = Out_IndexTmp.x;
				Out_IndexY = Out_IndexTmp.y;
				Out_IndexZ = Out_IndexTmp.z;
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::UnitToFloatIndexFunctionName)
	{
		static const TCHAR* FormatSample = TEXT(R"(
			void {FunctionName}(float3 In_Unit, out float3 Out_Index)
			{
				Out_Index = In_Unit * {NumCellsName} - .5;				
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::IndexToUnitFunctionName)
	{
		static const TCHAR* FormatSample = TEXT(R"(
			void {FunctionName}(float In_IndexX, float In_IndexY, float In_IndexZ, out float3 Out_Unit)
			{
				Out_Unit = (float3(In_IndexX, In_IndexY, In_IndexZ) + .5) * {UnitToUVName};
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::IndexToLinearFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, out int Out_Linear)
			{
				Out_Linear = In_IndexX + In_IndexY * {NumCellsName}.x + In_IndexZ * {NumCellsName}.x * {NumCellsName}.y;
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::LinearToIndexFunctionName)
	{
		static const TCHAR* FormatSample = TEXT(R"(
			void {FunctionName}(int In_Linear, out int Out_IndexX, out int Out_IndexY, out int Out_IndexZ)
			{
				Out_IndexX = In_Linear % {NumCellsName}.x;
				Out_IndexY = (In_Linear / {NumCellsName}.x) % {NumCellsName}.y;
				Out_IndexZ = In_Linear / ({NumCellsName}.x * {NumCellsName}.y);
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::ExecutionIndexToUnitFunctionName)
	{
		static const TCHAR* FormatSample = TEXT(R"(
			void {FunctionName}(out float3 Out_Unit)
			{
				#if NIAGARA_DISPATCH_TYPE == NIAGARA_DISPATCH_TYPE_THREE_D || NIAGARA_DISPATCH_TYPE == NIAGARA_DISPATCH_TYPE_CUSTOM
					Out_Unit = (float3(GDispatchThreadId.x, GDispatchThreadId.y, GDispatchThreadId.z) + .5) / {NumCellsName};
				#else
					const uint Linear = GLinearThreadId;
					const uint IndexX = Linear % {NumCellsName}.x;
					const uint IndexY = (Linear / {NumCellsName}.x) % {NumCellsName}.y;
					const uint IndexZ = Linear / ({NumCellsName}.x * {NumCellsName}.y);				

					Out_Unit = (float3(IndexX, IndexY, IndexZ) + .5) / {NumCellsName};				
				#endif
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::ExecutionIndexToGridIndexFunctionName)
	{
		static const TCHAR* FormatSample = TEXT(R"(
			void {FunctionName}(out int Out_IndexX, out int Out_IndexY, out int Out_IndexZ)
			{
				#if NIAGARA_DISPATCH_TYPE == NIAGARA_DISPATCH_TYPE_THREE_D || NIAGARA_DISPATCH_TYPE == NIAGARA_DISPATCH_TYPE_CUSTOM
					Out_IndexX = GDispatchThreadId.x;
					Out_IndexY = GDispatchThreadId.y;
					Out_IndexZ = GDispatchThreadId.z;
				#else
					const uint Linear = GLinearThreadId;
					Out_IndexX = Linear % {NumCellsName}.x;
					Out_IndexY = (Linear / {NumCellsName}.x) % {NumCellsName}.y;
					Out_IndexZ = Linear / ({NumCellsName}.x * {NumCellsName}.y);
				#endif
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::CellSizeFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(out float3 Out_CellSize)
			{
				Out_CellSize = {CellSizeName};
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}

	return false;
}
#endif

bool UNiagaraDataInterfaceGrid3D::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceGrid3D* OtherTyped = CastChecked<UNiagaraDataInterfaceGrid3D>(Destination);

	OtherTyped->ClearBeforeNonIterationStage = ClearBeforeNonIterationStage;
	OtherTyped->NumCells = NumCells;
	OtherTyped->CellSize = CellSize;
	OtherTyped->SetResolutionMethod = SetResolutionMethod;
	OtherTyped->WorldBBoxSize = WorldBBoxSize;
	OtherTyped->NumCellsMaxAxis = NumCellsMaxAxis;

	return true;
}

/*--------------------------------------------------------------------------------------------------------------------------*/

UNiagaraDataInterfaceGrid2D::UNiagaraDataInterfaceGrid2D(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, ClearBeforeNonIterationStage(true)
	, NumCellsX(3)
	, NumCellsY(3)
	, NumCellsMaxAxis(3)
	, NumAttributes(0)
	, SetGridFromMaxAxis(false)	
	, WorldBBoxSize(100., 100.)
{
}

void UNiagaraDataInterfaceGrid2D::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::NiagaraGrid2DDefaultUnnamedAttributesZero)
	{
		NumAttributes = 1;
	}

	Super::Serialize(Ar);
}

#if WITH_EDITOR
void UNiagaraDataInterfaceGrid2D::ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors)
{
	TArray<FNiagaraFunctionSignature> DIFuncs;
	GetFunctionsInternal(DIFuncs);
	
	// All the deprecated grid2d functions
	TSet<FName> DeprecatedFunctionNames;
	DeprecatedFunctionNames.Add(UNiagaraDataInterfaceRWBase::WorldBBoxSizeFunctionName);
	DeprecatedFunctionNames.Add(UNiagaraDataInterfaceRWBase::CellSizeFunctionName);

	if (DIFuncs.Contains(Function) && DeprecatedFunctionNames.Contains(Function.Name))
	{
		// #TODO(dmp): add validation warnings that aren't as strict as these errors
		// OutValidationErrors.Add(FText::Format(LOCTEXT("Grid2DDeprecationMsgFmt", "Grid2D DI Function {0} has been deprecated. Specify grid size on your emitter.\n"), FText::FromString(Function.GetName())));	
	}
	Super::ValidateFunction(Function, OutValidationErrors);
}

void UNiagaraDataInterfaceGrid2D::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{	
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::WorldBBoxSizeFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("WorldBBoxSize")));		
		Sig.bHidden = true;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::NumCellsFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY")));		

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::SimulationToUnitFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Simulation")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("SimulationToUnitTransform")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::UnitToSimulationFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("UnitToSimulationTransform")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Simulation")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::UnitToIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));		

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::UnitToFloatIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Index")));		

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::IndexToUnitFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("IndexY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::IndexToUnitStaggeredXFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("IndexY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::IndexToUnitStaggeredYFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("IndexY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::IndexToLinearFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::LinearToIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::ExecutionIndexToUnitFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::ExecutionIndexToGridIndexFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UNiagaraDataInterfaceRWBase::CellSizeFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("CellSize")));
		Sig.bHidden = true;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}
}
#endif

void UNiagaraDataInterfaceGrid2D::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	//if (BindingInfo.Name == WorldBBoxSizeFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == NumCellsFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == SimulationToUnitFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == UnitToSimulationFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == UnitToIndexFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == UnitToFloatIndexFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == IndexToUnitFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == IndexToUnitStaggeredXFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == IndexToUnitStaggeredYFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == IndexToLinearFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == ExecutionIndexToUnitFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == ExecutionIndexToGridIndexFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == LinearToIndexFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
	//else if (BindingInfo.Name == CellSizeFunctionName) { OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRWBase::EmptyVMFunction); }
}


bool UNiagaraDataInterfaceGrid2D::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceGrid2D* OtherTyped = CastChecked<const UNiagaraDataInterfaceGrid2D>(Other);

	return 
		OtherTyped->ClearBeforeNonIterationStage == ClearBeforeNonIterationStage &&
		OtherTyped->NumCellsX == NumCellsX &&
		OtherTyped->NumCellsY == NumCellsY &&
		OtherTyped->NumAttributes == NumAttributes &&
		OtherTyped->NumCellsMaxAxis == NumCellsMaxAxis &&		
		OtherTyped->WorldBBoxSize.Equals(WorldBBoxSize);
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceGrid2D::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	static const TCHAR *FormatDeclarations = TEXT(R"(
		int2 {NumCellsName};
		float2 {UnitToUVName};
		float2 {CellSizeName};		
		float2 {WorldBBoxSizeName};
	)");
	TMap<FString, FStringFormatArg> ArgsDeclarations = {
		{ TEXT("NumCellsName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName },
		{ TEXT("UnitToUVName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName },
		{ TEXT("CellSizeName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::CellSizeName },
		{ TEXT("WorldBBoxSizeName"),    ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::WorldBBoxSizeName },
	};
	OutHLSL += FString::Format(FormatDeclarations, ArgsDeclarations);
}

bool UNiagaraDataInterfaceGrid2D::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	TMap<FString, FStringFormatArg> ArgsDeclarations = {
		{ TEXT("FunctionName"), FunctionInfo.InstanceName},
		{ TEXT("NumCellsName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName },
		{ TEXT("UnitToUVName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName },
		{ TEXT("CellSizeName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::CellSizeName },		
		{ TEXT("WorldBBoxSizeName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::WorldBBoxSizeName },
	};

	if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::WorldBBoxSizeFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(out float2 Out_WorldBBox)
			{
				Out_WorldBBox = {WorldBBoxSizeName};				
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::NumCellsFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(out int Out_NumCellsX, out int Out_NumCellsY)
			{
				Out_NumCellsX = {NumCellsName}.x;
				Out_NumCellsY = {NumCellsName}.y;
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::SimulationToUnitFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(float3 In_Simulation, float4x4 In_SimulationToUnitTransform, out float3 Out_Unit)
			{
				Out_Unit = mul(float4(In_Simulation, 1.0), In_SimulationToUnitTransform).xyz;
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::UnitToSimulationFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(float3 In_Unit, float4x4 In_UnitToSimulationTransform, out float3 Out_Simulation)
			{
				Out_Simulation = mul(float4(In_Unit, 1.0), In_UnitToSimulationTransform).xyz;
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::UnitToIndexFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(float2 In_Unit, out int Out_IndexX, out int Out_IndexY)
			{
				int2 Out_IndexTmp = round(In_Unit * float2({NumCellsName})  - .5);
				Out_IndexX = Out_IndexTmp.x;
				Out_IndexY = Out_IndexTmp.y;				
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::UnitToFloatIndexFunctionName)
	{
		static const TCHAR* FormatSample = TEXT(R"(
			void {FunctionName}(float2 In_Unit, out float2 Out_Index)
			{
				Out_Index = In_Unit * float2({NumCellsName})  - .5;							
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::IndexToUnitFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(float In_IndexX, float In_IndexY, out float3 Out_Unit)
			{
				Out_Unit = float3((float2(In_IndexX, In_IndexY) + .5) * {UnitToUVName}, 0);
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::IndexToUnitStaggeredXFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(float In_IndexX, float In_IndexY, out float3 Out_Unit)
			{
				Out_Unit = float3((float2(In_IndexX, In_IndexY) + float2(0.0, 0.5)) * {UnitToUVName}, 0);
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::IndexToUnitStaggeredYFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(float In_IndexX, float In_IndexY, out float3 Out_Unit)
			{
				Out_Unit = float3((float2(In_IndexX, In_IndexY) +  + float2(0.5, 0.0)) * {UnitToUVName}, 0);
			}
		)");
		
		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::IndexToLinearFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, out int Out_Linear)
			{
				Out_Linear = In_IndexX + In_IndexY * {NumCellsName}.x;
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::LinearToIndexFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(int In_Linear, out int Out_IndexX, out int Out_IndexY)
			{
				Out_IndexX = In_Linear % {NumCellsName}.x;
				Out_IndexY = In_Linear / {NumCellsName}.x;				
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::ExecutionIndexToUnitFunctionName)
	{
		static const TCHAR* FormatSample = TEXT(R"(
			void {FunctionName}(out float2 Out_Unit)
			{
				#if NIAGARA_DISPATCH_TYPE == NIAGARA_DISPATCH_TYPE_TWO_D || NIAGARA_DISPATCH_TYPE == NIAGARA_DISPATCH_TYPE_CUSTOM
					Out_Unit = (float2(GDispatchThreadId.x, GDispatchThreadId.y) + .5) * rcp({NumCellsName});
				#else
					const uint Linear = GLinearThreadId;
					const uint IndexX = Linear % {NumCellsName}.x;
					const uint IndexY = Linear / {NumCellsName}.x;				
					Out_Unit = (float2(IndexX, IndexY) + .5) * {UnitToUVName};			
				#endif
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::ExecutionIndexToGridIndexFunctionName)
	{
		static const TCHAR* FormatSample = TEXT(R"(
			void {FunctionName}(out int Out_IndexX, out int Out_IndexY)
			{
				#if NIAGARA_DISPATCH_TYPE == NIAGARA_DISPATCH_TYPE_TWO_D || NIAGARA_DISPATCH_TYPE == NIAGARA_DISPATCH_TYPE_CUSTOM
					Out_IndexX = GDispatchThreadId.x;
					Out_IndexY = GDispatchThreadId.y;
				#else
					const uint Linear = GLinearThreadId;
					Out_IndexX = Linear % {NumCellsName}.x;
					Out_IndexY = Linear / {NumCellsName}.x;				
				#endif
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::CellSizeFunctionName)
	{
		static const TCHAR *FormatSample = TEXT(R"(
			void {FunctionName}(out float2 Out_CellSize)
			{
				Out_CellSize = {CellSizeName};
			}
		)");

		OutHLSL += FString::Format(FormatSample, ArgsDeclarations);
		return true;
	}

	return false;
}
#endif

bool UNiagaraDataInterfaceGrid2D::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceGrid2D* OtherTyped = CastChecked<UNiagaraDataInterfaceGrid2D>(Destination);

	OtherTyped->ClearBeforeNonIterationStage = ClearBeforeNonIterationStage;
	OtherTyped->NumCellsX = NumCellsX;
	OtherTyped->NumCellsY = NumCellsY;
	OtherTyped->NumAttributes = NumAttributes;
	OtherTyped->NumCellsMaxAxis = NumCellsMaxAxis;
	OtherTyped->SetGridFromMaxAxis = SetGridFromMaxAxis;	
	OtherTyped->WorldBBoxSize = WorldBBoxSize;

	return true;
}

#undef LOCTEXT_NAMESPACE

