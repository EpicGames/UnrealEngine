// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceRWUtils.h"
#include "NiagaraShader.h"
#include "ShaderParameterUtils.h"
#include "ClearQuad.h"

#include "NiagaraDataInterfaceGrid3DCollection.h"
#include "NiagaraDataInterfaceGrid2DCollection.h"

#include "UObject/UE5MainStreamObjectVersion.h"

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceRWUtils"

TArray<FString> FNiagaraDataInterfaceRWAttributeHelper::Channels = { "r", "g", "b", "a" };
FNiagaraDataInterfaceRWAttributeHelper::FNiagaraDataInterfaceRWAttributeHelper(const FNiagaraDataInterfaceGPUParamInfo& InParamInfo) : ParamInfo(InParamInfo)
{
}

template<typename T>
void FNiagaraDataInterfaceRWAttributeHelper::Init(TArray<FText>* OutWarnings /*= nullptr*/)
{
	AttributeInfos.Reserve(ParamInfo.GeneratedFunctions.Num());
	TotalChannels = 0;

	for (const FNiagaraDataInterfaceGeneratedFunction& Function : ParamInfo.GeneratedFunctions)
	{
		const FName* AttributeName = Function.FindSpecifierValue(UNiagaraDataInterfaceRWBase::NAME_Attribute);
		if (AttributeName == nullptr)
		{
			continue;
		}

		if (const FAttributeInfo* ExistingAttribute = FindAttributeInfo(*AttributeName))
		{
			if (OutWarnings != nullptr)
			{
				FNiagaraTypeDefinition AttributeTypeDef = T::GetValueTypeFromFuncName(Function.DefinitionName);
				if (ExistingAttribute->TypeDef != AttributeTypeDef)
				{
					OutWarnings->Emplace(FText::Format(LOCTEXT("BadType", "Same name, different types! {0} vs {1}, Attribute {2}"), AttributeTypeDef.GetNameText(), ExistingAttribute->TypeDef.GetNameText(), FText::FromName(ExistingAttribute->Name)));
				}
			}
			continue;
		}

		FAttributeInfo& AttributeInfo = AttributeInfos.AddDefaulted_GetRef();
		AttributeInfo.Name = *AttributeName;
		AttributeInfo.TypeDef = T::GetValueTypeFromFuncName(Function.DefinitionName);
		AttributeInfo.NumChannels = T::GetComponentCountFromFuncName(Function.DefinitionName);
		AttributeInfo.ChannelOffset = TotalChannels;
		AttributeInfo.AttributeIndex = AttributeInfos.Num() - 1;
		TotalChannels += AttributeInfo.NumChannels;
	}
}

const FNiagaraDataInterfaceRWAttributeHelper::FNiagaraDataInterfaceRWAttributeHelper::FAttributeInfo* FNiagaraDataInterfaceRWAttributeHelper::FindAttributeInfo(FName AttributeName) const
{
	return AttributeInfos.FindByPredicate([AttributeName](const FAttributeInfo& Info) { return Info.Name == AttributeName; });
}

#if WITH_EDITORONLY_DATA

FString FNiagaraDataInterfaceRWAttributeHelper::GetAttributeIndex(const FAttributeInfo* AttributeInfo, int Channel /*= 0*/) const
{
	check(AttributeInfo);

	return FString::Printf(TEXT("%d"), AttributeInfo->ChannelOffset + Channel);
}

FString FNiagaraDataInterfaceRWAttributeHelper::GetGridChannelString() const
{
	FString NumChannelsString = "";
	if (TotalChannels > 1)
	{
		NumChannelsString = FString::FromInt(TotalChannels);
	}
	return NumChannelsString;
}

void FNiagaraDataInterfaceRWAttributeHelper::GetChannelStrings(int AttributeIndex, int AttributeNumChannels, FString& NumChannelsString, FString& AttrGridChannels)  const
{
	NumChannelsString = "";
	if (AttributeNumChannels > 1)
	{
		NumChannelsString = FString::FromInt(AttributeNumChannels);
	}

	AttrGridChannels = FNiagaraDataInterfaceRWAttributeHelper::Channels[AttributeIndex];
	for (int i = 1; i < AttributeNumChannels; ++i)
	{
		AttrGridChannels += FNiagaraDataInterfaceRWAttributeHelper::Channels[AttributeIndex + i];
	}
}

void FNiagaraDataInterfaceRWAttributeHelper::GetChannelStrings(const FAttributeInfo* AttributeInfo, FString& NumChannelsString, FString& AttrGridChannels)  const
{
	NumChannelsString = "";
	if (AttributeInfo->NumChannels > 1)
	{
		NumChannelsString = FString::FromInt(AttributeInfo->NumChannels);
	}

	AttrGridChannels = FNiagaraDataInterfaceRWAttributeHelper::Channels[AttributeInfo->AttributeIndex];
	for (int i = 1; i < AttributeInfo->NumChannels; ++i)
	{
		AttrGridChannels += FNiagaraDataInterfaceRWAttributeHelper::Channels[AttributeInfo->AttributeIndex + i];
	}
}
#endif

template void FNiagaraDataInterfaceRWAttributeHelper::Init< UNiagaraDataInterfaceGrid3DCollection>(TArray<FText>* OutWarnings);
template void FNiagaraDataInterfaceRWAttributeHelper::Init< UNiagaraDataInterfaceGrid2DCollection>(TArray<FText>* OutWarnings);

#undef LOCTEXT_NAMESPACE

