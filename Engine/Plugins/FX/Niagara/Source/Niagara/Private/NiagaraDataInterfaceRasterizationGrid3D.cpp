// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceRasterizationGrid3D.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraRenderer.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderGraphUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceRasterizationGrid3D)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceRasterizationGrid3D"

namespace NDIRasterizationGrid3DLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
		SHADER_PARAMETER(FIntVector,						NumCells)
		SHADER_PARAMETER(FVector3f,							UnitToUV)
		SHADER_PARAMETER(float,								Precision)
		SHADER_PARAMETER(int,								NumAttributes)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D<int>,	IntGrid)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<int>,	OutputIntGrid)
		SHADER_PARAMETER_SRV(Buffer<float4>,				PerAttributeData)
	END_SHADER_PARAMETER_STRUCT()

	static const FString IntGridName(TEXT("_IntGrid"));
	static const FString OutputIntGridName(TEXT("_OutputIntGrid"));
	static const FString PrecisionName(TEXT("_Precision"));

	static const FName SetNumCellsFunctionName("SetNumCells");
	static const FName SetNumAttributesFunctionName("SetNumAttributes");
	static const FName SetFloatResetValueFunctionName("SetFloatResetValue");

	static const FString PerAttributeDataName(TEXT("_PerAttributeData"));

	// Global VM function names, also used by the shaders code generation methods.

	static const FName SetFloatValueFunctionName("SetFloatGridValue");
	static const FName GetFloatValueFunctionName("GetFloatGridValue");

	static const FName InterlockedAddFloatGridValueFunctionName("InterlockedAddFloatGridValue");
	static const FName InterlockedAddIntGridValueFunctionName("InterlockedAddIntGridValue");
	static const FName InterlockedAddFloatGridValueSafeFunctionName("InterlockedAddFloatGridValueSafe");


	static const FName InterlockedMinFloatGridValueFunctionName("InterlockedMinFloatGridValue");
	static const FName InterlockedMaxFloatGridValueFunctionName("InterlockedMaxFloatGridValue");

	static const FName IntToFloatFunctionName("IntToFloat");
	static const FName FloatToIntFunctionName("FloatToInt");

	static const FName SetIntValueFunctionName("SetIntGridValue");
	static const FName GetIntValueFunctionName("GetIntGridValue");

	static int32 GMaxNiagaraRasterizationGridCells = (1024 * 1024 * 1024);
	static FAutoConsoleVariableRef CVarMaxNiagaraRasterizationGridCells(
		TEXT("fx.MaxNiagaraRasterizationGridCells"),
		GMaxNiagaraRasterizationGridCells,
		TEXT("The max number of supported grid cells in Niagara. Overflowing this threshold will cause the sim to warn and fail. \n"),
		ECVF_Default
	);
}

/*--------------------------------------------------------------------------------------------------------------------------*/

UNiagaraDataInterfaceRasterizationGrid3D::UNiagaraDataInterfaceRasterizationGrid3D(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, NumAttributes(1)
	, Precision(1.f)
	, ResetValue(0)
{	
	Proxy.Reset(new FNiagaraDataInterfaceProxyRasterizationGrid3D());	
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceRasterizationGrid3D::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIRasterizationGrid3DLocal;

	Super::GetFunctionsInternal(OutFunctions);

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetNumCellsFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsZ")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetNumAttributesFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumAttributes")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		OutFunctions.Add(Sig);
	}
	
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetFloatResetValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("ResetValue")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetFloatValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IGNORE")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetIntValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IGNORE")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = InterlockedAddFloatGridValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IGNORE")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		
		Sig.bSoftDeprecatedFunction = true;
		Sig.bHidden = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = InterlockedAddFloatGridValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("OriginalValue")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;

		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = InterlockedAddIntGridValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IGNORE")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;

		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = InterlockedAddFloatGridValueSafeFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IsSafe")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;

		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = InterlockedMinFloatGridValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IGNORE")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;

		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;

		Sig.bSoftDeprecatedFunction = true;
		Sig.bHidden = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = InterlockedMinFloatGridValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("OriginalValue")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = InterlockedMaxFloatGridValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IGNORE")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;

		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bSoftDeprecatedFunction = true;
		Sig.bHidden = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = InterlockedMaxFloatGridValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("OriginalValue")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;

		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl2D_SetValueFunction", "Set the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");
		OutFunctions.Add(Sig);
	}

	{
		// Older, deprecated form
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetFloatValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_GetValueFunction", "Get the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");

		OutFunctions.Add(Sig);
	}

	{
		// Older, deprecated form
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetIntValueFunctionName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("AttributeIndex")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Particle;
		
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Description = NSLOCTEXT("Niagara", "NiagaraDataInterfaceGridColl3D_GetValueFunction", "Get the value at a specific index. Note that this is an older way of working with Grids. Consider using the SetFloat or other typed, named functions or parameter map variables with StackContext namespace instead.");

		OutFunctions.Add(Sig);
	}
}
#endif // WITH_EDITORONLY_DATA

void UNiagaraDataInterfaceRasterizationGrid3D::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	using namespace NDIRasterizationGrid3DLocal;

	Super::GetVMExternalFunction(BindingInfo, InstanceData, OutFunc);

	// #todo(dmp): this overrides the empty function set by the super class
	if (BindingInfo.Name == UNiagaraDataInterfaceRWBase::NumCellsFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceRasterizationGrid3D::VMGetNumCells);
	}
	else if (BindingInfo.Name == SetNumCellsFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceRasterizationGrid3D::VMSetNumCells);
	}
	else if (BindingInfo.Name == SetNumAttributesFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceRasterizationGrid3D::VMSetNumAttributes);
	}
	else if (BindingInfo.Name == SetFloatResetValueFunctionName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceRasterizationGrid3D::VMSetFloatResetValue);
	}
}


void UNiagaraDataInterfaceRasterizationGrid3D::VMGetNumCells(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<RasterizationGrid3DRWInstanceData> InstData(Context);

	FNDIOutputParam<int32> NumCellsX(Context);
	FNDIOutputParam<int32> NumCellsY(Context);
	FNDIOutputParam<int32> NumCellsZ(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		NumCellsX.SetAndAdvance(InstData->NumCells.X);
		NumCellsY.SetAndAdvance(InstData->NumCells.Y);
		NumCellsZ.SetAndAdvance(InstData->NumCells.Z);
	}
}


bool UNiagaraDataInterfaceRasterizationGrid3D::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceRasterizationGrid3D* OtherTyped = CastChecked<const UNiagaraDataInterfaceRasterizationGrid3D>(Other);

	return OtherTyped->NumAttributes == NumAttributes && OtherTyped->Precision == Precision && OtherTyped->ResetValue == ResetValue;
}

#if WITH_EDITOR
bool UNiagaraDataInterfaceRasterizationGrid3D::ShouldCompile(EShaderPlatform ShaderPlatform) const
{
	return
		RHISupportsVolumeTextureAtomics(ShaderPlatform) &&
		RHIVolumeTextureRenderingSupportGuaranteed(ShaderPlatform) &&
		Super::ShouldCompile(ShaderPlatform);
}
#endif

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceRasterizationGrid3D::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderParameters<NDIRasterizationGrid3DLocal::FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceRasterizationGrid3D::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	using namespace NDIRasterizationGrid3DLocal;

	static const TCHAR *FormatDeclarations = TEXT(R"(			
		Texture3D<int> {IntGridName};		
		RWTexture3D<int> {OutputIntGridName};
		float {Precision};
		Buffer<float4> {PerAttributeDataName};
		int {NumAttributesName};
	)");
	TMap<FString, FStringFormatArg> ArgsDeclarations = {				
		{ TEXT("IntGridName"),    ParamInfo.DataInterfaceHLSLSymbol + IntGridName },
		{ TEXT("OutputIntGridName"),    ParamInfo.DataInterfaceHLSLSymbol + OutputIntGridName},
		{ TEXT("Precision"), ParamInfo.DataInterfaceHLSLSymbol + PrecisionName},
		{ TEXT("PerAttributeDataName"), ParamInfo.DataInterfaceHLSLSymbol + PerAttributeDataName},
		{ TEXT("NumAttributesName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
	};
	OutHLSL += FString::Format(FormatDeclarations, ArgsDeclarations);

	// always generate the code for these functions that are used internally by other DI functions
	// to quantize values
	{
		static const TCHAR* FormatHLSL = TEXT(R"(
				float {IntToFloatFunction}(int IntValue)
				{
					return float(IntValue) / {Precision};
				}
			)");
		TMap<FString, FStringFormatArg> FormatArgs =
		{
			{ TEXT("IntToFloatFunction"), ParamInfo.DataInterfaceHLSLSymbol + IntToFloatFunctionName.ToString()},
			{ TEXT("FloatToIntFunction"), ParamInfo.DataInterfaceHLSLSymbol + FloatToIntFunctionName.ToString()},
			{ TEXT("Precision"), ParamInfo.DataInterfaceHLSLSymbol + PrecisionName},
		};
		OutHLSL += FString::Format(FormatHLSL, FormatArgs);
	}
	
	{
		static const TCHAR* FormatHLSL = TEXT(R"(
				int {FloatToIntFunction}(float FloatValue)
				{
					return FloatValue * {Precision};
				}
			)");

		TMap<FString, FStringFormatArg> FormatArgs =
		{
			{ TEXT("IntToFloatFunction"), ParamInfo.DataInterfaceHLSLSymbol + IntToFloatFunctionName.ToString()},
			{ TEXT("FloatToIntFunction"), ParamInfo.DataInterfaceHLSLSymbol + FloatToIntFunctionName.ToString()},
			{ TEXT("Precision"), ParamInfo.DataInterfaceHLSLSymbol + PrecisionName},
		};
		OutHLSL += FString::Format(FormatHLSL, FormatArgs);
	}

}

bool UNiagaraDataInterfaceRasterizationGrid3D::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIRasterizationGrid3DLocal;

	bool ParentRet = Super::GetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, OutHLSL);

	TMap<FString, FStringFormatArg> ArgsBounds =
	{
		{ TEXT("FunctionName"), FunctionInfo.InstanceName},
		{ TEXT("IntGrid"), ParamInfo.DataInterfaceHLSLSymbol + IntGridName},
		{ TEXT("OutputIntGrid"), ParamInfo.DataInterfaceHLSLSymbol + OutputIntGridName},
		{TEXT("NumAttributesName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumAttributesName},
		{ TEXT("NumCellsName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::NumCellsName},
		{ TEXT("UnitToUVName"), ParamInfo.DataInterfaceHLSLSymbol + UNiagaraDataInterfaceRWBase::UnitToUVName},
		{ TEXT("IntToFloatFunctionName"), ParamInfo.DataInterfaceHLSLSymbol + IntToFloatFunctionName.ToString()},
		{ TEXT("FloatToIntFunctionName"), ParamInfo.DataInterfaceHLSLSymbol + FloatToIntFunctionName.ToString()},
		{ TEXT("Precision"), ParamInfo.DataInterfaceHLSLSymbol + PrecisionName},
		{TEXT("PerAttributeDataName"), ParamInfo.DataInterfaceHLSLSymbol + PerAttributeDataName},
	};

	if (ParentRet)
	{
		return true;
	}	
	else if (FunctionInfo.DefinitionName == UNiagaraDataInterfaceRWBase::NumCellsFunctionName)
	{
		static const TCHAR* FormatHLSL = TEXT(R"(
			void {FunctionName}(out int OutNumCellsX, out int OutNumCellsY, out int OutNumCellsZ)
			{
				OutNumCellsX = {NumCellsName}.x;
				OutNumCellsY = {NumCellsName}.y;
				OutNumCellsZ = {NumCellsName}.z;
			}
		)");

		OutHLSL += FString::Format(FormatHLSL, ArgsBounds);
	}
	else if (FunctionInfo.DefinitionName == SetFloatValueFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, float In_Value, out int val)
			{			
				val = 0;
				if ( In_AttributeIndex < {NumAttributesName} )
				{
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;
					{OutputIntGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)] = {FloatToIntFunctionName}(In_Value);
				}
			}
		)");

		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == SetIntValueFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, int In_Value, out int val)
			{			
				val = 0;				
				if ( In_AttributeIndex < {NumAttributesName} )
				{	
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;			
					{OutputIntGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)] = In_Value;				
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == InterlockedAddFloatGridValueFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, float In_Value, out int val)
			{							
				val = 0;					
				if ( In_AttributeIndex < {NumAttributesName} )
				{
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;		
					InterlockedAdd({OutputIntGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)], {FloatToIntFunctionName}(In_Value), val);					
				}
			}

			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, float In_Value, out float Original_Value)
			{			
				Original_Value = 0.0;												
				if ( In_AttributeIndex < {NumAttributesName} )
				{
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;		
					int OriginalIntValue;
					InterlockedAdd({OutputIntGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)], {FloatToIntFunctionName}(In_Value), OriginalIntValue);
					Original_Value = {IntToFloatFunctionName}(OriginalIntValue);
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == InterlockedAddIntGridValueFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, int In_Value, out int val)
			{							
				val = 0;					
				if ( In_AttributeIndex < {NumAttributesName} )
				{
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;		
					InterlockedAdd({OutputIntGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)], In_Value);
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == InterlockedAddFloatGridValueSafeFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, float In_Value, out int val)
			{							
				val = 1;					
				if ( In_AttributeIndex < {NumAttributesName} )
				{
					int OriginalValue;
					int IntValue = {FloatToIntFunctionName}(In_Value);
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;	

					int3 Index = int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z);
					InterlockedAdd({OutputIntGrid}[Index], IntValue, OriginalValue);

					int StoredValue = IntValue + OriginalValue;

					// make sure to store max/min float value in the grid if we've over/underflowed
					[branch]
					if ((IntValue > 0 && StoredValue < OriginalValue) || (IntValue < 0 && StoredValue > OriginalValue))
					{
						val = 0;

						int NewValue = IntValue > 0 ? 2147483647 : -2147483648;
						{OutputIntGrid}[Index] = NewValue;
					}
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == InterlockedMinFloatGridValueFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, float In_Value, out int val)
			{							
				val = 0;
				
				if ( In_AttributeIndex < {NumAttributesName} )
				{
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;			
					InterlockedMin({OutputIntGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)], {FloatToIntFunctionName}(In_Value), val);
				}
			}

			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, float In_Value, out float Original_Value)
			{			
				Original_Value = 0.0;				
				if ( In_AttributeIndex < {NumAttributesName} )
				{
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;			

					int OriginalIntValue;
					InterlockedMin({OutputIntGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)], {FloatToIntFunctionName}(In_Value), OriginalIntValue);
					Original_Value = {IntToFloatFunctionName}(OriginalIntValue);
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == InterlockedMaxFloatGridValueFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, float In_Value, out int val)
			{							
				val = 0;				
				if ( In_AttributeIndex < {NumAttributesName} )
				{
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;			
					InterlockedMax({OutputIntGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)], {FloatToIntFunctionName}(In_Value), val);
				}
			}

			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, float In_Value, out float Original_Value)
			{		
				Original_Value = 0.0;
				if ( In_AttributeIndex < {NumAttributesName} )
				{
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;			

					int OriginalIntValue;
					InterlockedMax({OutputIntGrid}[int3(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z)], {FloatToIntFunctionName}(In_Value), OriginalIntValue);
					Original_Value = {IntToFloatFunctionName}(OriginalIntValue);
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetFloatValueFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, out float Out_Val)
			{		
				Out_Val = 0;
				if ( In_AttributeIndex < {NumAttributesName} )
				{
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;		
					Out_Val =  {IntToFloatFunctionName}({IntGrid}.Load(int4(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z, 0)));
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetIntValueFunctionName)
	{
		static const TCHAR* FormatBounds = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, int In_AttributeIndex, out int Out_Val)
			{
				Out_Val = 0;
				if ( In_AttributeIndex < {NumAttributesName} )
				{
					int3 TileOffset = {PerAttributeDataName}[In_AttributeIndex].xyz;
					Out_Val = {IntGrid}.Load(int4(In_IndexX + TileOffset.x, In_IndexY + TileOffset.y, In_IndexZ + TileOffset.z, 0));				
				}
			}
		)");
		OutHLSL += FString::Format(FormatBounds, ArgsBounds);
		return true;
	}
	return false;
}
#endif

void UNiagaraDataInterfaceRasterizationGrid3D::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIRasterizationGrid3DLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceRasterizationGrid3D::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyRasterizationGrid3D& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyRasterizationGrid3D>();
	RasterizationGrid3DRWInstanceData* InstanceData = DIProxy.SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());
	NDIRasterizationGrid3DLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDIRasterizationGrid3DLocal::FShaderParameters>();
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	const bool bBindInput = Context.IsResourceBound(&ShaderParameters->IntGrid);
	const bool bBindOutput = Context.IsResourceBound(&ShaderParameters->OutputIntGrid);
	bool bInputBound = false;
	bool bOutputBound = false;
	if (InstanceData && InstanceData->RasterizationTexture.IsValid())
	{
		ShaderParameters->NumCells = InstanceData->NumCells;
		ShaderParameters->UnitToUV = FVector3f(1.0f) / FVector3f(InstanceData->NumCells);
		ShaderParameters->Precision = InstanceData->Precision;
		ShaderParameters->NumAttributes = InstanceData->TotalNumAttributes;
		ShaderParameters->PerAttributeData = InstanceData->PerAttributeData.SRV;

		if (Context.IsOutputStage())
		{
			if (bBindOutput)
			{
				bOutputBound = true;
				ShaderParameters->OutputIntGrid = InstanceData->RasterizationTexture.GetOrCreateUAV(GraphBuilder);
			}
		}
		else
		{
			if (bBindInput)
			{
				bInputBound = true;
				ShaderParameters->IntGrid = InstanceData->RasterizationTexture.GetOrCreateSRV(GraphBuilder);
			}
		}
	}
	else
	{
		ShaderParameters->NumCells = FIntVector::ZeroValue;
		ShaderParameters->UnitToUV = FVector3f::ZeroVector;
		ShaderParameters->Precision = 0;
		ShaderParameters->NumAttributes = 0;
		ShaderParameters->PerAttributeData = FNiagaraRenderer::GetDummyFloat4Buffer();
	}

	if (bBindInput && !bInputBound)
	{
		ShaderParameters->IntGrid = Context.GetComputeDispatchInterface().GetBlackTextureSRV(GraphBuilder, ETextureDimension::Texture3D);
	}
	if (bBindOutput && !bOutputBound)
	{
		ShaderParameters->OutputIntGrid = Context.GetComputeDispatchInterface().GetEmptyTextureUAV(GraphBuilder, PF_A32B32G32R32F, ETextureDimension::Texture3D);
	}
}

bool UNiagaraDataInterfaceRasterizationGrid3D::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIRasterizationGrid3DLocal;

	RasterizationGrid3DRWInstanceData* InstanceData = new (PerInstanceData) RasterizationGrid3DRWInstanceData();

	FNiagaraDataInterfaceProxyRasterizationGrid3D* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRasterizationGrid3D>();

	int32 NumAttribChannelsFound = 0;

	int32 NumNamedAttribChannelsFound = 0;
	// #todo(dmp): implement named attributes
	//FindAttributes(InstanceData->Vars, InstanceData->Offsets, NumNamedAttribChannelsFound);

	NumAttribChannelsFound = NumAttributes + NumNamedAttribChannelsFound;
	int32 RT_TotalNumAttributes = NumAttribChannelsFound;

	FIntVector RT_NumCells = NumCells;		

	float RT_Precision = Precision;
	int RT_ResetValue = ResetValue;

	RT_NumCells.X = FMath::Max(RT_NumCells.X, 1);
	RT_NumCells.Y = FMath::Max(RT_NumCells.Y, 1);
	RT_NumCells.Z = FMath::Max(RT_NumCells.Z, 1);
	
	InstanceData->TotalNumAttributes = RT_TotalNumAttributes;
	InstanceData->NumCells = RT_NumCells;
	InstanceData->Precision = RT_Precision;
	InstanceData->ResetValue = RT_ResetValue;

	if ((RT_NumCells.X * RT_NumCells.Y * RT_NumCells.Z) > GMaxNiagaraRasterizationGridCells)
	{
		UE_LOGF(LogNiagara, Warning, "Dimensions are too big! Please adjust! %d x %d x %d > %d for ==> %ls", RT_NumCells.X, RT_NumCells.Y, RT_NumCells.Z, GMaxNiagaraRasterizationGridCells , *GetFullNameSafe(this))
			return false;
	}

	// Compute number of tiles based on resolution of individual attributes
	// #todo(dmp): refactor
	const float MaxDim = 2048;
	const int32 MaxTilesX = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.X));
	const int32 MaxTilesY = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.Y));
	const int32 MaxTilesZ = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.Z));
	const int32 MaxAttributes = MaxTilesX * MaxTilesY * MaxTilesZ;

	// need to determine number of tiles in x and y based on number of attributes and max dimension size
	const int32 NumTilesX = FMath::Min<int32>(MaxTilesX, NumAttribChannelsFound);
	const int32 NumTilesY = FMath::Min<int32>(MaxTilesY, FMath::CeilToInt(1.0f * float(NumAttribChannelsFound) / float(NumTilesX)));
	const int32 NumTilesZ = FMath::Min<int32>(MaxTilesZ, FMath::CeilToInt(1.0f * float(NumAttribChannelsFound) / float(NumTilesX * NumTilesY)));

	InstanceData->NumTiles.X = NumTilesX;
	InstanceData->NumTiles.Y = NumTilesY;
	InstanceData->NumTiles.Z = NumTilesZ;

	if (NumAttribChannelsFound == 0 && !Proxy->GetSourceDIName().IsNone())
	{
		UE_LOGF(LogNiagara, Log, "Zero attributes defined on %ls", *Proxy->GetSourceDIName().ToString());
	}

//	check(InstanceData->NumTiles.X > 0);
//	check(InstanceData->NumTiles.Y > 0);
//	check(InstanceData->NumTiles.Z > 0);

	InstanceData->ClearBeforeNonIterationStage = ClearBeforeNonIterationStage;

	// @todo-threadsafety. This would be a race but I'm taking a ref here. Not ideal in the long term.
	// Push Updates to Proxy.
	ENQUEUE_RENDER_COMMAND(FUpdateData)(
		[RT_Proxy, RT_TotalNumAttributes, RT_NumCells, RT_Precision, RT_ResetValue, InstanceID = SystemInstance->GetId(), RT_InstanceData = *InstanceData](FRHICommandListImmediate& RHICmdList)
	{
		check(!RT_Proxy->SystemInstancesToProxyData.Contains(InstanceID));
		RasterizationGrid3DRWInstanceData* TargetData = &RT_Proxy->SystemInstancesToProxyData.Add(InstanceID);
		
		TargetData->ClearBeforeNonIterationStage = RT_InstanceData.ClearBeforeNonIterationStage;
		TargetData->TotalNumAttributes = RT_TotalNumAttributes;
		TargetData->NumCells = RT_NumCells;		
		TargetData->NumTiles = RT_InstanceData.NumTiles;
		TargetData->Precision = RT_Precision;
		TargetData->ResetValue = RT_ResetValue;
		TargetData->NeedsRealloc = true;
	});

	return true;
}

void UNiagaraDataInterfaceRasterizationGrid3D::VMSetNumCells(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<RasterizationGrid3DRWInstanceData> InstData(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsX(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsY(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumCellsZ(Context);	
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		int NewNumCellsX = InNumCellsX.GetAndAdvance();
		int NewNumCellsY = InNumCellsY.GetAndAdvance();
		int NewNumCellsZ = InNumCellsZ.GetAndAdvance();		
		bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1 && NumCells.X >= 0 && NumCells.Y >= 0 && NumCells.Z >= 0);
		*OutSuccess.GetDestAndAdvance() = bSuccess;
		if (bSuccess)
		{
			FIntVector OldNumCells = InstData->NumCells;			

			InstData->NumCells.X = NewNumCellsX;
			InstData->NumCells.Y = NewNumCellsY;
			InstData->NumCells.Z = NewNumCellsZ;			
		
			InstData->NeedsRealloc = OldNumCells != InstData->NumCells;
		}
	}
}

void UNiagaraDataInterfaceRasterizationGrid3D::VMSetNumAttributes(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<RasterizationGrid3DRWInstanceData> InstData(Context);
	VectorVM::FExternalFuncInputHandler<int> InNumAttributes(Context);	
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		int NewNumAttributes = InNumAttributes.GetAndAdvance();
		bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1 && NumCells.X >= 0 && NumCells.Y >= 0 && NumCells.Z >= 0);
		*OutSuccess.GetDestAndAdvance() = bSuccess;
		if (bSuccess)
		{
			int OldNumAttributes = InstData->TotalNumAttributes;

			InstData->TotalNumAttributes = NewNumAttributes;

			InstData->NeedsRealloc = OldNumAttributes != InstData->TotalNumAttributes;
		}
	}
}

void UNiagaraDataInterfaceRasterizationGrid3D::VMSetFloatResetValue(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<RasterizationGrid3DRWInstanceData> InstData(Context);
	VectorVM::FExternalFuncInputHandler<float> InResetValue(Context);
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		float NewResetValue = InResetValue.GetAndAdvance();
		bool bSuccess = InstData.Get() != nullptr && Context.GetNumInstances() == 1;
		*OutSuccess.GetDestAndAdvance() = bSuccess;
		if (bSuccess)
		{
			// Quantize value
			InstData->ResetValue = FMath::FloorToInt(NewResetValue * InstData->Precision);
		}
	}
}

bool UNiagaraDataInterfaceRasterizationGrid3D::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	RasterizationGrid3DRWInstanceData* InstanceData = static_cast<RasterizationGrid3DRWInstanceData*>(PerInstanceData);
	bool bNeedsReset = false;

	if (InstanceData->NeedsRealloc && InstanceData->NumCells.X > 0 && InstanceData->NumCells.Y > 0 && InstanceData->NumCells.Z > 0 && InstanceData->TotalNumAttributes > 0)
	{
		InstanceData->NeedsRealloc = false;
		
		// Compute number of tiles based on resolution of individual attributes
		// #todo(dmp): refactor
		const float MaxDim = 2048;
		const int32 MaxTilesX = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.X));
		const int32 MaxTilesY = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.Y));
		const int32 MaxTilesZ = FMath::FloorToInt(MaxDim / float(InstanceData->NumCells.Z));
		const int32 MaxAttributes = MaxTilesX * MaxTilesY * MaxTilesZ;

		// need to determine number of tiles in x and y based on number of attributes and max dimension size
		const int32 NumTilesX = FMath::Min<int32>(MaxTilesX, InstanceData->TotalNumAttributes);
		const int32 NumTilesY = FMath::Min<int32>(MaxTilesY, FMath::CeilToInt(1.0f * float(InstanceData->TotalNumAttributes) / float(NumTilesX)));
		const int32 NumTilesZ = FMath::Min<int32>(MaxTilesZ, FMath::CeilToInt(1.0f * float(InstanceData->TotalNumAttributes) / float(NumTilesX * NumTilesY)));

		InstanceData->NumTiles.X = NumTilesX;
		InstanceData->NumTiles.Y = NumTilesY;
		InstanceData->NumTiles.Z = NumTilesZ;

		if (InstanceData->TotalNumAttributes <= 0)
		{
			UE_LOGF(LogNiagara, Warning, "Invalid number of attributes defined on %ls... max is %d, num defined is %d", *Proxy->GetSourceDIName().ToString(), MaxAttributes, InstanceData->TotalNumAttributes);
			return false;
		}

		check(InstanceData->NumTiles.X > 0);
		check(InstanceData->NumTiles.Y > 0);
		check(InstanceData->NumTiles.Z > 0);

		FNiagaraDataInterfaceProxyRasterizationGrid3D* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRasterizationGrid3D>();
		ENQUEUE_RENDER_COMMAND(FUpdateData)(
			[RT_Proxy, RT_NumCells = InstanceData->NumCells, RT_Precision = InstanceData->Precision, RT_ResetValue = InstanceData->ResetValue, InstanceID = SystemInstance->GetId(), RT_InstanceData = *InstanceData](FRHICommandListImmediate& RHICmdList)
		{
			check(RT_Proxy->SystemInstancesToProxyData.Contains(InstanceID));
			RasterizationGrid3DRWInstanceData* TargetData = RT_Proxy->SystemInstancesToProxyData.Find(InstanceID);

			TargetData->NumTiles = RT_InstanceData.NumTiles;
			TargetData->NumCells = RT_NumCells;					
			TargetData->Precision = RT_Precision;
			TargetData->TotalNumAttributes = RT_InstanceData.TotalNumAttributes;
			TargetData->ResetValue = RT_ResetValue;
			TargetData->NeedsRealloc = true;
			TargetData->PerAttributeData.Release();
		});
	}

	return false;
}


void UNiagaraDataInterfaceRasterizationGrid3D::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{		
	RasterizationGrid3DRWInstanceData* InstanceData = static_cast<RasterizationGrid3DRWInstanceData*>(PerInstanceData);
	InstanceData->~RasterizationGrid3DRWInstanceData();

	FNiagaraDataInterfaceProxyRasterizationGrid3D* ThisProxy = GetProxyAs<FNiagaraDataInterfaceProxyRasterizationGrid3D>();
	if (!ThisProxy)
		return;

	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			//check(ThisProxy->SystemInstancesToProxyData.Contains(InstanceID));
			ThisProxy->SystemInstancesToProxyData.Remove(InstanceID);
		}
	);
}

void FNiagaraDataInterfaceProxyRasterizationGrid3D::ResetData(const FNDIGpuComputeResetContext& Context)
{	
	RasterizationGrid3DRWInstanceData* ProxyData = SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());
	if (!ProxyData || ProxyData->NeedsRealloc || !ProxyData->RasterizationTexture.IsValid())
	{
		return;
	}

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	const FUintVector4 ResetValue(ProxyData->ResetValue, ProxyData->ResetValue, ProxyData->ResetValue, ProxyData->ResetValue);
	AddClearUAVPass(GraphBuilder, ProxyData->RasterizationTexture.GetOrCreateUAV(GraphBuilder), ResetValue);
}

void FNiagaraDataInterfaceProxyRasterizationGrid3D::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	using namespace NDIRasterizationGrid3DLocal;

	RasterizationGrid3DRWInstanceData& InstanceData = SystemInstancesToProxyData.FindChecked(Context.GetSystemInstanceID());

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	FRHICommandListBase& RHICmdList = GraphBuilder.RHICmdList;

	const uint32 NumTotalCells = InstanceData.NumCells.X * InstanceData.NumCells.Y * InstanceData.NumCells.Z * InstanceData.TotalNumAttributes;

	// Resize was requested
	if (InstanceData.NeedsRealloc || !InstanceData.RasterizationTexture.IsValid())
	{
		InstanceData.NeedsRealloc = false;
		
		
		if (NumTotalCells <= (uint32)GMaxNiagaraRasterizationGridCells && NumTotalCells > 0)
		{
			InstanceData.PerAttributeData.Release();

			const FIntVector TextureSize(InstanceData.NumCells.X * InstanceData.NumTiles.X, InstanceData.NumCells.Y * InstanceData.NumTiles.Y, InstanceData.NumCells.Z * InstanceData.NumTiles.Z);
			const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create3D(TextureSize, PF_R32_SINT, FClearValueBinding::Black, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);
			InstanceData.RasterizationTexture.Initialize(Context.GetGraphBuilder(), TEXT("NiagaraRasterizationGrid3D::IntGrid"), TextureDesc);
		}
	}

	// Generate per-attribute data
	if (InstanceData.PerAttributeData.NumBytes == 0)
	{
		TResourceArray<FVector4f> PerAttributeData;
		PerAttributeData.AddUninitialized((InstanceData.TotalNumAttributes * 2) + 1);
		for (int32 iAttribute = 0; iAttribute < InstanceData.TotalNumAttributes; ++iAttribute)
		{
			const FIntVector AttributeTileIndex(iAttribute % InstanceData.NumTiles.X, (iAttribute / InstanceData.NumTiles.X) % InstanceData.NumTiles.Y, iAttribute / (InstanceData.NumTiles.X * InstanceData.NumTiles.Y));
			PerAttributeData[iAttribute] = FVector4f(
				float(AttributeTileIndex.X * InstanceData.NumCells.X),
				float(AttributeTileIndex.Y * InstanceData.NumCells.Y),
				float(AttributeTileIndex.Z * InstanceData.NumCells.Z),
				0.0f
			);
			PerAttributeData[iAttribute + InstanceData.TotalNumAttributes] = FVector4f(
				(1.0f / float(InstanceData.NumTiles.X)) * float(AttributeTileIndex.X),
				(1.0f / float(InstanceData.NumTiles.Y)) * float(AttributeTileIndex.Y),
				(1.0f / float(InstanceData.NumTiles.Z)) * float(AttributeTileIndex.Z),
				0.0f
			);
		}
		PerAttributeData[InstanceData.TotalNumAttributes * 2] = FVector4f(65535, 65535, 65535, 65535);
		InstanceData.PerAttributeData.Initialize(RHICmdList, TEXT("Grid3D::PerAttributeData"), sizeof(FVector4f), PerAttributeData.Num(), EPixelFormat::PF_A32B32G32R32F, BUF_Static, &PerAttributeData);
	}

	if (Context.IsOutputStage() && NumTotalCells > 0 && InstanceData.ClearBeforeNonIterationStage)
	{
		const FUintVector4 ResetValue(InstanceData.ResetValue, InstanceData.ResetValue, InstanceData.ResetValue, InstanceData.ResetValue);
		AddClearUAVPass(GraphBuilder, InstanceData.RasterizationTexture.GetOrCreateUAV(GraphBuilder), ResetValue);
	}
}

void FNiagaraDataInterfaceProxyRasterizationGrid3D::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	if (Context.IsFinalPostSimulate())
	{
		for ( auto it=SystemInstancesToProxyData.CreateIterator(); it; ++it )
		{
			RasterizationGrid3DRWInstanceData& InstanceData = it.Value();
			InstanceData.RasterizationTexture.EndGraphUsage();
		}
	}
}

void FNiagaraDataInterfaceProxyRasterizationGrid3D::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context)
{
	if ( const RasterizationGrid3DRWInstanceData* TargetData = SystemInstancesToProxyData.Find(Context.GetSystemInstanceID()) )
	{
		Context.SetDirect(TargetData->NumCells);
	}
}

bool UNiagaraDataInterfaceRasterizationGrid3D::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceRasterizationGrid3D* OtherTyped = CastChecked<UNiagaraDataInterfaceRasterizationGrid3D>(Destination);

	OtherTyped->NumAttributes = NumAttributes;
	OtherTyped->Precision = Precision;
	OtherTyped->ResetValue = ResetValue;

	return true;
}

#undef LOCTEXT_NAMESPACE

