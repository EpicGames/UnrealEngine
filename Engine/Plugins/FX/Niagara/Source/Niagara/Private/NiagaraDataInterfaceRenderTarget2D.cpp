// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceRenderTarget2D.h"
#include "Engine/Engine.h"
#include "NiagaraCompileHashVisitor.h"
#include "TextureResource.h"
#include "CanvasTypes.h"
#include "Engine/TextureRenderTarget2D.h"

#include "DataInterface/NDIRenderTargetSimCacheData.h"
#include "NiagaraSimCache.h"
#include "NiagaraDataInterfaceRenderTargetCommon.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraGpuComputeDebugInterface.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraStats.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceRenderTarget2D)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceRenderTarget2D"

namespace NDIRenderTarget2DLocal
{
	int32 GSimCacheCompressed = true;
	static FAutoConsoleVariableRef CVarSimCacheCompressed(
		TEXT("fx.Niagara.RenderTarget2D.SimCacheCompressed"),
		GSimCacheCompressed,
		TEXT("When enabled compression is used for the sim cache data."),
		ECVF_Default
	);

	static const FName GetSimCacheCompressionType() { return GSimCacheCompressed ? NAME_Oodle : NAME_None; }

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FIntPoint,								TextureSize)
		SHADER_PARAMETER(int,									MipLevels)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>,	RWTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>,		Texture)
		SHADER_PARAMETER_SAMPLER(SamplerState,					TextureSampler)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceRenderTarget2DTemplate.ush");

	// Global VM function names, also used by the shaders code generation methods.
	static const FName SetValueFunctionName("SetRenderTargetValue");
	static const FName LoadValueFunctionName("LoadRenderTargetValue");
	static const FName SampleValueFunctionName("SampleRenderTargetValue");
	static const FName SetSizeFunctionName("SetRenderTargetSize");
	static const FName GetSizeFunctionName("GetRenderTargetSize");
	static const FName ReleaseResourcesFunctionName("ReleaseResources");
	static const FName GetNumMipLevelsName("GetNumMipLevels");
	static const FName SetFormatFunctionName("SetRenderTargetFormat");
	static const FName LinearToIndexName("LinearToIndex");
	static const FName ExecToIndexName("ExecToIndex");
	static const FName ExecToUnitName("ExecToUnit");
	static const FName Deprecated_GetValueFunctionName("GetRenderTargetValue");
}

FNiagaraVariableBase UNiagaraDataInterfaceRenderTarget2D::ExposedRTVar;

/*--------------------------------------------------------------------------------------------------------------------------*/

struct FNDIRenderTarget2DFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		AddedOptionalExecute = 1,
		AddedMipLevel = 2,
		AddedOptionalExecuteToSetSize = 3,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

/*--------------------------------------------------------------------------------------------------------------------------*/

#if STATS
void FRenderTarget2DRWInstanceData_RenderThread::UpdateMemoryStats()
{
	DEC_MEMORY_STAT_BY(STAT_NiagaraRenderTargetMemory, MemorySize);

	MemorySize = 0;
	if (FRHITexture* RHITexture = TextureRHI)
	{
		MemorySize = RHIComputeMemorySize(RHITexture);
	}

	INC_MEMORY_STAT_BY(STAT_NiagaraRenderTargetMemory, MemorySize);
}
#endif

/*--------------------------------------------------------------------------------------------------------------------------*/

UNiagaraDataInterfaceRenderTarget2D::UNiagaraDataInterfaceRenderTarget2D(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyRenderTarget2DProxy());

	FNiagaraTypeDefinition Def(UTextureRenderTarget::StaticClass());
	RenderTargetUserParameter.Parameter.SetType(Def);
}


void UNiagaraDataInterfaceRenderTarget2D::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);

		ExposedRTVar = FNiagaraVariableBase(FNiagaraTypeDefinition(UTexture::StaticClass()), TEXT("RenderTarget"));
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceRenderTarget2D::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIRenderTarget2DLocal;

	Super::GetFunctionsInternal(OutFunctions);

	const int32 EmitterSystemOnlyBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
	OutFunctions.Reserve(OutFunctions.Num() + 4);

	FNiagaraFunctionSignature DefaultSig;
	DefaultSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget"));
	DefaultSig.bMemberFunction = true;
	DefaultSig.bRequiresContext = false;
	DefaultSig.bSupportsCPU = true;
	DefaultSig.bSupportsGPU = true;
	DefaultSig.SetFunctionVersion(FNDIRenderTarget2DFunctionVersion::LatestVersion);

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = GetSizeFunctionName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Width"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Height"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetSizeFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Width"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Height"));
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success"));
		Sig.ModuleUsageBitmask = EmitterSystemOnlyBitmask;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsGPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = ReleaseResourcesFunctionName;
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);
		Sig.ModuleUsageBitmask = EmitterSystemOnlyBitmask;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsGPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = GetNumMipLevelsName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevels"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetFormatFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(StaticEnum<ETextureRenderTargetFormat>()), TEXT("Format"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success"));
		Sig.ModuleUsageBitmask = EmitterSystemOnlyBitmask;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsGPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetValueFunctionName;
		Sig.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled"))).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = LoadValueFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SampleValueFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = LinearToIndexName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = ExecToIndexName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = ExecToUnitName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = Deprecated_GetValueFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bSupportsCPU = false;
		Sig.bSoftDeprecatedFunction = true;
	}
}

bool UNiagaraDataInterfaceRenderTarget2D::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	using namespace NDIRenderTarget2DLocal;

	bool bWasChanged = false;

	if ( FunctionSignature.FunctionVersion < FNDIRenderTarget2DFunctionVersion::AddedOptionalExecute )
	{
		if ( FunctionSignature.Name == SetValueFunctionName )
		{
			check(FunctionSignature.Inputs.Num() == 4);
			FunctionSignature.Inputs.Insert_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled")), 1).SetValue(true);
			bWasChanged = true;
		}
	}
	if (FunctionSignature.FunctionVersion < FNDIRenderTarget2DFunctionVersion::AddedMipLevel)
	{
		if (FunctionSignature.Name == SampleValueFunctionName)
		{
			FunctionSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
			bWasChanged = true;
		}
	}

	if (FunctionSignature.FunctionVersion < FNDIRenderTarget2DFunctionVersion::AddedOptionalExecuteToSetSize)
	{
		if (FunctionSignature.Name == SetSizeFunctionName)
		{
			check(FunctionSignature.Inputs.Num() == 3);
			FunctionSignature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Execute"))).SetValue(true);			
			bWasChanged = true;
		}
	}

	// Set latest version
	FunctionSignature.FunctionVersion = FNDIRenderTarget2DFunctionVersion::LatestVersion;

	return bWasChanged;
}
#endif

void UNiagaraDataInterfaceRenderTarget2D::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIRenderTarget2DLocal;

	Super::GetVMExternalFunction(BindingInfo, InstanceData, OutFunc);
	if (BindingInfo.Name == GetSizeFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTarget2D::VMGetSize);
	}
	else if (BindingInfo.Name == SetSizeFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTarget2D::VMSetSize);
	}
	else if (BindingInfo.Name == ReleaseResourcesFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTarget2D::VMReleaseResources);
	}
	else if (BindingInfo.Name == GetNumMipLevelsName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTarget2D::VMGetNumMipLevels);
	}
	else if (BindingInfo.Name == SetFormatFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTarget2D::VMSetFormat);
	}
}

bool UNiagaraDataInterfaceRenderTarget2D::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	
	const UNiagaraDataInterfaceRenderTarget2D* OtherTyped = CastChecked<const UNiagaraDataInterfaceRenderTarget2D>(Other);
	return
		OtherTyped != nullptr &&
#if WITH_EDITORONLY_DATA
		OtherTyped->bPreviewRenderTarget == bPreviewRenderTarget &&
#endif
		OtherTyped->RenderTargetUserParameter == RenderTargetUserParameter &&
		OtherTyped->Size == Size &&
		OtherTyped->MipMapGeneration == MipMapGeneration &&
		OtherTyped->MipMapGenerationType == MipMapGenerationType &&
		OtherTyped->OverrideRenderTargetFormat == OverrideRenderTargetFormat &&
		OtherTyped->OverrideRenderTargetFilter == OverrideRenderTargetFilter &&
		OtherTyped->bInheritUserParameterSettings == bInheritUserParameterSettings &&
		OtherTyped->bOverrideFormat == bOverrideFormat;
}

bool UNiagaraDataInterfaceRenderTarget2D::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceRenderTarget2D* DestinationTyped = CastChecked<UNiagaraDataInterfaceRenderTarget2D>(Destination);
	if (!DestinationTyped)
	{
		return false;
	}

	DestinationTyped->Size = Size;
	DestinationTyped->MipMapGeneration = MipMapGeneration;
	DestinationTyped->MipMapGenerationType = MipMapGenerationType;
	DestinationTyped->OverrideRenderTargetFormat = OverrideRenderTargetFormat;
	DestinationTyped->OverrideRenderTargetFilter = OverrideRenderTargetFilter;
	DestinationTyped->bInheritUserParameterSettings = bInheritUserParameterSettings;
	DestinationTyped->bOverrideFormat = bOverrideFormat;
#if WITH_EDITORONLY_DATA
	DestinationTyped->bPreviewRenderTarget = bPreviewRenderTarget;
#endif
	DestinationTyped->RenderTargetUserParameter = RenderTargetUserParameter;
	return true;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceRenderTarget2D::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(NDIRenderTarget2DLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<NDIRenderTarget2DLocal::FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceRenderTarget2D::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIRenderTarget2DLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceRenderTarget2D::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIRenderTarget2DLocal;

	bool ParentRet = Super::GetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, OutHLSL);
	if (ParentRet)
	{
		return true;
	} 

	if ((FunctionInfo.DefinitionName == SetValueFunctionName) ||
		(FunctionInfo.DefinitionName == Deprecated_GetValueFunctionName) ||
		(FunctionInfo.DefinitionName == LoadValueFunctionName) ||
		(FunctionInfo.DefinitionName == SampleValueFunctionName) ||
		(FunctionInfo.DefinitionName == GetSizeFunctionName) ||
		(FunctionInfo.DefinitionName == GetNumMipLevelsName) ||
		(FunctionInfo.DefinitionName == LinearToIndexName) ||
		(FunctionInfo.DefinitionName == ExecToIndexName) ||
		(FunctionInfo.DefinitionName == ExecToUnitName))
	{
		return true;
	}

	return false;
}
#endif

void UNiagaraDataInterfaceRenderTarget2D::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIRenderTarget2DLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceRenderTarget2D::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyRenderTarget2DProxy& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyRenderTarget2DProxy>();
	FRenderTarget2DRWInstanceData_RenderThread* InstanceData_RT = DIProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	check(InstanceData_RT);

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	// Ensure RDG resources are ready to use
	if (InstanceData_RT->TransientRDGTexture == nullptr && InstanceData_RT->TextureRHI)
	{
		InstanceData_RT->TransientRDGTexture = GraphBuilder.FindExternalTexture(InstanceData_RT->TextureRHI);
		if (InstanceData_RT->TransientRDGTexture == nullptr)
		{
			InstanceData_RT->TransientRDGTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InstanceData_RT->TextureRHI, TEXT("NiagaraRenderTarget2D")));
		}
		InstanceData_RT->TransientRDGSRV = GraphBuilder.CreateSRV(InstanceData_RT->TransientRDGTexture);
		InstanceData_RT->TransientRDGUAV = GraphBuilder.CreateUAV(InstanceData_RT->TransientRDGTexture);

		GraphBuilder.UseInternalAccessMode(InstanceData_RT->TransientRDGTexture);
		Context.GetRDGExternalAccessQueue().Add(InstanceData_RT->TransientRDGTexture);
	}

	NDIRenderTarget2DLocal::FShaderParameters* Parameters = Context.GetParameterNestedStruct<NDIRenderTarget2DLocal::FShaderParameters>();
	Parameters->TextureSize = InstanceData_RT->Size;
	Parameters->MipLevels = InstanceData_RT->MipLevels;

	const bool bRTWrite = Context.IsResourceBound(&Parameters->RWTexture);
	const bool bRTRead = Context.IsResourceBound(&Parameters->Texture);

	if (bRTWrite)
	{
		if (InstanceData_RT->TransientRDGUAV)
		{
			Parameters->RWTexture = InstanceData_RT->TransientRDGUAV;
			InstanceData_RT->bRebuildMips = true;
			InstanceData_RT->bWroteThisFrame = true;
		}
		else
		{
			Parameters->RWTexture = Context.GetComputeDispatchInterface().GetEmptyTextureUAV(GraphBuilder, EPixelFormat::PF_A16B16G16R16, ETextureDimension::Texture2D);
		}
	}
	
	if (bRTRead)
	{
		if (bRTWrite == false && InstanceData_RT->TransientRDGSRV)
		{
			InstanceData_RT->bReadThisFrame = true;
			Parameters->Texture = InstanceData_RT->TransientRDGSRV;
		}
		else
		{
		#if WITH_NIAGARA_DEBUG_EMITTER_NAME
			GEngine->AddOnScreenDebugMessage(uint64(this), 1.f, FColor::White, *FString::Printf(TEXT("RenderTarget is read and wrote in the same stage, this is not allowed, read will be invalid. (%s)"), *Context.GetDebugString()));
		#endif
			Parameters->Texture = Context.GetComputeDispatchInterface().GetBlackTextureSRV(GraphBuilder, ETextureDimension::Texture2D);
		}

		if (InstanceData_RT->SamplerStateRHI)
		{
			Parameters->TextureSampler = InstanceData_RT->SamplerStateRHI;
		}
		else
		{
			Parameters->TextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		}
	}
}

bool UNiagaraDataInterfaceRenderTarget2D::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	check(Proxy);

	FRenderTarget2DRWInstanceData_GameThread* InstanceData = new (PerInstanceData) FRenderTarget2DRWInstanceData_GameThread();
	SystemInstancesToProxyData_GT.Emplace(SystemInstance->GetId(), InstanceData);

	//-TEMP: Until we prune data interface on cook this will avoid consuming memory
	const bool bValidGpuDataInterface = NiagaraDataInterfaceRenderTargetCommon::GIgnoreCookedOut == 0 || IsUsedWithGPUScript();

	ETextureRenderTargetFormat RenderTargetFormat;
	if ( NiagaraDataInterfaceRenderTargetCommon::GetRenderTargetFormat(bOverrideFormat, OverrideRenderTargetFormat, RenderTargetFormat) == false )
	{
		if (bValidGpuDataInterface)
		{
			UE_LOGF(LogNiagara, Warning, "NDIRT2D failed to find a render target format that supports UAV store");
			return false;
		}
		return true;
	}

	InstanceData->Size.X = FMath::Clamp<int>(int(float(Size.X) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
	InstanceData->Size.Y = FMath::Clamp<int>(int(float(Size.Y) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
	InstanceData->MipMapGeneration = MipMapGeneration;
	InstanceData->MipMapGenerationType = MipMapGenerationType;
	InstanceData->Format = RenderTargetFormat;
	InstanceData->Filter = OverrideRenderTargetFilter;
	InstanceData->RTUserParamBinding.Init(SystemInstance->GetInstanceParameters(), RenderTargetUserParameter.Parameter);
#if WITH_EDITORONLY_DATA
	InstanceData->bPreviewTexture = bPreviewRenderTarget;
#endif

	return true;
}

void UNiagaraDataInterfaceRenderTarget2D::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIRenderTarget2DLocal;

	SystemInstancesToProxyData_GT.Remove(SystemInstance->GetId());

	FNiagaraDataInterfaceProxyRenderTarget2DProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTarget2DProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[RT_Proxy, InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
#if STATS
			if (FRenderTarget2DRWInstanceData_RenderThread* TargetData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID))
			{
				TargetData->SamplerStateRHI.SafeRelease();
				TargetData->TextureRHI.SafeRelease();
				TargetData->UpdateMemoryStats();
			}
#endif
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		}
	);

	FRenderTarget2DRWInstanceData_GameThread* InstanceData = static_cast<FRenderTarget2DRWInstanceData_GameThread*>(PerInstanceData);
	NiagaraDataInterfaceRenderTargetCommon::ReleaseRenderTarget(SystemInstance, InstanceData);
	InstanceData->~FRenderTarget2DRWInstanceData_GameThread();
}

void UNiagaraDataInterfaceRenderTarget2D::GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const
{
	OutVariables.Emplace(ExposedRTVar);
}

bool UNiagaraDataInterfaceRenderTarget2D::GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const
{
	FRenderTarget2DRWInstanceData_GameThread* InstanceData = static_cast<FRenderTarget2DRWInstanceData_GameThread*>(InPerInstanceData);
	if (InVariable.IsValid() && InVariable == ExposedRTVar && InstanceData && InstanceData->TargetTexture)
	{
		UObject** Var = (UObject**)OutData;
		*Var = InstanceData->TargetTexture;
		return true;
	}
	return false;
}

void UNiagaraDataInterfaceRenderTarget2D::GetCanvasVariables(TArray<FNiagaraVariableBase>& OutVariables) const
{
	static const FName NAME_RenderTarget(TEXT("RenderTarget"));
	OutVariables.Emplace(FNiagaraTypeDefinition::GetVec4Def(), NAME_RenderTarget);
}

bool UNiagaraDataInterfaceRenderTarget2D::RenderVariableToCanvas(FNiagaraSystemInstanceID SystemInstanceID, FName VariableName, class FCanvas* Canvas, const FIntRect& DrawRect) const
{
	if (!Canvas)
	{
		return false;
	}

	FRenderTarget2DRWInstanceData_GameThread* GT_InstanceData = SystemInstancesToProxyData_GT.FindRef(SystemInstanceID);
	if (!GT_InstanceData)
	{
		return false;
	}

	if ( !GT_InstanceData->TargetTexture || !GT_InstanceData->TargetTexture->GetResource())
	{
		return false;
	}

	Canvas->DrawTile(
		DrawRect.Min.X, DrawRect.Min.Y, DrawRect.Width(), DrawRect.Height(),
		0.0f, 1.0f, 1.0f, 0.0f,
		FLinearColor::White,
		GT_InstanceData->TargetTexture->GetResource(),
		false
	);

	return true;
}

NIAGARA_API UObject* UNiagaraDataInterfaceRenderTarget2D::SimCacheBeginWrite(UObject* InSimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	if (!OptionalPerInstanceData)
	{
		return nullptr;
	}

	UNDIRenderTargetSimCacheData* SimCacheData = nullptr;
	SimCacheData = NewObject<UNDIRenderTargetSimCacheData>(InSimCache);
	SimCacheData->CompressionType = NDIRenderTarget2DLocal::GetSimCacheCompressionType();
	return SimCacheData;	
}

NIAGARA_API bool UNiagaraDataInterfaceRenderTarget2D::SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	check(OptionalPerInstanceData);
	const FRenderTarget2DRWInstanceData_GameThread* InstanceData_GT = reinterpret_cast<const FRenderTarget2DRWInstanceData_GameThread*>(OptionalPerInstanceData);

	if (InstanceData_GT->TargetTexture)
	{
		const FNiagaraDataInterfaceProxyRenderTarget2DProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTarget2DProxy>();
		const FTextureRenderTargetResource* RT_TargetTexture = InstanceData_GT->TargetTexture->GameThread_GetRenderTargetResource();

		TArray<uint8> TextureData;
		ENQUEUE_RENDER_COMMAND(NDIRenderTargetVolumeRead)
		(
			[RT_Proxy, RT_InstanceID = SystemInstance->GetId(), RT_TargetTexture, RT_TextureData = &TextureData, RT_VolumeResolution = InstanceData_GT->Size](FRHICommandListImmediate& RHICmdList)
			{
				if (const FRenderTarget2DRWInstanceData_RenderThread* InstanceData_RT = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID))
				{
					RHICmdList.Transition(FRHITransitionInfo(InstanceData_RT->TextureRHI, ERHIAccess::SRVMask, ERHIAccess::CopySrc));

					FRHIGPUTextureReadback RenderTargetReadback("ReadRTTexture");

					RenderTargetReadback.EnqueueCopy(RHICmdList, InstanceData_RT->TextureRHI, FIntVector(0, 0, 0), 0, FIntVector(RT_VolumeResolution.X, RT_VolumeResolution.Y, 1));

					// Sync the GPU. Unfortunately we can't use the fences because not all RHIs implement them yet.
					constexpr bool bFlushResources = true;
					RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

					//Lock the readback staging texture
					int32 RowPitchInPixels;
					int32 BufferHeight;
					const uint8* LockedData = (const uint8*)RenderTargetReadback.Lock(RowPitchInPixels, &BufferHeight);

					uint32 BlockBytes = GPixelFormats[InstanceData_RT->TextureRHI->GetFormat()].BlockBytes;
					int32 Count = InstanceData_RT->Size.X * InstanceData_RT->Size.Y * BlockBytes;
					RT_TextureData->AddUninitialized(Count);

					const uint8* SliceStart = LockedData;

					const uint8* RowStart = SliceStart;
					for (int32 Y = 0; Y < InstanceData_RT->Size.Y; ++Y)
					{
						int32 Offset = 0 + Y * InstanceData_RT->Size.X;
						FMemory::Memcpy(RT_TextureData->GetData() + Offset * BlockBytes, RowStart, BlockBytes * InstanceData_RT->Size.X);

						RowStart += RowPitchInPixels * BlockBytes;
					}
										
					//Unlock the staging texture
					RenderTargetReadback.Unlock();

					RHICmdList.Transition(FRHITransitionInfo(InstanceData_RT->TextureRHI, ERHIAccess::CopySrc, ERHIAccess::SRVMask));
				}
			}
		);
		FlushRenderingCommands();

		if (TextureData.Num() > 0)
		{
			UNDIRenderTargetSimCacheData* SimCacheData = CastChecked<UNDIRenderTargetSimCacheData>(StorageObject);
			SimCacheData->SetPixelData(
				FrameIndex,
				FIntVector(InstanceData_GT->Size.X, InstanceData_GT->Size.Y, 1),
				GetPixelFormatFromRenderTargetFormat(InstanceData_GT->Format),
				TextureData
			);
		}
	}

	return true;
}

NIAGARA_API bool UNiagaraDataInterfaceRenderTarget2D::SimCacheEndWrite(UObject* StorageObject) const
{
	return true;
}

NIAGARA_API bool UNiagaraDataInterfaceRenderTarget2D::SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext)
{
	UNDIRenderTargetSimCacheData* SimCacheData = ReadContext.GetStorageObject<UNDIRenderTargetSimCacheData>();

	FRenderTarget2DRWInstanceData_GameThread* InstanceData_GT = ReadContext.GetPerInstanceData<FRenderTarget2DRWInstanceData_GameThread>();

	const int FrameIndex = ReadContext.GetFrameInterp() >= 0.5f ? ReadContext.GetFrameIndexB() : ReadContext.GetFrameIndexA();
	if (!SimCacheData->IsValidFrame(FrameIndex))
	{
		return false;
	}

	{
		const FIntVector TextureSize = SimCacheData->GetTextureSize(FrameIndex);
		const EPixelFormat TextureFormat = SimCacheData->GetTextureFormat(FrameIndex);

		InstanceData_GT->Size = FIntPoint(TextureSize.X, TextureSize.Y);
		InstanceData_GT->Format = NiagaraDataInterfaceRenderTargetCommon::GetRenderTargetFormatFromPixelFormat(TextureFormat);
		InstanceData_GT->Filter = TextureFilter::TF_Default;
	}

	FNiagaraSystemInstance* SystemInstance = ReadContext.GetSystemInstance();

	PerInstanceTick(InstanceData_GT, SystemInstance, 0.0f);
	PerInstanceTickPostSimulate(InstanceData_GT, SystemInstance, 0.0f);

	if (InstanceData_GT->TargetTexture && SimCacheData->HasPixelData(FrameIndex))
	{
		FNiagaraDataInterfaceProxyRenderTarget2DProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTarget2DProxy>();
		FTextureRenderTargetResource* RT_TargetTexture = InstanceData_GT->TargetTexture->GameThread_GetRenderTargetResource();
		ENQUEUE_RENDER_COMMAND(NDIRenderTargetUpdate)
			(
				[RT_Proxy, RT_InstanceID=SystemInstance->GetId(), RT_TargetTexture, RT_SimCacheData=SimCacheData, RT_FrameIndex=FrameIndex](FRHICommandListImmediate& RHICmdList)
				{
					if (FRenderTarget2DRWInstanceData_RenderThread* InstanceData_RT = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID))
					{
						uint32 Stride;
						uint8* OutBuffer = static_cast<uint8*>(RHICmdList.LockTexture2D(InstanceData_RT->TextureRHI, 0, RLM_WriteOnly, Stride, true));
						RT_SimCacheData->GetPixelData(RT_FrameIndex, OutBuffer, Stride, 0);
						RHICmdList.UnlockTexture2D(InstanceData_RT->TextureRHI, 0, true);						
						//RHICmdList.UpdateTexture2D(InstanceData_RT->TextureRHI, 0, UpdateRegion, DestPitch, RT_CacheFrame->PixelData);													
					}
				}
		);
	}

	return true;
}

bool UNiagaraDataInterfaceRenderTarget2D::SimCacheCompareFrame(const UObject* LhsStorageObject, const UObject* RhsStorageObject, int FrameIndex, TOptional<float> Tolerance, FString& OutErrors) const
{
	const UNDIRenderTargetSimCacheData* LhsSimCacheData = CastChecked<UNDIRenderTargetSimCacheData>(LhsStorageObject);
	const UNDIRenderTargetSimCacheData* RhsSimCacheData = CastChecked<UNDIRenderTargetSimCacheData>(RhsStorageObject);
	return LhsSimCacheData->CompareFrame(RhsSimCacheData, FrameIndex, Tolerance, OutErrors);
}

TSharedPtr<FJsonObject> UNiagaraDataInterfaceRenderTarget2D::SimCacheToJson(const UObject* StorageObject, int FrameIndex, TOptional<FString> TargetFolder, TOptional<FString> FilenamePrefix) const
{
	const UNDIRenderTargetSimCacheData* SimCacheData = CastChecked<UNDIRenderTargetSimCacheData>(StorageObject);
	return SimCacheData->FrameToJson(FrameIndex, TargetFolder, FilenamePrefix);
}

void UNiagaraDataInterfaceRenderTarget2D::VMSetSize(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FRenderTarget2DRWInstanceData_GameThread> InstData(Context);
	FNDIInputParam<int> InSizeX(Context);
	FNDIInputParam<int> InSizeY(Context);
	VectorVM::FExternalFuncInputHandler<FNiagaraBool> InExecute(Context);
	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const int SizeX = InSizeX.GetAndAdvance();
		const int SizeY = InSizeY.GetAndAdvance();
		bool bExecute = InExecute.GetAndAdvance();
		const bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1 && SizeX >= 0 && SizeY >= 0) && bExecute;
		OutSuccess.SetAndAdvance(bSuccess);
		if (bSuccess)
		{
			InstData->bReleaseResources = false;
			InstData->Size.X = FMath::Clamp<int>(int(float(SizeX) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
			InstData->Size.Y = FMath::Clamp<int>(int(float(SizeY) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
		}
	}
}

void UNiagaraDataInterfaceRenderTarget2D::VMReleaseResources(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FRenderTarget2DRWInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncInputHandler<FNiagaraBool> InExecute(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const bool bExecute = InExecute.GetAndAdvance();
		if (bExecute)
		{
			InstData->bReleaseResources = true;
			InstData->Size.X = 1;
			InstData->Size.Y = 1;
		}
	}
}

void UNiagaraDataInterfaceRenderTarget2D::VMGetSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FRenderTarget2DRWInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int> OutSizeX(Context);
	FNDIOutputParam<int> OutSizeY(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutSizeX.SetAndAdvance(InstData->Size.X);
		OutSizeY.SetAndAdvance(InstData->Size.Y);
	}
}

void UNiagaraDataInterfaceRenderTarget2D::VMGetNumMipLevels(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FRenderTarget2DRWInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int> OutMipLevels(Context);

	int32 NumMipLevels = 1;
	if (InstData->TargetTexture != nullptr)
	{
		NumMipLevels = InstData->TargetTexture->GetNumMips();
	}
	else if (InstData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled)
	{
		NumMipLevels = FMath::FloorLog2(FMath::Max(InstData->Size.X, InstData->Size.Y)) + 1;
	}
	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutMipLevels.SetAndAdvance(NumMipLevels);
	}
}

void UNiagaraDataInterfaceRenderTarget2D::VMSetFormat(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FRenderTarget2DRWInstanceData_GameThread> InstData(Context);
	FNDIInputParam<ETextureRenderTargetFormat> InFormat(Context);
	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const ETextureRenderTargetFormat Format = InFormat.GetAndAdvance();

		const bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1);
		OutSuccess.SetAndAdvance(bSuccess);
		if (bSuccess)
		{
			InstData->Format = Format;

			if (bInheritUserParameterSettings && InstData->RTUserParamBinding.GetValue<UTextureRenderTarget2D>())
			{
				UE_LOGF(LogNiagara, Warning, "Overriding inherited volume render target format");
			}
		}
	}
}

bool UNiagaraDataInterfaceRenderTarget2D::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIRenderTarget2DLocal;

	FRenderTarget2DRWInstanceData_GameThread* InstanceData = static_cast<FRenderTarget2DRWInstanceData_GameThread*>(PerInstanceData);
	if (InstanceData->bReleaseResources == false)
	{
		// Pull from user parameter
		UTextureRenderTarget2D* UserTargetTexture = InstanceData->RTUserParamBinding.GetValue<UTextureRenderTarget2D>();
		if (UserTargetTexture && (InstanceData->TargetTexture != UserTargetTexture))
		{
			NiagaraDataInterfaceRenderTargetCommon::ReleaseRenderTarget(SystemInstance, InstanceData);
			InstanceData->TargetTexture = UserTargetTexture;
		}

		// Do we inherit the texture parameters from the user supplied texture?
		if (bInheritUserParameterSettings)
		{
			if (UserTargetTexture)
			{
				InstanceData->Size.X = UserTargetTexture->SizeX;
				InstanceData->Size.Y = UserTargetTexture->SizeY;
				if (UserTargetTexture->bAutoGenerateMips)
				{
					// We have to take a guess at user intention
					InstanceData->MipMapGeneration = MipMapGeneration == ENiagaraMipMapGeneration::Disabled ? ENiagaraMipMapGeneration::PostStage : MipMapGeneration;
					InstanceData->MipMapGenerationType = MipMapGenerationType;
				}
				else
				{
					InstanceData->MipMapGeneration = ENiagaraMipMapGeneration::Disabled;
					InstanceData->MipMapGenerationType = ENiagaraMipMapGenerationType::Unfiltered;
				}
				InstanceData->Format = InstanceData->TargetTexture->RenderTargetFormat;
				InstanceData->Filter = InstanceData->TargetTexture->Filter;
			}
			else
			{
				UE_LOGF(LogNiagara, Error, "RenderTarget UserParam is required but invalid.");
			}
		}
	}

	return false;
}

bool UNiagaraDataInterfaceRenderTarget2D::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIRenderTarget2DLocal;

	// Update InstanceData as the texture may have changed
	FRenderTarget2DRWInstanceData_GameThread* InstanceData = static_cast<FRenderTarget2DRWInstanceData_GameThread*>(PerInstanceData);
#if WITH_EDITORONLY_DATA
	InstanceData->bPreviewTexture = bPreviewRenderTarget;
#endif

	//-TEMP: Until we prune data interface on cook this will avoid consuming memory
	const bool bValidGpuDataInterface = NiagaraDataInterfaceRenderTargetCommon::GIgnoreCookedOut == 0 || IsUsedWithGPUScript();

	if (InstanceData->bReleaseResources)
	{
		if (InstanceData->TargetTexture)
		{
			NiagaraDataInterfaceRenderTargetCommon::ReleaseRenderTarget(SystemInstance, InstanceData);
		}
	}
	else
	{
		if (::IsValid(InstanceData->TargetTexture) == false)
		{
			InstanceData->TargetTexture = nullptr;
		}

		// Do we need to create a new texture?
		if (bValidGpuDataInterface && !bInheritUserParameterSettings && (InstanceData->TargetTexture == nullptr))
		{
			if (NiagaraDataInterfaceRenderTargetCommon::CreateRenderTarget<UTextureRenderTarget2D>(SystemInstance, InstanceData) == false)
			{
				return false;
			}
			check(InstanceData->TargetTexture);
			InstanceData->TargetTexture->bCanCreateUAV = true;
			InstanceData->TargetTexture->bAutoGenerateMips = InstanceData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled;
			InstanceData->TargetTexture->RenderTargetFormat = InstanceData->Format;
			InstanceData->TargetTexture->ClearColor = FLinearColor(0.0, 0, 0, 0);
			InstanceData->TargetTexture->Filter = InstanceData->Filter;
			InstanceData->TargetTexture->InitAutoFormat(InstanceData->Size.X, InstanceData->Size.Y);
			InstanceData->TargetTexture->UpdateResourceImmediate(true);
		}

		// Do we need to update the existing texture?
		if (InstanceData->TargetTexture)
		{
			const bool bAutoGenerateMips = InstanceData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled;
			if ((InstanceData->TargetTexture->SizeX != InstanceData->Size.X) || (InstanceData->TargetTexture->SizeY != InstanceData->Size.Y) ||
				(InstanceData->TargetTexture->RenderTargetFormat != InstanceData->Format) ||
				(InstanceData->TargetTexture->Filter != InstanceData->Filter) ||
				!InstanceData->TargetTexture->bCanCreateUAV ||
				(InstanceData->TargetTexture->bAutoGenerateMips != bAutoGenerateMips) ||
				!InstanceData->TargetTexture->GetResource())
			{
				// resize RT to match what we need for the output
				InstanceData->TargetTexture->bCanCreateUAV = true;
				InstanceData->TargetTexture->bAutoGenerateMips = bAutoGenerateMips;
				InstanceData->TargetTexture->RenderTargetFormat = InstanceData->Format;
				InstanceData->TargetTexture->Filter = InstanceData->Filter;
				InstanceData->TargetTexture->InitAutoFormat(InstanceData->Size.X, InstanceData->Size.Y);
				InstanceData->TargetTexture->UpdateResourceImmediate(true);
			}
		}
	}

	//-TODO: We could avoid updating each frame if we cache the resource pointer or a serial number
	const bool bUpdateRT = true && bValidGpuDataInterface;
	if (bUpdateRT)
	{
		FNiagaraDataInterfaceProxyRenderTarget2DProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTarget2DProxy>();
		FTextureRenderTargetResource* RT_TargetTexture = InstanceData->TargetTexture ? InstanceData->TargetTexture->GameThread_GetRenderTargetResource() : nullptr;
		ensureMsgf(InstanceData->bReleaseResources || RT_TargetTexture != nullptr, TEXT("NiagaraDataInterfaceRenderTarget2D - null RT for system (%s)"), *GetNameSafe(SystemInstance->GetSystem()));
		ENQUEUE_RENDER_COMMAND(NDIRenderTarget2DUpdate)
		(
			[RT_Proxy, RT_InstanceID=SystemInstance->GetId(), RT_InstanceData=*InstanceData, RT_TargetTexture](FRHICommandListImmediate& RHICmdList)
			{
				FRenderTarget2DRWInstanceData_RenderThread* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.FindOrAdd(RT_InstanceID);
				TargetData->Size = RT_InstanceData.Size;
				TargetData->MipLevels = 1;
				TargetData->MipMapGeneration = RT_InstanceData.MipMapGeneration;
				TargetData->MipMapGenerationType = RT_InstanceData.MipMapGenerationType;
			#if WITH_EDITORONLY_DATA
				TargetData->bPreviewTexture = RT_InstanceData.bPreviewTexture;
			#endif
				TargetData->SamplerStateRHI.SafeRelease();
				TargetData->TextureRHI.SafeRelease();
				if (RT_TargetTexture)
				{
					TargetData->MipLevels = RT_TargetTexture->GetCurrentMipCount();
					if (FTextureRenderTarget2DResource* Resource2D = RT_TargetTexture->GetTextureRenderTarget2DResource())
					{
						TargetData->SamplerStateRHI = Resource2D->SamplerStateRHI;
						TargetData->TextureRHI = Resource2D->GetTextureRHI();
					}
				}
			#if STATS
				TargetData->UpdateMemoryStats();
			#endif
			}
		);
	}

	return false;
}

void FNiagaraDataInterfaceProxyRenderTarget2DProxy::PostStage(const FNDIGpuComputePostStageContext& Context)
{
	if (FRenderTarget2DRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()))
	{
		if (ProxyData->bRebuildMips && (ProxyData->MipMapGeneration == ENiagaraMipMapGeneration::PostStage))
		{
			ProxyData->bRebuildMips = false;
			//-TODO:RDG:GPUProfiler:FNiagaraGpuProfileScope GpuProfileScope(RHICmdList, Context, GNiagaraRenderTarget2DGenerateMipsName);
			if (ProxyData->TransientRDGTexture)
			{
				NiagaraGenerateMips::GenerateMips(Context.GetGraphBuilder(), ProxyData->TransientRDGTexture, ProxyData->MipMapGenerationType);
			}
		}
	}
}

void FNiagaraDataInterfaceProxyRenderTarget2DProxy::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	FRenderTarget2DRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	if (ProxyData == nullptr)
	{
		return;
	}

	if (ProxyData->bRebuildMips && (ProxyData->MipMapGeneration == ENiagaraMipMapGeneration::PostSimulate))
	{
		//-TODO:RDG:GPUProfiler:FNiagaraGpuProfileScope GpuProfileScope(RHICmdList, Context, GNiagaraRenderTarget2DGenerateMipsName);
		NiagaraGenerateMips::GenerateMips(Context.GetGraphBuilder(), ProxyData->TransientRDGTexture, ProxyData->MipMapGenerationType);
	}

	// We only need to transfer this frame if it was written to.
	// If also read then we need to notify that the texture is important for the simulation
	// We also assume the texture is important for rendering, without discovering renderer bindings we don't really know
	if (ProxyData->bWroteThisFrame)
	{
		if (ProxyData->TransientRDGTexture)
		{
			Context.GetComputeDispatchInterface().MultiGPUResourceModified(Context.GetGraphBuilder(), ProxyData->TextureRHI, ProxyData->bReadThisFrame, true);
		}
	}

#if NIAGARA_COMPUTEDEBUG_ENABLED && WITH_EDITORONLY_DATA
	if (ProxyData->bPreviewTexture && ProxyData->TransientRDGTexture)
	{
		FNiagaraGpuComputeDebugInterface GpuComputeDebugInterface = Context.GetComputeDispatchInterface().GetGpuComputeDebugInterface();
		GpuComputeDebugInterface.AddTexture(Context.GetGraphBuilder(), Context.GetSystemInstanceID(), GetSourceDIName(), ProxyData->TransientRDGTexture);
	}
#endif

	// Clean up our temporary tracking
	ProxyData->bRebuildMips = false;
	ProxyData->bReadThisFrame = false;
	ProxyData->bWroteThisFrame = false;

	ProxyData->TransientRDGTexture = nullptr;
	ProxyData->TransientRDGSRV = nullptr;
	ProxyData->TransientRDGUAV = nullptr;
}

void FNiagaraDataInterfaceProxyRenderTarget2DProxy::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context)
{
	if ( const FRenderTarget2DRWInstanceData_RenderThread* TargetData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()) )
	{
		Context.SetDirect(TargetData->Size);
	}
}

#undef LOCTEXT_NAMESPACE
