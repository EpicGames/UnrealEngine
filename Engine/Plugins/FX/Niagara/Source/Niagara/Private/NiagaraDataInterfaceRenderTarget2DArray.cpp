// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceRenderTarget2DArray.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "TextureResource.h"
#include "NiagaraCompileHashVisitor.h"
#include "TextureRenderTarget2DArrayResource.h"
#include "Engine/TextureRenderTarget2DArray.h"

#include "NiagaraDataInterfaceRenderTargetCommon.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraStats.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraGpuComputeDebugInterface.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceRenderTarget2DArray)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceRenderTarget2DArray"

namespace NDIRenderTarget2DArrayLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FIntVector3,								TextureSize)
		SHADER_PARAMETER(int,										MipLevels)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>,	RWTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<float4>,	Texture)
		SHADER_PARAMETER_SAMPLER(SamplerState,						TextureSampler)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceRenderTarget2DArrayTemplate.ush");

	// Global VM function names, also used by the shaders code generation methods.
	static const FName SetValueFunctionName("SetRenderTargetValue");
	static const FName LoadValueFunctionName("LoadRenderTargetValue");
	static const FName SampleValueFunctionName("SampleRenderTargetValue");
	static const FName SetSizeFunctionName("SetRenderTargetSize");
	static const FName GetSizeFunctionName("GetRenderTargetSize");
	static const FName GetNumMipLevelsName("GetNumMipLevels");
	static const FName SetFormatFunctionName("SetRenderTargetFormat");
	static const FName LinearToIndexName("LinearToIndex");
	static const FName ExecToIndexName("ExecToIndex");
	static const FName ExecToUnitName("ExecToUnit");
	static const FName Deprecated_GetValueFunctionName("GetRenderTargetValue");
}

FNiagaraVariableBase UNiagaraDataInterfaceRenderTarget2DArray::ExposedRTVar;

/*--------------------------------------------------------------------------------------------------------------------------*/

struct FNDIRenderTarget2DArrayFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		AddedOptionalExecute = 1,
		AddedMipLevel = 2,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

/*--------------------------------------------------------------------------------------------------------------------------*/

#if STATS
void FRenderTarget2DArrayRWInstanceData_RenderThread::UpdateMemoryStats()
{
	DEC_MEMORY_STAT_BY(STAT_NiagaraRenderTargetMemory, MemorySize);

	MemorySize = 0;
	if (RenderTarget.IsValid())
	{
		MemorySize = RHIComputeMemorySize(RenderTarget->GetRHI());
	}

	INC_MEMORY_STAT_BY(STAT_NiagaraRenderTargetMemory, MemorySize);
}
#endif

/*--------------------------------------------------------------------------------------------------------------------------*/

UNiagaraDataInterfaceRenderTarget2DArray::UNiagaraDataInterfaceRenderTarget2DArray(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyRenderTarget2DArrayProxy());

	FNiagaraTypeDefinition Def(UTextureRenderTarget::StaticClass());
	RenderTargetUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceRenderTarget2DArray::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);

		ExposedRTVar = FNiagaraVariableBase(FNiagaraTypeDefinition(UTexture::StaticClass()), TEXT("RenderTarget"));
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceRenderTarget2DArray::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIRenderTarget2DArrayLocal;

	Super::GetFunctionsInternal(OutFunctions);

	const int32 EmitterSystemOnlyBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
	OutFunctions.Reserve(OutFunctions.Num() + 4);

	FNiagaraFunctionSignature DefaultSig;
	DefaultSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget"));
	DefaultSig.bMemberFunction = true;
	DefaultSig.bRequiresContext = false;
	DefaultSig.bSupportsCPU = true;
	DefaultSig.bSupportsGPU = true;
	DefaultSig.SetFunctionVersion(FNDIRenderTarget2DArrayFunctionVersion::LatestVersion);

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = GetSizeFunctionName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Width"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Height"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Slices"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetSizeFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Width"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Height"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Slices"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success"));
		Sig.ModuleUsageBitmask = EmitterSystemOnlyBitmask;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsGPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = GetNumMipLevelsName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevels"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetFormatFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(StaticEnum<ETextureRenderTargetFormat>()), TEXT("Format"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success"));
		Sig.ModuleUsageBitmask = EmitterSystemOnlyBitmask;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsGPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetValueFunctionName;
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled")).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = Deprecated_GetValueFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = LoadValueFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SampleValueFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Slice"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = LinearToIndexName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = ExecToIndexName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = ExecToUnitName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Slice"));
		Sig.bSupportsCPU = false;
	}
}

bool UNiagaraDataInterfaceRenderTarget2DArray::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	using namespace NDIRenderTarget2DArrayLocal;

	bool bWasChanged = false;

	if (FunctionSignature.FunctionVersion < FNDIRenderTarget2DArrayFunctionVersion::AddedOptionalExecute)
	{
		if (FunctionSignature.Name == SetValueFunctionName)
		{
			check(FunctionSignature.Inputs.Num() == 5);
			FunctionSignature.Inputs.Insert_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled")), 1).SetValue(true);
			bWasChanged = true;
		}
	}
	if (FunctionSignature.FunctionVersion < FNDIRenderTarget2DArrayFunctionVersion::AddedMipLevel)
	{
		if (FunctionSignature.Name == SampleValueFunctionName)
		{
			FunctionSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
			bWasChanged = true;
		}
	}

	// Set latest version
	FunctionSignature.FunctionVersion = FNDIRenderTarget2DArrayFunctionVersion::LatestVersion;

	return bWasChanged;
}
#endif

void UNiagaraDataInterfaceRenderTarget2DArray::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	using namespace NDIRenderTarget2DArrayLocal;

	Super::GetVMExternalFunction(BindingInfo, InstanceData, OutFunc);
	if (BindingInfo.Name == GetSizeFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTarget2DArray::VMGetSize);
	}
	else if (BindingInfo.Name == SetSizeFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTarget2DArray::VMSetSize);
	}
	else if (BindingInfo.Name == GetNumMipLevelsName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTarget2DArray::VMGetNumMipLevels);
	}
	else if (BindingInfo.Name == SetFormatFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTarget2DArray::VMSetFormat);
	}
}

bool UNiagaraDataInterfaceRenderTarget2DArray::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	
	const UNiagaraDataInterfaceRenderTarget2DArray* OtherTyped = CastChecked<const UNiagaraDataInterfaceRenderTarget2DArray>(Other);
	return
		OtherTyped != nullptr &&
#if WITH_EDITORONLY_DATA
		OtherTyped->bPreviewRenderTarget == bPreviewRenderTarget &&
#endif
		OtherTyped->RenderTargetUserParameter == RenderTargetUserParameter &&
		OtherTyped->Size == Size &&
		OtherTyped->OverrideRenderTargetFormat == OverrideRenderTargetFormat &&
		OtherTyped->OverrideRenderTargetFilter == OverrideRenderTargetFilter &&
		OtherTyped->bInheritUserParameterSettings == bInheritUserParameterSettings &&
		OtherTyped->bOverrideFormat == bOverrideFormat;
}

bool UNiagaraDataInterfaceRenderTarget2DArray::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceRenderTarget2DArray* DestinationTyped = CastChecked<UNiagaraDataInterfaceRenderTarget2DArray>(Destination);
	if (!DestinationTyped)
	{
		return false;
	}

	DestinationTyped->Size = Size;
	DestinationTyped->OverrideRenderTargetFormat = OverrideRenderTargetFormat;
	DestinationTyped->OverrideRenderTargetFilter = OverrideRenderTargetFilter;
	DestinationTyped->bInheritUserParameterSettings = bInheritUserParameterSettings;
	DestinationTyped->bOverrideFormat = bOverrideFormat;
#if WITH_EDITORONLY_DATA
	DestinationTyped->bPreviewRenderTarget = bPreviewRenderTarget;
#endif
	DestinationTyped->RenderTargetUserParameter = RenderTargetUserParameter;
	return true;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceRenderTarget2DArray::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(NDIRenderTarget2DArrayLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<NDIRenderTarget2DArrayLocal::FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceRenderTarget2DArray::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIRenderTarget2DArrayLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceRenderTarget2DArray::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIRenderTarget2DArrayLocal;

	bool ParentRet = Super::GetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, OutHLSL);
	if (ParentRet)
	{
		return true;
	}

	if ((FunctionInfo.DefinitionName == SetValueFunctionName) ||
		(FunctionInfo.DefinitionName == Deprecated_GetValueFunctionName) ||
		(FunctionInfo.DefinitionName == LoadValueFunctionName) ||
		(FunctionInfo.DefinitionName == SampleValueFunctionName) ||
		(FunctionInfo.DefinitionName == GetSizeFunctionName) ||
		(FunctionInfo.DefinitionName == LinearToIndexName) ||
		(FunctionInfo.DefinitionName == ExecToIndexName) ||
		(FunctionInfo.DefinitionName == ExecToUnitName))
	{
		return true;
	}

	return false;
}
#endif

void UNiagaraDataInterfaceRenderTarget2DArray::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIRenderTarget2DArrayLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceRenderTarget2DArray::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyRenderTarget2DArrayProxy& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyRenderTarget2DArrayProxy>();
	FRenderTarget2DArrayRWInstanceData_RenderThread* InstanceData_RT = DIProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	check(InstanceData_RT);

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	// Ensure RDG resources are ready to use
	if (InstanceData_RT->TransientRDGTexture == nullptr && InstanceData_RT->RenderTarget)
	{
		InstanceData_RT->TransientRDGTexture = GraphBuilder.RegisterExternalTexture(InstanceData_RT->RenderTarget);
		InstanceData_RT->TransientRDGSRV = GraphBuilder.CreateSRV(InstanceData_RT->TransientRDGTexture);
		InstanceData_RT->TransientRDGUAV = GraphBuilder.CreateUAV(InstanceData_RT->TransientRDGTexture);

		GraphBuilder.UseInternalAccessMode(InstanceData_RT->TransientRDGTexture);
		Context.GetRDGExternalAccessQueue().Add(InstanceData_RT->TransientRDGTexture);
	}

	NDIRenderTarget2DArrayLocal::FShaderParameters* Parameters = Context.GetParameterNestedStruct<NDIRenderTarget2DArrayLocal::FShaderParameters>();
	Parameters->TextureSize = InstanceData_RT->Size;
	Parameters->MipLevels = InstanceData_RT->MipLevels;

	const bool bRTWrite = Context.IsResourceBound(&Parameters->RWTexture);
	const bool bRTRead = Context.IsResourceBound(&Parameters->Texture);

	if (bRTWrite)
	{
		if (InstanceData_RT->RenderTarget.IsValid())
		{
			Parameters->RWTexture = InstanceData_RT->TransientRDGUAV;
			InstanceData_RT->bWroteThisFrame = true;
		}
		else
		{
			Parameters->RWTexture = Context.GetComputeDispatchInterface().GetEmptyTextureUAV(GraphBuilder, EPixelFormat::PF_A16B16G16R16, ETextureDimension::Texture2DArray);
		}
	}
	
	if (bRTRead)
	{
		if (bRTWrite == false && InstanceData_RT->RenderTarget.IsValid())
		{
			InstanceData_RT->bReadThisFrame = true;
			Parameters->Texture = InstanceData_RT->TransientRDGSRV;
		}
		else
		{
		#if WITH_NIAGARA_DEBUG_EMITTER_NAME
			GEngine->AddOnScreenDebugMessage(uint64(this), 1.f, FColor::White, *FString::Printf(TEXT("RenderTarget is read and wrote in the same stage, this is not allowed, read will be invalid. (%s)"), *Context.GetDebugString()));
		#endif
			Parameters->Texture = Context.GetComputeDispatchInterface().GetBlackTextureSRV(GraphBuilder, ETextureDimension::Texture2DArray);
		}

		if (InstanceData_RT->SamplerStateRHI)
		{
			Parameters->TextureSampler = InstanceData_RT->SamplerStateRHI;
		}
		else
		{
			Parameters->TextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		}
	}
}

bool UNiagaraDataInterfaceRenderTarget2DArray::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	check(Proxy);

	FRenderTarget2DArrayRWInstanceData_GameThread* InstanceData = new (PerInstanceData) FRenderTarget2DArrayRWInstanceData_GameThread();

	//-TEMP: Until we prune data interface on cook this will avoid consuming memory
	const bool bValidGpuDataInterface = NiagaraDataInterfaceRenderTargetCommon::GIgnoreCookedOut == 0 || IsUsedWithGPUScript();

	ETextureRenderTargetFormat RenderTargetFormat;
	if (NiagaraDataInterfaceRenderTargetCommon::GetRenderTargetFormat(bOverrideFormat, OverrideRenderTargetFormat, RenderTargetFormat) == false)
	{
		if (bValidGpuDataInterface)
		{
			UE_LOGF(LogNiagara, Warning, "NDIRT2DArray failed to find a render target format that supports UAV store");
			return false;
		}
		return true;
	}

	InstanceData->Size.X = FMath::Clamp<int>(int(float(Size.X) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
	InstanceData->Size.Y = FMath::Clamp<int>(int(float(Size.Y) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
	InstanceData->Size.Z = FMath::Clamp<int>(Size.Z, 1, GMaxTextureDimensions);
	InstanceData->Format = GetPixelFormatFromRenderTargetFormat(RenderTargetFormat);
	InstanceData->Filter = OverrideRenderTargetFilter;
	InstanceData->RTUserParamBinding.Init(SystemInstance->GetInstanceParameters(), RenderTargetUserParameter.Parameter);
#if WITH_EDITORONLY_DATA
	InstanceData->bPreviewTexture = bPreviewRenderTarget;
#endif

	return true;
}

void UNiagaraDataInterfaceRenderTarget2DArray::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIRenderTarget2DArrayLocal;

	FRenderTarget2DArrayRWInstanceData_GameThread* InstanceData = static_cast<FRenderTarget2DArrayRWInstanceData_GameThread*>(PerInstanceData);
	NiagaraDataInterfaceRenderTargetCommon::ReleaseRenderTarget(SystemInstance, InstanceData);
	InstanceData->~FRenderTarget2DArrayRWInstanceData_GameThread();

	FNiagaraDataInterfaceProxyRenderTarget2DArrayProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTarget2DArrayProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[RT_Proxy, InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
#if STATS
			if (FRenderTarget2DArrayRWInstanceData_RenderThread* TargetData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID))
			{
				TargetData->SamplerStateRHI.SafeRelease();
				TargetData->RenderTarget.SafeRelease();
				TargetData->UpdateMemoryStats();
			}
#endif
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		}
	);
}


void UNiagaraDataInterfaceRenderTarget2DArray::GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const
{
	OutVariables.Emplace(ExposedRTVar);
}

bool UNiagaraDataInterfaceRenderTarget2DArray::GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const
{
	FRenderTarget2DArrayRWInstanceData_GameThread* InstanceData = static_cast<FRenderTarget2DArrayRWInstanceData_GameThread*>(InPerInstanceData);
	if (InVariable.IsValid() && InVariable == ExposedRTVar && InstanceData && InstanceData->TargetTexture)
	{
		UObject** Var = (UObject**)OutData;
		*Var = InstanceData->TargetTexture;
		return true;
	}
	return false;
}

void UNiagaraDataInterfaceRenderTarget2DArray::VMSetSize(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FRenderTarget2DArrayRWInstanceData_GameThread> InstData(Context);
	FNDIInputParam<int> InSizeX(Context);
	FNDIInputParam<int> InSizeY(Context);
	FNDIInputParam<int> InSlices(Context);
	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const int SizeX = InSizeX.GetAndAdvance();
		const int SizeY = InSizeY.GetAndAdvance();
		const int Slices = InSlices.GetAndAdvance();
		const bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1 && SizeX > 0 && SizeY > 0 && Slices > 0);
		OutSuccess.SetAndAdvance(bSuccess);
		if (bSuccess)
		{
			InstData->Size.X = FMath::Clamp<int>(int(float(SizeX) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
			InstData->Size.Y = FMath::Clamp<int>(int(float(SizeY) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxTextureDimensions);
			InstData->Size.Z = FMath::Clamp<int>(Slices, 1, GMaxTextureDimensions);
		}
	}
}

void UNiagaraDataInterfaceRenderTarget2DArray::VMGetSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FRenderTarget2DArrayRWInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int> OutSizeX(Context);
	FNDIOutputParam<int> OutSizeY(Context);
	FNDIOutputParam<int> OutSlices(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutSizeX.SetAndAdvance(InstData->Size.X);
		OutSizeY.SetAndAdvance(InstData->Size.Y);
		OutSlices.SetAndAdvance(InstData->Size.Z);
	}
}

void UNiagaraDataInterfaceRenderTarget2DArray::VMGetNumMipLevels(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FRenderTarget2DArrayRWInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int> OutMipLevels(Context);

	int32 NumMipLevels = 1;
	if (InstData->TargetTexture != nullptr)
	{
		NumMipLevels = InstData->TargetTexture->GetNumMips();
	}
	//else if (InstData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled)
	//{
	//	NumMipLevels = FMath::FloorLog2(FMath::Max(InstData->Size.X, InstData->Size.Y)) + 1;
	//}
	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutMipLevels.SetAndAdvance(NumMipLevels);
	}
}

void UNiagaraDataInterfaceRenderTarget2DArray::VMSetFormat(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FRenderTarget2DArrayRWInstanceData_GameThread> InstData(Context);
	FNDIInputParam<ETextureRenderTargetFormat> InFormat(Context);
	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const ETextureRenderTargetFormat Format = InFormat.GetAndAdvance();


		const bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1);
		OutSuccess.SetAndAdvance(bSuccess);
		if (bSuccess)
		{
			InstData->Format = GetPixelFormatFromRenderTargetFormat(Format);

			if (bInheritUserParameterSettings && InstData->RTUserParamBinding.GetValue<UTextureRenderTarget2DArray>())
			{
				UE_LOGF(LogNiagara, Warning, "Overriding inherited volume render target format");
			}
		}
	}
}

bool UNiagaraDataInterfaceRenderTarget2DArray::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIRenderTarget2DArrayLocal;

	FRenderTarget2DArrayRWInstanceData_GameThread* InstanceData = static_cast<FRenderTarget2DArrayRWInstanceData_GameThread*>(PerInstanceData);

	// Pull from user parameter
	UTextureRenderTarget2DArray* UserTargetTexture = InstanceData->RTUserParamBinding.GetValue<UTextureRenderTarget2DArray>();
	if (UserTargetTexture && (InstanceData->TargetTexture != UserTargetTexture))
	{
		NiagaraDataInterfaceRenderTargetCommon::ReleaseRenderTarget(SystemInstance, InstanceData);
		InstanceData->TargetTexture = UserTargetTexture;
	}

	// Do we inherit the texture parameters from the user supplied texture?
	if (bInheritUserParameterSettings)
	{
		if (UserTargetTexture)
		{
			InstanceData->Size.X = UserTargetTexture->SizeX;
			InstanceData->Size.Y = UserTargetTexture->SizeY;
			InstanceData->Size.Z = UserTargetTexture->Slices;
			//if (UserTargetTexture->bAutoGenerateMips)
			//{
			//	// We have to take a guess at user intention
			//	InstanceData->MipMapGeneration = MipMapGeneration == ENiagaraMipMapGeneration::Disabled ? ENiagaraMipMapGeneration::PostStage : MipMapGeneration;
			//}
			//else
			//{
			//	InstanceData->MipMapGeneration = ENiagaraMipMapGeneration::Disabled;
			//}
			InstanceData->Format = InstanceData->TargetTexture->GetFormat();
			InstanceData->Filter = InstanceData->TargetTexture->Filter;
		}
		else
		{
			UE_LOGF(LogNiagara, Error, "RenderTarget UserParam is required but null or the wrong type.");
		}
	}

	return false;
}

bool UNiagaraDataInterfaceRenderTarget2DArray::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIRenderTarget2DArrayLocal;

	// Update InstanceData as the texture may have changed
	FRenderTarget2DArrayRWInstanceData_GameThread* InstanceData = static_cast<FRenderTarget2DArrayRWInstanceData_GameThread*>(PerInstanceData);
#if WITH_EDITORONLY_DATA
	InstanceData->bPreviewTexture = bPreviewRenderTarget;
#endif

	//-TEMP: Until we prune data interface on cook this will avoid consuming memory
	const bool bValidGpuDataInterface = NiagaraDataInterfaceRenderTargetCommon::GIgnoreCookedOut == 0 || IsUsedWithGPUScript();

	if (::IsValid(InstanceData->TargetTexture) == false)
	{
		InstanceData->TargetTexture = nullptr;
	}

	if (bValidGpuDataInterface && !bInheritUserParameterSettings && (InstanceData->TargetTexture == nullptr))
	{
		if (NiagaraDataInterfaceRenderTargetCommon::CreateRenderTarget<UTextureRenderTarget2DArray>(SystemInstance, InstanceData) == false)
		{
			return false;
		}
		check(InstanceData->TargetTexture);
		InstanceData->TargetTexture->bCanCreateUAV = true;
		//InstanceData->TargetTexture->bAutoGenerateMips = InstanceData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled;
		InstanceData->TargetTexture->ClearColor = FLinearColor(0.0, 0, 0, 0);
		InstanceData->TargetTexture->Filter = InstanceData->Filter;
		InstanceData->TargetTexture->Init(InstanceData->Size.X, InstanceData->Size.Y, InstanceData->Size.Z, InstanceData->Format);
		InstanceData->TargetTexture->UpdateResourceImmediate(true);
	}

	// Do we need to update the existing texture?
	if (InstanceData->TargetTexture)
	{
		//const bool bAutoGenerateMips = InstanceData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled;
		if ((InstanceData->TargetTexture->SizeX != InstanceData->Size.X) || (InstanceData->TargetTexture->SizeY != InstanceData->Size.Y) || (InstanceData->TargetTexture->Slices != InstanceData->Size.Z) ||
			(InstanceData->TargetTexture->GetFormat() != InstanceData->Format) ||
			(InstanceData->TargetTexture->Filter != InstanceData->Filter) ||
			!InstanceData->TargetTexture->bCanCreateUAV ||
			//(InstanceData->TargetTexture->bAutoGenerateMips != bAutoGenerateMips) ||
			!InstanceData->TargetTexture->GetResource())
		{
			// resize RT to match what we need for the output
			InstanceData->TargetTexture->bCanCreateUAV = true;
			//InstanceData->TargetTexture->bAutoGenerateMips = bAutoGenerateMips;
			InstanceData->TargetTexture->Filter = InstanceData->Filter;
			InstanceData->TargetTexture->Init(InstanceData->Size.X, InstanceData->Size.Y, InstanceData->Size.Z, InstanceData->Format);
			InstanceData->TargetTexture->UpdateResourceImmediate(true);
		}
	}

	//-TODO: We could avoid updating each frame if we cache the resource pointer or a serial number
	const bool bUpdateRT = true && bValidGpuDataInterface;
	if (bUpdateRT)
	{
		FNiagaraDataInterfaceProxyRenderTarget2DArrayProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTarget2DArrayProxy>();
		FTextureRenderTargetResource* RT_TargetTexture = InstanceData->TargetTexture ? InstanceData->TargetTexture->GameThread_GetRenderTargetResource() : nullptr;
		ENQUEUE_RENDER_COMMAND(NDIRenderTarget2DArrayUpdate)
		(
			[RT_Proxy, RT_InstanceID=SystemInstance->GetId(), RT_InstanceData=*InstanceData, RT_TargetTexture](FRHICommandListImmediate& RHICmdList)
			{
				FRenderTarget2DArrayRWInstanceData_RenderThread* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.FindOrAdd(RT_InstanceID);
				TargetData->Size = RT_InstanceData.Size;
				TargetData->MipLevels = 1;
				//TargetData->MipMapGeneration = RT_InstanceData.MipMapGeneration;
			#if WITH_EDITORONLY_DATA
				TargetData->bPreviewTexture = RT_InstanceData.bPreviewTexture;
			#endif
				TargetData->SamplerStateRHI.SafeRelease();
				TargetData->RenderTarget.SafeRelease();
				if (RT_TargetTexture)
				{
					TargetData->MipLevels = RT_TargetTexture->GetCurrentMipCount();
					if (FTextureRenderTarget2DArrayResource* Resource2DArray = RT_TargetTexture->GetTextureRenderTarget2DArrayResource())
					{
						TargetData->SamplerStateRHI = Resource2DArray->SamplerStateRHI;
						TargetData->RenderTarget = CreateRenderTarget(Resource2DArray->GetTextureRHI(), TEXT("NiagaraRenderTarget2DArray"));
					}
				}
			#if STATS
				TargetData->UpdateMemoryStats();
			#endif
			}
		);
	}

	return false;
}

void FNiagaraDataInterfaceProxyRenderTarget2DArrayProxy::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	FRenderTarget2DArrayRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	if (ProxyData == nullptr)
	{
		return;
	}

	// We only need to transfer this frame if it was written to.
	// If also read then we need to notify that the texture is important for the simulation
	// We also assume the texture is important for rendering, without discovering renderer bindings we don't really know
	if (ProxyData->bWroteThisFrame)
	{
		Context.GetComputeDispatchInterface().MultiGPUResourceModified(Context.GetGraphBuilder(), ProxyData->RenderTarget->GetRHI(), ProxyData->bReadThisFrame, true);
	}

#if NIAGARA_COMPUTEDEBUG_ENABLED && WITH_EDITORONLY_DATA
	if (ProxyData->bPreviewTexture && ProxyData->TransientRDGTexture)
	{
		FNiagaraGpuComputeDebugInterface GpuComputeDebugInterface = Context.GetComputeDispatchInterface().GetGpuComputeDebugInterface();
		GpuComputeDebugInterface.AddTexture(Context.GetGraphBuilder(), Context.GetSystemInstanceID(), GetSourceDIName(), ProxyData->TransientRDGTexture);
	}
#endif

	// Clean up our temporary tracking
	ProxyData->bReadThisFrame = false;
	ProxyData->bWroteThisFrame = false;

	ProxyData->TransientRDGTexture = nullptr;
	ProxyData->TransientRDGSRV = nullptr;
	ProxyData->TransientRDGUAV = nullptr;
}

void FNiagaraDataInterfaceProxyRenderTarget2DArrayProxy::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) 
{
	if ( const FRenderTarget2DArrayRWInstanceData_RenderThread* TargetData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()) )
	{
		Context.SetDirect(TargetData->Size);
	}
}

#undef LOCTEXT_NAMESPACE

