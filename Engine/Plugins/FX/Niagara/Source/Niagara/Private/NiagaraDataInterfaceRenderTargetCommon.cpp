// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceRenderTargetCommon.h"

#include "NiagaraCommon.h"
#include "NiagaraSettings.h"
#include "RHI.h"

namespace NiagaraDataInterfaceRenderTargetCommon
{
	//-TEMP: Until we prune data interface on cook this will avoid consuming memory
	int32 GIgnoreCookedOut = true;
	static FAutoConsoleVariableRef CVarNiagaraRenderTargetIgnoreCookedOut(
		TEXT("fx.Niagara.RenderTarget.IgnoreCookedOut"),
		GIgnoreCookedOut,
		TEXT("Ignores create render targets for cooked out emitter, i.e. ones that are not used by any GPU emitter."),
		ECVF_Default
	);

	float GResolutionMultiplier = 1.0f;
	static FAutoConsoleVariableRef CVarNiagaraRenderTargetResolutionMultiplier(
		TEXT("fx.Niagara.RenderTarget.ResolutionMultiplier"),
		GResolutionMultiplier,
		TEXT("Optional global modifier to Niagara render target resolution."),
		ECVF_Default
	);

	static bool GNiagaraRenderTargetOverrideFormatEnabled = false;
	static ETextureRenderTargetFormat GNiagaraRenderTargetOverrideFormat = ETextureRenderTargetFormat::RTF_RGBA32f;

	static FAutoConsoleCommandWithWorldAndArgs GCommandNiagaraRenderTargetOverrideFormat(
		TEXT("fx.Niagara.RenderTarget.OverrideFormat"),
		TEXT("Optional global format override for all Niagara render targets"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
			[](const TArray<FString>& Args, UWorld*)
			{
				static UEnum* TextureRenderTargetFormatEnum = StaticEnum<ETextureRenderTargetFormat>();
				if ( ensure(TextureRenderTargetFormatEnum) )
				{
					if (Args.Num() == 1)
					{
						const int32 EnumIndex = TextureRenderTargetFormatEnum->GetIndexByNameString(Args[0]);
						if (EnumIndex != INDEX_NONE)
						{
							GNiagaraRenderTargetOverrideFormatEnabled = true;
							GNiagaraRenderTargetOverrideFormat = ETextureRenderTargetFormat(EnumIndex);
						}
						else
						{
							GNiagaraRenderTargetOverrideFormatEnabled = false;
						}
					}
					UE_LOGF(LogNiagara, Log, "Niagara RenderTarget Override is '%ls' with format '%ls'.", GNiagaraRenderTargetOverrideFormatEnabled ? TEXT("Enabled") : TEXT("Disabled"), *TextureRenderTargetFormatEnum->GetNameStringByIndex(int32(GNiagaraRenderTargetOverrideFormat)));
				}
			}
		)
	);

	bool GetRenderTargetFormat(bool bOverrideFormat, ETextureRenderTargetFormat OverrideFormat, ETextureRenderTargetFormat& OutRenderTargetFormat)
	{
		OutRenderTargetFormat = bOverrideFormat ? OverrideFormat : GetDefault<UNiagaraSettings>()->DefaultRenderTargetFormat.GetValue();
		EPixelFormat PixelFormat = GetPixelFormatFromRenderTargetFormat(OutRenderTargetFormat);
		if (GNiagaraRenderTargetOverrideFormatEnabled)
		{
			OutRenderTargetFormat = GNiagaraRenderTargetOverrideFormat;
		}

		// If the format does not support typed store we need to find one that will
		while (UE::PixelFormat::HasCapabilities(GetPixelFormatFromRenderTargetFormat(OutRenderTargetFormat), EPixelFormatCapabilities::TypedUAVStore) == false)
		{
			static const TPair<ETextureRenderTargetFormat, ETextureRenderTargetFormat> FormatRemapTable[] =
			{
				TPairInitializer(RTF_R8,			RTF_R16f),
				TPairInitializer(RTF_RG8,			RTF_RG16f),
				TPairInitializer(RTF_RGBA8,			RTF_RGBA16f),
				TPairInitializer(RTF_RGBA8_SRGB,	RTF_RGBA16f),
				TPairInitializer(RTF_R16f,			RTF_R32f),
				TPairInitializer(RTF_RG16f,			RTF_RG32f),
				TPairInitializer(RTF_RGBA16f,		RTF_RGBA32f),
				TPairInitializer(RTF_R32f,			RTF_RGBA32f),
				TPairInitializer(RTF_RG32f,			RTF_RGBA32f),
				TPairInitializer(RTF_RGBA32f,		RTF_RGBA32f),
				TPairInitializer(RTF_RGB10A2,		RTF_RGBA32f),
			};

			const ETextureRenderTargetFormat PreviousFormat = OutRenderTargetFormat;
			for (const auto& FormatRemap : FormatRemapTable)
			{
				if (FormatRemap.Key == OutRenderTargetFormat)
				{
					OutRenderTargetFormat = FormatRemap.Value;
					break;
				}
			}
			if (PreviousFormat == OutRenderTargetFormat)
			{
				return false;
			}
		}

		return true;
	}

	const ETextureRenderTargetFormat GetRenderTargetFormatFromPixelFormat(EPixelFormat InPixelFormat)
	{
		switch (InPixelFormat)
		{
		case PF_G8:				return ETextureRenderTargetFormat::RTF_R8;
		case PF_R8G8:			return ETextureRenderTargetFormat::RTF_RG8;
		case PF_B8G8R8A8:		return ETextureRenderTargetFormat::RTF_RGBA8;
		case PF_R16F:			return ETextureRenderTargetFormat::RTF_R16f;
		case PF_G16R16F:		return ETextureRenderTargetFormat::RTF_RG16f;
		case PF_R32_FLOAT:		return ETextureRenderTargetFormat::RTF_R32f;
		case PF_G32R32F:		return ETextureRenderTargetFormat::RTF_RG32f;
		case PF_A32B32G32R32F:	return ETextureRenderTargetFormat::RTF_RGBA32f;
		case PF_A2B10G10R10:	return ETextureRenderTargetFormat::RTF_RGB10A2;
		case PF_FloatRGBA:		return ETextureRenderTargetFormat::RTF_RGBA16f;
		default:
			break;
		}

		UE_LOGF(LogNiagara, Error, "No mapping from pixel format to render target format is possible");
		return ETextureRenderTargetFormat::RTF_RGBA8;
	}

}
