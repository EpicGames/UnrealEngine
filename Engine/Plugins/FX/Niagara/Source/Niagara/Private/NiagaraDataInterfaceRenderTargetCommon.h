// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraWorldManager.h"

enum ETextureRenderTargetFormat : int;

namespace NiagaraDataInterfaceRenderTargetCommon
{
	extern int32 GIgnoreCookedOut;
	extern float GResolutionMultiplier;

	extern bool GetRenderTargetFormat(bool bOverrideFormat, ETextureRenderTargetFormat OverrideFormat, ETextureRenderTargetFormat& OutRenderTargetFormat);

	extern const ETextureRenderTargetFormat GetRenderTargetFormatFromPixelFormat(EPixelFormat InPixelFormat);

	template<typename TRenderTargetType, typename TInstanceDataType>
	static bool CreateRenderTarget(FNiagaraSystemInstance* SystemInstance, TInstanceDataType* InstanceData)
	{
		if (FNiagaraWorldManager* WorldManager = SystemInstance->GetWorldManager())
		{
			InstanceData->bManagedTexture = true;
			InstanceData->TargetTexture = WorldManager->ObjectPoolGetOrCreate<TRenderTargetType>();
			if (InstanceData->TargetTexture != nullptr)
			{
				WorldManager->AddReferencedObject(InstanceData->TargetTexture);
				return true;
			}
		}
		return false;
	}

	template<typename TInstanceDataType>
	static void ReleaseRenderTarget(FNiagaraSystemInstance* SystemInstance, TInstanceDataType* InstanceData)
	{
		if (InstanceData->bManagedTexture && InstanceData->TargetTexture)
		{
			if (FNiagaraWorldManager* WorldManager = SystemInstance->GetWorldManager())
			{
				InstanceData->TargetTexture->OverrideFormat = PF_Unknown;

				WorldManager->ObjectPoolReturn(InstanceData->TargetTexture);
				WorldManager->RemoveReferencedObject(InstanceData->TargetTexture);
				InstanceData->TargetTexture = nullptr;
				InstanceData->bManagedTexture = false;
			}
		}
	}
}
