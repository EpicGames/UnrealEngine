// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceRenderTargetCube.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "TextureResource.h"
#include "Engine/TextureRenderTargetCube.h"

#include "NiagaraCompileHashVisitor.h"
#include "NiagaraDataInterfaceRenderTargetCommon.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraStats.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraGpuComputeDebugInterface.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceRenderTargetCube)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceRenderTargetCube"

namespace NDIRenderTargetCubeLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(int, TextureSize)
		SHADER_PARAMETER(int, MipLevels)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTextureCube<float4>, RWTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(TextureCube<float4>, Texture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceRenderTargetCubeTemplate.ush");

	// Global VM function names, also used by the shaders code generation methods.
	static const FName SetValueFunctionName("SetRenderTargetValue");
	//static const FName GetValueFunctionName("GetRenderTargetValue");
	//static const FName LoadValueFunctionName("LoadRenderTargetValue");
	static const FName SampleValueFunctionName("SampleRenderTargetValue");
	static const FName SetSizeFunctionName("SetRenderTargetSize");
	static const FName GetSizeFunctionName("GetRenderTargetSize");
	static const FName GetNumMipLevelsName("GetNumMipLevels");
	static const FName SetFormatFunctionName("SetRenderTargetFormat");
	static const FName LinearToIndexName("LinearToIndex");
	static const FName ExecToIndexName("ExecToIndex");
	static const FName ExecToUnitName("ExecToUnit");

	struct EFunctionVersion
	{
		enum Type
		{
			InitialVersion = 0,
			AddedOptionalExecute = 1,
			AddedMipLevel = 2,

			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};
	};
}

FNiagaraVariableBase UNiagaraDataInterfaceRenderTargetCube::ExposedRTVar;

/*--------------------------------------------------------------------------------------------------------------------------*/

#if STATS
void FRenderTargetCubeRWInstanceData_RenderThread::UpdateMemoryStats()
{
	DEC_MEMORY_STAT_BY(STAT_NiagaraRenderTargetMemory, MemorySize);

	MemorySize = 0;
	if (RenderTarget.IsValid())
	{
		MemorySize = RHIComputeMemorySize(RenderTarget->GetRHI());
	}

	INC_MEMORY_STAT_BY(STAT_NiagaraRenderTargetMemory, MemorySize);
}
#endif

/*--------------------------------------------------------------------------------------------------------------------------*/

UNiagaraDataInterfaceRenderTargetCube::UNiagaraDataInterfaceRenderTargetCube(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyRenderTargetCubeProxy());

	FNiagaraTypeDefinition Def(UTextureRenderTarget::StaticClass());
	RenderTargetUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceRenderTargetCube::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);

		ExposedRTVar = FNiagaraVariableBase(FNiagaraTypeDefinition(UTexture::StaticClass()), TEXT("RenderTarget"));
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceRenderTargetCube::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIRenderTargetCubeLocal;

	Super::GetFunctionsInternal(OutFunctions);

	const int32 EmitterSystemOnlyBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
	OutFunctions.Reserve(OutFunctions.Num() + 7);

	FNiagaraFunctionSignature DefaultSig;
	DefaultSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget"));
	DefaultSig.bMemberFunction = true;
	DefaultSig.bRequiresContext = false;
	DefaultSig.bSupportsCPU = true;
	DefaultSig.bSupportsGPU = true;
	DefaultSig.SetFunctionVersion(EFunctionVersion::LatestVersion);

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = GetSizeFunctionName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Size"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetSizeFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Size"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success"));
		Sig.ModuleUsageBitmask = EmitterSystemOnlyBitmask;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsGPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = GetNumMipLevelsName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevels"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetFormatFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(StaticEnum<ETextureRenderTargetFormat>()), TEXT("Format"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success"));
		Sig.ModuleUsageBitmask = EmitterSystemOnlyBitmask;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsGPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetValueFunctionName;
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled")).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Face"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SampleValueFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UVW"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = LinearToIndexName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Face"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = ExecToIndexName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Face"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = ExecToUnitName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Unit"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Face"));
		Sig.bSupportsCPU = false;
	}
}

bool UNiagaraDataInterfaceRenderTargetCube::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	using namespace NDIRenderTargetCubeLocal;

	bool bWasChanged = false;

	if (FunctionSignature.FunctionVersion < EFunctionVersion::AddedOptionalExecute)
	{
		if (FunctionSignature.Name == SetValueFunctionName)
		{
			check(FunctionSignature.Inputs.Num() == 5);
			FunctionSignature.Inputs.Insert_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled")), 1).SetValue(true);
			bWasChanged = true;
		}
	}
	if (FunctionSignature.FunctionVersion < EFunctionVersion::AddedMipLevel)
	{
		if (FunctionSignature.Name == SampleValueFunctionName)
		{
			FunctionSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
			bWasChanged = true;
		}
	}

	// Set latest version
	FunctionSignature.FunctionVersion = EFunctionVersion::LatestVersion;

	return bWasChanged;
}
#endif

void UNiagaraDataInterfaceRenderTargetCube::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	using namespace NDIRenderTargetCubeLocal;

	Super::GetVMExternalFunction(BindingInfo, InstanceData, OutFunc);
	if (BindingInfo.Name == GetSizeFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTargetCube::VMGetSize);
	}
	else if (BindingInfo.Name == SetSizeFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTargetCube::VMSetSize);
	}
	else if (BindingInfo.Name == GetNumMipLevelsName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTargetCube::VMGetNumMipLevels);
	}
	else if (BindingInfo.Name == SetFormatFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTargetCube::VMSetFormat);
	}
}

bool UNiagaraDataInterfaceRenderTargetCube::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	
	const UNiagaraDataInterfaceRenderTargetCube* OtherTyped = CastChecked<const UNiagaraDataInterfaceRenderTargetCube>(Other);
	return
		OtherTyped != nullptr &&
#if WITH_EDITORONLY_DATA
		OtherTyped->bPreviewRenderTarget == bPreviewRenderTarget &&
#endif
		OtherTyped->RenderTargetUserParameter == RenderTargetUserParameter &&
		OtherTyped->Size == Size &&
		OtherTyped->OverrideRenderTargetFormat == OverrideRenderTargetFormat &&
		OtherTyped->OverrideRenderTargetFilter == OverrideRenderTargetFilter &&
		OtherTyped->bInheritUserParameterSettings == bInheritUserParameterSettings &&
		OtherTyped->bOverrideFormat == bOverrideFormat;
}

bool UNiagaraDataInterfaceRenderTargetCube::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if ( !Super::CopyToInternal(Destination) )
	{
		return false;
	}

	UNiagaraDataInterfaceRenderTargetCube* DestinationTyped = CastChecked<UNiagaraDataInterfaceRenderTargetCube>(Destination);
	if (!DestinationTyped)
	{
		return false;
	}

	DestinationTyped->Size = Size;
	DestinationTyped->OverrideRenderTargetFormat = OverrideRenderTargetFormat;
	DestinationTyped->OverrideRenderTargetFilter = OverrideRenderTargetFilter;
	DestinationTyped->bInheritUserParameterSettings = bInheritUserParameterSettings;
	DestinationTyped->bOverrideFormat = bOverrideFormat;
#if WITH_EDITORONLY_DATA
	DestinationTyped->bPreviewRenderTarget = bPreviewRenderTarget;
#endif
	DestinationTyped->RenderTargetUserParameter = RenderTargetUserParameter;
	return true;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceRenderTargetCube::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(NDIRenderTargetCubeLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<NDIRenderTargetCubeLocal::FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceRenderTargetCube::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIRenderTargetCubeLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceRenderTargetCube::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIRenderTargetCubeLocal;

	if ((FunctionInfo.DefinitionName == SetValueFunctionName) ||
		//(FunctionInfo.DefinitionName == GetValueFunctionName) ||
		//(FunctionInfo.DefinitionName == LoadValueFunctionName) ||
		(FunctionInfo.DefinitionName == SampleValueFunctionName) ||
		(FunctionInfo.DefinitionName == GetSizeFunctionName) ||
		(FunctionInfo.DefinitionName == LinearToIndexName) ||
		(FunctionInfo.DefinitionName == ExecToIndexName) ||
		(FunctionInfo.DefinitionName == ExecToUnitName))
	{
		return true;
	}

	return false;
}
#endif

void UNiagaraDataInterfaceRenderTargetCube::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIRenderTargetCubeLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceRenderTargetCube::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyRenderTargetCubeProxy& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyRenderTargetCubeProxy>();
	FRenderTargetCubeRWInstanceData_RenderThread* InstanceData_RT = DIProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	check(InstanceData_RT);

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	// Ensure RDG resources are ready to use
	if (InstanceData_RT->TransientRDGTexture == nullptr && InstanceData_RT->RenderTarget)
	{
		InstanceData_RT->TransientRDGTexture = GraphBuilder.RegisterExternalTexture(InstanceData_RT->RenderTarget);
		InstanceData_RT->TransientRDGSRV = GraphBuilder.CreateSRV(InstanceData_RT->TransientRDGTexture);
		InstanceData_RT->TransientRDGUAV = GraphBuilder.CreateUAV(InstanceData_RT->TransientRDGTexture);

		GraphBuilder.UseInternalAccessMode(InstanceData_RT->TransientRDGTexture);
		Context.GetRDGExternalAccessQueue().Add(InstanceData_RT->TransientRDGTexture);
	}

	NDIRenderTargetCubeLocal::FShaderParameters* Parameters = Context.GetParameterNestedStruct<NDIRenderTargetCubeLocal::FShaderParameters>();
	Parameters->TextureSize = InstanceData_RT->Size;
	Parameters->MipLevels = InstanceData_RT->MipLevels;

	const bool bRTWrite = Context.IsResourceBound(&Parameters->RWTexture);
	const bool bRTRead = Context.IsResourceBound(&Parameters->Texture);

	if (bRTWrite)
	{
		if (InstanceData_RT->RenderTarget.IsValid())
		{
			Parameters->RWTexture = InstanceData_RT->TransientRDGUAV;
			InstanceData_RT->bWroteThisFrame = true;
		}
		else
		{
			Parameters->RWTexture = Context.GetComputeDispatchInterface().GetEmptyTextureUAV(GraphBuilder, EPixelFormat::PF_A16B16G16R16, ETextureDimension::TextureCube);
		}
	}

	if (bRTRead)
	{
		if (bRTWrite == false && InstanceData_RT->RenderTarget.IsValid())
		{
			InstanceData_RT->bReadThisFrame = true;
			Parameters->Texture = InstanceData_RT->TransientRDGSRV;
		}
		else
		{
		#if WITH_NIAGARA_DEBUG_EMITTER_NAME
			GEngine->AddOnScreenDebugMessage(uint64(this), 1.f, FColor::White, *FString::Printf(TEXT("RenderTarget is read and wrote in the same stage, this is not allowed, read will be invalid. (%s)"), *Context.GetDebugString()));
		#endif
			Parameters->Texture = Context.GetComputeDispatchInterface().GetBlackTextureSRV(GraphBuilder, ETextureDimension::TextureCube);
		}

		if (InstanceData_RT->SamplerStateRHI)
		{
			Parameters->TextureSampler = InstanceData_RT->SamplerStateRHI;
		}
		else
		{
			Parameters->TextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		}
	}
}

bool UNiagaraDataInterfaceRenderTargetCube::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	check(Proxy);

	FRenderTargetCubeRWInstanceData_GameThread* InstanceData = new (PerInstanceData) FRenderTargetCubeRWInstanceData_GameThread();

	//-TEMP: Until we prune data interface on cook this will avoid consuming memory
	const bool bValidGpuDataInterface = NiagaraDataInterfaceRenderTargetCommon::GIgnoreCookedOut == 0 || IsUsedWithGPUScript();

	ETextureRenderTargetFormat RenderTargetFormat;
	if (NiagaraDataInterfaceRenderTargetCommon::GetRenderTargetFormat(bOverrideFormat, OverrideRenderTargetFormat, RenderTargetFormat) == false)
	{
		if (bValidGpuDataInterface)
		{
			UE_LOGF(LogNiagara, Warning, "NDIRTCube failed to find a render target format that supports UAV store");
			return false;
		}
		return true;
	}

	InstanceData->Size = FMath::Clamp<int>(int(float(Size) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxCubeTextureDimensions);
	InstanceData->Format = GetPixelFormatFromRenderTargetFormat(RenderTargetFormat);
	InstanceData->Filter = OverrideRenderTargetFilter;
	InstanceData->RTUserParamBinding.Init(SystemInstance->GetInstanceParameters(), RenderTargetUserParameter.Parameter);
#if WITH_EDITORONLY_DATA
	InstanceData->bPreviewTexture = bPreviewRenderTarget;
#endif

	return true;
}

void UNiagaraDataInterfaceRenderTargetCube::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIRenderTargetCubeLocal;

	FRenderTargetCubeRWInstanceData_GameThread* InstanceData = static_cast<FRenderTargetCubeRWInstanceData_GameThread*>(PerInstanceData);
	NiagaraDataInterfaceRenderTargetCommon::ReleaseRenderTarget(SystemInstance, InstanceData);
	InstanceData->~FRenderTargetCubeRWInstanceData_GameThread();

	FNiagaraDataInterfaceProxyRenderTargetCubeProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTargetCubeProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[RT_Proxy, InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
#if STATS
			if (FRenderTargetCubeRWInstanceData_RenderThread* TargetData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID))
			{
				TargetData->SamplerStateRHI.SafeRelease();
				TargetData->RenderTarget.SafeRelease();
				TargetData->UpdateMemoryStats();
			}
#endif
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		}
	);
}


void UNiagaraDataInterfaceRenderTargetCube::GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const
{
	OutVariables.Emplace(ExposedRTVar);
}

bool UNiagaraDataInterfaceRenderTargetCube::GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const
{
	FRenderTargetCubeRWInstanceData_GameThread* InstanceData = static_cast<FRenderTargetCubeRWInstanceData_GameThread*>(InPerInstanceData);
	if (InVariable.IsValid() && InVariable == ExposedRTVar && InstanceData && InstanceData->TargetTexture)
	{
		UObject** Var = (UObject**)OutData;
		*Var = InstanceData->TargetTexture;
		return true;
	}
	return false;
}

void UNiagaraDataInterfaceRenderTargetCube::VMSetSize(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FRenderTargetCubeRWInstanceData_GameThread> InstData(Context);
	FNDIInputParam<int> InSize(Context);
	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const int NewSize = InSize.GetAndAdvance();
		const bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1 && NewSize > 0);
		OutSuccess.SetAndAdvance(bSuccess);
		if (bSuccess)
		{
			InstData->Size = FMath::Clamp<int>(int(float(NewSize) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxCubeTextureDimensions);
		}
	}
}

void UNiagaraDataInterfaceRenderTargetCube::VMGetSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FRenderTargetCubeRWInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int> OutSize(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutSize.SetAndAdvance(InstData->Size);
	}
}

void UNiagaraDataInterfaceRenderTargetCube::VMGetNumMipLevels(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FRenderTargetCubeRWInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int> OutMipLevels(Context);

	int32 NumMipLevels = 1;
	if (InstData->TargetTexture != nullptr)
	{
		NumMipLevels = InstData->TargetTexture->GetNumMips();
	}
	//else if (InstData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled)
	//{
	//	NumMipLevels = FMath::FloorLog2(FMath::Max(InstData->Size.X, InstData->Size.Y)) + 1;
	//}
	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutMipLevels.SetAndAdvance(NumMipLevels);
	}
}

void UNiagaraDataInterfaceRenderTargetCube::VMSetFormat(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FRenderTargetCubeRWInstanceData_GameThread> InstData(Context);
	FNDIInputParam<ETextureRenderTargetFormat> InFormat(Context);
	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const ETextureRenderTargetFormat Format = InFormat.GetAndAdvance();


		const bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1);
		OutSuccess.SetAndAdvance(bSuccess);
		if (bSuccess)
		{
			InstData->Format = GetPixelFormatFromRenderTargetFormat(Format);

			if (bInheritUserParameterSettings && InstData->RTUserParamBinding.GetValue<UTextureRenderTargetCube>())
			{
				UE_LOGF(LogNiagara, Warning, "Overriding inherited volume render target format");
			}
		}
	}
}

bool UNiagaraDataInterfaceRenderTargetCube::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIRenderTargetCubeLocal;

	FRenderTargetCubeRWInstanceData_GameThread* InstanceData = static_cast<FRenderTargetCubeRWInstanceData_GameThread*>(PerInstanceData);

	// Pull from user parameter
	UTextureRenderTargetCube* UserTargetTexture = InstanceData->RTUserParamBinding.GetValue<UTextureRenderTargetCube>();
	if (UserTargetTexture && (InstanceData->TargetTexture != UserTargetTexture))
	{
		NiagaraDataInterfaceRenderTargetCommon::ReleaseRenderTarget(SystemInstance, InstanceData);
		InstanceData->TargetTexture = UserTargetTexture;
	}

	// Do we inherit the texture parameters from the user supplied texture?
	if (bInheritUserParameterSettings)
	{
		if (UserTargetTexture)
		{
			InstanceData->Size = UserTargetTexture->SizeX;
			//if (UserTargetTexture->bAutoGenerateMips)
			//{
			//	// We have to take a guess at user intention
			//	InstanceData->MipMapGeneration = MipMapGeneration == ENiagaraMipMapGeneration::Disabled ? ENiagaraMipMapGeneration::PostStage : MipMapGeneration;
			//}
			//else
			//{
			//	InstanceData->MipMapGeneration = ENiagaraMipMapGeneration::Disabled;
			//}
			InstanceData->Format = InstanceData->TargetTexture->GetFormat();
			InstanceData->Filter = InstanceData->TargetTexture->Filter;
		}
		else
		{
			UE_LOGF(LogNiagara, Error, "RenderTarget UserParam is required but invalid.");
		}
	}
		
	return false;
}

bool UNiagaraDataInterfaceRenderTargetCube::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIRenderTargetCubeLocal;

	// Update InstanceData as the texture may have changed
	FRenderTargetCubeRWInstanceData_GameThread* InstanceData = static_cast<FRenderTargetCubeRWInstanceData_GameThread*>(PerInstanceData);
#if WITH_EDITORONLY_DATA
	InstanceData->bPreviewTexture = bPreviewRenderTarget;
#endif

	//-TEMP: Until we prune data interface on cook this will avoid consuming memory
	const bool bValidGpuDataInterface = NiagaraDataInterfaceRenderTargetCommon::GIgnoreCookedOut == 0 || IsUsedWithGPUScript();

	if (::IsValid(InstanceData->TargetTexture) == false)
	{
		InstanceData->TargetTexture = nullptr;
	}

	// Do we need to create a new texture?
	if (bValidGpuDataInterface && !bInheritUserParameterSettings && (InstanceData->TargetTexture == nullptr))
	{
		if (NiagaraDataInterfaceRenderTargetCommon::CreateRenderTarget<UTextureRenderTargetCube>(SystemInstance, InstanceData) == false)
		{
			return false;
		}
		check(InstanceData->TargetTexture);
		InstanceData->TargetTexture->bCanCreateUAV = true;
		//InstanceData->TargetTexture->bAutoGenerateMips = InstanceData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled;
		InstanceData->TargetTexture->ClearColor = FLinearColor(0.0, 0, 0, 0);
		InstanceData->TargetTexture->Filter = InstanceData->Filter;
		InstanceData->TargetTexture->Init(InstanceData->Size, InstanceData->Format);
		InstanceData->TargetTexture->UpdateResourceImmediate(true);
	}

	// Do we need to update the existing texture?
	if (InstanceData->TargetTexture)
	{
		//const bool bAutoGenerateMips = InstanceData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled;
		if ((InstanceData->TargetTexture->SizeX != InstanceData->Size) || 
			(InstanceData->TargetTexture->GetFormat() != InstanceData->Format) ||
			(InstanceData->TargetTexture->Filter != InstanceData->Filter) ||
			!InstanceData->TargetTexture->bCanCreateUAV ||
			//(InstanceData->TargetTexture->bAutoGenerateMips != bAutoGenerateMips) ||
			!InstanceData->TargetTexture->GetResource())
		{
			// resize RT to match what we need for the output
			InstanceData->TargetTexture->bCanCreateUAV = true;
			//InstanceData->TargetTexture->bAutoGenerateMips = bAutoGenerateMips;
			InstanceData->TargetTexture->Filter = InstanceData->Filter;
			InstanceData->TargetTexture->Init(InstanceData->Size, InstanceData->Format);
			InstanceData->TargetTexture->UpdateResourceImmediate(true);
		}
	}

	//-TODO: We could avoid updating each frame if we cache the resource pointer or a serial number
	const bool bUpdateRT = true && bValidGpuDataInterface;
	if (bUpdateRT)
	{
		FNiagaraDataInterfaceProxyRenderTargetCubeProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTargetCubeProxy>();
		FTextureRenderTargetResource* RT_TargetTexture = InstanceData->TargetTexture ? InstanceData->TargetTexture->GameThread_GetRenderTargetResource() : nullptr;
		ENQUEUE_RENDER_COMMAND(NDIRenderTargetCubeUpdate)
		(
			[RT_Proxy, RT_InstanceID=SystemInstance->GetId(), RT_InstanceData=*InstanceData, RT_TargetTexture](FRHICommandListImmediate& RHICmdList)
			{
				FRenderTargetCubeRWInstanceData_RenderThread* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.FindOrAdd(RT_InstanceID);
				TargetData->Size = RT_InstanceData.Size;
				TargetData->MipLevels = 1;
				//TargetData->MipMapGeneration = RT_InstanceData.MipMapGeneration;
			#if WITH_EDITORONLY_DATA
				TargetData->bPreviewTexture = RT_InstanceData.bPreviewTexture;
			#endif
				TargetData->SamplerStateRHI.SafeRelease();
				TargetData->RenderTarget.SafeRelease();
				if (RT_TargetTexture)
				{
					TargetData->MipLevels = RT_TargetTexture->GetCurrentMipCount();
					if (FTextureRenderTargetCubeResource* ResourceCube = RT_TargetTexture->GetTextureRenderTargetCubeResource())
					{
						TargetData->SamplerStateRHI = ResourceCube->SamplerStateRHI;
						TargetData->RenderTarget = CreateRenderTarget(ResourceCube->GetTextureRHI(), TEXT("NiagaraRenderTargetCube"));
					}
				}
			#if STATS
				TargetData->UpdateMemoryStats();
			#endif
			}
		);
	}

	return false;
}

void FNiagaraDataInterfaceProxyRenderTargetCubeProxy::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	FRenderTargetCubeRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	if (ProxyData == nullptr)
	{
		return;
	}

	
	// We only need to transfer this frame if it was written to.
	// If also read then we need to notify that the texture is important for the simulation
	// We also assume the texture is important for rendering, without discovering renderer bindings we don't really know
	if (ProxyData->bWroteThisFrame)
	{
		Context.GetComputeDispatchInterface().MultiGPUResourceModified(Context.GetGraphBuilder(), ProxyData->RenderTarget->GetRHI(), ProxyData->bReadThisFrame, true);
	}

#if NIAGARA_COMPUTEDEBUG_ENABLED && WITH_EDITORONLY_DATA
	if (ProxyData->bPreviewTexture && ProxyData->TransientRDGTexture)
	{
		FNiagaraGpuComputeDebugInterface GpuComputeDebugInterface = Context.GetComputeDispatchInterface().GetGpuComputeDebugInterface();
		GpuComputeDebugInterface.AddTexture(Context.GetGraphBuilder(), Context.GetSystemInstanceID(), GetSourceDIName(), ProxyData->TransientRDGTexture);
	}
#endif

	// Clean up our temporary tracking
	ProxyData->bReadThisFrame = false;
	ProxyData->bWroteThisFrame = false;

	ProxyData->TransientRDGTexture = nullptr;
	ProxyData->TransientRDGSRV = nullptr;
	ProxyData->TransientRDGUAV = nullptr;
}

void FNiagaraDataInterfaceProxyRenderTargetCubeProxy::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context)
{
	if ( const FRenderTargetCubeRWInstanceData_RenderThread* TargetData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()) )
	{
		Context.SetDirect(FIntVector(TargetData->Size, TargetData->Size, 6));
	}
}

#undef LOCTEXT_NAMESPACE

