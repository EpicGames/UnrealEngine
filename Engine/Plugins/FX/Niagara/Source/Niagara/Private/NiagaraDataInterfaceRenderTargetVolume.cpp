// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceRenderTargetVolume.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "TextureResource.h"
#include "NiagaraCompileHashVisitor.h"
#include "TextureRenderTargetVolumeResource.h"
#include "Engine/TextureRenderTargetVolume.h"
#include "HAL/PlatformFileManager.h"

#include "DataInterface/NDIRenderTargetSimCacheData.h"
#include "NiagaraDataInterfaceRenderTargetCommon.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSettings.h"
#include "NiagaraSimCache.h"
#include "NiagaraSVTShaders.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraStats.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraShader.h"
#include "NiagaraGpuComputeDebugInterface.h"
#include "VolumeCache.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalRenderResources.h"
#include "ShaderCompilerCore.h"

#include "RenderGraphUtils.h"

#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "SparseVolumeTexture/SparseVolumeTextureData.h"
#include "SparseVolumeTexture/ISparseVolumeTextureStreamingManager.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceRenderTargetVolume)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceRenderTargetVolume"

namespace NDIRenderTargetVolumeLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FIntVector3, TextureSize)
		SHADER_PARAMETER(int, MipLevels)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D<float4>, Texture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceRenderTargetVolumeTemplate.ush");

	// Global VM function names, also used by the shaders code generation methods.
	static const FName SetValueFunctionName("SetRenderTargetValue");
	static const FName LoadValueFunctionName("LoadRenderTargetValue");
	static const FName SampleValueFunctionName("SampleRenderTargetValue");
	static const FName SetSizeFunctionName("SetRenderTargetSize");
	static const FName GetSizeFunctionName("GetRenderTargetSize");
	static const FName GetNumMipLevelsName("GetNumMipLevels");
	static const FName SetFormatFunctionName("SetRenderTargetFormat");
	static const FName LinearToIndexName("LinearToIndex");
	static const FName ExecToIndexName("ExecToIndex");
	static const FName ExecToUnitName("ExecToUnit");
	static const FName Deprecated_GetValueFunctionName("GetRenderTargetValue");

	struct EFunctionVersion
	{
		enum Type
		{
			InitialVersion = 0,
			AddedOptionalExecute = 1,
			AddedMipLevel = 2,

			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};
	};

	int32 GSimCacheEnabled = true;
	static FAutoConsoleVariableRef CVarSimCacheEnabled(
		TEXT("fx.Niagara.RenderTargetVolume.SimCacheEnabled"),
		GSimCacheEnabled,
		TEXT("When enabled we can write data into the simulation cache."),
		ECVF_Default
	);

	int32 GSimCacheCompressed = true;
	static FAutoConsoleVariableRef CVarSimCacheCompressed(
		TEXT("fx.Niagara.RenderTargetVolume.SimCacheCompressed"),
		GSimCacheCompressed,
		TEXT("When enabled compression is used for the sim cache data."),
		ECVF_Default
	);

	enum SimCacheStorageMode
	{
		Dense = 0,
		SVT = 1
	};

	int32 GSimCacheDataStorageMode = SimCacheStorageMode::SVT;
	static FAutoConsoleVariableRef CVarSimCacheDataStorageMode(
		TEXT("fx.Niagara.RenderTargetVolume.SimCacheDataStorageMode"),
		GSimCacheDataStorageMode,
		TEXT("Backing storage type for Volume RT sim cache data.  0 uses raw data, 1 uses OpenVDB, 2 uses SVT"),
		ECVF_Default
	);



	int32 GSimCacheUseOpenVDBFloatGrids = false;
	static FAutoConsoleVariableRef CVarSimCacheUseOpenVDBFloatGrids(
		TEXT("fx.Niagara.RenderTargetVolume.SimCacheUseOpenVDBFloatGrids"),
		GSimCacheUseOpenVDBFloatGrids,
		TEXT("Use OpenVDB float grids as output."),
		ECVF_Default
	);

	static const FName GetSimCacheCompressionType() { return GSimCacheCompressed ? NAME_Oodle : NAME_None; }
}

FNiagaraVariableBase UNiagaraDataInterfaceRenderTargetVolume::ExposedRTVar;

/*--------------------------------------------------------------------------------------------------------------------------*/

#if STATS
void FRenderTargetVolumeRWInstanceData_RenderThread::UpdateMemoryStats()
{
	DEC_MEMORY_STAT_BY(STAT_NiagaraRenderTargetMemory, MemorySize);

	MemorySize = 0;
	if (RenderTarget.IsValid())
	{
		MemorySize = RHIComputeMemorySize(RenderTarget->GetRHI());
	}

	INC_MEMORY_STAT_BY(STAT_NiagaraRenderTargetMemory, MemorySize);
}
#endif

/*--------------------------------------------------------------------------------------------------------------------------*/

UNiagaraDataInterfaceRenderTargetVolume::UNiagaraDataInterfaceRenderTargetVolume(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyRenderTargetVolumeProxy());

	FNiagaraTypeDefinition Def(UTextureRenderTarget::StaticClass());
	RenderTargetUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceRenderTargetVolume::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);

		ExposedRTVar = FNiagaraVariableBase(FNiagaraTypeDefinition(UTexture::StaticClass()), TEXT("RenderTarget"));
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceRenderTargetVolume::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIRenderTargetVolumeLocal;

	Super::GetFunctionsInternal(OutFunctions);

	const int32 EmitterSystemOnlyBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
	OutFunctions.Reserve(OutFunctions.Num() + 7);

	FNiagaraFunctionSignature DefaultSig;
	DefaultSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("RenderTarget"));
	DefaultSig.bMemberFunction = true;
	DefaultSig.bRequiresContext = false;
	DefaultSig.bSupportsCPU = true;
	DefaultSig.bSupportsGPU = true;
	DefaultSig.SetFunctionVersion(EFunctionVersion::LatestVersion);

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = GetSizeFunctionName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Width"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Height"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Depth"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetSizeFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Width"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Height"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Depth"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success"));
		Sig.ModuleUsageBitmask = EmitterSystemOnlyBitmask;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsGPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = GetNumMipLevelsName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevels"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetFormatFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(StaticEnum<ETextureRenderTargetFormat>()), TEXT("Format"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success"));
		Sig.ModuleUsageBitmask = EmitterSystemOnlyBitmask;
		Sig.bRequiresExecPin = true;
		Sig.bSupportsGPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SetValueFunctionName;
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled")).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bRequiresExecPin = true;
		Sig.bWriteFunction = true;
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = LoadValueFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = Deprecated_GetValueFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = SampleValueFunctionName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UVW"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = LinearToIndexName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Linear"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = ExecToIndexName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexX"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexY"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("IndexZ"));
		Sig.bSupportsCPU = false;
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = ExecToUnitName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit"));
		Sig.bSupportsCPU = false;
	}
}

bool UNiagaraDataInterfaceRenderTargetVolume::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	using namespace NDIRenderTargetVolumeLocal;

	bool bWasChanged = false;

	if (FunctionSignature.FunctionVersion < EFunctionVersion::AddedOptionalExecute)
	{
		if (FunctionSignature.Name == SetValueFunctionName)
		{
			check(FunctionSignature.Inputs.Num() == 5);
			FunctionSignature.Inputs.Insert_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled")), 1).SetValue(true);
			bWasChanged = true;
		}
	}
	if (FunctionSignature.FunctionVersion < EFunctionVersion::AddedMipLevel)
	{
		if (FunctionSignature.Name == SampleValueFunctionName)
		{
			FunctionSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
			bWasChanged = true;
		}
	}

	// Set latest version
	FunctionSignature.FunctionVersion = EFunctionVersion::LatestVersion;

	return bWasChanged;
}
#endif

void UNiagaraDataInterfaceRenderTargetVolume::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	using namespace NDIRenderTargetVolumeLocal;

	Super::GetVMExternalFunction(BindingInfo, InstanceData, OutFunc);
	if (BindingInfo.Name == GetSizeFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTargetVolume::VMGetSize);
	}
	else if (BindingInfo.Name == SetSizeFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTargetVolume::VMSetSize);
	}
	else if (BindingInfo.Name == GetNumMipLevelsName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTargetVolume::VMGetNumMipLevels);
	}
	else if (BindingInfo.Name == SetFormatFunctionName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceRenderTargetVolume::VMSetFormat);
	}
}

bool UNiagaraDataInterfaceRenderTargetVolume::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	
	const UNiagaraDataInterfaceRenderTargetVolume* OtherTyped = CastChecked<const UNiagaraDataInterfaceRenderTargetVolume>(Other);
	return
		OtherTyped != nullptr &&
#if WITH_EDITORONLY_DATA
		OtherTyped->bPreviewRenderTarget == bPreviewRenderTarget &&
#endif
		OtherTyped->RenderTargetUserParameter == RenderTargetUserParameter &&
		OtherTyped->Size == Size &&
		OtherTyped->OverrideRenderTargetFormat == OverrideRenderTargetFormat &&
		OtherTyped->OverrideRenderTargetFilter == OverrideRenderTargetFilter &&
		OtherTyped->bInheritUserParameterSettings == bInheritUserParameterSettings &&
		OtherTyped->bOverrideFormat == bOverrideFormat;
}

bool UNiagaraDataInterfaceRenderTargetVolume::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if ( !Super::CopyToInternal(Destination) )
	{
		return false;
	}

	UNiagaraDataInterfaceRenderTargetVolume* DestinationTyped = CastChecked<UNiagaraDataInterfaceRenderTargetVolume>(Destination);
	if (!DestinationTyped)
	{
		return false;
	}

	DestinationTyped->Size = Size;
	DestinationTyped->OverrideRenderTargetFormat = OverrideRenderTargetFormat;
	DestinationTyped->OverrideRenderTargetFilter = OverrideRenderTargetFilter;
	DestinationTyped->bInheritUserParameterSettings = bInheritUserParameterSettings;
	DestinationTyped->bOverrideFormat = bOverrideFormat;
#if WITH_EDITORONLY_DATA
	DestinationTyped->bPreviewRenderTarget = bPreviewRenderTarget;
#endif
	DestinationTyped->RenderTargetUserParameter = RenderTargetUserParameter;
	return true;
}

#if WITH_EDITOR
bool UNiagaraDataInterfaceRenderTargetVolume::ShouldCompile(EShaderPlatform ShaderPlatform) const
{
	return
		RHIVolumeTextureRenderingSupportGuaranteed(ShaderPlatform) &&
		Super::ShouldCompile(ShaderPlatform);
}
#endif

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceRenderTargetVolume::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIRenderTargetVolumeLocal::TemplateShaderFile);
	bSuccess &= InVisitor->UpdateShaderParameters<NDIRenderTargetVolumeLocal::FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceRenderTargetVolume::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),				ParamInfo.DataInterfaceHLSLSymbol},
	};

	FString TemplateFile;
	LoadShaderSourceFile(NDIRenderTargetVolumeLocal::TemplateShaderFile, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

bool UNiagaraDataInterfaceRenderTargetVolume::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIRenderTargetVolumeLocal;

	if ((FunctionInfo.DefinitionName == SetValueFunctionName) ||
		(FunctionInfo.DefinitionName == Deprecated_GetValueFunctionName) ||
		(FunctionInfo.DefinitionName == LoadValueFunctionName) ||
		(FunctionInfo.DefinitionName == SampleValueFunctionName) ||
		(FunctionInfo.DefinitionName == GetSizeFunctionName) ||
		(FunctionInfo.DefinitionName == LinearToIndexName) ||
		(FunctionInfo.DefinitionName == ExecToIndexName) ||
		(FunctionInfo.DefinitionName == ExecToUnitName))
	{
		return true;
	}

	return false;
}
#endif

void UNiagaraDataInterfaceRenderTargetVolume::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIRenderTargetVolumeLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceRenderTargetVolume::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyRenderTargetVolumeProxy& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyRenderTargetVolumeProxy>();
	FRenderTargetVolumeRWInstanceData_RenderThread* InstanceData_RT = DIProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	check(InstanceData_RT);

	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	// Ensure RDG resources are ready to use
	if (InstanceData_RT->TransientRDGTexture == nullptr && InstanceData_RT->RenderTarget)
	{
		InstanceData_RT->TransientRDGTexture = GraphBuilder.RegisterExternalTexture(InstanceData_RT->RenderTarget);
		InstanceData_RT->TransientRDGSRV = GraphBuilder.CreateSRV(InstanceData_RT->TransientRDGTexture);
		InstanceData_RT->TransientRDGUAV = GraphBuilder.CreateUAV(InstanceData_RT->TransientRDGTexture);
		GraphBuilder.UseInternalAccessMode(InstanceData_RT->TransientRDGTexture);
		Context.GetRDGExternalAccessQueue().Add(InstanceData_RT->TransientRDGTexture);
	}

	NDIRenderTargetVolumeLocal::FShaderParameters* Parameters = Context.GetParameterNestedStruct<NDIRenderTargetVolumeLocal::FShaderParameters>();
	Parameters->TextureSize = InstanceData_RT->Size;
	Parameters->MipLevels = InstanceData_RT->MipLevels;

	const bool bRTWrite = Context.IsResourceBound(&Parameters->RWTexture);
	const bool bRTRead = Context.IsResourceBound(&Parameters->Texture);

	if (bRTWrite)
	{
		if (InstanceData_RT->RenderTarget.IsValid())
		{
			Parameters->RWTexture = InstanceData_RT->TransientRDGUAV;
			InstanceData_RT->bWroteThisFrame = true;
		}
		else
		{
			Parameters->RWTexture = Context.GetComputeDispatchInterface().GetEmptyTextureUAV(GraphBuilder, EPixelFormat::PF_A16B16G16R16, ETextureDimension::Texture3D);
		}
	}

	if (bRTRead)
	{
		if (bRTWrite == false && InstanceData_RT->RenderTarget.IsValid())
		{
			InstanceData_RT->bReadThisFrame = true;
			Parameters->Texture = InstanceData_RT->TransientRDGSRV;
		}
		else
		{
		#if WITH_NIAGARA_DEBUG_EMITTER_NAME
			GEngine->AddOnScreenDebugMessage(uint64(this), 1.f, FColor::White, *FString::Printf(TEXT("RenderTarget is read and wrote in the same stage, this is not allowed, read will be invalid. (%s)"), *Context.GetDebugString()));
		#endif
			Parameters->Texture = Context.GetComputeDispatchInterface().GetBlackTextureSRV(GraphBuilder, ETextureDimension::Texture3D);
		}

		if (InstanceData_RT->SamplerStateRHI)
		{
			Parameters->TextureSampler = InstanceData_RT->SamplerStateRHI;
		}
		else
		{
			Parameters->TextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		}
	}
}

bool UNiagaraDataInterfaceRenderTargetVolume::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	check(Proxy);

	FRenderTargetVolumeRWInstanceData_GameThread* InstanceData = new (PerInstanceData) FRenderTargetVolumeRWInstanceData_GameThread();

	//-TEMP: Until we prune data interface on cook this will avoid consuming memory
	const bool bValidGpuDataInterface = NiagaraDataInterfaceRenderTargetCommon::GIgnoreCookedOut == 0 || IsUsedWithGPUScript();

	ETextureRenderTargetFormat RenderTargetFormat;
	if (NiagaraDataInterfaceRenderTargetCommon::GetRenderTargetFormat(bOverrideFormat, OverrideRenderTargetFormat, RenderTargetFormat) == false)
	{
		if (bValidGpuDataInterface)
		{
			UE_LOGF(LogNiagara, Warning, "NDIRTVolume failed to find a render target format that supports UAV store");
			return false;
		}
		return true;
	}

	InstanceData->Size.X = FMath::Clamp<int>(int(float(Size.X) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxVolumeTextureDimensions);
	InstanceData->Size.Y = FMath::Clamp<int>(int(float(Size.Y) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxVolumeTextureDimensions);
	InstanceData->Size.Z = FMath::Clamp<int>(int(float(Size.Z) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxVolumeTextureDimensions);
	InstanceData->Format = GetPixelFormatFromRenderTargetFormat(RenderTargetFormat);
	InstanceData->Filter = OverrideRenderTargetFilter;
	InstanceData->RTUserParamBinding.Init(SystemInstance->GetInstanceParameters(), RenderTargetUserParameter.Parameter);
#if WITH_EDITORONLY_DATA
	InstanceData->bPreviewTexture = bPreviewRenderTarget;
#endif

	return true;
}

void UNiagaraDataInterfaceRenderTargetVolume::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIRenderTargetVolumeLocal;

	FRenderTargetVolumeRWInstanceData_GameThread* InstanceData = static_cast<FRenderTargetVolumeRWInstanceData_GameThread*>(PerInstanceData);
	NiagaraDataInterfaceRenderTargetCommon::ReleaseRenderTarget(SystemInstance, InstanceData);
	InstanceData->~FRenderTargetVolumeRWInstanceData_GameThread();

	FNiagaraDataInterfaceProxyRenderTargetVolumeProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTargetVolumeProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[RT_Proxy, InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
#if STATS
			if (FRenderTargetVolumeRWInstanceData_RenderThread* TargetData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID))
			{
				TargetData->SamplerStateRHI.SafeRelease();
				TargetData->RenderTarget.SafeRelease();
				TargetData->UpdateMemoryStats();
			}
#endif
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		}
	);
}


void UNiagaraDataInterfaceRenderTargetVolume::GetExposedVariables(TArray<FNiagaraVariableBase>& OutVariables) const
{
	OutVariables.Emplace(ExposedRTVar);
}

bool UNiagaraDataInterfaceRenderTargetVolume::GetExposedVariableValue(const FNiagaraVariableBase& InVariable, void* InPerInstanceData, FNiagaraSystemInstance* InSystemInstance, void* OutData) const
{
	FRenderTargetVolumeRWInstanceData_GameThread* InstanceData = static_cast<FRenderTargetVolumeRWInstanceData_GameThread*>(InPerInstanceData);
	if (InVariable.IsValid() && InVariable == ExposedRTVar && InstanceData && InstanceData->TargetTexture)
	{
		UObject** Var = (UObject**)OutData;
		*Var = InstanceData->TargetTexture;
		return true;
	}
	return false;
}

UObject* UNiagaraDataInterfaceRenderTargetVolume::SimCacheBeginWrite(UObject* InSimCache, FNiagaraSystemInstance* NiagaraSystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	if (!OptionalPerInstanceData)
	{
		return nullptr;
	}

	if (NDIRenderTargetVolumeLocal::GSimCacheDataStorageMode == NDIRenderTargetVolumeLocal::SimCacheStorageMode::SVT)
	{		
		UNiagaraSimCache* SimCache = CastChecked<UNiagaraSimCache>(InSimCache);
		UAnimatedSparseVolumeTexture* CurrCache = NewObject<UAnimatedSparseVolumeTexture>(SimCache);			
		
		CurrCache->BeginInitialize(1);

		return CurrCache;
	}	
	else if (NDIRenderTargetVolumeLocal::GSimCacheEnabled)
	{
		UNDIRenderTargetSimCacheData* SimCacheData = NewObject<UNDIRenderTargetSimCacheData>(InSimCache);
		SimCacheData->CompressionType = NDIRenderTargetVolumeLocal::GetSimCacheCompressionType();

		return SimCacheData;
	}

	return nullptr;
}

bool UNiagaraDataInterfaceRenderTargetVolume::SimCacheWriteFrame(UObject* StorageObject, int FrameIndex, FNiagaraSystemInstance* SystemInstance, const void* OptionalPerInstanceData, FNiagaraSimCacheFeedbackContext& FeedbackContext) const
{
	check(OptionalPerInstanceData);

	const FRenderTargetVolumeRWInstanceData_GameThread* InstanceData_GT = reinterpret_cast<const FRenderTargetVolumeRWInstanceData_GameThread*>(OptionalPerInstanceData);

	if (InstanceData_GT->TargetTexture)
	{
		const FNiagaraDataInterfaceProxyRenderTargetVolumeProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTargetVolumeProxy>();
		const FTextureRenderTargetResource* RT_TargetTexture = InstanceData_GT->TargetTexture->GameThread_GetRenderTargetResource();

		//-OPT: Currently we are flushing rendering commands.  Do not remove this until making access to the frame data safe across threads.
		TArray<uint8> TextureData;

		ENQUEUE_RENDER_COMMAND(NDIRenderTargetVolume_CacheFrame)
		(
			[RT_Proxy, RT_InstanceID=SystemInstance->GetId(), RT_TargetTexture, RT_TextureData = &TextureData, RT_VolumeResolution = InstanceData_GT->Size](FRHICommandListImmediate& RHICmdList)
			{
				if (const FRenderTargetVolumeRWInstanceData_RenderThread* InstanceData_RT = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID))
				{					
					FRHIGPUTextureReadback RenderTargetReadback("ReadVolumeTexture");

					RenderTargetReadback.EnqueueCopy(RHICmdList, InstanceData_RT->RenderTarget->GetRHI(), FIntVector(0,0,0), 0, RT_VolumeResolution);
					
					// Sync the GPU. Unfortunately we can't use the fences because not all RHIs implement them yet.
					constexpr bool bFlushResources = true;
					RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

					//Lock the readback staging texture
					int32 RowPitchInPixels;
					int32 BufferHeight;
					const uint8* LockedData = (const uint8*) RenderTargetReadback.Lock(RowPitchInPixels, &BufferHeight);
					
					uint32 BlockBytes = GPixelFormats[InstanceData_RT->RenderTarget->GetRHI()->GetFormat()].BlockBytes;
					int32 Count = InstanceData_RT->Size.X * InstanceData_RT->Size.Y * InstanceData_RT->Size.Z * BlockBytes;
					RT_TextureData->AddUninitialized(Count);

					const uint8* SliceStart = LockedData;
					for (int32 Z = 0; Z < InstanceData_RT->Size.Z; ++Z)
					{
						const uint8* RowStart = SliceStart;
						for (int32 Y = 0; Y < InstanceData_RT->Size.Y; ++Y)
						{									
							int32 Offset = 0 + Y * InstanceData_RT->Size.X + Z * InstanceData_RT->Size.X * InstanceData_RT->Size.Y;
							FMemory::Memcpy(RT_TextureData->GetData() + Offset * BlockBytes, RowStart, BlockBytes * InstanceData_RT->Size.X);

							RowStart += RowPitchInPixels * BlockBytes;
						}

						SliceStart += BufferHeight * RowPitchInPixels * BlockBytes;
					}

					//Unlock the staging texture
					RenderTargetReadback.Unlock();
				}
			}
		);
		FlushRenderingCommands();

		if (TextureData.Num() > 0)
		{
			if (NDIRenderTargetVolumeLocal::GSimCacheDataStorageMode == NDIRenderTargetVolumeLocal::SimCacheStorageMode::SVT)
			{
#if WITH_EDITOR
				UAnimatedSparseVolumeTexture* CurrCache = CastChecked<UAnimatedSparseVolumeTexture>(StorageObject);
				
				UE::SVT::FTextureDataCreateInfo SVTCreateInfo;
				SVTCreateInfo.VirtualVolumeAABBMin = FIntVector3::ZeroValue;
				SVTCreateInfo.VirtualVolumeAABBMax = InstanceData_GT->Size;
				SVTCreateInfo.FallbackValues[0] = FVector4f(0, 0, 0, 0);
				SVTCreateInfo.FallbackValues[1] = FVector4f(0, 0, 0, 0);
				SVTCreateInfo.AttributesFormats[0] = InstanceData_GT->Format;
				SVTCreateInfo.AttributesFormats[1] = PF_Unknown;

				UE::SVT::FTextureData SparseTextureData{};
				bool Success = SparseTextureData.CreateFromDense(SVTCreateInfo, TArrayView<uint8, int64>((uint8*)TextureData.GetData(), (int64)TextureData.Num() * sizeof(TextureData[0])), TArrayView<uint8>());
				
				if (!Success)
				{
					UE_LOGF(LogNiagara, Error, "Cannot create SVT for volume render target");

					return false;
				}

				CurrCache->AppendFrame(SparseTextureData, FTransform::Identity);
#endif
			}
			else
			{
				UNDIRenderTargetSimCacheData* SimCacheData = CastChecked<UNDIRenderTargetSimCacheData>(StorageObject);
				SimCacheData->SetPixelData(
					FrameIndex,
					InstanceData_GT->Size,
					InstanceData_GT->Format,
					TextureData
				);
			}
		}
	}

	return true;
}

bool UNiagaraDataInterfaceRenderTargetVolume::SimCacheEndWrite(UObject* StorageObject) const
{
	if (NDIRenderTargetVolumeLocal::GSimCacheDataStorageMode == NDIRenderTargetVolumeLocal::SimCacheStorageMode::SVT)
	{	
		UAnimatedSparseVolumeTexture* CurrCache = CastChecked<UAnimatedSparseVolumeTexture>(StorageObject);
		CurrCache->EndInitialize();
	#if WITH_EDITOR
		CurrCache->PostEditChange();
	#endif
	}

	return true;
}

bool UNiagaraDataInterfaceRenderTargetVolume::SimCacheReadFrame(const FNiagaraSimCacheDataInterfaceReadContext& ReadContext)
{
	FRenderTargetVolumeRWInstanceData_GameThread* InstanceData_GT = ReadContext.GetPerInstanceData<FRenderTargetVolumeRWInstanceData_GameThread>();

	if (ReadContext.GetOptionalStorageObject<UVolumeCache>())
	{
		UE_LOGF(LogNiagara, Error, "vdb caches are no longer supported.  Regenerate your Sim Cache, which will use SVT to store volume data.");

	}
	else if (UAnimatedSparseVolumeTexture* SVT = ReadContext.GetOptionalStorageObject<UAnimatedSparseVolumeTexture>())
	{			
		const int32 MipLevel = 0;
		const float FrameRate = 0.0f;
		const bool bBlocking = true;
		const bool bHasFrameRate = false;
		const float SVYFrameIndex = FMath::RoundToFloat(float(ReadContext.GetFrameIndexA()) + ReadContext.GetFrameInterp());
		USparseVolumeTextureFrame *SVTFrame = USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest(SVT, GetTypeHash(this), FrameRate, SVYFrameIndex, MipLevel, bBlocking, bHasFrameRate);
		
		// The streaming manager normally ticks in FDeferredShadingSceneRenderer::Render(), but the SVT->DenseTexture conversion compute shader happens in a render command before that.
		// At execution time of that command, the streamer hasn't had the chance to do any streaming yet, so we force another tick here.
		// Assuming blocking requests are used, this guarantees that the requested frame is fully streamed in (if there is memory available).
		UE::SVT::GetStreamingManager().Update_GameThread();

		FNiagaraSystemInstance* SystemInstance = ReadContext.GetSystemInstance();
		const FIntVector VolumeResolution = SVT->GetVolumeResolution();						
		InstanceData_GT->Size = VolumeResolution;
		InstanceData_GT->Format = SVT->GetFormat(0);
		InstanceData_GT->Filter = TextureFilter::TF_Default;

		PerInstanceTick(InstanceData_GT, SystemInstance, 0.0f);
		PerInstanceTickPostSimulate(InstanceData_GT, SystemInstance, 0.0f);

		FNiagaraDataInterfaceProxyRenderTargetVolumeProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTargetVolumeProxy>();

		// Execute compute shader
		ENQUEUE_RENDER_COMMAND(NDIRenderTargetVolumeUpdate)
			(
				[RT_Proxy, RT_InstanceID = SystemInstance->GetId(), RT_Format = InstanceData_GT->Format,
				RT_VolumeResolution = VolumeResolution, RT_SVTRenderResources = SVTFrame ? SVTFrame->GetTextureRenderResources() : nullptr,
				FeatureLevel = SystemInstance->GetFeatureLevel()](FRHICommandListImmediate& RHICmdList)
				{
					if (RT_SVTRenderResources == nullptr)
					{
						return;
					}

					FUintVector4 CurrentPackedUniforms0 = FUintVector4();
					FUintVector4 CurrentPackedUniforms1 = FUintVector4();
					RT_SVTRenderResources->GetPackedUniforms(CurrentPackedUniforms0, CurrentPackedUniforms1);

					if (FRenderTargetVolumeRWInstanceData_RenderThread * InstanceData_RT = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID))
					{
						FRDGBuilder GraphBuilder(RHICmdList);

						FIntVector ThreadGroupSize = FNiagaraShader::GetDefaultThreadGroupSize(ENiagaraGpuDispatchType::ThreeD);

						TShaderMapRef<FNiagaraCopySVTToDenseBufferCS> ComputeShader(GetGlobalShaderMap(FeatureLevel));						

						const FIntVector NumThreadGroups(
							FMath::DivideAndRoundUp(RT_VolumeResolution.X, ThreadGroupSize.X),
							FMath::DivideAndRoundUp(RT_VolumeResolution.Y, ThreadGroupSize.Y),
							FMath::DivideAndRoundUp(RT_VolumeResolution.Z, ThreadGroupSize.Z)
						);
						
						FNiagaraCopySVTToDenseBufferCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FNiagaraCopySVTToDenseBufferCS::FParameters>();

						if (InstanceData_RT->RenderTarget.IsValid() && InstanceData_RT->TransientRDGTexture == nullptr)
						{
							InstanceData_RT->TransientRDGTexture = GraphBuilder.RegisterExternalTexture(InstanceData_RT->RenderTarget);							
							InstanceData_RT->TransientRDGUAV = GraphBuilder.CreateUAV(InstanceData_RT->TransientRDGTexture);

							// #todo(dmp): needed?							
							//Context.GetRDGExternalAccessQueue().Add(InstanceData_RT->TransientRDGTexture);
						}

						if (InstanceData_RT->RenderTarget.IsValid() && InstanceData_RT->TransientRDGUAV != nullptr)
						{
							PassParameters->DestinationBuffer = InstanceData_RT->TransientRDGUAV;
						}
						else
						{
							PassParameters->DestinationBuffer = GraphBuilder.CreateUAV(GraphBuilder.CreateTexture(
									FRDGTextureDesc::Create3D(FIntVector(1, 1, 1), RT_Format, FClearValueBinding::Black, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV),
									TEXT("NiagaraEmptyTextureUAV::Texture3D")
								),
								ERDGUnorderedAccessViewFlags::SkipBarrier
							);
						}						

						FRHITexture* PageTableTexture = RT_SVTRenderResources->GetPageTableTexture();
						FRHITexture* TextureA = RT_SVTRenderResources->GetPhysicalTileDataATexture();

						PassParameters->TileDataTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
						PassParameters->SparseVolumeTexturePageTable = PageTableTexture ? PageTableTexture : GBlackUintVolumeTexture->TextureRHI.GetReference();
						PassParameters->SparseVolumeTextureA = TextureA ? TextureA : GBlackVolumeTexture->TextureRHI.GetReference();
							
						PassParameters->PackedSVTUniforms0 = CurrentPackedUniforms0;
						PassParameters->PackedSVTUniforms1 = CurrentPackedUniforms1;

						PassParameters->TextureSize = RT_VolumeResolution;
						PassParameters->MipLevel = 0;
						
						GraphBuilder.AddPass(
							// Friendly name of the pass for profilers using printf semantics.
							RDG_EVENT_NAME("Copy SVT to Volume RT"),
							// Parameters provided to RDG.
							PassParameters,
							// Issues compute commands.
							ERDGPassFlags::Compute,
							// This is deferred until Execute. May execute in parallel with other passes.
							[PassParameters, ComputeShader, NumThreadGroups](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
							{
								FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *PassParameters, NumThreadGroups);
							});

						// Execute the graph.
						GraphBuilder.Execute();						
						
						InstanceData_RT->TransientRDGTexture = nullptr;
						InstanceData_RT->TransientRDGSRV = nullptr;
						InstanceData_RT->TransientRDGUAV = nullptr;
					}					
				});	
	}
	else if (UNDIRenderTargetSimCacheData* SimCacheData = ReadContext.GetOptionalStorageObject<UNDIRenderTargetSimCacheData>())
	{
		const int FrameIndex = ReadContext.GetFrameInterp() >= 0.5f ? ReadContext.GetFrameIndexB() : ReadContext.GetFrameIndexA();
		if (!SimCacheData->IsValidFrame(FrameIndex))
		{
			return false;
		}

		FNiagaraSystemInstance* SystemInstance = ReadContext.GetSystemInstance();

		InstanceData_GT->Size   = SimCacheData->GetTextureSize(FrameIndex);
		InstanceData_GT->Format = SimCacheData->GetTextureFormat(FrameIndex);
		InstanceData_GT->Filter = TextureFilter::TF_Default;

		PerInstanceTick(InstanceData_GT, SystemInstance, 0.0f);
		PerInstanceTickPostSimulate(InstanceData_GT, SystemInstance, 0.0f);

		if (InstanceData_GT->TargetTexture && SimCacheData->HasPixelData(FrameIndex))
		{
			FNiagaraDataInterfaceProxyRenderTargetVolumeProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTargetVolumeProxy>();
			FTextureRenderTargetResource* RT_TargetTexture = InstanceData_GT->TargetTexture->GameThread_GetRenderTargetResource();
			ENQUEUE_RENDER_COMMAND(NDIRenderTargetVolumeUpdate)
			(
				[RT_Proxy, RT_InstanceID=SystemInstance->GetId(), RT_TargetTexture, RT_SimCacheData=SimCacheData, RT_FrameIndex=FrameIndex](FRHICommandListImmediate& RHICmdList)
				{
					if (FRenderTargetVolumeRWInstanceData_RenderThread* InstanceData_RT = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_InstanceID))
					{
						FUpdateTextureRegion3D UpdateRegion(FIntVector::ZeroValue, FIntVector::ZeroValue, InstanceData_RT->Size);
						FUpdateTexture3DData UpdateTexture = RHICmdList.BeginUpdateTexture3D(InstanceData_RT->RenderTarget->GetRHI(), 0, UpdateRegion);
						RT_SimCacheData->GetPixelData(RT_FrameIndex, UpdateTexture.Data, UpdateTexture.RowPitch, UpdateTexture.DepthPitch);
						RHICmdList.EndUpdateTexture3D(UpdateTexture);
					}
				}
			);
		}
	}
	return true;
}

void UNiagaraDataInterfaceRenderTargetVolume::VMSetSize(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FRenderTargetVolumeRWInstanceData_GameThread> InstData(Context);
	FNDIInputParam<int> InSizeX(Context);
	FNDIInputParam<int> InSizeY(Context);
	FNDIInputParam<int> InSizeZ(Context);
	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const int SizeX = InSizeX.GetAndAdvance();
		const int SizeY = InSizeY.GetAndAdvance();
		const int SizeZ = InSizeZ.GetAndAdvance();
		const bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1 && SizeX > 0 && SizeY > 0 && SizeZ > 0);
		OutSuccess.SetAndAdvance(bSuccess);
		if (bSuccess)
		{
			InstData->Size.X = FMath::Clamp<int>(int(float(SizeX) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxVolumeTextureDimensions);
			InstData->Size.Y = FMath::Clamp<int>(int(float(SizeY) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxVolumeTextureDimensions);
			InstData->Size.Z = FMath::Clamp<int>(int(float(SizeZ) * NiagaraDataInterfaceRenderTargetCommon::GResolutionMultiplier), 1, GMaxVolumeTextureDimensions);

			if (bInheritUserParameterSettings && InstData->RTUserParamBinding.GetValue<UTextureRenderTargetVolume>())
			{
				UE_LOGF(LogNiagara, Warning, "Overriding inherited volume render target size");
			}
		}		
	}
}

void UNiagaraDataInterfaceRenderTargetVolume::VMGetSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FRenderTargetVolumeRWInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int> OutSizeX(Context);
	FNDIOutputParam<int> OutSizeY(Context);
	FNDIOutputParam<int> OutSizeZ(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutSizeX.SetAndAdvance(InstData->Size.X);
		OutSizeY.SetAndAdvance(InstData->Size.Y);
		OutSizeZ.SetAndAdvance(InstData->Size.Z);
	}
}

void UNiagaraDataInterfaceRenderTargetVolume::VMGetNumMipLevels(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FRenderTargetVolumeRWInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int> OutMipLevels(Context);

	int32 NumMipLevels = 1;
	if (InstData->TargetTexture != nullptr)
	{
		NumMipLevels = InstData->TargetTexture->GetNumMips();
	}
	//else if (InstData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled)
	//{
	//	NumMipLevels = FMath::FloorLog2(FMath::Max(InstData->Size.X, InstData->Size.Y)) + 1;
	//}
	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		OutMipLevels.SetAndAdvance(NumMipLevels);
	}
}

void UNiagaraDataInterfaceRenderTargetVolume::VMSetFormat(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FRenderTargetVolumeRWInstanceData_GameThread> InstData(Context);
	FNDIInputParam<ETextureRenderTargetFormat> InFormat(Context);
	FNDIOutputParam<FNiagaraBool> OutSuccess(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		const ETextureRenderTargetFormat Format = InFormat.GetAndAdvance();


		const bool bSuccess = (InstData.Get() != nullptr && Context.GetNumInstances() == 1);
		OutSuccess.SetAndAdvance(bSuccess);
		if (bSuccess)
		{
			InstData->Format = GetPixelFormatFromRenderTargetFormat(Format);

			if (bInheritUserParameterSettings && InstData->RTUserParamBinding.GetValue<UTextureRenderTargetVolume>())
			{
				UE_LOGF(LogNiagara, Warning, "Overriding inherited volume render target format");
			}
		}
	}
}

bool UNiagaraDataInterfaceRenderTargetVolume::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIRenderTargetVolumeLocal;

	FRenderTargetVolumeRWInstanceData_GameThread* InstanceData = static_cast<FRenderTargetVolumeRWInstanceData_GameThread*>(PerInstanceData);

	// Pull from user parameter
	UTextureRenderTargetVolume* UserTargetTexture = InstanceData->RTUserParamBinding.GetValue<UTextureRenderTargetVolume>();
	if (UserTargetTexture && (InstanceData->TargetTexture != UserTargetTexture))
	{
		NiagaraDataInterfaceRenderTargetCommon::ReleaseRenderTarget(SystemInstance, InstanceData);
		InstanceData->TargetTexture = UserTargetTexture;
	}

	// Do we inherit the texture parameters from the user supplied texture?
	if (bInheritUserParameterSettings)
	{
		if (UserTargetTexture)
		{
			InstanceData->Size.X = UserTargetTexture->SizeX;
			InstanceData->Size.Y = UserTargetTexture->SizeY;
			InstanceData->Size.Z = UserTargetTexture->SizeZ;
			//if (UserTargetTexture->bAutoGenerateMips)
			//{
			//	// We have to take a guess at user intention
			//	InstanceData->MipMapGeneration = MipMapGeneration == ENiagaraMipMapGeneration::Disabled ? ENiagaraMipMapGeneration::PostStage : MipMapGeneration;
			//}
			//else
			//{
			//	InstanceData->MipMapGeneration = ENiagaraMipMapGeneration::Disabled;
			//}
			InstanceData->Format = InstanceData->TargetTexture->GetFormat();
			InstanceData->Filter = InstanceData->TargetTexture->Filter;
		}
		else
		{
			UE_LOGF(LogNiagara, Error, "RenderTarget UserParam is required but invalid.");
		}
	}
		
	return false;
}

bool UNiagaraDataInterfaceRenderTargetVolume::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIRenderTargetVolumeLocal;

	// Update InstanceData as the texture may have changed
	FRenderTargetVolumeRWInstanceData_GameThread* InstanceData = static_cast<FRenderTargetVolumeRWInstanceData_GameThread*>(PerInstanceData);
#if WITH_EDITORONLY_DATA
	InstanceData->bPreviewTexture = bPreviewRenderTarget;
#endif

	//-TEMP: Until we prune data interface on cook this will avoid consuming memory
	const bool bValidGpuDataInterface = NiagaraDataInterfaceRenderTargetCommon::GIgnoreCookedOut == 0 || IsUsedWithGPUScript();

	if (::IsValid(InstanceData->TargetTexture) == false)
	{
		InstanceData->TargetTexture = nullptr;
	}

	// Do we need to create a new texture?
	if (bValidGpuDataInterface && !bInheritUserParameterSettings && (InstanceData->TargetTexture == nullptr))
	{
		if (NiagaraDataInterfaceRenderTargetCommon::CreateRenderTarget<UTextureRenderTargetVolume>(SystemInstance, InstanceData) == false)
		{
			return false;
		}
		check(InstanceData->TargetTexture);
		InstanceData->TargetTexture->bCanCreateUAV = true;
		//InstanceData->TargetTexture->bAutoGenerateMips = InstanceData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled;
		InstanceData->TargetTexture->ClearColor = FLinearColor(0.0, 0, 0, 0);
		InstanceData->TargetTexture->Filter = InstanceData->Filter;
		InstanceData->TargetTexture->Init(InstanceData->Size.X, InstanceData->Size.Y, InstanceData->Size.Z, InstanceData->Format);
		InstanceData->TargetTexture->UpdateResourceImmediate(true);
	}

	// Do we need to update the existing texture?
	if (InstanceData->TargetTexture)
	{
		//const bool bAutoGenerateMips = InstanceData->MipMapGeneration != ENiagaraMipMapGeneration::Disabled;
		if ((InstanceData->TargetTexture->SizeX != InstanceData->Size.X) || (InstanceData->TargetTexture->SizeY != InstanceData->Size.Y) || (InstanceData->TargetTexture->SizeZ != InstanceData->Size.Z) ||
			(InstanceData->TargetTexture->GetFormat() != InstanceData->Format) ||
			(InstanceData->TargetTexture->Filter != InstanceData->Filter) ||
			!InstanceData->TargetTexture->bCanCreateUAV ||
			//(InstanceData->TargetTexture->bAutoGenerateMips != bAutoGenerateMips) ||
			!InstanceData->TargetTexture->GetResource())
		{
			// resize RT to match what we need for the output
			InstanceData->TargetTexture->bCanCreateUAV = true;
			//InstanceData->TargetTexture->bAutoGenerateMips = bAutoGenerateMips;
			InstanceData->TargetTexture->Filter = InstanceData->Filter;
			InstanceData->TargetTexture->Init(InstanceData->Size.X, InstanceData->Size.Y, InstanceData->Size.Z, InstanceData->Format);
			InstanceData->TargetTexture->UpdateResourceImmediate(true);
		}
	}

	//-TODO: We could avoid updating each frame if we cache the resource pointer or a serial number
	const bool bUpdateRT = true && bValidGpuDataInterface;
	if (bUpdateRT)
	{
		FNiagaraDataInterfaceProxyRenderTargetVolumeProxy* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyRenderTargetVolumeProxy>();
		FTextureRenderTargetResource* RT_TargetTexture = InstanceData->TargetTexture ? InstanceData->TargetTexture->GameThread_GetRenderTargetResource() : nullptr;
		ENQUEUE_RENDER_COMMAND(NDIRenderTargetVolumeUpdate)
		(
			[RT_Proxy, RT_InstanceID=SystemInstance->GetId(), RT_InstanceData=*InstanceData, RT_TargetTexture](FRHICommandListImmediate& RHICmdList)
			{
				FRenderTargetVolumeRWInstanceData_RenderThread* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.FindOrAdd(RT_InstanceID);
				TargetData->Size = RT_InstanceData.Size;
				TargetData->MipLevels = 1;
				//TargetData->MipMapGeneration = RT_InstanceData.MipMapGeneration;
			#if WITH_EDITORONLY_DATA
				TargetData->bPreviewTexture = RT_InstanceData.bPreviewTexture;
			#endif
				TargetData->SamplerStateRHI.SafeRelease();
				TargetData->RenderTarget.SafeRelease();
				if (RT_TargetTexture)
				{
					TargetData->MipLevels = RT_TargetTexture->GetCurrentMipCount();
					if (FTextureRenderTargetVolumeResource* ResourceVolume = RT_TargetTexture->GetTextureRenderTargetVolumeResource())
					{
						TargetData->SamplerStateRHI = ResourceVolume->SamplerStateRHI;
						TargetData->RenderTarget = CreateRenderTarget(ResourceVolume->GetTextureRHI(), TEXT("NiagaraRenderTargetVolume"));
					}
				}
			#if STATS
				TargetData->UpdateMemoryStats();
			#endif
			}
		);
	}

	return false;
}

void FNiagaraDataInterfaceProxyRenderTargetVolumeProxy::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	FRenderTargetVolumeRWInstanceData_RenderThread* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
	if (ProxyData == nullptr)
	{
		return;
	}

	// We only need to transfer this frame if it was written to.
	// If also read then we need to notify that the texture is important for the simulation
	// We also assume the texture is important for rendering, without discovering renderer bindings we don't really know
	if (ProxyData->bWroteThisFrame)
	{
		Context.GetComputeDispatchInterface().MultiGPUResourceModified(Context.GetGraphBuilder(), ProxyData->RenderTarget->GetRHI(), ProxyData->bReadThisFrame, true);
	}

#if NIAGARA_COMPUTEDEBUG_ENABLED && WITH_EDITORONLY_DATA
	if (ProxyData->bPreviewTexture && ProxyData->TransientRDGTexture)
	{
		FNiagaraGpuComputeDebugInterface GpuComputeDebugInterface = Context.GetComputeDispatchInterface().GetGpuComputeDebugInterface();
		GpuComputeDebugInterface.AddTexture(Context.GetGraphBuilder(), Context.GetSystemInstanceID(), GetSourceDIName(), ProxyData->TransientRDGTexture);
	}
#endif

	// Clean up our temporary tracking
	ProxyData->bReadThisFrame = false;
	ProxyData->bWroteThisFrame = false;

	ProxyData->TransientRDGTexture = nullptr;
	ProxyData->TransientRDGSRV = nullptr;
	ProxyData->TransientRDGUAV = nullptr;
}

void FNiagaraDataInterfaceProxyRenderTargetVolumeProxy::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context)
{
	if ( const FRenderTargetVolumeRWInstanceData_RenderThread* TargetData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()) )
	{
		Context.SetDirect(TargetData->Size);
	}
}

#undef LOCTEXT_NAMESPACE

