// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceRigidMeshCollisionQuery.h"
#include "Algo/Accumulate.h"
#include "Algo/ForEach.h"
#include "Algo/RemoveIf.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Containers/StridedView.h"
#include "FXRenderingUtils.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/OverlapResult.h"
#include "Engine/SkeletalMesh.h"
#include "EngineUtils.h"

#include "Misc/LargeWorldRenderPosition.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraRenderer.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSimStageData.h"
#include "NiagaraSystemImpl.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemSimulation.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PrimitiveSceneInfo.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "RenderGraphBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceRigidMeshCollisionQuery)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceRigidMeshCollisionQuery"
DEFINE_LOG_CATEGORY_STATIC(LogRigidMeshCollision, Log, All);

// outstanding/known issues:
// -when actors change and the arrays are fully updated we'll experience a frame of 0 velocities
//		-potentially we could keep track of ranges of rigid bodies for given actors and then smartly reassign
//		the previous frame's transforms
// -could add a vM function for setting the maximum number of primitives


namespace NDIRigidMeshCollisionLocal
{

struct FNiagaraRigidMeshCollisionDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		LargeWorldCoordinates = 1,
		SetMaxDistance = 2,
		FindActorRotation = 3,
		MaxEncodedDistance = 4,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

//------------------------------------------------------------------------------------------------------------

BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
	SHADER_PARAMETER(uint32,				MaxTransforms)
	SHADER_PARAMETER(uint32,				CurrentOffset)
	SHADER_PARAMETER(uint32,				PreviousOffset)
	SHADER_PARAMETER(FUintVector4,			ElementOffsets)
	SHADER_PARAMETER_SRV(Buffer<float4>,	WorldTransformBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>,	InverseTransformBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>,	ElementExtentBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>,	MeshScaleBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint32>,	PhysicsTypeBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint32>,	DFIndexBuffer)
	SHADER_PARAMETER(FVector3f,				CombinedBBoxWorldMin)
	SHADER_PARAMETER(FVector3f,				CombinedBBoxWorldMax)
END_SHADER_PARAMETER_STRUCT()

static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceRigidMeshCollisionQuery.ush");

//------------------------------------------------------------------------------------------------------------

static const FName FindActorsName(TEXT("FindActors"));

//------------------------------------------------------------------------------------------------------------

static const FName GetNumBoxesName(TEXT("GetNumBoxes"));
static const FName GetNumSpheresName(TEXT("GetNumSpheres"));
static const FName GetNumCapsulesName(TEXT("GetNumCapsules"));
static const FName GetNumElementsName(TEXT("GetNumElements"));
static const FName GetBoxElementsStartIndexName(TEXT("GetBoxElementsStartIndex"));
static const FName GetSphereElementsStartIndexName(TEXT("GetSphereElementsStartIndex"));
static const FName GetCapsuleElementsStartIndexName(TEXT("GetCapsuleElementsStartIndex"));
static const FName GetSphereRadiusName(TEXT("GetSphereRadius"));
static const FName GetCapsuleSizeName(TEXT("GetCapsuleSize"));
static const FName GetBoxSizeName(TEXT("GetBoxSize"));

//------------------------------------------------------------------------------------------------------------
static const FName IsWorldPositionInsideCombinedBoundsName(TEXT("IsWorldPositionInsideCombinedBounds"));
static const FName GetClosestElementName(TEXT("GetClosestElement"));
static const FName GetElementPointName(TEXT("GetElementPoint"));
static const FName GetElementTransformName(TEXT("GetElementTransform"));
static const FName GetElementInverseTransformName(TEXT("GetElementInverseTransform"));
static const FName GetElementPointMeshDistanceFieldNoNormalName(TEXT("GetElementPointMeshDistanceFieldNoNormal"));
static const FName GetElementDistanceName(TEXT("GetElementDistance"));
static const FName GetClosestPointName(TEXT("GetClosestPoint"));
static const FName GetClosestPointSimpleName(TEXT("GetClosestPointSimple"));
static const FName GetClosestDistanceName(TEXT("GetClosestDistance"));
static const FName GetClosestPointMeshDistanceFieldName(TEXT("GetClosestPointMeshDistanceField"));
static const FName GetClosestPointMeshDistanceFieldAccurateName(TEXT("GetClosestPointMeshDistanceFieldAccurate"));
static const FName GetClosestPointMeshDistanceFieldNoNormalName(TEXT("GetClosestPointMeshDistanceFieldNoNormal"));
static const FName GetMaxEncodedDistanceMeshDistanceFieldName(TEXT("GetMaxEncodedDistanceMeshDistanceField"));

static const FText OverlapOriginDescription = IF_WITH_EDITORONLY_DATA(
	LOCTEXT("RigidBodyOverlapOriginDescription", "The center point, in world space, where the overlap trace will be performed."),
	FText()
);

static const FText OverlapRotationDescription = IF_WITH_EDITORONLY_DATA(
	LOCTEXT("RigidBodyOVerlapRotationDesciption", "The orientation of the box to be used for the overlap trace."),
	FText()
);

static const FText OverlapExtentDescription = IF_WITH_EDITORONLY_DATA(
	LOCTEXT("RigidBodyOverlapExtentDescription", "The full extent (max-min), in world space, of the overlap trace."),
	FText()
);

static const FText TraceChannelDescription = IF_WITH_EDITORONLY_DATA(
	LOCTEXT("RigidBodyTraceChannelDescription", "The trace channel to collide against. Trace channels can be configured in the project settings."),
	FText()
);

static const FText SkipOverlapDescription = IF_WITH_EDITORONLY_DATA(
	LOCTEXT("RigidBodySkipTraceDescription", "If enabled, the overlap test will not be performed."),
	FText()
);

//------------------------------------------------------------------------------------------------------------

bool IsMeshDistanceFieldEnabled()
{
	static const auto* CVarGenerateMeshDistanceFields = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GenerateMeshDistanceFields"));
	return CVarGenerateMeshDistanceFields != nullptr && CVarGenerateMeshDistanceFields->GetValueOnAnyThread() > 0;
}

//------------------------------------------------------------------------------------------------------------

template<typename BufferType, EPixelFormat PixelFormat>
void CreateInternalBuffer(FRHICommandListBase& RHICmdList, FReadBuffer& OutputBuffer, uint32 ElementCount)
{
	if (ElementCount > 0)
	{
		OutputBuffer.Initialize(RHICmdList, TEXT("FNDIRigidMeshCollisionBuffer"), sizeof(BufferType), ElementCount, PixelFormat, BUF_Static);
	}
}

template<typename BufferType, EPixelFormat PixelFormat>
void UpdateInternalBuffer(FRHICommandListBase& RHICmdList, const TArray<BufferType>& InputData, FReadBuffer& OutputBuffer)
{
	uint32 ElementCount = InputData.Num();
	if (ElementCount > 0 && OutputBuffer.Buffer.IsValid())
	{
		const uint32 BufferBytes = sizeof(BufferType) * ElementCount;

		void* OutputData = RHICmdList.LockBuffer(OutputBuffer.Buffer, 0, BufferBytes, RLM_WriteOnly);

		FMemory::Memcpy(OutputData, InputData.GetData(), BufferBytes);
		RHICmdList.UnlockBuffer(OutputBuffer.Buffer);
	}
}

void FillCurrentTransforms(const FTransform& ElementTransform, uint32 ElementCount, TArray<FVector4f>& OutCurrentTransform, TArray<FVector4f>& OutCurrentInverse)
{
	// LWC_TODO: precision loss
	const uint32 ElementOffset = 3 * ElementCount;
	const FMatrix44f ElementMatrix = FMatrix44f(ElementTransform.ToMatrixWithScale());
	const FMatrix44f ElementInverse = ElementMatrix.Inverse();

	ElementMatrix.To3x4MatrixTranspose(&OutCurrentTransform[ElementOffset].X);
	ElementInverse.To3x4MatrixTranspose(&OutCurrentInverse[ElementOffset].X);
}

template<typename TComponentType, typename TComponentFilterPredicate>
static void GenerateComponentList(
	TConstArrayView<AActor*> Actors,
	TConstArrayView<FName> ComponentTags,
	TComponentFilterPredicate FilterPredicate,
	TInlineComponentArray<TComponentType*>& Components)
{
	for (const AActor* Actor : Actors)
	{
		if (Actor)
		{
			for (UActorComponent* ActorComponent : Actor->GetComponents())
			{
				if (TComponentType* TypedComponent = Cast<TComponentType>(ActorComponent))
				{
					if (IsValid(TypedComponent) && FilterPredicate(TypedComponent))
					{
						if (ComponentTags.IsEmpty() || ComponentTags.ContainsByPredicate([&](const FName& Tag) { return Tag == NAME_None || TypedComponent->ComponentHasTag(Tag); }))
						{
							Components.Add(TypedComponent);
						}
					}
				}
			}
		}
	}
}

template<typename TComponentType>
void CollectComponents(TConstArrayView<AActor*> Actors, TConstArrayView<FName> ComponentTags, TInlineComponentArray<TComponentType*>& Components);

template<typename TComponentType, typename TBodySetupPredicate>
void ForEachBodySetup(TComponentType* Component, TBodySetupPredicate Predicate);

/// Begin UkeletalMeshComponent

template<>
void CollectComponents(TConstArrayView<AActor*> Actors, TConstArrayView<FName> ComponentTags, TInlineComponentArray<USkeletalMeshComponent*>& Components)
{
	auto SkeletalFilterPredicate = [&](USkeletalMeshComponent* Component)
	{
		if (UPhysicsAsset* PhysicsAsset = Component->GetPhysicsAsset())
		{
			USkeletalMesh* MeshAsset = Component->GetSkeletalMeshAsset() ? Component->GetSkeletalMeshAsset() : PhysicsAsset->GetPreviewMesh();
			if (!MeshAsset || !MeshAsset->GetRefSkeleton().GetNum())
			{
				return false;
			}

			return true;
		}

		return false;
	};

	GenerateComponentList(Actors, ComponentTags, SkeletalFilterPredicate, Components);
}

template<typename TBodySetupPredicate>
void ForEachBodySetup(USkeletalMeshComponent* Component, TBodySetupPredicate Predicate)
{
	if (UPhysicsAsset* PhysicsAsset = Component->GetPhysicsAsset())
	{
		USkeletalMesh* SkeletalMesh = Component->GetSkeletalMeshAsset() ? Component->GetSkeletalMeshAsset() : PhysicsAsset->GetPreviewMesh();
		const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();

		for (const UBodySetup* BodySetup : PhysicsAsset->SkeletalBodySetups)
		{
			const FName BoneName = BodySetup->BoneName;
			const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
			if (BoneIndex != INDEX_NONE && BoneIndex < RefSkeleton.GetNum())
			{
				Predicate(Component, BodySetup);
			}
		}
	}
}

uint32 MeshComponentHash(const USkeletalMeshComponent* SkeletalMeshComponent)
{
	uint32 HashResult = GetTypeHash(SkeletalMeshComponent);
	if (SkeletalMeshComponent)
	{
		HashResult = HashCombine(HashResult, GetTypeHash(SkeletalMeshComponent->GetSkeletalMeshAsset()));
	}
	return HashResult;
}

/// End UkeletalMeshComponent

/// Begin UStaticMeshComponent

template<>
void CollectComponents(TConstArrayView<AActor*> Actors, TConstArrayView<FName> ComponentTags, TInlineComponentArray<UStaticMeshComponent*>& Components)
{
	auto StaticFilterPredicate = [&](UStaticMeshComponent* Component)
	{
		return Component->GetBodySetup() != nullptr;
	};

	GenerateComponentList(Actors, ComponentTags, StaticFilterPredicate, Components);
}

template<typename TBodySetupPredicate>
void ForEachBodySetup(UStaticMeshComponent* Component, TBodySetupPredicate Predicate)
{
	Predicate(Component, Component->GetBodySetup());
}

uint32 MeshComponentHash(const UStaticMeshComponent* StaticMeshComponent)
{
	uint32 HashResult = GetTypeHash(StaticMeshComponent);
	if (StaticMeshComponent)
	{
		HashResult = HashCombine(HashResult, GetTypeHash(StaticMeshComponent->GetStaticMesh()));
	}
	return HashResult;
}

/// End UStaticMeshComponent

bool FilterShape(const FKShapeElem& Elem)
{
	// CollisionEnabledHasPhysics(Elem.GetCollisionEnabled());
	return Elem.GetCollisionEnabled() != ECollisionEnabled::NoCollision;
}

template<typename TComponentType>
void CountCollisionPrimitives(TConstArrayView<TComponentType*> Components, TArray<FNDIRigidMeshCollisionData::FComponentBodyCount>& PerComponentCounts, uint32& TotalBoxCount, uint32& TotalSphereCount, uint32& TotalCapsuleCount)
{
	PerComponentCounts.Reserve(Components.Num());

	for (TComponentType* Component : Components)
	{
		FNDIRigidMeshCollisionData::FComponentBodyCount& BodyCount = PerComponentCounts.AddDefaulted_GetRef();
		BodyCount.ComponentHash = MeshComponentHash(Component);

		ForEachBodySetup(Component, [&](TComponentType* Component, const UBodySetup* BodySetup)
		{
			for (const FKConvexElem& ConvexElem : BodySetup->AggGeom.ConvexElems)
			{
				if (FilterShape(ConvexElem))
				{
					++BodyCount.BoxCount;
				}
			}
			for (const FKBoxElem& BoxElem : BodySetup->AggGeom.BoxElems)
			{
				if (FilterShape(BoxElem))
				{
					++BodyCount.BoxCount;
				}
			}
			for (const FKSphereElem& SphereElem : BodySetup->AggGeom.SphereElems)
			{
				if (FilterShape(SphereElem))
				{
					++BodyCount.SphereCount;
				}
			}
			for (const FKSphylElem& CapsuleElem : BodySetup->AggGeom.SphylElems)
			{
				if (FilterShape(CapsuleElem))
				{
					++BodyCount.CapsuleCount;
				}
			}

			// if we have no supported bodies associated with a mesh, but we want to collide with it, add a single box
			if (BodySetup->AggGeom.ConvexElems.Num() == 0 &&
				BodySetup->AggGeom.BoxElems.Num() == 0 &&
				BodySetup->AggGeom.SphereElems.Num() == 0 &&
				BodySetup->AggGeom.SphylElems.Num() == 0)
			{
				++BodyCount.BoxCount;
			}
		});

		TotalBoxCount += BodyCount.BoxCount;
		TotalSphereCount += BodyCount.SphereCount;
		TotalCapsuleCount += BodyCount.CapsuleCount;
	}
}

template<typename TComponentType>
FTransform CreateElementTransform(const TComponentType* Component, const UBodySetup* BodySetup)
{
	return Component->GetComponentTransform();
}

template<>
FTransform CreateElementTransform<USkeletalMeshComponent>(const USkeletalMeshComponent* Component, const UBodySetup* BodySetup)
{
	if (USkeletalMesh* SkeletalMesh = Component->GetSkeletalMeshAsset())
	{
		const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
		const int32 BoneCount = RefSkeleton.GetNum();

		if (BoneCount > 0)
		{
			const FName BoneName = BodySetup->BoneName;
			const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
			if (BoneIndex != INDEX_NONE && BoneIndex < BoneCount)
			{
				return Component->GetBoneTransform(BoneIndex);
			}
		}
	}

	return Component->GetComponentTransform();
}

template<typename TComponentType, bool InitializeStatics>
void UpdateAssetArrays(TConstArrayView<TComponentType*> Components, const FVector& LWCTile, FNDIRigidMeshCollisionArrays* OutAssetArrays, uint32& BoxIndex, uint32& SphereIndex, uint32& CapsuleIndex, FBoxSphereBounds::Builder &CombinedWorldBoundsBuilder)
{
	auto UpdateAssetPredicate = [&](TComponentType* Component, const UBodySetup* BodySetup)
	{
		FTransform MeshTransform = CreateElementTransform(Component, BodySetup);
		MeshTransform.AddToTranslation(LWCTile * -FLargeWorldRenderScalar::GetTileSize());

		const FVector3f CurrMeshScale(MeshTransform.GetScale3D());

		const int32 ComponentIdIndex = OutAssetArrays->UniqueCompnentId.AddUnique(Component->GetPrimitiveSceneId());

		for (const FKConvexElem& ConvexElem : BodySetup->AggGeom.ConvexElems)
		{
			if (FilterShape(ConvexElem))
			{
				FBox BBox = ConvexElem.ElemBox;

				if (InitializeStatics)
				{
					FVector3f Extent = FVector3f(BBox.Max - BBox.Min);
					OutAssetArrays->ElementExtent[BoxIndex] = FVector4f(Extent.X, Extent.Y, Extent.Z, 0);					
					OutAssetArrays->MeshScale[BoxIndex] = FVector4f(CurrMeshScale.X, CurrMeshScale.Y, CurrMeshScale.Z, 0);
					OutAssetArrays->PhysicsType[BoxIndex] = (ConvexElem.GetCollisionEnabled() == ECollisionEnabled::QueryAndPhysics);
					OutAssetArrays->ComponentIdIndex[BoxIndex] = ComponentIdIndex;
				}

				FVector Center = (BBox.Max + BBox.Min) * .5;
				const FTransform ElementTransform = FTransform(Center) * MeshTransform;
				FillCurrentTransforms(ElementTransform, BoxIndex, OutAssetArrays->CurrentTransform, OutAssetArrays->CurrentInverse);
				++BoxIndex;
			}
		}
		for (const FKBoxElem& BoxElem : BodySetup->AggGeom.BoxElems)
		{
			if (FilterShape(BoxElem))
			{
				if (InitializeStatics)
				{
					OutAssetArrays->ElementExtent[BoxIndex] = FVector4f(BoxElem.X, BoxElem.Y, BoxElem.Z, 0);
					OutAssetArrays->MeshScale[BoxIndex] = FVector4f(CurrMeshScale.X, CurrMeshScale.Y, CurrMeshScale.Z, 0);
					OutAssetArrays->PhysicsType[BoxIndex] = (BoxElem.GetCollisionEnabled() == ECollisionEnabled::QueryAndPhysics);
					OutAssetArrays->ComponentIdIndex[BoxIndex] = ComponentIdIndex;
				}

				const FTransform ElementTransform = FTransform(BoxElem.Rotation, BoxElem.Center) * MeshTransform;
				FillCurrentTransforms(ElementTransform, BoxIndex, OutAssetArrays->CurrentTransform, OutAssetArrays->CurrentInverse);
				++BoxIndex;
			}
		}

		for (const FKSphereElem& SphereElem : BodySetup->AggGeom.SphereElems)
		{
			if (FilterShape(SphereElem))
			{
				if (InitializeStatics)
				{
					OutAssetArrays->ElementExtent[SphereIndex] = FVector4f(SphereElem.Radius, 0, 0, 0);
					OutAssetArrays->MeshScale[SphereIndex] = FVector4f(CurrMeshScale.X, CurrMeshScale.Y, CurrMeshScale.Z, 0);
					OutAssetArrays->PhysicsType[SphereIndex] = (SphereElem.GetCollisionEnabled() == ECollisionEnabled::QueryAndPhysics);
					OutAssetArrays->ComponentIdIndex[SphereIndex] = ComponentIdIndex;
				}

				const FTransform ElementTransform = FTransform(SphereElem.Center) * MeshTransform;
				FillCurrentTransforms(ElementTransform, SphereIndex, OutAssetArrays->CurrentTransform, OutAssetArrays->CurrentInverse);
				++SphereIndex;
			}
		}

		for (const FKSphylElem& CapsuleElem : BodySetup->AggGeom.SphylElems)
		{
			if (FilterShape(CapsuleElem))
			{
				if (InitializeStatics)
				{
					OutAssetArrays->ElementExtent[CapsuleIndex] = FVector4f(CapsuleElem.Radius, CapsuleElem.Length, 0, 0);
					OutAssetArrays->MeshScale[CapsuleIndex] = FVector4f(CurrMeshScale.X, CurrMeshScale.Y, CurrMeshScale.Z, 0);
					OutAssetArrays->PhysicsType[CapsuleIndex] = (CapsuleElem.GetCollisionEnabled() == ECollisionEnabled::QueryAndPhysics);
					OutAssetArrays->ComponentIdIndex[CapsuleIndex] = ComponentIdIndex;
				}

				const FTransform ElementTransform = FTransform(CapsuleElem.Rotation, CapsuleElem.Center) * MeshTransform;
				FillCurrentTransforms(ElementTransform, CapsuleIndex, OutAssetArrays->CurrentTransform, OutAssetArrays->CurrentInverse);
				++CapsuleIndex;
			}
		}

		// if we have no supported bodies associated with a mesh, but we want to collide with it, add a single box
		if (BodySetup->AggGeom.ConvexElems.Num() == 0 &&
			BodySetup->AggGeom.BoxElems.Num() == 0 &&
			BodySetup->AggGeom.SphereElems.Num() == 0 &&
			BodySetup->AggGeom.SphylElems.Num() == 0)
		{
			FVector Extent;
			FVector Center;
		
			FBoxSphereBounds Bounds = static_cast< USceneComponent* >(Component)->GetLocalBounds();
			Extent = Bounds.BoxExtent;
			Center = Bounds.Origin;
			
			if (InitializeStatics)
			{			
				// local bounds extent is half the world extents of the bounding box in local space
				Extent *= 2.0;

				OutAssetArrays->ElementExtent[BoxIndex] = FVector4f(float(Extent.X), float(Extent.Y), float(Extent.Z), 0);
				OutAssetArrays->MeshScale[BoxIndex] = FVector4f(CurrMeshScale.X, CurrMeshScale.Y, CurrMeshScale.Z, 0);
				OutAssetArrays->PhysicsType[BoxIndex] = true;
				OutAssetArrays->ComponentIdIndex[BoxIndex] = ComponentIdIndex;
			}
			
			const FTransform ElementTransform = FTransform(Center) * MeshTransform;
			FillCurrentTransforms(ElementTransform, BoxIndex, OutAssetArrays->CurrentTransform, OutAssetArrays->CurrentInverse);
			++BoxIndex;
		}
	};
	
	for (TComponentType* Component : Components)
	{				
		FBoxSphereBounds LocalBounds = static_cast<USceneComponent*>(Component)->GetLocalBounds();
		FBoxSphereBounds WorldBounds = LocalBounds.TransformBy(Component->GetComponentTransform());
		CombinedWorldBoundsBuilder += WorldBounds;		

		ForEachBodySetup(Component, UpdateAssetPredicate);
	}
}

template<bool FirstStage>
void RemapPreviousTransforms(
	TConstArrayView<FNDIRigidMeshCollisionData::FComponentBodyCount> PreviousCounts,
	TConstArrayView<FNDIRigidMeshCollisionData::FComponentBodyCount> CurrentCounts,
	const FNDIRigidMeshCollisionElementOffset& PreviousOffsets,
	const FNDIRigidMeshCollisionElementOffset& CurrentOffsets,
	FNDIRigidMeshCollisionArrays* OutAssetArrays)
{
	auto CopyComponent = []
		(const TArray<FVector4f>& Src, TArray<FVector4f>& Dst, 
			const uint32 SrcBoxCount, const uint32 SrcSphereCount, const uint32 SrcCapsuleCount,
			const uint32 DstBoxCount, const uint32 DstSphereCount, const uint32 DstCapsuleCount,
			const FNDIRigidMeshCollisionElementOffset& SrcOffsets, const FNDIRigidMeshCollisionElementOffset& DstOffsets,
			const FNDIRigidMeshCollisionData::FComponentBodyCount& BodyCounts) -> void
	{
		const uint32 SrcBoxStartIndex = 3 * (SrcOffsets.BoxOffset + SrcBoxCount);
		const uint32 DstBoxStartIndex = 3 * (DstOffsets.BoxOffset + DstBoxCount);
		for (uint32 ElementIt = 0; ElementIt < 3 * BodyCounts.BoxCount; ++ElementIt)
		{
			Dst[DstBoxStartIndex + ElementIt] = Src[SrcBoxStartIndex + ElementIt];
		}

		const uint32 SrcSphereStartIndex = 3 * (SrcOffsets.SphereOffset + SrcSphereCount);
		const uint32 DstSphereStartIndex = 3 * (DstOffsets.SphereOffset + DstSphereCount);
		for (uint32 ElementIt = 0; ElementIt < 3 * BodyCounts.SphereCount; ++ElementIt)
		{
			Dst[DstSphereStartIndex + ElementIt] = Src[SrcSphereStartIndex + ElementIt];
		}

		const uint32 SrcCapsuleStartIndex = 3 * (SrcOffsets.CapsuleOffset + SrcCapsuleCount);
		const uint32 DstCapsuleStartIndex = 3 * (DstOffsets.CapsuleOffset + DstCapsuleCount);
		for (uint32 ElementIt = 0; ElementIt < 3 * BodyCounts.CapsuleCount; ++ElementIt)
		{
			Dst[DstCapsuleStartIndex + ElementIt] = Src[SrcCapsuleStartIndex + ElementIt];
		}
	};

	uint32 CurrentBoxCount = 0;
	uint32 CurrentSphereCount = 0;
	uint32 CurrentCapsuleCount = 0;

	for (const FNDIRigidMeshCollisionData::FComponentBodyCount& CurrentCount : CurrentCounts)
	{
		bool bPreviousValuesCopied = false;

		uint32 PreviousBoxCount = 0;
		uint32 PreviousSphereCount = 0;
		uint32 PreviousCapsuleCount = 0;

		int32 PreviousIndex = 0;
		while (PreviousCounts.IsValidIndex(PreviousIndex) && !bPreviousValuesCopied)
		{
			const FNDIRigidMeshCollisionData::FComponentBodyCount& PreviousCount = PreviousCounts[PreviousIndex];

			if ((PreviousCount.ComponentHash == CurrentCount.ComponentHash)
				&& (PreviousCount.BoxCount == CurrentCount.BoxCount)
				&& (PreviousCount.SphereCount == CurrentCount.SphereCount)
				&& (PreviousCount.CapsuleCount == CurrentCount.CapsuleCount))
			{
				bPreviousValuesCopied = true;

				// For the body in the current tick, we found a match in the previous tick transforms.  Copy the previous tick transforms (stored in CurrentTransform)
				// over to PreviousTransform.  Note the indices for the transforms might differ between current and previous as bodies enter/leave the arrays
				if (FirstStage)
				{
					const TArray<FVector4f>& Src = OutAssetArrays->CurrentTransform;
					TArray<FVector4f>& Dst = OutAssetArrays->PreviousTransform;
					const TArray<FVector4f>& SrcInverse = OutAssetArrays->CurrentInverse;
					TArray<FVector4f>& DstInverse = OutAssetArrays->PreviousInverse;

					CopyComponent(Src, Dst,
						PreviousBoxCount, PreviousSphereCount, PreviousCapsuleCount,
						CurrentBoxCount, CurrentSphereCount, CurrentCapsuleCount,
						PreviousOffsets, CurrentOffsets,
						CurrentCount);

					CopyComponent(SrcInverse, DstInverse,
						PreviousBoxCount, PreviousSphereCount, PreviousCapsuleCount,
						CurrentBoxCount, CurrentSphereCount, CurrentCapsuleCount,
						PreviousOffsets, CurrentOffsets,
						CurrentCount);
				}

				break;
			}

			++PreviousIndex;
			PreviousBoxCount += PreviousCount.BoxCount;
			PreviousSphereCount += PreviousCount.SphereCount;
			PreviousCapsuleCount += PreviousCount.CapsuleCount;
		}

		// body is found in current tick but not in previous tick, so copy current transforms to previous buffer using indices and offsets from the current tick
		if (!bPreviousValuesCopied && !FirstStage)
		{
			CopyComponent(OutAssetArrays->CurrentTransform, OutAssetArrays->PreviousTransform,
				CurrentBoxCount, CurrentSphereCount, CurrentCapsuleCount,
				CurrentBoxCount, CurrentSphereCount, CurrentCapsuleCount,
				CurrentOffsets, CurrentOffsets,
				CurrentCount);

			CopyComponent(OutAssetArrays->CurrentInverse, OutAssetArrays->PreviousInverse,
				CurrentBoxCount, CurrentSphereCount, CurrentCapsuleCount,
				CurrentBoxCount, CurrentSphereCount, CurrentCapsuleCount,
				CurrentOffsets, CurrentOffsets,
				CurrentCount);
		}

		CurrentBoxCount += CurrentCount.BoxCount;
		CurrentSphereCount += CurrentCount.SphereCount;
		CurrentCapsuleCount += CurrentCount.CapsuleCount;
	}
}

bool UpdateInternalArrays(
	const FNiagaraSystemInstance* SystemInstance,
	TConstArrayView<UStaticMeshComponent*> StaticMeshView,
	TConstArrayView<USkeletalMeshComponent*> SkeletalMeshView,
	FVector LWCTile,
	bool bFullUpdate,
	TArray<FNDIRigidMeshCollisionData::FComponentBodyCount>& BodyCounts,
	FNDIRigidMeshCollisionArrays* OutAssetArrays,
	uint32& TotalBoxCount,
	uint32& TotalSphereCount,
	uint32& TotalCapsuleCount)
{
	TotalBoxCount = 0;
	TotalSphereCount = 0;
	TotalCapsuleCount = 0;

	if (OutAssetArrays == nullptr || OutAssetArrays->ElementOffsets.NumElements >= OutAssetArrays->MaxPrimitives)
	{
		return false;
	}

	// when we are in game and we don't need to worry about bodies changing for a given mesh component we can try to optimize
	// the update by just targeting the dynamic elements (transforms) and using the original values as the previous run
#if !WITH_EDITOR
	if (!bFullUpdate)
	{
		// if we're updating, then copy over last frame's transforms before we generate new ones
		Swap(OutAssetArrays->PreviousTransform, OutAssetArrays->CurrentTransform);
		Swap(OutAssetArrays->PreviousInverse, OutAssetArrays->CurrentInverse);

		uint32 BoxIndex = OutAssetArrays->ElementOffsets.BoxOffset;
		uint32 SphereIndex = OutAssetArrays->ElementOffsets.SphereOffset;
		uint32 CapsuleIndex = OutAssetArrays->ElementOffsets.CapsuleOffset;
		
		OutAssetArrays->CombinedBBoxWorldMin = FVector3f(TNumericLimits<float>::Max());
		OutAssetArrays->CombinedBBoxWorldMax = FVector3f(TNumericLimits<float>::Lowest());

		FBoxSphereBounds::Builder CombinedWorldBoundsBuilder;

		UpdateAssetArrays<UStaticMeshComponent, true>(StaticMeshView, LWCTile, OutAssetArrays, BoxIndex, SphereIndex, CapsuleIndex, CombinedWorldBoundsBuilder);
		UpdateAssetArrays<USkeletalMeshComponent, true>(SkeletalMeshView, LWCTile, OutAssetArrays, BoxIndex, SphereIndex, CapsuleIndex, CombinedWorldBoundsBuilder);

		const FBoxSphereBounds CombinedWorldBounds(CombinedWorldBoundsBuilder);
		const FBox CombinedWorldBoundingBox = CombinedWorldBounds.GetBox();

		OutAssetArrays->CombinedBBoxWorldMin = (FVector3f)CombinedWorldBoundingBox.Min;
		OutAssetArrays->CombinedBBoxWorldMax = (FVector3f)CombinedWorldBoundingBox.Max;

		FNiagaraLWCConverter LWCConverter = SystemInstance->GetLWCConverter();
		OutAssetArrays->CombinedBBoxWorldMin = LWCConverter.ConvertWorldToSimulationVector(CombinedWorldBoundingBox.Min);
		OutAssetArrays->CombinedBBoxWorldMax = LWCConverter.ConvertWorldToSimulationVector(CombinedWorldBoundingBox.Max);

		return true;
	}
#endif

	TArray<FNDIRigidMeshCollisionData::FComponentBodyCount> CurrentBodyCounts;

	CountCollisionPrimitives(StaticMeshView, CurrentBodyCounts, TotalBoxCount, TotalSphereCount, TotalCapsuleCount);
	CountCollisionPrimitives(SkeletalMeshView, CurrentBodyCounts, TotalBoxCount, TotalSphereCount, TotalCapsuleCount);

	if ((TotalBoxCount + TotalSphereCount + TotalCapsuleCount) >= OutAssetArrays->MaxPrimitives)
	{
		return false;
	}

	FNDIRigidMeshCollisionElementOffset PreviousElementOffsets = OutAssetArrays->ElementOffsets;

	OutAssetArrays->ElementOffsets.BoxOffset = 0;
	OutAssetArrays->ElementOffsets.SphereOffset = OutAssetArrays->ElementOffsets.BoxOffset + TotalBoxCount;
	OutAssetArrays->ElementOffsets.CapsuleOffset = OutAssetArrays->ElementOffsets.SphereOffset + TotalSphereCount;
	OutAssetArrays->ElementOffsets.NumElements = OutAssetArrays->ElementOffsets.CapsuleOffset + TotalCapsuleCount;

	uint32 BoxIndex = OutAssetArrays->ElementOffsets.BoxOffset;
	uint32 SphereIndex = OutAssetArrays->ElementOffsets.SphereOffset;
	uint32 CapsuleIndex = OutAssetArrays->ElementOffsets.CapsuleOffset;

	// where possible PreviousTransform & PreviousInverse should be pulled from the current values of CurrentTransform &
	// CurrentInverse based on the remapped entries
	RemapPreviousTransforms<true>(BodyCounts, CurrentBodyCounts, PreviousElementOffsets, OutAssetArrays->ElementOffsets, OutAssetArrays);

	// Initialize world bounds
	OutAssetArrays->CombinedBBoxWorldMin = FVector3f(TNumericLimits<float>::Max());
	OutAssetArrays->CombinedBBoxWorldMax = FVector3f(TNumericLimits<float>::Lowest());

	FBoxSphereBounds::Builder CombinedWorldBoundsBuilder;

	UpdateAssetArrays<UStaticMeshComponent, true>(StaticMeshView, LWCTile, OutAssetArrays, BoxIndex, SphereIndex, CapsuleIndex, CombinedWorldBoundsBuilder);
	UpdateAssetArrays<USkeletalMeshComponent, true>(SkeletalMeshView, LWCTile, OutAssetArrays, BoxIndex, SphereIndex, CapsuleIndex, CombinedWorldBoundsBuilder);

	const FBoxSphereBounds CombinedWorldBounds(CombinedWorldBoundsBuilder);
	const FBox CombinedWorldBoundingBox = CombinedWorldBounds.GetBox();

	OutAssetArrays->CombinedBBoxWorldMin = (FVector3f) CombinedWorldBoundingBox.Min;
	OutAssetArrays->CombinedBBoxWorldMax = (FVector3f) CombinedWorldBoundingBox.Max;

	FNiagaraLWCConverter LWCConverter = SystemInstance->GetLWCConverter();
	OutAssetArrays->CombinedBBoxWorldMin = LWCConverter.ConvertWorldToSimulationVector(CombinedWorldBoundingBox.Min);
	OutAssetArrays->CombinedBBoxWorldMax = LWCConverter.ConvertWorldToSimulationVector(CombinedWorldBoundingBox.Max);
			
	RemapPreviousTransforms<false>(BodyCounts, CurrentBodyCounts, PreviousElementOffsets, OutAssetArrays->ElementOffsets, OutAssetArrays);

	BodyCounts = MoveTemp(CurrentBodyCounts);

	return true;
}

static bool SystemHasFindActorsFunction(UNiagaraSystem* System)
{
	bool FindActorsFunctionFound = false;

	if (System)
	{
		System->ForEachScript([&](UNiagaraScript* NiagaraScript)
		{
			const FNiagaraVMExecutableData& ScriptExecutableData = NiagaraScript->GetVMExecutableData();
			if (ScriptExecutableData.IsValid())
			{
				for (const FVMExternalFunctionBindingInfo& FunctionBinding : ScriptExecutableData.CalledVMExternalFunctions)
				{
					if (FunctionBinding.Name == FindActorsName)
					{
						FindActorsFunctionFound = true;
						break;
					}
				}
			}
		});
	}

	return FindActorsFunctionFound;
}

bool WeakActorPtrLess(const TWeakObjectPtr<AActor>& Lhs, const TWeakObjectPtr<AActor>& Rhs)
{
	return GetTypeHash(Lhs) < GetTypeHash(Rhs);
}

} // NDIRigidMeshCollisionLocal

//------------------------------------------------------------------------------------------------------------

void FNDIRigidMeshCollisionBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	using namespace NDIRigidMeshCollisionLocal;

	CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, WorldTransformBuffer, 3 * MaxNumTransforms);
	CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, InverseTransformBuffer, 3 * MaxNumTransforms);

	CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, ElementExtentBuffer, MaxNumPrimitives);
	CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, MeshScaleBuffer, MaxNumPrimitives);
	CreateInternalBuffer<uint32, EPixelFormat::PF_R32_UINT>(RHICmdList, PhysicsTypeBuffer, MaxNumPrimitives);
	CreateInternalBuffer<uint32, EPixelFormat::PF_R32_UINT>(RHICmdList, DFIndexBuffer, MaxNumPrimitives);
}

void FNDIRigidMeshCollisionBuffer::ReleaseRHI()
{
	WorldTransformBuffer.Release();
	InverseTransformBuffer.Release();
	ElementExtentBuffer.Release();
	MeshScaleBuffer.Release();
	PhysicsTypeBuffer.Release();
	DFIndexBuffer.Release();
}

//------------------------------------------------------------------------------------------------------------



void FNDIRigidMeshCollisionData::ReleaseBuffers()
{
	if (AssetBuffer)
	{
		ENQUEUE_RENDER_COMMAND(DeleteResource)(
			[ParamPointerToRelease = AssetBuffer](FRHICommandListImmediate& RHICmdList)
			{
				ParamPointerToRelease->ReleaseResource();
				delete ParamPointerToRelease;
			});
		AssetBuffer = nullptr;
	}
}

bool FNDIRigidMeshCollisionData::HasActors() const
{
	return !ExplicitActors.IsEmpty() || !FoundActors.IsEmpty();
}

bool FNDIRigidMeshCollisionData::ShouldRunGlobalSearch(UNiagaraDataInterfaceRigidMeshCollisionQuery* Interface) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Interface->GlobalSearchAllowed && (Interface->GlobalSearchForced || (Interface->GlobalSearchFallback_Unscripted && !bHasScriptedFindActor));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FNDIRigidMeshCollisionData::MergeActors(FMergedActorArray& MergedActors) const
{
	MergedActors.Reserve(ExplicitActors.Num() + FoundActors.Num());

	auto AppendActors = [&](const TWeakObjectPtr<AActor>& ActorPtr)
	{
		if (AActor* Actor = ActorPtr.Get())
		{
			MergedActors.AddUnique(Actor);
		}
	};

	Algo::ForEach(ExplicitActors, AppendActors);
	Algo::ForEach(FoundActors, AppendActors);
}

bool FNDIRigidMeshCollisionData::TrimMissingActors()
{
	auto EvaluateWeakActor = [&](const TWeakObjectPtr<AActor>& ActorPtr)
	{
		return !ActorPtr.IsValid();
	};

	const int32 ExplicitActorCount = ExplicitActors.Num();
	ExplicitActors.SetNum(Algo::StableRemoveIf(ExplicitActors, EvaluateWeakActor));

	const int32 FoundActorCount = FoundActors.Num();
	FoundActors.SetNum(Algo::StableRemoveIf(FoundActors, EvaluateWeakActor));

	return ExplicitActorCount != ExplicitActors.Num() || FoundActorCount != FoundActors.Num();
}

void FNDIRigidMeshCollisionData::Init(int32 MaxNumPrimitives)
{
	const bool bHasActors = HasActors();
	const bool bWasInitialized = AssetArrays.IsValid();

	if (bHasActors)
	{
		if (!bWasInitialized)
		{
			AssetArrays = MakeUnique<FNDIRigidMeshCollisionArrays>(MaxNumPrimitives);

			AssetBuffer = new FNDIRigidMeshCollisionBuffer();
			AssetBuffer->SetMaxNumPrimitives(MaxNumPrimitives);

			BeginInitResource(AssetBuffer);
		}

		AssetArrays->Reset();
	}
	else if (bWasInitialized)
	{
		AssetArrays = nullptr;
		ReleaseBuffers();
	}

	MeshBodyCounts.Empty();

	bRequiresFullUpdate = true;
}

void FNDIRigidMeshCollisionData::Update(UNiagaraDataInterfaceRigidMeshCollisionQuery* Interface)
{
	using namespace NDIRigidMeshCollisionLocal;

	if (!Interface || !SystemInstance)
	{
		return;
	}

	Interface->GetExplicitActors(*this);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ShouldRunGlobalSearch(Interface))
	{
		Interface->GlobalFindActors(SystemInstance->GetWorld(), *this);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// see if we need to reinitialize the internals
	const bool bAlreadyInited = AssetArrays != nullptr;
	const bool bHasActors = HasActors();
	if (bAlreadyInited != bHasActors)
	{
		Init(Interface->MaxNumPrimitives);
	}

	if (bHasActors)
	{
		TrimMissingActors();

		TInlineComponentArray<UStaticMeshComponent*> StaticMeshes;
		TInlineComponentArray<USkeletalMeshComponent*> SkeletalMeshes;

		{
			FMergedActorArray MergedActors;
			MergeActors(MergedActors);

			CollectComponents(MergedActors, Interface->ComponentTags, StaticMeshes);
			CollectComponents(MergedActors, Interface->ComponentTags, SkeletalMeshes);

			// in order to better keep track of our possibly changing set of components over time we
			// sort our collected components by their TypeHash
			auto SortPredicateByTypeHash = [](const UMeshComponent& Lhs, const UMeshComponent& Rhs) -> bool
			{
				return GetTypeHash(&Lhs) < GetTypeHash(&Rhs);
			};

			StaticMeshes.Sort(SortPredicateByTypeHash);
			SkeletalMeshes.Sort(SortPredicateByTypeHash);

			auto AccumulateHash = [](uint32 Lhs, uint32 Rhs) -> uint32
			{
				return HashCombine(Lhs, Rhs);
			};

			// generate a hash with the collected components so that we can see if on subsequent frames we
			// have a different collection, in which case we'll want to do a full rebuild of the array of transforms
			uint32 NewHashValue = 0;

			NewHashValue = Algo::TransformAccumulate(StaticMeshes, [](const UStaticMeshComponent* MeshComponent) -> uint32 { return MeshComponentHash(MeshComponent); }, NewHashValue, AccumulateHash);
			NewHashValue = Algo::TransformAccumulate(SkeletalMeshes, [](const USkeletalMeshComponent* MeshComponent) -> uint32 { return MeshComponentHash(MeshComponent); }, NewHashValue, AccumulateHash);

			if (NewHashValue != ComponentCollectionHash)
			{
				ComponentCollectionHash = NewHashValue;
				bRequiresFullUpdate = true;
			}
		}

		TConstArrayView<UStaticMeshComponent*> StaticMeshView = MakeArrayView(StaticMeshes.GetData(), StaticMeshes.Num());
		TConstArrayView<USkeletalMeshComponent*> SkeletalMeshView = MakeArrayView(SkeletalMeshes.GetData(), SkeletalMeshes.Num());

		uint32 TotalBoxCount = 0;
		uint32 TotalSphereCount = 0;
		uint32 TotalCapsuleCount = 0;

		const bool bArraysUpdated = UpdateInternalArrays(
			SystemInstance,
			StaticMeshView,
			SkeletalMeshView,
			FVector(SystemInstance->GetLWCTile()),
			bRequiresFullUpdate,
			MeshBodyCounts,
			AssetArrays.Get(),
			TotalBoxCount,
			TotalSphereCount,
			TotalCapsuleCount);

		if (bArraysUpdated)
		{
			bRequiresFullUpdate = false;
			bExceedingComponentLimits = false;
		}
		else
		{
			if (TotalBoxCount + TotalSphereCount + TotalCapsuleCount > AssetArrays->MaxPrimitives)
			{
				if (!bExceedingComponentLimits)
				{
					UE_LOGF(LogRigidMeshCollision, Warning, "Number of Collision DI primitives (%d boxes, %d spheres, %d capsules) is higher than the %d limit for System[%ls].  Please increase it.",
						TotalBoxCount, TotalSphereCount, TotalCapsuleCount,
						AssetArrays->MaxPrimitives,
						*GetNameSafe(SystemInstance->GetSystem()));

					bExceedingComponentLimits = true;
				}
			}
		}
	}
	else
	{
		bRequiresFullUpdate = false;
	}
}

//------------------------------------------------------------------------------------------------------------

/** Proxy to send data to gpu */
struct FNDIRigidMeshCollisionProxy : public FNiagaraDataInterfaceProxy
{
	struct FRenderThreadData
	{
		FNDIRigidMeshCollisionElementOffset ElementOffsets;
		TArray<FVector4f> WorldTransform;
		TArray<FVector4f> InverseTransform;
		TArray<FVector4f> ElementExtent;
		TArray<FVector4f> MeshScale;
		TArray<uint32> PhysicsType;
		TArray<int32> ComponentIdIndex;
		uint32 MaxPrimitiveCount;
		TArray<FPrimitiveComponentId> UniqueComponentIds;

		FNDIRigidMeshCollisionBuffer* AssetBuffer = nullptr;

		FVector3f CombinedBBoxWorldMin;
		FVector3f CombinedBBoxWorldMax;
	};

	void RemoveInstance(const FNiagaraSystemInstanceID& Instance)
	{
		check(IsInRenderingThread());

		SystemInstancesToProxyData_RT.Remove(Instance);
	}

	/** Get the size of the data that will be passed to render*/
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override
	{
		return sizeof(FRenderThreadData);
	}

	/** Get the data that will be passed to render*/
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override
	{
		check(IsInRenderingThread());

		const FRenderThreadData* SourceData = reinterpret_cast<FRenderThreadData*>(PerInstanceData);
		FRenderThreadData& TargetData = SystemInstancesToProxyData_RT.FindOrAdd(Instance);

		if (ensure(SourceData))
		{
			TargetData = *SourceData;
			SourceData->~FRenderThreadData();
		}
	}

	/** Launch all pre stage functions */
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override
	{
		using namespace NDIRigidMeshCollisionLocal;

		check(SystemInstancesToProxyData_RT.Contains(Context.GetSystemInstanceID()));

		FRenderThreadData* ProxyData = SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());
		if (ProxyData != nullptr && ProxyData->AssetBuffer)
		{
			if (Context.GetSimStageData().bFirstStage)
			{
				FRHICommandListBase& RHICmdList = Context.GetGraphBuilder().RHICmdList;

				//-OPT: We may be able to avoid updating the buffers all the time
				UpdateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, ProxyData->WorldTransform, ProxyData->AssetBuffer->WorldTransformBuffer);
				UpdateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, ProxyData->InverseTransform, ProxyData->AssetBuffer->InverseTransformBuffer);
				UpdateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, ProxyData->ElementExtent, ProxyData->AssetBuffer->ElementExtentBuffer);
				UpdateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, ProxyData->MeshScale, ProxyData->AssetBuffer->MeshScaleBuffer);
				UpdateInternalBuffer<uint32, EPixelFormat::PF_R32_UINT>(RHICmdList, ProxyData->PhysicsType, ProxyData->AssetBuffer->PhysicsTypeBuffer);

				// the distance field indexing needs to be generated using the scene
				if (!ProxyData->ComponentIdIndex.IsEmpty() && ProxyData->AssetBuffer->DFIndexBuffer.Buffer.IsValid())
				{
					const int32 ElementCount = ProxyData->ComponentIdIndex.Num();
					const uint32 BufferBytes = sizeof(uint32) * ElementCount;
					void* BufferData = RHICmdList.LockBuffer(ProxyData->AssetBuffer->DFIndexBuffer.Buffer, 0, BufferBytes, RLM_WriteOnly);

					const FSceneInterface* Scene = Context.GetComputeDispatchInterface().GetSceneInterface();
					if (Scene && !ProxyData->UniqueComponentIds.IsEmpty())
					{
						TArray<uint32> UniqueDistanceFieldIndices;
						UniqueDistanceFieldIndices.Reserve(ProxyData->UniqueComponentIds.Num());

						for (const FPrimitiveComponentId ComponentId : ProxyData->UniqueComponentIds)
						{
							const FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene->GetPrimitiveSceneInfo(ComponentId);
							const uint32 DistanceFieldIndex = PrimitiveSceneInfo && PrimitiveSceneInfo->DistanceFieldInstanceIndices.Num() > 0 ? PrimitiveSceneInfo->DistanceFieldInstanceIndices[0] : INDEX_NONE;
							UniqueDistanceFieldIndices.Emplace(DistanceFieldIndex);
						}

						TArrayView<uint32> BufferView(reinterpret_cast<uint32*>(BufferData), ElementCount);
						for (int32 ElementIt = 0; ElementIt < ElementCount; ++ElementIt)
						{
							const int32 UniqueIdIndex = ProxyData->ComponentIdIndex[ElementIt];
							BufferView[ElementIt] = UniqueDistanceFieldIndices.IsValidIndex(UniqueIdIndex) ? UniqueDistanceFieldIndices[UniqueIdIndex] : INDEX_NONE;
						}
					}
					else
					{
						FMemory::Memset(BufferData, 0xFF, BufferBytes);
					}
					RHICmdList.UnlockBuffer(ProxyData->AssetBuffer->DFIndexBuffer.Buffer);
				}
			}
		}
	}

	/** List of proxy data for each system instances*/
	TMap<FNiagaraSystemInstanceID, FRenderThreadData> SystemInstancesToProxyData_RT;
};

//------------------------------------------------------------------------------------------------------------

UNiagaraDataInterfaceRigidMeshCollisionQuery::UNiagaraDataInterfaceRigidMeshCollisionQuery(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)	
{
	Proxy.Reset(new FNDIRigidMeshCollisionProxy());
}

#if WITH_NIAGARA_DEBUGGER

void UNiagaraDataInterfaceRigidMeshCollisionQuery::DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const
{
	const FNDIRigidMeshCollisionData* InstanceData_GT = DebugHudContext.GetSystemInstance()->FindTypedDataInterfaceInstanceData<FNDIRigidMeshCollisionData>(this);
	if (InstanceData_GT == nullptr || !InstanceData_GT->AssetArrays.IsValid())
	{
		return;
	}

	const FNDIRigidMeshCollisionElementOffset& ElementOffsets = InstanceData_GT->AssetArrays->ElementOffsets;

	const uint32 BoxCount = ElementOffsets.SphereOffset - ElementOffsets.BoxOffset;
	const uint32 SphereCount = ElementOffsets.CapsuleOffset - ElementOffsets.SphereOffset;
	const uint32 CapsuleCount = ElementOffsets.NumElements - ElementOffsets.CapsuleOffset;

	if (InstanceData_GT->bExceedingComponentLimits)
	{
		DebugHudContext.GetOutputString().Appendf(TEXT("EXCEEDED COMPONENT LIMIT - %d"), InstanceData_GT->AssetArrays->MaxPrimitives);
	}
	else
	{
		DebugHudContext.GetOutputString().Appendf(TEXT("Boxes(%d) Spheres(%d) Capsules(%d)"), BoxCount, SphereCount, CapsuleCount);
	}

	auto GetTransformFromArray = [&](TArray<FVector4f> &TransformArray, int32 ElementIndex)
	{
		const uint32 ElementOffset = 3 * ElementIndex;
		FVector4f* TransformVec = TransformArray.GetData() + ElementOffset;

		FMatrix ElementMatrix;
		ElementMatrix.SetIdentity();

		for (int32 RowIt = 0; RowIt < 3; ++RowIt)
		{
			for (int32 ColIt = 0; ColIt < 4; ++ColIt)
			{
				ElementMatrix.M[RowIt][ColIt] = TransformVec[RowIt][ColIt];
			}
		}

		return ElementMatrix.GetTransposed();
	};

	if (DebugHudContext.IsVerbose())
	{
		const UWorld* World = DebugHudContext.GetWorld();

		TArray<FVector4f>& CurrentTransformArray = InstanceData_GT->AssetArrays->CurrentTransform;
		TArray<FVector4f>& PreviousTransformArray = InstanceData_GT->AssetArrays->PreviousTransform;

		// Inclusive world bounds for DI
		FVector3f Center = (InstanceData_GT->AssetArrays->CombinedBBoxWorldMin + InstanceData_GT->AssetArrays->CombinedBBoxWorldMax) * .5;
		FVector3f HalfExtent = (InstanceData_GT->AssetArrays->CombinedBBoxWorldMax - InstanceData_GT->AssetArrays->CombinedBBoxWorldMin) * .5;
		DrawDebugBox(World, (FVector) Center, (FVector) HalfExtent, FQuat::Identity, FColor::Purple);

		// Boxes
		for (uint32 BoxIt = 0; BoxIt < BoxCount; ++BoxIt)
		{
			const FVector3f HalfBoxExtent = 0.5f * InstanceData_GT->AssetArrays->ElementExtent[ElementOffsets.BoxOffset + BoxIt];
			const FMatrix CurrentTransform = GetTransformFromArray(CurrentTransformArray, ElementOffsets.BoxOffset + BoxIt);
			DrawDebugBox(World, CurrentTransform.TransformPosition(FVector::ZeroVector), FVector(HalfBoxExtent), CurrentTransform.Rotator().Quaternion(), FColor::Blue);			

			const FMatrix PrevTransform = GetTransformFromArray(PreviousTransformArray, ElementOffsets.BoxOffset + BoxIt);
			DrawDebugBox(World, PrevTransform.TransformPosition(FVector::ZeroVector), FVector(HalfBoxExtent), PrevTransform.Rotator().Quaternion(), FColor::Red);
		}

		// Spheres
		for (uint32 SphereIt = 0; SphereIt < SphereCount; ++SphereIt)
		{
			const float Radius = InstanceData_GT->AssetArrays->ElementExtent[ElementOffsets.SphereOffset + SphereIt].X;
			const FMatrix CurrentTransform = GetTransformFromArray(CurrentTransformArray, ElementOffsets.SphereOffset + SphereIt);
			DrawDebugSphere(World, CurrentTransform.TransformPosition(FVector::ZeroVector), Radius, 20, FColor::Blue);

			const FMatrix PrevTransform = GetTransformFromArray(PreviousTransformArray, ElementOffsets.SphereOffset + SphereIt);
			DrawDebugSphere(World, PrevTransform.TransformPosition(FVector::ZeroVector), Radius, 20, FColor::Red);
		}

		// Capsules
		for (uint32 CapsuleIt = 0; CapsuleIt < CapsuleCount; ++CapsuleIt)
		{
			const FVector2f RadiusLength(InstanceData_GT->AssetArrays->ElementExtent[ElementOffsets.CapsuleOffset + CapsuleIt]);
			const FMatrix CurrentTransform = GetTransformFromArray(CurrentTransformArray, ElementOffsets.CapsuleOffset + CapsuleIt);
			DrawDebugCapsule(World, CurrentTransform.TransformPosition(FVector::ZeroVector), RadiusLength.Y * 0.5f, RadiusLength.X, CurrentTransform.Rotator().Quaternion(), FColor::Blue);

			const FMatrix PrevTransform = GetTransformFromArray(PreviousTransformArray, ElementOffsets.CapsuleOffset + CapsuleIt);
			DrawDebugCapsule(World, PrevTransform.TransformPosition(FVector::ZeroVector), RadiusLength.Y * 0.5f, RadiusLength.X, CurrentTransform.Rotator().Quaternion(), FColor::Red);
		}

		if (!InstanceData_GT->ExplicitActors.IsEmpty() || !InstanceData_GT->FoundActors.IsEmpty())
		{
			UCanvas* Canvas = DebugHudContext.GetCanvas();

			// the DrawDebugCanvas* functions don't reasoanbly handle the near clip plane (both in terms of clipping and in terms of
			// objects being behind the camera); so we introduce this culling behavior to work around it
			auto ShouldClip = [Canvas](const FMatrix& Transform, const FBoxSphereBounds& Bounds)
			{
				const FVector Origin = Transform.TransformPosition(Bounds.Origin);
				return (Canvas->Project(Origin).GetMin() < UE_KINDA_SMALL_NUMBER);
			};

			const UFont* Font = GEngine->GetMediumFont();
			Canvas->SetDrawColor(FColor::White);

			auto DrawDebugActor = [&](const TWeakObjectPtr<AActor>& InWeakActor, const TCHAR* ActorSourceString)
			{
				if (AActor* Actor = InWeakActor.Get())
				{
					FVector ActorOrigin;
					FVector ActorBoundsExtent;
					Actor->GetActorBounds(true, ActorOrigin, ActorBoundsExtent);

					const FMatrix CurrentTransform = FTranslationMatrix(ActorOrigin);
					DrawDebugBox(World, CurrentTransform.TransformPosition(FVector::ZeroVector), ActorBoundsExtent, FQuat::Identity, FColor::Yellow);
					if (!ShouldClip(CurrentTransform, FSphere(FVector::ZeroVector, ActorBoundsExtent.Size())))
					{
						FString ActorLabel;
#if WITH_EDITOR
						ActorLabel = Actor->GetActorLabel();
#endif
						if (ActorLabel.Len() == 0)
						{
							ActorLabel = Actor->GetName();
						}

						const FVector ScreenLoc = Canvas->Project(ActorOrigin);
						Canvas->DrawText(Font, FString::Printf(TEXT("RigidMeshDI[%s Actor] - %s"), ActorSourceString, *ActorLabel), float(ScreenLoc.X), float(ScreenLoc.Y));
					}
				}
			};

			for (const TWeakObjectPtr<AActor>& ExplicitActor : InstanceData_GT->ExplicitActors)
			{
				DrawDebugActor(ExplicitActor, TEXT("Explicit"));
			}

			for (const TWeakObjectPtr<AActor>& FoundActor : InstanceData_GT->FoundActors)
			{
				DrawDebugActor(FoundActor, TEXT("Found"));
			}
		}
	}
}
#endif

bool UNiagaraDataInterfaceRigidMeshCollisionQuery::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIRigidMeshCollisionLocal;

	const bool HasScriptedFindActor = SystemHasFindActorsFunction(SystemInstance->GetSystem());

	FNDIRigidMeshCollisionData* InstanceData = new (PerInstanceData) FNDIRigidMeshCollisionData(SystemInstance, HasScriptedFindActor);

	GetExplicitActors(*InstanceData);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// if we're running a global search, then run that now
	if (InstanceData->ShouldRunGlobalSearch(this))
	{
		GlobalFindActors(SystemInstance->GetWorld(), *InstanceData);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	InstanceData->Init(MaxNumPrimitives);

	return true;
}

ETickingGroup UNiagaraDataInterfaceRigidMeshCollisionQuery::CalculateTickGroup(const void* PerInstanceData) const
{
	using namespace NDIRigidMeshCollisionLocal;

	if (const FNDIRigidMeshCollisionData* InstanceData = static_cast<const FNDIRigidMeshCollisionData*>(PerInstanceData))
	{
		ETickingGroup TickingGroup = NiagaraFirstTickGroup;

		TInlineComponentArray<UStaticMeshComponent*> StaticMeshes;
		TInlineComponentArray<USkeletalMeshComponent*> SkeletalMeshes;

		FNDIRigidMeshCollisionData::FMergedActorArray MergedActors;
		InstanceData->MergeActors(MergedActors);

		CollectComponents(MergedActors, ComponentTags, StaticMeshes);
		CollectComponents(MergedActors, ComponentTags, SkeletalMeshes);

		auto ProcessComponent = [&](const UActorComponent* Component)
		{
			const ETickingGroup ComponentTickGroup = FMath::Max(Component->PrimaryComponentTick.TickGroup, Component->PrimaryComponentTick.EndTickGroup);
			const ETickingGroup PhysicsTickGroup = ComponentTickGroup;
			const ETickingGroup ClampedTickGroup = FMath::Clamp(static_cast<ETickingGroup>(PhysicsTickGroup + 1), NiagaraFirstTickGroup, NiagaraLastTickGroup);

			TickingGroup = FMath::Max(TickingGroup, ClampedTickGroup);
		};

		for (const UStaticMeshComponent* Component : StaticMeshes)
		{
			ProcessComponent(Component);
		}

		for (const USkeletalMeshComponent* Component : SkeletalMeshes)
		{
			ProcessComponent(Component);
		}

		return TickingGroup;
	}
	return NiagaraFirstTickGroup;
}

void UNiagaraDataInterfaceRigidMeshCollisionQuery::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIRigidMeshCollisionData* InstanceData = static_cast<FNDIRigidMeshCollisionData*>(PerInstanceData);

	InstanceData->ReleaseBuffers();	
	InstanceData->~FNDIRigidMeshCollisionData();

	FNDIRigidMeshCollisionProxy* ThisProxy = GetProxyAs<FNDIRigidMeshCollisionProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
	{
		ThisProxy->RemoveInstance(InstanceID);
	});
}

// we use PostSimulate so that the results from s will be ready and pushed to the render thread for handling the GPU queries
bool UNiagaraDataInterfaceRigidMeshCollisionQuery::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float InDeltaSeconds)
{
	FNDIRigidMeshCollisionData* InstanceData = static_cast<FNDIRigidMeshCollisionData*>(PerInstanceData);
	if (InstanceData && SystemInstance)
	{
		const FNiagaraTickInfo &TickInfo = SystemInstance->GetSystemSimulation()->GetTickInfo();

		// only update on the first tick
		if (TickInfo.TickNumber == 0)
		{
			check(InstanceData->SystemInstance == SystemInstance);
			InstanceData->Update(this);
		}
	}
	return false;
}

bool UNiagaraDataInterfaceRigidMeshCollisionQuery::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceRigidMeshCollisionQuery* OtherTyped = CastChecked<UNiagaraDataInterfaceRigidMeshCollisionQuery>(Destination);		
	
	OtherTyped->ActorTags = ActorTags;
	OtherTyped->ComponentTags = ComponentTags;
	OtherTyped->SourceActors = SourceActors;
	OtherTyped->OnlyUseMoveable = OnlyUseMoveable;
	OtherTyped->UseComplexCollisions = UseComplexCollisions;
	OtherTyped->bFilterByObjectType = bFilterByObjectType;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OtherTyped->GlobalSearchAllowed = GlobalSearchAllowed;
	OtherTyped->GlobalSearchForced = GlobalSearchForced;
	OtherTyped->GlobalSearchFallback_Unscripted = GlobalSearchFallback_Unscripted;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	OtherTyped->MaxNumPrimitives = MaxNumPrimitives;

	return true;
}

bool UNiagaraDataInterfaceRigidMeshCollisionQuery::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceRigidMeshCollisionQuery* OtherTyped = CastChecked<const UNiagaraDataInterfaceRigidMeshCollisionQuery>(Other);

	return (OtherTyped->ActorTags == ActorTags)
		&& (OtherTyped->ComponentTags == ComponentTags)
		&& (OtherTyped->SourceActors == SourceActors)
		&& (OtherTyped->OnlyUseMoveable == OnlyUseMoveable)
		&& (OtherTyped->UseComplexCollisions == UseComplexCollisions)
		&& (OtherTyped->bFilterByObjectType == bFilterByObjectType)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		&& (OtherTyped->GlobalSearchAllowed == GlobalSearchAllowed)
		&& (OtherTyped->GlobalSearchForced == GlobalSearchForced)
		&& (OtherTyped->GlobalSearchFallback_Unscripted == GlobalSearchFallback_Unscripted)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		&& (OtherTyped->MaxNumPrimitives == MaxNumPrimitives);
}

void UNiagaraDataInterfaceRigidMeshCollisionQuery::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceRigidMeshCollisionQuery::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (!Tag_DEPRECATED.IsEmpty())
	{
		FName Tag = *Tag_DEPRECATED;
		ActorTags.AddUnique(Tag);
		Tag_DEPRECATED = TEXT("");
	}
#endif
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceRigidMeshCollisionQuery::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIRigidMeshCollisionLocal;

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FindActorsName;
		Sig.SetDescription(LOCTEXT("FindActorsDescription", "Triggers an overlap test on the world to find actors to represent.."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = false;
		Sig.bSupportsCPU = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("RigidBody DI")));
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Overlap Origin")), OverlapOriginDescription);
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Overlap Rotation")), OverlapRotationDescription);
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Overlap Extent")), OverlapExtentDescription);
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(StaticEnum<ECollisionChannel>()), TEXT("TraceChannel")), TraceChannelDescription);
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Skip Overlap")), SkipOverlapDescription);
		Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Actors Changed")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNumBoxesName;
		Sig.SetDescription(LOCTEXT("GetNumBoxesNameDescription", "Returns the number of box primitives for the collection of static meshes the DI represents."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Boxes")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNumSpheresName;
		Sig.SetDescription(LOCTEXT("GetNumSpheresNameDescription", "Returns the number of sphere primitives for the collection of static meshes the DI represents."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Spheres")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNumCapsulesName;
		Sig.SetDescription(LOCTEXT("GetNumCapsulesNameDescription", "Returns the number of capsule primitives for the collection of static meshes the DI represents."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Capsules")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNumElementsName;
		Sig.SetDescription(LOCTEXT("GetNumElementsNameDescription", "Returns the number of primitive elements for the collection of static meshes the DI represents."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = true;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Elements")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetBoxElementsStartIndexName;
		Sig.SetDescription(LOCTEXT("GetBoxElementsStartIndexNameDescription", "Returns the Index of the first of box primitive for the collection of static meshes the DI represents."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetSphereElementsStartIndexName;
		Sig.SetDescription(LOCTEXT("GetSphereElementsStartIndexNameDescription", "Returns the Index of the first of sphere primitive for the collection of static meshes the DI represents."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetCapsuleElementsStartIndexName;
		Sig.SetDescription(LOCTEXT("GetCapsuleElementsStartIndexNameDescription", "Returns the Index of the first of capsule primitive for the collection of static meshes the DI represents."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetSphereRadiusName;
		Sig.SetDescription(LOCTEXT("GetSphereRadiusNameDescription", "Returns the radius of a sphere primitive in the collection of static meshes the DI represents."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Radius")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetCapsuleSizeName;
		Sig.SetDescription(LOCTEXT("GetCapsuleSizeNameDescription", "Returns the capsule size of a capsule primitive in the collection of static meshes the DI represents."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Size")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetBoxSizeName;
		Sig.SetDescription(LOCTEXT("GetBoxSizeNameNameDescription", "Returns the Extents of a box primitive in the collection of static meshes the DI represents."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Size")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = IsWorldPositionInsideCombinedBoundsName;
		Sig.SetDescription(LOCTEXT("IsWorldPositionInsideCombinedBoundsDescription", "Returns true if the query point is inside the bounding box of all the actors"));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = true;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsInside")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetClosestPointName;
		Sig.SetDescription(LOCTEXT("GetClosestPointDescription", "Given a world space position, computes the static mesh's closest point. Also returns normal and velocity for that point."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Closest Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Velocity")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetClosestPointSimpleName;
		Sig.SetDescription(LOCTEXT("GetClosestPointSimpleDescription", "Given a world space position, computes the static mesh's closest point. Also returns normal and velocity for that point."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));	
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Velocity")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetClosestElementName;
		Sig.SetDescription(LOCTEXT("GetClosestElementDescription", "Given a world space position, computes the static mesh's closest element."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Closest Element")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetElementPointName;
		Sig.SetDescription(LOCTEXT("GetElementPointDescription", "Given a world space position and an element index, computes the static mesh's closest point. Also returns normal and velocity for that point."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Closest Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Velocity")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetElementTransformName;
		Sig.SetDescription(LOCTEXT("GetElementTransformDescription", "Gets the World Transform of the specified element."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Translation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Is Valid")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetElementInverseTransformName;
		Sig.SetDescription(LOCTEXT("GetElementInverseTransformDescription", "Gets the Inverse World Transform of the specified element."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Translation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Is Valid")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetElementPointMeshDistanceFieldNoNormalName;
		Sig.SetDescription(LOCTEXT("GetClosestElementPointMeshDistanceFieldNoNormalDescription", "Given a world space position and an element index, computes the static mesh's closest distance and velocity."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetFloatDef()), TEXT("MaxDistance")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Velocity")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetElementDistanceName;
		Sig.SetDescription(LOCTEXT("GetElementDistanceDescription", "Given a world space position and element index, computes the distance to the closest point for the static mesh."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetClosestDistanceName;
		Sig.SetDescription(LOCTEXT("GetClosestDistanceDescription", "Given a world space position, computes the distance to the closest point for the static mesh."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetClosestPointMeshDistanceFieldName;
		Sig.SetDescription(LOCTEXT("GetClosestPointMeshDistanceFieldDescription", "Given a world space position, computes the distance to the closest point for the static mesh, using the mesh's distance field."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetFloatDef()), TEXT("MaxDistance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Closest Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Normal Is Valid")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Max Encoded Distance")));
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetClosestPointMeshDistanceFieldAccurateName;
		Sig.SetDescription(LOCTEXT("GetClosestPointMeshDistanceFieldDescription", "Given a world space position, computes the distance to the closest point for the static mesh, using the mesh's distance field."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetFloatDef()), TEXT("MaxDistance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Closest Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Normal")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Normal Is Valid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Max Encoded Distance")));
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetClosestPointMeshDistanceFieldNoNormalName;
		Sig.SetDescription(LOCTEXT("GetClosestPointMeshDistanceFieldNNDescription", "Given a world space position, computes the accurate closest distance and velocity for static meshes.  The closest position is approximate and computed by proximity to the physics assets and not the mesh itself."));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetFloatDef()), TEXT("MaxDistance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Closest Position")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Velocity")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetMaxEncodedDistanceMeshDistanceFieldName;
		Sig.SetDescription(LOCTEXT("GetMaxEncodedDistanceMeshDistanceFieldDescription", "Returns the maximum distance stored in the SDF according to the bandwidth it was created with"));
		Sig.SetFunctionVersion(FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion);
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Collision DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Max Distance")));

		OutFunctions.Add(Sig);
	}

}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceRigidMeshCollisionQuery, FindActorsCPU);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceRigidMeshCollisionQuery, GetNumElementsCPU);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceRigidMeshCollisionQuery, IsWorldPositionInsideCombinedBoundsCPU);

void UNiagaraDataInterfaceRigidMeshCollisionQuery::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIRigidMeshCollisionLocal;

	if (BindingInfo.Name == FindActorsName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceRigidMeshCollisionQuery, FindActorsCPU)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetNumElementsName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceRigidMeshCollisionQuery, GetNumElementsCPU)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == IsWorldPositionInsideCombinedBoundsName)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceRigidMeshCollisionQuery, IsWorldPositionInsideCombinedBoundsCPU)::Bind(this, OutFunc);
	}
	else
	{
		UE_LOGF(LogNiagara, Display, "Could not find data interface external function in %ls. %ls\n",
			*GetPathNameSafe(this), *BindingInfo.Name.ToString());
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceRigidMeshCollisionQuery::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIRigidMeshCollisionLocal;

	if ((FunctionInfo.DefinitionName == GetNumBoxesName) ||
		(FunctionInfo.DefinitionName == GetNumCapsulesName) ||
		(FunctionInfo.DefinitionName == GetNumSpheresName) ||
		(FunctionInfo.DefinitionName == GetNumElementsName) ||
		(FunctionInfo.DefinitionName == GetBoxElementsStartIndexName) ||
		(FunctionInfo.DefinitionName == GetSphereElementsStartIndexName) ||
		(FunctionInfo.DefinitionName == GetCapsuleElementsStartIndexName) ||
		(FunctionInfo.DefinitionName == GetSphereRadiusName) ||
		(FunctionInfo.DefinitionName == GetCapsuleSizeName) ||
		(FunctionInfo.DefinitionName == GetBoxSizeName) ||
		(FunctionInfo.DefinitionName == IsWorldPositionInsideCombinedBoundsName) ||
		(FunctionInfo.DefinitionName == GetClosestPointName) ||
		(FunctionInfo.DefinitionName == GetClosestPointSimpleName) ||		
		(FunctionInfo.DefinitionName == GetClosestElementName) ||
		(FunctionInfo.DefinitionName == GetElementTransformName) ||
		(FunctionInfo.DefinitionName == GetElementInverseTransformName) ||
		(FunctionInfo.DefinitionName == GetElementPointName) ||
		(FunctionInfo.DefinitionName == GetElementPointMeshDistanceFieldNoNormalName) ||
		(FunctionInfo.DefinitionName == GetElementDistanceName) ||
		(FunctionInfo.DefinitionName == GetClosestDistanceName) ||
		(FunctionInfo.DefinitionName == GetClosestPointMeshDistanceFieldName) ||
		(FunctionInfo.DefinitionName == GetClosestPointMeshDistanceFieldAccurateName) ||
		(FunctionInfo.DefinitionName == GetClosestPointMeshDistanceFieldNoNormalName) ||
		(FunctionInfo.DefinitionName == GetMaxEncodedDistanceMeshDistanceFieldName) )
	{
		return true;
	}
	OutHLSL += TEXT("\n");
	return false;
}

bool UNiagaraDataInterfaceRigidMeshCollisionQuery::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	using namespace NDIRigidMeshCollisionLocal;

	bool bChanged = false;
	
	// upgrade from lwc changes, only parameter types changed there
	if (FunctionSignature.FunctionVersion < FNiagaraRigidMeshCollisionDIFunctionVersion::LargeWorldCoordinates)
	{
		if (FunctionSignature.Name == GetClosestPointName && ensure(FunctionSignature.Inputs.Num() == 4) && ensure(FunctionSignature.Outputs.Num() == 4))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			FunctionSignature.Outputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
		if (FunctionSignature.Name == GetClosestElementName && ensure(FunctionSignature.Inputs.Num() == 3))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
		if (FunctionSignature.Name == GetElementPointName && ensure(FunctionSignature.Inputs.Num() == 5) && ensure(FunctionSignature.Outputs.Num() == 3))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			FunctionSignature.Outputs[0].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
		if (FunctionSignature.Name == GetElementPointMeshDistanceFieldNoNormalName && ensure(FunctionSignature.Inputs.Num() == 5) && ensure(FunctionSignature.Outputs.Num() == 2))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			FunctionSignature.Outputs[0].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
		if (FunctionSignature.Name == GetElementDistanceName && ensure(FunctionSignature.Inputs.Num() == 4))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
		if (FunctionSignature.Name == GetClosestDistanceName && ensure(FunctionSignature.Inputs.Num() == 3))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
		if (FunctionSignature.Name == GetClosestPointMeshDistanceFieldName && ensure(FunctionSignature.Inputs.Num() == 4) && ensure(FunctionSignature.Outputs.Num() == 4))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			FunctionSignature.Outputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
		if (FunctionSignature.Name == GetClosestPointMeshDistanceFieldNoNormalName && ensure(FunctionSignature.Inputs.Num() == 4) && ensure(FunctionSignature.Outputs.Num() == 3))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			FunctionSignature.Outputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
	}

	if (FunctionSignature.FunctionVersion < FNiagaraRigidMeshCollisionDIFunctionVersion::SetMaxDistance)
	{
		if (FunctionSignature.Name == GetClosestPointMeshDistanceFieldName)
		{
			FunctionSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetFloatDef()), TEXT("MaxDistance")));
			FunctionSignature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetBoolDef()), TEXT("Normal Is Valid")));
			bChanged = true;
		}
		if (FunctionSignature.Name == GetClosestPointMeshDistanceFieldNoNormalName)
		{
			FunctionSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetFloatDef()), TEXT("MaxDistance")));
			bChanged = true;
		}
	}

	if (FunctionSignature.FunctionVersion < FNiagaraRigidMeshCollisionDIFunctionVersion::FindActorRotation)
	{
		if (FunctionSignature.Name == FindActorsName)
		{
			FNiagaraVariable OverlapRotation(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Overlap Rotation"));

			FunctionSignature.Inputs.Insert(OverlapRotation, 2);
			FunctionSignature.InputDescriptions.Add(OverlapRotation, OverlapRotationDescription);
			bChanged = true;
		}
	}

	if (FunctionSignature.FunctionVersion < FNiagaraRigidMeshCollisionDIFunctionVersion::MaxEncodedDistance)
	{
		if (FunctionSignature.Name == GetClosestPointMeshDistanceFieldName || FunctionSignature.Name == GetClosestPointMeshDistanceFieldAccurateName)
		{
			FunctionSignature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Max Encoded Distance")));
			bChanged = true;
		}
	}

	FunctionSignature.FunctionVersion = FNiagaraRigidMeshCollisionDIFunctionVersion::LatestVersion;

	return bChanged;
}

void UNiagaraDataInterfaceRigidMeshCollisionQuery::GetCommonHLSL(FString& OutHLSL)
{
	Super::GetCommonHLSL(OutHLSL);

	OutHLSL += TEXT("#include \"/Engine/Private/DistanceFieldLightingShared.ush\"\n");
	OutHLSL += TEXT("#include \"/Engine/Private/MeshDistanceFieldCommon.ush\"\n");
}

void UNiagaraDataInterfaceRigidMeshCollisionQuery::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIRigidMeshCollisionLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceRigidMeshCollisionQuery::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}

	InVisitor->UpdateShaderFile(NDIRigidMeshCollisionLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<NDIRigidMeshCollisionLocal::FShaderParameters>();

	return true;
}
#endif

void UNiagaraDataInterfaceRigidMeshCollisionQuery::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIRigidMeshCollisionLocal::FShaderParameters>();
	ShaderParametersBuilder.AddIncludedStruct(UE::FXRenderingUtils::DistanceFields::GetObjectBufferParametersMetadata());
	ShaderParametersBuilder.AddIncludedStruct(UE::FXRenderingUtils::DistanceFields::GetAtlasParametersMetadata());
}

void UNiagaraDataInterfaceRigidMeshCollisionQuery::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	const FNDIRigidMeshCollisionProxy& InterfaceProxy = Context.GetProxy<FNDIRigidMeshCollisionProxy>();
	const FNDIRigidMeshCollisionProxy::FRenderThreadData* ProxyData = InterfaceProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID());

	NDIRigidMeshCollisionLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDIRigidMeshCollisionLocal::FShaderParameters>();

	const FShaderParametersMetadata* ObjectBufferParametersMetadata = UE::FXRenderingUtils::DistanceFields::GetObjectBufferParametersMetadata();
	const FShaderParametersMetadata* AtlasParametersMetadata = UE::FXRenderingUtils::DistanceFields::GetAtlasParametersMetadata();

	uint8* ShaderDistanceFieldObjectParameters = Context.GetParameterIncludedStruct(ObjectBufferParametersMetadata);
	uint8* ShaderDistanceFieldAtlasParameters = Context.GetParameterIncludedStruct(AtlasParametersMetadata);

	const bool bDistanceFieldDataBound =
		Context.IsStructBound(ShaderDistanceFieldObjectParameters, ObjectBufferParametersMetadata) ||
		Context.IsStructBound(ShaderDistanceFieldAtlasParameters, AtlasParametersMetadata);

	bool bBindDistanceFieldData = false;

	if (ProxyData != nullptr && ProxyData->AssetBuffer != nullptr && ProxyData->AssetBuffer->IsInitialized())
	{
		FNDIRigidMeshCollisionBuffer* AssetBuffer = ProxyData->AssetBuffer;

		ShaderParameters->WorldTransformBuffer = FNiagaraRenderer::GetSrvOrDefaultFloat4(AssetBuffer->WorldTransformBuffer.SRV);
		ShaderParameters->InverseTransformBuffer = FNiagaraRenderer::GetSrvOrDefaultFloat4(AssetBuffer->InverseTransformBuffer.SRV);
		ShaderParameters->ElementExtentBuffer = FNiagaraRenderer::GetSrvOrDefaultFloat4(AssetBuffer->ElementExtentBuffer.SRV);
		ShaderParameters->MeshScaleBuffer = FNiagaraRenderer::GetSrvOrDefaultFloat4(AssetBuffer->MeshScaleBuffer.SRV);
		ShaderParameters->PhysicsTypeBuffer = FNiagaraRenderer::GetSrvOrDefaultUInt(AssetBuffer->PhysicsTypeBuffer.SRV);
		ShaderParameters->DFIndexBuffer = FNiagaraRenderer::GetSrvOrDefaultUInt(AssetBuffer->DFIndexBuffer.SRV);

		ShaderParameters->MaxTransforms = ProxyData->MaxPrimitiveCount * 2;
		ShaderParameters->CurrentOffset = 0;
		ShaderParameters->PreviousOffset = ProxyData->MaxPrimitiveCount * 3;

		ShaderParameters->ElementOffsets.X = ProxyData->ElementOffsets.BoxOffset;
		ShaderParameters->ElementOffsets.Y = ProxyData->ElementOffsets.SphereOffset;
		ShaderParameters->ElementOffsets.Z = ProxyData->ElementOffsets.CapsuleOffset;
		ShaderParameters->ElementOffsets.W = ProxyData->ElementOffsets.NumElements;

		bBindDistanceFieldData = true;

		ShaderParameters->CombinedBBoxWorldMin = ProxyData->CombinedBBoxWorldMin;
		ShaderParameters->CombinedBBoxWorldMax = ProxyData->CombinedBBoxWorldMax;
	}
	else
	{
		ShaderParameters->WorldTransformBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->InverseTransformBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->ElementExtentBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->MeshScaleBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->PhysicsTypeBuffer = FNiagaraRenderer::GetDummyUIntBuffer();
		ShaderParameters->DFIndexBuffer = FNiagaraRenderer::GetDummyUIntBuffer();

		ShaderParameters->MaxTransforms = 0;
		ShaderParameters->CurrentOffset = 0;
		ShaderParameters->PreviousOffset = 0;
		ShaderParameters->ElementOffsets = FUintVector4(0, 0, 0, 0);

		ShaderParameters->CombinedBBoxWorldMin = FVector3f(0, 0, 0);
		ShaderParameters->CombinedBBoxWorldMax = FVector3f(0, 0, 0);
	}

	if (bDistanceFieldDataBound)
	{
		TConstStridedView<FSceneView> SimulationSceneViews = Context.GetComputeDispatchInterface().GetSimulationSceneViews();
		const FSceneView* PrimaryView = bBindDistanceFieldData && SimulationSceneViews.Num() > 0 ? &SimulationSceneViews[0] : nullptr;

		UE::FXRenderingUtils::DistanceFields::SetupObjectBufferParameters(Context.GetGraphBuilder(), ShaderDistanceFieldObjectParameters, PrimaryView);
		UE::FXRenderingUtils::DistanceFields::SetupAtlasParameters(Context.GetGraphBuilder(), ShaderDistanceFieldAtlasParameters, PrimaryView);
	}
}

#if WITH_EDITOR
void UNiagaraDataInterfaceRigidMeshCollisionQuery::ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors)
{
	using namespace NDIRigidMeshCollisionLocal;

	if (Function.Name == GetClosestPointMeshDistanceFieldName || Function.Name == GetClosestPointMeshDistanceFieldNoNormalName || Function.Name == GetClosestPointMeshDistanceFieldAccurateName)
	{
		if (!IsMeshDistanceFieldEnabled())
		{
			OutValidationErrors.Add(NSLOCTEXT("UNiagaraDataInterfaceRigidMeshCollisionQuery", "NiagaraDistanceFieldNotEnabledMsg", "The mesh distance field generation is currently not enabled, please check the project settings.\nNiagara cannot query the mesh distance fields otherwise."));
		}
	}
}
#endif

void UNiagaraDataInterfaceRigidMeshCollisionQuery::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	const FNDIRigidMeshCollisionData* GameThreadData = reinterpret_cast<const FNDIRigidMeshCollisionData*>(PerInstanceData);
	FNDIRigidMeshCollisionProxy::FRenderThreadData* RenderThreadData = new(DataForRenderThread) FNDIRigidMeshCollisionProxy::FRenderThreadData();

	if (ensure(GameThreadData != nullptr && RenderThreadData != nullptr))
	{
		if (const FNDIRigidMeshCollisionArrays* SourceArrayData = GameThreadData->AssetArrays.Get())
		{
			const int32 ElementCount = GameThreadData->AssetArrays->ElementOffsets.NumElements;

			RenderThreadData->ElementOffsets = GameThreadData->AssetArrays->ElementOffsets;

			// compact the world/inverse transforms
			const int32 TransformVectorCount = ElementCount * 3;

			auto CompactTransforms = [&](const TArray<FVector4f>& Current, const TArray<FVector4f>& Previous, TArray<FVector4f>& Compact)
			{
				check(Current.Num() >= TransformVectorCount);
				check(Previous.Num() >= TransformVectorCount);

				Compact.Reset(2 * TransformVectorCount); // space for current and previous transforms
				Compact.Append(Current.GetData(), TransformVectorCount);
				Compact.Append(Previous.GetData(), TransformVectorCount);
			};

			CompactTransforms(GameThreadData->AssetArrays->CurrentTransform, GameThreadData->AssetArrays->PreviousTransform, RenderThreadData->WorldTransform);
			CompactTransforms(GameThreadData->AssetArrays->CurrentInverse, GameThreadData->AssetArrays->PreviousInverse, RenderThreadData->InverseTransform);

			RenderThreadData->ElementExtent.Append(GameThreadData->AssetArrays->ElementExtent.GetData(), ElementCount);
			RenderThreadData->MeshScale.Append(GameThreadData->AssetArrays->MeshScale.GetData(), ElementCount);
			RenderThreadData->PhysicsType.Append(GameThreadData->AssetArrays->PhysicsType.GetData(), ElementCount);
			RenderThreadData->ComponentIdIndex.Append(GameThreadData->AssetArrays->ComponentIdIndex.GetData(), ElementCount);

			RenderThreadData->UniqueComponentIds = GameThreadData->AssetArrays->UniqueCompnentId;
			RenderThreadData->MaxPrimitiveCount = ElementCount;
			RenderThreadData->AssetBuffer = GameThreadData->AssetBuffer;

			RenderThreadData->CombinedBBoxWorldMin = GameThreadData->AssetArrays->CombinedBBoxWorldMin;
			RenderThreadData->CombinedBBoxWorldMax = GameThreadData->AssetArrays->CombinedBBoxWorldMax;
		}
	}
	check(Proxy);
}

bool UNiagaraDataInterfaceRigidMeshCollisionQuery::FilterComponent(const UPrimitiveComponent* Component) const
{
	return !(Component->IsA<USkeletalMeshComponent>() || Component->IsA<UStaticMeshComponent>());
}

bool UNiagaraDataInterfaceRigidMeshCollisionQuery::FilterActor(const AActor* Actor) const
{
	if (OnlyUseMoveable && !Actor->IsRootComponentMovable())
	{
		return true;
	}

	if (!ActorTags.IsEmpty() && !ActorTags.ContainsByPredicate([&](const FName& Tag) { return Tag == NAME_None || Actor->Tags.Contains(Tag); }))
	{
		return true;
	}

	return false;
}

bool UNiagaraDataInterfaceRigidMeshCollisionQuery::GlobalFindActors(UWorld* World, FNDIRigidMeshCollisionData& InstanceData) const
{
	TArray<TWeakObjectPtr<AActor>> PreviousActors;
	Swap(InstanceData.FoundActors, PreviousActors);

	if (ensure(World))
	{
		for (TActorIterator<AActor> It(World); It; ++It)
		{
			if (AActor* Actor = *It)
			{
				if (FilterActor(Actor))
				{
					continue;
				}

				InstanceData.FoundActors.AddUnique(Actor);
			}
		}
	}

	Algo::Sort(InstanceData.FoundActors, NDIRigidMeshCollisionLocal::WeakActorPtrLess);

	return PreviousActors != InstanceData.FoundActors;
}

bool UNiagaraDataInterfaceRigidMeshCollisionQuery::FindActors(UWorld* World, FNDIRigidMeshCollisionData& InstanceData, ECollisionChannel Channel, const FVector& OverlapLocation, const FVector& OverlapExtent, const FQuat& OverlapRotation) const
{
	TArray<TWeakObjectPtr<AActor>> PreviousActors;
	Swap(InstanceData.FoundActors, PreviousActors);

	if (ensure(World))
	{
		TArray<FOverlapResult> Overlaps;
		FCollisionQueryParams Params(SCENE_QUERY_STAT(NiagaraRigidMeshCollisionQuery), UseComplexCollisions);

		// it is not clear the best strategy for filtering out results between ByObjectType and ByChannel.  For now we're going to preserve existing behavior
		if (bFilterByObjectType)
		{
			FCollisionObjectQueryParams ObjectParams;
			ObjectParams.AddObjectTypesToQuery(Channel);

			World->OverlapMultiByObjectType(Overlaps, OverlapLocation, OverlapRotation, ObjectParams, FCollisionShape::MakeBox(0.5f * OverlapExtent), Params);
		}
		else
		{
			World->OverlapMultiByChannel(Overlaps, OverlapLocation, OverlapRotation, Channel, FCollisionShape::MakeBox(0.5f * OverlapExtent), Params);
		}

		for (const FOverlapResult& OverlapResult : Overlaps)
		{
			if (UPrimitiveComponent* PrimitiveComponent = OverlapResult.GetComponent())
			{
				if (FilterComponent(PrimitiveComponent))
				{
					continue;
				}

				if (AActor* ComponentActor = PrimitiveComponent->GetOwner())
				{
					if (FilterActor(ComponentActor))
					{
						continue;
					}
					InstanceData.FoundActors.AddUnique(ComponentActor);
				}
			}
		}
	}

	Algo::Sort(InstanceData.FoundActors, NDIRigidMeshCollisionLocal::WeakActorPtrLess);

	return PreviousActors != InstanceData.FoundActors;
}

bool UNiagaraDataInterfaceRigidMeshCollisionQuery::GetExplicitActors(FNDIRigidMeshCollisionData& InstanceData)
{
	TArray<TWeakObjectPtr<AActor>> PreviousActors;
	Swap(InstanceData.ExplicitActors, PreviousActors);

	for (const TSoftObjectPtr<AActor>& ActorPtr : SourceActors)
	{
		if (AActor* Actor = ActorPtr.Get())
		{
			InstanceData.ExplicitActors.AddUnique(Actor);
		}
	}

	return InstanceData.ExplicitActors != PreviousActors;
}

void UNiagaraDataInterfaceRigidMeshCollisionQuery::FindActorsCPU(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIRigidMeshCollisionData> InstanceData(Context);

	FNDIInputParam<FNiagaraPosition> OverlapOriginParam(Context);
	FNDIInputParam<FQuat4f> OverlapRotationParam(Context);
	FNDIInputParam<FVector3f> OverlapExtentParam(Context);
	FNDIInputParam<ECollisionChannel> TraceChannelParam(Context);
	FNDIInputParam<FNiagaraBool> SkipOverlapParam(Context);

	FNDIOutputParam<FNiagaraBool> ActorsChangedParam(Context);

	if (ensure(InstanceData->SystemInstance))
	{
		FNiagaraLWCConverter LWCConverter = InstanceData->SystemInstance->GetLWCConverter();
		UWorld* World = InstanceData->SystemInstance->GetWorld();

		if (World)
		{
			for (int32 i = 0; i < Context.GetNumInstances(); ++i)
			{
				FNiagaraPosition OverlapOrigin = OverlapOriginParam.GetAndAdvance();
				FQuat4f OverlapRotation = OverlapRotationParam.GetAndAdvance();
				FVector3f OverlapExtent = OverlapExtentParam.GetAndAdvance();
				ECollisionChannel TraceChannel = TraceChannelParam.GetAndAdvance();
				bool SkipOverlap = SkipOverlapParam.GetAndAdvance();

				bool ActorsChanged = false;

				if (!SkipOverlap)
				{
					const FVector ConvertedOrigin = LWCConverter.ConvertSimulationPositionToWorld(OverlapOrigin);
					if (FindActors(World, *InstanceData, TraceChannel, ConvertedOrigin, FVector(OverlapExtent), FQuat(OverlapRotation)))
					{
						ActorsChanged = true;
					}
				}

				ActorsChangedParam.SetAndAdvance(ActorsChanged);
			}

			return;
		}
	}

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		ActorsChangedParam.SetAndAdvance(false);
	}
}

void UNiagaraDataInterfaceRigidMeshCollisionQuery::GetNumElementsCPU(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIRigidMeshCollisionData> InstanceData(Context);

	FNDIOutputParam<int32> NumElementsParam(Context);

	if (InstanceData->SystemInstance != nullptr && InstanceData->AssetArrays != nullptr)
	{
		NumElementsParam.SetAndAdvance(InstanceData->AssetArrays->ElementOffsets.NumElements);
	}
	else
	{
		NumElementsParam.SetAndAdvance(0);
	}
}

void UNiagaraDataInterfaceRigidMeshCollisionQuery::IsWorldPositionInsideCombinedBoundsCPU(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIRigidMeshCollisionData> InstanceData(Context);
	
	FNDIInputParam<FNiagaraPosition> WorldPositionParam(Context);
	FNDIOutputParam<bool> IsInsideParam(Context);

	if (InstanceData->SystemInstance != nullptr && InstanceData->AssetArrays != nullptr)
	{
		FNiagaraPosition WorldPosition = WorldPositionParam.GetAndAdvance();

		FNiagaraLWCConverter LWCConverter = InstanceData->SystemInstance->GetLWCConverter();
		const FVector ConvertedWorldPosition = LWCConverter.ConvertSimulationPositionToWorld(WorldPosition);

		const bool IsInside =
			ConvertedWorldPosition.X >= InstanceData->AssetArrays->CombinedBBoxWorldMin.X &&
			ConvertedWorldPosition.Y >= InstanceData->AssetArrays->CombinedBBoxWorldMin.Y &&
			ConvertedWorldPosition.Z >= InstanceData->AssetArrays->CombinedBBoxWorldMin.Z &&
			ConvertedWorldPosition.X <= InstanceData->AssetArrays->CombinedBBoxWorldMax.X &&
			ConvertedWorldPosition.Y <= InstanceData->AssetArrays->CombinedBBoxWorldMax.Y &&
			ConvertedWorldPosition.Z <= InstanceData->AssetArrays->CombinedBBoxWorldMax.Z;

		IsInsideParam.SetAndAdvance(IsInside);
	}
	else
	{
		IsInsideParam.SetAndAdvance(false);
	}
}

#if WITH_EDITOR

void UNiagaraDataInterfaceRigidMeshCollisionQuery::GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	using namespace NDIRigidMeshCollisionLocal;

	auto IsValidActor = [](const TSoftObjectPtr<AActor>& SoftPtrActor) -> bool
	{
		return SoftPtrActor.IsValid();
	};

	const bool bHasScriptedFindActor = SystemHasFindActorsFunction(InAsset);
	const bool bHasSourceActor = SourceActors.ContainsByPredicate(IsValidActor);

	if (!bHasScriptedFindActor && !bHasSourceActor)
	{
		OutWarnings.Emplace(
			LOCTEXT("RigidMeshDI_MissingFindActor", "RigidMeshCollisionQuery is missing call to FindActors which is needed to find collision primitives."),
			LOCTEXT("RigidMeshDI_MissingFindActor_Summary", "Missing FIndActors."),
			FNiagaraDataInterfaceFix()
		);
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (GlobalSearchAllowed)
	{
		OutWarnings.Emplace(
			LOCTEXT("RigidMeshDI_GlobalSearch", "Use of Global Search options is deprecated in favor of using FindActors."),
			LOCTEXT("RigidMeshDI_GlobalSearch_Summary", "Deprecated Global Search."),
			FNiagaraDataInterfaceFix::CreateLambda(
				[this]()
				{
					GlobalSearchAllowed = false;
					GlobalSearchForced = false;
					GlobalSearchFallback_Unscripted = false;
					return true;
				}
			)
		);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#endif


void UNiagaraDIRigidMeshCollisionFunctionLibrary::SetSourceActors(UNiagaraComponent* NiagaraComponent, FName OverrideName, const TArray<AActor*>& InSourceActors)
{
	if (UNiagaraDataInterfaceRigidMeshCollisionQuery* QueryDI = UNiagaraFunctionLibrary::GetDataInterface<UNiagaraDataInterfaceRigidMeshCollisionQuery>(NiagaraComponent, OverrideName))
	{
		QueryDI->SourceActors.Reset(InSourceActors.Num());
		Algo::Transform(InSourceActors, QueryDI->SourceActors, [&](AActor* Actor)
		{
			return Actor;
		});
	}
}

#undef LOCTEXT_NAMESPACE
