// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSkeletalMesh.h"

#include "Animation/SkeletalMeshActor.h"
#include "Animation/Skeleton.h"
#include "Async/ParallelFor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/Canvas.h"
#include "Internationalization/Internationalization.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "NDISkeletalMeshCommon.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraComponent.h"
#include "NiagaraDataInterfaceMeshCommon.h"
#include "NiagaraDataInterfaceSkeletalMeshConnectivity.h"
#include "Experimental/NiagaraDataInterfaceSkeletalMeshUvMapping.h"
#include "NiagaraEmitter.h"
#include "NiagaraSettings.h"
#include "NiagaraStats.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraRenderer.h"
#include "NiagaraScript.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraWorldManager.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Templates/AlignmentTemplates.h"
#include "ShaderCore.h"
#include "CachedGeometry.h"
#include "SkeletalRenderPublic.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceSkeletalMesh)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceSkeletalMesh"

DECLARE_CYCLE_STAT(TEXT("PreSkin"), STAT_NiagaraSkel_PreSkin, STATGROUP_Niagara);

struct FNiagaraSkelMeshDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		AddedRandomInfo = 1,
		CleanUpVertexSampling = 2,
		CleanupBoneSampling = 3,
		AddTangentBasisToGetSkinnedVertexData = 4,
		RemoveUvSetFromMapping = 5,
		AddedEnabledUvMapping = 6,
		AddedValidConnectivity = 7,
		AddedInputBardCoordToGetFilteredTriangleAt = 8,
		LargeWorldCoordinates = 9,
		LargeWorldCoordinates2 = 10,
		AddBoneScale = 11,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

namespace NDISkelMeshLocal
{
	static bool GUseNewResetRequired = true;
	static FAutoConsoleVariableRef CUseNewResetRequired(
		TEXT("fx.Niagara.Rollback.SkeletalMeshDI.UseNewResetRequired"),
		GUseNewResetRequired,
		TEXT("Rollback console variable to use the old reset required detection vs the new one which is more concise."),
		ECVF_Default
	);

	static FName NAME_GetPreSkinnedLocalBounds("GetPreSkinnedLocalBounds");

	static const TCHAR* CommonShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceSkeletalMesh.ush");
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceSkeletalMeshTemplate.ush");

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
		SHADER_PARAMETER_SRV(Buffer<uint>,		MeshIndexBuffer)
		SHADER_PARAMETER_SRV(Buffer<float>,		MeshVertexBuffer)
		SHADER_PARAMETER_SRV(Buffer<uint>,		MeshSkinWeightBuffer)
		SHADER_PARAMETER_SRV(Buffer<uint>,		MeshSkinWeightLookupBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	MeshCurrBonesBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	MeshPrevBonesBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	MeshCurrSamplingBonesBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	MeshPrevSamplingBonesBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	MeshTangentBuffer)
		SHADER_PARAMETER_SRV(Buffer<float2>,	MeshTexCoordBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	MeshColorBuffer)
		SHADER_PARAMETER_SRV(Buffer<uint>,		MeshTriangleSamplerProbAliasBuffer)
		SHADER_PARAMETER(uint32,				MeshNumSamplingRegionTriangles)
		SHADER_PARAMETER(uint32,				MeshNumSamplingRegionVertices)
		SHADER_PARAMETER_SRV(Buffer<uint>,		MeshSamplingRegionsProbAliasBuffer)
		SHADER_PARAMETER_SRV(Buffer<uint>,		MeshSampleRegionsTriangleIndices)
		SHADER_PARAMETER_SRV(Buffer<uint>,		MeshSampleRegionsVertices)
		SHADER_PARAMETER_SRV(Buffer<uint>,		MeshTriangleMatricesOffsetBuffer)
		SHADER_PARAMETER(uint32,				MeshTriangleCount)
		SHADER_PARAMETER(uint32,				MeshVertexCount)
		SHADER_PARAMETER(uint32,				MeshNumTexCoord)
		SHADER_PARAMETER(uint32,				MeshNumWeights)
		SHADER_PARAMETER(uint32,				MeshBoneWeightStride)
		SHADER_PARAMETER(uint32,				MeshBoneIndexSizeBytes)
		SHADER_PARAMETER(uint32,				MeshBoneWeightSizeBytes)
		SHADER_PARAMETER(int,					NumBones)
		SHADER_PARAMETER(int,					NumFilteredBones)
		SHADER_PARAMETER(int,					NumUnfilteredBones)
		SHADER_PARAMETER(int,					RandomMaxBone)
		SHADER_PARAMETER(int,					ExcludeBoneIndex)
		SHADER_PARAMETER_SRV(Buffer<uint>,		FilteredAndUnfilteredBones)
		SHADER_PARAMETER(int,					NumFilteredSockets)
		SHADER_PARAMETER(int,					FilteredSocketBoneOffset)
		SHADER_PARAMETER_SRV(Buffer<int>,		UvMappingBuffer)
		SHADER_PARAMETER(uint32,				UvMappingBufferLength)
		SHADER_PARAMETER(uint32,				UvMappingSet)
		SHADER_PARAMETER_SRV(Buffer<uint>,		ConnectivityBuffer)
		SHADER_PARAMETER(uint32,				ConnectivityBufferLength)
		SHADER_PARAMETER(uint32,				ConnectivityMaxAdjacentPerVertex)
		SHADER_PARAMETER(FMatrix44f,			InstanceTransform)
		SHADER_PARAMETER(FMatrix44f,			InstancePrevTransform)
		SHADER_PARAMETER(FQuat4f,				InstanceRotation)
		SHADER_PARAMETER(FQuat4f,				InstancePrevRotation)
		SHADER_PARAMETER(float,					InstanceInvDeltaTime)
		SHADER_PARAMETER(FVector3f,				PreSkinnedLocalBoundsCenter)
		SHADER_PARAMETER(FVector3f,				PreSkinnedLocalBoundsExtents)
		SHADER_PARAMETER(uint32,				EnabledFeatures)

		SHADER_PARAMETER_SRV(Buffer<float>,		DeformedCurrPositionBuffer)
		SHADER_PARAMETER_SRV(Buffer<float>,		DeformedPrevPositionBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	DeformedTangentBuffer)
	END_SHADER_PARAMETER_STRUCT()

	int32 GetProbAliasDWORDSize(int32 TriangleCount)
	{
		const ENDISkelMesh_GpuUniformSamplingFormat::Type Format = GetDefault<UNiagaraSettings>()->NDISkelMesh_GpuUniformSamplingFormat;
		switch (Format)
		{
			case ENDISkelMesh_GpuUniformSamplingFormat::Full:
				return TriangleCount * 2;
			case ENDISkelMesh_GpuUniformSamplingFormat::Limited_24_8:
			case ENDISkelMesh_GpuUniformSamplingFormat::Limited_23_9:
				return TriangleCount * 1;
			default:
				UE_LOGF(LogNiagara, Fatal, "GpuUniformSamplingFormat %d is invalid", Format);
				return 0;
		}
	}

	void PackProbAlias(uint32* Dest, const FSkeletalMeshAreaWeightedTriangleSampler& triangleSampler, int32 AliasOffset = 0)
	{
		TArrayView<const float> ProbArray = triangleSampler.GetProb();
		TArrayView<const int32> AliasArray = triangleSampler.GetAlias();

		const ENDISkelMesh_GpuUniformSamplingFormat::Type Format = GetDefault<UNiagaraSettings>()->NDISkelMesh_GpuUniformSamplingFormat;
		switch (Format)
		{
			case ENDISkelMesh_GpuUniformSamplingFormat::Full:
			{
				for (int32 i = 0; i < triangleSampler.GetNumEntries(); ++i)
				{
					const float Probability = ProbArray[i];
					const int32 Alias = AliasArray[i] + AliasOffset;
					* Dest++ = *reinterpret_cast<const uint32*>(&Probability);
					*Dest++ = Alias;
				}
				break;
			}
			case ENDISkelMesh_GpuUniformSamplingFormat::Limited_24_8:
			{
				for (int32 i = 0; i < triangleSampler.GetNumEntries(); ++i)
				{
					const float Probability = ProbArray[i];
					const int32 Alias = AliasArray[i] + AliasOffset;
					if (ensureMsgf(Alias <= 0xffffff, TEXT("Triangle Alias %d is higher than possible %d"), Alias, 0xffffff))
					{
						*Dest++ = (Alias << 8) | (int(FMath::Clamp(Probability, 0.0f, 1.0f) * 255.0f) & 0xff);
					}
					else
					{
						*Dest++ = 0;
					}
				}

				break;
			}
			case ENDISkelMesh_GpuUniformSamplingFormat::Limited_23_9:
			{
				for (int32 i = 0; i < triangleSampler.GetNumEntries(); ++i)
				{
					const float Probability = ProbArray[i];
					const int32 Alias = AliasArray[i] + AliasOffset;
					if (ensureMsgf(Alias <= 0x7fffff, TEXT("Triangle Alias %d is higher than possible %d"), Alias, 0x7fffff))
					{
						*Dest++ = (Alias << 9) | (int(FMath::Clamp(Probability, 0.0f, 1.0f) * 511.0f) & 0x1ff);
					}
					else
					{
						*Dest++ = 0;
					}
				}
				break;
			}
			default:
				UE_LOGF(LogNiagara, Fatal, "GpuUniformSamplingFormat %d is invalid", Format);
				break;
		}
	}

	// Calculate which tick group the skeletal mesh component will be ready by
	ETickingGroup GetComponentTickGroup(USkinnedMeshComponent* SkinnedComponent, USkeletalMeshComponent* SkeletalComponent)
	{
		const ETickingGroup ComponentTickGroup = FMath::Max(SkinnedComponent->PrimaryComponentTick.TickGroup, SkinnedComponent->PrimaryComponentTick.EndTickGroup);
		const ETickingGroup PhysicsTickGroup = SkeletalComponent && SkeletalComponent->bBlendPhysics ? FMath::Max(ComponentTickGroup, TG_EndPhysics) : ComponentTickGroup;
		const ETickingGroup ClampedTickGroup = FMath::Clamp(ETickingGroup(PhysicsTickGroup + 1), NiagaraFirstTickGroup, NiagaraLastTickGroup);
		return ClampedTickGroup;
	}

	// Returns a pair where first is the number of lods and second is the first available LOD
	static const TTuple<int32, int32> GetStreamingLODData(const USkinnedAsset* SkinnedMesh)
	{
		const FStreamableRenderResourceState& SRRState = SkinnedMesh->GetStreamableResourceState();
		if (SRRState.IsValid())
		{
			const int32 NumValidLODs = FMath::Min(SRRState.NumRequestedLODs, SRRState.NumResidentLODs);
			const int32 CurrentFirstLOD = SRRState.LODCountToAssetFirstLODIdx(NumValidLODs);
			return MakeTuple(NumValidLODs, CurrentFirstLOD);
		}
		else if (FSkeletalMeshRenderData* RenderData = SkinnedMesh->GetResourceForRendering())
		{
			return MakeTuple(RenderData->LODRenderData.Num(), 0);
		}
		return MakeTuple(0, 0);
	}
}

//////////////////////////////////////////////////////////////////////////

FSkeletalMeshSamplingRegionAreaWeightedSampler::FSkeletalMeshSamplingRegionAreaWeightedSampler()
	: Owner(nullptr)
{
}

void FSkeletalMeshSamplingRegionAreaWeightedSampler::Init(FNDISkeletalMesh_InstanceData* InOwner)
{
	Owner = InOwner;
	Initialize();
}

float FSkeletalMeshSamplingRegionAreaWeightedSampler::GetWeights(TArray<float>& OutWeights)
{
	check(Owner);

	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Owner->WeakSkinnedMesh.Get());
	if (SkeletalMesh == nullptr)
	{
		OutWeights.Empty();
		return 0.0f;
	}

	check(SkeletalMesh->IsValidLODIndex(Owner->GetLODIndex()));

	float Total = 0.0f;
	int32 NumUsedRegions = Owner->SamplingRegionIndices.Num();
	if (NumUsedRegions <= 1)
	{
		//Use 0 or 1 Sampling region. Only need additional area weighting between regions if we're sampling from multiple.
		OutWeights.Empty();
		return 0.0f;
	}

	const FSkeletalMeshSamplingInfo& SamplingInfo = SkeletalMesh->GetSamplingInfo();
	OutWeights.Empty(NumUsedRegions);
	for (int32 i = 0; i < NumUsedRegions; ++i)
	{
		int32 RegionIdx = Owner->SamplingRegionIndices[i];
		float T = SamplingInfo.GetRegionBuiltData(RegionIdx).AreaWeightedSampler.GetTotalWeight();
		OutWeights.Add(T);
		Total += T;
	}
	return Total;
}

//////////////////////////////////////////////////////////////////////////
FSkeletalMeshSkinningDataHandle::FSkeletalMeshSkinningDataHandle()
	: SkinningData(nullptr)
{
}

FSkeletalMeshSkinningDataHandle::FSkeletalMeshSkinningDataHandle(FSkeletalMeshSkinningDataUsage InUsage, const TSharedPtr<FSkeletalMeshSkinningData>& InSkinningData, bool bNeedsDataImmediately)
	: Usage(InUsage)
	, SkinningData(InSkinningData)
{
	if (FSkeletalMeshSkinningData* SkinData = SkinningData.Get())
	{
		SkinData->RegisterUser(Usage, bNeedsDataImmediately);
	}
}

FSkeletalMeshSkinningDataHandle::~FSkeletalMeshSkinningDataHandle()
{
	if (FSkeletalMeshSkinningData* SkinData = SkinningData.Get())
	{
		SkinData->UnregisterUser(Usage);
	}
}

FSkeletalMeshSkinningDataHandle::FSkeletalMeshSkinningDataHandle(FSkeletalMeshSkinningDataHandle&& Other) noexcept
{
	Usage = Other.Usage;
	SkinningData = Other.SkinningData;
	Other.SkinningData = nullptr;
}

FSkeletalMeshSkinningDataHandle& FSkeletalMeshSkinningDataHandle::operator=(FSkeletalMeshSkinningDataHandle&& Other) noexcept
{
	if (this != &Other)
	{
		Usage = Other.Usage;
		SkinningData = Other.SkinningData;
		Other.SkinningData = nullptr;
	}
	return *this;
}

//////////////////////////////////////////////////////////////////////////
void FSkeletalMeshSkinningData::ForceDataRefresh()
{
	FRWScopeLock Lock(RWGuard, SLT_Write);
	bForceDataRefresh = true;
}

void FSkeletalMeshSkinningData::RegisterUser(FSkeletalMeshSkinningDataUsage Usage, bool bNeedsDataImmediately)
{
	FRWScopeLock Lock(RWGuard, SLT_Write);

	USkinnedMeshComponent* SkinnedComponent = WeakMeshComponent.Get();
	check(SkinnedComponent);

	USkinnedAsset* SkinnedMesh = SkinnedComponent->GetSkinnedAsset();
	int32 LODIndex = 0;
	int32 NumLODInfo = 1;

	if (SkinnedMesh != nullptr)
	{
		NumLODInfo = SkinnedMesh->GetLODNum();
		LODIndex = Usage.GetLODIndex();
		check(LODIndex != INDEX_NONE);
		check(LODIndex < NumLODInfo);
	}

	LODData.SetNum(NumLODInfo);

	if (Usage.NeedBoneMatrices())
	{
		++BoneMatrixUsers;
	}

	FLODData& LOD = LODData[LODIndex];
	if (Usage.NeedPreSkinnedVerts())
	{
		++LOD.PreSkinnedVertsUsers;
		++TotalPreSkinnedVertsUsers;
	}

	if (bNeedsDataImmediately)
	{
		check(IsInGameThread());
		const int32 NumExpectedTransforms = SkinnedMesh ? SkinnedMesh->GetRefBasesInvMatrix().Num() : 0;
		if (CurrBoneRefToLocals().Num() != NumExpectedTransforms || CurrComponentTransforms().Num() != NumExpectedTransforms)
		{
			UpdateBoneTransforms();
		}

		//Prime the prev matrices if they're missing.
		if (PrevBoneRefToLocals().Num() != CurrBoneRefToLocals().Num())
		{
			PrevBoneRefToLocals() = CurrBoneRefToLocals();
		}

		if (PrevComponentTransforms().Num() != CurrComponentTransforms().Num())
		{
			PrevComponentTransforms() = CurrComponentTransforms();
		}

		if (Usage.NeedPreSkinnedVerts() && CurrSkinnedPositions(LODIndex).Num() == 0 && SkinnedMesh && SkinnedMesh->GetLODInfo(LODIndex)->bAllowCPUAccess)
		{
			TRefCountPtr<const FSkeletalMeshLODRenderData> SkelMeshLODData = &SkinnedMesh->GetResourceForRendering()->LODRenderData[LODIndex];
			FSkinWeightVertexBuffer* SkinWeightBuffer = SkinnedComponent->GetSkinWeightBuffer(LODIndex);
			if (USkeletalMeshComponent* SkeletalComponent = Cast<USkeletalMeshComponent>(SkinnedComponent))
			{
				USkeletalMeshComponent::ComputeSkinnedPositions(SkeletalComponent, CurrSkinnedPositions(LODIndex), CurrBoneRefToLocals(), *SkelMeshLODData, *SkinWeightBuffer);
				USkeletalMeshComponent::ComputeSkinnedTangentBasis(SkeletalComponent, CurrSkinnedTangentBasis(LODIndex), CurrBoneRefToLocals(), *SkelMeshLODData, *SkinWeightBuffer);
			}
			else
			{
				//-TODO: Not supported currently
				//USkinnedMeshComponent::ComputeSkinnedPositions(SkinnedComponent, CurrSkinnedPositions(LODIndex), CurrBoneRefToLocals(), SkelMeshLODData, *SkinWeightBuffer);
				//USkinnedMeshComponent::ComputeSkinnedTangentBasis(SkinnedComponent, CurrSkinnedTangentBasis(LODIndex), CurrBoneRefToLocals(), SkelMeshLODData, *SkinWeightBuffer);
				const int32 NumVertices = SkelMeshLODData->GetNumVertices();

				TArray<FVector3f>& OutPositions = CurrSkinnedPositions(LODIndex);
				TArray<FVector3f>& OutTangents = CurrSkinnedTangentBasis(LODIndex);

				OutPositions.SetNumUninitialized(NumVertices);
				OutTangents.SetNumUninitialized(NumVertices * 3);
				for (int32 i = 0; i < NumVertices; ++i)
				{
					OutPositions[i] = FVector3f::ZeroVector;
					OutTangents[i * 3 + 0] = FVector3f::XAxisVector;
					OutTangents[i * 3 + 1] = FVector3f::YAxisVector;
					OutTangents[i * 3 + 2] = FVector3f::ZAxisVector;
				}
			}

			//Prime the previous positions if they're missing
			if (PrevSkinnedPositions(LODIndex).Num() != CurrSkinnedPositions(LODIndex).Num())
			{
				PrevSkinnedPositions(LODIndex) = CurrSkinnedPositions(LODIndex);
			}
			if (PrevSkinnedTangentBasis(LODIndex).Num() != CurrSkinnedTangentBasis(LODIndex).Num())
			{
				PrevSkinnedTangentBasis(LODIndex) = CurrSkinnedTangentBasis(LODIndex);
			}
		}
	}
}

void FSkeletalMeshSkinningData::UnregisterUser(FSkeletalMeshSkinningDataUsage Usage)
{
	FRWScopeLock Lock(RWGuard, SLT_Write);

	if (Usage.NeedBoneMatrices())
	{
		--BoneMatrixUsers;
	}

	int32 LODIndex = 0;

	USkinnedMeshComponent* SkinnedComp = WeakMeshComponent.Get();
	if (SkinnedComp && SkinnedComp->GetSkinnedAsset())
	{
		LODIndex = Usage.GetLODIndex();
	}

	// The first Niagara instance that detects a change to the SkeletalMeshComponent's SkeletalMesh will execute a
	// Unregister / Register which will modify the LOD count to the correct new count.  This means that a subsequent
	// Unregister call can be pointing to a LODIndex that is no longer valid.  We can safely ignore this as we do not
	// need to decrement the counters.
	if (Usage.NeedPreSkinnedVerts())
	{
		--TotalPreSkinnedVertsUsers;
		if (LODData.IsValidIndex(LODIndex))
		{
			--LODData[LODIndex].PreSkinnedVertsUsers;
		}
	}
}

bool FSkeletalMeshSkinningData::IsUsed() const
{
	if (BoneMatrixUsers > 0)
	{
		return true;
	}

	for (const FLODData& LOD : LODData)
	{
		if (LOD.PreSkinnedVertsUsers > 0)
		{
			return true;
		}
	}

	return false;
}

void FSkeletalMeshSkinningData::UpdateBoneTransforms()
{
	USkinnedMeshComponent* SkinnedComponent = WeakMeshComponent.Get();
	check(SkinnedComponent);

	const USkinnedAsset* SkinnedMesh = SkinnedComponent->GetSkinnedAsset();
	if (SkinnedMesh == nullptr)
	{
		return;
	}

	TArray<FMatrix44f>& CurrBones = CurrBoneRefToLocals();
	TArray<FTransform3f>& CurrTransforms = CurrComponentTransforms();

	if (USkinnedMeshComponent* LeaderComponent = SkinnedComponent->LeaderPoseComponent.Get())
	{
		const TArray<int32>& LeaderBoneMap = SkinnedComponent->GetLeaderBoneMap();
		const int32 NumBones = LeaderBoneMap.Num();

		if (NumBones == 0)
		{
			// This case indicates an invalid leader pose component (e.g. no skinned mesh)
			const int32 RefNumBones = SkinnedMesh->GetRefSkeleton().GetNum();
			CurrBones.Empty(RefNumBones);
			CurrBones.AddDefaulted(RefNumBones);
			CurrTransforms.Empty(RefNumBones);
			CurrTransforms.AddDefaulted(RefNumBones);
		}
		else
		{
			CurrBones.SetNumUninitialized(NumBones);
			CurrTransforms.SetNumUninitialized(NumBones);

			const TArray<FTransform>& LeaderTransforms = LeaderComponent->GetComponentSpaceTransforms();
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				bool bFoundLeader = false;
				if (LeaderBoneMap.IsValidIndex(BoneIndex))
				{
					const int32 LeaderIndex = LeaderBoneMap[BoneIndex];
					if (LeaderIndex != INDEX_NONE && LeaderIndex < LeaderTransforms.Num())
					{
						bFoundLeader = true;
						CurrTransforms[BoneIndex] = (FTransform3f)LeaderTransforms[LeaderIndex];
					}
				}

				if ( !bFoundLeader)
				{
					const int32 ParentIndex = SkinnedMesh->GetRefSkeleton().GetParentIndex(BoneIndex);
					FTransform3f BoneTransform = FTransform3f(SkinnedMesh->GetRefSkeleton().GetRefBonePose()[BoneIndex]);
					if ((ParentIndex >= 0) && (ParentIndex < BoneIndex))
					{
						BoneTransform = BoneTransform * CurrTransforms[ParentIndex];
					}
					CurrTransforms[BoneIndex] = BoneTransform;
				}

				if (SkinnedMesh->GetRefBasesInvMatrix().IsValidIndex(BoneIndex))
				{
					CurrBones[BoneIndex] = SkinnedMesh->GetRefBasesInvMatrix()[BoneIndex] * CurrTransforms[BoneIndex].ToMatrixWithScale();
				}
				else
				{
					CurrBones[BoneIndex] = CurrTransforms[BoneIndex].ToMatrixWithScale();
				}
			}
		}
	}
	else
	{
		SkinnedComponent->CacheRefToLocalMatrices(CurrBones);
		CurrTransforms = UE::LWC::ConvertArrayType<FTransform3f>(SkinnedComponent->GetComponentSpaceTransforms());	// LWC_TODO: Perf pessimization
	}
}

bool FSkeletalMeshSkinningData::Tick(float InDeltaSeconds, bool bRequirePreskin)
{
	FRWScopeLock Lock(RWGuard, SLT_Write);

	USkinnedMeshComponent* SkinnedComponent = WeakMeshComponent.Get();
	check(SkinnedComponent);

	DeltaSeconds = InDeltaSeconds;
	CurrIndex ^= 1;

	if (BoneMatrixUsers > 0)
	{
		UpdateBoneTransforms();
	}

	//Prime the prev matrices if they're missing.
	if ((PrevBoneRefToLocals().Num() != CurrBoneRefToLocals().Num()) || bForceDataRefresh)
	{
		PrevBoneRefToLocals() = CurrBoneRefToLocals();
	}

	if ((PrevComponentTransforms().Num() != CurrComponentTransforms().Num()) || bForceDataRefresh)
	{
		PrevComponentTransforms() = CurrComponentTransforms();
	}

	if (bRequirePreskin && SkinnedComponent->GetSkinnedAsset() != nullptr)
	{
		USkeletalMeshComponent* SkeletalComponent = Cast<USkeletalMeshComponent>(SkinnedComponent);
		const USkinnedAsset* SkinnedMesh = SkinnedComponent->GetSkinnedAsset();
		const FSkeletalMeshRenderData* RenderData = SkinnedMesh->GetResourceForRendering();
		check(RenderData);

		for (int32 LODIndex = RenderData->PendingFirstLODIdx; LODIndex < LODData.Num(); ++LODIndex)
		{
			FLODData& LOD = LODData[LODIndex];
			if (LOD.PreSkinnedVertsUsers > 0 && SkinnedMesh->GetLODInfo(LODIndex)->bAllowCPUAccess)
			{
				// Increment ref count to prevent stream out for happening while we are processing the CPU data.
				TRefCountPtr<const FSkeletalMeshLODRenderData> SkelMeshLODData = &RenderData->LODRenderData[LODIndex];

				if (SkeletalComponent)
				{
					//TODO: If we pass the sections in the usage too, we can probably skin a minimal set of verts just for the used regions.
					FSkinWeightVertexBuffer* SkinWeightBuffer = SkeletalComponent->GetSkinWeightBuffer(LODIndex);
					USkeletalMeshComponent::ComputeSkinnedPositions(SkeletalComponent, CurrSkinnedPositions(LODIndex), CurrBoneRefToLocals(), *SkelMeshLODData, *SkinWeightBuffer);
					USkeletalMeshComponent::ComputeSkinnedTangentBasis(SkeletalComponent, CurrSkinnedTangentBasis(LODIndex), CurrBoneRefToLocals(), *SkelMeshLODData, *SkinWeightBuffer);
					//check(CurrSkinnedPositions(LODIndex).Num() == SkelMeshLODData.NumVertices);
				}
				else
				{
					//-TODO: Not supported currently
					//USkinnedMeshComponent::ComputeSkinnedPositions(SkinnedComponent, CurrSkinnedPositions(LODIndex), CurrBoneRefToLocals(), SkelMeshLODData, *SkinWeightBuffer);
					//USkinnedMeshComponent::ComputeSkinnedTangentBasis(SkinnedComponent, CurrSkinnedTangentBasis(LODIndex), CurrBoneRefToLocals(), SkelMeshLODData, *SkinWeightBuffer);
					const int32 NumVertices = SkelMeshLODData->GetNumVertices();

					TArray<FVector3f>& OutPositions = CurrSkinnedPositions(LODIndex);
					TArray<FVector3f>& OutTangents = CurrSkinnedTangentBasis(LODIndex);

					OutPositions.SetNumUninitialized(NumVertices);
					OutTangents.SetNumUninitialized(NumVertices * 3);
					for (int32 i = 0; i < NumVertices; ++i)
					{
						OutPositions[i] = FVector3f::ZeroVector;
						OutTangents[i * 3 + 0] = FVector3f::XAxisVector;
						OutTangents[i * 3 + 1] = FVector3f::YAxisVector;
						OutTangents[i * 3 + 2] = FVector3f::ZAxisVector;
					}
				}

				//Prime the previous positions if they're missing
				if (PrevSkinnedPositions(LODIndex).Num() != CurrSkinnedPositions(LODIndex).Num())
				{
					PrevSkinnedPositions(LODIndex) = CurrSkinnedPositions(LODIndex);
				}
				if (PrevSkinnedTangentBasis(LODIndex).Num() != CurrSkinnedTangentBasis(LODIndex).Num())
				{
					PrevSkinnedTangentBasis(LODIndex) = CurrSkinnedTangentBasis(LODIndex);
				}
			}
		}
	}

	bForceDataRefresh = false;
	return true;
}

//////////////////////////////////////////////////////////////////////////

FSkeletalMeshSkinningDataHandle FNDI_SkeletalMesh_GeneratedData::GetCachedSkinningData(TWeakObjectPtr<USkinnedMeshComponent> Component, FSkeletalMeshSkinningDataUsage Usage, bool bNeedsDataImmediately)
{
	check(Component.Get() != nullptr);

	// Attempt to Find data
	{
		FRWScopeLock ReadLock(CachedSkinningDataGuard, SLT_ReadOnly);
		if (TSharedPtr<FSkeletalMeshSkinningData>* Existing = CachedSkinningData.Find(Component))
		{
			return FSkeletalMeshSkinningDataHandle(Usage, *Existing, bNeedsDataImmediately);
		}
	}

	// We need to add
	FRWScopeLock WriteLock(CachedSkinningDataGuard, SLT_Write);
	return FSkeletalMeshSkinningDataHandle(
		Usage,
		CachedSkinningData.Add(Component, MakeShared<FSkeletalMeshSkinningData>(Component)),
		bNeedsDataImmediately);
}

void FNDI_SkeletalMesh_GeneratedData::Tick(ETickingGroup TickGroup, float DeltaSeconds)
{
	check(IsInGameThread());
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_PreSkin);

	FRWScopeLock WriteLock(CachedSkinningDataGuard, SLT_Write);

	// We may want to look at separating out how we manage the ticks here
	//-OPT: Move into different arrays per tick group, manage promotions, demotions, etc, or add ourselves as a subsequent of the component's tick
	TArray<TWeakObjectPtr<USkinnedMeshComponent>, TInlineAllocator<32>> ToRemove;
	TArray<FSkeletalMeshSkinningData*, TInlineAllocator<32>> ToTickBonesOnly;
	TArray<FSkeletalMeshSkinningData*, TInlineAllocator<32>> ToTickPreskin;
	const bool bForceTick = TickGroup == NiagaraLastTickGroup;

	ToTickBonesOnly.Reserve(CachedSkinningData.Num());
	ToTickPreskin.Reserve(CachedSkinningData.Num());

	for (TPair<TWeakObjectPtr<USkinnedMeshComponent>, TSharedPtr<FSkeletalMeshSkinningData>>& Pair : CachedSkinningData)
	{
		USkinnedMeshComponent* SkinnedComponent = Pair.Key.Get();
		TSharedPtr<FSkeletalMeshSkinningData>& SkinningData = Pair.Value;

		if ( TickGroup == NiagaraFirstTickGroup )
		{
			SkinningData->bHasTicked = false;
		}

		// Should remove?
		if ( (SkinnedComponent == nullptr) || SkinningData.IsUnique() || !SkinningData->IsUsed() )
		{
			ToRemove.Add(Pair.Key);
			continue;
		}

		if ( SkinningData->bHasTicked == true )
		{
			continue;
		}

		// Has ticked or can be ticked
		if (bForceTick == false)
		{
			USkeletalMeshComponent* SkeletalComponent = Cast<USkeletalMeshComponent>(SkinnedComponent);
			const ETickingGroup PrereqTickGroup = NDISkelMeshLocal::GetComponentTickGroup(SkinnedComponent, SkeletalComponent);
			if ( PrereqTickGroup > TickGroup )
			{
				continue;
			}
		}

		// We are going to tick this one
		SkinningData->bHasTicked = true;

		FSkeletalMeshSkinningData* SkinningDataPtr = SkinningData.Get();
		check(SkinningDataPtr);

		if (SkinningDataPtr->NeedPreSkinnedVerts())
		{
			ToTickPreskin.Add(SkinningDataPtr);
		}
		else
		{
			ToTickBonesOnly.Add(SkinningDataPtr);
		}
	}

	for (TWeakObjectPtr<USkinnedMeshComponent> Key : ToRemove)
	{
		CachedSkinningData.Remove(Key);
	}

	// First tick the meshes that don't need pre-skinning.
	// This prevents additional threading overhead when we don't need to pre-skin.
	for (int i = 0; i < ToTickBonesOnly.Num(); i++)
	{
		ToTickBonesOnly[i]->Tick(DeltaSeconds, false);
	}

	// Then tick the remaining meshes requiring pre-skinning in parallel
	if (ToTickPreskin.Num() != 0)
	{
		ParallelFor(ToTickPreskin.Num(), [&](int32 Index)
		{
			ToTickPreskin[Index]->Tick(DeltaSeconds, true);
		});
	}

	{ // handle any changes to the UV mappings
		FRWScopeLock UvMappingWriteLock(CachedUvMappingGuard, SLT_Write);

		TArray<int32, TInlineAllocator<32>> MappingsToRemove;

		const int32 MappingCount = CachedUvMapping.Num();

		for (int32 MappingIt = 0; MappingIt < MappingCount; ++MappingIt)
		{
			const TSharedPtr<FSkeletalMeshUvMapping>& UvMappingData = CachedUvMapping[MappingIt];;

			if (UvMappingData->CanBeDestroyed())
			{
				MappingsToRemove.Add(MappingIt);
			}
		}

		while (MappingsToRemove.Num())
		{
			CachedUvMapping.RemoveAtSwap(MappingsToRemove.Pop(EAllowShrinking::No));
		}
	}

	{ // handle any changes to the connectivity handles
		FRWScopeLock ConnectivityWriteLock(CachedConnectivityGuard, SLT_Write);

		TArray<int32, TInlineAllocator<32>> EntriesToRemove;

		const int32 EntryCount = CachedConnectivity.Num();

		for (int32 EntryIt = 0; EntryIt < EntryCount; ++EntryIt)
		{
			const TSharedPtr<FSkeletalMeshConnectivity>& ConnectivityData = CachedConnectivity[EntryIt];;

			if (ConnectivityData->CanBeDestroyed())
			{
				EntriesToRemove.Add(EntryIt);
			}
		}

		while (EntriesToRemove.Num())
		{
			CachedConnectivity.RemoveAtSwap(EntriesToRemove.Pop(EAllowShrinking::No));
		}
	}
}

FSkeletalMeshUvMappingHandle FNDI_SkeletalMesh_GeneratedData::GetCachedUvMapping(TWeakObjectPtr<USkinnedAsset> MeshObject, int32 InLodIndex, int32 InUvSetIndex, FMeshUvMappingUsage Usage, bool bNeedsDataImmediately)
{
	check(MeshObject.Get() != nullptr);

	if (!FSkeletalMeshUvMapping::IsValidMeshObject(MeshObject, InLodIndex, InUvSetIndex))
	{
		return FSkeletalMeshUvMappingHandle();
	}

	// Attempt to Find data
	auto MappingMatchPredicate = [&](const TSharedPtr<FSkeletalMeshUvMapping>& UvMapping)
	{
		return UvMapping->Matches(MeshObject, InLodIndex, InUvSetIndex);
	};

	{
		FRWScopeLock ReadLock(CachedUvMappingGuard, SLT_ReadOnly);
		if (TSharedPtr<FSkeletalMeshUvMapping>* Existing = CachedUvMapping.FindByPredicate(MappingMatchPredicate))
		{
			return FSkeletalMeshUvMappingHandle(Usage, *Existing, bNeedsDataImmediately);
		}
	}

	// We need to add
	FRWScopeLock WriteLock(CachedUvMappingGuard, SLT_Write);
	if (TSharedPtr<FSkeletalMeshUvMapping>* Existing = CachedUvMapping.FindByPredicate(MappingMatchPredicate))
	{
		return FSkeletalMeshUvMappingHandle(Usage, *Existing, bNeedsDataImmediately);
	}

	return FSkeletalMeshUvMappingHandle(
		Usage,
		CachedUvMapping.Add_GetRef(MakeShared<FSkeletalMeshUvMapping>(MeshObject, InLodIndex, InUvSetIndex)),
		bNeedsDataImmediately);
}


FSkeletalMeshConnectivityHandle FNDI_SkeletalMesh_GeneratedData::GetCachedConnectivity(TWeakObjectPtr<USkinnedAsset> MeshObject, int32 InLodIndex, FSkeletalMeshConnectivityUsage Usage, bool bNeedsDataImmediately)
{
	check(MeshObject.Get() != nullptr);

	if (!FSkeletalMeshConnectivity::IsValidMeshObject(MeshObject, InLodIndex))
	{
		return FSkeletalMeshConnectivityHandle();
	}

	// Attempt to Find data
	auto ConnectivityMatchPredicate = [&](const TSharedPtr<FSkeletalMeshConnectivity>& Connectivity)
	{
		return Connectivity->CanBeUsed(MeshObject, InLodIndex);
	};


	{
		FRWScopeLock ReadLock(CachedConnectivityGuard, SLT_ReadOnly);
		if (TSharedPtr<FSkeletalMeshConnectivity>* Existing = CachedConnectivity.FindByPredicate(ConnectivityMatchPredicate))
		{
			return FSkeletalMeshConnectivityHandle(Usage, *Existing, bNeedsDataImmediately);
		}
	}

	// We need to add
	FRWScopeLock WriteLock(CachedConnectivityGuard, SLT_Write);
	if (TSharedPtr<FSkeletalMeshConnectivity>* Existing = CachedConnectivity.FindByPredicate(ConnectivityMatchPredicate))
	{
		return FSkeletalMeshConnectivityHandle(Usage, *Existing, bNeedsDataImmediately);
	}

	return FSkeletalMeshConnectivityHandle(
		Usage,
		CachedConnectivity.Add_GetRef(MakeShared<FSkeletalMeshConnectivity>(MeshObject, InLodIndex)),
		bNeedsDataImmediately);
}

//////////////////////////////////////////////////////////////////////////
// FStaticMeshGpuSpawnBuffer


FSkeletalMeshGpuSpawnStaticBuffers::~FSkeletalMeshGpuSpawnStaticBuffers()
{
	//ValidSections.Empty();
}

void FSkeletalMeshGpuSpawnStaticBuffers::Initialise(FNDISkeletalMesh_InstanceData* InstData, int32 InLODIndex, const FSkeletalMeshLODRenderData& SkeletalMeshLODRenderData, const FSkeletalMeshSamplingLODBuiltData* MeshSamplingLODBuiltData, FNiagaraSystemInstance* SystemInstance)
{
	SkeletalMeshSamplingLODBuiltData = nullptr;
	bUseGpuUniformlyDistributedSampling = false;

	LODIndex = InLODIndex;
	LODRenderData = nullptr;
	TriangleCount = 0;
	VertexCount = 0;

	NumFilteredBones = 0;
	NumUnfilteredBones = 0;
	FilteredAndUnfilteredBonesArray.Empty();
	NumFilteredSockets = 0;
	FilteredSocketBoneOffset = 0;

	if (InstData)
	{
		WeakSceneComponent = InstData->WeakSceneComponent;

		SkeletalMeshSamplingLODBuiltData = MeshSamplingLODBuiltData;
		bUseGpuUniformlyDistributedSampling = InstData->bIsGpuUniformlyDistributedSampling;

		LODRenderData = &SkeletalMeshLODRenderData;
		TriangleCount = SkeletalMeshLODRenderData.MultiSizeIndexContainer.GetIndexBuffer()->Num() / 3;
		VertexCount = SkeletalMeshLODRenderData.GetNumVertices();
		
		// TODO: Bring these back when we can know if they are for sure sampling from them. Disabled for now to suppress log spam
		//if (TriangleCount == 0)
		//{
		//	UE_LOGF(LogNiagara, Warning, "FSkeletalMeshGpuSpawnStaticBuffers> TriangleCount(0) is invalid for SkelMesh(%ls) System(%ls)", *GetFullNameSafe(InstData->SkeletalMesh.Get()), *GetFullNameSafe(SystemInstance->GetSystem()));
		//}
		//if (VertexCount == 0)
		//{
		//	UE_LOGF(LogNiagara, Warning, "FSkeletalMeshGpuSpawnStaticBuffers> VertexCount(0) is invalid for SkelMesh(%ls) System(%ls)", *GetFullNameSafe(InstData->SkeletalMesh.Get()), *GetFullNameSafe(SystemInstance->GetSystem()));
		//}

		if (bUseGpuUniformlyDistributedSampling)
		{
			check(SkeletalMeshSamplingLODBuiltData != nullptr);
			const int32 NumAreaSamples = SkeletalMeshSamplingLODBuiltData->AreaWeightedTriangleSampler.GetNumEntries();
			if (NumAreaSamples != TriangleCount)
			{
				UE_LOGF(LogNiagara, Warning, "FSkeletalMeshGpuSpawnStaticBuffers> AreaWeighted Triangle Sampling Count (%d) does not match triangle count (%d), disabling uniform sampling for SkelMesh(%ls) System(%ls)", NumAreaSamples, TriangleCount, *GetFullNameSafe(InstData->WeakSkinnedMesh.Get()), *GetFullNameSafe(SystemInstance->GetSystem()));
				bUseGpuUniformlyDistributedSampling = false;
			}
		}

		// Copy filtered Bones / Socket data into arrays that the renderer will use to create read buffers
		//-TODO: Exclude setting up these arrays if we don't sample from them
		NumFilteredBones = InstData->NumFilteredBones;
		NumUnfilteredBones = InstData->NumUnfilteredBones;
		ExcludedBoneIndex = InstData->ExcludedBoneIndex;

		FilteredAndUnfilteredBonesArray.Reserve(InstData->FilteredAndUnfilteredBones.Num());
		for (uint16 v : InstData->FilteredAndUnfilteredBones)
		{
			FilteredAndUnfilteredBonesArray.Add(v);
		}

		NumFilteredSockets = InstData->FilteredSocketInfo.Num();
		FilteredSocketBoneOffset = InstData->FilteredSocketBoneOffset;

		// Create triangle / vertex region sampling data
		if (InstData->SamplingRegionIndices.Num() > 0)
		{
			if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(InstData->WeakSkinnedMesh.Get()))
			{
				const FSkeletalMeshSamplingInfo& SamplingInfo = SkeletalMesh->GetSamplingInfo();

				// Count required regions
				bSamplingRegionsAllAreaWeighted = true;
				NumSamplingRegionTriangles = 0;
				NumSamplingRegionVertices = 0;

				for (const int32 RegionIndex : InstData->SamplingRegionIndices)
				{
					const FSkeletalMeshSamplingRegionBuiltData& SamplingRegionBuildData = SamplingInfo.GetRegionBuiltData(RegionIndex);
					NumSamplingRegionTriangles += SamplingRegionBuildData.TriangleIndices.Num();
					NumSamplingRegionVertices += SamplingRegionBuildData.Vertices.Num();
					bSamplingRegionsAllAreaWeighted &= SamplingRegionBuildData.AreaWeightedSampler.GetNumEntries() == SamplingRegionBuildData.TriangleIndices.Num();
				}

				// Build buffers
				SampleRegionsProbAlias.AddUninitialized(NDISkelMeshLocal::GetProbAliasDWORDSize(NumSamplingRegionTriangles));
				SampleRegionsTriangleIndicies.Reserve(NumSamplingRegionTriangles);
				SampleRegionsVertices.Reserve(NumSamplingRegionVertices);

				int32 RegionOffset = 0;
				int32 PABufferOffset = 0;
				for (const int32 RegionIndex : InstData->SamplingRegionIndices)
				{
					const FSkeletalMeshSamplingRegionBuiltData& SamplingRegionBuildData = SamplingInfo.GetRegionBuiltData(RegionIndex);
					if (bSamplingRegionsAllAreaWeighted)
					{
						NDISkelMeshLocal::PackProbAlias(SampleRegionsProbAlias.GetData() + PABufferOffset, SamplingRegionBuildData.AreaWeightedSampler, RegionOffset);
						PABufferOffset += NDISkelMeshLocal::GetProbAliasDWORDSize(SamplingRegionBuildData.AreaWeightedSampler.GetNumEntries());
					}
					for (int v : SamplingRegionBuildData.TriangleIndices)
					{
						SampleRegionsTriangleIndicies.Add(v / 3);
					}
					for (int v : SamplingRegionBuildData.Vertices)
					{
						SampleRegionsVertices.Add(v);
					}
					RegionOffset += SamplingRegionBuildData.TriangleIndices.Num();
				}
			}
		}
	}
}

void FSkeletalMeshGpuSpawnStaticBuffers::InitRHI(FRHICommandListBase& RHICmdList)
{
	// As of today, the UI does not allow to cull specific section of a mesh so this data could be generated on the Mesh. But Section culling might be added later?
	// Also see https://jira.it.epicgames.net/browse/UE-69376 : we would need to know if GPU sampling of the mesh surface is needed or not on the mesh to be able to do that.
	// ALso today we do not know if an interface is create from a CPU or GPU emitter. So always allocate for now. Follow up in https://jira.it.epicgames.net/browse/UE-69375.

	MeshIndexBufferSRV = LODRenderData->MultiSizeIndexContainer.GetIndexBuffer()->GetSRV();
	MeshVertexBufferSRV = LODRenderData->StaticVertexBuffers.PositionVertexBuffer.GetSRV();
	MeshTangentBufferSRV = LODRenderData->StaticVertexBuffers.StaticMeshVertexBuffer.GetTangentsSRV();
	bMeshValid = MeshIndexBufferSRV.IsValid() && MeshVertexBufferSRV.IsValid() && MeshTangentBufferSRV.IsValid();
	NumWeights = LODRenderData->SkinWeightVertexBuffer.GetMaxBoneInfluences();

	if ( bMeshValid == false )
	{
		MeshIndexBufferSRV = FNiagaraRenderer::GetDummyUIntBuffer();
		MeshVertexBufferSRV = FNiagaraRenderer::GetDummyFloatBuffer();
		MeshTangentBufferSRV = FNiagaraRenderer::GetDummyFloat4Buffer();
		NumWeights = 0;
		TriangleCount = 0;
		VertexCount = 0;
		bUseGpuUniformlyDistributedSampling = false;
		NumSamplingRegionTriangles = 0;
		bSamplingRegionsAllAreaWeighted = false;

		SampleRegionsProbAlias.Empty();
		SampleRegionsTriangleIndicies.Empty();
		SampleRegionsVertices.Empty();
	}

	MeshColorBufferSRV = LODRenderData->StaticVertexBuffers.ColorVertexBuffer.GetColorComponentsSRV();
	bHasMeshColors = bMeshValid && MeshColorBufferSRV.IsValid();
	if (bHasMeshColors == false)
	{
		MeshColorBufferSRV = FNiagaraRenderer::GetDummyFloat4Buffer();
	}

	MeshTexCoordBufferSRV = LODRenderData->StaticVertexBuffers.StaticMeshVertexBuffer.GetTexCoordsSRV();
	if (MeshTexCoordBufferSRV.IsValid())
	{
		NumTexCoord = LODRenderData->StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
	}
	else
	{
		MeshTexCoordBufferSRV = FNiagaraRenderer::GetDummyFloat2Buffer();
		NumTexCoord = 0;
	}

	const uint32 SectionCount = LODRenderData->RenderSections.Num();

#if STATS
	ensure(GPUMemoryUsage == 0);
#endif

	if (bUseGpuUniformlyDistributedSampling)
	{
		const FSkeletalMeshAreaWeightedTriangleSampler& triangleSampler = SkeletalMeshSamplingLODBuiltData->AreaWeightedTriangleSampler;
		check(TriangleCount == triangleSampler.GetNumEntries());

		const uint32 SizeByte = NDISkelMeshLocal::GetProbAliasDWORDSize(TriangleCount) * sizeof(uint32);

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("FSkeletalMeshGpuSpawnStaticBuffers"), SizeByte)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		TRHIBufferInitializer<uint32> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

		NDISkelMeshLocal::PackProbAlias(Initializer.GetWritableData(), triangleSampler);

		BufferTriangleUniformSamplerProbAliasRHI = Initializer.Finalize();
		BufferTriangleUniformSamplerProbAliasSRV = RHICmdList.CreateShaderResourceView(
			BufferTriangleUniformSamplerProbAliasRHI, 
			FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R32_UINT)
		);
#if STATS
		GPUMemoryUsage += SizeByte;
#endif
	}
	else
	{
		BufferTriangleUniformSamplerProbAliasSRV = FNiagaraRenderer::GetDummyUIntBuffer();
	}

	// Prepare sampling regions (if we have any)
	SampleRegionsProbAliasSRV = FNiagaraRenderer::GetDummyUIntBuffer();
	SampleRegionsTriangleIndicesSRV = FNiagaraRenderer::GetDummyUIntBuffer();
	SampleRegionsVerticesSRV = FNiagaraRenderer::GetDummyUIntBuffer();
	if (NumSamplingRegionTriangles > 0)
	{
		if (bSamplingRegionsAllAreaWeighted)
		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("SampleRegionsProbAliasBuffer"), SampleRegionsProbAlias.Num() * SampleRegionsProbAlias.GetTypeSize())
				.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
				.SetInitActionResourceArray(&SampleRegionsProbAlias)
				.DetermineInitialState();

			SampleRegionsProbAliasBuffer = RHICmdList.CreateBuffer(CreateDesc);
			SampleRegionsProbAliasSRV = RHICmdList.CreateShaderResourceView(
				SampleRegionsProbAliasBuffer, 
				FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT)
			);
#if STATS
			GPUMemoryUsage += SampleRegionsProbAlias.Num() * SampleRegionsProbAlias.GetTypeSize();
#endif
		}
		else
		{
			SampleRegionsProbAliasSRV = FNiagaraRenderer::GetDummyUIntBuffer();
		}

		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("SampleRegionsTriangleIndicesBuffer"), SampleRegionsTriangleIndicies.Num() * SampleRegionsTriangleIndicies.GetTypeSize())
				.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
				.SetInitActionResourceArray(&SampleRegionsTriangleIndicies)
				.DetermineInitialState();

			SampleRegionsTriangleIndicesBuffer = RHICmdList.CreateBuffer(CreateDesc);
			SampleRegionsTriangleIndicesSRV = RHICmdList.CreateShaderResourceView(
				SampleRegionsTriangleIndicesBuffer, 
				FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT)
			);
#if STATS
			GPUMemoryUsage += SampleRegionsTriangleIndicies.Num() * SampleRegionsTriangleIndicies.GetTypeSize();
#endif
		}

		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("SampleRegionsVerticesBuffer"), SampleRegionsVertices.Num() * SampleRegionsVertices.GetTypeSize())
				.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
				.SetInitActionResourceArray(&SampleRegionsVertices)
				.DetermineInitialState();

			SampleRegionsVerticesBuffer = RHICmdList.CreateBuffer(CreateDesc);
			SampleRegionsVerticesSRV = RHICmdList.CreateShaderResourceView(
				SampleRegionsVerticesBuffer, 
				FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT)
			);
#if STATS
			GPUMemoryUsage += SampleRegionsVertices.Num() * SampleRegionsVertices.GetTypeSize();
#endif
		}
	}
	else
	{
		SampleRegionsProbAliasSRV = FNiagaraRenderer::GetDummyUIntBuffer();
		SampleRegionsTriangleIndicesSRV = FNiagaraRenderer::GetDummyUIntBuffer();
		SampleRegionsVerticesSRV = FNiagaraRenderer::GetDummyUIntBuffer();
	}

	// Prepare the vertex matrix lookup offset for each of the sections. This is needed because per vertex BlendIndicies are stored relatively to each Section used matrices.
	// And these offset per section need to point to the correct matrix according to each section BoneMap.
	// There is not section selection/culling in the interface so technically we could compute that array in the pipeline.
	if ( bMeshValid )
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("FSkeletalMeshGpuSpawnStaticBuffers"), VertexCount * sizeof(uint32))
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		TRHIBufferInitializer<uint32> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

		uint32 AccumulatedMatrixOffset = 0;
		for (uint32 s = 0; s < SectionCount; ++s)
		{
			const FSkelMeshRenderSection& Section = LODRenderData->RenderSections[s];
			const uint32 SectionBaseVertexIndex = Section.BaseVertexIndex;
			const uint32 SectionNumVertices = Section.NumVertices;
			for (uint32 SectionVertex = 0; SectionVertex < SectionNumVertices; ++SectionVertex)
			{
				Initializer[SectionBaseVertexIndex + SectionVertex] = AccumulatedMatrixOffset;
			}
			AccumulatedMatrixOffset += Section.BoneMap.Num();
		}

		BufferTriangleMatricesOffsetRHI = Initializer.Finalize();
		BufferTriangleMatricesOffsetSRV = RHICmdList.CreateShaderResourceView(
			BufferTriangleMatricesOffsetRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
#if STATS
		GPUMemoryUsage += VertexCount * sizeof(uint32);
#endif
	}
	else
	{
		BufferTriangleMatricesOffsetSRV = FNiagaraRenderer::GetDummyUIntBuffer();
	}

	// Create arrays for filtered bones / sockets
	if ( FilteredAndUnfilteredBonesArray.Num() > 0 )
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("FilteredAndUnfilteredBonesBuffer"), FilteredAndUnfilteredBonesArray.Num() * FilteredAndUnfilteredBonesArray.GetTypeSize())
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitActionResourceArray(&FilteredAndUnfilteredBonesArray)
			.DetermineInitialState();

		FilteredAndUnfilteredBonesBuffer = RHICmdList.CreateBuffer(CreateDesc);
		FilteredAndUnfilteredBonesSRV = RHICmdList.CreateShaderResourceView(
			FilteredAndUnfilteredBonesBuffer,
			FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R16_UINT)
		);
	}
	else
	{
		FilteredAndUnfilteredBonesSRV = FNiagaraRenderer::GetDummyUIntBuffer();
	}
#if STATS
	GPUMemoryUsage += FilteredAndUnfilteredBonesArray.Num() * FilteredAndUnfilteredBonesArray.GetTypeSize();
#endif

#if STATS
	INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GPUMemoryUsage);
#endif
}

void FSkeletalMeshGpuSpawnStaticBuffers::ReleaseRHI()
{
#if STATS
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GPUMemoryUsage);
	GPUMemoryUsage = 0;
#endif

	BufferTriangleMatricesOffsetRHI.SafeRelease();
	BufferTriangleMatricesOffsetSRV.SafeRelease();

	BufferTriangleUniformSamplerProbAliasRHI.SafeRelease();
	BufferTriangleUniformSamplerProbAliasSRV.SafeRelease();

	SampleRegionsProbAliasBuffer.SafeRelease();
	SampleRegionsProbAliasSRV.SafeRelease();
	SampleRegionsTriangleIndicesBuffer.SafeRelease();
	SampleRegionsTriangleIndicesSRV.SafeRelease();
	SampleRegionsVerticesBuffer.SafeRelease();
	SampleRegionsVerticesSRV.SafeRelease();

	FilteredAndUnfilteredBonesBuffer.SafeRelease();
	FilteredAndUnfilteredBonesSRV.SafeRelease();

	MeshVertexBufferSRV.SafeRelease();
	MeshIndexBufferSRV.SafeRelease();
	MeshTangentBufferSRV.SafeRelease();
	MeshTexCoordBufferSRV.SafeRelease();
	MeshColorBufferSRV.SafeRelease();
}

//////////////////////////////////////////////////////////////////////////
//FSkeletalMeshGpuSpawnProxy
void FSkeletalMeshGpuDynamicBufferProxy::Initialise(const FReferenceSkeleton& RefSkel, const FSkeletalMeshLODRenderData& SkeletalMeshLODRenderData, uint32 InSamplingSocketCount)
{
	SectionBoneCount = 0;
	for (const FSkelMeshRenderSection& Section : SkeletalMeshLODRenderData.RenderSections)
	{
		SectionBoneCount += Section.BoneMap.Num();
	}

	SamplingBoneCount = RefSkel.GetNum();
	SamplingSocketCount = InSamplingSocketCount;
}

void FSkeletalMeshGpuDynamicBufferProxy::InitRHI(FRHICommandListBase& RHICmdList)
{
#if STATS
	ensure(GPUMemoryUsage == 0);
#endif
	const uint32 SectionBufferNumElements	= 3u * FMath::Max(1u, SectionBoneCount);
	const uint32 SamplingBufferNumElements	= 3u * FMath::Max(1u, SamplingBoneCount + SamplingSocketCount);

	for (FSkeletalBuffer& Buffer : RWBufferBones)
	{
		const FRHIBufferCreateDesc SectionBufferDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector4f>(TEXT("SkeletalMeshGpuDynamicBuffer"), SectionBufferNumElements)
			.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Dynamic)
			.DetermineInitialState();

		Buffer.SectionBuffer = RHICmdList.CreateBuffer(SectionBufferDesc);
		Buffer.SectionSRV = RHICmdList.CreateShaderResourceView(
			Buffer.SectionBuffer, 
			FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_A32B32G32R32F)
		);

		const FRHIBufferCreateDesc SamplingBufferDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector4f>(TEXT("SkeletalMeshGpuDynamicBuffer"), SamplingBufferNumElements)
			.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Dynamic)
			.DetermineInitialState();

		Buffer.SamplingBuffer = RHICmdList.CreateBuffer(SamplingBufferDesc);
		Buffer.SamplingSRV = RHICmdList.CreateShaderResourceView(
			Buffer.SamplingBuffer, 
			FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_A32B32G32R32F)
		);

#if STATS
		GPUMemoryUsage += sizeof(FVector4f) * SectionBufferNumElements;
		GPUMemoryUsage += sizeof(FVector4f) * SamplingBufferNumElements;
#endif
	}
#if STATS
	INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GPUMemoryUsage);
#endif
}

void FSkeletalMeshGpuDynamicBufferProxy::ReleaseRHI()
{
#if STATS
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GPUMemoryUsage);
	GPUMemoryUsage = 0;
#endif
	for (FSkeletalBuffer& Buffer : RWBufferBones)
	{
		Buffer.SectionBuffer.SafeRelease();
		Buffer.SectionSRV.SafeRelease();

		Buffer.SamplingBuffer.SafeRelease();
		Buffer.SamplingSRV.SafeRelease();
	}
}

void FSkeletalMeshGpuDynamicBufferProxy::NewFrame(const FNDISkeletalMesh_InstanceData* InstanceData, int32 LODIndex)
{
	// Grab Skeletal Component / Mesh, we must have a mesh at minimum to set the data
	USkinnedMeshComponent* SkinnedComponent = nullptr;
	USkinnedAsset* SkinnedMesh = nullptr;
	if (InstanceData != nullptr)
	{
		SkinnedComponent = Cast<USkinnedMeshComponent>(InstanceData->WeakSceneComponent.Get());
		if (SkinnedComponent != nullptr )
		{
			SkinnedMesh = SkinnedComponent->GetSkinnedAsset();
		}
		if (SkinnedMesh == nullptr)
		{
			SkinnedMesh = InstanceData->WeakSkinnedMesh.Get();
		}
	}

	if (SkinnedMesh == nullptr )
	{
		return;
	}

	static_assert(sizeof(FVector4f) == 4 * sizeof(float), "FVector4f should match 4 * floats");

	TArray<FVector4f> AllSectionsRefToLocalMatrices;
	TArray<FVector4f> BoneSamplingData;

	auto FillBuffers =
		[&](const TArray<FTransform>& BoneTransforms, const FReferenceSkeleton* ReferenceSkeleton)
		{
			check(BoneTransforms.Num() == SamplingBoneCount);

			// Fill AllSectionsRefToLocalMatrices
			TIndirectArray<FSkeletalMeshLODRenderData>& LODRenderDataArray = SkinnedMesh->GetResourceForRendering()->LODRenderData;
			check(0 <= LODIndex && LODIndex < LODRenderDataArray.Num());
			FSkeletalMeshLODRenderData& LODRenderData = LODRenderDataArray[LODIndex];
			TArray<FSkelMeshRenderSection>& Sections = LODRenderData.RenderSections;

			// Count number of matrices we want before appending all of them according to the per section mapping from BoneMap
			uint32 Float4Count = 0;
			for (const FSkelMeshRenderSection& Section : Sections)
			{
				Float4Count += Section.BoneMap.Num() * 3;
			}
			check(Float4Count == 3 * SectionBoneCount);
			AllSectionsRefToLocalMatrices.AddUninitialized(Float4Count);

			Float4Count = 0;
			for (const FSkelMeshRenderSection& Section : Sections)
			{
				const uint32 MatrixCount = Section.BoneMap.Num();
				for (uint32 m=0; m < MatrixCount; ++m)
				{
					const int32 BoneIndex = Section.BoneMap[m];
					const FTransform& BoneTransform = BoneTransforms[BoneIndex];
					// LWC_TODO: precision loss
					const FMatrix44f BoneMatrix = SkinnedMesh->GetRefBasesInvMatrix().IsValidIndex(BoneIndex) ? SkinnedMesh->GetRefBasesInvMatrix()[BoneIndex] * (FMatrix44f)BoneTransform.ToMatrixWithScale() : (FMatrix44f)BoneTransform.ToMatrixWithScale();
					BoneMatrix.To3x4MatrixTranspose(&AllSectionsRefToLocalMatrices[Float4Count].X);
					Float4Count += 3;
				}
			}

			// Fill BoneSamplingData
			const TArray<FTransform3f>& FilteredSocketsCurrBuffer = InstanceData->GetFilteredSocketsCurrBuffer();
			const int32 BoneSamplingDataCount = (BoneTransforms.Num() + FilteredSocketsCurrBuffer.Num()) * 3;

			BoneSamplingData.Reserve(BoneSamplingDataCount);
			for (int i=0; i < BoneTransforms.Num(); ++i )
			{
				const FTransform& BoneTransform = BoneTransforms[i];
				const FQuat4f Rotation = FQuat4f(BoneTransform.GetRotation());
				const int32 ParentIndex = ReferenceSkeleton ? ReferenceSkeleton->GetParentIndex(i) : -1;
				BoneSamplingData.Emplace((FVector3f)BoneTransform.GetLocation()); // LWC_TODO: precision loss
				BoneSamplingData.Emplace(Rotation.X, Rotation.Y, Rotation.Z, Rotation.W);
				BoneSamplingData.Emplace_GetRef((FVector3f)BoneTransform.GetScale3D()).W = reinterpret_cast<const float&>(ParentIndex); // LWC_TODO: precision loss
			}

			// Append sockets
			for (const FTransform3f& SocketTransform : FilteredSocketsCurrBuffer)
			{
				const FQuat4f Rotation = SocketTransform.GetRotation();
				const int32 ParentIndex = -1;
				BoneSamplingData.Emplace(SocketTransform.GetLocation());
				BoneSamplingData.Emplace(Rotation.X, Rotation.Y, Rotation.Z, Rotation.W);
				BoneSamplingData.Emplace_GetRef(SocketTransform.GetScale3D()).W = reinterpret_cast<const float&>(ParentIndex);
			}
		};

	// If we have a component pull transforms from component otherwise grab from skel mesh
	if (SkinnedComponent)
	{
		if (USkinnedMeshComponent* LeaderComponent = SkinnedComponent->LeaderPoseComponent.Get())
		{
			const FReferenceSkeleton* ReferenceSkeleton = nullptr;
			const TArray<int32>& LeaderBoneMap = SkinnedComponent->GetLeaderBoneMap();
			const int32 NumBones = LeaderBoneMap.Num();

			TArray<FTransform> TempBoneTransforms;
			TempBoneTransforms.Reserve(SamplingBoneCount);

			if (NumBones == 0)
			{
				// This case indicates an invalid leader pose component (e.g. no skeletal mesh)
				TempBoneTransforms.AddDefaulted(SamplingBoneCount);
			}
			else
			{
				ReferenceSkeleton = &SkinnedMesh->GetRefSkeleton();
				const TArray<FTransform>& LeaderTransforms = LeaderComponent->GetComponentSpaceTransforms();
				for (int32 BoneIndex=0; BoneIndex < NumBones; ++BoneIndex)
				{
					if (LeaderBoneMap.IsValidIndex(BoneIndex))
					{
						const int32 LeaderIndex = LeaderBoneMap[BoneIndex];
						if (LeaderIndex != INDEX_NONE && LeaderIndex < LeaderTransforms.Num())
						{
							TempBoneTransforms.Add(LeaderTransforms[LeaderIndex]);
							continue;
						}
					}

					const int32 ParentIndex  = ReferenceSkeleton->GetParentIndex(BoneIndex);
					FTransform BoneTransform = ReferenceSkeleton->GetRefBonePose()[BoneIndex];
					if (TempBoneTransforms.IsValidIndex(ParentIndex))
					{
						BoneTransform = BoneTransform * TempBoneTransforms[ParentIndex];
					}
					TempBoneTransforms.Add(BoneTransform);
				}
			}
			FillBuffers(TempBoneTransforms, ReferenceSkeleton);
		}
		else
		{
			const TArray<FTransform>& ComponentTransforms = SkinnedComponent->GetComponentSpaceTransforms();
			if (ComponentTransforms.Num() > 0)
			{
				FillBuffers(ComponentTransforms, &SkinnedMesh->GetRefSkeleton());
			}
			else
			{
				// Trying to catch cause of this case in the wild. Not supposed to be possible with a valid skeletal mesh
				ensureMsgf(
					false,
					TEXT("NiagaraSkelMeshDI: Mesh has no ComponentSpaceTransforms. Component - %s (Registered: %s, Flags: %d), Mesh - %s (Flags: %d)"),
					*GetFullNameSafe(SkinnedComponent),
					SkinnedComponent && SkinnedComponent->IsRegistered() ? TEXT("Yes") : TEXT("No"),
					SkinnedComponent ? SkinnedComponent->GetFlags() : 0,
					*GetFullNameSafe(SkinnedMesh),
					SkinnedMesh ? SkinnedMesh->GetFlags() : 0
				);

				TArray<FTransform> TempBoneTransforms;
				TempBoneTransforms.AddDefaulted(SamplingBoneCount);
				FillBuffers(TempBoneTransforms, nullptr);
			}
		}
	}
	else
	{
		//-TODO: Opt and combine with MaterPoseComponent
		const FReferenceSkeleton* ReferenceSkeleton = &SkinnedMesh->GetRefSkeleton();
		TArray<FTransform> TempBoneTransforms;
		TempBoneTransforms.Reserve(SamplingBoneCount);

		const TArray<FTransform>& RefTransforms = SkinnedMesh->GetRefSkeleton().GetRefBonePose();
		for (int32 i=0; i < RefTransforms.Num(); ++i)
		{
			FTransform BoneTransform = RefTransforms[i];
			const int32 ParentIndex = ReferenceSkeleton->GetParentIndex(i);
			if (TempBoneTransforms.IsValidIndex(ParentIndex))
			{
				BoneTransform = BoneTransform * TempBoneTransforms[ParentIndex];
			}
			TempBoneTransforms.Add(BoneTransform);
		}

		FillBuffers(TempBoneTransforms, ReferenceSkeleton);
	}

	FSkeletalMeshGpuDynamicBufferProxy* ThisProxy = this;
	ENQUEUE_RENDER_COMMAND(UpdateSpawnInfoForSkinnedMesh)(
		[ThisProxy, AllSectionsRefToLocalMatrices = MoveTemp(AllSectionsRefToLocalMatrices), BoneSamplingData = MoveTemp(BoneSamplingData)](FRHICommandListImmediate& RHICmdList) mutable
		{
			ThisProxy->CurrentBoneBufferId = (ThisProxy->CurrentBoneBufferId + 1) % BufferBoneCount;
			ThisProxy->bPrevBoneGpuBufferValid = ThisProxy->bBoneGpuBufferValid;
			ThisProxy->bBoneGpuBufferValid = true;

			// Copy bone remap data matrices
			{
				const uint32 NumBytes = AllSectionsRefToLocalMatrices.Num() * sizeof(FVector4f);
				if (NumBytes > 0)
				{
					void* DstData = RHICmdList.LockBuffer(ThisProxy->GetRWBufferBone().SectionBuffer, 0, NumBytes, RLM_WriteOnly);
					FMemory::Memcpy(DstData, AllSectionsRefToLocalMatrices.GetData(), NumBytes);
					RHICmdList.UnlockBuffer(ThisProxy->GetRWBufferBone().SectionBuffer);
				}
			}

			// Copy bone sampling data
			{
				const uint32 NumBytes = BoneSamplingData.Num() * sizeof(FVector4f);
				if (NumBytes > 0)
				{
					FVector4f* DstData = reinterpret_cast<FVector4f*>(RHICmdList.LockBuffer(ThisProxy->GetRWBufferBone().SamplingBuffer, 0, NumBytes, RLM_WriteOnly));
					FMemory::Memcpy(DstData, BoneSamplingData.GetData(), NumBytes);
					RHICmdList.UnlockBuffer(ThisProxy->GetRWBufferBone().SamplingBuffer);
				}
			}
		}
	);
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraDataInterfaceProxySkeletalMesh::ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance)
{
	FNiagaraDISkeletalMeshPassedDataToRT* SourceData = static_cast<FNiagaraDISkeletalMeshPassedDataToRT*>(PerInstanceData);

	FNiagaraDataInterfaceProxySkeletalMeshData& Data = SystemInstancesToData.FindOrAdd(Instance);

	Data.bIsGpuUniformlyDistributedSampling = SourceData->bIsGpuUniformlyDistributedSampling;
	Data.bReadDeformedGeometry = SourceData->bReadDeformedGeometry;
	Data.bUnlimitedBoneInfluences = SourceData->bUnlimitedBoneInfluences;
	Data.DeltaSeconds = SourceData->DeltaSeconds;
	Data.DynamicBuffer = SourceData->DynamicBuffer;
	Data.MeshBoneWeightStrideBytes = SourceData->MeshBoneWeightStrideBytes;
	Data.MeshBoneIndexSizeBytes = SourceData->MeshBoneIndexSizeBytes;
	Data.MeshBoneWeightSizeBytes = SourceData->MeshBoneWeightSizeBytes;
	Data.PrevTransform = SourceData->PrevTransform;
	Data.StaticBuffers = SourceData->StaticBuffers;
	Data.Transform = SourceData->Transform;
	Data.PreSkinnedLocalBoundsCenter = SourceData->PreSkinnedLocalBoundsCenter;
	Data.PreSkinnedLocalBoundsExtents = SourceData->PreSkinnedLocalBoundsExtents;

	Data.MeshSkinWeightBuffer = SourceData->MeshSkinWeightBuffer;
	Data.MeshSkinWeightLookupBuffer = SourceData->MeshSkinWeightLookupBuffer;

	Data.UvMappingBuffer = SourceData->UvMappingBuffer;
	Data.UvMappingSet = SourceData->UvMappingSet;

	Data.ConnectivityBuffer = SourceData->ConnectivityBuffer;

	SourceData->~FNiagaraDISkeletalMeshPassedDataToRT();
}

//////////////////////////////////////////////////////////////////////////
//FNDISkeletalMesh_InstanceData

void UNiagaraDataInterfaceSkeletalMesh::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	FNiagaraDISkeletalMeshPassedDataToRT* Data = static_cast<FNiagaraDISkeletalMeshPassedDataToRT*>(DataForRenderThread);
	FNDISkeletalMesh_InstanceData* SourceData = static_cast<FNDISkeletalMesh_InstanceData*>(PerInstanceData);

	Data->bIsGpuUniformlyDistributedSampling = SourceData->bIsGpuUniformlyDistributedSampling;
	Data->bReadDeformedGeometry = SourceData->bReadDeformedGeometry;
	Data->bUnlimitedBoneInfluences = SourceData->bUnlimitedBoneInfluences;
	Data->DeltaSeconds = SourceData->DeltaSeconds;
	Data->DynamicBuffer = SourceData->MeshGpuSpawnDynamicBuffers;
	Data->MeshBoneWeightStrideBytes = SourceData->MeshBoneWeightStrideBytes;
	Data->MeshBoneIndexSizeBytes = SourceData->MeshBoneIndexSizeBytes;
	Data->MeshBoneWeightSizeBytes = SourceData->MeshBoneWeightSizeBytes;
	Data->PrevTransform = FMatrix44f(SourceData->PrevTransform);	// LWC_TODO: Precision loss
	Data->StaticBuffers = SourceData->MeshGpuSpawnStaticBuffers;
	Data->Transform = FMatrix44f(SourceData->Transform);			// LWC_TODO: Precision loss
	Data->PreSkinnedLocalBoundsCenter = SourceData->PreSkinnedLocalBoundsCenter;
	Data->PreSkinnedLocalBoundsExtents = SourceData->PreSkinnedLocalBoundsExtents;

	Data->MeshSkinWeightBuffer = SourceData->MeshSkinWeightBuffer;
	Data->MeshSkinWeightLookupBuffer = SourceData->MeshSkinWeightLookupBuffer;

	Data->UvMappingBuffer = SourceData->UvMapping.GetQuadTreeProxy();
	Data->UvMappingSet = SourceData->UvMapping.GetUvSetIndex();

	Data->ConnectivityBuffer = SourceData->Connectivity.GetProxy();
}

USkinnedAsset* UNiagaraDataInterfaceSkeletalMesh::GetSkinnedMesh(FNiagaraSystemInstance* SystemInstance, USceneComponent* AttachComponent, TWeakObjectPtr<USceneComponent>& SceneComponent, USkinnedMeshComponent*& FoundSkinnedComponent, FNDISkeletalMesh_InstanceData* InstData)
{
	UClass* TargetComponentClass = OptionalComponentClass.Get() ? OptionalComponentClass.Get() : USkeletalMeshComponent::StaticClass();

	auto ConvertToValidComponent =
		[&](USceneComponent* Component, bool bAllowNullMesh=false) -> USkinnedMeshComponent*
		{
			if (IsValid(Component) && Component->IsA(TargetComponentClass) )
			{
				USkinnedMeshComponent* SkinnedComponent = CastChecked<USkinnedMeshComponent>(Component);
				if ((bAllowNullMesh || IsValid(SkinnedComponent->GetSkinnedAsset())) &&
					(ComponentTags.IsEmpty() || ComponentTags.ContainsByPredicate([&](const FName& Tag) { return Tag == NAME_None || Component->ComponentHasTag(Tag); })))
				{
					return SkinnedComponent;
				}
			}
			return nullptr;
		};

	// Helper to scour an actor (or its parents) for a valid skeletal mesh component
	auto FindActorSkelMeshComponent =
		[&](AActor* Actor, bool bRecurseParents = false) -> USkinnedMeshComponent*
		{
			if (ASkeletalMeshActor* SkelMeshActor = Cast<ASkeletalMeshActor>(Actor))
			{
				USkinnedMeshComponent* Comp = SkelMeshActor->GetSkeletalMeshComponent();
				if (ConvertToValidComponent(Comp, true))
				{
					return Comp;
				}
			}

			// Fall back on any valid component on the actor
			while (Actor)
			{
				if (USceneComponent* ActorRootComponent = Actor->GetRootComponent())
				{
					TArray<USceneComponent*> ActorComponents;
					ActorRootComponent->GetChildrenComponents(true /*bIncludeAllDescendants*/, ActorComponents);
					ActorComponents.Insert(ActorRootComponent, 0);

					for (USceneComponent* SceneComponent : ActorComponents)
					{
						if (USkinnedMeshComponent* SkinnedComponent = ConvertToValidComponent(SceneComponent))
						{
							return SkinnedComponent;
						}
					}
				}

				if (bRecurseParents)
				{
					Actor = Actor->GetParentActor();
				}
				else
				{
					break;
				}
			}

			return nullptr;
		};

	const bool bTrySource = SourceMode == ENDISkeletalMesh_SourceMode::Default || SourceMode == ENDISkeletalMesh_SourceMode::Source;
	const bool bTryAttachParent = SourceMode == ENDISkeletalMesh_SourceMode::Default || SourceMode == ENDISkeletalMesh_SourceMode::AttachParent;
	const bool bTryDefaultMesh = SourceMode == ENDISkeletalMesh_SourceMode::Default || SourceMode == ENDISkeletalMesh_SourceMode::DefaultMeshOnly;
	const bool bTryChildren = SourceMode == ENDISkeletalMesh_SourceMode::ChildrenOnly;

	if (MeshUserParameter.Parameter.IsValid() && InstData && SystemInstance != nullptr)
	{
		// Initialize the binding and retrieve the object. If a valid object is bound, we'll try and retrieve the SkelMesh component from it.
		// If it's not valid yet, we'll reset and do this again when/if a valid object is set on the binding
		UObject* UserParamObject = InstData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), MeshUserParameter.Parameter);
		InstData->CachedUserParam = UserParamObject;
		if (UserParamObject)
		{
			if (USkeletalMeshComponent* UserSkelMeshComp = Cast<USkeletalMeshComponent>(UserParamObject))
			{
				if (IsValid(UserSkelMeshComp))
				{
					FoundSkinnedComponent = UserSkelMeshComp;
				}
			}
			else if (AActor* Actor = Cast<AActor>(UserParamObject))
			{
				FoundSkinnedComponent = FindActorSkelMeshComponent(Actor);
			}
			else
			{
				//We have a valid, non-null UObject parameter type but it is not a type we can use to get a skeletal mesh from.
				UE_LOGF(LogNiagara, Warning, "SkeletalMesh data interface using object parameter with invalid type. Skeletal Mesh Data Interfaces can only get a valid mesh from SkeletalMeshComponents, SkeletalMeshActors or Actors.");
				UE_LOGF(LogNiagara, Warning, "Invalid Parameter : %ls", *UserParamObject->GetFullName());
				UE_LOGF(LogNiagara, Warning, "Niagara Component : %ls", *GetFullNameSafe(Cast<UNiagaraComponent>(AttachComponent)));
				UE_LOGF(LogNiagara, Warning, "System : %ls", *GetFullNameSafe(SystemInstance->GetSystem()));
			}
		}
		else
		{
			// The binding exists, but no object is bound. Not warning here in case the user knows what they're doing.
		}
	}
	else if (bTrySource && IsValid(SourceComponent))
	{
		FoundSkinnedComponent = SourceComponent;
	}
	else if (bTrySource && SoftSourceActor.Get())
	{
		FoundSkinnedComponent = FindActorSkelMeshComponent(SoftSourceActor.Get());
	}
	else if (bTryAttachParent && AttachComponent)
	{
		// First, try to find the mesh component up the attachment hierarchy
		for (USceneComponent* Curr = AttachComponent; Curr; Curr = Curr->GetAttachParent())
		{
			if (USkinnedMeshComponent* SkinnedComponent = ConvertToValidComponent(Curr, true))
			{
				FoundSkinnedComponent = SkinnedComponent;
				break;
			}
		}

		if (!FoundSkinnedComponent)
		{
			// Next, try to find one in our outer chain
			USkinnedMeshComponent* OuterComp = Cast<USkinnedMeshComponent>(AttachComponent->GetTypedOuter(TargetComponentClass));
			if (ConvertToValidComponent(OuterComp, true))
			{
				FoundSkinnedComponent = OuterComp;
			}
			else if (AActor* Actor = AttachComponent->GetAttachmentRootActor())
			{
				// Final fall-back, look for any mesh component on our root actor or any of its parents
				FoundSkinnedComponent = FindActorSkelMeshComponent(Actor, true);
			}
		}
	}
	else if (bTryChildren && AttachComponent)
	{
		const int32 NumChildren = AttachComponent->GetNumChildrenComponents();
		for (int32 iChild = 0; iChild < NumChildren; ++iChild)
		{
			if (USkinnedMeshComponent* SkinnedComponent = ConvertToValidComponent(AttachComponent->GetChildComponent(iChild)))
			{
				FoundSkinnedComponent = SkinnedComponent;
				break;
			}				
		}
	}

	USkinnedAsset* Mesh = nullptr;
	SceneComponent = nullptr;
	if (FoundSkinnedComponent)
	{
		Mesh = FoundSkinnedComponent->GetSkinnedAsset();
		SceneComponent = FoundSkinnedComponent;
	}
	else if (bTryDefaultMesh && DefaultMesh)
	{
		Mesh = DefaultMesh;
	}
#if WITH_EDITORONLY_DATA
	else if (!SystemInstance || !SystemInstance->GetWorld()->IsGameWorld())
	{
		// NOTE: We don't fall back on the preview mesh if we have a valid skeletal mesh component referenced
		Mesh = PreviewMesh.Get();

		if (!Mesh)
		{
			Mesh = PreviewMesh.LoadSynchronous();

			// if the soft object needed to be loaded we need to make sure that all the necessary streaming
			// meta data has been initialized
			if (Mesh)
			{
				Mesh->GetResourceForRendering();
			}
		}
	}
#endif

	return Mesh;
}

USkinnedAsset* UNiagaraDataInterfaceSkeletalMesh::GetSkinnedMesh(UNiagaraComponent* Component)
{
	// NOTE: We don't need the system instance when not initializing instance data, and when using a UNiagaraComponent, it is always the attach component
	TWeakObjectPtr<USceneComponent> SceneComponent;
	USkinnedMeshComponent* FoundSkinnedComponent = nullptr;
	return GetSkinnedMesh(nullptr, Component, SceneComponent, FoundSkinnedComponent, nullptr);
}

bool FNDISkeletalMesh_InstanceData::Init(UNiagaraDataInterfaceSkeletalMesh* Interface, FNiagaraSystemInstance* SystemInstance)
{
	check(Interface);
	check(SystemInstance);

	// Initialize members
	WeakSceneComponent = nullptr;
	WeakCachedAttachParent = nullptr;
	CachedUserParam = nullptr;
	WeakSkinnedMesh = nullptr;
	Transform = FMatrix::Identity;
	TransformInverseTransposed = FMatrix::Identity;
	PrevTransform = FMatrix::Identity;
	DeltaSeconds = SystemInstance->GetWorld()->GetDeltaSeconds();
	ChangeId = Interface->ChangeId;
	bIsGpuUniformlyDistributedSampling = false;
	bReadDeformedGeometry = Interface->bReadDeformedGeometry && GetDefault<UNiagaraSettings>()->NDISkelMesh_SupportReadingDeformedGeometry;
	bUnlimitedBoneInfluences = false;
	MeshBoneWeightStrideBytes = 0;
	MeshBoneIndexSizeBytes = 0;
	MeshBoneWeightSizeBytes = 0;
	MeshGpuSpawnStaticBuffers = nullptr;
	MeshGpuSpawnDynamicBuffers = nullptr;
	bAllowCPUMeshDataAccess = false;
	PreSkinnedLocalBoundsCenter = FVector3f::ZeroVector;
	PreSkinnedLocalBoundsExtents = FVector3f::ZeroVector;

	// Get skel mesh and confirm have valid data
	USkinnedMeshComponent* NewSkinnedComponent = nullptr;
	USceneComponent* AttachComponent = SystemInstance->GetAttachComponent();
	USkinnedAsset* SkinnedMesh = Interface->GetSkinnedMesh(SystemInstance, AttachComponent, WeakSceneComponent, NewSkinnedComponent, this);

	WeakSkinnedMesh = SkinnedMesh;
	bMeshValid = SkinnedMesh != nullptr;
	bComponentValid = WeakSceneComponent.IsValid();

	const FTransform ComponentTransform = CalculateComponentTransform(SystemInstance);
	Transform = ComponentTransform.ToMatrixWithScale();
	TransformInverseTransposed = Transform.Inverse().GetTransposed();
	PrevTransform = Transform;

	if (AttachComponent)
	{
		WeakCachedAttachParent = AttachComponent->GetAttachParent();
	}

	bResetOnLODStreamedIn = false;
	CachedLODIdx = 0;
	CachedLODData.SafeRelease();

	//Setup where to spawn from
	SamplingRegionIndices.Empty();
	bool bAllRegionsAreAreaWeighting = true;

	if (SkinnedMesh)
	{
		// Determine the LOD index and sampling region indices
		const TTuple<int32, int32> StreamingLODData = NDISkelMeshLocal::GetStreamingLODData(SkinnedMesh);
		const int32 NumValidLODs = StreamingLODData.Get<0>();
		if (NumValidLODs > 0)
		{
			const int32 CurrentFirstLOD = StreamingLODData.Get<1>();
			const int32 DesiredLODIndex = Interface->CalculateLODIndexAndSamplingRegions(SkinnedMesh, SamplingRegionIndices, bAllRegionsAreAreaWeighting);
			if (DesiredLODIndex != INDEX_NONE)
			{
				if (DesiredLODIndex >= CurrentFirstLOD)
				{
					CachedLODIdx = DesiredLODIndex;
				}
				else
				{
					CachedLODIdx = CurrentFirstLOD;
					bResetOnLODStreamedIn = true;
				}

				// Attempt to cache the LOD
				if (FSkeletalMeshRenderData* RenderData = SkinnedMesh->GetResourceForRendering())
				{
					if (RenderData->LODRenderData.IsValidIndex(CachedLODIdx))
					{
						CachedLODData = &RenderData->LODRenderData[CachedLODIdx];
					}
					
					if (!ensure(CachedLODData.IsValid()))
					{
						// NOTE: Assumption here is that the LOD render data is cacheable from GameThread as long as it's considered resident by
						// the StreamableRenderResourceState on GameThread. If this warning gets hit, that assumption has become incorrect.
						UE_LOGF(LogNiagara, Log, "SkeletalMesh data interface failed to cache LOD %d. Sampling will fail. %ls", CachedLODIdx, *Interface->GetFullName());
						SkinnedMesh = nullptr;
					}
				}
				else
				{
					// Warn and continue as if the component has no mesh
					UE_LOGF(LogNiagara, Log, "SkeletalMesh data interface with no Render data. Sampling will fail. %ls", *Interface->GetFullName());
					SkinnedMesh = nullptr;
				}
			}
			else
			{
				return false;
			}
		}
		else
		{
			// Warn and continue as if the component has no mesh
			UE_LOGF(LogNiagara, Log, "SkeletalMesh data interface with no resident LODs. Sampling will fail. %ls", *Interface->GetFullName());
			SkinnedMesh = nullptr;
		}

		if (SkinnedMesh == nullptr)
		{
			CachedLODIdx = 0;
			bResetOnLODStreamedIn = false;
		}
#if WITH_EDITOR
		else
		{
			// HACK! This only works on systems created by a Niagara component...should maybe move somewhere else to cover non-component systems
			if (UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(AttachComponent))
			{
				if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(SkinnedMesh))
				{
					SkeletalMesh->GetOnMeshChanged().AddUObject(NiagaraComponent, &UNiagaraComponent::ReinitializeSystem);
				}
				if (USkeleton* Skeleton = SkinnedMesh->GetSkeleton())
				{
					Skeleton->RegisterOnSkeletonHierarchyChanged(USkeleton::FOnSkeletonHierarchyChanged::CreateUObject(NiagaraComponent, &UNiagaraComponent::ReinitializeSystem));
				}
			}
		}
#endif

		if (SkinnedMesh != nullptr )
		{
			const FBoxSphereBounds LocalBounds = SkinnedMesh->GetBounds();
			PreSkinnedLocalBoundsCenter = FVector3f(LocalBounds.Origin);
			PreSkinnedLocalBoundsExtents = FVector3f(LocalBounds.BoxExtent) * 2.0f;
		}
	}

	check(CachedLODIdx >= 0);

	//Grab a handle to the skinning data if we have a component to skin.
	const ENDISkeletalMesh_SkinningMode SkinningMode = Interface->SkinningMode;
	FSkeletalMeshSkinningDataUsage Usage(
		CachedLODIdx,
		SkinningMode == ENDISkeletalMesh_SkinningMode::SkinOnTheFly || SkinningMode == ENDISkeletalMesh_SkinningMode::PreSkin,
		SkinningMode == ENDISkeletalMesh_SkinningMode::PreSkin
	);

	// GetSkeletalMeshGeneratedData() is meant to match with the required lod, so don't access unless we are using it.
	if (NewSkinnedComponent)
	{
		// TODO: This change is temporary to work around a crash that happens when you change the
		// source mesh on a system which is running in the level from the details panel.
		// bool bNeedsDataImmediately = SystemInstance->IsSolo();
		const bool bNeedsDataImmediately = true;

		TWeakObjectPtr<USkinnedMeshComponent> SkelWeakCompPtr = NewSkinnedComponent;
		FNDI_SkeletalMesh_GeneratedData& GeneratedData = SystemInstance->GetWorldManager()->EditGeneratedData<FNDI_SkeletalMesh_GeneratedData>();
		SkinningData = GeneratedData.GetCachedSkinningData(SkelWeakCompPtr, Usage, bNeedsDataImmediately);
	}
	else
	{
		SkinningData = FSkeletalMeshSkinningDataHandle(Usage, nullptr, false);
	}

	// support for UV mapping
	{
		bool UsedByCpuUvMapping = false;
		bool UsedByGpuUvMapping = false;

		SystemInstance->EvaluateBoundFunction(FSkeletalMeshInterfaceHelper::GetTriangleCoordAtUVName, UsedByCpuUvMapping, UsedByGpuUvMapping);
		SystemInstance->EvaluateBoundFunction(FSkeletalMeshInterfaceHelper::GetTriangleCoordInAabbName, UsedByCpuUvMapping, UsedByGpuUvMapping);

		const bool MeshValid = SkinnedMesh != nullptr;
		const bool SupportUvMappingCpu = UsedByCpuUvMapping && MeshValid;
		const bool SupportUvMappingGpu = UsedByGpuUvMapping && MeshValid && Interface->IsUsedWithGPUScript();

		FMeshUvMappingUsage UvMappingUsage(SupportUvMappingCpu, SupportUvMappingGpu);

		if (UvMappingUsage.IsValid())
		{
			const bool bNeedsDataImmediately = true;

			FNDI_SkeletalMesh_GeneratedData& GeneratedData = SystemInstance->GetWorldManager()->EditGeneratedData<FNDI_SkeletalMesh_GeneratedData>();
			UvMapping = GeneratedData.GetCachedUvMapping(WeakSkinnedMesh, CachedLODIdx, Interface->UvSetIndex, UvMappingUsage, bNeedsDataImmediately);
		}
		else
		{
			UvMapping = FSkeletalMeshUvMappingHandle(UvMappingUsage, nullptr, false);
		}
	}

	// mesh connectivity
	{
		bool UsedByCpuConnectivity = false;
		bool UsedByGpuConnectivity = false;

		SystemInstance->EvaluateBoundFunction(FSkeletalMeshInterfaceHelper::GetAdjacentTriangleIndexName, UsedByCpuConnectivity, UsedByGpuConnectivity);
		SystemInstance->EvaluateBoundFunction(FSkeletalMeshInterfaceHelper::GetTriangleNeighborName, UsedByCpuConnectivity, UsedByGpuConnectivity);

		const bool MeshValid = SkinnedMesh != nullptr;
		const bool SupportConnectivityCpu = UsedByCpuConnectivity && MeshValid;
		const bool SupportConnectivityGpu = UsedByGpuConnectivity && MeshValid && Interface->IsUsedWithGPUScript();

		FSkeletalMeshConnectivityUsage ConnectivityUsage(SupportConnectivityCpu, SupportConnectivityGpu);

		if (ConnectivityUsage.IsValid())
		{
			const bool bNeedsDataImmediately = true;

			FNDI_SkeletalMesh_GeneratedData& GeneratedData = SystemInstance->GetWorldManager()->EditGeneratedData<FNDI_SkeletalMesh_GeneratedData>();
			Connectivity = GeneratedData.GetCachedConnectivity(WeakSkinnedMesh, CachedLODIdx, ConnectivityUsage, bNeedsDataImmediately);
		}
		else
		{
			Connectivity = FSkeletalMeshConnectivityHandle(ConnectivityUsage, nullptr, false);
		}
	}

	//Init area weighting sampler for Sampling regions.
	if (SamplingRegionIndices.Num() > 1 && bAllRegionsAreAreaWeighting)
	{
		//We are sampling from multiple area weighted regions so setup the inter-region weighting sampler.
		SamplingRegionAreaWeightedSampler.Init(this);
	}

	if (SkinnedMesh)
	{
		check(CachedLODData);

		bAllowCPUMeshDataAccess = true; // Assume accessibility until proven otherwise below
		const FSkinWeightVertexBuffer* SkinWeightBuffer = GetSkinWeights();
		check(SkinWeightBuffer);

		// Check for the validity of the Mesh's cpu data.
		if (SkinnedMesh->GetLODInfo(CachedLODIdx)->bAllowCPUAccess)
		{
			const bool LODDataNumVerticesCorrect = CachedLODData->GetNumVertices() > 0;
			const bool LODDataPositionNumVerticesCorrect = CachedLODData->StaticVertexBuffers.PositionVertexBuffer.GetNumVertices() > 0;
			const bool SkinWeightBufferNumVerticesCorrect = SkinWeightBuffer->GetNumVertices() > 0;
			const bool bIndexBufferValid = CachedLODData->MultiSizeIndexContainer.IsIndexBufferValid();
			const bool bIndexBufferFound = bIndexBufferValid && (CachedLODData->MultiSizeIndexContainer.GetIndexBuffer() != nullptr);
			const bool bIndexBufferNumCorrect = bIndexBufferFound && (CachedLODData->MultiSizeIndexContainer.GetIndexBuffer()->Num() > 0);

			bAllowCPUMeshDataAccess = LODDataNumVerticesCorrect &&
				LODDataPositionNumVerticesCorrect &&
				SkinWeightBufferNumVerticesCorrect &&
				bIndexBufferValid &&
				bIndexBufferFound &&
				bIndexBufferNumCorrect;
		}
		else
		{
			bAllowCPUMeshDataAccess = false;
		}

		// Generate excluded root bone index (if any)
		FReferenceSkeleton& RefSkel = SkinnedMesh->GetRefSkeleton();
		ExcludedBoneIndex = INDEX_NONE;
		if (Interface->bExcludeBone && !Interface->ExcludeBoneName.IsNone())
		{
			ExcludedBoneIndex = RefSkel.FindBoneIndex(Interface->ExcludeBoneName);
			if (ExcludedBoneIndex == INDEX_NONE)
			{
				UE_LOGF(LogNiagara, Warning, "Skeletal Mesh Data Interface '%ls' is missing bone '%ls' this is ok but may not exclude what you want Mesh '%ls' Component '%ls'", *Interface->GetFullName(), *Interface->ExcludeBoneName.ToString(), *SkinnedMesh->GetFullName(), *GetFullNameSafe(WeakSceneComponent.Get()));
			}
		}

		// Gather filtered bones information
		if (Interface->FilteredBones.Num() > 0)
		{
			if (RefSkel.GetNum() > TNumericLimits<uint16>::Max())
			{
				UE_LOGF(LogNiagara, Warning, "Skeletal Mesh Data Interface '%ls' requires more bones '%d' than we currently support '%d' Mesh '%ls' Component '%ls'", *Interface->GetFullName(), RefSkel.GetNum(), TNumericLimits<uint16>::Max(), *SkinnedMesh->GetFullName(), *GetFullNameSafe(WeakSceneComponent.Get()));
				return false;
			}

			//-TODO: If the DI does not use unfiltered bones we can skip adding them here...
			TStringBuilder<256> MissingFilteredBones;

			FilteredAndUnfilteredBones.Reserve(RefSkel.GetNum());

			// Append filtered bones to the array first
			for (const FName& BoneName : Interface->FilteredBones)
			{
				const int32 Bone = RefSkel.FindBoneIndex(BoneName);
				if (Bone == INDEX_NONE)
				{
					if ( FNiagaraUtilities::LogVerboseWarnings() )
					{
						if (MissingFilteredBones.Len() > 0)
						{
							MissingFilteredBones.Append(TEXT(", "));
						}
						MissingFilteredBones.Append(BoneName.ToString());
					}
				}
				else
				{
					ensure(Bone <= TNumericLimits<uint16>::Max());
					FilteredAndUnfilteredBones.Add(uint16(Bone));
					++NumFilteredBones;
				}
			}

			// Append unfiltered bones to the array
			for (int32 i = 0; i < RefSkel.GetNum(); ++i)
			{
				// Don't append excluded bone
				if (i == ExcludedBoneIndex)
				{
					continue;
				}

				bool bExists = false;
				for (int32 j = 0; j < NumFilteredBones; ++j)
				{
					if (FilteredAndUnfilteredBones[j] == i)
					{
						bExists = true;
						break;
					}
				}
				if (!bExists)
				{
					FilteredAndUnfilteredBones.Add(uint16(i));
					++NumUnfilteredBones;
				}
			}

			if (MissingFilteredBones.Len() > 0)
			{
				UE_LOGF(LogNiagara, Warning, "Skeletal Mesh Data Interface is trying to sample from filtered bones that don't exist in it's skeleton. Mesh(%ls) Bones(%ls) System(%ls)", *GetFullNameSafe(SkinnedMesh), MissingFilteredBones.ToString(), *GetFullNameSafe(SystemInstance->GetSystem()));
			}
		}
		else
		{
			// Note: We do not allocate space in the array as that wastes memory, we handle this special case when reading from unfiltered
			NumUnfilteredBones = RefSkel.GetNum();
		}

		// Gather filtered socket information
		{
			TArray<FName>& FilteredSockets = Interface->FilteredSockets;
			FilteredSocketInfo.SetNum(FilteredSockets.Num());

			//-TODO: We may need to handle skinning mode changes here
			if (NewSkinnedComponent != nullptr)
			{
				for (int32 i = 0; i < FilteredSocketInfo.Num(); ++i)
				{
					FTransform SocketTransform;
					NewSkinnedComponent->GetSocketInfoByName(FilteredSockets[i], SocketTransform, FilteredSocketInfo[i].BoneIdx);
					FilteredSocketInfo[i].Transform = (FTransform3f)SocketTransform;
				}
			}
			else
			{
				for (int32 i = 0; i < FilteredSocketInfo.Num(); ++i)
				{
					FilteredSocketInfo[i].Transform = FTransform3f(FMatrix44f(SkinnedMesh->GetComposedRefPoseMatrix(FilteredSockets[i])));
					FilteredSocketInfo[i].BoneIdx = INDEX_NONE;
				}
			}

			FilteredSocketBoneOffset = SkinnedMesh->GetRefSkeleton().GetNum();

			FilteredSocketTransformsIndex = 0;
			FilteredSocketTransforms[0].Reset(FilteredSockets.Num());
			FilteredSocketTransforms[0].AddDefaulted(FilteredSockets.Num());
			UpdateFilteredSocketTransforms();
			for (int32 i = 1; i < FilteredSocketTransforms.Num(); ++i)
			{
				FilteredSocketTransforms[i].Reset(FilteredSockets.Num());
				FilteredSocketTransforms[i].Append(FilteredSocketTransforms[0]);
			}

			if ( FNiagaraUtilities::LogVerboseWarnings() )
			{
				TStringBuilder<512> MissingSockets;
				for (FName SocketName : FilteredSockets)
				{
					if (SkinnedMesh->FindSocket(SocketName) == nullptr)
					{
						if (MissingSockets.Len() != 0)
						{
							MissingSockets.Append(TEXT(", "));
						}
						MissingSockets.Append(SocketName.ToString());
					}
				}

				if (MissingSockets.Len() > 0)
				{
					UE_LOGF(LogNiagara, Warning, "Skeletal Mesh Data Interface is trying to sample from filtered sockets that don't exist in it's skeleton. Mesh(%ls) Sockets(%ls) System(%ls)", *GetFullNameSafe(SkinnedMesh), MissingSockets.ToString(), *GetFullNameSafe(SystemInstance->GetSystem()));
				}
			}
		}

		if (Interface->IsUsedWithGPUScript())
		{
			GPUSkinBoneInfluenceType BoneInfluenceType = SkinWeightBuffer->GetBoneInfluenceType();
			bUnlimitedBoneInfluences = (BoneInfluenceType == GPUSkinBoneInfluenceType::UnlimitedBoneInfluence);
			MeshBoneWeightStrideBytes = SkinWeightBuffer->GetConstantInfluencesVertexStride();
			MeshBoneIndexSizeBytes = SkinWeightBuffer->GetBoneIndexByteSize();
			MeshBoneWeightSizeBytes = SkinWeightBuffer->GetBoneWeightByteSize();

			MeshSkinWeightBuffer = SkinWeightBuffer->GetDataVertexBuffer();
			//check(MeshSkinWeightBufferSrv->IsValid()); // not available in this stream
			MeshSkinWeightLookupBuffer = SkinWeightBuffer->GetLookupVertexBuffer();

			FSkeletalMeshLODInfo* LODInfo = SkinnedMesh->GetLODInfo(CachedLODIdx);
			bIsGpuUniformlyDistributedSampling = LODInfo->bSupportUniformlyDistributedSampling && bAllRegionsAreAreaWeighting;

			const auto MaxInfluenceType = GetDefault<UNiagaraSettings>()->NDISkelMesh_GpuMaxInfluences;
			int32 MaxInfluenceCount = -1;
			if (MaxInfluenceType == ENDISkelMesh_GpuMaxInfluences::Type::AllowMax4)
			{
				MaxInfluenceCount = 4;
			}
			else if (MaxInfluenceType == ENDISkelMesh_GpuMaxInfluences::Type::AllowMax8)
			{
				MaxInfluenceCount = 8;
			}
			else
			{
				checkf(MaxInfluenceType == ENDISkelMesh_GpuMaxInfluences::Type::Unlimited, TEXT("Unknown value for NDISkelMesh_GpuMaxInfluences: %d"), MaxInfluenceType);
			}

			if (MaxInfluenceCount > 0 && (static_cast<uint32>(MaxInfluenceCount) < CachedLODData->GetVertexBufferMaxBoneInfluences()))
			{
				UE_LOGF(LogNiagara, Warning, "Skeletal Mesh %ls has bones extra influence: spawning from it might not work properly.", *SkinnedMesh->GetName());
			}

			const FSkeletalMeshSamplingLODBuiltData* MeshSamplingBuiltData = nullptr;
			if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(SkinnedMesh))
			{
				const FSkeletalMeshSamplingInfo& SamplingInfo = SkeletalMesh->GetSamplingInfo();
				MeshSamplingBuiltData = SamplingInfo.GetBuiltData().WholeMeshBuiltData.IsValidIndex(CachedLODIdx) ? &SamplingInfo.GetBuiltData().WholeMeshBuiltData[CachedLODIdx] : nullptr;
				bIsGpuUniformlyDistributedSampling &= MeshSamplingBuiltData != nullptr;
			}
			else
			{
				bIsGpuUniformlyDistributedSampling = false;
			}

			MeshGpuSpawnStaticBuffers = new FSkeletalMeshGpuSpawnStaticBuffers();
			MeshGpuSpawnStaticBuffers->Initialise(this, CachedLODIdx, *CachedLODData, MeshSamplingBuiltData, SystemInstance);
			BeginInitResource(MeshGpuSpawnStaticBuffers);

			MeshGpuSpawnDynamicBuffers = new FSkeletalMeshGpuDynamicBufferProxy();
			MeshGpuSpawnDynamicBuffers->Initialise(RefSkel, *CachedLODData, FilteredSocketInfo.Num());
			BeginInitResource(MeshGpuSpawnDynamicBuffers);

			// Trying to track down an invalid mesh that makes it to the GPU
			// Currently theory is it's from mutable, the logs should provide more information
			if (!MeshGpuSpawnDynamicBuffers->GetSectionBufferNumElements() || !MeshGpuSpawnDynamicBuffers->GetSamplingBufferNumElements())
			{
				UE_LOGF(
					LogNiagara, Warning,
					"SkeletalMeshDataInterface(%ls) has invalid dynamic buffer for LOD(%u). Section Elements(%u) Sampling Elements(%u) Mesh '%ls' Component '%ls' System '%ls'",
					*Interface->GetFullName(),
					CachedLODIdx,
					MeshGpuSpawnDynamicBuffers->GetSectionBufferNumElements(),
					MeshGpuSpawnDynamicBuffers->GetSamplingBufferNumElements(),
					*GetFullNameSafe(SkinnedMesh),
					*GetFullNameSafe(WeakSceneComponent.Get()),
					*GetFullNameSafe(SystemInstance->GetSystem())
				);
			}
		}
	}

	return true;
}

bool FNDISkeletalMesh_InstanceData::ResetRequired(UNiagaraDataInterfaceSkeletalMesh* Interface, FNiagaraSystemInstance* SystemInstance) const
{
	if (NDISkelMeshLocal::GUseNewResetRequired)
	{
		// Reset if any parameters changed on the data interface
		if (Interface->ChangeId != ChangeId)
		{
			return true;
		}

		// Reset if the scene component we've cached has been invalidated or unregistered
		USceneComponent* ExistingSceneComponent = WeakSceneComponent.Get();
		if (bComponentValid && (!ExistingSceneComponent || !ExistingSceneComponent->IsRegistered()))
		{
			return true;
		}

		// Reset if the skeletal mesh / component we are currently reading has changed
		TWeakObjectPtr<USceneComponent> WeakFoundSceneComponent;
		USkinnedMeshComponent* FoundSkinnedMeshComponent = nullptr;
		USkinnedAsset* FoundSkinnedMesh = Interface->GetSkinnedMesh(SystemInstance, SystemInstance->GetAttachComponent(), WeakFoundSceneComponent, FoundSkinnedMeshComponent, const_cast<FNDISkeletalMesh_InstanceData*>(this));

		if (WeakSceneComponent != WeakFoundSceneComponent)
		{
			return true;
		}

		USkinnedAsset* SkinnedMesh = WeakSkinnedMesh.Get();
		if (SkinnedMesh != FoundSkinnedMesh)
		{
			// Skinning Data is unique per component
			// Therefore if the component did not change but the SKM did we need to refresh the skinning data to ensure the bones are valid
			if (SkinningData.SkinningData.IsValid())
			{
				SkinningData.SkinningData->ForceDataRefresh();
			}
			return true;
		}

		// Reset if the LOD we relied on was streamed out, or if the LOD we need could now be available.
		if (SkinnedMesh != nullptr)
		{
			const TTuple<int32, int32> StreamingLODData = NDISkelMeshLocal::GetStreamingLODData(SkinnedMesh);
			const int32 NumValidLODs = StreamingLODData.Get<0>();
			if (NumValidLODs == 0)
			{
				return true;
			}

			const int32 CurrentFirstLOD = StreamingLODData.Get<1>();
			if (CurrentFirstLOD > CachedLODIdx || (CurrentFirstLOD < CachedLODIdx && bResetOnLODStreamedIn))
			{
				return true;
			}
		}
	}
	else
	{
		// Reset if the scene component we've cached has been invalidated or unregistered
		USceneComponent* SceneComponent = WeakSceneComponent.Get();
		if (bComponentValid && (!SceneComponent || !SceneComponent->IsRegistered()))
		{
			return true;
		}

		// Reset if any mesh was bound on init, but is now invalidated
		USkinnedAsset* SkinnedMesh = WeakSkinnedMesh.Get();
		if (bMeshValid && !SkinnedMesh)
		{
			return true;
		}

		if (Interface->MeshUserParameter.Parameter.IsValid())
		{
			// Reset if the user object ptr has been changed to look at a new object
			if (UserParamBinding.GetValue() != CachedUserParam)
			{
				return true;
			}
		}
		else if (Interface->GetSourceComponent())
		{
			// Reset if the source component changed (or there wasn't one and now there is)
			if (Interface->GetSourceComponent() != SceneComponent)
			{
				return true;
			}
		}
		else if (USceneComponent* AttachComponent = SystemInstance->GetAttachComponent())
		{
			// Reset if we detect any attachment change.
			// TODO: This check is not really comprehensive. What we really need to know is if the mesh we cached comes from a skeletal mesh component in our
			// attachment hierarchy, and if that hierarchy has changed in the chain between the system instance's attach component and the cached component,
			// therefore potentially invalidating the cached component and mesh as our best choice.
			if (WeakCachedAttachParent.Get() != AttachComponent->GetAttachParent())
			{
				// The scene component our system instance was associated with has changed attachment, so we need to reinit
				return true;
			}
		}

		// Reset if the LOD we relied on was streamed out, or if the LOD we need could now be available.
		if (SkinnedMesh != nullptr)
		{
			const TTuple<int32, int32> StreamingLODData = NDISkelMeshLocal::GetStreamingLODData(SkinnedMesh);
			const int32 NumValidLODs = StreamingLODData.Get<0>();
			if (NumValidLODs == 0)
			{
				return true;
			}

			const int32 CurrentFirstLOD = StreamingLODData.Get<1>();
			if (CurrentFirstLOD > CachedLODIdx || (CurrentFirstLOD < CachedLODIdx && bResetOnLODStreamedIn))
			{
				return true;
			}
		}

		// Reset if the skeletal mesh on the cached skeletal mesh component changed.
		if (USkinnedMeshComponent* SkinnedComponent = Cast<USkinnedMeshComponent>(SceneComponent))
		{
			if (SkinnedComponent->GetSkinnedAsset() != SkinnedMesh)
			{
				if (SkinningData.SkinningData.IsValid())
				{
					SkinningData.SkinningData.Get()->ForceDataRefresh();
				}
				return true;
			}
		}

		// Reset if any parameters changed on the data interface
		if (Interface->ChangeId != ChangeId)
		{
			return true;
		}
	}

	return false;
}

bool FNDISkeletalMesh_InstanceData::Tick(UNiagaraDataInterfaceSkeletalMesh* Interface, FNiagaraSystemInstance* SystemInstance, float InDeltaSeconds)
{
	if (ResetRequired(Interface, SystemInstance))
	{
		return true;
	}
	else
	{
		DeltaSeconds = InDeltaSeconds;

		PrevTransform = Transform;
		const FTransform ComponentTransform = CalculateComponentTransform(SystemInstance);
		Transform = ComponentTransform.ToMatrixWithScale();
		TransformInverseTransposed = Transform.Inverse().GetTransposed();

		// Cache socket transforms to avoid potentially calculating them multiple times during the VM
		FilteredSocketTransformsIndex = (FilteredSocketTransformsIndex + 1) % FilteredSocketTransforms.Num();
		UpdateFilteredSocketTransforms();

		if (MeshGpuSpawnDynamicBuffers)
		{
			MeshGpuSpawnDynamicBuffers->NewFrame(this, GetLODIndex());
		}

		return false;
	}
}

FTransform FNDISkeletalMesh_InstanceData::CalculateComponentTransform(FNiagaraSystemInstance* SystemInstance) const
{
	if (USceneComponent* SceneComponent = WeakSceneComponent.Get())
	{
		FTransform ComponentTransform = SceneComponent->GetComponentToWorld();
		ComponentTransform.AddToTranslation(FVector(SystemInstance->GetLWCTile()) * -FLargeWorldRenderScalar::GetTileSize());
		return ComponentTransform;
	}
	else
	{
		return SystemInstance->GetWorldTransform();
	}
}

void FNDISkeletalMesh_InstanceData::UpdateFilteredSocketTransforms()
{
	USkinnedMeshComponent* SceneComponent = Cast<USkinnedMeshComponent>(WeakSceneComponent.Get());
	TArray<FTransform3f>& WriteBuffer = GetFilteredSocketsWriteBuffer();

	for (int32 i = 0; i < FilteredSocketInfo.Num(); ++i)
	{
		const FCachedSocketInfo& SocketInfo = FilteredSocketInfo[i];
		const FTransform& BoneTransform = SocketInfo.BoneIdx != INDEX_NONE ? SceneComponent->GetBoneTransform(SocketInfo.BoneIdx, FTransform::Identity) : FTransform::Identity;
		WriteBuffer[i] = SocketInfo.Transform * FTransform3f(BoneTransform);
	}
}

bool FNDISkeletalMesh_InstanceData::HasColorData()
{
	return CachedLODData && CachedLODData->StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() != 0;
}

void FNDISkeletalMesh_InstanceData::Release()
{
	if (MeshGpuSpawnStaticBuffers || MeshGpuSpawnDynamicBuffers)
	{
		ENQUEUE_RENDER_COMMAND(DeleteResource)(
			[RT_StaticBuffers=MeshGpuSpawnStaticBuffers, RT_DynamicBuffers=MeshGpuSpawnDynamicBuffers](FRHICommandListImmediate& RHICmdList)
			{
				if (RT_StaticBuffers)
				{
					RT_StaticBuffers->ReleaseResource();
					delete RT_StaticBuffers;
				}
				if (RT_DynamicBuffers)
				{
					RT_DynamicBuffers->ReleaseResource();
					delete RT_DynamicBuffers;
				}
			}
		);
		MeshGpuSpawnStaticBuffers = nullptr;
		MeshGpuSpawnDynamicBuffers = nullptr;
	}
}

//Instance Data END
//////////////////////////////////////////////////////////////////////////


//////////////////////////////////////////////////////////////////////////
// UNiagaraDataInterfaceSkeletalMesh

UNiagaraDataInterfaceSkeletalMesh::UNiagaraDataInterfaceSkeletalMesh(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	  , SourceMode(ENDISkeletalMesh_SourceMode::Default)
#if WITH_EDITORONLY_DATA
	  , PreviewMesh(nullptr)
#endif
	  , SoftSourceActor(nullptr)
	  , SourceComponent(nullptr)
      , SkinningMode(ENDISkeletalMesh_SkinningMode::SkinOnTheFly)
	  , WholeMeshLOD(INDEX_NONE)
	  , ChangeId(0)
{
	FNiagaraTypeDefinition Def(UObject::StaticClass());
	MeshUserParameter.Parameter.SetType(Def);

	static const FName RootBoneName("root");
	ExcludeBoneName = RootBoneName;
	bExcludeBone = false;

	Proxy.Reset(new FNiagaraDataInterfaceProxySkeletalMesh());
}


void UNiagaraDataInterfaceSkeletalMesh::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags DIFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), DIFlags);

		//Still some issues with using custom structs. Convert node for example throws a wobbler. TODO after GDC.
		ENiagaraTypeRegistryFlags CoordFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FMeshTriCoordinate::StaticStruct(), CoordFlags);
	}
}

void UNiagaraDataInterfaceSkeletalMesh::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (Source_DEPRECATED != nullptr)
	{
		SoftSourceActor = Source_DEPRECATED;
	}
#endif
}

#if WITH_EDITOR
void UNiagaraDataInterfaceSkeletalMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	ChangeId++;

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceSkeletalMesh, SourceMode))
	{
		if (SourceMode != ENDISkeletalMesh_SourceMode::Default && SourceMode != ENDISkeletalMesh_SourceMode::Source)
		{
			// Ensure we don't have any reference to a source actor that we'll never use
			SoftSourceActor = nullptr;
			SourceComponent = nullptr;
		}

		if (SourceMode != ENDISkeletalMesh_SourceMode::Default && SourceMode != ENDISkeletalMesh_SourceMode::DefaultMeshOnly)
		{
			// Ensure we don't cook in a default mesh we'll never use
			DefaultMesh = nullptr;
		}
	}
}

bool UNiagaraDataInterfaceSkeletalMesh::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceSkeletalMesh, SoftSourceActor) || InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceSkeletalMesh, SourceComponent))
	{
		return SourceMode == ENDISkeletalMesh_SourceMode::Default || SourceMode == ENDISkeletalMesh_SourceMode::Source;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceSkeletalMesh, DefaultMesh))
	{
		return SourceMode == ENDISkeletalMesh_SourceMode::Default || SourceMode == ENDISkeletalMesh_SourceMode::DefaultMeshOnly;
	}

	return true;
}
#endif //WITH_EDITOR

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSkeletalMesh::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	const int32 FirstFunction = OutFunctions.Num();

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDISkelMeshLocal::NAME_GetPreSkinnedLocalBounds;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Center"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("ExtentsMin"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("ExtentsMax"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Extents"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Half Extents"));
		Sig.SetDescription(LOCTEXT("GetPreSkinnedLocalBoundsDesc", "Returns the local bounds of the Skeletal Mesh"));
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
	}

	GetTriangleSamplingFunctions(OutFunctions);
	GetVertexSamplingFunctions(OutFunctions);
	GetSkeletonSamplingFunctions(OutFunctions);

	for ( int i=FirstFunction; i < OutFunctions.Num(); ++i )
	{
		OutFunctions[i].FunctionVersion = FNiagaraSkelMeshDIFunctionVersion::LatestVersion;
	}
}

#endif //WITH_EDITORONLY_DATA

void UNiagaraDataInterfaceSkeletalMesh::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	FNDISkeletalMesh_InstanceData* InstData = (FNDISkeletalMesh_InstanceData*)InstanceData;

	if (!InstData)
	{
		OutFunc = FVMExternalFunction();
		return;
	}

	// Bind misc functions
	if (BindingInfo.Name == NDISkelMeshLocal::NAME_GetPreSkinnedLocalBounds)
	{
		OutFunc.BindUObject(this, &UNiagaraDataInterfaceSkeletalMesh::VMGetPreSkinnedLocalBounds);
		return;
	}

	// Bind skeleton sampling function
	BindSkeletonSamplingFunction(BindingInfo, InstData, OutFunc);
	if (OutFunc.IsBound())
	{
		return;
	}

	// Bind triangle sampling function
	BindTriangleSamplingFunction(BindingInfo, InstData, OutFunc);
	if (OutFunc.IsBound())
	{
		// Only warn about failed access if we are bound to a skeletal mesh, there are cases where the user chooses to have no SKM so this warning is invalid
		if (!InstData->bAllowCPUMeshDataAccess && InstData->WeakSkinnedMesh.IsValid())
		{
			UE_LOGF(LogNiagara, Log, "Skeletal Mesh Data Interface is trying to use triangle sampling function '%ls', but either no CPU access is set on the mesh or the data is invalid. Interface: %ls",
				*BindingInfo.Name.ToString(), *GetFullName());
		}
		return;
	}

	// Bind vertex sampling function
	BindVertexSamplingFunction(BindingInfo, InstData, OutFunc);
	if (OutFunc.IsBound())
	{
		// Only warn about failed access if we are bound to a skeletal mesh, there are cases where the user chooses to have no SKM so this warning is invalid
		if (!InstData->bAllowCPUMeshDataAccess && InstData->WeakSkinnedMesh.IsValid())
		{
			UE_LOGF(LogNiagara, Log, "Skeletal Mesh Data Interface is trying to use vertex sampling function '%ls' but either no CPU access is set on the mesh, or the data is invalid. Interface: %ls",
				*BindingInfo.Name.ToString(), *GetFullName());
		}
		return;
	}
}


bool UNiagaraDataInterfaceSkeletalMesh::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceSkeletalMesh* OtherTyped = CastChecked<UNiagaraDataInterfaceSkeletalMesh>(Destination);
	OtherTyped->UnbindSourceDelegates();
	OtherTyped->SourceMode = SourceMode;
	OtherTyped->SoftSourceActor = SoftSourceActor;
	OtherTyped->MeshUserParameter = MeshUserParameter;
	OtherTyped->ComponentTags = ComponentTags;
	OtherTyped->OptionalComponentClass = OptionalComponentClass;
	OtherTyped->SourceComponent = SourceComponent;
	OtherTyped->SkinningMode = SkinningMode;
	OtherTyped->SamplingRegions = SamplingRegions;
	OtherTyped->WholeMeshLOD = WholeMeshLOD;
	OtherTyped->FilteredBones = FilteredBones;
	OtherTyped->FilteredSockets = FilteredSockets;
	OtherTyped->bExcludeBone = bExcludeBone;
	OtherTyped->ExcludeBoneName = ExcludeBoneName;
	OtherTyped->bRequireCurrentFrameData = bRequireCurrentFrameData;
	OtherTyped->bReadDeformedGeometry = bReadDeformedGeometry;
	OtherTyped->UvSetIndex = UvSetIndex;
#if WITH_EDITORONLY_DATA
	OtherTyped->PreviewMesh = PreviewMesh;
#endif
	OtherTyped->DefaultMesh = DefaultMesh;
	OtherTyped->BindSourceDelegates();

	return true;
}

bool UNiagaraDataInterfaceSkeletalMesh::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceSkeletalMesh* OtherTyped = CastChecked<const UNiagaraDataInterfaceSkeletalMesh>(Other);
	return OtherTyped->SourceMode == SourceMode &&
#if WITH_EDITORONLY_DATA
		OtherTyped->PreviewMesh == PreviewMesh &&
#endif
		OtherTyped->DefaultMesh == DefaultMesh &&
		OtherTyped->SoftSourceActor == SoftSourceActor &&
		OtherTyped->MeshUserParameter == MeshUserParameter &&
		OtherTyped->ComponentTags == ComponentTags &&
		OtherTyped->OptionalComponentClass == OptionalComponentClass &&
		OtherTyped->SourceComponent == SourceComponent &&
		OtherTyped->SkinningMode == SkinningMode &&
		OtherTyped->SamplingRegions == SamplingRegions &&
		OtherTyped->WholeMeshLOD == WholeMeshLOD &&
		OtherTyped->FilteredBones == FilteredBones &&
		OtherTyped->FilteredSockets == FilteredSockets &&
		OtherTyped->bExcludeBone == bExcludeBone &&
		OtherTyped->ExcludeBoneName == ExcludeBoneName &&
		OtherTyped->UvSetIndex == UvSetIndex &&
		OtherTyped->bRequireCurrentFrameData == bRequireCurrentFrameData &&
		OtherTyped->bReadDeformedGeometry == bReadDeformedGeometry;
}

bool UNiagaraDataInterfaceSkeletalMesh::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDISkeletalMesh_InstanceData* Inst = new (PerInstanceData) FNDISkeletalMesh_InstanceData();
	check(IsAligned(PerInstanceData, 16));
	return Inst->Init(this, SystemInstance);
}

void UNiagaraDataInterfaceSkeletalMesh::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDISkeletalMesh_InstanceData* Inst = (FNDISkeletalMesh_InstanceData*)PerInstanceData;

#if WITH_EDITOR
	if (USkinnedAsset* SkinnedMesh = Inst->WeakSkinnedMesh.Get())
	{
		if (UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(SystemInstance->GetAttachComponent()))
		{
			if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(SkinnedMesh))
			{
				SkeletalMesh->GetOnMeshChanged().RemoveAll(NiagaraComponent);
			}
			if (USkeleton* Skeleton = SkinnedMesh->GetSkeleton())
			{
				Skeleton->UnregisterOnSkeletonHierarchyChanged(NiagaraComponent);
			}
		}
	}
#endif

	ENQUEUE_RENDER_COMMAND(FNiagaraDestroySkeletalMeshInstanceData) (
		[RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxySkeletalMesh>(), InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->SystemInstancesToData.Remove(InstanceID);
		}
	);

	Inst->Release();
	Inst->~FNDISkeletalMesh_InstanceData();
}

bool UNiagaraDataInterfaceSkeletalMesh::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float InDeltaSeconds)
{
	FNDISkeletalMesh_InstanceData* Inst = (FNDISkeletalMesh_InstanceData*)PerInstanceData;
	return Inst->Tick(this, SystemInstance, InDeltaSeconds);
}

#if WITH_NIAGARA_DEBUGGER
void UNiagaraDataInterfaceSkeletalMesh::DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const
{
	const FNDISkeletalMesh_InstanceData* InstanceData_GT = DebugHudContext.GetSystemInstance()->FindTypedDataInterfaceInstanceData<FNDISkeletalMesh_InstanceData>(this);
	if (InstanceData_GT == nullptr)
	{
		return;
	}

	USceneComponent* SceneComponent = InstanceData_GT->WeakSceneComponent.Get();
	AActor* SceneActor = SceneComponent ? SceneComponent->GetAttachmentRootActor() : nullptr;
	USkinnedAsset* SkinnedMesh = InstanceData_GT->WeakSkinnedMesh.Get();
	DebugHudContext.GetOutputString().Appendf(TEXT("Skeleton(%s) SkelComp(%s:%s)"), *GetNameSafe(SkinnedMesh), SceneActor ? *SceneActor->GetActorNameOrLabel() : TEXT("None"), *GetNameSafe(SceneComponent));

	if ( DebugHudContext.IsVerbose() && SceneComponent && SkinnedMesh)
	{
		FSkeletalMeshAccessorHelper MeshAccessor;
		MeshAccessor.Init<TNDISkelMesh_FilterModeNone, TNDISkelMesh_AreaWeightingOff>(InstanceData_GT);
		if (MeshAccessor.AreBonesAccessible())
		{
			const TArray<FTransform3f>& BoneTransforms = MeshAccessor.SkinningData->CurrComponentTransforms();
			const FMatrix& InstanceTransform = InstanceData_GT->Transform;

			UCanvas* Canvas = DebugHudContext.GetCanvas();
			for (const FTransform3f& BoneTransform : BoneTransforms)
			{
				const FVector BoneLocation = InstanceTransform.TransformPosition(FVector(BoneTransform.GetLocation()));
				const FVector ScreenPos = Canvas->Project(BoneLocation, false);
				if (ScreenPos.Z <= 0.0f)
				{
					continue;
				}

				Canvas->Canvas->DrawNGon(FVector2D(ScreenPos), FColor::Red, 8, 4.0f);
			}
		}
	}
}
#endif

#if WITH_EDITOR
void UNiagaraDataInterfaceSkeletalMesh::GetFeedback(UNiagaraSystem* Asset, UNiagaraComponent* Component, TArray<FNiagaraDataInterfaceError>& OutErrors,
	TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	if (Asset == nullptr)
	{
		return;
	}

	bool bHasCPUAccessWarning = false;
	bool bHasNoMeshAssignedWarning = false;
	USkinnedAsset* SkinnedMesh = GetSkinnedMesh(Component);

	// Collect Errors
	if (SkinnedMesh != nullptr)
	{
		bool bHasCPUAccess = true;
		for (int32 LODIndex = 0, LODNum = SkinnedMesh->GetLODNum(); LODIndex < LODNum; ++LODIndex)
		{
			if (!SkinnedMesh->GetLODInfo(LODIndex)->bAllowCPUAccess)
			{
				bHasCPUAccess = false;
				break;
			}
		}

		// Check for the possibility that this mesh won't behave properly because of no CPU access
		if (!bHasCPUAccess)
		{
			// Collect all scripts used by the system
			// NOTE: We don't descriminate between CPU or GPU scripts here because while GPU access will "Just Work"
			// on some platforms, other platforms (like Mobile or OpenGL) do not create a shader resource view for the
			// buffers unless the CPU access flag is enabled.
			TArray<UNiagaraScript*> Scripts;
			Scripts.Add(Asset->GetSystemSpawnScript());
			Scripts.Add(Asset->GetSystemUpdateScript());
			for (const FNiagaraEmitterHandle& EmitterHandle : Asset->GetEmitterHandles())
			{
				if (FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData())
				{
					EmitterData->GetScripts(Scripts, false);
				}
			}

			// Now check if any script uses functions that require CPU access
			// TODO: This isn't complete enough. It doesn't guarantee that the DI used by these functions are THIS DI.
			// Finding that information out is currently non-trivial so just pop a warning with the possibility of false
			// positives
			TArray<FNiagaraFunctionSignature> Functions;
			GetTriangleSamplingFunctions(Functions);
			GetVertexSamplingFunctions(Functions);

			bHasCPUAccessWarning = [this, &Scripts, &Functions]()
			{
				for (const auto Script : Scripts)
				{
					for (const auto& DIInfo : Script->GetVMExecutableData().DataInterfaceInfo)
					{
						if (DIInfo.MatchesClass(GetClass()))
						{
							for (const auto& Func : DIInfo.RegisteredFunctions)
							{
								auto Filter = [&Func](const FNiagaraFunctionSignature& CPUSig)
								{
									return CPUSig.Name == Func.Name;
								};
								if (Functions.FindByPredicate(Filter))
								{
									return true;
								}
							}
						}
					}
				}
				return false;
			}();
		}
	}
	else
	{
		bHasNoMeshAssignedWarning = true;
	}

	// Report Errors/Warnings
	if (SkinnedMesh && bHasCPUAccessWarning)
	{
		FNiagaraDataInterfaceFeedback CPUAccessNotAllowedWarning(
			FText::Format(LOCTEXT("CPUAccessNotAllowedError", "This mesh may need CPU access in order to be used properly (even when used by GPU emitters). ({0})"), FText::FromString(SkinnedMesh->GetName())),
			LOCTEXT("CPUAccessNotAllowedErrorSummary", "CPU access error"),
			FNiagaraDataInterfaceFix::CreateLambda([=]()
				{
					SkinnedMesh->Modify();
				
					for (int32 LODIndex = 0, LODNum = SkinnedMesh->GetLODNum(); LODIndex < LODNum; ++LODIndex)
					{
						SkinnedMesh->GetLODInfo(LODIndex)->bAllowCPUAccess = true;
					}
					return true;
				}));

		OutWarnings.Add(CPUAccessNotAllowedWarning);
	}

	if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(SkinnedMesh))
	{
		if (SkeletalMesh->HasActiveClothingAssets())
		{
			const FText ClothSummary = LOCTEXT("UsingClothSummary", "Active clothing assets found.  Results may be incorrect.");
			if (GetDefault<UNiagaraSettings>()->NDISkelMesh_SupportReadingDeformedGeometry == false)
			{
				OutInfo.Emplace(
					FText::Format(LOCTEXT("UsingClothNeedsProjectSetting", "Sampling from clothing regions on mesh '{0} will be incorrect.  Please enable reading from deformed geometry is disabled in project settings."), FText::FromString(SkeletalMesh->GetName())),
					ClothSummary,
					FNiagaraDataInterfaceFix()
				);
			}
			else
			{
				OutInfo.Emplace(
					FText::Format(LOCTEXT("UsingClothCheckValid", "Sampling from clothing regions on mesh '{0}' may be incorrect.  Ensure you are using a GPU emitter and the skin cache or deformer graph are enabled."), FText::FromString(SkeletalMesh->GetName())),
					ClothSummary,
					FNiagaraDataInterfaceFix()
				);
			}
		}

		// Do we have any sampling regions enabled, if so check they are valid.
		if (SkeletalMesh && SamplingRegions.Num() > 0)
		{
			FString MissingRegions;

			const FSkeletalMeshSamplingInfo& SamplingInfo = SkeletalMesh->GetSamplingInfo();
			for (FName RegionName : SamplingRegions)
			{
				const int32 RegionIdx = SamplingInfo.IndexOfRegion(RegionName);
				if (RegionIdx == INDEX_NONE)
				{
					if (!MissingRegions.IsEmpty())
					{
						MissingRegions.Append(TEXT(", "));
					}
					RegionName.AppendString(MissingRegions);
				}
			}

			if (!MissingRegions.IsEmpty())
			{
				OutWarnings.Emplace(
					FText::Format(LOCTEXT("MissingSamplingRegions", "The following regions were not found '{0}'"), FText::FromString(MissingRegions)),
					LOCTEXT("MissingSamplingRegionsSummary", "Invalid Sampling Regions"),
					FNiagaraDataInterfaceFix(
					)
				);
			}
		}
	}

	if (SoftSourceActor.Get() == nullptr && bHasNoMeshAssignedWarning)
	{
		FNiagaraDataInterfaceFeedback NoMeshAssignedError(LOCTEXT("NoMeshAssignedError", "This Data Interface should be assigned a skeletal mesh to operate correctly."),
			LOCTEXT("NoMeshAssignedErrorSummary", "No mesh assigned warning"),
			FNiagaraDataInterfaceFix());

		OutWarnings.Add(NoMeshAssignedError);
	}

	if (SourceMode == ENDISkeletalMesh_SourceMode::Default && DefaultMesh != nullptr)
	{
		OutWarnings.Emplace(
			LOCTEXT("DefaultSourceAndDefaultMesh", "Default Mesh is set but Source Mode is set to Default.  Verify you have not left it set incorrectly to avoid unused meshes being referenced."),
			LOCTEXT("DefaultSourceAndDefaultMeshSummary", "Potentially unused Default Mesh set!"),
			FNiagaraDataInterfaceFix::CreateLambda(
				[this]()
				{
					FScopedTransaction ST(LOCTEXT("ClearDefaultMesh", "Clear Default Mesh"));
					Modify();
					DefaultMesh = nullptr;
					return true;
				}
			)
		);
	}

	// Look for bones being used that are LOD'ed out
	if ( (SkinnedMesh != nullptr) && (SkinnedMesh->GetResourceForRendering() != nullptr) )
	{
		FSkeletalMeshRenderData* SkelResource = SkinnedMesh->GetResourceForRendering();
		auto IsBoneRequiredInAllLODs =
			[&](const FName BoneName)
			{
				const int32 BoneIndex = SkinnedMesh->GetRefSkeleton().FindBoneIndex(BoneName);
				if (BoneIndex == INDEX_NONE)
				{
					return false;
				}

				for ( const FSkeletalMeshLODRenderData& LODData : SkelResource->LODRenderData )
				{
					if ( !LODData.RequiredBones.Contains(BoneIndex) )
					{
						return false;
					}
				}
				return true;
			};

		if (FilteredBones.Num() > 0)
		{
			FString MissingBoneList;
			for (FName Bone : FilteredBones)
			{
				if ( !IsBoneRequiredInAllLODs(Bone) )
				{
					MissingBoneList.Append(TEXT("\n"));
					Bone.AppendString(MissingBoneList);
				}
			}

			if (MissingBoneList.Len() > 0 )
			{
				OutWarnings.Emplace(
					FText::Format(LOCTEXT("BonesLODOutError", "Filtered Bones may not animate in all LODs, this can lead to incorrect results when animating at those LOD levels.\n{0}"), FText::FromString(MissingBoneList)),
					LOCTEXT("BonesLODOutErrorSummary", "Filtered bones may not animate in all LODs."),
					FNiagaraDataInterfaceFix()
				);
			}
		}
	}
}

//Deprecated functions we check for and advise on updates in ValidateFunction
static const FName GetTriPositionName_DEPRECATED("GetTriPosition");
static const FName GetTriPositionWSName_DEPRECATED("GetTriPositionWS");
static const FName GetTriPositionVelocityAndNormalName_DEPRECATED("GetTriPositionVelocityAndNormal");
static const FName GetTriPositionVelocityAndNormalWSName_DEPRECATED("GetTriPositionVelocityAndNormalWS");
static const FName GetTriPositionVelocityAndNormalBinormalTangentName_DEPRECATED("GetTriPositionVelocityAndNormalBinormalTangent");
static const FName GetTriPositionVelocityAndNormalBinormalTangentWSName_DEPRECATED("GetTriPositionVelocityAndNormalBinormalTangentWS");

void UNiagaraDataInterfaceSkeletalMesh::ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors)
{
	TArray<FNiagaraFunctionSignature> DIFuncs;
	GetFunctionsInternal(DIFuncs);

	if (!DIFuncs.Contains(Function))
	{
		TArray<FNiagaraFunctionSignature> SkinnedDataDeprecatedFunctions;
		{
			FNiagaraFunctionSignature Sig;
			Sig.Name = GetTriPositionName_DEPRECATED;
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position")));
			Sig.bMemberFunction = true;
			Sig.bRequiresContext = false;
			SkinnedDataDeprecatedFunctions.Add(Sig);
		}

		{
			FNiagaraFunctionSignature Sig;
			Sig.Name = GetTriPositionWSName_DEPRECATED;
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position")));
			Sig.bMemberFunction = true;
			Sig.bRequiresContext = false;
			SkinnedDataDeprecatedFunctions.Add(Sig);
		}

		{
			FNiagaraFunctionSignature Sig;
			Sig.Name = GetTriPositionVelocityAndNormalName_DEPRECATED;
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
			Sig.bMemberFunction = true;
			Sig.bRequiresContext = false;
			SkinnedDataDeprecatedFunctions.Add(Sig);
		}

		{
			FNiagaraFunctionSignature Sig;
			Sig.Name = GetTriPositionVelocityAndNormalWSName_DEPRECATED;
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
			Sig.bMemberFunction = true;
			Sig.bRequiresContext = false;
			SkinnedDataDeprecatedFunctions.Add(Sig);
		}

		{
			FNiagaraFunctionSignature Sig;
			Sig.Name = GetTriPositionVelocityAndNormalBinormalTangentName_DEPRECATED;
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("UV Set")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV")));
			Sig.bMemberFunction = true;
			Sig.bRequiresContext = false;
			SkinnedDataDeprecatedFunctions.Add(Sig);
		}

		{
			FNiagaraFunctionSignature Sig;
			Sig.Name = GetTriPositionVelocityAndNormalBinormalTangentWSName_DEPRECATED;
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("SkeletalMesh")));
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FMeshTriCoordinate::StaticStruct()), TEXT("Coord")));
			Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("UV Set")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
			Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV")));
			Sig.bMemberFunction = true;
			Sig.bRequiresContext = false;
			SkinnedDataDeprecatedFunctions.Add(Sig);
		}

		if (SkinnedDataDeprecatedFunctions.Contains(Function))
		{
			OutValidationErrors.Add(FText::Format(LOCTEXT("SkinnedDataFunctionDeprecationMsgFmt", "Skeletal Mesh DI Function {0} has been deprecated. Use GetSinnedTriangleData or GetSkinnedTriangleDataWS instead.\n"), FText::FromName(Function.Name)));
		}
		else
		{
			Super::ValidateFunction(Function, OutValidationErrors);
		}
	}
}

#endif

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceSkeletalMesh::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
		return false;

	InVisitor->UpdateShaderFile(NDISkelMeshLocal::CommonShaderFile);
	InVisitor->UpdateShaderFile(NDISkelMeshLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<NDISkelMeshLocal::FShaderParameters>();

	InVisitor->UpdatePOD(TEXT("NDISkelmesh_Influences"), int(GetDefault<UNiagaraSettings>()->NDISkelMesh_GpuMaxInfluences));
	InVisitor->UpdatePOD(TEXT("NDISkelmesh_ProbAliasFormat"), int(GetDefault<UNiagaraSettings>()->NDISkelMesh_GpuUniformSamplingFormat));

	return true;
}
#endif

#if WITH_EDITOR
void UNiagaraDataInterfaceSkeletalMesh::ModifyCompilationEnvironment(EShaderPlatform ShaderPlatform, struct FShaderCompilerEnvironment& OutEnvironment) const
{
	Super::ModifyCompilationEnvironment(ShaderPlatform, OutEnvironment);

	OutEnvironment.SetDefine(TEXT("DISKELMESH_ALLOW_DEFORMED"), int(GetDefault<UNiagaraSettings>()->NDISkelMesh_SupportReadingDeformedGeometry));
	OutEnvironment.SetDefine(TEXT("DISKELMESH_ALLOW_16BIT"), int(GetDefault<UNiagaraSettings>()->NDISkelMesh_Support16BitIndexWeight));
	OutEnvironment.SetDefine(TEXT("DISKELMESH_BONE_INFLUENCES"), int(GetDefault<UNiagaraSettings>()->NDISkelMesh_GpuMaxInfluences));
	OutEnvironment.SetDefine(TEXT("DISKELMESH_PROBALIAS_FORMAT"), int(GetDefault<UNiagaraSettings>()->NDISkelMesh_GpuUniformSamplingFormat));
	OutEnvironment.SetDefine(TEXT("DISKELMESH_ADJ_INDEX_FORMAT"), int(GetDefault<UNiagaraSettings>()->NDISkelMesh_AdjacencyTriangleIndexFormat));
}
#endif

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSkeletalMesh::GetCommonHLSL(FString& OutHLSL)
{
	OutHLSL.Appendf(TEXT("#include \"%s\"\n"), NDISkelMeshLocal::CommonShaderFile);
	OutHLSL.Append(TEXT("#include \"/Plugin/FX/Niagara/Private/Experimental/NiagaraUvMappingUtils.ush\"\n"));
}

void UNiagaraDataInterfaceSkeletalMesh::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDISkelMeshLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceSkeletalMesh::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	static const TSet<FName> ValidGpuFunctions =
	{
		NDISkelMeshLocal::NAME_GetPreSkinnedLocalBounds,
		//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
		// Triangle Sampling
		FSkeletalMeshInterfaceHelper::GetTriangleIndicesName,
		FSkeletalMeshInterfaceHelper::GetTriCoordVerticesName,
		FSkeletalMeshInterfaceHelper::GetTriangleCountName,
		FSkeletalMeshInterfaceHelper::GetFilteredTriangleCountName,
		FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataName,
		FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataWSName,
		FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataInterpName,
		FSkeletalMeshInterfaceHelper::GetSkinnedTriangleVertexDataWSInterpName,
		//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
		// Bone Sampling
		FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataName,
		FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataInterpolatedName,
		FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSName,
		FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSInterpolatedName,
		//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
		// Vertex Sampling
		FSkeletalMeshInterfaceHelper::GetVertexDataName,
		FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataName,
		FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataWSName,
		FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataInterpolatedName,
		FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataInterpolatedWSName,
		FSkeletalMeshInterfaceHelper::GetVertexColorName,
		FSkeletalMeshInterfaceHelper::GetVertexUVName,
		FSkeletalMeshInterfaceHelper::IsValidVertexName,
		FSkeletalMeshInterfaceHelper::RandomVertexName,
		FSkeletalMeshInterfaceHelper::GetVertexCountName,
		FSkeletalMeshInterfaceHelper::IsValidFilteredVertexName,
		FSkeletalMeshInterfaceHelper::RandomFilteredVertexName,
		FSkeletalMeshInterfaceHelper::GetFilteredVertexCountName,
		FSkeletalMeshInterfaceHelper::GetFilteredVertexAtName,
		//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
		// Filtered Bone
		FSkeletalMeshInterfaceHelper::IsValidBoneName,
		FSkeletalMeshInterfaceHelper::RandomBoneName,
		FSkeletalMeshInterfaceHelper::GetBoneCountName,
		FSkeletalMeshInterfaceHelper::GetParentBoneName,
		FSkeletalMeshInterfaceHelper::GetFilteredBoneCountName,
		FSkeletalMeshInterfaceHelper::GetFilteredBoneAtName,
		FSkeletalMeshInterfaceHelper::RandomFilteredBoneName,
		FSkeletalMeshInterfaceHelper::GetUnfilteredBoneCountName,
		FSkeletalMeshInterfaceHelper::GetUnfilteredBoneAtName,
		FSkeletalMeshInterfaceHelper::RandomUnfilteredBoneName,
		//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
		// Filtered Socket
		FSkeletalMeshInterfaceHelper::GetFilteredSocketCountName,
		FSkeletalMeshInterfaceHelper::GetFilteredSocketBoneAtName,
		FSkeletalMeshInterfaceHelper::GetFilteredSocketTransformName,
		FSkeletalMeshInterfaceHelper::RandomFilteredSocketName,
		//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
		// Misc bone functions
		FSkeletalMeshInterfaceHelper::RandomFilteredSocketOrBoneName,
		FSkeletalMeshInterfaceHelper::GetFilteredSocketOrBoneCountName,
		FSkeletalMeshInterfaceHelper::GetFilteredSocketOrBoneAtName,
		//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
		// Uv Mapping
		FSkeletalMeshInterfaceHelper::GetAdjacentTriangleIndexName,
		FSkeletalMeshInterfaceHelper::GetTriangleNeighborName,
	};

	if ( ValidGpuFunctions.Contains(FunctionInfo.DefinitionName) )
	{
		return true;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Special cases due to using FMeshTriCoordinate
	TMap<FString, FStringFormatArg> FormatArgs =
	{
		{TEXT("InstanceFunctionName"),	FunctionInfo.InstanceName},
		{TEXT("ParameterName"),			ParamInfo.DataInterfaceHLSLSymbol},
	};

	const TCHAR* FuncFormat = nullptr;
	if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::IsValidTriCoordName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(in MeshTriCoordinate InCoord, out bool bIsValid) { IsValidTriCoord_{ParameterName}(InCoord.Tri, bIsValid); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::RandomTriangleName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(NiagaraRandInfo InRandomInfo, out MeshTriCoordinate OutCoord) { RandomTriangle_{ParameterName}(InRandomInfo.Seed1, InRandomInfo.Seed2, InRandomInfo.Seed3, OutCoord.Tri, OutCoord.BaryCoord); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::RandomTriCoordName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(NiagaraRandInfo InRandomInfo, out MeshTriCoordinate OutCoord) { RandomTriangle_{ParameterName}(InRandomInfo.Seed1, InRandomInfo.Seed2, InRandomInfo.Seed3, OutCoord.Tri, OutCoord.BaryCoord); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::RandomFilteredTriangleName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(NiagaraRandInfo InRandomInfo, out MeshTriCoordinate OutCoord) { RandomFilteredTriangle_{ParameterName}(InRandomInfo.Seed1, InRandomInfo.Seed2, InRandomInfo.Seed3, OutCoord.Tri, OutCoord.BaryCoord); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::GetFilteredTriangleAtName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(int FilteredIndex, in float3 BaryCoord, out MeshTriCoordinate OutCoord) { GetFilteredTriangleAt_{ParameterName}(FilteredIndex, OutCoord.Tri); OutCoord.BaryCoord = BaryCoord; }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::GetTriangleDataName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(in MeshTriCoordinate InCoord, out float3 OutPosition, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent) { GetPointOnTriangle_{ParameterName}(InCoord.Tri, InCoord.BaryCoord, OutPosition, OutTangent, OutBinormal, OutNormal); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataWSName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(in MeshTriCoordinate InCoord, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent) { GetSkinnedTriangleDataWS_{ParameterName}(InCoord.Tri, InCoord.BaryCoord, OutPosition, OutVelocity, OutNormal, OutBinormal, OutTangent); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataWSInterpName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(in MeshTriCoordinate InCoord, in float InInterp, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent) { GetSkinnedTriangleDataInterpolatedWS_{ParameterName}(InCoord.Tri, InCoord.BaryCoord, InInterp, OutPosition, OutVelocity, OutNormal, OutBinormal, OutTangent); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(in MeshTriCoordinate InCoord, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent) { GetSkinnedTriangleData_{ParameterName}(InCoord.Tri, InCoord.BaryCoord, OutPosition, OutVelocity, OutNormal, OutBinormal, OutTangent); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataInterpName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(in MeshTriCoordinate InCoord, in float InInterp, out float3 OutPosition, out float3 OutVelocity, out float3 OutNormal, out float3 OutBinormal, out float3 OutTangent) { GetSkinnedTriangleDataInterpolated_{ParameterName}(InCoord.Tri, InCoord.BaryCoord, InInterp, OutPosition, OutVelocity, OutNormal, OutBinormal, OutTangent); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::GetTriUVName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(in MeshTriCoordinate InCoord, in int InUVSet, out float2 OutUV) { GetTriUV_{ParameterName}(InCoord.Tri, InCoord.BaryCoord, InUVSet, OutUV); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::GetTriColorName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(in MeshTriCoordinate InCoord, out float4 OutColor) { GetTriColor_{ParameterName}(InCoord.Tri, InCoord.BaryCoord, OutColor); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::GetTriangleCoordAtUVName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(in bool InEnabled, in float2 InUV, in float InTolerance, out MeshTriCoordinate OutCoord, out bool OutIsValid) { GetTriangleCoordAtUV_{ParameterName}(InEnabled, InUV, InTolerance, OutCoord.Tri, OutCoord.BaryCoord, OutIsValid); }\n");
	}
	else if (FunctionInfo.DefinitionName == FSkeletalMeshInterfaceHelper::GetTriangleCoordInAabbName)
	{
		FuncFormat = TEXT("void {InstanceFunctionName}(in bool InEnabled, in float2 InUvMin, in float2 InUvMax, out MeshTriCoordinate OutCoord, out bool OutIsValid) { GetTriangleCoordInAabb_{ParameterName}(InEnabled, InUvMin, InUvMax, OutCoord.Tri, OutCoord.BaryCoord, OutIsValid); }\n");
	}

	if ( FuncFormat != nullptr )
	{
		OutHLSL += FString::Format(FuncFormat, FormatArgs);
		return true;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Unsupported functionality
	return false;
}

bool UNiagaraDataInterfaceSkeletalMesh::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	bool bWasChanged = false;

	// Early out for version matching
	if (FunctionSignature.FunctionVersion == FNiagaraSkelMeshDIFunctionVersion::LatestVersion)
	{
		return bWasChanged;
	}

	// Renamed some functions and added Random Info to Various functions for consistency
	if (FunctionSignature.FunctionVersion < FNiagaraSkelMeshDIFunctionVersion::AddedRandomInfo)
	{
		static const TPair<FName, FName> FunctionRenames[] =
		{
			MakeTuple(FName("IsValidBone"), FSkeletalMeshInterfaceHelper::IsValidBoneName),
			MakeTuple(FName("RandomSpecificBone"), FSkeletalMeshInterfaceHelper::RandomFilteredBoneName),
			MakeTuple(FName("GetSpecificBoneCount"), FSkeletalMeshInterfaceHelper::GetFilteredBoneCountName),
			MakeTuple(FName("GetSpecificBone"), FSkeletalMeshInterfaceHelper::GetFilteredBoneAtName),
			MakeTuple(FName("RandomSpecificSocketBone"), FSkeletalMeshInterfaceHelper::RandomFilteredSocketName),
			MakeTuple(FName("GetSpecificSocketCount"), FSkeletalMeshInterfaceHelper::GetFilteredSocketCountName),
			MakeTuple(FName("GetSpecificSocketTransform"), FSkeletalMeshInterfaceHelper::GetFilteredSocketTransformName),
			MakeTuple(FName("GetSpecificSocketBone"), FSkeletalMeshInterfaceHelper::GetFilteredSocketBoneAtName),
			MakeTuple(FName("RandomFilteredSocketBone"), FSkeletalMeshInterfaceHelper::RandomFilteredSocketName),
		};

		for (const auto& RenamePair : FunctionRenames)
		{
			if (FunctionSignature.Name == RenamePair.Key)
			{
				FunctionSignature.Name = RenamePair.Value;
				bWasChanged = true;
				break;
			}
		}

		if (FunctionSignature.Name == FSkeletalMeshInterfaceHelper::RandomTriCoordName)
		{
			if (FunctionSignature.Inputs.Num() == 1)
			{
				FunctionSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
				bWasChanged = true;
			}
		}
		else if (FunctionSignature.Name == FSkeletalMeshInterfaceHelper::RandomFilteredBoneName)
		{
			FunctionSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
			bWasChanged = true;
		}
		else if (FunctionSignature.Name == FSkeletalMeshInterfaceHelper::RandomFilteredSocketName)
		{
			FunctionSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
			bWasChanged = true;
		}
		else if (FunctionSignature.Name == FSkeletalMeshInterfaceHelper::RandomVertexName)
		{
			FunctionSignature.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraRandInfo::StaticStruct()), TEXT("RandomInfo")));
			bWasChanged = true;
		}
	}

	// Vertex sampling clean up
	if (FunctionSignature.FunctionVersion < FNiagaraSkelMeshDIFunctionVersion::CleanUpVertexSampling)
	{
		static const TPair<FName, FName> FunctionRenames[] =
		{
			MakeTuple(FName("IsValidVertex"), FSkeletalMeshInterfaceHelper::IsValidVertexName),
			MakeTuple(FName("RandomVertex"), FSkeletalMeshInterfaceHelper::RandomFilteredVertexName),
		};

		for (const auto& RenamePair : FunctionRenames)
		{
			if (FunctionSignature.Name == RenamePair.Key)
			{
				FunctionSignature.Name = RenamePair.Value;
				bWasChanged = true;
				break;
			}
		}
	}

	// Clean up CleanupBoneSampling
	if (FunctionSignature.FunctionVersion < FNiagaraSkelMeshDIFunctionVersion::CleanupBoneSampling)
	{
		static const TPair<FName, FName> FunctionRenames[] =
		{
			MakeTuple(FName("GetFilteredSocketBone"), FSkeletalMeshInterfaceHelper::GetFilteredSocketBoneAtName),
		};

		for (const auto& RenamePair : FunctionRenames)
		{
			if (FunctionSignature.Name == RenamePair.Key)
			{
				FunctionSignature.Name = RenamePair.Value;
				bWasChanged = true;
				break;
			}
		}
	}

	// Added tangent basis to GetSkinnedVertexData
	if (FunctionSignature.FunctionVersion < FNiagaraSkelMeshDIFunctionVersion::AddTangentBasisToGetSkinnedVertexData)
	{
		if ((FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataWSName))
		{
			if ( ensure(FunctionSignature.Outputs.Num() == 2) )
			{
				FunctionSignature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));
				FunctionSignature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Binormal")));
				FunctionSignature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
				bWasChanged = true;
			}
		}
	}

	// Added a new Tolerance parameter to GetTriangleCoordAtUV
	if (FunctionSignature.FunctionVersion < FNiagaraSkelMeshDIFunctionVersion::RemoveUvSetFromMapping)
	{
		if (FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetTriangleCoordAtUVName)
		{
			for (int32 InputIndex = FunctionSignature.Inputs.Num() - 1; InputIndex >= 0; --InputIndex)
			{
				if (FunctionSignature.Inputs[InputIndex].GetName() == TEXT("UV Set"))
				{
					FunctionSignature.Inputs.RemoveAt(InputIndex);
				}
			}

			FNiagaraVariable ToleranceVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Tolerance"));
			ToleranceVariable.SetValue(KINDA_SMALL_NUMBER);

			FunctionSignature.Inputs.Add(ToleranceVariable);
			bWasChanged = true;
		}
	}

	if (FunctionSignature.FunctionVersion < FNiagaraSkelMeshDIFunctionVersion::AddedEnabledUvMapping)
	{
		if ((FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetTriangleCoordAtUVName)
			|| (FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetTriangleCoordInAabbName))
		{
			FNiagaraVariable EnabledVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enabled"));
			EnabledVariable.SetValue(true);
			FunctionSignature.Inputs.Insert(EnabledVariable, 1);
			bWasChanged = true;
		}
	}

	if (FunctionSignature.FunctionVersion < FNiagaraSkelMeshDIFunctionVersion::AddedValidConnectivity)
	{
		if ((FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetAdjacentTriangleIndexName)
			|| (FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetTriangleNeighborName))
		{
			FunctionSignature.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid")));
			bWasChanged = true;
		}
	}

	if (FunctionSignature.FunctionVersion < FNiagaraSkelMeshDIFunctionVersion::AddedInputBardCoordToGetFilteredTriangleAt)
	{
		if ( FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetFilteredTriangleAtName )
		{
			FunctionSignature.Inputs.Add_GetRef(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BaryCoord"))).SetValue(FVector3f(1.0f / 3.0f));
			bWasChanged = true;
		}
	}

	if (FunctionSignature.FunctionVersion < FNiagaraSkelMeshDIFunctionVersion::LargeWorldCoordinates2)
	{
		if (
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetTriangleDataName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataWSName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataInterpName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedTriangleDataWSInterpName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataInterpolatedName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSInterpolatedName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetVertexDataName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedVertexDataWSName) )
		{
			check(FunctionSignature.Outputs[0].GetName() == TEXT("Position"));
			check(FunctionSignature.Outputs[0].GetType() == FNiagaraTypeDefinition::GetVec3Def() || FunctionSignature.Outputs[0].GetType() == FNiagaraTypeDefinition::GetPositionDef());
			FunctionSignature.Outputs[0].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bWasChanged = true;
		}
	}

	if (FunctionSignature.FunctionVersion < FNiagaraSkelMeshDIFunctionVersion::AddBoneScale)
	{
		if (
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataInterpolatedName) ||
			(FunctionSignature.Name == FSkeletalMeshInterfaceHelper::GetSkinnedBoneDataWSInterpolatedName) )
		{
			// Some bogus content appears to exist which only contains Position & Velocity outputs
			if ((FunctionSignature.Outputs.Num() <= 2) && (FunctionSignature.Outputs[1].GetName() == TEXT("Velocity")))
			{
				FunctionSignature.Outputs.EmplaceAt(1, FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation"));
			}

			check(FunctionSignature.Outputs[2].GetName() == TEXT("Velocity"));
			FunctionSignature.Outputs.EmplaceAt(2, FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale"));
			bWasChanged = true;
		}
	}

	// Set latest version
	FunctionSignature.FunctionVersion = FNiagaraSkelMeshDIFunctionVersion::LatestVersion;

	return bWasChanged;
}
#endif

void UNiagaraDataInterfaceSkeletalMesh::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDISkelMeshLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceSkeletalMesh::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxySkeletalMesh& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxySkeletalMesh>();
	FNiagaraDataInterfaceProxySkeletalMeshData* InstanceData = DIProxy.SystemInstancesToData.Find(Context.GetSystemInstanceID());

	NDISkelMeshLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDISkelMeshLocal::FShaderParameters>();
	if (InstanceData && InstanceData->StaticBuffers)
	{
		FSkeletalMeshGpuSpawnStaticBuffers* StaticBuffers = InstanceData->StaticBuffers;

		// Bind mesh buffers
		ShaderParameters->MeshVertexBuffer = StaticBuffers->GetBufferPositionSRV();
		ShaderParameters->MeshIndexBuffer = StaticBuffers->GetBufferIndexSRV();
		ShaderParameters->MeshTangentBuffer = StaticBuffers->GetBufferTangentSRV();
		ShaderParameters->MeshTexCoordBuffer = StaticBuffers->GetBufferTexCoordSRV();
		ShaderParameters->MeshColorBuffer = StaticBuffers->GetBufferColorSRV();

		ShaderParameters->MeshNumTexCoord = StaticBuffers->GetNumTexCoord();
		ShaderParameters->MeshTriangleCount = StaticBuffers->GetTriangleCount();
		ShaderParameters->MeshVertexCount = StaticBuffers->GetVertexCount();

		// Set triangle sampling buffer
		ShaderParameters->MeshTriangleSamplerProbAliasBuffer = StaticBuffers->GetBufferTriangleUniformSamplerProbAliasSRV();

		// Set triangle sampling region buffer
		ShaderParameters->MeshNumSamplingRegionTriangles = StaticBuffers->GetNumSamplingRegionTriangles();
		ShaderParameters->MeshNumSamplingRegionVertices = StaticBuffers->GetNumSamplingRegionVertices();
		ShaderParameters->MeshSamplingRegionsProbAliasBuffer = StaticBuffers->GetSampleRegionsProbAliasSRV();
		ShaderParameters->MeshSampleRegionsTriangleIndices = StaticBuffers->GetSampleRegionsTriangleIndicesSRV();
		ShaderParameters->MeshSampleRegionsVertices = StaticBuffers->GetSampleRegionsVerticesSRV();

		ShaderParameters->MeshSkinWeightBuffer = FNiagaraRenderer::GetSrvOrDefaultUInt(InstanceData->MeshSkinWeightBuffer->GetSRV());
		ShaderParameters->MeshSkinWeightLookupBuffer = FNiagaraRenderer::GetSrvOrDefaultUInt(InstanceData->MeshSkinWeightLookupBuffer->GetSRV());

		uint32 EnabledFeaturesBits = 0;
		EnabledFeaturesBits |= StaticBuffers->IsUseGpuUniformlyDistributedSampling() ? 0x01 : 0;
		EnabledFeaturesBits |= StaticBuffers->IsSamplingRegionsAllAreaWeighted() ? 0x02 : 0;
		EnabledFeaturesBits |= InstanceData->bUnlimitedBoneInfluences ? 0x04 : 0;
		EnabledFeaturesBits |= StaticBuffers->HasMeshColors() ? 0x08 : 0;

		// If we are allowed to read from the skin cache then find the information
		ShaderParameters->DeformedCurrPositionBuffer = FNiagaraRenderer::GetDummyFloatBuffer();
		ShaderParameters->DeformedPrevPositionBuffer = FNiagaraRenderer::GetDummyFloatBuffer();
		ShaderParameters->DeformedTangentBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();

		if (InstanceData->bReadDeformedGeometry)
		{
			USkinnedMeshComponent* SkinnedComponent = Cast<USkinnedMeshComponent>(StaticBuffers->GetSceneComponent());
			FSkeletalMeshObject* SkelMeshObject = SkinnedComponent ? SkinnedComponent->MeshObject : nullptr;
			if (SkelMeshObject)
			{
				FCachedGeometry CacheGeometry;
				bool bCacheValid = SkelMeshObject->GetCachedGeometry(Context.GetGraphBuilder(), CacheGeometry);
				bCacheValid &= CacheGeometry.Sections.Num() > 0;
				bCacheValid &= StaticBuffers->GetLODIndex() == CacheGeometry.LODIndex;
				if (bCacheValid)
				{
					if (CacheGeometry.Sections[0].PositionBuffer && (CacheGeometry.Sections[0].PositionBuffer != ShaderParameters->MeshVertexBuffer))
					{
						ShaderParameters->DeformedCurrPositionBuffer = CacheGeometry.Sections[0].PositionBuffer;
						ShaderParameters->DeformedPrevPositionBuffer = CacheGeometry.Sections[0].PreviousPositionBuffer ? CacheGeometry.Sections[0].PreviousPositionBuffer : CacheGeometry.Sections[0].PositionBuffer;
						EnabledFeaturesBits |= 0x10;
					}
					if (CacheGeometry.Sections[0].TangentBuffer && (CacheGeometry.Sections[0].TangentBuffer != ShaderParameters->MeshTangentBuffer))
					{
						ShaderParameters->DeformedTangentBuffer = CacheGeometry.Sections[0].TangentBuffer;
						EnabledFeaturesBits |= 0x20;
					}
				}
			}
		}

		FSkeletalMeshGpuDynamicBufferProxy* DynamicBuffers = InstanceData->DynamicBuffer;
		check(DynamicBuffers);
		if (DynamicBuffers->DoesBoneDataExist())
		{
			ShaderParameters->MeshNumWeights = StaticBuffers->GetNumWeights();
			ShaderParameters->MeshBoneWeightStride = InstanceData->MeshBoneWeightStrideBytes / 4;
			ShaderParameters->MeshBoneIndexSizeBytes = InstanceData->MeshBoneIndexSizeBytes;
			ShaderParameters->MeshBoneWeightSizeBytes = InstanceData->MeshBoneWeightSizeBytes;

			ShaderParameters->MeshCurrBonesBuffer = DynamicBuffers->GetRWBufferBone().SectionSRV;
			ShaderParameters->MeshPrevBonesBuffer = DynamicBuffers->GetRWBufferPrevBone().SectionSRV;
			ShaderParameters->MeshCurrSamplingBonesBuffer = DynamicBuffers->GetRWBufferBone().SamplingSRV;
			ShaderParameters->MeshPrevSamplingBonesBuffer = DynamicBuffers->GetRWBufferPrevBone().SamplingSRV;
			ShaderParameters->MeshTriangleMatricesOffsetBuffer = StaticBuffers->GetBufferTriangleMatricesOffsetSRV();
		}
		// Bind dummy data for validation purposes only.  Code will not execute due to "EnabledFeatures" bits but validation can not determine that.
		else
		{
			ShaderParameters->MeshNumWeights = 0;
			ShaderParameters->MeshBoneWeightStride = 0;
			ShaderParameters->MeshBoneIndexSizeBytes = 0;
			ShaderParameters->MeshBoneWeightSizeBytes = 0;

			ShaderParameters->MeshCurrBonesBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
			ShaderParameters->MeshPrevBonesBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
			ShaderParameters->MeshCurrSamplingBonesBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
			ShaderParameters->MeshPrevSamplingBonesBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
			ShaderParameters->MeshTriangleMatricesOffsetBuffer = FNiagaraRenderer::GetDummyUIntBuffer();
		}

		ShaderParameters->NumBones = DynamicBuffers->GetNumBones();

		ShaderParameters->NumFilteredBones = StaticBuffers->GetNumFilteredBones();
		ShaderParameters->NumUnfilteredBones = StaticBuffers->GetNumUnfilteredBones();
		ShaderParameters->RandomMaxBone = StaticBuffers->GetExcludedBoneIndex() >= 0 ? DynamicBuffers->GetNumBones() - 2 : DynamicBuffers->GetNumBones() - 1;
		ShaderParameters->ExcludeBoneIndex = StaticBuffers->GetExcludedBoneIndex();
		ShaderParameters->FilteredAndUnfilteredBones = StaticBuffers->GetFilteredAndUnfilteredBonesSRV();

		ShaderParameters->NumFilteredSockets = StaticBuffers->GetNumFilteredSockets();
		ShaderParameters->FilteredSocketBoneOffset = StaticBuffers->GetFilteredSocketBoneOffset();

		if (InstanceData->UvMappingBuffer)
		{
			ShaderParameters->UvMappingBuffer = InstanceData->UvMappingBuffer->GetSrv();
			ShaderParameters->UvMappingBufferLength = InstanceData->UvMappingBuffer->GetBufferSize();
			ShaderParameters->UvMappingSet = InstanceData->UvMappingSet;
		}
		else
		{
			ShaderParameters->UvMappingBuffer = FNiagaraRenderer::GetDummyIntBuffer();
			ShaderParameters->UvMappingBufferLength = 0;
			ShaderParameters->UvMappingSet = 0;
		}

		if (InstanceData->ConnectivityBuffer)
		{
			const uint32 NumBufferElements = FMath::DivideAndRoundUp<uint32>(InstanceData->ConnectivityBuffer->GetBufferSize(), sizeof(uint32));
			ShaderParameters->ConnectivityBuffer = InstanceData->ConnectivityBuffer->GetSrv();
			ShaderParameters->ConnectivityBufferLength = NumBufferElements;
			ShaderParameters->ConnectivityMaxAdjacentPerVertex = InstanceData->ConnectivityBuffer->MaxAdjacentTriangleCount;
		}
		else
		{
			ShaderParameters->ConnectivityBuffer = FNiagaraRenderer::GetDummyUIntBuffer();
			ShaderParameters->ConnectivityBufferLength = 0;
			ShaderParameters->ConnectivityMaxAdjacentPerVertex = 0;
		}

		ShaderParameters->InstanceTransform = InstanceData->Transform;
		ShaderParameters->InstancePrevTransform = InstanceData->PrevTransform;
		ShaderParameters->InstanceRotation = InstanceData->Transform.GetMatrixWithoutScale().ToQuat();
		ShaderParameters->InstancePrevRotation = InstanceData->PrevTransform.GetMatrixWithoutScale().ToQuat();
		ShaderParameters->InstanceInvDeltaTime = 1.0f / InstanceData->DeltaSeconds;

		ShaderParameters->PreSkinnedLocalBoundsCenter = InstanceData->PreSkinnedLocalBoundsCenter;
		ShaderParameters->PreSkinnedLocalBoundsExtents = InstanceData->PreSkinnedLocalBoundsExtents;

		ShaderParameters->EnabledFeatures = EnabledFeaturesBits;
	}
	else
	{
		// Bind dummy buffers
		ShaderParameters->MeshVertexBuffer = FNiagaraRenderer::GetDummyFloatBuffer();
		ShaderParameters->MeshIndexBuffer = FNiagaraRenderer::GetDummyUIntBuffer();
		ShaderParameters->MeshTangentBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();

		ShaderParameters->MeshNumTexCoord = 0;
		ShaderParameters->MeshTexCoordBuffer = FNiagaraRenderer::GetDummyFloat2Buffer();
		ShaderParameters->MeshColorBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->MeshTriangleCount = 0;
		ShaderParameters->MeshVertexCount = 0;
		ShaderParameters->MeshTriangleSamplerProbAliasBuffer = FNiagaraRenderer::GetDummyUIntBuffer();

		ShaderParameters->MeshNumSamplingRegionTriangles = 0;
		ShaderParameters->MeshNumSamplingRegionVertices = 0;
		ShaderParameters->MeshSamplingRegionsProbAliasBuffer = FNiagaraRenderer::GetDummyUIntBuffer();
		ShaderParameters->MeshSampleRegionsTriangleIndices = FNiagaraRenderer::GetDummyUIntBuffer();
		ShaderParameters->MeshSampleRegionsVertices = FNiagaraRenderer::GetDummyUIntBuffer();

		ShaderParameters->MeshSkinWeightBuffer = FNiagaraRenderer::GetDummyUIntBuffer();
		ShaderParameters->MeshSkinWeightLookupBuffer = FNiagaraRenderer::GetDummyUIntBuffer();

		ShaderParameters->MeshNumWeights = 0;
		ShaderParameters->MeshBoneWeightStride = 0;
		ShaderParameters->MeshBoneIndexSizeBytes = 0;
		ShaderParameters->MeshBoneWeightSizeBytes = 0;

		ShaderParameters->MeshCurrBonesBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->MeshPrevBonesBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->MeshCurrSamplingBonesBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->MeshPrevSamplingBonesBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->MeshTriangleMatricesOffsetBuffer = FNiagaraRenderer::GetDummyUIntBuffer();

		ShaderParameters->NumBones = 0;
		ShaderParameters->NumFilteredBones = 0;
		ShaderParameters->NumUnfilteredBones = 0;
		ShaderParameters->RandomMaxBone = 0;
		ShaderParameters->ExcludeBoneIndex = 0;
		ShaderParameters->FilteredAndUnfilteredBones = FNiagaraRenderer::GetDummyUIntBuffer();
		ShaderParameters->NumFilteredSockets = 0;
		ShaderParameters->FilteredSocketBoneOffset = 0;

		ShaderParameters->UvMappingBuffer = FNiagaraRenderer::GetDummyIntBuffer();
		ShaderParameters->UvMappingBufferLength = 0;
		ShaderParameters->UvMappingSet = 0;

		ShaderParameters->ConnectivityBuffer = FNiagaraRenderer::GetDummyUIntBuffer();
		ShaderParameters->ConnectivityBufferLength = 0;
		ShaderParameters->ConnectivityMaxAdjacentPerVertex = 0;

		ShaderParameters->InstanceTransform = FMatrix44f::Identity;
		ShaderParameters->InstancePrevTransform = FMatrix44f::Identity;
		ShaderParameters->InstanceRotation = FQuat4f::Identity;
		ShaderParameters->InstancePrevRotation = FQuat4f::Identity;
		ShaderParameters->InstanceInvDeltaTime = 0.0f;

		ShaderParameters->PreSkinnedLocalBoundsCenter = FVector3f::ZeroVector;
		ShaderParameters->PreSkinnedLocalBoundsExtents = FVector3f::ZeroVector;

		ShaderParameters->EnabledFeatures = 0;

		ShaderParameters->DeformedCurrPositionBuffer = FNiagaraRenderer::GetDummyFloatBuffer();
		ShaderParameters->DeformedPrevPositionBuffer = FNiagaraRenderer::GetDummyFloatBuffer();
		ShaderParameters->DeformedTangentBuffer = FNiagaraRenderer::GetDummyFloat4Buffer();
	}
}

void UNiagaraDataInterfaceSkeletalMesh::BindSourceDelegates()
{
	if (AActor* Source = SoftSourceActor.Get())
	{
		Source->OnEndPlay.AddDynamic(this, &UNiagaraDataInterfaceSkeletalMesh::OnSourceEndPlay);
	}
	else if (SourceComponent)
	{
		UE_CLOGF(!UObjectBaseUtility::IsGarbageEliminationEnabled(), 
			LogNiagara, Warning, "%ls: Unable to bind OnEndPlay for actor-less source component %ls, this may extend the lifetime of the component", 
			*GetFullName(), *SourceComponent->GetPathName());
	}
}

void UNiagaraDataInterfaceSkeletalMesh::UnbindSourceDelegates()
{
	if (AActor* Source = SoftSourceActor.Get())
	{
		Source->OnEndPlay.RemoveAll(this);
	}
}

void UNiagaraDataInterfaceSkeletalMesh::OnSourceEndPlay(AActor* InSource, EEndPlayReason::Type Reason)
{
	// Increment change id in case we're able to find a new source component 
	++ChangeId;
	UnbindSourceDelegates();
	SoftSourceActor = nullptr;
	SourceComponent = nullptr;
}

void UNiagaraDataInterfaceSkeletalMesh::SetSourceComponentFromBlueprints(USkinnedMeshComponent* ComponentToUse)
{
	// NOTE: When ChangeId changes the next tick will be skipped and a reset of the per-instance data will be initiated.
	++ChangeId;
	UnbindSourceDelegates();
	SourceComponent = ComponentToUse;
	SoftSourceActor = ComponentToUse ? ComponentToUse->GetOwner() : nullptr;
	BindSourceDelegates();
}

void UNiagaraDataInterfaceSkeletalMesh::SetSamplingRegionsFromBlueprints(const TArray<FName>& InSamplingRegions)
{
	// NOTE: When ChangeId changes the next tick will be skipped and a reset of the per-instance data will be initiated.
	++ChangeId;
	SamplingRegions = InSamplingRegions;
}

void UNiagaraDataInterfaceSkeletalMesh::SetFilteredBonesFromBlueprints(const TArray<FName>& InFilteredBones)
{
	// NOTE: When ChangeId changes the next tick will be skipped and a reset of the per-instance data will be initiated.
	++ChangeId;
	FilteredBones = InFilteredBones;
}

void UNiagaraDataInterfaceSkeletalMesh::SetFilteredSocketsFromBlueprints(const TArray<FName>& InFilteredSockets)
{
	// NOTE: When ChangeId changes the next tick will be skipped and a reset of the per-instance data will be initiated.
	++ChangeId;
	FilteredSockets = InFilteredSockets;
}

void UNiagaraDataInterfaceSkeletalMesh::SetWholeMeshLODFromBlueprints(int32 InWholeMeshLOD)
{
	// NOTE: When ChangeId changes the next tick will be skipped and a reset of the per-instance data will be initiated.
	++ChangeId;
	WholeMeshLOD = InWholeMeshLOD;
}

ETickingGroup UNiagaraDataInterfaceSkeletalMesh::CalculateTickGroup(const void* PerInstanceData) const
{
	const FNDISkeletalMesh_InstanceData* InstData = static_cast<const FNDISkeletalMesh_InstanceData*>(PerInstanceData);
	USkinnedMeshComponent* SkinnedComponent = Cast<USkinnedMeshComponent>(InstData->WeakSceneComponent.Get());
	if (SkinnedComponent && bRequireCurrentFrameData)
	{
		USkeletalMeshComponent* SkeletalComponent = Cast<USkeletalMeshComponent>(SkinnedComponent);
		return NDISkelMeshLocal::GetComponentTickGroup(SkinnedComponent, SkeletalComponent);
	}
	return NiagaraFirstTickGroup;
}


int32 UNiagaraDataInterfaceSkeletalMesh::CalculateLODIndexAndSamplingRegions(USkinnedAsset* SkinnedMesh, TArray<int32>& OutSamplingRegionIndices, bool& OutAllRegionsAreAreaWeighting) const
{
	check(SkinnedMesh);

	if (SamplingRegions.Num())
	{
		if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(SkinnedMesh))
		{
			int32 LastRegionLODIndex = INDEX_NONE;

			//Sampling from regions. Gather the indices of the regions we'll sample from.
			const FSkeletalMeshSamplingInfo& SamplingInfo = SkeletalMesh->GetSamplingInfo();
			for (FName RegionName : SamplingRegions)
			{
				const int32 RegionIdx = SamplingInfo.IndexOfRegion(RegionName);
				if (RegionIdx != INDEX_NONE)
				{
					const FSkeletalMeshSamplingRegion& Region = SamplingInfo.GetRegion(RegionIdx);
					const FSkeletalMeshSamplingRegionBuiltData& RegionBuiltData = SamplingInfo.GetRegionBuiltData(RegionIdx);

					const int32 RegionLODIndex = Region.LODIndex == INDEX_NONE ? (SkinnedMesh->GetLODNum() - 1) : (FMath::Clamp(Region.LODIndex, 0, SkinnedMesh->GetLODNum() - 1));

					if (LastRegionLODIndex == INDEX_NONE)
					{
						LastRegionLODIndex = RegionLODIndex;
					}
					else if (RegionLODIndex != LastRegionLODIndex)
					{
						UE_LOGF(LogNiagara, Warning, "Skeletal Mesh Data Interface is trying to use regions on LODs levels that are either streamed or cooked out. This is currently unsupported. Interface(%ls) Mesh(%ls) Region(%ls)",
							*GetFullName(),
							*SkinnedMesh->GetFullName(),
							*RegionName.ToString());
						return INDEX_NONE;
					}

					if (RegionBuiltData.TriangleIndices.Num() > 0)
					{
						OutSamplingRegionIndices.Add(RegionIdx);
						OutAllRegionsAreAreaWeighting &= Region.bSupportUniformlyDistributedSampling;
					}
					else
					{
						UE_LOGF(LogNiagara, Warning, "Skeletal Mesh Data Interface is trying to use a region with no associated triangles. LOD(%d) Interface(%ls) Mesh(%ls) Region(%ls)",
							 RegionLODIndex,
							*GetFullName(),
							*SkinnedMesh->GetFullName(),
							*RegionName.ToString());

						return INDEX_NONE;
					}
				}
				else
				{
					UE_LOGF(LogNiagara, Warning, "Skeletal Mesh Data Interface is trying to use a region on a mesh that does not provide this region. Interface(%ls) Mesh(%ls) Region(%ls)",
						*GetFullName(),
						*SkinnedMesh->GetFullName(),
						*RegionName.ToString());

					return INDEX_NONE;
				}
			}
			return LastRegionLODIndex;
		}
	}

	//If we have no regions, sample the whole mesh at the specified LOD.
	if (WholeMeshLOD == INDEX_NONE)
	{
		return SkinnedMesh->GetLODNum() - 1;
	}
	else
	{
		return FMath::Clamp(WholeMeshLOD, 0, SkinnedMesh->GetLODNum() - 1);
	}
}

void UNiagaraDataInterfaceSkeletalMesh::VMGetPreSkinnedLocalBounds(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISkeletalMesh_InstanceData> InstData(Context);
	FNDIOutputParam<FVector3f>	OutCenter(Context);
	FNDIOutputParam<FVector3f>	OutExtentsMin(Context);
	FNDIOutputParam<FVector3f>	OutExtentsMax(Context);
	FNDIOutputParam<FVector3f>	OutExtents(Context);
	FNDIOutputParam<FVector3f>	OutHalfExtents(Context);

	const FVector3f Center = InstData->PreSkinnedLocalBoundsCenter;
	const FVector3f Extents = InstData->PreSkinnedLocalBoundsExtents;
	const FVector3f HalfExtents = Extents * 0.5f;
	const FVector3f ExtentsMin = Center - HalfExtents;
	const FVector3f ExtentsMax = Center + HalfExtents;

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCenter.SetAndAdvance(Center);
		OutExtentsMin.SetAndAdvance(ExtentsMin);
		OutExtentsMax.SetAndAdvance(ExtentsMax);
		OutExtents.SetAndAdvance(Extents);
		OutHalfExtents.SetAndAdvance(HalfExtents);
	}
}

//UNiagaraDataInterfaceSkeletalMesh END
//////////////////////////////////////////////////////////////////////////

template<>
void FSkeletalMeshAccessorHelper::Init<TNDISkelMesh_FilterModeSingle, TNDISkelMesh_AreaWeightingOff>(const FNDISkeletalMesh_InstanceData* InstData)
{
	Comp = Cast<USkeletalMeshComponent>(InstData->WeakSceneComponent.Get());
	Mesh = Cast<USkeletalMesh>(InstData->WeakSkinnedMesh.Get());
	LODData = InstData->CachedLODData;
	SkinWeightBuffer = InstData->GetSkinWeights();
	IndexBuffer = LODData ? LODData->MultiSizeIndexContainer.GetIndexBuffer() : nullptr;
	SkinningData = InstData->SkinningData.SkinningData.Get();
	Usage = InstData->SkinningData.Usage;

	if (Mesh)
	{
		const FSkeletalMeshSamplingInfo& SamplingInfo = Mesh->GetSamplingInfo();
		SamplingRegion = &SamplingInfo.GetRegion(InstData->SamplingRegionIndices[0]);
		SamplingRegionBuiltData = &SamplingInfo.GetRegionBuiltData(InstData->SamplingRegionIndices[0]);
	}
	else
	{
		SamplingRegion = nullptr;
		SamplingRegionBuiltData = nullptr;
	}

	if (SkinningData != nullptr)
	{
		SkinningData->EnterRead();
	}
}

template<>
void FSkeletalMeshAccessorHelper::Init<TNDISkelMesh_FilterModeSingle, TNDISkelMesh_AreaWeightingOn>(const FNDISkeletalMesh_InstanceData* InstData)
{
	Comp = Cast<USkeletalMeshComponent>(InstData->WeakSceneComponent.Get());
	Mesh = Cast<USkeletalMesh>(InstData->WeakSkinnedMesh.Get());
	LODData = InstData->CachedLODData;
	SkinWeightBuffer = InstData->GetSkinWeights();
	IndexBuffer = LODData ? LODData->MultiSizeIndexContainer.GetIndexBuffer() : nullptr;
	SkinningData = InstData->SkinningData.SkinningData.Get();
	Usage = InstData->SkinningData.Usage;

	if (Mesh)
	{
		const FSkeletalMeshSamplingInfo& SamplingInfo = Mesh->GetSamplingInfo();
		SamplingRegion = &SamplingInfo.GetRegion(InstData->SamplingRegionIndices[0]);
		SamplingRegionBuiltData = &SamplingInfo.GetRegionBuiltData(InstData->SamplingRegionIndices[0]);
	}
	else
	{
		SamplingRegion = nullptr;
		SamplingRegionBuiltData = nullptr;
	}

	if (SkinningData != nullptr)
	{
		SkinningData->EnterRead();
	}
}

#undef LOCTEXT_NAMESPACE
