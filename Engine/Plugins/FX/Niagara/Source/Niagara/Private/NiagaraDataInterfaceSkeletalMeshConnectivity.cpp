// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSkeletalMeshConnectivity.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "NiagaraDataInterfaceSkeletalMesh.h"
#include "NiagaraSettings.h"
#include "NiagaraStats.h"

#include "Rendering/SkeletalMeshRenderData.h"
#include <limits>
#include "RenderingThread.h"

DECLARE_CYCLE_STAT(TEXT("Niagara - SkelMesh - BuildAdjacencyBuffer"), STAT_NiagaraSkel_Connectivity_Adjacency, STATGROUP_Niagara);

FSkeletalMeshConnectivityHandle::FSkeletalMeshConnectivityHandle()
	: ConnectivityData(nullptr)
{
}

FSkeletalMeshConnectivityHandle::FSkeletalMeshConnectivityHandle(FSkeletalMeshConnectivityUsage InUsage, const TSharedPtr<FSkeletalMeshConnectivity>& InMappingData, bool bNeedsDataImmediately)
	: Usage(InUsage)
	, ConnectivityData(InMappingData)
{
	if (FSkeletalMeshConnectivity* MappingData = ConnectivityData.Get())
	{
		MappingData->RegisterUser(Usage, bNeedsDataImmediately);
	}
}

FSkeletalMeshConnectivityHandle::~FSkeletalMeshConnectivityHandle()
{
	if (FSkeletalMeshConnectivity* MappingData = ConnectivityData.Get())
	{
		MappingData->UnregisterUser(Usage);
	}
}

FSkeletalMeshConnectivityHandle::FSkeletalMeshConnectivityHandle(FSkeletalMeshConnectivityHandle&& Other) noexcept
{
	Usage = Other.Usage;
	ConnectivityData = Other.ConnectivityData;
	Other.ConnectivityData = nullptr;
}

FSkeletalMeshConnectivityHandle& FSkeletalMeshConnectivityHandle::operator=(FSkeletalMeshConnectivityHandle&& Other) noexcept
{
	if (this != &Other)
	{
		Usage = Other.Usage;
		ConnectivityData = Other.ConnectivityData;
		Other.ConnectivityData = nullptr;
	}
	return *this;
}

FSkeletalMeshConnectivityHandle::operator bool() const
{
	return ConnectivityData.IsValid();
}

int32 FSkeletalMeshConnectivityHandle::GetAdjacentTriangleIndex(int32 VertexIndex, int32 AdjacencyIndex) const
{
	if (ConnectivityData)
	{
		return ConnectivityData->GetAdjacentTriangleIndex(VertexIndex, AdjacencyIndex);
	}

	return INDEX_NONE;
}

const FSkeletalMeshConnectivityProxy* FSkeletalMeshConnectivityHandle::GetProxy() const
{
	if (ConnectivityData)
	{
		return ConnectivityData->GetProxy();
	}
	return nullptr;
}

void FSkeletalMeshConnectivityHandle::PinAndInvalidateHandle()
{
	ConnectivityData.Reset();
}

FSkeletalMeshConnectivity::FSkeletalMeshConnectivity(TWeakObjectPtr<USkinnedAsset> InMeshObject, int32 InLodIndex)
	: LodIndex(InLodIndex)
	, MeshObject(InMeshObject)
	, GpuUserCount(0)
{
}

FSkeletalMeshConnectivity::~FSkeletalMeshConnectivity()
{
	Release();
}

bool FSkeletalMeshConnectivity::IsUsed() const
{
	return (GpuUserCount > 0);
}

bool FSkeletalMeshConnectivity::CanBeDestroyed() const
{
	return !IsUsed();
}

void FSkeletalMeshConnectivity::RegisterUser(FSkeletalMeshConnectivityUsage Usage, bool bNeedsDataImmediately)
{
	if (Usage.RequiresGpuAccess)
	{
		if (GpuUserCount++ == 0)
		{
			check(Proxy == nullptr);
			Proxy.Reset(new FSkeletalMeshConnectivityProxy());
			if (Proxy->Initialize(*this))
			{
				BeginInitResource(Proxy.Get());
			}
			else
			{
				Proxy.Reset();
			}
		}
	}
}

void FSkeletalMeshConnectivity::UnregisterUser(FSkeletalMeshConnectivityUsage Usage)
{
	if (Usage.RequiresGpuAccess)
	{
		if (--GpuUserCount == 0)
		{
			Release();
		}
	}
}

void FSkeletalMeshConnectivity::Release()
{
	if (FSkeletalMeshConnectivityProxy* ProxyPtr = Proxy.Release())
	{
		ENQUEUE_RENDER_COMMAND(BeginDestroyCommand)([RT_Proxy = ProxyPtr](FRHICommandListImmediate& RHICmdList)
		{
			RT_Proxy->ReleaseResource();
			delete RT_Proxy;
		});
	}
}

bool FSkeletalMeshConnectivity::CanBeUsed(const TWeakObjectPtr<USkinnedAsset>& InMeshObject, int32 InLodIndex) const
{
	return IsUsed() && LodIndex == InLodIndex && MeshObject == InMeshObject;
}

bool FSkeletalMeshConnectivity::IsValidMeshObject(TWeakObjectPtr<USkinnedAsset>& MeshObject, int32 InLodIndex)
{
	USkinnedAsset* Mesh = MeshObject.Get();

	if (!Mesh)
	{
		return false;
	}

	const FSkeletalMeshLODInfo* LodInfo = Mesh->GetLODInfo(InLodIndex);
	if (!LodInfo)
	{
		// invalid Lod index
		return false;
	}

	if (!LodInfo->bAllowCPUAccess)
	{
		// we need CPU access to buffers in order to generate our UV mapping quad tree
		return false;
	}

	if (!GetLodRenderData(*Mesh, InLodIndex))
	{
		// no render data available
		return false;
	}

	return true;
}

int32 FSkeletalMeshConnectivity::GetAdjacentTriangleIndex(int32 VertexIndex, int32 AdjacencyIndex) const
{
	return INDEX_NONE;
}

void FSkeletalMeshConnectivity::GetTriangleVertices(int32 TriangleIndex, int32& OutVertex0, int32& OutVertex1, int32& OutVertex2) const
{
	OutVertex0 = INDEX_NONE;
	OutVertex1 = INDEX_NONE;
	OutVertex2 = INDEX_NONE;
}

const FSkeletalMeshLODRenderData* FSkeletalMeshConnectivity::GetLodRenderData(const USkinnedAsset& Mesh, int32 LodIndex)
{
	if (FSkeletalMeshRenderData* RenderData = Mesh.GetResourceForRendering())
	{
		if (RenderData->LODRenderData.IsValidIndex(LodIndex))
		{
			return &RenderData->LODRenderData[LodIndex];
		}
	}

	return nullptr;
}

const FSkeletalMeshLODRenderData* FSkeletalMeshConnectivity::GetLodRenderData() const
{
	if (const USkinnedAsset* Mesh = MeshObject.Get())
	{
		return GetLodRenderData(*Mesh, LodIndex);
	}

	return nullptr;
}

FString FSkeletalMeshConnectivity::GetMeshName() const
{
	if (const USkinnedAsset* Mesh = MeshObject.Get())
	{
		return Mesh->GetPathName();
	}
	return TEXT("<none>");
}

const FSkeletalMeshConnectivityProxy* FSkeletalMeshConnectivity::GetProxy() const
{
	return Proxy.Get();
}

struct FAdjacencyVertexOverlapKey
{
	FVector Position;
};

uint32 GetTypeHash(const FAdjacencyVertexOverlapKey& Key)
{
	return GetTypeHash(Key.Position);
}

bool operator==(const FAdjacencyVertexOverlapKey& Lhs, const FAdjacencyVertexOverlapKey& Rhs)
{
	return Lhs.Position == Rhs.Position;
}

template<typename TriangleIndexType, bool SortBySize>
static bool BuildAdjacencyBuffer(const FSkeletalMeshLODRenderData& LodRenderData, int32 MaxAdjacencyCount, TResourceArray<uint8>& Buffer, int32& MaxFoundAdjacentTriangleCount)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSkel_Connectivity_Adjacency);

	const FRawStaticIndexBuffer16or32Interface* IndexBuffer = LodRenderData.MultiSizeIndexContainer.GetIndexBuffer();
	const FPositionVertexBuffer& VertexBuffer = LodRenderData.StaticVertexBuffers.PositionVertexBuffer;
	const uint32 IndexCount = IndexBuffer->Num();
	const uint32 TriangleCount = IndexCount / 3;

	const TriangleIndexType MaxValidTriangleIndex = std::numeric_limits<TriangleIndexType>::max() - 1; // reserve -1 for an invalid index

	if (TriangleCount >= MaxValidTriangleIndex)
	{
		return false;
	}

	const uint32 VertexCount = LodRenderData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
	const int32 SizePerVertex = MaxAdjacencyCount * sizeof(TriangleIndexType);
	const int32 BufferSize = VertexCount * SizePerVertex;
	const int32 PaddedBufferSize = 4 * FMath::DivideAndRoundUp(BufferSize, 4);
	Buffer.SetNum(PaddedBufferSize, EAllowShrinking::Yes);
	FMemory::Memset(Buffer.GetData(), 0xFF, Buffer.Num());
	TriangleIndexType* AdjacencyBuffer = reinterpret_cast<TriangleIndexType*>(Buffer.GetData());

	TMap<FVector3f, int32 /*UniqueVertexIndex*/> UniqueIndexMap;

	TArray<uint32> RedirectionArray;
	TArray<float> TriangleSizes;

	RedirectionArray.SetNum(VertexCount);
	if (SortBySize)
	{
		TriangleSizes.SetNum(TriangleCount);
	}

	for (TriangleIndexType TriangleIt = 0; TriangleIt < TriangleCount; ++TriangleIt)
	{
		const uint32 V[3] = 
		{
			IndexBuffer->Get(TriangleIt * 3 + 0),
			IndexBuffer->Get(TriangleIt * 3 + 1),
			IndexBuffer->Get(TriangleIt * 3 + 2)
		};

		const FVector3f P[3] =
		{
			VertexBuffer.VertexPosition(V[0]),
			VertexBuffer.VertexPosition(V[1]),
			VertexBuffer.VertexPosition(V[2])
		};

		if (SortBySize)
		{
			TriangleSizes[TriangleIt] = 0.5f * ((P[2] - P[0]) ^ (P[1] - P[0])).Size();
		}

		for (int32 i = 0; i < 3; ++i)
		{
			const uint32 VertexIndex = RedirectionArray[V[i]] = UniqueIndexMap.FindOrAdd(P[i], V[i]);

			TArrayView<TriangleIndexType> AdjacentTriangles = MakeArrayView(AdjacencyBuffer + VertexIndex * MaxAdjacencyCount, MaxAdjacencyCount);

			int32 InsertionPoint = 0;
			while (InsertionPoint < MaxAdjacencyCount)
			{
				const TriangleIndexType TestTriangle = AdjacentTriangles[InsertionPoint];

				if (TestTriangle == TriangleIndexType(INDEX_NONE))
				{
					AdjacentTriangles[InsertionPoint] = TriangleIt;
					break;
				}

				if ((SortBySize && (TriangleSizes[TriangleIt] > TriangleSizes[TestTriangle]))
					|| (!SortBySize && TriangleIt < TestTriangle))
				{
					// skip empty entries
					int32 ShiftIt = MaxAdjacencyCount - 1;
					while (AdjacentTriangles[ShiftIt - 1] == TriangleIndexType(INDEX_NONE))
					{
						--ShiftIt;
					}

					// shift the results down and then insert
					do 
					{
						AdjacentTriangles[ShiftIt] = AdjacentTriangles[ShiftIt - 1];
						--ShiftIt;
					} while (ShiftIt > InsertionPoint);

					AdjacentTriangles[InsertionPoint] = TriangleIt;
					break;
				}

				++InsertionPoint;
			}
		}
	}

	for (uint32 VertexIt = 1; VertexIt < VertexCount; ++VertexIt)
	{
		// if this vertex has a sibling we just copy the data over
		const int32 SiblingIndex = RedirectionArray[VertexIt];
		if (SiblingIndex != VertexIt)
		{
			FMemory::Memcpy(AdjacencyBuffer + VertexIt * MaxAdjacencyCount, AdjacencyBuffer + SiblingIndex * MaxAdjacencyCount, SizePerVertex);
		}
	}

	return true;
}

bool
FSkeletalMeshConnectivityProxy::Initialize(const FSkeletalMeshConnectivity& Connectivity)
{
	if (const FSkeletalMeshLODRenderData* LodRenderData = Connectivity.GetLodRenderData())
	{
		int32 MaxFoundAdjacentTriangleCount = 0;
		bool AdjacencySuccess = false;

		const auto IndexFormat = GetDefault<UNiagaraSettings>()->NDISkelMesh_AdjacencyTriangleIndexFormat;
		if (IndexFormat == ENDISkelMesh_AdjacencyTriangleIndexFormat::Full)
		{
			AdjacencySuccess = BuildAdjacencyBuffer<uint32, true>(*LodRenderData, MaxAdjacentTriangleCount, AdjacencyResource, MaxFoundAdjacentTriangleCount);
		}
		else if (IndexFormat == ENDISkelMesh_AdjacencyTriangleIndexFormat::Half)
		{
			AdjacencySuccess = BuildAdjacencyBuffer<uint16, true>(*LodRenderData, MaxAdjacentTriangleCount, AdjacencyResource, MaxFoundAdjacentTriangleCount);
		}

		if (!AdjacencySuccess)
		{
			UE_LOGF(LogNiagara, Warning, "Failed to build adjacency for %ls.  Check project settings for NDISkelMesh_AdjacencyTriangleIndexFormat.  Currently using %ls.",
				*Connectivity.GetMeshName(), *StaticEnum<ENDISkelMesh_AdjacencyTriangleIndexFormat::Type>()->GetValueAsString(IndexFormat));
		}
		if (MaxFoundAdjacentTriangleCount > MaxAdjacentTriangleCount)
		{
			UE_LOGF(LogNiagara, Warning, "Max adjacency limit of %d exceeded (up to %d found) when processing %ls.  Some connections will be ignored.",
				MaxAdjacentTriangleCount, MaxFoundAdjacentTriangleCount, *Connectivity.GetMeshName());
		}

		return AdjacencySuccess;
	}

	return false;
}

void
FSkeletalMeshConnectivityProxy::InitRHI(FRHICommandListBase& RHICmdList)
{
	const int32 BufferSize = AdjacencyResource.GetResourceDataSize();

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(TEXT("FSkeletalMeshConnectivityProxy_AdjacencyBuffer"), BufferSize)
		.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Static)
		.SetInitActionResourceArray(&AdjacencyResource)
		.DetermineInitialState();

	AdjacencyBuffer = RHICmdList.CreateBuffer(CreateDesc);
	AdjacencySrv = RHICmdList.CreateShaderResourceView(
		AdjacencyBuffer, 
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R32_UINT));

#if STATS
	check(GpuMemoryUsage == 0);
	GpuMemoryUsage = BufferSize;
#endif

	INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GpuMemoryUsage);
}

void
FSkeletalMeshConnectivityProxy::ReleaseRHI()
{
	AdjacencyBuffer.SafeRelease();
	AdjacencySrv.SafeRelease();

	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, GpuMemoryUsage);

#if STATS
	GpuMemoryUsage = 0;
#endif
}
