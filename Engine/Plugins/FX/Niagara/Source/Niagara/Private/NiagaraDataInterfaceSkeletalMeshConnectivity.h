// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/DynamicRHIResourceArray.h"
#include "RenderResource.h"
#include "RHIResources.h"
#include "UObject/WeakObjectPtr.h"

class FSkeletalMeshLODRenderData;
class USkinnedAsset;
struct FSkeletalMeshConnectivityUsage;

struct FSkeletalMeshConnectivity;

class FSkeletalMeshConnectivityProxy : public FRenderResource
{
public:
	bool Initialize(const FSkeletalMeshConnectivity& ConnectivityData);

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	FShaderResourceViewRHIRef GetSrv() const { return AdjacencySrv; }
	uint32 GetBufferSize() const { return AdjacencyBuffer ? AdjacencyBuffer->GetSize() : 0; }

	const int32 MaxAdjacentTriangleCount = 8;

private:
	TResourceArray<uint8> AdjacencyResource;

	FBufferRHIRef AdjacencyBuffer;
	FShaderResourceViewRHIRef AdjacencySrv;

#if STATS
	int64 GpuMemoryUsage = 0;
#endif
};

struct FSkeletalMeshConnectivity
{
	FSkeletalMeshConnectivity() = delete;
	FSkeletalMeshConnectivity(const FSkeletalMeshConnectivity&) = delete;
	FSkeletalMeshConnectivity(TWeakObjectPtr<USkinnedAsset> InMeshObject, int32 InLodIndex);
	~FSkeletalMeshConnectivity();

	bool IsUsed() const;
	bool CanBeDestroyed() const;
	void RegisterUser(FSkeletalMeshConnectivityUsage Usage, bool bNeedsDataImmediately);
	void UnregisterUser(FSkeletalMeshConnectivityUsage Usage);
	bool CanBeUsed(const TWeakObjectPtr<USkinnedAsset>& InMeshObject, int32 InLodIndex) const;
	const FSkeletalMeshConnectivityProxy* GetProxy() const;

	static bool IsValidMeshObject(TWeakObjectPtr<USkinnedAsset>& MeshObject, int32 InLodIndex);

	int32 GetAdjacentTriangleIndex(int32 VertexIndex, int32 AdjacencyIndex) const;
	void GetTriangleVertices(int32 TriangleIndex, int32& OutVertex0, int32& OutVertex1, int32& OutVertex2) const;
	const FSkeletalMeshLODRenderData* GetLodRenderData() const;
	FString GetMeshName() const;

	const int32 LodIndex;

private:
	static const FSkeletalMeshLODRenderData* GetLodRenderData(const USkinnedAsset& Mesh, int32 LodIndex);

	void Release();

	TWeakObjectPtr<USkinnedAsset> MeshObject;
	TUniquePtr<FSkeletalMeshConnectivityProxy> Proxy;

	std::atomic<int32> GpuUserCount;
};
