// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSparseVolumeTexture.h"
#include "NiagaraCompileHashVisitor.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraShader.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "RHIStaticStates.h"
#include "ShaderCompilerCore.h"
#include "GlobalRenderResources.h"
#include "SparseVolumeTexture/ISparseVolumeTextureStreamingManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceSparseVolumeTexture)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterfaceSparseVolumeTexture"

const TCHAR* UNiagaraDataInterfaceSparseVolumeTexture::TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceSparseVolumeTextureTemplate.ush");
const FName UNiagaraDataInterfaceSparseVolumeTexture::LoadSparseVolumeTextureName(TEXT("LoadSparseVolumeTexture"));
const FName UNiagaraDataInterfaceSparseVolumeTexture::SampleSparseVolumeTextureName(TEXT("SampleSparseVolumeTexture"));
const FName UNiagaraDataInterfaceSparseVolumeTexture::GetTextureDimensionsName(TEXT("GetSparseVolumeTextureDimensions"));
const FName UNiagaraDataInterfaceSparseVolumeTexture::GetNumMipLevelsName(TEXT("GetSparseVolumeTextureNumMipLevels"));
const FName UNiagaraDataInterfaceSparseVolumeTexture::RequestSparseVolumeTextureFrameName(TEXT("RequestSparseVolumeTextureFrame"));
const FName UNiagaraDataInterfaceSparseVolumeTexture::GetNumFramesName(TEXT("GetSparseVolumeTextureNumFrames"));

struct FNDISparseVolumeTextureFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

struct FNDISparseVolumeTextureInstanceData_GameThread
{
	TWeakObjectPtr<USparseVolumeTexture> CurrentTexture = nullptr;
	const UE::SVT::FTextureRenderResources* CurrentRenderResources = nullptr;
	FIntVector3 CurrentTextureSize = FIntVector3::ZeroValue;
	int32 CurrentTextureMipLevels = 0;
	FNiagaraParameterDirectBinding<UObject*> UserParamBinding;
	float CurrentFrame = 0;
	float FrameRate = 30.f;
	int32 NumFrames = 0;
};

struct FNDISparseVolumeTextureInstanceData_RenderThread
{
	const UE::SVT::FTextureRenderResources* RenderResources = nullptr;
	FIntVector3 TextureSize = FIntVector3::ZeroValue;
	int32 MipLevels = 0;
	int32 NumFrames = 0;
};

struct FNiagaraDataInterfaceProxySparseVolumeTexture : public FNiagaraDataInterfaceProxy
{
	TMap<FNiagaraSystemInstanceID, FNDISparseVolumeTextureInstanceData_RenderThread> InstanceData_RT;

	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { checkNoEntry(); }
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }
};

UNiagaraDataInterfaceSparseVolumeTexture::UNiagaraDataInterfaceSparseVolumeTexture(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, SparseVolumeTexture(nullptr)
	, BlockingStreamingRequests(true)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxySparseVolumeTexture());

	FNiagaraTypeDefinition Def(UObject::StaticClass());
	SparseVolumeTextureUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceSparseVolumeTexture::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceSparseVolumeTexture::PostLoad()
{
	Super::PostLoad();
}

void UNiagaraDataInterfaceSparseVolumeTexture::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSparseVolumeTexture::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature DefaultGpuSig;
	DefaultGpuSig.bMemberFunction = true;
	DefaultGpuSig.bRequiresContext = false;
	DefaultGpuSig.bSupportsCPU = false;
	DefaultGpuSig.bSupportsGPU = true;
	DefaultGpuSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("SparseVolumeTexture"));
	DefaultGpuSig.SetFunctionVersion(FNDISparseVolumeTextureFunctionVersion::LatestVersion);

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultGpuSig);
		Sig.Name = LoadSparseVolumeTextureName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelZ"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("AttributesA"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("AttributesB"));
		Sig.SetDescription(LOCTEXT("SparseVolumeTextureLoadDesc", "Read a voxel from the provided location & mip without any filtering or sampling."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultGpuSig);
		Sig.Name = SampleSparseVolumeTextureName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UVW"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("AttributesA"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("AttributesB"));
		Sig.SetDescription(LOCTEXT("SparseVolumeTextureSampleDesc", "Sample supplied mip level from input sparse volume texture at the specified UVW coordinates."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = GetTextureDimensionsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = true;
		Sig.SetFunctionVersion(FNDISparseVolumeTextureFunctionVersion::LatestVersion);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("SparseVolumeTexture"));		
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SizeX"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SizeY"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("SizeZ"));
		Sig.SetDescription(LOCTEXT("SparseVolumeTextureDimsDesc", "Get the dimensions of the provided mip level."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = GetNumMipLevelsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = true;
		Sig.SetFunctionVersion(FNDISparseVolumeTextureFunctionVersion::LatestVersion);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("SparseVolumeTexture"));		
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumMipLevels"));
		Sig.SetDescription(LOCTEXT("SparseVolumeGetNumMipLevelsDesc", "Get the number of mip levels."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = RequestSparseVolumeTextureFrameName;
		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;		
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		Sig.SetFunctionVersion(FNDISparseVolumeTextureFunctionVersion::LatestVersion);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("SparseVolumeTexture"));		
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("FrameRate"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Frame"));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));
		Sig.SetDescription(LOCTEXT("RequestSparseVolumeTextureFrameDesc", "Queue up the frame to load on tick"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = GetNumFramesName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = true;
		Sig.SetFunctionVersion(FNDISparseVolumeTextureFunctionVersion::LatestVersion);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("SparseVolumeTexture"));		
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumFrames"));		
		Sig.SetDescription(LOCTEXT("SparseVolumeNumFramesDesc", "Get the number of frames."));
	}
}
#endif

void UNiagaraDataInterfaceSparseVolumeTexture::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	if (BindingInfo.Name == GetTextureDimensionsName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSparseVolumeTexture::VMGetTextureDimensions);
	}
	else if (BindingInfo.Name == GetNumMipLevelsName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSparseVolumeTexture::VMGetNumMipLevels);
	}
	else if (BindingInfo.Name == RequestSparseVolumeTextureFrameName)
	{
		check(BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 1);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSparseVolumeTexture::VMRequestSparseVolumeTextureFrame);
	}	
	else if (BindingInfo.Name == GetNumFramesName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSparseVolumeTexture::VMGetNumFrames);
	}
}

int32 UNiagaraDataInterfaceSparseVolumeTexture::PerInstanceDataSize() const
{
	return sizeof(FNDISparseVolumeTextureInstanceData_GameThread);
}

bool UNiagaraDataInterfaceSparseVolumeTexture::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDISparseVolumeTextureInstanceData_GameThread* InstanceData = new (PerInstanceData) FNDISparseVolumeTextureInstanceData_GameThread();
	InstanceData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), SparseVolumeTextureUserParameter.Parameter);
	return true;
}

void UNiagaraDataInterfaceSparseVolumeTexture::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDISparseVolumeTextureInstanceData_GameThread* InstanceData = static_cast<FNDISparseVolumeTextureInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FNDISparseVolumeTextureInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(NDISparseVolumeTextureTexture_RemoveInstance)
	(
		[RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxySparseVolumeTexture>(), RT_InstanceID = SystemInstance->GetId()](FRHICommandListImmediate&)
		{
			RT_Proxy->InstanceData_RT.Remove(RT_InstanceID);
		}
	);
}

bool UNiagaraDataInterfaceSparseVolumeTexture::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FNDISparseVolumeTextureInstanceData_GameThread* InstanceData = static_cast<FNDISparseVolumeTextureInstanceData_GameThread*>(PerInstanceData);
	
	USparseVolumeTexture* CurrentTexture = InstanceData->UserParamBinding.GetValueOrDefault<USparseVolumeTexture>(SparseVolumeTexture);	

	if (CurrentTexture)
	{	
		InstanceData->NumFrames = CurrentTexture->GetNumFrames();		

		// skip frame request if we are passed in a frame (ie: from BP)
		USparseVolumeTextureFrame* SparseVolumeTextureFrame = Cast<USparseVolumeTextureFrame>(CurrentTexture);

		if (SparseVolumeTextureFrame == nullptr)
		{			
			// todo: we only support querying mip 0 for now
			SparseVolumeTextureFrame = USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest(CurrentTexture, GetTypeHash(InstanceData), InstanceData->FrameRate, InstanceData->CurrentFrame, 0, BlockingStreamingRequests, true);
			UE::SVT::GetStreamingManager().Update_GameThread();
		}

		if (SparseVolumeTextureFrame)
		{
			const UE::SVT::FTextureRenderResources* CurrentRenderResources = SparseVolumeTextureFrame->GetTextureRenderResources();
			FIntVector3 CurrentTextureSize = FIntVector3(SparseVolumeTextureFrame->GetVolumeResolution());
			int32 CurrentTextureMipLevels = SparseVolumeTextureFrame->GetNumMipLevels();

			// if the loaded SVT frame is different from what the instance data specifies
			if ((InstanceData->CurrentTexture != SparseVolumeTextureFrame)
				|| (InstanceData->CurrentRenderResources != CurrentRenderResources)
				|| (InstanceData->CurrentTextureSize != CurrentTextureSize) 
				|| (InstanceData->CurrentTextureMipLevels != CurrentTextureMipLevels))
			{
				InstanceData->CurrentTexture = SparseVolumeTextureFrame;
				InstanceData->CurrentRenderResources = CurrentRenderResources;
				InstanceData->CurrentTextureSize = CurrentTextureSize;
				InstanceData->CurrentTextureMipLevels = CurrentTextureMipLevels;

				ENQUEUE_RENDER_COMMAND(NDISparseVolumeTexture_UpdateInstance)
				(
					[RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxySparseVolumeTexture>(),
					RT_InstanceID = SystemInstance->GetId(),
					RT_RenderResources = CurrentRenderResources,
					RT_TextureSize = CurrentTextureSize, 
					RT_MipLevels = CurrentTextureMipLevels,
					RT_NumFrames = InstanceData->NumFrames]
				(FRHICommandListImmediate&)
					{
						FNDISparseVolumeTextureInstanceData_RenderThread& RTInstanceData = RT_Proxy->InstanceData_RT.FindOrAdd(RT_InstanceID);
						RTInstanceData.RenderResources = RT_RenderResources;
						RTInstanceData.TextureSize = RT_TextureSize;
						RTInstanceData.MipLevels = RT_MipLevels;
						RTInstanceData.NumFrames = RT_NumFrames;
					}
				);
			}
		}
	}
	return false;
}

bool UNiagaraDataInterfaceSparseVolumeTexture::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceSparseVolumeTexture* OtherSparseVolumeTexture = CastChecked<const UNiagaraDataInterfaceSparseVolumeTexture>(Other);
	return OtherSparseVolumeTexture->SparseVolumeTexture == SparseVolumeTexture && OtherSparseVolumeTexture->SparseVolumeTextureUserParameter == SparseVolumeTextureUserParameter && OtherSparseVolumeTexture->BlockingStreamingRequests == BlockingStreamingRequests;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceSparseVolumeTexture::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceSparseVolumeTexture::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceSparseVolumeTexture::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	static const TSet<FName> ValidGpuFunctions =
	{
		LoadSparseVolumeTextureName,
		SampleSparseVolumeTextureName,
		GetTextureDimensionsName,
		GetNumMipLevelsName,
		GetNumFramesName,
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

bool UNiagaraDataInterfaceSparseVolumeTexture::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	return false;
}
#endif

void UNiagaraDataInterfaceSparseVolumeTexture::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceSparseVolumeTexture::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxySparseVolumeTexture& TextureProxy = Context.GetProxy<FNiagaraDataInterfaceProxySparseVolumeTexture>();
	FNDISparseVolumeTextureInstanceData_RenderThread* RTInstanceData = TextureProxy.InstanceData_RT.Find(Context.GetSystemInstanceID());

	FShaderParameters* Parameters = Context.GetParameterNestedStruct<FShaderParameters>();
	Parameters->TileDataTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters->PageTableTexture = GBlackUintVolumeTexture->TextureRHI;
	Parameters->PhysicalTileDataATexture = GBlackVolumeTexture->TextureRHI;
	Parameters->PhysicalTileDataBTexture = GBlackVolumeTexture->TextureRHI;
	Parameters->PackedUniforms0 = FUintVector4();
	Parameters->PackedUniforms1 = FUintVector4();
	Parameters->TextureSize = FIntVector3::ZeroValue;
	Parameters->MipLevels = 0;
	Parameters->NumFrames = 0;
	
	if (RTInstanceData && RTInstanceData->RenderResources)
	{
		FRHITexture* PageTableTexture = RTInstanceData->RenderResources->GetPageTableTexture();
		FRHITexture* PhysicalTileDataATexture = RTInstanceData->RenderResources->GetPhysicalTileDataATexture();
		FRHITexture* PhysicalTileDataBTexture = RTInstanceData->RenderResources->GetPhysicalTileDataBTexture();

		Parameters->PageTableTexture = PageTableTexture ? PageTableTexture : Parameters->PageTableTexture;
		Parameters->PhysicalTileDataATexture = PhysicalTileDataATexture ? PhysicalTileDataATexture : Parameters->PhysicalTileDataATexture;
		Parameters->PhysicalTileDataBTexture = PhysicalTileDataBTexture ? PhysicalTileDataBTexture : Parameters->PhysicalTileDataBTexture;
		RTInstanceData->RenderResources->GetPackedUniforms(Parameters->PackedUniforms0, Parameters->PackedUniforms1);
		Parameters->TextureSize = RTInstanceData->TextureSize;
		Parameters->MipLevels = RTInstanceData->MipLevels;
		Parameters->NumFrames = RTInstanceData->NumFrames;
	}
}

void UNiagaraDataInterfaceSparseVolumeTexture::VMGetTextureDimensions(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISparseVolumeTextureInstanceData_GameThread> InstData(Context);
	FNDIInputParam<int32>		InMipLevel(Context);
	FNDIOutputParam<int32>		OutSizeX(Context);
	FNDIOutputParam<int32>		OutSizeY(Context);
	FNDIOutputParam<int32>		OutSizeZ(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 MipLevel = InMipLevel.GetAndAdvance();

		OutSizeX.SetAndAdvance(FMath::Max(InstData->CurrentTextureSize.X >> MipLevel, 1));
		OutSizeY.SetAndAdvance(FMath::Max(InstData->CurrentTextureSize.Y >> MipLevel, 1));
		OutSizeZ.SetAndAdvance(FMath::Max(InstData->CurrentTextureSize.Z >> MipLevel, 1));
	}
}

void UNiagaraDataInterfaceSparseVolumeTexture::VMGetNumMipLevels(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISparseVolumeTextureInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int32> OutNumMipLevels(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutNumMipLevels.SetAndAdvance(InstData->CurrentTextureMipLevels);
	}
}

void UNiagaraDataInterfaceSparseVolumeTexture::VMRequestSparseVolumeTextureFrame(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISparseVolumeTextureInstanceData_GameThread> InstData(Context);
	FNDIInputParam<float>	InFrameRate(Context);
	FNDIInputParam<float>	InFrame(Context);
	FNDIOutputParam<bool> OutSuccess(Context);

	InstData->FrameRate = InFrameRate.GetAndAdvance();
	InstData->CurrentFrame = InFrame.GetAndAdvance();
	
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutSuccess.SetAndAdvance(true);
	}
}

void UNiagaraDataInterfaceSparseVolumeTexture::VMGetNumFrames(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISparseVolumeTextureInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int32> OutNumFrames(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutNumFrames.SetAndAdvance(InstData->NumFrames);
	}
}

void UNiagaraDataInterfaceSparseVolumeTexture::SetTexture(USparseVolumeTexture* InSparseVolumeTexture)
{
	SparseVolumeTexture = InSparseVolumeTexture;
}

bool UNiagaraDataInterfaceSparseVolumeTexture::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceSparseVolumeTexture* DestinationTexture = CastChecked<UNiagaraDataInterfaceSparseVolumeTexture>(Destination);
	DestinationTexture->SparseVolumeTexture = SparseVolumeTexture;
	DestinationTexture->SparseVolumeTextureUserParameter = SparseVolumeTextureUserParameter;
	DestinationTexture->BlockingStreamingRequests = BlockingStreamingRequests;

	return true;
}

#undef LOCTEXT_NAMESPACE
