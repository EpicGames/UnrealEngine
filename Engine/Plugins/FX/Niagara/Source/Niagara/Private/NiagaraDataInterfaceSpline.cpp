// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSpline.h"
#include "Engine/Canvas.h"
#include "Internationalization/Internationalization.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "CanvasItem.h"
#include "ShaderCompilerCore.h"

#include "NiagaraComponent.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraRenderer.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraStats.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceSpline)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceSpline"

struct FNiagaraSplineDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		LWCConversion = 1,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

void FNDISpline_InstanceData_RenderThread::Reset()
{
	check(IsInRenderingThread());
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, SplinePositionsLUT.NumBytes);
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, SplineScalesLUT.NumBytes);
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, SplineRotationsLUT.NumBytes);
	SplinePositionsLUT.Release();
	SplineScalesLUT.Release();
	SplineRotationsLUT.Release();

	SplineTransform = FMatrix44f::Identity;
	SplineTransformRotationMat = FMatrix44f::Identity;
	SplineTransformInverse = FMatrix44f::Identity;
	SplineTransformInverseTranspose = FMatrix44f::Identity;
	SplineTransformRotation = FQuat4f::Identity;

	DefaultUpVector = FVector3f::ZAxisVector;

	SplineLength = 0.0f;
	SplineDistanceStep = 0.0f;
	InvSplineDistanceStep = 0.0f;
	MaxIndex = INDEX_NONE;
}

namespace NDISplineLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FMatrix44f,	SplineTransform)
		SHADER_PARAMETER(FMatrix44f,	SplineTransformRotationMat)
		SHADER_PARAMETER(FMatrix44f,	SplineTransformInverse)
		SHADER_PARAMETER(FMatrix44f,	SplineTransformInverseTranspose)
		SHADER_PARAMETER(FQuat4f,		SplineTransformRotation)
		SHADER_PARAMETER(FVector3f,		DefaultUpVector)
		SHADER_PARAMETER(float,			SplineLength)
		SHADER_PARAMETER(float,			SplineDistanceStep)
		SHADER_PARAMETER(float,			InvSplineDistanceStep)
		SHADER_PARAMETER(int,			MaxIndex)

		SHADER_PARAMETER_SRV(Buffer<float4>, SplinePositionsLUT)
		SHADER_PARAMETER_SRV(Buffer<float4>, SplineScalesLUT)
		SHADER_PARAMETER_SRV(Buffer<float4>, SplineRotationsLUT)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceSplineTemplate.ush");

	static const FName SampleSplinePositionByUnitDistanceName("SampleSplinePositionByUnitDistance");
	static const FName SampleSplinePositionByUnitDistanceWSName("SampleSplinePositionByUnitDistanceWS");

	static const FName SampleSplineRotationByUnitDistanceName("SampleSplineRotationByUnitDistance");
	static const FName SampleSplineRotationByUnitDistanceWSName("SampleSplineRotationByUnitDistanceWS");

	static const FName SampleSplineUpVectorByUnitDistanceName("SampleSplineUpVectorByUnitDistance");
	static const FName SampleSplineUpVectorByUnitDistanceWSName("SampleSplineUpVectorByUnitDistanceWS");

	static const FName SampleSplineDirectionByUnitDistanceName("SampleSplineDirectionByUnitDistance");
	static const FName SampleSplineDirectionByUnitDistanceWSName("SampleSplineDirectionByUnitDistanceWS");

	static const FName SampleSplineRightVectorByUnitDistanceName("SampleSplineRightVectorByUnitDistance");
	static const FName SampleSplineRightVectorByUnitDistanceWSName("SampleSplineRightVectorByUnitDistanceWS");

	static const FName SampleSplineTangentByUnitDistanceName("SampleSplineTangentByUnitDistance");
	static const FName SampleSplineTangentByUnitDistanceWSName("SampleSplineTangentByUnitDistanceWS");

	static const FName FindClosestUnitDistanceFromPositionWSName("FindClosestUnitDistanceFromPositionWS");

	static const FName GetSplineLengthName("GetSplineLength");

	/** Temporary solution for exposing the transform of a mesh. Ideally this would be done by allowing interfaces to add to the uniform set for a simulation. */
	static const FName GetSplineLocalToWorldName("GetSplineLocalToWorld");
	static const FName GetSplineLocalToWorldInverseTransposedName("GetSplineLocalToWorldInverseTransposed");


	static bool GbNiagaraDISplineDisableLUTs = false;
	static FAutoConsoleVariableRef CVarGbNiagaraDISplineDisableLUTs(
		TEXT("fx.Niagara.NDISpline.GDisableLUTs"),
		GbNiagaraDISplineDisableLUTs,
		TEXT("Should we turn off all LUTs on CPU?"),
		ECVF_Default
	);
}

UNiagaraDataInterfaceSpline::UNiagaraDataInterfaceSpline(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, SoftSourceActor(nullptr)
#if WITH_EDITORONLY_DATA
	, Source_DEPRECATED(nullptr)
#endif
, bUseLUT(false)
	, NumLUTSteps(256)
{
	FNiagaraTypeDefinition Def(UObject::StaticClass());
	SplineUserParameter.Parameter.SetType(Def);
	
	Proxy.Reset(new FNiagaraDataInterfaceProxySpline());
	MarkRenderDataDirty();
}

void UNiagaraDataInterfaceSpline::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we regitser data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceSpline::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (Source_DEPRECATED != nullptr)
	{
		SoftSourceActor = Source_DEPRECATED;
	}
#endif
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSpline::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature DefaultSig;
	DefaultSig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Spline")));
	DefaultSig.bMemberFunction = true;
	DefaultSig.bRequiresContext = false;
	DefaultSig.SetFunctionVersion(FNiagaraSplineDIFunctionVersion::LatestVersion);

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplinePositionByUnitDistanceName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplinePositionByUnitDistance", "Sample the spline Position where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the local space of the referenced USplineComponent."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplinePositionByUnitDistanceWSName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplinePositionByUnitDistanceWS", "Sample the spline Position where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the world space of the level."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplineRotationByUnitDistanceName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplineRotationByUnitDistance", "Sample the spline Rotation where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the local space of the referenced USplineComponent."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplineRotationByUnitDistanceWSName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplineRotationByUnitDistanceWS", "Sample the spline Rotation where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the world space of the level."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplineDirectionByUnitDistanceName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Direction")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplineDirectionByUnitDistance", "Sample the spline direction vector where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the local space of the referenced USplineComponent."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplineDirectionByUnitDistanceWSName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Direction")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplineDirectionByUnitDistanceWS", "Sample the spline direction vector where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the world space of the level."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplineUpVectorByUnitDistanceName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UpVector")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplineUpVectorByUnitDistance", "Sample the spline up vector where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the local space of the referenced USplineComponent."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplineUpVectorByUnitDistanceWSName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UpVector")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplineUpVectorByUnitDistanceWS", "Sample the spline up vector where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the world space of the level."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplineRightVectorByUnitDistanceName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("RightVector")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplineRightVectorByUnitDistance", "Sample the spline right vector where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the local space of the referenced USplineComponent."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplineRightVectorByUnitDistanceWSName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("RightVector")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplineRightVectorByUnitDistanceWS", "Sample the spline right vector where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the world space of the level."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplineTangentByUnitDistanceName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplineTangentVectorByUnitDistance", "Sample the spline tangent vector where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the local space of the referenced USplineComponent."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::SampleSplineTangentByUnitDistanceWSName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Tangent")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_SampleSplineTangentVectorByUnitDistanceWS", "Sample the spline tangent vector where U is a 0 to 1 value representing the start and normalized length of the spline.\nThis is in the world space of the level."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::GetSplineLocalToWorldName;
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Transform")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_GetSplineLocalToWorld", "Get the transform from the USplineComponent's local space to world space."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::GetSplineLocalToWorldInverseTransposedName;
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Transform")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_GetSplineLocalToWorldInverseTransposed", "Get the transform from the world space to the USplineComponent's local space."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::FindClosestUnitDistanceFromPositionWSName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("PositionWS")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("U")));
		Sig.SetDescription(LOCTEXT("DataInterfaceSpline_FindClosestUnitDistanceFromPositionWS", "Given a world space position, find the closest value 'U' on the USplineComponent to that point."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultSig);
		Sig.Name = NDISplineLocal::GetSplineLengthName;
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Length")));
	}
}
#endif

DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplinePositionByUnitDistance);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineRotationByUnitDistance);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineUpVectorByUnitDistance);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineRightVectorByUnitDistance);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineDirectionByUnitDistance);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineTangentByUnitDistance);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, FindClosestUnitDistanceFromPositionWS);


template<typename NextBinder>
struct TSplineUseLUTBinder
{
	template<typename... ParamTypes>
	static void Bind(UNiagaraDataInterface* Interface, const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
	{
		UNiagaraDataInterfaceSpline* SplineInterface = CastChecked<UNiagaraDataInterfaceSpline>(Interface);
		if (SplineInterface->bUseLUT && !NDISplineLocal::GbNiagaraDISplineDisableLUTs)
		{
			NextBinder::template Bind<ParamTypes..., TIntegralConstant<bool, true>>(Interface, BindingInfo, InstanceData, OutFunc);
		}
		else
		{
			NextBinder::template Bind<ParamTypes..., TIntegralConstant<bool, false>>(Interface, BindingInfo, InstanceData, OutFunc);
		}
	}
};

void UNiagaraDataInterfaceSpline::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == NDISplineLocal::SampleSplinePositionByUnitDistanceName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplinePositionByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::SampleSplinePositionByUnitDistanceWSName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandler, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplinePositionByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::SampleSplineRotationByUnitDistanceName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 4);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineRotationByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::SampleSplineRotationByUnitDistanceWSName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 4);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandler, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineRotationByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::SampleSplineUpVectorByUnitDistanceName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineUpVectorByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::SampleSplineUpVectorByUnitDistanceWSName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandler, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineUpVectorByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::SampleSplineDirectionByUnitDistanceName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineDirectionByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::SampleSplineDirectionByUnitDistanceWSName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandler, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineDirectionByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::SampleSplineRightVectorByUnitDistanceName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineRightVectorByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::SampleSplineRightVectorByUnitDistanceWSName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandler, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineRightVectorByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::SampleSplineTangentByUnitDistanceName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandlerNoop, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineTangentByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::SampleSplineTangentByUnitDistanceWSName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		TSplineUseLUTBinder<TNDIExplicitBinder<FNDITransformHandler, TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, SampleSplineTangentByUnitDistance)>>>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::FindClosestUnitDistanceFromPositionWSName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		TSplineUseLUTBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceSpline, FindClosestUnitDistanceFromPositionWS)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == NDISplineLocal::GetSplineLocalToWorldName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 16);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSpline::GetLocalToWorld);
	}
	else if (BindingInfo.Name == NDISplineLocal::GetSplineLocalToWorldInverseTransposedName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 16);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSpline::GetLocalToWorldInverseTransposed);
	}
	else if (BindingInfo.Name == NDISplineLocal::GetSplineLengthName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSpline::VMGetSplineLength);
	}
}

bool UNiagaraDataInterfaceSpline::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceSpline* OtherTyped = CastChecked<UNiagaraDataInterfaceSpline>(Destination);
	OtherTyped->SourceMode = SourceMode;
	OtherTyped->SoftSourceActor = SoftSourceActor;
	OtherTyped->SplineUserParameter = SplineUserParameter;
	OtherTyped->bUseLUT = bUseLUT;
	OtherTyped->NumLUTSteps = NumLUTSteps;
	
	OtherTyped->MarkRenderDataDirty();
	return true;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceSpline::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	// LWC upgrades
	if (FunctionSignature.FunctionVersion < FNiagaraSplineDIFunctionVersion::LWCConversion)
	{
		TArray<FNiagaraFunctionSignature> AllFunctions;
		GetFunctionsInternal(AllFunctions);
		for (const FNiagaraFunctionSignature& Sig : AllFunctions)
		{
			if (FunctionSignature.Name == Sig.Name)
			{
				FunctionSignature = Sig;
				return true;
			}
		}
	}
	return false;
}

void UNiagaraDataInterfaceSpline::GetCommonHLSL(FString& OutHLSL)
{
	OutHLSL += TEXT("#include \"/Plugin/FX/Niagara/Private/NiagaraCommon.ush\"\n");
}

void UNiagaraDataInterfaceSpline::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"), ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDISplineLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceSpline::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDISplineLocal;	
	static const TSet<FName> ValidGpuFunctions =
	{
		SampleSplinePositionByUnitDistanceName,
		SampleSplinePositionByUnitDistanceWSName,
		SampleSplineRotationByUnitDistanceName,
		SampleSplineRotationByUnitDistanceWSName,
		SampleSplineDirectionByUnitDistanceName,
		SampleSplineDirectionByUnitDistanceWSName,
		SampleSplineUpVectorByUnitDistanceName,
		SampleSplineUpVectorByUnitDistanceWSName,
		SampleSplineRightVectorByUnitDistanceName,
		SampleSplineRightVectorByUnitDistanceWSName,
		SampleSplineTangentByUnitDistanceName,
		SampleSplineTangentByUnitDistanceWSName,
		GetSplineLocalToWorldName,
		GetSplineLocalToWorldInverseTransposedName,
		FindClosestUnitDistanceFromPositionWSName,
		GetSplineLengthName
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

bool UNiagaraDataInterfaceSpline::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(NDISplineLocal::TemplateShaderFile);
	InVisitor->UpdateShaderParameters<NDISplineLocal::FShaderParameters>();
	return bSuccess;
}
#endif

void UNiagaraDataInterfaceSpline::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDISplineLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceSpline::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxySpline& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxySpline>();
	NDISplineLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDISplineLocal::FShaderParameters>();
	if (FNDISpline_InstanceData_RenderThread* InstanceData_RT = DIProxy.SystemInstancesToProxyData_RT.Find(Context.GetSystemInstanceID()))
	{
		ShaderParameters->SplineTransform					= InstanceData_RT->SplineTransform;
		ShaderParameters->SplineTransformRotationMat		= InstanceData_RT->SplineTransformRotationMat;
		ShaderParameters->SplineTransformInverse			= InstanceData_RT->SplineTransformInverse;
		ShaderParameters->SplineTransformInverseTranspose	= InstanceData_RT->SplineTransformInverseTranspose;
		ShaderParameters->SplineTransformRotation			= InstanceData_RT->SplineTransformRotation;
		ShaderParameters->DefaultUpVector					= InstanceData_RT->DefaultUpVector;
		ShaderParameters->SplineLength						= InstanceData_RT->SplineLength;
		ShaderParameters->SplineDistanceStep				= InstanceData_RT->SplineDistanceStep;
		ShaderParameters->InvSplineDistanceStep				= InstanceData_RT->InvSplineDistanceStep;
		ShaderParameters->MaxIndex							= InstanceData_RT->MaxIndex;
		ShaderParameters->SplinePositionsLUT				= InstanceData_RT->SplinePositionsLUT.SRV;
		ShaderParameters->SplineScalesLUT					= InstanceData_RT->SplineScalesLUT.SRV;
		ShaderParameters->SplineRotationsLUT				= InstanceData_RT->SplineRotationsLUT.SRV;
	}
	else
	{
		ShaderParameters->SplineTransform					= FMatrix44f::Identity;
		ShaderParameters->SplineTransformRotationMat		= FMatrix44f::Identity;
		ShaderParameters->SplineTransformInverse			= FMatrix44f::Identity;
		ShaderParameters->SplineTransformInverseTranspose	= FMatrix44f::Identity;
		ShaderParameters->SplineTransformRotation			= FQuat4f::Identity;
		ShaderParameters->DefaultUpVector					= FVector3f::UnitZ();
		ShaderParameters->SplineLength						= 0.0f;
		ShaderParameters->SplineDistanceStep				= 0.0f;
		ShaderParameters->InvSplineDistanceStep				= 0.0f;
		ShaderParameters->MaxIndex							= 0;
		ShaderParameters->SplinePositionsLUT				= FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->SplineScalesLUT					= FNiagaraRenderer::GetDummyFloat4Buffer();
		ShaderParameters->SplineRotationsLUT				= FNiagaraRenderer::GetDummyFloat4Buffer();
	}
}

bool UNiagaraDataInterfaceSpline::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceSpline* OtherTyped = CastChecked<const UNiagaraDataInterfaceSpline>(Other);
	return OtherTyped->SourceMode == SourceMode
		&& OtherTyped->SoftSourceActor == SoftSourceActor
		&& OtherTyped->SplineUserParameter == SplineUserParameter
		&& OtherTyped->bUseLUT == bUseLUT
		&& OtherTyped->NumLUTSteps == NumLUTSteps;
}

#if WITH_NIAGARA_DEBUGGER
void UNiagaraDataInterfaceSpline::DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const
{
	const FNDISpline_InstanceData* InstanceData_GT = DebugHudContext.GetSystemInstance()->FindTypedDataInterfaceInstanceData<FNDISpline_InstanceData>(this);
	if (InstanceData_GT == nullptr)
	{
		return;
	}

	USplineComponent* SplineComponent = InstanceData_GT->Component.Get();
	AActor* SceneActor = SplineComponent ? SplineComponent->GetAttachmentRootActor() : nullptr;
	DebugHudContext.GetOutputString().Appendf(TEXT("SplineComponent(%s:%s)"), SceneActor ? *SceneActor->GetActorNameOrLabel() : TEXT("None"), *GetNameSafe(SplineComponent));

	if (!DebugHudContext.IsVerbose() || SplineComponent == nullptr)
	{
		return;
	}

	UCanvas* Canvas = DebugHudContext.GetCanvas();

	auto DrawDebugSpline =
		[Canvas](int32 NumPoints, TFunction<FVector(int)> GetPoint, const FColor& Color)
		{
			TOptional<FVector> LastScreenPosition;
			for (int32 i = 0; i < NumPoints; ++i)
			{
				const FVector WorldPosition = GetPoint(i);
				const FVector ScreenPosition = Canvas->Project(WorldPosition, false);
				if (ScreenPosition.Z > 0.0f)
				{
					if (LastScreenPosition.IsSet())
					{
						FCanvasLineItem LineItem(FVector2D(ScreenPosition), FVector2D(LastScreenPosition.GetValue()));
						LineItem.SetColor(Color);
						Canvas->Canvas->DrawItem(LineItem);
					}

					Canvas->Canvas->DrawNGon(FVector2D(ScreenPosition), Color, 8, 4.0f);
					LastScreenPosition.Emplace(ScreenPosition);
				}
				else
				{
					LastScreenPosition.Reset();
				}
			}
		};

	// Draw points in spline
	DrawDebugSpline(
		SplineComponent->GetNumberOfSplinePoints(),
		[SplineComponent](int32 PointIndex)
		{
			const FSplinePoint SplinePoint = SplineComponent->GetSplinePointAt(PointIndex, ESplineCoordinateSpace::World);
			return SplinePoint.Position;
		},
		FColor::Red
	);

	// Draw LUT points
	if (bUseLUT)
	{
		DrawDebugSpline(
			InstanceData_GT->SplineLUT.Positions.Num(),
			[InstanceData_GT, Transform=SplineComponent->GetComponentTransform()](int32 PointIndex)
			{
				const FVector Position = InstanceData_GT->SplineLUT.Positions[PointIndex];
				return Transform.TransformPosition(Position);
			},
			FColor::Green
		);
	}
}
#endif

int32 UNiagaraDataInterfaceSpline::PerInstanceDataSize()const
{
	return sizeof(FNDISpline_InstanceData);
}

bool UNiagaraDataInterfaceSpline::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDISpline_InstanceData* InstData = new (PerInstanceData) FNDISpline_InstanceData();
	SystemInstancesToProxyData_GT.Emplace(SystemInstance->GetId(), InstData);

	InstData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), SplineUserParameter.Parameter);

	return true;
}

void UNiagaraDataInterfaceSpline::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	SystemInstancesToProxyData_GT.Remove(SystemInstance->GetId());
	
	FNDISpline_InstanceData* InstData = static_cast<FNDISpline_InstanceData*>(PerInstanceData);
	InstData->~FNDISpline_InstanceData();

	FNiagaraDataInterfaceProxySpline* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxySpline>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[RT_Proxy, InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
#if STATS
			if (FNDISpline_InstanceData_RenderThread* TargetData = RT_Proxy->SystemInstancesToProxyData_RT.Find(InstanceID))
			{
				TargetData->Reset();
			}
#endif
			RT_Proxy->SystemInstancesToProxyData_RT.Remove(InstanceID);
		}
	);
}

USplineComponent* UNiagaraDataInterfaceSpline::ResolveSplineComponent(FNiagaraSystemInstance* SystemInstance, FNDISpline_InstanceData* InstanceData) const
{
	const bool bTryParameterBinding = SourceMode == ENDISpline_SourceMode::Default || SourceMode == ENDISpline_SourceMode::ParameterBindingOnly;
	const bool bTrySourceActor = SourceMode == ENDISpline_SourceMode::Default;
	const bool bTryAttachParent = SourceMode == ENDISpline_SourceMode::Default || SourceMode == ENDISpline_SourceMode::AttachParentOnly;

	// Parameter binding
	UObject* ParameterBindingValue = InstanceData->UserParamBinding.GetValue();
	if (bTryParameterBinding && ParameterBindingValue)
	{
		if (USplineComponent* BoundComponent = Cast<USplineComponent>(ParameterBindingValue))
		{
			if (IsValid(BoundComponent))
			{
				return BoundComponent;
			}
		}
		else if (AActor* Actor = Cast<AActor>(ParameterBindingValue))
		{
			if (USplineComponent* ActorComponent = Actor->FindComponentByClass<USplineComponent>())
			{
				return ActorComponent;
			}
		}
		else
		{
			UE_LOGF(LogNiagara, Warning, "Spline data interface has an invalid object parameter type. Only USplineComponent or AActor are supported.");
			UE_LOGF(LogNiagara, Warning, "Invalid Parameter : %ls", *ParameterBindingValue->GetFullName());
			UE_LOGF(LogNiagara, Warning, "Niagara Component : %ls", *GetFullNameSafe(SystemInstance ? Cast<UNiagaraComponent>(SystemInstance->GetAttachComponent()) : nullptr));
			UE_LOGF(LogNiagara, Warning, "System : %ls", *GetFullNameSafe(SystemInstance ? SystemInstance->GetSystem() : nullptr));
		}
	}

	// Source actor
	if (bTrySourceActor)
	{
		if (AActor* SourceActor = SoftSourceActor.Get())
		{
			return SourceActor->FindComponentByClass<USplineComponent>();
		}
	}

	// Attach parent
	if (bTryAttachParent && SystemInstance)
	{
		if (USceneComponent* AttachComp = SystemInstance->GetAttachComponent())
		{
			if (AActor* Owner = AttachComp->GetAttachmentRootActor())
			{
				return Owner->FindComponentByClass<USplineComponent>();
			}
		}
	}

	return nullptr;
}

bool UNiagaraDataInterfaceSpline::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	check(SystemInstance && PerInstanceData);
	FNDISpline_InstanceData* InstData = static_cast<FNDISpline_InstanceData*>(PerInstanceData);

	// Reolve the spline parameter
	USplineComponent* SplineComponent = nullptr;
	if (SourceMode == ENDISpline_SourceMode::Default)
	{
		if (SplineUserParameter.Parameter.IsValid())
		{
			if (InstData->CachedUserParam && InstData->UserParamBinding.GetValue() != InstData->CachedUserParam)
			{
				return true;
			}
		}

		SplineComponent = InstData->Component.Get();
		if (SplineComponent == nullptr)
		{
			SplineComponent = ResolveSplineComponent(SystemInstance, InstData);
			InstData->Component = SplineComponent;
		}
	}
	else
	{
		SplineComponent = ResolveSplineComponent(SystemInstance, InstData);
	}

	// Handle the parameter changing
	if (SplineComponent != InstData->Component.Get())
	{
		InstData->Component = SplineComponent;
		InstData->SplineCurvesVersion.Reset();
	}

	// Re-evaluate source in case it's changed?
	if (SplineComponent != nullptr)
	{
		FTransform SplineTransform = SplineComponent->GetComponentToWorld();
		SplineTransform.AddToTranslation(FVector(SystemInstance->GetLWCTile()) * -FLargeWorldRenderScalar::GetTileSize());
		InstData->TransformQuat = SplineTransform.GetRotation();
		InstData->Transform = SplineTransform.ToMatrixWithScale();
		InstData->TransformInverseTransposed = InstData->Transform.InverseFast().GetTransposed();
		InstData->ComponentTransform = SplineComponent->GetComponentTransform();
		InstData->DefaultUpVector = SplineComponent->DefaultUpVector;
		InstData->LwcConverter = SystemInstance->GetLWCConverter();

		bool bShouldBuildLUT = (bUseLUT || IsUsedWithGPUScript()) && InstData->SplineLUT.MaxIndex < 0;
		
		if (!InstData->SplineCurvesVersion.IsSet() || (InstData->SplineCurvesVersion != SplineComponent->GetVersion()))
		{
			InstData->SplineCurves = SplineComponent->GetSplineCurves();
			InstData->SplineCurvesVersion = InstData->SplineCurves.Version;
			InstData->bSyncedGPUCopy = false;
			InstData->SplineLUT.Reset();

			bShouldBuildLUT = bUseLUT || IsUsedWithGPUScript();
		}
		
		bool bShouldSyncToGPU = IsUsedWithGPUScript() && !InstData->bSyncedGPUCopy && InstData->SplineLUT.MaxIndex != INDEX_NONE;
		
		// We must build the LUT if this is for GPU regardless of settings
		if (bShouldBuildLUT)
		{
			InstData->SplineLUT.BuildLUT(InstData->SplineCurves, bUseLUT? NumLUTSteps : 256/*Default the LUT to a reasonable value if it's not specifically enabled*/);

			bShouldSyncToGPU = IsUsedWithGPUScript();
		}


		if (bShouldSyncToGPU)
		{
			FNiagaraDataInterfaceProxySpline* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxySpline>();	
			InstData->bSyncedGPUCopy = true;
			

			// Push Updates to Proxy.
			ENQUEUE_RENDER_COMMAND(FUpdateDIColorCurve)(
				[RT_Proxy, InstanceId=SystemInstance->GetId(), Transform=InstData->Transform, TransformRot=InstData->TransformQuat, DefaultUp=InstData->DefaultUpVector, rtShaderLUT=InstData->SplineLUT](FRHICommandListImmediate& RHICmdList)
			{

				FNDISpline_InstanceData_RenderThread* TargetData = &RT_Proxy->SystemInstancesToProxyData_RT.FindOrAdd(InstanceId);
					
				TargetData->SplineTransform = FMatrix44f(Transform);			// LWC_TODO: Precision loss
				TargetData->SplineTransformRotationMat =  FMatrix44f(Transform.RemoveTranslation());
				TargetData->SplineTransformRotationMat.RemoveScaling();
				TargetData->SplineTransformInverse = TargetData->SplineTransform.Inverse();
				TargetData->SplineTransformInverseTranspose = TargetData->SplineTransformInverse.GetTransposed();
				TargetData->SplineTransformRotation = FQuat4f(TransformRot);
					
				TargetData->DefaultUpVector = (FVector3f)DefaultUp;
					
				TargetData->SplineLength = rtShaderLUT.SplineLength;
				TargetData->SplineDistanceStep = rtShaderLUT.SplineDistanceStep;
				TargetData->InvSplineDistanceStep = rtShaderLUT.InvSplineDistanceStep;
				TargetData->MaxIndex = rtShaderLUT.MaxIndex;
		
				DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TargetData->SplinePositionsLUT.NumBytes);
				DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TargetData->SplineScalesLUT.NumBytes);
				DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TargetData->SplineRotationsLUT.NumBytes);
				TargetData->SplinePositionsLUT.Release();
				TargetData->SplineScalesLUT.Release();
				TargetData->SplineRotationsLUT.Release();
			
				check(rtShaderLUT.Positions.Num());
		
				uint32 BufferSize;
		
				// Bind positions
				TargetData->SplinePositionsLUT.Initialize(RHICmdList, TEXT("SplinePositionsLUT"), sizeof(FVector4f), rtShaderLUT.Positions.Num(), EPixelFormat::PF_A32B32G32R32F, BUF_Static);
				BufferSize = rtShaderLUT.Positions.Num() * sizeof(FVector4f);
				FVector4f* PositionBufferData = static_cast<FVector4f*>(RHICmdList.LockBuffer(TargetData->SplinePositionsLUT.Buffer, 0, BufferSize, EResourceLockMode::RLM_WriteOnly));
				for (int32 Index = 0; Index < rtShaderLUT.Positions.Num(); Index++)
				{
					PositionBufferData[Index].X = float(rtShaderLUT.Positions[Index].X);	// LWC Precision Loss
					PositionBufferData[Index].Y = float(rtShaderLUT.Positions[Index].Y);
					PositionBufferData[Index].Z = float(rtShaderLUT.Positions[Index].Z);
					PositionBufferData[Index].W = 1.0f;
				}
				RHICmdList.UnlockBuffer(TargetData->SplinePositionsLUT.Buffer);
		
				// Bind scales
				TargetData->SplineScalesLUT.Initialize(RHICmdList, TEXT("SplineScalesLUT"), sizeof(FVector4f), rtShaderLUT.Scales.Num(), EPixelFormat::PF_A32B32G32R32F, BUF_Static);
				BufferSize = rtShaderLUT.Scales.Num() * sizeof(FVector4f);
				FVector4f* ScaleBufferData = static_cast<FVector4f*>(RHICmdList.LockBuffer(TargetData->SplineScalesLUT.Buffer, 0, BufferSize, EResourceLockMode::RLM_WriteOnly));
				for (int32 Index = 0; Index < rtShaderLUT.Scales.Num(); Index++)
				{
					ScaleBufferData[Index].X = float(rtShaderLUT.Scales[Index].X);
					ScaleBufferData[Index].Y = float(rtShaderLUT.Scales[Index].Y);
					ScaleBufferData[Index].Z = float(rtShaderLUT.Scales[Index].Z);
					ScaleBufferData[Index].W = 1.0f;
				}
				RHICmdList.UnlockBuffer(TargetData->SplineScalesLUT.Buffer);
				
				// Bind rotations
				TargetData->SplineRotationsLUT.Initialize(RHICmdList, TEXT("SplineRotationsLUT"), sizeof(FQuat4f), rtShaderLUT.Rotations.Num(), EPixelFormat::PF_A32B32G32R32F, BUF_Static);
				BufferSize = rtShaderLUT.Rotations.Num() * sizeof(FQuat4f);
				FQuat4f* RotationBufferData = static_cast<FQuat4f*>(RHICmdList.LockBuffer(TargetData->SplineRotationsLUT.Buffer, 0, BufferSize, EResourceLockMode::RLM_WriteOnly));
				for (int32 Index=0; Index < rtShaderLUT.Rotations.Num(); Index++)
				{
					RotationBufferData[Index] = FQuat4f(rtShaderLUT.Rotations[Index]);
				}
				RHICmdList.UnlockBuffer(TargetData->SplineRotationsLUT.Buffer);
				
				INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TargetData->SplinePositionsLUT.NumBytes);
				INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TargetData->SplineScalesLUT.NumBytes);
				INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, TargetData->SplineRotationsLUT.NumBytes);
			});
		}	
	}

	//Any situations requiring a rebind?
	return false;
}

void FNiagaraDataInterfaceSplineLUT::BuildLUT(const FSplineCurves& SplineCurves, int32 NumSteps)
{
	Positions.Empty(NumSteps);
	Scales.Empty(NumSteps);
	Rotations.Empty(NumSteps);
	MaxIndex = NumSteps - 1;

	SplineLength = SplineCurves.GetSplineLength();
	SplineDistanceStep = MaxIndex ? ((1.0f / MaxIndex) * SplineLength) : 0.0f;
	InvSplineDistanceStep = SplineDistanceStep > 0.0f ? 1.0f / SplineDistanceStep : 0.0f;
	
	for (int32 Index = 0; Index < NumSteps; Index++)
	{
		float Key = SplineCurves.ReparamTable.Eval(Index * SplineDistanceStep);
		Positions.Add(SplineCurves.Position.Eval(Key, FVector::ZeroVector));
		Scales.Add(SplineCurves.Scale.Eval(Key, FVector::ZeroVector));
		Rotations.Add(SplineCurves.Rotation.Eval(Key, FQuat::Identity).GetNormalized());
	}

}

void FNiagaraDataInterfaceSplineLUT::Reset()
{
	Positions.Empty();
	Scales.Empty();
	Rotations.Empty();
	SplineLength = 0;
	SplineDistanceStep = 0;
	InvSplineDistanceStep = 0;
	MaxIndex = INDEX_NONE;
}

void FNiagaraDataInterfaceSplineLUT::FindNeighborKeys(float InDistance, int32& PrevKey, int32& NextKey, float& Alpha) const
{
	const float Key = InDistance * InvSplineDistanceStep;
	
	PrevKey = FMath::Clamp(FMath::FloorToInt(Key), 0, MaxIndex);
	NextKey = FMath::Clamp(FMath::CeilToInt(Key), 0, MaxIndex);

	Alpha = FMath::Frac(Key);
}

template<>
float FNDISpline_InstanceData::GetSplineLength<TIntegralConstant<bool, false>>() const
{
	return SplineCurves.GetSplineLength();
}

template<>
float FNDISpline_InstanceData::GetSplineLength<TIntegralConstant<bool, true>>() const
{
	return SplineLUT.SplineLength;
}

bool FNDISpline_InstanceData::IsValid() const
{
	return Component.IsValid();
}

template<typename UseLUT>
FVector FNDISpline_InstanceData::GetLocationAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Key = ConvertDistanceToKey<UseLUT>(Distance);	
	FVector Location = EvaluatePosition<UseLUT>(Key);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Location = ComponentTransform.TransformPosition(Location);
	}

	return Location;
}

template<typename UseLUT>
FQuat FNDISpline_InstanceData::GetQuaternionAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Key = ConvertDistanceToKey<UseLUT>(Distance);	
	const FQuat Quat = EvaluateRotation<UseLUT>(Key);

	const FVector Direction = EvaluatePosition<UseLUT>(Key).GetSafeNormal();
	const FVector UpVector = Quat.RotateVector(DefaultUpVector);

	FQuat Rot = (FRotationMatrix::MakeFromXZ(Direction, UpVector)).ToQuat();

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Rot = ComponentTransform.GetRotation() * Rot;
	}

	return Rot;
}


template<typename UseLUT>
FVector FNDISpline_InstanceData::GetUpVectorAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const FQuat Quat = GetQuaternionAtDistanceAlongSpline<UseLUT>(Distance, ESplineCoordinateSpace::Local);
	FVector UpVector = Quat.RotateVector(FVector::UpVector);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		UpVector = ComponentTransform.TransformVectorNoScale(UpVector);
	}

	return UpVector;
}

template<typename UseLUT>
FVector FNDISpline_InstanceData::GetRightVectorAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const FQuat Quat = GetQuaternionAtDistanceAlongSpline<UseLUT>(Distance, ESplineCoordinateSpace::Local);
	FVector RightVector = Quat.RotateVector(FVector::RightVector);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		RightVector = ComponentTransform.TransformVectorNoScale(RightVector);
	}

	return RightVector;
}

template <>
float FNDISpline_InstanceData::ConvertDistanceToKey<TIntegralConstant<bool, false>>(float InDistance) const
{
	return SplineCurves.ReparamTable.Eval(InDistance, 0.0f);
}

template <>
float FNDISpline_InstanceData::ConvertDistanceToKey<TIntegralConstant<bool, true>>(float InDistance) const
{
	return InDistance;
}




template <>
FVector FNDISpline_InstanceData::EvaluatePosition<TIntegralConstant<bool, false>>(float InKey) const
{
	return SplineCurves.Position.Eval(InKey, FVector::ZeroVector);
}

template <>
FVector FNDISpline_InstanceData::EvaluatePosition<TIntegralConstant<bool, true>>(float InKey) const
{
	int32 PrevKey, NextKey;
	float Alpha;
	SplineLUT.FindNeighborKeys(InKey, PrevKey, NextKey, Alpha);

	if (NextKey == PrevKey)
	{
		if (PrevKey >= 0)
		{
			return SplineLUT.Positions[PrevKey];
		}
		else
		{
			return FVector::ZeroVector;
		}
	}

	return FMath::Lerp(SplineLUT.Positions[PrevKey], SplineLUT.Positions[NextKey], Alpha);
}

template <>
FVector FNDISpline_InstanceData::EvaluateScale<TIntegralConstant<bool, false>>(float InKey) const
{
	return SplineCurves.Scale.Eval(InKey, FVector::ZeroVector);
}

template <>
FVector FNDISpline_InstanceData::EvaluateScale<TIntegralConstant<bool, true>>(float InKey) const
{
	int32 PrevKey, NextKey;
	float Alpha;
	SplineLUT.FindNeighborKeys(InKey, PrevKey, NextKey, Alpha);

	if (NextKey == PrevKey)
	{
		if (PrevKey >= 0)
		{
			return SplineLUT.Scales[PrevKey];
		}
		else
		{
			return FVector::OneVector;
		}
	}

	return FMath::Lerp(SplineLUT.Scales[PrevKey], SplineLUT.Scales[NextKey], Alpha);
}

template <>
FQuat FNDISpline_InstanceData::EvaluateRotation<TIntegralConstant<bool, false>>(float InKey) const
{
	return SplineCurves.Rotation.Eval(InKey, FQuat::Identity).GetNormalized();
}

template <>
FQuat FNDISpline_InstanceData::EvaluateRotation<TIntegralConstant<bool, true>>(float InKey) const
{
	int32 PrevKey, NextKey;
	float Alpha;
	SplineLUT.FindNeighborKeys(InKey, PrevKey, NextKey, Alpha);

	if (NextKey == PrevKey)
	{
		if (PrevKey >= 0)
		{
			return SplineLUT.Rotations[PrevKey];
		}
		else
		{
			return FQuat::Identity;
		}
	}

	return FQuat::Slerp(SplineLUT.Rotations[PrevKey], SplineLUT.Rotations[NextKey], Alpha);
}

template <>
FVector FNDISpline_InstanceData::EvaluateDerivativePosition<TIntegralConstant<bool, false>>(float InKey) const
{
	return SplineCurves.Position.EvalDerivative(InKey, FVector::ZeroVector);
}

template <>
FVector FNDISpline_InstanceData::EvaluateDerivativePosition<TIntegralConstant<bool, true>>(float InKey) const
{
	int32 PrevKey, NextKey;
	float Alpha;
	SplineLUT.FindNeighborKeys(InKey, PrevKey, NextKey, Alpha);

	if (NextKey == PrevKey)
	{
		if (NextKey < SplineLUT.MaxIndex)
		{
			NextKey++;
		}
		else if (PrevKey > 0)
		{
			PrevKey--;
		}
		else
		{
			// We only have one point, so can't find a direction
			return FVector::ZeroVector;
		}
	}

	return SplineLUT.Positions[NextKey] - SplineLUT.Positions[PrevKey];
}

template <>
float FNDISpline_InstanceData::EvaluateFindNearestPosition<TIntegralConstant<bool, false>>(FVector InPosition) const
{
	float Dummy;
	return SplineCurves.Position.FindNearest(InPosition, Dummy);
}

template <>
float FNDISpline_InstanceData::EvaluateFindNearestPosition<TIntegralConstant<bool, true>>(FVector InPosition) const
{
	// This is a brute force search, definitely not a great idea with large tables, but also not too many ways around it without more data.
	float MinDistance = TNumericLimits<float>::Max();
	int32 KeyToNearest = 0;
	for (int32 i=0; i < SplineLUT.Positions.Num(); i++)
	{
		const float Distance = float(FVector::DistSquared(InPosition, SplineLUT.Positions[i]));
		if (Distance < MinDistance)
		{
			MinDistance = Distance;
			KeyToNearest = i;
		}
	}
	const int32 NumPointsMinusOne = SplineLUT.Positions.Num() - 1;
	return KeyToNearest > 0 && NumPointsMinusOne > 0 ? float(KeyToNearest) / float(NumPointsMinusOne) : 0.0f;
}

template<typename UseLUT>
FVector FNDISpline_InstanceData::GetTangentAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Key = ConvertDistanceToKey<UseLUT>(Distance);;	
	FVector Tangent = EvaluateDerivativePosition<UseLUT>(Key);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Tangent = ComponentTransform.TransformVector(Tangent);
	}

	return Tangent;
}


template<typename UseLUT>
FVector FNDISpline_InstanceData::GetDirectionAtDistanceAlongSpline(float Distance, ESplineCoordinateSpace::Type CoordinateSpace) const
{
	const float Key = ConvertDistanceToKey<UseLUT>(Distance);	
	FVector Direction = EvaluateDerivativePosition<UseLUT>(Key).GetSafeNormal();

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Direction = ComponentTransform.TransformVector(Direction);
		Direction.Normalize();
	}

	return Direction;
}

template <>
float FNDISpline_InstanceData::GetFinalKeyTime<TIntegralConstant<bool, false>>() const
{
	const int32 NumPoints = SplineCurves.Position.Points.Num();
	return NumPoints > 0 ? SplineCurves.Position.Points[NumPoints - 1].InVal : 1.0f;
}

template <>
float FNDISpline_InstanceData::GetFinalKeyTime<TIntegralConstant<bool, true>>() const
{
	return 1.0f;
}

template<typename UseLUT>
float FNDISpline_InstanceData::FindInputKeyClosestToWorldLocation(const FVector& WorldLocation) const
{
	const FVector LocalLocation = ComponentTransform.InverseTransformPosition(WorldLocation);
	return EvaluateFindNearestPosition<UseLUT>(LocalLocation);
}

template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
void UNiagaraDataInterfaceSpline::SampleSplinePositionByUnitDistance(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISpline_InstanceData> InstData(Context);
	TransformHandlerType TransformHandler;
	SplineSampleType SplineSampleParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosZ(Context);

	if (InstData->IsValid())
	{
		const float SplineLength = InstData->GetSplineLength<UseLUT>();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const float DistanceUnitDistance = SplineSampleParam.Get();

			FVector Pos = InstData->GetLocationAtDistanceAlongSpline<UseLUT>(DistanceUnitDistance * SplineLength, ESplineCoordinateSpace::Local);
			TransformHandler.TransformPosition(Pos, InstData->Transform);

			*OutPosX.GetDest() = float(Pos.X);	// LWC Precision Loss
			*OutPosY.GetDest() = float(Pos.Y);
			*OutPosZ.GetDest() = float(Pos.Z);
			SplineSampleParam.Advance();
			OutPosX.Advance();
			OutPosY.Advance();
			OutPosZ.Advance();
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			FVector Pos = FVector(EForceInit::ForceInitToZero);
			TransformHandler.TransformPosition(Pos, InstData->Transform);

			*OutPosX.GetDest() = float(Pos.X);	// LWC Precision Loss
			*OutPosY.GetDest() = float(Pos.Y);
			*OutPosZ.GetDest() = float(Pos.Z);
			SplineSampleParam.Advance();
			OutPosX.Advance();
			OutPosY.Advance();
			OutPosZ.Advance();
		}
	}
}

template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
void UNiagaraDataInterfaceSpline::SampleSplineRotationByUnitDistance(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISpline_InstanceData> InstData(Context);
	TransformHandlerType TransformHandler;
	SplineSampleType SplineSampleParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutQuatX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutQuatY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutQuatZ(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutQuatW(Context);

	if (InstData->IsValid())
	{
		const FQuat TransformQuat = InstData->TransformQuat;
		const float SplineLength = InstData->GetSplineLength<UseLUT>();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const float DistanceUnitDistance = SplineSampleParam.GetAndAdvance();

			FQuat Quat = InstData->GetQuaternionAtDistanceAlongSpline<UseLUT>(DistanceUnitDistance * SplineLength, ESplineCoordinateSpace::Local);
			TransformHandler.TransformRotation(Quat, TransformQuat);

			*OutQuatX.GetDestAndAdvance() = float(Quat.X);	// LWC Precison Loss
			*OutQuatY.GetDestAndAdvance() = float(Quat.Y);
			*OutQuatZ.GetDestAndAdvance() = float(Quat.Z);
			*OutQuatW.GetDestAndAdvance() = float(Quat.W);
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			*OutQuatX.GetDestAndAdvance() = FQuat4f::Identity.X;
			*OutQuatY.GetDestAndAdvance() = FQuat4f::Identity.Y;
			*OutQuatZ.GetDestAndAdvance() = FQuat4f::Identity.Z;
			*OutQuatW.GetDestAndAdvance() = FQuat4f::Identity.W;
		}
	}
}

template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
void UNiagaraDataInterfaceSpline::SampleSplineUpVectorByUnitDistance(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISpline_InstanceData> InstData(Context);
	TransformHandlerType TransformHandler;
	SplineSampleType SplineSampleParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosZ(Context);

	if (InstData->IsValid())
	{
		const float SplineLength = InstData->GetSplineLength<UseLUT>();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			float DistanceUnitDistance = SplineSampleParam.Get();

			FVector Vec = InstData->GetUpVectorAtDistanceAlongSpline<UseLUT>(DistanceUnitDistance * SplineLength, ESplineCoordinateSpace::Local);
			TransformHandler.TransformUnitVector(Vec, InstData->Transform);

			*OutPosX.GetDest() = float(Vec.X);		// LWC Precision Loss
			*OutPosY.GetDest() = float(Vec.Y);
			*OutPosZ.GetDest() = float(Vec.Z);
			SplineSampleParam.Advance();
			OutPosX.Advance();
			OutPosY.Advance();
			OutPosZ.Advance();
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			FVector Vec = FVector(0.0f, 0.0f, 1.0f); 
			TransformHandler.TransformUnitVector(Vec, InstData->Transform);

			*OutPosX.GetDest() = float(Vec.X);		// LWC Precision Loss
			*OutPosY.GetDest() = float(Vec.Y);
			*OutPosZ.GetDest() = float(Vec.Z);
			SplineSampleParam.Advance();
			OutPosX.Advance();
			OutPosY.Advance();
			OutPosZ.Advance();
		}
	}
}

template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
void UNiagaraDataInterfaceSpline::SampleSplineRightVectorByUnitDistance(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISpline_InstanceData> InstData(Context);
	TransformHandlerType TransformHandler;
	SplineSampleType SplineSampleParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosZ(Context);
	
	
	if (InstData->IsValid())
	{
		const float SplineLength = InstData->GetSplineLength<UseLUT>();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			float DistanceUnitDistance = SplineSampleParam.Get();

			FVector Vec = InstData->GetRightVectorAtDistanceAlongSpline<UseLUT>(DistanceUnitDistance * SplineLength, ESplineCoordinateSpace::Local);
			TransformHandler.TransformUnitVector(Vec, InstData->Transform);

			*OutPosX.GetDest() = float(Vec.X);	// LWC Precision Loss
			*OutPosY.GetDest() = float(Vec.Y);
			*OutPosZ.GetDest() = float(Vec.Z);
			SplineSampleParam.Advance();
			OutPosX.Advance();
			OutPosY.Advance();
			OutPosZ.Advance();
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			FVector Vec = FVector(-1.0f, 0.0f, 0.0f); 
			TransformHandler.TransformUnitVector(Vec, InstData->Transform);

			*OutPosX.GetDest() = float(Vec.X);	// LWC Precision Loss
			*OutPosY.GetDest() = float(Vec.Y);
			*OutPosZ.GetDest() = float(Vec.Z);
			SplineSampleParam.Advance();
			OutPosX.Advance();
			OutPosY.Advance();
			OutPosZ.Advance();
		}
	}
}

template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
void UNiagaraDataInterfaceSpline::SampleSplineTangentByUnitDistance(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISpline_InstanceData> InstData(Context);
	TransformHandlerType TransformHandler;
	SplineSampleType SplineSampleParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosZ(Context);

	if (InstData->IsValid())
	{
		const float SplineLength = InstData->GetSplineLength<UseLUT>();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			float DistanceUnitDistance = SplineSampleParam.Get();

			FVector Vec = InstData->GetTangentAtDistanceAlongSpline<UseLUT>(DistanceUnitDistance * SplineLength, ESplineCoordinateSpace::Local);
			TransformHandler.TransformUnitVector(Vec, InstData->Transform);

			*OutPosX.GetDest() = float(Vec.X);	// LWC Precision Loss
			*OutPosY.GetDest() = float(Vec.Y);
			*OutPosZ.GetDest() = float(Vec.Z);
			SplineSampleParam.Advance();
			OutPosX.Advance();
			OutPosY.Advance();
			OutPosZ.Advance();
		}
	}
	else
	{
		// Note if GetTangentAtDistanceAlongSpline fails it will return FVector::ZeroVector
		const FVector3f Vec = FVector3f::ZeroVector;
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			*OutPosX.GetDest() = Vec.X;
			*OutPosY.GetDest() = Vec.Y;
			*OutPosZ.GetDest() = Vec.Z;
			SplineSampleParam.Advance();
			OutPosX.Advance();
			OutPosY.Advance();
			OutPosZ.Advance();
		}
	}
}

template<typename UseLUT, typename TransformHandlerType, typename SplineSampleType>
void UNiagaraDataInterfaceSpline::SampleSplineDirectionByUnitDistance(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISpline_InstanceData> InstData(Context);
	TransformHandlerType TransformHandler;
	SplineSampleType SplineSampleParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutPosZ(Context);

	if (InstData->IsValid())
	{
		const float SplineLength = InstData->GetSplineLength<UseLUT>();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			float DistanceUnitDistance = SplineSampleParam.Get();

			FVector Vec = InstData->GetDirectionAtDistanceAlongSpline<UseLUT>(DistanceUnitDistance * SplineLength, ESplineCoordinateSpace::Local);
			TransformHandler.TransformUnitVector(Vec, InstData->Transform);

			*OutPosX.GetDest() = float(Vec.X);		// LWC Precision Loss
			*OutPosY.GetDest() = float(Vec.Y);
			*OutPosZ.GetDest() = float(Vec.Z);
			SplineSampleParam.Advance();
			OutPosX.Advance();
			OutPosY.Advance();
			OutPosZ.Advance();
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			FVector Vec = FVector(0.0f, 1.0f, 0.0f);
			TransformHandler.TransformUnitVector(Vec, InstData->Transform);

			*OutPosX.GetDest() = float(Vec.X);	// LWC Precision Loss
			*OutPosY.GetDest() = float(Vec.Y);
			*OutPosZ.GetDest() = float(Vec.Z);
			SplineSampleParam.Advance();
			OutPosX.Advance();
			OutPosY.Advance();
			OutPosZ.Advance();
		}
	}
}

void UNiagaraDataInterfaceSpline::WriteTransform(const FMatrix44f& ToWrite, FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FExternalFuncRegisterHandler<float> Out00(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out01(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out02(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out03(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out04(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out05(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out06(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out07(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out08(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out09(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out10(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out11(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out12(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out13(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out14(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out15(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*Out00.GetDest() = ToWrite.M[0][0]; Out00.Advance();
		*Out01.GetDest() = ToWrite.M[0][1]; Out01.Advance();
		*Out02.GetDest() = ToWrite.M[0][2]; Out02.Advance();
		*Out03.GetDest() = ToWrite.M[0][3]; Out03.Advance();
		*Out04.GetDest() = ToWrite.M[1][0]; Out04.Advance();
		*Out05.GetDest() = ToWrite.M[1][1]; Out05.Advance();
		*Out06.GetDest() = ToWrite.M[1][2]; Out06.Advance();
		*Out07.GetDest() = ToWrite.M[1][3]; Out07.Advance();
		*Out08.GetDest() = ToWrite.M[2][0]; Out08.Advance();
		*Out09.GetDest() = ToWrite.M[2][1]; Out09.Advance();
		*Out10.GetDest() = ToWrite.M[2][2]; Out10.Advance();
		*Out11.GetDest() = ToWrite.M[2][3]; Out11.Advance();
		*Out12.GetDest() = ToWrite.M[3][0]; Out12.Advance();
		*Out13.GetDest() = ToWrite.M[3][1]; Out13.Advance();
		*Out14.GetDest() = ToWrite.M[3][2]; Out14.Advance();
		*Out15.GetDest() = ToWrite.M[3][3]; Out15.Advance();
	}
}

template<typename UseLUT>
void UNiagaraDataInterfaceSpline::FindClosestUnitDistanceFromPositionWS(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISpline_InstanceData> InstData(Context);
	FNDIInputParam<FNiagaraPosition> PosParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutUnitDistance(Context);

	if (InstData->IsValid())
	{
		const float FinalKeyTime = InstData->GetFinalKeyTime<UseLUT>();

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const FNiagaraPosition SimPos = PosParam.GetAndAdvance();
			const FVector WorldPos = InstData->LwcConverter.ConvertSimulationPositionToWorld(SimPos);

			// This first call finds the key time, but this is not in 0..1 range for the spline. 
			const float KeyTime = InstData->FindInputKeyClosestToWorldLocation<UseLUT>(WorldPos);
			// We need to convert into the range by dividing through by the overall duration of the spline according to the keys.
			const float UnitDistance = FinalKeyTime > 0.0f ? KeyTime / FinalKeyTime : 0.0f;

			*OutUnitDistance.GetDest() = UnitDistance;
			OutUnitDistance.Advance();
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			*OutUnitDistance.GetDest() = 0.0f;

			PosParam.GetAndAdvance();
			OutUnitDistance.Advance();
		}
	}
}

void UNiagaraDataInterfaceSpline::GetLocalToWorld(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISpline_InstanceData> InstData(Context);
	WriteTransform(FMatrix44f(InstData->Transform), Context);	//LWC Precision Loss
}

void UNiagaraDataInterfaceSpline::GetLocalToWorldInverseTransposed(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISpline_InstanceData> InstData(Context);
	WriteTransform(FMatrix44f(InstData->TransformInverseTransposed), Context);	//LWC Precision Loss
}

void UNiagaraDataInterfaceSpline::VMGetSplineLength(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDISpline_InstanceData> InstData(Context);
	FNDIOutputParam<float> OutLength(Context);

	float SplineLength = 0.0f;
	if (InstData->IsValid())
	{
		SplineLength = bUseLUT ? InstData->GetSplineLength<TIntegralConstant<bool, true>>() : InstData->GetSplineLength<TIntegralConstant<bool, false>>();
	}

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutLength.SetAndAdvance(SplineLength);
	}
}

#undef LOCTEXT_NAMESPACE

