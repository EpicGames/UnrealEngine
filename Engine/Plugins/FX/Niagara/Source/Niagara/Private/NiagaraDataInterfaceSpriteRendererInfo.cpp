// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceSpriteRendererInfo.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraSpriteRendererProperties.h"
#include "NiagaraShaderParametersBuilder.h"
#include "RenderingThread.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceSpriteRendererInfo)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceSpriteRendererInfo"

namespace NDISpriteRendererInfoLocal
{
	static const FName IsValidName("IsValid");
	static const FName GetSourceModeName("GetSourceMode");
	static const FName GetAlignmentName("GetAlignment");
	static const FName GetFacingModeName("GetFacingMode");
	static const FName GetSubUVDetailsName("GetSubUVDetails");

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(uint32,			bIsValid)
		SHADER_PARAMETER(int32,				SourceMode)
		SHADER_PARAMETER(int32,				Alignment)
		SHADER_PARAMETER(int32,				FacingMode)
		SHADER_PARAMETER(uint32,			bSubImageBlend)
		SHADER_PARAMETER(FVector2f,			SubImageSize)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceSpriteRendererInfoTemplate.ush");

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		FShaderParameters	ShaderParameters_RT;

		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

		void PushToRenderThread(const UNiagaraSpriteRendererProperties* InProperties)
		{
			const UNiagaraSpriteRendererProperties* Properties = InProperties ? InProperties : GetDefault<UNiagaraSpriteRendererProperties>();

			FShaderParameters Parameters;
			Parameters.bIsValid			= InProperties != nullptr;
			Parameters.SourceMode		= uint32(Properties->SourceMode);
			Parameters.Alignment		= uint32(Properties->Alignment);
			Parameters.FacingMode		= uint32(Properties->FacingMode);
			Parameters.bSubImageBlend	= Properties->bSubImageBlend ? 1 : 0;
			Parameters.SubImageSize		= FVector2f(Properties->SubImageSize);

			ENQUEUE_RENDER_COMMAND(NDISpriteRendererInfo_Update)
			(
				[this, Parameters](FRHICommandList& RHICmdList)
				{
					ShaderParameters_RT = Parameters;
				}
			);
		}
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////

UNiagaraDataInterfaceSpriteRendererInfo::UNiagaraDataInterfaceSpriteRendererInfo(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new NDISpriteRendererInfoLocal::FNDIProxy());
	MarkRenderDataDirty();
}

void UNiagaraDataInterfaceSpriteRendererInfo::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		
		// We can't allow user variables of this type because it will cause components to have external reference (the renderer)
		Flags &= ~ENiagaraTypeRegistryFlags::AllowUserVariable;
		Flags &= ~ENiagaraTypeRegistryFlags::AllowSystemVariable;

		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}	
}

void UNiagaraDataInterfaceSpriteRendererInfo::PostLoad()
{
	Super::PostLoad();

	if (SpriteRenderer)
	{
		SpriteRenderer->ConditionalPostLoad();
		OnSpriteRendererChanged(SpriteRenderer);
	}
}

void UNiagaraDataInterfaceSpriteRendererInfo::BeginDestroy()
{
#if WITH_EDITOR
	OnSpriteRendererChanged(nullptr);
#endif
	Super::BeginDestroy();
}

#if WITH_EDITOR
void UNiagaraDataInterfaceSpriteRendererInfo::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (SpriteRenderer && PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceSpriteRendererInfo, SpriteRenderer))
	{
		OnSpriteRendererChanged(nullptr);
	}
}

void UNiagaraDataInterfaceSpriteRendererInfo::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// If coming from undo, property will be nullptr and since we copy the info, we need to reacquire if new.
	if (PropertyChangedEvent.Property == nullptr || (PropertyChangedEvent.Property &&
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceSpriteRendererInfo, SpriteRenderer)))
	{
		OnSpriteRendererChanged(SpriteRenderer);
	}
}
#endif // WITH_EDITOR

void UNiagaraDataInterfaceSpriteRendererInfo::OnSpriteRendererChanged(UNiagaraSpriteRendererProperties* NewSpriteRenderer)
{
#if WITH_EDITOR
	if ( SpriteRenderer )
	{
		SpriteRenderer->OnChanged().Remove(OnSpriteRendererChangedHandle);
		OnSpriteRendererChangedHandle.Reset();
	}

	if ( NewSpriteRenderer )
	{
		OnSpriteRendererChangedHandle = NewSpriteRenderer->OnChanged().AddLambda([this]() { MarkRenderDataDirty(); } );
	}
#endif
	MarkRenderDataDirty();
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceSpriteRendererInfo::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature DefaultImmutableSignature;
	DefaultImmutableSignature.bMemberFunction = true;
	DefaultImmutableSignature.Inputs.Emplace(GetClass(), TEXT("SpriteRendererInfo"));

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultImmutableSignature);
		Sig.Name = NDISpriteRendererInfoLocal::IsValidName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultImmutableSignature);
		Sig.Name = NDISpriteRendererInfoLocal::GetSourceModeName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition(StaticEnum<ENiagaraRendererSourceDataMode>()), TEXT("SourceMode"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultImmutableSignature);
		Sig.Name = NDISpriteRendererInfoLocal::GetAlignmentName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition(StaticEnum<ENiagaraSpriteAlignment>()), TEXT("Alignment"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultImmutableSignature);
		Sig.Name = NDISpriteRendererInfoLocal::GetFacingModeName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition(StaticEnum<ENiagaraSpriteFacingMode>()), TEXT("FacingMode"));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultImmutableSignature);
		Sig.Name = NDISpriteRendererInfoLocal::GetSubUVDetailsName;
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("BlendEnabled"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("SubImageSize"));
	}
}
#endif

void UNiagaraDataInterfaceSpriteRendererInfo::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	if (BindingInfo.Name == NDISpriteRendererInfoLocal::IsValidName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSpriteRendererInfo::VMIsValid);
	}
	else if (BindingInfo.Name == NDISpriteRendererInfoLocal::GetSourceModeName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSpriteRendererInfo::VMGetSourceMode);
	}
	else if (BindingInfo.Name == NDISpriteRendererInfoLocal::GetAlignmentName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSpriteRendererInfo::VMGetAlignment);
	}
	else if (BindingInfo.Name == NDISpriteRendererInfoLocal::GetFacingModeName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSpriteRendererInfo::VMGetFacingMode);
	}
	else if (BindingInfo.Name == NDISpriteRendererInfoLocal::GetSubUVDetailsName)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceSpriteRendererInfo::VMGetSubUVDetails);
	}
}

bool UNiagaraDataInterfaceSpriteRendererInfo::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	auto OtherTyped = CastChecked<const UNiagaraDataInterfaceSpriteRendererInfo>(Other);
	return SpriteRenderer == OtherTyped->SpriteRenderer;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceSpriteRendererInfo::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDISpriteRendererInfoLocal::TemplateShaderFile);
	bSuccess &= InVisitor->UpdateShaderParameters<NDISpriteRendererInfoLocal::FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceSpriteRendererInfo::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDISpriteRendererInfoLocal::TemplateShaderFile, TemplateArgs);
}

bool UNiagaraDataInterfaceSpriteRendererInfo::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if ((FunctionInfo.DefinitionName == NDISpriteRendererInfoLocal::IsValidName) ||
		(FunctionInfo.DefinitionName == NDISpriteRendererInfoLocal::GetSourceModeName) ||
		(FunctionInfo.DefinitionName == NDISpriteRendererInfoLocal::GetAlignmentName) ||
		(FunctionInfo.DefinitionName == NDISpriteRendererInfoLocal::GetFacingModeName) ||
		(FunctionInfo.DefinitionName == NDISpriteRendererInfoLocal::GetSubUVDetailsName) )
	{
		return true;
	}

	return false;
}
#endif

void UNiagaraDataInterfaceSpriteRendererInfo::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDISpriteRendererInfoLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceSpriteRendererInfo::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	NDISpriteRendererInfoLocal::FNDIProxy& DIProxy = Context.GetProxy<NDISpriteRendererInfoLocal::FNDIProxy>();
	NDISpriteRendererInfoLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDISpriteRendererInfoLocal::FShaderParameters>();
	*ShaderParameters = DIProxy.ShaderParameters_RT;
}

#if WITH_EDITOR

void UNiagaraDataInterfaceSpriteRendererInfo::GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	if (SpriteRenderer == nullptr)
	{
		OutWarnings.Emplace(
			LOCTEXT("NoRendererSelectedWarning", "A Sprite Renderer applied to an emitter in this system is expected to be selected here"),
			LOCTEXT("NoRendererSelectedWarningSummary", "No Sprite Renderer selected"),
			FNiagaraDataInterfaceFix()
		);
	}
	else if ( !SpriteRenderer->GetIsEnabled() )
	{
		OutWarnings.Emplace(
			LOCTEXT("RendererDisabledWarning", "The selected Sprite Renderer is disabled"),
			LOCTEXT("RendererDisabledWarningSummary", "Sprite Renderer is disabled"),
			FNiagaraDataInterfaceFix::CreateLambda(
				[this]()
				{
					SpriteRenderer->SetIsEnabled(true);
					return true;
				}
			)
		);
	}
	else if ( !SpriteRenderer->IsContainedInEmitterRendererArray() )
	{
		OutWarnings.Emplace(
			LOCTEXT("OrphanedRendererWarning", "The selected Sprite Renderer is invalid"),
			LOCTEXT("OrphanedRendererWarningSummary", "Sprite Renderer is orphaned and no longer valid"),
			FNiagaraDataInterfaceFix::CreateLambda(
				[this]()
				{
					const FScopedTransaction Transaction(LOCTEXT("ClearSpriteRenderer", "Clear Sprite Renderer"));
					Modify();
					SpriteRenderer = nullptr;
					return true;
				}
			)
		);
	}
}

#endif // WITH_EDITOR

bool UNiagaraDataInterfaceSpriteRendererInfo::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceSpriteRendererInfo* DestinationTyped = CastChecked<UNiagaraDataInterfaceSpriteRendererInfo>(Destination);
	DestinationTyped->OnSpriteRendererChanged(nullptr);
	DestinationTyped->SpriteRenderer = SpriteRenderer;
	DestinationTyped->OnSpriteRendererChanged(SpriteRenderer);

	return true;
}

void UNiagaraDataInterfaceSpriteRendererInfo::PushToRenderThreadImpl()
{
	GetProxyAs<NDISpriteRendererInfoLocal::FNDIProxy>()->PushToRenderThread(SpriteRenderer);
}

void UNiagaraDataInterfaceSpriteRendererInfo::VMIsValid(FVectorVMExternalFunctionContext& Context)
{
	FNDIOutputParam<bool> OutIsValid(Context);

	const bool bIsValid = SpriteRenderer != nullptr;
	for (int32 i=0; i < Context.GetNumInstances(); ++i)
	{
		OutIsValid.SetAndAdvance(bIsValid);
	}
}

void UNiagaraDataInterfaceSpriteRendererInfo::VMGetSourceMode(FVectorVMExternalFunctionContext& Context)
{
	FNDIOutputParam<uint32> OutSourceMode(Context);

	const UNiagaraSpriteRendererProperties* Properties = SpriteRenderer ? SpriteRenderer.Get() : GetDefault<UNiagaraSpriteRendererProperties>();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutSourceMode.SetAndAdvance(uint32(Properties->SourceMode));
	}
}

void UNiagaraDataInterfaceSpriteRendererInfo::VMGetAlignment(FVectorVMExternalFunctionContext& Context)
{
	FNDIOutputParam<uint32> OutAlignment(Context);

	const UNiagaraSpriteRendererProperties* Properties = SpriteRenderer ? SpriteRenderer.Get() : GetDefault<UNiagaraSpriteRendererProperties>();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutAlignment.SetAndAdvance(uint32(Properties->Alignment));
	}
}

void UNiagaraDataInterfaceSpriteRendererInfo::VMGetFacingMode(FVectorVMExternalFunctionContext& Context)
{
	FNDIOutputParam<uint32> OutFacingMode(Context);

	const UNiagaraSpriteRendererProperties* Properties = SpriteRenderer ? SpriteRenderer.Get() : GetDefault<UNiagaraSpriteRendererProperties>();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutFacingMode.SetAndAdvance(uint32(Properties->FacingMode));
	}
}

void UNiagaraDataInterfaceSpriteRendererInfo::VMGetSubUVDetails(FVectorVMExternalFunctionContext& Context)
{
	FNDIOutputParam<bool> OutBlendEnabled(Context);
	FNDIOutputParam<FVector2f> OutSubImageSize(Context);

	const UNiagaraSpriteRendererProperties* Properties = SpriteRenderer ? SpriteRenderer.Get() : GetDefault<UNiagaraSpriteRendererProperties>();
	const FVector2f SubImageSize(Properties->SubImageSize);
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutBlendEnabled.SetAndAdvance(Properties->bSubImageBlend != 0);
		OutSubImageSize.SetAndAdvance(SubImageSize);
	}
}

#undef LOCTEXT_NAMESPACE
