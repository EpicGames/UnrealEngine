// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceStaticMeshUvMapping.h"

#include "Experimental/NiagaraMeshUvMapping.h"
#include "RawIndexBuffer.h"
#include "Engine/StaticMesh.h"
#include "StaticMeshResources.h"

struct FStaticMeshVertexAccessorHelper
{
public:
	FStaticMeshVertexAccessorHelper(const FStaticMeshLODResources* InLodRenderData)
		: LodRenderData(InLodRenderData)
		, IndexBuffer(LodRenderData->IndexBuffer.GetArrayView())
	{}

	FORCEINLINE FVector2D GetVertexUV(int32 VertexIndex, int32 UvChannel) const
	{
		return FVector2D(LodRenderData->VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(IndexBuffer[VertexIndex], UvChannel));
	}

	FORCEINLINE int32 GetTriangleCount() const
	{
		return LodRenderData->IndexBuffer.GetNumIndices() / 3;
	}

private:
	const FStaticMeshLODResources* LodRenderData;
	const FIndexArrayView IndexBuffer;
};

FStaticMeshUvMapping::FStaticMeshUvMapping(TWeakObjectPtr<UStaticMesh> InMeshObject, int32 InLodIndex, int32 InUvSetIndex)
	: FMeshUvMapping(InLodIndex, InUvSetIndex)
	, MeshObject(InMeshObject)
{
}

bool FStaticMeshUvMapping::IsValidMeshObject(const TWeakObjectPtr<UStaticMesh>& MeshObject, int32 InLodIndex, int32 InUvSetIndex)
{
	UStaticMesh* Mesh = MeshObject.Get();

	if (!Mesh || !Mesh->bAllowCPUAccess)
	{
		return false;
	}

	if (!GetLodRenderData(*Mesh, InLodIndex, InUvSetIndex))
	{
		// no render data available
		return false;
	}

	return true;
}

bool FStaticMeshUvMapping::Matches(const TWeakObjectPtr<UStaticMesh>& InMeshObject, int32 InLodIndex, int32 InUvSetIndex) const
{
	return LodIndex == InLodIndex && MeshObject == InMeshObject && UvSetIndex == InUvSetIndex;
}

const FStaticMeshLODResources* FStaticMeshUvMapping::GetLodRenderData() const
{
	if (const UStaticMesh* Mesh = MeshObject.Get())
	{
		return GetLodRenderData(*Mesh, LodIndex, UvSetIndex);
	}
	return nullptr;
}

void FStaticMeshUvMapping::FindOverlappingTriangles(const FVector2D& InUv, float Tolerance, TArray<int32>& TriangleIndices) const
{
	TriangleIndices.Empty();

	if (const FStaticMeshLODResources* LodRenderData = GetLodRenderData())
	{
		FStaticMeshVertexAccessorHelper VertexHelper(LodRenderData);
		FQuadTreeQueryHelper QueryHelper(TriangleIndexQuadTree, VertexHelper, UvSetIndex);
		return QueryHelper.FindOverlappingTriangle(InUv, Tolerance, TriangleIndices);
	}
}

int32 FStaticMeshUvMapping::FindFirstTriangle(const FVector2D& InUv, float Tolerance, FVector& BarycentricCoord) const
{
	if (const FStaticMeshLODResources* LodRenderData = GetLodRenderData())
	{
		FStaticMeshVertexAccessorHelper VertexHelper(LodRenderData);
		FQuadTreeQueryHelper QueryHelper(TriangleIndexQuadTree, VertexHelper, UvSetIndex);
		return QueryHelper.FindFirstTriangle(InUv, Tolerance, BarycentricCoord);
	}

	return INDEX_NONE;
}

int32 FStaticMeshUvMapping::FindFirstTriangle(const FBox2D& InUvBox, FVector& BarycentricCoord) const
{
	if (const FStaticMeshLODResources* LodRenderData = GetLodRenderData())
	{
		FStaticMeshVertexAccessorHelper VertexHelper(LodRenderData);
		FQuadTreeQueryHelper QueryHelper(TriangleIndexQuadTree, VertexHelper, UvSetIndex);
		return QueryHelper.FindFirstTriangle(InUvBox, BarycentricCoord);
	}

	return INDEX_NONE;
}

const FStaticMeshLODResources* FStaticMeshUvMapping::GetLodRenderData(const UStaticMesh& Mesh, int32 LodIndex, int32 UvSetIndex)
{
	if (const FStaticMeshRenderData* RenderData = Mesh.GetRenderData())
	{
		if (RenderData->LODResources.IsValidIndex(LodIndex))
		{
			const FStaticMeshLODResources& LodRenderData = RenderData->LODResources[LodIndex];
			const int32 NumTexCoords = LodRenderData.GetNumTexCoords();

			if (NumTexCoords > UvSetIndex)
			{
				return &LodRenderData;
			}
		}
	}

	return nullptr;
}

void FStaticMeshUvMapping::BuildQuadTree()
{
	if (const FStaticMeshLODResources* LodRenderData = GetLodRenderData())
	{
		FStaticMeshVertexAccessorHelper VertexHelper(LodRenderData);
		FQuadTreeBuildHelper BuildHelper(VertexHelper, UvSetIndex, TriangleIndexQuadTree);
		BuildHelper.Build();
	}
}
