// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceTexture.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "RenderGraphBuilder.h"
#include "RHIStaticStates.h"
#include "RenderGraphUtils.h"
#include "ShaderCompilerCore.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceTexture)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterfaceTexture"

const TCHAR* UNiagaraDataInterfaceTexture::TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceTextureTemplate.ush");
const FName UNiagaraDataInterfaceTexture::LoadTexture2DName(TEXT("LoadTexture2D"));
const FName UNiagaraDataInterfaceTexture::GatherRedTexture2DName(TEXT("GatherRedTexture2D"));
const FName UNiagaraDataInterfaceTexture::SampleTexture2DName(TEXT("SampleTexture2D"));
const FName UNiagaraDataInterfaceTexture::SamplePseudoVolumeTextureName(TEXT("SamplePseudoVolumeTexture"));
const FName UNiagaraDataInterfaceTexture::GetTextureDimensionsName(TEXT("GetTextureDimensions"));
const FName UNiagaraDataInterfaceTexture::GetNumMipLevelsName(TEXT("GetNumMipLevels"));

struct FNDITextureFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		AddMipLevelSupport = 1,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

struct FNDITextureInstanceData_GameThread
{
	TWeakObjectPtr<UTexture> CurrentTexture = nullptr;
	FIntPoint CurrentTextureSize = FIntPoint::ZeroValue;
	int32 CurrentTextureMipLevels = 0;
	FNiagaraParameterDirectBinding<UObject*> UserParamBinding;
};

struct FNDITextureInstanceData_RenderThread
{
	FSamplerStateRHIRef		SamplerStateRHI;
	FTextureReferenceRHIRef	TextureReferenceRHI;
	FIntPoint				TextureSize = FIntPoint(0, 0);
	int32					MipLevels = 0;

	FRDGTextureRef			TransientRDGTexture = nullptr;
};

struct FNiagaraDataInterfaceProxyTexture : public FNiagaraDataInterfaceProxy
{
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { checkNoEntry(); }
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override
	{
		if (Context.IsFinalPostSimulate())
		{
			if (FNDITextureInstanceData_RenderThread* InstanceData = InstanceData_RT.Find(Context.GetSystemInstanceID()))
			{
				InstanceData->TransientRDGTexture = nullptr;
			}
		}
	}

	TMap<FNiagaraSystemInstanceID, FNDITextureInstanceData_RenderThread> InstanceData_RT;
};

UNiagaraDataInterfaceTexture::UNiagaraDataInterfaceTexture(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, Texture(nullptr)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyTexture());

	FNiagaraTypeDefinition Def(UTexture::StaticClass());
	TextureUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceTexture::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

void UNiagaraDataInterfaceTexture::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
	if (NiagaraVer < FNiagaraCustomVersion::TextureDataInterfaceUsesCustomSerialize)
	{
		if (Texture != nullptr)
		{
			Texture->ConditionalPostLoad();
		}
	}
#endif
}

void UNiagaraDataInterfaceTexture::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsLoading() == false || Ar.CustomVer(FNiagaraCustomVersion::GUID) >= FNiagaraCustomVersion::TextureDataInterfaceUsesCustomSerialize)
	{
		TArray<uint8> StreamData;
		Ar << StreamData;
	}
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);
}

bool UNiagaraDataInterfaceTexture::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceTexture* DestinationTexture = CastChecked<UNiagaraDataInterfaceTexture>(Destination);
	DestinationTexture->Texture = Texture;
	DestinationTexture->TextureUserParameter = TextureUserParameter;

	return true;
}

bool UNiagaraDataInterfaceTexture::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceTexture* OtherTexture = CastChecked<const UNiagaraDataInterfaceTexture>(Other);
	return
		OtherTexture->Texture == Texture &&
		OtherTexture->TextureUserParameter == TextureUserParameter;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceTexture::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature DefaultGpuSig;
	DefaultGpuSig.bMemberFunction = true;
	DefaultGpuSig.bRequiresContext = false;
	DefaultGpuSig.bSupportsCPU = false;
	DefaultGpuSig.bSupportsGPU = true;
	DefaultGpuSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Texture"));
	DefaultGpuSig.SetFunctionVersion(FNDITextureFunctionVersion::LatestVersion);

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultGpuSig);
		Sig.Name = LoadTexture2DName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("TextureLoadTexture2DDesc", "Read a texel from the provided location & mip without any filtering or sampling."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultGpuSig);
		Sig.Name = GatherRedTexture2DName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("TextureGatherTexture2DDesc", "Gather the 4 samples (Red only) that would be used in bilinear interpolation."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultGpuSig);
		Sig.Name = SampleTexture2DName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("TextureSampleTexture2DDesc", "Sample supplied mip level from input 2d texture at the specified UV coordinates. The UV origin (0,0) is in the upper left hand corner of the image."));
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultGpuSig);
		Sig.Name = SamplePseudoVolumeTextureName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UVW"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("XYNumFrames"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("TotalNumFrames"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipMode"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("DDX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("DDY"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("TextureSamplePseudoVolumeTextureDesc", "Return a pseudovolume texture sample.\nUseful for simulating 3D texturing with a 2D texture or as a texture flipbook with lerped transitions.\nTreats 2d layout of frames as a 3d texture and performs bilinear filtering by blending with an offset Z frame.\nTexture = Input Texture Object storing Volume Data\nUVW = Input float3 for Position, 0 - 1\nXYNumFrames = Input float for num frames in x, y directions\nTotalNumFrames = Input float for num total frames\nMipMode = Sampling mode : 0 = use miplevel, 1 = use UV computed gradients, 2 = Use gradients(default = 0)\nMipLevel = MIP level to use in mipmode = 0 (default 0)\nDDX, DDY = Texture gradients in mipmode = 2\n"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = GetTextureDimensionsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Texture"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Dimensions2D"));
		Sig.SetDescription(LOCTEXT("TextureDimsDesc", "Get the dimensions of the provided Mip level."));
		Sig.SetFunctionVersion(FNDITextureFunctionVersion::LatestVersion);
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = GetNumMipLevelsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Texture"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumMipLevels"));
		Sig.SetDescription(LOCTEXT("GetNumMipLevelsDesc", "Get the number of Mip Levels."));
		Sig.SetFunctionVersion(FNDITextureFunctionVersion::LatestVersion);
	}
}
#endif

void UNiagaraDataInterfaceTexture::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == SampleTexture2DName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 4);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceTexture::VMSampleTexture);
	}
	else if (BindingInfo.Name == SamplePseudoVolumeTextureName)
	{
		check(BindingInfo.GetNumInputs() == 13 && BindingInfo.GetNumOutputs() == 4);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceTexture::VMSamplePseudoVolumeTexture);
	}
	else if (BindingInfo.Name == GetTextureDimensionsName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 2);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceTexture::VMGetTextureDimensions);
	}
	else if (BindingInfo.Name == GetNumMipLevelsName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceTexture::VMGetNumMipLevels);
	}
}

int32 UNiagaraDataInterfaceTexture::PerInstanceDataSize() const
{
	return sizeof(FNDITextureInstanceData_GameThread);
}

bool UNiagaraDataInterfaceTexture::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDITextureInstanceData_GameThread* InstanceData = new (PerInstanceData) FNDITextureInstanceData_GameThread();
	InstanceData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), TextureUserParameter.Parameter);
	return true;
}

void UNiagaraDataInterfaceTexture::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDITextureInstanceData_GameThread* InstanceData = static_cast<FNDITextureInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FNDITextureInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(NDITexture_RemoveInstance)
	(
		[RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxyTexture>(), RT_InstanceID=SystemInstance->GetId()](FRHICommandListImmediate&)
		{
			RT_Proxy->InstanceData_RT.Remove(RT_InstanceID);
		}
	);
}

bool UNiagaraDataInterfaceTexture::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FNDITextureInstanceData_GameThread* InstanceData = static_cast<FNDITextureInstanceData_GameThread*>(PerInstanceData);

	UTexture* CurrentTexture = InstanceData->UserParamBinding.GetValueOrDefault<UTexture>(Texture);
	FIntPoint CurrentTextureSize = FIntPoint::ZeroValue;
	int32 CurrentTextureMipLevels = 0;

	if (CurrentTexture != nullptr)
	{
		CurrentTextureSize.X = int32(CurrentTexture->GetSurfaceWidth());
		CurrentTextureSize.Y = int32(CurrentTexture->GetSurfaceHeight());
		if (UTexture2D* CurrentTexture2D = Cast<UTexture2D>(CurrentTexture))
		{
			CurrentTextureMipLevels = CurrentTexture2D->GetNumMips();
		}
		else if (UTextureRenderTarget2D* CurrentTexture2DRT = Cast<UTextureRenderTarget2D>(CurrentTexture))
		{
			CurrentTextureMipLevels = CurrentTexture2DRT->GetNumMips();
		}
		else
		{
			CurrentTextureMipLevels = 1;
		}
	}

	if ( (InstanceData->CurrentTexture != CurrentTexture) || (InstanceData->CurrentTextureSize != CurrentTextureSize) || (InstanceData->CurrentTextureMipLevels != CurrentTextureMipLevels) )
	{
		InstanceData->CurrentTexture = CurrentTexture;
		InstanceData->CurrentTextureSize = CurrentTextureSize;
		InstanceData->CurrentTextureMipLevels = CurrentTextureMipLevels;

		ENQUEUE_RENDER_COMMAND(NDITexture_UpdateInstance)
		(
			[RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxyTexture>(), RT_InstanceID=SystemInstance->GetId(), RT_Texture=CurrentTexture, RT_TextureSize=CurrentTextureSize, RT_MipLevels=CurrentTextureMipLevels](FRHICommandListImmediate&)
			{
				FNDITextureInstanceData_RenderThread& InstanceData = RT_Proxy->InstanceData_RT.FindOrAdd(RT_InstanceID);
				if (RT_Texture)
				{
					InstanceData.TextureReferenceRHI = RT_Texture->TextureReference.TextureReferenceRHI;
					InstanceData.SamplerStateRHI = RT_Texture->GetResource() ? RT_Texture->GetResource()->SamplerStateRHI.GetReference() : TStaticSamplerState<SF_Point>::GetRHI();
					InstanceData.TextureSize = RT_TextureSize;
					InstanceData.MipLevels = RT_MipLevels;
				}
				else
				{
					InstanceData.TextureReferenceRHI = nullptr;
					InstanceData.SamplerStateRHI = nullptr;
					InstanceData.TextureSize = FIntPoint(0, 0);
					InstanceData.MipLevels = 0;
				}
			}
		);
	}

	return false;
}

void UNiagaraDataInterfaceTexture::VMSampleTexture(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDITextureInstanceData_GameThread> InstData(Context);
	FNDIInputParam<FVector2f>	InUV(Context);
	FNDIInputParam<float>		InMipLevel(Context);
	FNDIOutputParam<FVector4f>	OutValue(Context);

	const FVector4f DefaultValue(1.0f, 0.0f, 1.0f, 1.0f);
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutValue.SetAndAdvance(DefaultValue);
	}
}

void UNiagaraDataInterfaceTexture::VMSamplePseudoVolumeTexture(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDITextureInstanceData_GameThread> InstData(Context);
	FNDIInputParam<FVector3f>	InUVW(Context);
	FNDIInputParam<FVector2f>	InXYNumFrames(Context);
	FNDIInputParam<float>		InTotalNumFrames(Context);
	FNDIInputParam<int32>		InMipMode(Context);
	FNDIInputParam<float>		InMipLevel(Context);
	FNDIInputParam<FVector2f>	InDDX(Context);
	FNDIInputParam<FVector2f>	InDDY(Context);
	FNDIOutputParam<FVector4f>	OutValue(Context);

	const FVector4f DefaultValue(1.0f, 0.0f, 1.0f, 1.0f);
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutValue.SetAndAdvance(DefaultValue);
	}
}

void UNiagaraDataInterfaceTexture::VMGetTextureDimensions(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDITextureInstanceData_GameThread> InstData(Context);
	FNDIInputParam<int32>		InMipLevel(Context);
	FNDIOutputParam<FVector2f>	OutSize(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 MipLevel = InMipLevel.GetAndAdvance();
		const FVector2f TextureSize(
			float(FMath::Max(InstData->CurrentTextureSize.X >> MipLevel, 1)),
			float(FMath::Max(InstData->CurrentTextureSize.Y >> MipLevel, 1))
		);
		OutSize.SetAndAdvance(TextureSize);
	}
}

void UNiagaraDataInterfaceTexture::VMGetNumMipLevels(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDITextureInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int32> OutNumMipLevels(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutNumMipLevels.SetAndAdvance(InstData->CurrentTextureMipLevels);
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceTexture::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceTexture::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceTexture::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	static const TSet<FName> ValidGpuFunctions =
	{
		LoadTexture2DName,
		GatherRedTexture2DName,
		SampleTexture2DName,
		SamplePseudoVolumeTextureName,
		GetTextureDimensionsName,
		GetNumMipLevelsName,
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

bool UNiagaraDataInterfaceTexture::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	if (FunctionSignature.FunctionVersion < FNDITextureFunctionVersion::AddMipLevelSupport)
	{
		static FName LegacyDimsName("TextureDimensions2D");
		if ( FunctionSignature.Name == SampleTexture2DName )
		{
			FunctionSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		}
		else if (FunctionSignature.Name == LegacyDimsName )
		{
			FunctionSignature.Name = GetTextureDimensionsName;
			FunctionSignature.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		}
	}

	const bool bUpdated = FunctionSignature.FunctionVersion < FNDITextureFunctionVersion::LatestVersion;
	FunctionSignature.FunctionVersion = FNDITextureFunctionVersion::LatestVersion;
	return bUpdated;
}
#endif

void UNiagaraDataInterfaceTexture::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceTexture::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyTexture& TextureProxy = Context.GetProxy<FNiagaraDataInterfaceProxyTexture>();
	FNDITextureInstanceData_RenderThread* InstanceData = TextureProxy.InstanceData_RT.Find(Context.GetSystemInstanceID());
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	FShaderParameters* Parameters = Context.GetParameterNestedStruct<FShaderParameters>();
	if (InstanceData && InstanceData->TextureReferenceRHI.IsValid())
	{
		Parameters->TextureSize = InstanceData->TextureSize;
		Parameters->MipLevels = InstanceData->MipLevels;
		Parameters->TextureSampler = InstanceData->SamplerStateRHI;
		if (Context.IsResourceBound(&Parameters->Texture))
		{
			FRDGTextureRef RDGTexture = InstanceData ? InstanceData->TransientRDGTexture : nullptr;
			if (InstanceData && RDGTexture == nullptr)
			{
				FTextureRHIRef ResolvedTextureRHI = InstanceData->TextureReferenceRHI->GetReferencedTexture();
				if (ResolvedTextureRHI.IsValid())
				{
					InstanceData->TransientRDGTexture = GraphBuilder.FindExternalTexture(ResolvedTextureRHI);
					if (InstanceData->TransientRDGTexture == nullptr)
					{
						InstanceData->TransientRDGTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(ResolvedTextureRHI, TEXT("NiagaraTexture2D")));

						GraphBuilder.UseInternalAccessMode(InstanceData->TransientRDGTexture);
						Context.GetRDGExternalAccessQueue().Add(InstanceData->TransientRDGTexture);
					}
					RDGTexture = InstanceData->TransientRDGTexture;
				}
			}

			Parameters->Texture = RDGTexture ? RDGTexture : Context.GetComputeDispatchInterface().GetBlackTexture(GraphBuilder, ETextureDimension::Texture2D);
		}
	}
	else
	{
		Parameters->TextureSize = FIntPoint(0, 0);
		Parameters->MipLevels = 0;
		Parameters->TextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		Parameters->Texture = Context.GetComputeDispatchInterface().GetBlackTexture(GraphBuilder, ETextureDimension::Texture2D);
	}
}

void UNiagaraDataInterfaceTexture::SetTexture(UTexture* InTexture)
{
	Texture = InTexture;
}

#undef LOCTEXT_NAMESPACE

