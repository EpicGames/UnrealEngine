// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceVector2DCurve.h"
#include "NiagaraTypes.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceVector2DCurve)

#if WITH_EDITORONLY_DATA
#include "Interfaces/ITargetPlatform.h"
#endif

//////////////////////////////////////////////////////////////////////////
//Vector2D Curve

const FName UNiagaraDataInterfaceVector2DCurve::SampleCurveName("SampleVector2DCurve");

UNiagaraDataInterfaceVector2DCurve::UNiagaraDataInterfaceVector2DCurve(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExposedName = TEXT("Vector2 Curve");

	SetDefaultLUT();
}

void UNiagaraDataInterfaceVector2DCurve::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}

#if WITH_EDITORONLY_DATA
	UpdateLUT();
#endif
}

void UNiagaraDataInterfaceVector2DCurve::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (bUseLUT && Ar.IsCooking() && Ar.CookingTarget()->RequiresCookedData())
	{
		UpdateLUT(true);

		Exchange(XCurve, XCurveCookedEditorCache);
		Exchange(YCurve, YCurveCookedEditorCache);

		Super::Serialize(Ar);

		Exchange(XCurve, XCurveCookedEditorCache);
		Exchange(YCurve, YCurveCookedEditorCache);
	}
	else if (bUseLUT && Ar.IsLoading() && GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		Super::Serialize(Ar);

		Exchange(XCurve, XCurveCookedEditorCache);
		Exchange(YCurve, YCurveCookedEditorCache);
	}
	else
#endif
	{
		Super::Serialize(Ar);
	}
}

void UNiagaraDataInterfaceVector2DCurve::UpdateTimeRanges()
{
	if ((XCurve.GetNumKeys() > 0 || YCurve.GetNumKeys() > 0))
	{
		LUTMinTime = FLT_MAX;
		LUTMinTime = FMath::Min(XCurve.GetNumKeys() > 0 ? XCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);
		LUTMinTime = FMath::Min(YCurve.GetNumKeys() > 0 ? YCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);

		LUTMaxTime = -FLT_MAX;
		LUTMaxTime = FMath::Max(XCurve.GetNumKeys() > 0 ? XCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTMaxTime = FMath::Max(YCurve.GetNumKeys() > 0 ? YCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTInvTimeRange = 1.0f / (LUTMaxTime - LUTMinTime);
	}
	else
	{
		LUTMinTime = 0.0f;
		LUTMaxTime = 1.0f;
		LUTInvTimeRange = 1.0f;
	}
}

TArray<float> UNiagaraDataInterfaceVector2DCurve::BuildLUT(int32 NumEntries) const
{
	TArray<float> OutputLUT;
	const float InvEntryCountFactor = (NumEntries > 1) ? (1.0f / float(NumEntries - 1.0f)) : 0.0f;

	OutputLUT.Reserve(NumEntries * 2);
	for (int32 i = 0; i < NumEntries; i++)
	{
		const float X = UnnormalizeTime(i * InvEntryCountFactor);
		OutputLUT.Add(XCurve.Eval(X));
		OutputLUT.Add(YCurve.Eval(X));
	}
	return OutputLUT;
}

bool UNiagaraDataInterfaceVector2DCurve::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceVector2DCurve* DestinationVector2DCurve = CastChecked<UNiagaraDataInterfaceVector2DCurve>(Destination);
	DestinationVector2DCurve->XCurve = XCurve;
	DestinationVector2DCurve->YCurve = YCurve;
#if WITH_EDITORONLY_DATA
	DestinationVector2DCurve->UpdateLUT();
	if (!CompareLUTS(DestinationVector2DCurve->ShaderLUT))
	{
		UE_LOGF(LogNiagara, Log, "Post CopyToInternal LUT generation is out of sync. Please investigate. %ls", *GetPathName());
	}
#endif
	return true;
}

bool UNiagaraDataInterfaceVector2DCurve::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceVector2DCurve* OtherVector2DCurve = CastChecked<const UNiagaraDataInterfaceVector2DCurve>(Other);
	return OtherVector2DCurve->XCurve == XCurve &&
		OtherVector2DCurve->YCurve == YCurve;
}
void UNiagaraDataInterfaceVector2DCurve::GetCurveData(TArray<FCurveData>& OutCurveData)
{
	OutCurveData.Add(FCurveData(&XCurve, TEXT("X"), FLinearColor(1.0f, 0.05f, 0.05f)));
	OutCurveData.Add(FCurveData(&YCurve, TEXT("Y"), FLinearColor(0.05f, 1.0f, 0.05f)));
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceVector2DCurve::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature Sig;
	Sig.Name = SampleCurveName;
	Sig.bMemberFunction = true;
	Sig.bRequiresContext = false;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Vector2DCurve")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("X")));
	Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Value")));
	//Sig.Owner = *GetName();

	OutFunctions.Add(Sig);
}

// build the shader function HLSL; function name is passed in, as it's defined per-DI; that way, configuration could change
// the HLSL in the spirit of a static switch
// TODO: need a way to identify each specific function here
// 
bool UNiagaraDataInterfaceVector2DCurve::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if (FunctionInfo.DefinitionName == SampleCurveName)
	{
		return true;
	}
	return false;
}
#endif

DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceVector2DCurve, SampleCurve);
void UNiagaraDataInterfaceVector2DCurve::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == SampleCurveName && BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 2)
	{
		TCurveUseLUTBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceVector2DCurve, SampleCurve)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else
	{
		UE_LOGF(LogNiagara, Display, "Could not find data interface external function in %ls.\n\tExpected Name: SampleVector2DCurve  Actual Name: %ls\n\tExpected Inputs: 1  Actual Inputs: %i\n\tExpected Outputs: 2  Actual Outputs: %i",
			*GetPathNameSafe(this), *BindingInfo.Name.ToString(), BindingInfo.GetNumInputs(), BindingInfo.GetNumOutputs());
	}
}

template<>
FORCEINLINE_DEBUGGABLE FVector2f UNiagaraDataInterfaceVector2DCurve::SampleCurveInternal<TIntegralConstant<bool, true>>(float X)
{
	const float RemappedX = FMath::Clamp(NormalizeTime(X) * LUTNumSamplesMinusOne, 0.0f, LUTNumSamplesMinusOne);
	const float PrevEntry = FMath::TruncToFloat(RemappedX);
	const float NextEntry = PrevEntry < LUTNumSamplesMinusOne ? PrevEntry + 1.0f : PrevEntry;
	const float Interp = RemappedX - PrevEntry;

	const int32 AIndex = (int32)(PrevEntry * (float)CurveLUTNumElems);
	const int32 BIndex = (int32)(NextEntry * (float)CurveLUTNumElems);
	const FVector2f A = FVector2f(ShaderLUT[AIndex], ShaderLUT[AIndex + 1]);
	const FVector2f B = FVector2f(ShaderLUT[BIndex], ShaderLUT[BIndex + 1]);
	return FMath::Lerp(A, B, Interp);
}

template<>
FORCEINLINE_DEBUGGABLE FVector2f UNiagaraDataInterfaceVector2DCurve::SampleCurveInternal<TIntegralConstant<bool, false>>(float X)
{
	return FVector2f(XCurve.Eval(X), YCurve.Eval(X));
}

template<typename UseLUT>
void UNiagaraDataInterfaceVector2DCurve::SampleCurve(FVectorVMExternalFunctionContext& Context)
{
	//TODO: Create some SIMDable optimized representation of the curve to do this faster.
	VectorVM::FExternalFuncInputHandler<float> XParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleY(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const float X = XParam.GetAndAdvance();
		const FVector2f V = SampleCurveInternal<UseLUT>(X);
		*OutSampleX.GetDestAndAdvance() = V.X;
		*OutSampleY.GetDestAndAdvance() = V.Y;
	}
}
