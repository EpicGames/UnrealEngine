// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceVector4Curve.h"
#include "NiagaraTypes.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceVector4Curve)

#if WITH_EDITORONLY_DATA
#include "Interfaces/ITargetPlatform.h"
#endif

//////////////////////////////////////////////////////////////////////////
//Color Curve

const FName UNiagaraDataInterfaceVector4Curve::SampleCurveName(TEXT("SampleColorCurve"));

UNiagaraDataInterfaceVector4Curve::UNiagaraDataInterfaceVector4Curve(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExposedName = TEXT("Vector4 Curve");

	SetDefaultLUT();
}

void UNiagaraDataInterfaceVector4Curve::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}

#if WITH_EDITORONLY_DATA
	UpdateLUT();
#endif
}

void UNiagaraDataInterfaceVector4Curve::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (bUseLUT && Ar.IsCooking() && Ar.CookingTarget()->RequiresCookedData())
	{
		UpdateLUT(true);

		Exchange(XCurve, XCurveCookedEditorCache);
		Exchange(YCurve, YCurveCookedEditorCache);
		Exchange(ZCurve, ZCurveCookedEditorCache);
		Exchange(WCurve, WCurveCookedEditorCache);

		Super::Serialize(Ar);

		Exchange(XCurve, XCurveCookedEditorCache);
		Exchange(YCurve, YCurveCookedEditorCache);
		Exchange(ZCurve, ZCurveCookedEditorCache);
		Exchange(WCurve, WCurveCookedEditorCache);
	}
	else if (bUseLUT && Ar.IsLoading() && GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		Super::Serialize(Ar);

		Exchange(XCurve, XCurveCookedEditorCache);
		Exchange(YCurve, YCurveCookedEditorCache);
		Exchange(ZCurve, ZCurveCookedEditorCache);
		Exchange(WCurve, WCurveCookedEditorCache);
	}
	else
#endif
	{
		Super::Serialize(Ar);
	}
}

void UNiagaraDataInterfaceVector4Curve::UpdateTimeRanges()
{
	if ((XCurve.GetNumKeys() > 0 || YCurve.GetNumKeys() > 0 || ZCurve.GetNumKeys() > 0 || WCurve.GetNumKeys() > 0))
	{
		LUTMinTime = FLT_MAX;
		LUTMinTime = FMath::Min(XCurve.GetNumKeys() > 0 ? XCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);
		LUTMinTime = FMath::Min(YCurve.GetNumKeys() > 0 ? YCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);
		LUTMinTime = FMath::Min(ZCurve.GetNumKeys() > 0 ? ZCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);
		LUTMinTime = FMath::Min(WCurve.GetNumKeys() > 0 ? WCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);

		LUTMaxTime = -FLT_MAX;
		LUTMaxTime = FMath::Max(XCurve.GetNumKeys() > 0 ? XCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTMaxTime = FMath::Max(YCurve.GetNumKeys() > 0 ? YCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTMaxTime = FMath::Max(ZCurve.GetNumKeys() > 0 ? ZCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTMaxTime = FMath::Max(WCurve.GetNumKeys() > 0 ? WCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTInvTimeRange = 1.0f / (LUTMaxTime - LUTMinTime);
	}
	else
	{
		LUTMinTime = 0.0f;
		LUTMaxTime = 1.0f;
		LUTInvTimeRange = 1.0f;
	}
}

TArray<float> UNiagaraDataInterfaceVector4Curve::BuildLUT(int32 NumEntries) const
{
	TArray<float> OutputLUT;
	const float InvEntryCountFactor = (NumEntries > 1) ? (1.0f / float(NumEntries - 1.0f)) : 0.0f;

	OutputLUT.Reserve(NumEntries * 4);
	for (int32 i = 0; i < NumEntries; i++)
	{
		float X = UnnormalizeTime(i * InvEntryCountFactor);
		FLinearColor C(XCurve.Eval(X), YCurve.Eval(X), ZCurve.Eval(X), WCurve.Eval(X));
		OutputLUT.Add(C.R);
		OutputLUT.Add(C.G);
		OutputLUT.Add(C.B);
		OutputLUT.Add(C.A);
	}
	return OutputLUT;
}

bool UNiagaraDataInterfaceVector4Curve::CopyToInternal(UNiagaraDataInterface* Destination) const 
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceVector4Curve* DestinationColorCurve = CastChecked<UNiagaraDataInterfaceVector4Curve>(Destination);
	DestinationColorCurve->XCurve = XCurve;
	DestinationColorCurve->YCurve = YCurve;
	DestinationColorCurve->ZCurve = ZCurve;
	DestinationColorCurve->WCurve = WCurve;
#if WITH_EDITORONLY_DATA
	DestinationColorCurve->UpdateLUT();
	if (!CompareLUTS(DestinationColorCurve->ShaderLUT))
	{
		UE_LOGF(LogNiagara, Log, "Post CopyToInternal LUT generation is out of sync. Please investigate. %ls", *GetPathName());
	}
#endif

	return true;
}

bool UNiagaraDataInterfaceVector4Curve::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceVector4Curve* OtherCurve = CastChecked<const UNiagaraDataInterfaceVector4Curve>(Other);
	return OtherCurve->XCurve == XCurve &&
		OtherCurve->YCurve == YCurve &&
		OtherCurve->ZCurve == ZCurve &&
		OtherCurve->WCurve == WCurve;
}

void UNiagaraDataInterfaceVector4Curve::GetCurveData(TArray<FCurveData>& OutCurveData)
{
	OutCurveData.Add(FCurveData(&XCurve, TEXT("X"), FLinearColor(1.0f, 0.05f, 0.05f)));
	OutCurveData.Add(FCurveData(&YCurve, TEXT("Y"), FLinearColor(0.05f, 1.0f, 0.05f)));
	OutCurveData.Add(FCurveData(&ZCurve, TEXT("Z"), FLinearColor(0.1f, 0.2f, 1.0f)));
	OutCurveData.Add(FCurveData(&WCurve, TEXT("W"), FLinearColor(0.2f, 0.2f, 0.2f)));
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceVector4Curve::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature Sig;
	Sig.Name = SampleCurveName;
	Sig.bMemberFunction = true;
	Sig.bRequiresContext = false;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Vector4Curve")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("X")));
	Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Value")));
	//Sig.Owner = *GetName();

	OutFunctions.Add(Sig);
}

// build the shader function HLSL; function name is passed in, as it's defined per-DI; that way, configuration could change
// the HLSL in the spirit of a static switch
// TODO: need a way to identify each specific function here
// 
bool UNiagaraDataInterfaceVector4Curve::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if (FunctionInfo.DefinitionName == SampleCurveName)
	{
		return true;
	}
	return false;
}
#endif

DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceVector4Curve, SampleCurve);
void UNiagaraDataInterfaceVector4Curve::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == SampleCurveName && BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 4)
	{
		TCurveUseLUTBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceVector4Curve, SampleCurve)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else
	{
		UE_LOGF(LogNiagara, Display, "Could not find data interface external function in %ls.\n\tExpected Name: SampleColorCurve  Actual Name: %ls\n\tExpected Inputs: 1  Actual Inputs: %i\n\tExpected Outputs: 4  Actual Outputs: %i",
			*GetPathNameSafe(this), *BindingInfo.Name.ToString(), BindingInfo.GetNumInputs(), BindingInfo.GetNumOutputs());
	}
}

template<>
FORCEINLINE_DEBUGGABLE FVector4f UNiagaraDataInterfaceVector4Curve::SampleCurveInternal<TIntegralConstant<bool, true>>(float X)
{
	const float RemappedX = FMath::Clamp(NormalizeTime(X) * LUTNumSamplesMinusOne, 0.0f, LUTNumSamplesMinusOne);
	const float PrevEntry = FMath::TruncToFloat(RemappedX);
	const float NextEntry = PrevEntry < LUTNumSamplesMinusOne ? PrevEntry + 1.0f : PrevEntry;
	const float Interp = RemappedX - PrevEntry;

	const int32 AIndex = (int32)(PrevEntry * (float)CurveLUTNumElems);
	const int32 BIndex = (int32)(NextEntry * (float)CurveLUTNumElems);
	const FVector4f A = FVector4f(ShaderLUT[AIndex], ShaderLUT[AIndex + 1], ShaderLUT[AIndex + 2], ShaderLUT[AIndex + 3]);
	const FVector4f B = FVector4f(ShaderLUT[BIndex], ShaderLUT[BIndex + 1], ShaderLUT[BIndex + 2], ShaderLUT[BIndex + 3]);
	return FMath::Lerp(A, B, Interp);
}

template<>
FORCEINLINE_DEBUGGABLE FVector4f UNiagaraDataInterfaceVector4Curve::SampleCurveInternal<TIntegralConstant<bool, false>>(float X)
{
	return FVector4f(XCurve.Eval(X), YCurve.Eval(X), ZCurve.Eval(X), WCurve.Eval(X));
}


template<typename UseLUT>
void UNiagaraDataInterfaceVector4Curve::SampleCurve(FVectorVMExternalFunctionContext& Context)
{
	//TODO: Create some SIMDable optimized representation of the curve to do this faster.
	VectorVM::FExternalFuncInputHandler<float> XParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> SamplePtrR(Context);
	VectorVM::FExternalFuncRegisterHandler<float> SamplePtrG(Context);
	VectorVM::FExternalFuncRegisterHandler<float> SamplePtrB(Context);
	VectorVM::FExternalFuncRegisterHandler<float> SamplePtrA(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const float X = XParam.GetAndAdvance();
		const FVector4f Sample = SampleCurveInternal<UseLUT>(X);
		*SamplePtrR.GetDestAndAdvance() = Sample.X;
		*SamplePtrG.GetDestAndAdvance() = Sample.Y;
		*SamplePtrB.GetDestAndAdvance() = Sample.Z;
		*SamplePtrA.GetDestAndAdvance() = Sample.W;
	}
}

