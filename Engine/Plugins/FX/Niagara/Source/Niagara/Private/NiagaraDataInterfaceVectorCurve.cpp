// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceVectorCurve.h"
#include "NiagaraTypes.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceVectorCurve)

#if WITH_EDITORONLY_DATA
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveVector.h"
#include "Curves/CurveFloat.h"
#include "Interfaces/ITargetPlatform.h"
#endif

//////////////////////////////////////////////////////////////////////////
//Vector Curve

const FName UNiagaraDataInterfaceVectorCurve::SampleCurveName(TEXT("SampleVectorCurve"));

UNiagaraDataInterfaceVectorCurve::UNiagaraDataInterfaceVectorCurve(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExposedName = TEXT("Vector3 Curve");

	SetDefaultLUT();
}

void UNiagaraDataInterfaceVectorCurve::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}

#if WITH_EDITORONLY_DATA
	UpdateLUT();
#endif
}

void UNiagaraDataInterfaceVectorCurve::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if ( bUseLUT && Ar.IsCooking() && Ar.CookingTarget()->RequiresCookedData() )
	{
		UpdateLUT(true);

		Exchange(XCurve, XCurveCookedEditorCache);
		Exchange(YCurve, YCurveCookedEditorCache);
		Exchange(ZCurve, ZCurveCookedEditorCache);

		Super::Serialize(Ar);

		Exchange(XCurve, XCurveCookedEditorCache);
		Exchange(YCurve, YCurveCookedEditorCache);
		Exchange(ZCurve, ZCurveCookedEditorCache);
	}
	else if (bUseLUT && Ar.IsLoading() && GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		Super::Serialize(Ar);

		Exchange(XCurve, XCurveCookedEditorCache);
		Exchange(YCurve, YCurveCookedEditorCache);
		Exchange(ZCurve, ZCurveCookedEditorCache);
	}
	else
#endif
	{
		Super::Serialize(Ar);
	}
}

void UNiagaraDataInterfaceVectorCurve::UpdateTimeRanges()
{
	if ((XCurve.GetNumKeys() > 0 || YCurve.GetNumKeys() > 0 || ZCurve.GetNumKeys() > 0))
	{
		LUTMinTime = FLT_MAX;
		LUTMinTime = FMath::Min(XCurve.GetNumKeys() > 0 ? XCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);
		LUTMinTime = FMath::Min(YCurve.GetNumKeys() > 0 ? YCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);
		LUTMinTime = FMath::Min(ZCurve.GetNumKeys() > 0 ? ZCurve.GetFirstKey().Time : LUTMinTime, LUTMinTime);

		LUTMaxTime = -FLT_MAX;
		LUTMaxTime = FMath::Max(XCurve.GetNumKeys() > 0 ? XCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTMaxTime = FMath::Max(YCurve.GetNumKeys() > 0 ? YCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTMaxTime = FMath::Max(ZCurve.GetNumKeys() > 0 ? ZCurve.GetLastKey().Time : LUTMaxTime, LUTMaxTime);
		LUTInvTimeRange = 1.0f / (LUTMaxTime - LUTMinTime);
	}
	else
	{
		LUTMinTime = 0.0f;
		LUTMaxTime = 1.0f;
		LUTInvTimeRange = 1.0f;
	}
}

TArray<float> UNiagaraDataInterfaceVectorCurve::BuildLUT(int32 NumEntries) const
{
	TArray<float> OutputLUT;
	const float InvEntryCountFactor = (NumEntries > 1) ? (1.0f / float(NumEntries - 1.0f)) : 0.0f;

	OutputLUT.Reserve(NumEntries * 3);
	for (int32 i=0; i < NumEntries; i++)
	{
		const float X = UnnormalizeTime(i * InvEntryCountFactor);
		const FVector3f C(XCurve.Eval(X), YCurve.Eval(X), ZCurve.Eval(X));
		OutputLUT.Add(C.X);
		OutputLUT.Add(C.Y);
		OutputLUT.Add(C.Z);
	}
	return OutputLUT;
}

bool UNiagaraDataInterfaceVectorCurve::CopyToInternal(UNiagaraDataInterface* Destination) const 
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceVectorCurve* DestinationVectorCurve = CastChecked<UNiagaraDataInterfaceVectorCurve>(Destination);
	DestinationVectorCurve->XCurve = XCurve;
	DestinationVectorCurve->YCurve = YCurve;
	DestinationVectorCurve->ZCurve = ZCurve;
#if WITH_EDITORONLY_DATA
	DestinationVectorCurve->UpdateLUT();

	if (!CompareLUTS(DestinationVectorCurve->ShaderLUT))
	{
		UE_LOGF(LogNiagara, Log, "Post CopyToInternal LUT generation is out of sync. Please investigate. %ls", *GetPathName());
	}
#endif
	return true;
}

bool UNiagaraDataInterfaceVectorCurve::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceVectorCurve* OtherVectorCurve = CastChecked<const UNiagaraDataInterfaceVectorCurve>(Other);
	return OtherVectorCurve->XCurve == XCurve &&
		OtherVectorCurve->YCurve == YCurve &&
		OtherVectorCurve->ZCurve == ZCurve;
}

void UNiagaraDataInterfaceVectorCurve::GetCurveData(TArray<FCurveData>& OutCurveData)
{
	OutCurveData.Add(FCurveData(&XCurve, TEXT("X"), FLinearColor(1.0f, 0.05f, 0.05f)));
	OutCurveData.Add(FCurveData(&YCurve, TEXT("Y"), FLinearColor(0.05f, 1.0f, 0.05f)));
	OutCurveData.Add(FCurveData(&ZCurve, TEXT("Z"), FLinearColor(0.1f, 0.2f, 1.0f)));
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceVectorCurve::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature Sig;
	Sig.Name = SampleCurveName;
	Sig.bMemberFunction = true;
	Sig.bRequiresContext = false;
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("VectorCurve")));
	Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("X")));
	Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Value")));
	//Sig.Owner = *GetName();

	OutFunctions.Add(Sig);
}

// build the shader function HLSL; function name is passed in, as it's defined per-DI; that way, configuration could change
// the HLSL in the spirit of a static switch
// TODO: need a way to identify each specific function here
// 
void UNiagaraDataInterfaceVectorCurve::SyncCurvesToAsset()
{
	if (UCurveLinearColor* ColorCurve = Cast<UCurveLinearColor>(CurveAsset))
	{
		XCurve = ColorCurve->FloatCurves[0];
		YCurve = ColorCurve->FloatCurves[1];
		ZCurve = ColorCurve->FloatCurves[2];
	}
	else if (UCurveVector* VecCurve = Cast<UCurveVector>(CurveAsset))
	{
		XCurve = VecCurve->FloatCurves[0];
		YCurve = VecCurve->FloatCurves[1];
		ZCurve = VecCurve->FloatCurves[2];
	}
	else if (UCurveFloat* FloatCurve = Cast<UCurveFloat>(CurveAsset))
	{
		XCurve = FloatCurve->FloatCurve;
		YCurve = FloatCurve->FloatCurve;
		ZCurve = FloatCurve->FloatCurve;
	}
}

bool UNiagaraDataInterfaceVectorCurve::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if (FunctionInfo.DefinitionName == SampleCurveName)
	{
		return true;
	}
	return false;
}
#endif

DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceVectorCurve, SampleCurve);
void UNiagaraDataInterfaceVectorCurve::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == SampleCurveName && BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3)
	{
		TCurveUseLUTBinder<NDI_FUNC_BINDER(UNiagaraDataInterfaceVectorCurve, SampleCurve)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else
	{
		UE_LOGF(LogNiagara, Display, "Could not find data interface external function in %ls.\n\tExpected Name: SampleVectorCurve  Actual Name: %ls\n\tExpected Inputs: 1  Actual Inputs: %i\n\tExpected Outputs: 3  Actual Outputs: %i",
			*GetPathNameSafe(this), *BindingInfo.Name.ToString(), BindingInfo.GetNumInputs(), BindingInfo.GetNumOutputs());
	}
}

template<>
FORCEINLINE_DEBUGGABLE FVector3f UNiagaraDataInterfaceVectorCurve::SampleCurveInternal<TIntegralConstant<bool, true>>(float X)
{
	const float RemappedX = FMath::Clamp(NormalizeTime(X) * LUTNumSamplesMinusOne, 0.0f, LUTNumSamplesMinusOne);
	const float PrevEntry = FMath::TruncToFloat(RemappedX);
	const float NextEntry = PrevEntry < LUTNumSamplesMinusOne ? PrevEntry + 1.0f : PrevEntry;
	const float Interp = RemappedX - PrevEntry;

	const int32 AIndex = (int32)(PrevEntry * (float)CurveLUTNumElems);
	const int32 BIndex = (int32)(NextEntry * (float)CurveLUTNumElems);
	const FVector3f A = FVector3f(ShaderLUT[AIndex], ShaderLUT[AIndex + 1], ShaderLUT[AIndex + 2]);
	const FVector3f B = FVector3f(ShaderLUT[BIndex], ShaderLUT[BIndex + 1], ShaderLUT[BIndex + 2]);
	return FMath::Lerp(A, B, Interp);
}

template<>
FORCEINLINE_DEBUGGABLE FVector3f UNiagaraDataInterfaceVectorCurve::SampleCurveInternal<TIntegralConstant<bool, false>>(float X)
{
	return FVector3f(XCurve.Eval(X), YCurve.Eval(X), ZCurve.Eval(X));
}

template<typename UseLUT>
void UNiagaraDataInterfaceVectorCurve::SampleCurve(FVectorVMExternalFunctionContext& Context)
{
	//TODO: Create some SIMDable optimized representation of the curve to do this faster.
	VectorVM::FExternalFuncInputHandler<float> XParam(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleZ(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const float X = XParam.GetAndAdvance();
		const FVector3f V = SampleCurveInternal<UseLUT>(X);
		*OutSampleX.GetDestAndAdvance() = V.X;
		*OutSampleY.GetDestAndAdvance() = V.Y;
		*OutSampleZ.GetDestAndAdvance() = V.Z;
	}
}

