// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceVectorField.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraEmitter.h"
#include "VectorField/VectorFieldStatic.h"
#include "VectorField/VectorFieldAnimated.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemImpl.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraComponent.h"
#include "NiagaraShaderParametersBuilder.h"
#include "RHIStaticStates.h"
#include "GlobalRenderResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceVectorField)

#if INTEL_ISPC
#include "NiagaraDataInterfaceVectorField.ispc.generated.h"
#endif

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceVectorField"

// Global VM function names, also used by the shaders code generation methods.
static const FName SampleVectorFieldName("SampleField");
static const FName LoadVectorFieldName("LoadField");
static const FName GetVectorFieldTilingAxesName("FieldTilingAxes");
static const FName GetVectorFieldDimensionsName("FieldDimensions");
static const FName GetVectorFieldBoundsName("FieldBounds");

#if INTEL_ISPC
static_assert(sizeof(ispc::FVector) == sizeof(FVector), "sizeof(ispc::FVector) != sizeof(FVector)");

#if !UE_BUILD_SHIPPING
bool GNiagaraVectorFieldUseIspc = NIAGARA_VECTOR_FIELD_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarNiagaraVectorFieldUseIspc(
	TEXT("fx.NiagaraVectorFieldUseIspc"),
	GNiagaraVectorFieldUseIspc,
	TEXT("When enabled VectorField will use ISPC for sampling if appropriate."),
	ECVF_Default
);
#endif

#endif

/*--------------------------------------------------------------------------------------------------------------------------*/

UNiagaraDataInterfaceVectorField::UNiagaraDataInterfaceVectorField(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, Field(nullptr)
	, bTileX(false)
	, bTileY(false)
	, bTileZ(false)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyVectorField());
	MarkRenderDataDirty();
}

/*--------------------------------------------------------------------------------------------------------------------------*/


#if WITH_EDITOR
void UNiagaraDataInterfaceVectorField::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);
	
	// Flush the rendering thread before making any changes to make sure the 
	// data read by the compute shader isn't subject to a race condition.
	// TODO(mv): Solve properly using something like a RT Proxy.
	//FlushRenderingCommands();
}

void UNiagaraDataInterfaceVectorField::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	MarkRenderDataDirty();
}
#endif //WITH_EDITOR


void UNiagaraDataInterfaceVectorField::PostLoad()
{
	Super::PostLoad();
	if (Field)
	{
		Field->ConditionalPostLoad();
	}

	MarkRenderDataDirty();
}

void UNiagaraDataInterfaceVectorField::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

/*--------------------------------------------------------------------------------------------------------------------------*/

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceVectorField::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SampleVectorFieldName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Vector Field")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Sample Point")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Sampled Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = LoadVectorFieldName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Vector Field")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index X")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index Y")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Index Z")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Loaded Value")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVectorFieldDimensionsName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Vector Field")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Dimensions")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVectorFieldTilingAxesName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Vector Field")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TilingAxes")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVectorFieldBoundsName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Vector Field")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("MinBounds")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("MaxBounds")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceVectorField, SampleVectorField);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceVectorField, LoadVectorField);

void UNiagaraDataInterfaceVectorField::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == SampleVectorFieldName && BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 3)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceVectorField, SampleVectorField)::Bind(this, OutFunc);
	}
	if (BindingInfo.Name == LoadVectorFieldName && BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 3)
	{
		NDI_FUNC_BINDER(UNiagaraDataInterfaceVectorField, LoadVectorField)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetVectorFieldDimensionsName && BindingInfo.GetNumInputs() == 0 && BindingInfo.GetNumOutputs() == 3)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceVectorField::GetFieldDimensions);
	}
	else if (BindingInfo.Name == GetVectorFieldBoundsName && BindingInfo.GetNumInputs() == 0 && BindingInfo.GetNumOutputs() == 6)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceVectorField::GetFieldBounds);
	}
	else if (BindingInfo.Name == GetVectorFieldTilingAxesName && BindingInfo.GetNumInputs() == 0 && BindingInfo.GetNumOutputs() == 3)
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceVectorField::GetFieldTilingAxes);
	}
}

bool UNiagaraDataInterfaceVectorField::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceVectorField* OtherTyped = CastChecked<const UNiagaraDataInterfaceVectorField>(Other);
	return OtherTyped->Field == Field 
		&& OtherTyped->bTileX == bTileX
		&& OtherTyped->bTileY == bTileY
		&& OtherTyped->bTileZ == bTileZ;
}

bool UNiagaraDataInterfaceVectorField::CanExecuteOnTarget(ENiagaraSimTarget Target) const
{
	return true;
}

/*--------------------------------------------------------------------------------------------------------------------------*/

#if WITH_EDITOR	
void UNiagaraDataInterfaceVectorField::GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	OutWarnings.Empty();
	OutInfo.Empty();

	UVectorFieldStatic* StaticVectorField = Cast<UVectorFieldStatic>(Field);
	UVectorFieldAnimated* AnimatedVectorField = Cast<UVectorFieldAnimated>(Field);

	// There are a few cases that we are trying to handle here:
	// 1) We have selected the DataInterface inline in the stack, in which case Component will be nullptr and we won't be in any of the ExposedParameters.
	// 2) We have selected the DataInterface in the User Parameters editor in the stack, in which case Component will be nullptr and we WILL be in the ExposedParameters.
	// 3) We have selected the DataInterface in the component panel. Component won't be nullptr in this case.
	TArray<FName> DIAliases;
	if (InComponent)
	{
		for (const UNiagaraDataInterface* DI : InComponent->GetOverrideParameters().GetDataInterfaces())
		{
			if (DI && (DI == this || DI->GetClass() == this->GetClass()))
			{
				if (DI == this || DI->Equals(this))
				{
					const FNiagaraVariableBase* Var = InComponent->GetOverrideParameters().FindVariableFromDataInterface(DI);
					if (Var)
					{
						DIAliases.AddUnique(Var->GetName());
					}
				}
			}
		}
	}
	else if (InAsset)
	{
		for (const UNiagaraDataInterface* DI : InAsset->GetExposedParameters().GetDataInterfaces())
		{
			if (DI && (DI == this || DI->GetClass() == this->GetClass()))
			{
				if (DI == this || DI->Equals(this))
				{
					const FNiagaraVariableBase* Var = InAsset->GetExposedParameters().FindVariableFromDataInterface(DI);
					if (Var)
					{
						DIAliases.AddUnique(Var->GetName());
					}
				}
			}
		}
	}
	
	// Filter through all the relevant CPU scripts
	bool bHasCPUFunctions = false;
	if (InAsset)
	{
		InAsset->ForEachScriptWithRuntimeData([&bHasCPUFunctions,&DIAliases,this](const UNiagaraScript* Script, const FNiagaraScriptRuntimeData* ScriptRuntimeData)
		{
			TConstArrayView<FNiagaraScriptResolvedDataInterfaceInfo> ResolvedDIs = ScriptRuntimeData->GetResolvedDataInterfaces();
			for (int32 Idx = 0; Idx < Script->GetVMExecutableData().DataInterfaceInfo.Num(); Idx++)
			{
				const auto& DIInfo = Script->GetVMExecutableData().DataInterfaceInfo[Idx];
				if (DIInfo.MatchesClass(GetClass()))
				{
					// Only the SampleField/LoadField functions are relevant for CPU access
					bool bHasCPUFunctionsReferenced = false;
					for (const FNiagaraFunctionSignature& Sig : DIInfo.RegisteredFunctions)
					{
						if (Sig.Name == SampleVectorFieldName || Sig.Name == LoadVectorFieldName)
						{
							bHasCPUFunctionsReferenced = true;
							break;
						}
					}

					if (bHasCPUFunctionsReferenced)
					{
						// We assume that if the properties match or we are referencing an external variable whose name is in the list of candidates found in the prior search, it's a valid match for us.
						if (ResolvedDIs.IsValidIndex(Idx) && ResolvedDIs[Idx].ResolvedDataInterface != nullptr &&
							(ResolvedDIs[Idx].ResolvedDataInterface->Equals(this) || DIAliases.Contains(ResolvedDIs[Idx].CompileName)))
						{
							FVersionedNiagaraEmitter OuterEmitter = Script->GetOuterEmitter();
							if (OuterEmitter.Emitter && OuterEmitter.GetEmitterData() && (OuterEmitter.GetEmitterData()->SimTarget == ENiagaraSimTarget::CPUSim || Script->IsSystemScript(Script->Usage)))
							{
								bHasCPUFunctions = true;
							}
						}
					}
				}
			}
		});
	}

	if (StaticVectorField != nullptr && !StaticVectorField->bAllowCPUAccess && bHasCPUFunctions)
	{
		FNiagaraDataInterfaceError CPUAccessNotAllowedError(
			FText::Format(
				LOCTEXT("CPUAccessNotAllowedError", "This Vector Field needs CPU access in order to be used properly.({0})"), 
				FText::FromString(StaticVectorField->GetName())
			),
			LOCTEXT("CPUAccessNotAllowedErrorSummary", "CPU access error"),
			FNiagaraDataInterfaceFix::CreateLambda(
				[=]()
				{
					StaticVectorField->SetCPUAccessEnabled();
					return true;
				}
			)
		);
		OutErrors.Add(CPUAccessNotAllowedError);
	}
	else if (AnimatedVectorField != nullptr)
	{
		FNiagaraDataInterfaceError AnimatedVectorFieldsNotSupportedError(
			LOCTEXT("AnimatedVectorFieldsNotSupportedErrorSummary", "Invalid vector field type."),
			LOCTEXT("AnimatedVectorFieldsNotSupportedError", "Animated vector fields are not supported."),
			nullptr
		);
		OutErrors.Add(AnimatedVectorFieldsNotSupportedError);
	}
	else if (Field == nullptr)
	{
		FNiagaraDataInterfaceError VectorFieldNotLoadedError(
			LOCTEXT("VectorFieldNotLoadedErrorSummary", "No Vector Field is loaded."),
			LOCTEXT("VectorFieldNotLoadedError", "No Vector Field is loaded."),
			nullptr
		);
		OutErrors.Add(VectorFieldNotLoadedError);
	}
}
#endif // WITH_EDITOR	

/*--------------------------------------------------------------------------------------------------------------------------*/

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceVectorField::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceVectorField::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	OutHLSL.Appendf(TEXT("float3 %s_TilingAxes;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("float3 %s_Dimensions;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("float3 %s_MinBounds;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("float3 %s_MaxBounds;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("Texture3D %s_Texture;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
	OutHLSL.Appendf(TEXT("SamplerState %s_Sampler;\n"), *ParamInfo.DataInterfaceHLSLSymbol);
}

bool UNiagaraDataInterfaceVectorField::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	TMap<FString, FStringFormatArg> FormatArgs;
	FormatArgs.Add(TEXT("FunctionName"),	FunctionInfo.InstanceName);
	FormatArgs.Add(TEXT("TilingAxesName"),	ParamInfo.DataInterfaceHLSLSymbol + TEXT("_TilingAxes"));
	FormatArgs.Add(TEXT("DimensionsName"),	ParamInfo.DataInterfaceHLSLSymbol + TEXT("_Dimensions"));
	FormatArgs.Add(TEXT("MinBoundsName"),	ParamInfo.DataInterfaceHLSLSymbol + TEXT("_MinBounds"));
	FormatArgs.Add(TEXT("MaxBoundsName"),	ParamInfo.DataInterfaceHLSLSymbol + TEXT("_MaxBounds"));
	FormatArgs.Add(TEXT("TextureName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_Texture"));
	FormatArgs.Add(TEXT("SamplerName"),		ParamInfo.DataInterfaceHLSLSymbol + TEXT("_Sampler"));

	if (FunctionInfo.DefinitionName == SampleVectorFieldName)
	{
		static const TCHAR* FunctionFormat = TEXT(R"(
			void {FunctionName}(float3 In_SamplePoint, out float3 Out_Sample)
			{
				float3 SamplePoint = (In_SamplePoint - {MinBoundsName}) / ({MaxBoundsName} - {MinBoundsName});
				Out_Sample = Texture3DSample({TextureName}, {SamplerName}, SamplePoint).xyz;
			}
		)");
		OutHLSL += FString::Format(FunctionFormat, FormatArgs);
		return true;
	}
	else if (FunctionInfo.DefinitionName == LoadVectorFieldName)
	{
		static const TCHAR* FunctionFormat = TEXT(R"(
			void {FunctionName}(int In_IndexX, int In_IndexY, int In_IndexZ, out float3 Out_Sample)
			{
				const int3 SampleIndex = int3(In_IndexX, In_IndexY, In_IndexZ);
				const int MipIndex = 0;
				Out_Sample = {TextureName}.Load(int4(SampleIndex, MipIndex)).xyz;
			}
		)");
		OutHLSL += FString::Format(FunctionFormat, FormatArgs);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVectorFieldTilingAxesName)
	{
		static const TCHAR* FunctionFormat = TEXT(R"(
			void {FunctionName}(out float3 Out_TilingAxes)
			{
				Out_TilingAxes = {TilingAxesName};
			}
		)");
		OutHLSL += FString::Format(FunctionFormat, FormatArgs);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVectorFieldDimensionsName)
	{
		static const TCHAR* FunctionFormat = TEXT(R"(
			void {FunctionName}(out float3 Out_Dimensions)
			{
				Out_Dimensions = {DimensionsName};
			}
		)");
		OutHLSL += FString::Format(FunctionFormat, FormatArgs);
		return true;
	}
	else if (FunctionInfo.DefinitionName == GetVectorFieldBoundsName)
	{
		static const TCHAR* FunctionFormat = TEXT(R"(
			void {FunctionName}(out float3 Out_MinBounds, out float3 Out_MaxBounds)
			{
				Out_MinBounds = {MinBoundsName};
				Out_MaxBounds = {MaxBoundsName};
			}
		)");
		OutHLSL += FString::Format(FunctionFormat, FormatArgs);
		return true;
	}
	return false;
}
#endif

void UNiagaraDataInterfaceVectorField::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceVectorField::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	// Different sampler states used by the computer shader to sample 3D vector field. 
	// Encoded as bitflags. To sample: 
	//     1st bit: X-axis tiling flag
	//     2nd bit: Y-axis tiling flag
	//     3rd bit: Z-axis tiling flag
	static FRHISamplerState* SamplerStates[8] = { nullptr };
	if (SamplerStates[0] == nullptr)
	{
		SamplerStates[0] = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		SamplerStates[1] = TStaticSamplerState<SF_Bilinear, AM_Wrap,  AM_Clamp, AM_Clamp>::GetRHI();
		SamplerStates[2] = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Wrap,  AM_Clamp>::GetRHI();
		SamplerStates[3] = TStaticSamplerState<SF_Bilinear, AM_Wrap,  AM_Wrap,  AM_Clamp>::GetRHI();
		SamplerStates[4] = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Wrap>::GetRHI();
		SamplerStates[5] = TStaticSamplerState<SF_Bilinear, AM_Wrap,  AM_Clamp, AM_Wrap>::GetRHI();
		SamplerStates[6] = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Wrap,  AM_Wrap>::GetRHI();
		SamplerStates[7] = TStaticSamplerState<SF_Bilinear, AM_Wrap,  AM_Wrap,  AM_Wrap>::GetRHI();
	}

	FNiagaraDataInterfaceProxyVectorField& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyVectorField>();

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->TilingAxes	= (FVector3f)DIProxy.GetTilingAxes();
	ShaderParameters->Dimensions	= (FVector3f)DIProxy.Dimensions;
	ShaderParameters->MinBounds		= (FVector3f)DIProxy.MinBounds;
	ShaderParameters->MaxBounds		= (FVector3f)DIProxy.MaxBounds;
	ShaderParameters->Texture		= DIProxy.TextureRHI.IsValid() ? DIProxy.TextureRHI : GBlackVolumeTexture->TextureRHI;
	ShaderParameters->Sampler		= SamplerStates[int(DIProxy.bTileX) + 2 * int(DIProxy.bTileY) + 4 * int(DIProxy.bTileZ)];
}

/*--------------------------------------------------------------------------------------------------------------------------*/

void UNiagaraDataInterfaceVectorField::GetFieldTilingAxes(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FExternalFuncRegisterHandler<float> OutSizeX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSizeY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSizeZ(Context);

	const FVector3f Tilings(GetTilingAxes());
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*OutSizeX.GetDest() = Tilings.X;
		*OutSizeY.GetDest() = Tilings.Y;
		*OutSizeZ.GetDest() = Tilings.Z;

		OutSizeX.Advance();
		OutSizeY.Advance();
		OutSizeZ.Advance();
	}
}

void UNiagaraDataInterfaceVectorField::GetFieldDimensions(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FExternalFuncRegisterHandler<float> OutSizeX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSizeY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSizeZ(Context);

	const FVector3f Dim(GetDimensions());
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*OutSizeX.GetDest() = Dim.X;
		*OutSizeY.GetDest() = Dim.Y;
		*OutSizeZ.GetDest() = Dim.Z;

		OutSizeX.Advance();
		OutSizeY.Advance();
		OutSizeZ.Advance();
	}
}

void UNiagaraDataInterfaceVectorField::GetFieldBounds(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FExternalFuncRegisterHandler<float> OutMinX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutMinY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutMinZ(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutMaxX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutMaxY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutMaxZ(Context);

	const FVector3f MinBounds(GetMinBounds());
	const FVector3f MaxBounds(GetMaxBounds());
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*OutMinX.GetDest() = MinBounds.X;
		*OutMinY.GetDest() = MinBounds.Y;
		*OutMinZ.GetDest() = MinBounds.Z;
		*OutMaxX.GetDest() = MaxBounds.X;
		*OutMaxY.GetDest() = MaxBounds.Y;
		*OutMaxZ.GetDest() = MaxBounds.Z;

		OutMinX.Advance();
		OutMinY.Advance();
		OutMinZ.Advance();
		OutMaxX.Advance();
		OutMaxY.Advance();
		OutMaxZ.Advance();
	}
}

void UNiagaraDataInterfaceVectorField::SampleVectorField(FVectorVMExternalFunctionContext& Context)
{
	// Input arguments...
	VectorVM::FExternalFuncInputHandler<float> XParam(Context);
	VectorVM::FExternalFuncInputHandler<float> YParam(Context);
	VectorVM::FExternalFuncInputHandler<float> ZParam(Context);

	// Outputs...
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleZ(Context);

	UVectorFieldStatic* StaticVectorField = Cast<UVectorFieldStatic>(Field);
	UVectorFieldAnimated* AnimatedVectorField = Cast<UVectorFieldAnimated>(Field);

	bool bSuccess = false;

	if (StaticVectorField != nullptr && StaticVectorField->bAllowCPUAccess)
	{
		const FVector TilingAxes = FVector(bTileX ? 1.0f : 0.0f, bTileY ? 1.0f : 0.0f, bTileZ ? 1.0f : 0.0f);

		const uint32 SizeX = (uint32)StaticVectorField->SizeX;
		const uint32 SizeY = (uint32)StaticVectorField->SizeY;
		const uint32 SizeZ = (uint32)StaticVectorField->SizeZ;
		const FVector Size(SizeX, SizeY, SizeZ);

		const FVector MinBounds(StaticVectorField->Bounds.Min.X, StaticVectorField->Bounds.Min.Y, StaticVectorField->Bounds.Min.Z);
		const FVector BoundSize = StaticVectorField->Bounds.GetSize();

		if (ensure(StaticVectorField->HasCPUData() && FMath::Min3(SizeX, SizeY, SizeZ) > 0 && BoundSize.GetMin() > SMALL_NUMBER))
		{
			const FVector OneOverBoundSize(FVector::OneVector / BoundSize);

#if INTEL_ISPC && VECTOR_FIELD_DATA_AS_HALF
			if (GNiagaraVectorFieldUseIspc)
			{
				constexpr int32 DataSamplesPerVector = sizeof(ispc::FHalfVector) / sizeof(FFloat16);

				TConstArrayView<FFloat16> FieldSamplesData = StaticVectorField->ReadCPUData();
				TConstArrayView<ispc::FHalfVector> FieldSamples(reinterpret_cast<const ispc::FHalfVector*>(FieldSamplesData.GetData()), FieldSamplesData.Num() / DataSamplesPerVector);

				ispc::SampleVectorField(XParam.GetDest(), YParam.GetDest(), ZParam.GetDest(),
					XParam.IsConstant(), YParam.IsConstant(), ZParam.IsConstant(),
					OutSampleX.GetDest(), OutSampleY.GetDest(), OutSampleZ.GetDest(),
					OutSampleX.IsValid(), OutSampleY.IsValid(), OutSampleZ.IsValid(),
					FieldSamples.GetData(), FieldSamples.Num() - 1, (ispc::FVector&)MinBounds, (ispc::FVector&)OneOverBoundSize,
					(ispc::FVector&)Size, (ispc::FVector&)TilingAxes, Context.GetNumInstances());
			}
			else
#endif
			{
				for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
				{
					// Position in Volume Space
					FVector Pos(XParam.Get(), YParam.Get(), ZParam.Get());

					// Normalize position
					Pos = (Pos - MinBounds) * OneOverBoundSize;

					// Scaled position
					Pos = Pos * Size;

					// Offset by half a cell size due to sample being in the center of its cell
					Pos = Pos - FVector(0.5f, 0.5f, 0.5f);

					const FVector3f V(StaticVectorField->FilteredSample(Pos, TilingAxes));

					// Write final output...
					*OutSampleX.GetDest() = V.X;
					*OutSampleY.GetDest() = V.Y;
					*OutSampleZ.GetDest() = V.Z;

					XParam.Advance();
					YParam.Advance();
					ZParam.Advance();
					OutSampleX.Advance();
					OutSampleY.Advance();
					OutSampleZ.Advance();
				}
			}

			bSuccess = true;
		}
	}

	if (!bSuccess)
	{
		// TODO(mv): Add warnings?
		if (StaticVectorField != nullptr && !StaticVectorField->bAllowCPUAccess)
		{
			// No access to static vector data
		}
		else if (AnimatedVectorField != nullptr)
		{
			// Animated vector field not supported
		}
		else if (Field == nullptr)
		{
			// Vector field not loaded
		}

		// Set the default vector to positive X axis corresponding to a velocity of 100 cm/s
		// Rationale: Setting to the zero vector can be visually confusing and likely to cause problems elsewhere
		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			*OutSampleX.GetDest() = 0.0f;
			*OutSampleY.GetDest() = 0.0f;
			*OutSampleZ.GetDest() = 0.0f;

			XParam.Advance();
			YParam.Advance();
			ZParam.Advance();
			OutSampleX.Advance();
			OutSampleY.Advance();
			OutSampleZ.Advance();
		}
	}
}

void UNiagaraDataInterfaceVectorField::LoadVectorField(FVectorVMExternalFunctionContext& Context)
{
	// Input arguments...
	VectorVM::FExternalFuncInputHandler<int32> XParam(Context);
	VectorVM::FExternalFuncInputHandler<int32> YParam(Context);
	VectorVM::FExternalFuncInputHandler<int32> ZParam(Context);

	// Outputs...
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutSampleZ(Context);

	UVectorFieldStatic* StaticVectorField = Cast<UVectorFieldStatic>(Field);
	UVectorFieldAnimated* AnimatedVectorField = Cast<UVectorFieldAnimated>(Field);

	bool bSuccess = false;

	if (StaticVectorField != nullptr && StaticVectorField->bAllowCPUAccess)
	{
		const FIntVector MinSampleIndex(0, 0, 0);
		const FIntVector MaxSampleIndex(StaticVectorField->SizeX - 1, StaticVectorField->SizeY - 1, StaticVectorField->SizeZ - 1);

		if (ensure(StaticVectorField->HasCPUData() && MaxSampleIndex.GetMin() >= 0))
		{
			for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
			{
				// follow d3d convention of returning 0 if index is out of bounds
				FVector3f Result = FVector3f(0.0f, 0.0f, 0.0f);

				const FIntVector SampleIndex(XParam.Get(), YParam.Get(), ZParam.Get());
				const FIntVector ClampedIndex(
					FMath::Clamp(SampleIndex.X, MinSampleIndex.X, MaxSampleIndex.X),
					FMath::Clamp(SampleIndex.Y, MinSampleIndex.Y, MaxSampleIndex.Y),
					FMath::Clamp(SampleIndex.Z, MinSampleIndex.Z, MaxSampleIndex.Z)
				);

				if (ClampedIndex == SampleIndex)
				{
					Result = FVector3f(StaticVectorField->Sample(SampleIndex));
				}

				// Write final output...
				*OutSampleX.GetDest() = Result.X;
				*OutSampleY.GetDest() = Result.Y;
				*OutSampleZ.GetDest() = Result.Z;

				XParam.Advance();
				YParam.Advance();
				ZParam.Advance();
				OutSampleX.Advance();
				OutSampleY.Advance();
				OutSampleZ.Advance();
			}
		}
	}

	if (!bSuccess)
	{
		// TODO(mv): Add warnings?
		if (StaticVectorField != nullptr && !StaticVectorField->bAllowCPUAccess)
		{
			// No access to static vector data
		}
		else if (AnimatedVectorField != nullptr)
		{
			// Animated vector field not supported
		}
		else if (Field == nullptr)
		{
			// Vector field not loaded
		}

		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			*OutSampleX.GetDest() = 0.0f;
			*OutSampleY.GetDest() = 0.0f;
			*OutSampleZ.GetDest() = 0.0f;

			XParam.Advance();
			YParam.Advance();
			ZParam.Advance();
			OutSampleX.Advance();
			OutSampleY.Advance();
			OutSampleZ.Advance();
		}
	}
}

/*--------------------------------------------------------------------------------------------------------------------------*/

FVector UNiagaraDataInterfaceVectorField::GetTilingAxes() const
{
	return FVector(float(bTileX), float(bTileY), float(bTileZ));
}

FVector UNiagaraDataInterfaceVectorField::GetDimensions() const
{
	UVectorFieldStatic* StaticVectorField = Cast<UVectorFieldStatic>(Field);
	if (StaticVectorField)
	{
		return FVector(StaticVectorField->SizeX, StaticVectorField->SizeY, StaticVectorField->SizeZ);
	}
	return FVector{ 1.0f, 1.0f, 1.0f }; // Matches GBlackVolumeTexture
}

FVector UNiagaraDataInterfaceVectorField::GetMinBounds() const
{
	UVectorFieldStatic* StaticVectorField = Cast<UVectorFieldStatic>(Field);
	if (StaticVectorField)
	{
		return StaticVectorField->Bounds.Min;
	}
	return FVector{-1.0f, -1.0f, -1.0f};
}

FVector UNiagaraDataInterfaceVectorField::GetMaxBounds() const
{
	UVectorFieldStatic* StaticVectorField = Cast<UVectorFieldStatic>(Field);
	if (StaticVectorField)
	{
		return StaticVectorField->Bounds.Max;
	}
	return FVector{1.0f, 1.0f, 1.0f};
}

/*--------------------------------------------------------------------------------------------------------------------------*/

bool UNiagaraDataInterfaceVectorField::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceVectorField* OtherTyped = CastChecked<UNiagaraDataInterfaceVectorField>(Destination);
	OtherTyped->Field = Field;
	OtherTyped->bTileX = bTileX;
	OtherTyped->bTileY = bTileY;
	OtherTyped->bTileZ = bTileZ;

	OtherTyped->MarkRenderDataDirty();
	return true;
}

void UNiagaraDataInterfaceVectorField::PushToRenderThreadImpl()
{
	FNiagaraDataInterfaceProxyVectorField* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyVectorField>();

	FVector RT_Dimensions = GetDimensions();
	FVector RT_MinBounds = GetMinBounds();
	FVector RT_MaxBounds = GetMaxBounds();
	bool RT_bTileX = bTileX;
	bool RT_bTileY = bTileY;
	bool RT_bTileZ = bTileZ;

	FVectorFieldTextureAccessor TextureAccessor(Field);

	// Push Updates to Proxy.
	ENQUEUE_RENDER_COMMAND(FUpdateDIVectorField)(
		[RT_Proxy, RT_Dimensions, RT_MinBounds, RT_MaxBounds, RT_bTileX, RT_bTileY, RT_bTileZ, TextureAccessor](FRHICommandListImmediate& RHICmdList)
	{
		RT_Proxy->bTileX = RT_bTileX;
		RT_Proxy->bTileY = RT_bTileY;
		RT_Proxy->bTileZ = RT_bTileZ;
		RT_Proxy->Dimensions = RT_Dimensions;
		RT_Proxy->MinBounds = RT_MinBounds;
		RT_Proxy->MaxBounds = RT_MaxBounds;
		RT_Proxy->TextureRHI = TextureAccessor.GetTexture();
	});
}

#undef LOCTEXT_NAMESPACE

