// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "NiagaraCommon.isph"

struct FHalfVector
{
	int16 V[3];
};

inline varying FVector ReadHalfVector(const uniform FHalfVector Data[], int DataSize, int SampleIndex)
{
	#pragma ignore warning(perf)
	const FHalfVector Sample = Data[SampleIndex >= 0 && SampleIndex <= DataSize ? SampleIndex : 0];

	return SetVector((FReal)(half_to_float_fast(Sample.V[0])), (FReal)(half_to_float_fast(Sample.V[1])), (FReal)(half_to_float_fast(Sample.V[2])));
}

export void SampleVectorField(const uniform float XParam[], const uniform float YParam[], const uniform float ZParam[],
							const uniform bool bXParamConst, const uniform bool bYParamConst, const uniform bool bZParamConst,
							uniform float OutSampleX[], uniform float OutSampleY[], uniform float OutSampleZ[],
							const uniform bool bOutputXConst, const uniform bool bOutputYConst, const uniform bool bOutputZConst,
							const uniform FHalfVector Data[], const uniform int DataSize, const uniform FVector &MinBounds, const uniform FVector &OneOverBoundSize,
							const uniform FVector &Size, const uniform FVector &TilingAxes, const uniform int NumInstances)
{
	const uniform FVector SizeMinusOne = Size - OneVector;
	const uniform FVector OneOverSize = ONE / Size;

	foreach(InstanceIdx = 0 ... NumInstances)
	{
		// Position in Volume Space
		FVector Pos = SetVector(
			(FReal)(LoadFloat(XParam, InstanceIdx, bXParamConst)),
			(FReal)(LoadFloat(YParam, InstanceIdx, bYParamConst)),
			(FReal)(LoadFloat(ZParam, InstanceIdx, bZParamConst)));

		// Normalize position
		Pos = (Pos - MinBounds) * OneOverBoundSize;

		// Scaled position
		Pos = Pos * Size;

		// Offset by half a cell size due to sample being in the center of its cell
		Pos = Pos - HalfVector;

		//
		FVector Index0 = VectorFloor(Pos);
		FVector Index1 = Index0 + OneVector;

		//
		const FVector Fraction = Pos - Index0;

		Index0 = Index0 - TilingAxes*VectorFloor(Index0 * OneOverSize)*Size;
		Index1 = Index1 - TilingAxes*VectorFloor(Index1 * OneOverSize)*Size;

		Index0 = VectorClamp(Index0, ZeroVector, SizeMinusOne);
		Index1 = VectorClamp(Index1, ZeroVector, SizeMinusOne);

		// Sample by regular trilinear interpolation:

		// Fetch corners & Blend axes
		const FVector V000 = ReadHalfVector(Data, DataSize, (int)(Index0.V[0] + Size.V[0] * Index0.V[1] + Size.V[0] * Size.V[1] * Index0.V[2]));
		const FVector V100 = ReadHalfVector(Data, DataSize, (int)(Index1.V[0] + Size.V[0] * Index0.V[1] + Size.V[0] * Size.V[1] * Index0.V[2]));

		const FVector V00 = VectorLerp(V000, V100, Fraction.V[0]);

		const FVector V010 = ReadHalfVector(Data, DataSize, (int)(Index0.V[0] + Size.V[0] * Index1.V[1] + Size.V[0] * Size.V[1] * Index0.V[2]));
		const FVector V110 = ReadHalfVector(Data, DataSize, (int)(Index1.V[0] + Size.V[0] * Index1.V[1] + Size.V[0] * Size.V[1] * Index0.V[2]));

		const FVector V10 = VectorLerp(V010, V110, Fraction.V[0]);
		const FVector V0 = VectorLerp(V00, V10, Fraction.V[1]);

		const FVector V001 = ReadHalfVector(Data, DataSize, (int)(Index0.V[0] + Size.V[0] * Index0.V[1] + Size.V[0] * Size.V[1] * Index1.V[2]));
		const FVector V101 = ReadHalfVector(Data, DataSize, (int)(Index1.V[0] + Size.V[0] * Index0.V[1] + Size.V[0] * Size.V[1] * Index1.V[2]));

		const FVector V01 = VectorLerp(V001, V101, Fraction.V[0]);

		const FVector V011 = ReadHalfVector(Data, DataSize, (int)(Index0.V[0] + Size.V[0] * Index1.V[1] + Size.V[0] * Size.V[1] * Index1.V[2]));
		const FVector V111 = ReadHalfVector(Data, DataSize, (int)(Index1.V[0] + Size.V[0] * Index1.V[1] + Size.V[0] * Size.V[1] * Index1.V[2]));

		const FVector V11 = VectorLerp(V011, V111, Fraction.V[0]);
		const FVector V1 = VectorLerp(V01, V11, Fraction.V[1]);
		const FVector V = VectorLerp(V0, V1, Fraction.V[2]);

		// Write final output...
		if (bOutputXConst)
		{
			OutSampleX[InstanceIdx] = V.V[0];
		}

		if (bOutputYConst)
		{
			OutSampleY[InstanceIdx] = V.V[1];
		}

		if (bOutputZConst)
		{
			OutSampleZ[InstanceIdx] = V.V[2];
		}
	}
}
