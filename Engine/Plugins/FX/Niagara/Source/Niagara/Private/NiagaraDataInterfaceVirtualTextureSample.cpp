// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceVirtualTextureSample.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "EngineModule.h"
#include "Engine/TextureRenderTarget2D.h"
#include "NiagaraRenderer.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "RenderGraphBuilder.h"
#include "GlobalRenderResources.h"
#include "RHIStaticStates.h"
#include "RenderGraphUtils.h"
#include "ShaderCompilerCore.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceVirtualTextureSample)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterfaceVirtualTextureSample"

const TCHAR* UNiagaraDataInterfaceVirtualTextureSample::TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceVirtualTextureSampleTemplate.ush");
const FName UNiagaraDataInterfaceVirtualTextureSample::SampleTexture2DName(TEXT("SampleTexture2D"));
const FName UNiagaraDataInterfaceVirtualTextureSample::GetTextureDimensionsName(TEXT("GetTextureDimensions"));
const FName UNiagaraDataInterfaceVirtualTextureSample::GetNumMipLevelsName(TEXT("GetNumMipLevels"));

struct FNDIVirtualTextureInstanceData_GameThread
{
	TWeakObjectPtr<UTexture> CurrentTexture = nullptr;
	FIntPoint CurrentTextureSize = FIntPoint::ZeroValue;
	int32 CurrentTextureMipLevels = 0;
	FNiagaraParameterDirectBinding<UObject*> UserParamBinding;
};

struct FNDIVirtualTextureSampleInstanceData_RenderThread
{
	FVirtualTexture2DResource *VirtualTextureResource = nullptr;
	uint8					VirtualTextureResource_SRGB = 0;	
	FIntPoint				TextureSize = FIntPoint(0, 0);
	int32					MipLevels = 0;	
};

struct FNiagaraDataInterfaceProxyVirtualTextureSample : public FNiagaraDataInterfaceProxy
{
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { checkNoEntry(); }
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

	TMap<FNiagaraSystemInstanceID, FNDIVirtualTextureSampleInstanceData_RenderThread> InstanceData_RT;
};

UNiagaraDataInterfaceVirtualTextureSample::UNiagaraDataInterfaceVirtualTextureSample(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, Texture(nullptr)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyVirtualTextureSample());

	FNiagaraTypeDefinition Def(UTexture::StaticClass());
	TextureUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceVirtualTextureSample::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceVirtualTextureSample::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceVirtualTextureSample* DestinationTexture = CastChecked<UNiagaraDataInterfaceVirtualTextureSample>(Destination);
	DestinationTexture->Texture = Texture;
	DestinationTexture->TextureUserParameter = TextureUserParameter;

	return true;
}

bool UNiagaraDataInterfaceVirtualTextureSample::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceVirtualTextureSample* OtherTexture = CastChecked<const UNiagaraDataInterfaceVirtualTextureSample>(Other);
	return
		OtherTexture->Texture == Texture &&
		OtherTexture->TextureUserParameter == TextureUserParameter;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceVirtualTextureSample::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	FNiagaraFunctionSignature DefaultGpuSig;
	DefaultGpuSig.bExperimental = true;
	DefaultGpuSig.bMemberFunction = true;
	DefaultGpuSig.bRequiresContext = false;
	DefaultGpuSig.bSupportsCPU = false;
	DefaultGpuSig.bSupportsGPU = true;
	DefaultGpuSig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("VirtualTexture"));	

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.Add_GetRef(DefaultGpuSig);
		Sig.Name = SampleTexture2DName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("UV"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("VirtualTextureSampleTexture2DDesc", "Sample supplied mip level from input 2d virtual texture at the specified UV coordinates. The UV origin (0,0) is in the upper left hand corner of the image."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = GetTextureDimensionsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("VirtualTexture"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Dimensions2D"));
		Sig.SetDescription(LOCTEXT("VirtualTextureDimsDesc", "Get the dimensions of the provided Mip level for the virtual texture."));		
	}
	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = GetNumMipLevelsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("VirtualTexture"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumMipLevels"));
		Sig.SetDescription(LOCTEXT("GetNumMipLevelsDesc", "Get the number of Mip Levels for the virtual texture."));		
	}
}
#endif

void UNiagaraDataInterfaceVirtualTextureSample::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == GetTextureDimensionsName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 2);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceVirtualTextureSample::VMGetTextureDimensions);
	}
	else if (BindingInfo.Name == GetNumMipLevelsName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceVirtualTextureSample::VMGetNumMipLevels);
	}
}

int32 UNiagaraDataInterfaceVirtualTextureSample::PerInstanceDataSize() const
{
	return sizeof(FNDIVirtualTextureInstanceData_GameThread);
}

bool UNiagaraDataInterfaceVirtualTextureSample::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIVirtualTextureInstanceData_GameThread* InstanceData = new (PerInstanceData) FNDIVirtualTextureInstanceData_GameThread();
	InstanceData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), TextureUserParameter.Parameter);
	return true;
}

void UNiagaraDataInterfaceVirtualTextureSample::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIVirtualTextureInstanceData_GameThread* InstanceData = static_cast<FNDIVirtualTextureInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FNDIVirtualTextureInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(NDIVirtualTexture_RemoveInstance)
	(
		[RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxyVirtualTextureSample>(), RT_InstanceID=SystemInstance->GetId()](FRHICommandListImmediate&)
		{
			RT_Proxy->InstanceData_RT.Remove(RT_InstanceID);
		}
	);
}

bool UNiagaraDataInterfaceVirtualTextureSample::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FNDIVirtualTextureInstanceData_GameThread* InstanceData = static_cast<FNDIVirtualTextureInstanceData_GameThread*>(PerInstanceData);

	UTexture* CurrentTexture = InstanceData->UserParamBinding.GetValueOrDefault<UTexture>(Texture);
	FIntPoint CurrentTextureSize = FIntPoint::ZeroValue;
	int32 CurrentTextureMipLevels = 0;

	bool IsValid = true;

	if (CurrentTexture != nullptr)
	{
		CurrentTextureSize.X = int32(CurrentTexture->GetSurfaceWidth());
		CurrentTextureSize.Y = int32(CurrentTexture->GetSurfaceHeight());
		if (UTexture2D* CurrentTexture2D = Cast<UTexture2D>(CurrentTexture))
		{
			CurrentTextureMipLevels = CurrentTexture2D->GetNumMips();
		}
		else if (UTextureRenderTarget2D* CurrentTexture2DRT = Cast<UTextureRenderTarget2D>(CurrentTexture))
		{
			CurrentTextureMipLevels = CurrentTexture2DRT->GetNumMips();
		}
		else
		{
			CurrentTextureMipLevels = 1;
		}

		// error if the texture isn't virtual
		if (CurrentTexture->VirtualTextureStreaming == false)
		{			
			IsValid = false;		
		}

		// error if texture isn't valid
		if (!CurrentTexture->IsValidLowLevel())
		{		
			IsValid = false;			
		}
	}
	else
	{		
		IsValid = false;		
	}

	IsValid = IsValid && CurrentTexture->IsFullyStreamedIn() && !CurrentTexture->HasPendingInitOrStreaming() && !CurrentTexture->IsCompiling();

	if (IsValid)
	{
		InstanceData->CurrentTexture = CurrentTexture;
		InstanceData->CurrentTextureSize = CurrentTextureSize;
		InstanceData->CurrentTextureMipLevels = CurrentTextureMipLevels;
	
		ENQUEUE_RENDER_COMMAND(NDIVirtualTexture_UpdateInstance)
		(
			[RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxyVirtualTextureSample>(), RT_InstanceID=SystemInstance->GetId(), RT_Texture=CurrentTexture, RT_TextureSize=CurrentTextureSize, RT_MipLevels=CurrentTextureMipLevels](FRHICommandListImmediate &RHI)
			{
				FNDIVirtualTextureSampleInstanceData_RenderThread& InstanceData = RT_Proxy->InstanceData_RT.FindOrAdd(RT_InstanceID);
				
				InstanceData.TextureSize = RT_TextureSize;
				InstanceData.MipLevels = RT_MipLevels;
				InstanceData.VirtualTextureResource = static_cast<FVirtualTexture2DResource*>(RT_Texture->GetResource());
				if (InstanceData.VirtualTextureResource && InstanceData.VirtualTextureResource->IsInitialized())
				{
					// We need to acquire here as it's not valid to acquire during set shader parameters and will cause a crash
					InstanceData.VirtualTextureResource->AcquireAllocatedVT();
				}

				InstanceData.VirtualTextureResource_SRGB = RT_Texture->SRGB;				
			}
		);
	}
	else
	{
		InstanceData->CurrentTexture = nullptr;
		InstanceData->CurrentTextureSize = FIntPoint(0,0);
		InstanceData->CurrentTextureMipLevels = 0;

		ENQUEUE_RENDER_COMMAND(NDIVirtualTexture_UpdateInstance)
		(
			[RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxyVirtualTextureSample>(), RT_InstanceID=SystemInstance->GetId()](FRHICommandListImmediate &RHI)
			{
				FNDIVirtualTextureSampleInstanceData_RenderThread& InstanceData = RT_Proxy->InstanceData_RT.FindOrAdd(RT_InstanceID);
				
				InstanceData.TextureSize = 0;
				InstanceData.MipLevels = 0;
				InstanceData.VirtualTextureResource = nullptr;
				InstanceData.VirtualTextureResource_SRGB = false;				
			}
		);
	}

	return false;
}

void UNiagaraDataInterfaceVirtualTextureSample::VMGetTextureDimensions(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIVirtualTextureInstanceData_GameThread> InstData(Context);
	FNDIInputParam<int32>		InMipLevel(Context);
	FNDIOutputParam<FVector2f>	OutSize(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 MipLevel = InMipLevel.GetAndAdvance();
		const FVector2f TextureSize(
			float(FMath::Max(InstData->CurrentTextureSize.X >> MipLevel, 1)),
			float(FMath::Max(InstData->CurrentTextureSize.Y >> MipLevel, 1))
		);
		OutSize.SetAndAdvance(TextureSize);
	}
}

void UNiagaraDataInterfaceVirtualTextureSample::VMGetNumMipLevels(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIVirtualTextureInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<int32> OutNumMipLevels(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutNumMipLevels.SetAndAdvance(InstData->CurrentTextureMipLevels);
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceVirtualTextureSample::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceVirtualTextureSample::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceVirtualTextureSample::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	static const TSet<FName> ValidGpuFunctions =
	{			
		SampleTexture2DName,
		GetTextureDimensionsName,
		GetNumMipLevelsName,
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

#endif

void UNiagaraDataInterfaceVirtualTextureSample::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceVirtualTextureSample::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyVirtualTextureSample& TextureProxy = Context.GetProxy<FNiagaraDataInterfaceProxyVirtualTextureSample>();
	FNDIVirtualTextureSampleInstanceData_RenderThread* InstanceData = TextureProxy.InstanceData_RT.Find(Context.GetSystemInstanceID());
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	bool IsValid = false;

	FShaderParameters* Parameters = Context.GetParameterNestedStruct<FShaderParameters>();
	if (InstanceData)
	{
		FVirtualTexture2DResource* VirtualTextureResource = InstanceData->VirtualTextureResource;
		if (VirtualTextureResource && VirtualTextureResource->IsInitialized())
		{
			if (IAllocatedVirtualTexture* AllocatedVT = VirtualTextureResource->GetAllocatedVT())
			{
				FRHIShaderResourceView* InPhysicalTexture = AllocatedVT->GetPhysicalTextureSRV((uint32)0, InstanceData->VirtualTextureResource_SRGB == 1);

				if (InPhysicalTexture && InPhysicalTexture->IsValid())
				{
					Parameters->TextureSize = InstanceData->TextureSize;
					Parameters->MipLevels = InstanceData->MipLevels;

					Parameters->PhysicalTexture = InPhysicalTexture;
					Parameters->TextureSampler = VirtualTextureResource->SamplerStateRHI.GetReference();

					Parameters->PageTableTexture0 = AllocatedVT->GetPageTableTexture(0);
					Parameters->PageTableTexture1 = AllocatedVT->GetNumPageTableTextures() > 1 ? AllocatedVT->GetPageTableTexture(1) : GBlackTexture->TextureRHI.GetReference();

					if (Parameters->TextureSampler->IsValid() && Parameters->PageTableTexture0->IsValid() && Parameters->PageTableTexture1->IsValid())
					{
						FUintVector4 VTPackedPageTableUniform[2];
						FUintVector4 VTPackedUniform;

						AllocatedVT->GetPackedPageTableUniform(VTPackedPageTableUniform);
						AllocatedVT->GetPackedUniform(&VTPackedUniform, (uint32)0);

						Parameters->VTPackedPageTableUniform[0] = VTPackedPageTableUniform[0];
						Parameters->VTPackedPageTableUniform[1] = VTPackedPageTableUniform[1];
						Parameters->VTPackedUniform = VTPackedUniform;

						IsValid = true;
					}
				}
			}
		}
	}
	
	if (!IsValid)
	{
		Parameters->TextureSize = FIntPoint(0, 0);
		Parameters->MipLevels = 0;
		Parameters->TextureSampler = TStaticSamplerState<SF_Point>::GetRHI();		
		Parameters->VTPackedPageTableUniform[0] = FUintVector4(0, 0, 0, 0);
		Parameters->VTPackedPageTableUniform[1] = FUintVector4(0, 0, 0, 0);
		Parameters->VTPackedUniform = FUintVector4();
		Parameters->PhysicalTexture = GBlackTextureWithSRV->ShaderResourceViewRHI;
		Parameters->PageTableTexture0 = GBlackTexture->TextureRHI;
		Parameters->PageTableTexture1 = GBlackTexture->TextureRHI;		
	}
}

#undef LOCTEXT_NAMESPACE

