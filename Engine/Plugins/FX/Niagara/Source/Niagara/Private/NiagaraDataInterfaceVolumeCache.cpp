// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataInterfaceVolumeCache.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystem.h"
#include "RenderGraphBuilder.h"
#include "RHIStaticStates.h"
#include "GlobalRenderResources.h"
#include "VolumeCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceVolumeCache)


#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceVolumeCache"

const TCHAR* UNiagaraDataInterfaceVolumeCache::TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceVolumeCache.ush");
const FName UNiagaraDataInterfaceVolumeCache::SetFrameName("SetFrame");
const FName UNiagaraDataInterfaceVolumeCache::ReadFileName("ReadFile");
const FName UNiagaraDataInterfaceVolumeCache::GetNumCellsName("GetNumCells");
const FName UNiagaraDataInterfaceVolumeCache::IndexToUnitName("IndexToUnit");
const FName UNiagaraDataInterfaceVolumeCache::SampleCurrentFrameValueName(TEXT("SampleCurrentFrameValue"));
const FName UNiagaraDataInterfaceVolumeCache::GetCurrentFrameValue(TEXT("GetCurrentFrameValue"));
const FName UNiagaraDataInterfaceVolumeCache::GetCurrentFrameNumCells(TEXT("GetCurrentFrameNumCells"));

struct FVolumeCacheInstanceData_RenderThread
{
	int						CurrFrame = 0;
	bool					ReadFile = false;

	FSamplerStateRHIRef		SamplerStateRHI = nullptr;
	FTextureReferenceRHIRef	TextureReferenceRHI = nullptr;
	FTextureRHIRef			ResolvedTextureRHI = nullptr;
	FVector3f				TextureSize = FVector3f::ZeroVector;
	
	TSharedPtr<FVolumeCacheData> VolumeCacheData = nullptr;
};

struct FVolumeCacheInstanceData_GameThread
{
	int CurrFrame = 0;
	int PrevFrame = -1;

	FIntVector NumCells = FIntVector::ZeroValue;

	bool ReadFile = false;		
};

struct FNiagaraDataInterfaceVolumeCacheProxy : public FNiagaraDataInterfaceProxy
{	
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FVolumeCacheInstanceData_GameThread); }

	virtual void ConsumePerInstanceDataFromGameThread(void* FromGameThreadData, const FNiagaraSystemInstanceID& InstanceID) override
	{
		FVolumeCacheInstanceData_GameThread* FromGameThread = reinterpret_cast<FVolumeCacheInstanceData_GameThread*>(FromGameThreadData);
		FVolumeCacheInstanceData_RenderThread* InstanceData = InstanceData_RT.Find(InstanceID);

		if (FromGameThread != nullptr && InstanceData != nullptr)
		{
			InstanceData->CurrFrame = FromGameThread->CurrFrame;
			InstanceData->ReadFile = FromGameThread->ReadFile;
			InstanceData->TextureSize.X = float(FromGameThread->NumCells.X);
			InstanceData->TextureSize.Y = float(FromGameThread->NumCells.Y);
			InstanceData->TextureSize.Z = float(FromGameThread->NumCells.Z);

			FromGameThread->~FVolumeCacheInstanceData_GameThread();
		}
	}
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override;


	TMap<FNiagaraSystemInstanceID, FVolumeCacheInstanceData_RenderThread> InstanceData_RT;
};



UNiagaraDataInterfaceVolumeCache::UNiagaraDataInterfaceVolumeCache(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNiagaraDataInterfaceVolumeCacheProxy());

}

void UNiagaraDataInterfaceVolumeCache::PostInitProperties()
{
	Super::PostInitProperties();

	//Can we register data interfaces as regular types and fold them into the FNiagaraVariable framework for UI and function calls etc?
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);		
	}
}


bool UNiagaraDataInterfaceVolumeCache::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceVolumeCache* OtherTyped = CastChecked<const UNiagaraDataInterfaceVolumeCache>(Other);

	return OtherTyped != nullptr && VolumeCache == OtherTyped->VolumeCache;
}

int32 UNiagaraDataInterfaceVolumeCache::PerInstanceDataSize() const 
{ 
	return sizeof(FVolumeCacheInstanceData_GameThread); 
}

bool UNiagaraDataInterfaceVolumeCache::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceVolumeCache* OtherTyped = CastChecked<UNiagaraDataInterfaceVolumeCache>(Destination);

	OtherTyped->VolumeCache = VolumeCache;
	return true;
}

bool UNiagaraDataInterfaceVolumeCache::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	check(Proxy);
	
	FVolumeCacheInstanceData_GameThread* InstanceData = new (PerInstanceData) FVolumeCacheInstanceData_GameThread();
	SystemInstancesToProxyData_GT.Emplace(SystemInstance->GetId(), InstanceData);	

	if (VolumeCache != nullptr)
	{
		InstanceData->NumCells = VolumeCache->Resolution;

		// Push Updates to Proxy.
		FNiagaraDataInterfaceVolumeCacheProxy* TheProxy = GetProxyAs<FNiagaraDataInterfaceVolumeCacheProxy>();
		ENQUEUE_RENDER_COMMAND(FUpdateData)(
			[TheProxy, RT_VolumeCacheData = VolumeCache->GetData(), InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& RHICmdList)
		{
			check(!TheProxy->InstanceData_RT.Contains(InstanceID));
			FVolumeCacheInstanceData_RenderThread* TargetData = &TheProxy->InstanceData_RT.Add(InstanceID);
			TargetData->VolumeCacheData = RT_VolumeCacheData;
		});
	}

	

	return true;
}

void UNiagaraDataInterfaceVolumeCache::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FVolumeCacheInstanceData_GameThread* InstanceData = SystemInstancesToProxyData_GT.FindRef(SystemInstance->GetId());	
	
	InstanceData->~FVolumeCacheInstanceData_GameThread();
	SystemInstancesToProxyData_GT.Remove(SystemInstance->GetId());

	ENQUEUE_RENDER_COMMAND(RemoveInstance)
		(
			[RT_Proxy = GetProxyAs<FNiagaraDataInterfaceVolumeCacheProxy>(), RT_InstanceID = SystemInstance->GetId()](FRHICommandListImmediate&)
	{
		RT_Proxy->InstanceData_RT.Remove(RT_InstanceID);
	}
	);
}

void FNiagaraDataInterfaceVolumeCacheProxy::PreStage(const FNDIGpuComputePreStageContext& Context)
{	
	FVolumeCacheInstanceData_RenderThread* InstanceData = InstanceData_RT.Find(Context.GetSystemInstanceID());
	
	if (Context.IsInputStage() && InstanceData && InstanceData->ReadFile && InstanceData->VolumeCacheData != nullptr)
	{
		if (!InstanceData->ResolvedTextureRHI.IsValid())
		{
			const FIntVector IntTextureSize = FIntVector(int32(InstanceData->TextureSize.X), int32(InstanceData->TextureSize.Y), int32(InstanceData->TextureSize.Z));

			// #todo(dmp): this should be in instance data
			EPixelFormat Format = PF_FloatRGBA;

			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create3D(TEXT("NiagaraVolumeCache"), IntTextureSize.X, IntTextureSize.Y, IntTextureSize.Z, Format)
				.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::NoTiling);

			InstanceData->ResolvedTextureRHI = Context.GetGraphBuilder().RHICmdList.CreateTexture(Desc);
			InstanceData->SamplerStateRHI = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		}
		// must make sure readfile, curr frame, size are in the instance data - must sync in provide instance data for render thread

		InstanceData->VolumeCacheData->Fill3DTexture(InstanceData->CurrFrame, InstanceData->ResolvedTextureRHI);
	}
}

void UNiagaraDataInterfaceVolumeCache::ProvidePerInstanceDataForRenderThread(void* InDataFromGT, void* InInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	FVolumeCacheInstanceData_GameThread* InstanceData = static_cast<FVolumeCacheInstanceData_GameThread*>(InInstanceData);
	FVolumeCacheInstanceData_GameThread* DataFromGT = static_cast<FVolumeCacheInstanceData_GameThread*>(InDataFromGT);

	DataFromGT->CurrFrame = InstanceData->CurrFrame;
	DataFromGT->ReadFile = InstanceData->ReadFile;
	DataFromGT->NumCells = InstanceData->NumCells;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceVolumeCache::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{	
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetFrameName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Frame")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
		Sig.bExperimental = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ReadFileName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("ReadFile")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Success")));

		Sig.ModuleUsageBitmask = ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::System;
		Sig.bExperimental = true;
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = true;
		Sig.bSupportsGPU = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNumCellsName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsX")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsY")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumCellsZ")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = IndexToUnitName;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Index")));		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Unit")));

		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SampleCurrentFrameValueName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Texture")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UVW")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel")));
		Sig.SetDescription(LOCTEXT("TextureSampleVolumeTextureDesc", "Sample the specified mip level of the input 3d texture at the specified UVW coordinates. The UVW origin (0, 0, 0) is in the bottom left hand corner of the volume."));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		//Sig.Owner = *GetName();

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetCurrentFrameValue;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.bReadFunction = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Texture")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("x")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("y")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("z")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel")));
		Sig.SetDescription(LOCTEXT("TextureLoadVolumeTextureDesc", "load the specified mip level of the input 3d texture at the specified x, y, z voxel coordinates."));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value")));
		//Sig.Owner = *GetName();

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetCurrentFrameNumCells;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Texture")));
		Sig.SetDescription(LOCTEXT("TextureDimsDesc", "Get the dimensions of mip 0 of the texture."));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Dimensions3D")));
		//Sig.Owner = *GetName();

		OutFunctions.Add(Sig);
	}
}
#endif

void UNiagaraDataInterfaceVolumeCache::SetFrame(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FVolumeCacheInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncInputHandler<int> InFrame(Context);	
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> OutSuccess(Context);

	InstData->CurrFrame = InFrame.GetAndAdvance();

	// cannot read from cache...spew errors or let it go?
	if (VolumeCache == nullptr || !VolumeCache->LoadFile(InstData->CurrFrame))
	{
		UE_LOGF(LogNiagara, Warning, "Cache Read failed: %ls", *GetName());
	}

	*OutSuccess.GetDestAndAdvance() = true;
}

void UNiagaraDataInterfaceVolumeCache::ReadFile(FVectorVMExternalFunctionContext& Context)
{
	// This should only be called from a system or emitter script due to a need for only setting up initially.
	VectorVM::FUserPtrHandler<FVolumeCacheInstanceData_GameThread> InstData(Context);
	VectorVM::FExternalFuncInputHandler<bool> Read(Context);
	VectorVM::FExternalFuncRegisterHandler<FNiagaraBool> OutSuccess(Context);

	InstData->ReadFile = Read.GetAndAdvance();

	*OutSuccess.GetDestAndAdvance() = true;
}

void UNiagaraDataInterfaceVolumeCache::GetNumCells(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FVolumeCacheInstanceData_GameThread> InstData(Context);

	FNDIOutputParam<int32> NumCellsX(Context);
	FNDIOutputParam<int32> NumCellsY(Context);
	FNDIOutputParam<int32> NumCellsZ(Context);

	int32 TmpNumCellsX = InstData->NumCells.X;
	int32 TmpNumCellsY = InstData->NumCells.Y;
	int32 TmpNumCellsZ = InstData->NumCells.Z;

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		NumCellsX.SetAndAdvance(TmpNumCellsX);
		NumCellsY.SetAndAdvance(TmpNumCellsY);
		NumCellsZ.SetAndAdvance(TmpNumCellsZ);
	}
}

FString UNiagaraDataInterfaceVolumeCache::GetAssetPath(FString PathFormat, int32 FrameIndex) const
{
	UNiagaraSystem* NiagaraSystem = GetTypedOuter<UNiagaraSystem>();
	check(NiagaraSystem);

	const TMap<FString, FStringFormatArg> PathFormatArgs =
	{		
		{TEXT("SavedDir"),		FPaths::ProjectSavedDir()},
		{TEXT("FrameIndex"),	FString::Printf(TEXT("%03d"), FrameIndex)},
	};
	FString AssetPath = FString::Format(*PathFormat, PathFormatArgs);
	AssetPath.ReplaceInline(TEXT("//"), TEXT("/"));
	return AssetPath;
}

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceVolumeCache, SetFrame);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceVolumeCache, ReadFile);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceVolumeCache, GetNumCells);
void UNiagaraDataInterfaceVolumeCache::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	if (BindingInfo.Name == SetFrameName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceVolumeCache, SetFrame)::Bind(this, OutFunc);
	}
	if (BindingInfo.Name == ReadFileName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceVolumeCache, ReadFile)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetNumCellsName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceVolumeCache, GetNumCells)::Bind(this, OutFunc);
	}
}

bool UNiagaraDataInterfaceVolumeCache::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{	
	return false;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceVolumeCache::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceVolumeCache::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};

	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceVolumeCache::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if ((FunctionInfo.DefinitionName == SampleCurrentFrameValueName) ||
		(FunctionInfo.DefinitionName == GetCurrentFrameNumCells) ||
		(FunctionInfo.DefinitionName == GetCurrentFrameValue) || 
		(FunctionInfo.DefinitionName == IndexToUnitName))
	{
		return true;
	}
	return false;
}
#endif

void UNiagaraDataInterfaceVolumeCache::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceVolumeCache::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	const FNiagaraDataInterfaceVolumeCacheProxy& TextureProxy = Context.GetProxy<FNiagaraDataInterfaceVolumeCacheProxy>();
	const FVolumeCacheInstanceData_RenderThread* InstanceData = TextureProxy.InstanceData_RT.Find(Context.GetSystemInstanceID());

	FShaderParameters* Parameters = Context.GetParameterNestedStruct<FShaderParameters>();
	if (InstanceData && InstanceData->ResolvedTextureRHI.IsValid())
	{
		Parameters->TextureSize = InstanceData->TextureSize;
		Parameters->Texture = InstanceData->ResolvedTextureRHI;
		Parameters->TextureSampler = InstanceData->SamplerStateRHI ? InstanceData->SamplerStateRHI : GBlackVolumeTexture->SamplerStateRHI;
	}
	else
	{
		Parameters->TextureSize = FVector3f::ZeroVector;
		Parameters->Texture = GBlackVolumeTexture->TextureRHI;
		Parameters->TextureSampler = GBlackVolumeTexture->SamplerStateRHI;
	}
}

#undef LOCTEXT_NAMESPACE

