// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceVolumeTexture.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "Engine/TextureRenderTargetVolume.h"
#include "Engine/VolumeTexture.h"
#include "RenderGraphBuilder.h"
#include "RHIStaticStates.h"
#include "RenderGraphUtils.h"
#include "ShaderCompilerCore.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceVolumeTexture)

#define LOCTEXT_NAMESPACE "UNiagaraDataInterfaceVolumeTexture"

const TCHAR* UNiagaraDataInterfaceVolumeTexture::TemplateShaderFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraDataInterfaceVolumeTextureTemplate.ush");
const FName UNiagaraDataInterfaceVolumeTexture::LoadVolumeTextureName(TEXT("LoadVolumeTexture"));
const FName UNiagaraDataInterfaceVolumeTexture::SampleVolumeTextureName(TEXT("SampleVolumeTexture"));
const FName UNiagaraDataInterfaceVolumeTexture::TextureDimsName(TEXT("TextureDimensions3D"));

struct FNDIVolumeTextureInstanceData_GameThread
{
	TWeakObjectPtr<UTexture> CurrentTexture = nullptr;
	FIntVector CurrentTextureSize = FIntVector::ZeroValue;
	FNiagaraParameterDirectBinding<UObject*> UserParamBinding;
};

struct FNDIVolumeTextureInstanceData_RenderThread
{
	FSamplerStateRHIRef		SamplerStateRHI;
	FTextureReferenceRHIRef	TextureReferenceRHI;
	FVector3f				TextureSize;

	FRDGTextureRef			TransientRDGTexture = nullptr;
};

struct FNiagaraDataInterfaceProxyVolumeTexture : public FNiagaraDataInterfaceProxy
{
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override { check(false); }
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return 0; }

	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override
	{
		if (Context.IsFinalPostSimulate())
		{
			if (FNDIVolumeTextureInstanceData_RenderThread* InstanceData = InstanceData_RT.Find(Context.GetSystemInstanceID()))
			{
				InstanceData->TransientRDGTexture = nullptr;
			}
		}
	}

	TMap<FNiagaraSystemInstanceID, FNDIVolumeTextureInstanceData_RenderThread> InstanceData_RT;
};

UNiagaraDataInterfaceVolumeTexture::UNiagaraDataInterfaceVolumeTexture(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, Texture(nullptr)
{
	Proxy.Reset(new FNiagaraDataInterfaceProxyVolumeTexture());

	FNiagaraTypeDefinition Def(UTexture::StaticClass());
	TextureUserParameter.Parameter.SetType(Def);
}

void UNiagaraDataInterfaceVolumeTexture::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

bool UNiagaraDataInterfaceVolumeTexture::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}
	UNiagaraDataInterfaceVolumeTexture* DestinationTexture = CastChecked<UNiagaraDataInterfaceVolumeTexture>(Destination);
	DestinationTexture->Texture = Texture;
	DestinationTexture->TextureUserParameter = TextureUserParameter;

	return true;
}

bool UNiagaraDataInterfaceVolumeTexture::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceVolumeTexture* OtherTexture = CastChecked<const UNiagaraDataInterfaceVolumeTexture>(Other);
	return	
		OtherTexture->Texture == Texture &&
		OtherTexture->TextureUserParameter == TextureUserParameter;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceVolumeTexture::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	{
		FNiagaraFunctionSignature & Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = LoadVolumeTextureName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Texture"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelX"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelY"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("TexelZ"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("TextureLoadTextureVolumeDesc", "Read a texel from the provided location & mip without any filtering or sampling."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = SampleVolumeTextureName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsCPU = false;
		Sig.bSupportsGPU = true;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Texture"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("UVW"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MipLevel"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Value"));
		Sig.SetDescription(LOCTEXT("TextureSampleVolumeTextureDesc", "Sample the specified mip level of the input 3d texture at the specified UVW coordinates. The UVW origin (0, 0, 0) is in the bottom left hand corner of the volume."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = TextureDimsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Texture"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Dimensions3D"));
		Sig.SetDescription(LOCTEXT("TextureDimsDesc", "Get the dimensions of mip 0 of the texture."));
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceVolumeTexture, SampleVolumeTexture)
void UNiagaraDataInterfaceVolumeTexture::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == SampleVolumeTextureName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 4);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceVolumeTexture, SampleVolumeTexture)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == TextureDimsName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceVolumeTexture::GetTextureDimensions);
	}
}

int32 UNiagaraDataInterfaceVolumeTexture::PerInstanceDataSize() const
{
	return sizeof(FNDIVolumeTextureInstanceData_GameThread);
}

bool UNiagaraDataInterfaceVolumeTexture::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIVolumeTextureInstanceData_GameThread* InstanceData = new (PerInstanceData) FNDIVolumeTextureInstanceData_GameThread();
	InstanceData->UserParamBinding.Init(SystemInstance->GetInstanceParameters(), TextureUserParameter.Parameter);
	return true;
}

void UNiagaraDataInterfaceVolumeTexture::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIVolumeTextureInstanceData_GameThread* InstanceData = static_cast<FNDIVolumeTextureInstanceData_GameThread*>(PerInstanceData);
	InstanceData->~FNDIVolumeTextureInstanceData_GameThread();

	ENQUEUE_RENDER_COMMAND(NDITexture_RemoveInstance)
	(
		[RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyVolumeTexture>(), RT_InstanceID = SystemInstance->GetId()](FRHICommandListImmediate&)
		{
			RT_Proxy->InstanceData_RT.Remove(RT_InstanceID);
		}
	);
}

bool UNiagaraDataInterfaceVolumeTexture::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	FNDIVolumeTextureInstanceData_GameThread* InstanceData = static_cast<FNDIVolumeTextureInstanceData_GameThread*>(PerInstanceData);

	UTexture* CurrentTexture = InstanceData->UserParamBinding.GetValueOrDefault<UTexture>(Texture);
	if ( InstanceData->CurrentTexture != CurrentTexture )
	{
		UVolumeTexture* CurrentTextureVolume = Cast<UVolumeTexture>(CurrentTexture);
		UTextureRenderTargetVolume* CurrentTextureRT = Cast<UTextureRenderTargetVolume>(CurrentTexture);
		if (CurrentTextureVolume || CurrentTextureRT)
		{
			const FIntVector CurrentTextureSize = CurrentTextureVolume != nullptr ?
				FIntVector(CurrentTextureVolume->GetSizeX(), CurrentTextureVolume->GetSizeY(), CurrentTextureVolume->GetSizeZ()) :
				FIntVector(CurrentTextureRT->SizeX, CurrentTextureRT->SizeY, CurrentTextureRT->SizeZ);

			InstanceData->CurrentTexture = CurrentTexture;
			InstanceData->CurrentTextureSize = CurrentTextureSize;

			ENQUEUE_RENDER_COMMAND(NDITexture_UpdateInstance)
			(
				[RT_Proxy=GetProxyAs<FNiagaraDataInterfaceProxyVolumeTexture>(), RT_InstanceID=SystemInstance->GetId(), RT_Texture=CurrentTexture, RT_TextureSize=CurrentTextureSize](FRHICommandListImmediate&)
				{
					FNDIVolumeTextureInstanceData_RenderThread& InstanceData = RT_Proxy->InstanceData_RT.FindOrAdd(RT_InstanceID);
					if (RT_Texture)
					{
						InstanceData.TextureReferenceRHI = RT_Texture->TextureReference.TextureReferenceRHI;
						InstanceData.SamplerStateRHI = RT_Texture->GetResource() ? RT_Texture->GetResource()->SamplerStateRHI.GetReference() : TStaticSamplerState<SF_Point>::GetRHI();
						InstanceData.TextureSize.X = float(RT_TextureSize.X);
						InstanceData.TextureSize.Y = float(RT_TextureSize.Y);
						InstanceData.TextureSize.Z = float(RT_TextureSize.Z);
					}
					else
					{
						InstanceData.TextureReferenceRHI = nullptr;
						InstanceData.SamplerStateRHI = nullptr;
						InstanceData.TextureSize = FVector3f::ZeroVector;
					}
				}
			);
		}
	}
	return false;
}

void UNiagaraDataInterfaceVolumeTexture::GetTextureDimensions(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIVolumeTextureInstanceData_GameThread> InstData(Context);
	FNDIOutputParam<float> OutWidth(Context);
	FNDIOutputParam<float> OutHeight(Context);
	FNDIOutputParam<float> OutDepth(Context);

	const FVector3f FloatTextureSize(float(InstData->CurrentTextureSize.X), float(InstData->CurrentTextureSize.Y), float(InstData->CurrentTextureSize.Z));
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutWidth.SetAndAdvance(FloatTextureSize.X);
		OutHeight.SetAndAdvance(FloatTextureSize.Y);
		OutDepth.SetAndAdvance(FloatTextureSize.Z);
	}
}

void UNiagaraDataInterfaceVolumeTexture::SampleVolumeTexture(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIVolumeTextureInstanceData_GameThread> InstData(Context);
	FNDIInputParam<FVector3f> UVWParam(Context);
	FNDIInputParam<float> MipLevelParam(Context);
	FNDIOutputParam<FVector4f> OutSample(Context);

	const FVector4f DefaultSample(1.0f, 0.0f, 1.0f, 1.0f);
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const FVector3f UVW = UVWParam.GetAndAdvance();
		const float Mip = MipLevelParam.GetAndAdvance();
		OutSample.SetAndAdvance(DefaultSample);
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceVolumeTexture::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	InVisitor->UpdateShaderFile(TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceVolumeTexture::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};

	AppendTemplateHLSL(OutHLSL, TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceVolumeTexture::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	static const TSet<FName> ValidGpuFunctions =
	{
		LoadVolumeTextureName,
		SampleVolumeTextureName,
		TextureDimsName,
	};
	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}
#endif

void UNiagaraDataInterfaceVolumeTexture::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceVolumeTexture::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyVolumeTexture& TextureProxy = Context.GetProxy<FNiagaraDataInterfaceProxyVolumeTexture>();
	FNDIVolumeTextureInstanceData_RenderThread* InstanceData = TextureProxy.InstanceData_RT.Find(Context.GetSystemInstanceID());
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();

	FShaderParameters* Parameters = Context.GetParameterNestedStruct<FShaderParameters>();
	if (InstanceData && InstanceData->TextureReferenceRHI.IsValid())
	{
		Parameters->TextureSize = InstanceData->TextureSize;
		Parameters->TextureSampler = InstanceData->SamplerStateRHI;
		if (Context.IsResourceBound(&Parameters->Texture))
		{
			FRDGTextureRef RDGTexture = InstanceData ? InstanceData->TransientRDGTexture : nullptr;
			if (InstanceData && RDGTexture == nullptr)
			{
				FTextureRHIRef ResolvedTextureRHI = InstanceData->TextureReferenceRHI->GetReferencedTexture();
				if (ResolvedTextureRHI.IsValid())
				{
					InstanceData->TransientRDGTexture = GraphBuilder.FindExternalTexture(ResolvedTextureRHI);
					if (InstanceData->TransientRDGTexture == nullptr)
					{
						InstanceData->TransientRDGTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(ResolvedTextureRHI, TEXT("NiagaraTextureVolume")));

						GraphBuilder.UseInternalAccessMode(InstanceData->TransientRDGTexture);
						Context.GetRDGExternalAccessQueue().Add(InstanceData->TransientRDGTexture);
					}
					RDGTexture = InstanceData->TransientRDGTexture;
				}
			}

			Parameters->Texture = RDGTexture ? RDGTexture : Context.GetComputeDispatchInterface().GetBlackTexture(GraphBuilder, ETextureDimension::Texture3D);
		}
	}
	else
	{
		Parameters->TextureSize = FVector3f::ZeroVector;
		Parameters->TextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		Parameters->Texture = Context.GetComputeDispatchInterface().GetBlackTexture(GraphBuilder, ETextureDimension::Texture3D);
	}
}

void UNiagaraDataInterfaceVolumeTexture::SetTexture(UTexture* InTexture)
{
	if (InTexture)
	{
		Texture = InTexture;
	}
}

#undef LOCTEXT_NAMESPACE

