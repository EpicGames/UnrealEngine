// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataManager.h"

#include "NiagaraWorldManager.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataManager)


void UNiagaraDataManager::Init(FNiagaraWorldManager* InWorldManager)
{
	check(WorldManager == nullptr); //Called initialize multiple times.
	check(InWorldManager);
	WorldManager = InWorldManager;
	ComputeDispatchInterface = FNiagaraGpuComputeDispatchInterface::Get(WorldManager->GetWorld());
}

void UNiagaraDataManager::BeginFrame()
{

}

void UNiagaraDataManager::EndFrame()
{
	//Remove component to ref map entry.
	for (auto ComponentToDataIt = NiagaraComponentToSharedData.CreateIterator(); ComponentToDataIt; ++ComponentToDataIt)
	{
		UNiagaraComponent* Comp = ComponentToDataIt.Key().Get();
		TArray<FNiagaraSharedDataPtr>& ComponentSharedData = ComponentToDataIt.Value();

		if(IsValid(Comp) == false)
		{
			ComponentSharedData.Empty();
		}

		for(auto DataIt = ComponentSharedData.CreateIterator(); DataIt; ++DataIt)
		{
			FNiagaraSharedDataPtr& Data = *DataIt;
			if(!Data.IsValid() || Data->bInvalidated)
			{
				DataIt.RemoveCurrentSwap();
			}
		}

		if(ComponentSharedData.Num() == 0)
		{
			if(IsValid(Comp))
			{
				Comp->OnSystemFinished.RemoveDynamic(this, &UNiagaraDataManager::OnReferencingSystemFinished);
			}
			ComponentToDataIt.RemoveCurrent();
		}
	}
}

void UNiagaraDataManager::RegisterReferencingSystem(UNiagaraComponent* Component, FNiagaraSharedDataPtr Data)
{
	check(Component && Data.IsValid());
	NiagaraComponentToSharedData.FindOrAdd(Component).AddUnique(Data);
	Component->OnSystemFinished.AddUniqueDynamic(this, &UNiagaraDataManager::OnReferencingSystemFinished);
}

void UNiagaraDataManager::OnReferencingSystemFinished(UNiagaraComponent* Component)
{
	NiagaraComponentToSharedData.Remove(Component);
	Component->OnSystemFinished.RemoveDynamic(this, &UNiagaraDataManager::OnReferencingSystemFinished);
}

