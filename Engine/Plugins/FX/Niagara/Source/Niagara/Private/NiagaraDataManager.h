// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraTypes.h"
#include "NiagaraGpuComputeDataManager.h"
#include "NiagaraSharedDataUtilities.h"
#include "NiagaraDataManager.generated.h"

class FNiagaraWorldManager;
class FNiagaraGpuComputeDispatchInterface;

/** 
A generic manager class holding some data requiring management or marshaling for Niagara.
A counterpart for Gpu accessible data is FNiagaraGpuComputeDataManager.
Some cases may utilize a UNiagaraDataManager and a FNiagaraGpuComputeDataManager together.
*/
UCLASS()
class UNiagaraDataManager : public UObject
{
	GENERATED_BODY()

public:

	virtual void Init(FNiagaraWorldManager* WorldMan);		
	virtual void BeginFrame();
	virtual void EndFrame();
	virtual void PreTick(ETickingGroup TickGroup, float DeltaSeconds){}

	/** Track the connection between a NiagaraComponent and a FNiagaraSharedDataPtr. The data will remain valid at least until the valid component completes. */
	void RegisterReferencingSystem(UNiagaraComponent* Component, FNiagaraSharedDataPtr SharedData);
	
	/** Callback when a Niagara component completes so we can remove any tracked data associated with it. */
	UFUNCTION()
	void OnReferencingSystemFinished(UNiagaraComponent* Component);

protected:
	
	FNiagaraWorldManager* WorldManager = nullptr;
	FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = nullptr;

	/** 
	Map of active Niagara Components referencing FNiagaraSharedDataPtrs.
	Allows us to maintain the lifetime of a FNiagaraSharedDataPtr until a referencing Niagara Component completes.
	*/
	TMap<TWeakObjectPtr<UNiagaraComponent>, TArray<FNiagaraSharedDataPtr>> NiagaraComponentToSharedData;
};

struct FNiagaraDataInterfaceSetShaderParametersContext;
typedef FNiagaraDataInterfaceSetShaderParametersContext FNiagaraDataManagerSetShaderParametersContext;

/** Base class for RT Proxies of data managers. */
struct FNiagaraDataManager_RTProxy : public FNiagaraGpuComputeDataManager
{
	FNiagaraDataManager_RTProxy(FNiagaraGpuComputeDispatchInterface* InOwnerInterface)
	: FNiagaraGpuComputeDataManager(InOwnerInterface)
	{
	}
};