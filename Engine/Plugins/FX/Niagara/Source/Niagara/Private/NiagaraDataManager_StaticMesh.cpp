// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataManager_StaticMesh.h"

#include "PrimitiveSceneInfo.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraDataInterface.h"
#include "RenderGraphUtils.h"
#include "SystemTextures.h"
#include "Components/StaticMeshComponent.h"
#include "SceneInterface.h"

//////////////////////////////////////////////////////////////////////////
// UNiagaraDataManager_StaticMesh
// Allows indirect referencing of static mesh data from Niagara via a persistent ID.
// 

UNiagaraDataManager_StaticMesh::UNiagaraDataManager_StaticMesh(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(FNiagaraSharedDataID_StaticMesh::StaticStruct()), ENiagaraTypeRegistryFlags::AllowParameter | ENiagaraTypeRegistryFlags::AllowAnyVariable);
}

void UNiagaraDataManager_StaticMesh::EndFrame()
{
	++IDAcquireTag;

	TArray<FPrimitiveComponentId> ToRemove;
	int32 MaxSlotsUsed = -1;
	//Remove and invalidate any invalid/refs
	for (auto It = TrackedComponents.CreateIterator(); It; ++It)
	{
		TWeakObjectPtr<UStaticMeshComponent> WeakComp = It.Key();
		FNiagaraSharedData_StaticMeshPtr& MeshData = It.Value();
		
		check(MeshData.IsValid());
		
		bool bRefIsValid = WeakComp.IsValid() && !MeshData.IsUnique();
		int32 Slot = MeshData->ID.Index;
		if (bRefIsValid)
		{
			MaxSlotsUsed = FMath::Max(Slot, MaxSlotsUsed);
			continue;
		}

		//Signal to anyone else holding this ref that it should be removed.
		MeshData->bInvalidated = true;

		ToRemove.Add(MeshData->PrimitiveId);
		FreeSlots.Add(Slot);		
		It.RemoveCurrent();
	}

	Super::EndFrame();

	NumSlots = MaxSlotsUsed + 1;

	if (ToRemove.Num() > 0)
	{
		ENQUEUE_RENDER_COMMAND(RemovePrimIDtoMeshID)(
			[RTDispatchInterface = ComputeDispatchInterface, RTToRemove = MoveTemp(ToRemove), RTNumSlots = NumSlots](FRHICommandListImmediate& CmdList)
			{
				if (RTDispatchInterface)
				{
					FNiagaraDataManager_StaticMesh_RTProxy& Manager = RTDispatchInterface->GetOrCreateDataManager<FNiagaraDataManager_StaticMesh_RTProxy>();
					for (const FPrimitiveComponentId& PrimitiveIdToRemove : RTToRemove)
					{
						Manager.Remove(PrimitiveIdToRemove);
					}
					Manager.SetNumSlots(RTNumSlots);
				}
			}
		);
	}
}

FNiagaraSharedData_StaticMeshPtr UNiagaraDataManager_StaticMesh::GetOrCreateSharedMeshData(UStaticMeshComponent* ComponentToReference)
{
	if(FNiagaraSharedData_StaticMeshPtr* FoundData = TrackedComponents.Find(ComponentToReference))
	{
		return *FoundData;
	}

	FNiagaraSharedData_StaticMeshPtr& NewMeshData = TrackedComponents.Add(ComponentToReference);
	NewMeshData = MakeShared<FNiagaraSharedData_StaticMesh>();
	NewMeshData->PrimitiveId = ComponentToReference->GetPrimitiveSceneId();
	NewMeshData->ID.AcquireTag = IDAcquireTag;
	NewMeshData->SourceComponent = ComponentToReference;

	int32 IDIndex = NumSlots;
	if(FreeSlots.Num() > 0)
	{
		IDIndex = FreeSlots.Pop();
	}

	NewMeshData->ID.Index = IDIndex;
	NumSlots = FMath::Max(IDIndex + 1, NumSlots);

	ENQUEUE_RENDER_COMMAND(AddPrimIDtoMeshID)(
		[RTDispatchInterface = ComputeDispatchInterface, RTPrimId = NewMeshData->PrimitiveId, RTRefID = NewMeshData->ID, RTNumSlots = NumSlots](FRHICommandListImmediate& CmdList)
		{
			if (RTDispatchInterface)
			{
				FNiagaraDataManager_StaticMesh_RTProxy& Manager = RTDispatchInterface->GetOrCreateDataManager<FNiagaraDataManager_StaticMesh_RTProxy>();
				Manager.Add(RTPrimId, RTRefID);
				Manager.SetNumSlots(RTNumSlots);
			}
		}
	);

	return NewMeshData;
}

//////////////////////////////////////////////////////////////////////////
// FNiagaraDataManager_StaticMesh_RTProxy
// Manager class on the GPU for static mesh data. Effectively an RT Proxy for the GT side manager.

void FNiagaraDataManager_StaticMesh_RTProxy::Add(FPrimitiveComponentId PrimitiveId, FNiagaraSharedDataID_StaticMesh MeshID)
{
	PrimitiveToMeshID.Add(PrimitiveId) = MeshID;
}
void FNiagaraDataManager_StaticMesh_RTProxy::Remove(FPrimitiveComponentId PrimitiveId)
{
	PrimitiveToMeshID.Remove(PrimitiveId);
}
	
void FNiagaraDataManager_StaticMesh_RTProxy::BeginFrame(FNiagaraGpuComputeDataManagerTickContext& Context)
{

}

void FNiagaraDataManager_StaticMesh_RTProxy::PreTick(FNiagaraGpuComputeDataManagerTickContext& Context, ENiagaraGpuComputeTickStage::Type TickStage)
{
	if(MeshDataBuffer == nullptr)
	{
		static_assert(sizeof(FNiagaraSharedDataGPU_StaticMesh) == 16u);
		//TODO: If we have some hooks into add/remove primitive and can be sure if/when the DF indices can change then we could keep this data persistent until it's dirty.	
		//NOTE: This building must be done after the DF Scene has been updated so shouldn't be done early from PreInitViews etc.
		const FSceneInterface* Scene = GetOwnerInterface()->GetSceneInterface();
		check(Scene);

		TArray<FNiagaraSharedDataGPU_StaticMesh> Buffer;
		Buffer.SetNum(NumSlots);
		for (auto It = PrimitiveToMeshID.CreateConstIterator(); It; ++It)
		{
			FPrimitiveComponentId PrimitiveId = It.Key();
			const FNiagaraSharedDataID_StaticMesh& StaticMeshID = It.Value();
			if (const FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene ? Scene->GetPrimitiveSceneInfo(PrimitiveId) : nullptr)
			{
				int32 Slot = StaticMeshID.Index;
				
				if(Buffer.IsValidIndex(Slot))
				{
					Buffer[Slot].IDTag = StaticMeshID.AcquireTag;

					Buffer[Slot].DistanceFieldIndex = PrimitiveSceneInfo->DistanceFieldInstanceIndices.Num() > 0 ? PrimitiveSceneInfo->DistanceFieldInstanceIndices[0] : INDEX_NONE;
				
					//TODO: if we bind GPU scene buffers we can access a bunch of data.
					Buffer[Slot].PrimitiveIndex = PrimitiveSceneInfo->GetIndex();

					//TODO: Could pass bindless handles to vertex data etc...
				
					//TODO: Can pass in bounds here if we want but could be better to access via GPU scene info above?
	// 				FBoxSphereBounds Bounds = PrimitiveSceneInfo->Proxy->GetBounds();
	// 				Buffer[Slot].BoundsRadius = Bounds.SphereRadius;
	// 				Buffer[Slot].BoundsExtents = FVector3f(Bounds.BoxExtent);
	// 				
	// 				FVector3f LWCTile  = FLargeWorldRenderScalar::GetTileFor(Bounds.Origin);
	// 				FVector3f SimLocalSWC = FVector3f(Bounds.Origin - FVector(LWCTile) * FLargeWorldRenderScalar::GetTileSize());
	// 				Buffer[Slot].BoundsOrigin = SimLocalSWC;
	// 				Buffer[Slot].BoundsOriginTile = LWCTile;
				}
			}
		}
		
		if(Buffer.Num() == 0)
		{
			Buffer.SetNum(1);
		}
			
		MeshDataBuffer = CreateStructuredBuffer(Context.GetGraphBuilder(), TEXT("NiagaraSharedMeshDataBuffer"), MoveTemp(Buffer));
		MeshDataBufferSRV = Context.GetGraphBuilder().CreateSRV(MeshDataBuffer);				
	}
}

void FNiagaraDataManager_StaticMesh_RTProxy::EndFrame(FNiagaraGpuComputeDataManagerTickContext& Context)
{
	MeshDataBufferSRV = nullptr;
	MeshDataBuffer = nullptr;
}

FRDGBufferSRVRef FNiagaraDataManager_StaticMesh_RTProxy::GetGPUBufferSRV()const
{
	return MeshDataBufferSRV;
}

const FShaderParametersMetadata* FNiagaraDataManager_StaticMesh_RTProxy::GetParametersMetadata()
{
	return TShaderParameterStructTypeInfo<FNiagaraDataManager_StaticMesh_Parameters>::GetStructMetadata();
}

void FNiagaraDataManager_StaticMesh_RTProxy::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context)
{
	const FShaderParametersMetadata* ParametersMetadata = GetParametersMetadata();
	if(uint8* ShaderParameters = Context.GetParameterIncludedStruct(ParametersMetadata))
	{
		if (Context.IsStructBound(ShaderParameters, ParametersMetadata))
		{
			FNiagaraDataManager_StaticMesh_Parameters* TypedParameters = reinterpret_cast<FNiagaraDataManager_StaticMesh_Parameters*>(Context.GetParameterIncludedStruct(ParametersMetadata));
			
			if (const FNiagaraDataManager_StaticMesh_RTProxy* MeshDataManager = Context.GetComputeDispatchInterface().GetDataManager<FNiagaraDataManager_StaticMesh_RTProxy>())
			{
				TypedParameters->NiagaraSharedData_StaticMesh_Buffer = MeshDataManager->GetGPUBufferSRV();
				TypedParameters->NiagaraSharedData_StaticMesh_NumSlots = MeshDataManager->GetNumSlots();
			}
			else
			{			
				TArray<FNiagaraSharedDataGPU_StaticMesh> Buffer;
				Buffer.SetNum(1);
				FRDGBufferRef MeshDataBuffer = CreateStructuredBuffer(Context.GetGraphBuilder(), TEXT("DummyNiagaraSharedMeshDataBuffer"), Buffer);;
				FRDGBufferSRVRef MeshDataBufferSRV = Context.GetGraphBuilder().CreateSRV(MeshDataBuffer);
				TypedParameters->NiagaraSharedData_StaticMesh_Buffer = MeshDataBufferSRV;
				TypedParameters->NiagaraSharedData_StaticMesh_NumSlots = 0;
			}
		}
	}
}
