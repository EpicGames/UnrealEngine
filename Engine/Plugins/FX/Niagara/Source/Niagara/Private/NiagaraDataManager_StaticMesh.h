// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraTypes.h"
#include "MeshPassProcessor.h"
#include "PrimitiveComponentId.h"
#include "RenderGraphFwd.h"

#include "NiagaraDataManager.h"
#include "NiagaraDataManager_StaticMesh.generated.h"


class UPrimitiveComponent;
struct FNiagaraDataInterfaceSetShaderParametersContext;

/** Persistent ID allowing Niagara particles access to the static mesh info provided by the manager. */
USTRUCT(BlueprintType)
struct FNiagaraSharedDataID_StaticMesh : public FNiagaraID
{
	GENERATED_BODY()
};

//Shared data about a static mesh we are tracking for indirect access in Niagara.
struct FNiagaraSharedData_StaticMesh : public FNiagaraSharedData
{
	TWeakObjectPtr<UStaticMeshComponent> SourceComponent;
	
	FPrimitiveComponentId PrimitiveId;

	//The mesh reference generated for this primitive. 
	FNiagaraSharedDataID_StaticMesh ID;
};

typedef TSharedPtr<FNiagaraSharedData_StaticMesh> FNiagaraSharedData_StaticMeshPtr;

UCLASS()
class UNiagaraDataManager_StaticMesh : public UNiagaraDataManager
{
	GENERATED_BODY()

public:
	
	UNiagaraDataManager_StaticMesh(FObjectInitializer const& ObjectInitializer);

	//UNiagaraDataManager Interface
	virtual void EndFrame()override;
	//END UNiagaraDataManager Interface
	
	/** 
	Finds or creates shared data tracking info about the given mesh component for use later in Niagara.
	Once no-one is holding this shared data pointer, the data will be cleaned up and invalid to access on the GPU.
	Access is still safe but will fail.
	You can have Niagara Components hold the reference until they complete using RegisterReferencingSystem().
	*/
	[[nodiscard]] FNiagaraSharedData_StaticMeshPtr GetOrCreateSharedMeshData(UStaticMeshComponent* ComponentToReference);	

private:
	
	TMap<TWeakObjectPtr<UStaticMeshComponent>, FNiagaraSharedData_StaticMeshPtr> TrackedComponents;
	
	/** Currently free slots in the MeshDataBuffer. */
	TArray<int32> FreeSlots;
	int32 NumSlots = 0;
	int32 IDAcquireTag = 0;
};

/** 
Render thread proxy for UNiagaraDataManager_StaticMesh.
Tracks link between PrimitiveIds for tracked static mesh components and associated FNiagaraSharedDataID_StaticMesh.
The manager generates a buffer each frame allowing particles to use a FNiagaraSharedDataID_StaticMesh to access info about each static mesh component.
*/
struct FNiagaraDataManager_StaticMesh_RTProxy final : public FNiagaraDataManager_RTProxy
{	
	static FName GetManagerName()
	{
		static FName ManagerName("FNiagaraDataManager_StaticMesh_RTProxy");
		return ManagerName;
	}
	
	FNiagaraDataManager_StaticMesh_RTProxy(FNiagaraGpuComputeDispatchInterface* InOwnerInterface)
	: FNiagaraDataManager_RTProxy(InOwnerInterface)
	{
	}
	
	//FNiagaraDataManager_RTProxy Interface
	virtual void BeginFrame(FNiagaraGpuComputeDataManagerTickContext& Context) override;
	virtual void EndFrame(FNiagaraGpuComputeDataManagerTickContext& Context) override;
	virtual void PreTick(FNiagaraGpuComputeDataManagerTickContext& Context, ENiagaraGpuComputeTickStage::Type TickStage) override;	
	//END FNiagaraDataManager_RTProxy Interface
	
	void Add(FPrimitiveComponentId PrimitiveId, FNiagaraSharedDataID_StaticMesh MeshID);
	void Remove(FPrimitiveComponentId PrimitiveId);
	void SetNumSlots(int32 InNumSlots){ NumSlots = InNumSlots; }

	[[nodiscard]] FRDGBufferSRVRef GetGPUBufferSRV()const;
	[[nodiscard]] int32 GetNumSlots()const{ return NumSlots; }
	
	[[nodiscard]] static const FShaderParametersMetadata* GetParametersMetadata();
	static void SetShaderParameters(const FNiagaraDataManagerSetShaderParametersContext& Context);

private:

	TMap<FPrimitiveComponentId, FNiagaraSharedDataID_StaticMesh> PrimitiveToMeshID;
	int32 NumSlots = 0;

	FRDGBufferRef MeshDataBuffer = nullptr;
	FRDGBufferSRVRef MeshDataBufferSRV = nullptr;
};

/** Counterpart to struct in NiagaraManagedReference_StaticMesh.ush */
struct FNiagaraSharedDataGPU_StaticMesh
{
	int32 IDTag = INDEX_NONE;
	int32 DistanceFieldIndex = INDEX_NONE;
	int32 PrimitiveIndex = INDEX_NONE;
	int32 Padding = INDEX_NONE;
};

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraDataManager_StaticMesh_Parameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FNiagaraSharedDataGPU_StaticMesh>, NiagaraSharedData_StaticMesh_Buffer)
	SHADER_PARAMETER(int32, NiagaraSharedData_StaticMesh_NumSlots)
END_SHADER_PARAMETER_STRUCT()

