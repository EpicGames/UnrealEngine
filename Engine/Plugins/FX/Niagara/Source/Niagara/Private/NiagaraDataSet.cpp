// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataSet.h"
#include "NiagaraCommon.h"
#include "GlobalShader.h"
#include "NiagaraStats.h"
#include "NiagaraRenderer.h"
#include "NiagaraGPUInstanceCountManager.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraShaderParticleID.h"
#include "NiagaraShared.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DECLARE_CYCLE_STAT(TEXT("InitRenderData"), STAT_InitRenderData, STATGROUP_Niagara);

//////////////////////////////////////////////////////////////////////////

FCriticalSection FNiagaraSharedObject::CritSec;
TArray<FNiagaraSharedObject*> FNiagaraSharedObject::DeferredDeletionList;
void FNiagaraSharedObject::Destroy()
{
	FScopeLock Lock(&CritSec);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(IsThisNotNull(this, "FNiagaraSharedObject::Destroy"));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	check(DeferredDeletionList.Contains(this) == false);
	DeferredDeletionList.Add(this);
}

void FNiagaraSharedObject::FlushDeletionList()
{
	//Always do this on RT. GPU buffers must be freed on RT and we may as well do CPU frees at the same time.
	ENQUEUE_RENDER_COMMAND(FlushDeletionListCommand)([&](FRHICommandListImmediate& RHICmdList)
	{
		FScopeLock Lock(&CritSec);//Possibly make this a lock free queue?
		int32 i = 0;
		while (i < DeferredDeletionList.Num())
		{
			check(DeferredDeletionList[i] != nullptr);
			if (DeferredDeletionList[i]->IsInUse() == false)
			{
				delete DeferredDeletionList[i];
				DeferredDeletionList.RemoveAtSwap(i);
			}
			else
			{
				++i;
			}
		}
	});
}

//////////////////////////////////////////////////////////////////////////
static int32 GNiagaraDataBufferMinSize = 512;
static FAutoConsoleVariableRef CVarRenderDataBlockSize(
	TEXT("fx.NiagaraDataBufferMinSize"),
	GNiagaraDataBufferMinSize,
	TEXT("Niagara data buffer minimum allocation size in bytes (Default=512)."),
	ECVF_Default
);

static int32 GNiagaraDataBufferShrinkFactor = 3;
static FAutoConsoleVariableRef CVarNiagaraRenderBufferShrinkFactor(
	TEXT("fx.NiagaraDataBufferShrinkFactor"),
	GNiagaraDataBufferShrinkFactor,
	TEXT("Niagara data buffer size threshold for shrinking. (Default=3) \n")
	TEXT("The buffer will be reallocated when the used size becomes 1/F of the allocated size."),
	ECVF_Default
);

static int32 GNiagaraGPUDataBufferChunkSize = 4096;
static FAutoConsoleVariableRef CVarNiagaraGPUDataBufferChunkSize(
	TEXT("fx.NiagaraGPUDataBufferChunkSize"),
	GNiagaraGPUDataBufferChunkSize,
	TEXT("Niagara GPU data buffer allocation chunk size used to round GPU allocations in bytes, must be power of 2 (Default=4096)\n"),
	ECVF_Default
);

static int32 GNiagaraGPUDataBufferShrinkFactor = 2;
static FAutoConsoleVariableRef CVarNiagaraGPUDataBufferShrinkFactor(
	TEXT("fx.NiagaraGPUDataBufferShrinkFactor"),
	GNiagaraGPUDataBufferShrinkFactor,
	TEXT("Niagara GPU data buffer size threshold for shrinking. (Default=2) \n")
	TEXT("The buffer will be reallocated when the used size becomes 1/F of the allocated size. \n"),
	ECVF_Default
);

static int32 GNiagaraGPUDataWarningSize = 0;
static FAutoConsoleVariableRef CVarNiagaraGPUDataWarningSize(
	TEXT("fx.NiagaraGPUDataWarningSize"),
	GNiagaraGPUDataWarningSize,
	TEXT("Allocation size where we should log a warning."),
	ECVF_Scalability
);

static bool GNiagaraReleaseBuffersOnReset = true;
static FAutoConsoleVariableRef CVarNiagaraReleaseBuffersOnReset(
	TEXT("fx.NiagaraReleaseBuffersOnReset"),
	GNiagaraReleaseBuffersOnReset,
	TEXT("Will release all memory associated with data buffers when the dataset is reset."),
	ECVF_Scalability
);

void NiagaraWarnGpuBufferSize(const int32 ByteSize, const TCHAR* DebugSimName)
{
	static int32 MaxEncountered = 0;
	if ((GNiagaraGPUDataWarningSize > 0) && (ByteSize > GNiagaraGPUDataWarningSize) && (MaxEncountered < ByteSize))
	{
		MaxEncountered = ByteSize;
		UE_LOGF(LogNiagara, Warning, "GpuBuffer Allocation (%d bytes) from %ls", ByteSize, DebugSimName);
	}
}

FNiagaraDataSet::FNiagaraDataSet()
	: NumFreeIDs(0)
	, MaxUsedID(0)
	, IDAcquireTag(0)
	, GPUNumAllocatedIDs(0)
	, CurrentData(nullptr)
	, DestinationData(nullptr)
#if NIAGARA_MEMORY_TRACKING
	, BufferSizeBytes(0)
#endif
	, MaxInstanceCount(UINT_MAX)
	, MaxAllocationCount(UINT_MAX)
	, bInitialized(false)
{
	CompiledData.Init(&FNiagaraDataSetCompiledData::DummyCompiledData);
}

FNiagaraDataSet::~FNiagaraDataSet()
{
	CompiledData.Reset();

// 	int32 CurrBytes = RenderDataFloat.NumBytes + RenderDataInt.NumBytes;
// 	DEC_MEMORY_STAT_BY(STAT_NiagaraVBMemory, CurrBytes);
	if (Data.Num() > 0)
	{
		for (FNiagaraDataBuffer* Buffer : Data)
		{
			Buffer->Destroy();
		}
		Data.Empty();
	}

	if (GPUFreeIDs.Buffer)
	{
		check(IsInRenderingThread());
		GPUFreeIDs.Release();
	}

	GPUNumAllocatedIDs = 0;
}

void FNiagaraDataSet::Init(const FNiagaraDataSetCompiledData* InDataSetCompiledData, int32 DefaultNumBuffers)
{
	CompiledData.Init(InDataSetCompiledData != nullptr ? InDataSetCompiledData : &FNiagaraDataSetCompiledData::DummyCompiledData);

	while(Data.Num() < DefaultNumBuffers)
	{
		Data.Add(new FNiagaraDataBuffer(this));
	}

	if (bInitialized)
	{
		Reset();
	}
	else
	{
		ResetBuffersInternal();
		bInitialized = true;
	}
}

void FNiagaraDataSet::Reset()
{
	ResetBuffers();
}

void FNiagaraDataSet::ResetBuffers()
{
	//checkSlow(CompiledData);
	
	if (GetSimTarget() == ENiagaraSimTarget::CPUSim)
	{
		ResetBuffersInternal();

		if (GNiagaraReleaseBuffersOnReset)
		{
			for (auto it=Data.CreateIterator(); it; ++it)
			{
				FNiagaraDataBuffer* Buffer = Data[it.GetIndex()];
				if (Buffer->IsBeingRead())
				{
				
					// In this path the buffer might still be in use on the render thread for rendering the last frame therefore we push it into the deferred destruction queue as we can not release the data as we would race.
#if NIAGARA_MEMORY_TRACKING
					BufferSizeBytes -= Buffer->GetFloatBuffer().GetAllocatedSize() + Buffer->GetInt32Buffer().GetAllocatedSize() + Buffer->GetHalfBuffer().GetAllocatedSize();
#endif
					Buffer->Destroy();
					it.RemoveCurrentSwap();
				}
				else
				{
					Buffer->ReleaseCPU();
				}
			}
		}
	}
	else
	{
		checkSlow(GetSimTarget() == ENiagaraSimTarget::GPUComputeSim);
		ENQUEUE_RENDER_COMMAND(ResetBuffersCommand)(
			[this](FRHICommandListImmediate& RHICmdList)
			{
				for (FNiagaraDataBuffer* Buffer : Data)
				{
					Buffer->ReleaseGPU();
				}
				ResetBuffersInternal();
			}
		);
	}
}

void FNiagaraDataSet::ResetBuffersInternal()
{
#if NIAGARA_MEMORY_TRACKING
	BufferSizeBytes -= FreeIDsTable.GetAllocatedSize() + SpawnedIDsTable.GetAllocatedSize();
#endif

	if (bInitialized == true)
	{
		CheckCorrectThread();
	}

	CurrentData = nullptr;
	DestinationData = nullptr;

	FreeIDsTable.Reset();
	NumFreeIDs = 0;
	MaxUsedID = INDEX_NONE;
	NumSpawnedIDs = 0;
	SpawnedIDsTable.Reset();

	//Ensure we have a valid current buffer
	BeginSimulate();
	EndSimulate();

#if NIAGARA_MEMORY_TRACKING
	BufferSizeBytes += FreeIDsTable.GetAllocatedSize() + SpawnedIDsTable.GetAllocatedSize();
#endif
}

FNiagaraDataBuffer& FNiagaraDataSet::AllocateBuffer()
{
	//Find a free buffer we can write into.
	//Linear search but there should only be 2 or three entries.
	for (FNiagaraDataBuffer* Buffer : Data)
	{
		check(Buffer);
		if (Buffer != CurrentData && Buffer->TryLock())
		{
			return *Buffer;
		}
	}

	FNiagaraDataBuffer* Buffer = Data.Add_GetRef(new FNiagaraDataBuffer(this));
	verify(Buffer->TryLock());
	check(Buffer->IsBeingWritten());
	return *Buffer;
}

FNiagaraDataBuffer& FNiagaraDataSet::BeginSimulate(bool bResetDestinationData)
{
	//CheckCorrectThread();
	check(DestinationData == nullptr);

	DestinationData = &AllocateBuffer();

	if (bResetDestinationData)
	{
		DestinationData->SetNumInstances(0);
		DestinationData->GetIDTable().Reset();
	}

	return GetDestinationDataChecked();
}

void FNiagaraDataSet::EndSimulate(bool SetCurrentData)
{
	//CheckCorrectThread();
	//Destination is now complete so make it the current simulation state.
	DestinationData->Unlock();
	checkSlow(!DestinationData->IsInUse());

	if (SetCurrentData)
	{
		CurrentData = DestinationData;
	}

	DestinationData = nullptr;
}

void FNiagaraDataSet::SetCurrentData(FNiagaraDataBuffer* InCurrentData)
{
	check(DestinationData == nullptr);

	CurrentData = InCurrentData;
}

void FNiagaraDataSet::Allocate(int32 NumInstances, bool bMaintainExisting)
{
#if NIAGARA_MEMORY_TRACKING
	BufferSizeBytes -= FreeIDsTable.GetAllocatedSize() + SpawnedIDsTable.GetAllocatedSize();
#endif

	checkSlow(IsInitialized());
	CheckCorrectThread();
	checkSlow(DestinationData);

	DestinationData->Allocate(NumInstances);
	if (bMaintainExisting)
	{
		CurrentData->CopyTo(*DestinationData, 0, 0, CurrentData->GetNumInstances());
	}

#if NIAGARA_NAN_CHECKING
	CheckForNaNs();
#endif

	if (RequiresPersistentIDs())
	{
		TArray<int32>& CurrentIDTable = CurrentData->GetIDTable();
		TArray<int32>& DestinationIDTable = DestinationData->GetIDTable();

		int32 NumUsedIDs = MaxUsedID + 1;

		int32 RequiredIDs = FMath::Max(NumInstances, NumUsedIDs);
		int32 ExistingNumIDs = CurrentIDTable.Num();

		//////////////////////////////////////////////////////////////////////////
		//TODO: We should replace this with a lock free list that uses just a single table with RequiredIDs elements.
		//Unused slots in the array can form a linked list so that we need only one array with a Head index for the FreeID list
		//This will be faster and likely simpler than the current implementation while also working on GPU.
		//////////////////////////////////////////////////////////////////////////
		if (RequiredIDs > ExistingNumIDs)
		{
			//UE_LOGF(LogNiagara, Warning, "Growing ID Table! OldSize:%d | NewSize:%d", ExistingNumIDs, RequiredIDs);
			int32 NewNumIds = RequiredIDs - ExistingNumIDs;

			//Free ID Table must always be at least as large as the data buffer + it's current size in the case all particles die this frame.
			FreeIDsTable.AddUninitialized(NewNumIds);

			//Free table should always have enough room for these new IDs.
			check(NumFreeIDs + NewNumIds <= FreeIDsTable.Num());

			//ID Table grows so add any new IDs to the free array. Add in reverse order to maintain a continuous increasing allocation when popping.
			for (int32 NewFreeID = RequiredIDs - 1; NewFreeID >= ExistingNumIDs; --NewFreeID)
			{
				FreeIDsTable[NumFreeIDs++] = NewFreeID;
			}
			//UE_LOGF(LogNiagara, Warning, "DataSetAllocate: Adding New Free IDs: %d - ", NewNumIds);
		}
#if 0
		else if (RequiredIDs < ExistingNumIDs >> 1)//Configurable?
		{
			//If the max id we use has reduced significantly then we can shrink the tables.
			//Have to go through the FreeIDs and remove any that are greater than the new table size.
			//UE_LOGF(LogNiagara, Warning, "DataSetAllocate: Shrinking ID Table! OldSize:%d | NewSize:%d", ExistingNumIDs, RequiredIDs);
			for (int32 CheckedFreeID = 0; CheckedFreeID < NumFreeIDs;)
			{
				checkSlow(NumFreeIDs <= FreeIDsTable.Num());
				if (FreeIDsTable[CheckedFreeID] >= RequiredIDs)
				{
					//UE_LOGF(LogNiagara, Warning, "RemoveSwap FreeID: Removed:%d | Swapped:%d", FreeIDsTable[CheckedFreeID], FreeIDsTable.Last());		
					int32 FreeIDIndex = --NumFreeIDs;
					FreeIDsTable[CheckedFreeID] = FreeIDsTable[FreeIDIndex];
					FreeIDsTable[FreeIDIndex] = INDEX_NONE;
				}
				else
				{
					++CheckedFreeID;
				}
			}

			check(NumFreeIDs <= RequiredIDs);
			FreeIDsTable.SetNumUninitialized(NumFreeIDs);
		}
#endif
		else
		{
			//Drop in required size not great enough so just allocate same size.
			RequiredIDs = ExistingNumIDs;
		}

		// We need to clear the ID to index table to -1 so we don't have stale entries for particles which died in the previous
		// frame (when the results were written to another buffer). All the entries which are in use will be filled in by the script.
		DestinationIDTable.SetNumUninitialized(RequiredIDs);
		FMemory::Memset(DestinationIDTable.GetData(), -1, RequiredIDs * DestinationIDTable.GetTypeSize());

		//reset the max ID ready for it to be filled in during simulation.
		MaxUsedID = INDEX_NONE;

// 		UE_LOGF(LogNiagara, Warning, "DataSetAllocate: NumInstances:%d | ID Table Size:%d | NumFreeIDs:%d | FreeTableSize:%d", NumInstances, DestinationData->GetIDTable().Num(), NumFreeIDs, FreeIDsTable.Num());
// 		UE_LOGF(LogNiagara, Warning, "== FreeIDs %d ==", NumFreeIDs);
// 		for (int32 i=0; i< NumFreeIDs;++i)
// 		{
// 			UE_LOGF(LogNiagara, Warning, "%d", FreeIDsTable[i]);
// 		}
	}

#if NIAGARA_MEMORY_TRACKING
	BufferSizeBytes += FreeIDsTable.GetAllocatedSize() + SpawnedIDsTable.GetAllocatedSize();
#endif
}

int64 FNiagaraDataSet::GetSizeBytes() const
{
#if NIAGARA_MEMORY_TRACKING
	return BufferSizeBytes;
#else
	return 0ll;
#endif
}

void FNiagaraDataSet::CheckForNaNs() const
{
	for (const FNiagaraDataBuffer* Buffer : Data)
	{
		if (Buffer->CheckForNaNs())
		{
			Buffer->Dump(0, Buffer->GetNumInstances(), TEXT("Found Niagara buffer containing NaNs!"));
			ensureAlwaysMsgf(false, TEXT("NiagaraDataSet contains NaNs!"));
		}
	}
}

void FNiagaraDataSet::Dump(int32 StartIndex, int32 NumInstances, const FString& Label, const FName& SortParameterKey)const
{
	if (CurrentData)
	{
		CurrentData->Dump(StartIndex, NumInstances, Label, SortParameterKey);
	}

	if (GetDestinationData())
	{
		FString DestLabel = Label + TEXT("[Destination]");
		DestinationData->Dump(StartIndex, NumInstances, DestLabel, SortParameterKey);
	}
}

void FNiagaraDataSet::AllocateGPUFreeIDs(uint32 InNumInstances, FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, const TCHAR* DebugSimName)
{
	checkSlow(GetSimTarget() == ENiagaraSimTarget::GPUComputeSim && RequiresPersistentIDs());

	// Clearing and compacting the ID table must run over all the allocated elements, so we must use a chunk size which balances
	// between reallocation frequency and the cost of processing unused elements.
	constexpr uint32 ID_ALLOC_CHUNKSIZE = 1024;
	const uint32 NumIDsToAlloc = FMath::DivideAndRoundUp(InNumInstances, ID_ALLOC_CHUNKSIZE) * ID_ALLOC_CHUNKSIZE;

	if (NumIDsToAlloc <= GPUNumAllocatedIDs)
	{
		// We can never shrink the ID buffer, because IDs with numeric values larger than the current number of instances
		// might still be in use.
		return;
	}

#if NIAGARA_MEMORY_TRACKING
	BufferSizeBytes -= GPUFreeIDs.NumBytes;
#endif

	SCOPED_DRAW_EVENT(RHICmdList, NiagaraGPUComputeInitFreeIDs);

	FNameBuilder DebugBufferName;
	DebugBufferName.Append(TEXT("NiagaraFreeIDList_"));
	DebugBufferName.Append(DebugSimName);

	FRWBuffer NewFreeIDsBuffer;
	NewFreeIDsBuffer.Initialize(RHICmdList, DebugBufferName.ToString(), sizeof(int32), NumIDsToAlloc, EPixelFormat::PF_R32_SINT, BUF_Static);

	FRHIShaderResourceView* ExistingBuffer = GPUNumAllocatedIDs > 0 ? GPUFreeIDs.SRV.GetReference() : FNiagaraRenderer::GetDummyIntBuffer();

	RHICmdList.Transition(FRHITransitionInfo(NewFreeIDsBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
	NiagaraInitGPUFreeIDList(RHICmdList, FeatureLevel, NewFreeIDsBuffer.UAV, NumIDsToAlloc, ExistingBuffer, GPUNumAllocatedIDs);
	RHICmdList.Transition(FRHITransitionInfo(NewFreeIDsBuffer.UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute));

	GPUFreeIDs = MoveTemp(NewFreeIDsBuffer);
	GPUNumAllocatedIDs = NumIDsToAlloc;

#if NIAGARA_MEMORY_TRACKING
	BufferSizeBytes += GPUFreeIDs.NumBytes;
#endif
}

const FNiagaraVariableLayoutInfo* FNiagaraDataSet::GetVariableLayout(const FNiagaraVariableBase& Var)const
{
	int32 VarLayoutIndex = GetVariables().IndexOfByKey(Var);
	return VarLayoutIndex != INDEX_NONE ? &CompiledData->VariableLayouts[VarLayoutIndex] : nullptr;
}

bool FNiagaraDataSet::GetVariableComponentOffsets(const FNiagaraVariableBase& Var, int32 &FloatStart, int32 &IntStart, int32& HalfStart) const
{
	const FNiagaraVariableLayoutInfo *Info = GetVariableLayout(Var);
	if (Info)
	{
		FloatStart = Info->GetFloatComponentStart();
		IntStart = Info->GetInt32ComponentStart();
		HalfStart = Info->GetHalfComponentStart();
		return true;
	}

	FloatStart = INDEX_NONE;
	IntStart = INDEX_NONE;
	HalfStart = INDEX_NONE;
	return false;
}

void FNiagaraDataSet::CopyTo(FNiagaraDataSet& Other, int32 StartIdx, int32 NumInstances, bool bResetOther)const
{
	//check(CompiledData);

	CheckCorrectThread();

	if (bResetOther)
	{
		Other.CompiledData = CompiledData;
		Other.Reset();
	}
	else
	{
		checkSlow(Other.GetVariables() == GetVariables());
	}

	//Read the most current data. Even if it's possibly partially complete simulation data.
	FNiagaraDataBuffer* SourceBuffer = GetDestinationData() ? GetDestinationData() : GetCurrentData();
	FNiagaraDataBuffer* OtherCurrentBuffer = Other.GetCurrentData();

	if (SourceBuffer != nullptr)
	{
		int32 SourceInstances = SourceBuffer->GetNumInstances();
		int32 OrigNumInstances = OtherCurrentBuffer->GetNumInstances();

		if (StartIdx >= SourceInstances)
		{
			return; //We can't start beyond the end of the source buffer.
		}

		if (NumInstances == INDEX_NONE || StartIdx + NumInstances >= SourceInstances)
		{
			NumInstances = SourceBuffer->GetNumInstances() - StartIdx;
		}

		FNiagaraDataBuffer& OtherDestBuffer = Other.BeginSimulate();

		//We need to allocate enough space for the new data and existing data if we're keeping it.
		int32 RequiredInstances = bResetOther ? NumInstances : NumInstances + OrigNumInstances;
		OtherDestBuffer.Allocate(RequiredInstances);
		OtherDestBuffer.SetNumInstances(RequiredInstances);

		//Copy the data in our current buffer over into the new buffer.
		if (!bResetOther)
		{
			OtherCurrentBuffer->CopyTo(OtherDestBuffer, 0, 0, OtherCurrentBuffer->GetNumInstances());
		}

		//Now copy the data from the source buffer into the newly allocated space.
		SourceBuffer->CopyTo(OtherDestBuffer, 0, OrigNumInstances, NumInstances);

		Other.EndSimulate();
	}
}

void FNiagaraDataSet::CopyFromGPUReadback(const float* GPUReadBackFloat, const int* GPUReadBackInt, const FFloat16* GPUReadBackHalf, int32 StartIdx /* = 0 */, int32 NumInstances /* = INDEX_NONE */, uint32 FloatStride, uint32 IntStride, uint32 HalfStride)
{
	check(IsInRenderingThread());
	check(IsInitialized());//We should be finalized with proper layout information already.

	FNiagaraDataBuffer& DestBuffer = BeginSimulate();
	DestBuffer.GPUCopyFrom(GPUReadBackFloat, GPUReadBackInt, GPUReadBackHalf, StartIdx, NumInstances, FloatStride, IntStride, HalfStride);
	EndSimulate();
}
 
//////////////////////////////////////////////////////////////////////////

FNiagaraDataBuffer::FNiagaraDataBuffer(FNiagaraDataSet* InOwner)
	: Owner(InOwner)
	, GPUInstanceCountBufferOffset(INDEX_NONE)
	, NumInstances(0)
	, NumInstancesAllocated(0)
	, FloatStride(0)
	, Int32Stride(0)
	, HalfStride(0)
	, NumSpawnedInstances(0)
	, IDAcquireTag(0)
{
}

FNiagaraDataBuffer::~FNiagaraDataBuffer()
{
	check(!IsInUse());
	DEC_MEMORY_STAT_BY(STAT_NiagaraParticleMemory, FloatData.GetAllocatedSize() + Int32Data.GetAllocatedSize() + HalfData.GetAllocatedSize());
#if NIAGARA_MEMORY_TRACKING
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUParticleMemory, AllocationSizeBytes);
#endif
}

int32 FNiagaraDataBuffer::TransferInstance(FNiagaraDataBuffer& SourceBuffer, int32 InstanceIndex, bool bRemoveFromSource)
{
	CheckUsage(false);
	if (SourceBuffer.GetNumInstances() > (uint32)InstanceIndex)
	{
		int32 OldNumInstances = NumInstances;
		if (NumInstances == NumInstancesAllocated)
		{
			//Have to allocate some more space.
			Allocate(NumInstancesAllocated + 1, true);
		}

		SetNumInstances(OldNumInstances + 1);

		/** Copy the instance data. */
		int32 FloatComponents = Owner->GetNumFloatComponents();
		for (int32 CompIdx = FloatComponents - 1; CompIdx >= 0; --CompIdx)
		{
			float* Src = SourceBuffer.GetInstancePtrFloat(CompIdx, InstanceIndex);
			float* Dst = GetInstancePtrFloat(CompIdx, OldNumInstances);
			*Dst = *Src;
		}
		int32 IntComponents = Owner->GetNumInt32Components();
		for (int32 CompIdx = IntComponents - 1; CompIdx >= 0; --CompIdx)
		{
			int32* Src = SourceBuffer.GetInstancePtrInt32(CompIdx, InstanceIndex);
			int32* Dst = GetInstancePtrInt32(CompIdx, OldNumInstances);
			*Dst = *Src;
		}

		if (bRemoveFromSource)
		{
			SourceBuffer.KillInstance(InstanceIndex);
		}

		return OldNumInstances;
	}

	return INDEX_NONE;
}

bool FNiagaraDataBuffer::CheckForNaNs()const
{
	CheckUsage(true);
	bool bContainsNaNs = false;
	int32 NumFloatComponents = Owner->GetNumFloatComponents();
	for (int32 CompIdx = 0; CompIdx < NumFloatComponents && !bContainsNaNs; ++CompIdx)
	{
		for (int32 InstIdx = 0; InstIdx < (int32)NumInstances && !bContainsNaNs; ++InstIdx)
		{
			float Val = *GetInstancePtrFloat(CompIdx, InstIdx);
			bContainsNaNs = FMath::IsNaN(Val) || !FMath::IsFinite(Val);
		}
	}

	return bContainsNaNs;
}

void FNiagaraDataBuffer::ZeroCPUBuffers()
{
	if(FloatData.GetData())
	{
		FMemory::Memzero(FloatData.GetData(), FloatData.Num());
	}

	if (Int32Data.GetData())
	{
		FMemory::Memzero(Int32Data.GetData(), Int32Data.Num());
	}

	if (HalfData.GetData())
	{
		FMemory::Memzero(HalfData.GetData(), HalfData.Num());
	}
}

void FNiagaraDataBuffer::Allocate(uint32 InNumInstances, bool bMaintainExisting)
{
	NumInstances = 0;
	if (!bMaintainExisting)
	{
		IDToIndexTable.Reset();
	}

	// Calculate allocation size
	uint32 NewFloatStride = GetSafeComponentBufferSize(InNumInstances * sizeof(float));
	int32 NewFloatNum = NewFloatStride * Owner->GetNumFloatComponents();

	uint32 NewInt32Stride = GetSafeComponentBufferSize(InNumInstances * sizeof(int32));
	int32 NewInt32Num = NewInt32Stride * Owner->GetNumInt32Components();

	uint32 NewHalfStride = GetSafeComponentBufferSize(InNumInstances * sizeof(FFloat16));
	int32 NewHalfNum = NewHalfStride * Owner->GetNumHalfComponents();

	// Do sizes match?
	if (NewFloatNum != FloatData.Num() || NewInt32Num != Int32Data.Num() || NewHalfNum != HalfData.Num())
	{
		// Do we need to grow or shrink?
		const bool bGrowData = NewFloatNum > FloatData.Num() || NewInt32Num > Int32Data.Num() || NewHalfNum > HalfData.Num();
		const bool bShrinkFloatData = !bGrowData && (GNiagaraDataBufferShrinkFactor * FMath::Max(GNiagaraDataBufferMinSize, NewFloatNum) < FloatData.Max() || !NewFloatNum);
		const bool bShrinkIntData = !bGrowData && (GNiagaraDataBufferShrinkFactor * FMath::Max(GNiagaraDataBufferMinSize, NewInt32Num) < Int32Data.Max() || !NewInt32Num);
		const bool bShrinkHalfData = !bGrowData && (GNiagaraDataBufferShrinkFactor * FMath::Max(GNiagaraDataBufferMinSize, NewHalfNum) < HalfData.Max() || !NewHalfNum);
		if (bGrowData || bShrinkFloatData || bShrinkIntData || bShrinkHalfData)
		{
			NumInstancesAllocated = InNumInstances;

			DEC_MEMORY_STAT_BY(STAT_NiagaraParticleMemory, FloatData.GetAllocatedSize() + Int32Data.GetAllocatedSize() + HalfData.GetAllocatedSize());
#if NIAGARA_MEMORY_TRACKING
			Owner->BufferSizeBytes -= FloatData.GetAllocatedSize() + Int32Data.GetAllocatedSize() + HalfData.GetAllocatedSize();
#endif
			if (bMaintainExisting)
			{
				TArray<uint8> NewFloatData;
				TArray<uint8> NewInt32Data;
				TArray<uint8> NewHalfData;
				NewFloatData.SetNum(NewFloatNum);
				NewInt32Data.SetNum(NewInt32Num);
				NewHalfData.SetNum(NewHalfNum);

				if (NewFloatStride > 0 && FloatStride > 0)
				{
					const int32 FloatComponents = Owner->GetNumFloatComponents();
					const uint32 BytesToCopy = FMath::Min(NewFloatStride, FloatStride);
					for (int32 CompIdx = FloatComponents - 1; CompIdx >= 0; --CompIdx)
					{
						const uint8* Src = FloatData.GetData() + FloatStride * CompIdx;
						uint8* Dst = NewFloatData.GetData() + NewFloatStride * CompIdx;
						FMemory::Memcpy(Dst, Src, BytesToCopy);
					}
				}

				if (NewInt32Stride > 0 && Int32Stride > 0)
				{
					const int32 IntComponents = Owner->GetNumInt32Components();
					const uint32 BytesToCopy = FMath::Min(NewInt32Stride, Int32Stride);
					for (int32 CompIdx = IntComponents - 1; CompIdx >= 0; --CompIdx)
					{
						const uint8* Src = Int32Data.GetData() + Int32Stride * CompIdx;
						uint8* Dst = NewInt32Data.GetData() + NewInt32Stride * CompIdx;
						FMemory::Memcpy(Dst, Src, BytesToCopy);
					}
				}

				if (NewHalfStride > 0 && HalfStride > 0)
				{
					const int32 HalfComponents = Owner->GetNumHalfComponents();
					const uint32 BytesToCopy = FMath::Min(NewHalfStride, HalfStride);
					for (int32 CompIdx = HalfComponents - 1; CompIdx >= 0; --CompIdx)
					{
						const uint8* Src = HalfData.GetData() + HalfStride * CompIdx;
						uint8* Dst = NewHalfData.GetData() + NewHalfStride * CompIdx;
						FMemory::Memcpy(Dst, Src, BytesToCopy);
					}
				}

				FloatData = MoveTemp(NewFloatData);
				Int32Data = MoveTemp(NewInt32Data);
				HalfData = MoveTemp(NewHalfData);
			}
			else
			{
				FloatData.SetNum(NewFloatNum, bShrinkFloatData ? EAllowShrinking::Yes : EAllowShrinking::No);
				Int32Data.SetNum(NewInt32Num, bShrinkIntData ? EAllowShrinking::Yes : EAllowShrinking::No);
				HalfData.SetNum(NewHalfNum, bShrinkHalfData ? EAllowShrinking::Yes : EAllowShrinking::No);
			}
			INC_MEMORY_STAT_BY(STAT_NiagaraParticleMemory, FloatData.GetAllocatedSize() + Int32Data.GetAllocatedSize() + HalfData.GetAllocatedSize());
#if NIAGARA_MEMORY_TRACKING
			Owner->BufferSizeBytes += FloatData.GetAllocatedSize() + Int32Data.GetAllocatedSize() + HalfData.GetAllocatedSize();
#endif
		}
		// Calculate strides based upon max of instance counts
		// This allows us to skip building the register table when shrinking
		else
		{
			NumInstancesAllocated = FMath::Max(NumInstancesAllocated, InNumInstances);
			NewFloatStride = GetSafeComponentBufferSize(NumInstancesAllocated * sizeof(float));
			NewInt32Stride = GetSafeComponentBufferSize(NumInstancesAllocated * sizeof(int32));
			NewHalfStride = GetSafeComponentBufferSize(NumInstancesAllocated * sizeof(FFloat16));
		}
	}
	else
	{
		NumInstancesAllocated = InNumInstances;
	}

	if ((NewFloatStride != FloatStride) || (NewInt32Stride != Int32Stride) || (NewHalfStride != HalfStride))
	{
		FloatStride = NewFloatStride;
		Int32Stride = NewInt32Stride;
		HalfStride = NewHalfStride;
		BuildRegisterTable();
	}

	check(GetNumInstances() <= NumInstancesAllocated);
	check(FloatData.Num() >= NewFloatNum);
	check(Int32Data.Num() >= NewInt32Num);
	check(HalfData.Num() >= NewHalfNum);
}

void FNiagaraDataBuffer::ReleaseCPU()
{
	NumInstances = 0;
	NumInstancesAllocated = 0;
	IDToIndexTable.Reset();

	FloatStride = 0;
	Int32Stride = 0;
	HalfStride = 0;

	DEC_MEMORY_STAT_BY(STAT_NiagaraParticleMemory, FloatData.GetAllocatedSize() + Int32Data.GetAllocatedSize() + HalfData.GetAllocatedSize());
#if NIAGARA_MEMORY_TRACKING
	Owner->BufferSizeBytes -= FloatData.GetAllocatedSize() + Int32Data.GetAllocatedSize() + HalfData.GetAllocatedSize();
#endif

	FloatData.Empty();
	Int32Data.Empty();
	HalfData.Empty();
}

void FNiagaraDataBuffer::AllocateGPU(FRHICommandListBase& RHICmdList, uint32 InNumInstances, ERHIFeatureLevel::Type FeatureLevel, const TCHAR* DebugSimName)
{
	static constexpr EBufferUsageFlags GPUBufferFlags = BUF_Static | BUF_SourceCopy;

	CheckUsage(false);
	checkSlow(Owner->GetSimTarget() == ENiagaraSimTarget::GPUComputeSim);

	// Clear count offset
	GPUInstanceCountBufferOffset = INDEX_NONE;

	NumInstancesAllocated = InNumInstances;

	// GetMaxInstanceCount() returns the maximum number of usable instances
	const uint32 MaxAllocatedInstances = Owner->GetMaxAllocationCount();

	// Round the count up to the nearest threadgroup size. GetMaxNumInstances() ensures that the returned value is aligned to NiagaraComputeMaxThreadGroupSize, so if the calling
	// code clamps the instance count correctly, this operation should never exceed the max instance count.
	const uint32 PaddedNumInstances = FMath::DivideAndRoundUp(NumInstancesAllocated, NiagaraComputeMaxThreadGroupSize) * NiagaraComputeMaxThreadGroupSize;
	check(PaddedNumInstances <= MaxAllocatedInstances);

	// Pack the data so that the space between elements is the padded thread group size
	FloatStride = PaddedNumInstances * sizeof(float);
	Int32Stride = PaddedNumInstances * sizeof(int32);
	HalfStride = PaddedNumInstances * sizeof(FFloat16);

#if NIAGARA_MEMORY_TRACKING
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUParticleMemory, AllocationSizeBytes);
	Owner->BufferSizeBytes -= AllocationSizeBytes;
	AllocationSizeBytes = 0;
#endif

	if (PaddedNumInstances == 0)
	{
		if (GPUBufferFloat.Buffer)
		{
			GPUBufferFloat.Release();
		}
		if (GPUBufferInt.Buffer)
		{
			GPUBufferInt.Release();
		}
		if (GPUBufferHalf.Buffer)
		{
			GPUBufferHalf.Release();
		}

		if (GPUIDToIndexTable.Buffer)
		{
			GPUIDToIndexTable.Release();
		}
	}
	else // Otherwise check for growing and possibly shrinking (if GNiagaraGPUDataBufferBufferSlack > 1) .
	{
		// Float buffer requires growing or shrinking?
		const int32 RequiredFloatByteSize = Align(FloatStride * Owner->GetNumFloatComponents(), GNiagaraGPUDataBufferChunkSize);
		const int32 CurrentFloatByteSize = (int32)GPUBufferFloat.NumBytes;
		if ((RequiredFloatByteSize > CurrentFloatByteSize) || (RequiredFloatByteSize * GNiagaraGPUDataBufferShrinkFactor < CurrentFloatByteSize))
		{
			if (GPUBufferFloat.Buffer)
			{
				GPUBufferFloat.Release();
			}
			NiagaraWarnGpuBufferSize(RequiredFloatByteSize, DebugSimName);
			GPUBufferFloat.Initialize(RHICmdList, TEXT("GPUBufferFloat"), sizeof(float), RequiredFloatByteSize / sizeof(float), EPixelFormat::PF_R32_FLOAT, ERHIAccess::SRVMask, GPUBufferFlags);
		}

		// Half buffer requires growing or shrinking?
		const int32 RequiredHalfByteSize = Align(HalfStride * Owner->GetNumHalfComponents(), GNiagaraGPUDataBufferChunkSize);
		const int32 CurrentHalfByteSize = (int32)GPUBufferHalf.NumBytes;
		if ((RequiredHalfByteSize > CurrentHalfByteSize) || (RequiredHalfByteSize * GNiagaraGPUDataBufferShrinkFactor < CurrentHalfByteSize))
		{
			if (GPUBufferHalf.Buffer)
			{
				GPUBufferHalf.Release();
			}
			NiagaraWarnGpuBufferSize(RequiredHalfByteSize, DebugSimName);
			GPUBufferHalf.Initialize(RHICmdList, TEXT("GPUBufferHalf"), sizeof(FFloat16), RequiredHalfByteSize / sizeof(FFloat16), EPixelFormat::PF_R16F, ERHIAccess::SRVMask, GPUBufferFlags);
		}

		// Int buffer requires growing or shrinking?
		const int32 RequiredInt32ByteSize = Align(Int32Stride * Owner->GetNumInt32Components(), GNiagaraGPUDataBufferChunkSize);
		const int32 CurrentInt32ByteSize = (int32)GPUBufferInt.NumBytes;
		if ((RequiredInt32ByteSize > CurrentInt32ByteSize) || (RequiredInt32ByteSize * GNiagaraGPUDataBufferShrinkFactor < CurrentInt32ByteSize))
		{
			if (GPUBufferInt.Buffer)
			{
				GPUBufferInt.Release();
			}
			NiagaraWarnGpuBufferSize(RequiredInt32ByteSize, DebugSimName);
			GPUBufferInt.Initialize(RHICmdList, TEXT("GPUBufferInt"), sizeof(int32), RequiredInt32ByteSize / sizeof(int32), EPixelFormat::PF_R32_SINT, ERHIAccess::SRVMask, GPUBufferFlags);
		}

		// Allocate persistent IDs?
		if (Owner->RequiresPersistentIDs())
		{
			uint32 NumExistingElems = GPUIDToIndexTable.Buffer ? GPUIDToIndexTable.Buffer->GetSize() / sizeof(int32) : 0;
			uint32 NumNeededElems = Owner->GetGPUNumAllocatedIDs();
			if (NumExistingElems < NumNeededElems)
			{
				if (GPUIDToIndexTable.Buffer)
				{
					GPUIDToIndexTable.Release();
				}
				NiagaraWarnGpuBufferSize(NumNeededElems * sizeof(int32), DebugSimName);
				TStringBuilder<128> DebugBufferName;
				DebugBufferName.Appendf(TEXT("NiagaraIDToIndexTable_%s_%p"), DebugSimName ? DebugSimName : TEXT(""), this);
				GPUIDToIndexTable.Initialize(RHICmdList, DebugBufferName.ToString(), sizeof(int32), NumNeededElems, EPixelFormat::PF_R32_SINT, ERHIAccess::SRVCompute, GPUBufferFlags);
			}
		}
	}
#if NIAGARA_MEMORY_TRACKING
	AllocationSizeBytes = GPUBufferFloat.NumBytes + GPUBufferHalf.NumBytes + GPUBufferInt.NumBytes + GPUIDToIndexTable.NumBytes;
	Owner->BufferSizeBytes += AllocationSizeBytes;
	INC_MEMORY_STAT_BY(STAT_NiagaraGPUParticleMemory, AllocationSizeBytes);
#endif
}

void FNiagaraDataBuffer::SwapGPU(FNiagaraDataBuffer* BufferToSwap)
{
#if NIAGARA_MEMORY_TRACKING
	Swap(AllocationSizeBytes, BufferToSwap->AllocationSizeBytes);
#endif

	Swap(GPUDataReadyStage, BufferToSwap->GPUDataReadyStage);
	Swap(GPUInstanceCountBufferOffset, BufferToSwap->GPUInstanceCountBufferOffset);
	Swap(GPUBufferFloat, BufferToSwap->GPUBufferFloat);
	Swap(GPUBufferInt, BufferToSwap->GPUBufferInt);
	Swap(GPUIDToIndexTable, BufferToSwap->GPUIDToIndexTable);
	Swap(GPUBufferHalf, BufferToSwap->GPUBufferHalf);
	Swap(NumInstances, BufferToSwap->NumInstances);
	Swap(NumInstancesAllocated, BufferToSwap->NumInstancesAllocated);
	Swap(FloatStride, BufferToSwap->FloatStride);
	Swap(Int32Stride, BufferToSwap->Int32Stride);
	Swap(HalfStride, BufferToSwap->HalfStride);
	Swap(NumSpawnedInstances, BufferToSwap->NumSpawnedInstances);
	Swap(IDAcquireTag, BufferToSwap->IDAcquireTag);
}

void FNiagaraDataBuffer::ReleaseGPU()
{
	check(IsInRenderingThread());

#if NIAGARA_MEMORY_TRACKING
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUParticleMemory, AllocationSizeBytes);
	Owner->BufferSizeBytes -= AllocationSizeBytes;
	AllocationSizeBytes = 0;
#endif

	NumInstancesAllocated = 0;
	FloatStride = 0;
	Int32Stride = 0;
	HalfStride = 0;
	GPUInstanceCountBufferOffset = INDEX_NONE;

	if (GPUBufferFloat.Buffer)
	{
		GPUBufferFloat.Release();
	}
	if (GPUBufferInt.Buffer)
	{
		GPUBufferInt.Release();
	}
	if (GPUBufferHalf.Buffer)
	{
		GPUBufferHalf.Release();
	}

	if (GPUIDToIndexTable.Buffer)
	{
		GPUIDToIndexTable.Release();
	}
}

void FNiagaraDataBuffer::SwapInstances(uint32 OldIndex, uint32 NewIndex) 
{
	CheckUsage(false);

	uint32 FloatComponents = Owner->GetNumFloatComponents();
	for (uint32 CompIdx = 0; CompIdx < FloatComponents; ++CompIdx)
	{
		float* Src = GetInstancePtrFloat(CompIdx, OldIndex);
		float* Dst = GetInstancePtrFloat(CompIdx, NewIndex);
		float Temp = *Dst;
		*Dst = *Src;
		*Src = Temp;
	}
	uint32 IntComponents = Owner->GetNumInt32Components();
	for (uint32 CompIdx = 0; CompIdx < IntComponents; ++CompIdx)
	{
		int32* Src = GetInstancePtrInt32(CompIdx, OldIndex);
		int32* Dst = GetInstancePtrInt32(CompIdx, NewIndex);
		int32 Temp = *Dst;
		*Dst = *Src;
		*Src = Temp;
	}
	uint32 HalfComponents = Owner->GetNumHalfComponents();
	for (uint32 CompIdx = 0; CompIdx < HalfComponents; ++CompIdx)
	{
		FFloat16* Src = GetInstancePtrHalf(CompIdx, OldIndex);
		FFloat16* Dst = GetInstancePtrHalf(CompIdx, NewIndex);
		Swap(*Dst, *Src);
	}
}

void FNiagaraDataBuffer::KillInstance(uint32 InstanceIdx)
{
	CheckUsage(false);
	check(InstanceIdx < NumInstances);
	--NumInstances;

	uint32 FloatComponents = Owner->GetNumFloatComponents();
	for (uint32 CompIdx = 0; CompIdx < FloatComponents; ++CompIdx)
	{
		float* Src = GetInstancePtrFloat(CompIdx, NumInstances);
		float* Dst = GetInstancePtrFloat(CompIdx, InstanceIdx);
		*Dst = *Src;
	}
	uint32 IntComponents = Owner->GetNumInt32Components();
	for (uint32 CompIdx = 0; CompIdx < IntComponents; ++CompIdx)
	{
		int32* Src = GetInstancePtrInt32(CompIdx, NumInstances);
		int32* Dst = GetInstancePtrInt32(CompIdx, InstanceIdx);
		*Dst = *Src;
	}
	uint32 HalfComponents = Owner->GetNumHalfComponents();
	for (uint32 CompIdx = 0; CompIdx < HalfComponents; ++CompIdx)
	{
		const FFloat16* Src = GetInstancePtrHalf(CompIdx, NumInstances);
		FFloat16* Dst = GetInstancePtrHalf(CompIdx, InstanceIdx);
		*Dst = *Src;
	}

#if NIAGARA_NAN_CHECKING
	CheckForNaNs();
#endif
}

void FNiagaraDataBuffer::CopyToUnrelated(FNiagaraDataBuffer& DestBuffer, int32 StartIdx, int32 DestStartIdx, int32 InNumInstances)const
{
	if (StartIdx < 0 || (uint32)StartIdx >= NumInstances)
	{
		return;
	}

	uint32 InstancesToCopy = InNumInstances;
	if (InstancesToCopy == INDEX_NONE)
	{
		InstancesToCopy = NumInstances - StartIdx;
	}

	if (InstancesToCopy != 0)
	{
		const uint32 NewNumInstances = DestStartIdx + InstancesToCopy;

		// Only allocate if we need to increase the number of instances as the caller may have previously
		// allocated the array and may not be expecting it to shrink inside this call.
		if (DestStartIdx < 0 || NewNumInstances > DestBuffer.GetNumInstancesAllocated())
		{
			DestBuffer.Allocate(NewNumInstances, true);
		}
		DestBuffer.SetNumInstances(NewNumInstances);

		const FNiagaraDataSetCompiledData& DestCompiledData = DestBuffer.GetOwner()->GetCompiledData();
		const FNiagaraDataSetCompiledData& CompiledData = GetOwner()->GetCompiledData();

		for (int32 VarIdx = 0; VarIdx < DestCompiledData.Variables.Num(); ++VarIdx)
		{
			const FNiagaraVariableBase& DestVar = DestCompiledData.Variables[VarIdx];
			const FNiagaraVariableLayoutInfo& DestVarLayout = DestCompiledData.VariableLayouts[VarIdx];

			const int32 SrcVarIdx = CompiledData.Variables.IndexOfByKey(DestVar);
			if (SrcVarIdx != INDEX_NONE)
			{
				//Found a the variable to copy over.
				const FNiagaraVariableLayoutInfo& SrcVarLayout = CompiledData.VariableLayouts[SrcVarIdx];

				const uint32 FloatComponents = SrcVarLayout.GetNumFloatComponents();
				check(FloatComponents == DestVarLayout.GetNumFloatComponents());
				for (uint32 CompIdx = 0; CompIdx < FloatComponents; ++CompIdx)
				{
					const int32 SrcCompOffset = SrcVarLayout.GetFloatComponentStart() + CompIdx;
					const int32 DestCompOffest = DestVarLayout.GetFloatComponentStart() + CompIdx;
					const float* SrcStart = GetInstancePtrFloat(SrcCompOffset, StartIdx);
					const float* SrcEnd = GetInstancePtrFloat(SrcCompOffset, StartIdx + InstancesToCopy);
					float* Dst = DestBuffer.GetInstancePtrFloat(DestCompOffest, DestStartIdx);
					size_t Count = SrcEnd - SrcStart;
					FMemory::Memcpy(Dst, SrcStart, Count * sizeof(float));

					if (Count > 0)
					{
						for (size_t i = 0; i < Count; i++)
						{
							checkSlow(SrcStart[i] == Dst[i]);
						}
					}
				}
				const uint32 IntComponents = SrcVarLayout.GetNumInt32Components();
				check(IntComponents == DestVarLayout.GetNumInt32Components());
				for (uint32 CompIdx = 0; CompIdx < IntComponents; ++CompIdx)
				{
					const int32 SrcCompOffset = SrcVarLayout.GetInt32ComponentStart() + CompIdx;
					const int32 DestCompOffest = DestVarLayout.GetInt32ComponentStart() + CompIdx;
					const int32* SrcStart = GetInstancePtrInt32(SrcCompOffset, StartIdx);
					const int32* SrcEnd = GetInstancePtrInt32(SrcCompOffset, StartIdx + InstancesToCopy);
					int32* Dst = DestBuffer.GetInstancePtrInt32(DestCompOffest, DestStartIdx);
					size_t Count = SrcEnd - SrcStart;
					FMemory::Memcpy(Dst, SrcStart, Count * sizeof(int32));

					if (Count > 0)
					{
						for (size_t i = 0; i < Count; i++)
						{
							checkSlow(SrcStart[i] == Dst[i]);
						}
					}
				}
				const uint32 HalfComponents = SrcVarLayout.GetNumHalfComponents();
				check(HalfComponents == DestVarLayout.GetNumHalfComponents());
				for (uint32 CompIdx = 0; CompIdx < HalfComponents; ++CompIdx)
				{
					const int32 SrcCompOffset = SrcVarLayout.GetHalfComponentStart() + CompIdx;
					const int32 DestCompOffest = DestVarLayout.GetHalfComponentStart() + CompIdx;
					const FFloat16* SrcStart = GetInstancePtrHalf(SrcCompOffset, StartIdx);
					const FFloat16* SrcEnd = GetInstancePtrHalf(SrcCompOffset, StartIdx + InstancesToCopy);
					FFloat16* Dst = DestBuffer.GetInstancePtrHalf(DestCompOffest, DestStartIdx);
					size_t Count = SrcEnd - SrcStart;
					FMemory::Memcpy(Dst, SrcStart, Count * sizeof(FFloat16));

					if (Count > 0)
					{
						for (size_t i = 0; i < Count; i++)
						{
							checkSlow(SrcStart[i] == Dst[i]);
						}
					}
				}
			}
			else
			{
				const uint32 FloatComponents = DestVarLayout.GetNumFloatComponents();
				for (uint32 CompIdx = 0; CompIdx < FloatComponents; ++CompIdx)
				{
					const int32 DestCompOffest = DestVarLayout.GetFloatComponentStart() + CompIdx;
					float* Dst = DestBuffer.GetInstancePtrFloat(DestCompOffest, DestStartIdx);
					FMemory::Memzero(Dst, InstancesToCopy * sizeof(float));
				}

				const uint32 IntComponents = DestVarLayout.GetNumInt32Components();
				for (uint32 CompIdx = 0; CompIdx < IntComponents; ++CompIdx)
				{
					const int32 DestCompOffest = DestVarLayout.GetInt32ComponentStart() + CompIdx;
					int32* Dst = DestBuffer.GetInstancePtrInt32(DestCompOffest, DestStartIdx);
					FMemory::Memzero(Dst, InstancesToCopy * sizeof(int32));
				}

				const uint32 HalfComponents = DestVarLayout.GetNumHalfComponents();
				for (uint32 CompIdx = 0; CompIdx < HalfComponents; ++CompIdx)
				{
					const int32 DestCompOffest = DestVarLayout.GetHalfComponentStart() + CompIdx;
					FFloat16* Dst = DestBuffer.GetInstancePtrHalf(DestCompOffest, DestStartIdx);
					FMemory::Memzero(Dst, InstancesToCopy * sizeof(FFloat16));
				}
			}
		}
	}
}

void FNiagaraDataBuffer::CopyTo(FNiagaraDataBuffer& DestBuffer, int32 StartIdx, int32 DestStartIdx, int32 InNumInstances)const
{
	CheckUsage(false);

	if (StartIdx < 0 || (uint32)StartIdx >= NumInstances)
	{
		return;
	}

	uint32 InstancesToCopy = InNumInstances;
	if (InstancesToCopy == INDEX_NONE)
	{
		InstancesToCopy = NumInstances - StartIdx;
	}

	if (InstancesToCopy != 0)
	{
		const uint32 NewNumInstances = DestStartIdx + InstancesToCopy;

		// Only allocate if we need to increase the number of instances as the caller may have previously
		// allocated the array and may not be expecting it to shrink inside this call.
		if (DestStartIdx < 0 || NewNumInstances >= DestBuffer.GetNumInstancesAllocated())
		{
			DestBuffer.Allocate(NewNumInstances, true);
		}
		DestBuffer.SetNumInstances(NewNumInstances);

		uint32 FloatComponents = Owner->GetNumFloatComponents();
		for (uint32 CompIdx = 0; CompIdx < FloatComponents; ++CompIdx)
		{
			const float* SrcStart = GetInstancePtrFloat(CompIdx, StartIdx);
			const float* SrcEnd = GetInstancePtrFloat(CompIdx, StartIdx + InstancesToCopy);
			float* Dst = DestBuffer.GetInstancePtrFloat(CompIdx, DestStartIdx);
			size_t Count = SrcEnd - SrcStart;
			FMemory::Memcpy(Dst, SrcStart, Count*sizeof(float));

			if (Count > 0)
			{
				for (size_t i = 0; i < Count; i++)
				{
					checkSlow(SrcStart[i] == Dst[i]);
				}
			}
		}
		uint32 IntComponents = Owner->GetNumInt32Components();
		for (uint32 CompIdx = 0; CompIdx < IntComponents; ++CompIdx)
		{
			const int32* SrcStart = GetInstancePtrInt32(CompIdx, StartIdx);
			const int32* SrcEnd = GetInstancePtrInt32(CompIdx, StartIdx + InstancesToCopy);
			int32* Dst = DestBuffer.GetInstancePtrInt32(CompIdx, DestStartIdx);
			size_t Count = SrcEnd - SrcStart;
			FMemory::Memcpy(Dst, SrcStart, Count * sizeof(int32));

			if (Count > 0)
			{
				for (size_t i = 0; i < Count; i++)
				{
					checkSlow(SrcStart[i] == Dst[i]);
				}
			}
		}
		uint32 HalfComponents = Owner->GetNumHalfComponents();
		for (uint32 CompIdx = 0; CompIdx < HalfComponents; ++CompIdx)
		{
			const FFloat16* SrcStart = GetInstancePtrHalf(CompIdx, StartIdx);
			const FFloat16* SrcEnd = GetInstancePtrHalf(CompIdx, StartIdx + InstancesToCopy);
			FFloat16* Dst = DestBuffer.GetInstancePtrHalf(CompIdx, DestStartIdx);
			size_t Count = SrcEnd - SrcStart;
			FMemory::Memcpy(Dst, SrcStart, Count * sizeof(FFloat16));

			if (Count > 0)
			{
				for (size_t i = 0; i < Count; i++)
				{
					checkSlow(SrcStart[i] == Dst[i]);
				}
			}
		}
	}
}

void FNiagaraDataBuffer::GPUCopyFrom(const float* GPUReadBackFloat, const int* GPUReadBackInt, const FFloat16* GPUReadBackHalf, int32 InStartIdx, int32 InNumInstances, uint32 InSrcFloatStride, uint32 InSrcIntStride, uint32 InSrcHalfStride)
{
	//CheckUsage(false); //Have to disable this as in this specific case we write to a "CPUSim" from the RT.

	Allocate(InNumInstances);
	SetNumInstances(InNumInstances);

	if (InNumInstances <= 0)
	{
		return;
	}

	if (GPUReadBackFloat)
	{
		uint32 FloatComponents = Owner->GetNumFloatComponents();
		for (uint32 CompIdx = 0; CompIdx < FloatComponents; ++CompIdx)
		{
			// We have to reimplement the logic from GetInstancePtrFloat here because the incoming stride may be different than this 
			// data buffer's stride.
			const float* SrcStart = (const float*)((uint8*)GPUReadBackFloat + InSrcFloatStride * CompIdx) + InStartIdx; 
			const float* SrcEnd = (const float*)((uint8*)GPUReadBackFloat + InSrcFloatStride * CompIdx) + InStartIdx + InNumInstances;
			float* Dst = GetInstancePtrFloat(CompIdx, 0);
			size_t Count = SrcEnd - SrcStart;
			FMemory::Memcpy(Dst, SrcStart, Count * sizeof(float));
		}
	}
	if (GPUReadBackInt)
	{
		uint32 IntComponents = Owner->GetNumInt32Components();
		for (uint32 CompIdx = 0; CompIdx < IntComponents; ++CompIdx)
		{
			// We have to reimplement the logic from GetInstancePtrInt here because the incoming stride may be different than this 
			// data buffer's stride.
			const int32* SrcStart = (const int32*)((uint8*)GPUReadBackInt + InSrcIntStride * CompIdx) + InStartIdx;
			const int32* SrcEnd = (const int32*)((uint8*)GPUReadBackInt + InSrcIntStride * CompIdx) + InStartIdx + InNumInstances;
			int32* Dst = GetInstancePtrInt32(CompIdx, 0);
			size_t Count = SrcEnd - SrcStart;
			FMemory::Memcpy(Dst, SrcStart, Count * sizeof(int32));
		}
	}
	if (GPUReadBackHalf)
	{
		uint32 HalfComponents = Owner->GetNumHalfComponents();
		for (uint32 CompIdx = 0; CompIdx < HalfComponents; ++CompIdx)
		{
			// We have to reimplement the logic from GetInstancePtrInt here because the incoming stride may be different than this 
			// data buffer's stride.
			const FFloat16* SrcStart = (const FFloat16*)((uint8*)GPUReadBackHalf + InSrcHalfStride * CompIdx) + InStartIdx;
			const FFloat16* SrcEnd = (const FFloat16*)((uint8*)GPUReadBackHalf + InSrcHalfStride * CompIdx) + InStartIdx + InNumInstances;
			FFloat16* Dst = GetInstancePtrHalf(CompIdx, 0);
			size_t Count = SrcEnd - SrcStart;
			FMemory::Memcpy(Dst, SrcStart, Count * sizeof(FFloat16));
		}
	}
}

void FNiagaraDataBuffer::PushCPUBuffersToGPU(const TArray<FNiagaraDataBufferRef>& SourceBuffers, bool bReleaseRef, FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, const TCHAR* DebugSimName, bool bAllocate)
{
	uint32 NewCount = 0;
	check(GetOwner()->GetSimTarget() == ENiagaraSimTarget::GPUComputeSim);
	
	bool bIdenticalLayout = true;
	for (FNiagaraDataBuffer* Buffer : SourceBuffers)
	{
		if (Buffer)
		{
			NewCount += Buffer->GetNumInstances();

			bIdenticalLayout &= Buffer->GetOwner()->GetCompiledData().GetLayoutHash() == GetOwner()->GetCompiledData().GetLayoutHash();
		}
	}

	if(bAllocate)
	{
		AllocateGPU(RHICmdList, NewCount, FeatureLevel, DebugSimName);
	}
	SetNumInstances(NewCount);

	if (NewCount > 0)
	{
		uint32 FloatComponents = Owner->GetNumFloatComponents();
		uint32 IntComponents = Owner->GetNumInt32Components();
		uint32 HalfComponents = Owner->GetNumHalfComponents();

		uint8* MappedBufferFloat = GPUBufferFloat.Buffer ? (uint8*)RHICmdList.LockBuffer(GPUBufferFloat.Buffer, 0, FloatComponents * FloatStride, RLM_WriteOnly) : nullptr;
		uint8* MappedBufferInt32 = GPUBufferInt.Buffer ? (uint8*)RHICmdList.LockBuffer(GPUBufferInt.Buffer, 0, IntComponents * Int32Stride, RLM_WriteOnly) : nullptr;
		uint8* MappedBufferHalf = GPUBufferHalf.Buffer ? (uint8*)RHICmdList.LockBuffer(GPUBufferHalf.Buffer, 0, HalfComponents * HalfStride, RLM_WriteOnly) : nullptr;

		if(bIdenticalLayout)
		{
			//We have identical layout between all src buffers and the dest buffer so we can copy easily without worrying about missing variables.
			for (uint32 CompIdx = 0; CompIdx < FloatComponents; ++CompIdx)
			{
				float* Dest = (float*)MappedBufferFloat;
				MappedBufferFloat += FloatStride;
				for (FNiagaraDataBuffer* Buffer : SourceBuffers)
				{
					const float* Src = Buffer->GetInstancePtrFloat(CompIdx, 0);
					int32 CopyInstances = Buffer->GetNumInstances();
					FMemory::Memcpy(Dest, Src, CopyInstances * sizeof(float));
					Dest += CopyInstances;
				}
			}

			for (uint32 CompIdx = 0; CompIdx < IntComponents; ++CompIdx)
			{
				int32* Dest = (int32*)MappedBufferInt32;
				MappedBufferInt32 += Int32Stride;
				for (FNiagaraDataBuffer* Buffer : SourceBuffers)
				{
					const int32* Src = Buffer->GetInstancePtrInt32(CompIdx, 0);
					int32 CopyInstances = Buffer->GetNumInstances();
					FMemory::Memcpy(Dest, Src, CopyInstances * sizeof(int32));
					Dest += CopyInstances;
				}
			}

			for (uint32 CompIdx = 0; CompIdx < HalfComponents; ++CompIdx)
			{
				FFloat16* Dest = (FFloat16*)MappedBufferHalf;
				MappedBufferHalf += HalfStride;
				for (FNiagaraDataBuffer* Buffer : SourceBuffers)
				{
					const FFloat16* Src = Buffer->GetInstancePtrHalf(CompIdx, 0);
					int32 CopyInstances = Buffer->GetNumInstances();
					FMemory::Memcpy(Dest, Src, CopyInstances * sizeof(FFloat16));
					Dest += CopyInstances;
				}
			}
		}
		else
		{
			//The layouts between src and dest buffer locations don't perfectly match so we must go variable by variable and copy over those that exist in the src data.
			//Zeroing any that do not.
			const FNiagaraDataSetCompiledData& CompiledData = GetOwner()->GetCompiledData();
			
			for (int32 VarIdx = 0; VarIdx < CompiledData.Variables.Num(); ++VarIdx)
			{
				const FNiagaraVariableBase& DestVar = CompiledData.Variables[VarIdx];
				const FNiagaraVariableLayoutInfo& DestVarLayout = CompiledData.VariableLayouts[VarIdx];

				uint32 DestInstStart = 0;
				for (FNiagaraDataBuffer* SrcBuffer : SourceBuffers)
				{
					if(SrcBuffer->GetNumInstances() == 0)
					{
						continue;
					}

					const FNiagaraDataSetCompiledData& SrcCompiledData = SrcBuffer->GetOwner()->GetCompiledData();

					const int32 SrcVarIdx = SrcCompiledData.Variables.IndexOfByKey(DestVar);
					if (SrcVarIdx != INDEX_NONE)
					{
						//The variable was found so memcpy from the relevant src->dest buffer locations.
						const FNiagaraVariableLayoutInfo& SrcVarLayout = SrcCompiledData.VariableLayouts[SrcVarIdx];
						check(SrcVarLayout.GetNumFloatComponents() == DestVarLayout.GetNumFloatComponents());
						for (uint32 CompIdx = 0; CompIdx < DestVarLayout.GetNumFloatComponents(); ++CompIdx)
						{
							const int32 SrcCompOffset = SrcVarLayout.GetFloatComponentStart() + CompIdx;
							const int32 DestCompOffest = DestVarLayout.GetFloatComponentStart() + CompIdx;
							const float* SrcStart = SrcBuffer->GetInstancePtrFloat(SrcCompOffset, 0);

							float* Dst = (float*)(MappedBufferFloat + (DestCompOffest * FloatStride)) + DestInstStart;
							FMemory::Memcpy(Dst, SrcStart, SrcBuffer->GetNumInstances() * sizeof(float));
						}
						check(SrcVarLayout.GetNumInt32Components() == DestVarLayout.GetNumInt32Components());
						for (uint32 CompIdx = 0; CompIdx < DestVarLayout.GetNumInt32Components(); ++CompIdx)
						{
							const int32 SrcCompOffset = SrcVarLayout.GetInt32ComponentStart() + CompIdx;
							const int32 DestCompOffest = DestVarLayout.GetInt32ComponentStart() + CompIdx;
							const int32* SrcStart = SrcBuffer->GetInstancePtrInt32(SrcCompOffset, 0);

							int32* Dst = (int32*)(MappedBufferInt32 + (DestCompOffest * Int32Stride)) + DestInstStart;
							FMemory::Memcpy(Dst, SrcStart, SrcBuffer->GetNumInstances() * sizeof(int32));
						}
						check(SrcVarLayout.GetNumHalfComponents() == DestVarLayout.GetNumHalfComponents());
						for (uint32 CompIdx = 0; CompIdx < DestVarLayout.GetNumHalfComponents(); ++CompIdx)
						{
							const int32 SrcCompOffset = SrcVarLayout.GetHalfComponentStart() + CompIdx;
							const int32 DestCompOffest = DestVarLayout.GetHalfComponentStart() + CompIdx;
							const FFloat16* SrcStart = SrcBuffer->GetInstancePtrHalf(SrcCompOffset, 0);

							FFloat16* Dst = (FFloat16*)(MappedBufferHalf + (DestCompOffest * HalfStride)) + DestInstStart;
							FMemory::Memcpy(Dst, SrcStart, SrcBuffer->GetNumInstances() * sizeof(FFloat16));
						}
					}
					else
					{
						//This variable is not present in the source buffer so zero this out.
						for (uint32 CompIdx = 0; CompIdx < DestVarLayout.GetNumFloatComponents(); ++CompIdx)
						{
							const int32 DestCompOffest = DestVarLayout.GetFloatComponentStart() + CompIdx;
							float* Dst = (float*)(MappedBufferFloat + (DestCompOffest * FloatStride)) + DestInstStart;
							FMemory::Memzero(Dst, SrcBuffer->GetNumInstances() * sizeof(float));
						}
						for (uint32 CompIdx = 0; CompIdx < DestVarLayout.GetNumInt32Components(); ++CompIdx)
						{
							const int32 DestCompOffest = DestVarLayout.GetInt32ComponentStart() + CompIdx;
							int32* Dst = (int32*)(MappedBufferInt32 + (DestCompOffest * Int32Stride)) + DestInstStart;
							FMemory::Memzero(Dst, SrcBuffer->GetNumInstances() * sizeof(int32));
						}
						for (uint32 CompIdx = 0; CompIdx < DestVarLayout.GetNumHalfComponents(); ++CompIdx)
						{
							const int32 DestCompOffest = DestVarLayout.GetHalfComponentStart() + CompIdx;
							FFloat16* Dst = (FFloat16*)(MappedBufferHalf + (DestCompOffest * HalfStride)) + DestInstStart;
							FMemory::Memzero(Dst, SrcBuffer->GetNumInstances() * sizeof(FFloat16));
						}
					}

					DestInstStart += SrcBuffer->GetNumInstances();
				}
			}
		}

		if (MappedBufferFloat)
		{
			RHICmdList.UnlockBuffer(GPUBufferFloat.Buffer);
		}

		if (MappedBufferInt32)
		{
			RHICmdList.UnlockBuffer(GPUBufferInt.Buffer);
		}

		if (MappedBufferHalf)
		{
			RHICmdList.UnlockBuffer(GPUBufferHalf.Buffer);
		}
	}
}

void FNiagaraDataBuffer::TransferGPUToCPUImmediate(FRHICommandListImmediate& RHICmdList, FNiagaraGpuComputeDispatchInterface* ComputeInterface, FNiagaraDataBuffer* CPUBuffer) const
{
	check(GetOwner()->GetCompiledData().GetLayoutHash() == CPUBuffer->GetOwner()->GetCompiledData().GetLayoutHash());

	// No count offset means no data
	if (GPUInstanceCountBufferOffset == INDEX_NONE)
	{
		CPUBuffer->Allocate(0);
		CPUBuffer->SetNumInstances(0);
		return;
	}

	// Enqueue buffers readbacks
	FStagingBufferRHIRef CountReadbackBuffer = RHICreateStagingBuffer();
	{
		FNiagaraGPUInstanceCountManager& CountManager = ComputeInterface->GetGPUInstanceCounterManager();
		RHICmdList.Transition(FRHITransitionInfo(CountManager.GetInstanceCountBuffer().Buffer, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState, ERHIAccess::CopySrc));
		RHICmdList.CopyToStagingBuffer(CountManager.GetInstanceCountBuffer().Buffer, CountReadbackBuffer, GPUInstanceCountBufferOffset * sizeof(int32), sizeof(int32));
		RHICmdList.Transition(FRHITransitionInfo(CountManager.GetInstanceCountBuffer().Buffer, ERHIAccess::CopySrc, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState));
	}

	FStagingBufferRHIRef FloatReadbackBuffer = nullptr;
	if (GPUBufferFloat.NumBytes > 0)
	{
		FloatReadbackBuffer = RHICreateStagingBuffer();
		RHICmdList.Transition(FRHITransitionInfo(GPUBufferFloat.Buffer, ERHIAccess::SRVMask, ERHIAccess::CopySrc));
		RHICmdList.CopyToStagingBuffer(GPUBufferFloat.Buffer, FloatReadbackBuffer, 0, GPUBufferFloat.NumBytes);
		RHICmdList.Transition(FRHITransitionInfo(GPUBufferFloat.Buffer, ERHIAccess::CopySrc, ERHIAccess::SRVMask));
	}

	FStagingBufferRHIRef HalfReadbackBuffer = nullptr;
	if (GPUBufferHalf.NumBytes > 0)
	{
		HalfReadbackBuffer = RHICreateStagingBuffer();
		RHICmdList.Transition(FRHITransitionInfo(GPUBufferHalf.Buffer, ERHIAccess::SRVMask, ERHIAccess::CopySrc));
		RHICmdList.CopyToStagingBuffer(GPUBufferHalf.Buffer, HalfReadbackBuffer, 0, GPUBufferHalf.NumBytes);
		RHICmdList.Transition(FRHITransitionInfo(GPUBufferHalf.Buffer, ERHIAccess::CopySrc, ERHIAccess::SRVMask));
	}

	FStagingBufferRHIRef Int32ReadbackBuffer = nullptr;
	if (GPUBufferInt.NumBytes > 0)
	{
		Int32ReadbackBuffer = RHICreateStagingBuffer();
		RHICmdList.Transition(FRHITransitionInfo(GPUBufferInt.Buffer, ERHIAccess::SRVMask, ERHIAccess::CopySrc));
		RHICmdList.CopyToStagingBuffer(GPUBufferInt.Buffer, Int32ReadbackBuffer, 0, GPUBufferInt.NumBytes);
		RHICmdList.Transition(FRHITransitionInfo(GPUBufferInt.Buffer, ERHIAccess::CopySrc, ERHIAccess::SRVMask));
	}

	// Ensure all readbacks are submitted and complete
	RHICmdList.SubmitAndBlockUntilGPUIdle();

	// Read the count and allocato space for results
	uint32 GPUNumInstances = 0;
	{
		const uint32* DataPtr = static_cast<const uint32*>(RHICmdList.LockStagingBuffer(CountReadbackBuffer, 0, sizeof(int32)));
		GPUNumInstances = *DataPtr;
		RHICmdList.UnlockStagingBuffer(CountReadbackBuffer);
	}

	CPUBuffer->Allocate(GPUNumInstances);
	CPUBuffer->SetNumInstances(GPUNumInstances);
	if (GPUNumInstances == 0)
	{
		return;
	}

	// Copy out buffers
	if (FloatReadbackBuffer)
	{
		const uint32 GPUStride	= FloatStride;
		const uint8* GPUData	= reinterpret_cast<uint8*>(RHICmdList.LockStagingBuffer(FloatReadbackBuffer, 0, GPUBufferFloat.NumBytes));

		const uint32 CPUStride	= CPUBuffer->GetFloatStride();
		uint8* CPUData			= CPUBuffer->GetComponentPtrFloat(0);

		const int32 NumComponents = CPUBuffer->GetOwner()->GetNumFloatComponents();
		for (int32 i = 0; i < NumComponents; ++i)
		{
			FMemory::Memcpy(CPUData + (CPUStride * i), GPUData + (GPUStride * i), GPUNumInstances * sizeof(float));
		}

		RHICmdList.UnlockStagingBuffer(FloatReadbackBuffer);
	}

	if (HalfReadbackBuffer)
	{
		const uint32 GPUStride = HalfStride;
		const uint8* GPUData = reinterpret_cast<uint8*>(RHICmdList.LockStagingBuffer(HalfReadbackBuffer, 0, GPUBufferHalf.NumBytes));

		const uint32 CPUStride = CPUBuffer->GetHalfStride();
		uint8* CPUData = CPUBuffer->GetComponentPtrHalf(0);

		const int32 NumComponents = CPUBuffer->GetOwner()->GetNumHalfComponents();
		for (int32 i = 0; i < NumComponents; ++i)
		{
			FMemory::Memcpy(CPUData + (CPUStride * i), GPUData + (GPUStride * i), GPUNumInstances * sizeof(FFloat16));
		}

		RHICmdList.UnlockStagingBuffer(HalfReadbackBuffer);
	}

	if (Int32ReadbackBuffer)
	{
		const uint32 GPUStride = Int32Stride;
		const uint8* GPUData = reinterpret_cast<uint8*>(RHICmdList.LockStagingBuffer(Int32ReadbackBuffer, 0, GPUBufferInt.NumBytes));

		const uint32 CPUStride = CPUBuffer->GetInt32Stride();
		uint8* CPUData = CPUBuffer->GetComponentPtrInt32(0);

		const int32 NumComponents = CPUBuffer->GetOwner()->GetNumInt32Components();
		for (int32 i = 0; i < NumComponents; ++i)
		{
			FMemory::Memcpy(CPUData + (CPUStride * i), GPUData + (GPUStride * i), GPUNumInstances * sizeof(int32));
		}

		RHICmdList.UnlockStagingBuffer(Int32ReadbackBuffer);
	}
}

void FNiagaraDataBuffer::Dump(int32 StartIndex, int32 InNumInstances, const FString& Label, const FName& SortParameterKey)const
{
	FNiagaraDataVariableIterator Itr(this, StartIndex);

	if (InNumInstances == INDEX_NONE)
	{
		InNumInstances = GetNumInstances();
		InNumInstances -= StartIndex;
	}

	int32 NumInstancesDumped = 0;
	TArray<TPair<float, FString>> Lines;
	Lines.Reserve(GetNumInstances());
	bool bKeyFound = false;
	int32 MaxKeySize = 0;
	while (Itr.IsValid() && NumInstancesDumped < InNumInstances)
	{
		Itr.Get();

		FString Line = TEXT("| ");
		float Key = 0.0f;
		for (const FNiagaraVariable& Var : Itr.GetVariables())
		{
			Line += Var.ToString() + TEXT(" | ");
			if (SortParameterKey.IsNone() == false && Var.GetName() == SortParameterKey)
			{
				// Dump value to string so that we can sort on it later.
				if (Var.GetType() == FNiagaraTypeDefinition::GetFloatDef())
				{
					Key = Var.GetValue<float>();
					bKeyFound = true;
				}
				else if (Var.GetType() == FNiagaraTypeDefinition::GetIntDef())
				{
					Key = (float)Var.GetValue<int32>();
					bKeyFound = true;
				}
			}
		}
		Lines.Add(TPair<float, FString>(Key, Line));
		Itr.Advance();
		NumInstancesDumped++;
	}

	static FString Sep;
	if (Sep.Len() == 0)
	{
		for (int32 i = 0; i < 50; ++i)
		{
			Sep.AppendChar(TEXT('='));
		}
	}

	if (bKeyFound)
	{
		Lines.StableSort([](const TPair<float, FString>& Lhs, const TPair<float, FString>& Rhs)
			{
				return Lhs.Key < Rhs.Key;
			});
	}

	UE_LOGF(LogNiagara, Log, "%ls", *Sep);
	UE_LOGF(LogNiagara, Log, " %ls ", *Label);
	UE_LOGF(LogNiagara, Log, "%ls", *Sep);
	// 	UE_LOGF(LogNiagara, Log, "%ls", *HeaderStr);
	// 	UE_LOGF(LogNiagara, Log, "%ls", *Sep);
	if (bKeyFound)
	{
		for (TPair<float, FString>& Str : Lines)
		{
			UE_LOGF(LogNiagara, Log, "[%f] %ls", Str.Key, *Str.Value);
		}
	}
	else
	{
		for (TPair<float, FString>& Str : Lines)
	{
			UE_LOGF(LogNiagara, Log, "%ls", *Str.Value);
		}
	}
	if (IDToIndexTable.Num() > 0)
	{
		UE_LOGF(LogNiagara, Log, "== ID Table ==");
		for (int32 i = 0; i < IDToIndexTable.Num(); ++i)
		{
			UE_LOGF(LogNiagara, Log, "%d = %d", i, IDToIndexTable[i]);
		}
	}
	UE_LOGF(LogNiagara, Log, "%ls", *Sep);

}

#if WITH_EDITOR
void FScopedNiagaraDataSetGPUReadback::ReadbackData(FNiagaraGpuComputeDispatchInterface* InComputeDispatchInterface, FNiagaraDataSet* InDataSet)
{
	check(DataSet == nullptr);
	check(InDataSet != nullptr);

	ComputeDispatchInterface = InComputeDispatchInterface && !InComputeDispatchInterface->IsPendingKill() ? ComputeDispatchInterface : nullptr;
	DataSet = InDataSet;
	DataBuffer = DataSet->GetCurrentData();

	// These should be zero if we are GPU and aren't inside a readback scope already
	check((DataBuffer->FloatData.Num() == 0) && (DataBuffer->Int32Data.Num() == 0));

	// Readback data
	ENQUEUE_RENDER_COMMAND(ReadbackGPUBuffers)
	(
		[&](FRHICommandListImmediate& RHICmdList)
		{
			// Read DrawIndirect Params
			const uint32 BufferOffset = DataBuffer->GetGPUInstanceCountBufferOffset();
			if (ComputeDispatchInterface && BufferOffset != INDEX_NONE)
			{
				FRHIBuffer* InstanceCountBuffer = ComputeDispatchInterface->GetGPUInstanceCounterManager().GetInstanceCountBuffer().Buffer;

				void* Data = RHICmdList.LockBuffer(InstanceCountBuffer, 0, (BufferOffset + 1) * sizeof(int32), RLM_ReadOnly);
				NumInstances = reinterpret_cast<int32*>(Data)[BufferOffset];
				RHICmdList.UnlockBuffer(InstanceCountBuffer);
			}
			else
			{
				NumInstances = DataBuffer->GetNumInstances();
			}

			// Read float data
			const FRWBuffer& GPUFloatBuffer = DataBuffer->GetGPUBufferFloat();
			if (GPUFloatBuffer.Buffer.IsValid())
			{
				DataBuffer->FloatData.AddUninitialized(GPUFloatBuffer.NumBytes);

				void* CPUFloatBuffer = RHICmdList.LockBuffer(GPUFloatBuffer.Buffer, 0, GPUFloatBuffer.NumBytes, RLM_ReadOnly);
				FMemory::Memcpy(DataBuffer->FloatData.GetData(), CPUFloatBuffer, GPUFloatBuffer.NumBytes);
				RHICmdList.UnlockBuffer(GPUFloatBuffer.Buffer);
			}

			// Read int data
			const FRWBuffer& GPUIntBuffer = DataBuffer->GetGPUBufferInt();
			if (GPUIntBuffer.Buffer.IsValid())
			{
				DataBuffer->Int32Data.AddUninitialized(GPUIntBuffer.NumBytes);

				void* CPUIntBuffer = RHICmdList.LockBuffer(GPUIntBuffer.Buffer, 0, GPUIntBuffer.NumBytes, RLM_ReadOnly);
				FMemory::Memcpy(DataBuffer->Int32Data.GetData(), CPUIntBuffer, GPUIntBuffer.NumBytes);
				RHICmdList.UnlockBuffer(GPUIntBuffer.Buffer);
			}
	
			// Read half data
			const FRWBuffer& GPUHalfBuffer = DataBuffer->GetGPUBufferHalf();
			if (GPUHalfBuffer.Buffer.IsValid())
			{
				DataBuffer->HalfData.AddUninitialized(GPUHalfBuffer.NumBytes);

				void* CPUHalfBuffer = RHICmdList.LockBuffer(GPUHalfBuffer.Buffer, 0, GPUHalfBuffer.NumBytes, RLM_ReadOnly);
				FMemory::Memcpy(DataBuffer->HalfData.GetData(), CPUHalfBuffer, GPUHalfBuffer.NumBytes);
				RHICmdList.UnlockBuffer(GPUHalfBuffer.Buffer);
			}
	}
	);
	FlushRenderingCommands();
}
#endif

//////////////////////////////////////////////////////////////////////////

void FNiagaraDataBuffer::BuildRegisterTable()
{
	int32 TotalRegisters = Owner->GetNumFloatComponents() + Owner->GetNumInt32Components() + Owner->GetNumHalfComponents();
	RegisterTable.Reset();
	RegisterTable.SetNumUninitialized(TotalRegisters);
	int32 NumRegisters = 0;

	memset(RegisterTypeOffsets, 0, sizeof(RegisterTypeOffsets));
	for (const FNiagaraVariableLayoutInfo& VarLayout : Owner->GetVariableLayouts())
	{
		const int32 NumFloats = VarLayout.GetNumFloatComponents();
		for (int32 CompIdx = 0; CompIdx < NumFloats; ++CompIdx)
		{
			uint32 CompBufferOffset = VarLayout.GetFloatComponentStart() + CompIdx;
			uint32 CompRegisterOffset = VarLayout.LayoutInfo.GetFloatComponentRegisterOffset(CompIdx);
			RegisterTable[NumRegisters + CompRegisterOffset] = (uint8*)GetComponentPtrFloat(CompBufferOffset);
			check(RegisterTable[NumRegisters + CompRegisterOffset]);
		}
		NumRegisters += NumFloats;
	}

	RegisterTypeOffsets[1] = NumRegisters;
	for (const FNiagaraVariableLayoutInfo& VarLayout : Owner->GetVariableLayouts())
	{
		const int32 NumInts = VarLayout.GetNumInt32Components();
		for (int32 CompIdx = 0; CompIdx < NumInts; ++CompIdx)
		{
			uint32 CompBufferOffset = VarLayout.GetInt32ComponentStart() + CompIdx;
			uint32 CompRegisterOffset = VarLayout.LayoutInfo.GetInt32ComponentRegisterOffset(CompIdx);
			RegisterTable[NumRegisters + CompRegisterOffset] = (uint8*)GetComponentPtrInt32(CompBufferOffset);
			check(RegisterTable[NumRegisters + CompRegisterOffset]);
		}
		NumRegisters += NumInts;
	}
		
	RegisterTypeOffsets[2] = NumRegisters;
	for (const FNiagaraVariableLayoutInfo& VarLayout : Owner->GetVariableLayouts())
	{
		const int32 NumHalfs = VarLayout.GetNumHalfComponents();
		for (int32 CompIdx = 0; CompIdx < NumHalfs; ++CompIdx)
		{
			uint32 CompBufferOffset = VarLayout.GetHalfComponentStart() + CompIdx;
			uint32 CompRegisterOffset = VarLayout.LayoutInfo.GetHalfComponentRegisterOffset(CompIdx);
			RegisterTable[NumRegisters + CompRegisterOffset] = (uint8*)GetComponentPtrHalf(CompBufferOffset);
			check(RegisterTable[NumRegisters + CompRegisterOffset]);
		}
		NumRegisters += NumHalfs;
	}

	for (void* Register : RegisterTable)
	{
		check(Register);
	}
}

//////////////////////////////////////////////////////////////////////////

FNiagaraDataSetCompiledData FNiagaraDataSetCompiledData::DummyCompiledData;

const FNiagaraVariableLayoutInfo* FNiagaraDataSetCompiledData::FindVariableLayoutInfo(const FNiagaraVariableBase& VariableDef) const
{
	for ( int i=0; i < Variables.Num(); ++i )
	{
		if ( Variables[i].GetName() == VariableDef.GetName() && Variables[i].GetType() == VariableDef.GetType() )
		{
			return &VariableLayouts[i];
		}
	}
	return nullptr;
}

// note, this method is also implemented in FNiagaraDataSet::BuildLayout()
void FNiagaraDataSetCompiledData::BuildLayout()
{
	VariableLayouts.Empty();
	TotalFloatComponents = 0;
	TotalInt32Components = 0;
	TotalHalfComponents = 0;

	for (const FNiagaraVariableBase& Var : Variables)
	{
		FNiagaraVariableLayoutInfo& VarInfo = VariableLayouts[VariableLayouts.AddDefaulted()];
		VarInfo.LayoutInfo.GenerateLayoutInfo(Var.GetType().GetScriptStruct());
		VarInfo.SetFloatComponentStart(TotalFloatComponents);
		VarInfo.SetInt32ComponentStart(TotalInt32Components);
		VarInfo.SetHalfComponentStart(TotalHalfComponents);
		TotalFloatComponents += VarInfo.GetNumFloatComponents();
		TotalInt32Components += VarInfo.GetNumInt32Components();
		TotalHalfComponents += VarInfo.GetNumHalfComponents();
	}

	//Prime the layout hash for cases this happens at runtime.
	GetLayoutHash();
}

FNiagaraDataSetCompiledData::FNiagaraDataSetCompiledData()
{
	Empty();
}

void FNiagaraDataSetCompiledData::Empty()
{
	CachedLayoutHash = INDEX_NONE;
	bRequiresPersistentIDs = 0;
	TotalFloatComponents = 0;
	TotalInt32Components = 0;
	TotalHalfComponents = 0;
	Variables.Empty();
	VariableLayouts.Empty();
	ID = FNiagaraDataSetID();
	SimTarget = ENiagaraSimTarget::CPUSim;
}
