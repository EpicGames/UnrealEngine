// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraDataSetDebugAccessor.h"
#include "NiagaraDataSet.h"

bool FNiagaraDataSetDebugAccessor::Init(const FNiagaraDataSetCompiledData& CompiledData, FName InVariableName)
{
	VariableName = InVariableName;
	bIsEnum = false;
	bIsBool = false;
	NumComponentsInt32 = 0;
	NumComponentsFloat = 0;
	NumComponentsHalf = 0;
	ComponentIndexInt32 = INDEX_NONE;
	ComponentIndexFloat = INDEX_NONE;
	ComponentIndexHalf = INDEX_NONE;

	for (int32 i = 0; i < CompiledData.Variables.Num(); ++i)
	{
		const FNiagaraVariableBase& Variable = CompiledData.Variables[i];
		if (Variable.GetName() != VariableName)
		{
			continue;
		}

		const FNiagaraVariableLayoutInfo& VariableLayout = CompiledData.VariableLayouts[i];
		NumComponentsInt32 = VariableLayout.GetNumInt32Components();
		NumComponentsFloat = VariableLayout.GetNumFloatComponents();
		NumComponentsHalf = VariableLayout.GetNumHalfComponents();
		ComponentIndexInt32 = VariableLayout.GetInt32ComponentStart();
		ComponentIndexFloat = VariableLayout.GetFloatComponentStart();
		ComponentIndexHalf = VariableLayout.GetHalfComponentStart();
		NiagaraType = Variable.GetType();

		bIsBool = Variable.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef());
		bIsEnum = Variable.GetType().IsEnum();
		return NumComponentsInt32 + NumComponentsFloat + NumComponentsHalf > 0;
	}
	return false;
}

float FNiagaraDataSetDebugAccessor::ReadFloat(const FNiagaraDataBuffer* DataBuffer, uint32 Instance, uint32 Component) const
{
	if (DataBuffer != nullptr && ComponentIndexFloat != INDEX_NONE && Instance < DataBuffer->GetNumInstances() && Component < NumComponentsFloat)
	{
		const float* FloatData = reinterpret_cast<const float*>(DataBuffer->GetComponentPtrFloat(ComponentIndexFloat + Component));
		return FloatData[Instance];
	}
	return 0.0f;
}

float FNiagaraDataSetDebugAccessor::ReadHalf(const FNiagaraDataBuffer* DataBuffer, uint32 Instance, uint32 Component) const
{
	if (DataBuffer != nullptr && ComponentIndexHalf != INDEX_NONE && Instance < DataBuffer->GetNumInstances() && Component < NumComponentsHalf)
	{
		const FFloat16* HalfData = reinterpret_cast<const FFloat16*>(DataBuffer->GetComponentPtrHalf(ComponentIndexHalf + Component));
		return HalfData[Instance];
	}
	return 0.0f;
}

int32 FNiagaraDataSetDebugAccessor::ReadInt(const FNiagaraDataBuffer* DataBuffer, uint32 Instance, uint32 Component) const
{
	if (DataBuffer != nullptr && ComponentIndexInt32 != INDEX_NONE && Instance < DataBuffer->GetNumInstances() && Component < NumComponentsInt32)
	{
		const int32* IntData = reinterpret_cast<const int32*>(DataBuffer->GetComponentPtrInt32(ComponentIndexInt32 + Component));
		return IntData[Instance];
	}
	return 0;
}

bool FNiagaraDataSetDebugAccessor::ValidateDataBuffer(const FNiagaraDataSetCompiledData& CompiledData, const FNiagaraDataBuffer* DataBuffer, uint32 iInstance, TFunction<void(const FNiagaraVariableBase&, int32)> ErrorCallback)
{
	bool bIsValid = true;

	// If it's not a valid index skip it
	if ( iInstance >= DataBuffer->GetNumInstances() )
	{
		return bIsValid;
	}

	// For each variable look at data
	for (int32 iVariable = 0; iVariable < CompiledData.Variables.Num(); ++iVariable)
	{
		// Look over float data
		if (CompiledData.VariableLayouts[iVariable].GetNumFloatComponents() > 0)
		{
			const int32 ComponentIndex = CompiledData.VariableLayouts[iVariable].GetFloatComponentStart();
			const int32 NumComponents = CompiledData.VariableLayouts[iVariable].GetNumFloatComponents();

			for (int32 iComponent = 0; iComponent < NumComponents; ++iComponent)
			{
				const float* FloatData = reinterpret_cast<const float*>(DataBuffer->GetComponentPtrFloat(ComponentIndex + iComponent));
				check(FloatData);

				if (!FMath::IsFinite(FloatData[iInstance]))
				{
					bIsValid = false;
					ErrorCallback(CompiledData.Variables[iVariable], iComponent);
				}
			}
		}
		else if (CompiledData.VariableLayouts[iVariable].GetNumHalfComponents() > 0)
		{
			const int32 ComponentIndex = CompiledData.VariableLayouts[iVariable].GetHalfComponentStart();
			const int32 NumComponents = CompiledData.VariableLayouts[iVariable].GetNumHalfComponents();

			for (int32 iComponent = 0; iComponent < NumComponents; ++iComponent)
			{
				const FFloat16* HalfData = reinterpret_cast<const FFloat16*>(DataBuffer->GetComponentPtrHalf(ComponentIndex + iComponent));
				check(HalfData);

				const float Value = HalfData[iInstance].GetFloat();
				if (!FMath::IsFinite(Value))
				{
					bIsValid = false;
					ErrorCallback(CompiledData.Variables[iVariable], iComponent);
				}
			}
		}
	}

	return bIsValid;
}

bool FNiagaraDataSetDebugAccessor::ValidateDataBuffer(const FNiagaraDataSetCompiledData& CompiledData, const FNiagaraDataBuffer* DataBuffer, TFunction<void(const FNiagaraVariableBase&, uint32, int32)> ErrorCallback)
{
	bool bIsValid = true;

	// For each variable look at data
	for (int32 iVariable=0; iVariable < CompiledData.Variables.Num(); ++iVariable)
	{
		// Look over float data
		if (CompiledData.VariableLayouts[iVariable].GetNumFloatComponents() > 0)
		{
			const int32 ComponentIndex = CompiledData.VariableLayouts[iVariable].GetFloatComponentStart();
			const int32 NumComponents = CompiledData.VariableLayouts[iVariable].GetNumFloatComponents();

			for ( int32 iComponent=0; iComponent < NumComponents; ++iComponent )
			{
				const float* FloatData = reinterpret_cast<const float*>(DataBuffer->GetComponentPtrFloat(ComponentIndex + iComponent));
				check(FloatData);

				for ( uint32 iInstance=0; iInstance < DataBuffer->GetNumInstances(); ++iInstance )
				{
					if ( !FMath::IsFinite(FloatData[iInstance]) )
					{
						bIsValid = false;
						ErrorCallback(CompiledData.Variables[iVariable], iInstance, iComponent);
					}
				}
			}
		}
		else if (CompiledData.VariableLayouts[iVariable].GetNumHalfComponents() > 0)
		{
			const int32 ComponentIndex = CompiledData.VariableLayouts[iVariable].GetHalfComponentStart();
			const int32 NumComponents = CompiledData.VariableLayouts[iVariable].GetNumHalfComponents();

			for (int32 iComponent = 0; iComponent < NumComponents; ++iComponent)
			{
				const FFloat16* HalfData = reinterpret_cast<const FFloat16*>(DataBuffer->GetComponentPtrHalf(ComponentIndex + iComponent));
				check(HalfData);

				for (uint32 iInstance = 0; iInstance < DataBuffer->GetNumInstances(); ++iInstance)
				{
					const float Value = HalfData[iInstance].GetFloat();
					if (!FMath::IsFinite(Value))
					{
						bIsValid = false;
						ErrorCallback(CompiledData.Variables[iVariable], iInstance, iComponent);
					}
				}
			}
		}
	}

	return bIsValid;
}
