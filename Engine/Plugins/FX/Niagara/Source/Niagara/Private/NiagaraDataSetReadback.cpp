// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataSetReadback.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGPUInstanceCountManager.h"
#include "NiagaraGpuReadbackManager.h"
#include "Stateless/NiagaraStatelessEmitterInstance.h"

#include "Async/Async.h"

class FNiagaraEmitterInstance;

void FNiagaraDataSetReadback::SetReadbackRead(FOnReadbackReady InOnReadbackReady)
{
	check(IsReady());
	OnReadbackReady = InOnReadbackReady;
}

void FNiagaraDataSetReadback::EnqueueReadback(FNiagaraEmitterInstance* EmitterInstance)
{
	check(EmitterInstance);
	check(IsReady());

	SourceName = EmitterInstance->GetEmitterHandle().GetName();
	DataSet.Init(&EmitterInstance->GetParticleData().GetCompiledData());
	if ( FNiagaraComputeExecutionContext* GPUExecContext = EmitterInstance->GetGPUContext() )
	{
		ParameterStore = GPUExecContext->CombinedParamStore;

		FNiagaraSystemInstance* SystemInstance = EmitterInstance->GetParentSystemInstance();

		++PendingReadbacks;
		ENQUEUE_RENDER_COMMAND(NiagaraDataSetReadback)
		(
			[RT_DataSetReadback=AsShared(), RT_ComputeDispatchInterface=SystemInstance->GetComputeDispatchInterface(), RT_GPUExecContext=GPUExecContext](FRHICommandListImmediate& RHICmdList)
			{
				RT_DataSetReadback->GPUReadbackInternal(RHICmdList, RT_ComputeDispatchInterface, RT_GPUExecContext);
			}
		);
	}
	else if (FNiagaraEmitterInstanceImpl* StatefulEmitterInstance = EmitterInstance->AsStateful())
	{
		const FNiagaraDataSet& SourceDataSet = EmitterInstance->GetParticleData();
		if (FNiagaraDataBuffer* SourceDataBuffer = SourceDataSet.GetCurrentData())
		{
			EmitterInstance->GetParticleData().CopyTo(DataSet, 0, SourceDataBuffer->GetNumInstances());
		}
		else
		{
			FNiagaraDataBuffer& DestBuffer = DataSet.BeginSimulate();
			DestBuffer.SetNumInstances(0);
			DestBuffer.Allocate(0);
			DataSet.EndSimulate();
		}
		ParameterStore = StatefulEmitterInstance->GetUpdateExecutionContext().Parameters;
	}
	else if (FNiagaraStatelessEmitterInstance* StatelessEmitterInstance = EmitterInstance->AsStateless())
	{
		++PendingReadbacks;

		DataSet.BeginSimulate();
		DataSet.EndSimulate();

		StatelessEmitterInstance->EnqueueParticleDataBufferReadback(
			DataSet.GetCurrentData(),
			[RT_DataSetReadback=AsShared()]()
			{
				RT_DataSetReadback->ReadbackCompleteInternal();
			}
		);
		ParameterStore = FNiagaraParameterStore();
	}
}

void FNiagaraDataSetReadback::ImmediateReadback(FNiagaraEmitterInstance* EmitterInstance)
{
	EnqueueReadback(EmitterInstance);
	if ( !IsReady() )
	{
		FNiagaraSystemInstance* SystemInstance = EmitterInstance->GetParentSystemInstance();
		FNiagaraGpuComputeDispatchInterface* DispatchInterface = SystemInstance->GetComputeDispatchInterface();
		ENQUEUE_RENDER_COMMAND(NiagaraFlushReadback)
		(
			[RT_ComputeDispatchInterface=SystemInstance->GetComputeDispatchInterface()](FRHICommandListImmediate& RHICmdList)
			{
				RT_ComputeDispatchInterface->GetGpuReadbackManager()->WaitCompletion(RHICmdList);
			}
		);
		FlushRenderingCommands();
		check(IsReady());
	}
}

void FNiagaraDataSetReadback::ReadbackCompleteInternal()
{
	if ( OnReadbackReady.IsBound() )
	{
		AsyncTask(
			ENamedThreads::GameThread,
			[Readback=this]()
			{
				Readback->PendingReadbacks--;
				Readback->OnReadbackReady.Execute(*Readback);
			}
		);
	}
	else
	{
		PendingReadbacks--;
	}
}

void FNiagaraDataSetReadback::GPUReadbackInternal(FRHICommandListImmediate& RHICmdList, FNiagaraGpuComputeDispatchInterface* DispatchInterface, FNiagaraComputeExecutionContext* GPUContext)
{
	FNiagaraGpuReadbackManager* ReadbackManager = DispatchInterface->GetGpuReadbackManager();
	FNiagaraDataBuffer* CurrentDataBuffer = GPUContext->MainDataSet->GetCurrentData();
	if (CurrentDataBuffer == nullptr || ReadbackManager == nullptr)
	{
		DataSet.CopyFromGPUReadback(nullptr, nullptr, nullptr, 0, 0, 0, 0, 0);
		ReadbackCompleteInternal();
		return;
	}

	const uint32 CountOffset = CurrentDataBuffer->GetGPUInstanceCountBufferOffset();
	if (CountOffset == INDEX_NONE)
	{
		DataSet.CopyFromGPUReadback(nullptr, nullptr, nullptr, 0, 0, 0, 0, 0);
		ReadbackCompleteInternal();
		return;
	}

	const FNiagaraGPUInstanceCountManager& CountManager = DispatchInterface->GetGPUInstanceCounterManager();
	FRWBuffer& FloatBuffer = CurrentDataBuffer->GetGPUBufferFloat();
	FRWBuffer& HalfBuffer = CurrentDataBuffer->GetGPUBufferHalf();
	FRWBuffer& IntBuffer = CurrentDataBuffer->GetGPUBufferInt();
	FRWBuffer& IDtoIndexBuffer = CurrentDataBuffer->GetGPUIDToIndexTable();

	constexpr int32 NumReadbackBuffers = 5;
	TArray<FNiagaraGpuReadbackManager::FBufferRequest, TInlineAllocator<NumReadbackBuffers>> ReadbackBuffers;
	ReadbackBuffers.Emplace(CountManager.GetInstanceCountBuffer().Buffer, uint32(CountOffset * sizeof(uint32)), uint32(sizeof(uint32)));
	const int32 FloatBufferIndex = FloatBuffer.NumBytes == 0 ? INDEX_NONE : ReadbackBuffers.Emplace(FloatBuffer.Buffer, 0, FloatBuffer.NumBytes);
	const int32 HalfBufferIndex = HalfBuffer.NumBytes == 0 ? INDEX_NONE : ReadbackBuffers.Emplace(HalfBuffer.Buffer, 0, HalfBuffer.NumBytes);
	const int32 IntBufferIndex = IntBuffer.NumBytes == 0 ? INDEX_NONE : ReadbackBuffers.Emplace(IntBuffer.Buffer, 0, IntBuffer.NumBytes);
	const int32 IDtoIndexBufferIndex = IDtoIndexBuffer.NumBytes == 0 ? INDEX_NONE : ReadbackBuffers.Emplace(IDtoIndexBuffer.Buffer, 0, IDtoIndexBuffer.NumBytes);

	const int32 FloatBufferStride = CurrentDataBuffer->GetFloatStride();
	const int32 HalfBufferStride = CurrentDataBuffer->GetHalfStride();
	const int32 IntBufferStride = CurrentDataBuffer->GetInt32Stride();

	// Transition buffers to copy
	TArray<FRHITransitionInfo, TInlineAllocator<NumReadbackBuffers>> Transitions;
	Transitions.Emplace(ReadbackBuffers[0].Buffer, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState, ERHIAccess::CopySrc);
	for (int32 i=1; i < ReadbackBuffers.Num(); ++i)
	{
		Transitions.Emplace(
			ReadbackBuffers[i].Buffer,
			(i == IDtoIndexBufferIndex) ? ERHIAccess::SRVCompute : ERHIAccess::SRVMask,
			ERHIAccess::CopySrc
		);
	}
	RHICmdList.Transition(Transitions);

	// Enqueue readback
	ReadbackManager->EnqueueReadbacks(
		RHICmdList,
		MakeArrayView(ReadbackBuffers),
		[=, Readback=AsShared()](TConstArrayView<TPair<void*, uint32>> BufferData)
		{
			const int32 InstanceCount = *reinterpret_cast<int32*>(BufferData[0].Key);

			// Copy dataset databuffer
			const float* FloatDataBuffer = FloatBufferIndex == INDEX_NONE ? nullptr : reinterpret_cast<float*>(BufferData[FloatBufferIndex].Key);
			const FFloat16* HalfDataBuffer = HalfBufferIndex == INDEX_NONE ? nullptr : reinterpret_cast<FFloat16*>(BufferData[HalfBufferIndex].Key);
			const int32* IntDataBuffer = IntBufferIndex == INDEX_NONE ? nullptr : reinterpret_cast<int32*>(BufferData[IntBufferIndex].Key);
			Readback->DataSet.CopyFromGPUReadback(FloatDataBuffer, IntDataBuffer, HalfDataBuffer, 0, InstanceCount, FloatBufferStride, IntBufferStride, HalfBufferStride);

			// Copy ID to Index table
			if (FNiagaraDataBuffer* CurrentData = Readback->DataSet.GetCurrentData())
			{
				TArray<int32>& IDTable = CurrentData->GetIDTable();

				const int32* IDtoIndexBuffer = IDtoIndexBufferIndex == INDEX_NONE ? nullptr : reinterpret_cast<int32*>(BufferData[IDtoIndexBufferIndex].Key);
				if (IDtoIndexBuffer != nullptr)
				{
					const int32 NumIDs = BufferData[IDtoIndexBufferIndex].Value / sizeof(int32);
					check(NumIDs >= InstanceCount);
					IDTable.SetNumUninitialized(NumIDs);
					FMemory::Memcpy(IDTable.GetData(), IDtoIndexBuffer, NumIDs * sizeof(int32));
				}
				else
				{
					IDTable.Empty();
				}
			}

			Readback->ReadbackCompleteInternal();
		}
	);

	// Transition buffers from copy
	for (FRHITransitionInfo& Transition : Transitions)
	{
		Swap(Transition.AccessBefore, Transition.AccessAfter);
	}
	RHICmdList.Transition(Transitions);
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraDataBufferReadback::EnqueueReadback(FRHICommandList& RHICmdList, FNiagaraDataBufferRef InDataBuffer, FNiagaraGpuReadbackManager* ReadbackManager, FNiagaraGPUInstanceCountManager& InstanceCountManager)
{
	check(InDataBuffer);
	check(IsInRenderingThread());

	bool bHasGPUData = InDataBuffer->GetGPUBufferFloat().NumBytes > 0 || InDataBuffer->GetGPUBufferInt().NumBytes > 0 || InDataBuffer->GetGPUBufferHalf().NumBytes > 0;
	++PendingReadbacks;
	if (bHasGPUData)
	{
		GPUReadbackInternal(RHICmdList, ReadbackManager, InstanceCountManager, InDataBuffer);
	}
	else
	{
		GatherResults(	InDataBuffer->GetNumInstances(), 
						(const float*)InDataBuffer->GetFloatBuffer().GetData(), 
						(const int32*)InDataBuffer->GetInt32Buffer().GetData(), 
						(const FFloat16*)InDataBuffer->GetHalfBuffer().GetData(), 
						InDataBuffer->GetFloatStride(), 
						InDataBuffer->GetInt32Stride(), 
						InDataBuffer->GetHalfStride(),
						InDataBuffer->GetOwner()->GetNumFloatComponents(),
						InDataBuffer->GetOwner()->GetNumInt32Components(),
						InDataBuffer->GetOwner()->GetNumHalfComponents());

		ReadbackCompleteInternal();
	}
}

void FNiagaraDataBufferReadback::ReadbackCompleteInternal()
{	
	if (OnReadbackComplete.IsBound())
	{
		AsyncTask(
			ENamedThreads::GameThread,
			[Readback = AsShared()]()
			{
				Readback->PendingReadbacks--;
				Readback->OnReadbackComplete.Execute(Readback);
			}
		);
	}
	else
	{
		PendingReadbacks--;
	}
}

void FNiagaraDataBufferReadback::ReadResultsToDataBuffer(FNiagaraDataBuffer* DestBuffer)const
{
	check(DestBuffer);

	check(ReadbackData_NumFloatComponents == DestBuffer->GetOwner()->GetNumFloatComponents());
	check(ReadbackData_NumInt32Components == DestBuffer->GetOwner()->GetNumInt32Components());
	check(ReadbackData_NumHalfComponents == DestBuffer->GetOwner()->GetNumHalfComponents());

	DestBuffer->GPUCopyFrom(ReadbackData_Float.GetData(), ReadbackData_Int32.GetData(), ReadbackData_Half.GetData(), 0, ReadbackData_Count, ReadbackData_FloatStride, ReadbackData_Int32Stride, ReadbackData_HalfStride);
}

void FNiagaraDataBufferReadback::GPUReadbackInternal(FRHICommandList& RHICmdList, FNiagaraGpuReadbackManager* ReadbackManager, FNiagaraGPUInstanceCountManager& InstanceCountManager, FNiagaraDataBufferRef SrcDataBuffer)
{
	if (SrcDataBuffer == nullptr || ReadbackManager == nullptr)
	{
		ReadbackCompleteInternal();
		return;
	}

	const uint32 CountOffset = SrcDataBuffer->GetGPUInstanceCountBufferOffset();
	if (CountOffset == INDEX_NONE)
	{
		ReadbackCompleteInternal();
		return;
	}

	FRWBuffer& FloatBuffer = SrcDataBuffer->GetGPUBufferFloat();
	FRWBuffer& HalfBuffer = SrcDataBuffer->GetGPUBufferHalf();
	FRWBuffer& IntBuffer = SrcDataBuffer->GetGPUBufferInt();
	FRWBuffer& IDtoIndexBuffer = SrcDataBuffer->GetGPUIDToIndexTable();

	constexpr int32 NumReadbackBuffers = 5;
	TArray<FNiagaraGpuReadbackManager::FBufferRequest, TInlineAllocator<NumReadbackBuffers>> ReadbackBuffers;
	const int32 CountBufferIndex = ReadbackBuffers.Emplace(InstanceCountManager.GetInstanceCountBuffer().Buffer, sizeof(uint32) * CountOffset, sizeof(uint32));
	const int32 FloatBufferIndex = FloatBuffer.NumBytes == 0 ? INDEX_NONE : ReadbackBuffers.Emplace(FloatBuffer.Buffer, 0, FloatBuffer.NumBytes);
	const int32 HalfBufferIndex = HalfBuffer.NumBytes == 0 ? INDEX_NONE : ReadbackBuffers.Emplace(HalfBuffer.Buffer, 0, HalfBuffer.NumBytes);
	const int32 IntBufferIndex = IntBuffer.NumBytes == 0 ? INDEX_NONE : ReadbackBuffers.Emplace(IntBuffer.Buffer, 0, IntBuffer.NumBytes);
	const int32 IDtoIndexBufferIndex = IDtoIndexBuffer.NumBytes == 0 ? INDEX_NONE : ReadbackBuffers.Emplace(IDtoIndexBuffer.Buffer, 0, IDtoIndexBuffer.NumBytes);

	const int32 FloatBufferStride = SrcDataBuffer->GetFloatStride();
	const int32 HalfBufferStride = SrcDataBuffer->GetHalfStride();
	const int32 IntBufferStride = SrcDataBuffer->GetInt32Stride();

	const int32 NumFloatComponents = SrcDataBuffer->GetOwner()->GetNumFloatComponents();
	const int32 NumInt32Components = SrcDataBuffer->GetOwner()->GetNumInt32Components();
	const int32 NumHalfComponents = SrcDataBuffer->GetOwner()->GetNumHalfComponents();

	int32 SourceNumInstancesAllocated = (int32)SrcDataBuffer->GetNumInstancesAllocated();

	// Transition buffers to copy
	TArray<FRHITransitionInfo, TInlineAllocator<NumReadbackBuffers>> Transitions;
	Transitions.Emplace(ReadbackBuffers[0].Buffer, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState, ERHIAccess::CopySrc);
	for (int32 i = 1; i < ReadbackBuffers.Num(); ++i)
	{
		Transitions.Emplace(
			ReadbackBuffers[i].Buffer,
			(i == IDtoIndexBufferIndex) ? ERHIAccess::SRVCompute : ERHIAccess::SRVMask,
			ERHIAccess::CopySrc
		);
	}
	RHICmdList.Transition(Transitions);

	// Enqueue readback
	ReadbackManager->EnqueueReadbacks(
		RHICmdList,
		MakeArrayView(ReadbackBuffers),
		[=, Readback = AsShared()](TConstArrayView<TPair<void*, uint32>> BufferData)
		{
			int32 InstanceCount = reinterpret_cast<int32*>(BufferData[CountBufferIndex].Key)[0];

			ensure(SourceNumInstancesAllocated >= InstanceCount);
			InstanceCount = FMath::Min(InstanceCount, SourceNumInstancesAllocated);

			// Copy dataset databuffer
			const float* FloatDataBuffer = FloatBufferIndex == INDEX_NONE ? nullptr : reinterpret_cast<float*>(BufferData[FloatBufferIndex].Key);
			const FFloat16* HalfDataBuffer = HalfBufferIndex == INDEX_NONE ? nullptr : reinterpret_cast<FFloat16*>(BufferData[HalfBufferIndex].Key);
			const int32* IntDataBuffer = IntBufferIndex == INDEX_NONE ? nullptr : reinterpret_cast<int32*>(BufferData[IntBufferIndex].Key);
			
			Readback->GatherResults(InstanceCount, FloatDataBuffer, IntDataBuffer, HalfDataBuffer, FloatBufferStride, IntBufferStride, HalfBufferStride, NumFloatComponents, NumInt32Components, NumHalfComponents);

			//TODO: Safely copy IDToIndex Table.
			// Copy ID to Index table
// 			if (SrcDataBuffer)
// 			{
// 				TArray<int32>& IDTable = SrcDataBuffer->GetIDTable();
// 
// 				const int32* IDtoIndexBuffer = IDtoIndexBufferIndex == INDEX_NONE ? nullptr : reinterpret_cast<int32*>(BufferData[IDtoIndexBufferIndex].Key);
// 				if (IDtoIndexBuffer != nullptr)
// 				{
// 					const int32 NumIDs = BufferData[IDtoIndexBufferIndex].Value / sizeof(int32);
// 					check(NumIDs >= InstanceCount);
// 					IDTable.SetNumUninitialized(NumIDs);
// 					FMemory::Memcpy(IDTable.GetData(), IDtoIndexBuffer, NumIDs * sizeof(int32));
// 				}
// 				else
// 				{
// 					IDTable.Empty();
// 				}
// 			}

			Readback->ReadbackCompleteInternal();
		}
	);

	// Transition buffers from copy
	for (FRHITransitionInfo& Transition : Transitions)
	{
		Swap(Transition.AccessBefore, Transition.AccessAfter);
	}
	RHICmdList.Transition(Transitions);
}

void FNiagaraDataBufferReadback::GatherResults(	int32 InstanceCount, const float* FloatData, const int32* Int32Data, const FFloat16* HalfData, 
												int32 FloatStride, int32 Int32Stride, int32 HalfStride,
												int32 NumFloatComponents, int32 NumInt32Components, int32 NumHalfComponents)
{
	//Copy over data to local temp buffers for use to pass over the to GT.
	//It is not safe for us to copy directly into an RT owned data buffer at this point.
	//We must wait until we're on the GT and copy into a GT owned data buffer.
	if(FloatData)
	{
		int32 BufferSize = NumFloatComponents * FloatStride;
		ReadbackData_Float.SetNumUninitialized(BufferSize);
		FMemory::Memcpy(ReadbackData_Float.GetData(), FloatData, BufferSize);
	}

	if(Int32Data)
	{
		int32 BufferSize = NumInt32Components * Int32Stride;
		ReadbackData_Int32.SetNumUninitialized(BufferSize);
		FMemory::Memcpy(ReadbackData_Int32.GetData(), Int32Data, BufferSize);
	}
	
	if(HalfData)
	{
		int32 BufferSize = NumHalfComponents * HalfStride;
		ReadbackData_Half.SetNumUninitialized(BufferSize);
		FMemory::Memcpy(ReadbackData_Half.GetData(), HalfData, BufferSize);
	}

	ReadbackData_Count = InstanceCount;
	ReadbackData_FloatStride = FloatStride;
	ReadbackData_Int32Stride = Int32Stride;
	ReadbackData_HalfStride = HalfStride;
	ReadbackData_NumFloatComponents = NumFloatComponents;
	ReadbackData_NumInt32Components = NumInt32Components;
	ReadbackData_NumHalfComponents = NumHalfComponents;
}