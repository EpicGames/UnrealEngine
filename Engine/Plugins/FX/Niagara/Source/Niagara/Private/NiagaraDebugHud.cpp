// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDebugHud.h"
#include "BatchedElements.h"
#include "Engine/Engine.h"
#include "DrawDebugHelpers.h"
#include "GameFramework/Pawn.h"
#include "NiagaraComponent.h"
#include "NiagaraConstants.h"
#include "NiagaraCullProxyComponent.h"
#include "NiagaraDataSetDebugAccessor.h"
#include "NiagaraDataSetReadback.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraMeshRendererProperties.h"
#include "NiagaraScript.h"
#include "NiagaraScriptRuntimeData.h"
#include "NiagaraSimCache.h"
#include "NiagaraSpriteRendererProperties.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemGpuComputeProxy.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraWorldManager.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/NiagaraStatelessEmitterData.h"

#include "Components/LineBatchComponent.h"
#include "Debug/DebugDrawService.h"
#include "Engine/Canvas.h"
#include "GameFramework/PlayerController.h"
#include "Particles/FXBudget.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "UObject/UObjectIterator.h"

#include "ParticleEmitterInstances.h"

#if WITH_NIAGARA_DEBUGGER

namespace NiagaraDebugLocal
{
	FCriticalSection RTFramesGuard;

	enum class EParameterStoreLocation
	{
		 UserOverride,
		 SystemSpawn,
		 SystemUpdate,
		 EmitterSpawn,
		 EmitterUpdate,
		 EmitterRenderBindings,
	};

	struct FParameterStoreVariable
	{
		EParameterStoreLocation	StoreLocation;
		int32					EmitterIndex = INDEX_NONE;
		FNiagaraVariableBase	Variable;
	};

	struct FCachedVariables
	{
		TWeakObjectPtr<UNiagaraSystem> WeakNiagaraSystem;

		TArray<int32>										EngineVariables;			// Engine varibles that are visible on the HUD, these are special because they are not in the buffers
		TArray<FNiagaraDataSetDebugAccessor>				SystemVariables;			// System & Emitter variables since both are inside the same DataBuffer
		TArray<FParameterStoreVariable>						ParameterStoreVariables;	// Variables Stored inside Parameter Stores
	#if WITH_EDITORONLY_DATA
		TArray<FNiagaraVariable>							StaticVariables;			// Static variables do not exist in cooked builds
	#endif

		TArray<TArray<FNiagaraDataSetDebugAccessor>>		ParticleVariables;			// Per Emitter Particle variables
		TArray<FNiagaraDataSetAccessor<FNiagaraPosition>>	ParticlePositionAccessors;	// Only valid if we have particle attributes
	};

	static const TPair<FString, TFunction<FString(FNiagaraSystemInstance*)>> GEngineVariables[] =
	{
		{TEXT("Engine.LODDistance"),		[](FNiagaraSystemInstance* SystemInstance) -> FString { return FString::Printf(TEXT("%6.2f"), SystemInstance->GetLODDistance()); }},
		{TEXT("Engine.LODFraction"),		[](FNiagaraSystemInstance* SystemInstance) -> FString { return FString::Printf(TEXT("%6.2f"), SystemInstance->GetLODDistance() / SystemInstance->GetMaxLODDistance()); }},
		{TEXT("Engine.System.TickCount"),	[](FNiagaraSystemInstance* SystemInstance) -> FString { return FString::Printf(TEXT("%d"), SystemInstance->GetTickCount()); }},
	};

	static TMap<TWeakObjectPtr<UNiagaraSystem>, FCachedVariables> GCachedSystemVariables;

	FNiagaraDebugHUDSettingsData Settings;

	static FDelegateHandle	GDebugDrawHandle;
	static int32			GDebugDrawHandleUsers = 0;

	static const TCHAR*		GCompileForEditString = TEXT(" [EditMode]");

	static FVector StringToVector(const FString& Arg, const FVector& DefaultValue)
	{
		TArray<FString> Values;
		Arg.ParseIntoArray(Values, TEXT(","));
		FVector OutValue;
		OutValue.X = Values.Num() > 0 ? FCString::Atof(*Values[0]) : DefaultValue.X;
		OutValue.Y = Values.Num() > 1 ? FCString::Atof(*Values[1]) : DefaultValue.Y;
		OutValue.Z = Values.Num() > 2 ? FCString::Atof(*Values[2]) : DefaultValue.Z;
		return OutValue;
	}

	static FVector2f StringToVector2f(const FString& Arg, const FVector2f& DefaultValue)
	{
		TArray<FString> Values;
		Arg.ParseIntoArray(Values, TEXT(","));
		FVector2f OutValue;
		OutValue.X = Values.Num() > 0 ? FCString::Atof(*Values[0]) : DefaultValue.X;
		OutValue.Y = Values.Num() > 1 ? FCString::Atof(*Values[1]) : DefaultValue.Y;
		return OutValue;
	}

	static ENiagaraDebugHudFont StringToFont(const FString& Arg)
	{
		return FMath::Clamp(ENiagaraDebugHudFont(FCString::Atoi(*Arg)), ENiagaraDebugHudFont::Small, ENiagaraDebugHudFont::Normal);
	}

	static TTuple<const TCHAR*, const TCHAR*, TFunction<void(FString)>> GDebugConsoleCommands[] =
	{
		// Main HUD commands
		MakeTuple(TEXT("Enabled="), TEXT("Enable or disable the HUD"),
			[](FString Arg)
			{
				Settings.bHudEnabled = FCString::Atoi(*Arg) != 0;
				#if WITH_EDITORONLY_DATA
				Settings.bWidgetEnabled = true;
				#endif
			}
		),
		MakeTuple(TEXT("ValidationEnabled="), TEXT("Enable or disable validation features in general"), [](FString Arg) {Settings.bValidationEnabled = FCString::Atoi(*Arg) != 0; }),
		MakeTuple(TEXT("ValidateSystemSimulationDataBuffers="), TEXT("Enable or disable validation on system data buffers"), [](FString Arg) {Settings.bValidateSystemSimulationDataBuffers = FCString::Atoi(*Arg) != 0; }),
		MakeTuple(TEXT("ValidateParticleDataBuffers="), TEXT("Enable or disable validation on particle data buffers"), [](FString Arg) {Settings.bValidateParticleDataBuffers = FCString::Atoi(*Arg) != 0; }),
		MakeTuple(TEXT("ValidationLogErrors="), TEXT("When enabled validation errors will be logged as we as on screen"), [](FString Arg) {Settings.bValidationLogErrors = FCString::Atoi(*Arg) != 0; }),
		MakeTuple(TEXT("ValidationAttributeDisplayTruncate="), TEXT("When > 0 limits the number of attributes we log"), [](FString Arg) {Settings.ValidationAttributeDisplayTruncate = FCString::Atoi(*Arg); }),

		MakeTuple(TEXT("OverviewEnabled="), TEXT("Enable or disable the main overview display"), [](FString Arg) {Settings.bOverviewEnabled = FCString::Atoi(*Arg) != 0; }),
		MakeTuple(TEXT("OverviewMode="), TEXT("Change the mode of the debug overivew"), [](FString Arg) {Settings.OverviewMode = (ENiagaraDebugHUDOverviewMode)FCString::Atoi(*Arg); }),

		MakeTuple(TEXT("OverviewLocation="), TEXT("Set the overview location"), [](FString Arg) { Settings.OverviewLocation = FVector2D(StringToVector2f(Arg, FVector2f(Settings.OverviewLocation))); }),
		MakeTuple(TEXT("FiltersEnabled="), TEXT("Enable or disable the filters."), [](FString Arg) {Settings.bFiltersEnabled = FCString::Atoi(*Arg) != 0; }),
		MakeTuple(TEXT("SystemFilter="), TEXT("Set the system filter"), [](FString Arg) {Settings.SystemFilter = Arg; Settings.bSystemFilterEnabled = !Arg.IsEmpty(); }),
		MakeTuple(TEXT("EmitterFilter="), TEXT("Set the emitter filter"), [](FString Arg) {Settings.EmitterFilter = Arg; Settings.bEmitterFilterEnabled = !Arg.IsEmpty(); GCachedSystemVariables.Empty(); }),
		MakeTuple(TEXT("ActorFilter="), TEXT("Set the actor filter"), [](FString Arg) {Settings.ActorFilter = Arg; Settings.bActorFilterEnabled = !Arg.IsEmpty(); }),
		MakeTuple(TEXT("ComponentFilter="), TEXT("Set the component filter"), [](FString Arg) {Settings.ComponentFilter = Arg; Settings.bComponentFilterEnabled = !Arg.IsEmpty(); }),
		MakeTuple(TEXT("OverviewFont="), TEXT("Set the overview font to use"), [](FString Arg) { Settings.OverviewFont = StringToFont(Arg); }),
		MakeTuple(TEXT("SystemTextFont="), TEXT("Set the system display text font to use"), [](FString Arg) { Settings.SystemTextOptions.Font = StringToFont(Arg); }),
		MakeTuple(TEXT("ParticleTextFont="), TEXT("Set the particle display text font to use"), [](FString Arg) { Settings.ParticleTextOptions.Font = StringToFont(Arg); }),

		MakeTuple(TEXT("ShowRegisteredComponents="), TEXT("Should we show registered components in the overview."), [](FString Arg) {Settings.bShowRegisteredComponents = FCString::Atoi(*Arg) != 0; }),
		MakeTuple(TEXT("ShowGlobalBudgetInfo="), TEXT("Shows global budget information"), [](FString Arg) {Settings.bShowGlobalBudgetInfo = FCString::Atoi(*Arg) != 0; }),

		MakeTuple(TEXT("PerfGraphMode="), TEXT("Change the mode of the perf graph"), [](FString Arg) {Settings.PerfGraphMode = (ENiagaraDebugHUDPerfGraphMode)FCString::Atoi(*Arg); }),
		MakeTuple(TEXT("PerfGraphTimeRange="), TEXT("Time range for the Y axis of the perf graph"), [](FString Arg) {Settings.PerfGraphTimeRange = FCString::Atof(*Arg); }),

		// Bounds commands
		MakeTuple(TEXT("DrawBoundsEnabled="), TEXT("Draw the component bounds"), [](FString Arg) {Settings.bDrawBoundsEnabled = FCString::Atoi(*Arg) != 0; }),
		MakeTuple(TEXT("DrawBoundsWireframe="), TEXT("Draw the component bounds"), [](FString Arg) {Settings.bDrawBoundsWireframe = FCString::Atoi(*Arg) != 0; }),
		MakeTuple(TEXT("DrawBoundsAlpha="), TEXT("Draw the component bounds"), [](FString Arg) {Settings.DrawBoundsAlpha = FCString::Atof(*Arg); }),

		// System commands
		MakeTuple(TEXT("SystemShowActiveOnlyInWorld="), TEXT("When enabled only active systems are shown in world"), [](FString Arg) {Settings.bSystemShowActiveOnlyInWorld = FCString::Atoi(*Arg) != 0; }),
		MakeTuple(TEXT("SystemDebugVerbosity="), TEXT("Set the in world system debug verbosity"), [](FString Arg) {Settings.SystemDebugVerbosity = FMath::Clamp(ENiagaraDebugHudVerbosity(FCString::Atoi(*Arg)), ENiagaraDebugHudVerbosity::None, ENiagaraDebugHudVerbosity::Verbose); }),
		MakeTuple(TEXT("SystemEmitterVerbosity="), TEXT("Set the in world system emitter debug verbosity"), [](FString Arg) {Settings.SystemEmitterVerbosity = FMath::Clamp(ENiagaraDebugHudVerbosity(FCString::Atoi(*Arg)), ENiagaraDebugHudVerbosity::None, ENiagaraDebugHudVerbosity::Verbose); }),
		MakeTuple(TEXT("DataInterfaceVerbosity="), TEXT("Set the in world system data interface debug verbosity"), [](FString Arg) {Settings.DataInterfaceVerbosity = FMath::Clamp(ENiagaraDebugHudVerbosity(FCString::Atoi(*Arg)), ENiagaraDebugHudVerbosity::None, ENiagaraDebugHudVerbosity::Verbose); }),
		MakeTuple(TEXT("SystemVariables="), TEXT("Set the system variables to display"), [](FString Arg) {Settings.SystemVariables.Empty(); FNiagaraDebugHUDVariable::InitFromString(Arg, Settings.SystemVariables); GCachedSystemVariables.Empty(); }),

		// Particle commands
		MakeTuple(TEXT("EnableGpuParticleReadback="), TEXT("Enables GPU readback support for particle attributes"), [](FString Arg) {Settings.bEnableGpuParticleReadback = FCString::Atoi(*Arg) != 0;}),
		MakeTuple(TEXT("ParticleVariables="), TEXT("Set the particle variables to display"), [](FString Arg) {Settings.ParticlesVariables.Empty(); FNiagaraDebugHUDVariable::InitFromString(Arg, Settings.ParticlesVariables); GCachedSystemVariables.Empty(); }),
		MakeTuple(TEXT("ShowParticleVariables="), TEXT("Set Particle variables visibility"), [](FString Arg) {Settings.bShowParticleVariables = FCString::Atoi(*Arg) != 0; GCachedSystemVariables.Empty(); }),
		MakeTuple(TEXT("MaxParticlesToDisplay="), TEXT("Maximum number of particles to show variables on"), [](FString Arg) {Settings.MaxParticlesToDisplay = FMath::Max(FCString::Atoi(*Arg), 0); Settings.bUseMaxParticlesToDisplay = Settings.MaxParticlesToDisplay > 0; }),
		MakeTuple(TEXT("ShowParticlesVariablesWithSystem="), TEXT("When enabled particle variables are shown with the system display"), [](FString Arg) {Settings.bShowParticlesVariablesWithSystem = FCString::Atoi(*Arg) != 0; }),
	};

	static FAutoConsoleCommandWithWorldAndArgs CmdNiagaraDebugHud(
		TEXT("NiagaraDebugHud"),
		TEXT("Shorter version to quickly toggle debug hud modes\n")
		TEXT(" No value will toggle the overview on / off\n")
		TEXT(" A numberic value selects which overmode to set, where 0 is off\n"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
			[](const TArray<FString>& Args, UWorld*)
			{
				if ( Args.Num() == 0 )
				{
					const bool bIsEnabled = Settings.bHudEnabled && Settings.bOverviewEnabled;
					Settings.bHudEnabled = true;
					Settings.bOverviewEnabled = !bIsEnabled;
					Settings.OverviewMode = ENiagaraDebugHUDOverviewMode::Overview;
				}
				else
				{
					const int32 Mode = FCString::Atoi(*Args[0]);

					Settings.bHudEnabled = true;
					Settings.bOverviewEnabled = Mode > 0;
					Settings.OverviewMode = FMath::Clamp(ENiagaraDebugHUDOverviewMode(Mode - 1), ENiagaraDebugHUDOverviewMode::Overview, ENiagaraDebugHUDOverviewMode::GpuComputePerformance);
				}
			}
		)
	);

	static FAutoConsoleCommandWithWorldAndArgs CmdDebugHud(
		TEXT("fx.Niagara.Debug.Hud"),
		TEXT("Set options for debug hud display"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
			[](const TArray<FString>& Args, UWorld*)
			{
				if ( Args.Num() == 0 )
				{
					UE_LOGF(LogNiagara, Log, "fx.Niagara.Debug.Hud - CommandList");
					for ( const auto& Command : GDebugConsoleCommands )
					{
						UE_LOGF(LogNiagara, Log, " \"%ls\" %ls", Command.Get<0>(), Command.Get<1>());
					}
					return;
				}

				for ( FString Arg : Args )
				{
					bool bFound = false;
					for (const auto& Command : GDebugConsoleCommands)
					{
						if ( Arg.RemoveFromStart(Command.Get<0>()) )
						{
							Command.Get<2>()(Arg);
							bFound = true;
							break;
						}
					}

					if ( !bFound )
					{
						UE_LOGF(LogNiagara, Warning, "Command '%ls' not found", *Arg);
					}
				}
			}
		)
	);

	TArray<TWeakObjectPtr<UNiagaraComponent>> GDebugSpawnedComponents;
	static FAutoConsoleCommandWithWorldAndArgs CmdSpawnComponent(
		TEXT("fx.Niagara.Debug.SpawnComponent"),
		TEXT("Spawns a NiagaraComponent using the given parameters"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
			[](const TArray<FString>& Args, UWorld* World)
			{
				// While it works, not sure this makes sense for anything other than PIE
				if (!World->IsGameWorld() && !World->IsPlayInEditor())
				{
					return;
				}

				if ( Args.Num() == 0 )
				{
					UE_LOGF(LogNiagara, Log, "fx.Niagara.Debug.SpawnSystem <AssetPath>");
					return;
				}

				UNiagaraSystem* NiagaraSystem = LoadObject<UNiagaraSystem>(nullptr, *Args[0]);
				if (NiagaraSystem == nullptr)
				{
					UE_LOGF(LogNiagara, Warning, "Failed to load NiagaraSystem '%ls'", *Args[0]);
					return;
				}

				bool bAttachToPlayer = false;
				bool bAutoDestroy = true;
				bool bAutoActivate = true;
				FVector Location = FVector::ZeroVector;
				ENCPoolMethod PoolingMethod = ENCPoolMethod::None;
				bool bPreCullCheck = true;

				AActor* CameraTarget = nullptr;
				FVector PlayerLocation = FVector::ZeroVector;
				FRotator CameraRotation = FRotator::ZeroRotator;
				for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
				{
					APlayerController* PlayerController = Iterator->Get();
					if (PlayerController && PlayerController->IsLocalPlayerController())
					{
						PlayerController->GetPlayerViewPoint(PlayerLocation, CameraRotation);
						if ( APawn* PlayerPawn = PlayerController->GetPawnOrSpectator() )
						{
							PlayerLocation = PlayerPawn->GetActorLocation();
						}

						if (PlayerController->PlayerCameraManager)
						{
							CameraRotation = PlayerController->PlayerCameraManager->GetCameraRotation();
							CameraTarget = PlayerController->PlayerCameraManager->GetViewTarget();
						}
					}
				}

				for (int32 i=1; i < Args.Num(); ++i )
				{
					FString Arg = Args[i];
					if (Arg.RemoveFromStart(TEXT("AttachToPlayer="))) { bAttachToPlayer = FCString::Atoi(*Arg) != 0; }
					else if (Arg.RemoveFromStart(TEXT("AutoDestroy="))) { bAutoDestroy = FCString::Atoi(*Arg) != 0; }
					else if (Arg.RemoveFromStart(TEXT("AutoActivate="))) { bAutoActivate = FCString::Atoi(*Arg) != 0; }
					else if (Arg.RemoveFromStart(TEXT("PreCullCheck="))) { bPreCullCheck = FCString::Atoi(*Arg) != 0; }
					else if (Arg.RemoveFromStart(TEXT("Location="))) { Location = StringToVector(Arg, FVector::ZeroVector); }
					else if (Arg.RemoveFromStart(TEXT("LocationFromPlayer="))) { Location = StringToVector(Arg, FVector::ZeroVector); Location = CameraRotation.RotateVector(Location) + PlayerLocation; }
					//FVector Location = FVector::ZeroVector;
					//ENCPoolMethod PoolingMethod = ENCPoolMethod::None;
				}

				UNiagaraComponent* SpawnedComponent = nullptr;
				if (bAttachToPlayer)
				{
					if (CameraTarget != nullptr)
					{
						SpawnedComponent = UNiagaraFunctionLibrary::SpawnSystemAttached(NiagaraSystem, CameraTarget->GetRootComponent(), NAME_None, Location, FRotator::ZeroRotator, EAttachLocation::KeepWorldPosition, bAutoDestroy, bAutoActivate, PoolingMethod, bPreCullCheck);
					}
				}
				else
				{
					SpawnedComponent = UNiagaraFunctionLibrary::SpawnSystemAtLocation(World, NiagaraSystem, Location, FRotator::ZeroRotator, FVector(1.f), bAutoDestroy, bAutoActivate, PoolingMethod, bPreCullCheck);
				}

				if (SpawnedComponent)
				{
					GDebugSpawnedComponents.Add(SpawnedComponent);
				}
			}
		)
	);

	static FAutoConsoleCommandWithWorldAndArgs CmdSpawnSystem(
		TEXT("fx.Niagara.Debug.KillSpawned"),
		TEXT("Kills all spawned compoonents"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
			[](const TArray<FString>& Args, UWorld* World)
			{
				for (TWeakObjectPtr<UNiagaraComponent> WeakComponent : GDebugSpawnedComponents)
				{
					UNiagaraComponent* NiagaraComponent = WeakComponent.Get();
					if (NiagaraComponent)
					{
						NiagaraComponent->DestroyComponent();
					}
				}
				GDebugSpawnedComponents.Reset();
			}
		)
	);

	template<typename TVariableList, typename TPredicate>
	void FindVariablesByWildcard(const TVariableList& Variables, const TArray<FNiagaraDebugHUDVariable>& DebugVariables, TPredicate Predicate)
	{
		if (DebugVariables.Num() == 0)
		{
			return;
		}

		for (const auto& Variable : Variables)
		{
			const FString VariableName = Variable.GetName().ToString();
			for (const FNiagaraDebugHUDVariable& DebugVariable : DebugVariables)
			{
				if (DebugVariable.bEnabled && (DebugVariable.Name.Len() > 0) && VariableName.MatchesWildcard(DebugVariable.Name))
				{
					Predicate(Variable);
					break;
				}
			}
		}
	}

	void FindParameterStoreVariablesByWildcard(const FNiagaraParameterStore* ParameterStore, const TArray<FNiagaraDebugHUDVariable>& DebugVariables, FCachedVariables* CachedVariables, EParameterStoreLocation StoreLocation, int32 EmitterIndex = INDEX_NONE, ENiagaraSimTarget SimTarget = ENiagaraSimTarget::CPUSim)
	{
		if (ParameterStore)
		{
			FindVariablesByWildcard(
				ParameterStore->ReadParameterVariables(),
				DebugVariables,
				[&](const FNiagaraVariableWithOffset& Variable)
				{
					if (CachedVariables->ParameterStoreVariables.ContainsByPredicate([&](const FParameterStoreVariable& Existing) { return Existing.Variable == Variable; }) == false)
					{
						FParameterStoreVariable& NewVariable = CachedVariables->ParameterStoreVariables.AddDefaulted_GetRef();
						NewVariable.StoreLocation = StoreLocation;
						NewVariable.EmitterIndex = EmitterIndex;
						NewVariable.Variable = Variable;
					}
				}
			);
		}
	}

	void FindScriptVariablesByWildcard(UNiagaraScript* NiagaraScript, const FNiagaraScriptRuntimeData* ScriptRuntimeData, const TArray<FNiagaraDebugHUDVariable>& DebugVariables, FCachedVariables* CachedVariables, EParameterStoreLocation StoreLocation, int32 EmitterIndex = INDEX_NONE, ENiagaraSimTarget SimTarget = ENiagaraSimTarget::CPUSim)
	{
		if (NiagaraScript && ScriptRuntimeData)
		{
			FindParameterStoreVariablesByWildcard(ScriptRuntimeData->GetExecutionReadyParameterStore(), DebugVariables, CachedVariables, StoreLocation, EmitterIndex, SimTarget);

		#if WITH_EDITORONLY_DATA
			FindVariablesByWildcard(
				NiagaraScript->GetVMExecutableData().StaticVariablesWritten,
				DebugVariables,
				[&](const FNiagaraVariable& Variable)
				{
					CachedVariables->StaticVariables.AddUnique(Variable);
				}
			);
		#endif
		}
	}

	void FindSystemVariablesByWildcard(UNiagaraSystem* NiagaraSystem, const TArray<FNiagaraDebugHUDVariable>& DebugVariables, FCachedVariables* CachedVariables)
	{
		if (DebugVariables.Num() == 0)
		{
			return;
		}

		const FNiagaraDataSetCompiledData& SystemCompiledData = NiagaraSystem->GetSystemCompiledData().DataSetCompiledData;
		FindVariablesByWildcard(
			SystemCompiledData.Variables,
			DebugVariables,
			[&](const FNiagaraVariable& Variable) { CachedVariables->SystemVariables.AddDefaulted_GetRef().Init(SystemCompiledData, Variable.GetName()); }
		);

		FindParameterStoreVariablesByWildcard(&NiagaraSystem->GetExposedParameters(), DebugVariables, CachedVariables, EParameterStoreLocation::UserOverride);
		FindScriptVariablesByWildcard(NiagaraSystem->GetSystemSpawnScript(), NiagaraSystem->GetScriptRuntimeData(nullptr, *NiagaraSystem->GetSystemSpawnScript()).Get(),
			DebugVariables, CachedVariables, EParameterStoreLocation::SystemSpawn);
		FindScriptVariablesByWildcard(NiagaraSystem->GetSystemUpdateScript(), NiagaraSystem->GetScriptRuntimeData(nullptr, *NiagaraSystem->GetSystemUpdateScript()).Get(),
			DebugVariables, CachedVariables, EParameterStoreLocation::SystemUpdate);

		int32 EmitterIndex = 0;
		for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem->GetEmitterHandles())
		{
			FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
			if (EmitterData && EmitterHandle.GetIsEnabled())
			{
				FindScriptVariablesByWildcard(EmitterData->SpawnScriptProps.Script, NiagaraSystem->GetScriptRuntimeData(&EmitterHandle, *EmitterData->SpawnScriptProps.Script).Get(),
					DebugVariables, CachedVariables, EParameterStoreLocation::EmitterSpawn, EmitterIndex, EmitterData->SimTarget);
				FindScriptVariablesByWildcard(EmitterData->UpdateScriptProps.Script, NiagaraSystem->GetScriptRuntimeData(&EmitterHandle, *EmitterData->UpdateScriptProps.Script).Get(),
					DebugVariables, CachedVariables, EParameterStoreLocation::EmitterUpdate, EmitterIndex, EmitterData->SimTarget);
				FindParameterStoreVariablesByWildcard(&EmitterData->RendererBindings, DebugVariables, CachedVariables, EParameterStoreLocation::EmitterRenderBindings, EmitterIndex);
			}
			++EmitterIndex;
		}

	#if WITH_EDITORONLY_DATA
		// Remove any static variables we found that are used inside renderer bindings to avoid doubling up
		CachedVariables->StaticVariables.RemoveAll(
			[&](const FNiagaraVariableBase& StaticVariable)
			{
				return CachedVariables->ParameterStoreVariables.ContainsByPredicate(
						[&](const FParameterStoreVariable& Existing)
						{
							return Existing.Variable == StaticVariable;
						}
					);
			}
		);
	#endif
	}

	double GetMillisecondsToPerfUnits()
	{
		return (Settings.PerfUnits == ENiagaraDebugHUDPerfUnits::Milliseconds) ? 1.0 / 1000.0 : 1.0;
	}

	FString FormatPerfValue(double Microseconds, int32 Length = 7)
	{
		FString TempString;
		if (Settings.PerfUnits == ENiagaraDebugHUDPerfUnits::Milliseconds)
		{
			TempString = FString::Printf(TEXT("%5.3f"), float(Microseconds / 1000.0));
		}
		else //if (Settings.PerfUnits == ENiagaraDebugHUDPerfUnits::Milliseconds)
		{
			TempString = FString::FormatAsNumber(int32(Microseconds));
		}
		while (TempString.Len() < Length)
		{
			TempString.AppendChar(' ');
		}
		return TempString;
	}

	FString FormatPerfString(const TCHAR* InText)
	{
		if (Settings.PerfUnits == ENiagaraDebugHUDPerfUnits::Milliseconds)
		{
			return FString::Printf(TEXT("%s (ms)"), InText);
		}
		return FString::Printf(TEXT("%s (us)"), InText);
	}

	const FCachedVariables& GetCachedVariables(UNiagaraSystem* NiagaraSystem)
	{
		FCachedVariables* CachedVariables = GCachedSystemVariables.Find(NiagaraSystem);
		if (CachedVariables == nullptr)
		{
			CachedVariables = &GCachedSystemVariables.Emplace(NiagaraSystem);
			CachedVariables->WeakNiagaraSystem = MakeWeakObjectPtr(NiagaraSystem);

			if (Settings.SystemVariables.Num() > 0)
			{
				FindSystemVariablesByWildcard(NiagaraSystem, Settings.SystemVariables, CachedVariables);
				
				for (int32 iVariable=0; iVariable < UE_ARRAY_COUNT(GEngineVariables); ++iVariable)
				{
					for (const FNiagaraDebugHUDVariable& DebugVariable : Settings.SystemVariables)
					{
						if (DebugVariable.bEnabled && GEngineVariables[iVariable].Key.MatchesWildcard(DebugVariable.Name))
						{
							CachedVariables->EngineVariables.Add(iVariable);
							break;
						}
					}
				}
			}

			if (Settings.bShowParticleVariables && Settings.ParticlesVariables.Num() > 0)
			{
				const TConstArrayView<FNiagaraEmitterHandle> EmitterHandles = NiagaraSystem->GetEmitterHandles();
				const TArray<TSharedRef<const FNiagaraEmitterCompiledData>>& AllStatefulEmittersCompiledData = NiagaraSystem->GetEmitterCompiledData();

				TArray<FNiagaraDebugHUDVariable> LocalParticlesVariables;
				LocalParticlesVariables.Reserve(Settings.ParticlesVariables.Num());
				Algo::TransformIf(
					Settings.ParticlesVariables,
					LocalParticlesVariables,
					[](const FNiagaraDebugHUDVariable& Variable) { return Variable.bEnabled; },
					[](const FNiagaraDebugHUDVariable& Variable)
					{
						FNiagaraDebugHUDVariable NewVariable = Variable;
						NewVariable.Name.RemoveFromStart(FNiagaraConstants::ParticleAttributeNamespaceString);
						return NewVariable;
					}
				);				

				CachedVariables->ParticleVariables.AddDefaulted(EmitterHandles.Num());
				CachedVariables->ParticlePositionAccessors.AddDefaulted(EmitterHandles.Num());
				for (int32 iEmitter=0; iEmitter < EmitterHandles.Num(); ++iEmitter)
				{
					const FNiagaraEmitterHandle& EmitterHandle = EmitterHandles[iEmitter];
					if (!EmitterHandle.IsValid() || !EmitterHandle.GetIsEnabled())
					{
						continue;
					}

					if (Settings.bEmitterFilterEnabled && !Settings.EmitterFilter.IsEmpty() && !EmitterHandle.GetUniqueInstanceName().MatchesWildcard(Settings.EmitterFilter))
					{
						continue;
					}

					const FNiagaraDataSetCompiledData* EmitterCompiledData = nullptr;
					if (EmitterHandle.GetEmitterMode() == ENiagaraEmitterMode::Stateless)
					{
						UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandle.GetStatelessEmitter();
						FNiagaraStatelessEmitterDataPtr EmitterData = StatelessEmitter ? StatelessEmitter->GetEmitterData() : nullptr;
						EmitterCompiledData = EmitterData ? EmitterData->ParticleDataSetCompiledData.Get() : nullptr;
					}
					else
					{
						EmitterCompiledData = AllStatefulEmittersCompiledData.IsValidIndex(iEmitter) ? &AllStatefulEmittersCompiledData[iEmitter]->DataSetCompiledData : nullptr;
					}

					if (EmitterCompiledData == nullptr)
					{
						continue;
					}

					FindVariablesByWildcard(
						EmitterCompiledData->Variables,
						LocalParticlesVariables,
						[&](const FNiagaraVariable& Variable) { CachedVariables->ParticleVariables[iEmitter].AddDefaulted_GetRef().Init(*EmitterCompiledData, Variable.GetName()); }
					);

					if (CachedVariables->ParticleVariables[iEmitter].Num() > 0)
					{
						static const FName PositionName(TEXT("Position"));
						CachedVariables->ParticlePositionAccessors[iEmitter].Init(*EmitterCompiledData, PositionName);
					}
				}
			}
		}
		return *CachedVariables;
	}

	UFont* GetFont(ENiagaraDebugHudFont Font)
	{
		switch (Font)
		{
			default:
			case ENiagaraDebugHudFont::Small:	return GEngine->GetTinyFont();
			case ENiagaraDebugHudFont::Normal:	return GEngine->GetSmallFont();
		}
	};

	FVector2f GetStringSize(UFont* Font, const TCHAR* Text)
	{
		FVector2f MaxSize = FVector2f::ZeroVector;
		FVector2f CurrSize = FVector2f::ZeroVector;

		const float fAdvanceHeight = Font->GetMaxCharHeight();
		const TCHAR* PrevChar = nullptr;
		while (*Text)
		{
			if ( *Text == '\n' )
			{
				CurrSize.X = 0.0f;
				CurrSize.Y = CurrSize.Y + fAdvanceHeight;
				PrevChar = nullptr;
				++Text;
				continue;
			}

			float TmpWidth, TmpHeight;
			Font->GetCharSize(*Text, TmpWidth, TmpHeight);

			int8 CharKerning = 0;
			if (PrevChar)
			{
				CharKerning = Font->GetCharKerning(*PrevChar, *Text);
			}

			CurrSize.X += TmpWidth + CharKerning;
			MaxSize.X = FMath::Max(MaxSize.X, CurrSize.X);
			MaxSize.Y = FMath::Max(MaxSize.Y, CurrSize.Y + TmpHeight);

			PrevChar = Text++;
		}

		return MaxSize;
	}

	TPair<FVector2f, FVector2f> GetTextLocation(UFont* Font, const TCHAR* Text, const FNiagaraDebugHudTextOptions& TextOptions, const FVector2f ScreenLocation)
	{
		FVector2f StringSize = GetStringSize(Font, Text);
		FVector2f OutLocation = ScreenLocation + FVector2f(TextOptions.ScreenOffset);
		if (TextOptions.HorizontalAlignment == ENiagaraDebugHudHAlign::Center )
		{
			OutLocation.X -= StringSize.X * 0.5f;
		}
		else if (TextOptions.HorizontalAlignment == ENiagaraDebugHudHAlign::Right)
		{
			OutLocation.X -= StringSize.X;
		}
		if (TextOptions.VerticalAlignment == ENiagaraDebugHudVAlign::Center )
		{
			OutLocation.Y -= StringSize.Y * 0.5f;
		}
		else if (TextOptions.VerticalAlignment == ENiagaraDebugHudVAlign::Bottom)
		{
			OutLocation.Y -= StringSize.Y;
		}
		return TPair<FVector2f, FVector2f>(StringSize, OutLocation);
	}

	void DrawWireframeBox(UWorld* World, const FVector& Location, const FVector& Extents, const FLinearColor& Color, float Thickness = 3.0f)
	{
		if (ULineBatchComponent* LineBatcher = World->GetLineBatcher(UWorld::ELineBatcherType::World))
		{
			LineBatcher->DrawLine(Location + FVector(Extents.X, Extents.Y, Extents.Z), Location + FVector(Extents.X, -Extents.Y, Extents.Z), Color, 0, Thickness);
			LineBatcher->DrawLine(Location + FVector(Extents.X, -Extents.Y, Extents.Z), Location + FVector(-Extents.X, -Extents.Y, Extents.Z), Color, 0, Thickness);
			LineBatcher->DrawLine(Location + FVector(-Extents.X, -Extents.Y, Extents.Z), Location + FVector(-Extents.X, Extents.Y, Extents.Z), Color, 0, Thickness);
			LineBatcher->DrawLine(Location + FVector(-Extents.X, Extents.Y, Extents.Z), Location + FVector(Extents.X, Extents.Y, Extents.Z), Color, 0, Thickness);

			LineBatcher->DrawLine(Location + FVector(Extents.X, Extents.Y, -Extents.Z), Location + FVector(Extents.X, -Extents.Y, -Extents.Z), Color, 0, Thickness);
			LineBatcher->DrawLine(Location + FVector(Extents.X, -Extents.Y, -Extents.Z), Location + FVector(-Extents.X, -Extents.Y, -Extents.Z), Color, 0, Thickness);
			LineBatcher->DrawLine(Location + FVector(-Extents.X, -Extents.Y, -Extents.Z), Location + FVector(-Extents.X, Extents.Y, -Extents.Z), Color, 0, Thickness);
			LineBatcher->DrawLine(Location + FVector(-Extents.X, Extents.Y, -Extents.Z), Location + FVector(Extents.X, Extents.Y, -Extents.Z), Color, 0, Thickness);

			LineBatcher->DrawLine(Location + FVector(Extents.X, Extents.Y, Extents.Z), Location + FVector(Extents.X, Extents.Y, -Extents.Z), Color, 0, Thickness);
			LineBatcher->DrawLine(Location + FVector(Extents.X, -Extents.Y, Extents.Z), Location + FVector(Extents.X, -Extents.Y, -Extents.Z), Color, 0, Thickness);
			LineBatcher->DrawLine(Location + FVector(-Extents.X, -Extents.Y, Extents.Z), Location + FVector(-Extents.X, -Extents.Y, -Extents.Z), Color, 0, Thickness);
			LineBatcher->DrawLine(Location + FVector(-Extents.X, Extents.Y, Extents.Z), Location + FVector(-Extents.X, Extents.Y, -Extents.Z), Color, 0, Thickness);
		}
	}

	void DrawBox(UWorld* World, const FVector& Location, const FVector& Extents, const FLinearColor& Color)
	{
		if (ULineBatchComponent* LineBatcher = World->GetLineBatcher(UWorld::ELineBatcherType::World))
		{
			const FBox BoundsBox(-Extents, Extents);
			FColor BoxColor = Color.ToFColor(false);
			LineBatcher->DrawSolidBox(BoundsBox, FTransform(FQuat::Identity, Location), BoxColor, 0, 0.0f);
		}
	}

	void DrawSystemLocation(UCanvas* Canvas, bool bIsActive, const FVector& ScreenLocation, const FRotator& Rotation)
	{
		FSceneView* SceneView = Canvas->SceneView;
		FCanvas* DrawCanvas = Canvas->Canvas;
		if (SceneView && DrawCanvas)
		{
			const FMatrix& ViewMatrix = SceneView->ViewMatrices.GetWorldToView();
			const float AxisLength = 50.0f;
			const float BoxSize = 10.0f;
			const FVector XAxis(ViewMatrix.TransformVector(Rotation.RotateVector(FVector(1.0f, 0.0f, 0.0f))));
			const FVector YAxis(ViewMatrix.TransformVector(Rotation.RotateVector(FVector(0.0f, 1.0f, 0.0f))));
			const FVector ZAxis(ViewMatrix.TransformVector(Rotation.RotateVector(FVector(0.0f, 0.0f, 1.0f))));

			FBatchedElements* BatchedLineElements = DrawCanvas->GetBatchedElements(FCanvas::ET_Line);

			if ( ensure(BatchedLineElements) )
			{
				FHitProxyId HitProxyId = DrawCanvas->GetHitProxyId();
				const FVector ScreenLocation2D(ScreenLocation.X, ScreenLocation.Y, 0.0f);
				const FVector XAxis2D(XAxis.X, -XAxis.Y, 0.0f);
				const FVector YAxis2D(YAxis.X, -YAxis.Y, 0.0f);
				const FVector ZAxis2D(ZAxis.X, -ZAxis.Y, 0.0f);
				BatchedLineElements->AddLine(ScreenLocation2D, ScreenLocation2D + (XAxis2D * AxisLength), bIsActive ? FLinearColor::Red : FLinearColor::Black, HitProxyId, 1.0f);
				BatchedLineElements->AddLine(ScreenLocation2D, ScreenLocation2D + (YAxis2D * AxisLength), bIsActive ? FLinearColor::Green : FLinearColor::Black, HitProxyId, 1.0f);
				BatchedLineElements->AddLine(ScreenLocation2D, ScreenLocation2D + (ZAxis2D * AxisLength), bIsActive ? FLinearColor::Blue : FLinearColor::Black, HitProxyId, 1.0f);

				const FVector BoxPoints[] =
				{
					ScreenLocation2D + ((-XAxis2D - YAxis2D - ZAxis2D) * BoxSize),
					ScreenLocation2D + (( XAxis2D - YAxis2D - ZAxis2D) * BoxSize),
					ScreenLocation2D + (( XAxis2D + YAxis2D - ZAxis2D) * BoxSize),
					ScreenLocation2D + ((-XAxis2D + YAxis2D - ZAxis2D) * BoxSize),
					ScreenLocation2D + ((-XAxis2D - YAxis2D + ZAxis2D) * BoxSize),
					ScreenLocation2D + (( XAxis2D - YAxis2D + ZAxis2D) * BoxSize),
					ScreenLocation2D + (( XAxis2D + YAxis2D + ZAxis2D) * BoxSize),
					ScreenLocation2D + ((-XAxis2D + YAxis2D + ZAxis2D) * BoxSize),
				};
				const FLinearColor BoxColor = bIsActive ? FLinearColor::White : FLinearColor::Black;
				BatchedLineElements->AddLine(BoxPoints[0], BoxPoints[1], BoxColor, HitProxyId, 1.0f);
				BatchedLineElements->AddLine(BoxPoints[1], BoxPoints[2], BoxColor, HitProxyId, 1.0f);
				BatchedLineElements->AddLine(BoxPoints[2], BoxPoints[3], BoxColor, HitProxyId, 1.0f);
				BatchedLineElements->AddLine(BoxPoints[3], BoxPoints[0], BoxColor, HitProxyId, 1.0f);

				BatchedLineElements->AddLine(BoxPoints[4], BoxPoints[5], BoxColor, HitProxyId, 1.0f);
				BatchedLineElements->AddLine(BoxPoints[5], BoxPoints[6], BoxColor, HitProxyId, 1.0f);
				BatchedLineElements->AddLine(BoxPoints[6], BoxPoints[7], BoxColor, HitProxyId, 1.0f);
				BatchedLineElements->AddLine(BoxPoints[7], BoxPoints[4], BoxColor, HitProxyId, 1.0f);

				BatchedLineElements->AddLine(BoxPoints[0], BoxPoints[4], BoxColor, HitProxyId, 1.0f);
				BatchedLineElements->AddLine(BoxPoints[1], BoxPoints[5], BoxColor, HitProxyId, 1.0f);
				BatchedLineElements->AddLine(BoxPoints[2], BoxPoints[6], BoxColor, HitProxyId, 1.0f);
				BatchedLineElements->AddLine(BoxPoints[3], BoxPoints[7], BoxColor, HitProxyId, 1.0f);
			}
		}
	}

	template<typename TOutput>
	void BuildGpuHudInformation(TOutput& Output, UNiagaraComponent* NiagaraComponent, FNiagaraSystemInstance* SystemInstance, ERHIFeatureLevel::Type FeatureLevel)
	{
		static UEnum* GpuComputeTickStageEnum = StaticEnum<ENiagaraGpuComputeTickStage::Type>();
		const FNiagaraSystemGpuComputeProxy* SystemInstanceComputeProxy = SystemInstance->GetSystemGpuComputeProxy();
		if (GpuComputeTickStageEnum == nullptr || SystemInstanceComputeProxy == nullptr)
		{
			return;
		}

		const ENiagaraGpuComputeTickStage::Type GpuTickStage = SystemInstanceComputeProxy->GetComputeTickStage();
		Output.Appendf(TEXT("GpuTickStage - %s\n"), *GpuComputeTickStageEnum->GetNameStringByValue(GpuTickStage));
		if (Settings.SystemDebugVerbosity == ENiagaraDebugHudVerbosity::Verbose)
		{
			TStringBuilder<128> GpuFeaturesBuilder;
			if (SystemInstance->RequiresGlobalDistanceField())
			{
				GpuFeaturesBuilder.Append(TEXT(" GlobalDistanceField"));
			}
			if (SystemInstance->RequiresDepthBuffer())
			{
				GpuFeaturesBuilder.Append(TEXT(" DepthBuffer"));
			}
			if (SystemInstance->RequiresEarlyViewData())
			{
				GpuFeaturesBuilder.Append(TEXT(" EarlyViewData"));
			}
			if (SystemInstance->RequiresViewUniformBuffer())
			{
				GpuFeaturesBuilder.Append(TEXT(" ViewUniformBuffer"));
			}
			if (SystemInstance->RequiresRayTracingScene())
			{
				GpuFeaturesBuilder.Append(TEXT(" RayTracingScene"));
			}
			if (SystemInstance->RequiresCurrentFrameNDC())
			{
				GpuFeaturesBuilder.Append(TEXT(" CurrentFrameNDC"));
			}
			if (GpuFeaturesBuilder.Len() > 0)
			{
				Output.Appendf(TEXT("GpuFeatures -%s\n"), *GpuFeaturesBuilder);
			}
		}

		// Attempt to give feedback to the user if an emitter will be latent or not on the GPU
		if (GpuTickStage == ENiagaraGpuComputeTickStage::PostOpaqueRender)
		{
			TStringBuilder<128> GpuLatentBuilder;

			for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
			{
				UNiagaraEmitter* NiagaraEmitter = EmitterInstance->GetEmitter();
				if (NiagaraEmitter == nullptr)
				{
					continue;
				}

				bool bLowLatencyFailed = false;
				EmitterInstance->ForEachEnabledRenderer(
					[&](const UNiagaraRendererProperties* RenderProperties)
					{
						ENiagaraRendererGpuTranslucentLatency RequestedLatency = ENiagaraRendererGpuTranslucentLatency::ProjectDefault;
						if (const UNiagaraMeshRendererProperties* MeshRenderProperties = Cast<const UNiagaraMeshRendererProperties>(RenderProperties))
						{
							RequestedLatency = MeshRenderProperties->GpuTranslucentLatency;
						}
						else if (const UNiagaraSpriteRendererProperties* SpriteRendererProperties = Cast<const UNiagaraSpriteRendererProperties>(RenderProperties))
						{
							RequestedLatency = SpriteRendererProperties->GpuTranslucentLatency;
						}
						else
						{
							// Renderer does not support low latency
							return;
						}

						const bool bWantsThisFrameData = UNiagaraRendererProperties::ShouldGpuTranslucentThisFrame(RequestedLatency);

						bool bSupportsThisFrameData =
							!NiagaraComponent->bCastVolumetricTranslucentShadow &&
							UNiagaraRendererProperties::IsGpuTranslucentThisFrame(FeatureLevel, RequestedLatency);

						if (bSupportsThisFrameData)
						{
							TArray<UMaterialInterface*> UsedMaterials;
							RenderProperties->GetUsedMaterials(&EmitterInstance.Get(), UsedMaterials);
							for (UMaterialInterface* Material : UsedMaterials)
							{
								if (Material && (!IsTranslucentBlendMode(*Material) || Material->GetCastDynamicShadowAsMasked()))
								{
									bSupportsThisFrameData = false;
									break;
								}
							}
						}
						bLowLatencyFailed |= bWantsThisFrameData != bSupportsThisFrameData;
					}
				);

				if (bLowLatencyFailed)
				{
					GpuLatentBuilder.AppendChar(' ');
					GpuLatentBuilder.Append(NiagaraEmitter->GetUniqueEmitterName());
				}
			}

			if ( GpuLatentBuilder.Len() > 0 )
			{
				Output.Appendf(TEXT("GpuHasLatentEmitters -%s\n"), *GpuLatentBuilder);
			}
		}
	}

#if WITH_NIAGARA_GPU_PROFILER
	bool NeedsGpuStatCapture()
	{
		return
			Settings.bOverviewEnabled &&
			(
				(Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::Performance) ||
				(Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::GpuComputePerformance) ||
				(Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph && Settings.PerfGraphMode == ENiagaraDebugHUDPerfGraphMode::GPU)
			);
	}
#endif
}

FNiagaraDebugHud::FNiagaraDebugHud(UWorld* World)
{
	using namespace NiagaraDebugLocal;

	WeakWorld = World;

	LongestSystemPrettyName = TEXT("NS_SomeBigLongNiagaraSystemName");

	if ( !GDebugDrawHandle.IsValid() )
	{
		GDebugDrawHandle = UDebugDrawService::Register(TEXT("Particles"), FDebugDrawDelegate::CreateStatic(&FNiagaraDebugHud::DebugDrawCallback));
	}
	++GDebugDrawHandleUsers;
#if WITH_NIAGARA_GPU_PROFILER
	GpuProfilerListener.SetHandler(
		[&](const FNiagaraGpuFrameResultsPtr& InGpuResults)
		{
			FNiagaraGpuComputeDispatchInterface* DispatchInterface = FNiagaraGpuComputeDispatchInterface::Get(WeakWorld.Get());
			if ( InGpuResults->OwnerContext == uintptr_t(DispatchInterface) )
			{
				GpuResults = InGpuResults;
				GpuResultsGameFrameCounter = GFrameCounter;

				GpuTotalDispatches.Accumulate(GpuResultsGameFrameCounter, GpuResults->TotalDispatches);
				GpuTotalMicroseconds.Accumulate(GpuResultsGameFrameCounter, GpuResults->TotalDurationMicroseconds);

				for ( const auto& DispatchResult : GpuResults->DispatchResults )
				{
					if ( DispatchResult.OwnerEmitter.Emitter.IsExplicitlyNull() )
					{
						FGpuUsagePerEvent& EventUsage = GpuUsagePerEvent.FindOrAdd(DispatchResult.StageName);
						EventUsage.InstanceCount.Accumulate(GpuResultsGameFrameCounter, 1);
						EventUsage.Microseconds.Accumulate(GpuResultsGameFrameCounter, DispatchResult.DurationMicroseconds);
					}
					else
					{
						UNiagaraEmitter* OwnerEmitter = DispatchResult.OwnerEmitter.Emitter.Get();
						UNiagaraSystem* OwnerSystem = OwnerEmitter ? OwnerEmitter->GetTypedOuter<UNiagaraSystem>() : nullptr;
						if (OwnerSystem == nullptr)
						{
							continue;
						}

						FGpuUsagePerSystem& SystemUsage = GpuUsagePerSystem.FindOrAdd(OwnerSystem);
						SystemUsage.Microseconds.Accumulate(GpuResultsGameFrameCounter, DispatchResult.DurationMicroseconds);

						FGpuUsagePerEmitter& EmitterUsage = SystemUsage.Emitters.FindOrAdd(OwnerEmitter);
						EmitterUsage.Microseconds.Accumulate(GpuResultsGameFrameCounter, DispatchResult.DurationMicroseconds);
						EmitterUsage.InstanceCount.Accumulate(GpuResultsGameFrameCounter, 1);

						FGpuUsagePerStage& StageUsage = EmitterUsage.Stages.FindOrAdd(DispatchResult.StageName);
						StageUsage.Microseconds.Accumulate(GpuResultsGameFrameCounter, DispatchResult.DurationMicroseconds);
						StageUsage.InstanceCount.Accumulate(GpuResultsGameFrameCounter, 1);

						if ( DispatchResult.bUniqueInstance )
						{
							SystemUsage.InstanceCount.Accumulate(GpuResultsGameFrameCounter, 1);
						}
					}
				}

				// Prune data that hasn't been seen in a while
				constexpr uint64 FramesBeforePrune = SmoothedNumFrames;
				for ( auto SystemIt=GpuUsagePerSystem.CreateIterator(); SystemIt; ++SystemIt)
				{
					if ( SystemIt.Value().Microseconds.ShouldPrune(GpuResultsGameFrameCounter) )
					{
						SystemIt.RemoveCurrent();
						continue;
					}

					for (auto EmitterIt=SystemIt.Value().Emitters.CreateIterator(); EmitterIt; ++EmitterIt)
					{
						if (EmitterIt.Value().Microseconds.ShouldPrune(GpuResultsGameFrameCounter) )
						{
							EmitterIt.RemoveCurrent();
							continue;
						}

						for (auto StageIt=EmitterIt.Value().Stages.CreateIterator(); StageIt; ++StageIt)
						{
							if (StageIt.Value().Microseconds.ShouldPrune(GpuResultsGameFrameCounter))
							{
								StageIt.RemoveCurrent();
							}
						}
					}
				}

				for ( auto EventIt=GpuUsagePerEvent.CreateIterator(); EventIt; ++EventIt)
				{
					if (EventIt.Value().Microseconds.ShouldPrune(GpuResultsGameFrameCounter))
					{
						EventIt.RemoveCurrent();
					}
				}
			}
		}
	);
#endif
}

FNiagaraDebugHud::~FNiagaraDebugHud()
{
	using namespace NiagaraDebugLocal;

	--GDebugDrawHandleUsers;
	if (GDebugDrawHandleUsers == 0)
	{
		UDebugDrawService::Unregister(GDebugDrawHandle);
		GDebugDrawHandle.Reset();
	}

#if WITH_EDITORONLY_DATA
	for (auto It = SystemCompiledDelegates.CreateIterator(); It; ++It)
	{
		if ( UNiagaraSystem* NiagaraSystem = It.Key().Get() )
		{
			NiagaraSystem->OnSystemCompiled().Remove(It.Value());
		}
	}
	SystemCompiledDelegates.Empty();
#endif
}

void FNiagaraDebugHud::UpdateSettings(const FNiagaraDebugHUDSettingsData& NewSettings)
{
	using namespace NiagaraDebugLocal;

	FNiagaraDebugHUDSettingsData::StaticStruct()->CopyScriptStruct(&Settings, &NewSettings);
	GCachedSystemVariables.Empty();
}

void FNiagaraDebugHud::AddMessage(FName Key, const FNiagaraDebugMessage& Message)
{
	Messages.FindOrAdd(Key) = Message;
}

void FNiagaraDebugHud::RemoveMessage(FName Key)
{
	Messages.Remove(Key);
}

void FNiagaraDebugHud::GatherSystemInfo()
{
	using namespace NiagaraDebugLocal;

	GlobalTotalRegistered = 0;
	GlobalTotalActive = 0;
	GlobalTotalScalability = 0;
	GlobalTotalEmitters = 0;
	GlobalTotalParticles = 0;
	GlobalTotalBytes = 0;

	GlobalTotalCulled = 0;
	GlobalTotalCulledByDistance = 0;
	GlobalTotalCulledByVisibility = 0;
	GlobalTotalCulledByInstanceCount = 0;
	GlobalTotalCulledByBudget = 0;
	
	GlobalTotalPlayerSystems = 0;

	//PerSystemDebugInfo.Reset();
	InWorldComponents.Reset();

	UWorld* World = WeakWorld.Get();
	if (World == nullptr)
	{
		return;
	}

#if WITH_NIAGARA_GPU_PROFILER
	GpuProfilerListener.SetEnabled(NeedsGpuStatCapture());
#endif

	// When not enabled do nothing
	if (!Settings.IsEnabled())
	{
		return;
	}

	// If the overview is not enabled and we don't have any filters we can skip everything below as nothing will be displayed
	if (!Settings.bOverviewEnabled)
	{
		if ( !Settings.bActorFilterEnabled && !Settings.bComponentFilterEnabled )
		{
			return;
		}
	}

#if WITH_PARTICLE_PERF_STATS
	//-TODO: Enable based one perf warnings
	bool bUpdateStats = false;
	if (Settings.bOverviewEnabled && (Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::Performance || Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph))
	{
		if (StatsListener.IsValid() == false)
		{
			StatsListener = MakeShared<FNiagaraDebugHUDStatsListener, ESPMode::ThreadSafe>(*this);
			FParticlePerfStatsManager::AddListener(StatsListener);
		}
	}
	else
	{
		if (StatsListener)
		{
			FParticlePerfStatsManager::RemoveListener(StatsListener);
			StatsListener.Reset();
		}
	}
#endif

	//Clear transient data we're about to gather.
	//Keeping some persistent data.
	for (auto& Pair : PerSystemDebugInfo)
	{
		FSystemDebugInfo& SysInfo = Pair.Value;
		SysInfo.Reset();
		++SysInfo.FramesSinceVisible;

		#if WITH_PARTICLE_PERF_STATS
		if (SysInfo.PerfStats.IsUnique())
		{
			SysInfo.PerfStats = nullptr;
		}
		#endif
	}

	// Iterate all components looking for active ones in the world we are in
	for (TObjectIterator<UFXSystemComponent> It; It; ++It)
	{
		UFXSystemComponent* FXComponent = *It;
		UNiagaraComponent* NiagaraComponent = Settings.bIncludeNiagara ? Cast<UNiagaraComponent>(FXComponent) : nullptr;
		UParticleSystemComponent* CascadeComponent = Settings.bIncludeCascade ? Cast<UParticleSystemComponent>(FXComponent) : nullptr;

		FNiagaraSystemInstanceControllerPtr SystemInstanceController = NiagaraComponent ? NiagaraComponent->GetSystemInstanceController() : nullptr;
		FNiagaraSystemInstance* SystemInstance = SystemInstanceController.IsValid() ? SystemInstanceController->GetSystemInstance_Unsafe() : nullptr;
		
		if (!IsValidChecked(FXComponent) || (!IsValid(NiagaraComponent) && !IsValid(CascadeComponent)) || FXComponent->IsUnreachable() || FXComponent->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
		{
			continue;
		}
		if (FXComponent->GetWorld() != World)
		{
			continue;
		}
		if (FXComponent->GetFXSystemAsset() == nullptr)
		{
			continue;
		}

		const bool bIsActive = FXComponent->IsActive();
		const bool bHasScalability = NiagaraComponent ? NiagaraComponent->IsRegisteredWithScalabilityManager() : (CascadeComponent ? CascadeComponent->bIsManagingSignificance : false);
		const bool bUsingCullProxy = NiagaraComponent ? NiagaraComponent->IsUsingCullProxy() : false;
		const bool bIsCullProxy = FXComponent->IsA<UNiagaraCullProxyComponent>();

		FSystemDebugInfo& SystemDebugInfo = PerSystemDebugInfo.FindOrAdd(FXComponent->GetFXSystemAsset()->GetFName());
		if (SystemDebugInfo.SystemName.IsEmpty())
		{
			SystemDebugInfo.SystemName = GetNameSafe(FXComponent->GetFXSystemAsset());
		}

		if (SystemDebugInfo.SystemPrettyName.IsEmpty())
		{
			SystemDebugInfo.SystemPrettyName = SystemDebugInfo.SystemName;
			if (UNiagaraSystem* NiagaraSystem = Cast<UNiagaraSystem>(FXComponent->GetFXSystemAsset()))
			{
			#if WITH_EDITORONLY_DATA
				if (NiagaraSystem->GetCompileForEdit())
				{
					SystemDebugInfo.SystemPrettyName.Append(GCompileForEditString);
				}
			#endif
				if (const TCHAR* SystemStateModeString = NiagaraSystem->GetSystemStateModeString())
				{
					SystemDebugInfo.SystemPrettyName.AppendChar(' ');
					SystemDebugInfo.SystemPrettyName.Append(SystemStateModeString);
				}

			#if WITH_EDITORONLY_DATA
				if (!SystemCompiledDelegates.Contains(NiagaraSystem))
				{
					SystemCompiledDelegates.Emplace(
						NiagaraSystem,
						NiagaraSystem->OnSystemCompiled().AddRaw(this, &FNiagaraDebugHud::OnSystemCompiled)
					);
				}
			#endif
			}

			if (SystemDebugInfo.SystemPrettyName.Len() > LongestSystemPrettyName.Len())
			{
				LongestSystemPrettyName = SystemDebugInfo.SystemPrettyName;
			}
		}
		SystemDebugInfo.bPassesSystemFilter = !(Settings.bFiltersEnabled && Settings.bSystemFilterEnabled) || Settings.SystemFilter.IsEmpty() || SystemDebugInfo.SystemName.MatchesWildcard(Settings.SystemFilter);
		SystemDebugInfo.bShowInWorld = (Settings.SystemDebugVerbosity != ENiagaraDebugHudVerbosity::None) && SystemDebugInfo.bPassesSystemFilter;

		const bool bCanShowInWorld = 
			SystemDebugInfo.bShowInWorld &&
			((bIsActive || !Settings.bSystemShowActiveOnlyInWorld) || bUsingCullProxy) &&
			!bIsCullProxy;

		if (bCanShowInWorld)
		{
			bool bIsMatch = true;

			// Filter by actor
			if (Settings.bFiltersEnabled && Settings.bActorFilterEnabled && !Settings.ActorFilter.IsEmpty())
			{
				AActor* Actor = FXComponent->GetOwner();
				bIsMatch &= (Actor != nullptr) && Actor->GetActorNameOrLabel().MatchesWildcard(Settings.ActorFilter);
			}

			// Filter by component
			if ( bIsMatch && Settings.bFiltersEnabled && Settings.bComponentFilterEnabled && !Settings.ComponentFilter.IsEmpty())
			{
				bIsMatch &= FXComponent->GetName().MatchesWildcard(Settings.ComponentFilter);
			}

			if (bIsMatch)
			{
				InWorldComponents.Add(FXComponent);
			}
		}

		if (FXComponent->IsRegistered())
		{
			++GlobalTotalRegistered;
			++SystemDebugInfo.TotalRegistered;
		}

		if ( bHasScalability )
		{
			++GlobalTotalScalability;
			++SystemDebugInfo.TotalScalability;
		}

		if (NiagaraComponent && NiagaraComponent->IsLocalPlayerEffect() && bIsActive)
		{
			++GlobalTotalPlayerSystems;
			++SystemDebugInfo.TotalPlayerSystems;
		}

		if (NiagaraComponent && NiagaraComponent->IsRegisteredWithScalabilityManager())
		{
#if WITH_NIAGARA_DEBUGGER
			if (NiagaraComponent->DebugCachedScalabilityState.bCulled)
			{
				++GlobalTotalCulled;
				++SystemDebugInfo.TotalCulled;
				
				if (NiagaraComponent->DebugCachedScalabilityState.bCulledByDistance)
				{
					++GlobalTotalCulledByDistance;
					++SystemDebugInfo.TotalCulledByDistance;
				}
				if (NiagaraComponent->DebugCachedScalabilityState.bCulledByVisibility)
				{
					++GlobalTotalCulledByVisibility;
					++SystemDebugInfo.TotalCulledByVisibility;
				}
				if (NiagaraComponent->DebugCachedScalabilityState.bCulledByInstanceCount)
				{
					++GlobalTotalCulledByInstanceCount;
					++SystemDebugInfo.TotalCulledByInstanceCount;
				}
				if (NiagaraComponent->DebugCachedScalabilityState.bCulledByGlobalBudget)
				{
					++GlobalTotalCulledByBudget;
					++SystemDebugInfo.TotalCulledByBudget;
				}
			}
#endif//WITH_NIAGARA_DEBUGGER
		}

		// Track rough memory usage
		const int64 BytesUsed = FXComponent->GetApproxMemoryUsage();
		SystemDebugInfo.TotalBytes += BytesUsed;
		GlobalTotalBytes += BytesUsed;

		if (bIsActive)
		{
			// Accumulate totals
			int32 ActiveEmitters = 0;
			int32 TotalEmitters = 0;
			int32 ActiveParticles = 0;
			if( NiagaraComponent)
			{
				check(SystemInstance);
				for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
				{
					if (EmitterInstance->IsDisabled())
					{
						continue;
					}

					++TotalEmitters;
					if (EmitterInstance->GetExecutionState() == ENiagaraExecutionState::Active)
					{
						++ActiveEmitters;
					}
					ActiveParticles += EmitterInstance->GetNumParticles();
				}
			}
			else if (CascadeComponent)
			{
				for(FParticleEmitterInstance* Emitter : CascadeComponent->EmitterInstances)
				{					
					if(Emitter)
					{
						++TotalEmitters;
						ActiveParticles += Emitter->ActiveParticles;
						if(Emitter->HasCompleted() == false)
						{
							++ActiveEmitters;
						}
					}
				}
			}

			++SystemDebugInfo.TotalActive;
			SystemDebugInfo.TotalEmitters += ActiveEmitters;
			SystemDebugInfo.TotalParticles += ActiveParticles;

			++GlobalTotalActive;
			GlobalTotalEmitters += ActiveEmitters;
			GlobalTotalParticles += ActiveParticles;

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
			if(StatsListener)
			{
				SystemDebugInfo.PerfStats = StatsListener->GetSystemStats(FXComponent->GetFXSystemAsset());
			}
#endif
		}

		//Generate a unique-ish random color for use in graphs and the world to help visually ID this system.
		FRandomStream Rands(GetTypeHash(FXComponent->GetFXSystemAsset()->GetName()) + Settings.SystemColorSeed);
		uint8 RandomHue = (uint8)Rands.RandRange(int32(Settings.SystemColorHSVMin.X), int32(Settings.SystemColorHSVMax.X));
		uint8 RandomSat = (uint8)Rands.RandRange(int32(Settings.SystemColorHSVMin.Y), int32(Settings.SystemColorHSVMax.Y));
		uint8 RandomValue = (uint8)Rands.RandRange(int32(Settings.SystemColorHSVMin.Z), int32(Settings.SystemColorHSVMax.Z));
		SystemDebugInfo.UniqueColor = FLinearColor::MakeFromHSV8(RandomHue, RandomSat, RandomValue);


		bool bWillBeVisible = false;
		
		if (Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::Performance || Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph)
		{
			bWillBeVisible = SystemDebugInfo.TotalActive > 0;
		}
		else
		{
			bWillBeVisible = (SystemDebugInfo.TotalActive > 0 || SystemDebugInfo.TotalScalability > 0);
			if ( Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::Overview )
			{
				bWillBeVisible |= Settings.bShowRegisteredComponents && SystemDebugInfo.TotalRegistered > 0;
			}
		}
		if (bWillBeVisible)
		{
			SystemDebugInfo.FramesSinceVisible = 0;
		}
		else
		{
			SystemDebugInfo.FramesSinceVisible++;
		}
	}

	GlobalScalabiltiyKillCounts.Reset();

	if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World))
	{
		const TMap<TWeakObjectPtr<UNiagaraSystem>, FNiagaraSystemScalabilityKillCounts>& SystemKillCounts = WorldMan->GetSystemScalabilityKillCounts();
		for (auto& KillCountEntry : SystemKillCounts)
		{
			if (UNiagaraSystem* System = KillCountEntry.Key.Get())
			{
				GlobalScalabiltiyKillCounts.Add(KillCountEntry.Value);
				FSystemDebugInfo& SystemDebugInfo = PerSystemDebugInfo.FindOrAdd(System->GetFName());
				SystemDebugInfo.ScalabilityKillCounts = KillCountEntry.Value;
			}
		}
	}
}

const FNiagaraDataSet* FNiagaraDebugHud::GetParticleDataSet(FNiagaraSystemInstance* SystemInstance, FNiagaraEmitterInstance* EmitterInstance, int32 iEmitter)
{
	using namespace NiagaraDebugLocal;

	if (EmitterInstance == nullptr || SystemInstance == nullptr)
	{
		return nullptr;
	}

	// For GPU context we need to readback and cache the data
	if (EmitterInstance->GetSimTarget() == ENiagaraSimTarget::GPUComputeSim)
	{
		if (!Settings.bEnableGpuParticleReadback)
		{
			return nullptr;
		}

		FGpuEmitterCache* GpuCachedData = GpuEmitterData.Find(SystemInstance->GetId());
		if ( GpuCachedData == nullptr )
		{
			const int32 NumEmitters = SystemInstance->GetEmitters().Num();
			GpuCachedData = &GpuEmitterData.Emplace(SystemInstance->GetId());
			GpuCachedData->CurrentEmitterData.AddDefaulted(NumEmitters);
			GpuCachedData->PendingEmitterData.AddDefaulted(NumEmitters);
		}
		else if (!GpuCachedData->PendingEmitterData.IsValidIndex(iEmitter))
		{
			// In case we didn't get the change notification make sure we update the arrays
			GpuCachedData->CurrentEmitterData.SetNum(iEmitter + 1);
			GpuCachedData->PendingEmitterData.SetNum(iEmitter + 1);
		}
		GpuCachedData->LastAccessedCycles = FPlatformTime::Cycles64();

		// Pending readback complete?
		if (GpuCachedData->PendingEmitterData[iEmitter] && GpuCachedData->PendingEmitterData[iEmitter]->IsReady())
		{
			GpuCachedData->CurrentEmitterData[iEmitter] = GpuCachedData->PendingEmitterData[iEmitter];
			GpuCachedData->PendingEmitterData[iEmitter] = nullptr;
		}

		// Enqueue a readback?
		if ( GpuCachedData->PendingEmitterData[iEmitter] == nullptr )
		{
			GpuCachedData->PendingEmitterData[iEmitter] = MakeShared<FNiagaraDataSetReadback, ESPMode::ThreadSafe>();
			GpuCachedData->PendingEmitterData[iEmitter]->EnqueueReadback(EmitterInstance);
		}

		// Pull current data if we have one
		if ( GpuCachedData->CurrentEmitterData[iEmitter] )
		{
			return &GpuCachedData->CurrentEmitterData[iEmitter]->GetDataSet();
		}
		return nullptr;
	}

	return &EmitterInstance->GetParticleData();
}

FNiagaraDebugHud::FValidationErrorInfo& FNiagaraDebugHud::GetValidationErrorInfo(UNiagaraComponent* NiagaraComponent)
{
	FValidationErrorInfo* InfoOut = ValidationErrors.Find(NiagaraComponent);
	if (!InfoOut)
	{
		InfoOut = &ValidationErrors.Add(NiagaraComponent);
		InfoOut->DisplayName = GetNameSafe(NiagaraComponent->GetOwner()) / *NiagaraComponent->GetName() / *GetNameSafe(NiagaraComponent->GetAsset());
	}
	InfoOut->LastWarningTime = FPlatformTime::Seconds();
	return *InfoOut;
}

void FNiagaraDebugHud::DebugDrawCallback(UCanvas* Canvas, APlayerController* PC)
{
	using namespace NiagaraDebugLocal;

	if (!Settings.IsEnabled())
	{
		return;
	}

	if (!Canvas || !Canvas->Canvas || !Canvas->SceneView || !Canvas->SceneView->Family || !Canvas->SceneView->Family->Scene)
	{
		return;
	}

	if ( UWorld* World = Canvas->SceneView->Family->Scene->GetWorld())
	{
		if ( FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World) )
		{
			if (FNiagaraDebugHud* DebugHud = WorldManager->GetNiagaraDebugHud())
			{
				DebugHud->Draw(WorldManager, Canvas, PC);
			}
		}
	}
}

void FNiagaraDebugHud::Draw(FNiagaraWorldManager* WorldManager, UCanvas* Canvas, APlayerController* PC)
{
	using namespace NiagaraDebugLocal;

	const double CurrTime = WorldManager->GetWorld()->GetRealTimeSeconds();
	DeltaSeconds = float(CurrTime - LastDrawTime);

	// We may want to use the HUD to pause / step but not actually display anything on screen
	if (Settings.bHudRenderingEnabled && Canvas && Canvas->Canvas)
	{
		// Draw in world components
		DrawComponents(WorldManager, Canvas);

		DrawDebugGeomerty(WorldManager, Canvas);

		// Draw overview
		DrawOverview(WorldManager, Canvas->Canvas);
	}

	// Scrub any gpu cached emitters we haven't used in a while
	{
		static double ScrubDurationSeconds = 1.0;
		const uint64 ScrubDurationCycles = uint64(ScrubDurationSeconds / FPlatformTime::GetSecondsPerCycle64());
		const uint64 ScrubCycles = FPlatformTime::Cycles64() - ScrubDurationCycles;

		for ( auto it=GpuEmitterData.CreateIterator(); it; ++it )
		{
			const FGpuEmitterCache& CachedData = it.Value();
			if ( CachedData.LastAccessedCycles < ScrubCycles )
			{
				it.RemoveCurrent();
			}
		}
	}

#if WITH_NIAGARA_GPU_PROFILER
	// Kill GPU results that are no longer relevant
	if ( GpuResults.IsValid() && (GFrameCounter - GpuResultsGameFrameCounter) > SmoothedNumFrames)
	{
		GpuResults = nullptr;
	}
#endif

	LastDrawTime = CurrTime;
}

void FNiagaraDebugHud::AddLine2D(FVector2f Start, FVector2f End, FLinearColor Color, float Thickness, float Lifetime)
{
	if (NiagaraDebugLocal::Settings.IsEnabled())
	{
		FDebugLine2D NewLine;
		NewLine.Start = Start;
		NewLine.End = End;
		NewLine.Color = Color;
		NewLine.Thickness = Thickness;
		NewLine.Lifetime = Lifetime;
		Lines2D.Add(NewLine);
	}
}

void FNiagaraDebugHud::AddCircle2D(FVector2f Pos, float Rad, int32 Segments, FLinearColor Color, float Thickness, float Lifetime)
{
	if (NiagaraDebugLocal::Settings.IsEnabled())
	{
		FDebugCircle2D NewCircle;
		NewCircle.Pos = Pos;
		NewCircle.Rad = Rad;
		NewCircle.Segments = Segments;
		NewCircle.Color = Color;
		NewCircle.Thickness = Thickness;
		NewCircle.Lifetime = Lifetime;
		Circles2D.Add(NewCircle);
	}
}

void FNiagaraDebugHud::AddBox2D(FVector2f Pos, FVector2f Extents, FLinearColor Color, float Thickness, float Lifetime)
{
	if (NiagaraDebugLocal::Settings.IsEnabled())
	{
		FDebugBox2D NewBox;
		NewBox.Pos = Pos;
		NewBox.Extents = Extents;
		NewBox.Color = Color;
		NewBox.Thickness = Thickness;
		NewBox.Lifetime = Lifetime;
		Boxes2D.Add(NewBox);
	}
}

void FNiagaraDebugHud::DrawDebugGeomerty(class FNiagaraWorldManager* WorldManager, class UCanvas* Canvas)
{
	UWorld* World = WorldManager->GetWorld();

	if (World == nullptr || Canvas == nullptr)
	{
		return;
	}

	FCanvas* DrawCanvas = Canvas->Canvas;

	FVector2f Size(Canvas->ClipX, Canvas->ClipY);
	
	for (auto it = Lines2D.CreateIterator(); it; ++it)
	{
		FDebugLine2D& Line = *it;
		DrawDebugCanvas2DLine(Canvas, FVector2D(Line.Start * Size), FVector2D(Line.End * Size), Line.Color, Line.Thickness);

		Line.Lifetime -= DeltaSeconds;
		if (Line.Lifetime <= 0.0f)
		{
			it.RemoveCurrent();
		}
	}
	for (auto it = Circles2D.CreateIterator(); it; ++it)
	{
		FDebugCircle2D& Circle = *it;
		DrawDebugCanvas2DCircle(Canvas, FVector2D(Circle.Pos * Size), Circle.Rad * Size.X, Circle.Segments, Circle.Color, Circle.Thickness);

		Circle.Lifetime -= DeltaSeconds;
		if (Circle.Lifetime <= 0.0f)
		{
			it.RemoveCurrent();
		}
	}
	for (auto it = Boxes2D.CreateIterator(); it; ++it)
	{
		FDebugBox2D& Box = *it;

		FBox2D Box2D;
		Box2D.Min = FVector2D((Box.Pos - Box.Extents) * Size);
		Box2D.Max = FVector2D((Box.Pos + Box.Extents) * Size);
		DrawDebugCanvas2DBox(Canvas, Box2D, Box.Color, Box.Thickness);

		Box.Lifetime -= DeltaSeconds;
		if (Box.Lifetime <= 0.0f)
		{
			it.RemoveCurrent();
		}
	}
}

#if WITH_EDITORONLY_DATA
void FNiagaraDebugHud::OnSystemCompiled(UNiagaraSystem * NiagaraSystem)
{
	if (FSystemDebugInfo* DebugInfo = PerSystemDebugInfo.Find(NiagaraSystem->GetFName()))
	{
		DebugInfo->SystemPrettyName.Empty();
	}

	NiagaraDebugLocal::GCachedSystemVariables.Remove(NiagaraSystem);
}
#endif

template<typename T>
struct FGraph
{
	FCanvas* Canvas;
	UFont* Font;

	FVector2f Location;
	FVector2f Size;

	FVector2f MaxValues;

	FString XLabel;
	FString YLabel;

	float EstYAxisLabelWidth = 35.0f;

	FGraph(FCanvas* InCanvas, UFont* InFont, FVector2f InLocation, FVector2f InSize, FVector2f InMaxValues, FString InXLabel, FString InYLabel)
		: Canvas(InCanvas)
		, Font(InFont)
		, Location(InLocation)
		, Size(InSize)
		, MaxValues(InMaxValues)
		, XLabel(InXLabel)
		, YLabel(InYLabel)
	{
	}

	void DrawLine(FString Name, FLinearColor Color, const TArray<T>& Samples)
	{
		if(Samples.IsEmpty())
		{
			return;
		}

		FBatchedElements* BatchedElements = Canvas->GetBatchedElements(FCanvas::ET_Line);
		BatchedElements->AddReserveLines(Samples.Num() - 1);
		FHitProxyId HitProxyId = Canvas->GetHitProxyId();

		float Top = Location.Y;
		float Bottom = Top + Size.Y;
		float Left = Location.X + EstYAxisLabelWidth;
		float Right = Left + Size.X + EstYAxisLabelWidth;

		float PerSampleWidth = Size.X / Samples.Num();
		for (int32 i = 0; i < Samples.Num() - 1; ++i)
		{
			double Sample = Samples[i];
			double NextSample = Samples[i + 1];

			float StartX = Left + i * PerSampleWidth;
			float EndX = Left + (i + 1) * PerSampleWidth;
			float StartY = FMath::Lerp(Bottom, Top, Sample / MaxValues.Y);
			float EndY = FMath::Lerp(Bottom, Top, NextSample / MaxValues.Y);
			BatchedElements->AddLine(FVector(StartX, StartY, 0.0f), FVector(EndX, EndY, 0.0f), Color, HitProxyId, 1.0f);
		}
	}

	void Draw(FLinearColor AxesColor, FLinearColor BackgroundColor)
	{
		const float fAdvanceHeight = Font->GetMaxCharHeight() + 1.0f;

		float Top = Location.Y;
		float Bottom = Top + Size.Y;
		float Left = Location.X + EstYAxisLabelWidth;
		float Right = Left + Size.X;

		//Draw BG
		Canvas->DrawTile(Left - EstYAxisLabelWidth, Top - fAdvanceHeight, Size.X + EstYAxisLabelWidth+ EstYAxisLabelWidth, Size.Y + fAdvanceHeight + fAdvanceHeight, 0.0f, 0.0f, 0.0f, 0.0f, BackgroundColor);

		int32 NumGridLines = 5;
		FBatchedElements* BatchedElements = Canvas->GetBatchedElements(FCanvas::ET_Line);
		BatchedElements->AddReserveLines(2 + NumGridLines);
		FHitProxyId HitProxyId = Canvas->GetHitProxyId();

		FLinearColor GraphAxesColor = AxesColor;

		//Draw Axes
		BatchedElements->AddLine(FVector(Left, Top, 0.0f), FVector(Left, Bottom, 0.0f), GraphAxesColor, HitProxyId, 3.0f);
		BatchedElements->AddLine(FVector(Left, Bottom, 0.0f), FVector(Right, Bottom, 0.0f), GraphAxesColor, HitProxyId, 3.0f);

		//Draw Gridlines
		for (int32 i = 1; i <= NumGridLines; ++i)
		{
			float LineTime = (MaxValues.Y / NumGridLines) * i;
			FString LineTimeString = LexToSanitizedString(LineTime);
			FVector2f LabelSize = FVector2f(float(Font->GetStringSize(*LineTimeString)), Font->GetMaxCharHeight());
			float LineX = Left;
			float LineY = FMath::Lerp(Bottom, Top, (float)i / NumGridLines);
			BatchedElements->AddLine(FVector(Left, LineY, 0.0f), FVector(LineX + Size.X, LineY, 0.0f), GraphAxesColor * 0.5f, HitProxyId, 1.0f);

			float LabelX = LineX - LabelSize.X - 2.0f;
			float LabelY = LineY - (LabelSize.Y * 0.5f);
			Canvas->DrawShadowedString(LabelX, LabelY, *LineTimeString, Font, GraphAxesColor);
		}
	}
};

void FNiagaraDebugHud::DrawOverview(class FNiagaraWorldManager* WorldManager, FCanvas* DrawCanvas)
{
	using namespace NiagaraDebugLocal;

	UFont* Font = GetFont(Settings.OverviewFont);
	const float fAdvanceHeight = Font->GetMaxCharHeight() + 1.0f;

	const FLinearColor HeadingColor = Settings.OverviewHeadingColor;
	const FLinearColor DetailColor = Settings.OverviewDetailColor;
	const FLinearColor DetailHighlightColor = Settings.OverviewDetailHighlightColor;
	const FLinearColor BackgroundColor = Settings.DefaultBackgroundColor;

	FVector2f TextLocation = FVector2f(Settings.OverviewLocation);

	struct FOverviewColumn
	{
		float ColumnSeparator = 10.0f;
		float GlobalDataSeparator = 5.0f;

		FString GlobalHeader;
		FString GlobalData;
		FString SystemHeader;

		float Offset;
		float HeaderWidth;
		float SystemWidth;
		float MaxWidth;

		typedef TFunction<void(FCanvas*, UFont*, float, float, FOverviewColumn&, const FSystemDebugInfo&)>  FSystemDrawFunc;
		FSystemDrawFunc SystemDrawFunc;

		FOverviewColumn(FString InGlobalHeader, FString InGlobalData, FString InSystemHeader, float& InOffset, UFont* Font, const TCHAR* ExampleSystemString, FSystemDrawFunc InSystemDrawFunc)
			: GlobalHeader(InGlobalHeader)
			, GlobalData(InGlobalData)
			, SystemHeader(InSystemHeader)
			, Offset(InOffset)
			, SystemDrawFunc(InSystemDrawFunc)
		{
			HeaderWidth = float(GetStringSize(Font, *GlobalHeader).X);
			SystemWidth = float(FMath::Max(GetStringSize(Font, *SystemHeader).X, GetStringSize(Font, ExampleSystemString).X));
			MaxWidth = float(FMath::Max(HeaderWidth + GlobalDataSeparator + GetStringSize(Font, *GlobalData).X, SystemWidth)) + ColumnSeparator;

			InOffset += MaxWidth;
		}

		FOverviewColumn(FString InGlobalHeader, FString InGlobalData, int32 GlobalDataStringSize, FString InSystemHeader, float& InOffset, UFont* Font, int32 SystemStringSize, FSystemDrawFunc InSystemDrawFunc)
			: GlobalHeader(InGlobalHeader)
			, GlobalData(InGlobalData)
			, SystemHeader(InSystemHeader)
			, Offset(InOffset)
			, SystemDrawFunc(InSystemDrawFunc)
		{
			GlobalDataStringSize = FMath::Max(GetStringSize(Font, *GlobalData).X, GlobalDataStringSize);

			HeaderWidth = float(GetStringSize(Font, *GlobalHeader).X);
			SystemWidth = float(FMath::Max(GetStringSize(Font, *SystemHeader).X, SystemStringSize));
			MaxWidth = float(FMath::Max(HeaderWidth + GlobalDataSeparator + GlobalDataStringSize, SystemWidth)) + ColumnSeparator;

			InOffset += MaxWidth;
		}

		void DrawGlobalHeader(FCanvas* DrawCanvas, UFont* Font, float X, float Y, FLinearColor HeadingColor, FLinearColor DetailColor)
		{
			DrawCanvas->DrawShadowedString(X + Offset, Y, *GlobalHeader, Font, HeadingColor);
			DrawCanvas->DrawShadowedString(X + Offset + HeaderWidth + GlobalDataSeparator, Y, *GlobalData, Font, DetailColor);
		}

		void DrawSystemHeader(FCanvas* DrawCanvas, UFont* Font, float X, float Y, FLinearColor HeadingColor)
		{
			DrawCanvas->DrawShadowedString(X + Offset, Y, *SystemHeader, Font, HeadingColor);
		}

		void DrawSystemData(FCanvas* DrawCanvas, UFont* Font, float X, float Y, const FSystemDebugInfo& SystemInfo)
		{
			SystemDrawFunc(DrawCanvas, Font, X + Offset, Y, *this, SystemInfo);
		}
	};

	float TotalOverviewWidth = 750.0f;
	float ColumnOffset = 0.0f;
	TArray<FOverviewColumn, TInlineAllocator<16>> OverviewColumns;

	if (Settings.bOverviewEnabled)
	{
		OverviewColumns.Emplace(TEXT(""), TEXT(""), TEXT("System Name"), ColumnOffset, Font, *LongestSystemPrettyName,
			[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
			{
				FLinearColor RowBGColor = SystemInfo.UniqueColor;
				RowBGColor.A = Settings.SystemColorTableOpacity;
				Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0,0,0,0, RowBGColor);
				const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;

				FNameBuilder SystemNameString;
				SystemNameString.Append(*SystemInfo.SystemPrettyName);
				Canvas->DrawShadowedString(X, Y, SystemNameString.ToString(), Font, RowColor);
		});

		if (Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::Overview)
		{
			if (Settings.bShowRegisteredComponents)
			{
				OverviewColumns.Emplace(TEXT("TotalRegistered:"), FString::FromInt(GlobalTotalRegistered), TEXT("# Registered"), ColumnOffset, Font, TEXT("0000"),
					[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FString::FromInt(SystemInfo.TotalRegistered), Font, RowColor);
				});
			}

			OverviewColumns.Emplace(TEXT("TotalActive:"), FString::FromInt(GlobalTotalActive), TEXT("# Active"), ColumnOffset, Font, TEXT("0000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FString::FromInt(SystemInfo.TotalActive), Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("TotalScalability:"), FString::FromInt(GlobalTotalScalability), TEXT("# Scalability"), ColumnOffset, Font, TEXT("0000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FString::FromInt(SystemInfo.TotalScalability), Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("TotalEmitters:"), FString::FromInt(GlobalTotalEmitters), TEXT("# Emitters"), ColumnOffset, Font, TEXT("0000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FString::FromInt(SystemInfo.TotalEmitters), Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("TotalParticles:"), FString::FromInt(GlobalTotalParticles), TEXT("# Particles"), ColumnOffset, Font, TEXT("0000000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FString::FromInt(SystemInfo.TotalParticles), Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("TotalMemory:"), FString::Printf(TEXT("%6.2fmb"), float(double(GlobalTotalBytes) / (1024.0 * 1024.0))), TEXT("# MBytes"), ColumnOffset, Font, TEXT("00000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FString::Printf(TEXT("%6.2f"), double(SystemInfo.TotalBytes) / (1024.0 * 1024.0)), Font, RowColor);
				});
		}
		else if (Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::Scalability)
		{
			FString ScalabilitySleepVsKilledFmt(TEXT("%d | %d"));
			OverviewColumns.Emplace(TEXT("TotalActive:"), FString::FromInt(GlobalTotalActive), TEXT("# Active"), ColumnOffset, Font, TEXT("0000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FString::FromInt(SystemInfo.TotalActive), Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("TotalScalability:"), FString::FromInt(GlobalTotalScalability), TEXT("# Scalability"), ColumnOffset, Font, TEXT("0000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FString::FromInt(SystemInfo.TotalScalability), Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("TotalCulled:"), FString::Printf(TEXT("%d | %d"), GlobalTotalCulled, GlobalScalabiltiyKillCounts.Total), TEXT("# Culled (Asleep | Killed)"), ColumnOffset, Font, TEXT("000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					FString ValueStr = FString::Printf(TEXT("%d | %d"), SystemInfo.TotalCulled, SystemInfo.ScalabilityKillCounts.Total);
					Canvas->DrawShadowedString(X, Y, *ValueStr, Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("Distance:"), FString::Printf(TEXT("%d | %d"), GlobalTotalCulledByDistance, GlobalScalabiltiyKillCounts.Distance), TEXT("# Distance (Asleep | Killed)"), ColumnOffset, Font, TEXT("000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					FString ValueStr = FString::Printf(TEXT("%d | %d"), SystemInfo.TotalCulledByDistance, SystemInfo.ScalabilityKillCounts.Distance);
					Canvas->DrawShadowedString(X, Y, *ValueStr, Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("Visibility:"), FString::Printf(TEXT("%d | %d"), GlobalTotalCulledByVisibility, GlobalScalabiltiyKillCounts.Visibility), TEXT("# Visibility (Asleep | Killed)"), ColumnOffset, Font, TEXT("000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					FString ValueStr = FString::Printf(TEXT("%d | %d"), SystemInfo.TotalCulledByVisibility, SystemInfo.ScalabilityKillCounts.Visibility);
					Canvas->DrawShadowedString(X, Y, *ValueStr, Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("InstanceCount:"), FString::Printf(TEXT("%d | %d"), GlobalTotalCulledByInstanceCount, GlobalScalabiltiyKillCounts.InstanceCounts), TEXT("# Inst Count (Asleep | Killed)"), ColumnOffset, Font, TEXT("000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					FString ValueStr = FString::Printf(TEXT("%d | %d"), SystemInfo.TotalCulledByInstanceCount, SystemInfo.ScalabilityKillCounts.InstanceCounts);
					Canvas->DrawShadowedString(X, Y, *ValueStr, Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("Budget:"), FString::Printf(TEXT("%d | %d"), GlobalTotalCulledByBudget, GlobalScalabiltiyKillCounts.GlobalBudget), TEXT("# Budget (Asleep | Killed)"), ColumnOffset, Font, TEXT("000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					FString ValueStr = FString::Printf(TEXT("%d | %d"), SystemInfo.TotalCulledByBudget, SystemInfo.ScalabilityKillCounts.GlobalBudget);
					Canvas->DrawShadowedString(X, Y, *ValueStr, Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("Player FX:"), FString::FromInt(GlobalTotalPlayerSystems), TEXT("# Player"), ColumnOffset, Font, TEXT("000"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FString::FromInt(SystemInfo.TotalPlayerSystems), Font, RowColor);
				});
		}
#if WITH_PARTICLE_PERF_STATS
		else if (Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::Performance && StatsListener)
		{
			FNiagaraDebugHUDPerfStats& GlobalPerfStats = StatsListener->GetGlobalStats();

			const int32 GlobalDataStringSize = GetStringSize(Font, TEXT("000,000")).X;
			const int32 SystemStringSize = GetStringSize(Font, TEXT("000,000")).X;

			OverviewColumns.Emplace(TEXT("Game Thread Avg:"), FormatPerfValue(GlobalPerfStats.Avg.Time_GT), GlobalDataStringSize, FormatPerfString(TEXT("GT Avg")), ColumnOffset, Font, SystemStringSize,
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FormatPerfValue(SystemInfo.PerfStats ? SystemInfo.PerfStats->Avg.Time_GT : 0.0), Font, RowColor);
				});

			if (Settings.bShowPerfColumGameThreadOnly)
			{
				OverviewColumns.Emplace(TEXT("Game Thread Only Avg:"), FormatPerfValue(GlobalPerfStats.Avg.Time_GTOnly), GlobalDataStringSize, FormatPerfString(TEXT("GT Only Avg")), ColumnOffset, Font, SystemStringSize,
					[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
					{
						FLinearColor RowBGColor = SystemInfo.UniqueColor;
						RowBGColor.A = Settings.SystemColorTableOpacity;
						Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
						const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
						Canvas->DrawShadowedString(X, Y, *FormatPerfValue(SystemInfo.PerfStats ? SystemInfo.PerfStats->Avg.Time_GTOnly : 0.0), Font, RowColor);
					});
			}

			OverviewColumns.Emplace(TEXT("Game Thread Max:"), FormatPerfValue(GlobalPerfStats.Max.Time_GT), GlobalDataStringSize, FormatPerfString(TEXT("GT Max")), ColumnOffset, Font, SystemStringSize,
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FormatPerfValue(SystemInfo.PerfStats ? SystemInfo.PerfStats->Max.Time_GT : 0.0), Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("Render Thread Avg:"), FormatPerfValue(GlobalPerfStats.Avg.Time_RT), GlobalDataStringSize, FormatPerfString(TEXT("RT Avg")), ColumnOffset, Font, SystemStringSize,
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FormatPerfValue(SystemInfo.PerfStats ? SystemInfo.PerfStats->Avg.Time_RT : 0.0), Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("Render Thread Max:"), FormatPerfValue(GlobalPerfStats.Max.Time_RT), GlobalDataStringSize, FormatPerfString(TEXT("RT Max")), ColumnOffset, Font, SystemStringSize,
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FormatPerfValue(SystemInfo.PerfStats ? SystemInfo.PerfStats->Max.Time_RT : 0.0), Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("Gpu Avg:"), FormatPerfValue(GlobalPerfStats.Avg.Time_GPU), GlobalDataStringSize, FormatPerfString(TEXT("Gpu Avg")), ColumnOffset, Font, SystemStringSize,
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FormatPerfValue(SystemInfo.PerfStats ? SystemInfo.PerfStats->Avg.Time_GPU : 0.0), Font, RowColor);
				});

			OverviewColumns.Emplace(TEXT("Gpu Max:"), FormatPerfValue(GlobalPerfStats.Max.Time_GPU), GlobalDataStringSize, FormatPerfString(TEXT("Gpu Max")), ColumnOffset, Font, SystemStringSize,
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					FLinearColor RowBGColor = SystemInfo.UniqueColor;
					RowBGColor.A = Settings.SystemColorTableOpacity;
					Canvas->DrawTile(X, Y, Col.MaxWidth, fAdvanceHeight, 0, 0, 0, 0, RowBGColor);
					const FLinearColor RowColor = SystemInfo.bShowInWorld ? DetailHighlightColor : DetailColor;
					Canvas->DrawShadowedString(X, Y, *FormatPerfValue(SystemInfo.PerfStats ? SystemInfo.PerfStats->Max.Time_GPU : 0.0), Font, RowColor);
				});
		}
		else if (Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph && StatsListener)
		{
			OverviewColumns.Emplace(TEXT("Total Active"), TEXT(""), TEXT(""), ColumnOffset, Font, TEXT("------"),
				[&DetailColor, &DetailHighlightColor, &fAdvanceHeight](FCanvas* Canvas, UFont* Font, float X, float Y, FOverviewColumn& Col, const FSystemDebugInfo& SystemInfo)
				{
					Canvas->DrawTile(X, Y, Col.MaxWidth - 6.0f, fAdvanceHeight - 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, SystemInfo.UniqueColor);
					FString SysCountString = LexToSanitizedString(SystemInfo.TotalActive);
					float StringWidth = float(Font->GetStringSize(*SysCountString));
					Canvas->DrawShadowedString((X + (Col.MaxWidth * 0.5f)) - StringWidth * 0.5f, Y, *SysCountString, Font, FLinearColor::Black);
				});
		}
#endif//WITH_PARTICLE_PERF_STATS

		TotalOverviewWidth = ColumnOffset;
	}

	// Display overview
	{
		TStringBuilder<1024> OverviewString;
		{
			static const auto CVarGlobalLoopTime = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.Niagara.Debug.GlobalLoopTime"));

			const TCHAR* Separator = TEXT("    ");
			const ENiagaraDebugPlaybackMode PlaybackMode = WorldManager->GetDebugPlaybackMode();
			const float PlaybackRate = WorldManager->GetDebugPlaybackRate();
			const float PlaybackLoopTime = CVarGlobalLoopTime ? CVarGlobalLoopTime->GetFloat() : 0.0f;

			bool bRequiresNewline = false;
			if (PlaybackMode != ENiagaraDebugPlaybackMode::Play)
			{
				bRequiresNewline = true;
				OverviewString.Append(TEXT("PlaybackMode: "));
				switch (WorldManager->GetDebugPlaybackMode())
				{
					case ENiagaraDebugPlaybackMode::Loop:	OverviewString.Append(TEXT("Looping")); break;
					case ENiagaraDebugPlaybackMode::Paused:	OverviewString.Append(TEXT("Paused")); break;
					case ENiagaraDebugPlaybackMode::Step:	OverviewString.Append(TEXT("Step")); break;
					default:								OverviewString.Append(TEXT("Unknown")); break;
				}
				OverviewString.Append(Separator);
			}
			if (!FMath::IsNearlyEqual(PlaybackRate, 1.0f))
			{
				bRequiresNewline = true;
				OverviewString.Appendf(TEXT("PlaybackRate: %.4f"), PlaybackRate);
				OverviewString.Append(Separator);
			}
			if (!FMath::IsNearlyEqual(PlaybackLoopTime, 0.0f))
			{
				bRequiresNewline = true;
				OverviewString.Appendf(TEXT("LoopTime: %.2f"), PlaybackLoopTime);
				OverviewString.Append(Separator);
			}
			if (bRequiresNewline)
			{
				bRequiresNewline = false;
				OverviewString.Append(TEXT("\n"));
			}	

			// Display any filters we may have
			if (Settings.bSystemFilterEnabled || Settings.bEmitterFilterEnabled || Settings.bActorFilterEnabled || Settings.bComponentFilterEnabled)
			{
				if (Settings.bSystemFilterEnabled)
				{
					OverviewString.Appendf(TEXT("SystemFilter: %s"), *Settings.SystemFilter);
					OverviewString.Append(Separator);
				}
				if (Settings.bEmitterFilterEnabled)
				{
					OverviewString.Appendf(TEXT("EmitterFilter: %s"), *Settings.EmitterFilter);
					OverviewString.Append(Separator);
				}
				if (Settings.bActorFilterEnabled)
				{
					OverviewString.Appendf(TEXT("ActorFilter: %s"), *Settings.ActorFilter);
					OverviewString.Append(Separator);
				}
				if (Settings.bComponentFilterEnabled)
				{
					OverviewString.Appendf(TEXT("ComponentFilter: %s"), *Settings.ComponentFilter);
					OverviewString.Append(Separator);
				}
			}
		}

		if (Settings.bOverviewEnabled || OverviewString.Len() > 0)
		{
			const int32 NumLines = 2 + (Settings.bOverviewEnabled ? 1 : 0);
			const FVector2f OverviewStringSize = GetStringSize(Font, OverviewString.ToString());
			const FVector2f ActualSize(FMath::Max(OverviewStringSize.X, TotalOverviewWidth), (NumLines*fAdvanceHeight) + OverviewStringSize.Y);

			// Draw background
			DrawCanvas->DrawTile(TextLocation.X - 1.0f, TextLocation.Y - 1.0f, ActualSize.X + 2.0f, ActualSize.Y + 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, BackgroundColor);

			// Draw string
			DrawCanvas->DrawShadowedString(TextLocation.X, TextLocation.Y, TEXT("Niagara DebugHud"), Font, HeadingColor);
			TextLocation.Y += fAdvanceHeight;

			const UEnum* EnumPtr = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Niagara.ENiagaraDebugHUDOverviewMode"), EFindObjectFlags::ExactClass);
			TStringBuilder<128> ModelStringBuilder;
			ModelStringBuilder.Append(EnumPtr->GetDisplayNameTextByValue((int32)Settings.OverviewMode).ToString());
			if(Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph)
			{
				const UEnum* PerfGraphModeEnumPtr = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Niagara.ENiagaraDebugHUDPerfGraphMode"), EFindObjectFlags::ExactClass);
				const UEnum* PerfGraphSampleModeEnumPtr = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Niagara.ENiagaraDebugHUDPerfSampleMode"), EFindObjectFlags::ExactClass);
				
				ModelStringBuilder.Append(TEXT(" - "));
				ModelStringBuilder.Append(PerfGraphModeEnumPtr->GetDisplayNameTextByValue((int32)Settings.PerfGraphMode).ToString());
				ModelStringBuilder.Append(TEXT(" - "));
				ModelStringBuilder.Append(PerfGraphSampleModeEnumPtr->GetDisplayNameTextByValue((int32)Settings.PerfSampleMode).ToString());
			}			
			DrawCanvas->DrawShadowedString(TextLocation.X, TextLocation.Y, ModelStringBuilder.ToString(), Font, HeadingColor);
			ModelStringBuilder.Reset();

			TextLocation.Y += fAdvanceHeight;

			if (OverviewString.Len() > 0)
			{
				DrawCanvas->DrawShadowedString(TextLocation.X, TextLocation.Y, OverviewString.ToString(), Font, HeadingColor);
				TextLocation.Y += OverviewStringSize.Y;
			}

			// Display global system information
			if (Settings.bOverviewEnabled)
			{
				for (FOverviewColumn& Column : OverviewColumns)
				{
					Column.DrawGlobalHeader(DrawCanvas, Font, TextLocation.X, TextLocation.Y, HeadingColor, DetailColor);
				}
				TextLocation.Y += fAdvanceHeight;
			}
		}
	}

	//-TODO: Improve the column view to allow custom views like this
#if WITH_NIAGARA_GPU_PROFILER
	if (Settings.bOverviewEnabled && Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::GpuComputePerformance)
	{
		DrawGpuComputeOverriew(WorldManager, DrawCanvas, TextLocation);
	}
#endif //WITH_NIAGARA_GPU_PROFILER

	// Display active systems information
	if (Settings.bOverviewEnabled && Settings.OverviewMode != ENiagaraDebugHUDOverviewMode::GpuComputePerformance)
	{
		TextLocation.Y += fAdvanceHeight;		

		// Filter out what we won't display on the HUD and calculate number of lines
		using FSortedPerSystemDebugInfo = TPair<FName, const FSystemDebugInfo&>;
		TArray<FSortedPerSystemDebugInfo> SortedPerSystemDebugInfo;
		SortedPerSystemDebugInfo.Reserve(PerSystemDebugInfo.Num());

		for (const auto& Pair : PerSystemDebugInfo)
		{
			const FSystemDebugInfo& SystemInfo = Pair.Value;
			if ((SystemInfo.FramesSinceVisible >= Settings.PerfHistoryFrames) ||
				(Settings.bSystemFilterEnabled && Settings.bOverviewShowFilteredSystemOnly && !SystemInfo.bPassesSystemFilter))
			{
				continue;
			}

			SortedPerSystemDebugInfo.Emplace(Pair);
		}

		// Sort our display list, if enabled
		switch (Settings.OverviewSortMode)
		{
			case ENiagaraDebugHUDDOverviewSort::Name:				Algo::Sort(SortedPerSystemDebugInfo, [](const FSortedPerSystemDebugInfo& Lhs, const FSortedPerSystemDebugInfo& Rhs) -> bool { return Lhs.Key.LexicalLess(Rhs.Key); });	break;
			case ENiagaraDebugHUDDOverviewSort::NumberRegistered:	Algo::Sort(SortedPerSystemDebugInfo, [](const FSortedPerSystemDebugInfo& Lhs, const FSortedPerSystemDebugInfo& Rhs) -> bool { return Lhs.Value.TotalRegistered    > Rhs.Value.TotalRegistered; }); break;
			case ENiagaraDebugHUDDOverviewSort::NumberActive:		Algo::Sort(SortedPerSystemDebugInfo, [](const FSortedPerSystemDebugInfo& Lhs, const FSortedPerSystemDebugInfo& Rhs) -> bool { return Lhs.Value.TotalActive        > Rhs.Value.TotalActive; }); break;
			case ENiagaraDebugHUDDOverviewSort::NumberScalability:	Algo::Sort(SortedPerSystemDebugInfo, [](const FSortedPerSystemDebugInfo& Lhs, const FSortedPerSystemDebugInfo& Rhs) -> bool { return Lhs.Value.TotalScalability   > Rhs.Value.TotalScalability; });	break;
			case ENiagaraDebugHUDDOverviewSort::MemoryUsage:		Algo::Sort(SortedPerSystemDebugInfo, [](const FSortedPerSystemDebugInfo& Lhs, const FSortedPerSystemDebugInfo& Rhs) -> bool { return Lhs.Value.TotalBytes         > Rhs.Value.TotalBytes; });	break;
			case ENiagaraDebugHUDDOverviewSort::RecentlyVisibilty:	Algo::Sort(SortedPerSystemDebugInfo, [](const FSortedPerSystemDebugInfo& Lhs, const FSortedPerSystemDebugInfo& Rhs) -> bool { return Lhs.Value.FramesSinceVisible < Rhs.Value.FramesSinceVisible; });	break;

			default:
				break;
		}

		// Draw background + headers
		DrawCanvas->DrawTile(TextLocation.X - 1.0f, TextLocation.Y - 1.0f, TotalOverviewWidth + 1.0f, 2.0f + (float(SortedPerSystemDebugInfo.Num() + 1) * fAdvanceHeight), 0.0f, 0.0f, 0.0f, 0.0f, BackgroundColor);

		float SystemDataY = TextLocation.Y;
		for (FOverviewColumn& Column : OverviewColumns)
		{
			Column.DrawSystemHeader(DrawCanvas, Font, TextLocation.X, TextLocation.Y, HeadingColor);
		}

		TextLocation.Y += fAdvanceHeight;

		// Draw each row
		for (const FSortedPerSystemDebugInfo& Pair : SortedPerSystemDebugInfo)
		{
			const FSystemDebugInfo& SystemInfo = Pair.Value;
			for (FOverviewColumn& Column : OverviewColumns)
			{
				Column.DrawSystemData(DrawCanvas, Font, TextLocation.X, TextLocation.Y, SystemInfo);
			}
			TextLocation.Y += fAdvanceHeight;
		}
		
		#if WITH_PARTICLE_PERF_STATS
		//Draw graph
		if (Settings.OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph && StatsListener)
		{
			FLinearColor GraphAxesColor = Settings.PerfGraphAxisColor;

			TextLocation.Y += fAdvanceHeight;

			FVector2f GraphLocation(TextLocation.X, SystemDataY + fAdvanceHeight);

			if (OverviewColumns.Num())
			{
				GraphLocation.X += OverviewColumns.Last().Offset + OverviewColumns.Last().MaxWidth + 50.0f;
			}

			const double MillisecondsToPerfUnits = GetMillisecondsToPerfUnits();
			const float GraphTimeRange = (Settings.bUsePerfGraphTimeRange ? Settings.PerfGraphTimeRange : AutoGraphTimeRange) * MillisecondsToPerfUnits;

			FGraph<double> Graph(DrawCanvas, Font, GraphLocation, FVector2f(Settings.PerfGraphSize), FVector2f(float(Settings.PerfHistoryFrames), GraphTimeRange), TEXT("Frame"), FormatPerfString(TEXT("Time")));

			Graph.Draw(Settings.PerfGraphAxisColor, BackgroundColor);

			//Add each line to the graph.
			double NewGraphTimeRange = 0.0;
			for (const auto& Pair : PerSystemDebugInfo)
			{
				const FSystemDebugInfo& SysInfo = Pair.Value;
				if (SysInfo.PerfStats && SysInfo.bPassesSystemFilter)
				{
					TArray<double> Frames;
					if (Settings.PerfGraphMode == ENiagaraDebugHUDPerfGraphMode::GameThread)
					{
						SysInfo.PerfStats->History.GetHistoryFrames_GT(Frames);
					}
					else if (Settings.PerfGraphMode == ENiagaraDebugHUDPerfGraphMode::RenderThread)
					{
						SysInfo.PerfStats->History.GetHistoryFrames_RT(Frames);
					}
					else if (Settings.PerfGraphMode == ENiagaraDebugHUDPerfGraphMode::GPU)
					{
						SysInfo.PerfStats->History.GetHistoryFrames_GPU(Frames);
					}

					if (Settings.SmoothingWidth > 0)
					{
						TArray<double> Smoothed;
						Smoothed.Reserve(Frames.Num());
						for (int32 i = 0; i < Frames.Num(); ++i)
						{
							int32 SmoothSamples = 0;
							double SmoothTotal = 0.0f;
							for (int32 j = i - Settings.SmoothingWidth; j < i + Settings.SmoothingWidth; ++j)
							{
								if (j >= 0 && j < Frames.Num())
								{
									++SmoothSamples;
									SmoothTotal += Frames[j];
								}
							}

							Smoothed.Add(SmoothTotal / SmoothSamples);
						}

						Frames = MoveTemp(Smoothed);
					}

					for (double& Value : Frames)
					{
						NewGraphTimeRange = FMath::Max(Value, NewGraphTimeRange);
						Value *= MillisecondsToPerfUnits;
					}

					Graph.DrawLine(SysInfo.SystemName, SysInfo.UniqueColor, Frames);
				}
			}

			constexpr double AutoGraphTimeRangeChangeRate = 5.0;
			constexpr double AutoGraphMinUnits = 100.0;

			NewGraphTimeRange = FMath::CeilToDouble(NewGraphTimeRange / AutoGraphMinUnits) * AutoGraphMinUnits;
			NewGraphTimeRange = FMath::Max(NewGraphTimeRange, AutoGraphMinUnits);
			AutoGraphTimeRangeChangeMax = FMath::Max(AutoGraphTimeRangeChangeMax, NewGraphTimeRange);
			if (AutoGraphTimeRangeChangeMax > AutoGraphTimeRange || (FPlatformTime::Seconds() - AutoGraphTimeRangeChangedTime > AutoGraphTimeRangeChangeRate))
			{
				AutoGraphTimeRange				= AutoGraphTimeRangeChangeMax;
				AutoGraphTimeRangeChangeMax		= AutoGraphMinUnits;
				AutoGraphTimeRangeChangedTime	= FPlatformTime::Seconds();
			}
		}
		#endif//WITH_PARTICLE_PERF_STATS
	}

	TextLocation.Y += fAdvanceHeight;

	DrawGlobalBudgetInfo(WorldManager, DrawCanvas, TextLocation);
	DrawValidation(WorldManager, DrawCanvas, TextLocation);
	DrawMessages(WorldManager, DrawCanvas, TextLocation);
}

void FNiagaraDebugHud::DrawGpuComputeOverriew(class FNiagaraWorldManager* WorldManager, class FCanvas* DrawCanvas, FVector2f& TextLocation)
{
#if WITH_NIAGARA_GPU_PROFILER
	using namespace NiagaraDebugLocal;

	UFont* Font = GetFont(Settings.OverviewFont);
	const float fAdvanceHeight = Font->GetMaxCharHeight() + 1.0f;

	const FLinearColor HeadingColor = Settings.OverviewHeadingColor;
	const FLinearColor DetailColor = Settings.OverviewDetailColor;
	const FLinearColor DetailHighlightColor = Settings.OverviewDetailHighlightColor;
	const FLinearColor BackgroundColor = Settings.DefaultBackgroundColor;

	if (GpuResults == nullptr)
	{
		static const auto ProfilingEnabledCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.Niagara.GpuProfiling.Enabled"));

		static const FString EnableCVarWarning(TEXT("GPU Profiling is disabled, enable 'fx.Niagara.GpuProfiling.Enabled'"));
		static const FString NoDataWarning(TEXT("No GPU data is ready"));

		const FVector2f StringSize = GetStringSize(Font, *EnableCVarWarning);
		DrawCanvas->DrawTile(TextLocation.X - 1.0f, TextLocation.Y - 1.0f, StringSize.X + 1.0f, 2.0f + StringSize.Y, 0.0f, 0.0f, 0.0f, 0.0f, BackgroundColor);
		DrawCanvas->DrawShadowedString(TextLocation.X, TextLocation.Y, *(ProfilingEnabledCVar && ProfilingEnabledCVar->GetBool() ? NoDataWarning : EnableCVarWarning), Font, HeadingColor);
		TextLocation.Y += StringSize.Y;
		
		return;
	}

	struct FSimpleTableDraw
	{
		struct FColumn
		{
			float				DefaultWidth = 150.0f;
			FVector2f			MeasuredSize = FVector2f::ZeroVector;
			float				DrawOffset = 0.0f;
			TStringBuilder<128>	AllRowsText;
		};

		void AddColumns(int32 NumToAdd, float DefaultWidth = 150.0f)
		{
			for ( int32 i=0; i < NumToAdd; ++i )
			{
				FColumn& Column = Columns.AddDefaulted_GetRef();
				Column.DefaultWidth = DefaultWidth;
			}
		}

		TStringBuilder<128>& GetColumnText(int32 Column)
		{
			return Columns[Column].AllRowsText;
		}

		void RowComplete()
		{
			for ( int32 i=0; i < Columns.Num(); ++i )
			{
				Columns[i].AllRowsText.AppendChar(TEXT('\n'));
			}
		}

		void Draw(UFont* Font, FCanvas* DrawCanvas, FVector2f& Location, FLinearColor TextColor, FLinearColor BackgroundColor)
		{
			FVector2f TotalSize = FVector2f::ZeroVector;
			for ( FColumn& Column : Columns)
			{
				Column.MeasuredSize	= GetStringSize(Font, Column.AllRowsText.ToString());
				Column.DrawOffset	= TotalSize.X;

				const float SpaceBetweenColumns = (&Column == &Columns.Last()) ? 0.0f : DefaultSpaceBetweenColumns;
				TotalSize.X = Column.DrawOffset + FMath::Max(Column.MeasuredSize.X + SpaceBetweenColumns, Column.DefaultWidth);
				TotalSize.Y = FMath::Max(TotalSize.Y, Column.MeasuredSize.Y);
			}

			DrawCanvas->DrawTile(Location.X - 1.0f, Location.Y - 1.0f, TotalSize.X + 1.0f, 2.0f + TotalSize.Y, 0.0f, 0.0f, 0.0f, 0.0f, BackgroundColor);
			for (FColumn& Column : Columns)
			{
				DrawCanvas->DrawShadowedString(Location.X + Column.DrawOffset, Location.Y, Column.AllRowsText.ToString(), Font, TextColor);
			}

			Location.Y += TotalSize.Y;
		}

	private:
		float DefaultSpaceBetweenColumns = 10.0f;
		TArray<FColumn, TInlineAllocator<16>> Columns;
	};

	// Display overview of results
	{
		FSimpleTableDraw SimpleTable;
		SimpleTable.AddColumns(1, 250.0f);
		SimpleTable.AddColumns(2, 150.0f);
		SimpleTable.GetColumnText(0).Append(TEXT("Gpu Compute Overview"));
		SimpleTable.RowComplete();

		const UEnum* PerfUnitsEnum = StaticEnum<ENiagaraDebugHUDPerfUnits>();
		const FString PerfUnits = PerfUnitsEnum ? PerfUnitsEnum->GetNameStringByValue((int)Settings.PerfUnits) : FString();

		SimpleTable.GetColumnText(0).Appendf(TEXT("Total %s : %s"), *PerfUnits, *FormatPerfValue(GpuTotalMicroseconds.GetAverage()));
		SimpleTable.GetColumnText(1).Appendf(TEXT("Total Dispatches : %d"), GpuTotalDispatches.GetAverage());

		SimpleTable.Draw(Font, DrawCanvas, TextLocation, DetailColor, BackgroundColor);
		TextLocation.Y += fAdvanceHeight;
	}

	// Do a pass to determine what will be detailed view
	bool bHasDetailedView = false;
	bool bHasSimpleView = false;
	for (auto SystemIt = GpuUsagePerSystem.CreateIterator(); SystemIt; ++SystemIt)
	{
		// Prune data systems
		UNiagaraSystem* OwnerSystem = SystemIt.Key().Get();
		if (OwnerSystem == nullptr)
		{
			SystemIt.RemoveCurrent();
			continue;
		}

		// Prune dead emitters
		for (auto EmitterIt = SystemIt.Value().Emitters.CreateIterator(); EmitterIt; ++EmitterIt)
		{
			UNiagaraEmitter* OwnerEmitter = EmitterIt.Key().Get();
			if (OwnerEmitter == nullptr)
			{
				EmitterIt.RemoveCurrent();
			}
		}

		const bool bShowDetailed = !Settings.bSystemFilterEnabled || Settings.SystemFilter.IsEmpty() || OwnerSystem->GetName().MatchesWildcard(Settings.SystemFilter);
		SystemIt.Value().bShowDetailed = bShowDetailed;
		bHasDetailedView |= bShowDetailed;
		bHasSimpleView |= !bShowDetailed;
	}

	// Show detailed information first
	if (bHasDetailedView)
	{
		FSimpleTableDraw SimpleTable;
		SimpleTable.AddColumns(3, 200.0f);
		SimpleTable.AddColumns(3, 100.0f);

		bool bFirst = true;
		for (auto SystemIt = GpuUsagePerSystem.CreateIterator(); SystemIt; ++SystemIt)
		{
			const FGpuUsagePerSystem& SystemUsage = SystemIt.Value();
			if ( SystemUsage.bShowDetailed == false )
			{
				continue;
			}

			if (!bFirst)
			{
				SimpleTable.RowComplete();
			}
			bFirst = false;

			UNiagaraSystem* OwnerSystem = SystemIt.Key().Get();
			check(OwnerSystem);

			SimpleTable.GetColumnText(0).Append(TEXT("System Name"));
			SimpleTable.GetColumnText(1).Append(TEXT("Emitter Name"));
			SimpleTable.GetColumnText(2).Append(TEXT("Stage Name"));
			SimpleTable.GetColumnText(3).Append(TEXT("Avg Instances"));
			SimpleTable.GetColumnText(4).Append(FormatPerfString(TEXT("Avg")));
			SimpleTable.GetColumnText(5).Append(FormatPerfString(TEXT("Max")));
			SimpleTable.RowComplete();

			for (auto EmitterIt = SystemIt.Value().Emitters.CreateIterator(); EmitterIt; ++EmitterIt)
			{
				UNiagaraEmitter* OwnerEmitter = EmitterIt.Key().Get();
				check(OwnerEmitter);
				for (auto StageIt=EmitterIt.Value().Stages.CreateIterator(); StageIt; ++StageIt)
				{
					const FGpuUsagePerStage& StageUsage = StageIt.Value();
					OwnerSystem->GetFName().AppendString(SimpleTable.GetColumnText(0));
#if WITH_EDITORONLY_DATA
					if (OwnerSystem->GetCompileForEdit())
					{
						SimpleTable.GetColumnText(0).Append(GCompileForEditString);
					}
#endif
					OwnerEmitter->GetFName().AppendString(SimpleTable.GetColumnText(1));
					StageIt.Key().AppendString(SimpleTable.GetColumnText(2));
					SimpleTable.GetColumnText(3).Appendf(TEXT("%4.1f"), StageUsage.InstanceCount.GetAverage<float>());
					SimpleTable.GetColumnText(4).Append(FormatPerfValue(StageUsage.Microseconds.GetAverage()));
					SimpleTable.GetColumnText(5).Append(FormatPerfValue(StageUsage.Microseconds.GetMax()));
					SimpleTable.RowComplete();
				}
			}

			SimpleTable.GetColumnText(2).Append(TEXT("Total"));
			SimpleTable.GetColumnText(3).Appendf(TEXT("%4.1f"), SystemUsage.InstanceCount.GetAverage<float>());
			SimpleTable.GetColumnText(4).Append(FormatPerfValue(SystemUsage.Microseconds.GetAverage()));
			SimpleTable.GetColumnText(5).Append(FormatPerfValue(SystemUsage.Microseconds.GetMax()));
			SimpleTable.RowComplete();
		}
		SimpleTable.Draw(Font, DrawCanvas, TextLocation, DetailColor, BackgroundColor);
		TextLocation.Y += fAdvanceHeight;
	}

	// Show simple information
	if (bHasSimpleView)
	{
		FSimpleTableDraw SimpleTable;
		SimpleTable.AddColumns(1, 200.0f);
		SimpleTable.AddColumns(3, 100.0f);
		SimpleTable.GetColumnText(0).Append(TEXT("SystemName"));
		SimpleTable.GetColumnText(1).Append(TEXT("Avg Instances"));
		SimpleTable.GetColumnText(2).Append(FormatPerfString(TEXT("Avg")));
		SimpleTable.GetColumnText(3).Append(FormatPerfString(TEXT("Max")));
		SimpleTable.RowComplete();

		for (auto SystemIt=GpuUsagePerSystem.CreateIterator(); SystemIt; ++SystemIt)
		{
			if (SystemIt.Value().bShowDetailed == true)
			{
				continue;
			}

			UNiagaraSystem* OwnerSystem = SystemIt.Key().Get();
			check(OwnerSystem);

			const FGpuUsagePerSystem& SystemUsage = SystemIt.Value();

			OwnerSystem->GetFName().AppendString(SimpleTable.GetColumnText(0));
#if WITH_EDITORONLY_DATA
			if (OwnerSystem->GetCompileForEdit())
			{
				SimpleTable.GetColumnText(0).Append(GCompileForEditString);
			}
#endif
			SimpleTable.GetColumnText(1).Appendf(TEXT("%4.1f"), SystemUsage.InstanceCount.GetAverage<float>());
			SimpleTable.GetColumnText(2).Append(FormatPerfValue(SystemUsage.Microseconds.GetAverage()));
			SimpleTable.GetColumnText(3).Append(FormatPerfValue(SystemUsage.Microseconds.GetMax()));
			SimpleTable.RowComplete();
		}
		SimpleTable.Draw(Font, DrawCanvas, TextLocation, DetailColor, BackgroundColor);
		TextLocation.Y += fAdvanceHeight;
	}

	// Show events
	if (GpuUsagePerEvent.Num() > 0)
	{
		FSimpleTableDraw SimpleTable;
		SimpleTable.AddColumns(1, 200.0f);
		SimpleTable.AddColumns(3, 100.0f);
		SimpleTable.GetColumnText(0).Append(TEXT("EventName"));
		SimpleTable.GetColumnText(1).Append(TEXT("Avg Instances"));
		SimpleTable.GetColumnText(2).Append(FormatPerfString(TEXT("Avg")));
		SimpleTable.GetColumnText(3).Append(FormatPerfString(TEXT("Max")));
		SimpleTable.RowComplete();

		for (auto EventIt=GpuUsagePerEvent.CreateIterator(); EventIt; ++EventIt)
		{
			const FGpuUsagePerEvent& EventUsage = EventIt.Value();
			EventIt.Key().AppendString(SimpleTable.GetColumnText(0));
			SimpleTable.GetColumnText(1).Appendf(TEXT("%4.1f"), EventUsage.InstanceCount.GetAverage<float>());
			SimpleTable.GetColumnText(2).Append(FormatPerfValue(EventUsage.Microseconds.GetAverage()));
			SimpleTable.GetColumnText(3).Append(FormatPerfValue(EventUsage.Microseconds.GetMax()));
			SimpleTable.RowComplete();
		}
		SimpleTable.Draw(Font, DrawCanvas, TextLocation, DetailColor, BackgroundColor);
		TextLocation.Y += fAdvanceHeight;
	}
#endif //WITH_NIAGARA_GPU_PROFILER
}

void FNiagaraDebugHud::DrawGlobalBudgetInfo(class FNiagaraWorldManager* WorldManager, class FCanvas* DrawCanvas, FVector2f& TextLocation)
{
	using namespace NiagaraDebugLocal;

	if ( !Settings.bShowGlobalBudgetInfo )
	{
		return;
	}

	static const float GuessWidth = 400.0f;

	UFont* Font = GetFont(Settings.OverviewFont);
	const float fAdvanceHeight = Font->GetMaxCharHeight() + 1.0f;

	const FLinearColor HeadingColor = Settings.OverviewHeadingColor;
	const FLinearColor DetailColor = Settings.OverviewDetailColor;
	const FLinearColor DetailHighlightColor = Settings.OverviewDetailHighlightColor;
	const FLinearColor BackgroundColor = Settings.DefaultBackgroundColor;

	if (FFXBudget::Enabled())
	{
		const FFXTimeData Time = FFXBudget::GetTime();
		const FFXTimeData Budget = FFXBudget::GetBudget();
		const FFXTimeData Usage = FFXBudget::GetUsage();
		const FFXTimeData AdjustedUsage = FFXBudget::GetAdjustedUsage();

		const int32 NumLines = 5;//Header, time, budget, usage and adjusted usage.
		DrawCanvas->DrawTile(TextLocation.X - 1.0f, TextLocation.Y - 1.0f, GuessWidth + 1.0f, 2.0f + (float(NumLines) * fAdvanceHeight), 0.0f, 0.0f, 0.0f, 0.0f, BackgroundColor);

		DrawCanvas->DrawShadowedString(TextLocation.X, TextLocation.Y, TEXT("Global Budget Info"), Font, HeadingColor);

		TextLocation.Y += fAdvanceHeight;
		FString TimeLabels[] =
		{
			TEXT("Time: "),
			TEXT("Budget: "),
			TEXT("Usage: "),
			TEXT("Adjusted: ")
		};

		int32 LabelSizes[] =
		{
			Font->GetStringSize(*TimeLabels[0]),
			Font->GetStringSize(*TimeLabels[1]),
			Font->GetStringSize(*TimeLabels[2]),
			Font->GetStringSize(*TimeLabels[3])
		};
		int32 MaxLabelSize = FMath::Max(FMath::Max(LabelSizes[0], LabelSizes[1]), FMath::Max(LabelSizes[2], LabelSizes[3]));

		auto DrawTimeData = [&](FString Heading, FFXTimeData Time, float HighlightThreshold = FLT_MAX)
		{
			//Draw Time
			int32 XOff = 0;
			int32 AlignSize = 50;
			FString Text[] = {
				Heading,
				FString::Printf(TEXT("GT = %2.3f"), Time.GT),
				FString::Printf(TEXT("CNC = %2.3f"), Time.GTConcurrent),
				FString::Printf(TEXT("RT = %2.3f"), Time.RT)
			};
			int32 Sizes[] =
			{
				MaxLabelSize,
				Font->GetStringSize(*Text[1]),
				Font->GetStringSize(*Text[2]),
				Font->GetStringSize(*Text[3])
			};
			DrawCanvas->DrawShadowedString(TextLocation.X + XOff, TextLocation.Y, *Text[0], Font, HeadingColor);
			XOff = AlignArbitrary(XOff + Sizes[0], AlignSize);
			DrawCanvas->DrawShadowedString(TextLocation.X + XOff, TextLocation.Y, *Text[1], Font, Time.GT > HighlightThreshold ? DetailHighlightColor : DetailColor);
			XOff = AlignArbitrary(XOff + Sizes[1], AlignSize);
			DrawCanvas->DrawShadowedString(TextLocation.X + XOff, TextLocation.Y, *Text[2], Font, Time.GTConcurrent > HighlightThreshold ? DetailHighlightColor : DetailColor);
			XOff = AlignArbitrary(XOff + Sizes[2], AlignSize);
			DrawCanvas->DrawShadowedString(TextLocation.X + XOff, TextLocation.Y, *Text[3], Font, Time.RT > HighlightThreshold ? DetailHighlightColor : DetailColor);
			XOff = AlignArbitrary(XOff + Sizes[3], AlignSize);

			TextLocation.Y += fAdvanceHeight;
		};
		DrawTimeData(TimeLabels[0], Time);
		DrawTimeData(TimeLabels[1], Budget);
		DrawTimeData(TimeLabels[2], Usage, 1.0f);
		DrawTimeData(TimeLabels[3], AdjustedUsage, 1.0f);
	}
	else
	{
		int32 NumLines = 2;
		DrawCanvas->DrawTile(TextLocation.X - 1.0f, TextLocation.Y - 1.0f, GuessWidth + 1.0f, 2.0f + (float(NumLines) * fAdvanceHeight), 0.0f, 0.0f, 0.0f, 0.0f, BackgroundColor);
		DrawCanvas->DrawShadowedString(TextLocation.X, TextLocation.Y, TEXT("Global Budget Info"), Font, HeadingColor);
		TextLocation.Y += fAdvanceHeight;
		DrawCanvas->DrawShadowedString(TextLocation.X, TextLocation.Y, TEXT("Global budget tracking is disabled. Can be enabled by setting cvars fx.Budget.Enabled and fx.Budget.EnabledInEditor to true."), Font, DetailHighlightColor);
		TextLocation.Y += fAdvanceHeight;
	}
}

void FNiagaraDebugHud::DrawValidation(class FNiagaraWorldManager* WorldManager, class FCanvas* DrawCanvas, FVector2f& TextLocation)
{
	using namespace NiagaraDebugLocal;

	if (!Settings.bValidationEnabled || (!Settings.bValidateSystemSimulationDataBuffers && !Settings.bValidateParticleDataBuffers))
	{
		return;
	}

	for (TWeakObjectPtr<UFXSystemComponent> WeakComponent : InWorldComponents)
	{
		UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(WeakComponent.Get());
		if (NiagaraComponent == nullptr)
		{
			continue;
		}

		UNiagaraSystem* NiagaraSystem = NiagaraComponent->GetAsset();
		FNiagaraSystemInstanceControllerPtr SystemInstanceController = NiagaraComponent->GetSystemInstanceController();
		FNiagaraSystemInstance* SystemInstance = SystemInstanceController.IsValid() ? SystemInstanceController->GetSystemInstance_Unsafe() : nullptr;
		if ((NiagaraSystem == nullptr) || (SystemInstance == nullptr))
		{
			continue;
		}

		auto SystemSimulation = SystemInstance->GetSystemSimulation();
		if (!SystemSimulation.IsValid() || !SystemSimulation->IsValid())
		{
			continue;
		}

		// Ensure systems are complete
		SystemSimulation->WaitForInstancesTickComplete();

		// Look for validation errors
		if (Settings.bValidateSystemSimulationDataBuffers)
		{
			FNiagaraDataSetDebugAccessor::ValidateDataBuffer(
				SystemSimulation->MainDataSet.GetCompiledData(),
				SystemSimulation->MainDataSet.GetCurrentData(),
				SystemInstance->GetSystemInstanceIndex(),
				[&](const FNiagaraVariableBase& Variable, int32 ComponentIndex)
				{
					auto& ValidationError = GetValidationErrorInfo(NiagaraComponent);
					ValidationError.SystemVariablesWithErrors.AddUnique(Variable.GetName());
				}
			);
		}

		if (Settings.bValidateParticleDataBuffers)
		{
			TConstArrayView<FNiagaraEmitterInstanceRef> EmitterHandles = SystemInstance->GetEmitters();
			for ( int32 iEmitter=0; iEmitter < EmitterHandles.Num(); ++iEmitter)
			{
				FNiagaraEmitterInstance* EmitterInstance = &EmitterHandles[iEmitter].Get();
				if ( !EmitterInstance )
				{
					continue;
				}

				const FNiagaraDataSet* ParticleDataSet = GetParticleDataSet(SystemInstance, EmitterInstance, iEmitter);
				if (ParticleDataSet == nullptr)
				{
					continue;
				}

				const FNiagaraDataBuffer* DataBuffer = ParticleDataSet->GetCurrentData();
				if (!DataBuffer || !DataBuffer->GetNumInstances())
				{
					continue;
				}

				FNiagaraDataSetDebugAccessor::ValidateDataBuffer(
					ParticleDataSet->GetCompiledData(),
					DataBuffer,
					[&](const FNiagaraVariableBase& Variable, int32 InstanceIndex, int32 ComponentIndex)
					{
						auto& ValidationError = GetValidationErrorInfo(NiagaraComponent);
						const FName EmitterName(*EmitterInstance->GetEmitter()->GetUniqueEmitterName());
						ValidationError.ParticleVariablesWithErrors.FindOrAdd(EmitterName).AddUnique(Variable.GetName());
					}
				);
			}
		}
	}

	if ( ValidationErrors.Num() > 0 )
	{
		const double TrimSeconds = FPlatformTime::Seconds() - 5.0;

		TStringBuilder<1024> ErrorString;
		for (auto ValidationIt=ValidationErrors.CreateIterator(); ValidationIt; ++ValidationIt)
		{
			const FValidationErrorInfo& ErrorInfo = ValidationIt.Value();
			if (ErrorInfo.LastWarningTime < TrimSeconds)
			{
				ValidationIt.RemoveCurrent();
				continue;
			}

			ErrorString.Append(ErrorInfo.DisplayName);
			ErrorString.Append(TEXT("\n"));

			// System Variables
			{
				const int32 NumVariables = FMath::Min(ErrorInfo.SystemVariablesWithErrors.Num(), 3);
				if (NumVariables > 0)
				{
					for (int32 iVariable=0; iVariable < NumVariables; ++iVariable)
					{
						if (iVariable == 0)
						{
							ErrorString.Append(TEXT("- SystemVars - "));
						}
						else
						{
							ErrorString.Append(TEXT(", "));
						}
						ErrorString.Append(ErrorInfo.SystemVariablesWithErrors[iVariable].ToString());
					}
					if (NumVariables != ErrorInfo.SystemVariablesWithErrors.Num())
					{
						ErrorString.Append(TEXT(", ..."));
					}
					ErrorString.Append(TEXT("\n"));
				}
			}

			// Particle Variables
			for (auto EmitterIt=ErrorInfo.ParticleVariablesWithErrors.CreateConstIterator(); EmitterIt; ++EmitterIt)
			{
				const TArray<FName>& EmitterVariables = EmitterIt.Value();
				const int32 NumVariables = Settings.ValidationAttributeDisplayTruncate > 0 ? FMath::Min(EmitterVariables.Num(), Settings.ValidationAttributeDisplayTruncate) : EmitterVariables.Num();
				if (NumVariables > 0)
				{
					for (int32 iVariable = 0; iVariable < NumVariables; ++iVariable)
					{
						if (iVariable == 0)
						{
							ErrorString.Appendf(TEXT("- Particles(%s) - "), *EmitterIt.Key().ToString());
						}
						else
						{
							ErrorString.Append(TEXT(", "));
						}
						ErrorString.Append(EmitterVariables[iVariable].ToString());
					}
					if (NumVariables != EmitterVariables.Num())
					{
						ErrorString.Append(TEXT(", ..."));
					}
					ErrorString.Append(TEXT("\n"));

				}
			}
		}

		if (ErrorString.Len() > 0)
		{
			UFont* Font = GetFont(Settings.OverviewFont);
			const float fAdvanceHeight = Font->GetMaxCharHeight() + 1.0f;

			const FVector2f ErrorStringSize = GetStringSize(Font, ErrorString.ToString());

			TextLocation.Y += fAdvanceHeight;
			DrawCanvas->DrawTile(TextLocation.X - 1.0f, TextLocation.Y - 1.0f, ErrorStringSize.X + 2.0f, ErrorStringSize.Y + fAdvanceHeight + 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, Settings.DefaultBackgroundColor);

			DrawCanvas->DrawShadowedString(TextLocation.X, TextLocation.Y, TEXT("Found Errors:"), Font, Settings.MessageErrorTextColor);
			TextLocation.Y += fAdvanceHeight;

			DrawCanvas->DrawShadowedString(TextLocation.X, TextLocation.Y, ErrorString.ToString(), Font, Settings.MessageErrorTextColor);

			if (Settings.bValidationEnabled && Settings.bValidationLogErrors)
			{
				UE_LOGF(LogNiagara, Warning, "Validation Errors - %ls", ErrorString.ToString());
			}
		}
	}
}

void FNiagaraDebugHud::DrawComponents(FNiagaraWorldManager* WorldManager, UCanvas* Canvas)
{
	using namespace NiagaraDebugLocal;

	FCanvas* DrawCanvas = Canvas->Canvas;
	UWorld* World = WorldManager->GetWorld();
	UFont* SystemFont = GetFont(Settings.SystemTextOptions.Font);
	UFont* ParticleFont = GetFont(Settings.ParticleTextOptions.Font);

	// Draw in world components
	UEnum* ExecutionStateEnum = StaticEnum<ENiagaraExecutionState>();
	UEnum* PoolingMethodEnum = StaticEnum<ENCPoolMethod>();
	UEnum* SystemInstanceState = StaticEnum<ENiagaraSystemInstanceState>();
	for (TWeakObjectPtr<UFXSystemComponent> WeakComponent : InWorldComponents)
	{
		UFXSystemComponent* FXComponent = WeakComponent.Get();
		if (FXComponent == nullptr)
		{
			continue;
		}

		const FVector ComponentLocation = FXComponent->GetComponentLocation();
		const FRotator ComponentRotation = FXComponent->GetComponentRotation();
		const bool bIsActive = FXComponent->IsActive();

		// Show system bounds (only active components)
		if (Settings.bDrawBoundsEnabled)
		{
			const FBox BoundsToDraw = FXComponent->CalcBounds(FXComponent->GetComponentTransform()).GetBox();
			if (BoundsToDraw.IsValid)
			{
				FLinearColor BoundsColor = bIsActive ? FLinearColor::Red : FLinearColor::Black;
				BoundsColor.A = Settings.DrawBoundsAlpha;

				if (Settings.bDrawBoundsWireframe)
				{
					DrawWireframeBox(World, BoundsToDraw.GetCenter(), BoundsToDraw.GetExtent(), BoundsColor);
				}
				else
				{
					DrawBox(World, BoundsToDraw.GetCenter(), BoundsToDraw.GetExtent(), BoundsColor);
				}
			}
		}

		UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(FXComponent);
		if (NiagaraComponent == nullptr)
		{
			continue;
		}

		UNiagaraSystem* NiagaraSystem = NiagaraComponent->GetAsset();
		FNiagaraSystemInstanceControllerPtr SystemInstanceController = NiagaraComponent->GetSystemInstanceController();
		FNiagaraSystemInstance* SystemInstance = SystemInstanceController.IsValid() ? SystemInstanceController->GetSystemInstance_Unsafe() : nullptr;
		if (NiagaraSystem == nullptr || SystemInstance == nullptr)
		{
			continue;
		}

		// Get system simulation
		auto SystemSimulation = SystemInstance->GetSystemSimulation();
		const bool bSystemSimulationValid = SystemSimulation.IsValid() && SystemSimulation->IsValid();
		if (bSystemSimulationValid)
		{
			SystemSimulation->WaitForInstancesTickComplete();
		}

		const FLinearColor TextColor = ValidationErrors.Contains(NiagaraComponent) ? Settings.InWorldErrorTextColor : Settings.InWorldTextColor;

		// Show particle data in world
		if (!Settings.bShowParticlesVariablesWithSystem && bSystemSimulationValid && Canvas->SceneView)
		{
			const FCachedVariables& CachedVariables = GetCachedVariables(NiagaraSystem);
			for (int32 iEmitter = 0; iEmitter < CachedVariables.ParticleVariables.Num(); ++iEmitter)
			{
				if ((CachedVariables.ParticleVariables[iEmitter].Num() == 0) || !CachedVariables.ParticlePositionAccessors[iEmitter].IsValid())
				{
					continue;
				}

				FNiagaraEmitterInstance* EmitterInstance = &SystemInstance->GetEmitters()[iEmitter].Get();
				const FNiagaraDataSet* ParticleDataSet = GetParticleDataSet(SystemInstance, EmitterInstance, iEmitter);
				if (ParticleDataSet == nullptr)
				{
					continue;
				}

				const FNiagaraDataBuffer* DataBuffer = ParticleDataSet->GetCurrentData();
				if (!DataBuffer || !DataBuffer->GetNumInstances())
				{
					continue;
				}

				// No positions accessor, we can't show this in world
				auto PositionReader = CachedVariables.ParticlePositionAccessors[iEmitter].GetReader(*ParticleDataSet);
				if (!PositionReader.IsValid())
				{
					continue;
				}

				FSceneView* SceneView = Canvas->SceneView;

				const FTransform& SystemTransform = SystemInstance->GetWorldTransform();
				const bool bParticlesLocalSpace = EmitterInstance->IsLocalSpace();
				//const float ClipRadius = Settings.bUseParticleDisplayRadius ? 1.0f : 0.0f;
				const float ParticleDisplayCenterRadiusSq = Settings.bUseParticleDisplayCenterRadius ? (Settings.ParticleDisplayCenterRadius * Settings.ParticleDisplayCenterRadius) : 0.0f;
				const float ParticleDisplayClipNearPlane = Settings.bUseParticleDisplayClip ? float(FMath::Max(Settings.ParticleDisplayClip.X, 0.0)) : 0.0f;
				const float ParticleDisplayClipFarPlane = Settings.bUseParticleDisplayClip ? float(FMath::Max(Settings.ParticleDisplayClip.Y, ParticleDisplayClipNearPlane)) : FLT_MAX;
				const uint32 MaxDisplayParticles = Settings.bUseMaxParticlesToDisplay ? Settings.MaxParticlesToDisplay : 0xFFFFFFFF;
				uint32 NumDisplayedParticles = 0;

				FNiagaraLWCConverter LwcConverter = SystemInstance->GetLWCConverter(false);
				for (uint32 iInstance = 0; iInstance < DataBuffer->GetNumInstances(); ++iInstance)
				{
					const FVector ParticalLocalPosition = LwcConverter.ConvertSimulationPositionToWorld(PositionReader.Get(iInstance));
					const FVector ParticleWorldPosition = bParticlesLocalSpace ? SystemTransform.TransformPosition(ParticalLocalPosition) : ParticalLocalPosition;

					const FPlane ClipPosition = SceneView->Project(ParticleWorldPosition);
					if ((ParticleDisplayCenterRadiusSq > 0) && (((ClipPosition.X * ClipPosition.X) + (ClipPosition.Y * ClipPosition.Y)) >= ParticleDisplayCenterRadiusSq))
					{
						continue;
					}

					if ((ClipPosition.W <= ParticleDisplayClipNearPlane) || (ClipPosition.W > ParticleDisplayClipFarPlane))
					{
						continue;
					}

					const FVector ParticleScreenLocation = Canvas->Project(ParticleWorldPosition);

					TStringBuilder<1024> StringBuilder;
					if ( Settings.bShowParticleIndex )
					{
						StringBuilder.Appendf(TEXT("Particle(%u)"), iInstance);
					}
					for (const auto& ParticleVariable : CachedVariables.ParticleVariables[iEmitter])
					{
						StringBuilder.AppendChar(Settings.bShowParticleVariablesVertical ? TEXT('\n') : TEXT(' '));
						StringBuilder.Append(ParticleVariable.GetName().ToString());
						StringBuilder.AppendChar(TEXT('('));
						ParticleVariable.StringAppend(StringBuilder, DataBuffer, iInstance);
						StringBuilder.AppendChar(TEXT(')'));
					}

					const TCHAR* FinalString = StringBuilder.ToString();
					const TPair<FVector2f, FVector2f> SizeAndLocation = GetTextLocation(ParticleFont, FinalString, Settings.ParticleTextOptions, FVector2f(float(ParticleScreenLocation.X), float(ParticleScreenLocation.Y)));
					DrawCanvas->DrawTile(SizeAndLocation.Value.X - 1.0f, SizeAndLocation.Value.Y - 1.0f, SizeAndLocation.Key.X + 2.0f, SizeAndLocation.Key.Y + 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, Settings.DefaultBackgroundColor);
					DrawCanvas->DrawShadowedString(SizeAndLocation.Value.X, SizeAndLocation.Value.Y, FinalString, ParticleFont, TextColor);

					++NumDisplayedParticles;
					if (NumDisplayedParticles >= MaxDisplayParticles)
					{
						break;
					}
				}
			}
		}

		const FVector ScreenLocation = Canvas->Project(ComponentLocation);
		if (!FMath::IsNearlyZero(ScreenLocation.Z))
		{
			// Show locator
			DrawSystemLocation(Canvas, bIsActive, ScreenLocation, ComponentRotation);

			// Show system text
			if ( Settings.SystemDebugVerbosity != ENiagaraDebugHudVerbosity::None)
			{
				AActor* OwnerActor = NiagaraComponent->GetOwner();

				TStringBuilder<1024> StringBuilder;

				StringBuilder.Appendf(TEXT("System - %s\n"), *GetNameSafe(NiagaraSystem));

				// Build component name
				StringBuilder.Append(TEXT("Component - "));
				if ( OwnerActor )
				{
					StringBuilder.Append(*OwnerActor->GetActorNameOrLabel());
					StringBuilder.Append(TEXT("/"));
				}
				StringBuilder.Append(*GetNameSafe(NiagaraComponent));
				if (OwnerActor && Settings.SystemDebugVerbosity == ENiagaraDebugHudVerbosity::Verbose)
				{
					StringBuilder.Appendf(TEXT(" OwnerClass(%s)"), *OwnerActor->GetClass()->GetName());
				}
				StringBuilder.Append(TEXT("\n"));

				if (Settings.SystemDebugVerbosity == ENiagaraDebugHudVerbosity::Verbose)
				{
					StringBuilder.Appendf(
						TEXT("System InstanceState(%s) VMActualState(%s) VMRequestedState(%s)\n"),
						*SystemInstanceState->GetNameStringByIndex((int32)SystemInstance->SystemInstanceState),
						*ExecutionStateEnum->GetNameStringByIndex((int32)SystemInstance->GetActualExecutionState()),
						*ExecutionStateEnum->GetNameStringByIndex((int32)SystemInstance->GetRequestedExecutionState())
					);
					if (NiagaraComponent->PoolingMethod != ENCPoolMethod::None)
					{
						StringBuilder.Appendf(TEXT("Pooled - %s\n"), *PoolingMethodEnum->GetNameStringByIndex((int32)NiagaraComponent->PoolingMethod));
					}

					if (SystemInstanceController->IsSolo())
					{
						StringBuilder.Append(TEXT("Solo(true)"));
						if ( NiagaraComponent->GetForceSolo() )
						{
							StringBuilder.Append(TEXT(" ForceSolo"));
						}
						
						switch (NiagaraComponent->GetAgeUpdateMode())
						{
							case ENiagaraAgeUpdateMode::DesiredAge:
							case ENiagaraAgeUpdateMode::DesiredAgeNoSeek:
								StringBuilder.Appendf(TEXT(" DesiredAge(%6.3f)"), NiagaraComponent->GetDesiredAge());
								break;
						}

						StringBuilder.Append(TEXT("\n"));
					}

					if ( NiagaraComponent->bHiddenInGame || !NiagaraComponent->IsVisible() || (OwnerActor && OwnerActor->IsHidden()) )
					{
						StringBuilder.Appendf(TEXT("HiddenInGame(%d) IsVisible(%d) GetVisibleFlag(%d)"), NiagaraComponent->bHiddenInGame, NiagaraComponent->IsVisible(), NiagaraComponent->GetVisibleFlag());
						if ( OwnerActor )
						{
							StringBuilder.Appendf(TEXT(" OwnerActorHidden(%d)"), OwnerActor->IsHidden());
						}
						StringBuilder.Append(TEXT("\n"));
					}

					if (NiagaraComponent->bAutoManageAttachment)
					{
						StringBuilder.Appendf(TEXT("Auto Attachment - Parent(%s) Socket(%s)\n"), *GetNameSafe(NiagaraComponent->AutoAttachParent.Get()), *NiagaraComponent->AutoAttachSocketName.ToString());
					}

					if (UNiagaraSimCache* SimCache = NiagaraComponent->GetSimCache())
					{
						StringBuilder.Appendf(TEXT("SimCache(%s)\n"), *GetFullNameSafe(SimCache));
					}

					if (bIsActive)
					{
						if (NiagaraComponent->IsRegisteredWithScalabilityManager())
						{
							StringBuilder.Appendf(TEXT("Scalability - %s\n"), *GetNameSafe(NiagaraSystem->GetEffectType()));
							if (SystemInstance->SignificanceIndex != INDEX_NONE)
							{
								StringBuilder.Appendf(TEXT("SignificanceIndex - %d\n"), SystemInstance->SignificanceIndex);
							}
						}
						static UEnum* TickingGroupEnum = StaticEnum<ETickingGroup>();
						if (TickingGroupEnum)
						{
							StringBuilder.Appendf(TEXT("TickGroup - %s\n"), *TickingGroupEnum->GetNameStringByValue(SystemInstance->CalculateTickGroup()));
						}
						BuildGpuHudInformation(StringBuilder, NiagaraComponent, SystemInstance, World->GetFeatureLevel());

						StringBuilder.Appendf(TEXT("LastRenderTime - %5.2f"), NiagaraComponent->GetLastRenderTime());
						if (NiagaraComponent->bRenderCustomDepth)
						{
							StringBuilder.Append(TEXT(" bRenderCustomDepth"));
						}
						switch (NiagaraComponent->GetOcclusionQueryMode())
						{
							case ENiagaraOcclusionQueryMode::AlwaysEnabled:		StringBuilder.Append(TEXT(" OcclusionQuery(AlwaysEnabled)")); break;
							case ENiagaraOcclusionQueryMode::AlwaysDisabled:	StringBuilder.Append(TEXT(" OcclusionQuery(AlwaysDisable)")); break;
						}
						StringBuilder.Append(TEXT("\n"));

						if ( NiagaraSystem->bSupportLargeWorldCoordinates )
						{
							const FVector3f LWCTile = SystemInstance->GetLWCTile();
							if (!LWCTile.Equals(FVector3f::ZeroVector) )
							{
								StringBuilder.Appendf(TEXT("LWCTile - %.1f, %.1f, %.1f"), LWCTile.X, LWCTile.Y, LWCTile.Z);
							}
						}
					}

					const int64 TotalBytes = SystemInstanceController->GetTotalBytesUsed();
					StringBuilder.Appendf(TEXT("Memory - %6.2fMB\n"), float(double(TotalBytes) / (1024.0*1024.0)));
				}

				if (bIsActive)
				{
					int32 ActiveEmitters = 0;
					int32 TotalEmitters = 0;
					int32 ActiveParticles = 0;
					for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
					{
						if (EmitterInstance->IsDisabled())
						{
							continue;
						}

						++TotalEmitters;
						if (EmitterInstance->GetExecutionState() == ENiagaraExecutionState::Active)
						{
							++ActiveEmitters;
						}
						ActiveParticles += EmitterInstance->GetNumParticles();

						if (Settings.SystemEmitterVerbosity == ENiagaraDebugHudVerbosity::Verbose)
						{
							const FString EmitterName = EmitterInstance->GetEmitterHandle().GetUniqueInstanceName();
							if ( EmitterInstance->GetSimTarget() == ENiagaraSimTarget::GPUComputeSim )
							{
								StringBuilder.Appendf(TEXT("Emitter(GPU) %s - State %s - Particles %d\n"), *EmitterName, *ExecutionStateEnum->GetNameStringByIndex((int32)EmitterInstance->GetExecutionState()), EmitterInstance->GetNumParticles());
							}
							else
							{
								StringBuilder.Appendf(TEXT("Emitter %s - State %s - Particles %d\n"), *EmitterName, *ExecutionStateEnum->GetNameStringByIndex((int32)EmitterInstance->GetExecutionState()), EmitterInstance->GetNumParticles());
							}
						}
					}

					if (Settings.SystemEmitterVerbosity == ENiagaraDebugHudVerbosity::Basic)
					{
						StringBuilder.Appendf(TEXT("Emitters - %d / %d\n"), ActiveEmitters, TotalEmitters);
						StringBuilder.Appendf(TEXT("Particles - %d\n"), ActiveParticles);
					}

					// Any variables to display?
					if (bSystemSimulationValid)
					{
						const FCachedVariables& CachedVariables = GetCachedVariables(NiagaraSystem);

						// Engine Variables
						for (int32 EngineVariable : CachedVariables.EngineVariables)
						{
							StringBuilder.Appendf(TEXT("%s = %s\n"), *GEngineVariables[EngineVariable].Key, *GEngineVariables[EngineVariable].Value(SystemInstance));
						}

						// System variables
						if (CachedVariables.SystemVariables.Num() > 0)
						{
							FNiagaraDataBuffer* DataBuffer = SystemSimulation->MainDataSet.GetCurrentData();
							const uint32 InstanceIndex = SystemInstance->GetSystemInstanceIndex();

							for (const auto& SystemVariable : CachedVariables.SystemVariables)
							{
								StringBuilder.Append(SystemVariable.GetName().ToString());
								StringBuilder.Append(TEXT(" = "));
								SystemVariable.StringAppend(StringBuilder, DataBuffer, InstanceIndex);
								StringBuilder.Append(TEXT("\n"));
							}
						}

						// Parameter Store Variables (i.e. user variables)
						if (CachedVariables.ParameterStoreVariables.Num() > 0)
						{
							for (const FParameterStoreVariable& ParameterStoreVariable : CachedVariables.ParameterStoreVariables)
							{
								FNiagaraParameterStore* ParameterStore = nullptr;
								FNiagaraEmitterInstanceImpl* StatefulEmitter = SystemInstance->Emitters.IsValidIndex(ParameterStoreVariable.EmitterIndex) ? SystemInstance->Emitters[ParameterStoreVariable.EmitterIndex]->AsStateful() : nullptr;
								switch (ParameterStoreVariable.StoreLocation)
								{
									case EParameterStoreLocation::UserOverride:			ParameterStore = SystemInstance->GetOverrideParameters(); break;
									case EParameterStoreLocation::SystemSpawn:			ParameterStore = &SystemSimulation->GetSpawnExecutionContext()->Parameters; break;
									case EParameterStoreLocation::SystemUpdate:			ParameterStore = &SystemSimulation->GetUpdateExecutionContext()->Parameters; break;
									case EParameterStoreLocation::EmitterSpawn:			ParameterStore = StatefulEmitter ? &StatefulEmitter->GetSpawnExecutionContext().Parameters : nullptr; break;
									case EParameterStoreLocation::EmitterUpdate:		ParameterStore = StatefulEmitter ? &StatefulEmitter->GetUpdateExecutionContext().Parameters : nullptr; break;
									case EParameterStoreLocation::EmitterRenderBindings:ParameterStore = StatefulEmitter ? &StatefulEmitter->GetRendererBoundVariables() : nullptr; break;
								}
								if (ParameterStore == nullptr)
								{
									continue;
								}

								if (ParameterStoreVariable.Variable.IsDataInterface())
								{
									UNiagaraDataInterface* DataInterface = ParameterStore->GetDataInterface(ParameterStoreVariable.Variable);
									FNDIDrawDebugHudContext DebugHudContext(Settings.DataInterfaceVerbosity == ENiagaraDebugHudVerbosity::Verbose, World, Canvas, SystemInstance);
									if ( DataInterface != nullptr && Settings.DataInterfaceVerbosity != ENiagaraDebugHudVerbosity::None )
									{
										DataInterface->DrawDebugHud(DebugHudContext);
									}

									if (DebugHudContext.GetOutputString().IsEmpty())
									{
										StringBuilder.Appendf(TEXT("%s(%s %s)"), *ParameterStoreVariable.Variable.GetName().ToString(), *GetNameSafe(ParameterStoreVariable.Variable.GetType().GetClass()) , *GetNameSafe(DataInterface));
									}
									else
									{
										StringBuilder.Appendf(TEXT("%s(%s)"), *ParameterStoreVariable.Variable.GetName().ToString(), *DebugHudContext.GetOutputString());
									}
								}
								else if (ParameterStoreVariable.Variable.IsUObject())
								{
									StringBuilder.Appendf(TEXT("%s(%s %s)"), *ParameterStoreVariable.Variable.GetName().ToString(), *GetNameSafe(ParameterStoreVariable.Variable.GetType().GetClass()) , *GetNameSafe(ParameterStore->GetUObject(ParameterStoreVariable.Variable)));
								}
								else
								{
									FNiagaraVariable VariableWithValue(ParameterStoreVariable.Variable);
									if ( const uint8* ParameterData = ParameterStore->GetParameterData(VariableWithValue) )
									{
										VariableWithValue.SetData(ParameterData);
									}
									StringBuilder.Append(*VariableWithValue.ToString());
								}
								StringBuilder.AppendChar('\n');
							}
						}

						// Static variables
					#if WITH_EDITORONLY_DATA
						for (const FNiagaraVariable& StaticVar : CachedVariables.StaticVariables)
						{
							StringBuilder.Append(*StaticVar.ToString());
							StringBuilder.Append(TEXT(" (Static)\n"));
						}
					#endif

						// Append particle data if we don't show them in world
						if (Settings.bShowParticlesVariablesWithSystem)
						{
							for (int32 iEmitter = 0; iEmitter < CachedVariables.ParticleVariables.Num(); ++iEmitter)
							{
								if (CachedVariables.ParticleVariables[iEmitter].Num() == 0)
								{
									continue;
								}

								FNiagaraEmitterInstance* EmitterInstance = &SystemInstance->GetEmitters()[iEmitter].Get();
								const FNiagaraDataSet* ParticleDataSet = GetParticleDataSet(SystemInstance, EmitterInstance, iEmitter);
								if (ParticleDataSet == nullptr)
								{
									continue;
								}

								const FNiagaraDataBuffer* DataBuffer = ParticleDataSet->GetCurrentData();
								if (!DataBuffer || !DataBuffer->GetNumInstances())
								{
									continue;
								}

								StringBuilder.Appendf(TEXT("Emitter (%s)\n"), *EmitterInstance->GetEmitter()->GetUniqueEmitterName());
								const uint32 NumParticles = Settings.bUseMaxParticlesToDisplay ? FMath::Min((uint32)Settings.MaxParticlesToDisplay, DataBuffer->GetNumInstances()) : DataBuffer->GetNumInstances();
								for (uint32 iInstance = 0; iInstance < NumParticles; ++iInstance)
								{
									StringBuilder.Appendf(TEXT(" Particle(%u) "), iInstance);
									for (const auto& ParticleVariable : CachedVariables.ParticleVariables[iEmitter])
									{
										StringBuilder.Append(ParticleVariable.GetName().ToString());
										StringBuilder.Append(TEXT("("));
										ParticleVariable.StringAppend(StringBuilder, DataBuffer, iInstance);
										StringBuilder.Append(TEXT(") "));
									}
									StringBuilder.Append(TEXT("\n"));
								}

								if (NumParticles < DataBuffer->GetNumInstances())
								{
									StringBuilder.Appendf(TEXT(" ...Truncated"));
								}
							}
						}
					}
				}
				// Instance is considered inactive
				else
				{
					if (Settings.SystemDebugVerbosity >= ENiagaraDebugHudVerbosity::Basic)
					{
						if (NiagaraComponent->IsUsingCullProxy())
						{
							StringBuilder.Appendf(TEXT("Using Cull Proxy"));
						}
						else
						{
							StringBuilder.Appendf(TEXT("Deactivated by Scalability - %s "), *GetNameSafe(NiagaraSystem->GetEffectType()));
							if (Settings.SystemDebugVerbosity >= ENiagaraDebugHudVerbosity::Verbose)
							{
								FNiagaraScalabilityState ScalabilityState;
								if (WorldManager->GetScalabilityState(NiagaraComponent, ScalabilityState))
								{
									StringBuilder.Appendf(TEXT("- Significance(%.2f)"), ScalabilityState.Significance);
#if DEBUG_SCALABILITY_STATE
									if (ScalabilityState.bCulledByDistance)
									{
										StringBuilder.Append(TEXT(" DistanceCulled"));
									}
									if (ScalabilityState.bCulledByInstanceCount)
									{
										StringBuilder.Append(TEXT(" InstanceCulled"));
									}
									if (ScalabilityState.bCulledByVisibility)
									{
										StringBuilder.Append(TEXT(" VisibilityCulled"));
									}
									if (ScalabilityState.bCulledByGlobalBudget)
									{
										StringBuilder.Append(TEXT(" GlobalBudgetCulled"));
									}
#endif
									StringBuilder.Append(TEXT("\n"));
								}
								else
								{
									StringBuilder.Appendf(TEXT("- Scalability State Unknown\n"));
								}
							}
						}
					}
				}

				const TCHAR* FinalString = StringBuilder.ToString();
				const TPair<FVector2f, FVector2f> SizeAndLocation = GetTextLocation(SystemFont, FinalString, Settings.SystemTextOptions, FVector2f(float(ScreenLocation.X), float(ScreenLocation.Y)));
				DrawCanvas->DrawTile(SizeAndLocation.Value.X - 1.0f, SizeAndLocation.Value.Y - 1.0f, SizeAndLocation.Key.X + 2.0f, SizeAndLocation.Key.Y + 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, Settings.DefaultBackgroundColor);
				DrawCanvas->DrawShadowedString(SizeAndLocation.Value.X, SizeAndLocation.Value.Y, FinalString, SystemFont, TextColor);
			}
		}
	}
}

void FNiagaraDebugHud::DrawMessages(class FNiagaraWorldManager* WorldManager, class FCanvas* DrawCanvas, FVector2f& TextLocation)
{
	using namespace NiagaraDebugLocal;

	static const float MinWidth = 500.0f;

	UFont* Font = GetFont(Settings.OverviewFont);
	const float fAdvanceHeight = Font->GetMaxCharHeight() + 1.0f;

	FVector2f BackgroundSize(MinWidth, 0.0f);
	TArray<FName, TInlineAllocator<8>> ToRemove;
	for (TPair<FName, FNiagaraDebugMessage>& Pair : Messages)
	{
		FName& Key = Pair.Key;
		FNiagaraDebugMessage& Message = Pair.Value;

		Message.Lifetime -= DeltaSeconds;
		if (Message.Lifetime > 0.0f)
		{
			BackgroundSize = FVector2f::Max(BackgroundSize, GetStringSize(Font, *Message.Message));
		}
		else
		{
			ToRemove.Add(Key);
		}
	}

	//Not sure why but the get size always underestimates slightly.
	BackgroundSize.X += 20.0f;

	for (FName DeadMessage : ToRemove)
	{
		Messages.Remove(DeadMessage);
	}

	if (Messages.Num())
	{
		TextLocation.Y += fAdvanceHeight;
		DrawCanvas->DrawTile(TextLocation.X - 1.0f, TextLocation.Y - 1.0f, BackgroundSize.X + 2.0f, BackgroundSize.Y + 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, Settings.DefaultBackgroundColor);

		for (TPair<FName, FNiagaraDebugMessage>& Pair : Messages)
		{
			FName& Key = Pair.Key;
			FNiagaraDebugMessage& Message = Pair.Value;

			FLinearColor MessageColor;
			if (Message.Type == ENiagaraDebugMessageType::Info) MessageColor = Settings.MessageInfoTextColor;
			else if (Message.Type == ENiagaraDebugMessageType::Warning) MessageColor = Settings.MessageWarningTextColor;
			else if (Message.Type == ENiagaraDebugMessageType::Error) MessageColor = Settings.MessageErrorTextColor;

			DrawCanvas->DrawShadowedString(TextLocation.X, TextLocation.Y, *Message.Message, Font, MessageColor);//TODO: Sort by type / lifetime?
			TextLocation.Y += fAdvanceHeight;
		}
	}
}

//////////////////////////////////////////////////////////////////////////

#if WITH_PARTICLE_PERF_STATS

void FNiagaraDebugHudStatHistory::AddFrame_GT(double Time)
{
	GTFrames.SetNumZeroed(NiagaraDebugLocal::Settings.PerfHistoryFrames);
	CurrFrame = FMath::WrapExclusive(CurrFrame + 1, 0, GTFrames.Num());
	GTFrames[CurrFrame] = Time;
}

void FNiagaraDebugHudStatHistory::AddFrame_RT(double Time)
{
	FScopeLock Lock(&NiagaraDebugLocal::RTFramesGuard);
	RTFrames.SetNumZeroed(NiagaraDebugLocal::Settings.PerfHistoryFrames);
	CurrFrameRT = FMath::WrapExclusive(CurrFrameRT + 1, 0, RTFrames.Num());
	RTFrames[CurrFrameRT] = Time;
}

void FNiagaraDebugHudStatHistory::AddFrame_GPU(double Time)
{
	FScopeLock Lock(&NiagaraDebugLocal::RTFramesGuard);
	GPUFrames.SetNumZeroed(NiagaraDebugLocal::Settings.PerfHistoryFrames);
	CurrFrameGPU = FMath::WrapExclusive(CurrFrameGPU + 1, 0, GPUFrames.Num());
	GPUFrames[CurrFrameGPU] = Time;
}

void FNiagaraDebugHudStatHistory::GetHistoryFrames_GT(TArray<double>& OutHistoryGT)
{
	if (GTFrames.Num() == 0)
	{
		return;
	}
	OutHistoryGT.Reserve(GTFrames.Num());
	int32 WriteFrame = CurrFrame;
	do
	{
		OutHistoryGT.Add(GTFrames[WriteFrame]);
		WriteFrame = FMath::WrapExclusive(WriteFrame + 1, 0, GTFrames.Num());
	} while (WriteFrame != CurrFrame);
}

void FNiagaraDebugHudStatHistory::GetHistoryFrames_RT(TArray<double>& OutHistoryRT)
{
	FScopeLock Lock(&NiagaraDebugLocal::RTFramesGuard);

	if (RTFrames.Num() == 0)
	{
		return;
	}

	OutHistoryRT.Reserve(RTFrames.Num());
	int32 WriteFrame = CurrFrameRT;
	do 
	{
		OutHistoryRT.Add(RTFrames[WriteFrame]);
		WriteFrame = FMath::WrapExclusive(WriteFrame + 1, 0, RTFrames.Num());
	} while (WriteFrame != CurrFrameRT);
}

void FNiagaraDebugHudStatHistory::GetHistoryFrames_GPU(TArray<double>& OutHistoryGPU)
{
	FScopeLock Lock(&NiagaraDebugLocal::RTFramesGuard);

	if (GPUFrames.Num() == 0)
	{
		return;
	}

	OutHistoryGPU.Reserve(GPUFrames.Num());
	int32 WriteFrame = CurrFrameGPU;
	do
	{
		OutHistoryGPU.Add(GPUFrames[WriteFrame]);
		WriteFrame = FMath::WrapExclusive(WriteFrame + 1, 0, GPUFrames.Num());
	} while (WriteFrame != CurrFrameGPU);
}

TSharedPtr<FNiagaraDebugHUDPerfStats> FNiagaraDebugHUDStatsListener::GetSystemStats(UFXSystemAsset* System)
{
	FScopeLock Lock(&SystemStatsGuard);//Locking on every get isn't great but it's debug hud code so meh.
	if (TSharedPtr<FNiagaraDebugHUDPerfStats>* Stats = SystemStats.Find(System))
	{
		return *Stats;
	}
	return nullptr;
}

FNiagaraDebugHUDPerfStats& FNiagaraDebugHUDStatsListener::GetGlobalStats()
{
	return GlobalStats;
}

void FNiagaraDebugHUDStatsListener::OnAddSystem(const TWeakObjectPtr<const UFXSystemAsset>& NewSystem)
{
	FParticlePerfStatsListener_GatherAll::OnAddSystem(NewSystem);

	FScopeLock Lock(&AccumulatedStatsGuard);
	FScopeLock SysInfoLock(&SystemStatsGuard);

	TSharedPtr<FNiagaraDebugHUDPerfStats>& SysStats = SystemStats.Add(NewSystem);
	SysStats = MakeShared<FNiagaraDebugHUDPerfStats>();
}

void FNiagaraDebugHUDStatsListener::OnRemoveSystem(const TWeakObjectPtr<const UFXSystemAsset>& System)
{
	FParticlePerfStatsListener_GatherAll::OnRemoveSystem(System);

	FScopeLock Lock(&AccumulatedStatsGuard);
	FScopeLock SysInfoLock(&SystemStatsGuard);

	SystemStats.FindAndRemoveChecked(System);
}

bool FNiagaraDebugHUDStatsListener::Tick()
{
	using namespace NiagaraDebugLocal;
	FParticlePerfStatsListener_GatherAll::Tick();

	FScopeLock Lock(&AccumulatedStatsGuard);
	FScopeLock SysInfoLock(&SystemStatsGuard);

	bool bPushStats = false;
	if (NumFrames++ > Settings.PerfReportFrames)
	{
		NumFrames = 0;
		bPushStats = true;
	}

	double GlobalAvg = 0.0;
	double GlobalMax = 0.0;
	for (auto& WorldStatsPair : AccumulatedWorldStats)
	{
		if (const UWorld* World = Cast<UWorld>(WorldStatsPair.Key.Get()))
		{
			FAccumulatedParticlePerfStats* Stats = WorldStatsPair.Value.Get();
			check(Stats);

			if (Settings.PerfSampleMode == ENiagaraDebugHUDPerfSampleMode::FrameTotal)
			{
				GlobalAvg += Stats->GetGameThreadStats().GetPerFrameAvg();
				GlobalMax += Stats->GetGameThreadStats().GetPerFrameMax();
			}
			else
			{
				GlobalAvg += Stats->GetGameThreadStats().GetPerInstanceAvg();
				GlobalMax += Stats->GetGameThreadStats().GetPerInstanceMax();
			}

			Stats->ResetGT();
		}
	}

	GlobalStats.History.AddFrame_GT(GlobalAvg);

	if (bPushStats)
	{
		GlobalStats.Avg.Time_GT = GlobalAvg;
		GlobalStats.Max.Time_GT = GlobalMax;
	}

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	for (auto& HUDStatsPair : SystemStats)
	{
		TWeakObjectPtr<const UFXSystemAsset> WeakSystem = HUDStatsPair.Key;
		TSharedPtr<FNiagaraDebugHUDPerfStats>& HUDStats = HUDStatsPair.Value;

		const UFXSystemAsset* System = WeakSystem.Get();
		if(System == nullptr)
		{
			continue;
		}

		if (FAccumulatedParticlePerfStats* Stats = GetStats(System))
		{
			double SysAvg = 0.0;
			double SysAvgGTOnly = 0.0;
			double SysMax = 0.0;
			if (Settings.PerfSampleMode == ENiagaraDebugHUDPerfSampleMode::FrameTotal)
			{
				SysAvg = Stats->GetGameThreadStats().GetPerFrameAvg();
				SysAvgGTOnly = Stats->GetGameThreadStats().GetPerFrameAvg_GTOnly();
				SysMax = Stats->GetGameThreadStats().GetPerFrameMax();
			}
			else
			{
				SysAvg = Stats->GetGameThreadStats().GetPerInstanceAvg();
				SysAvgGTOnly = Stats->GetGameThreadStats().GetPerInstanceAvg_GTOnly();
				SysMax = Stats->GetGameThreadStats().GetPerInstanceMax();
			}

			HUDStats->History.AddFrame_GT(SysAvg);

			if (bPushStats)
			{
				HUDStats->Avg.Time_GT = SysAvg;
				HUDStats->Avg.Time_GTOnly = SysAvgGTOnly;
				HUDStats->Max.Time_GT = SysMax;
			}

			Stats->ResetGT();
		}
	}
#endif

	return true;
}

void FNiagaraDebugHUDStatsListener::TickRT()
{
	using namespace NiagaraDebugLocal;
	FParticlePerfStatsListener_GatherAll::TickRT();


	bool bPushStats = false;
	if (NumFramesRT++ > Settings.PerfReportFrames)
	{
		NumFramesRT = 0;
		bPushStats = true;
	}

	FScopeLock Lock(&AccumulatedStatsGuard);
	FScopeLock SysInfoLock(&SystemStatsGuard);

	double RTGlobalAvg = 0.0;
	double RTGlobalMax = 0.0;
	double GPUGlobalAvg = 0.0;
	double GPUGlobalMax = 0.0;
	for (auto& WorldStatsPair : AccumulatedWorldStats)
	{
		if (const UWorld* World = Cast<UWorld>(WorldStatsPair.Key.Get()))
		{
			FAccumulatedParticlePerfStats* Stats = WorldStatsPair.Value.Get();
			check(Stats);

			if (Settings.PerfSampleMode == ENiagaraDebugHUDPerfSampleMode::FrameTotal)
			{
				RTGlobalAvg += Stats->GetRenderThreadStats().GetPerFrameAvg();
				RTGlobalMax += Stats->GetRenderThreadStats().GetPerFrameMax();
				GPUGlobalAvg += Stats->GetGPUStats().GetPerFrameAvgMicroseconds();
				GPUGlobalMax += Stats->GetGPUStats().GetPerFrameMaxMicroseconds();
			}
			else
			{
				RTGlobalAvg += Stats->GetRenderThreadStats().GetPerInstanceAvg();
				RTGlobalMax += Stats->GetRenderThreadStats().GetPerInstanceMax();
				GPUGlobalAvg += Stats->GetGPUStats().GetPerInstanceAvgMicroseconds();
				GPUGlobalMax += Stats->GetGPUStats().GetPerInstanceMaxMicroseconds();
			}

			Stats->ResetRT();
		}
	}
	GlobalStats.History.AddFrame_RT(RTGlobalAvg);
	GlobalStats.History.AddFrame_GPU(GPUGlobalAvg);

	if (bPushStats)
	{
		GlobalStats.Avg.Time_RT = RTGlobalAvg;
		GlobalStats.Max.Time_RT = RTGlobalMax;
		GlobalStats.Avg.Time_GPU = GPUGlobalAvg;
		GlobalStats.Max.Time_GPU = GPUGlobalMax;
	}

#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
	for (auto& HUDStatsPair : SystemStats)
	{
		TWeakObjectPtr<const UFXSystemAsset> WeakSystem = HUDStatsPair.Key;
		TSharedPtr<FNiagaraDebugHUDPerfStats>& HUDStats = HUDStatsPair.Value;

		const UFXSystemAsset* System = WeakSystem.Get();
		if (System == nullptr)
		{
			continue;
		}

		if (FAccumulatedParticlePerfStats* Stats = GetStats(System))
		{
			float RTSysAvg;
			float RTSysMax;
			float GPUSysAvg;
			float GPUSysMax;
			if (Settings.PerfSampleMode == ENiagaraDebugHUDPerfSampleMode::FrameTotal)
			{
				RTSysAvg = Stats->GetRenderThreadStats().GetPerFrameAvg();
				RTSysMax = Stats->GetRenderThreadStats().GetPerFrameAvg();
				GPUSysAvg = float(Stats->GetGPUStats().GetPerFrameAvgMicroseconds());
				GPUSysMax = float(Stats->GetGPUStats().GetPerFrameAvgMicroseconds());
			}
			else
			{
				RTSysAvg = Stats->GetRenderThreadStats().GetPerInstanceAvg();
				RTSysMax = Stats->GetRenderThreadStats().GetPerInstanceMax();
				GPUSysAvg = float(Stats->GetGPUStats().GetPerInstanceAvgMicroseconds());
				GPUSysMax = float(Stats->GetGPUStats().GetPerInstanceMaxMicroseconds());
			}

			HUDStats->History.AddFrame_RT(RTSysAvg);
			HUDStats->History.AddFrame_GPU(GPUSysAvg);

			if (bPushStats)
			{
				HUDStats->Avg.Time_RT = RTSysAvg;
				HUDStats->Max.Time_RT = RTSysMax;
				HUDStats->Avg.Time_GPU = GPUSysAvg;
				HUDStats->Max.Time_GPU = GPUSysMax;
			}

			Stats->ResetRT();
		}
	}
#endif
}
#endif//WITH_PARTICLE_PERF_STATS

#endif //WITH_NIAGARA_DEBUGGER
