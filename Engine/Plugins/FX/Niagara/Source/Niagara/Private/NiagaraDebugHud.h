// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
Class used help debugging Niagara simulations
==============================================================================*/
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraDefines.h"
#include "NiagaraTypes.h"
#include "RHICommandList.h"
#include "NiagaraDebuggerCommon.h"
#include "NiagaraGPUProfilerInterface.h"
#include "Particles/ParticlePerfStatsManager.h"

#if WITH_NIAGARA_DEBUGGER

#if WITH_PARTICLE_PERF_STATS

class FNiagaraDataSet;
class FNiagaraDebugHud;

struct FNiagaraDebugHudFrameStat
{
	double Time_GT = 0.0;
	double Time_GTOnly = 0.0;
	double Time_RT = 0.0;
	double Time_GPU = 0.0;
};

/** Ring buffer history of stats. */
struct FNiagaraDebugHudStatHistory
{
	TArray<double> GTFrames;
	TArray<double> RTFrames;
	TArray<double> GPUFrames;

	int32 CurrFrame = 0;
	int32 CurrFrameRT = 0;
	int32 CurrFrameGPU = 0;

	void AddFrame_GT(double Time);
	void AddFrame_RT(double Time);
	void AddFrame_GPU(double Time);

	void GetHistoryFrames_GT(TArray<double>& OutHistoryGT);
	void GetHistoryFrames_RT(TArray<double>& OutHistoryRT);
	void GetHistoryFrames_GPU(TArray<double>& OutHistoryGPU);
};

struct FNiagaraDebugHUDPerfStats
{
	FNiagaraDebugHudFrameStat Avg;
	FNiagaraDebugHudFrameStat Max;
	FNiagaraDebugHudStatHistory History;
};

/**
Listener that accumulates short runs of stats and reports then to the debug hud.
*/
class FNiagaraDebugHUDStatsListener : public FParticlePerfStatsListener_GatherAll
{
public:
	FNiagaraDebugHud& Owner;
	FNiagaraDebugHUDStatsListener(FNiagaraDebugHud& InOwner) 
	: FParticlePerfStatsListener_GatherAll(true, true, false)//TODO: Also gather component stats and display that info in world.
	, Owner(InOwner)
	{
	}

	int32 NumFrames = 0;
	int32 NumFramesRT = 0;

	FNiagaraDebugHUDPerfStats GlobalStats;

	TMap<TWeakObjectPtr<const UFXSystemAsset>, TSharedPtr<FNiagaraDebugHUDPerfStats>> SystemStats;
	FCriticalSection SystemStatsGuard;

	virtual bool Tick()override;
	virtual void TickRT()override;
	TSharedPtr<FNiagaraDebugHUDPerfStats> GetSystemStats(UFXSystemAsset* System);
	FNiagaraDebugHUDPerfStats& GetGlobalStats();

	virtual void OnAddSystem(const TWeakObjectPtr<const UFXSystemAsset>& NewSystem)override;
	virtual void OnRemoveSystem(const TWeakObjectPtr<const UFXSystemAsset>& System)override;
};

#endif

class FNiagaraDebugHud
{
	typedef TSharedPtr<class FNiagaraDataSetReadback, ESPMode::ThreadSafe> FGpuDataSetPtr;

	struct FSystemDebugInfo
	{
		FString		SystemName;
		FString		SystemPrettyName;

		#if WITH_PARTICLE_PERF_STATS
		TSharedPtr<FNiagaraDebugHUDPerfStats> PerfStats = nullptr;
		#endif
		FLinearColor UniqueColor = FLinearColor::Red;
		int32		FramesSinceVisible = 0;

		bool		bShowInWorld = false;
		bool		bPassesSystemFilter = true;
		int32		TotalRegistered = 0;
		int32		TotalActive = 0;
		int32		TotalScalability = 0;
		int32		TotalEmitters = 0;
		int32		TotalParticles = 0;
		int64		TotalBytes = 0;

		int32		TotalCulled = 0;
		int32		TotalCulledByDistance = 0;
		int32		TotalCulledByVisibility = 0;
		int32		TotalCulledByInstanceCount = 0;
		int32		TotalCulledByBudget = 0;
		
		int32		TotalPlayerSystems = 0;

		FNiagaraSystemScalabilityKillCounts ScalabilityKillCounts;

		void Reset()
		{
			bShowInWorld = false;
			bPassesSystemFilter = true;
			TotalRegistered = 0;
			TotalActive = 0;
			TotalScalability = 0;
			TotalEmitters = 0;
			TotalParticles = 0;
			TotalBytes = 0;

			TotalCulled = 0;
			TotalCulledByDistance = 0;
			TotalCulledByVisibility = 0;
			TotalCulledByInstanceCount = 0;
			TotalCulledByBudget = 0;

			TotalPlayerSystems = 0;
		}
	};

	struct FGpuEmitterCache
	{
		uint64					LastAccessedCycles;
		TArray<FGpuDataSetPtr>	CurrentEmitterData;
		TArray<FGpuDataSetPtr>	PendingEmitterData;
	};

	struct FValidationErrorInfo
	{
		double						LastWarningTime = 0.0;
		FString						DisplayName;
		TArray<FName>				SystemVariablesWithErrors;
		TMap<FName, TArray<FName>>	ParticleVariablesWithErrors;
	};

	static constexpr int32 SmoothedNumFrames = 32;
	template<typename TCounterType>
	struct FSmoothedCounter
	{
		FSmoothedCounter() {}

		uint64			LastFrameSeen = 0;
		bool			SmoothedSetOnce = false;
		TCounterType	SmoothedFrameMax = TCounterType();
		float			SmoothedTotalAvg = 0.0f;

		uint32			CurrentCount = 0;
		TCounterType	CurrentFrameMax = TCounterType();
		TCounterType	CurrentTotalSum = TCounterType();
		TCounterType	CurrentFrameSum = TCounterType();

		void Reset()
		{
			LastFrameSeen = 0;
			SmoothedSetOnce = false;
			SmoothedFrameMax = TCounterType();
			SmoothedTotalAvg = 0.0f;

			CurrentCount = 0;
			CurrentFrameMax = TCounterType();
			CurrentTotalSum = TCounterType();
			CurrentFrameSum = TCounterType();
		}

		bool ShouldPrune(uint64 FrameCounter)
		{
			return (FrameCounter - LastFrameSeen) >= SmoothedNumFrames;
		}

		void Accumulate(uint64 InLastFrameSeen, TCounterType Value)
		{
			if ( LastFrameSeen != InLastFrameSeen )
			{
				LastFrameSeen = InLastFrameSeen;

				CurrentFrameMax = FMath::Max(CurrentFrameSum, CurrentFrameMax);
				CurrentFrameSum = TCounterType();

				const bool bResetCurrent = ++CurrentCount >= SmoothedNumFrames;
				if (!SmoothedSetOnce || bResetCurrent)
				{
					SmoothedSetOnce |= bResetCurrent;
					SmoothedFrameMax = CurrentFrameMax;
					SmoothedTotalAvg = float(CurrentTotalSum) / float(CurrentCount);
				}

				if (bResetCurrent)
				{
					CurrentCount = 0;
					CurrentFrameMax = TCounterType();
					CurrentTotalSum = TCounterType();
				}
			}
			CurrentTotalSum += Value;
			CurrentFrameSum += Value;
		}

		template<typename TReturnType = TCounterType>
		TReturnType GetAverage() const { return TReturnType(SmoothedTotalAvg); }
		TCounterType GetMax() const { return SmoothedFrameMax; }
	};

public:
	FNiagaraDebugHud(class UWorld* World);
	~FNiagaraDebugHud();

	void GatherSystemInfo();

	void UpdateSettings(const FNiagaraDebugHUDSettingsData& NewSettings);

	void AddMessage(FName Key, const FNiagaraDebugMessage& Message);
	void RemoveMessage(FName Key);

private:
	const FNiagaraDataSet* GetParticleDataSet(class FNiagaraSystemInstance* SystemInstance, class FNiagaraEmitterInstance* EmitterInstance, int32 iEmitter);
	FValidationErrorInfo& GetValidationErrorInfo(class UNiagaraComponent* NiagaraComponent);

	static void DebugDrawCallback(class UCanvas* Canvas, class APlayerController* PC);

	void Draw(class FNiagaraWorldManager* WorldManager, class UCanvas* Canvas, class APlayerController* PC);
	void DrawOverview(class FNiagaraWorldManager* WorldManager, class FCanvas* DrawCanvas);
	void DrawGpuComputeOverriew(class FNiagaraWorldManager* WorldManager, class FCanvas* DrawCanvas, FVector2f& TextLocation);
	void DrawGlobalBudgetInfo(class FNiagaraWorldManager* WorldManager, class FCanvas* DrawCanvas, FVector2f& TextLocation);
	void DrawValidation(class FNiagaraWorldManager* WorldManager, class FCanvas* DrawCanvas, FVector2f& TextLocation);
	void DrawComponents(class FNiagaraWorldManager* WorldManager, class UCanvas* Canvas);
	void DrawMessages(class FNiagaraWorldManager* WorldManager, class FCanvas* DrawCanvas, FVector2f& TextLocation);
	void DrawDebugGeomerty(class FNiagaraWorldManager* WorldManager, class UCanvas* DrawCanvas);

#if WITH_EDITORONLY_DATA
	void OnSystemCompiled(UNiagaraSystem* NiagaraSystem);
#endif

private:
	TWeakObjectPtr<class UWorld>	WeakWorld;

#if WITH_EDITORONLY_DATA
	TMap<TWeakObjectPtr<UNiagaraSystem>, FDelegateHandle> SystemCompiledDelegates;
#endif

	int32 GlobalTotalRegistered = 0;
	int32 GlobalTotalActive = 0;
	int32 GlobalTotalScalability = 0;
	int32 GlobalTotalEmitters = 0;
	int32 GlobalTotalParticles = 0;
	int64 GlobalTotalBytes = 0;

	int32 GlobalTotalCulled = 0;
	int32 GlobalTotalCulledByDistance = 0;
	int32 GlobalTotalCulledByVisibility = 0;
	int32 GlobalTotalCulledByInstanceCount = 0;
	int32 GlobalTotalCulledByBudget = 0;

	int32 GlobalTotalPlayerSystems = 0;

	FNiagaraSystemScalabilityKillCounts GlobalScalabiltiyKillCounts;

	FString LongestSystemPrettyName;
	TMap<FName, FSystemDebugInfo>	PerSystemDebugInfo;

	TArray<TWeakObjectPtr<class UFXSystemComponent>>	InWorldComponents;

	TMap<TWeakObjectPtr<class UNiagaraComponent>, FValidationErrorInfo> ValidationErrors;

	TMap<FNiagaraSystemInstanceID, FGpuEmitterCache> GpuEmitterData;

#if WITH_PARTICLE_PERF_STATS
	TSharedPtr<FNiagaraDebugHUDStatsListener, ESPMode::ThreadSafe> StatsListener;
#endif

#if WITH_NIAGARA_GPU_PROFILER
	FNiagaraGpuProfilerListener	GpuProfilerListener;
	FNiagaraGpuFrameResultsPtr	GpuResults;
	uint64						GpuResultsGameFrameCounter = 0;

	FSmoothedCounter<int32>		GpuTotalDispatches;
	FSmoothedCounter<uint64>	GpuTotalMicroseconds;

	struct FGpuUsagePerStage
	{
		FSmoothedCounter<uint32> InstanceCount;
		FSmoothedCounter<uint64> Microseconds;
	};

	struct FGpuUsagePerEmitter
	{
		FSmoothedCounter<uint32> InstanceCount;
		FSmoothedCounter<uint64> Microseconds;
		TMap<FName, FGpuUsagePerStage> Stages;
	};

	struct FGpuUsagePerSystem
	{
		bool bShowDetailed = false;
		FSmoothedCounter<uint32> InstanceCount;
		FSmoothedCounter<uint64> Microseconds;
		TMap<TWeakObjectPtr<UNiagaraEmitter>, FGpuUsagePerEmitter> Emitters;
	};

	struct FGpuUsagePerEvent
	{
		FSmoothedCounter<uint32> InstanceCount;
		FSmoothedCounter<uint64> Microseconds;
	};

	TMap<TWeakObjectPtr<UNiagaraSystem>, FGpuUsagePerSystem> GpuUsagePerSystem;
	TMap<FName, FGpuUsagePerEvent> GpuUsagePerEvent;
#endif

	double LastDrawTime = 0.0f;
	float DeltaSeconds = 0.0f;

	/** Generic messages that the debugger or other parts of Niagara can post to the HUD. */
	TMap<FName, FNiagaraDebugMessage> Messages;

	//Additional debug geometry helpers
	struct FDebugLine2D
	{
		FVector2f Start;
		FVector2f End;
		FLinearColor Color;
		float Thickness;
		float Lifetime;
	};
	TArray<FDebugLine2D> Lines2D;
	struct FDebugCircle2D
	{
		FVector2f Pos;
		float Rad;
		int32 Segments;
		FLinearColor Color;
		float Thickness;
		float Lifetime;
	};
	TArray<FDebugCircle2D> Circles2D;
	struct FDebugBox2D
	{
		FVector2f Pos;
		FVector2f Extents;
		FLinearColor Color;
		float Thickness;
		float Lifetime;
	};
	TArray<FDebugBox2D> Boxes2D;

	double AutoGraphTimeRange = 100.0;
	double AutoGraphTimeRangeChangeMax = 100.0;
	double AutoGraphTimeRangeChangedTime = 0.0;

public:
	/** Add a 2D line to the debug renering. Positions are in normalized screen space. (0,0) in top left, (1,1) bottom right.*/
	void AddLine2D(FVector2f Start, FVector2f End, FLinearColor Color, float Thickness, float Lifetime);
	void AddCircle2D(FVector2f Pos, float Rad, int32 Segments, FLinearColor Color, float Thickness, float Lifetime);
	void AddBox2D(FVector2f Pos, FVector2f Extents, FLinearColor Color, float Thickness, float Lifetime);
	//Additional debug geometry helpers END
};

#endif //WITH_NIAGARA_DEBUGGER
