// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDebuggerClient.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "IMessageContext.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraWorldManager.h"
#include "NiagaraComponent.h"
#include "NiagaraModule.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraDebugHud.h"
#include "Containers/Ticker.h"
#include "NiagaraSimCache.h"
#include "NiagaraSystemGpuComputeProxy.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

#if WITH_NIAGARA_DEBUGGER

DEFINE_LOG_CATEGORY(LogNiagaraDebuggerClient);

FNiagaraDebuggerClient::FNiagaraDebuggerClient()
{
	MessageEndpoint = FMessageEndpoint::Builder("FNiagaraDebuggerClient")
		.Handling<FNiagaraDebuggerRequestConnection>(this, &FNiagaraDebuggerClient::HandleConnectionRequestMessage)
		.Handling<FNiagaraDebuggerConnectionClosed>(this, &FNiagaraDebuggerClient::HandleConnectionClosedMessage)
		.Handling<FNiagaraDebuggerExecuteConsoleCommand>(this, &FNiagaraDebuggerClient::HandleExecConsoleCommandMessage)
 		.Handling<FNiagaraDebugHUDSettingsData>(this, &FNiagaraDebuggerClient::HandleDebugHUDSettingsMessage)
		.Handling<FNiagaraRequestSimpleClientInfoMessage>(this, &FNiagaraDebuggerClient::HandleRequestSimpleClientInfoMessage)
		.Handling<FNiagaraOutlinerCaptureSettings>(this, &FNiagaraDebuggerClient::HandleOutlinerSettingsMessage)
		.Handling<FNiagaraSystemSimCacheCaptureRequest>(this, &FNiagaraDebuggerClient::HandleSimCacheCaptureRequestMessage);

 	if (MessageEndpoint.IsValid())
 	{
		MessageEndpoint->Subscribe<FNiagaraDebuggerRequestConnection>();
		MessageEndpoint->Subscribe<FNiagaraDebuggerConnectionClosed>();
 	}

	SessionId = FApp::GetSessionId();
	InstanceId = FApp::GetInstanceId();
	InstanceName = FApp::GetInstanceName();
	UE_LOGF(LogNiagaraDebuggerClient, Log, "Niagara Debugger Client Initialized | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *InstanceName);

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FNiagaraDebuggerClient::Tick));
}

FNiagaraDebuggerClient::~FNiagaraDebuggerClient()
{
	FTSTicker::RemoveTicker(TickerHandle);

	CloseConnection();
}

bool FNiagaraDebuggerClient::Tick(float DeltaSeconds)
{
	//Keep ticking until we destroy the debugger.


	//Process any pending sim cache captures we have.
	for (auto It = SimCacheCaptures.CreateIterator(); It ; ++It)
	{
		FNiagaraSimCacheCaptureInfo& SimCacheCapture = *It;
		if (SimCacheCapture.Process())
		{
			It.RemoveCurrent();
		}
	}
	return true;
}

void FNiagaraDebuggerClient::GetSimpleClientInfo(FNiagaraSimpleClientInfo& OutInfo)
{
	OutInfo.Systems.Reset();
	OutInfo.Emitters.Reset();
	for (TObjectIterator<UNiagaraSystem> It; It; ++It)
	{
		UNiagaraSystem* System = *It;
		if (System)
		{
			OutInfo.Systems.Add(System->GetName());
			for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
			{
				if (Handle.GetInstance().Emitter)
				{
					OutInfo.Emitters.AddUnique(Handle.GetUniqueInstanceName());
				}
			}
		}
	}

	OutInfo.Actors.Reset();
	OutInfo.Components.Reset();
	TSet<AActor*> Actors;
	for (TObjectIterator<UNiagaraComponent> It; It; ++It)
	{
		UNiagaraComponent* Comp = *It;
		if (IsValid(Comp) && !Comp->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
		{
			OutInfo.Components.AddUnique(Comp->GetName());
			Actors.Add(Comp->GetOwner());
		}
	}

	for (AActor* Actor : Actors)
	{
		if (IsValid(Actor) && !Actor->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
		{
			OutInfo.Actors.AddUnique(Actor->GetActorNameOrLabel());
		}
	}
}

void FNiagaraDebuggerClient::UpdateClientInfo()
{
	if (MessageEndpoint.IsValid() && Connection.IsValid())
	{
		FNiagaraSimpleClientInfo* NewInfo = FMessageEndpoint::MakeMessage<FNiagaraSimpleClientInfo>();
		GetSimpleClientInfo(*NewInfo);
		MessageEndpoint->Send(NewInfo, EMessageFlags::Reliable, nullptr, TArrayBuilder<FMessageAddress>().Add(Connection), FTimespan::Zero(), FDateTime::MaxValue());
	}
}

void FNiagaraDebuggerClient::HandleConnectionRequestMessage(const FNiagaraDebuggerRequestConnection& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (MessageEndpoint.IsValid() && Message.SessionId == SessionId && Message.InstanceId == InstanceId)
	{
		if (Connection.IsValid())
		{
			UE_LOGF(LogNiagaraDebuggerClient, Warning, "Connection request received but we already have a connected debugger. Current connection being dropped and new connection accepted. | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *InstanceName);
			CloseConnection();
		}
		else
		{
			UE_LOGF(LogNiagaraDebuggerClient, Log, "Connection request received and accepted. | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *InstanceName);
		}
		
		//Accept the connection and inform the debugger we have done so with an accepted message.
		Connection = Context->GetSender();
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FNiagaraDebuggerAcceptConnection>(SessionId, InstanceId), EMessageFlags::Reliable, nullptr, TArrayBuilder<FMessageAddress>().Add(Connection), FTimespan::Zero(), FDateTime::MaxValue());
		
		//Also send an initial update of the client info.
		UpdateClientInfo();		
	}
}

void FNiagaraDebuggerClient::HandleConnectionClosedMessage(const FNiagaraDebuggerConnectionClosed& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (MessageEndpoint.IsValid() && Message.SessionId == SessionId && Message.InstanceId == InstanceId)
	{
		if (Connection == Context->GetSender())
		{
			UE_LOGF(LogNiagaraDebuggerClient, Log, "Connection Closed. | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *InstanceName);
			OnConnectionClosed();
		}
		else
		{
			UE_LOGF(LogNiagaraDebuggerClient, Warning, "Received connection closed message for unconnected debugger. | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *InstanceName);
		}
	}
}



void FNiagaraDebuggerClient::ExecConsoleCommand(const FNiagaraDebuggerExecuteConsoleCommand& Message)
{
	ExecuteConsoleCommand(*Message.Command, Message.bRequiresWorld);
}

void FNiagaraDebuggerClient::UpdateDebugHUDSettings(const FNiagaraDebugHUDSettingsData& Message)
{
	//Pass along the new settings.
	auto ApplySettingsToWorldMan =
		[&Message](FNiagaraWorldManager& WorldMan)
	{
		WorldMan.GetNiagaraDebugHud()->UpdateSettings(Message);

		//TODO: Move these to just take direct from the debug hud per worldman?
		//Possibly move the debug hud itself to the debugger client rather than having one per world manager and they all share global state.
		const ENiagaraDebugPlaybackMode PlaybackMode = Message.IsEnabled() ? Message.PlaybackMode : ENiagaraDebugPlaybackMode::Play;
		const float PlaybackRate = Message.IsEnabled() && Message.bPlaybackRateEnabled ? Message.PlaybackRate : 1.0f;
		WorldMan.SetDebugPlaybackMode(PlaybackMode);
		WorldMan.SetDebugPlaybackRate(PlaybackRate);
	};

	FNiagaraWorldManager::ForAllWorldManagers(ApplySettingsToWorldMan);

	//TODO: Move usage to come direct from settings struct instead of this CVar.
	const float GlobalLoopTime = Message.IsEnabled() && Message.bLoopTimeEnabled && Message.PlaybackMode == ENiagaraDebugPlaybackMode::Loop ? Message.LoopTime : 0.0f;
	ExecuteConsoleCommand(*FString::Printf(TEXT("fx.Niagara.Debug.GlobalLoopTime %.3f"), GlobalLoopTime), true);
}

void FNiagaraDebuggerClient::UpdateOutlinerSettings(const FNiagaraOutlinerCaptureSettings& Message, FOnNiagaraDebuggerClientOutlinerCapture OnCapture)
{
	FNiagaraOutlinerCaptureSettings::StaticStruct()->CopyScriptStruct(&OutlinerSettings, &Message);
	if (ensure(OutlinerSettings.bTriggerCapture))
	{
		if (OutlinerCountdown == 0)
		{
#if WITH_PARTICLE_PERF_STATS
			if (Message.bGatherPerfData)
			{
				if (StatsListener)
				{
					FParticlePerfStatsManager::RemoveListener(StatsListener);
				}

				StatsListener = MakeShared<FNiagaraOutlinerPerfListener, ESPMode::ThreadSafe>();
				FParticlePerfStatsManager::AddListener(StatsListener);
			}
#endif
			OutlinerOnCapture = OnCapture;
			if (Message.CaptureDelayFrames <= 0)
			{
				UE_LOGF(LogNiagaraDebuggerClient, Log, "Received request to capture outliner data. Capturing now. | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *InstanceName);
				UpdateOutliner(0.001f);
			}
			else
			{
				OutlinerCountdown = Message.CaptureDelayFrames;
				UE_LOGF(LogNiagaraDebuggerClient, Log, "Received request to capture outliner data. Capturing in %u frames. | Session: %ls | Instance: %ls (%ls).", Message.CaptureDelayFrames, *SessionId.ToString(), *InstanceId.ToString(), *InstanceName);
				FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FNiagaraDebuggerClient::UpdateOutliner));
			}
		}
		else
		{
			UE_LOGF(LogNiagaraDebuggerClient, Log, "Received request to capture outliner data. Ignoring as we already have a pending outliner capture. | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *InstanceName);
		}
	}
	else
	{
		UE_LOGF(LogNiagaraDebuggerClient, Log, "Received request to capture outliner data but the capture bool is false. | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *InstanceName);
	}
}

void FNiagaraDebuggerClient::SimCacheCaptureRequest(const FNiagaraSystemSimCacheCaptureRequest& Message, FOnNiagaraDebuggerClientSimCacheCapture OnCapture)
{
	UNiagaraComponent* Comp = nullptr;

	for (TObjectIterator<UNiagaraComponent> It; It; ++It)
	{
		if (*It->GetPathName() == Message.ComponentName)
		{
			Comp = *It;
			break;
		}
	}

	if (Comp)
	{
		FNiagaraSimCacheCaptureInfo& NewCapture = SimCacheCaptures.AddDefaulted_GetRef();
		NewCapture.Request = Message;
		NewCapture.Component = Comp;
		NewCapture.OnCapture = OnCapture;
	}
}

void FNiagaraDebuggerClient::HandleExecConsoleCommandMessage(const FNiagaraDebuggerExecuteConsoleCommand& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (ensure(Context->GetSender() == Connection))
	{
		UE_LOGF(LogNiagaraDebuggerClient, Log, "Executing console command. %ls | Session: %ls | Instance: %ls (%ls).", *Message.Command, *SessionId.ToString(), *InstanceId.ToString(), *InstanceName);
		ExecConsoleCommand(Message);
	}
}

void FNiagaraDebuggerClient::HandleDebugHUDSettingsMessage(const FNiagaraDebugHUDSettingsData& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (ensure(Context->GetSender() == Connection))
	{
		UE_LOGF(LogNiagaraDebuggerClient, Log, "Received updated DebugHUD settings. | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *InstanceName);

		UpdateDebugHUDSettings(Message);
	}
}
void FNiagaraDebuggerClient::HandleRequestSimpleClientInfoMessage(const FNiagaraRequestSimpleClientInfoMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (ensure(Context->GetSender() == Connection))
	{
		UpdateClientInfo();
	}
}

void FNiagaraDebuggerClient::HandleOutlinerSettingsMessage(const FNiagaraOutlinerCaptureSettings& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (ensure(Context->GetSender() == Connection))
	{
		UpdateOutlinerSettings(Message, FOnNiagaraDebuggerClientOutlinerCapture::CreateLambda(
			[&](const FNiagaraOutlinerData& NewOutlinerData)
			{
				if (ensure(Connection.IsValid()))
				{
					FNiagaraDebuggerOutlinerUpdate* Message = FMessageEndpoint::MakeMessage<FNiagaraDebuggerOutlinerUpdate>();
					FNiagaraOutlinerData::StaticStruct()->CopyScriptStruct(&Message->OutlinerData, &NewOutlinerData);
					//Send the updated data to the debugger;
					MessageEndpoint->Send(Message, EMessageFlags::Reliable, nullptr, TArrayBuilder<FMessageAddress>().Add(Connection), FTimespan::Zero(), FDateTime::MaxValue());
				}
			})
		);
	}
}

void FNiagaraDebuggerClient::HandleSimCacheCaptureRequestMessage(const FNiagaraSystemSimCacheCaptureRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (ensure(Context->GetSender() == Connection))
	{
		SimCacheCaptureRequest(Message,
			FOnNiagaraDebuggerClientSimCacheCapture::CreateLambda([&](const FNiagaraSystemSimCacheCaptureRequest& Request, TObjectPtr<UNiagaraSimCache> CapturedSimCache)
			{
				if (MessageEndpoint.IsValid() && Connection.IsValid() && IsValid(CapturedSimCache))
				{
					//Sim cache is complete. Send it back to the connected debugger.
					FNiagaraSystemSimCacheCaptureReply* SimCacheCaptureReply = FMessageEndpoint::MakeMessage<FNiagaraSystemSimCacheCaptureReply>();
					SimCacheCaptureReply->ComponentName = Request.ComponentName;

					SimCacheCaptureReply->SimCacheData.Reset();
					FMemoryWriter ArWriter(SimCacheCaptureReply->SimCacheData);
					FObjectAndNameAsStringProxyArchive ProxyArWriter(ArWriter, false);
					CapturedSimCache->Serialize(ProxyArWriter);

					MessageEndpoint->Send(SimCacheCaptureReply, EMessageFlags::Reliable, nullptr, TArrayBuilder<FMessageAddress>().Add(Connection), FTimespan::Zero(), FDateTime::MaxValue());
				}
			})
		);
	}
}

void FNiagaraDebuggerClient::CloseConnection()
{
	if (MessageEndpoint.IsValid() && Connection.IsValid())
	{
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FNiagaraDebuggerConnectionClosed>(SessionId, InstanceId), EMessageFlags::Reliable, nullptr, TArrayBuilder<FMessageAddress>().Add(Connection), FTimespan::Zero(), FDateTime::MaxValue());
	}

	OnConnectionClosed();
}

void FNiagaraDebuggerClient::OnConnectionClosed()
{
	Connection.Invalidate();
}

void FNiagaraDebuggerClient::ExecuteConsoleCommand(const TCHAR* Cmd, bool bRequiresWorld)
{
	if (bRequiresWorld)
	{
		for (TObjectIterator<UWorld> WorldIt; WorldIt; ++WorldIt)
		{
			UWorld* World = *WorldIt;
			if ((World != nullptr) &&
				(World->PersistentLevel != nullptr) &&
				(World->PersistentLevel->OwningWorld == World) &&
				((World->GetNetMode() == ENetMode::NM_Client) || (World->GetNetMode() == ENetMode::NM_Standalone)))
			{
				GEngine->Exec(*WorldIt, Cmd);
			}
		}
	}
	else
	{
		GEngine->Exec(nullptr, Cmd);
	}
}

void FNiagaraDebuggerClient::CaptureOutlinerData(FNiagaraOutlinerData& OutlinerData)
{
	//Gather all high level state data to pass to the outliner in the debugger.
	//TODO: Move out to somewhere neater and add more info.
	for (TObjectIterator<UNiagaraComponent> CompIt; CompIt; ++CompIt)
	{
		UNiagaraComponent* Comp = *CompIt;
		if (!::IsValid(Comp))
		{
			continue;
		}

		UWorld* World = Comp->GetWorld();
		FNiagaraOutlinerWorldData& WorldData = OutlinerData.WorldData.FindOrAdd(World ? World->GetPathName() : TEXT("Null World"));
		if (World)
		{
			WorldData.bHasBegunPlay = World->HasBegunPlay();
			WorldData.WorldType = uint8(World->WorldType);
			WorldData.NetMode = uint8(World->GetNetMode());

#if WITH_PARTICLE_PERF_STATS
			if (StatsListener)
			{
				if (FAccumulatedParticlePerfStats* WorldStats = StatsListener->GetStats(World))
				{
					WorldData.AveragePerFrameTime.GameThread = WorldStats->GetGameThreadStats().GetPerFrameAvg();
					WorldData.AveragePerFrameTime.RenderThread = WorldStats->GetRenderThreadStats().GetPerFrameAvg();

					WorldData.MaxPerFrameTime.GameThread = WorldStats->GetGameThreadStats().GetPerFrameMax();
					WorldData.MaxPerFrameTime.RenderThread = WorldStats->GetRenderThreadStats().GetPerFrameMax();
				}
			}
#endif
		}

		UNiagaraSystem* System = Comp->GetAsset();
		FNiagaraOutlinerSystemData& Instances = WorldData.Systems.FindOrAdd(System ? System->GetPathName() : TEXT("Null System"));
		if (System)
		{
			Instances.AssetPath = FSoftObjectPath(System);

			//Add System specific data.
#if WITH_PARTICLE_PERF_STATS
			if (StatsListener)
			{
				if (FAccumulatedParticlePerfStats* SystemStats = StatsListener->GetStats(System))
				{
					Instances.AveragePerFrameTime.GameThread = SystemStats->GetGameThreadStats().GetPerFrameAvg();
					Instances.AveragePerFrameTime.RenderThread = SystemStats->GetRenderThreadStats().GetPerFrameAvg();

					Instances.MaxPerFrameTime.GameThread = SystemStats->GetGameThreadStats().GetPerFrameMax();
					Instances.MaxPerFrameTime.RenderThread = SystemStats->GetRenderThreadStats().GetPerFrameMax();


					Instances.AveragePerInstanceTime.GameThread = SystemStats->GetGameThreadStats().GetPerInstanceAvg();
					Instances.AveragePerInstanceTime.RenderThread = SystemStats->GetRenderThreadStats().GetPerInstanceAvg();

					Instances.MaxPerInstanceTime.GameThread = SystemStats->GetGameThreadStats().GetPerInstanceMax();
					Instances.MaxPerInstanceTime.RenderThread = SystemStats->GetRenderThreadStats().GetPerInstanceMax();
				}
			}
#endif
		}

		FNiagaraOutlinerSystemInstanceData& InstData = Instances.SystemInstances.AddDefaulted_GetRef();
		InstData.ComponentName = Comp->GetPathName();

		if (FNiagaraSystemInstanceControllerPtr InstController = Comp->GetSystemInstanceController())
		{
			InstData.ActualExecutionState = InstController->GetActualExecutionState();
			InstData.RequestedExecutionState = InstController->GetRequestedExecutionState();

			InstData.ScalabilityState = Comp->DebugCachedScalabilityState;
			InstData.bPendingKill = !IsValidChecked(Comp) || Comp->IsUnreachable();
			InstData.bUsingCullProxy = Comp->IsUsingCullProxy();

			InstData.PoolMethod = Comp->PoolingMethod;

			// TODO: need to be able to access at least a shadow copy of per-emitter execution state and num particles. For now, we'll just allow unsafe access
			if (FNiagaraSystemInstance* Inst = InstController->GetSystemInstance_Unsafe())
			{
				InstData.TickGroup = Inst->CalculateTickGroup();
				if (const FNiagaraSystemGpuComputeProxy* SystemInstanceComputeProxy = Inst->GetSystemGpuComputeProxy())
				{
					InstData.GpuTickStage = SystemInstanceComputeProxy->GetComputeTickStage();
				}
				InstData.LWCTile = Inst->GetLWCTile();
				InstData.bIsSolo = Inst->IsSolo();
				InstData.bRequiresGlobalDistanceField = Inst->RequiresGlobalDistanceField();
				InstData.bRequiresDepthBuffer = Inst->RequiresDepthBuffer();
				InstData.bRequiresEarlyViewData = Inst->RequiresEarlyViewData();
				InstData.bRequiresViewUniformBuffer = Inst->RequiresViewUniformBuffer();
				InstData.bRequiresRayTracingScene = Inst->RequiresRayTracingScene();
				InstData.bRequiresCurrentFrameNDC = Inst->RequiresCurrentFrameNDC();

				InstData.Emitters.Reserve(Inst->GetEmitters().Num());
				for (const FNiagaraEmitterInstanceRef& EmitterInst : Inst->GetEmitters())
				{
					FNiagaraOutlinerEmitterInstanceData& EmitterData = InstData.Emitters.AddDefaulted_GetRef();
					const FNiagaraEmitterHandle& EmitterHandle = EmitterInst->GetEmitterHandle();

					//TODO: This is a bit wasteful to copy the name into each instance data. Though we can't rely on the debugger side data matchin the actul running data on the device.
					//We need to build a shared representation of the asset data from the client that we then reference from this per instance data.
					EmitterData.EmitterName = EmitterHandle.GetUniqueInstanceName();
					EmitterData.SimTarget = EmitterInst->GetSimTarget();
					//Move all above to a shared asset representation.

					EmitterData.ExecState = EmitterInst->GetExecutionState();
					EmitterData.NumParticles = EmitterInst->GetNumParticles();

					EmitterData.bRequiresPersistentIDs = EmitterInst->GetParticleData().RequiresPersistentIDs();
					EmitterData.bIsLightweight = EmitterInst->AsStateless() != nullptr;
				}
			}
		}
		else
		{
			InstData.ActualExecutionState = ENiagaraExecutionState::Num;
			InstData.RequestedExecutionState = ENiagaraExecutionState::Num;
		}

#if WITH_PARTICLE_PERF_STATS
		if (StatsListener)
		{
			if (FAccumulatedParticlePerfStats* ComponentStats = StatsListener->GetStats(Comp))
			{
				InstData.AverageTime.GameThread = ComponentStats->GetGameThreadStats().GetPerFrameAvg();
				InstData.AverageTime.RenderThread = ComponentStats->GetRenderThreadStats().GetPerFrameAvg();

				InstData.MaxTime.GameThread = ComponentStats->GetGameThreadStats().GetPerFrameMax();
				InstData.MaxTime.RenderThread = ComponentStats->GetRenderThreadStats().GetPerFrameMax();
			}
		}
#endif
	}

	//TODO: Add any component less systems too if and when they are a thing.
	//TODO: Gather some info for unloaded or currently unused systems.

}

static const FName NiagaraDebuggerClientOutlienrUpdateMessage("NiagaraDebuggerClientOutlienrUpdateMessage");
bool FNiagaraDebuggerClient::UpdateOutliner(float DeltaSeconds)
{
	if (OutlinerCountdown > 0)
	{
		--OutlinerCountdown;
		FString HudMessage = FString::Printf(TEXT("Capturing Niagara Outliner in %u frames..."), OutlinerCountdown);
		FNiagaraWorldManager::ForAllWorldManagers([&HudMessage](FNiagaraWorldManager& WorldMan){ WorldMan.GetNiagaraDebugHud()->AddMessage(NiagaraDebuggerClientOutlienrUpdateMessage, FNiagaraDebugMessage(ENiagaraDebugMessageType::Info, HudMessage, 1.0f)); });
		return true;
	}
	
	FString HudMessage = FString::Printf(TEXT("Captured Niagara Outliner Info."));
	FNiagaraWorldManager::ForAllWorldManagers([&HudMessage](FNiagaraWorldManager& WorldMan) { WorldMan.GetNiagaraDebugHud()->AddMessage(NiagaraDebuggerClientOutlienrUpdateMessage, FNiagaraDebugMessage(ENiagaraDebugMessageType::Info, HudMessage, 3.0f)); });

	//Ensure any RT writes to perf or state info are complete.
	FlushRenderingCommands();

	OutlinerCountdown = 0;
	FNiagaraOutlinerData OutlinerData;
	CaptureOutlinerData(OutlinerData);
	OutlinerOnCapture.ExecuteIfBound(OutlinerData);

#if WITH_PARTICLE_PERF_STATS
	if (StatsListener)
	{
		FParticlePerfStatsManager::RemoveListener(StatsListener);
		StatsListener = nullptr;
	}
#endif

	//Clear up the timer now that we've sent the capture.
	//TODO: continuous/repeated capture mode?
	return false;
}

//////////////////////////////////////////////////////////////////////////

bool FNiagaraSimCacheCaptureInfo::Process()
{
	UNiagaraComponent* Comp = Component.Get();
	if (Comp == nullptr || ProcessedFrames >= Request.CaptureDelayFrames + Request.CaptureFrames)
	{
		//Capture is complete		
		if (SimCache.IsValid())
		{
			SimCache->EndWrite();

			OnCapture.ExecuteIfBound(Request, SimCache.Get());
		}
		return true;
	}

	if(ProcessedFrames >= Request.CaptureDelayFrames)
	{
		//First Frame of Capture, init the sim cache.
		if (ProcessedFrames == Request.CaptureDelayFrames)
		{	
			FNiagaraSimCacheCreateParameters CacheCreateParameters = FNiagaraSimCacheCreateParameters::CreateForDebugging();
			CacheCreateParameters.bAllowSerializeLargeCache = false;

			SimCache.Reset(NewObject<UNiagaraSimCache>(GetTransientPackage()));
			SimCache->BeginWrite(CacheCreateParameters, Comp);
		}

		SimCache->WriteFrame(Comp);
	}
	++ProcessedFrames;

	return false;
}

#endif//WITH_NIAGARA_DEBUGGER
