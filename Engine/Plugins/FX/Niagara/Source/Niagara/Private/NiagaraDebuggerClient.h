// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
Controller class running on game clients that handles the passing of messages to a connected Niagara debugger.
==============================================================================*/

#pragma once

#include "Containers/Ticker.h"
#include "NiagaraCommon.h"
#include "NiagaraDebuggerCommon.h"
#include "IMessageContext.h"
#include "NiagaraDefines.h"
#include "Particles/ParticlePerfStatsManager.h"
#include "UObject/StrongObjectPtr.h"

#if WITH_NIAGARA_DEBUGGER
#include "NiagaraSimCache.h"
#endif

class FMessageEndpoint;

#if WITH_PARTICLE_PERF_STATS

/**
Listener that accumulates a short run of stats for all systems and components in the scene and reports those stats to the outliner.
*/
class FNiagaraOutlinerPerfListener : public FParticlePerfStatsListener_GatherAll
{
public:
	FNiagaraOutlinerPerfListener() : FParticlePerfStatsListener_GatherAll(true, true, true)
	{

	}
};

#endif

#if WITH_NIAGARA_DEBUGGER


struct FNiagaraSimCacheCaptureInfo
{
	uint32 ProcessedFrames = 0;
	FNiagaraSystemSimCacheCaptureRequest Request;
	TWeakObjectPtr<UNiagaraComponent> Component;

	TStrongObjectPtr<UNiagaraSimCache> SimCache = nullptr;

	FOnNiagaraDebuggerClientSimCacheCapture OnCapture;

	/** Process this request. Captures data where needed. Returns true if complete. */
	bool Process();
};

DECLARE_LOG_CATEGORY_EXTERN(LogNiagaraDebuggerClient, Log, All);

class FNiagaraDebuggerClient : public INiagaraDebuggerClient
{
public:

	FNiagaraDebuggerClient();
	virtual ~FNiagaraDebuggerClient();

	bool Tick(float DeltaSeconds);

	//INiagaraDebuggerClient Interface
	virtual void ExecConsoleCommand(const FNiagaraDebuggerExecuteConsoleCommand& Message) override;
	virtual void UpdateDebugHUDSettings(const FNiagaraDebugHUDSettingsData& Message) override;
	virtual void GetSimpleClientInfo(FNiagaraSimpleClientInfo& OutClientInfo) override;
	virtual void UpdateOutlinerSettings(const FNiagaraOutlinerCaptureSettings& Message, FOnNiagaraDebuggerClientOutlinerCapture OnCapture) override;
	virtual void SimCacheCaptureRequest(const FNiagaraSystemSimCacheCaptureRequest& Message, FOnNiagaraDebuggerClientSimCacheCapture OnCapture) override;
	//INiagaraDebuggerClient End

private:

	void HandleConnectionRequestMessage(const FNiagaraDebuggerRequestConnection& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void HandleConnectionClosedMessage(const FNiagaraDebuggerConnectionClosed& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void HandleExecConsoleCommandMessage(const FNiagaraDebuggerExecuteConsoleCommand& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void HandleDebugHUDSettingsMessage(const FNiagaraDebugHUDSettingsData& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void HandleRequestSimpleClientInfoMessage(const FNiagaraRequestSimpleClientInfoMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void HandleOutlinerSettingsMessage(const FNiagaraOutlinerCaptureSettings& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void HandleSimCacheCaptureRequestMessage(const FNiagaraSystemSimCacheCaptureRequest& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	void UpdateClientInfo();

	/** Closes any currently active connection. */
	void CloseConnection();

	/** Handle any cleanup needed whether we close the connection or the client does. */
	void OnConnectionClosed();

	void ExecuteConsoleCommand(const TCHAR* Cmd, bool bRequiresWorld);

	bool UpdateOutliner(float DeltaSeconds);

	void CaptureOutlinerData(FNiagaraOutlinerData& OutlinerData);

	/** Holds the session and instance identifier. */
	FGuid SessionId;
	FGuid InstanceId;
	FString InstanceName;

	/** Holds the messaging endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** The address of the connected debugger, if any. */
	FMessageAddress Connection;

	FNiagaraOutlinerCaptureSettings OutlinerSettings;
	FTSTicker::FDelegateHandle TickerHandle;

	uint32 OutlinerCountdown = 0;
	FOnNiagaraDebuggerClientOutlinerCapture OutlinerOnCapture;

#if WITH_PARTICLE_PERF_STATS
	TSharedPtr<FNiagaraOutlinerPerfListener, ESPMode::ThreadSafe> StatsListener;
#endif

	/* All currently ongoing sim cache captures. */
	TArray<FNiagaraSimCacheCaptureInfo> SimCacheCaptures;
};

#endif
