// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDebuggerCommon.h"
#include "NiagaraModule.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDebuggerCommon)

INiagaraDebuggerClient* INiagaraDebuggerClient::Get()
{
#if WITH_NIAGARA_DEBUGGER
	INiagaraModule& NiagaraModule = FModuleManager::LoadModuleChecked<INiagaraModule>("Niagara");
	return reinterpret_cast<INiagaraDebuggerClient*>(NiagaraModule.GetDebuggerClient());
#else
	return nullptr;
#endif
}

//////////////////////////////////////////////////////////////////////////

FNiagaraDebugHUDSettingsData::FNiagaraDebugHUDSettingsData()
{
	ActorFilter = TEXT("*");
	ComponentFilter = TEXT("*");
	SystemFilter = TEXT("*");
	EmitterFilter = TEXT("*");
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraDebugHUDSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// Reset the properties we don't want to carry between editor sessions, we have to do this as all the properties are inside a
	// data structure, and the whole data structure will go into the config rather than individual properties that specify if they should.
	FNiagaraDebugHUDSettingsData Defaults;
	Data.PlaybackMode			= Defaults.PlaybackMode;
	Data.bPlaybackRateEnabled	= Defaults.bPlaybackRateEnabled;
}

void UNiagaraDebugHUDSettings::NotifyPropertyChanged()
{
	OnChangedDelegate.Broadcast();
	SaveConfig();
}

//////////////////////////////////////////////////////////////////////////

FString FNiagaraDebugHUDVariable::BuildVariableString(const TArray<FNiagaraDebugHUDVariable>& Variables)
{
	FString Output;
	for (const FNiagaraDebugHUDVariable& Variable : Variables)
	{
		if (Variable.bEnabled && Variable.Name.Len() > 0)
		{
			if (Output.Len() > 0)
			{
				Output.Append(TEXT(","));
			}
			Output.Append(Variable.Name);
		}
	}
	return Output;
};

void FNiagaraDebugHUDVariable::InitFromString(const FString& VariablesString, TArray<FNiagaraDebugHUDVariable>& OutVariables)
{
	TArray<FString> Variables;
	VariablesString.ParseIntoArray(Variables, TEXT(","));
	for (const FString& Var : Variables)
	{
		FNiagaraDebugHUDVariable& NewVar = OutVariables.AddDefaulted_GetRef();
		NewVar.bEnabled = true;
		NewVar.Name = Var;
	}
}
