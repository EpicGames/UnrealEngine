// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDecalRendererProperties.h"
#include "NiagaraConstants.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraRenderer.h"
#include "NiagaraRendererDecals.h"
#include "Components/DecalComponent.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDecalRendererProperties)

#if WITH_EDITOR
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SWidget.h"
#include "Styling/SlateIconFinder.h"
#include "AssetThumbnail.h"
#endif

#define LOCTEXT_NAMESPACE "UNiagaraDecalRendererProperties"

namespace NiagaraDecalRendererPropertiesLocal
{
	TArray<TWeakObjectPtr<UNiagaraDecalRendererProperties>> RendererPropertiesToDeferredInit;

	template<typename TValueType>
	static FNiagaraVariable MakeNiagaraVariableWithValue(const FNiagaraTypeDefinition& TypeDef, const FName& Name, const TValueType& Value)
	{
		FNiagaraVariable Variable(TypeDef, Name);
		Variable.SetValue(Value);
		return Variable;
	}

	static void SetupBindings(UNiagaraDecalRendererProperties* Props)
	{
		if (Props->PositionBinding.IsValid())
		{
			return;
		}
		Props->InitRendererBaseBindings();
		Props->PositionBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
		Props->DecalOrientationBinding.Setup(UNiagaraDecalRendererProperties::GetDecalOrientationVariable(), UNiagaraDecalRendererProperties::GetDecalOrientationVariable());
		Props->DecalSizeBinding.Setup(UNiagaraDecalRendererProperties::GetDecalSizeVariable(), UNiagaraDecalRendererProperties::GetDecalSizeVariable());
		Props->DecalFadeBinding.Setup(UNiagaraDecalRendererProperties::GetDecalFadeVariable(), UNiagaraDecalRendererProperties::GetDecalFadeVariable());
		Props->DecalSortOrderBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_UNIQUE_ID);
		Props->DecalColorBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_COLOR);
		Props->DecalVisibleBinding.Setup(UNiagaraDecalRendererProperties::GetDecalVisibleVariable(), UNiagaraDecalRendererProperties::GetDecalVisibleVariable());
		Props->RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);

	#if WITH_EDITORONLY_DATA
		Props->MaterialParameterBinding.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		Props->MaterialParameterBinding.SetAllowedObjects({ UMaterialInterface::StaticClass() });
	#endif
	}
}

UNiagaraDecalRendererProperties::UNiagaraDecalRendererProperties()
{
	AttributeBindings =
	{
		&PositionBinding,
		&DecalOrientationBinding,
		&DecalSizeBinding,
		&DecalFadeBinding,
		&DecalSortOrderBinding,
		&DecalColorBinding,
		&DecalVisibleBinding,
		&RendererVisibilityTagBinding,
	};
}

void UNiagaraDecalRendererProperties::PostLoad()
{
	Super::PostLoad();
	
#if WITH_EDITORONLY_DATA
	SourceMode = ValidateSourceMode(SourceMode);
	ChangeToPositionBinding(PositionBinding);
#endif
	PostLoadBindings(ENiagaraRendererSourceDataMode::Particles);
	MaterialParameters.ConditionalPostLoad();
}

void UNiagaraDecalRendererProperties::PostInitProperties()
{
	using namespace NiagaraDecalRendererPropertiesLocal;

	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		// We can end up hitting PostInitProperties before the Niagara Module has initialized bindings this needs, mark this object for deferred init and early out.
		if (FModuleManager::Get().IsModuleLoaded("Niagara") == false)
		{
			NiagaraDecalRendererPropertiesLocal::RendererPropertiesToDeferredInit.Add(this);
			return;
		}
		SetupBindings(this);
	}
}

void UNiagaraDecalRendererProperties::Serialize(FArchive& Ar)
{
	// MIC will replace the main material during serialize
	// Be careful if adding code that looks at the material to make sure you get the correct one
	{
#if WITH_EDITORONLY_DATA
		TOptional<TGuardValue<TObjectPtr<UMaterialInterface>>> MICGuard;
		if (Ar.IsSaving() && Ar.IsCooking() && MICMaterial)
		{
			MICGuard.Emplace(Material, MICMaterial);
		}
#endif

		Super::Serialize(Ar);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDecalRendererProperties::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraDecalRendererProperties, SourceMode))
	{
		UpdateSourceModeDerivates(SourceMode, true);
	}

	// Update our MICs if we change material / material bindings
	//-OPT: Could narrow down further to only static materials
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraDecalRendererProperties, Material)) ||
		(MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraDecalRendererProperties, MaterialParameters)))
	{
		UpdateMICs();
	}
}
#endif// WITH_EDITORONLY_DATA

void UNiagaraDecalRendererProperties::InitCDOPropertiesAfterModuleStartup()
{
	using namespace NiagaraDecalRendererPropertiesLocal;

	UNiagaraDecalRendererProperties* CDO = CastChecked<UNiagaraDecalRendererProperties>(UNiagaraDecalRendererProperties::StaticClass()->GetDefaultObject());
	SetupBindings(CDO);

	for (TWeakObjectPtr<UNiagaraDecalRendererProperties>& WeakProps : NiagaraDecalRendererPropertiesLocal::RendererPropertiesToDeferredInit)
	{
		if (UNiagaraDecalRendererProperties* Props = WeakProps.Get())
		{
			SetupBindings(Props);
		}
	}
}

FNiagaraRenderer* UNiagaraDecalRendererProperties::CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
	FNiagaraRenderer* NewRenderer = new FNiagaraRendererDecals(FeatureLevel, this, Emitter);
	NewRenderer->Initialize(this, Emitter, InController);
	return NewRenderer;
}

class FNiagaraBoundsCalculatorDecals : public FNiagaraBoundsCalculator
{
public:
	FNiagaraBoundsCalculatorDecals(const FNiagaraDataSetAccessor<FNiagaraPosition>& InPositionAccessor, const FNiagaraDataSetAccessor<FVector3f>& InDecalSizeAccessor)
		: PositionAccessor(InPositionAccessor)
		, DecalSizeAccessor(InDecalSizeAccessor)
	{}

	FNiagaraBoundsCalculatorDecals() = delete;
	virtual ~FNiagaraBoundsCalculatorDecals() = default;

	virtual void InitAccessors(const FNiagaraDataSetCompiledData* CompiledData) override {}
	virtual FBox CalculateBounds(const FTransform& SystemTransform, const FNiagaraDataSet& DataSet, const int32 NumInstances) const override
	{
		if (!NumInstances || !PositionAccessor.IsValid())
		{
			return FBox(ForceInit);
		}

		FNiagaraPosition BoundsMin(ForceInitToZero);
		FNiagaraPosition BoundsMax(ForceInitToZero);
		PositionAccessor.GetReader(DataSet).GetMinMax(BoundsMin, BoundsMax);
		FBox Bounds(BoundsMin, BoundsMax);

		if (DecalSizeAccessor.IsValid())
		{
			Bounds = Bounds.ExpandBy(DecalSizeAccessor.GetReader(DataSet).GetMax().GetAbsMax());
		}
		return Bounds;
	}

protected:
	const FNiagaraDataSetAccessor<FNiagaraPosition>& PositionAccessor;
	const FNiagaraDataSetAccessor<FVector3f>& DecalSizeAccessor;
};

FNiagaraBoundsCalculator* UNiagaraDecalRendererProperties::CreateBoundsCalculator()
{
	if (GetCurrentSourceMode() == ENiagaraRendererSourceDataMode::Emitter)
	{
		return nullptr;
	}

	return new FNiagaraBoundsCalculatorDecals(PositionDataSetAccessor, DecalSizeDataSetAccessor);
}

void UNiagaraDecalRendererProperties::GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const
{
	if ( UMaterialInterface* MaterialInterface = GetMaterial(InEmitter) )
	{
		OutMaterials.Add(MaterialInterface);
	}
}

void UNiagaraDecalRendererProperties::CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData)
{
	UpdateSourceModeDerivates(SourceMode);
	UpdateMICs();

	InitParticleDataSetAccessor(PositionDataSetAccessor, CompiledData, PositionBinding);
	InitParticleDataSetAccessor(DecalOrientationDataSetAccessor, CompiledData, DecalOrientationBinding);
	InitParticleDataSetAccessor(DecalSizeDataSetAccessor, CompiledData, DecalSizeBinding);
	InitParticleDataSetAccessor(DecalFadeDataSetAccessor, CompiledData, DecalFadeBinding);
	InitParticleDataSetAccessor(DecalSortOrderDataSetAccessor, CompiledData, DecalSortOrderBinding);
	InitParticleDataSetAccessor(DecalColorDataSetAccessor, CompiledData, DecalColorBinding);
	InitParticleDataSetAccessor(DecalVisibleAccessor, CompiledData, DecalVisibleBinding);
	InitParticleDataSetAccessor(RendererVisibilityTagAccessor, CompiledData, RendererVisibilityTagBinding);
}

void UNiagaraDecalRendererProperties::UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit)
{
	Super::UpdateSourceModeDerivates(InSourceMode, bFromPropertyEdit);

	if ( UNiagaraEmitterBase* SrcEmitter = GetTypedOuter<UNiagaraEmitterBase>() )
	{
		for (FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
		{
			MaterialParamBinding.CacheValues(SrcEmitter);
		}
	}
}

bool UNiagaraDecalRendererProperties::PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore)
{
	bool bAnyAdded = Super::PopulateRequiredBindings(InParameterStore);

	for (const FNiagaraVariableAttributeBinding* Binding : AttributeBindings)
	{
		if (Binding && Binding->CanBindToHostParameterMap())
		{
			InParameterStore.AddParameter(Binding->GetParamMapBindableVariable(), false);
			bAnyAdded = true;
		}
	}

	for (const FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		InParameterStore.AddParameter(MaterialParamBinding.GetParamMapBindableVariable(), false);
		bAnyAdded = true;
	}

	return bAnyAdded;
}

#if WITH_EDITORONLY_DATA
const TArray<FNiagaraVariable>& UNiagaraDecalRendererProperties::GetOptionalAttributes()
{
	using namespace NiagaraDecalRendererPropertiesLocal;

	static TArray<FNiagaraVariable> Attrs;
	if (Attrs.Num() == 0)
	{
		Attrs.Add(SYS_PARAM_PARTICLES_POSITION);
		Attrs.Add(SYS_PARAM_PARTICLES_COLOR);
		Attrs.Add(GetDecalSizeVariable());
		Attrs.Add(GetDecalOrientationVariable());
		Attrs.Add(GetDecalFadeVariable());
		Attrs.Add(SYS_PARAM_PARTICLES_UNIQUE_ID);
		Attrs.Add(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
	}
	return Attrs;
}

const FSlateBrush* UNiagaraDecalRendererProperties::GetStackIcon() const
{
	return FSlateIconFinder::FindIconBrushForClass(UDecalComponent::StaticClass());
}

void UNiagaraDecalRendererProperties::GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	OutWidgets.Add(SNew(STextBlock).Text(LOCTEXT("DecalRenderer", "Decal Renderer")));
}

void UNiagaraDecalRendererProperties::GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const
{
	Super::GetRendererFeedback(InEmitter, OutErrors, OutWarnings, OutInfo);

	OutInfo.Emplace(LOCTEXT("DecalRenderingPreviewSceneInfo", "Decal Rendering does not show in the default preview scene, please modify the Preview Scene Settings to add ground geometry to visualize in preview."));

	if (MaterialParameters.HasAnyBindings())
	{
		TArray<UMaterialInterface*> Materials;
		GetUsedMaterials(nullptr, Materials);
		MaterialParameters.GetFeedback(Materials, OutWarnings);
	}
}

TArray<FNiagaraVariable> UNiagaraDecalRendererProperties::GetBoundAttributes() const
{
	TArray<FNiagaraVariable> BoundAttributes = Super::GetBoundAttributes();
	BoundAttributes.Reserve(BoundAttributes.Num() + MaterialParameters.AttributeBindings.Num() + 1);

	for (const FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		BoundAttributes.AddUnique(MaterialParamBinding.GetParamMapBindableVariable());
	}

	if (MaterialParameterBinding.IsSetToDefault() == false)
	{
		BoundAttributes.AddUnique(MaterialParameterBinding.AliasedParameter);
	}

	return BoundAttributes;
}

void UNiagaraDecalRendererProperties::RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RenameVariable(OldVariable, NewVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MaterialParameters.RenameVariable(OldVariable, NewVariable, InEmitter, GetCurrentSourceMode());
#endif
}

void UNiagaraDecalRendererProperties::RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RemoveVariable(OldVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MaterialParameters.RemoveVariable(OldVariable, InEmitter, GetCurrentSourceMode());
#endif
}
#endif // WITH_EDITORONLY_DATA

void UNiagaraDecalRendererProperties::UpdateMICs()
{
#if WITH_EDITORONLY_DATA
	UpdateMaterialParametersMIC(MaterialParameters, Material, MICMaterial);
#endif
}

UMaterialInterface* UNiagaraDecalRendererProperties::GetMaterial(const FNiagaraEmitterInstance* InEmitter) const
{
	UMaterialInterface* MaterialInterface = nullptr;
	if (InEmitter != nullptr)
	{
		MaterialInterface = Cast<UMaterialInterface>(InEmitter->FindBinding(MaterialParameterBinding.ResolvedParameter));
	}

#if WITH_EDITORONLY_DATA
	MaterialInterface = MaterialInterface ? MaterialInterface : ToRawPtr(MICMaterial);
#endif

	return MaterialInterface ? MaterialInterface : ToRawPtr(Material);
}

FNiagaraVariable UNiagaraDecalRendererProperties::GetDecalOrientationVariable()
{
	using namespace NiagaraDecalRendererPropertiesLocal;
	static FNiagaraVariable Variable = MakeNiagaraVariableWithValue(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Particles.DecalOrientation"), UNiagaraDecalRendererProperties::GetDefaultOrientation());
	return Variable;
}

FNiagaraVariable UNiagaraDecalRendererProperties::GetDecalSizeVariable()
{
	using namespace NiagaraDecalRendererPropertiesLocal;
	static FNiagaraVariable Variable = MakeNiagaraVariableWithValue(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particles.DecalSize"), UNiagaraDecalRendererProperties::GetDefaultDecalSize());
	return Variable;
}

FNiagaraVariable UNiagaraDecalRendererProperties::GetDecalFadeVariable()
{
	using namespace NiagaraDecalRendererPropertiesLocal;
	static FNiagaraVariable Variable = MakeNiagaraVariableWithValue(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.DecalFade"), UNiagaraDecalRendererProperties::GetDefaultDecalFade());
	return Variable;
}

FNiagaraVariable UNiagaraDecalRendererProperties::GetDecalVisibleVariable()
{
	using namespace NiagaraDecalRendererPropertiesLocal;
	static FNiagaraVariable Variable = MakeNiagaraVariableWithValue(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Particles.DecalVisible"), UNiagaraDecalRendererProperties::GetDefaultDecalVisible());
	return Variable;
}


#undef LOCTEXT_NAMESPACE
