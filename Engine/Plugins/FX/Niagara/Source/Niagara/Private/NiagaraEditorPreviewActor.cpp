// Copyright Epic Games, Inc. All Rights Reserved.


#include "NiagaraEditorPreviewActor.h"

#include "NiagaraComponent.h"
#include "NiagaraSystemInstanceController.h"

#include "UObject/ConstructorHelpers.h"
#include "Components/ArrowComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEditorPreviewActor)

namespace NiagaraEditorPreviewActorPrivate
{
	TOptional<FVector> InterpolatePoints(float ShapeU, TConstArrayView<FVector> ShapePoints, ENiagaraEditorPreviewActorPlaybackType PlaybackType, double Tension)
	{
		const int32 NumPoints = ShapePoints.Num();
		if (NumPoints == 0)
		{
			return {};
		}

		const int32 NumPointsMinusOne = ShapePoints.Num() - 1;

		FVector Points[4];
		double Interp = 0.0f;
		{
			switch (PlaybackType)
			{
				default:
				case ENiagaraEditorPreviewActorPlaybackType::Once:
				{
					const double PointU = ShapeU * double(NumPointsMinusOne);
					const int32 Point = FMath::FloorToInt(PointU);
					Points[0] = ShapePoints[FMath::Clamp(Point - 1, 0, NumPointsMinusOne)];
					Points[1] = ShapePoints[FMath::Clamp(Point    , 0, NumPointsMinusOne)];
					Points[2] = ShapePoints[FMath::Clamp(Point + 1, 0, NumPointsMinusOne)];
					Points[3] = ShapePoints[FMath::Clamp(Point + 2, 0, NumPointsMinusOne)];
					Interp = FMath::Fractional(PointU);
					break;
				}
				case ENiagaraEditorPreviewActorPlaybackType::Looping:
				{
					const double PointU = ShapeU * double(NumPoints);
					const int32 Point = FMath::FloorToInt(PointU) + NumPoints;
					Points[0] = ShapePoints[FMath::Modulo(Point - 1, NumPoints)];
					Points[1] = ShapePoints[FMath::Modulo(Point    , NumPoints)];
					Points[2] = ShapePoints[FMath::Modulo(Point + 1, NumPoints)];
					Points[3] = ShapePoints[FMath::Modulo(Point + 2, NumPoints)];
					Interp = FMath::Fractional(PointU);
					break;
				}
				case ENiagaraEditorPreviewActorPlaybackType::PingPong:
				{
					const double PointU = ShapeU * double(NumPointsMinusOne);
					const int32 Point = FMath::FloorToInt(PointU);
					Points[0] = ShapePoints[FMath::Abs(FMath::Modulo(Point - 1, NumPoints))];
					Points[1] = ShapePoints[FMath::Abs(FMath::Modulo(Point    , NumPoints))];
					Points[2] = ShapePoints[FMath::Abs(FMath::Modulo(Point + 1, NumPoints))];
					Points[3] = ShapePoints[FMath::Abs(FMath::Modulo(Point + 2, NumPoints))];
					Interp = FMath::Fractional(PointU);
					break;
				}
			}
		}

		const double T = FMath::Fractional(Interp);
		const double T2 = T * T;
		const double T3 = T2 * T;
		const double C1 =  2.0 * T3 - 3.0 * T2 + 1.0;
		const double C2 = -2.0 * T3 + 3.0 * T2;
		const double C3 = T3 - 2.0 * T2 + T;
		const double C4 = T3 - T2;

		FVector Temp;
		Temp  = (C1 * Points[1]) + (C2 * Points[2]);
		Temp += Tension * C3 * (Points[2] - Points[0]);
		Temp += Tension * C4 * (Points[3] - Points[1]);
		return Temp * 0.5;
	}
}

ANiagaraEditorPreviewActor::ANiagaraEditorPreviewActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("ArrowComponent0"));
	RootComponent = ArrowComponent;

	NiagaraComponent = CreateDefaultSubobject<UNiagaraComponent>(TEXT("NiagaraComponent0"));
	NiagaraComponent->SetupAttachment(ArrowComponent);
	NiagaraComponent->SetAllowScalability(false);

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;

	bIsEditorOnlyActor = true;

	CustomShapePoints.Emplace(-200.0f, 0.0f, 0.0f);
	CustomShapePoints.Emplace( 200.0f, 0.0f, 0.0f);
}

void ANiagaraEditorPreviewActor::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	//if (NiagaraComponent)
	//{
	//	NiagaraComponent->OnSystemFinished.AddUniqueDynamic(this, &ANiagaraActor::OnNiagaraSystemFinished);
	//}
}

void ANiagaraEditorPreviewActor::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();

	//// Clear Notification Delegate
	//if (NiagaraComponent)
	//{
	//	NiagaraComponent->OnSystemFinished.RemoveAll(this);
	//}
}

void ANiagaraEditorPreviewActor::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}

	if ( NiagaraComponent->IsActive() == false )
	{
		NiagaraComponent->Activate();
	}

	FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = NiagaraComponent->GetSystemInstanceController();
	if (SystemInstanceController == nullptr)
	{
		return;
	}

	// Calculate our playback time
	const float CurrentTime = SystemInstanceController->GetAge();
	float CurrMotionTime = CurrentTime / MotionDuration;
	float NextMotionTime = (CurrentTime + 0.1f) / MotionDuration;

	switch (PlaybackType)
	{
		case ENiagaraEditorPreviewActorPlaybackType::Once:
			CurrMotionTime = FMath::Clamp(CurrMotionTime, 0.0f, 1.0f);
			NextMotionTime = FMath::Clamp(NextMotionTime, 0.0f, 1.0f);
			break;
		case ENiagaraEditorPreviewActorPlaybackType::Looping:
			CurrMotionTime = FMath::Fractional(CurrMotionTime);
			NextMotionTime = FMath::Fractional(NextMotionTime);
			break;
		case ENiagaraEditorPreviewActorPlaybackType::PingPong:
			CurrMotionTime = ((FMath::FloorToInt(CurrMotionTime) & 1) == 0) ? FMath::Fractional(CurrMotionTime) : 1.0f - FMath::Fractional(CurrMotionTime);
			NextMotionTime = ((FMath::FloorToInt(NextMotionTime) & 1) == 0) ? FMath::Fractional(NextMotionTime) : 1.0f - FMath::Fractional(NextMotionTime);
			break;
	}

	// Calculate the location of the component
	TOptional<FVector> CurrLocation = InternalCalculateLocation(CurrMotionTime);

	// Calculate the rotation of the component
	TOptional<FQuat> CurrRotation;
	switch ( RotationMode )
	{
		case ENiagaraEditorPreviewActorRotationMode::None:
		{
			CurrRotation = FQuat::Identity;
			break;
		}

		case ENiagaraEditorPreviewActorRotationMode::DirectionOfTravel:
		{
			const TOptional<FVector> NextLocation = InternalCalculateLocation(NextMotionTime);
			if (CurrLocation.IsSet() && NextLocation.IsSet())
			{
				FVector XAxis = NextLocation.GetValue() - CurrLocation.GetValue();
				if (XAxis.Normalize())
				{
					FVector ZAxis = FVector::ZAxisVector;
					FVector YAxis = FVector::CrossProduct(ZAxis, XAxis);
					XAxis = FVector::CrossProduct(YAxis, ZAxis);

					CurrRotation = FMatrix(XAxis, YAxis, ZAxis, FVector::ZeroVector).ToQuat();
				}
			}
			break;
		}

		case ENiagaraEditorPreviewActorRotationMode::Blueprint:
		{
			FQuat TempRotation = FQuat::Identity;
			CalculateRotation(NextMotionTime, TempRotation);
			CurrRotation = TempRotation;
			break;
		}
	}

	// Set on the component
	if (CurrLocation.IsSet() || CurrRotation.IsSet())
	{
		FVector FinalLocation;
		if (CurrLocation.IsSet())
		{
			FinalLocation = ShapeRotation.RotateVector(CurrLocation.GetValue()) * ShapeScale;
		}
		else
		{
			FinalLocation = NiagaraComponent->GetRelativeLocation();
		}

		FRotator FinalRotation;
		if (CurrRotation.IsSet())
		{
			FinalRotation = (ShapeRotation.Quaternion() * CurrRotation.GetValue()).Rotator();
		}
		else
		{
			FinalRotation = NiagaraComponent->GetRelativeRotation();
		}

		NiagaraComponent->SetRelativeLocationAndRotation(FinalLocation, FinalRotation);
	}
}

TOptional<FVector> ANiagaraEditorPreviewActor::InternalCalculateLocation(float MotionTime)
{
	TOptional<FVector> OutLocation;
	switch (MotionType)
	{
		case ENiagaraEditorPreviewActorShapeType::Square:
		{
			const FVector2D HSize = SquareSize * 0.5;
			const FVector Points[] =
			{
				FVector(-HSize.X, -HSize.Y, 0.0),
				FVector( HSize.X, -HSize.Y, 0.0),
				FVector( HSize.X,  HSize.Y, 0.0),
				FVector(-HSize.X,  HSize.Y, 0.0),
			};
			OutLocation = NiagaraEditorPreviewActorPrivate::InterpolatePoints(MotionTime, Points, PlaybackType, ShapeTension);
			break;
		}

		case ENiagaraEditorPreviewActorShapeType::Triangle:
		{
			const FVector2D HSize = TriangleSize * 0.5;
			const FVector Points[] =
			{
				FVector(     0.0, -HSize.Y, 0.0),
				FVector( HSize.X,  HSize.Y, 0.0),
				FVector(-HSize.X,  HSize.Y, 0.0),
			};
			OutLocation = NiagaraEditorPreviewActorPrivate::InterpolatePoints(MotionTime, Points, PlaybackType, ShapeTension);
			break;
		}

		case ENiagaraEditorPreviewActorShapeType::Circle:
		{
			const double R = CircleRotationRate.Get(1.0);
			const double S = FMath::Sin(MotionTime * R * UE_TWO_PI);
			const double C = FMath::Cos(MotionTime * R * UE_TWO_PI);

			const double Radius = FMath::Lerp(CircleRadius, CircleEndRadius.Get(CircleRadius), MotionTime);
			OutLocation = FVector(S, C, 0.0) * Radius;
			break;
		}

		case ENiagaraEditorPreviewActorShapeType::Custom:
		{
			OutLocation = NiagaraEditorPreviewActorPrivate::InterpolatePoints(MotionTime, CustomShapePoints, PlaybackType, ShapeTension);
			break;
		}

		case ENiagaraEditorPreviewActorShapeType::Blueprint:
		{
			FVector TempLocation = FVector::ZeroVector;
			CalculateLocation(MotionTime, TempLocation);
			OutLocation = TempLocation;
			break;
		}
	}

	return OutLocation;
}

#if WITH_EDITOR
bool ANiagaraEditorPreviewActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (UNiagaraSystem* System = NiagaraComponent->GetAsset())
	{
		Objects.Add(System);
	}

	return true;
}
#endif // WITH_EDITOR

