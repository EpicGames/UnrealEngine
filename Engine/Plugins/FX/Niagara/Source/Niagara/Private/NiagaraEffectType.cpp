// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraEffectType.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/World.h"
#include "NiagaraCommon.h"
#include "Modules/ModuleManager.h"
#include "NiagaraComponent.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstanceController.h"
#include "UObject/UObjectIterator.h"

//In an effort to cut the impact of runtime perf tracking, I limit the number of fames we actually sample on.
int32 GNumFramesBetweenRuntimePerfSamples = 5;
static FAutoConsoleVariableRef CVarNumFramesBetweenRuntimePerfSamples(TEXT("fx.NumFramesBetweenRuntimePerfSamples"), GNumFramesBetweenRuntimePerfSamples, TEXT("How many frames between each sample of Niagara runtime perf. \n"), ECVF_ReadOnly);

int32 GNiagaraRuntimeCycleHistorySize = 15;
static FAutoConsoleVariableRef CVarNiagaraRuntimeCycleHistorySize(TEXT("fx.NiagaraRuntimeCycleHistorySize"), GNiagaraRuntimeCycleHistorySize, TEXT("How many frames history to use in Niagara's runtime performance trackers. \n"), ECVF_ReadOnly);

UNiagaraEffectType::UNiagaraEffectType(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, UpdateFrequency(ENiagaraScalabilityUpdateFrequency::SpawnOnly)
	, CullReaction(ENiagaraCullReaction::DeactivateImmediate)
	, SignificanceHandler(nullptr)
	, NumInstances(0)
	, bNewSystemsSinceLastScalabilityUpdate(false)
	, PerformanceBaselineController(nullptr)
{
}

void UNiagaraEffectType::BeginDestroy()
{
	Super::BeginDestroy();
}

bool UNiagaraEffectType::IsReadyForFinishDestroy()
{
	return Super::IsReadyForFinishDestroy();
}

void UNiagaraEffectType::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);
}

void UNiagaraEffectType::PostLoad()
{
	Super::PostLoad();

	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);

	/** Init signficance handlers to match previous behavior. */
	if (NiagaraVer < FNiagaraCustomVersion::SignificanceHandlers)
	{
		if (UpdateFrequency == ENiagaraScalabilityUpdateFrequency::SpawnOnly)
		{
			SignificanceHandler = nullptr;
		}
		else
		{
			SignificanceHandler = NewObject<UNiagaraSignificanceHandlerDistance>(this);
		}
	}

	for (FNiagaraSystemScalabilitySettings& SysScalabilitySetting : SystemScalabilitySettings.Settings)
	{
		SysScalabilitySetting.PostLoad(NiagaraVer);
	}

	//Apply platform set redirectors
	auto ApplyPlatformSetRedirects = [](FNiagaraPlatformSet& Platforms)
	{
		Platforms.ApplyRedirects();
	};
	ForEachPlatformSet(ApplyPlatformSetRedirects);


#if !WITH_EDITOR && NIAGARA_PERF_BASELINES
	//When not in the editor we clear out the baseline so that it's regenerated for play tests.
	//We cannot use the saved editor/development config settings.
	InvalidatePerfBaseline();
#endif
}

#if WITH_EDITORONLY_DATA
void UNiagaraEffectType::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraSignificanceHandlerDistance::StaticClass()));
}
#endif

const FNiagaraSystemScalabilitySettings& UNiagaraEffectType::GetActiveSystemScalabilitySettings()const
{
	for (const FNiagaraSystemScalabilitySettings& Settings : SystemScalabilitySettings.Settings)
	{
		if (Settings.Platforms.IsActive())
		{
			return Settings;
		}
	}

	//UE_LOGF(LogNiagara, Warning, "Could not find active system scalability settings for EffectType %ls", *GetFullName());

	static FNiagaraSystemScalabilitySettings Dummy;
	return Dummy;
}

const FNiagaraEmitterScalabilitySettings& UNiagaraEffectType::GetActiveEmitterScalabilitySettings()const
{
	for (const FNiagaraEmitterScalabilitySettings& Settings : EmitterScalabilitySettings.Settings)
	{
		if (Settings.Platforms.IsActive())
		{
			return Settings;
		}
	}

	//UE_LOGF(LogNiagara, Warning, "Could not find active emitter scalability settings for EffectType %ls", *GetFullName());

	static FNiagaraEmitterScalabilitySettings Dummy;
	return Dummy;
}

const FNiagaraReadinessSettingsProfile* UNiagaraEffectType::FindReadinessSettingsProfile(FGameplayTag Profile)const
{
	FGameplayTag ProfileToUse = Profile.IsValid() ? Profile : ReadinessProfile;
	if (ReadinessSettings)
	{
		return ReadinessSettings->Find(ProfileToUse);
	}
	
	return nullptr;
}

#if WITH_PER_FXTYPE_PARTICLE_PERF_STATS
void UNiagaraEffectType::PopulatePerfStatNames()
{
	// Lazy populate to keep unneeded FNames down
	if (CSVStat_Total == NAME_None)
	{
		const FString FNameString = GetFName().ToString();
		CSVStat_Total = *FString::Printf(TEXT("FXType_Total/%s"), *FNameString);
		CSVStat_GTOnly = *FString::Printf(TEXT("FXType_GTOnly/%s"), *FNameString);
		CSVStat_RT = *FString::Printf(TEXT("FXType_RT/%s"), *FNameString);
		CSVStat_GPU = *FString::Printf(TEXT("FXType_GPU/%s"), *FNameString);
		CSVStat_Count = *FString::Printf(TEXT("FXType_Count/%s"), *FNameString);
	}
}
#endif

#if WITH_EDITOR

void UNiagaraEffectType::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FNiagaraSystemUpdateContext UpdateContext;
	for (TObjectIterator<UNiagaraSystem> It; It; ++It)
	{
		UNiagaraSystem* System = *It;
		if (System->GetEffectType() == this)
		{
			System->UpdateScalability();
			UpdateContext.Add(System, true);
		}
	}

	if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraEffectType, PerformanceBaselineController))
	{
		PerfBaselineVersion.Invalidate();
	}
}
#endif

#if NIAGARA_PERF_BASELINES
void UNiagaraEffectType::UpdatePerfBaselineStats(FNiagaraPerfBaselineStats& NewBaselineStats)
{
	PerfBaselineStats = NewBaselineStats;
	PerfBaselineVersion = CurrentPerfBaselineVersion;

#if WITH_EDITOR
	SaveConfig();
#endif
}

void UNiagaraEffectType::InvalidatePerfBaseline()
{
	PerfBaselineVersion.Invalidate();
	PerfBaselineStats = FNiagaraPerfBaselineStats();

#if WITH_EDITOR
	SaveConfig();
#endif
}
#endif

//////////////////////////////////////////////////////////////////////////

static const FNiagaraLinearRamp DefaultBudgetScaleRamp(0.5f, 1.0f, 1.0f, 0.5f);

FNiagaraGlobalBudgetScaling::FNiagaraGlobalBudgetScaling()
{
	bCullByGlobalBudget = false;
	bScaleMaxDistanceByGlobalBudgetUse = false;
	bScaleMaxInstanceCountByGlobalBudgetUse = false;
	bScaleSystemInstanceCountByGlobalBudgetUse = false;
	MaxGlobalBudgetUsage = 1.0f;
	MaxDistanceScaleByGlobalBudgetUse = DefaultBudgetScaleRamp;
	MaxInstanceCountScaleByGlobalBudgetUse = DefaultBudgetScaleRamp;
	MaxSystemInstanceCountScaleByGlobalBudgetUse = DefaultBudgetScaleRamp;
}

//////////////////////////////////////////////////////////////////////////
 
FNiagaraSystemVisibilityCullingSettings::FNiagaraSystemVisibilityCullingSettings()
	: bCullWhenNotRendered(false)
	, bCullByViewFrustum(false)
	, bAllowPreCullingByViewFrustum(false)
	, MaxTimeOutsideViewFrustum(1.0f)
	, MaxTimeWithoutRender(1.0f)
{
}

//////////////////////////////////////////////////////////////////////////

FNiagaraSystemScalabilityOverride::FNiagaraSystemScalabilityOverride()
	: bOverrideDistanceSettings(false)
	, bOverrideInstanceCountSettings(false)
	, bOverridePerSystemInstanceCountSettings(false)
	, bOverrideVisibilitySettings(false)
	, bOverrideGlobalBudgetScalingSettings(false)
	, bOverrideCullProxySettings(false)
{
}

FNiagaraSystemScalabilitySettings::FNiagaraSystemScalabilitySettings()
{
	Clear();
}

void FNiagaraSystemScalabilitySettings::Clear()
{
	Platforms = FNiagaraPlatformSet();
	bCullByDistance = false;
	bCullMaxInstanceCount = false;
	bCullPerSystemMaxInstanceCount = false;	
	MaxDistance = 0.0f;
	MaxInstances = 0;
	MaxSystemInstances = 0;


	MaxTimeWithoutRender_DEPRECATED = 0.0f;
	bCullByMaxTimeWithoutRender_DEPRECATED = false;
	VisibilityCulling = FNiagaraSystemVisibilityCullingSettings();

	BudgetScaling = FNiagaraGlobalBudgetScaling();
	CullProxyMode = ENiagaraCullProxyMode::None;
	MaxSystemProxies = 32;
}

void FNiagaraSystemScalabilitySettings::PostLoad(int32 Version)
{
	if (Version < FNiagaraCustomVersion::VisibilityCullingImprovements)
	{
		VisibilityCulling.bCullWhenNotRendered = bCullByMaxTimeWithoutRender_DEPRECATED;
		VisibilityCulling.MaxTimeWithoutRender = MaxTimeWithoutRender_DEPRECATED;
	}
}

FNiagaraEmitterScalabilitySettings::FNiagaraEmitterScalabilitySettings()
{
	Clear();
}

void FNiagaraEmitterScalabilitySettings::Clear()
{
	SpawnCountScale = 1.0f;
	bScaleSpawnCount = false;
}

FNiagaraEmitterScalabilityOverride::FNiagaraEmitterScalabilityOverride()
	: bOverrideSpawnCountScale(false)
{
}


//////////////////////////////////////////////////////////////////////////

#include "NiagaraScalabilityManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEffectType)

void UNiagaraSignificanceHandlerDistance::CalculateSignificance(TConstArrayView<UNiagaraComponent*> Components, TArrayView<FNiagaraScalabilityState> OutState, TConstArrayView<FNiagaraScalabilitySystemData> SystemData, TArray<int32>& OutIndices)
{
	const int32 ComponentCount = Components.Num();
	check(ComponentCount == OutState.Num());

	for (int32 CompIdx = 0; CompIdx < ComponentCount; ++CompIdx)
	{
		FNiagaraScalabilityState& State = OutState[CompIdx];
		const FNiagaraScalabilitySystemData& SysData = SystemData[State.SystemDataIndex];

		const bool AddIndex = (SysData.bNeedsSignificanceForActiveOrDirty && (!State.bCulled || State.IsDirty())) || SysData.bNeedsSignificanceForCulled;

		if (State.bCulled && !SysData.bNeedsSignificanceForCulled)
		{
			State.Significance = 0.0f;
		}
		else
		{
			UNiagaraComponent* Component = Components[CompIdx];

			float LODDistance = 0.0f;
			if (Component->bEnablePreviewLODDistance)
			{
				LODDistance = Component->PreviewLODDistance;
			}
			else if (FNiagaraSystemInstanceControllerConstPtr Controller = Component->GetSystemInstanceController())
			{
				LODDistance = Controller->GetLODDistance();
			}

			State.Significance = 1.0f / LODDistance;
		}

		if (AddIndex)
		{
			OutIndices.Add(CompIdx);
		}
	}
}

void UNiagaraSignificanceHandlerAge::CalculateSignificance(TConstArrayView<UNiagaraComponent*> Components, TArrayView<FNiagaraScalabilityState> OutState, TConstArrayView<FNiagaraScalabilitySystemData> SystemData, TArray<int32>& OutIndices)
{
	const int32 ComponentCount = Components.Num();
	check(ComponentCount == OutState.Num());

	for (int32 CompIdx = 0; CompIdx < ComponentCount; ++CompIdx)
	{
		FNiagaraScalabilityState& State = OutState[CompIdx];
		const FNiagaraScalabilitySystemData& SysData = SystemData[State.SystemDataIndex];
		const bool AddIndex = (SysData.bNeedsSignificanceForActiveOrDirty && (!State.bCulled || State.IsDirty())) || SysData.bNeedsSignificanceForCulled;

		if (State.bCulled)
		{
			State.Significance = 0.0f;
		}
		else
		{
			UNiagaraComponent* Component = Components[CompIdx];

			if (FNiagaraSystemInstanceControllerConstPtr Controller = Component->GetSystemInstanceController())
			{
				State.Significance = 1.0f / Controller->GetAge();//Newer Systems are higher significance.
			}
		}

		if (AddIndex)
		{
			OutIndices.Add(CompIdx);
		}
	}
}


#if NIAGARA_PERF_BASELINES

#include "AssetRegistry/AssetRegistryModule.h"

//Invalidate this to regenerate perf baseline info.
//For example if there are some significant code optimizations.
const FGuid UNiagaraEffectType::CurrentPerfBaselineVersion = FGuid(0xD854D103, 0x87C17A44, 0x87CA4524, 0x5F72FBC2);
UNiagaraEffectType::FGeneratePerfBaselines UNiagaraEffectType::GeneratePerfBaselinesDelegate;

void UNiagaraEffectType::GeneratePerfBaselines()
{
	if (GeneratePerfBaselinesDelegate.IsBound())
	{
		//Load all effect types so we generate all baselines at once.
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		TArray<FAssetData> EffectTypeAssets;
		AssetRegistryModule.Get().GetAssetsByClass(UNiagaraEffectType::StaticClass()->GetClassPathName(), EffectTypeAssets);

		TArray<UNiagaraEffectType*> EffectTypesToGenerate;
		for (FAssetData& Asset : EffectTypeAssets)
		{
			if (UNiagaraEffectType* FXType = Cast<UNiagaraEffectType>(Asset.GetAsset()))
			{
				if (FXType->IsPerfBaselineValid() == false && FXType->GetPerfBaselineController())
				{
					EffectTypesToGenerate.Add(FXType);
				}
			}
		}

		GeneratePerfBaselinesDelegate.Execute(EffectTypesToGenerate);
	}
}

void UNiagaraEffectType::SpawnBaselineActor(UWorld* World)
{
	if (PerformanceBaselineController && World)
	{
		//Update with dummy stats so we don't try to regen them again.
		FNiagaraPerfBaselineStats DummyStats;
		UpdatePerfBaselineStats(DummyStats);

		FActorSpawnParameters SpawnParams;
		ANiagaraPerfBaselineActor* BaselineActor = CastChecked<ANiagaraPerfBaselineActor>(World->SpawnActorDeferred<ANiagaraPerfBaselineActor>(ANiagaraPerfBaselineActor::StaticClass(), FTransform::Identity));
		BaselineActor->Controller = CastChecked<UNiagaraBaselineController>(StaticDuplicateObject(PerformanceBaselineController, BaselineActor));
		BaselineActor->Controller->EffectType = this;
		BaselineActor->Controller->Owner = BaselineActor;

		BaselineActor->FinishSpawning(FTransform::Identity);
		BaselineActor->RegisterAllActorTickFunctions(true, true);
	}
}

void InvalidatePerfBaselines()
{
	for (TObjectIterator<UNiagaraEffectType> It; It; ++It)
	{
		It->InvalidatePerfBaseline();
	}
}

FAutoConsoleCommand InvalidatePerfBaselinesCommand(
	TEXT("fx.InvalidateNiagaraPerfBaselines"),
	TEXT("Invalidates all Niagara performance baseline data."),
	FConsoleCommandDelegate::CreateStatic(&InvalidatePerfBaselines)
);

#endif
//////////////////////////////////////////////////////////////////////////


//////////////////////////////////////////////////////////////////////////

#if WITH_PER_FXTYPE_PARTICLE_PERF_STATS

bool FParticlePerfStatsListener_EffectType::Tick()
{
	if (FCsvProfiler* CSVProfiler = FCsvProfiler::Get())
	{
		//First we gather up stats for all systems in each FXType
		TMap<UNiagaraEffectType*, FParticlePerfStats> FXTypeStatsMap;
		FParticlePerfStatsManager::ForAllSystemStats(
			[&](TWeakObjectPtr<const UFXSystemAsset>& WeakSystem, TUniquePtr<FParticlePerfStats>& Stats)
			{
				if (Stats->GetGameThreadStats().NumInstances > 0)
				{
					if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(WeakSystem.Get()))
					{
						FParticlePerfStats& FXTypeStats = FXTypeStatsMap.FindOrAdd(System->GetEffectType());
						FParticlePerfStats_GT& GTStats = FXTypeStats.GetGameThreadStats();

						GTStats += Stats->GetGameThreadStats();
					}
				}
			}
		);

		//Now add to CSV
		for (auto It = FXTypeStatsMap.CreateIterator(); It; ++It)
		{
			if (UNiagaraEffectType* FXType = It.Key())
			{
				FParticlePerfStats& Stats = It.Value();

				if (Stats.GetGameThreadStats().NumInstances > 0)
				{
					FXType->PopulatePerfStatNames();

					float TotalTime = static_cast<float>(FPlatformTime::ToMilliseconds64(Stats.GetGameThreadStats().GetTotalCycles()) * 1000.0f);
					float GTTime = static_cast<float>(FPlatformTime::ToMilliseconds64(Stats.GetGameThreadStats().GetTotalCycles_GTOnly()) * 1000.0f);
					//float AvgTime = FPlatformTime::ToMilliseconds64(Stats->GetGameThreadStats().GetPerInstanceAvgCycles()) * 1000.0f;
					int32 Count = static_cast<int32>(Stats.GetGameThreadStats().NumInstances);
					//float Activation = FPlatformTime::ToMilliseconds64(Stats->GetGameThreadStats().ActivationCycles) * 1000.0f;
					//float Wait = FPlatformTime::ToMilliseconds64(Stats->GetGameThreadStats().WaitCycles) * 1000.0f;

					CSVProfiler->RecordCustomStat(FXType->CSVStat_Total, CSV_CATEGORY_INDEX(Particles), TotalTime, ECsvCustomStatOp::Set);
					CSVProfiler->RecordCustomStat(FXType->CSVStat_GTOnly, CSV_CATEGORY_INDEX(Particles), GTTime, ECsvCustomStatOp::Set);
					//CSVProfiler->RecordCustomStat(FXType->CSVStat_InstAvgGT, CSV_CATEGORY_INDEX(Particles), AvgTime, ECsvCustomStatOp::Set);
					CSVProfiler->RecordCustomStat(FXType->CSVStat_Count, CSV_CATEGORY_INDEX(Particles), Count, ECsvCustomStatOp::Set);

					//CSVProfiler->RecordCustomStat(System->CSVStat_Activation, CSV_CATEGORY_INDEX(Particles), Activation, ECsvCustomStatOp::Set);
					//CSVProfiler->RecordCustomStat(System->CSVStat_Waits, CSV_CATEGORY_INDEX(Particles), Wait, ECsvCustomStatOp::Set);
				}
			}
		}
	}
	return true;
}

void FParticlePerfStatsListener_EffectType::TickRT()
{
	if (FCsvProfiler* CSVProfiler = FCsvProfiler::Get())
	{
		//First we gather up stats for all systems in each FXType
		TMap<UNiagaraEffectType*, FParticlePerfStats> FXTypeStatsMap;
		FParticlePerfStatsManager::ForAllSystemStats(
			[&](TWeakObjectPtr<const UFXSystemAsset>& WeakSystem, TUniquePtr<FParticlePerfStats>& Stats)
			{
				if (Stats->GetGameThreadStats().NumInstances > 0)
				{
					if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(WeakSystem.Get()))
					{
						FParticlePerfStats& FXTypeStats = FXTypeStatsMap.FindOrAdd(System->GetEffectType());
						FParticlePerfStats_RT& RTStats = FXTypeStats.GetRenderThreadStats();
						FParticlePerfStats_GPU& GPUStats = FXTypeStats.GetGPUStats();

						RTStats += Stats->GetRenderThreadStats();
						GPUStats += Stats->GetGPUStats();
					}
				}
			}
		);

		//Now add to CSV
		for (auto It = FXTypeStatsMap.CreateIterator(); It; ++It)
		{
			if (UNiagaraEffectType* FXType = It.Key())
			{
				FParticlePerfStats& Stats = It.Value();

				if (Stats.GetRenderThreadStats().NumInstances > 0)
				{
					FXType->PopulatePerfStatNames();

					const float RTTime = static_cast<float>(FPlatformTime::ToMilliseconds64(Stats.GetRenderThreadStats().GetTotalCycles()) * 1000.0f);
					//const float RTAvgTime = FPlatformTime::ToMilliseconds64(Stats->GetRenderThreadStats().GetPerInstanceAvgCycles()) * 1000.0f;
					CSVProfiler->RecordCustomStat(FXType->CSVStat_RT, CSV_CATEGORY_INDEX(Particles), RTTime, ECsvCustomStatOp::Set);
					//CSVProfiler->RecordCustomStat(System->CSVStat_InstAvgRT, CSV_CATEGORY_INDEX(Particles), RTAvgTime, ECsvCustomStatOp::Set);

					const float GpuTime = static_cast<float>(Stats.GetGPUStats().GetTotalMicroseconds());
					//const float GpuAvgTime = float(Stats->GetGPUStats().GetPerInstanceAvgMicroseconds());
					CSVProfiler->RecordCustomStat(FXType->CSVStat_GPU, CSV_CATEGORY_INDEX(Particles), GpuTime, ECsvCustomStatOp::Set);
					//CSVProfiler->RecordCustomStat(System->CSVStat_InstAvgGPU, CSV_CATEGORY_INDEX(Particles), GpuAvgTime, ECsvCustomStatOp::Set);
				}
			}
		}
	}
}

#endif //WITH_PER_FXTYPE_PARTICLE_PERF_STATS