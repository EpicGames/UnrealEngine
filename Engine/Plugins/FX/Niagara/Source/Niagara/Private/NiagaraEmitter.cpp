// Copyright Epic Games, Inc. All Rights Reserved.


#include "NiagaraEmitter.h"

#include "INiagaraEditorOnlyDataUtlities.h"
#include "NiagaraAnalytics.h"
#include "NiagaraBoundsCalculator.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraComponentSettings.h"
#include "NiagaraMessageDataBase.h"
#include "NiagaraEditorDataBase.h"
#include "NiagaraModule.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraScratchPadContainer.h"
#include "NiagaraScript.h"
#include "NiagaraScriptRuntimeData.h"
#include "NiagaraScriptSourceBase.h"
#include "NiagaraSettings.h"
#include "NiagaraShader.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraStats.h"
#include "NiagaraSystem.h"
#include "NiagaraTrace.h"

#include "Engine/Engine.h"
#include "HAL/LowLevelMemTracker.h"
#include "Interfaces/ITargetPlatform.h"
#include "Modules/ModuleManager.h"
#include "Templates/Greater.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "WorldCollision.h"
#include "PSOPrecacheComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEmitter)

#if WITH_EDITOR
const FName UNiagaraEmitter::PrivateMemberNames::EventHandlerScriptProps = GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, EventHandlerScriptProps);

const FString InitialNotSynchronizedReason("Emitter created");

const FGuid UNiagaraEmitter::EmitterMergeMessageId(0xDAF26E73, 0x4D1B416B, 0x815FA6C2, 0x6D5D0A75);
#endif

static bool GNiagaraEmitterPostLoadCleanup = true;
static FAutoConsoleVariableRef CVarRollback_NiagaraEmitterPostLoadCleanup(
	TEXT("fx.Niagara.Rollback.NiagaraEmitterPostLoadCleanup"),
	GNiagaraEmitterPostLoadCleanup,
	TEXT("When true use a leaner version of post load"),
	ECVF_Default
);

static int32 GbForceNiagaraCompileOnLoad = 0;
static FAutoConsoleVariableRef CVarForceNiagaraCompileOnLoad(
	TEXT("fx.ForceCompileOnLoad"),
	GbForceNiagaraCompileOnLoad,
	TEXT("If > 0 emitters will be forced to compile on load. \n"),
	ECVF_Default
	);

static int32 GbForceNiagaraMergeOnLoad = 0;
static FAutoConsoleVariableRef CVarForceNiagaraMergeOnLoad(
	TEXT("fx.ForceMergeOnLoad"),
	GbForceNiagaraMergeOnLoad,
	TEXT("If > 0 emitters will be forced to merge on load. \n"),
	ECVF_Default
);

static int32 GbForceNiagaraFailToCompile = 0;
static FAutoConsoleVariableRef CVarForceNiagaraCompileToFail(
	TEXT("fx.ForceNiagaraCompileToFail"),
	GbForceNiagaraFailToCompile,
	TEXT("If > 0 emitters will go through the motions of a compile, but will never set valid bytecode. \n"),
	ECVF_Default
);

static int32 GbEnableEmitterChangeIdMergeLogging = 0;
static FAutoConsoleVariableRef CVarEnableEmitterChangeIdMergeLogging(
	TEXT("fx.EnableEmitterMergeChangeIdLogging"),
	GbEnableEmitterChangeIdMergeLogging,
	TEXT("If > 0 verbose change id information will be logged to help with debuggin merge issues. \n"),
	ECVF_Default
);

static int32 GNiagaraEmitterComputePSOPrecacheMode = 1;
static FAutoConsoleVariableRef CVarNiagaraEmitterComputePSOPrecacheMode(
	TEXT("fx.Niagara.Emitter.ComputePSOPrecacheMode"),
	GNiagaraEmitterComputePSOPrecacheMode,
	TEXT("Controlls how PSO precaching should be done for Niagara compute shaders\n")
	TEXT("0 = Disabled (Default).\n")
	TEXT("1 = Enabled if r.PSOPrecaching is also enabled. Emitters are not allowed to run until they complete if r.PSOPrecache.ProxyCreationStrategy is not set to AlwaysCreate\n")
	TEXT("2 = Force Enabled.\n")
	TEXT("3 = Force Enabled, emitters are not allowed to run until they complete."),
	ECVF_Scalability
);

static int32 GNiagaraEmitterMaxGPUBufferElements = 0;
static FAutoConsoleVariableRef CVarNiagaraEmitterMaxGPUBufferElements(
	TEXT("fx.Niagara.Emitter.MaxGPUBufferElements"),
	GNiagaraEmitterMaxGPUBufferElements,
	TEXT("Maximum elements per GPU buffer, for example 4k elements would restrict a float buffer to be 16k maximum per buffer.\n")
	TEXT("Note: If you request something smaller than what will satisfy a single unit of work it will be increased to that size.\n")
	TEXT("Default 0 which will allow the buffer to be the maximum allowed by the RHI.\n"),
	FConsoleVariableDelegate::CreateLambda(
		[](IConsoleVariable*)
		{
			FNiagaraSystemUpdateContext UpdateCtx;
			UpdateCtx.SetDestroyOnAdd(true);
			UpdateCtx.SetOnlyActive(true);

			for (TObjectIterator<UNiagaraSystem> It; It; ++It)
			{
				UNiagaraSystem* System = *It;
				if ( IsValid(System) )
				{
					UpdateCtx.Add(System, true);
				}
			}
		}
	),
	ECVF_Scalability
);

FNiagaraDetailsLevelScaleOverrides::FNiagaraDetailsLevelScaleOverrides()
{
	Low = 0.125f;
	Medium = 0.25f;
	High = 0.5f;
	Epic = 1.0f;
	Cine = 1.0f;
}

void FNiagaraEmitterScriptProperties::InitDataSetAccess()
{
	EventReceivers.Empty();
	EventGenerators.Empty();

	if (Script && Script->IsReadyToRun(ENiagaraSimTarget::CPUSim))
	{
		//UE_LOGF(LogNiagara, Log, "InitDataSetAccess: %ls %d %d", *Script->GetPathName(), Script->ReadDataSets.Num(), Script->WriteDataSets.Num());
		// TODO: add event receiver and generator lists to the script properties here
		//
		for (FNiagaraDataSetID &ReadID : Script->GetVMExecutableData().ReadDataSets)
		{
			EventReceivers.Add( FNiagaraEventReceiverProperties(ReadID.Name, NAME_None, NAME_None) );
		}

		for (FNiagaraDataSetProperties &WriteID : Script->GetVMExecutableData().WriteDataSets)
		{
			FNiagaraEventGeneratorProperties Props(WriteID, NAME_None);
			EventGenerators.Add(Props);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

UNiagaraEmitter::UNiagaraEmitter(const FObjectInitializer& Initializer)
: Super(Initializer)
#if WITH_EDITORONLY_DATA
, PreAllocationCount_DEPRECATED(0)
, FixedBounds_DEPRECATED(FBox(FVector(-100), FVector(100)))
, MinDetailLevel_DEPRECATED(0)
, MaxDetailLevel_DEPRECATED(4)
, bInterpolatedSpawning_DEPRECATED(false)
, bFixedBounds_DEPRECATED(false)
, bUseMinDetailLevel_DEPRECATED(false)
, bUseMaxDetailLevel_DEPRECATED(false)
, bRequiresPersistentIDs_DEPRECATED(false)
, MaxGPUParticlesSpawnPerFrame_DEPRECATED(0)
#endif
{
}

void UNiagaraEmitter::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad) == false)
	{
#if WITH_EDITORONLY_DATA
    	CheckVersionDataAvailable();
#endif
		for (FVersionedNiagaraEmitterData& Data : VersionData)
		{
			Data.PostInitProperties(this);
		}
	}

#if WITH_EDITORONLY_DATA
	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
		if (Data.GPUComputeScript)
		{
			Data.GPUComputeScript->OnGPUScriptCompiled().RemoveAll(this);
			Data.GPUComputeScript->OnGPUScriptCompiled().AddUObject(this, &UNiagaraEmitter::RaiseOnEmitterGPUCompiled);
		}
	}
#endif

	UniqueEmitterName = TEXT("Emitter");

	ResolveScalabilitySettings();
}

void FVersionedNiagaraEmitterData::CopyFrom(const FVersionedNiagaraEmitterData& Source)
{
	// copy over uproperties
    for (TFieldIterator<FProperty> PropertyIt(StaticStruct()); PropertyIt; ++PropertyIt)
    {
    	FProperty* Property = *PropertyIt;
    	const uint8* SourceAddr = Property->ContainerPtrToValuePtr<uint8>(&Source);
        uint8* DestinationAddr = Property->ContainerPtrToValuePtr<uint8>(this);

        Property->CopyCompleteValue(DestinationAddr, SourceAddr);
    }

#if STATS
	StatDatabase.Init();
#endif
}

FName GetVersionedName(const FString& BaseName, const FNiagaraAssetVersion& VersionPostfix)
{
	return FName(FString::Printf(TEXT("%s_%i_%i"), *BaseName, VersionPostfix.MajorVersion, VersionPostfix.MinorVersion));
}

void FVersionedNiagaraEmitterData::PostInitProperties(UNiagaraEmitter* Outer)
{
		SpawnScriptProps.Script = NewObject<UNiagaraScript>(Outer, GetVersionedName("SpawnScript", Version), RF_Transactional);
		SpawnScriptProps.Script->SetUsage(ENiagaraScriptUsage::ParticleSpawnScript);

		UpdateScriptProps.Script = NewObject<UNiagaraScript>(Outer, GetVersionedName("UpdateScript", Version), RF_Transactional);
		UpdateScriptProps.Script->SetUsage(ENiagaraScriptUsage::ParticleUpdateScript);

#if WITH_EDITORONLY_DATA
		EmitterSpawnScriptProps.Script = NewObject<UNiagaraScript>(Outer, GetVersionedName("EmitterSpawnScript", Version), RF_Transactional);
		EmitterSpawnScriptProps.Script->SetUsage(ENiagaraScriptUsage::EmitterSpawnScript);
		
		EmitterUpdateScriptProps.Script = NewObject<UNiagaraScript>(Outer, GetVersionedName("EmitterUpdateScript", Version), RF_Transactional);
		EmitterUpdateScriptProps.Script->SetUsage(ENiagaraScriptUsage::EmitterUpdateScript);
#endif

		GPUComputeScript = NewObject<UNiagaraScript>(Outer, GetVersionedName("GPUComputeScript", Version), RF_Transactional);
		GPUComputeScript->SetUsage(ENiagaraScriptUsage::ParticleGPUComputeScript);

#if WITH_EDITORONLY_DATA && WITH_EDITOR
	if(EditorParameters == nullptr || EditorData == nullptr)
	{
		INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");

		if(EditorParameters == nullptr)
		{
			EditorParameters = NiagaraModule.GetEditorOnlyDataUtilities().CreateDefaultEditorParameters(Outer);
		}

		if(EditorData == nullptr)
		{
			EditorData = NiagaraModule.GetEditorOnlyDataUtilities().CreateDefaultEditorData(Outer);
		}
	}	
#endif
}

bool FVersionedNiagaraEmitterData::UsesCollection(const UNiagaraParameterCollection* Collection) const
{
	if (SpawnScriptProps.Script && SpawnScriptProps.Script->UsesCollection(Collection))
	{
		return true;
	}
	if (UpdateScriptProps.Script && UpdateScriptProps.Script->UsesCollection(Collection))
	{
		return true;
	}
	for (int32 i = 0; i < EventHandlerScriptProps.Num(); i++)
	{
		if (EventHandlerScriptProps[i].Script && EventHandlerScriptProps[i].Script->UsesCollection(Collection))
		{
			return true;
		}
	}
	return false;
}

bool FVersionedNiagaraEmitterData::UsesScript(const UNiagaraScript* Script) const
{
	if (SpawnScriptProps.Script == Script || UpdateScriptProps.Script == Script)
	{
		return true;
	}
#if WITH_EDITORONLY_DATA
	if (EmitterSpawnScriptProps.Script == Script || EmitterUpdateScriptProps.Script == Script)
	{
		return true;
	}
#endif
	for (int32 i = 0; i < EventHandlerScriptProps.Num(); i++)
	{
		if (EventHandlerScriptProps[i].Script == Script)
		{
			return true;
		}
	}
	return false;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraEmitter::GetForceCompileOnLoad()
{
	return GbForceNiagaraCompileOnLoad > 0;
}

bool UNiagaraEmitter::IsSynchronizedWithParent() const
{
	for (const FVersionedNiagaraEmitterData& Data : VersionData)
	{
		if (!Data.IsSynchronizedWithParent())
		{
			return false;
		}
	}
	return true;
}

bool FVersionedNiagaraEmitterData::IsSynchronizedWithParent() const
{
	if (VersionedParent.Emitter == nullptr)
	{
		// If the emitter has no parent than it is synchronized by default.
		return true;
	}

	if (VersionedParentAtLastMerge.Emitter == nullptr)
	{
		// If the parent was valid but the parent at last merge isn't, they we don't know if it's up to date so we say it's not, and let 
		// the actual merge code print an appropriate message to the log.
		return false;
	}

	if (VersionedParent.Emitter->GetChangeId().IsValid() == false ||
		VersionedParentAtLastMerge.Emitter->GetChangeId().IsValid() == false)
	{
		// If any of the change Ids aren't valid then we assume we're out of sync.
		return false;
	}

	// Otherwise check the change ids, and the force flag.
	return VersionedParent.Emitter->GetChangeId() == VersionedParentAtLastMerge.Emitter->GetChangeId() && GbForceNiagaraMergeOnLoad <= 0;
}

TArray<INiagaraMergeManager::FMergeEmitterResults> UNiagaraEmitter::MergeChangesFromParent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MergeEmitter);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(*GetPathName(), NiagaraChannel);

	// we need to merge each version separately
	TArray<INiagaraMergeManager::FMergeEmitterResults> Results;
	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
		FVersionedNiagaraEmitter& VersionedParent = Data.VersionedParent;
		FVersionedNiagaraEmitter& VersionedParentAtLastMerge = Data.VersionedParentAtLastMerge;

		if (VersionedParent.Emitter && VersionedParent.Emitter->IsVersioningEnabled() && VersionedParent.Version.IsValid() == false)
		{
			// parent was versioned after this emitter was created, so we switch to version 1.0 of the parent.
			VersionedParent.Version = VersionedParent.Emitter->GetAllAvailableVersions()[0].VersionGuid;
		}
		if (GbEnableEmitterChangeIdMergeLogging)
		{
			UE_LOGF(LogNiagara, Log, "Emitter %ls is merging changes from parent %ls because its Change ID was updated.", *GetPathName(),
				Data.VersionedParent.Emitter != nullptr ? *Data.VersionedParent.Emitter->GetPathName() : TEXT("(null)"));

			UE_LOGF(LogNiagara, Log, "\nEmitter %ls Id=%ls \nParentAtLastMerge %ls id=%ls \nParent %ls Id=%ls.", 
				*GetPathName(), *ChangeId.ToString(),
				VersionedParentAtLastMerge.Emitter != nullptr ? *VersionedParentAtLastMerge.Emitter->GetPathName() : TEXT("(null)"), VersionedParentAtLastMerge.Emitter != nullptr ? *VersionedParentAtLastMerge.Emitter->GetChangeId().ToString() : TEXT("(null)"),
				VersionedParent.Emitter != nullptr ? *VersionedParent.Emitter->GetPathName() : TEXT("(null)"), VersionedParent.Emitter != nullptr ? *VersionedParent.Emitter->GetChangeId().ToString() : TEXT("(null)"));
		}

		if (VersionedParent.GetEmitterData() == nullptr)
		{
			// If we don't have a copy of the parent emitter, this emitter can't safely be merged.
			INiagaraMergeManager::FMergeEmitterResults MergeResults;
			MergeResults.MergeResult = INiagaraMergeManager::EMergeEmitterResult::FailedToDiff;
			MergeResults.bModifiedGraph = false;
			MergeResults.ErrorMessages.Add(NSLOCTEXT("NiagaraEmitter", "NoParentErrorMessage", "This emitter has no 'Parent' so changes can't be merged in."));
			Results.Add(MergeResults);
			continue;
		}

		INiagaraModule& NiagaraModule = FModuleManager::Get().GetModuleChecked<INiagaraModule>("Niagara");
		const INiagaraMergeManager& MergeManager = NiagaraModule.GetMergeManager();
		FVersionedNiagaraEmitter InstanceToMerge = FVersionedNiagaraEmitter(this, Data.Version.VersionGuid);
		INiagaraMergeManager::FMergeEmitterResults MergeResults = MergeManager.MergeEmitter(VersionedParent, VersionedParentAtLastMerge, InstanceToMerge);
		Results.Add(MergeResults);
		if (MergeResults.MergeResult == INiagaraMergeManager::EMergeEmitterResult::SucceededDifferencesApplied || MergeResults.MergeResult == INiagaraMergeManager::EMergeEmitterResult::SucceededNoDifferences)
		{
			if (MergeResults.MergeResult == INiagaraMergeManager::EMergeEmitterResult::SucceededDifferencesApplied)
			{
				UpdateFromMergedCopy(MergeManager, MergeResults.MergedInstance, &Data);
			}

			// Update the last merged source and clear it's stand alone and public flags since it's not an asset.
			VersionedParentAtLastMerge.Emitter = VersionedParent.Emitter->DuplicateWithoutMerging(this);
			VersionedParentAtLastMerge.Emitter->DisableVersioning(VersionedParent.Version);
			VersionedParentAtLastMerge.Version = VersionedParent.Version;
		}
		else
		{
			UE_LOGF(LogNiagara, Warning, "Failed to merge changes for parent emitter.  Emitter: %ls  Parent Emitter: %ls  Error Message: %ls",
				*GetPathName(), VersionedParent.Emitter != nullptr ? *VersionedParent.Emitter->GetPathName() : TEXT("(null)"), *MergeResults.GetErrorMessagesString());
		}

		if (MergeResults.MergeNiagaraMessage != nullptr)
		{
			MessageStore.AddMessage(EmitterMergeMessageId, CastChecked<UNiagaraMessageDataBase>(StaticDuplicateObject(MergeResults.MergeNiagaraMessage, this)));
		}
		else
		{
			MessageStore.RemoveMessage(EmitterMergeMessageId);
		}
	}
	return Results;
}

bool FVersionedNiagaraEmitterData::UsesEmitter(const UNiagaraEmitter& InEmitter) const
{
	return VersionedParent.Emitter == &InEmitter || (VersionedParent.GetEmitterData() && VersionedParent.GetEmitterData()->UsesEmitter(InEmitter));
}

void FVersionedNiagaraEmitterData::GatherStaticVariables(TArray<FNiagaraVariable>& OutVars) const
{
	TArray<UNiagaraScript*> OutScripts;
	GetScripts(OutScripts, false, true);

	for (UNiagaraScript* Script : OutScripts)
	{
		if (Script)
		{
			Script->GatherScriptStaticVariables(OutVars);
		}
	}
}

struct TParentGuard : private FNoncopyable
{
	TParentGuard(TArray<FVersionedNiagaraEmitterData>& InVersionData) : VersionData(InVersionData)
	{
		for (FVersionedNiagaraEmitterData& Data : VersionData)
		{
			OldParentValues.Add(Data.VersionedParent);
			OldParentLastMergeValues.Add(Data.VersionedParentAtLastMerge);
			Data.RemoveParent();
		}
	}
	~TParentGuard()
	{
		for (int i = 0; i < VersionData.Num(); i++)
		{
			VersionData[i].VersionedParent = OldParentValues[i];
			VersionData[i].VersionedParentAtLastMerge = OldParentLastMergeValues[i];
		}
	}

private:
	TArray<FVersionedNiagaraEmitterData>& VersionData;
	TArray<FVersionedNiagaraEmitter> OldParentValues;
	TArray<FVersionedNiagaraEmitter> OldParentLastMergeValues;
};

UNiagaraEmitter* UNiagaraEmitter::DuplicateWithoutMerging(UObject* InOuter)
{
	UNiagaraEmitter* Duplicate;
	{
		TParentGuard ParentGuard(VersionData);

		// Reset the linker to the last version to make sure PostLoad on the duplicated object is not doing any data conversion, squashing new properties with not properly initialized _DEPRECATED members.
		ResetLoaders(InOuter->GetPackage());
		FObjectDuplicationParameters DuplicationParameters(this, InOuter);
		DuplicationParameters.FlagMask = GetFlags() & ~(RF_Standalone | RF_Public);
		Duplicate = Cast<UNiagaraEmitter>(StaticDuplicateObjectEx(DuplicationParameters));
	}
	return Duplicate;
}
#endif

void UNiagaraEmitter::Serialize(FArchive& Ar)
{

#if WITH_EDITORONLY_DATA
	bool bHasTransientFlag = HasAnyFlags(RF_Transient);
	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
		for (UNiagaraSimulationStageBase* Stage : Data.SimulationStages)
		{
			if (Stage)
			{
				if (Stage->Script)
				{
					if (!bHasTransientFlag && Stage->Script->HasAnyFlags(RF_Transient))
					{
						UE_LOGF(LogNiagara, Error, "Emitter \"%ls\" has a simulation stage with a Transient script and the emitter itself isn't transient!", *GetPathName());
					}
				}
				else
				{
					UE_LOGF(LogNiagara, Error, "Emitter \"%ls\" has a simulation stage with a null Script entry!", *GetPathName());
				}

			}
			else
			{
				UE_LOGF(LogNiagara, Error, "Emitter \"%ls\" has a simulation stage with a null entry!", *GetPathName());
			}
		}
	}
	
#endif
	Super::Serialize(Ar);
	

	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
}

void FVersionedNiagaraEmitterData::EnsureScriptsPostLoaded()
{
	TArray<UNiagaraScript*> AllScripts;
	GetScripts(AllScripts, false);

	// Post load scripts for use below.
	for (UNiagaraScript* Script : AllScripts)
	{
		Script->ConditionalPostLoad();
	}

	// Additionally we want to make sure that the GPUComputeScript, if it exists, is also post loaded immediately even if we're not using it.
	// Currently an unused GPUComputeScript will cause the cached data to be invalidated and rebuilt because it will never get
	// a valid CompilerVersionID assigned to it (since it's not being compiled because it's not being used).  The side effect of this is that
	// the invalidation occurs in a non-deterministic location (based on PostLoad order) and can mess up with the cooking process
	if (GPUComputeScript)
	{
		GPUComputeScript->ConditionalPostLoad();
	}
}

void UNiagaraEmitter::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// It is unclear in what conditions we can have our PostLoad() invoked without the package having been fully serialized.  Failure to
	// have the data loaded results in the emitter (and potentially all other objects that reference it) being left in an indeterminate
	// state.  While investigation continues into hwo this happens, this call to Preload() seems to resolve the somewhat reproducible case
	// that we have.  There's some evidence that UNiagaraScript having instanced properties is leading to an edge case in object serialization
	// but more investigation is required.
	ConditionalPreload();
#endif

#if WITH_EDITORONLY_DATA
	CheckVersionDataAvailable();
#endif

	if (GIsEditor)
	{
		SetFlags(RF_Transactional);
	}

	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (NiagaraVer < FNiagaraCustomVersion::PlatformScalingRefactor)
	{
		int32 MinDetailLevel = bUseMaxDetailLevel_DEPRECATED ? MinDetailLevel_DEPRECATED : 0;
		int32 MaxDetailLevel = bUseMaxDetailLevel_DEPRECATED ? MaxDetailLevel_DEPRECATED : 4;
		int32 NewQLMask = 0;
		//Currently all detail levels were direct mappings to quality level so just transfer them over to the new mask in PlatformSet.
		for (int32 QL = MinDetailLevel; QL <= MaxDetailLevel; ++QL)
		{
			NewQLMask |= (1 << QL);
		}

		FVersionedNiagaraEmitterData* LatestEmitterData = GetLatestEmitterData();
		LatestEmitterData->Platforms = FNiagaraPlatformSet(NewQLMask);

		//Transfer spawn rate scaling overrides
		if (bOverrideGlobalSpawnCountScale_DEPRECATED)
		{
			FNiagaraEmitterScalabilityOverride& LowOverride = LatestEmitterData->ScalabilityOverrides.Overrides.AddDefaulted_GetRef();
			LowOverride.Platforms = FNiagaraPlatformSet(FNiagaraPlatformSet::CreateQualityLevelMask(0));
			LowOverride.bOverrideSpawnCountScale = true;
			LowOverride.bScaleSpawnCount = true;
			LowOverride.SpawnCountScale = GlobalSpawnCountScaleOverrides_DEPRECATED.Low;

			FNiagaraEmitterScalabilityOverride& MediumOverride = LatestEmitterData->ScalabilityOverrides.Overrides.AddDefaulted_GetRef();
			MediumOverride.Platforms = FNiagaraPlatformSet(FNiagaraPlatformSet::CreateQualityLevelMask(1));
			MediumOverride.bOverrideSpawnCountScale = true;
			MediumOverride.bScaleSpawnCount = true;
			MediumOverride.SpawnCountScale = GlobalSpawnCountScaleOverrides_DEPRECATED.Medium;

			FNiagaraEmitterScalabilityOverride& HighOverride = LatestEmitterData->ScalabilityOverrides.Overrides.AddDefaulted_GetRef();
			HighOverride.Platforms = FNiagaraPlatformSet(FNiagaraPlatformSet::CreateQualityLevelMask(2));
			HighOverride.bOverrideSpawnCountScale = true;
			HighOverride.bScaleSpawnCount = true;
			HighOverride.SpawnCountScale = GlobalSpawnCountScaleOverrides_DEPRECATED.High;

			FNiagaraEmitterScalabilityOverride& EpicOverride = LatestEmitterData->ScalabilityOverrides.Overrides.AddDefaulted_GetRef();
			EpicOverride.Platforms = FNiagaraPlatformSet(FNiagaraPlatformSet::CreateQualityLevelMask(3));
			EpicOverride.bOverrideSpawnCountScale = true;
			EpicOverride.bScaleSpawnCount = true;
			EpicOverride.SpawnCountScale = GlobalSpawnCountScaleOverrides_DEPRECATED.Epic;

			FNiagaraEmitterScalabilityOverride& CineOverride = LatestEmitterData->ScalabilityOverrides.Overrides.AddDefaulted_GetRef();
			CineOverride.Platforms = FNiagaraPlatformSet(FNiagaraPlatformSet::CreateQualityLevelMask(4));
			CineOverride.bOverrideSpawnCountScale = true;
			CineOverride.bScaleSpawnCount = true;
			CineOverride.SpawnCountScale = GlobalSpawnCountScaleOverrides_DEPRECATED.Cine;
		}
	}
	
	// this can only ever be true for old assets that haven't been loaded yet, so this won't overwrite subsequent changes to the template specification
	ENiagaraScriptTemplateSpecification CurrentTemplateSpecification = TemplateSpecification_DEPRECATED;
	if(bIsTemplateAsset_DEPRECATED)
	{
		CurrentTemplateSpecification = ENiagaraScriptTemplateSpecification::Template;
	}

	if(bExposeToLibrary_DEPRECATED)
	{
		LibraryVisibility = ENiagaraScriptLibraryVisibility::Library;
	}

	if(NiagaraVer < FNiagaraCustomVersion::InheritanceUxRefactor)
	{
		if(CurrentTemplateSpecification == ENiagaraScriptTemplateSpecification::Template || CurrentTemplateSpecification == ENiagaraScriptTemplateSpecification::Behavior)
		{
			bIsInheritable = false;
		}
	}
	
	if (MessageKeyToMessageMap_DEPRECATED.IsEmpty() == false)
	{
		MessageStore.SetMessages(MessageKeyToMessageMap_DEPRECATED);
		MessageKeyToMessageMap_DEPRECATED.Empty();
	}

	if(UPackage* Package = GetPackage())
	{
		if(Package->HasAnyPackageFlags(PKG_Cooked))
		{
			// We remove the thumbnail for cooked emitters as it can cause issues with cooked emitter being put into uncooked systems
			ThumbnailImage = nullptr;
		}
	}
#endif

#if WITH_EDITORONLY_DATA	
	if(NiagaraVer < FNiagaraCustomVersion::Type::UserAssetTagStorageInMetaData && AssetTags_DEPRECATED.Num() > 0)
	{
		INiagaraModule::Get().GetEditorOnlyDataUtilities().MoveUserAssetTagsToMetaData(this);
	}
#endif
	const int32 UE5MainVer = GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID);

	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
#if WITH_EDITORONLY_DATA
		Data.PostLoad(*this, NiagaraVer);
		Data.GPUComputeScript->OnGPUScriptCompiled().RemoveAll(this);
		Data.GPUComputeScript->OnGPUScriptCompiled().AddUObject(this, &UNiagaraEmitter::RaiseOnEmitterGPUCompiled);
		if (UE5MainVer < FUE5MainStreamObjectVersion::FixGpuAlwaysRunningUpdateScriptNoneInterpolated)
		{
			if ( Data.SimTarget == ENiagaraSimTarget::GPUComputeSim )
			{
				Data.bGpuAlwaysRunParticleUpdateScript = Data.UsesInterpolatedSpawning();
			}
			else
			{
				Data.bGpuAlwaysRunParticleUpdateScript = false;
			}
		}
#else
		Data.PostLoad(*this, NiagaraVer);
#endif
	}

#if !WITH_EDITOR
	// When running without the editor in a cooked build we run the update immediately in post load since
	// there will be no merging or compiling which makes it safe to do so.
	UpdateEmitterAfterLoad();
#endif
}

#if WITH_EDITORONLY_DATA
void UNiagaraEmitter::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraScratchPadContainer::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(TEXT("/Script/NiagaraEditor.NiagaraEditorParametersAdapter")));
}
#endif

void FVersionedNiagaraEmitterData::PostLoad(UNiagaraEmitter& Emitter, int32 NiagaraVer)
{
#if STATS
	StatDatabase.Init();
#endif

#if WITH_EDITORONLY_DATA
	//Force old data to not update Initial values to maintain existing behavior.
	//New data and anything resaved will update Initial. values with writes from Event scripts. 
	if (NiagaraVer < FNiagaraCustomVersion::EventSpawnsUpdateInitialAttributeValues)
	{
		for (FNiagaraEventScriptProperties& EventScript : EventHandlerScriptProps)
		{
			EventScript.UpdateAttributeInitialValues = false;
		}
	}

	if (NiagaraVer < FNiagaraCustomVersion::AddedNewInterpolatedSpawnMode && bInterpolatedSpawning_DEPRECATED)
	{
		InterpolatedSpawnMode = ENiagaraInterpolatedSpawnMode::Interpolation;
	}
#endif

#if WITH_EDITORONLY_DATA && WITH_EDITOR
	if (EditorParameters == nullptr)
	{
		INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");
		EditorParameters = NiagaraModule.GetEditorOnlyDataUtilities().CreateDefaultEditorParameters(&Emitter);
	}
#endif
	
	for (int32 RendererIndex = RendererProperties.Num() - 1; RendererIndex >= 0; --RendererIndex)
	{
		if (RendererProperties[RendererIndex] == nullptr)
		{
#if WITH_EDITORONLY_DATA
			//In cooked builds these can be cooked out and null on purpose.
			bool bIsCooked = Emitter.GetPackage()->bIsCookedForEditor;
			ensureMsgf(bIsCooked, TEXT("Null renderer found in %s at index %i, removing it to prevent crashes."), *Emitter.GetPathName(), RendererIndex);
#endif
			RendererProperties.RemoveAt(RendererIndex);
		}
		else
		{
			RendererProperties[RendererIndex]->OuterEmitterVersion = Version.VersionGuid;
			RendererProperties[RendererIndex]->ConditionalPostLoad();
		}
	}

	for (int32 SimulationStageIndex = SimulationStages.Num() - 1; SimulationStageIndex >= 0; --SimulationStageIndex)
	{
		if (ensureMsgf(SimulationStages[SimulationStageIndex] != nullptr && SimulationStages[SimulationStageIndex]->Script != nullptr, TEXT("Null simulation stage, or simulation stage with a null script found in %s at index %i, removing it to prevent crashes."), *Emitter.GetPathName(), SimulationStageIndex) == false)
		{
			SimulationStages.RemoveAt(SimulationStageIndex);
		}
		else
		{
#if WITH_EDITORONLY_DATA
			SimulationStages[SimulationStageIndex]->OuterEmitterVersion = Version.VersionGuid;
#endif
			SimulationStages[SimulationStageIndex]->ConditionalPostLoad();
		}
	}

	if (SpawnScriptProps.Script)
	{
		SpawnScriptProps.Script->ConditionalPostLoad();
	}
	
#if WITH_EDITORONLY_DATA
	if (EditorData != nullptr)
	{
		EditorData->ConditionalPostLoad();
		EditorData->PostLoadFromOwner(&Emitter);
	}
	else
	{
		INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");
		EditorData = NiagaraModule.GetEditorOnlyDataUtilities().CreateDefaultEditorData(&Emitter);
	}

	if (!GPUComputeScript)
	{
		GPUComputeScript = NewObject<UNiagaraScript>(&Emitter, "GPUComputeScript", RF_Transactional);
		GPUComputeScript->SetUsage(ENiagaraScriptUsage::ParticleGPUComputeScript);
		GPUComputeScript->SetLatestSource(SpawnScriptProps.Script ? SpawnScriptProps.Script->GetLatestSource() : nullptr);
	}

	if (EmitterSpawnScriptProps.Script == nullptr || EmitterUpdateScriptProps.Script == nullptr)
	{
		EmitterSpawnScriptProps.Script = NewObject<UNiagaraScript>(&Emitter, "EmitterSpawnScript", RF_Transactional);
		EmitterSpawnScriptProps.Script->SetUsage(ENiagaraScriptUsage::EmitterSpawnScript);

		EmitterUpdateScriptProps.Script = NewObject<UNiagaraScript>(&Emitter, "EmitterUpdateScript", RF_Transactional);
		EmitterUpdateScriptProps.Script->SetUsage(ENiagaraScriptUsage::EmitterUpdateScript);

		if (SpawnScriptProps.Script)
		{
			EmitterSpawnScriptProps.Script->SetLatestSource(SpawnScriptProps.Script->GetLatestSource());
			EmitterUpdateScriptProps.Script->SetLatestSource(SpawnScriptProps.Script->GetLatestSource());
		}
	}

	if (!Emitter.GetOutermost()->bIsCookedForEditor)
	{
		if (ensureMsgf(GraphSource != nullptr, TEXT("Niagara emitter %s [Flags: %x] - Version (%d.%d - %s) - Missing GraphSource"),
			*Emitter.GetPathName(), Emitter.GetFlags(),
			Version.MajorVersion, Version.MinorVersion, *Version.VersionGuid.ToString()))
		{
			GraphSource->ConditionalPostLoad();
			GraphSource->PostLoadFromEmitter(FVersionedNiagaraEmitter(&Emitter, Version.VersionGuid));
		}

		if (GNiagaraEmitterPostLoadCleanup == false)
		{
			if (GetParent().Emitter != nullptr)
			{
				GetParent().Emitter->ConditionalPostLoad();
			}
			if (GetParentAtLastMerge().Emitter != nullptr)
			{
				GetParentAtLastMerge().Emitter->ConditionalPostLoad();
			}

			for (auto ParentScratchPadScript : ParentScratchPads->Scripts)
			{
				if (ParentScratchPadScript)
				{
					ParentScratchPadScript->ConditionalPostLoad();
				}
			}
		}

		for (auto ScratchPadScript : ScratchPads->Scripts)
		{
			if (ScratchPadScript)
			{
				ScratchPadScript->ConditionalPostLoad();
			}
		}

		if (IsRunningCommandlet() && Emitter.IsSynchronizedWithParent() == false)
		{
			// Modify here so that the asset will be marked dirty when using the resave commandlet.  This will be ignored during regular post load.
			Emitter.Modify();
		}
	}
#else
	check(GPUComputeScript == nullptr || SimTarget == ENiagaraSimTarget::GPUComputeSim);
#endif

	// make sure to PostLoad any ParameterStore so that they remains searchable (sorted)
	RendererBindings.PostLoad(&Emitter);

	//Temporarily disabling interpolated spawn if the script type and flag don't match.
	if (SpawnScriptProps.Script)
	{
		const bool bActualInterpolatedSpawning = SpawnScriptProps.Script->IsInterpolatedParticleSpawnScript();
		const bool bUsesInterpolatedSpawning = InterpolatedSpawnMode != ENiagaraInterpolatedSpawnMode::NoInterpolation;
		if (bActualInterpolatedSpawning != bUsesInterpolatedSpawning)
		{
			InterpolatedSpawnMode = ENiagaraInterpolatedSpawnMode::NoInterpolation;
			if (bActualInterpolatedSpawning)
			{
#if WITH_EDITORONLY_DATA
				//clear out the script as it was compiled with interpolated spawn.
				SpawnScriptProps.Script->InvalidateCompileResults(TEXT("Interpolated spawn changed."));
#endif
				SpawnScriptProps.Script->SetUsage(ENiagaraScriptUsage::ParticleSpawnScript);
			}
			UE_LOGF(LogNiagara, Warning, "Disabling interpolated spawn because emitter flag and script type don't match. Did you adjust this value in the UI? Emitter may need recompile.. %ls", *Emitter.GetFullName());
		}
	}

	EnsureScriptsPostLoaded();

#if WITH_EDITORONLY_DATA
	// If we have a simulation stage using the old bSpawnOnly we need to modify all generic stages to the new method
	if (SimulationStages.ContainsByPredicate([](UNiagaraSimulationStageBase* Stage) { UNiagaraSimulationStageGeneric* StageAsGeneric = Cast<UNiagaraSimulationStageGeneric>(Stage); return StageAsGeneric && StageAsGeneric->bSpawnOnly_DEPRECATED; }))
	{
		for (UNiagaraSimulationStageBase* Stage : SimulationStages)
		{
			if ( UNiagaraSimulationStageGeneric* StageAsGeneric = Cast<UNiagaraSimulationStageGeneric>(Stage) )
			{
				StageAsGeneric->ExecuteBehavior = StageAsGeneric->bSpawnOnly_DEPRECATED ? ENiagaraSimStageExecuteBehavior::OnSimulationReset : ENiagaraSimStageExecuteBehavior::NotOnSimulationReset;
				StageAsGeneric->bSpawnOnly_DEPRECATED = false;
			}
		}
	}
#endif
}

bool UNiagaraEmitter::IsEditorOnly() const
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return false;
	}

	// if the emitter does not have a system as it's outer than it is likely a standalone emitter/parent emitter and so is editor only
	if (const UNiagaraSystem* SystemOwner = Cast<const UNiagaraSystem>(GetOuter()))
	{
		for (const auto& SystemEmitterHandle : SystemOwner->GetEmitterHandles())
		{
			if (SystemEmitterHandle.GetInstance().Emitter == this)
			{
				return false;
			}
		}
	}

	return true;
#else
	return Super::IsEditorOnly();
#endif
}


void UNiagaraEmitter::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UNiagaraEmitter::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITOR
	Context.AddTag(FAssetRegistryTag("VersioningEnabled", bVersioningEnabled ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));

	const FVersionedNiagaraEmitterData* AssetData = GetLatestEmitterData();
	FVersionedNiagaraEmitterData DefaultData;
	const FVersionedNiagaraEmitterData& EmitterData = AssetData ? *AssetData : DefaultData; // the CDO does not have any version data, so just use the default struct values in that case 
	Context.AddTag(FAssetRegistryTag("HasGPUEmitter", ( EmitterData.SimTarget == ENiagaraSimTarget::GPUComputeSim) ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));

	const float BoundsSize =  float(EmitterData.FixedBounds.GetSize().GetMax());
	Context.AddTag(FAssetRegistryTag("FixedBoundsSize",  EmitterData.CalculateBoundsMode == ENiagaraEmitterCalculateBoundMode::Fixed ? FString::Printf(TEXT("%.2f"), BoundsSize) : FString(TEXT("None")), FAssetRegistryTag::TT_Numerical));


	uint32 NumActiveRenderers = 0;
	TArray<const UNiagaraRendererProperties*> ActiveRenderers;

	for (const UNiagaraRendererProperties* Props :  EmitterData.GetRenderers())
	{
		if (Props)
		{
			NumActiveRenderers++;
			ActiveRenderers.Add(Props);
		}
	}

	Context.AddTag(FAssetRegistryTag("ActiveRenderers", LexToString(NumActiveRenderers), FAssetRegistryTag::TT_Numerical));

	// Gather up NumActive emitters based off of quality level.
	if (const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>())
	{
		int32 NumQualityLevels = Settings->QualityLevels.Num();
		TArray<int32> QualityLevelsNumActive;
		QualityLevelsNumActive.AddZeroed(NumQualityLevels);

		// Keeping structure from UNiagaraSystem for easy code comparison
		for (int32 i = 0; i < NumQualityLevels; i++)
		{
			if ( EmitterData.Platforms.IsEffectQualityEnabled(i))
			{
				QualityLevelsNumActive[i]++;
			}
		}

		for (int32 i = 0; i < NumQualityLevels; i++)
		{
			FString QualityLevelKey = Settings->QualityLevels[i].ToString() + TEXT("Emitters");
			Context.AddTag(FAssetRegistryTag(*QualityLevelKey, LexToString(QualityLevelsNumActive[i]), FAssetRegistryTag::TT_Numerical));
		}
	}

	TMap<FName, uint32> NumericKeys;
	TMap<FName, FString> StringKeys;
	// Gather up custom asset tags for  RendererProperties
	{
		TArray<UClass*> RendererClasses;
		GetDerivedClasses(UNiagaraRendererProperties::StaticClass(), RendererClasses);

		for (UClass* RendererClass : RendererClasses)
		{
			const UNiagaraRendererProperties* PropDefault = RendererClass->GetDefaultObject< UNiagaraRendererProperties>();
			if (PropDefault)
			{
				PropDefault->GetAssetTagsForContext(this, EmitterData.Version.VersionGuid, ActiveRenderers, NumericKeys, StringKeys);
			}
		}
	}

	// Gather up custom asset tags for DataInterfaces
	{
		TArray<const UNiagaraDataInterface*> DataInterfaces;
		auto AddDIs = [&](UNiagaraScript* Script)
		{
			if (Script)
			{
				for (FNiagaraScriptDataInterfaceCompileInfo& Info : Script->GetVMExecutableData().DataInterfaceInfo)
				{
					const UNiagaraDataInterface* DefaultDataInterface = Info.GetDefaultDataInterface();
					if (DefaultDataInterface != nullptr)
					{
						DataInterfaces.AddUnique(DefaultDataInterface);
					}
				}
			}
		};

	
		TArray<UNiagaraScript*> Scripts;
		EmitterData.GetScripts(Scripts);
		for (UNiagaraScript* Script : Scripts)
		{
			AddDIs(Script);
		}

		TArray<UClass*> DIClasses;
		GetDerivedClasses(UNiagaraDataInterface::StaticClass(), DIClasses);

		for (UClass* DIClass : DIClasses)
		{
			const UNiagaraDataInterface* PropDefault = DIClass->GetDefaultObject< UNiagaraDataInterface>();
			if (PropDefault)
			{
				PropDefault->GetAssetTagsForContext(this, EmitterData.Version.VersionGuid, DataInterfaces, NumericKeys, StringKeys);
			}
		}
		Context.AddTag(FAssetRegistryTag("ActiveDIs", LexToString(DataInterfaces.Num()), FAssetRegistryTag::TT_Numerical));
	}


	// Now propagate the custom numeric and string tags from the DataInterfaces and RendererProperties above
	auto NumericIter = NumericKeys.CreateConstIterator();
	while (NumericIter)
	{

		Context.AddTag(FAssetRegistryTag(NumericIter.Key(), LexToString(NumericIter.Value()), FAssetRegistryTag::TT_Numerical));
		++NumericIter;
	}

	auto StringIter = StringKeys.CreateConstIterator();
	while (StringIter)
	{

		Context.AddTag(FAssetRegistryTag(StringIter.Key(), LexToString(StringIter.Value()), FAssetRegistryTag::TT_Alphabetical));
		++StringIter;
	}
	
	INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");
	Context.AddTag(NiagaraModule.GetEditorOnlyDataUtilities().CreateClassUsageAssetRegistryTag(this));

#endif
	Super::GetAssetRegistryTags(Context);
}

bool UNiagaraEmitter::NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform)const
{
	// Don't load disabled emitters.
	// Awkwardly, this requires us to look for ourselves in the owning system.
	const UNiagaraSystem* OwnerSystem = GetTypedOuter<const UNiagaraSystem>();
	if ( OwnerSystem )
	{
		for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
		{
			if (EmitterHandle.GetInstance().Emitter == this)
			{
				if (!EmitterHandle.GetIsEnabled())
				{
					return false;
				}
				break;
			}
		}
	}

	if (!FNiagaraPlatformSet::ShouldPruneEmittersOnCook(TargetPlatform->IniPlatformName()))
	{
		return true;
	}

	if (OwnerSystem && !OwnerSystem->GetScalabilityPlatformSet().IsEnabledForPlatform(TargetPlatform->IniPlatformName()))
	{
		UE_LOGF(LogNiagara, Verbose, "Pruned emitter %ls for platform %ls from system scalability", *GetFullName(), *TargetPlatform->DisplayName().ToString())
		return false;
	}


	bool bIsEnabled = IsEnabledOnPlatform(TargetPlatform->IniPlatformName());
	if(!bIsEnabled)
	{
		UE_LOGF(LogNiagara, Verbose, "Pruned emitter %ls for platform %ls", *GetFullName(), *TargetPlatform->DisplayName().ToString())
	}
	return bIsEnabled;
}

// Begin UNiagaraEmitterBase Interface
#if WITH_EDITORONLY_DATA
void UNiagaraEmitter::OnUniqueEmitterNameChanged(const FString& OldName)
{
	for (FVersionedNiagaraEmitterData& EmitterData : VersionData)
	{
		EmitterData.SyncEmitterAlias(OldName, *this);
	}
}
#endif //WITH_EDITORONLY_DATA

#if WITH_EDITOR
/** Creates a new emitter with the supplied emitter as a parent emitter and the supplied system as it's owner. */
UNiagaraEmitter* UNiagaraEmitter::CreateWithParentAndOwner(FVersionedNiagaraEmitter InParentEmitter, UObject* InOwner, FName InName, EObjectFlags FlagMask)
{
	UNiagaraEmitter* NewEmitter = Cast<UNiagaraEmitter>(StaticDuplicateObject(InParentEmitter.Emitter, InOwner, InName, FlagMask));
	NewEmitter->DisableVersioning(InParentEmitter.Version);
	NewEmitter->SetUniqueEmitterName(InName.GetPlainNameString());
	FVersionedNiagaraEmitterData* EmitterData = NewEmitter->GetLatestEmitterData();
	EmitterData->VersionedParent = InParentEmitter;
	EmitterData->VersionedParentAtLastMerge.Emitter = Cast<UNiagaraEmitter>(StaticDuplicateObject(InParentEmitter.Emitter, NewEmitter));
	EmitterData->VersionedParentAtLastMerge.Emitter->ClearFlags(RF_Standalone | RF_Public);
	EmitterData->VersionedParentAtLastMerge.Emitter->DisableVersioning(InParentEmitter.Version);
	EmitterData->VersionedParentAtLastMerge.Version = InParentEmitter.Version;
	EmitterData->ParentScratchPads->AppendScripts(EmitterData->ScratchPads);
	EmitterData->GraphSource->MarkNotSynchronized(InitialNotSynchronizedReason);
	NewEmitter->RebindNotifications();
	return NewEmitter;
}

/** Creates a new emitter by duplicating an existing emitter.  The new emitter  will reference the same parent emitter if one is available. */
UNiagaraEmitter* UNiagaraEmitter::CreateAsDuplicate(const UNiagaraEmitter& InEmitterToDuplicate, FName InDuplicateName, UNiagaraSystem& InDuplicateOwnerSystem)
{
	UNiagaraEmitter* NewEmitter = Cast<UNiagaraEmitter>(StaticDuplicateObject(&InEmitterToDuplicate, &InDuplicateOwnerSystem));
	NewEmitter->ClearFlags(RF_Standalone | RF_Public);
	for (FVersionedNiagaraEmitterData& Data : NewEmitter->VersionData)
	{
		if (const FVersionedNiagaraEmitterData* OldData = InEmitterToDuplicate.GetEmitterData(Data.Version.VersionGuid))
		{
			ensure(Data.VersionedParent == OldData->VersionedParent);
			if (OldData->VersionedParentAtLastMerge.Emitter != nullptr)
			{
				Data.VersionedParentAtLastMerge.Emitter = Cast<UNiagaraEmitter>(StaticDuplicateObject(OldData->VersionedParentAtLastMerge.Emitter, NewEmitter));
				Data.VersionedParentAtLastMerge.Emitter->ClearFlags(RF_Standalone | RF_Public);
				Data.VersionedParentAtLastMerge.Emitter->DisableVersioning(OldData->VersionedParentAtLastMerge.Version);
				Data.VersionedParentAtLastMerge.Version = OldData->VersionedParentAtLastMerge.Version;
			}
		}
	}
	NewEmitter->SetUniqueEmitterName(InDuplicateName.GetPlainNameString());
	for (FVersionedNiagaraEmitterData& Data : NewEmitter->VersionData)
	{
		Data.GraphSource->MarkNotSynchronized(InitialNotSynchronizedReason);
	}
	NewEmitter->RebindNotifications();

	return NewEmitter;
}


void UNiagaraEmitter::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (IsAsset() && DuplicateMode == EDuplicateMode::Normal)
	{
		SetUniqueEmitterName(GetFName().GetPlainNameString());
	}
	
	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
#if STATS
		Data.StatDatabase.Init();
#endif
	}
}

void UNiagaraEmitter::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	if (IsAsset())
	{
		SetUniqueEmitterName(GetFName().GetPlainNameString());
	}
	UpdateStatID();
}

void UNiagaraEmitter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// post undo, some of the properties need to be resolved again, as they are reset to their default
	ResolveScalabilitySettings();
	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
#if STATS
		Data.StatDatabase.Init();
#endif
		Data.UpdateDebugName(*this, nullptr);
	}
	UpdateChangeId(TEXT("PostEditChangeProperty"));
	
	OnPropertiesChangedDelegate.Broadcast();
}

void UNiagaraEmitter::PostEditChangeVersionedProperty(FPropertyChangedEvent& PropertyChangedEvent, const FGuid& Version)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName;
	if (PropertyChangedEvent.Property)
	{
		PropertyName = PropertyChangedEvent.Property->GetFName();
	}
	FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(Version);
	if (!EmitterData)
	{
		PostEditChangeProperty(PropertyChangedEvent);
		return;
	}

	bool bNeedsRecompile = false;
	bool bRecomputeExecutionOrder = false;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, InterpolatedSpawnMode))
	{
		ENiagaraInterpolatedSpawnMode ScriptInterpolationMode = EmitterData->SpawnScriptProps.Script->GetComputedVMCompilationId().InterpolatedSpawnMode;
		ENiagaraInterpolatedSpawnMode EmitterInterpolationMode = EmitterData->InterpolatedSpawnMode;
		if (EmitterInterpolationMode != ScriptInterpolationMode)
		{
			//Recompile spawn script if we've altered the interpolated spawn property.
			EmitterData->SpawnScriptProps.Script->SetUsage(EmitterData->UsesInterpolatedSpawning() ? ENiagaraScriptUsage::ParticleSpawnScriptInterpolated : ENiagaraScriptUsage::ParticleSpawnScript);
			UE_LOGF(LogNiagara, Log, "Updating script usage: Script->IsInterpolatdSpawn %d Emitter->bInterpolatedSpawning %d", (int32)EmitterData->SpawnScriptProps.Script->IsInterpolatedParticleSpawnScript(), EmitterData->UsesInterpolatedSpawning());
			if (EmitterData->GraphSource != nullptr)
			{
				EmitterData->GraphSource->MarkNotSynchronized(TEXT("Emitter interpolated spawn changed"));
			}
			bNeedsRecompile = true;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, bGpuAlwaysRunParticleUpdateScript))
	{
		const bool bActualInterpolatedSpawning = EmitterData->SpawnScriptProps.Script->IsInterpolatedParticleSpawnScript();
		if ((bActualInterpolatedSpawning == false) && (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim))
		{
			if (EmitterData->GraphSource != nullptr)
			{
				EmitterData->GraphSource->MarkNotSynchronized(TEXT("Emitter GPU interpolated spawn changed"));
			}
			bNeedsRecompile = true;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, SimTarget))
	{
		if (EmitterData->GraphSource != nullptr)
		{
			EmitterData->GraphSource->MarkNotSynchronized(TEXT("Emitter simulation target changed."));
		}
		bNeedsRecompile = true;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, bRequiresPersistentIDs))
	{
		if (EmitterData->GraphSource != nullptr)
		{
			EmitterData->GraphSource->MarkNotSynchronized(TEXT("Emitter Requires Persistent IDs changed."));
		}
		bNeedsRecompile = true;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, bLocalSpace))
	{
		if (EmitterData->GraphSource != nullptr)
		{
			EmitterData->GraphSource->MarkNotSynchronized(TEXT("Emitter LocalSpace changed."));
		}

		bNeedsRecompile = true;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, bDeterminism))
	{
		if (EmitterData->GraphSource != nullptr)
		{
			EmitterData->GraphSource->MarkNotSynchronized(TEXT("Emitter Determinism changed."));
		}

		bNeedsRecompile = true;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraEventScriptProperties, SourceEmitterID))
	{
		bRecomputeExecutionOrder = true;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, AttributesToPreserve))
	{
		if (EmitterData->GraphSource != nullptr)
		{
			EmitterData->GraphSource->MarkNotSynchronized(TEXT("AttributesToPreserve changed."));
		}
		bNeedsRecompile = true;
	}

	ResolveScalabilitySettings();

	UpdateChangeId(TEXT("PostEditChangeProperty"));

#if WITH_EDITORONLY_DATA
	if (bNeedsRecompile)
	{
		UNiagaraSystem::RequestCompileForEmitter(FVersionedNiagaraEmitter(this, Version));
	}
	else if (bRecomputeExecutionOrder)
	{
		UNiagaraSystem::RecomputeExecutionOrderForEmitter(FVersionedNiagaraEmitter(this, Version));
	}
#endif

	// make sure to call this after the request for compilation is performed above (if necessary).  This broadcast
	// will result in emitters being reset, which could involve activation (when warmup is involved) and we may be
	// in an invalid state where a recompile is required.
	OnPropertiesChangedDelegate.Broadcast();
}

void UNiagaraEmitter::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
	UpdateEmitterAfterLoad();
}


UNiagaraEmitter::FOnPropertiesChanged& UNiagaraEmitter::OnPropertiesChanged()
{
	return OnPropertiesChangedDelegate;
}

UNiagaraEmitter::FOnSimStagesChanged& UNiagaraEmitter::OnSimStagesChanged()
{
	return OnSimStagesChangedDelegate;
}

UNiagaraEmitter::FOnSimStagesChanged& UNiagaraEmitter::OnEventHandlersChanged()
{
	return OnEventHandlersChangedDelegate;
}

void UNiagaraEmitter::HandleVariableRenamed(const FNiagaraVariable& InOldVariable, const FNiagaraVariable& InNewVariable, bool bUpdateContexts, FGuid EmitterVersion)
{
	// Rename the variable if it is in use by any renderer properties
	FVersionedNiagaraEmitterData* Data = GetEmitterData(EmitterVersion);
	for (UNiagaraRendererProperties* Prop : Data->GetRenderers())
	{
		Prop->Modify(false);
		Prop->RenameVariable(InOldVariable, InNewVariable, FVersionedNiagaraEmitterBase(this, EmitterVersion));
	}

	// Rename any simulation stage variables
	for (UNiagaraSimulationStageBase* SimStage : Data->SimulationStages)
	{
		if (UNiagaraSimulationStageGeneric* GenericStage = Cast<UNiagaraSimulationStageGeneric>(SimStage))
		{
			GenericStage->Modify(false);
			GenericStage->RenameVariable(InOldVariable, InNewVariable, FVersionedNiagaraEmitter(this, EmitterVersion));
		}
	}

	Data->RebuildRendererBindings(*this);

	if (bUpdateContexts)
	{
		FNiagaraSystemUpdateContext UpdateCtx(FVersionedNiagaraEmitterBase(this, EmitterVersion), true);
	}
}

void UNiagaraEmitter::HandleVariableRemoved(const FNiagaraVariable& InOldVariable, bool bUpdateContexts, FGuid EmitterVersion)
{
	// Reset the variable if it is in use by any renderer properties
	FVersionedNiagaraEmitterData* Data = GetEmitterData(EmitterVersion);
	for (UNiagaraRendererProperties* Prop : Data->GetRenderers())
	{
		Prop->Modify(false);
		Prop->RemoveVariable(InOldVariable, FVersionedNiagaraEmitterBase(this, EmitterVersion));
	}

	// Remove any simulation stage variables
	for (UNiagaraSimulationStageBase* SimStage : Data->SimulationStages)
	{
		if (UNiagaraSimulationStageGeneric* GenericStage = Cast<UNiagaraSimulationStageGeneric>(SimStage))
		{
			GenericStage->Modify(false);
			GenericStage->RemoveVariable(InOldVariable, FVersionedNiagaraEmitter(this, EmitterVersion));
		}
	}

	Data->RebuildRendererBindings(*this);

	if (bUpdateContexts)
	{
		FNiagaraSystemUpdateContext UpdateCtx(FVersionedNiagaraEmitterBase(this, EmitterVersion), true);
	}
}
#endif

#if WITH_EDITORONLY_DATA
TArray<UNiagaraScriptSourceBase*> UNiagaraEmitter::GetAllSourceScripts()
{
	TArray<UNiagaraScriptSourceBase*> OutScriptSources;
	TArray<UNiagaraScript*> Scripts;
	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
		Data.GetScripts(Scripts, false);
	}
	for (UNiagaraScript* Script : Scripts)
	{
		if (Script != nullptr)
		{
			UNiagaraScriptSourceBase* LatestSource = Script->GetLatestSource();
			if (LatestSource != nullptr)
			{
				OutScriptSources.Add(LatestSource);
			}
		}
	}
	return OutScriptSources;
}

FString UNiagaraEmitter::GetSourceObjectPathName() const
{
	return GetPathName();
}

TArray<UNiagaraEditorParametersAdapterBase*> UNiagaraEmitter::GetEditorOnlyParametersAdapters()
{
	return { GetLatestEmitterData()->GetEditorParameters() };
}

const TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe>& UNiagaraEmitter::GetCachedTraversalData(const FGuid& EmitterVersion) const
{
	const FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(EmitterVersion);
	if (EmitterData->CachedTraversalData.IsValid() && EmitterData->CachedTraversalData->IsValidForEmitter(EmitterData))
	{
		return EmitterData->CachedTraversalData;
	}

	INiagaraModule& NiagaraModule = FModuleManager::Get().LoadModuleChecked<INiagaraModule>(TEXT("Niagara"));
	EmitterData->CachedTraversalData = NiagaraModule.CacheGraphTraversal(this, EmitterVersion);
	return EmitterData->CachedTraversalData;
}

void UNiagaraEmitter::InvalidateCachedTraversalData(const FGuid& EmitterVersion)
{
	const FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(EmitterVersion);
	if (EmitterData->CachedTraversalData.IsValid() && EmitterData->CachedTraversalData->IsValidForEmitter(EmitterData))
	{
		EmitterData->CachedTraversalData.Reset();
	}
}

#endif // WITH_EDITORONLY_DATA

bool UNiagaraEmitter::IsEnabledOnPlatform(const FString& PlatformName)const
{
	for (const FVersionedNiagaraEmitterData& EmitterData : VersionData)
	{
		if (EmitterData.Platforms.IsEnabledForPlatform(PlatformName))
		{
			return true;
		}
	}
	return false;
}

bool FVersionedNiagaraEmitterData::IsValidInternal() const
{
	if (!SpawnScriptProps.Script || !UpdateScriptProps.Script)
	{
		return false;
	}

	if (SimTarget == ENiagaraSimTarget::CPUSim)
	{
		if (!SpawnScriptProps.Script->IsScriptCompilationPending(false) && !SpawnScriptProps.Script->DidScriptCompilationSucceed(false))
		{
			return false;
		}
		if (!UpdateScriptProps.Script->IsScriptCompilationPending(false) && !UpdateScriptProps.Script->DidScriptCompilationSucceed(false))
		{
			return false;
		}
		if (EventHandlerScriptProps.Num() != 0)
		{
			for (int32 i = 0; i < EventHandlerScriptProps.Num(); i++)
			{
				if (!EventHandlerScriptProps[i].Script->IsScriptCompilationPending(false) &&
					!EventHandlerScriptProps[i].Script->DidScriptCompilationSucceed(false))
				{
					return false;
				}
			}
		}
	}

	if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		if (!GPUComputeScript->IsScriptCompilationPending(true) && 
			!GPUComputeScript->DidScriptCompilationSucceed(true))
		{
			return false;
		}
	}
	return true;
}

bool FVersionedNiagaraEmitterData::IsReadyToRunInternal() const
{
	//Check for various failure conditions and bail.
	if (!UpdateScriptProps.Script || !SpawnScriptProps.Script)
	{
		return false;
	}

	if (SimTarget == ENiagaraSimTarget::CPUSim)
	{
		if (SpawnScriptProps.Script->IsScriptCompilationPending(false))
		{
			return false;
		}
		if (UpdateScriptProps.Script->IsScriptCompilationPending(false))
		{
			return false;
		}
		if (EventHandlerScriptProps.Num() != 0)
		{
			for (int32 i = 0; i < EventHandlerScriptProps.Num(); i++)
			{
				if (EventHandlerScriptProps[i].Script->IsScriptCompilationPending(false))
				{
					return false;
				}
			}
		}
	}

	if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		// event scripts on GPU emitters are not currently supported
		if (!EventHandlerScriptProps.IsEmpty())
		{
			return false;
		}

		if (GPUComputeScript->IsScriptCompilationPending(true))
		{
			return false;
		}
	}

	return true;
}

bool FVersionedNiagaraEmitterData::IsValid() const
{
#if WITH_EDITORONLY_DATA
	return IsValidInternal();
#else
	if (!IsValidCached.IsSet())
	{
		IsValidCached = IsValidInternal();
	}
	return IsValidCached.GetValue();
#endif
}

bool FVersionedNiagaraEmitterData::IsReadyToRun() const
{
#if WITH_EDITORONLY_DATA
	return IsReadyToRunInternal();
#else
	if (!IsReadyToRunCached.IsSet())
	{
		IsReadyToRunCached = IsReadyToRunInternal();
	}
	return IsReadyToRunCached.GetValue();
#endif
}

void FVersionedNiagaraEmitterData::GetScripts(TArray<UNiagaraScript*>& OutScripts, bool bCompilableOnly, bool bEnabledOnly) const
{
	OutScripts.Add(SpawnScriptProps.Script);
	OutScripts.Add(UpdateScriptProps.Script);
	if (!bCompilableOnly)
	{
#if WITH_EDITORONLY_DATA
		OutScripts.Add(EmitterSpawnScriptProps.Script);
		OutScripts.Add(EmitterUpdateScriptProps.Script);
#endif
	}

	for (int32 i = 0; i < EventHandlerScriptProps.Num(); i++)
	{
		if (EventHandlerScriptProps[i].Script)
		{
			OutScripts.Add(EventHandlerScriptProps[i].Script);
		}
	}

	if (!bCompilableOnly)
	{
		for (int32 i = 0; i < SimulationStages.Num(); i++)
		{
			if (SimulationStages[i] && SimulationStages[i]->Script)
			{
				if (!bEnabledOnly || SimulationStages[i]->bEnabled)
				{
					OutScripts.Add(SimulationStages[i]->Script);
				}
			}
		}
	}

	if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		OutScripts.Add(GPUComputeScript);
	}
}

UNiagaraScript* FVersionedNiagaraEmitterData::GetScript(ENiagaraScriptUsage Usage, FGuid UsageId) const
{
	TArray<UNiagaraScript*> Scripts;
	GetScripts(Scripts, false);
	for (UNiagaraScript* Script : Scripts)
	{
		if (Script->IsEquivalentUsage(Usage) && Script->GetUsageId() == UsageId)
		{
			return Script;
		}
	}
	return nullptr;
}

void FVersionedNiagaraEmitterData::CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData, const UNiagaraEmitter& Emitter, const FGuid& EmitterHandleId)
{
	bIsAllowedToExecute = true;
	bRequiresViewUniformBuffer = false;
	bNeedsPartialDepthTexture = false;

	MaxInstanceCount = 0;
	MaxAllocationCount = 0;
	BoundsCalculators.Empty();

	// Allow renderers to cache the bindings also
	for (UNiagaraRendererProperties* Renderer : RendererProperties)
	{
		Renderer->CacheFromCompiledData(CompiledData);
	}

	// Initialize bounds calculators - skip creating if we won't ever use it.  We leave the GPU sims in there with the editor so that we can
	// generate the bounds from the readback in the tool.
#if !WITH_EDITOR
	bool bUseDynamicBounds = CalculateBoundsMode == ENiagaraEmitterCalculateBoundMode::Dynamic && SimTarget == ENiagaraSimTarget::CPUSim;
	if (bUseDynamicBounds)
#endif
	{
		BoundsCalculators.Reserve(RendererProperties.Num());
		for (UNiagaraRendererProperties* Renderer : RendererProperties)
		{
			if ((Renderer != nullptr) && Renderer->GetIsEnabled())
			{
				if (FNiagaraBoundsCalculator* BoundsCalculator = Renderer->CreateBoundsCalculator(); BoundsCalculator != nullptr)
				{
					BoundsCalculator->InitAccessors(CompiledData);
					BoundsCalculators.Emplace(BoundsCalculator);
				}
			}
		}
	}

	// Cache shaders for all GPU scripts
#if WITH_EDITORONLY_DATA
	if (!UNiagaraScript::AreGpuScriptsCompiledBySystem())
	{
		if (AreAllScriptAndSourcesSynchronized())
		{
			ForEachScript(
				[](UNiagaraScript* Script)
				{
					Script->CacheResourceShadersForRendering(false, false);
				}
			);
		}
	}
#endif

	// Cache information for GPU compute sims
	CacheFromShaderCompiled();

	// Find number maximum number of instance we can support for this emitter
	if (CompiledData != nullptr)
	{
		// Prevent division by 0 in case there are no renderers.
		uint32 MaxGPUBufferComponents = 1;
		if (SimTarget == ENiagaraSimTarget::CPUSim)
		{
			// CPU emitters only upload the data needed by the renderers to the GPU. Compute the maximum number of components per particle
			// among all the enabled renderers, since this will decide how many particles we can upload.
			ForEachEnabledRenderer(
				[&](UNiagaraRendererProperties* RendererProperty)
				{
					const uint32 RendererMaxNumComponents = RendererProperty->ComputeMaxUsedComponents(CompiledData);
					MaxGPUBufferComponents = FMath::Max(MaxGPUBufferComponents, RendererMaxNumComponents);
				}
			);
		}
		else
		{
			// GPU emitters must store the entire particle payload on GPU buffers, so get the maximum component count from the dataset.
			MaxGPUBufferComponents = FMath::Max(MaxGPUBufferComponents, FMath::Max3(CompiledData->TotalFloatComponents, CompiledData->TotalInt32Components, CompiledData->TotalHalfComponents));
		}

		// See how many particles we can fit in a GPU buffer. This number can be quite small on some platforms.
		const uint64 MaxViewDimensionForTypedBuffer = GRHIGlobals.MaxViewDimensionForTypedBuffer;
		const uint64 MaxBufferElements = (GNiagaraEmitterMaxGPUBufferElements > 0) ? uint64(GNiagaraEmitterMaxGPUBufferElements) : MaxViewDimensionForTypedBuffer;
		const uint64 ClampedMaxBufferElements = (int32)FMath::Min<int64>(MaxBufferElements, TNumericLimits<int32>::Max());
		// Don't just cast the result of the division to 32-bit, since that will produce garbage if MaxNumInstances is larger than UINT_MAX. Saturate instead.
		MaxAllocationCount = (uint32)FMath::Min(ClampedMaxBufferElements / MaxGPUBufferComponents, (uint64)UINT_MAX);
		MaxInstanceCount = MaxAllocationCount;

		if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			// Round down to nearest thread group size
			MaxAllocationCount = FMath::DivideAndRoundDown(MaxAllocationCount, NiagaraComputeMaxThreadGroupSize) * NiagaraComputeMaxThreadGroupSize;

			// We can't go below a thread group size, the rest of the code does not handle this and we should only hit this when buffer size forcing is enabled
			if (MaxAllocationCount < NiagaraComputeMaxThreadGroupSize)
			{
				MaxAllocationCount = NiagaraComputeMaxThreadGroupSize;
				check(MaxAllocationCount * MaxGPUBufferComponents <= MaxViewDimensionForTypedBuffer);
			}

			// -1 because we need a scratch instance on the GPU
			MaxInstanceCount = MaxAllocationCount - 1;
		}

		if (AllocationMode == EParticleAllocationMode::FixedCount)
		{
			MaxInstanceCount = FMath::Min(MaxInstanceCount, uint32(FMath::Max(PreAllocationCount, 0)));
		}
	}
	else
	{
		MaxInstanceCount = 0;
	}
	UpdateDebugName(Emitter, CompiledData);

	SimStageExecutionData = nullptr;
	if (GPUComputeScript)
	{
		if (CompiledData)
		{
			for (FSimulationStageMetaData& SimStageMetaData : GPUComputeScript->GetVMExecutableData().SimulationStageMetaData)
			{
				if (SimStageMetaData.bParticleIterationStateEnabled)
				{
					if (const FNiagaraVariableLayoutInfo* VariableInfo = CompiledData->FindVariableLayoutInfo(FNiagaraVariableBase(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.ParticleIterationStateBinding)))
					{
						SimStageMetaData.ParticleIterationStateComponentIndex = VariableInfo->GetInt32ComponentStart();
					}
				}
			}
		}

		SimStageExecutionData = MakeShared<FNiagaraSimStageExecutionData>();
#if WITH_EDITORONLY_DATA
		SimStageExecutionData->Build(GPUComputeScript->GetVMExecutableData().SimulationStageMetaData, SimStageExecutionLoopEditorData);
		SimStageExecutionLoops = SimStageExecutionData->ExecutionLoops;
#else
		SimStageExecutionData->ExecutionLoops = SimStageExecutionLoops;
		SimStageExecutionData->SimStageMetaData = GPUComputeScript->GetVMExecutableData().SimulationStageMetaData;
#endif
	}

	// Detemine if we are allowed to execute or not
	bIsAllowedToExecute = IsAllowedByScalability();
	if (bIsAllowedToExecute && (SimTarget == ENiagaraSimTarget::GPUComputeSim))
	{
		if (const FNiagaraShaderScript* ShaderScript = GPUComputeScript ? GPUComputeScript->GetRenderThreadScript() : nullptr)
		{
			TSharedRef<FNiagaraShaderScriptParametersMetadata> ShaderParametersMetadata = ShaderScript->GetScriptParametersMetadata();
			if (ShaderParametersMetadata->ShaderParametersMetadata != nullptr)
			{
				const uint64 ScriptCBufferSize = ShaderParametersMetadata->ShaderParametersMetadata->GetLayout().ConstantBufferSize;
				const uint64 RHIMaxCBufferSize = GetMaxConstantBufferByteSize();
				if (ScriptCBufferSize > RHIMaxCBufferSize)
				{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					GEngine->AddOnScreenDebugMessage(uint64(this), 1.f, FColor::Red, *FString::Printf(TEXT("GPU Simulation(%s) is disabled due to using too much constant buffer space (%" UINT64_FMT "/%" UINT64_FMT ")."), GetDebugSimName(), ScriptCBufferSize, RHIMaxCBufferSize));
#endif
					bIsAllowedToExecute = false;
				}
			}
			else
			{
				bIsAllowedToExecute = false;
			}
		}
		else
		{
			bIsAllowedToExecute = false;
		}
	}

	bIsAllowedToExecute &= FNiagaraComponentSettings::IsEmitterAllowedToRun(*this, Emitter, EmitterHandleId);
}

void FVersionedNiagaraEmitterData::UpdateDebugName(const UNiagaraEmitter& Emitter, const FNiagaraDataSetCompiledData* CompiledData)
{
#if WITH_NIAGARA_DEBUG_EMITTER_NAME
	// Ensure our debug simulation name is up to date
	DebugSimName.Empty();
	if (const UNiagaraSystem* SystemOwner = Emitter.GetTypedOuter<class UNiagaraSystem>())
	{
		DebugSimName = SystemOwner->GetName();
		DebugSimName.AppendChar(':');
	}
#if WITH_EDITORONLY_DATA
	DebugSimName.Append(Emitter.GetName());
	if (Emitter.IsVersioningEnabled())
	{
		DebugSimName.AppendChar(':');
		DebugSimName.AppendInt(Version.MajorVersion);
		DebugSimName.AppendChar('.');
		DebugSimName.AppendInt(Version.MinorVersion);
	}
#endif
#endif

	RebuildRendererBindings(Emitter);
}

bool FVersionedNiagaraEmitterData::BuildParameterStoreRendererBindings(FNiagaraParameterStore& ParameterStore) const
{
	bool bAnyBindingsAdded = false;
	for (UNiagaraRendererProperties* Props : GetRenderers())
	{
		if (Props && Props->bIsEnabled)
		{
			bAnyBindingsAdded |= Props->PopulateRequiredBindings(ParameterStore);
		}
	}

	if (GPUComputeScript)
	{
		for (const FSimulationStageMetaData& SimStageMetaData : GPUComputeScript->GetSimulationStageMetaData())
		{
			if (!SimStageMetaData.EnabledBinding.IsNone())
			{
				bAnyBindingsAdded |= ParameterStore.AddParameter(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), SimStageMetaData.EnabledBinding), false);
			}
			if (SimStageMetaData.IterationSourceType == ENiagaraIterationSource::DirectSet)
			{
				if (!SimStageMetaData.ElementCountXBinding.IsNone())
				{
					bAnyBindingsAdded |= ParameterStore.AddParameter(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.ElementCountXBinding), false);
				}
				if (!SimStageMetaData.ElementCountYBinding.IsNone())
				{
					bAnyBindingsAdded |= ParameterStore.AddParameter(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.ElementCountYBinding), false);
				}
				if (!SimStageMetaData.ElementCountZBinding.IsNone())
				{
					bAnyBindingsAdded |= ParameterStore.AddParameter(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.ElementCountZBinding), false);
				}
			}
			if (!SimStageMetaData.NumIterationsBinding.IsNone())
			{
				bAnyBindingsAdded |= ParameterStore.AddParameter(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.NumIterationsBinding), false);
			}
		}
		for ( const FNiagaraSimStageExecutionLoopData& LoopData : SimStageExecutionLoops)
		{
			if (!LoopData.NumLoopsBinding.IsNone())
			{
				bAnyBindingsAdded |= ParameterStore.AddParameter(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), LoopData.NumLoopsBinding), false);
			}
		}
	}
	return bAnyBindingsAdded;
}

void FVersionedNiagaraEmitterData::RebuildRendererBindings(const UNiagaraEmitter& Emitter)
{
#if WITH_EDITORONLY_DATA
	// For right now we only capture the values of the static variables here. So we create a temp
	// parameter store for ALL possible values, then fill in the real one with the static variables only.
	FNiagaraParameterStore TempStore;
	const bool bAnyRendererBindingsAdded = BuildParameterStoreRendererBindings(TempStore);

	RendererBindings.Empty();
	RendererBindingsExternalObjects.Empty();

	if (bAnyRendererBindingsAdded)
	{
		TArray<TPair<UNiagaraScript*,const FNiagaraScriptRuntimeData*>> TargetScriptPairs;
		const FNiagaraEmitterHandle* EmitterHandle = nullptr;
		UNiagaraSystem* SystemOwner = Emitter.GetTypedOuter<UNiagaraSystem>();
		if (SystemOwner != nullptr)
		{
			TargetScriptPairs.Add(TPair<UNiagaraScript*,const FNiagaraScriptRuntimeData*>(
				SystemOwner->GetSystemSpawnScript(), SystemOwner->GetScriptRuntimeData(nullptr, *SystemOwner->GetSystemSpawnScript()).Get()));
			TargetScriptPairs.Add(TPair<UNiagaraScript*, const FNiagaraScriptRuntimeData*>(
				SystemOwner->GetSystemUpdateScript(), SystemOwner->GetScriptRuntimeData(nullptr, *SystemOwner->GetSystemUpdateScript()).Get()));
			EmitterHandle = SystemOwner->GetEmitterHandles().FindByPredicate([this]
				(const FNiagaraEmitterHandle& EmitterHandle) { return EmitterHandle.GetInstance().GetEmitterData() == this; });
		}

		if (EmitterHandle != nullptr)
		{
			TArray<UNiagaraScript*> EmitterScripts;
			GetScripts(EmitterScripts, false, true);
			for (UNiagaraScript* EmitterScript : EmitterScripts)
			{
				TargetScriptPairs.Add(TPair<UNiagaraScript*, const FNiagaraScriptRuntimeData*>(EmitterScript, SystemOwner->GetScriptRuntimeData(EmitterHandle, *EmitterScript).Get()));
			}
		}

		TArrayView<const FNiagaraVariableWithOffset> Vars = TempStore.ReadParameterVariables();
		for (const FNiagaraVariableWithOffset& Var : Vars)
		{
			for (TPair<UNiagaraScript*, const FNiagaraScriptRuntimeData*>& TargetScriptPair : TargetScriptPairs)
			{
				UNiagaraScript* Script = TargetScriptPair.Key;
				if (Script == nullptr)
				{
					continue;
				}
				bool bVariableFound = false;

				// Find Resolved UObjects
				// Note: Most parameters are pushed in as part of the DataSet -> Parameters process
				if (Var.GetType().IsUObject() && !Var.GetType().IsDataInterface())
				{
					const FNiagaraScriptExecutionParameterStore* ScriptParameterStore = TargetScriptPair.Value != nullptr
						? TargetScriptPair.Value->GetExecutionReadyParameterStore() : nullptr;
					if (ScriptParameterStore != nullptr)
					{
						if (ScriptParameterStore->IndexOf(Var) != INDEX_NONE)
						{
							int32 ParameterOffset = INDEX_NONE;
							RendererBindings.AddParameter(Var, true, false, &ParameterOffset);
							bVariableFound = true;
						}
					}
				}
				// Find static variables
				else
				{
					for (const FNiagaraVariable& StaticVar : Script->GetVMExecutableData().StaticVariablesWritten)
					{
						if (StaticVar.GetType().IsSameBaseDefinition(Var.GetType()) && StaticVar.GetName() == Var.GetName())
						{
							RendererBindings.AddParameter(StaticVar, true, false);
							bVariableFound = true;
							break;
						}
					}
				}

				if (bVariableFound)
				{
					break;
				}
			}
		}

		if (RendererBindings.GetUObjects().Num() > 0)
		{
			for (TPair<UNiagaraScript*, const FNiagaraScriptRuntimeData*>& TargetScriptPair : TargetScriptPairs)
			{
				const UNiagaraScript* Script = TargetScriptPair.Key;
				for (const FNiagaraScriptUObjectCompileInfo& DefaultInfo : Script->GetCachedDefaultUObjects())
				{
					const FNiagaraVariableBase ReadVariable(DefaultInfo.Variable.GetType(), DefaultInfo.RegisteredParameterMapRead);
					const bool bIsUserVariable = ReadVariable.IsInNameSpace(FNiagaraConstants::UserNamespaceString);
					if (!bIsUserVariable)
					{
						continue;
					}
					for (const FName WriteName : DefaultInfo.RegisteredParameterMapWrites)
					{
						const FNiagaraVariableBase WriteVariable(DefaultInfo.Variable.GetType(), WriteName);
						if (RendererBindings.IndexOf(WriteVariable) != INDEX_NONE)
						{
							FNiagaraExternalUObjectInfo& ExternalInfo = RendererBindingsExternalObjects.AddDefaulted_GetRef();
							ExternalInfo.Variable = WriteVariable;
							ExternalInfo.ExternalName = ReadVariable.GetName();
						}
					}
				}
			}
		}

		RendererBindings.TriggerOnLayoutChanged();
	}
#endif
}

void FVersionedNiagaraEmitterData::CacheFromShaderCompiled()
{
	bRequiresViewUniformBuffer = false;
	bNeedsPartialDepthTexture = false;
	if (GPUComputeScript && (SimTarget == ENiagaraSimTarget::GPUComputeSim))
	{
		if (const FNiagaraShaderScript* NiagaraShaderScript = GPUComputeScript->GetRenderThreadScript())
		{
			for (int i=0; i < NiagaraShaderScript->GetNumPermutations(); ++i)
			{
				FNiagaraShaderRef NiagaraShaderRef = NiagaraShaderScript->GetShaderGameThread(i);
				if (NiagaraShaderRef.IsValid())
				{
					bRequiresViewUniformBuffer |= NiagaraShaderRef->bNeedsViewUniformBuffer;
					bNeedsPartialDepthTexture |= (NiagaraShaderRef->MiscUsageBitMask & uint16(ENiagaraScriptMiscUsageMask::UsesPartialDepthCollisionQuery)) != 0;
				}
			}
		}
	}
}

bool FVersionedNiagaraEmitterData::UsesInterpolatedSpawning() const
{
	return InterpolatedSpawnMode != ENiagaraInterpolatedSpawnMode::NoInterpolation;
}

void UNiagaraEmitter::UpdateEmitterAfterLoad()
{
	if (bFullyLoaded)
	{
		return;
	}
	bFullyLoaded = true;
	LLM_SCOPE(ELLMTag::Niagara);
	
#if WITH_EDITORONLY_DATA
	check(IsInGameThread());

	// We remove emitters and scripts on dedicated servers (and platforms which don't use AV data), so skip further work.
	const bool bIsDedicatedServer = !GIsClient && GIsServer;
	const bool bTargetRequiresAvData = WillNeedAudioVisualData();
	if (bIsDedicatedServer || !bTargetRequiresAvData)
	{
		return;
	}

	// Synchronize with definitions before merging.
	PostLoadDefinitionsSubscriptions(
		MakeWeakObjectPtr(this),
		[](UObject* ObjectOwner) -> TArray<INiagaraParameterDefinitionsSubscriber*>
		{
			TArray<INiagaraParameterDefinitionsSubscriber*> OutArray;
			if (UNiagaraEmitter* Owner = Cast<UNiagaraEmitter>(ObjectOwner))
			{
				OutArray.Add(Owner);
			}
			return OutArray;
		}
	);

	// If we're owned by another emitter then we're a VersionedParentAtLastMerge
	if (GetOuter()->IsA<UNiagaraEmitter>())
	{
		ensure(VersionData.Num() == 1);
		
		// If this emitter is owned by another emitter, remove it's inheritance information so that it doesn't try to merge changes.
		VersionData[0].VersionedParent = FVersionedNiagaraEmitter();
		VersionData[0].VersionedParentAtLastMerge = FVersionedNiagaraEmitter();
	}

	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
		if (Data.VersionedParent.Emitter != nullptr)
		{
			Data.VersionedParent.Emitter->UpdateEmitterAfterLoad();
		}
		if (Data.VersionedParentAtLastMerge.Emitter != nullptr)
		{
			Data.VersionedParentAtLastMerge.Emitter->UpdateEmitterAfterLoad();
		}
	}
	
	if (!GetOutermost()->bIsCookedForEditor)
	{
		if (IsSynchronizedWithParent() == false)
		{
			bool bIsPackageDirty = GetOutermost()->IsDirty();
			MergeChangesFromParent();
			if (bIsPackageDirty == false)
			{
				// we do not want to dirty the system from the merge on load
				GetOutermost()->SetDirtyFlag(false);
			}
		}

		// Reset scripts if recompile is forced.
		bool bGenerateNewChangeId = false;
		FString GenerateNewChangeIdReason;
		if (GetForceCompileOnLoad())
		{
			// If we are a standalone emitter, then we invalidate id's, which should cause systems dependent on us to regenerate.
			UObject* OuterObj = GetOuter();
			if (OuterObj == GetOutermost())
			{
				for (FVersionedNiagaraEmitterData& Data : VersionData)
				{
					Data.GraphSource->ForceGraphToRecompileOnNextCheck();
				}
				bGenerateNewChangeId = true;
				GenerateNewChangeIdReason = TEXT("PostLoad - Force compile on load");
				if (GEnableVerboseNiagaraChangeIdLogging)
				{
					UE_LOGF(LogNiagara, Log, "InvalidateCachedCompileIds for %ls because GbForceNiagaraCompileOnLoad = %d", *GetPathName(), GbForceNiagaraCompileOnLoad);
				}
			}
		}
	
		if (ChangeId.IsValid() == false)
		{
			// If the change id is already invalid we need to generate a new one, and can skip checking the owned scripts.
			bGenerateNewChangeId = true;
			GenerateNewChangeIdReason = TEXT("PostLoad - Change id was invalid.");
			if (GEnableVerboseNiagaraChangeIdLogging)
			{
				UE_LOGF(LogNiagara, Log, "Change ID updated for emitter %ls because the ID was invalid.", *GetPathName());
			}
		}

		if (bGenerateNewChangeId)
		{
			UpdateChangeId(GenerateNewChangeIdReason);
		}

		RebindNotifications();
	}
#endif

	ResolveScalabilitySettings();

	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
		Data.UpdateDebugName(*this, nullptr);
	}
}

const FNiagaraEmitterScalabilityOverride& FVersionedNiagaraEmitterData::GetCurrentOverrideSettings() const
{
	for (const FNiagaraEmitterScalabilityOverride& Override : ScalabilityOverrides.Overrides)
	{
		if (Override.Platforms.IsActive())
		{
			return Override;
		}
	}	

	static FNiagaraEmitterScalabilityOverride Dummy;
	return Dummy;
}

bool FVersionedNiagaraEmitterData::IsAllowedByScalability() const
{
	return Platforms.IsActive();
}

bool FVersionedNiagaraEmitterData::RequiresPersistentIDs() const
{
	return bRequiresPersistentIDs;
}

#if WITH_EDITORONLY_DATA

FGuid UNiagaraEmitter::GetChangeId() const
{
	return ChangeId;
}

UNiagaraEditorDataBase* FVersionedNiagaraEmitterData::GetEditorData() const
{
	return EditorData;
}

UNiagaraEditorParametersAdapterBase* FVersionedNiagaraEmitterData::GetEditorParameters()
{
	return EditorParameters;
}

void UNiagaraEmitter::SetEditorData(UNiagaraEditorDataBase* InEditorData, const FGuid& VersionGuid)
{
	FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(VersionGuid);
	if (EmitterData == nullptr)
	{
		return;
	}
	if (EmitterData->EditorData == InEditorData)
	{
		return;
	}
	if (EmitterData->EditorData != nullptr)
	{
		EmitterData->EditorData->OnPersistentDataChanged().RemoveAll(this);
	}

	EmitterData->EditorData = InEditorData;
	
	if (EmitterData->EditorData != nullptr)
	{
		EmitterData->EditorData->OnPersistentDataChanged().AddUObject(this, &UNiagaraEmitter::PersistentEditorDataChanged);
	}
}

bool FVersionedNiagaraEmitterData::AreAllScriptAndSourcesSynchronized() const
{
	if (SpawnScriptProps.Script->IsCompilable() && !SpawnScriptProps.Script->AreScriptAndSourceSynchronized())
	{
		return false;
	}

	if (UpdateScriptProps.Script->IsCompilable() && !UpdateScriptProps.Script->AreScriptAndSourceSynchronized())
	{
		return false;
	}

	if (EmitterSpawnScriptProps.Script->IsCompilable() && !EmitterSpawnScriptProps.Script->AreScriptAndSourceSynchronized())
	{
		return false;
	}

	if (EmitterUpdateScriptProps.Script->IsCompilable() && !EmitterUpdateScriptProps.Script->AreScriptAndSourceSynchronized())
	{
		return false;
	}

	for (int32 i = 0; i < EventHandlerScriptProps.Num(); i++)
	{
		if (EventHandlerScriptProps[i].Script && EventHandlerScriptProps[i].Script->IsCompilable() && !EventHandlerScriptProps[i].Script->AreScriptAndSourceSynchronized())
		{
			return false;
		}
	}

	for (int32 i = 0; i < SimulationStages.Num(); i++)
	{
		if (SimulationStages[i] && SimulationStages[i]->Script  && SimulationStages[i]->Script->IsCompilable() && SimulationStages[i]->bEnabled && !SimulationStages[i]->Script->AreScriptAndSourceSynchronized())
		{
			return false;
		}
	}

	if (SimTarget == ENiagaraSimTarget::GPUComputeSim && GPUComputeScript->IsCompilable())
	{
		if (!GPUComputeScript->AreScriptAndSourceSynchronized())
		{
			return false;
		}

		if (UNiagaraScript::AreGpuScriptsCompiledBySystem())
		{
			// need to also check if the shader resource is available for the GPUScript
			if (!GPUComputeScript->IsScriptShaderSynchronized())
			{
				return false;
			}
		}
	}

	return true;
}


UNiagaraEmitter::FOnEmitterCompiled& UNiagaraEmitter::OnEmitterVMCompiled()
{
	return OnVMScriptCompiledDelegate;
}

UNiagaraEmitter::FOnEmitterCompiled& UNiagaraEmitter::OnEmitterGPUCompiled()
{
	return OnGPUScriptCompiledDelegate;
}

void FVersionedNiagaraEmitterData::InvalidateCompileResults()
{
	TArray<UNiagaraScript*> Scripts;
	GetScripts(Scripts, false);
	for (int32 i = 0; i < Scripts.Num(); i++)
	{
		Scripts[i]->InvalidateCompileResults(TEXT("Emitter compile invalidated."));
	}
}

void UNiagaraEmitter::OnPostCompile(const FGuid& EmitterVersion)
{
	FVersionedNiagaraEmitterData* Data = GetEmitterData(EmitterVersion);
	if (Data == nullptr)
	{
		return;
	}
	Data->OnPostCompile(*this);

	OnEmitterVMCompiled().Broadcast(FVersionedNiagaraEmitter(this, EmitterVersion));
}

void FVersionedNiagaraEmitterData::OnPostCompile(const UNiagaraEmitter& InEmitter)
{
	SyncEmitterAlias(TEXT("Emitter"), InEmitter);

	SpawnScriptProps.InitDataSetAccess();
	UpdateScriptProps.InitDataSetAccess();

	TSet<FName> SpawnIds;
	TSet<FName> UpdateIds;
	for (const FNiagaraEventGeneratorProperties& SpawnGeneratorProps : SpawnScriptProps.EventGenerators)
	{
		SpawnIds.Add(SpawnGeneratorProps.ID);
	}
	for (const FNiagaraEventGeneratorProperties& UpdateGeneratorProps : UpdateScriptProps.EventGenerators)
	{
		UpdateIds.Add(UpdateGeneratorProps.ID);
	}

	SharedEventGeneratorIds.Empty();
	SharedEventGeneratorIds.Append(SpawnIds.Intersect(UpdateIds).Array());

	for (int32 i = 0; i < EventHandlerScriptProps.Num(); i++)
	{
		if (EventHandlerScriptProps[i].Script)
		{
			EventHandlerScriptProps[i].InitDataSetAccess();
		}
	}

	if (GbForceNiagaraFailToCompile != 0)
	{
		TArray<UNiagaraScript*> Scripts;
		GetScripts(Scripts, false);
		for (int32 i = 0; i < Scripts.Num(); i++)
		{
			Scripts[i]->InvalidateCompileResults(TEXT("Console variable forced recompile.")); 
		}
	}

	// If we have a GPU script but the SimTarget isn't GPU, we need to clear out the old results.
	if (SimTarget != ENiagaraSimTarget::GPUComputeSim && GPUComputeScript->GetLastCompileStatus() != ENiagaraScriptCompileStatus::NCS_Unknown)
	{
		GPUComputeScript->InvalidateCompileResults(TEXT("Not a GPU emitter."));
	}

	RuntimeEstimation.RuntimeAllocations.Empty();
	RuntimeEstimation.AllocationEstimate = 0;
	RuntimeEstimation.SetIsEstimationDirty(false);
#if STATS
	StatDatabase.Init();
	StatDatabase.ClearStatCaptures();
#endif
}

void UNiagaraEmitter::RebindNotifications()
{
	for (FVersionedNiagaraEmitterData& EmitterData : VersionData)
	{
		if (EmitterData.GraphSource)
		{
			EmitterData.GraphSource->OnChanged().RemoveAll(this);
			EmitterData.GraphSource->OnChanged().AddUObject(this, &UNiagaraEmitter::GraphSourceChanged);
		}

		if (EmitterData.EmitterSpawnScriptProps.Script)
		{
			FNiagaraParameterStore& Store = EmitterData.EmitterSpawnScriptProps.Script->RapidIterationParameters;
			Store.RemoveAllOnChangedHandlers(this);
			Store.AddOnChangedHandler(FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));
		}

		if (EmitterData.EmitterUpdateScriptProps.Script)
		{
			FNiagaraParameterStore& Store = EmitterData.EmitterUpdateScriptProps.Script->RapidIterationParameters;
			Store.RemoveAllOnChangedHandlers(this);
			Store.AddOnChangedHandler(FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));
		}

		if (EmitterData.SpawnScriptProps.Script)
		{
			FNiagaraParameterStore& Store = EmitterData.SpawnScriptProps.Script->RapidIterationParameters;
			Store.RemoveAllOnChangedHandlers(this);
			Store.AddOnChangedHandler(FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));
		}

		if (EmitterData.UpdateScriptProps.Script)
		{
			FNiagaraParameterStore& Store = EmitterData.UpdateScriptProps.Script->RapidIterationParameters;
			Store.RemoveAllOnChangedHandlers(this);
			Store.AddOnChangedHandler(FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));
		}

		for (FNiagaraEventScriptProperties& EventScriptProperties : EmitterData.EventHandlerScriptProps)
		{
			if (EventScriptProperties.Script)
			{
				FNiagaraParameterStore& Store = EventScriptProperties.Script->RapidIterationParameters;
				Store.RemoveAllOnChangedHandlers(this);
				Store.AddOnChangedHandler(FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));
			}
		}

		for (UNiagaraSimulationStageBase* SimulationStage : EmitterData.SimulationStages)
		{
			if (SimulationStage)
			{
				SimulationStage->OuterEmitterVersion = EmitterData.Version.VersionGuid;
				SimulationStage->OnChanged().RemoveAll(this);
				SimulationStage->OnChanged().AddUObject(this, &UNiagaraEmitter::SimulationStageChanged);

				if (SimulationStage->Script)
				{
					FNiagaraParameterStore& Store = SimulationStage->Script->RapidIterationParameters;
					Store.RemoveAllOnChangedHandlers(this);
					Store.AddOnChangedHandler(FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));
				}
			}
		}

		for (UNiagaraRendererProperties* Renderer : EmitterData.RendererProperties)
		{
			if (Renderer)
			{
				Renderer->OuterEmitterVersion = EmitterData.Version.VersionGuid;
				Renderer->OnChanged().RemoveAll(this);
				Renderer->OnChanged().AddUObject(this, &UNiagaraEmitter::RendererChanged);
			}
		}
		
		if (EmitterData.EditorData != nullptr)
		{
			EmitterData.EditorData->OnPersistentDataChanged().RemoveAll(this);
			EmitterData.EditorData->OnPersistentDataChanged().AddUObject(this, &UNiagaraEmitter::PersistentEditorDataChanged);
		}
	}
}

#endif


void FVersionedNiagaraEmitterData::GatherCompiledParticleAttributes(TArray<FNiagaraVariableBase>& OutVariables) const
{
	if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		OutVariables = GetGPUComputeScript()->GetVMExecutableData().Attributes;
	}
	else
	{
		OutVariables = UpdateScriptProps.Script->GetVMExecutableData().Attributes;

		for (const FNiagaraVariableBase& Var : SpawnScriptProps.Script->GetVMExecutableData().Attributes)
		{
			OutVariables.AddUnique(Var);
		}
	}

}

FNiagaraVariableBase UNiagaraEmitter::GetResolvedDIBinding(const FNiagaraVariableBase& InVariable) const
{
	const FNiagaraVariableBase* ResolvedDIBinding = GetLatestEmitterData()->ResolvedDIBindings.Find(InVariable);
	return ResolvedDIBinding ? *ResolvedDIBinding : InVariable;
}

bool UNiagaraEmitter::CanObtainParticleAttribute(const FNiagaraVariableBase& InVar, const FGuid& EmitterVersion, FNiagaraTypeDefinition& OutBoundType) const
{
	check(!HasAnyFlags(RF_NeedPostLoad));

	const FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(EmitterVersion);
	if (EmitterData && EmitterData->SpawnScriptProps.Script)
	{
		// make sure that this isn't called before our dependents are fully loaded
		const FNiagaraEmitterScriptProperties& SpawnScriptProps = EmitterData->SpawnScriptProps;
		check(!SpawnScriptProps.Script->HasAnyFlags(RF_NeedPostLoad));

		bool bContainsAttribute = SpawnScriptProps.Script->GetVMExecutableData().Attributes.ContainsByPredicate(FNiagaraVariableMatch(InVar.GetType(), InVar.GetName()));
		if (!bContainsAttribute && InVar.GetType() == FNiagaraTypeDefinition::GetPositionDef())
		{
			// if we don't find a position type var we check for a vec3 type for backwards compatibility
			OutBoundType = FNiagaraTypeDefinition::GetVec3Def();
			bContainsAttribute = SpawnScriptProps.Script->GetVMExecutableData().Attributes.ContainsByPredicate(FNiagaraVariableMatch(OutBoundType, InVar.GetName()));
		}
		return bContainsAttribute;
	}
	return false;
}


bool UNiagaraEmitter::CanObtainEmitterAttribute(const FNiagaraVariableBase& InVarWithUniqueNameNamespace, FNiagaraTypeDefinition& OutBoundType) const
{
	check(!HasAnyFlags(RF_NeedPostLoad));

	if (const UNiagaraSystem* Sys = GetTypedOuter<UNiagaraSystem>())
	{
		// make sure that this isn't called before our dependents are fully loaded
		check(!Sys->HasAnyFlags(RF_NeedPostLoad));

		return Sys->CanObtainEmitterAttribute(InVarWithUniqueNameNamespace, OutBoundType);
	}
	return false;
}

FVersionedNiagaraEmitterData* UNiagaraEmitter::GetLatestEmitterData()
{
	if (VersionData.Num() == 0)
	{
		return nullptr;
	}
	if (!bVersioningEnabled)
	{
		return &VersionData[0];
	}
	FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(ExposedVersion);
	ensureMsgf(EmitterData, TEXT("Invalid exposed version for Niagara emitter %s, asset might be corrupted!"), *GetPathName());
	return EmitterData;
}

const FVersionedNiagaraEmitterData* UNiagaraEmitter::GetLatestEmitterData() const
{
	return const_cast<UNiagaraEmitter*>(this)->GetLatestEmitterData();
}

const FVersionedNiagaraEmitterData* UNiagaraEmitter::GetEmitterData(const FGuid& VersionGuid) const
{
	return const_cast<UNiagaraEmitter*>(this)->GetEmitterData(VersionGuid);
}

FVersionedNiagaraEmitterData* UNiagaraEmitter::GetEmitterData(const FGuid& VersionGuid)
{
	if (VersionData.Num() == 0)
	{
		return nullptr;
	}
	
	// check if we even need to support different versions
	if (!bVersioningEnabled)
	{
		return &VersionData[0];
	}

	if (!VersionGuid.IsValid())
	{
		for (FVersionedNiagaraEmitterData& Data : VersionData)
		{
			if (Data.Version.VersionGuid == ExposedVersion)
			{
				return &Data;
			}
		}
		ensureMsgf(false, TEXT("Invalid exposed version for Niagara emitter %s, asset might be corrupted!"), *GetPathName());
		return nullptr;
	}
	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
		if (Data.Version.VersionGuid == VersionGuid)
		{
			return &Data;
		}
	}
	return nullptr;
}

TArray<FNiagaraAssetVersion> UNiagaraEmitter::GetAllAvailableVersions() const
{
	TArray<FNiagaraAssetVersion> Versions;
	for (const FVersionedNiagaraEmitterData& Data : VersionData)
	{
		Versions.Add(Data.Version);
	}
	return Versions;
}

#if WITH_EDITORONLY_DATA

struct FNiagaraEmitterVersionDataAccessor final : FNiagaraVersionDataAccessor
{
	virtual ~FNiagaraEmitterVersionDataAccessor() override = default;
	explicit FNiagaraEmitterVersionDataAccessor(FVersionedNiagaraEmitterData* InEmitterData) : EmitterData(InEmitterData) {}

	virtual FNiagaraAssetVersion& GetObjectVersion() override { return EmitterData->Version; }
	virtual FText& GetVersionChangeDescription() override { return EmitterData->VersionChangeDescription; }
	virtual FText& GetDeprecationMessage() override { return EmitterData->DeprecationMessage; }
	virtual bool& IsDeprecated() override { return EmitterData->bDeprecated; }
	virtual ENiagaraPythonUpdateScriptReference& GetUpdateScriptExecutionType() override { return EmitterData->UpdateScriptExecution; }
	virtual FString& GetPythonUpdateScript() override { return EmitterData->PythonUpdateScript; }
	virtual FFilePath& GetScriptAsset() override { return EmitterData->ScriptAsset; }
	virtual TSubclassOf<UNiagaraVersionUpgradeBase>& GetUpdateUtilityClass() override { return EmitterData->UpdateUtilityClass; }
	
	FVersionedNiagaraEmitterData* EmitterData;
};

TSharedPtr<FNiagaraVersionDataAccessor> UNiagaraEmitter::GetVersionDataAccessor(const FGuid& Version)
{
	if (FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(Version))
	{
		return MakeShared<FNiagaraEmitterVersionDataAccessor>(EmitterData);
	}
	return TSharedPtr<FNiagaraVersionDataAccessor>();
}

FNiagaraAssetVersion UNiagaraEmitter::GetExposedVersion() const
{
	const FVersionedNiagaraEmitterData* EmitterData = GetLatestEmitterData();
	return EmitterData ? EmitterData->Version : FNiagaraAssetVersion();
}

FNiagaraAssetVersion const* UNiagaraEmitter::FindVersionData(const FGuid& VersionGuid) const
{
	for (const FVersionedNiagaraEmitterData& Data : VersionData)
	{
		if (Data.Version.VersionGuid == VersionGuid)
		{
			return &Data.Version;
		}
	}
	return nullptr;
}

UNiagaraScriptSourceBase* DuplicateScriptSource(UNiagaraScriptSourceBase* Source, UNiagaraEmitter* Outer, const FNiagaraAssetVersion& NewVersion)
{
	if (Source)
	{
		FObjectDuplicationParameters ObjParameters(Source, Outer);
		ObjParameters.DestClass = Source->GetClass();
		ObjParameters.DestName = MakeUniqueObjectName(Outer, Source->GetClass(), GetVersionedName("NiagaraScriptSource", NewVersion));
		if (UNiagaraScriptSourceBase* NewSource = Cast<UNiagaraScriptSourceBase>(StaticDuplicateObjectEx(ObjParameters)))
		{
			return NewSource;
		}
	}
	return nullptr;
}

UNiagaraScript* DuplicateScript(UNiagaraScript* Script, UObject* Outer, UNiagaraScriptSourceBase* Source, FString ScriptName, const FNiagaraAssetVersion& NewVersion)
{
	if (Script)
	{
		FObjectDuplicationParameters ObjParameters(Script, Outer);
		ObjParameters.DestClass = UNiagaraScript::StaticClass();
		ObjParameters.DestName = MakeUniqueObjectName(Outer, UNiagaraScript::StaticClass(), GetVersionedName(ScriptName, NewVersion));
		ObjParameters.ApplyFlags = RF_Transactional;
		if (UNiagaraScript* NewScript = Cast<UNiagaraScript>(StaticDuplicateObjectEx(ObjParameters)))
		{
			if (Source)
			{
				NewScript->SetLatestSource(Source);
			}
			return NewScript;
		}
	}
	return nullptr;
}

UNiagaraSimulationStageBase* DuplicateSimStage(UNiagaraSimulationStageBase* SimulationStage, UNiagaraEmitter* Outer, UNiagaraScriptSourceBase* Source, const FNiagaraAssetVersion& NewVersion)
{
	if (SimulationStage)
	{
		FObjectDuplicationParameters ObjParameters(SimulationStage, Outer);
		ObjParameters.DestClass = SimulationStage->GetClass();
		ObjParameters.DestName = MakeUniqueObjectName(Outer, SimulationStage->GetClass(), GetVersionedName(SimulationStage->GetClass()->GetName(), NewVersion));
		ObjParameters.ApplyFlags = RF_Transactional;
		if (UNiagaraSimulationStageBase* NewStage = Cast<UNiagaraSimulationStageBase>(StaticDuplicateObjectEx(ObjParameters)))
		{
			NewStage->Script = DuplicateScript(SimulationStage->Script, NewStage, Source, "SimulationStage", NewVersion);
			return NewStage;
		}
	}
	return nullptr;
}

FGuid UNiagaraEmitter::AddNewVersion(int32 MajorVersion, int32 MinorVersion)
{
	// check preconditions
	check(MajorVersion >= 1);
	check(MajorVersion != 1 || MinorVersion != 0);
	Modify();

	FNiagaraSystemUpdateContext UpdateContext;
	UpdateContext.SetDestroyOnAdd(true);
	UpdateContext.SetDestroySystemSim(true);
	if (UNiagaraSystem* Owner = GetTypedOuter<UNiagaraSystem>())
	{
		UpdateContext.Add(Owner, true);
	}

	FVersionedNiagaraEmitterData NewVersionData;
	FNiagaraAssetVersion NewVersion = { MajorVersion, MinorVersion, FGuid::NewGuid() };
	
	for (int i = VersionData.Num() - 1; i >= 0; i--)
	{
		FVersionedNiagaraEmitterData& Data = VersionData[i];
		check(Data.Version.MajorVersion != MajorVersion || Data.Version.MinorVersion != MinorVersion); // the version should not already exist

		if (Data.Version.MajorVersion < MajorVersion || (Data.Version.MajorVersion == MajorVersion && Data.Version.MinorVersion < MinorVersion))
		{
			// copy the data
			NewVersionData.CopyFrom(Data);

			// duplicate the scripts and source graph
			NewVersionData.GraphSource = DuplicateScriptSource(Data.GraphSource, this, NewVersion);
			NewVersionData.SpawnScriptProps.Script = DuplicateScript(Data.SpawnScriptProps.Script, this, NewVersionData.GraphSource, "SpawnScript", NewVersion);
			NewVersionData.UpdateScriptProps.Script = DuplicateScript(Data.UpdateScriptProps.Script, this, NewVersionData.GraphSource, "UpdateScript", NewVersion);
			NewVersionData.EmitterSpawnScriptProps.Script = DuplicateScript(Data.EmitterSpawnScriptProps.Script, this, NewVersionData.GraphSource, "EmitterSpawnScript", NewVersion);
			NewVersionData.EmitterUpdateScriptProps.Script = DuplicateScript(Data.EmitterUpdateScriptProps.Script, this, NewVersionData.GraphSource, "EmitterUpdateScript", NewVersion);
			NewVersionData.GPUComputeScript = DuplicateScript(Data.GPUComputeScript, this, NewVersionData.GraphSource, "GPUComputeScript", NewVersion);

			// duplicate the event scripts
			for (FNiagaraEventScriptProperties& EventProp : NewVersionData.EventHandlerScriptProps)
			{
				EventProp.Script = DuplicateScript(EventProp.Script, this, NewVersionData.GraphSource, "EventScript", NewVersion);
			}

			// duplicate the simulation stages
			NewVersionData.SimulationStages.Empty();
			for (UNiagaraSimulationStageBase* SimulationStage : Data.SimulationStages)
			{
				UNiagaraSimulationStageBase* NewStage = DuplicateSimStage(SimulationStage, this, NewVersionData.GraphSource, NewVersion);
				NewStage->OuterEmitterVersion = NewVersion.VersionGuid;
				NewVersionData.SimulationStages.Add(NewStage);
			}

			// duplicate the renderer settings
			NewVersionData.RendererProperties.Empty();
			for (UNiagaraRendererProperties* Renderer : Data.RendererProperties)
			{
				UNiagaraRendererProperties* NewRenderer = Cast<UNiagaraRendererProperties>(StaticDuplicateObject(Renderer, this, NAME_None, RF_Transactional));
				NewRenderer->OuterEmitterVersion = NewVersion.VersionGuid;
				NewVersionData.RendererProperties.Add(NewRenderer);
			}

			// duplicate the scratch pad scripts and replace their usage in the graphs
			Data.ScratchPads->CheckConsistency();
			NewVersionData.ScratchPads = CastChecked<UNiagaraScratchPadContainer>(StaticDuplicateObject(Data.ScratchPads, this));
			for (int k = 0; k < Data.ScratchPads->Scripts.Num(); k++) {
				UNiagaraScript* OldScript = Data.ScratchPads->Scripts[k];
				UNiagaraScript* NewScript = NewVersionData.ScratchPads->Scripts[k];
				NewVersionData.GraphSource->ReplaceScriptReferences(OldScript, NewScript);
			}

			// duplicate the parent scratch pad script container.
			Data.ParentScratchPads->CheckConsistency();
			NewVersionData.ParentScratchPads = CastChecked<UNiagaraScratchPadContainer>(StaticDuplicateObject(Data.ParentScratchPads, this));
			for (int k = 0; k < Data.ParentScratchPads->Scripts.Num(); k++) {
				UNiagaraScript* OldScript = Data.ParentScratchPads->Scripts[k];
				UNiagaraScript* NewScript = NewVersionData.ParentScratchPads->Scripts[k];
				NewVersionData.GraphSource->ReplaceScriptReferences(OldScript, NewScript);
			}

			// duplicate the editor data
			if (Data.EditorData)
			{
				NewVersionData.EditorData = CastChecked<UNiagaraEditorDataBase>(StaticDuplicateObject(Data.EditorData, this));
			}
			if (Data.EditorParameters)
			{
				NewVersionData.EditorParameters = CastChecked<UNiagaraEditorParametersAdapterBase>(StaticDuplicateObject(Data.EditorParameters, this));
			}

			// init stats critical sections
#if STATS
			NewVersionData.StatDatabase.Init();
#endif
			break;
		}
	}

	// set default data for fields we don't want to copy
	NewVersionData.Version = NewVersion;
	NewVersionData.VersionChangeDescription = FText();
	NewVersionData.UpdateScriptExecution = ENiagaraPythonUpdateScriptReference::None;
	NewVersionData.PythonUpdateScript = FString();
	NewVersionData.ScriptAsset = FFilePath();
	
	VersionData.Add(NewVersionData);
	VersionData.Sort([](const FVersionedNiagaraEmitterData& A, const FVersionedNiagaraEmitterData& B) { return A.Version < B.Version; });
	RebindNotifications();

	return NewVersionData.Version.VersionGuid;
}

void UNiagaraEmitter::DeleteVersion(const FGuid& VersionGuid)
{
	check(VersionGuid != ExposedVersion);
	
	for (int i = 0; i < VersionData.Num(); i++)
	{
		FNiagaraAssetVersion& AssetVersion = VersionData[i].Version;
		if (AssetVersion.VersionGuid == VersionGuid)
		{
			check(AssetVersion.MajorVersion != 1 || AssetVersion.MinorVersion != 0);
			Modify();
			VersionData.RemoveAt(i);
			return;
		}
	}
}

void UNiagaraEmitter::ExposeVersion(const FGuid& VersionGuid)
{
	// check if the requested version already exists
	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
		if (Data.Version.VersionGuid == VersionGuid)
		{
			Modify();
			ExposedVersion = VersionGuid;
			Data.Version.bIsVisibleInVersionSelector = true;
			return;
		}
	}
}

void UNiagaraEmitter::EnableVersioning()
{
	if (bVersioningEnabled)
	{
		return;
	}
	Modify();
	ensure(VersionData.Num() == 1);
	bVersioningEnabled = true;	
	ExposedVersion = VersionData[0].Version.VersionGuid;

	NiagaraAnalytics::RecordEvent("Versioning.EmitterEnabled");
}

void UNiagaraEmitter::DisableVersioning(const FGuid& VersionGuidToUse)
{
	CheckVersionDataAvailable();
	if (!bVersioningEnabled)
	{
		return;
	}
	
	bVersioningEnabled = false;
	FVersionedNiagaraEmitterData DataToUse;
	DataToUse.CopyFrom(VersionData[0]);
	if (VersionGuidToUse.IsValid())
	{
		for (const FVersionedNiagaraEmitterData& Data : VersionData)
		{
			if (Data.Version.VersionGuid == VersionGuidToUse)
			{
				DataToUse.CopyFrom(Data);
				break;
			}
		}
	}
	DataToUse.Version = FNiagaraAssetVersion(); // reset and create new guid
	ExposedVersion = DataToUse.Version.VersionGuid;	
	VersionData.Empty();
	VersionData.Add(DataToUse);
}

void UNiagaraEmitter::CheckVersionDataAvailable()
{
	if (VersionData.Num() > 0) {
		return;
	}

	// copy over existing data of assets that were created pre-versioning
	FVersionedNiagaraEmitterData& Data = VersionData.AddDefaulted_GetRef();
	Data.SimTarget = SimTarget_DEPRECATED;
	Data.Platforms = Platforms_DEPRECATED;
	Data.RandomSeed = RandomSeed_DEPRECATED;
	Data.EditorData = EditorData_DEPRECATED;
	Data.bLocalSpace = bLocalSpace_DEPRECATED;
	Data.FixedBounds = FixedBounds_DEPRECATED;
	Data.GraphSource = GraphSource_DEPRECATED;
	Data.bDeterminism = bDeterminism_DEPRECATED;
	Data.AllocationMode = AllocationMode_DEPRECATED;
	Data.SpawnScriptProps = SpawnScriptProps_DEPRECATED;
	Data.SimulationStages = SimulationStages_DEPRECATED;
	Data.EditorParameters = EditorParameters_DEPRECATED;
	Data.GPUComputeScript = GPUComputeScript_DEPRECATED;
	Data.UpdateScriptProps = UpdateScriptProps_DEPRECATED;
	Data.PreAllocationCount = PreAllocationCount_DEPRECATED;
	Data.RendererProperties = RendererProperties_DEPRECATED;
	Data.ScalabilityOverrides = ScalabilityOverrides_DEPRECATED;
	Data.AttributesToPreserve = AttributesToPreserve_DEPRECATED;
	Data.bRequiresPersistentIDs = bRequiresPersistentIDs_DEPRECATED;
	Data.EventHandlerScriptProps = EventHandlerScriptProps_DEPRECATED;
	Data.SharedEventGeneratorIds = SharedEventGeneratorIds_DEPRECATED;
	Data.EmitterSpawnScriptProps = EmitterSpawnScriptProps_DEPRECATED;
	Data.EmitterUpdateScriptProps = EmitterUpdateScriptProps_DEPRECATED;
	Data.MaxGPUParticlesSpawnPerFrame = MaxGPUParticlesSpawnPerFrame_DEPRECATED;
	
	Data.bInterpolatedSpawning_DEPRECATED = bInterpolatedSpawning_DEPRECATED;
	Data.InterpolatedSpawnMode = bInterpolatedSpawning_DEPRECATED ? ENiagaraInterpolatedSpawnMode::Interpolation : ENiagaraInterpolatedSpawnMode::NoInterpolation;

	if (bFixedBounds_DEPRECATED)
	{
		Data.CalculateBoundsMode = ENiagaraEmitterCalculateBoundMode::Fixed;
	}

	Data.ScratchPads = NewObject<UNiagaraScratchPadContainer>(this);
	Data.ScratchPads->SetScripts(ScratchPadScripts_DEPRECATED);

	Data.ParentScratchPads = NewObject<UNiagaraScratchPadContainer>(this);
	Data.ParentScratchPads->SetScripts(ParentScratchPadScripts_DEPRECATED);

	if (Parent_DEPRECATED)
	{
		Data.VersionedParent.Emitter = Parent_DEPRECATED;
	}

	if (ParentAtLastMerge_DEPRECATED)
	{
		// we don't need to store a version here, as the parent at last merge is always just a snapshot of one version
		Data.VersionedParentAtLastMerge.Emitter = ParentAtLastMerge_DEPRECATED;
	}

	// create a stable initial version guid for our versioned data based on the UNiagaraEmitter
	Data.Version.VersionGuid = FNiagaraAssetVersion::CreateStableVersionGuid(this);

	ExposedVersion = Data.Version.VersionGuid;

	for (UNiagaraRendererProperties* Renderer : Data.RendererProperties)
	{
		Renderer->OuterEmitterVersion = ExposedVersion;
	}
	for (UNiagaraSimulationStageBase* SimStage : Data.SimulationStages)
	{
		SimStage->OuterEmitterVersion = ExposedVersion;
	}
}

#endif

FVersionedNiagaraEmitterData::FVersionedNiagaraEmitterData()
	: InterpolatedSpawnMode(ENiagaraInterpolatedSpawnMode::NoInterpolation)
#if WITH_EDITORONLY_DATA
	, bInterpolatedSpawning_DEPRECATED(false)
	, bGpuAlwaysRunParticleUpdateScript(false)
#endif
	, bRequiresPersistentIDs(false)
	, MaxGPUParticlesSpawnPerFrame(0)
	, bIsAllowedToExecute(true)
	, bRequiresViewUniformBuffer(false)
	, bNeedsPartialDepthTexture(false)
#if WITH_EDITORONLY_DATA
	, EditorData(nullptr)
	, EditorParameters(nullptr)
#endif
{
	FixedBounds = GetDefaultFixedBounds();
}

#if WITH_EDITORONLY_DATA

void UNiagaraEmitter::UpdateFromMergedCopy(const INiagaraMergeManager& MergeManager, UNiagaraEmitter* MergedEmitter, FVersionedNiagaraEmitterData* EmitterData)
{
	auto ReouterMergedObject = [](UObject* NewOuter, UObject* TargetObject)
	{
		FName MergedObjectUniqueName = MakeUniqueObjectName(NewOuter, TargetObject->GetClass(), TargetObject->GetFName());
		TargetObject->Rename(*MergedObjectUniqueName.ToString(), NewOuter);
	};

	// The merged copy was based on the parent emitter so its name might be wrong, check and fix that first,
	// otherwise the rapid iteration parameter names will be wrong from the copied scripts.
	if (MergedEmitter->GetUniqueEmitterName() != UniqueEmitterName)
	{
		MergedEmitter->SetUniqueEmitterName(UniqueEmitterName);
	}

	// Copy base editable emitter properties.
	TArray<FProperty*> DifferentProperties;
	MergeManager.DiffEditableProperties(this, MergedEmitter, *StaticClass(), DifferentProperties);
	MergeManager.CopyPropertiesToBase(this, MergedEmitter, DifferentProperties);

	ensure(MergedEmitter->IsVersioningEnabled() == false);
	FVersionedNiagaraEmitterData* MergedData = MergedEmitter->GetLatestEmitterData();
	DifferentProperties.Empty();
	MergeManager.DiffEditableProperties(EmitterData, MergedData, *FVersionedNiagaraEmitterData::StaticStruct(), DifferentProperties);
	MergeManager.CopyPropertiesToBase(EmitterData, MergedData, DifferentProperties);

	// Copy source and scripts
	ReouterMergedObject(this, MergedData->GraphSource);
	EmitterData->GraphSource->OnChanged().RemoveAll(this);
	EmitterData->GraphSource = MergedData->GraphSource;
	EmitterData->GraphSource->OnChanged().AddUObject(this, &UNiagaraEmitter::GraphSourceChanged);

	ReouterMergedObject(this, MergedData->SpawnScriptProps.Script);
	EmitterData->SpawnScriptProps.Script->RapidIterationParameters.RemoveAllOnChangedHandlers(this);
	EmitterData->SpawnScriptProps.Script = MergedData->SpawnScriptProps.Script;
	EmitterData->SpawnScriptProps.Script->RapidIterationParameters.AddOnChangedHandler(
		FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));

	ReouterMergedObject(this, MergedData->UpdateScriptProps.Script);
	EmitterData->UpdateScriptProps.Script->RapidIterationParameters.RemoveAllOnChangedHandlers(this);
	EmitterData->UpdateScriptProps.Script = MergedData->UpdateScriptProps.Script;
	EmitterData->UpdateScriptProps.Script->RapidIterationParameters.AddOnChangedHandler(
		FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));

	ReouterMergedObject(this, MergedData->EmitterSpawnScriptProps.Script);
	EmitterData->EmitterSpawnScriptProps.Script->RapidIterationParameters.RemoveAllOnChangedHandlers(this);
	EmitterData->EmitterSpawnScriptProps.Script = MergedData->EmitterSpawnScriptProps.Script;
	EmitterData->EmitterSpawnScriptProps.Script->RapidIterationParameters.AddOnChangedHandler(
		FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));

	ReouterMergedObject(this, MergedData->EmitterUpdateScriptProps.Script);
	EmitterData->EmitterUpdateScriptProps.Script->RapidIterationParameters.RemoveAllOnChangedHandlers(this);
	EmitterData->EmitterUpdateScriptProps.Script = MergedData->EmitterUpdateScriptProps.Script;
	EmitterData->EmitterUpdateScriptProps.Script->RapidIterationParameters.AddOnChangedHandler(
		FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));

	ReouterMergedObject(this, MergedData->GPUComputeScript);
	EmitterData->GPUComputeScript->RapidIterationParameters.RemoveAllOnChangedHandlers(this);
	EmitterData->GPUComputeScript = MergedData->GPUComputeScript;
	EmitterData->GPUComputeScript->RapidIterationParameters.AddOnChangedHandler(
		FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));

	// Copy event handlers
	for (FNiagaraEventScriptProperties& EventScriptProperties : EmitterData->EventHandlerScriptProps)
	{
		EventScriptProperties.Script->RapidIterationParameters.RemoveAllOnChangedHandlers(this);
	}
	EmitterData->EventHandlerScriptProps.Empty();

	for (FNiagaraEventScriptProperties& MergedEventScriptProperties : MergedData->EventHandlerScriptProps)
	{
		EmitterData->EventHandlerScriptProps.Add(MergedEventScriptProperties);
		ReouterMergedObject(this, MergedEventScriptProperties.Script);
		MergedEventScriptProperties.Script->RapidIterationParameters.AddOnChangedHandler(
			FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));
	}

	// Copy shader stages
	for (TObjectPtr<UNiagaraSimulationStageBase>& SimulationStage : EmitterData->SimulationStages)
	{
		SimulationStage->OnChanged().RemoveAll(this);
		SimulationStage->Script->RapidIterationParameters.RemoveAllOnChangedHandlers(this);
	}
	EmitterData->SimulationStages.Empty();

	for (UNiagaraSimulationStageBase* MergedSimulationStage : MergedData->SimulationStages)
	{
		ReouterMergedObject(this, MergedSimulationStage);
		EmitterData->SimulationStages.Add(MergedSimulationStage);
		MergedSimulationStage->OnChanged().AddUObject(this, &UNiagaraEmitter::SimulationStageChanged);
		MergedSimulationStage->Script->RapidIterationParameters.AddOnChangedHandler(
			FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));
	}

	// Copy renderers
	for (UNiagaraRendererProperties* Renderer : EmitterData->RendererProperties)
	{
		Renderer->OnChanged().RemoveAll(this);

		// some renderer properties have been incorrectly flagged as RF_Public meaning that even if we remove them here with the merge
		// they will be included in a cook; so clear the flag while we're removing them
		Renderer->ClearFlags(RF_Public);
	}
	EmitterData->RendererProperties.Empty();

	for (UNiagaraRendererProperties* MergedRenderer : MergedData->RendererProperties)
	{
		ReouterMergedObject(this, MergedRenderer);
		EmitterData->RendererProperties.Add(MergedRenderer);
		MergedRenderer->OnChanged().AddUObject(this, &UNiagaraEmitter::RendererChanged);
	}

	// Copy scratch pad scripts.
	EmitterData->ParentScratchPads = NewObject<UNiagaraScratchPadContainer>(this);
	EmitterData->ParentScratchPads->AppendScripts(MergedData->ParentScratchPads->Scripts);

	EmitterData->ScratchPads = NewObject<UNiagaraScratchPadContainer>(this);
	EmitterData->ScratchPads->AppendScripts(MergedData->ScratchPads->Scripts);

	UNiagaraEditorDataBase* NewEditorData = MergedData->GetEditorData();
	ReouterMergedObject(this, NewEditorData);
	SetEditorData(MergedData->GetEditorData(), EmitterData->Version.VersionGuid);

	// Update messages
	for (const TPair<FGuid, TObjectPtr<UNiagaraMessageDataBase>>& Message : MergedEmitter->GetMessageStore().GetMessages())
	{
		if (Message.Value != nullptr)
		{
			ReouterMergedObject(this, Message.Value);
			MessageStore.AddMessage(Message.Key, Message.Value);
		}
	}

	// Update the change id since we don't know what's changed.
	UpdateChangeId(TEXT("Updated from merged copy"));
}

void FVersionedNiagaraEmitterData::SyncEmitterAlias(const FString& InOldName, const UNiagaraEmitter& InEmitter)
{
	TArray<UNiagaraScript*> Scripts;
	GetScripts(Scripts, false, true); // Get all the scripts...

	for (UNiagaraScript* Script : Scripts)
	{
		// We don't mark the package dirty here because this can happen as a result of a compile and we don't want to dirty files
		// due to compilation, in cases where the package should be marked dirty an previous modify would have already done this.
		Script->Modify(false);
		Script->SyncAliases(FNiagaraAliasContext(Script->GetUsage())
			.ChangeEmitterName(InOldName, InEmitter.GetUniqueEmitterName()));
	}

	// if we haven't yet been postloaded then we'll hold off on updating the renderers as they are dependent on everything
	// (System/Emitter/Scripts) being fully loaded.
	if (!InEmitter.HasAnyFlags(RF_NeedPostLoad))
	{
		for (UNiagaraRendererProperties* Renderer : RendererProperties)
		{
			if (Renderer)
			{
				Renderer->Modify(false);
				Renderer->RenameEmitter(*InOldName, &InEmitter);
			}
		}

		for (UNiagaraSimulationStageBase* SimStage : SimulationStages)
		{
			if (UNiagaraSimulationStageGeneric* GenericStage = Cast<UNiagaraSimulationStageGeneric>(SimStage))
			{
				GenericStage->Modify(false);
				GenericStage->RenameEmitter(*InOldName, &InEmitter);
			}
		}
	}
}

#endif

void UNiagaraEmitter::AddRenderer(UNiagaraRendererProperties* Renderer, FGuid EmitterVersion)
{
	FNiagaraSystemUpdateContext UpdateContext;
	UpdateContext.SetDestroyOnAdd(true);
	if (UNiagaraSystem* Owner = GetTypedOuter<UNiagaraSystem>())
	{
		UpdateContext.Add(Owner, true);
	}

	Modify();
	Renderer->OuterEmitterVersion = EmitterVersion;
	FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(EmitterVersion);
	EmitterData->RendererProperties.Add(Renderer);
#if WITH_EDITOR
	Renderer->OnChanged().AddUObject(this, &UNiagaraEmitter::RendererChanged);
	UpdateChangeId(TEXT("Renderer added"));
	OnRenderersChangedDelegate.Broadcast();
#endif
	EmitterData->RebuildRendererBindings(*this);
}

void UNiagaraEmitter::RemoveRenderer(UNiagaraRendererProperties* Renderer, FGuid EmitterVersion)
{
	FNiagaraSystemUpdateContext UpdateContext;
	UpdateContext.SetDestroyOnAdd(true);
	if (UNiagaraSystem* Owner = GetTypedOuter<UNiagaraSystem>())
	{
		UpdateContext.Add(Owner, true);
	}

	Modify();
	FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(EmitterVersion);
	EmitterData->RendererProperties.Remove(Renderer);
#if WITH_EDITOR
	Renderer->OnChanged().RemoveAll(this);
	UpdateChangeId(TEXT("Renderer removed"));
	OnRenderersChangedDelegate.Broadcast();
#endif
	EmitterData->RebuildRendererBindings(*this);
}

void UNiagaraEmitter::MoveRenderer(UNiagaraRendererProperties* Renderer, int32 NewIndex, FGuid EmitterVersion)
{
	FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(EmitterVersion);
	int32 CurrentIndex = EmitterData->RendererProperties.IndexOfByKey(Renderer);
	if (CurrentIndex == INDEX_NONE || CurrentIndex == NewIndex || !EmitterData->RendererProperties.IsValidIndex(NewIndex))
	{
		return;
	}
	
	FNiagaraSystemUpdateContext UpdateContext;
	UpdateContext.SetDestroyOnAdd(true);
	if (UNiagaraSystem* Owner = GetTypedOuter<UNiagaraSystem>())
	{
		UpdateContext.Add(Owner, true);
	}

	Modify();
	EmitterData->RendererProperties.RemoveAt(CurrentIndex);
	EmitterData->RendererProperties.Insert(Renderer, NewIndex);
#if WITH_EDITOR
	UpdateChangeId(TEXT("Renderer moved"));
	OnRenderersChangedDelegate.Broadcast();
#endif
	EmitterData->RebuildRendererBindings(*this);
}

FNiagaraEventScriptProperties* FVersionedNiagaraEmitterData::GetEventHandlerByIdUnsafe(FGuid ScriptUsageId)
{
	for (FNiagaraEventScriptProperties& EventScriptProperties : EventHandlerScriptProps)
	{
		if (EventScriptProperties.Script->GetUsageId() == ScriptUsageId)
		{
			return &EventScriptProperties;
		}
	}
	return nullptr;
}

void UNiagaraEmitter::AddEventHandler(FNiagaraEventScriptProperties EventHandler, FGuid EmitterVersion)
{
	Modify();
	GetEmitterData(EmitterVersion)->EventHandlerScriptProps.Add(EventHandler);
#if WITH_EDITOR
	EventHandler.Script->RapidIterationParameters.AddOnChangedHandler(
		FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));
	UpdateChangeId(TEXT("Event handler added"));
	OnEventHandlersChangedDelegate.Broadcast();
#endif
}

void UNiagaraEmitter::RemoveEventHandlerByUsageId(FGuid EventHandlerUsageId, FGuid EmitterVersion)
{
	Modify();
	auto FindEventHandlerById = [=](const FNiagaraEventScriptProperties& EventHandler) { return EventHandler.Script->GetUsageId() == EventHandlerUsageId; };
	TArray<FNiagaraEventScriptProperties>& EventHandlerScriptProps = GetEmitterData(EmitterVersion)->EventHandlerScriptProps;
#if WITH_EDITOR
	FNiagaraEventScriptProperties* EventHandler = EventHandlerScriptProps.FindByPredicate(FindEventHandlerById);
	if (EventHandler != nullptr)
	{
		EventHandler->Script->RapidIterationParameters.RemoveAllOnChangedHandlers(this);
	}
#endif
	EventHandlerScriptProps.RemoveAll(FindEventHandlerById);
#if WITH_EDITOR
	OnEventHandlersChangedDelegate.Broadcast();
	UpdateChangeId(TEXT("Event handler removed"));
#endif
}

UNiagaraSimulationStageBase* FVersionedNiagaraEmitterData::GetSimulationStageById(FGuid ScriptUsageId) const
{
	TObjectPtr<UNiagaraSimulationStageBase> const* FoundSimulationStagePtr = SimulationStages.FindByPredicate([&ScriptUsageId](UNiagaraSimulationStageBase* SimulationStage) { return SimulationStage->Script->GetUsageId() == ScriptUsageId; });
	return FoundSimulationStagePtr != nullptr ? *FoundSimulationStagePtr : nullptr;
}

void UNiagaraEmitter::AddSimulationStage(UNiagaraSimulationStageBase* SimulationStage, FGuid EmitterVersion)
{
	Modify();
	GetEmitterData(EmitterVersion)->SimulationStages.Add(SimulationStage);
#if WITH_EDITOR
	SimulationStage->OuterEmitterVersion = EmitterVersion;
	SimulationStage->OnChanged().AddUObject(this, &UNiagaraEmitter::SimulationStageChanged);
	SimulationStage->Script->RapidIterationParameters.AddOnChangedHandler(
		FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraEmitter::ScriptRapidIterationParameterChanged));
	UpdateChangeId(TEXT("Shader stage added"));
	OnSimStagesChangedDelegate.Broadcast();
#endif
}

void UNiagaraEmitter::RemoveSimulationStage(UNiagaraSimulationStageBase* SimulationStage, FGuid EmitterVersion)
{
	Modify();
	bool bRemoved = GetEmitterData(EmitterVersion)->SimulationStages.Remove(SimulationStage) != 0;
#if WITH_EDITOR
	if (bRemoved)
	{
		SimulationStage->OnChanged().RemoveAll(this);
		SimulationStage->Script->RapidIterationParameters.RemoveAllOnChangedHandlers(this);
		UpdateChangeId(TEXT("Simulation stage removed"));
		OnSimStagesChangedDelegate.Broadcast();
	}
#endif
}

void UNiagaraEmitter::MoveSimulationStageToIndex(UNiagaraSimulationStageBase* SimulationStageToMove, int32 TargetIndex, FGuid EmitterVersion)
{
	TArray<TObjectPtr<UNiagaraSimulationStageBase>>& SimulationStages = GetEmitterData(EmitterVersion)->SimulationStages;
	int32 CurrentIndex = SimulationStages.IndexOfByKey(SimulationStageToMove);
	checkf(CurrentIndex != INDEX_NONE, TEXT("Simulation stage could not be moved because it is not owned by this emitter."));
	if (TargetIndex != CurrentIndex)
	{
		int32 AdjustedTargetIndex = CurrentIndex < TargetIndex
			? TargetIndex - 1 // If the current index is less than the target index, the target index needs to be decreased to make up for the item being removed.
			: TargetIndex;

		SimulationStages.Remove(SimulationStageToMove);
		SimulationStages.Insert(SimulationStageToMove, AdjustedTargetIndex);
#if WITH_EDITOR
		UpdateChangeId("Simulation stage moved.");
		OnSimStagesChangedDelegate.Broadcast();
#endif
	}
}

bool FVersionedNiagaraEmitterData::IsEventGeneratorShared(FName EventGeneratorId) const
{
	return SharedEventGeneratorIds.Contains(EventGeneratorId);
}

void UNiagaraEmitter::BeginDestroy()
{
#if WITH_EDITOR
	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
		if (Data.GraphSource != nullptr)
		{
			Data.GraphSource->OnChanged().RemoveAll(this);
		}
		if (Data.GPUComputeScript)
		{
			Data.GPUComputeScript->OnGPUScriptCompiled().RemoveAll(this);
		}
	}
#endif
	Super::BeginDestroy();
}

#if WITH_EDITORONLY_DATA

void UNiagaraEmitter::UpdateChangeId(const FString& Reason)
{
	// We don't mark the package dirty here because this can happen as a result of a compile and we don't want to dirty files
	// due to compilation, in cases where the package should be marked dirty an previous modify would have already done this.
	Modify(false);
	FGuid OldId = ChangeId;
	ChangeId = FGuid::NewGuid();
	if (GbEnableEmitterChangeIdMergeLogging)
	{
		UE_LOGF(LogNiagara, Log, "Emitter %ls change id updated. Reason: %ls OldId: %ls NewId: %ls",
			*GetPathName(), *Reason, *OldId.ToString(), *ChangeId.ToString());
	}

	for (FVersionedNiagaraEmitterData& Data : VersionData)
	{
		Data.CachedTraversalData.Reset();
#if STATS
		Data.StatDatabase.Init();
		Data.StatDatabase.ClearStatCaptures();
#endif
	}
}

void UNiagaraEmitter::ScriptRapidIterationParameterChanged()
{
	UpdateChangeId(TEXT("Script rapid iteration parameter changed."));
}

void UNiagaraEmitter::SimulationStageChanged()
{
	UpdateChangeId(TEXT("Simulation Stage Changed"));
}

void UNiagaraEmitter::RendererChanged()
{
	UpdateChangeId(TEXT("Renderer changed."));
}

void UNiagaraEmitter::GraphSourceChanged()
{
	UpdateChangeId(TEXT("Graph source changed."));

	UNiagaraSystem* Sys = Cast<UNiagaraSystem>(GetOuter());
	if (Sys)
		Sys->GraphSourceChanged();
}

void UNiagaraEmitter::RaiseOnEmitterGPUCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion)
{
	OnGPUScriptCompiledDelegate.Broadcast(InScript->GetOuterEmitter());
}

void UNiagaraEmitter::PersistentEditorDataChanged()
{
	UpdateChangeId(TEXT("Persistent editor data changed."));
}
#endif

TStatId UNiagaraEmitter::GetStatID(bool bGameThread, bool bConcurrent)const
{
#if STATS
	if (!StatID_GT.IsValidStat())
	{
		GenerateStatID();
	}

	if (bGameThread)
	{
		if (bConcurrent)
		{
			return StatID_GT_CNC;
		}
		else
		{
			return StatID_GT;
		}
	}
	else
	{
		if (bConcurrent)
		{
			return StatID_RT_CNC;
		}
		else
		{
			return StatID_RT;
		}
	}
#else
	return TStatId();
#endif
}

void FVersionedNiagaraEmitterData::ClearRuntimeAllocationEstimate(uint64 ReportHandle)
{
	FScopeLock Lock(RuntimeEstimation.GetCriticalSection());
	if (ReportHandle == INDEX_NONE)
	{
		RuntimeEstimation.AllocationEstimate = 0;
		RuntimeEstimation.RuntimeAllocations.Empty();
		RuntimeEstimation.SetIsEstimationDirty(true);
	}
	else
	{
		RuntimeEstimation.RuntimeAllocations.Remove(ReportHandle);
		RuntimeEstimation.SetIsEstimationDirty(true);
	}
}

int32 FVersionedNiagaraEmitterData::AddRuntimeAllocation(uint64 ReporterHandle, int32 AllocationCount)
{
	FScopeLock Lock(RuntimeEstimation.GetCriticalSection());
	int32* Estimate = RuntimeEstimation.RuntimeAllocations.Find(ReporterHandle);
	if (!Estimate || *Estimate < AllocationCount)
	{
		RuntimeEstimation.RuntimeAllocations.Add(ReporterHandle, AllocationCount);
		RuntimeEstimation.SetIsEstimationDirty(true);

		// Remove a random entry when there are enough logged allocations already
		if (RuntimeEstimation.RuntimeAllocations.Num() > 10)
		{
			TArray<uint64> Keys;
			RuntimeEstimation.RuntimeAllocations.GetKeys(Keys);
			RuntimeEstimation.RuntimeAllocations.Remove(Keys[FMath::RandHelper(Keys.Num())]);
		}
	}
	return RuntimeEstimation.RuntimeAllocations.Num();
}

int32 FVersionedNiagaraEmitterData::GetMaxParticleCountEstimate()
{
	if ((AllocationMode == EParticleAllocationMode::ManualEstimate)
		|| (AllocationMode == EParticleAllocationMode::FixedCount))
	{
		return PreAllocationCount;
	}
	
	if (RuntimeEstimation.GetIsEstimationDirty())
	{
		FScopeLock lock(RuntimeEstimation.GetCriticalSection());
		int32 EstimationCount = RuntimeEstimation.RuntimeAllocations.Num();
		RuntimeEstimation.AllocationEstimate = 0;
		if (EstimationCount > 0)
		{
			RuntimeEstimation.RuntimeAllocations.ValueSort(TGreater<int32>());
			int32 i = 0;
			for (TPair<uint64, int32> pair : RuntimeEstimation.RuntimeAllocations)
			{
				if (i >= (EstimationCount - 1) / 2)
				{
					// to prevent overallocation from outliers we take the median instead of the global max
					RuntimeEstimation.AllocationEstimate = pair.Value;
					break;
				}
				i++;
			}
			RuntimeEstimation.SetIsEstimationDirty(false);
		}
	}
	return RuntimeEstimation.AllocationEstimate;
}

FMemoryRuntimeEstimation::FMemoryRuntimeEstimation()
{
	EstimationCriticalSection = MakeShared<FCriticalSection>();
}

FCriticalSection* FMemoryRuntimeEstimation::GetCriticalSection()
{
	ensure(EstimationCriticalSection.IsValid());
	return EstimationCriticalSection.Get();
}

void UNiagaraEmitter::UpdateStatID() const
{
#if STATS
	if (StatID_GT.IsValidStat())
	{
		GenerateStatID();
	}
#endif
}

void UNiagaraEmitter::GenerateStatID() const
{
#if STATS
	FString Name = GetOuter() ? GetOuter()->GetFName().ToString() : TEXT("");
	Name += TEXT("/") + UniqueEmitterName;
	StatID_GT = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_NiagaraEmitters>(Name + TEXT("[GT]"));
	StatID_GT_CNC = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_NiagaraEmitters>(Name + TEXT("[GT_CNC]"));
	StatID_RT = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_NiagaraEmitters>(Name + TEXT("[RT]"));
	StatID_RT_CNC = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_NiagaraEmitters>(Name + TEXT("[RT_CNC]"));
#endif
}

// Register global PSO collector so compile time and tracked in the stats
static const TCHAR* NiagaraComputePSOCollectorName = TEXT("NiagaraComputePSOCollector");
FRegisterGlobalPSOCollectorFunction RegisterNiagaraComputePSOCollector(nullptr, NiagaraComputePSOCollectorName);

FGraphEventArray FVersionedNiagaraEmitterData::PrecacheComputePSOs(const UNiagaraEmitter& NiagaraEmitter)
{
	FGraphEventArray PSOReadyGraphTasks;
	PSOPrecacheResult = EPSOPrecacheResult::Complete;
	if (GNiagaraEmitterComputePSOPrecacheMode == 0 || !FApp::CanEverRender() || SimTarget != ENiagaraSimTarget::GPUComputeSim)
	{
		return PSOReadyGraphTasks;
	}
	
	FNiagaraShaderScript* ShaderScript = GPUComputeScript->GetRenderThreadScript();
	if (ShaderScript == nullptr || GPUComputeScript->DidScriptCompilationSucceed(true) == false)
	{
		return PSOReadyGraphTasks;
	}

	// Compute PSO can fail on some platforms even though the shader compiled successfully this path will wait for the PSO precache to complete before the emitter is allowed to run
	if (GNiagaraEmitterComputePSOPrecacheMode == 3)
	{
		FPSOPrecacheRequestParams PSORequestParams;
		PSORequestParams.InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
		PSORequestParams.PrecachePriority = EPSOPrecachePriority::High;
		PSORequestParams.CollectorName = NiagaraComputePSOCollectorName;
		PSORequestParams.ResourceName = NiagaraEmitter.GetName();
		PSORequestParams.bForcePrecache = true;

		FGraphEventArray PSOPrecacheEvents;
		for (int32 i = 0; i < ShaderScript->GetNumPermutations(); ++i)
		{
			FNiagaraShaderRef ShaderRef = ShaderScript->GetShaderGameThread(i);
			if (ShaderRef.IsValid() == false)
			{
				continue;
			}

			FRHIComputeShader* ComputeShader = ShaderRef.GetComputeShader();
			FPSOPrecacheRequestResult PSOPrecacheRequestResult = PipelineStateCache::PrecacheComputePipelineState(ComputeShader, PSORequestParams);
			if (PSOPrecacheRequestResult.AsyncCompileEvent.IsValid())
			{
				PSOPrecacheEvents.Add(PSOPrecacheRequestResult.AsyncCompileEvent);
				continue;
			}

			const EPSOPrecacheResult ShaderResult = PipelineStateCache::CheckPipelineStateInCache(ComputeShader);
			if (ShaderResult != EPSOPrecacheResult::Complete)
			{
				PSOPrecacheResult = EPSOPrecacheResult::NotSupported;
				return PSOReadyGraphTasks;
			}
		}

		const FNiagaraVMExecutableDataId VMid = GPUComputeScript->GetComputedVMCompilationId();	//-TODO:
		struct FNiagaraEmitterPSOPrecacheReadyTask
		{
			explicit FNiagaraEmitterPSOPrecacheReadyTask(FVersionedNiagaraEmitterWeakPtr InVersionedEmitter)
				: WeakVersionedEmitter(InVersionedEmitter)
			{
			}

			FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraEmitterPSOPrecacheReadyTask, STATGROUP_TaskGraphTasks); }
			ENamedThreads::Type GetDesiredThread() { return ENamedThreads::GameThread; }
			static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
			void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
			{
				FVersionedNiagaraEmitter VersionedEmitter = WeakVersionedEmitter.ResolveWeakPtr();
				FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
				if (EmitterData == nullptr)
				{
					return;
				}

				FNiagaraShaderScript* ShaderScript = EmitterData->GPUComputeScript->GetRenderThreadScript();
				if (ShaderScript == nullptr)
				{
					return;
				}

				for (int32 i = 0; i < ShaderScript->GetNumPermutations(); ++i)
				{
					FNiagaraShaderRef ShaderRef = ShaderScript->GetShaderGameThread(i);
					if (ShaderRef.IsValid() == false)
					{
						continue;
					}

					FRHIComputeShader* ComputeShader = ShaderRef.GetComputeShader();
					const EPSOPrecacheResult ShaderResult = PipelineStateCache::CheckPipelineStateInCache(ComputeShader);
					if (ShaderResult != EPSOPrecacheResult::Complete)
					{
						EmitterData->PSOPrecacheResult = EPSOPrecacheResult::NotSupported;

						UNiagaraSystem* NiagaraSystem = VersionedEmitter.Emitter ? VersionedEmitter.Emitter->GetTypedOuter<UNiagaraSystem>() : nullptr;
						UE_LOGF(LogNiagara, Warning, "Niagara ComputePSOPrecache Failed for Emitter(%ls) System(%ls), emitter will not run.", *GetNameSafe(VersionedEmitter.Emitter), *GetNameSafe(NiagaraSystem));
						return;
					}
				}
				EmitterData->PSOPrecacheResult = EPSOPrecacheResult::Complete;
			}

			FVersionedNiagaraEmitterWeakPtr WeakVersionedEmitter;
		};

		if (PSOPrecacheEvents.Num() > 0)
		{
			PSOPrecacheResult = EPSOPrecacheResult::Active;

			FVersionedNiagaraEmitterWeakPtr EmitterWeakPtr((UNiagaraEmitter*)(&NiagaraEmitter), Version.VersionGuid);
			PSOReadyGraphTasks.Add( TGraphTask<FNiagaraEmitterPSOPrecacheReadyTask>::CreateTask(&PSOPrecacheEvents, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(EmitterWeakPtr));
		}
		else
		{
			PSOPrecacheResult = EPSOPrecacheResult::Complete;
		}
	}
	// In this mode we either force them to cache or respect that precaching is enabled
	else if (GNiagaraEmitterComputePSOPrecacheMode == 2 || PipelineStateCache::IsPSOPrecachingEnabled() )
	{
		FPSOPrecacheDataArray PSOInitializers;
		PSOInitializers.Reserve(ShaderScript->GetNumPermutations());
		
		static const int32 NiagaraEmitterPSOCollectorIndex = FGlobalPSOCollectorManager::GetIndex(NiagaraComputePSOCollectorName);

		for (int32 i=0; i < ShaderScript->GetNumPermutations(); ++i)
		{
			FNiagaraShaderRef ShaderRef = ShaderScript->GetShaderGameThread(i);
			if (ShaderRef.IsValid() == false)
			{
				continue;
			}

			FPSOPrecacheData PSOPrecacheData;
			PSOPrecacheData.Type = FPSOPrecacheData::EType::Compute;
			PSOPrecacheData.ComputeInitializer = FComputePipelineStateInitializer(ShaderRef.GetComputeShader());
			PSOPrecacheData.InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
			PSOPrecacheData.Priority = EPSOPrecachePriority::High;
		#if PSO_PRECACHING_VALIDATE
			PSOPrecacheData.PSOCollectorIndex = NiagaraEmitterPSOCollectorIndex;
			PSOPrecacheData.VertexFactoryType = nullptr;
		#endif // PSO_PRECACHING_VALIDATE
			PSOInitializers.Add(MoveTemp(PSOPrecacheData));
		}

		FPSOPrecacheRequestResultArray ResultArray = RequestPrecachePSOs(EPSOPrecacheType::Global, PSOInitializers);

		EPSOPrecacheProxyCreationStrategy ProxyCreationStrategy = GetPSOPrecacheProxyCreationStrategy();
		bool bAddToPSOReadyGraphTasks = GNiagaraEmitterComputePSOPrecacheMode == 1 && ProxyCreationStrategy != EPSOPrecacheProxyCreationStrategy::AlwaysCreate;
		if (bAddToPSOReadyGraphTasks)
		{			
			for (FPSOPrecacheRequestResult PSOPrecacheRequestResult : ResultArray)
			{
				if (PSOPrecacheRequestResult.AsyncCompileEvent != nullptr && bAddToPSOReadyGraphTasks)
				{
					PSOReadyGraphTasks.Add(PSOPrecacheRequestResult.AsyncCompileEvent);
				}
			}
		}
	}
	return PSOReadyGraphTasks;
}

#if WITH_EDITORONLY_DATA
FVersionedNiagaraEmitter FVersionedNiagaraEmitterData::GetParent() const
{
	return VersionedParent;
}

FVersionedNiagaraEmitter FVersionedNiagaraEmitterData::GetParentAtLastMerge() const
{
	return VersionedParentAtLastMerge;
}

void FVersionedNiagaraEmitterData::RemoveParent()
{
	VersionedParent = FVersionedNiagaraEmitter();
	VersionedParentAtLastMerge = FVersionedNiagaraEmitter();
}

void UNiagaraEmitter::SetParent(const FVersionedNiagaraEmitter& InParent)
{
	if (ensure(VersionData.Num() == 1))
	{
		FVersionedNiagaraEmitterData& EmitterData = VersionData[0];
		EmitterData.VersionedParent = InParent;
		EmitterData.VersionedParentAtLastMerge.Emitter = InParent.Emitter->DuplicateWithoutMerging(this);
		EmitterData.VersionedParentAtLastMerge.Emitter->DisableVersioning(InParent.Version);
		EmitterData.VersionedParentAtLastMerge.Version = InParent.Version;

		// Since this API is only valid for the "Create duplicate parent" operation we move the emitters scratch pad script to the parent array since that's where they're defined now.
		// Normally we would duplicate the parent scratch pad scripts here, but since the whole parent is already a duplicate, we can skip this here.
		EmitterData.ParentScratchPads->AppendScripts(EmitterData.ScratchPads);
		EmitterData.GraphSource->MarkNotSynchronized(TEXT("Emitter parent changed"));
		UpdateChangeId(TEXT("Parent Set"));
	}
}

void FVersionedNiagaraEmitterData::Reparent(const FVersionedNiagaraEmitter& InParent)
{
	VersionedParent = InParent;
	VersionedParentAtLastMerge = FVersionedNiagaraEmitter();
	GraphSource->MarkNotSynchronized(TEXT("Emitter parent changed"));
}

void UNiagaraEmitter::ChangeParentVersion(const FGuid& NewParentVersion, const FGuid& EmitterVersion)
{
	FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(EmitterVersion);
	if (ensure(EmitterData) && EmitterData->VersionedParent.Emitter && EmitterData->VersionedParent.Emitter->IsVersioningEnabled() && EmitterData->VersionedParent.Emitter->GetEmitterData(NewParentVersion))
	{
		Modify();
		EmitterData->VersionedParent.Version = NewParentVersion;
		MergeChangesFromParent();
	}
}

void UNiagaraEmitter::NotifyScratchPadScriptsChanged()
{
	UpdateChangeId(TEXT("Scratch pad scripts changed."));
}
#endif

void UNiagaraEmitter::ResolveScalabilitySettings()
{
	for (FVersionedNiagaraEmitterData& EmitterData : VersionData)
	{
		EmitterData.CurrentScalabilitySettings.Clear();

		if (UNiagaraSystem* Owner = GetTypedOuter<UNiagaraSystem>())
		{
			if(UNiagaraEffectType* ActualEffectType = Owner->GetEffectType())
			{
				EmitterData.CurrentScalabilitySettings = ActualEffectType->GetActiveEmitterScalabilitySettings();
			}
		}

		for (FNiagaraEmitterScalabilityOverride& Override : EmitterData.ScalabilityOverrides.Overrides)
		{
			if (Override.Platforms.IsActive())
			{
				if (Override.bOverrideSpawnCountScale)
				{
					EmitterData.CurrentScalabilitySettings.bScaleSpawnCount = Override.bScaleSpawnCount;
					EmitterData.CurrentScalabilitySettings.SpawnCountScale = Override.SpawnCountScale;
				}
			}
		}

		// Clear out any allocation estimates as changing scalability could change the max spawn count
		if (EmitterData.RuntimeEstimation.AllocationEstimate > 0)
		{
			EmitterData.ClearRuntimeAllocationEstimate(INDEX_NONE);
		}
	}
}

void UNiagaraEmitter::UpdateScalability()
{
	ResolveScalabilitySettings();
}


