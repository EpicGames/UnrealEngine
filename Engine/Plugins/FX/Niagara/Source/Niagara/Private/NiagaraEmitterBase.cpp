// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEmitterBase.h"
#include "NiagaraSystem.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEmitterBase)

FVersionedNiagaraEmitterBase::FVersionedNiagaraEmitterBase(const FVersionedNiagaraEmitterBaseWeakPtr& VersionedEmitterWeak)
{
	Emitter = VersionedEmitterWeak.Emitter.Get();
	Version = Emitter ? VersionedEmitterWeak.Version : FGuid();
}

FVersionedNiagaraEmitterBaseWeakPtr FVersionedNiagaraEmitterBase::ToWeakPtr() const
{
	return FVersionedNiagaraEmitterBaseWeakPtr(Emitter, Version);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FVersionedNiagaraEmitterBaseWeakPtr::FVersionedNiagaraEmitterBaseWeakPtr(const FVersionedNiagaraEmitterBase& VersionedEmitter)
{
	Emitter = VersionedEmitter.Emitter;
	Version = VersionedEmitter.Version;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

const FString& UNiagaraEmitterBase::GetUniqueEmitterName() const
{
	return UniqueEmitterName;
}

bool UNiagaraEmitterBase::SetUniqueEmitterName(const FString& InName)
{
	if (InName != UniqueEmitterName)
	{
		Modify();
		FString OldName = UniqueEmitterName;
		UniqueEmitterName = InName;

		// Note: Assets don't care about the number portion so we need to compare without the number otherwise renaming can collide
		const FString ExistingName = IsAsset() ? GetFName().GetPlainNameString() : GetName();
		if (ExistingName != InName)
		{
			// Also rename the underlying uobject to keep things consistent.
			FName UniqueObjectName = MakeUniqueObjectName(GetOuter(), StaticClass(), *InName);
			Rename(*UniqueObjectName.ToString(), GetOuter());
		}

	#if WITH_EDITORONLY_DATA
		OnUniqueEmitterNameChanged(OldName);
	#endif
		return true;
	}

	return false;
}

bool UNiagaraEmitterBase::CanObtainSystemAttribute(const FNiagaraVariableBase& InVariable, FNiagaraTypeDefinition& OutBoundType) const
{
	check(!HasAnyFlags(RF_NeedPostLoad));

	if (const UNiagaraSystem* NiagaraSystem = GetTypedOuter<UNiagaraSystem>())
	{
		// make sure that this isn't called before our dependents are fully loaded
		check(!NiagaraSystem->HasAnyFlags(RF_NeedPostLoad));

		return NiagaraSystem->CanObtainSystemAttribute(InVariable, OutBoundType);
	}
	return false;
}

bool UNiagaraEmitterBase::CanObtainUserVariable(const FNiagaraVariableBase& InVariable) const
{
	check(!HasAnyFlags(RF_NeedPostLoad));

	if (const UNiagaraSystem* NiagaraSystem = GetTypedOuter<UNiagaraSystem>())
	{
		// make sure that this isn't called before our dependents are fully loaded
		check(!NiagaraSystem->HasAnyFlags(RF_NeedPostLoad));

		return NiagaraSystem->CanObtainUserVariable(InVariable);
	}
	return false;
}
