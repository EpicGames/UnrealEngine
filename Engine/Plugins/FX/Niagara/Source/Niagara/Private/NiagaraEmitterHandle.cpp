// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEmitterHandle.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitter.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "NiagaraScriptSourceBase.h"
#include "NiagaraCommon.h"
#include "NiagaraCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEmitterHandle)

const FNiagaraEmitterHandle FNiagaraEmitterHandle::InvalidHandle;

FNiagaraEmitterHandle::FNiagaraEmitterHandle() 
	: bIsEnabled(true)
#if WITH_EDITORONLY_DATA
	, Source_DEPRECATED(nullptr)
	, LastMergedSource_DEPRECATED(nullptr)
	, bIsolated(false)
	, Instance_DEPRECATED(nullptr)
#endif
{
}

#if WITH_EDITORONLY_DATA
FNiagaraEmitterHandle::FNiagaraEmitterHandle(UNiagaraEmitter& InEmitter, const FGuid& Version)
	: Name(*InEmitter.GetUniqueEmitterName())
	, Id(FGuid::NewGuid())
	, IdName(*Id.ToString())
	, bIsEnabled(true)
	, EmitterMode(ENiagaraEmitterMode::Standard)
	, Source_DEPRECATED(nullptr)
	, LastMergedSource_DEPRECATED(nullptr)
	, bIsolated(false)
	, Instance_DEPRECATED(nullptr)
	, VersionedInstance(FVersionedNiagaraEmitter(&InEmitter, Version))
{
}

FNiagaraEmitterHandle::FNiagaraEmitterHandle(const FVersionedNiagaraEmitter& InEmitter)
	: Name(*InEmitter.Emitter->GetUniqueEmitterName())
	, Id(FGuid::NewGuid())
	, IdName(*Id.ToString())
	, bIsEnabled(true)
	, EmitterMode(ENiagaraEmitterMode::Standard)
	, Source_DEPRECATED(nullptr)
	, LastMergedSource_DEPRECATED(nullptr)
	, bIsolated(false)
	, Instance_DEPRECATED(nullptr)
	, VersionedInstance(InEmitter)
{
}

//-TODO:Stateless
FNiagaraEmitterHandle::FNiagaraEmitterHandle(UNiagaraStatelessEmitter& InEmitter)
	: Name(*InEmitter.GetUniqueEmitterName())
	, Id(FGuid::NewGuid())
	, IdName(*Id.ToString())
	, bIsEnabled(true)
	, EmitterMode(ENiagaraEmitterMode::Stateless)
	, Source_DEPRECATED(nullptr)
	, LastMergedSource_DEPRECATED(nullptr)
	, bIsolated(false)
	, Instance_DEPRECATED(nullptr)
	, StatelessEmitter(&InEmitter)
{
}
//-TODO:Stateless
#endif

bool FNiagaraEmitterHandle::IsValid() const
{
	return Id.IsValid();
}

FGuid FNiagaraEmitterHandle::GetId() const
{
	return Id;
}

FName FNiagaraEmitterHandle::GetIdName() const
{
	return IdName;
}

FName FNiagaraEmitterHandle::GetName() const
{
	return Name;
}

void FNiagaraEmitterHandle::SetName(FName InName, UNiagaraSystem& InOwnerSystem)
{
	FName SanitizedName = *FNiagaraUtilities::SanitizeNameForObjectsAndPackages(InName.ToString());
	if (SanitizedName.IsEqual(Name, ENameCase::CaseSensitive, false))
	{
		return;
	}

	TSet<FName> OtherEmitterNames;
	for (const FNiagaraEmitterHandle& OtherEmitterHandle : InOwnerSystem.GetEmitterHandles())
	{
		if (OtherEmitterHandle.GetId() != GetId())
		{
			OtherEmitterNames.Add(OtherEmitterHandle.GetName());
		}
	}
	FName UniqueName = FNiagaraUtilities::GetUniqueName(SanitizedName, OtherEmitterNames);

	Name = UniqueName;

	if (StatelessEmitter != nullptr)
	{
		StatelessEmitter->SetUniqueEmitterName(Name.ToString());
	}

	if (VersionedInstance.Emitter && VersionedInstance.Emitter->SetUniqueEmitterName(Name.ToString()))
	{
 #if WITH_EDITOR
		if (InOwnerSystem.GetSystemSpawnScript() && InOwnerSystem.GetSystemSpawnScript()->GetLatestSource())
		{
			// Just invalidate the system scripts here. The emitter scripts have their important variables 
			// changed in the SetUniqueEmitterName method above.
			InOwnerSystem.GetSystemSpawnScript()->GetLatestSource()->MarkNotSynchronized(TEXT("EmitterHandleRenamed"));
		}
#endif
	}
}

bool FNiagaraEmitterHandle::GetIsEnabled() const
{
	return bIsEnabled;
}

bool FNiagaraEmitterHandle::SetIsEnabled(bool bInIsEnabled, UNiagaraSystem& InOwnerSystem, bool bRecompileIfChanged)
{
	if (bIsEnabled != bInIsEnabled)
	{
		bIsEnabled = bInIsEnabled;

#if WITH_EDITOR
		if (InOwnerSystem.GetSystemSpawnScript() && InOwnerSystem.GetSystemSpawnScript()->GetLatestSource())
		{
			// We need to get the NiagaraNodeEmitters to update their enabled state based on what happened.
			InOwnerSystem.GetSystemSpawnScript()->GetLatestSource()->RefreshFromExternalChanges();

			// Need to cause us to recompile in the future if necessary...
			FString InvalidateReason = TEXT("Emitter enabled changed.");
			InOwnerSystem.GetSystemSpawnScript()->InvalidateCompileResults(InvalidateReason);
			InOwnerSystem.GetSystemUpdateScript()->InvalidateCompileResults(InvalidateReason);

			// Clean out the emitter's compile results for cleanliness.
			FVersionedNiagaraEmitterData* EmitterData = VersionedInstance.GetEmitterData();
			if (EmitterData)
			{
				EmitterData->InvalidateCompileResults();
			}

			// In some cases we may do the recompile now.
			if (bRecompileIfChanged)
			{
				InOwnerSystem.RequestCompile(false);
			}
		}
#endif
		return true;
	}
	return false;
}

bool FNiagaraEmitterHandle::IsAllowedByScalability() const
{
	if (EmitterMode == ENiagaraEmitterMode::Standard)
	{
		FVersionedNiagaraEmitterData* EmitterData = VersionedInstance.GetEmitterData();
		return EmitterData ? EmitterData->IsAllowedByScalability() : false;
	}
	else
	{
		return StatelessEmitter ? StatelessEmitter->IsAllowedByScalability() : false;
	}
}

#if WITH_EDITORONLY_DATA
bool FNiagaraEmitterHandle::IsEnabledOnEffectQualityLevel(int32 QualityLevel) const
{
	if (EmitterMode == ENiagaraEmitterMode::Standard)
	{
		FVersionedNiagaraEmitterData* EmitterData = VersionedInstance.GetEmitterData();
		return EmitterData ? EmitterData->Platforms.IsEffectQualityEnabled(QualityLevel) : false;
	}
	else
	{
		return StatelessEmitter ? StatelessEmitter->GetPlatformSet().IsEffectQualityEnabled(QualityLevel) : false;
	}
}

const FNiagaraPlatformSet* FNiagaraEmitterHandle::GetPlatformSet() const
{
	if (EmitterMode == ENiagaraEmitterMode::Standard)
	{
		FVersionedNiagaraEmitterData* EmitterData = VersionedInstance.GetEmitterData();
		return EmitterData ? &EmitterData->Platforms : nullptr;
	}
	else
	{
		return StatelessEmitter ? &StatelessEmitter->GetPlatformSet() : nullptr;
	}
}

void FNiagaraEmitterHandle::SetEmitterMode(UNiagaraSystem& InOwningSystem, ENiagaraEmitterMode InEmitterMode)
{
	if (EmitterMode != InEmitterMode)
	{
		EmitterMode = InEmitterMode;
		if (EmitterMode == ENiagaraEmitterMode::Stateless && StatelessEmitter == nullptr)
		{
			StatelessEmitter = NewObject<UNiagaraStatelessEmitter>(&InOwningSystem, "StatelessEmitter", RF_Transactional);
			StatelessEmitter->SetUniqueEmitterName(*Name.ToString());
			StatelessEmitter->CacheFromCompiledData();
		}
		InOwningSystem.RequestCompile(false);
		OnEmitterModeChangedDelegate.Broadcast();
	}
}
#endif

const FVersionedNiagaraEmitter FNiagaraEmitterHandle::GetInstance() const
{
	return VersionedInstance;
}

void FNiagaraEmitterHandle::SetInstance(const FVersionedNiagaraEmitter& VersionedData)
{
	VersionedInstance = VersionedData;
}

void FNiagaraEmitterHandle::SetInstanceVersion(const FGuid& InVersion)
{
	VersionedInstance.Version = InVersion;
}

FVersionedNiagaraEmitterData* FNiagaraEmitterHandle::GetEmitterData() const
{
	return EmitterMode == ENiagaraEmitterMode::Standard	? VersionedInstance.GetEmitterData() : nullptr;
}

UNiagaraEmitterBase* FNiagaraEmitterHandle::GetEmitterBase() const
{
	return EmitterMode == ENiagaraEmitterMode::Stateless ? StaticCast<UNiagaraEmitterBase*>(StatelessEmitter.Get()) : StaticCast<UNiagaraEmitterBase*>(VersionedInstance.Emitter.Get());
}

FString FNiagaraEmitterHandle::GetUniqueInstanceName() const
{
	// We might not have an instance if this is a cooked object that we're loading in the editor
	if (VersionedInstance.Emitter)
	{
		return VersionedInstance.Emitter->GetUniqueEmitterName();
	}
	//-TODO:Stateless: We need an emitter data adapter / base class
	else if (StatelessEmitter)
	{
		return StatelessEmitter->GetUniqueEmitterName();
	}
	return FString();
}

void FNiagaraEmitterHandle::ForEachEnabledRendererWithIndex(TFunction<void(const class UNiagaraRendererProperties*, int32)> Delegate) const
{
	if (EmitterMode == ENiagaraEmitterMode::Standard)
	{
		if (FVersionedNiagaraEmitterData* EmitterData = VersionedInstance.GetEmitterData())
		{
			EmitterData->ForEachEnabledRendererWithIndex(Delegate);
		}
	}
	else if (StatelessEmitter)
	{
		StatelessEmitter->ForEachEnabledRendererWithIndex(Delegate);
	}
}

#if WITH_EDITORONLY_DATA

bool FNiagaraEmitterHandle::NeedsRecompile() const
{
	if (GetIsEnabled())
	{
		if (FVersionedNiagaraEmitterData* EmitterData = VersionedInstance.GetEmitterData())
		{
			TArray<UNiagaraScript*> Scripts;
			EmitterData->GetScripts(Scripts);

			for (UNiagaraScript* Script : Scripts)
			{
				if (Script->IsCompilable() && !Script->AreScriptAndSourceSynchronized())
				{
					return true;
				}
			}
		}
	}
	return false;
}

void FNiagaraEmitterHandle::ConditionalPostLoad(int32 NiagaraCustomVersion)
{
#if WITH_EDITORONLY_DATA
	if (Instance_DEPRECATED != nullptr)
	{
		VersionedInstance.Emitter = Instance_DEPRECATED;
		VersionedInstance.Version = Instance_DEPRECATED->IsVersioningEnabled() ? Instance_DEPRECATED->GetExposedVersion().VersionGuid : FGuid(); 
	}
#endif

	if (UNiagaraEmitter* InstanceEmitter = VersionedInstance.Emitter)
	{
		InstanceEmitter->ConditionalPostLoad();
		if (NiagaraCustomVersion < FNiagaraCustomVersion::MoveInheritanceDataFromTheEmitterHandleToTheEmitter)
		{
			FVersionedNiagaraEmitterData* EmitterData = GetEmitterData();
			if (Source_DEPRECATED && EmitterData)
			{
				Source_DEPRECATED->ConditionalPostLoad();
				EmitterData->VersionedParent = FVersionedNiagaraEmitter(Source_DEPRECATED, Source_DEPRECATED->GetExposedVersion().VersionGuid);
				Source_DEPRECATED = nullptr;
			}
			if (LastMergedSource_DEPRECATED && EmitterData)
			{
				LastMergedSource_DEPRECATED->ConditionalPostLoad();
				EmitterData->VersionedParentAtLastMerge = FVersionedNiagaraEmitter(LastMergedSource_DEPRECATED, LastMergedSource_DEPRECATED->GetExposedVersion().VersionGuid);
				EmitterData->VersionedParentAtLastMerge.Emitter->Rename(nullptr, InstanceEmitter);
				LastMergedSource_DEPRECATED = nullptr;
			}
		}

		FText Reason;
		if (InstanceEmitter->GetFName().IsValidObjectName(Reason) == false)
		{
			UNiagaraSystem* OwningSystem = InstanceEmitter->GetTypedOuter<UNiagaraSystem>();
			if (OwningSystem != nullptr)
			{
				// If the name isn't a valid object name, set the name again so that it will be properly sanitized.
				SetName(Name, *OwningSystem);
			}
		}

		if (bIsEnabled == false)
		{
			FVersionedNiagaraEmitterData* EmitterData = VersionedInstance.GetEmitterData();
			if (EmitterData != nullptr)
			{
				EmitterData->InvalidateCompileResults();
			}
		}
	}
	if (StatelessEmitter != nullptr)
	{
		StatelessEmitter->ConditionalPostLoad();
	}
}

bool FNiagaraEmitterHandle::UsesEmitter(const FVersionedNiagaraEmitter& InEmitter) const
{
	if (ensure(InEmitter.Emitter))
	{
		if (InEmitter.Emitter->IsVersioningEnabled())
		{
			return VersionedInstance == InEmitter || (VersionedInstance.GetEmitterData() && VersionedInstance.GetEmitterData()->UsesEmitter(*InEmitter.Emitter));
		}
		else
		{
			return FNiagaraEmitterHandle::UsesEmitter(*InEmitter.Emitter);
		}
	}
	return false;
}

bool FNiagaraEmitterHandle::UsesEmitter(const FVersionedNiagaraEmitterBase& InEmitter) const
{
	return FNiagaraEmitterHandle::UsesEmitter(*InEmitter.Emitter);
}

bool FNiagaraEmitterHandle::UsesEmitter(const UNiagaraEmitterBase& InEmitter) const
{
	return VersionedInstance.Emitter == &InEmitter || (VersionedInstance.GetEmitterData() && VersionedInstance.GetEmitterData()->UsesEmitter(static_cast<const UNiagaraEmitter&>(InEmitter)));
}

void FNiagaraEmitterHandle::ClearEmitter()
{
	VersionedInstance = FVersionedNiagaraEmitter();
	Instance_DEPRECATED = nullptr;
	Source_DEPRECATED = nullptr;
	LastMergedSource_DEPRECATED = nullptr;
}


#endif
