// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEmitterInstance.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraUserRedirectionParameterStore.h"

FNiagaraEmitterInstance::FNiagaraEmitterInstance(FNiagaraSystemInstance* InParentSystemInstance)
	: ParentSystemInstance(InParentSystemInstance)
{
}

void FNiagaraEmitterInstance::Init(int32 InEmitterIdx)
{
	check(ParentSystemInstance);
	check(ParticleDataSet);

	EmitterIndex = InEmitterIdx;
}

#if STATS
TStatId FNiagaraEmitterInstance::GetEmitterStatID(bool bGameThread, bool bConcurrent) const
{
	return VersionedEmitter.Emitter ? VersionedEmitter.Emitter->GetStatID(bGameThread, bConcurrent) : TStatId();
}

TStatId FNiagaraEmitterInstance::GetSystemStatID(bool bGameThread, bool bConcurrent) const
{
	return ParentSystemInstance->GetSystem()->GetStatID(bGameThread, bConcurrent);
}
#endif //STATS

int32 FNiagaraEmitterInstance::GetNumParticles() const
{
	// Note: For GPU simulations the data is latent we can not read directly from GetCurrentData() until we have passed a fence
	// which guarantees that at least one tick has occurred inside the batcher.  The count will still technically be incorrect
	// but hopefully adequate for a system script update.
	if (GPUExecContext)
	{
		if (GPUExecContext->ParticleCountReadFence.load(std::memory_order_relaxed) <= GPUExecContext->ParticleCountWriteFence.load(std::memory_order_relaxed))
		{
			// Fence has passed we read directly from the GPU Exec Context which will have the most up-to-date information
			return GPUExecContext->GetCurrentNumInstances();
		}
		// Fence has not been passed return the TotalSpawnedParticles as a 'guess' to the amount
		return TotalSpawnedParticles;
	}

	if (ParticleDataSet->GetCurrentData())
	{
		return ParticleDataSet->GetCurrentData()->GetNumInstances();
	}
	return 0;
}

void FNiagaraEmitterInstance::SetSystemFixedBoundsOverride(FBox SystemFixedBounds)
{
	//-TODO: This is ugly can we detangle it
	CachedSystemFixedBounds = SystemFixedBounds;
}

void FNiagaraEmitterInstance::SetFixedBounds(const FBox& InLocalBounds)
{
	FRWScopeLock ScopeLock(FixedBoundsGuard, SLT_Write);
	FixedBounds = InLocalBounds;
}

FBox FNiagaraEmitterInstance::GetFixedBounds() const
{
	{
		FRWScopeLock ScopeLock(FixedBoundsGuard, SLT_ReadOnly);
		if (FixedBounds.IsValid)
		{
			return FixedBounds;
		}
	}
	//-TODO: FUGLY
	//FVersionedNiagaraEmitterData* EmitterData = CachedEmitter.GetEmitterData();
	//if (EmitterData && EmitterData->CalculateBoundsMode == ENiagaraEmitterCalculateBoundMode::Fixed)
	//{
	//	return EmitterData->FixedBounds;
	//}
	return FBox(ForceInit);
}

#if WITH_EDITORONLY_DATA
bool FNiagaraEmitterInstance::IsDisabledFromIsolation() const
{
	return ParentSystemInstance && ParentSystemInstance->GetIsolateEnabled() && !GetEmitterHandle().IsIsolated();
}
#endif

int64 FNiagaraEmitterInstance::GetTotalBytesUsed() const
{
	int64 ByteSize = 0;
	if (ParticleDataSet)
	{
		ByteSize += ParticleDataSet->GetSizeBytes();
	}
	return ByteSize;
}

const FNiagaraEmitterHandle& FNiagaraEmitterInstance::GetEmitterHandle() const
{
	UNiagaraSystem* Sys = ParentSystemInstance->GetSystem();
	return Sys->GetEmitterHandles()[EmitterIndex];
}

bool FNiagaraEmitterInstance::GetBoundRendererValue_GT(const FNiagaraVariableBase& InBaseVar, const FNiagaraVariableBase& InSubVar, void* OutValueData) const
{
	if (InBaseVar.IsDataInterface())
	{
		UNiagaraDataInterface* UObj = RendererBindings.GetDataInterface(InBaseVar);
		if (UObj && InSubVar.GetName() == NAME_None)
		{
			UNiagaraDataInterface** Var = (UNiagaraDataInterface**)OutValueData;
			*Var = UObj;
			return true;
		}
		else if (UObj && UObj->CanExposeVariables())
		{
			void* PerInstanceData = ParentSystemInstance->FindDataInterfaceInstanceData(UObj);
			return UObj->GetExposedVariableValue(InSubVar, PerInstanceData, ParentSystemInstance, OutValueData);
		}
	}
	else if (InBaseVar.IsUObject())
	{
		UObject* UObj = RendererBindings.GetUObject(InBaseVar);
		UObject** Var = (UObject**)OutValueData;
		*Var = UObj;
		return true;
	}
	else
	{
		const uint8* Data = RendererBindings.GetParameterData(InBaseVar);
		if (Data && InBaseVar.GetSizeInBytes() != 0)
		{
			memcpy(OutValueData, Data, InBaseVar.GetSizeInBytes());
			return true;
		}
	}
	return false;
}

TObjectPtr<UObject> FNiagaraEmitterInstance::FindBinding(const FNiagaraVariable& InVariable) const
{
	if (!InVariable.IsValid())
	{
		return nullptr;
	}

	if (FNiagaraSystemInstance* SystemInstance = GetParentSystemInstance())
	{
		if (FNiagaraUserRedirectionParameterStore* OverrideParameters = SystemInstance->GetOverrideParameters())
		{
			return OverrideParameters->GetUObject(InVariable);
		}
	}
	return nullptr;
}

UNiagaraDataInterface* FNiagaraEmitterInstance::FindDataInterface(const FNiagaraVariable& InVariable) const
{
	if (!InVariable.IsValid())
	{
		return nullptr;
	}

	if (FNiagaraSystemInstance* SystemInstance = GetParentSystemInstance())
	{
		if (FNiagaraUserRedirectionParameterStore* OverrideParameters = SystemInstance->GetOverrideParameters())
		{
			return OverrideParameters->GetDataInterface(InVariable);
		}
	}
	return nullptr;
}

void FNiagaraEmitterInstance::UpdatePSOStatus()
{
	// nothing to do here, can be overriden by the emitter implementation
}

FNiagaraScriptExecutionContext& FNiagaraEmitterInstance::GetSpawnExecutionContext()
{
	FNiagaraEmitterInstanceImpl* StatefulEmitter = AsStateful();
	check(StatefulEmitter);
	return StatefulEmitter->GetSpawnExecutionContext();
}

FNiagaraScriptExecutionContext& FNiagaraEmitterInstance::GetUpdateExecutionContext()
{
	FNiagaraEmitterInstanceImpl* StatefulEmitter = AsStateful();
	check(StatefulEmitter);
	return StatefulEmitter->GetUpdateExecutionContext();
}

TArrayView<FNiagaraScriptExecutionContext> FNiagaraEmitterInstance::GetEventExecutionContexts()
{
	FNiagaraEmitterInstanceImpl* StatefulEmitter = AsStateful();
	check(StatefulEmitter);
	return StatefulEmitter->GetEventExecutionContexts();
}

TArray<FNiagaraSpawnInfo>& FNiagaraEmitterInstance::GetSpawnInfo()
{
	FNiagaraEmitterInstanceImpl* StatefulEmitter = AsStateful();
	check(StatefulEmitter);
	return StatefulEmitter->GetSpawnInfo();
}

void FNiagaraEmitterInstance::SetParticleComponentActive(FObjectKey ComponentKey, int32 ParticleID) const
{
	const FNiagaraEmitterInstanceImpl* StatefulEmitter = AsStateful();
	check(StatefulEmitter);
	StatefulEmitter->SetParticleComponentActive(ComponentKey, ParticleID);
}

bool FNiagaraEmitterInstance::IsParticleComponentActive(FObjectKey ComponentKey, int32 ParticleID) const
{
	const FNiagaraEmitterInstanceImpl* StatefulEmitter = AsStateful();
	check(StatefulEmitter);
	return StatefulEmitter->IsParticleComponentActive(ComponentKey, ParticleID);
}

bool FNiagaraEmitterInstance::IsReadyToRun() const
{
	const FNiagaraEmitterInstanceImpl* StatefulEmitter = AsStateful();
	check(StatefulEmitter);
	return StatefulEmitter->IsReadyToRun();
}

bool FNiagaraEmitterInstance::HasTicked() const
{
	const FNiagaraEmitterInstanceImpl* StatefulEmitter = AsStateful();
	check(StatefulEmitter);
	return StatefulEmitter->HasTicked();
}

void FNiagaraEmitterInstance::SetExecutionState(ENiagaraExecutionState InState)
{
	FNiagaraEmitterInstanceImpl* StatefulEmitter = AsStateful();
	check(StatefulEmitter);
	return StatefulEmitter->SetExecutionState(InState);
}

#if WITH_EDITOR
void FNiagaraEmitterInstance::CalculateFixedBounds(const FTransform& ToWorldSpace)
{
	FNiagaraEmitterInstanceImpl* StatefulEmitter = AsStateful();
	check(StatefulEmitter);
	return StatefulEmitter->CalculateFixedBounds(ToWorldSpace);
}
#endif

float FNiagaraEmitterInstance::GetTotalCPUTimeMS() const
{
#if WITH_EDITORONLY_DATA
	return FPlatformTime::ToMilliseconds(TickTimeCycles);
#else
	return 0.0f;
#endif
}

FName FNiagaraEmitterInstance::GetCachedIDName() const
{
	const FNiagaraEmitterInstanceImpl* StatefulEmitter = AsStateful();
	check(StatefulEmitter);
	return StatefulEmitter->GetCachedIDName();
}
