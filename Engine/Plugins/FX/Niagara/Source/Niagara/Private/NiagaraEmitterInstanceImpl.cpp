// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEmitterInstanceImpl.h"
#include "Engine/Engine.h"
#include "NiagaraBoundsCalculator.h"
#include "NiagaraStats.h"
#include "NiagaraConstants.h"
#include "NiagaraModule.h"
#include "NiagaraParameterCollectionResolver.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraDataInterface.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraScriptExecutionContext.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraComponentSettings.h"
#include "NiagaraSystemSimulation.h"

DECLARE_DWORD_COUNTER_STAT(TEXT("Num Custom Events"), STAT_NiagaraNumCustomEvents, STATGROUP_Niagara);

//DECLARE_CYCLE_STAT(TEXT("Tick"), STAT_NiagaraTick, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Emitter Simulate [CNC]"), STAT_NiagaraSimulate, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Emitter Spawn [CNC]"), STAT_NiagaraSpawn, STATGROUP_Niagara);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Emitter Post Tick [CNC]"), STAT_NiagaraEmitterPostTick, STATGROUP_Niagara, EStatFlags::Verbose);
DECLARE_CYCLE_STAT(TEXT("Emitter Event Handling [CNC]"), STAT_NiagaraEventHandle, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Emitter Event CopyBuffer [CNC]"), STAT_NiagaraEvent_CopyBuffer, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Emitter Error Check [CNC]"), STAT_NiagaraEmitterErrorCheck, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Init Emitters [GT]"), STAT_NiagaraEmitterInit, STATGROUP_Niagara);

static int32 GbDumpParticleData = 0;
static FAutoConsoleVariableRef CVarNiagaraDumpParticleData(
	TEXT("fx.DumpParticleData"),
	GbDumpParticleData,
	TEXT("If > 0 current frame particle data will be dumped after simulation. \n"),
	ECVF_Default
	);

static int32 GbNiagaraDumpNans = 0;
static FAutoConsoleVariableRef CVarNiagaraDumpNans(
	TEXT("fx.Niagara.DumpNans"),
	GbNiagaraDumpNans,
	TEXT("If not 0 any NaNs will be dumped always.\n"),
	ECVF_Default
);

static int32 GbNiagaraDumpNansOnce = 0;
static FAutoConsoleVariableRef CVarNiagaraDumpNansOnce(
	TEXT("fx.Niagara.DumpNansOnce"),
	GbNiagaraDumpNansOnce,
	TEXT("If not 0 any NaNs will be dumped for the first emitter that encounters NaNs.\n"),
	ECVF_Default
);

static int32 GbNiagaraShowAllocationWarnings = 0;
static FAutoConsoleVariableRef CVarNiagaraShowAllocationWarnings(
	TEXT("fx.Niagara.ShowAllocationWarnings"),
	GbNiagaraShowAllocationWarnings,
	TEXT("If not 0 then frequent reallocations and over-allocations of particle memory will cause warnings in the log.\n"),
	ECVF_Default
);

/**
TODO: This is mainly to avoid hard limits in our storage/alloc code etc rather than for perf reasons.
We should improve our hard limit/safety code and possibly add a max for perf reasons.
*/
static int32 GMaxNiagaraCPUParticlesPerEmitter = 1000000;
static FAutoConsoleVariableRef CVarMaxNiagaraCPUParticlesPerEmitter(
	TEXT("fx.MaxNiagaraCPUParticlesPerEmitter"),
	GMaxNiagaraCPUParticlesPerEmitter,
	TEXT("The max number of supported CPU particles per emitter in Niagara. \n"),
	ECVF_Default
);

static int32 GMaxNiagaraGPUParticlesSpawnPerFrame = 2000000;
static FAutoConsoleVariableRef CVarMaxNiagaraGPUParticlesSpawnPerFrame(
	TEXT("fx.MaxNiagaraGPUParticlesSpawnPerFrame"),
	GMaxNiagaraGPUParticlesSpawnPerFrame,
	TEXT("The max number of GPU particles we expect to spawn in a single frame.\n"),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////

#if WITH_EDITORONLY_DATA
template<bool bAccumulate>
struct FNiagaraEditorOnlyCycleTimer
{
	FORCEINLINE FNiagaraEditorOnlyCycleTimer(uint32& InCyclesOut)
		: CyclesOut(InCyclesOut)
		, StartCycles(FPlatformTime::Cycles())
	{
	}

	FORCEINLINE ~FNiagaraEditorOnlyCycleTimer()
	{
		uint32 DeltaCycles = FPlatformTime::Cycles() - StartCycles;
		if (bAccumulate)
		{
			CyclesOut += DeltaCycles;
		}
		else
		{
			CyclesOut = DeltaCycles;
		}
	}

	uint32& CyclesOut;
	uint32 StartCycles;
};
#endif

//////////////////////////////////////////////////////////////////////////

FNiagaraEmitterInstanceImpl::FNiagaraEmitterInstanceImpl(FNiagaraSystemInstance* InParentSystemInstance)
	: FNiagaraEmitterInstance(InParentSystemInstance)
{
	ParticleDataSet = new FNiagaraDataSet();

	if (FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = ParentSystemInstance->GetComputeDispatchInterface())
	{
		ComputeDispatchInterfacePtr = StaticCastSharedRef<FNiagaraGpuComputeDispatchInterface>(ComputeDispatchInterface->AsShared());
	}
}

FNiagaraEmitterInstanceImpl::~FNiagaraEmitterInstanceImpl()
{
	// Clear the cached emitter as it is not safe to access the CacheEmitter due to deferred deleted which can happen after the VersionedEmitter has been GCed
	VersionedEmitter = FVersionedNiagaraEmitter();

	//UE_LOGF(LogNiagara, Warning, "~Simulator %p", this);
	CachedBounds.Init();
	UnbindParameters(false);

	if (GPUExecContext != nullptr)
	{
		//This has down stream stores now too so we need to unbind them here otherwise we'll get a in the dtor on the RT.
		GPUExecContext->CombinedParamStore.UnbindAll();

		/** We defer the deletion of the particle dataset and the compute context to the RT to be sure all in-flight RT commands have finished using it.*/
		ENQUEUE_RENDER_COMMAND(FDeleteContextCommand)(
			[ExecContext=GPUExecContext, DataSet=ParticleDataSet](FRHICommandListImmediate& RHICmdList)
			{
				if ( ExecContext != nullptr )
				{
					delete ExecContext;
				}
				if ( DataSet != nullptr )
				{
					delete DataSet;
				}
			}
		);
			
		GPUExecContext = nullptr;
		ParticleDataSet = nullptr;
	}
	else
	{
		if ( ParticleDataSet != nullptr )
		{
			delete ParticleDataSet;
			ParticleDataSet = nullptr;
		}
	}
}

TArrayView<FNiagaraScriptExecutionContext> FNiagaraEmitterInstanceImpl::GetEventExecutionContexts()
{
	if (EventInstanceData.IsValid())
	{
		return MakeArrayView(EventInstanceData->EventExecContexts);
	}
	return MakeArrayView<FNiagaraScriptExecutionContext>(nullptr, 0);
}

bool FNiagaraEmitterInstanceImpl::IsReadyToRun() const
{
	if (!IsDisabled() && !VersionedEmitter.GetEmitterData()->IsReadyToRun())
	{
		return false;
	}

	return true;
}

void FNiagaraEmitterInstanceImpl::Dump()const
{
	if (IsDisabled())
	{
		return;
	}

	UE_LOGF(LogNiagara, Log, "==  %ls ========", *VersionedEmitter.Emitter->GetUniqueEmitterName());
	UE_LOGF(LogNiagara, Log, ".................Spawn.................");
	SpawnExecContext.Parameters.DumpParameters(true);
	UE_LOGF(LogNiagara, Log, ".................Update.................");
	UpdateExecContext.Parameters.DumpParameters(true);
	if (VersionedEmitter.GetEmitterData()->SimTarget == ENiagaraSimTarget::GPUComputeSim && GPUExecContext != nullptr)
	{
		UE_LOGF(LogNiagara, Log, "................. %ls Combined Parameters .................", TEXT("GPU Script"));
		GPUExecContext->CombinedParamStore.DumpParameters();
		//-TODO: Add dump for GPU particles
	}
	else
	{
		ParticleDataSet->Dump(0, INDEX_NONE, TEXT("Particle Data"), TEXT("UniqueID"));
	}
}

bool FNiagaraEmitterInstanceImpl::IsAllowedToExecute() const
{
	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	return GetEmitterHandle().GetIsEnabled() && EmitterData && EmitterData->IsAllowedToExecute();
}

void FNiagaraEmitterInstanceImpl::Init(int32 InEmitterIndex)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEmitterInit);

	Super::Init(InEmitterIndex);

	const FNiagaraEmitterHandle& EmitterHandle = GetEmitterHandle();
	VersionedEmitter = EmitterHandle.GetInstance();
	CachedIDName = EmitterHandle.GetIdName();

	MaxAllocationCount = 0;
	ReallocationCount = 0;
	MinOverallocation = -1;

	bResetPending = false;

	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	if (EmitterData == nullptr)
	{
		//@todo(message manager) Error bubbling here
		ExecutionState = ENiagaraExecutionState::Disabled;
		return;
	}

	bLocalSpace = EmitterData->bLocalSpace;
	bDeterministic = EmitterData->bDeterminism;
	bNeedsPartialDepthTexture = EmitterData->NeedsPartialDepthTexture();
	bNeedsEarlyViewUniformBuffer = EmitterData->RequiresViewUniformBuffer();
	SimTarget = EmitterData->SimTarget;

	RandomSeed = EmitterData->RandomSeed + ParentSystemInstance->GetRandomSeedOffset();

	MaxAllocationCount = EmitterData->GetMaxParticleCountEstimate();
	if (!IsAllowedToExecute())
	{
		ExecutionState = ENiagaraExecutionState::Disabled;
		return;
	}
	
	const TArray<TSharedRef<const FNiagaraEmitterCompiledData>>& EmitterCompiledDatas = ParentSystemInstance->GetSystem()->GetEmitterCompiledData();
	if (EmitterCompiledDatas.IsValidIndex(EmitterIndex) == false)
	{
		//@todo(message manager) Error bubbling here
		ExecutionState = ENiagaraExecutionState::Disabled;
		return;
	}

	EmitterCompiledData = EmitterCompiledDatas[EmitterIndex];

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST 
	CheckForErrors();
#endif

	if (IsDisabled())
	{
		return;
	}

	//Init the spawn infos to the correct number for this system.
	SpawnInfos.SetNum(EmitterCompiledData->SpawnAttributes.Num());

	{
		ParticleDataSet->Init(&EmitterCompiledData->DataSetCompiledData);

		// We do not need to kill the existing particles as we will have none
		ResetSimulation(false);

		//Warn the user if there are any attributes used in the update script that are not initialized in the spawn script.
		//TODO: We need some window in the System editor and possibly the graph editor for warnings and errors.

		const bool bVerboseAttributeLogging = false;

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST 
		if (bVerboseAttributeLogging)
		{
			for (const FNiagaraVariableBase& Attr : EmitterData->UpdateScriptProps.Script->GetVMExecutableData().Attributes)
			{
				int32 FoundIdx;
				if (!EmitterData->SpawnScriptProps.Script->GetVMExecutableData().Attributes.Find(Attr, FoundIdx))
				{
					UE_LOGF(LogNiagara, Warning, "Attribute %ls is used in the Update script for %ls but it is not initialised in the Spawn script!", *Attr.GetName().ToString(), *EmitterHandle.GetName().ToString());
				}
				for (int32 i = 0; i < EmitterData->GetEventHandlers().Num(); i++)
				{
					if (EmitterData->GetEventHandlers()[i].Script && !EmitterData->GetEventHandlers()[i].Script->GetVMExecutableData().Attributes.Find(Attr, FoundIdx))
					{
						UE_LOGF(LogNiagara, Warning, "Attribute %ls is used in the event handler script for %ls but it is not initialised in the Spawn script!", *Attr.GetName().ToString(), *EmitterHandle.GetName().ToString());
					}
				}
			}
		}
#endif
	}

	UNiagaraSystem* ParentSystem = ParentSystemInstance->GetSystem();

	{
		const int32 UpdateEventGeneratorCount = EmitterData->UpdateScriptProps.EventGenerators.Num();
		const int32 SpawnEventGeneratorCount = EmitterData->SpawnScriptProps.EventGenerators.Num();
		const int32 NumEvents = EmitterData->GetEventHandlers().Num();

		if (UpdateEventGeneratorCount || SpawnEventGeneratorCount || NumEvents)
		{
			EventInstanceData = MakeUnique<FEventInstanceData>();
			EventInstanceData->UpdateScriptEventDataSets.Empty(UpdateEventGeneratorCount);
			EventInstanceData->UpdateEventGeneratorIsSharedByIndex.SetNumZeroed(UpdateEventGeneratorCount);
			int32 UpdateEventGeneratorIndex = 0;
			for (const FNiagaraEventGeneratorProperties &GeneratorProps : EmitterData->UpdateScriptProps.EventGenerators)
			{
				FNiagaraDataSet *Set = ParentSystemInstance->CreateEventDataSet(EmitterHandle.GetIdName(), GeneratorProps.ID);
				Set->Init(&GeneratorProps.DataSetCompiledData);

				EventInstanceData->UpdateScriptEventDataSets.Add(Set);
				EventInstanceData->UpdateEventGeneratorIsSharedByIndex[UpdateEventGeneratorIndex] = EmitterData->IsEventGeneratorShared(GeneratorProps.ID);
				++UpdateEventGeneratorIndex;
			}

			EventInstanceData->SpawnScriptEventDataSets.Empty(SpawnEventGeneratorCount);
			EventInstanceData->SpawnEventGeneratorIsSharedByIndex.SetNumZeroed(SpawnEventGeneratorCount);
			int32 SpawnEventGeneratorIndex = 0;
			for (const FNiagaraEventGeneratorProperties &GeneratorProps : EmitterData->SpawnScriptProps.EventGenerators)
			{
				FNiagaraDataSet *Set = ParentSystemInstance->CreateEventDataSet(EmitterHandle.GetIdName(), GeneratorProps.ID);
				Set->Init(&GeneratorProps.DataSetCompiledData);

				EventInstanceData->SpawnScriptEventDataSets.Add(Set);
				EventInstanceData->SpawnEventGeneratorIsSharedByIndex[SpawnEventGeneratorIndex] = EmitterData->IsEventGeneratorShared(GeneratorProps.ID);
				++SpawnEventGeneratorIndex;
			}

			EventInstanceData->EventExecContexts.SetNum(NumEvents);
			EventInstanceData->EventExecCountBindings.SetNum(NumEvents);

			for (int32 i = 0; i < NumEvents; i++)
			{
				UNiagaraScript* EventScript = EmitterData->GetEventHandlers()[i].Script;

				//This is cpu explicitly? Are we doing event handlers on GPU?
				EventInstanceData->EventExecContexts[i].Init(ParentSystemInstance, EventScript, ParentSystem->GetScriptRuntimeData(&EmitterHandle, *EventScript));
				EventInstanceData->EventExecCountBindings[i].Init(EventInstanceData->EventExecContexts[i].Parameters, SYS_PARAM_ENGINE_EXEC_COUNT);
			}
		}
	}

	{
		SpawnExecContext.Init(ParentSystemInstance, EmitterData->SpawnScriptProps.Script, ParentSystem->GetScriptRuntimeData(&EmitterHandle, *EmitterData->SpawnScriptProps.Script));
		UpdateExecContext.Init(ParentSystemInstance, EmitterData->UpdateScriptProps.Script, ParentSystem->GetScriptRuntimeData(&EmitterHandle, *EmitterData->UpdateScriptProps.Script));

		// setup the parameer store for the GPU execution context; since spawn and update are combined here, we build one with params from both script props
		if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim && EmitterData->GetGPUComputeScript() != nullptr)
		{
			GPUExecContext = new FNiagaraComputeExecutionContext();
			GPUExecContext->InitParams(EmitterData->GetGPUComputeScript(), ParentSystem->GetScriptRuntimeData(&EmitterHandle, *EmitterData->GetGPUComputeScript()), EmitterData->GetSimStageExcecutionData());
			GPUExecContext->SetDebugSimName(EmitterData->GetDebugSimName(), ParentSystemInstance ? ParentSystemInstance->GetSystem()->GetFName() : NAME_None);
			GPUExecContext->ProfilingComponentPtr = ParentSystemInstance ? ParentSystemInstance->GetAttachComponent() : nullptr;
			GPUExecContext->ProfilingEmitterPtr = VersionedEmitter.ToWeakPtr();
			GPUExecContext->MainDataSet = ParticleDataSet;
			GPUExecContext->GPUScript_RT = EmitterData->GetGPUComputeScript()->GetRenderThreadScript();
		#if STATS
			GPUExecContext->SystemStatID = GetSystemStatID(false, false);
			GPUExecContext->EmitterStatID = GetEmitterStatID(false, false);
		#endif
			GPUDataBufferInterfaces = GPUExecContext;
		}
	}

	//Setup direct bindings for setting parameter values.
	{
		//Setup direct bindings for setting parameter values.
		SpawnIntervalBinding.Init(SpawnExecContext.Parameters, EmitterCompiledData->EmitterSpawnIntervalVar);
		InterpSpawnStartBinding.Init(SpawnExecContext.Parameters, EmitterCompiledData->EmitterInterpSpawnStartDTVar);
		SpawnGroupBinding.Init(SpawnExecContext.Parameters, EmitterCompiledData->EmitterSpawnGroupVar);

		// Initialize the exec count
		SpawnExecCountBinding.Init(SpawnExecContext.Parameters, SYS_PARAM_ENGINE_EXEC_COUNT);
		UpdateExecCountBinding.Init(UpdateExecContext.Parameters, SYS_PARAM_ENGINE_EXEC_COUNT);
	}

	{
		// Collect script defined data interface parameters.
		TArray<UNiagaraScript*, TInlineAllocator<8>> Scripts;
		Scripts.Add(EmitterData->SpawnScriptProps.Script);
		Scripts.Add(EmitterData->UpdateScriptProps.Script);
		for (const FNiagaraEventScriptProperties& EventHandler : EmitterData->GetEventHandlers())
		{
			Scripts.Add(EventHandler.Script);
		}
		for (const UNiagaraSimulationStageBase* SimStage : EmitterData->GetSimulationStages())
		{
			Scripts.Add(SimStage->Script);
		}

		if (EventInstanceData.IsValid())
		{
			const int32 NumEventHandlers = EmitterData->GetEventHandlers().Num();
			EventInstanceData->EventHandlingInfo.Reset();
			EventInstanceData->EventHandlingInfo.SetNum(NumEventHandlers);
			for (int32 i = 0; i < NumEventHandlers; i++)
			{
				const FNiagaraEventScriptProperties& EventHandlerProps = EmitterData->GetEventHandlers()[i];
				FNiagaraEventHandlingInfo& Info = EventInstanceData->EventHandlingInfo[i];
				Info.SourceEmitterGuid = EventHandlerProps.SourceEmitterID;
				Info.SourceEmitterName = Info.SourceEmitterGuid.IsValid() ? *Info.SourceEmitterGuid.ToString() : CachedIDName;
				Info.SpawnCounts.Reset();
				Info.TotalSpawnCount = 0;
				Info.EventData = nullptr;
			}
		}

		// We may need to populate bindings that will be used in rendering
		RendererBindings.UnbindAll();

		const bool bAnyRendererBindingsAdded = EmitterData->BuildParameterStoreRendererBindings(RendererBindings);

		if (bAnyRendererBindingsAdded)
		{
			if (ParentSystemInstance)
			{
				ParentSystemInstance->BindToParameterStore(RendererBindings);
			}

			if (GPUExecContext && EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
			{
				GPUExecContext->CombinedParamStore.Bind(&RendererBindings);
			}
		}
	}

	UpdatePSOStatus();

	MaxInstanceCount = EmitterData->GetMaxInstanceCount();
	ParticleDataSet->SetMaxInstanceCount(MaxInstanceCount);
	ParticleDataSet->SetMaxAllocationCount(EmitterData->GetMaxAllocationCount());
}

void FNiagaraEmitterInstanceImpl::InitDITickLists()
{
	if(IsDisabled())
	{
		// Make sure that our function tables need to be regenerated...
		SpawnExecContext.GetNDIStageTickHander().Empty();
		UpdateExecContext.GetNDIStageTickHander().Empty();

		for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
		{
			EventContext.GetNDIStageTickHander().Empty();
		}
		return;
	}

	// Make sure that our function tables need to be regenerated...
	SpawnExecContext.InitDITickLists(ParentSystemInstance);
	UpdateExecContext.InitDITickLists(ParentSystemInstance);

	for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
	{
		EventContext.InitDITickLists(ParentSystemInstance);
	}
}


void FNiagaraEmitterInstanceImpl::ResetSimulation(bool bKillExisting /*= true*/)
{
	EmitterAge = 0;
	TickCount = 0;
	InstanceSeed = FGenericPlatformMath::Rand();
	bCachedBoundsDynamic = false;
	CachedBounds.Init();
	ParticlesWithComponents.Empty();

	if (MinOverallocation > 100 && GbNiagaraShowAllocationWarnings)
	{
		FString SystemName = this->GetParentSystemInstance()->GetSystem()->GetName();
		FString FullName = SystemName + "::" + this->GetEmitterHandle().GetName().ToString();
		UE_LOGF(LogNiagara, Warning, "The emitter %ls over-allocated %i particles during its runtime. If this happens frequently, consider setting the emitter's AllocationMode property to 'manual' to improve runtime performance.", *FullName, MinOverallocation);
	}

	if (IsDisabled())
	{
		return;
	}

	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	check(EmitterData);
	RandomSeed = EmitterData->RandomSeed + ParentSystemInstance->GetRandomSeedOffset();

	SetExecutionState(ENiagaraExecutionState::Active);

	if (bKillExisting)
	{
		bResetPending = true;
		TotalSpawnedParticles = 0;
	}
}

void FNiagaraEmitterInstanceImpl::OnPooledReuse()
{
	// Ensure we kill any existing particles and mark our buffers for reset
	bResetPending = true;
	TotalSpawnedParticles = 0;
	bAllowSpawning_GT = true;
	bAllowSpawning_CNC = true;

	FixedBounds.Init();
}

void FNiagaraEmitterInstanceImpl::SetParticleComponentActive(FObjectKey ComponentKey, int32 ParticleID) const
{
	ParticlesWithComponents.FindOrAdd(ComponentKey).Add(ParticleID);
}

bool FNiagaraEmitterInstanceImpl::IsParticleComponentActive(FObjectKey ComponentKey, int32 ParticleID) const
{
	return ParticlesWithComponents.FindOrAdd(ComponentKey).Contains(ParticleID);
}

void FNiagaraEmitterInstanceImpl::CheckForErrors()
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEmitterErrorCheck);
	
	checkSlow(VersionedEmitter.Emitter);

	//Check for various failure conditions and bail.
	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	if (!EmitterData->UpdateScriptProps.Script || !EmitterData->SpawnScriptProps.Script )
	{
		//TODO - Arbitrary named scripts. Would need some base functionality for Spawn/Udpate to be called that can be overriden in BPs for emitters with custom scripts.
		UE_LOGF(LogNiagara, Error, "Emitter %ls cannot be enabled because it's doesn't have both an update and spawn script.", *VersionedEmitter.Emitter->GetFullName());
		SetExecutionState(ENiagaraExecutionState::Disabled);
		return;
	}

	if (!EmitterData->UpdateScriptProps.Script->IsReadyToRun(ENiagaraSimTarget::CPUSim) || !EmitterData->SpawnScriptProps.Script->IsReadyToRun(ENiagaraSimTarget::CPUSim))
	{
		//TODO - Arbitrary named scripts. Would need some base functionality for Spawn/Udpate to be called that can be overriden in BPs for emitters with custom scripts.
		UE_LOGF(LogNiagara, Error, "Emitter %ls cannot be enabled because it's doesn't have both an update and spawn script ready to run CPU scripts.", *VersionedEmitter.Emitter->GetFullName());
		SetExecutionState(ENiagaraExecutionState::Disabled);
		return;
	}

	if (EmitterData->SpawnScriptProps.Script->GetVMExecutableData().DataUsage.bReadsAttributeData)
	{
		UE_LOGF(LogNiagara, Error, "%ls reads attribute data and so cannot be used as a spawn script. The data being read would be invalid.", *EmitterData->SpawnScriptProps.Script->GetName());
		SetExecutionState(ENiagaraExecutionState::Disabled);
		return;
	}
	if (EmitterData->UpdateScriptProps.Script->GetVMExecutableData().Attributes.Num() == 0 || EmitterData->SpawnScriptProps.Script->GetVMExecutableData().Attributes.Num() == 0)
	{
		UE_LOGF(LogNiagara, Error, "This emitter cannot be enabled because its spawn or update script doesn't have any attributes..");
		SetExecutionState(ENiagaraExecutionState::Disabled);
		return;
	}

	if (EmitterData->SimTarget == ENiagaraSimTarget::CPUSim)
	{
		bool bFailed = false;
		if (!EmitterData->SpawnScriptProps.Script->DidScriptCompilationSucceed(false))
		{
			bFailed = true;
			UE_LOGF(LogNiagara, Error, "This emitter cannot be enabled because its CPU Spawn script failed to compile.");
		}

		if (!EmitterData->UpdateScriptProps.Script->DidScriptCompilationSucceed(false))
		{
			bFailed = true;
			UE_LOGF(LogNiagara, Error, "This emitter cannot be enabled because its CPU Update script failed to compile.");
		}

		if (EmitterData->GetEventHandlers().Num() != 0)
		{
			for (int32 i = 0; i < EmitterData->GetEventHandlers().Num(); i++)
			{
				if (!EmitterData->GetEventHandlers()[i].Script->DidScriptCompilationSucceed(false))
				{
					bFailed = true;
					UE_LOGF(LogNiagara, Error, "This emitter cannot be enabled because one of its CPU Event scripts failed to compile.");
				}
			}
		}

		if (bFailed)
		{
			SetExecutionState(ENiagaraExecutionState::Disabled);
			return;
		}
	}

	if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		if (EmitterData->GetGPUComputeScript()->IsScriptCompilationPending(true))
		{
			UE_LOGF(LogNiagara, Error, "This emitter cannot be enabled because its GPU script hasn't been compiled..");
			SetExecutionState(ENiagaraExecutionState::Disabled);
			return;
		}
		if (!EmitterData->GetGPUComputeScript()->DidScriptCompilationSucceed(true))
		{
			UE_LOGF(LogNiagara, Error, "This emitter cannot be enabled because its GPU script failed to compile.");
			SetExecutionState(ENiagaraExecutionState::Disabled);
			return;
		}
	}
}

void FNiagaraEmitterInstanceImpl::UpdatePSOStatus()
{
	bWaitingForPSO = false;
#if UE_WITH_PSO_PRECACHING
	UNiagaraSystem* ParentSystem = ParentSystemInstance->GetSystem();
	const FNiagaraEmitterHandle& EmitterHandle = GetEmitterHandle();
	if (FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData())
	{
		FMaterialInterfacePSOPrecacheParamsList PSOPrecacheList;
		EmitterHandle.ForEachEnabledRendererWithIndex([&PSOPrecacheList, this](const UNiagaraRendererProperties* Properties, int32)
		{
			Properties->CollectPSOPrecacheData(this, PSOPrecacheList);
		});
		for (FMaterialInterfacePSOPrecacheParams& PSOPrecacheParams : PSOPrecacheList)
		{
			if (ParentSystem->IsLoadingPSOCache(PSOPrecacheParams.MaterialInterface))
			{
				bWaitingForPSO = true;
				break;
			}
		}
	}
#endif
}

void FNiagaraEmitterInstanceImpl::DirtyDataInterfaces()
{
	if (IsDisabled())
	{
		return;
	}

	// Make sure that our function tables need to be regenerated...
	SpawnExecContext.DirtyDataInterfaces();
	UpdateExecContext.DirtyDataInterfaces();

	if (VersionedEmitter.GetEmitterData()->SimTarget == ENiagaraSimTarget::GPUComputeSim && GPUExecContext != nullptr)
	{
		GPUExecContext->DirtyDataInterfaces();
	}

	for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
	{
		EventContext.DirtyDataInterfaces();
	}
}

TConstArrayView<UNiagaraRendererProperties*> FNiagaraEmitterInstanceImpl::GetRenderers() const
{
	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	return EmitterData ? EmitterData->GetRenderers() : MakeArrayView<UNiagaraRendererProperties*>({});
}

void FNiagaraEmitterInstanceImpl::UnbindParameters(bool bExternalOnly)
{
	if (bExternalOnly && !IsDisabled())
	{
		const FNiagaraParameterCollectionResolver ParameterCollectionResolver = ParentSystemInstance->CreateParameterCollectionResolver();
		ParameterCollectionResolver.UnbindCollections(SpawnExecContext.Script, SpawnExecContext.Parameters);
		ParameterCollectionResolver.UnbindCollections(UpdateExecContext.Script, UpdateExecContext.Parameters);

		for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
		{
			ParameterCollectionResolver.UnbindCollections(EventContext.Script, EventContext.Parameters);
		}

		if (FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData())
		{
			FNiagaraScriptInstanceParameterStore& TargetParamStore = EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim ? GPUExecContext->CombinedParamStore : UpdateExecContext.Parameters;
			for (const UNiagaraSimulationStageBase* SimStage : EmitterData->GetSimulationStages())
			{
				if (SimStage->bEnabled == false)
				{
					continue;
				}

				ParameterCollectionResolver.UnbindCollections(SimStage->Script, TargetParamStore);
			}
		}

		ParentSystemInstance->UnbindFromParameterStore(RendererBindings);

		VersionedEmitter.GetEmitterData()->RendererBindings.Unbind(&RendererBindings);
	}
	else
	{
		SpawnExecContext.Parameters.UnbindFromSourceStores();
		UpdateExecContext.Parameters.UnbindFromSourceStores();
		if (GPUExecContext != nullptr)
		{
			GPUExecContext->CombinedParamStore.UnbindFromSourceStores();
		}

		for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
		{
			EventContext.Parameters.UnbindFromSourceStores();
		}
	}
}

void FNiagaraEmitterInstanceImpl::BindParameters(bool bExternalOnly)
{
	if (IsDisabled())
	{
		return;
	}

	const FNiagaraParameterCollectionResolver ParameterCollectionResolver = ParentSystemInstance->CreateParameterCollectionResolver();
	ParameterCollectionResolver.BindCollections(SpawnExecContext.Script, SpawnExecContext.Parameters);
	ParameterCollectionResolver.BindCollections(UpdateExecContext.Script, UpdateExecContext.Parameters);

	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		ParameterCollectionResolver.BindCollections(SpawnExecContext.Script, GPUExecContext->CombinedParamStore);
		ParameterCollectionResolver.BindCollections(UpdateExecContext.Script, GPUExecContext->CombinedParamStore);
	}

	for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
	{
		ParameterCollectionResolver.BindCollections(EventContext.Script, EventContext.Parameters);
	}

	FNiagaraScriptInstanceParameterStore& TargetParamStore = EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim ? GPUExecContext->CombinedParamStore : UpdateExecContext.Parameters;
	for (const UNiagaraSimulationStageBase* SimStage : EmitterData->GetSimulationStages())
	{
		if (SimStage->bEnabled == false)
		{
			continue;
		}

		ParameterCollectionResolver.BindCollections(SimStage->Script, TargetParamStore);
	}

	if (!bExternalOnly)
	{
		//Now bind parameters from the component and system.
		FNiagaraParameterStore& InstanceParams = ParentSystemInstance->GetInstanceParameters();

		InstanceParams.Bind(&SpawnExecContext.Parameters);
		InstanceParams.Bind(&UpdateExecContext.Parameters);

		for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
		{
			InstanceParams.Bind(&EventContext.Parameters);
		}

#if WITH_EDITORONLY_DATA
		EmitterData->SpawnScriptProps.Script->RapidIterationParameters.Bind(&SpawnExecContext.Parameters);
		EmitterData->UpdateScriptProps.Script->RapidIterationParameters.Bind(&UpdateExecContext.Parameters);

		if (EventInstanceData.IsValid())
		{
			ensure(EmitterData->GetEventHandlers().Num() == EventInstanceData->EventExecContexts.Num());
			for (int32 i = 0; i < EmitterData->GetEventHandlers().Num(); i++)
			{
				EmitterData->GetEventHandlers()[i].Script->RapidIterationParameters.Bind(&EventInstanceData->EventExecContexts[i].Parameters);
			}
		}
	#endif

		if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			InstanceParams.Bind(&GPUExecContext->CombinedParamStore);
#if WITH_EDITORONLY_DATA
			EmitterData->SpawnScriptProps.Script->RapidIterationParameters.Bind(&GPUExecContext->CombinedParamStore);
			EmitterData->UpdateScriptProps.Script->RapidIterationParameters.Bind(&GPUExecContext->CombinedParamStore);

			for (int32 i = 0; i < EmitterData->GetSimulationStages().Num(); i++)
			{
				EmitterData->GetSimulationStages()[i]->Script->RapidIterationParameters.Bind(&GPUExecContext->CombinedParamStore);
			}
#endif
		}
	}

	//if (bAnyRendererBindingsAdded)
	{
		if (ParentSystemInstance)
		{
			ParentSystemInstance->BindToParameterStore(RendererBindings);
		}

		VersionedEmitter.GetEmitterData()->RendererBindings.Bind(&RendererBindings);

		//SystemScriptDefinedDataInterfaceParameters.Bind(&RendererBindings);
		//ScriptDefinedDataInterfaceParameters.Bind(&RendererBindings);
	}
}

void FNiagaraEmitterInstanceImpl::RebindParameterCollection(UNiagaraParameterCollectionInstance* OldCollection, UNiagaraParameterCollectionInstance* NewCollection)
{
	if (IsDisabled())
	{
		return;
	}

	const auto RebindToParameterStore =
		[CollectionParent=OldCollection->GetParent(), OldCollection, NewCollection](UNiagaraScript* Script, FNiagaraParameterStore* ParameterStore)
		{
			if (Script && Script->GetCachedParameterCollectionReferences().Contains(CollectionParent))
			{
				OldCollection->GetParameterStore().Unbind(ParameterStore);
				NewCollection->GetParameterStore().Bind(ParameterStore);
			}
		};

	RebindToParameterStore(SpawnExecContext.Script, &SpawnExecContext.Parameters);
	RebindToParameterStore(UpdateExecContext.Script, &UpdateExecContext.Parameters);

	if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		RebindToParameterStore(SpawnExecContext.Script, &GPUExecContext->CombinedParamStore);
		RebindToParameterStore(UpdateExecContext.Script, &GPUExecContext->CombinedParamStore);
	}

	for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
	{
		RebindToParameterStore(EventContext.Script, &EventContext.Parameters);
	}

	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	FNiagaraScriptInstanceParameterStore& TargetParamStore = SimTarget == ENiagaraSimTarget::GPUComputeSim ? GPUExecContext->CombinedParamStore : UpdateExecContext.Parameters;
	for (const UNiagaraSimulationStageBase* SimStage : EmitterData->GetSimulationStages())
	{
		if (SimStage->bEnabled == false)
		{
			continue;
		}

		RebindToParameterStore(SimStage->Script, &TargetParamStore);
	}
}

FBox FNiagaraEmitterInstanceImpl::InternalCalculateDynamicBounds(int32 ParticleCount) const
{
	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	if (!ParticleCount || !EmitterData || !ParentSystemInstance)
	{
		return FBox(ForceInit);
	}

	const auto BoundsCalculators = EmitterData->GetBoundsCalculators();
	if (!BoundsCalculators.Num())
	{
		return FBox(ForceInit);
	}

	FBox Ret;
	Ret.Init();

	const FTransform& Transform = ParentSystemInstance->GetWorldTransform();
	for (const TSharedPtr<FNiagaraBoundsCalculator>& BoundsCalculator : BoundsCalculators)
	{
		Ret += BoundsCalculator->CalculateBounds(Transform, *ParticleDataSet, ParticleCount);
	}

	return Ret;
}

#if WITH_EDITOR
void FNiagaraEmitterInstanceImpl::CalculateFixedBounds(const FTransform& ToWorldSpace)
{
	check(VersionedEmitter.Emitter);
	if (IsComplete())
	{
		return;
	}

	FScopedNiagaraDataSetGPUReadback ScopedGPUReadback;
	int32 NumInstances = 0;
	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		if (GPUExecContext == nullptr)
		{
			return;
		}

		ScopedGPUReadback.ReadbackData(ComputeDispatchInterfacePtr.Get(), GPUExecContext->MainDataSet);
		NumInstances = ScopedGPUReadback.GetNumInstances();
	}
	else
	{
		NumInstances = ParticleDataSet->GetCurrentDataChecked().GetNumInstances();
	}

	if (NumInstances == 0)
	{
		return;
	}

	FBox Bounds = InternalCalculateDynamicBounds(NumInstances);
	if (!Bounds.IsValid)
		return;

	VersionedEmitter.Emitter->Modify();
	EmitterData->CalculateBoundsMode = ENiagaraEmitterCalculateBoundMode::Fixed;
	if (EmitterData->bLocalSpace)
	{
		EmitterData->FixedBounds = Bounds;
	}
	else
	{
		EmitterData->FixedBounds = Bounds.TransformBy(ToWorldSpace);
	}

	bCachedBoundsDynamic = false;
	CachedBounds = Bounds;
}
#endif

/** 
  * Do any post work such as calculating dynamic bounds.
  */
void FNiagaraEmitterInstanceImpl::PostTick()
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEmitterPostTick);

	//We reset the spawn infos to it's minimum size for script spawning.
	//It can exceed this size as external DIs etc can inject additional spawn infos.
	SpawnInfos.SetNum(EmitterCompiledData->SpawnAttributes.Num());

	if (EventInstanceData.IsValid())
	{
		//Clear refs to event data buffers.
		for (FNiagaraEventHandlingInfo& Info : EventInstanceData->EventHandlingInfo)
		{
			Info.SetEventData(nullptr);
		}
	}

	bCachedBoundsDynamic = false;
	CachedBounds.Init();
	
	if (FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData(); ensure(EmitterData))
	{
		// Read lock can be smaller in scope, but probably not necessary
		FRWScopeLock ScopeLock(FixedBoundsGuard, SLT_ReadOnly);
		if (FixedBounds.IsValid)
		{
			CachedBounds = FixedBounds;
		}
		else if (CachedSystemFixedBounds.IsValid)
		{
			CachedBounds = CachedSystemFixedBounds;
		}
		else if ( EmitterData->CalculateBoundsMode == ENiagaraEmitterCalculateBoundMode::Fixed )
		{
			CachedBounds = EmitterData->FixedBounds;
		}
		else if ( EmitterData->CalculateBoundsMode == ENiagaraEmitterCalculateBoundMode::Dynamic )
		{
			// We do not support dynamic bounds for GPU yet
			// Therefore we need to return the default bounds in this case to avoid flickering (some content relies on this path)
			if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
			{
				CachedBounds = FVersionedNiagaraEmitterData::GetDefaultFixedBounds();
			}
			else
			{
				FBox DynamicBounds = InternalCalculateDynamicBounds(ParticleDataSet->GetCurrentDataChecked().GetNumInstances());
				if (DynamicBounds.IsValid)
				{
					bCachedBoundsDynamic = true;
					if (EmitterData->bLocalSpace)
					{
						CachedBounds = DynamicBounds;
					}
					else
					{
						CachedBounds = DynamicBounds.TransformBy(FMatrix(ParentSystemInstance->GetOwnerParameters().EngineWorldToLocal));
					}
				}
			}
		}

#if STATS
		EmitterData->GetStatData().AddStatCapture(TTuple<uint64, ENiagaraScriptUsage>((uint64)this, ENiagaraScriptUsage::ParticleSpawnScript), GetSpawnExecutionContext().ReportStats());
		EmitterData->GetStatData().AddStatCapture(TTuple<uint64, ENiagaraScriptUsage>((uint64)this, ENiagaraScriptUsage::ParticleUpdateScript), GetUpdateExecutionContext().ReportStats());
#endif
	}
}

void FNiagaraEmitterInstanceImpl::SetEmitterEnable(bool bNewEnableState)
{
	bAllowSpawning_GT = bNewEnableState;
}

bool FNiagaraEmitterInstanceImpl::HandleCompletion(bool bForce)
{
	if (IsDisabled())
	{
		return true;
	}

	if (bForce)
	{
		SetExecutionState(ENiagaraExecutionState::Complete);
	}

	if (IsComplete())
	{
		if( GPUExecContext )
		{
			GPUExecContext->Reset(ComputeDispatchInterfacePtr.Get());
		}
		ParticleDataSet->ResetBuffers();
		if (EventInstanceData.IsValid())
		{
			for (FNiagaraDataSet* EventDataSet : EventInstanceData->UpdateScriptEventDataSets)
			{
				EventDataSet->ResetBuffers();
			}

			for (FNiagaraDataSet* EventDataSet : EventInstanceData->SpawnScriptEventDataSets)
			{
				EventDataSet->ResetBuffers();
			}
		}
		return true;
	}

	return false;
}

bool FNiagaraEmitterInstanceImpl::RequiresPersistentIDs() const
{
	//TODO: can we have this be enabled at runtime from outside the system?
	return GetEmitterHandle().GetEmitterData()->RequiresPersistentIDs() || ParticleDataSet->HasVariable(SYS_PARAM_PARTICLES_ID);
}

#if WITH_EDITOR
void FNiagaraEmitterInstanceImpl::TickRapidIterationParameters()
{
	if (IsComplete())
	{
		return;
	}

	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	EmitterData->SpawnScriptProps.Script->RapidIterationParameters.Tick();
	EmitterData->UpdateScriptProps.Script->RapidIterationParameters.Tick();
	if (EventInstanceData.IsValid())
	{
		ensure(EmitterData->GetEventHandlers().Num() == EventInstanceData->EventExecContexts.Num());
		for (int32 i = 0; i < EmitterData->GetEventHandlers().Num(); i++)
		{
			EmitterData->GetEventHandlers()[i].Script->RapidIterationParameters.Tick();
		}
	}
}
#endif

bool FNiagaraEmitterInstanceImpl::ShouldTick() const
{
	return ExecutionState == ENiagaraExecutionState::Active || GetNumParticles() > 0;
}

/** 
  * PreTick - handles killing dead particles, emitter death, and buffer swaps
  */
void FNiagaraEmitterInstanceImpl::PreTick()
{
	if (IsComplete())
	{
		return;
	}
	FVersionedNiagaraEmitterData* NiagaraEmitterData = VersionedEmitter.GetEmitterData();

#if STATS
	FScopeCycleCounter SystemStatCounter(GetEmitterStatID(true, true));
#endif

	checkSlow(ParticleDataSet);
	FNiagaraDataSet& Data = *ParticleDataSet;

	bool bOk = true;
	bOk &= SpawnExecContext.Tick(ParentSystemInstance, NiagaraEmitterData->SimTarget);
	bOk &= UpdateExecContext.Tick(ParentSystemInstance, NiagaraEmitterData->SimTarget);

	// @todo THREADSAFETY We should not tick GPU contexts on the game thread!
	if (NiagaraEmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim && GPUExecContext != nullptr)
	{
		bOk &= GPUExecContext->Tick(ParentSystemInstance);
	}

	for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
	{
		bOk &= EventContext.Tick(ParentSystemInstance, NiagaraEmitterData->SimTarget);
	}

	if (!bOk)
	{
		ResetSimulation();
		SetExecutionState(ENiagaraExecutionState::Disabled);
		return;
	}

	if (TickCount == 0)
	{
		//On our very first frame we prime any previous params (for interpolation).
		SpawnExecContext.PostTick();
		UpdateExecContext.PostTick();
		if (NiagaraEmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim && GPUExecContext != nullptr)
		{
			//We PostTick the GPUExecContext here to prime crucial PREV parameters (such as PREV_Engine.Owner.Position). This PostTick call is necessary as the GPUExecContext has not been sent to the dispatch interface yet.
			GPUExecContext->PostTick();
		}

		for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
		{
			EventContext.PostTick();
		}
	}

	checkSlow(Data.GetNumVariables() > 0);
	checkSlow(VersionedEmitter.GetEmitterData()->SpawnScriptProps.Script);
	checkSlow(VersionedEmitter.GetEmitterData()->UpdateScriptProps.Script);

	if (bResetPending)
	{
		bResetPending = false;

		if (EventInstanceData.IsValid())
		{
			for (FNiagaraDataSet* SpawnScriptEventDataSet : EventInstanceData->SpawnScriptEventDataSets)
			{
				SpawnScriptEventDataSet->ResetBuffers();
			}
			for (FNiagaraDataSet* UpdateScriptEventDataSet : EventInstanceData->UpdateScriptEventDataSets)
			{
				UpdateScriptEventDataSet->ResetBuffers();
			}
		}

		if ( GPUExecContext )
		{
			GPUExecContext->bResetPending_GT = true;
			GPUExecContext->GpuSpawnInfo_GT.Reset();
		}
		else
		{
			Data.ResetBuffers();
		}
	}

	bAllowSpawning_CNC = bAllowSpawning_GT;

	if (ParentSystemInstance && RendererBindings.GetUObjectsDirty())
	{
		if ( const FNiagaraParameterStore* SrcStore = ParentSystemInstance->GetOverrideParameters() )
		{
			FNiagaraParameterStore* DstStore = &RendererBindings;

			for ( const FNiagaraExternalUObjectInfo& ExternalObject : VersionedEmitter.GetEmitterData()->RendererBindingsExternalObjects )
			{
				const int32 SrcIndex = SrcStore->IndexOf(ExternalObject.GetExternalVariable());
				const int32 DstIndex = DstStore->IndexOf(ExternalObject.Variable);
				if (SrcIndex != INDEX_NONE && DstIndex != INDEX_NONE)
				{
					DstStore->SetUObject(SrcStore->GetUObject(SrcIndex), DstIndex);
				}
			}
		}
		RendererBindings.Tick();
	}


	++TickCount;
	ParticleDataSet->SetIDAcquireTag(TickCount);
}

bool FNiagaraEmitterInstanceImpl::WaitForDebugInfo()
{
	FNiagaraComputeExecutionContext* DebugContext = GPUExecContext;
	if (VersionedEmitter.GetEmitterData()->SimTarget == ENiagaraSimTarget::GPUComputeSim && DebugContext)
	{
		FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = ComputeDispatchInterfacePtr.Get();
		if (ComputeDispatchInterface)
		{
			ENQUEUE_RENDER_COMMAND(CaptureCommand)([ComputeDispatchInterface](FRHICommandListImmediate& RHICmdList) { ComputeDispatchInterface->ProcessDebugReadbacks(RHICmdList, true); });
		}
		else
		{
			UE_LOGF(LogNiagara, Fatal, "Compute interface is invalid and we have a GPU simulation, this may result in a softlock.");
		}
		FlushRenderingCommands(); 
		return true;
	}
	return false;
}

static int32 GbTriggerCrash = 0;
static FAutoConsoleVariableRef CVarTriggerCrash(
	TEXT("fx.TriggerDebugCrash"),
	GbTriggerCrash,
	TEXT("If > 0 we deliberately crash to test Crash Reporter integration."),
	ECVF_Default
);

FORCENOINLINE void NiagaraTestCrash()
{
	check(0);
}

void FNiagaraEmitterInstanceImpl::Tick(float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraTick);
	FScopeCycleCounterUObject AdditionalScope(VersionedEmitter.Emitter, GET_STATID(STAT_NiagaraTick));
#if WITH_EDITORONLY_DATA
	FNiagaraEditorOnlyCycleTimer<false> TickTime(TickTimeCycles);
#endif

#if STATS
	FScopeCycleCounter SystemStatCounter(GetEmitterStatID(true, true));
#endif

	if (HandleCompletion())
	{
		return;
	}

	//Test crash allowing us to test CR functionality.
#if !UE_BUILD_SHIPPING
	if (GbTriggerCrash)
	{
		GbTriggerCrash = 0;
		NiagaraTestCrash();
	}
#endif

	checkSlow(ParticleDataSet);
	FNiagaraDataSet& Data = *ParticleDataSet;
	EmitterAge += DeltaSeconds;

	//UE_LOGF(LogNiagara, Warning, "Emitter Tick %f", EmitterAge);

	if (ExecutionState == ENiagaraExecutionState::InactiveClear)
	{
		if (GPUExecContext)
		{
			GPUExecContext->Reset(ComputeDispatchInterfacePtr.Get());
		}
		Data.ResetBuffers();
		ExecutionState = ENiagaraExecutionState::Inactive;
		return;
	}

	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	if (EmitterData->SimTarget == ENiagaraSimTarget::CPUSim && Data.GetCurrentDataChecked().GetNumInstances() == 0 && ExecutionState != ENiagaraExecutionState::Active)
	{
		Data.ResetBuffers();
		return;
	}

	UNiagaraSystem* System = ParentSystemInstance->GetSystem();

	if (GbDumpParticleData || System->bDumpDebugEmitterInfo)
	{
		UE_LOGF(LogNiagara, Log, "|||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||");
		UE_LOGF(LogNiagara, Log, "|=== FNiagaraEmitterInstanceImpl::Tick [ %ls ] ===============|", *VersionedEmitter.Emitter->GetPathName());
	}


	checkSlow(Data.GetNumVariables() > 0);
	checkSlow(VersionedEmitter.GetEmitterData()->SpawnScriptProps.Script);
	checkSlow(VersionedEmitter.GetEmitterData()->UpdateScriptProps.Script);
	
	if (EventInstanceData.IsValid())
	{
		// Set up the spawn counts and source datasets for the events. The system ensures that we will run after any emitters
		// we're receiving from, so we can use the data buffers that our sources have computed this tick.
		const int32 NumEventHandlers = EmitterData->GetEventHandlers().Num();
		EventInstanceData->EventSpawnTotal = 0;
		for (int32 i = 0; i < NumEventHandlers; i++)
		{
			const FNiagaraEventScriptProperties& EventHandlerProps = EmitterData->GetEventHandlers()[i];
			FNiagaraEventHandlingInfo& Info = EventInstanceData->EventHandlingInfo[i];

			Info.TotalSpawnCount = 0;//This was being done every frame but should be done in init?
			Info.SpawnCounts.Reset();

			//TODO: We can move this lookup into the init and just store a ptr to the other set?
			if (bAllowSpawning_CNC)
			{
				if (FNiagaraDataSet* EventSet = ParentSystemInstance->GetEventDataSet(Info.SourceEmitterName, EventHandlerProps.SourceEventName))
				{
					Info.SetEventData(&EventSet->GetCurrentDataChecked());
					uint32 EventSpawnNum = CalculateEventSpawnCount(EventHandlerProps, Info.SpawnCounts, EventSet);
					Info.TotalSpawnCount += EventSpawnNum;
					EventInstanceData->EventSpawnTotal += EventSpawnNum;
				}
			}
		}
	}

	// Calculate number of new particles from regular spawning 
	uint32 SpawnTotal = 0;
	if (ExecutionState == ENiagaraExecutionState::Active && bAllowSpawning_CNC)
	{
		for (FNiagaraSpawnInfo& Info : SpawnInfos)
		{
			if (Info.Count > 0)
			{
				SpawnTotal += Info.Count;
			}
		}
	}

	int32 EventSpawnTotal = (EventInstanceData.IsValid() ? EventInstanceData->EventSpawnTotal : 0);
	int32 OrigNumParticles = GetNumParticles();
	int32 AllocationEstimate = EmitterData->GetMaxParticleCountEstimate();
	int32 RequiredSize = OrigNumParticles + SpawnTotal + EventSpawnTotal;

	if (RequiredSize == 0)
	{
		//Early out if we have no particles to process.
		//return;
	}

	int32 AllocationSize = FMath::Max<int32>(AllocationEstimate, RequiredSize);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (uint32(AllocationSize) > MaxInstanceCount)
	{
		GEngine->AddOnScreenDebugMessage(uint64(uintptr_t(this)), 5.f, FColor::Yellow, FString::Printf(TEXT("%s has exceeded the maximum particle count (%u), requested (%d)"), *VersionedEmitter.Emitter->GetFullName(), MaxInstanceCount, AllocationSize));
	}
#endif
	AllocationSize = (int32)FMath::Min((uint32)AllocationSize, MaxInstanceCount);

	if (AllocationSize > MaxAllocationCount)
	{
		ReallocationCount++;
		MaxAllocationCount = AllocationSize;
		int32 Estimations = EmitterData->AddRuntimeAllocation((uint64)this, MaxAllocationCount);
		if (GbNiagaraShowAllocationWarnings && Estimations >= 5 && ReallocationCount == 3)
		{
			FString SystemName = System->GetName();
			FString FullName = SystemName + "::" + this->GetEmitterHandle().GetName().ToString();
			UE_LOGF(LogNiagara, Warning, "The emitter %ls required many memory reallocation due to changing particle counts. Consider setting the emitter's AllocationMode property to 'manual' to improve runtime performance.", *FullName);
		}
	}
	
	int32 Overallocation = AllocationSize - RequiredSize;
	if (Overallocation >= 0 && (MinOverallocation < 0 || Overallocation < MinOverallocation))
	{
		MinOverallocation = Overallocation;
	}

	// add system constants
	{
		SCOPE_CYCLE_COUNTER(STAT_NiagaraConstants);

		auto& EmitterParameters = ParentSystemInstance->EditEmitterParameters(EmitterIndex);
		EmitterParameters.EmitterTotalSpawnedParticles = TotalSpawnedParticles;
		EmitterParameters.EmitterAge = EmitterAge;
		EmitterParameters.EmitterRandomSeed = RandomSeed;
		EmitterParameters.EmitterInstanceSeed = InstanceSeed;
	}

	/* GPU simulation -  we just create an FNiagaraComputeExecutionContext, queue it, and let the dispatch interface take care of the rest
	 */
	if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim && GPUExecContext != nullptr)
	{
		check(GPUExecContext->GPUScript_RT == EmitterData->GetGPUComputeScript()->GetRenderThreadScript());

#if NIAGARA_SYSTEM_CAPTURE
		if (ParentSystemInstance->ShouldCaptureThisFrame())
		{
			TSharedPtr<FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> DebugInfo = ParentSystemInstance->GetActiveCaptureWrite(CachedIDName, ENiagaraScriptUsage::ParticleGPUComputeScript, FGuid());
			if (DebugInfo && ComputeDispatchInterfacePtr.IsValid())
			{
				//Data.Dump(DebugInfo->Frame, true, 0, OrigNumParticles);
				//DebugInfo->Frame.Dump(true, 0, OrigNumParticles);
				DebugInfo->Parameters = GPUExecContext->CombinedParamStore;
				
				//TODO: This layout info can be pulled into the emitter/systems etc and all sets just refer to them. They are becoming an annoyance here.
				DebugInfo->Frame.Init(&EmitterCompiledData->GetGPUCaptureDataSetCompiledData());

				// Execute a readback
				ENQUEUE_RENDER_COMMAND(NiagaraReadbackGpuSim)(
					[RT_ComputeDispatchInterface = ComputeDispatchInterfacePtr.Get(), RT_InstanceID = ParentSystemInstance->GetId(), RT_DebugInfo = DebugInfo, RT_Context = GPUExecContext](FRHICommandListImmediate& RHICmdList)
					{
						RT_ComputeDispatchInterface->AddDebugReadback(RT_InstanceID, RT_DebugInfo, RT_Context);
					}
				);
			}
		}
#endif

		// Calculate spawn information to pass to the RT
		{
			static_assert(((NIAGARA_MAX_GPU_SPAWN_INFOS % 4) == 0) && (NIAGARA_MAX_GPU_SPAWN_INFOS > 0), "NIAGARA_MAX_GPU_SPAWN_INFOS should be greater than zero and a multiple of 4");

			FNiagaraGpuSpawnInfo& GpuSpawnInfo = GPUExecContext->GpuSpawnInfo_GT;
			GpuSpawnInfo.EventSpawnTotal = EventSpawnTotal;
			GpuSpawnInfo.SpawnRateInstances = 0;
			GpuSpawnInfo.MaxParticleCount = AllocationSize;

			int NumSpawnInfos = 0;
			int32 NumSpawnedOnGPUThisFrame = 0;
			if (ExecutionState == ENiagaraExecutionState::Active && bAllowSpawning_CNC)
			{
				for (int32 SpawnInfoIdx = 0; SpawnInfoIdx < SpawnInfos.Num(); SpawnInfoIdx++)
				{
					const FNiagaraSpawnInfo& Info = SpawnInfos[SpawnInfoIdx];
					if (Info.Count > 0 && (NumSpawnInfos < NIAGARA_MAX_GPU_SPAWN_INFOS))
					{
						// Ideally, we should clamp the spawn count here, to make sure that we don't exceed the maximum number of particles. However, the value returned by
						// GetNumParticles() can lag behind the real number, so we can't actually determine on the game thread how many particles we're still allowed to
						// spawn. Therefore, we'll send the spawn requests to the render thread as if there was no limit, and we'll clamp the values there, when we prepare
						// the destination dataset for simulation.
						NumSpawnedOnGPUThisFrame += Info.Count;

						int32 MaxParticlesSpawnedPerFrame = EmitterData->MaxGPUParticlesSpawnPerFrame <= 0 ? GMaxNiagaraGPUParticlesSpawnPerFrame : EmitterData->MaxGPUParticlesSpawnPerFrame;

						if (NumSpawnedOnGPUThisFrame > MaxParticlesSpawnedPerFrame)
						{
							FString DebugMsg = FString::Printf(TEXT("%s has attempted to execeed max GPU per frame spawn! | Max: %d | Requested: %d | SpawnInfoEntry: %d"), *VersionedEmitter.Emitter->GetFullName(), MaxParticlesSpawnedPerFrame, NumSpawnedOnGPUThisFrame, SpawnInfoIdx);
							UE_LOGF(LogNiagara, Warning, "%ls", *DebugMsg);
							GEngine->AddOnScreenDebugMessage(uint64(uintptr_t(this)), 5.f, FColor::Yellow, DebugMsg);
							break;
						}

						GpuSpawnInfo.SpawnInfoParams[NumSpawnInfos].IntervalDt = Info.IntervalDt;
						GpuSpawnInfo.SpawnInfoParams[NumSpawnInfos].InterpStartDt = Info.InterpStartDt;
						GpuSpawnInfo.SpawnInfoParams[NumSpawnInfos].SpawnGroup = Info.SpawnGroup;
						GpuSpawnInfo.SpawnInfoParams[NumSpawnInfos].GroupSpawnStartIndex = (int32)GpuSpawnInfo.SpawnRateInstances;

						GpuSpawnInfo.SpawnRateInstances += Info.Count;
						GpuSpawnInfo.SpawnInfoStartOffsets[NumSpawnInfos] = (int32)GpuSpawnInfo.SpawnRateInstances;

						++NumSpawnInfos;
					}
					else if (Info.Count > 0)
					{
						FString DebugMsg = FString::Printf(TEXT("%s Exceeded Gpu spawn info count, see NIAGARA_MAX_GPU_SPAWN_INFOS for more information!"), *VersionedEmitter.Emitter->GetUniqueEmitterName());
						UE_LOGF(LogNiagara, Warning, "%ls", *DebugMsg);
						GEngine->AddOnScreenDebugMessage(uint64(uintptr_t(this)), 5.f, FColor::Yellow, DebugMsg);
						break;
					}

					// Warning: this will be be inaccurate if the render thread clamps the spawn count to keep the total particle count below the limit.
					TotalSpawnedParticles += Info.Count;
				}
			}

			// Clear out the remaining data and leave the end slot as MAX to avoid reading off end of the array on the GPU
			while (NumSpawnInfos < NIAGARA_MAX_GPU_SPAWN_INFOS)
			{
				GpuSpawnInfo.SpawnInfoParams[NumSpawnInfos].IntervalDt = 0.0f;
				GpuSpawnInfo.SpawnInfoParams[NumSpawnInfos].InterpStartDt = 0.0f;
				GpuSpawnInfo.SpawnInfoParams[NumSpawnInfos].SpawnGroup = 0;
				GpuSpawnInfo.SpawnInfoParams[NumSpawnInfos].GroupSpawnStartIndex = (int32)GpuSpawnInfo.SpawnRateInstances;
				GpuSpawnInfo.SpawnInfoStartOffsets[NumSpawnInfos] = INT32_MAX;
				++NumSpawnInfos;
			}
		}

		//GPUExecContext.UpdateInterfaces = VersionedEmitter->UpdateScriptProps.Script->GetCachedDefaultDataInterfaces();

		// copy over the constants for the render thread
		//
		if (GbDumpParticleData || System->bDumpDebugEmitterInfo)
		{
			UE_LOGF(LogNiagara, Log, ".................Spawn.................");
			SpawnExecContext.Parameters.DumpParameters(true);
			UE_LOGF(LogNiagara, Log, ".................Update.................");
			UpdateExecContext.Parameters.DumpParameters(true);
			UE_LOGF(LogNiagara, Log, "................. %ls Combined Parameters (%d Spawned ).................", TEXT("GPU Script"), SpawnTotal);
			GPUExecContext->CombinedParamStore.DumpParameters();
		}

		// Because each context is only ran once each frame, the CBuffer layout stays constant for the lifetime duration of the CBuffer (one frame).

		// @todo-threadsafety do this once during init. Should not change during runtime...
		const uint32 ConstantBufferSize = GPUExecContext->GetConstantBufferSize();
		if (GPUExecContext->ExternalCBufferLayoutSize != ConstantBufferSize)
		{
			GPUExecContext->ExternalCBufferLayoutSize = ConstantBufferSize;
			ENQUEUE_RENDER_COMMAND(UpdateConstantBuffer)(
				[GPUExecContext_RT=GPUExecContext, ConstantBufferSize_RT=ConstantBufferSize](FRHICommandListImmediate& RHICmdList)
				{
					GPUExecContext_RT->ExternalCBufferLayout = new FNiagaraRHIUniformBufferLayout(TEXT("Niagara GPU External CBuffer"), ConstantBufferSize_RT);
				}
			);
		}

		// Call PostTick to adjust bounds, etc
		PostTick();

		// Need to call post-tick, which calls the copy to previous for interpolated spawning
		SpawnExecContext.PostTick();
		UpdateExecContext.PostTick();

		// At this stage GPU execution is being handled by the dispatch interface so we do not need to call PostTick() for it
		for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
		{
			EventContext.PostTick();
		}

		return;
	}

	//Ensure we don't blow our current hard limits on cpu particle count.
	//TODO: These current limits can be improved relatively easily. Though perf in at these counts will obviously be an issue anyway.
	if (EmitterData->SimTarget == ENiagaraSimTarget::CPUSim && AllocationSize > GMaxNiagaraCPUParticlesPerEmitter)
	{
		FString DebugMsg = FString::Printf(TEXT("%s has attempted to exceed the max CPU particle count! | Max: %d | Requested: %u"), *VersionedEmitter.Emitter->GetFullName(), GMaxNiagaraCPUParticlesPerEmitter, AllocationSize);
		UE_LOGF(LogNiagara, Warning, "%ls", *DebugMsg);
		GEngine->AddOnScreenDebugMessage(uint64(uintptr_t(this)), 5.f, FColor::Yellow, DebugMsg);

		//We clear the emitters estimate otherwise we get stuck in this state forever.
		EmitterData->ClearRuntimeAllocationEstimate();

		//For now we completely bail out of spawning new particles. Possibly should improve this in future.
		AllocationSize = OrigNumParticles;
		SpawnTotal = 0;

		if (EventInstanceData.IsValid())
		{
			EventSpawnTotal = 0;
			EventInstanceData->EventSpawnTotal = 0;

			for (FNiagaraEventHandlingInfo& Info : EventInstanceData->EventHandlingInfo)
			{
				Info.SpawnCounts.Empty();
				Info.TotalSpawnCount = 0;
			}
		}
	}

	Data.BeginSimulate();
	Data.Allocate(AllocationSize);

	if (EventInstanceData.IsValid())
	{
		int32 SpawnEventGeneratorIndex = 0;
		for (FNiagaraDataSet* SpawnEventDataSet : EventInstanceData->SpawnScriptEventDataSets)
		{
			int32 NumToAllocate = SpawnTotal + EventSpawnTotal;
			if (EventInstanceData->SpawnEventGeneratorIsSharedByIndex[SpawnEventGeneratorIndex])
			{
				// For shared event data sets we need to allocate storage for the current particles since
				// the same data set will be used in the update execution.
				NumToAllocate += OrigNumParticles;
			}
			SpawnEventDataSet->BeginSimulate();
			SpawnEventDataSet->Allocate(NumToAllocate);
			SpawnEventGeneratorIndex++;
		}

		int32 UpdateEventGeneratorIndex = 0;
		for (FNiagaraDataSet* UpdateEventDataSet : EventInstanceData->UpdateScriptEventDataSets)
		{
			if (EventInstanceData->UpdateEventGeneratorIsSharedByIndex[UpdateEventGeneratorIndex] == false)
			{
				// We only allocate update event data sets if they're not shared, because shared event datasets will have already
				// been allocated as part of the spawn event data set handling.
				UpdateEventDataSet->BeginSimulate();
				UpdateEventDataSet->Allocate(OrigNumParticles);
			}
			UpdateEventGeneratorIndex++;
		}
	}

	// Simulate existing particles forward by DeltaSeconds.
	if (OrigNumParticles > 0)
	{
		Data.GetDestinationDataChecked().SetNumInstances(OrigNumParticles);
		SCOPE_CYCLE_COUNTER(STAT_NiagaraSimulate);

		UpdateExecCountBinding.SetValue(OrigNumParticles);
		UpdateExecContext.BindData(0, Data, 0, true);

		if (EventInstanceData.IsValid())
		{
			int32 EventDataSetIdx = 1;
			for (FNiagaraDataSet* EventDataSet : EventInstanceData->UpdateScriptEventDataSets)
			{
				check(EventDataSet);
				EventDataSet->GetDestinationDataChecked().SetNumInstances(OrigNumParticles);
				UpdateExecContext.BindData(EventDataSetIdx++, *EventDataSet, 0, true);
			}
		}

		FScriptExecutionConstantBufferTable UpdateConstantBufferTable;
		BuildConstantBufferTable(UpdateExecContext, UpdateConstantBufferTable);

		UpdateExecContext.Execute(ParentSystemInstance, DeltaSeconds, OrigNumParticles, UpdateConstantBufferTable);
		int32 DeltaParticles = Data.GetDestinationDataChecked().GetNumInstances() - OrigNumParticles;

		ensure(DeltaParticles <= 0); // We either lose particles or stay the same, we should never add particles in update!

		if (GbDumpParticleData || System->bDumpDebugEmitterInfo)
		{
			Data.GetDestinationDataChecked().Dump(0, OrigNumParticles, FString::Printf(TEXT("=== Updated %d Particles (%d Died) ==="), OrigNumParticles, -DeltaParticles));

			if (EventInstanceData.IsValid())
			{
				for (int32 EventIdx = 0; EventIdx < EventInstanceData->UpdateScriptEventDataSets.Num(); ++EventIdx)
				{
					FNiagaraDataSet* EventDataSet = EventInstanceData->UpdateScriptEventDataSets[EventIdx];
					if (EventDataSet && EventDataSet->GetDestinationDataChecked().GetNumInstances() > 0)
					{
						EventDataSet->GetDestinationDataChecked().Dump(0, INDEX_NONE, FString::Printf(TEXT("Update Script Event %d"), EventIdx));
					}
				}
			}
		//	UE_LOGF(LogNiagara, Log, "=== Update Parameters ===" );
			UpdateExecContext.Parameters.Dump();
		}
	}
	
	// Temporary code to track down an intermitent issue where we had particles but now have none and destination has become invalid.
	if (Data.GetDestinationData() == nullptr)
	{
		UE_LOGF(LogNiagara, Error, "NiagaraEmitterInstance: DestinationData is nullptr.");
		UE_LOGF(LogNiagara, Error, "ExecutionState(%d) TickCount(%d) TotalSpawnedParticles(%d) ParentSystemInstance(%p)", int(ExecutionState), TickCount, TotalSpawnedParticles, ParentSystemInstance);
		if (ParentSystemInstance)
		{
			USceneComponent* AttachComponent = ParentSystemInstance->GetAttachComponent();
			UNiagaraSystem* NiagaraSystem = ParentSystemInstance->GetSystem();
			UE_LOGF(LogNiagara, Error, "SystemInstanceState(%d) AttachComponent(%p=%ls) NiagaraSystem(%p=%ls)",
				int(ParentSystemInstance->SystemInstanceState),
				AttachComponent, *GetNameSafe(AttachComponent),
				NiagaraSystem, *GetNameSafe(NiagaraSystem)
			);
		}
	}

	uint32 EventSpawnStart = Data.GetDestinationDataChecked().GetNumInstances();
	const int32 NumBeforeSpawn = Data.GetDestinationDataChecked().GetNumInstances();
	uint32 TotalActualEventSpawns = 0;

	Data.NumSpawnedIDs = 0;

	int32 SpawnCountRemaining = AllocationSize - OrigNumParticles;

	//Init new particles with the spawn script.
	if (SpawnTotal + EventSpawnTotal > 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_NiagaraSpawn);

		// note that this constant buffer table is used for each invocation of the spawn, the data within the
		// table will get modified between invocations (TotalSpawnedParticles).
		FScriptExecutionConstantBufferTable SpawnConstantBufferTable;
		BuildConstantBufferTable(SpawnExecContext, SpawnConstantBufferTable);

		//Handle main spawn rate spawning
		auto SpawnParticles = [&](int32 Num, const TCHAR* DumpLabel)
		{
			int32 OrigNum = Data.GetDestinationDataChecked().GetNumInstances();
			Data.GetDestinationDataChecked().SetNumInstances(OrigNum + Num);

			// We need to update Engine.Emitter.TotalSpawnedParticles for each event spawn invocation.
			ParentSystemInstance->EditEmitterParameters(EmitterIndex).EmitterTotalSpawnedParticles = TotalSpawnedParticles;
			
			// NOTE(mv): Updates the count after setting the variable, such that the TotalSpawnedParticles value read 
			//           in the script has the count at the start of the frame. 
			//           This way UniqueID = TotalSpawnedParticles + ExecIndex provide unique and sequential identifiers. 
			// NOTE(mv): Only for CPU particles, as GPU particles early outs further up and has a separate increment. 
			TotalSpawnedParticles += Num;

			SpawnExecCountBinding.SetValue(Num);
			SpawnExecContext.BindData(0, Data, OrigNum, true);

			if (EventInstanceData.IsValid())
			{
				//UE_LOGF(LogNiagara, Log, "SpawnScriptEventDataSets: %d", SpawnScriptEventDataSets.Num());
				int32 EventDataSetIdx = 1;
				for (FNiagaraDataSet* EventDataSet : EventInstanceData->SpawnScriptEventDataSets)
				{
					//UE_LOGF(LogNiagara, Log, "SpawnScriptEventDataSets.. %d", EventDataSet->GetNumVariables());
					int32 EventOrigNum = EventDataSet->GetDestinationDataChecked().GetNumInstances();
					EventDataSet->GetDestinationDataChecked().SetNumInstances(EventOrigNum + Num);
					SpawnExecContext.BindData(EventDataSetIdx++, *EventDataSet, EventOrigNum, true);
				}
			}

			SpawnExecContext.Execute(ParentSystemInstance, DeltaSeconds, Num, SpawnConstantBufferTable);

			if (GbDumpParticleData || System->bDumpDebugEmitterInfo)
			{
				Data.GetDestinationDataChecked().Dump(OrigNum, Num, FString::Printf(TEXT("===  %s Spawned %d Particles==="), DumpLabel, Num));

				if (EventInstanceData.IsValid())
				{
					for (int32 EventIdx = 0; EventIdx < EventInstanceData->SpawnScriptEventDataSets.Num(); ++EventIdx)
					{
						FNiagaraDataSet* EventDataSet = EventInstanceData->SpawnScriptEventDataSets[EventIdx];
						if (EventDataSet && EventDataSet->GetDestinationDataChecked().GetNumInstances() > 0)
						{
							EventDataSet->GetDestinationDataChecked().Dump(0, INDEX_NONE, FString::Printf(TEXT("Spawn Script Event %d"), EventIdx));
						}
					}
				}

				//UE_LOGF(LogNiagara, Log, "=== %ls Spawn Parameters ===", *DumpLabel);
				SpawnExecContext.Parameters.Dump();
			}
		};

		//Perform all our regular spawning that's driven by our emitter script.
		for (const FNiagaraSpawnInfo& Info : SpawnInfos)
		{
			const int32 AdjustedSpawnCount = FMath::Min(Info.Count, SpawnCountRemaining);
			if ( AdjustedSpawnCount > 0 )
			{
				SpawnIntervalBinding.SetValue(Info.IntervalDt);
				InterpSpawnStartBinding.SetValue(Info.InterpStartDt);
				SpawnGroupBinding.SetValue(Info.SpawnGroup);
				SpawnParticles(AdjustedSpawnCount, TEXT("Regular Spawn"));
			}
			SpawnCountRemaining -= AdjustedSpawnCount;
		}

		EventSpawnStart = Data.GetDestinationDataChecked().GetNumInstances();
		if (EventInstanceData.IsValid())
		{
			for (int32 EventScriptIdx = 0; EventScriptIdx < EmitterData->GetEventHandlers().Num(); EventScriptIdx++)
			{
				FNiagaraEventHandlingInfo& Info = EventInstanceData->EventHandlingInfo[EventScriptIdx];

				for (int32 i = 0; i < Info.SpawnCounts.Num(); i++)
				{
					const int32 EventNumToSpawn = FMath::Min(Info.SpawnCounts[i], SpawnCountRemaining);
					if (EventNumToSpawn > 0)
					{
						const int32 CurrNumParticles = Data.GetDestinationDataChecked().GetNumInstances();

						//Event spawns are instantaneous at the middle of the frame?
						SpawnIntervalBinding.SetValue(0.0f);
						InterpSpawnStartBinding.SetValue(DeltaSeconds * 0.5f);
						SpawnGroupBinding.SetValue(0);
						SpawnParticles(EventNumToSpawn, TEXT("Event Spawn"));

						//Update EventSpawnCounts to the number actually spawned.
						const int32 NumActuallySpawned = Data.GetDestinationDataChecked().GetNumInstances() - CurrNumParticles;
						TotalActualEventSpawns += NumActuallySpawned;
						Info.SpawnCounts[i] = NumActuallySpawned;
						SpawnCountRemaining -= NumActuallySpawned;
					}
					else
					{
						Info.SpawnCounts[i] = 0;
					}
				}
			}
		}
	}

	const int32 NumAfterSpawn = Data.GetDestinationDataChecked().GetNumInstances();
	const int32 TotalNumSpawned = NumAfterSpawn - NumBeforeSpawn;

	Data.GetDestinationDataChecked().SetNumSpawnedInstances(TotalNumSpawned);
	Data.GetDestinationDataChecked().SetIDAcquireTag(Data.GetIDAcquireTag());

	//We're done with this simulation pass.
	Data.EndSimulate();

	if (EventInstanceData.IsValid())
	{
		for (FNiagaraDataSet* SpawnEventDataSet : EventInstanceData->SpawnScriptEventDataSets)
		{
			if (SpawnEventDataSet && SpawnEventDataSet->GetDestinationData())
			{
				SpawnEventDataSet->EndSimulate();
			}
		}

		for (FNiagaraDataSet* UpdateEventDataSet : EventInstanceData->UpdateScriptEventDataSets)
		{
			if (UpdateEventDataSet && UpdateEventDataSet->GetDestinationData())
			{
				UpdateEventDataSet->EndSimulate();
			}
		}
	}

	//Now pull out any debug info we need.
#if NIAGARA_SYSTEM_CAPTURE
	if (ParentSystemInstance->ShouldCaptureThisFrame())
	{
		//Pull out update data.
		TSharedPtr<FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> DebugInfo = ParentSystemInstance->GetActiveCaptureWrite(CachedIDName, ENiagaraScriptUsage::ParticleUpdateScript, FGuid());
		if (DebugInfo)
		{
			Data.CopyTo(DebugInfo->Frame, 0, OrigNumParticles);
			DebugInfo->Parameters = UpdateExecContext.Parameters;
			DebugInfo->bWritten = true;
		}
		//Pull out update data.
		DebugInfo = ParentSystemInstance->GetActiveCaptureWrite(CachedIDName, ENiagaraScriptUsage::ParticleSpawnScript, FGuid());
		if (DebugInfo)
		{
			Data.CopyTo(DebugInfo->Frame, NumBeforeSpawn, TotalNumSpawned);
			DebugInfo->Parameters = SpawnExecContext.Parameters;
			DebugInfo->bWritten = true;
		}
	}
#endif
	/*else if (SpawnTotal + EventSpawnTotal > 0)
	{
		UE_LOGF(LogNiagara, Log, "Skipping spawning due to execution state! %d", (uint32)ExecutionState)
	}*/

	if (EventInstanceData.IsValid())
	{
		if (TotalActualEventSpawns > 0)
		{
			SCOPE_CYCLE_COUNTER(STAT_NiagaraEvent_CopyBuffer);
			if (GbDumpParticleData || System->bDumpDebugEmitterInfo)
			{
				Data.Dump(0, INDEX_NONE, TEXT("Existing Data - Pre Event Alloc"));
			}
			//Allocate a new dest buffer to write spawn event handler results into.
			//Can just do one allocate here for all spawn event handlers.
			//Though this requires us to copy the contents of the instances we're not writing to in this pass over from the previous buffer.
			FNiagaraDataBuffer& DestBuffer = Data.BeginSimulate();
			Data.Allocate(Data.GetCurrentDataChecked().GetNumInstances(), true);
			DestBuffer.SetNumInstances(EventSpawnStart);

			//if (GbDumpParticleData || System->bDumpDebugEmitterInfo)
			//{
			//	DestBuffer.Dump(0, INDEX_NONE, TEXT("Existing Data - Post Event Alloc, Pre Events"));
			//}
		}

		for (int32 EventScriptIdx = 0; EventScriptIdx < EmitterData->GetEventHandlers().Num(); EventScriptIdx++)
		{
			FNiagaraEventHandlingInfo& Info = EventInstanceData->EventHandlingInfo[EventScriptIdx];

			FScriptExecutionConstantBufferTable EventConstantBufferTable;
			BuildConstantBufferTable(EventInstanceData->EventExecContexts[EventScriptIdx], EventConstantBufferTable);

			if (Info.EventData && Info.SpawnCounts.Num() > 0)
			{
				SCOPE_CYCLE_COUNTER(STAT_NiagaraEventHandle);

				for (int32 i = 0; i < Info.SpawnCounts.Num(); i++)
				{
					int32 EventNumToSpawn = Info.SpawnCounts[i];
					if (EventNumToSpawn > 0)
					{
						EventInstanceData->EventExecCountBindings[EventScriptIdx].SetValue(EventNumToSpawn);

						EventInstanceData->EventExecContexts[EventScriptIdx].BindData(0, Data, EventSpawnStart, true);
						EventInstanceData->EventExecContexts[EventScriptIdx].BindData(1, EventInstanceData->EventHandlingInfo[EventScriptIdx].EventData, i, false);

						EventInstanceData->EventExecContexts[EventScriptIdx].Execute(ParentSystemInstance, DeltaSeconds, EventNumToSpawn, EventConstantBufferTable);

						const uint32 PostHandlerNumInstances = Data.GetDestinationData()->GetNumInstances();
						const uint32 EventSpawnsStillAlive = PostHandlerNumInstances - EventSpawnStart;
						if (GbDumpParticleData || System->bDumpDebugEmitterInfo)
						{
							EventInstanceData->EventHandlingInfo[EventScriptIdx].EventData->Dump(i, 1, FString::Printf(TEXT("=== Event Data %d [%d] ==="), EventScriptIdx, i));
							Data.GetDestinationDataChecked().Dump(EventSpawnStart, EventSpawnsStillAlive, FString::Printf(TEXT("=== Event %d %d Particles (%d Alive) ==="), EventScriptIdx, EventNumToSpawn, EventSpawnsStillAlive));
							//UE_LOGF(LogNiagara, Log, "=== Event %d Parameters ===", EventScriptIdx);
							EventInstanceData->EventExecContexts[EventScriptIdx].Parameters.Dump();
						}

	#if NIAGARA_SYSTEM_CAPTURE
						if (ParentSystemInstance->ShouldCaptureThisFrame())
						{
							FGuid EventGuid = EventInstanceData->EventExecContexts[EventScriptIdx].Script->GetUsageId();
							TSharedPtr<FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> DebugInfo = ParentSystemInstance->GetActiveCaptureWrite(CachedIDName, ENiagaraScriptUsage::ParticleEventScript, EventGuid);
							if (DebugInfo)
							{
								Data.CopyTo(DebugInfo->Frame, EventSpawnStart, EventSpawnsStillAlive);
								DebugInfo->Parameters = EventInstanceData->EventExecContexts[EventScriptIdx].Parameters;
								DebugInfo->bWritten = true;
							}
						}
	#endif
						//Spawn events from the current end point. Possible the last event killed some particles.
						EventSpawnStart = PostHandlerNumInstances;
					}
				}
			}
		}

		//If we processed any events we need to end simulate to update the current sim state.
		if (Data.GetDestinationData())
		{
			Data.EndSimulate();
		}

		// Update events need a copy per event so that the previous event's data can be used.
		for (int32 EventScriptIdx = 0; EventScriptIdx < EmitterData->GetEventHandlers().Num(); EventScriptIdx++)
		{
			const FNiagaraEventScriptProperties &EventHandlerProps = EmitterData->GetEventHandlers()[EventScriptIdx];
			FNiagaraDataBuffer* EventData = EventInstanceData->EventHandlingInfo[EventScriptIdx].EventData;

			FScriptExecutionConstantBufferTable EventConstantBufferTable;
			BuildConstantBufferTable(EventInstanceData->EventExecContexts[EventScriptIdx], EventConstantBufferTable);

			// handle all-particle events
			if (EventHandlerProps.Script && EventHandlerProps.ExecutionMode == EScriptExecutionMode::EveryParticle && EventData)
			{
				uint32 NumParticles = Data.GetCurrentDataChecked().GetNumInstances();

				if (EventData->GetNumInstances())
				{
					SCOPE_CYCLE_COUNTER(STAT_NiagaraEventHandle);

					for (uint32 i = 0; i < EventData->GetNumInstances(); i++)
					{
						Data.BeginSimulate();
						Data.Allocate(NumParticles);

						uint32 NumInstancesPrev = Data.GetCurrentDataChecked().GetNumInstances();
						EventInstanceData->EventExecCountBindings[EventScriptIdx].SetValue(NumInstancesPrev);

						EventInstanceData->EventExecContexts[EventScriptIdx].BindData(0, Data, 0, true);
						EventInstanceData->EventExecContexts[EventScriptIdx].BindData(1, EventData, i, false);

						EventInstanceData->EventExecContexts[EventScriptIdx].Execute(ParentSystemInstance, DeltaSeconds, NumInstancesPrev, EventConstantBufferTable);

						Data.EndSimulate();

						if (GbDumpParticleData || System->bDumpDebugEmitterInfo)
						{
							EventData->Dump(i, 1, FString::Printf(TEXT("=== Event Data %d [%d] ==="), EventScriptIdx, i));
							Data.GetCurrentDataChecked().Dump(0, NumInstancesPrev, FString::Printf(TEXT("=== Event %d %d Particles ==="), EventScriptIdx, NumInstancesPrev));
							EventInstanceData->EventExecContexts[EventScriptIdx].Parameters.Dump();
						}

#if NIAGARA_SYSTEM_CAPTURE
						if (ParentSystemInstance->ShouldCaptureThisFrame())
						{
							FGuid EventGuid = EventInstanceData->EventExecContexts[EventScriptIdx].Script->GetUsageId();
							TSharedPtr<FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> DebugInfo = ParentSystemInstance->GetActiveCaptureWrite(CachedIDName, ENiagaraScriptUsage::ParticleEventScript, EventGuid);
							if (DebugInfo)
							{
								Data.CopyTo(DebugInfo->Frame, 0, NumInstancesPrev);
								DebugInfo->Parameters = EventInstanceData->EventExecContexts[EventScriptIdx].Parameters;
								DebugInfo->bWritten = true;
							}
						}
#endif
						ensure(NumParticles == Data.GetCurrentDataChecked().GetNumInstances());
					}
				}
			}
		}

		//TODO: Disabling this event mode for now until it can be reworked. Currently it uses index directly with can easily be invalid and cause undefined behavior.
		//
//		// handle single-particle events
//		// TODO: we'll need a way to either skip execution of the VM if an index comes back as invalid, or we'll have to pre-process
//		// event/particle arrays; this is currently a very naive (and comparatively slow) implementation, until full indexed reads work
// 		if (EventHandlerProps.Script && EventHandlerProps.ExecutionMode == EScriptExecutionMode::SingleParticle && EventSet[EventScriptIdx])
// 		{
// 
// 			SCOPE_CYCLE_COUNTER(STAT_NiagaraEventHandle);
// 			FNiagaraVariable IndexVar(FNiagaraTypeDefinition::GetIntDef(), "ParticleIndex");
// 			FNiagaraDataSetIterator<int32> IndexItr(*EventSet[EventScriptIdx], IndexVar, 0, false);
// 			if (IndexItr.IsValid() && EventSet[EventScriptIdx]->GetPrevNumInstances() > 0)
// 			{
// 				EventExecCountBindings[EventScriptIdx].SetValue(1);
// 
// 				Data.CopyCurToPrev();
// 				uint32 NumParticles = Data.GetNumInstances();
// 
// 				for (uint32 i = 0; i < EventSet[EventScriptIdx]->GetPrevNumInstances(); i++)
// 				{
// 					int32 Index = *IndexItr;
// 					IndexItr.Advance();
// 					DataSetExecInfos.SetNum(1, false);
// 					DataSetExecInfos[0].StartInstance = Index;
// 					DataSetExecInfos[0].bUpdateInstanceCount = false;
// 					DataSetExecInfos.Emplace(EventSet[EventScriptIdx], i, false, false);
// 					EventExecContexts[EventScriptIdx].Execute(1, DataSetExecInfos);
// 
// 					if (GbDumpParticleData || System->bDumpDebugEmitterInfo)
// 					{
// 						ensure(EventHandlerProps.Script->RapidIterationParameters.VerifyBinding(&EventExecContexts[EventScriptIdx].Parameters));
// 						UE_LOGF(LogNiagara, Log, "=== Event %d Src Parameters ===", EventScriptIdx);
// 						EventHandlerProps.Script->RapidIterationParameters.Dump();
// 						UE_LOGF(LogNiagara, Log, "=== Event %d Context Parameters ===", EventScriptIdx);
// 						EventExecContexts[EventScriptIdx].Parameters.Dump();
// 						UE_LOGF(LogNiagara, Log, "=== Event %d Particles (%d index written, %d total) ===", EventScriptIdx, Index, Data.GetNumInstances());
// 						Data.Dump(true, Index, 1);
// 					}
// 
// 
// #if NIAGARA_SYSTEM_CAPTURE
// 					if (ParentSystemInstance->ShouldCaptureThisFrame())
// 					{
// 						FGuid EventGuid = EventExecContexts[EventScriptIdx].Script->GetUsageId();
// 						TSharedPtr<struct FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> DebugInfo = ParentSystemInstance->GetActiveCaptureWrite(CachedIDName, ENiagaraScriptUsage::ParticleEventScript, EventGuid);
// 						if (DebugInfo)
// 						{
// 							Data.Dump(DebugInfo->Frame, true, Index, 1);
// 							//DebugInfo->Frame.Dump(true, 0, 1);
// 							DebugInfo->Parameters = EventExecContexts[EventScriptIdx].Parameters;
// 						}
// 					}
// #endif
// 					ensure(NumParticles == Data.GetNumInstances());
// 				}
// 			}
// 		}
	}

	PostTick();

	SpawnExecContext.PostTick();
	UpdateExecContext.PostTick();
	// At this stage GPU execution is being handled by the dispatch interface so we do not need to call PostTick() for it

	for (FNiagaraScriptExecutionContext& EventContext : GetEventExecutionContexts())
	{
		EventContext.PostTick();
	}

	if (GbDumpParticleData || System->bDumpDebugEmitterInfo)
	{
		UE_LOGF(LogNiagara, Log, "|=== END OF FNiagaraEmitterInstanceImpl::Tick [ %ls ] ===============|", *VersionedEmitter.Emitter->GetPathName());
		UE_LOGF(LogNiagara, Log, "|||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||||");
	}


	INC_DWORD_STAT_BY(STAT_NiagaraNumParticles, Data.GetCurrentDataChecked().GetNumInstances());
}

/** Calculate total number of spawned particles from events; these all come from event handler script with the SpawnedParticles execution mode
 *  We get the counts ahead of event processing time so we only have to allocate new particles once
 *  TODO: augment for multiple spawning event scripts
 */
uint32 FNiagaraEmitterInstanceImpl::CalculateEventSpawnCount(const FNiagaraEventScriptProperties &EventHandlerProps, TArray<int32, TInlineAllocator<16>>& EventSpawnCounts, FNiagaraDataSet *EventSet)
{
	uint32 SpawnTotal = 0;
	uint32 MaxSpawnCount = INT_MAX; //We could probably do to have a CVar for limiting the max event spawn directly but for now just keep the count from overflowing so it's caught by the overall partcle count checks later.
	int32 NumEventsToProcess = 0;

	if (EventSet)
	{
		NumEventsToProcess = EventSet->GetCurrentDataChecked().GetNumInstances();
		if(EventHandlerProps.MaxEventsPerFrame > 0)
		{
			NumEventsToProcess = FMath::Min<int32>(EventSet->GetCurrentDataChecked().GetNumInstances(), EventHandlerProps.MaxEventsPerFrame);
		}

		const bool bUseRandom = EventHandlerProps.bRandomSpawnNumber && EventHandlerProps.MinSpawnNumber < EventHandlerProps.SpawnNumber;
		for (int32 i = 0; i < NumEventsToProcess; i++)
		{
			const uint32 SpawnNumber = bUseRandom ? FMath::RandRange((int32)EventHandlerProps.MinSpawnNumber, (int32)EventHandlerProps.SpawnNumber) : EventHandlerProps.SpawnNumber;
			uint32 NewSpawnTotal = SpawnTotal + SpawnNumber;
			if (ExecutionState == ENiagaraExecutionState::Active && SpawnNumber > 0 && NewSpawnTotal < MaxSpawnCount)
			{
				EventSpawnCounts.Add(SpawnNumber);
				SpawnTotal = NewSpawnTotal;
			}
		}
	}

	return SpawnTotal;
}

void FNiagaraEmitterInstanceImpl::SetExecutionState(ENiagaraExecutionState InState)
{
	/*if (InState != ExecutionState)
	{
		const UEnum* EnumPtr = FNiagaraTypeDefinition::GetExecutionStateEnum();
		UE_LOGF(LogNiagara, Log, "Emitter \"%ls\" change state: %ls to %ls", *GetEmitterHandle().GetName().ToString(), *EnumPtr->GetNameStringByValue((int64)ExecutionState),
			*EnumPtr->GetNameStringByValue((int64)InState));
	}*/

	/*if (InState == ENiagaraExecutionState::Active && ExecutionState == ENiagaraExecutionState::Inactive)
	{
		UE_LOGF(LogNiagara, Log, "Emitter \"%ls\" change state N O O O O O ", *GetEmitterHandle().GetName().ToString());
	}*/
	if (ensureMsgf(InState >= ENiagaraExecutionState::Active && InState < ENiagaraExecutionState::Num, 
					TEXT("Setting invalid emitter execution state! %d\nEmitter=%s\nSystem=%s\nComponent=%s"),
					(int32)InState,
					*GetFullNameSafe(VersionedEmitter.Emitter),
					*GetFullNameSafe(ParentSystemInstance ? ParentSystemInstance->GetSystem() : nullptr),
					*GetFullNameSafe(ParentSystemInstance ? ParentSystemInstance->GetAttachComponent() : nullptr))
		)
	{
		//We can't move out of disabled without a proper reinit.
		if (ExecutionState != ENiagaraExecutionState::Disabled)
		{
			ExecutionState = InState;
		}
	}
	else
	{
		//Try to gracefully fail in this case.
		ExecutionState = ENiagaraExecutionState::Inactive;
	}

}

void FNiagaraEmitterInstanceImpl::BuildConstantBufferTable(
	const FNiagaraScriptExecutionContext& ExecContext,
	FScriptExecutionConstantBufferTable& ConstantBufferTable) const
{
	const auto ScriptLiterals = ExecContext.GetScriptLiterals();
	const auto& ExternalParameterData = ExecContext.Parameters.GetParameterDataArray();
	uint8* ExternalParameterBuffer = const_cast<uint8*>(ExternalParameterData.GetData());

	const uint32 ExternalParameterSize = ExecContext.Parameters.GetExternalParameterSize();

	const uint32 TableCount = 5 * (ExecContext.HasInterpolationParameters ? 2 : 1) + 1;
	ConstantBufferTable.Reset(TableCount);

	ConstantBufferTable.AddTypedBuffer(ParentSystemInstance->GetGlobalParameters());
	ConstantBufferTable.AddTypedBuffer(ParentSystemInstance->GetSystemParameters());
	ConstantBufferTable.AddTypedBuffer(ParentSystemInstance->GetOwnerParameters());
	ConstantBufferTable.AddTypedBuffer(ParentSystemInstance->GetEmitterParameters(EmitterIndex));
	ConstantBufferTable.AddRawBuffer(ExternalParameterBuffer, ExternalParameterSize);

	if (ExecContext.HasInterpolationParameters)
	{
		ConstantBufferTable.AddTypedBuffer(ParentSystemInstance->GetGlobalParameters(true));
		ConstantBufferTable.AddTypedBuffer(ParentSystemInstance->GetSystemParameters(true));
		ConstantBufferTable.AddTypedBuffer(ParentSystemInstance->GetOwnerParameters(true));
		ConstantBufferTable.AddTypedBuffer(ParentSystemInstance->GetEmitterParameters(EmitterIndex, true));
		ConstantBufferTable.AddRawBuffer(ExternalParameterBuffer + ExternalParameterSize, ExternalParameterSize);
	}

	ConstantBufferTable.AddRawBuffer(ScriptLiterals.GetData(), ScriptLiterals.Num());
}
