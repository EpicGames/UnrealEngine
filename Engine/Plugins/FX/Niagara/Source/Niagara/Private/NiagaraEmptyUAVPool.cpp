// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEmptyUAVPool.h"
#include "NiagaraStats.h"
#include "NiagaraTypes.h"
#include "RHICommandList.h"
#include "RenderGraphBuilder.h"

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("# EmptyUAVs"), STAT_NiagaraEmptyUAVPool, STATGROUP_Niagara);

FNiagaraEmptyUAVPoolScopedAccess::FNiagaraEmptyUAVPoolScopedAccess(class FNiagaraEmptyUAVPool* InEmptyUAVPool)
	: EmptyUAVPool(InEmptyUAVPool)
{
	check(IsInRenderingThread());
	check(EmptyUAVPool);
	++EmptyUAVPool->UAVAccessCounter;
}

FNiagaraEmptyUAVPoolScopedAccess::~FNiagaraEmptyUAVPoolScopedAccess()
{
	check(IsInRenderingThread());
	--EmptyUAVPool->UAVAccessCounter;
	if (EmptyUAVPool->UAVAccessCounter == 0)
	{
		EmptyUAVPool->ResetEmptyUAVPools();
	}
}

FNiagaraEmptyRDGUAVPoolScopedAccess::FNiagaraEmptyRDGUAVPoolScopedAccess(class FNiagaraEmptyUAVPool* InEmptyUAVPool)
	: EmptyUAVPool(InEmptyUAVPool)
{
	check(IsInRenderingThread());
	check(EmptyUAVPool);
	++EmptyUAVPool->RDGUAVAccessCounter;
}

FNiagaraEmptyRDGUAVPoolScopedAccess::~FNiagaraEmptyRDGUAVPoolScopedAccess()
{
	check(IsInRenderingThread());
	--EmptyUAVPool->RDGUAVAccessCounter;
	if (EmptyUAVPool->RDGUAVAccessCounter == 0)
	{
		EmptyUAVPool->ResetEmptyRDGUAVPools();
	}
}

void FNiagaraEmptyUAVPool::Tick()
{
	check(IsInRenderingThread());
	BufferRDGUAVPool.Empty();
	TextureRDGUAVPool.Empty();
}

FRHIUnorderedAccessView* FNiagaraEmptyUAVPool::GetEmptyUAVFromPool(FRHICommandList& RHICmdList, EPixelFormat Format, ENiagaraEmptyUAVType Type)
{
	check(IsInRenderingThread());
	checkf(UAVAccessCounter != 0, TEXT("Accessing Niagara's UAV Pool while not within a scope, this could result in a memory leak!"));

	TMap<EPixelFormat, FEmptyUAVPool>& UAVMap = UAVPools[(int)Type];
	FEmptyUAVPool& Pool = UAVMap.FindOrAdd(Format);
	checkSlow(Pool.NextFreeIndex <= Pool.UAVs.Num());
	if (Pool.NextFreeIndex == Pool.UAVs.Num())
	{
		FEmptyUAV& NewUAV = Pool.UAVs.AddDefaulted_GetRef();

		// Initialize the UAV
		const TCHAR* ResourceName = TEXT("FNiagaraGpuComputeDispatch::EmptyUAV");
		switch ( Type )
		{
			case ENiagaraEmptyUAVType::Buffer:
			{
				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateVertex(ResourceName, GPixelFormats[Format].BlockBytes)
					.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource)
					.DetermineInitialState();

				NewUAV.Buffer = RHICmdList.CreateBuffer(CreateDesc);
				NewUAV.UAV = RHICmdList.CreateUnorderedAccessView(
					NewUAV.Buffer, 
					FRHIViewDesc::CreateBufferUAV()
						.SetType(FRHIViewDesc::EBufferType::Typed)
						.SetFormat(Format));
				break;
			}
			
			case ENiagaraEmptyUAVType::Texture2D:
			{
				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::Create2D(ResourceName, 1, 1, Format)
					.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);

				NewUAV.Texture = RHICmdList.CreateTexture(Desc);
				NewUAV.UAV = RHICmdList.CreateUnorderedAccessView(NewUAV.Texture, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(NewUAV.Texture));
				break;
			}
			
			case ENiagaraEmptyUAVType::Texture2DArray:
			{
				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::Create2DArray(ResourceName, 1, 1, 1, Format)
					.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);

				NewUAV.Texture = RHICmdList.CreateTexture(Desc);
				NewUAV.UAV = RHICmdList.CreateUnorderedAccessView(NewUAV.Texture, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(NewUAV.Texture));
				break;
			}
			
			case ENiagaraEmptyUAVType::Texture3D:
			{
				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::Create3D(ResourceName, 1, 1, 1, Format)
					.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);

				NewUAV.Texture = RHICmdList.CreateTexture(Desc);
				NewUAV.UAV = RHICmdList.CreateUnorderedAccessView(NewUAV.Texture, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(NewUAV.Texture));
				break;
			}
			
			case ENiagaraEmptyUAVType::TextureCube:
			{
				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::CreateCube(ResourceName, 1, Format)
					.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);

				NewUAV.Texture = RHICmdList.CreateTexture(Desc);
				NewUAV.UAV = RHICmdList.CreateUnorderedAccessView(NewUAV.Texture, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(NewUAV.Texture));
				break;
			}


			case ENiagaraEmptyUAVType::TextureCubeArray:
			{
				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::CreateCubeArray(ResourceName, 1, 1, Format)
					.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);

				NewUAV.Texture = RHICmdList.CreateTexture(Desc);
				NewUAV.UAV = RHICmdList.CreateUnorderedAccessView(NewUAV.Texture, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(NewUAV.Texture));
				break;
			}

			default:
			{
				checkNoEntry();
				return nullptr;
			}
		}

		RHICmdList.Transition(FRHITransitionInfo(NewUAV.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

		// Dispatches which use Empty UAVs are allowed to overlap, since we don't care about the contents of these buffers.
		// We never need to calll EndUAVOverlap() on these.
		RHICmdList.BeginUAVOverlap(NewUAV.UAV);

		INC_DWORD_STAT(STAT_NiagaraEmptyUAVPool);
	}

	FRHIUnorderedAccessView* UAV = Pool.UAVs[Pool.NextFreeIndex].UAV;
	++Pool.NextFreeIndex;
	return UAV;
}

void FNiagaraEmptyUAVPool::ResetEmptyUAVPools()
{
	for (int Type = 0; Type < UE_ARRAY_COUNT(UAVPools); ++Type)
	{
		for (TPair<EPixelFormat, FEmptyUAVPool>& Entry : UAVPools[Type])
		{
			Entry.Value.NextFreeIndex = 0;
		}
	}
}

FRDGBufferUAVRef FNiagaraEmptyUAVPool::GetEmptyRDGUAVFromPool(FRDGBuilder& GraphBuilder, EPixelFormat Format)
{
	check(IsInRenderingThread());
	checkf(RDGUAVAccessCounter != 0, TEXT("Accessing Niagara's RDG UAV Pool while not within a scope, this could result in a memory leak!"));
	check(UE::PixelFormat::HasCapabilities(Format, EPixelFormatCapabilities::TypedUAVStore));

	FBufferRDGUAVPool& Pool = BufferRDGUAVPool.FindOrAdd(Format);
	if (Pool.NextFreeIndex == Pool.UAVs.Num())
	{
		const uint32 BytesPerElement = GPixelFormats[Format].BlockBytes;
		Pool.UAVs.Add(
			GraphBuilder.CreateUAV(
				GraphBuilder.CreateBuffer(
					FRDGBufferDesc::CreateBufferDesc(BytesPerElement, 1),
					TEXT("ENiagaraEmptyUAVType::Buffer")
				),
				Format,
				ERDGUnorderedAccessViewFlags::SkipBarrier
			)
		);
	}
	return Pool.UAVs[Pool.NextFreeIndex++];
}

FRDGTextureUAVRef FNiagaraEmptyUAVPool::GetEmptyRDGUAVFromPool(FRDGBuilder& GraphBuilder, EPixelFormat Format, ETextureDimension TextureDimension)
{
	check(IsInRenderingThread());
	checkf(RDGUAVAccessCounter != 0, TEXT("Accessing Niagara's RDG UAV Pool while not within a scope, this could result in a memory leak!"));
	check(UE::PixelFormat::HasCapabilities(Format, EPixelFormatCapabilities::TypedUAVStore));

	FTextureRDGUAVPool& Pool = TextureRDGUAVPool.FindOrAdd(TPairInitializer(Format, TextureDimension));
	if (Pool.NextFreeIndex == Pool.UAVs.Num())
	{
		switch (TextureDimension)
		{
			case ETextureDimension::Texture2D:
				Pool.UAVs.Add(
					GraphBuilder.CreateUAV(
						GraphBuilder.CreateTexture(
							FRDGTextureDesc::Create2D(FIntPoint(1, 1), Format, FClearValueBinding::Black, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV),
							TEXT("NiagaraEmptyTextureUAV::Texture2D")
						),
						ERDGUnorderedAccessViewFlags::SkipBarrier
					)
				);
				break;

			case ETextureDimension::Texture2DArray:
				Pool.UAVs.Add(
					GraphBuilder.CreateUAV(
						GraphBuilder.CreateTexture(
							FRDGTextureDesc::Create2DArray(FIntPoint(1, 1), Format, FClearValueBinding::Black, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV, 1),
							TEXT("NiagaraEmptyTextureUAV::Texture2DArray")
						),
						ERDGUnorderedAccessViewFlags::SkipBarrier
					)
				);
				break;

			case ETextureDimension::Texture3D:
				Pool.UAVs.Add(
					GraphBuilder.CreateUAV(
						GraphBuilder.CreateTexture(
							FRDGTextureDesc::Create3D(FIntVector(1, 1, 1), Format, FClearValueBinding::Black, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV),
							TEXT("NiagaraEmptyTextureUAV::Texture3D")
						),
						ERDGUnorderedAccessViewFlags::SkipBarrier
					)
				);
				break;

			case ETextureDimension::TextureCube:
				Pool.UAVs.Add(
					GraphBuilder.CreateUAV(
						GraphBuilder.CreateTexture(
							FRDGTextureDesc::CreateCube(1, Format, FClearValueBinding::Black, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV),
							TEXT("NiagaraEmptyTextureUAV::TextureCube")
						),
						ERDGUnorderedAccessViewFlags::SkipBarrier
					)
				);
				break;

			case ETextureDimension::TextureCubeArray:
				Pool.UAVs.Add(
					GraphBuilder.CreateUAV(
						GraphBuilder.CreateTexture(
							FRDGTextureDesc::CreateCubeArray(1, Format, FClearValueBinding::Black, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV, 1),
							TEXT("NiagaraEmptyTextureUAV::TextureCubeArray")
						),
						ERDGUnorderedAccessViewFlags::SkipBarrier
					)
				);
				break;

			default:
				checkNoEntry();
				return nullptr;
		}
	}
	return Pool.UAVs[Pool.NextFreeIndex++];
}

void FNiagaraEmptyUAVPool::ResetEmptyRDGUAVPools()
{
	for ( auto it=BufferRDGUAVPool.CreateIterator(); it; ++it )
	{
		it.Value().NextFreeIndex = 0;
	}

	for ( auto it=TextureRDGUAVPool.CreateIterator(); it; ++it )
	{
		it.Value().NextFreeIndex = 0;
	}
}

FNiagaraEmptyUAVPool::FEmptyUAV::~FEmptyUAV()
{
	Buffer.SafeRelease();
	Texture.SafeRelease();
	UAV.SafeRelease();
}

FNiagaraEmptyUAVPool::FEmptyUAVPool::~FEmptyUAVPool()
{
	UE_CLOGF(NextFreeIndex != 0, LogNiagara, Warning, "EmptyUAVPool is potentially in use during destruction.");
	DEC_DWORD_STAT_BY(STAT_NiagaraEmptyUAVPool, UAVs.Num());
}

