// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEvents.h"
#include "NiagaraStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEvents)


DECLARE_DWORD_COUNTER_STAT(TEXT("Num Death Events"), STAT_NiagaraNumDeathEvents, STATGROUP_Niagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Spawn Events"), STAT_NiagaraNumSpawnEvents, STATGROUP_Niagara);

void UNiagaraEventReceiverEmitterAction_SpawnParticles::PerformAction(FNiagaraEmitterInstance& OwningSim, const FNiagaraEventReceiverProperties& OwningEventReceiver)
{
	//Find the generator set we're bound to and spawn NumParticles particles for every event it generated last frame.
// 	FNiagaraDataSet* GeneratorSet = OwningSim.GetParentSystemInstance()->GetDataSet(FNiagaraDataSetID(OwningEventReceiver.SourceEventGenerator, ENiagaraDataSetType::Event), OwningEventReceiver.SourceEmitter);
// 	if (GeneratorSet)
// 	{
// 		OwningSim.SpawnBurst(GeneratorSet->GetPrevNumInstances()*NumParticles);
// 	}
}
//////////////////////////////////////////////////////////////////////////

