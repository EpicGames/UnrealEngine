// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraFunctionLibrary.h"
#include "Components/StaticMeshComponent.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/Engine.h"
#include "Engine/SkeletalMesh.h"
#include "NiagaraComponent.h"
#include "Engine/StaticMesh.h"
#include "NiagaraSystem.h"
#include "ContentStreaming.h"
#include "Engine/Texture2D.h"
#include "Internationalization/Internationalization.h"

#include "NiagaraAsyncGpuTraceHelper.h"
#include "NiagaraComponentPool.h"
#include "NiagaraWorldManager.h"
#include "DataInterface/NiagaraDataInterfaceStaticMesh.h"
#include "DataInterface/NiagaraDataInterfaceSceneCapture2D.h"
#include "NiagaraDataInterfaceSkeletalMesh.h"
#include "NiagaraDataInterfaceTexture.h"
#include "NiagaraDataInterface2DArrayTexture.h"
#include "NiagaraDataInterfaceVolumeTexture.h"
#include "NiagaraDataInterfaceCubeTexture.h"
#include "Engine/VolumeTexture.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCube.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Engine/TextureRenderTargetVolume.h"
#include "Engine/TextureRenderTargetCube.h"
#include "NiagaraGpuComputeDispatchInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraFunctionLibrary)

#define LOCTEXT_NAMESPACE "NiagaraFunctionLibrary"

//DECLARE_CYCLE_STAT(TEXT("FastDot4"), STAT_NiagaraFastDot4, STATGROUP_Niagara);

#if WITH_EDITOR
int32 GForceNiagaraSpawnAttachedSolo = 0;
static FAutoConsoleVariableRef CVarForceNiagaraSpawnAttachedSolo(
	TEXT("fx.ForceNiagaraSpawnAttachedSolo"),
	GForceNiagaraSpawnAttachedSolo,
	TEXT("If > 0 Niagara systems which are spawned attached will be force to spawn in solo mode for debugging.\n"),
	ECVF_Default
);
#endif


int32 GAllowFastPathFunctionLibrary = 0;
static FAutoConsoleVariableRef CVarAllowFastPathFunctionLibrary(
	TEXT("fx.AllowFastPathFunctionLibrary"),
	GAllowFastPathFunctionLibrary,
	TEXT("If > 0 Allow the graph to insert custom fastpath operations into the graph.\n"),
	ECVF_Default
);

TArray<FNiagaraFunctionSignature> UNiagaraFunctionLibrary::VectorVMOps;
TArray<FString> UNiagaraFunctionLibrary::VectorVMOpsHLSL;

UNiagaraFunctionLibrary::UNiagaraFunctionLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}


UNiagaraComponent* CreateNiagaraSystem(UNiagaraSystem* SystemTemplate, UWorld* World, AActor* Actor, bool bAutoDestroy, ENCPoolMethod PoolingMethod)
{
	UNiagaraComponent* NiagaraComponent = nullptr;

	if (FApp::CanEverRender() && World && !World->IsNetMode(NM_DedicatedServer))
	{
		if (PoolingMethod == ENCPoolMethod::None)
		{
			NiagaraComponent = NewObject<UNiagaraComponent>((Actor ? Actor : (UObject*)World), NAME_None, RF_Transient);
			NiagaraComponent->SetAsset(SystemTemplate);
			NiagaraComponent->bAutoActivate = false;
			NiagaraComponent->SetAutoDestroy(bAutoDestroy);
			NiagaraComponent->bAllowAnyoneToDestroyMe = true;
			NiagaraComponent->SetVisibleInRayTracing(false);
		}
		else if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World))
		{
			UNiagaraComponentPool* ComponentPool = WorldManager->GetComponentPool();
			NiagaraComponent = ComponentPool->CreateWorldParticleSystem(SystemTemplate, World, PoolingMethod);
		}
	}
	return NiagaraComponent;
}


UNiagaraComponent* UNiagaraFunctionLibrary::SpawnSystemAtLocation(const UObject* WorldContextObject, UNiagaraSystem* SystemTemplate, FVector SpawnLocation, FRotator SpawnRotation, FVector Scale, bool bAutoDestroy, bool bAutoActivate, ENCPoolMethod PoolingMethod, bool bPreCullCheck)
{
	FFXSystemSpawnParameters SpawnParams;
	SpawnParams.WorldContextObject = WorldContextObject;
	SpawnParams.SystemTemplate = SystemTemplate;
	SpawnParams.Location = SpawnLocation;
	SpawnParams.Rotation = SpawnRotation;
	SpawnParams.Scale = Scale;
	SpawnParams.bAutoDestroy = bAutoDestroy;
	SpawnParams.bAutoActivate = bAutoActivate;
	SpawnParams.PoolingMethod = ToPSCPoolMethod(PoolingMethod);
	SpawnParams.bPreCullCheck = bPreCullCheck;
	return SpawnSystemAtLocationWithParams(SpawnParams);
}

/**
* Spawns a Niagara System at the specified world location/rotation
* @return			The spawned UNiagaraComponent
*/
UNiagaraComponent* UNiagaraFunctionLibrary::SpawnSystemAtLocationWithParams(const FFXSystemSpawnParameters& SpawnParams)
{
	UNiagaraComponent* PSC = NULL;
	UNiagaraSystem* NiagaraSystem = Cast<UNiagaraSystem>(SpawnParams.SystemTemplate);
	if (NiagaraSystem)
	{
		UWorld* World = GEngine->GetWorldFromContextObject(SpawnParams.WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if (World != nullptr)
		{
			bool bShouldCull = false;
			if (SpawnParams.bPreCullCheck)
			{
				bool bIsPlayerEffect = SpawnParams.bIsPlayerEffect;
				if (NiagaraSystem->AllowCullingForLocalPlayers() || bIsPlayerEffect == false)
				{
					FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World);
					bShouldCull = WorldManager->ShouldPreCull(NiagaraSystem, SpawnParams.Location);
				}
			}

			if (!bShouldCull)
			{
				PSC = CreateNiagaraSystem(NiagaraSystem, World, World->GetWorldSettings(), SpawnParams.bAutoDestroy, ToNiagaraPooling(SpawnParams.PoolingMethod));
				if(PSC)
				{
					if (SpawnParams.bIsPlayerEffect)
					{
						PSC->SetForceLocalPlayerEffect(true);
					}

#if WITH_EDITORONLY_DATA
					PSC->bWaitForCompilationOnActivate = GIsAutomationTesting;
#endif

					if (!PSC->IsRegistered())
					{
						PSC->RegisterComponentWithWorld(World);
					}

					PSC->SetAbsolute(true, true, true);
					PSC->SetWorldLocationAndRotation(SpawnParams.Location, SpawnParams.Rotation);
					PSC->SetRelativeScale3D(SpawnParams.Scale);
					if (SpawnParams.bAutoActivate)
					{
						PSC->Activate(true);
					}
				}
			}
		}
	}
	return PSC;
}


/**
* Spawns a Niagara System attached to a component
* @return			The spawned UNiagaraComponent
*/
UNiagaraComponent* UNiagaraFunctionLibrary::SpawnSystemAttached(UNiagaraSystem* SystemTemplate, USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, FRotator Rotation, EAttachLocation::Type LocationType, bool bAutoDestroy, bool bAutoActivate, ENCPoolMethod PoolingMethod, bool bPreCullCheck)
{
	return SpawnSystemAttached(SystemTemplate, AttachToComponent, AttachPointName, Location, Rotation, FVector(1, 1, 1), LocationType, bAutoDestroy, PoolingMethod, bAutoActivate, bPreCullCheck);
}

UNiagaraComponent* UNiagaraFunctionLibrary::SpawnSystemAttached(
	UNiagaraSystem* SystemTemplate,
	USceneComponent* AttachToComponent,
	FName AttachPointName,
	FVector Location,
	FRotator Rotation,
	FVector Scale,
	EAttachLocation::Type LocationType,
	bool bAutoDestroy,
	ENCPoolMethod PoolingMethod,
	bool bAutoActivate,
	bool bPreCullCheck
)
{
	FFXSystemSpawnParameters SpawnParams;
	SpawnParams.SystemTemplate = SystemTemplate;
	SpawnParams.AttachToComponent = AttachToComponent;
	SpawnParams.AttachPointName = AttachPointName;
	SpawnParams.Location = Location;
	SpawnParams.Rotation = Rotation;
	SpawnParams.Scale = Scale;
	SpawnParams.LocationType = LocationType;
	SpawnParams.bAutoDestroy = bAutoDestroy;
	SpawnParams.bAutoActivate = bAutoActivate;
	SpawnParams.PoolingMethod = ToPSCPoolMethod(PoolingMethod);
	SpawnParams.bPreCullCheck = bPreCullCheck;
	return SpawnSystemAttachedWithParams(SpawnParams);
}

/**
* Spawns a Niagara System attached to a component
* @return			The spawned UNiagaraComponent
*/

UNiagaraComponent* UNiagaraFunctionLibrary::SpawnSystemAttachedWithParams(const FFXSystemSpawnParameters& SpawnParams)
{
	LLM_SCOPE(ELLMTag::Niagara);
	
	UNiagaraComponent* PSC = nullptr;
	UNiagaraSystem* NiagaraSystem = Cast<UNiagaraSystem>(SpawnParams.SystemTemplate);
	if (NiagaraSystem)
	{
		if (!SpawnParams.AttachToComponent)
		{
			UE_LOGF(LogScript, Warning, "UGameplayStatics::SpawnNiagaraEmitterAttached: NULL AttachComponent specified!");
		}
		else
		{
			UWorld* const World = SpawnParams.AttachToComponent->GetWorld();
			if (World && !World->IsNetMode(NM_DedicatedServer))
			{
				bool bShouldCull = false;
				if (SpawnParams.bPreCullCheck)
				{
					//Don't precull if this is a local player linked effect and the system doesn't allow us to cull those.
					bool bIsPlayerEffect = SpawnParams.bIsPlayerEffect || FNiagaraWorldManager::IsComponentLocalPlayerLinked(SpawnParams.AttachToComponent);
					if (NiagaraSystem->AllowCullingForLocalPlayers() || bIsPlayerEffect == false)
					{
						FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World);
						//TODO: For now using the attach parent location and ignoring the emitters relative location which is clearly going to be a bit wrong in some cases.
						bShouldCull = WorldManager->ShouldPreCull(NiagaraSystem, SpawnParams.AttachToComponent->GetComponentLocation());
					}
				}

				if (!bShouldCull)
				{
					PSC = CreateNiagaraSystem(NiagaraSystem, World, SpawnParams.AttachToComponent->GetOwner(), SpawnParams.bAutoDestroy, ToNiagaraPooling(SpawnParams.PoolingMethod));
					if (PSC)
					{
						if (SpawnParams.bIsPlayerEffect)
						{
							PSC->SetForceLocalPlayerEffect(true);
						}
						
#if WITH_EDITOR
						if (GForceNiagaraSpawnAttachedSolo > 0)
						{
							PSC->SetForceSolo(true);
						}
#endif
						auto SetupRelativeTransforms = [&]()
						{
							if (SpawnParams.LocationType == EAttachLocation::KeepWorldPosition)
							{
								const FTransform ParentToWorld = SpawnParams.AttachToComponent->GetSocketTransform(SpawnParams.AttachPointName);
								const FTransform ComponentToWorld(SpawnParams.Rotation, SpawnParams.Location, SpawnParams.Scale);
								const FTransform RelativeTM = ComponentToWorld.GetRelativeTransform(ParentToWorld);
								PSC->SetRelativeLocation_Direct(RelativeTM.GetLocation());
								PSC->SetRelativeRotation_Direct(RelativeTM.GetRotation().Rotator());
								PSC->SetRelativeScale3D_Direct(RelativeTM.GetScale3D());
							}
							else
							{
								PSC->SetRelativeLocation_Direct(SpawnParams.Location);
								PSC->SetRelativeRotation_Direct(SpawnParams.Rotation);

								if (SpawnParams.LocationType == EAttachLocation::SnapToTarget)
								{
									// SnapToTarget indicates we "keep world scale", this indicates we we want the inverse of the parent-to-world scale 
									// to calculate world scale at Scale 1, and then apply the passed in Scale
									const FTransform ParentToWorld = SpawnParams.AttachToComponent->GetSocketTransform(SpawnParams.AttachPointName);
									PSC->SetRelativeScale3D_Direct(SpawnParams.Scale * ParentToWorld.GetSafeScaleReciprocal(ParentToWorld.GetScale3D()));
								}
								else
								{
									PSC->SetRelativeScale3D_Direct(SpawnParams.Scale);
								}
							}
						};

						if (PSC->IsRegistered())
						{
							//It is now possible for us to be already regisetered so we must use AttachToComponent() instead.
							SetupRelativeTransforms();
							PSC->AttachToComponent(SpawnParams.AttachToComponent, FAttachmentTransformRules::KeepRelativeTransform, SpawnParams.AttachPointName);
						}
						else
						{
							PSC->SetupAttachment(SpawnParams.AttachToComponent, SpawnParams.AttachPointName);
							SetupRelativeTransforms();
							PSC->RegisterComponentWithWorld(World);
						}

						if (SpawnParams.bAutoActivate)
						{
							PSC->Activate(true);
						}

						// Notify the texture streamer so that PSC gets managed as a dynamic component.
						IStreamingManager::Get().NotifyPrimitiveUpdated(PSC);
					}
				}
			}
		}
	}
	return PSC;
}

void UNiagaraFunctionLibrary::OverrideSystemUserVariableStaticMeshComponent(UNiagaraComponent* NiagaraSystem, const FString& OverrideName, UStaticMeshComponent* StaticMeshComponent)
{
	if (!NiagaraSystem)
	{
		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "NiagaraSystem in \"Set Niagara Static Mesh Component\" is NULL, OverrideName \"%ls\" and StaticMeshComponent \"%ls\", skipping.", *OverrideName, *GetFullNameSafe(StaticMeshComponent));
		}
		return;
	}

	if (!StaticMeshComponent)
	{
		UE_LOGF(LogNiagara, Warning, "StaticMeshComponent in \"Set Niagara Static Mesh Component\" is NULL, OverrideName \"%ls\" and NiagaraSystem \"%ls\", skipping.", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}

	const FNiagaraParameterStore& OverrideParameters = NiagaraSystem->GetOverrideParameters();

	FNiagaraVariable Variable(FNiagaraTypeDefinition(UNiagaraDataInterfaceStaticMesh::StaticClass()), *OverrideName);
	
	int32 Index = OverrideParameters.IndexOf(Variable);
	if (Index == INDEX_NONE)
	{
		UE_LOGF(LogNiagara, Warning, "Could not find index of variable \"%ls\" in the OverrideParameters map of NiagaraSystem \"%ls\".", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}
	
	UNiagaraDataInterfaceStaticMesh* StaticMeshInterface = Cast<UNiagaraDataInterfaceStaticMesh>(OverrideParameters.GetDataInterface(Index));
	if (!StaticMeshInterface)
	{
		UE_LOGF(LogNiagara, Warning, "Did not find a matching Static Mesh Data Interface variable named \"%ls\" in the User variables of NiagaraSystem \"%ls\" .", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}

	StaticMeshInterface->SetSourceComponentFromBlueprints(StaticMeshComponent);
}

void UNiagaraFunctionLibrary::OverrideSystemUserVariableStaticMesh(UNiagaraComponent* NiagaraSystem, const FString& OverrideName, UStaticMesh* StaticMesh)
{
	if (!NiagaraSystem)
	{
		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "NiagaraSystem in \"Set Niagara Static Mesh Component\" is NULL, OverrideName \"%ls\" and StaticMesh \"%ls\", skipping.", *OverrideName, *GetFullNameSafe(StaticMesh));
		}
		return;
	}

	if (!StaticMesh)
	{
		UE_LOGF(LogNiagara, Warning, "StaticMesh in \"Set Niagara Static Mesh Component\" is NULL, OverrideName \"%ls\" and NiagaraSystem \"%ls\", skipping.", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}

	const FNiagaraParameterStore& OverrideParameters = NiagaraSystem->GetOverrideParameters();

	FNiagaraVariable Variable(FNiagaraTypeDefinition(UNiagaraDataInterfaceStaticMesh::StaticClass()), *OverrideName);

	int32 Index = OverrideParameters.IndexOf(Variable);
	if (Index == INDEX_NONE)
	{
		UE_LOGF(LogNiagara, Warning, "Could not find index of variable \"%ls\" in the OverrideParameters map of NiagaraSystem \"%ls\".", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}

	UNiagaraDataInterfaceStaticMesh* StaticMeshInterface = Cast<UNiagaraDataInterfaceStaticMesh>(OverrideParameters.GetDataInterface(Index));
	if (!StaticMeshInterface)
	{
		UE_LOGF(LogNiagara, Warning, "Did not find a matching Static Mesh Data Interface variable named \"%ls\" in the User variables of NiagaraSystem \"%ls\" .", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}

	StaticMeshInterface->SetDefaultMeshFromBlueprints(StaticMesh);
}

UNiagaraDataInterfaceSkeletalMesh* UNiagaraFunctionLibrary::GetSkeletalMeshDataInterface(UNiagaraComponent* NiagaraSystem, const FString& OverrideName)
{
	if (!NiagaraSystem)
	{
		return nullptr;
	}

	const FNiagaraParameterStore& OverrideParameters = NiagaraSystem->GetOverrideParameters();
	FNiagaraVariable Variable(FNiagaraTypeDefinition(UNiagaraDataInterfaceSkeletalMesh::StaticClass()), *OverrideName);

	const int32 Index = OverrideParameters.IndexOf(Variable);
	return Index != INDEX_NONE ? Cast<UNiagaraDataInterfaceSkeletalMesh>(OverrideParameters.GetDataInterface(Index)) : nullptr;
}

void UNiagaraFunctionLibrary::OverrideSystemUserVariableSkeletalMeshComponent(UNiagaraComponent* NiagaraSystem, const FString& OverrideName, USkeletalMeshComponent* SkeletalMeshComponent)
{
	if (!NiagaraSystem)
	{
		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "NiagaraSystem in \"Set Niagara Skeletal Mesh Component\" is NULL, OverrideName \"%ls\" and SkeletalMeshComponent \"%ls\" SkeletalMesh \"%ls\", skipping.", *OverrideName, *GetFullNameSafe(SkeletalMeshComponent), *GetFullNameSafe(SkeletalMeshComponent ? SkeletalMeshComponent->GetSkeletalMeshAsset() : nullptr));
		}
		return;
	}

	if (!SkeletalMeshComponent)
	{
		UE_LOGF(LogNiagara, Warning, "SkeletalMeshComponent in \"Set Niagara Skeletal Mesh Component\" is NULL, OverrideName \"%ls\" and NiagaraSystem \"%ls\", skipping.", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}

	UNiagaraDataInterfaceSkeletalMesh* SkeletalMeshInterface = GetSkeletalMeshDataInterface(NiagaraSystem, OverrideName);
	if (!SkeletalMeshInterface)
	{
		UE_LOGF(LogNiagara, Warning, "UNiagaraFunctionLibrary::OverrideSystemUserVariableSkeletalMeshComponent: Did not find a matching Skeletal Mesh Data Interface variable named \"%ls\" in the User variables of NiagaraSystem \"%ls\" .", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}

	SkeletalMeshInterface->SetSourceComponentFromBlueprints(SkeletalMeshComponent);
}

void UNiagaraFunctionLibrary::SetSkeletalMeshDataInterfaceSamplingRegions(UNiagaraComponent* NiagaraSystem, const FString& OverrideName, const TArray<FName>& SamplingRegions)
{
	if (!NiagaraSystem)
	{
		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "NiagaraSystem in \"Set Skeletal Mesh Data Interface Sampling Regions\" is NULL, OverrideName \"%ls\", skipping.", *OverrideName);
		}
		return;
	}

	UNiagaraDataInterfaceSkeletalMesh* SkeletalMeshInterface = GetSkeletalMeshDataInterface(NiagaraSystem, OverrideName);
	if (!SkeletalMeshInterface)
	{
		UE_LOGF(LogNiagara, Warning, "UNiagaraFunctionLibrary::SetSkeletalMeshDataInterfaceSamplingRegions: Did not find a matching Skeletal Mesh Data Interface variable named \"%ls\" in the User variables of NiagaraSystem \"%ls\" .", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}

	SkeletalMeshInterface->SetSamplingRegionsFromBlueprints(SamplingRegions);
}


void UNiagaraFunctionLibrary::SetSkeletalMeshDataInterfaceFilteredBones(UNiagaraComponent* NiagaraSystem, const FString& OverrideName, const TArray<FName>& FilteredBones)
{
	if (!NiagaraSystem)
	{
		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "NiagaraSystem in \"Set Skeletal Mesh Data Interface Filtered Bones\" is NULL, OverrideName \"%ls\", skipping.", *OverrideName);
		}
		return;
	}

	UNiagaraDataInterfaceSkeletalMesh* SkeletalMeshInterface = GetSkeletalMeshDataInterface(NiagaraSystem, OverrideName);
	if (!SkeletalMeshInterface)
	{
		UE_LOGF(LogNiagara, Warning, "UNiagaraFunctionLibrary::SetSkeletalMeshDataInterfaceFilteredBones: Did not find a matching Skeletal Mesh Data Interface variable named \"%ls\" in the User variables of NiagaraSystem \"%ls\" .", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}

	SkeletalMeshInterface->SetFilteredBonesFromBlueprints(FilteredBones);
}

void UNiagaraFunctionLibrary::SetSkeletalMeshDataInterfaceFilteredSockets(UNiagaraComponent* NiagaraSystem, const FString& OverrideName, const TArray<FName>& FilteredSockets)
{
	if (!NiagaraSystem)
	{
		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "NiagaraSystem in \"Set Skeletal Mesh Data Interface Filtered Sockets\" is NULL, OverrideName \"%ls\", skipping.", *OverrideName);
		}
		return;
	}

	UNiagaraDataInterfaceSkeletalMesh* SkeletalMeshInterface = GetSkeletalMeshDataInterface(NiagaraSystem, OverrideName);
	if (!SkeletalMeshInterface)
	{
		UE_LOGF(LogNiagara, Warning, "UNiagaraFunctionLibrary::SetSkeletalMeshDataInterfaceFilteredSockets: Did not find a matching Skeletal Mesh Data Interface variable named \"%ls\" in the User variables of NiagaraSystem \"%ls\" .", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}

	SkeletalMeshInterface->SetFilteredSocketsFromBlueprints(FilteredSockets);
}

NIAGARA_API void UNiagaraFunctionLibrary::SetSceneCapture2DDataInterfaceManagedMode(UNiagaraComponent* NiagaraSystem, const FName& DIName, 
	ESceneCaptureSource ManagedCaptureSource, FIntPoint ManagedTextureSize, ETextureRenderTargetFormat ManagedTextureFormat,
	ECameraProjectionMode::Type ManagedProjectionType, float ManagedFOVAngle, float ManagedOrthoWidth, bool bManagedCaptureEveryFrame,
	bool bManagedCaptureOnMovement, const TArray<AActor*>& ShowOnlyActors, float ManagedLODDistanceFactor)
{
	if (!NiagaraSystem)
	{
		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "NiagaraSystem in \"SetSceneCapture2DDataInterfaceManagedMode\" is NULL, skipping.");
		}
		return;
	}

	UNiagaraDataInterfaceSceneCapture2D* CaptureDI =
		UNiagaraFunctionLibrary::GetDataInterface< UNiagaraDataInterfaceSceneCapture2D>(NiagaraSystem, DIName);

	if (!CaptureDI)
	{
		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "Scene Capture 2D DI \"%ls\" is Null in \"SetSceneCapture2DDataInterfaceManagedMode\" is NULL, skipping.", *DIName.ToString());
		}
		return;
	}

	CaptureDI->SourceMode = ENDISceneCapture2DSourceMode::Managed;

	CaptureDI->ManagedCaptureSource = ManagedCaptureSource;
	CaptureDI->ManagedTextureSize = ManagedTextureSize;
	CaptureDI->ManagedTextureFormat = ManagedTextureFormat;
	CaptureDI->ManagedProjectionType = ManagedProjectionType;
	CaptureDI->ManagedFOVAngle = ManagedFOVAngle;
	CaptureDI->ManagedOrthoWidth = ManagedOrthoWidth;
	CaptureDI->bManagedCaptureEveryFrame = bManagedCaptureEveryFrame;
	CaptureDI->bManagedCaptureOnMovement = bManagedCaptureOnMovement;
	CaptureDI->ManagedLODDistanceFactor = ManagedLODDistanceFactor;
	
	CaptureDI->ManagedShowOnlyActors.Empty();

	TArray<TObjectPtr<AActor>> AllActors;
	for (AActor* CurrActor : ShowOnlyActors)
	{
		AllActors.Add(CurrActor);
	}
	
	CaptureDI->ManagedShowOnlyActors.Append(AllActors);
}

void UNiagaraFunctionLibrary::SetTextureObject(UNiagaraComponent* NiagaraSystem, const FString& OverrideName, UTexture* Texture)
{
	if (!NiagaraSystem)
	{
		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "NiagaraSystem in \"SetTextureObject\" is NULL, OverrideName \"%ls\" and Texture \"%ls\", skipping.", *OverrideName, *GetFullNameSafe(Texture));
		}
		return;
	}

	if (!Texture)
	{
		UE_LOGF(LogNiagara, Warning, "Texture in \"SetTextureObject\" is NULL, OverrideName \"%ls\" and NiagaraSystem \"%ls\", skipping.", *OverrideName, *GetFullNameSafe(NiagaraSystem));
		return;
	}

	const FNiagaraParameterStore& OverrideParameters = NiagaraSystem->GetOverrideParameters();

	if (Texture->IsA<UTexture2D>() || Texture->IsA<UTextureRenderTarget2D>())
	{
		const FNiagaraVariable Variable(FNiagaraTypeDefinition(UNiagaraDataInterfaceTexture::StaticClass()), *OverrideName);
		const int32 Index = OverrideParameters.IndexOf(Variable);
		if (Index == INDEX_NONE)
		{
			UE_LOGF(LogNiagara, Warning, "Could not find index of variable \"%ls\" in the OverrideParameters map of NiagaraSystem \"%ls\".", *OverrideName, *GetFullNameSafe(NiagaraSystem));
			return;
		}

		UNiagaraDataInterfaceTexture* TextureDI = Cast<UNiagaraDataInterfaceTexture>(OverrideParameters.GetDataInterface(Index));
		if (!TextureDI)
		{
			UE_LOGF(LogNiagara, Warning, "Did not find a matching Texture Data Interface variable named \"%ls\" in the User variables of NiagaraSystem \"%ls\" .", *OverrideName, *GetFullNameSafe(NiagaraSystem));
			return;
		}

		TextureDI->SetTexture(Texture);
	}
	else if (Texture->IsA<UTexture2DArray>() || Texture->IsA<UTextureRenderTarget2DArray>())
	{
		const FNiagaraVariable Variable(FNiagaraTypeDefinition(UNiagaraDataInterface2DArrayTexture::StaticClass()), *OverrideName);
		const int32 Index = OverrideParameters.IndexOf(Variable);
		if (Index == INDEX_NONE)
		{
			UE_LOGF(LogNiagara, Warning, "Could not find index of variable \"%ls\" in the OverrideParameters map of NiagaraSystem \"%ls\".", *OverrideName, *GetFullNameSafe(NiagaraSystem));
			return;
		}

		UNiagaraDataInterface2DArrayTexture* TextureDI = Cast<UNiagaraDataInterface2DArrayTexture>(OverrideParameters.GetDataInterface(Index));
		if (!TextureDI)
		{
			UE_LOGF(LogNiagara, Warning, "Did not find a matching Texture Data Interface variable named \"%ls\" in the User variables of NiagaraSystem \"%ls\" .", *OverrideName, *GetFullNameSafe(NiagaraSystem));
			return;
		}

		TextureDI->SetTexture(Texture);
	}
	else if (Texture->IsA<UVolumeTexture>() || Texture->IsA<UTextureRenderTargetVolume>())
	{
		const FNiagaraVariable Variable(FNiagaraTypeDefinition(UNiagaraDataInterfaceVolumeTexture::StaticClass()), *OverrideName);
		const int32 Index = OverrideParameters.IndexOf(Variable);
		if (Index == INDEX_NONE)
		{
			UE_LOGF(LogNiagara, Warning, "Could not find index of variable \"%ls\" in the OverrideParameters map of NiagaraSystem \"%ls\".", *OverrideName, *GetFullNameSafe(NiagaraSystem));
			return;
		}

		UNiagaraDataInterfaceVolumeTexture* TextureDI = Cast<UNiagaraDataInterfaceVolumeTexture>(OverrideParameters.GetDataInterface(Index));
		if (!TextureDI)
		{
			UE_LOGF(LogNiagara, Warning, "Did not find a matching Texture Data Interface variable named \"%ls\" in the User variables of NiagaraSystem \"%ls\" .", *OverrideName, *GetFullNameSafe(NiagaraSystem));
			return;
		}

		TextureDI->SetTexture(Texture);
	}
	else if (Texture->IsA<UTextureCube>() || Texture->IsA<UTextureRenderTargetCube>())
	{
		const FNiagaraVariable Variable(FNiagaraTypeDefinition(UNiagaraDataInterfaceCubeTexture::StaticClass()), *OverrideName);
		const int32 Index = OverrideParameters.IndexOf(Variable);
		if (Index == INDEX_NONE)
		{
			UE_LOGF(LogNiagara, Warning, "Could not find index of variable \"%ls\" in the OverrideParameters map of NiagaraSystem \"%ls\".", *OverrideName, *GetFullNameSafe(NiagaraSystem));
			return;
		}

		UNiagaraDataInterfaceCubeTexture* TextureDI = Cast<UNiagaraDataInterfaceCubeTexture>(OverrideParameters.GetDataInterface(Index));
		if (!TextureDI)
		{
			UE_LOGF(LogNiagara, Warning, "Did not find a matching Texture Data Interface variable named \"%ls\" in the User variables of NiagaraSystem \"%ls\" .", *OverrideName, *GetFullNameSafe(NiagaraSystem));
			return;
		}

		TextureDI->SetTexture(Texture);
	}
	else
	{
		UE_LOGF(LogNiagara, Warning, "Texture in \"SetTextureObject\" is of an unsupported type \"%ls\", OverrideName \"%ls\" and NiagaraSystem \"%ls\", skipping.", *OverrideName, *GetNameSafe(Texture->GetClass()), *GetFullNameSafe(NiagaraSystem));
		return;
	}
}

void UNiagaraFunctionLibrary::SetTexture2DArrayObject(UNiagaraComponent* NiagaraSystem, const FString& OverrideName, UTexture2DArray* Texture)
	{
	SetTextureObject(NiagaraSystem, OverrideName, Texture);
	}

void UNiagaraFunctionLibrary::SetVolumeTextureObject(UNiagaraComponent* NiagaraSystem, const FString& OverrideName, UVolumeTexture* Texture)
{
	SetTextureObject(NiagaraSystem, OverrideName, Texture);
}

TArray<FNiagaraUserParameterInfo> UNiagaraFunctionLibrary::GetAllUserParameters(UNiagaraSystem* NiagaraSystem)
{
	TArray<FNiagaraUserParameterInfo> Result;
	if (NiagaraSystem)
	{
		TArray<FNiagaraVariable> Parameters;
		NiagaraSystem->GetExposedParameters().GetUserParameters(Parameters);
		for (const FNiagaraVariable& UserVar : Parameters)
		{
			FNiagaraUserParameterInfo& ParameterInfo = Result.AddDefaulted_GetRef();
			ParameterInfo.ParameterName = UserVar.GetName();
			ParameterInfo.ParameterType = UserVar.GetType().GetStruct();
			ParameterInfo.TypeName = UserVar.GetType().GetFName();
			ParameterInfo.ParameterSizeBytes = UserVar.GetType().GetSize();
			ParameterInfo.bIsEnumType = UserVar.GetType().IsEnum();
		}
	}
	return Result;
}

TArray<FNiagaraMinimalEmitterInfo> UNiagaraFunctionLibrary::GetAllEmitters(UNiagaraSystem* NiagaraSystem)
{
	TArray<FNiagaraMinimalEmitterInfo> Result;
	if (NiagaraSystem)
	{
		for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem->GetEmitterHandles())
		{
			
			FNiagaraMinimalEmitterInfo& EmitterInfo = Result.AddDefaulted_GetRef();
			EmitterInfo.EmitterName = EmitterHandle.GetName();
			EmitterInfo.bIsEnabled = EmitterHandle.GetIsEnabled();
			EmitterInfo.bIsLightweight = EmitterHandle.GetEmitterMode() == ENiagaraEmitterMode::Stateless;
			
			EmitterHandle.ForEachEnabledRendererWithIndex([&EmitterInfo](const UNiagaraRendererProperties* Properties, int32)
				{
					TArray<UMaterialInterface*> Materials;
					Properties->GetUsedMaterials(nullptr, Materials);
					EmitterInfo.UsedRendererMaterials.Append(Materials);

					TArray<UObject*> Meshes;
					Properties->GetUsedMeshes(nullptr, Meshes);
					EmitterInfo.UsedRendererMeshes.Append(Meshes);
				});
		}
	}
	return Result;
}

UNiagaraDataInterface* UNiagaraFunctionLibrary::GetDataInterface(UClass* DIClass, UNiagaraComponent* NiagaraSystem, FName OverrideName)
{
	if (NiagaraSystem == nullptr)
	{
		if (FNiagaraUtilities::LogVerboseWarnings())
		{
			UE_LOGF(LogNiagara, Warning, "NiagaraSystem was nullptr for OverrideName \"%ls\".", *OverrideName.ToString());
		}
		return nullptr;
	}

	const FNiagaraParameterStore& OverrideParameters = NiagaraSystem->GetOverrideParameters();
	FNiagaraVariableBase Variable(FNiagaraTypeDefinition(DIClass), OverrideName);
	const int32* Index = OverrideParameters.FindParameterOffset(Variable, true);
	if (Index == nullptr)
	{
		UE_LOGF(LogNiagara, Warning, "OverrideParameter(%ls) System(%ls) DataInterface(%ls) was not found", *OverrideName.ToString(), *GetNameSafe(NiagaraSystem->GetAsset()), *GetNameSafe(DIClass));
		return nullptr;
	}

	UNiagaraDataInterface* UntypedDI = OverrideParameters.GetDataInterface(*Index);
	if (UntypedDI == nullptr)
	{
		UE_LOGF(LogNiagara, Warning, "OverrideParameter(%ls) System(%ls) DataInterface is nullptr.", *OverrideName.ToString(), *GetNameSafe(NiagaraSystem->GetAsset()));
		return nullptr;
	}

	if (UntypedDI->IsA(DIClass))
	{
		return UntypedDI;
	}
	UE_LOGF(LogNiagara, Warning, "OverrideParameter(%ls) System(%ls) DataInterface is a (%ls) and not expected type (%ls)", *OverrideName.ToString(), *GetNameSafe(NiagaraSystem->GetAsset()), *GetNameSafe(UntypedDI->GetClass()), *GetNameSafe(DIClass));
	return nullptr;
}

UNiagaraParameterCollectionInstance* UNiagaraFunctionLibrary::GetNiagaraParameterCollection(UObject* WorldContextObject, UNiagaraParameterCollection* Collection)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World != nullptr)
	{
		return FNiagaraWorldManager::Get(World)->GetParameterCollection(Collection);
	}
	return nullptr;
}

UNiagaraParameterCollectionInstance* UNiagaraFunctionLibrary::CreateNiagaraParameterCollectionInstance(UObject* WorldContextObject, UNiagaraParameterCollection* ParentCollection)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ParentCollection || !World)
	{
		return nullptr;
	}

	UObject* CollectionOuter = GetTransientPackage();
	static const FName CollectionBaseName("RuntimeNPC");
	const FName CollectionName = MakeUniqueObjectName(CollectionOuter, UNiagaraParameterCollectionInstance::StaticClass(), CollectionBaseName);

	UNiagaraParameterCollectionInstance* DefaultInstance = ParentCollection->GetDefaultInstance();
	UNiagaraParameterCollectionInstance* NewInstance = CastChecked<UNiagaraParameterCollectionInstance>(StaticDuplicateObject(DefaultInstance, CollectionOuter, CollectionName));

#if WITH_EDITOR
	// Bind to the default instance so that changes to the collection propagate through.
	DefaultInstance->GetParameterStore().Bind(&NewInstance->GetParameterStore());
#endif

	NewInstance->Bind(World);
	return NewInstance;
}

const TArray<FNiagaraFunctionSignature>& UNiagaraFunctionLibrary::GetVectorVMFastPathOps(bool bIgnoreConsoleVariable)
{
	if (!bIgnoreConsoleVariable && (GAllowFastPathFunctionLibrary == 0))
	{
		static TArray<FNiagaraFunctionSignature> Empty;
		return Empty;
	}

	InitVectorVMFastPathOps();
	return VectorVMOps;
}

bool UNiagaraFunctionLibrary::DefineFunctionHLSL(const FNiagaraFunctionSignature& FunctionSignature, FString& HlslOutput)
{
	InitVectorVMFastPathOps();

	const int32 i = VectorVMOps.IndexOfByKey(FunctionSignature);
	if ( i == INDEX_NONE )
	{
		return false;
	}

	HlslOutput += VectorVMOpsHLSL[i];
	return true;
}

const FName FastPathLibraryName(TEXT("FastPathLibrary"));
const FName FastPathDot4Name(TEXT("FastPathDot4"));
const FName FastPathTransformPositionName(TEXT("FastPathTransformPosition"));
const FName FastMatrixToQuaternionName(TEXT("FastMatrixToQuaternion"));
const FName FastUniqueRandomName(TEXT("FastUniqueRandom"));

struct FVectorKernelFastDot4
{
	static FNiagaraFunctionSignature GetFunctionSignature()
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FastPathDot4Name;
		Sig.OwnerName = FastPathLibraryName;
		Sig.bMemberFunction = false;
		Sig.bRequiresContext = false;
		Sig.SetDescription(LOCTEXT("FastPathDot4Desc", "Fast path for Vector4 dot product."));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("A")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("B")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Value")));
		return Sig;
	}

	static FString GetFunctionHLSL()
	{
		return FString();
	}

	static void Exec(FVectorVMExternalFunctionContext& Context)
	{
		//SCOPE_CYCLE_COUNTER(STAT_NiagaraFastDot4);

		VectorVM::FExternalFuncInputHandler<float>InVecA[4] =
		{
			VectorVM::FExternalFuncInputHandler<float>(Context), 
			VectorVM::FExternalFuncInputHandler<float>(Context), 
			VectorVM::FExternalFuncInputHandler<float>(Context),
			VectorVM::FExternalFuncInputHandler<float>(Context)
		};

		VectorVM::FExternalFuncInputHandler<float>InVecB[4] =
		{
			VectorVM::FExternalFuncInputHandler<float>(Context),
			VectorVM::FExternalFuncInputHandler<float>(Context),
			VectorVM::FExternalFuncInputHandler<float>(Context),
			VectorVM::FExternalFuncInputHandler<float>(Context)
		};


		VectorVM::FExternalFuncRegisterHandler<float> OutValue(Context);

		VectorRegister4Float* RESTRICT AX = (VectorRegister4Float *)InVecA[0].GetDest();
		VectorRegister4Float* RESTRICT AY = (VectorRegister4Float *)InVecA[1].GetDest();
		VectorRegister4Float* RESTRICT AZ = (VectorRegister4Float *)InVecA[2].GetDest();
		VectorRegister4Float* RESTRICT AW = (VectorRegister4Float *)InVecA[3].GetDest();

		VectorRegister4Float* RESTRICT BX = (VectorRegister4Float *)InVecB[0].GetDest();
		VectorRegister4Float* RESTRICT BY = (VectorRegister4Float *)InVecB[1].GetDest();
		VectorRegister4Float* RESTRICT BZ = (VectorRegister4Float *)InVecB[2].GetDest();
		VectorRegister4Float* RESTRICT BW = (VectorRegister4Float *)InVecB[3].GetDest();
		VectorRegister4Float* RESTRICT Out = (VectorRegister4Float *)OutValue.GetDest();

		const int32 Loops = Context.GetNumLoops<4>();
		for (int32 i = 0; i < Loops; ++i)
		{

			VectorRegister4Float AVX0= VectorLoadAligned(&AX[i]);
			VectorRegister4Float AVY0= VectorLoadAligned(&AY[i]);
			VectorRegister4Float AVZ0= VectorLoadAligned(&AZ[i]);
			VectorRegister4Float AVW0= VectorLoadAligned(&AW[i]);
			VectorRegister4Float BVX0= VectorLoadAligned(&BX[i]);
			VectorRegister4Float BVY0= VectorLoadAligned(&BY[i]);
			VectorRegister4Float BVZ0= VectorLoadAligned(&BZ[i]);
			VectorRegister4Float BVW0= VectorLoadAligned(&BW[i]);

			/*
				 R[19] = :mul(R[21], R[25]);
				 R[21] = :mad(R[20], R[24], R[19]);
				 R[19] = :mad(R[22], R[26], R[21]);
				 R[20] = :mad(R[23], R[27], R[19]);
			*/
			VectorRegister4Float AMBX0	= VectorMultiply(AVX0, BVX0);
			VectorRegister4Float AMBXY0= VectorMultiplyAdd(AVY0, BVY0, AMBX0);
			VectorRegister4Float AMBXYZ0= VectorMultiplyAdd(AVZ0, BVZ0, AMBXY0);
			VectorRegister4Float AMBXYZW0= VectorMultiplyAdd(AVW0, BVW0, AMBXYZ0);
			VectorStoreAligned(AMBXYZW0, &Out[i]) ;

			/*
			(float Sum = 0.0f;
			for (int32 VecIdx = 0; VecIdx < 4; VecIdx++)
			{
				Sum += InVecA[VecIdx].GetAndAdvance() * InVecB[VecIdx].GetAndAdvance();
			}

			*OutValue.GetDestAndAdvance() = Sum;
			*/
		}
	}
};

struct FVectorKernelFastTransformPosition
{
	static FNiagaraFunctionSignature GetFunctionSignature()
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FastPathTransformPositionName;
		Sig.OwnerName = FastPathLibraryName;
		Sig.bMemberFunction = false;
		Sig.bRequiresContext = false;
		Sig.SetDescription(LOCTEXT("FastPathTransformPositionDesc", "Fast path for Matrix4 transforming a Vector3 position"));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Mat")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("PositionTransformed")));
		return Sig;
	}

	static FString GetFunctionHLSL()
	{
		return FString();
	}

	static void Exec(FVectorVMExternalFunctionContext& Context)
	{
#if 0
		TArray<VectorVM::FExternalFuncInputHandler<float>, TInlineAllocator<16>> InMatrix;
		for (int i = 0; i < 16; i++)
		{
			InMatrix.Emplace(Context);
		}

		TArray<VectorVM::FExternalFuncInputHandler<float>, TInlineAllocator<16>> InVec;
		for (int i = 0; i < 3; i++)
		{
			InVec.Emplace(Context);
		}

		TArray<VectorVM::FExternalFuncInputHandler<float>, TInlineAllocator<16>> OutVec;
		for (int i = 0; i < 3; i++)
		{
			OutVec.Emplace(Context);
		}

		/*
	29	| R[19] = :mul(R[20], R[26]);
	30	| R[27] = :mul(R[21], R[26]);
	31	| R[28] = :mul(R[22], R[26]);
	32	| R[29] = :mul(R[23], R[26]);
	33	| R[26] = :mul(R[20], R[25]);
	34	| R[30] = :mul(R[21], R[25]);
	35	| R[31] = :mul(R[22], R[25]);
	36	| R[32] = :mul(R[23], R[25]);
	37	| R[25] = :mul(R[20], R[24]);
	38	| R[33] = :mul(R[21], R[24]);
	39	| R[34] = :mul(R[22], R[24]);
	40	| R[35] = :mul(R[23], R[24]);
	41	| R[24] = :add(R[26], R[25]);
	42	| R[25] = :add(R[30], R[33]);
	43	| R[26] = :add(R[31], R[34]);
	44	| R[30] = :add(R[32], R[35]);
	45	| R[31] = :add(R[19], R[24]);
	46	| R[19] = :add(R[27], R[25]);
	47	| R[24] = :add(R[28], R[26]);
	48	| R[25] = :add(R[29], R[30]);
	49	| R[26] = :add(R[20], R[31]);
	50	| R[20] = :add(R[21], R[19]);
	51	| R[19] = :add(R[22], R[24]);
		*/
		/*float* RESTRICT M00 = (float *)InMatrix[0].GetDest();
		float* RESTRICT M01 = (float *)InMatrix[1].GetDest();
		float* RESTRICT M02 = (float *)InMatrix[2].GetDest();
		float* RESTRICT M03 = (float *)InMatrix[3].GetDest();
		float* RESTRICT M10 = (float *)InMatrix[4].GetDest();
		float* RESTRICT M11 = (float *)InMatrix[5].GetDest();
		float* RESTRICT M12 = (float *)InMatrix[6].GetDest();
		float* RESTRICT M13 = (float *)InMatrix[7].GetDest(); 
		float* RESTRICT M20 = (float *)InMatrix[8].GetDest();
		float* RESTRICT M21 = (float *)InMatrix[9].GetDest();
		float* RESTRICT M22 = (float *)InMatrix[10].GetDest();
		float* RESTRICT M23 = (float *)InMatrix[11].GetDest();
		float* RESTRICT M30 = (float *)InMatrix[12].GetDest();
		float* RESTRICT M31 = (float *)InMatrix[13].GetDest();
		float* RESTRICT M32 = (float *)InMatrix[14].GetDest();
		float* RESTRICT M33 = (float *)InMatrix[15].GetDest();*/
		TArray<float* RESTRICT, TInlineAllocator<16>> InMatrixDest;
		for (int i = 0; i < 16; i++)
		{
			InMatrixDest.Emplace((float *)InMatrix[i].GetDest());
		}

		float* RESTRICT BX = (float *)InVec[0].GetDest();
		float* RESTRICT BY = (float *)InVec[1].GetDest();
		float* RESTRICT BZ = (float *)InVec[2].GetDest();
		
		float* RESTRICT OutValueX = (float *)OutVec[0].GetDest();
		float* RESTRICT OutValueY = (float *)OutVec[1].GetDest();
		float* RESTRICT OutValueZ = (float *)OutVec[2].GetDest();
		
		int32 LoopInstances = Align(Context.NumInstances, 1) /1;
		for (int32 i = 0; i < Context.NumInstances; ++i)
		{
			/*FMatrix LocalMat;
			for (int32 Row = 0; Row < 4; Row++)
			{
				for (int32 Column = 0; Column < 4; Column++)
				{
					LocalMat.M[Row][Column] = *InMatrixDest[Row * 4 + Column]++;
				}
			}*/

			for (int32 Row = 0; Row < 4; Row++)
			{
				for (int32 Column = 0; Column < 4; Column++)
				{
					*InMatrixDest[Row * 4 + Column]++;
				}
			}
			*BX++; *BY++; *BZ++;

			//FVector LocalVec(*BX++, *BY++, *BZ++);

			//FVector LocalOutVec = LocalMat.TransformPosition(LocalVec);

			//*(OutValueX)++ = LocalOutVec.X;
			//*(OutValueY)++ = LocalOutVec.Y;
			//*(OutValueZ)++ = LocalOutVec.Z;
		}
#endif
	}
};

struct FVectorKernelFastMatrixToQuaternion
{
	static FNiagaraFunctionSignature GetFunctionSignature()
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FastMatrixToQuaternionName;
		Sig.OwnerName = FastPathLibraryName;
		Sig.bMemberFunction = false;
		Sig.bRequiresContext = false;
		Sig.SetDescription(LOCTEXT("FastMatrixToQuaternionDesc", "Fast path for Matrix4 to Quaternion"));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Mat")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Quat")));
		return Sig;
	}

	static FString GetFunctionHLSL()
	{
		static FString FunctionHLSL = TEXT(R"(
	void FastMatrixToQuaternion_FastPathLibrary(float4x4 Mat, out float4 Quat)
	{
		float tr = Mat[0][0] + Mat[1][1] + Mat[2][2];
		if (tr > 0.0f)
		{
			float InvS = rsqrt(tr + 1.f);
			float s = 0.5f * InvS;

			Quat.x = (Mat[1][2] - Mat[2][1]) * s;
			Quat.y = (Mat[2][0] - Mat[0][2]) * s;
			Quat.z = (Mat[0][1] - Mat[1][0]) * s;
			Quat.w = 0.5f * rcp(InvS);
		}
		else if ( (Mat[0][0] > Mat[1][1]) && (Mat[0][0] > Mat[2][2]) )
		{
			float s = Mat[0][0] - Mat[1][1] - Mat[2][2] + 1.0f;
			float InvS = rsqrt(s);
			s = 0.5f * InvS;

			Quat.x = 0.5f * rcp(InvS);
			Quat.y = (Mat[0][1] + Mat[1][0]) * s;
			Quat.z = (Mat[0][2] + Mat[2][0]) * s;
			Quat.w = (Mat[1][2] - Mat[2][1]) * s;
		}
		else if ( Mat[1][1] > Mat[2][2] )
		{
			float s = Mat[1][1] - Mat[2][2] - Mat[0][0] + 1.0f;
			float InvS = rsqrt(s);
			s = 0.5f * InvS;

			Quat.x = (Mat[1][0] + Mat[0][1]) * s;
			Quat.y = 0.5f * rcp(InvS);
			Quat.z = (Mat[1][2] + Mat[2][1]) * s;
			Quat.w = (Mat[2][0] - Mat[0][2]) * s;

		}
		else
		{
			float s = Mat[2][2] - Mat[0][0] - Mat[1][1] + 1.0f;
			float InvS = rsqrt(s);
			s = 0.5f * InvS;

			Quat.x = (Mat[2][0] + Mat[0][2]) * s;
			Quat.y = (Mat[2][1] + Mat[1][2]) * s;
			Quat.z = 0.5f * rcp(InvS);
			Quat.w = (Mat[0][1] - Mat[1][0]) * s;
		}
	}
)");

		return FunctionHLSL;
	}

	static void Exec(FVectorVMExternalFunctionContext& Context)
	{
		TArray<VectorVM::FExternalFuncInputHandler<float>, TInlineAllocator<16>> InMatrix;
		for (int i=0; i < 16; i++)
		{
			InMatrix.Emplace(Context);
		}

		TArray<VectorVM::FExternalFuncRegisterHandler<float>, TInlineAllocator<4>> OutQuat;
		for (int i=0; i < 4; ++i)
		{
			OutQuat.Emplace(Context);
		}

		for ( int32 iInstance=0; iInstance < Context.GetNumInstances(); ++iInstance)
		{
			// Read Matrix
			FMatrix44f Mat;
			for ( int32 j=0; j < 16; ++j )
			{
				Mat.M[j >> 2][j & 3] = InMatrix[j].GetAndAdvance();
			}

			// Generate Quat (without checking for consistency with the GPU)
			FQuat4f Quat;
			{
				// Check diagonal (trace)
				const float tr = Mat.M[0][0] + Mat.M[1][1] + Mat.M[2][2];
				if (tr > 0.0f)
				{
					float InvS = FMath::InvSqrt(tr + 1.f);
					Quat.W = 0.5f * (1.f / InvS);
					const float s = 0.5f * InvS;

					Quat.X = (Mat.M[1][2] - Mat.M[2][1]) * s;
					Quat.Y = (Mat.M[2][0] - Mat.M[0][2]) * s;
					Quat.Z = (Mat.M[0][1] - Mat.M[1][0]) * s;
				}
				else
				{
					// diagonal is negative
					int32 i = 0;

					if (Mat.M[1][1] > Mat.M[0][0])
						i = 1;

					if (Mat.M[2][2] > Mat.M[i][i])
						i = 2;

					const int32 j = (i + 1) % 3;
					const int32 k = (j + 1) % 3;

					float s = Mat.M[i][i] - Mat.M[j][j] - Mat.M[k][k] + 1.0f;

					float InvS = FMath::InvSqrt(s);

					float qt[4] = {};
					qt[i] = 0.5f * (1.f / InvS);

					s = 0.5f * InvS;

					qt[3] = (Mat.M[j][k] - Mat.M[k][j]) * s;
					qt[j] = (Mat.M[i][j] + Mat.M[j][i]) * s;
					qt[k] = (Mat.M[i][k] + Mat.M[k][i]) * s;

					Quat.X = qt[0];
					Quat.Y = qt[1];
					Quat.Z = qt[2];
					Quat.W = qt[3];
				}
			}

			*OutQuat[0].GetDestAndAdvance() = Quat.X;
			*OutQuat[1].GetDestAndAdvance() = Quat.Y;
			*OutQuat[2].GetDestAndAdvance() = Quat.Z;
			*OutQuat[3].GetDestAndAdvance() = Quat.W;
		}
	}
};

struct FVectorKernelFastUniqueRandom
{
	static FNiagaraFunctionSignature GetFunctionSignature()
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = FastUniqueRandomName;
		Sig.OwnerName = FastPathLibraryName;
		Sig.bMemberFunction = false;
		Sig.bRequiresContext = false;
		Sig.SetDescription(LOCTEXT("FastUniqueRandomDesc", "Fast path for GetUniqueRandom function."));
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Seed")));
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("NumUniqueValues")), LOCTEXT("UniqueValuesTooltip", "The number of unique values to draw from. Example: a value of 20 will generate numbers between [0..19]."));
		FNiagaraVariable IterationInput(FNiagaraTypeDefinition::GetIntDef(), TEXT("Iterations"));
		IterationInput.SetValue(3); // default value
		Sig.AddInput(IterationInput, LOCTEXT("IterationTooltip", "More iterations increase entropy at the cost of performance"));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Value")));
		return Sig;
	}

	static FString GetFunctionHLSL()
	{
		static FString FunctionHLSL = TEXT(R"(
			uint CalcLFSRValue(uint Seed, uint NumUniqueValues, int Iterations, uint4 Taps, uint Mask)
			{
				uint NextValue = (Seed + 1) & Mask;
				for (int i = 0; i < Iterations; i++) {
					do {
						uint Tap = 0;
						Tap ^= (Taps.x & NextValue) == 0 ? 0 : 1;
						Tap ^= (Taps.y & NextValue) == 0 ? 0 : 1;
						Tap ^= (Taps.z & NextValue) == 0 ? 0 : 1;
						Tap ^= (Taps.w & NextValue) == 0 ? 0 : 1;
						NextValue = (((NextValue << 1) | Tap) & Mask);
					} while (NextValue > NumUniqueValues);
				}
				return NextValue - 1;
			}

			void FastUniqueRandom_FastPathLibrary(int Seed, int NumUniqueValues, int Iterations, out int Result)
			{
				if (NumUniqueValues < 2) {
					NumUniqueValues = 2;
				}
				Seed %= NumUniqueValues;

				if (NumUniqueValues < 16) {
					uint4 Taps = {(1 << 3), (1 << 2), 0, 0};
					uint Mask = 0xF;
					Result = CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
				}
				else if (NumUniqueValues < 256) {
					uint4 Taps = {(1 << 7), (1 << 5), (1 << 4), (1 << 3)};
					uint Mask = 0xFF;
					Result = CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
				}
				else if (NumUniqueValues < 4096) {
					uint4 Taps = {(1 << 11), (1 << 10), (1 << 7), (1 << 5)};
					uint Mask = 0xFFF;
					Result = CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
				}
				else if (NumUniqueValues < 65536) {
					uint4 Taps = {(1 << 15), (1 << 13), (1 << 12), (1 << 10)};
					uint Mask = 0xFFFF;
					Result = CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
				}
				else if (NumUniqueValues < 1048576) {
					uint4 Taps ={(1 << 19), (1 << 16), 0, 0};
					uint Mask = 0xFFFFF;
					Result = CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
				}
				else {
					uint4 Taps = {(1 << 23), (1 << 22), (1 << 20), (1 << 19)};
					uint Mask = 0xFFFFFF;
					Result = CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
				}
			}
		)");
		return FunctionHLSL;
	}

	static unsigned int CalcLFSRValue(const unsigned int& Seed, const unsigned int& NumUniqueValues, const int& Iterations, const FIntVector4& Taps, const unsigned int& Mask)
	{
		unsigned int NextValue = (Seed + 1) & Mask;
		for (int i = 0; i < Iterations; i++) {
			do {
				unsigned int Tap = 0;
				Tap ^= (Taps.X & NextValue) == 0 ? 0 : 1;
				Tap ^= (Taps.Y & NextValue) == 0 ? 0 : 1;
				Tap ^= (Taps.Z & NextValue) == 0 ? 0 : 1;
				Tap ^= (Taps.W & NextValue) == 0 ? 0 : 1;
				NextValue = (((NextValue << 1) | Tap) & Mask);
			} while (NextValue > NumUniqueValues);
		}
		return NextValue - 1;
	}

	static unsigned int FastUniqueRandom_FastPathLibrary(int Seed, int NumUniqueValues, int Iterations)
	{
		if (NumUniqueValues < 2) {
			NumUniqueValues = 2;
		}
		Seed %= NumUniqueValues;
		
		// Source for the tap values: Table of Linear Feedback Shift Registers: Roy Ward, Tim Molteno, 2007
		if (NumUniqueValues < 16) {
			FIntVector4 Taps = {(1 << 3), (1 << 2), 0, 0};
			unsigned int Mask = 0xF;
			return CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
		}
		if (NumUniqueValues < 256) {
			FIntVector4 Taps = {(1 << 7), (1 << 5), (1 << 4), (1 << 3)};
			unsigned int Mask = 0xFF;
			return CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
		}
		if (NumUniqueValues < 4096) {
			FIntVector4 Taps = {(1 << 11), (1 << 10), (1 << 7), (1 << 5)};
			unsigned int Mask = 0xFFF;
			return CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
		}
		if (NumUniqueValues < 65536) {
			FIntVector4 Taps = {(1 << 15), (1 << 13), (1 << 12), (1 << 10)};
			unsigned int Mask = 0xFFFF;
			return CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
		}
		if (NumUniqueValues < 1048576) {
			FIntVector4 Taps ={(1 << 19), (1 << 16), 0, 0};
			unsigned int Mask = 0xFFFFF;
			return CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
		}
		FIntVector4 Taps = {(1 << 23), (1 << 22), (1 << 20), (1 << 19)};
		unsigned int Mask = 0xFFFFFF;
		return CalcLFSRValue(Seed, NumUniqueValues, Iterations, Taps, Mask);
	}

	static void Exec(FVectorVMExternalFunctionContext& Context)
	{
		FNDIInputParam<int32> InSeed(Context);
		FNDIInputParam<int32> InNumUniqueValues(Context);
		FNDIInputParam<int32> InIterations(Context);
		FNDIOutputParam<int32> OutValue(Context);
		
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			int32 Seed = InSeed.GetAndAdvance();
			int32 NumUniqueValues = InNumUniqueValues.GetAndAdvance();
			int32 Iterations = InIterations.GetAndAdvance();

			int32 Result = FastUniqueRandom_FastPathLibrary(Seed, NumUniqueValues, Iterations);
			OutValue.SetAndAdvance(Result);
		}
	}
};

bool UNiagaraFunctionLibrary::GetVectorVMFastPathExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, FVMExternalFunction &OutFunc)
{
	if (BindingInfo.Name == FastPathDot4Name)
	{
		OutFunc = FVMExternalFunction::CreateStatic(FVectorKernelFastDot4::Exec);
		return true;
	}
	else if (BindingInfo.Name == FastPathTransformPositionName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(FVectorKernelFastTransformPosition::Exec);
		return true;
	}
	else if (BindingInfo.Name == FastMatrixToQuaternionName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(FVectorKernelFastMatrixToQuaternion::Exec);
		return true;
	}
	else if (BindingInfo.Name == FastUniqueRandomName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(FVectorKernelFastUniqueRandom::Exec);
		return true;
	}
	return false;
}

void UNiagaraFunctionLibrary::InitVectorVMFastPathOps()
{
	static FRWLock FastPathBuildLock;

	{
		FReadScopeLock Lock(FastPathBuildLock);

		if (!VectorVMOps.IsEmpty())
		{
			return;
		}
	}

	FWriteScopeLock Lock(FastPathBuildLock);
	if (!VectorVMOps.IsEmpty())
	{
		return;
	}

	VectorVMOps =
	{
		FVectorKernelFastDot4::GetFunctionSignature(),
		FVectorKernelFastTransformPosition::GetFunctionSignature(),
		FVectorKernelFastMatrixToQuaternion::GetFunctionSignature(),
		FVectorKernelFastUniqueRandom::GetFunctionSignature(),
	};

	VectorVMOpsHLSL =
	{
		FVectorKernelFastDot4::GetFunctionHLSL(),
		FVectorKernelFastTransformPosition::GetFunctionHLSL(),
		FVectorKernelFastMatrixToQuaternion::GetFunctionHLSL(),
		FVectorKernelFastUniqueRandom::GetFunctionHLSL(),
	};

	check(VectorVMOps.Num() == VectorVMOpsHLSL.Num());
}


//////////////////////////////////////////////////////////////////////////
//GPU Ray Traced Collision Functions

void UNiagaraFunctionLibrary::SetComponentNiagaraGPURayTracedCollisionGroup(UObject* WorldContextObject, UPrimitiveComponent* Primitive, int32 CollisionGroup)
{
#if NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS
	if ( UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) )
	{
		if ( FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = FNiagaraGpuComputeDispatchInterface::Get(World) )
		{
			ComputeDispatchInterface->GetAsyncGpuTraceHelper().SetPrimitiveRayTracingCollisionGroup_GT(Primitive, CollisionGroup);
		}
	}
#endif
}

void UNiagaraFunctionLibrary::SetActorNiagaraGPURayTracedCollisionGroup(UObject* WorldContextObject, AActor* Actor, int32 CollisionGroup)
{
#if NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS
	if (Actor == nullptr)
	{
		return;
	}

	if ( UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) )
	{
		if (FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = FNiagaraGpuComputeDispatchInterface::Get(World))
		{
			FNiagaraAsyncGpuTraceHelper& Helper = ComputeDispatchInterface->GetAsyncGpuTraceHelper();

			Actor->ForEachComponent<UPrimitiveComponent>(
				/*bIncludeFromChildActors*/true,
				[&Helper, &CollisionGroup](UPrimitiveComponent* PrimitiveComponent)
				{
					Helper.SetPrimitiveRayTracingCollisionGroup_GT(PrimitiveComponent, CollisionGroup);
				});
			}
	}
#endif
}

int32 UNiagaraFunctionLibrary::AcquireNiagaraGPURayTracedCollisionGroup(UObject* WorldContextObject)
{
#if NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		if (FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = FNiagaraGpuComputeDispatchInterface::Get(World))
		{
			return ComputeDispatchInterface->GetAsyncGpuTraceHelper().AcquireGPURayTracedCollisionGroup_GT();
		}
	}
#endif
	return INDEX_NONE;
}

void UNiagaraFunctionLibrary::ReleaseNiagaraGPURayTracedCollisionGroup(UObject* WorldContextObject, int32 CollisionGroup)
{
#if NIAGARA_ASYNC_GPU_TRACE_COLLISION_GROUPS
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		if (FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = FNiagaraGpuComputeDispatchInterface::Get(World))
		{
			ComputeDispatchInterface->GetAsyncGpuTraceHelper().ReleaseGPURayTracedCollisionGroup_GT(CollisionGroup);
		}
	}
#endif
}
#undef LOCTEXT_NAMESPACE
