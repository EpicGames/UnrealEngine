// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGPUInstanceCountManager.h"
#include "NiagaraEmptyUAVPool.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraRenderer.h"
#include "NiagaraStats.h"
#include "NiagaraShader.h"

#include "Containers/DynamicRHIResourceArray.h"
#include "GPUSortManager.h" // CopyUIntBufferToTargets
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "PipelineStateCache.h"
#include "RHIGPUReadback.h"

int32 GNiagaraMinGPUInstanceCount = 2048;
static FAutoConsoleVariableRef CVarNiagaraMinGPUInstanceCount(
	TEXT("Niagara.MinGPUInstanceCount"),
	GNiagaraMinGPUInstanceCount,
	TEXT("Minimum number of instance count entries allocated in the global buffer. (default=2048)"),
	ECVF_Default
);

int32 GNiagaraGPUCountManagerAllocateIncrement = 64;
static FAutoConsoleVariableRef CVarNiagaraGPUCountManagerAllocateIncrement(
	TEXT("Niagara.GPUCountManager.AllocateIncrement"),
	GNiagaraGPUCountManagerAllocateIncrement,
	TEXT("If we run out of space for allocations this is how many allocate rather than a single entry. (default=64)"),
	ECVF_Default
);

int32 GNiagaraMinCulledGPUInstanceCount = 2048;
static FAutoConsoleVariableRef CVarNiagaraMinCulledGPUInstanceCount(
	TEXT("Niagara.MinCulledGPUInstanceCount"),
	GNiagaraMinCulledGPUInstanceCount,
	TEXT("Minimum number of culled (per-view) instance count entries allocated in the global buffer. (default=2048)"),
	ECVF_Default
);

float GNiagaraGPUCountBufferSlack = 1.5f;
static FAutoConsoleVariableRef CVarNiagaraGPUCountBufferSlack(
	TEXT("Niagara.GPUCountBufferSlack"),
	GNiagaraGPUCountBufferSlack,
	TEXT("Multiplier of the GPU count buffer size to prevent frequent re-allocation."),
	ECVF_Default
);

int32 GNiagaraIndirectArgsPoolMinSize = 256;
static FAutoConsoleVariableRef CVarNiagaraIndirectArgsPoolMinSize(
	TEXT("fx.Niagara.IndirectArgsPool.MinSize"),
	GNiagaraIndirectArgsPoolMinSize,
	TEXT("Minimum number of draw indirect args allocated into the pool. (default=256)"),
	ECVF_Default
);

float GNiagaraIndirectArgsPoolBlockSizeFactor = 2.0f;
static FAutoConsoleVariableRef CNiagaraIndirectArgsPoolBlockSizeFactor(
	TEXT("fx.Niagara.IndirectArgsPool.BlockSizeFactor"),
	GNiagaraIndirectArgsPoolBlockSizeFactor,
	TEXT("Multiplier on the indirect args pool size when needing to increase it from running out of space. (default=2.0)"),
	ECVF_Default
);

int32 GNiagaraIndirectArgsPoolAllowShrinking = 1;
static FAutoConsoleVariableRef CVarNiagaraIndirectArgsPoolAllowShrinking(
	TEXT("fx.Niagara.IndirectArgsPool.AllowShrinking"),
	GNiagaraIndirectArgsPoolAllowShrinking,
	TEXT("Allow the indirect args pool to shrink after a number of frames below a low water mark."),
	ECVF_Default
);

float GNiagaraIndirectArgsPoolLowWaterAmount = 0.5f;
static FAutoConsoleVariableRef CVarNiagaraIndirectArgsPoolLowWaterAmount(
	TEXT("fx.Niagara.IndirectArgsPool.LowWaterAmount"),
	GNiagaraIndirectArgsPoolLowWaterAmount,
	TEXT("Percentage (0-1) of the indirect args pool that is considered low and worthy of shrinking"),
	ECVF_Default
);

int32 GNiagaraIndirectArgsPoolLowWaterFrames = 150;
static FAutoConsoleVariableRef CVarNiagaraIndirectArgsPoolLowWaterFrames(
	TEXT("fx.Niagara.IndirectArgsPool.LowWaterFrames"),
	GNiagaraIndirectArgsPoolLowWaterFrames,
	TEXT("The number of frames to wait to shrink the indirect args pool for being below the low water mark. (default=150)"),
	ECVF_Default
);

DECLARE_DWORD_COUNTER_STAT(TEXT("Used GPU Instance Counters"), STAT_NiagaraUsedGPUInstanceCounters, STATGROUP_Niagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("Indirect Draw Calls"), STAT_NiagaraIndirectDraws, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("GPU Readback Lock"), STAT_NiagaraGPUReadbackLock, STATGROUP_Niagara);

#ifndef ENABLE_NIAGARA_INDIRECT_ARG_POOL_LOG
#define ENABLE_NIAGARA_INDIRECT_ARG_POOL_LOG 0
#endif

#if ENABLE_NIAGARA_INDIRECT_ARG_POOL_LOG
#define INDIRECT_ARG_POOL_LOG(Format, ...) UE_LOG(LogNiagara, Log, TEXT("NIAGARA INDIRECT ARG POOL: ") TEXT(Format), __VA_ARGS__)
#else
#define INDIRECT_ARG_POOL_LOG(Format, ...) do {} while(0)
#endif

//*****************************************************************************

const ERHIAccess FNiagaraGPUInstanceCountManager::kCountBufferDefaultState = ERHIAccess::SRVMask | ERHIAccess::CopySrc;
const ERHIAccess FNiagaraGPUInstanceCountManager::kIndirectArgsDefaultState = ERHIAccess::IndirectArgs | ERHIAccess::SRVMask;

FNiagaraGPUInstanceCountManager::FNiagaraGPUInstanceCountManager(ERHIFeatureLevel::Type InFeatureLevel)
	: FeatureLevel(InFeatureLevel)
{
}

FNiagaraGPUInstanceCountManager::~FNiagaraGPUInstanceCountManager()
{
	ReleaseRHI();
}

void FNiagaraGPUInstanceCountManager::InitRHI(FRHICommandListBase& RHICmdList)
{
}

void FNiagaraGPUInstanceCountManager::ReleaseRHI()
{
	ReleaseCounts();

	for (auto& PoolEntry : DrawIndirectPool)
	{
		PoolEntry->Buffer.Release();
	}
	DrawIndirectPool.Empty();
}

void FNiagaraGPUInstanceCountManager::ReleaseCounts()
{
	CountBuffer.Release();
	CulledCountBuffer.Release();
	MultiViewCountBuffer.Release();

	AllocatedInstanceCounts = 0;
	AllocatedCulledCounts = 0;

	if (CountReadback)
	{
		delete CountReadback;
		CountReadback = nullptr;
		CountReadbackSize = 0;
	}
}

uint32 FNiagaraGPUInstanceCountManager::AcquireEntry()
{
	check(IsInParallelRenderingThread());

	UE::TScopeLock LockGuard(AcquireEntryGuard);

	if (FreeEntries.Num())
	{
		return FreeEntries.Pop();
	}
	else if (UsedInstanceCounts < AllocatedInstanceCounts)
	{
		// We can't reallocate on the fly, the buffer must be correctly resized before any tick gets scheduled.
		return UsedInstanceCounts++;
	}
	else
	{
		return INDEX_NONE;
	}
}

uint32 FNiagaraGPUInstanceCountManager::AcquireOrAllocateEntry(FRHICommandListImmediate& RHICmdList)
{
	check(IsInRenderingThread());

	// Free entries?
	if (FreeEntries.Num())
	{
		return FreeEntries.Pop();
	}
	else if (UsedInstanceCounts < AllocatedInstanceCounts)
	{
		return UsedInstanceCounts++;
	}

	// We need to resize
	ResizeBuffers(RHICmdList, AllocatedInstanceCounts + GNiagaraGPUCountManagerAllocateIncrement);

	check(UsedInstanceCounts < AllocatedInstanceCounts);
	return UsedInstanceCounts++;
}

void FNiagaraGPUInstanceCountManager::FreeEntry(uint32& BufferOffset)
{
	check(IsInRenderingThread());

	if (BufferOffset != INDEX_NONE)
	{
		checkf(!FreeEntries.Contains(BufferOffset), TEXT("BufferOffset %u exists in FreeEntries"), BufferOffset);
		checkf(!InstanceCountClearTasks.Contains(BufferOffset), TEXT("BufferOffset %u exists in InstanceCountClearTasks"), BufferOffset);

		InstanceCountClearTasks.Add(BufferOffset);
		BufferOffset = INDEX_NONE;
	}
}

void FNiagaraGPUInstanceCountManager::FreeEntryArray(TConstArrayView<uint32> EntryArray)
{
	check(IsInRenderingThread());

	const int32 NumToFree = EntryArray.Num();
	if (NumToFree > 0)
	{
#if DO_CHECK
		for (uint32 BufferOffset : EntryArray)
		{
			checkf(!FreeEntries.Contains(BufferOffset), TEXT("BufferOffset %u exists in FreeEntries"), BufferOffset);
			checkf(!InstanceCountClearTasks.Contains(BufferOffset), TEXT("BufferOffset %u exists in InstanceCountClearTasks"), BufferOffset);
		}
#endif
		InstanceCountClearTasks.Append(EntryArray.GetData(), NumToFree);
	}
}

FRWBuffer* FNiagaraGPUInstanceCountManager::AcquireCulledCountsBuffer(FRHICommandListImmediate& RHICmdList)
{
	check(IsInRenderingThread());

	if (RequiredCulledCounts > 0)
	{
		if (!bAcquiredCulledCounts)
		{
			const int32 RecommendedCulledCounts = FMath::Max(GNiagaraMinCulledGPUInstanceCount, (int32)(RequiredCulledCounts * GNiagaraGPUCountBufferSlack));
			if (RecommendedCulledCounts > AllocatedCulledCounts)
			{
				// We need a bigger buffer
				CulledCountBuffer.Release();

				AllocatedCulledCounts = RecommendedCulledCounts;
				CulledCountBuffer.Initialize(RHICmdList, TEXT("NiagaraCulledGPUInstanceCounts"), sizeof(uint32), AllocatedCulledCounts, EPixelFormat::PF_R32_UINT, ERHIAccess::UAVCompute);
				CulledCountsRHIAccess = ERHIAccess::UAVCompute;
			}
			else if (CulledCountsRHIAccess != ERHIAccess::UAVCompute)
			{
				RHICmdList.Transition(FRHITransitionInfo(CulledCountBuffer.UAV, CulledCountsRHIAccess, ERHIAccess::UAVCompute));
				CulledCountsRHIAccess = ERHIAccess::UAVCompute;
			}

			// Initialize the buffer by clearing it to zero then transition it to be ready to write to
			RHICmdList.ClearUAVUint(CulledCountBuffer.UAV, FUintVector4(EForceInit::ForceInitToZero));
			RHICmdList.Transition(FRHITransitionInfo(CulledCountBuffer.UAV, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute));

			bAcquiredCulledCounts = true;
		}
		else if (CulledCountsRHIAccess != ERHIAccess::UAVCompute)
		{
			RHICmdList.Transition(FRHITransitionInfo(CulledCountBuffer.UAV, CulledCountsRHIAccess, ERHIAccess::UAVCompute));
			CulledCountsRHIAccess = ERHIAccess::UAVCompute;
		}

		return &CulledCountBuffer;
	}

	return nullptr;
}

void FNiagaraGPUInstanceCountManager::ResizeBuffers(FRHICommandListImmediate& RHICmdList, int32 ReservedInstanceCounts)
{
	const int32 RequiredInstanceCounts = UsedInstanceCounts + FMath::Max<int32>(ReservedInstanceCounts - FreeEntries.Num(), 0);
	if (RequiredInstanceCounts > 0)
	{
		const int32 RecommendedInstanceCounts = FMath::Max(GNiagaraMinGPUInstanceCount, (int32)(RequiredInstanceCounts * GNiagaraGPUCountBufferSlack));
		// If the buffer is not allocated, allocate it to the recommended size.
		if (!AllocatedInstanceCounts)
		{
			AllocatedInstanceCounts = RecommendedInstanceCounts;
			TResourceArray<uint32> InitData;
			InitData.AddZeroed(AllocatedInstanceCounts);
			CountBuffer.Initialize(RHICmdList, TEXT("NiagaraGPUInstanceCounts"), sizeof(uint32), AllocatedInstanceCounts, EPixelFormat::PF_R32_UINT, kCountBufferDefaultState, BUF_Static | BUF_SourceCopy, &InitData);
			//UE_LOGF(LogNiagara, Log, "FNiagaraGPUInstanceCountManager::ResizeBuffers Alloc AllocatedInstanceCounts: %d ReservedInstanceCounts: %d", AllocatedInstanceCounts, ReservedInstanceCounts);
		}
		// If we need to increase the buffer size to RecommendedInstanceCounts because the buffer is too small.
		else if (RequiredInstanceCounts > AllocatedInstanceCounts)
		{
			SCOPED_DRAW_EVENT(RHICmdList, ResizeNiagaraGPUCounts);

			// Init a bigger buffer filled with 0.
			TResourceArray<uint32> InitData;
			InitData.AddZeroed(RecommendedInstanceCounts);
			FRWBuffer NextCountBuffer;
			NextCountBuffer.Initialize(RHICmdList, TEXT("NiagaraGPUInstanceCounts"), sizeof(uint32), RecommendedInstanceCounts, EPixelFormat::PF_R32_UINT, ERHIAccess::UAVCompute, BUF_Static | BUF_SourceCopy, &InitData);

			// Copy the current buffer in the next buffer. We don't need to transition any of the buffers, because the current buffer is transitioned to readable after
			// the simulation, and the new buffer is created in the UAVCompute state.
			FRHIUnorderedAccessView* UAVs[] = { NextCountBuffer.UAV };
			int32 UsedIndexCounts[] = { AllocatedInstanceCounts };
			CopyUIntBufferToTargets(RHICmdList, FeatureLevel, CountBuffer.SRV, UAVs, UsedIndexCounts, 0, UE_ARRAY_COUNT(UAVs));

			// FNiagaraGpuComputeDispatch expects the count buffer to be readable and copyable before running the sim.
			RHICmdList.Transition(FRHITransitionInfo(NextCountBuffer.UAV, ERHIAccess::UAVCompute, kCountBufferDefaultState));

			// Swap the buffers
			AllocatedInstanceCounts = RecommendedInstanceCounts;
			Swap(NextCountBuffer, CountBuffer);
			//UE_LOGF(LogNiagara, Log, "FNiagaraGPUInstanceCountManager::ResizeBuffers Resize AllocatedInstanceCounts: %d ReservedInstanceCounts: %d", AllocatedInstanceCounts, ReservedInstanceCounts);
		}
		// If we need to shrink the buffer size because use way to much buffer size.
		else if ((int32)(RecommendedInstanceCounts * GNiagaraGPUCountBufferSlack) < AllocatedInstanceCounts)
		{
			// possibly shrink but hard to do because of sparse array allocation.
		}
	}
	else
	{
		ReleaseCounts();
	}

	INC_DWORD_STAT_BY(STAT_NiagaraUsedGPUInstanceCounters, RequiredInstanceCounts);
}

void FNiagaraGPUInstanceCountManager::FlushIndirectArgsPool(FRHICommandListBase& RHICmdList)
{
	check(IsInRenderingThread());
	checkf(DrawIndirectArgMap.IsEmpty(), TEXT("DrawIndirectArgMap is not empty in FlushIndirectArgsPool.  This means that OnPostRenderOpaque was not called on the GPU sort manager."));

	// Cull indirect draw pool entries so that we only keep the last pool
	while (DrawIndirectPool.Num() > 1)
	{
		FIndirectArgsPoolEntryPtr& PoolEntry = DrawIndirectPool[0];
		PoolEntry->Buffer.Release();

		DrawIndirectPool.RemoveAt(0, EAllowShrinking::No);
	}

	// If shrinking is allowed and we've been under the low water mark
	if (GNiagaraIndirectArgsPoolAllowShrinking && DrawIndirectPool.Num() > 0 && DrawIndirectLowWaterFrames >= uint32(GNiagaraIndirectArgsPoolLowWaterFrames))
	{
		FIndirectArgsPoolEntryPtr& PoolEntry = DrawIndirectPool[0];
		const uint32 NewSize = FMath::Max<uint32>(GNiagaraIndirectArgsPoolMinSize, FMath::FloorToInt(float(PoolEntry->AllocatedEntries) / GNiagaraIndirectArgsPoolBlockSizeFactor));

		INDIRECT_ARG_POOL_LOG("Shrinking pool from size %d to %d", PoolEntry->AllocatedEntries, NewSize);

		PoolEntry->Buffer.Release();
		PoolEntry->AllocatedEntries = NewSize;

		TResourceArray<uint32> InitData;
		InitData.AddZeroed(PoolEntry->AllocatedEntries * NIAGARA_DRAW_INDIRECT_ARGS_SIZE);
		PoolEntry->Buffer.Initialize(RHICmdList, TEXT("NiagaraGPUDrawIndirectArgs"), sizeof(uint32), PoolEntry->AllocatedEntries * NIAGARA_DRAW_INDIRECT_ARGS_SIZE, EPixelFormat::PF_R32_UINT, kIndirectArgsDefaultState, BUF_Static | BUF_DrawIndirect, &InitData);

		// Reset the timer
		DrawIndirectLowWaterFrames = 0;
	}
}

FNiagaraGPUInstanceCountManager::FIndirectArgSlot FNiagaraGPUInstanceCountManager::AddDrawIndirect(FRHICommandListBase& RHICmdList, uint32 InstanceCountBufferOffset, uint32 NumIndicesPerInstance, uint32 StartIndexLocation, bool bIsInstancedStereoEnabled, bool bCulled, ENiagaraGpuComputeTickStage::Type ReadyTickStage)
{
	UE::TScopeLock Lock(AddDrawIndirectGuard);

	const ENiagaraDrawIndirectArgGenTaskFlags TaskFlags =
		(bIsInstancedStereoEnabled ? ENiagaraDrawIndirectArgGenTaskFlags::InstancedStereo : ENiagaraDrawIndirectArgGenTaskFlags::None)
		| (bCulled ? ENiagaraDrawIndirectArgGenTaskFlags::UseCulledCounts : ENiagaraDrawIndirectArgGenTaskFlags::None)
		| (ReadyTickStage == ENiagaraGpuComputeTickStage::PostOpaqueRender ? ENiagaraDrawIndirectArgGenTaskFlags::PostOpaque : ENiagaraDrawIndirectArgGenTaskFlags::None);
	FNiagaraDrawIndirectArgGenTaskInfo Info(InstanceCountBufferOffset, NumIndicesPerInstance, StartIndexLocation, TaskFlags);

	FNiagaraDrawIndirectArgGenSlotInfo* SlotInfo = DrawIndirectArgMap.Find(Info);
	if ( SlotInfo == nullptr )
	{
		// Attempt to allocate a new slot from the pool, or add to the pool if it's full
		FIndirectArgsPoolEntry* PoolEntry = DrawIndirectPool.Num() > 0 ? DrawIndirectPool.Last().Get() : nullptr;
		if (PoolEntry == nullptr || PoolEntry->UsedEntriesTotal >= PoolEntry->AllocatedEntries)
		{
			FIndirectArgsPoolEntryPtr NewEntry = MakeUnique<FIndirectArgsPoolEntry>();
			NewEntry->AllocatedEntries = PoolEntry ? uint32(PoolEntry->AllocatedEntries * GNiagaraIndirectArgsPoolBlockSizeFactor) : uint32(GNiagaraIndirectArgsPoolMinSize);

			INDIRECT_ARG_POOL_LOG("Increasing pool from size %d to %d", PoolEntry ? PoolEntry->AllocatedEntries : 0, NewEntry->AllocatedEntries);

			TResourceArray<uint32> InitData;
			InitData.AddZeroed(NewEntry->AllocatedEntries * NIAGARA_DRAW_INDIRECT_ARGS_SIZE);
			NewEntry->Buffer.Initialize(RHICmdList, TEXT("NiagaraGPUDrawIndirectArgs"), sizeof(uint32), NewEntry->AllocatedEntries * NIAGARA_DRAW_INDIRECT_ARGS_SIZE, EPixelFormat::PF_R32_UINT, kIndirectArgsDefaultState, BUF_Static | BUF_DrawIndirect, &InitData);

			PoolEntry = NewEntry.Get();
			DrawIndirectPool.Emplace(MoveTemp(NewEntry));
		}

		Info.IndirectArgsBufferOffset = PoolEntry->UsedEntriesTotal * NIAGARA_DRAW_INDIRECT_ARGS_SIZE;
		++PoolEntry->UsedEntriesTotal;

		SlotInfo = &DrawIndirectArgMap.Add(Info);
		SlotInfo->PoolIndex = DrawIndirectPool.Num() - 1;
		SlotInfo->BufferOffset = Info.IndirectArgsBufferOffset * sizeof(uint32);

		const ENiagaraGPUCountUpdatePhase::Type CountPhase = ReadyTickStage == ENiagaraGpuComputeTickStage::PostOpaqueRender ? ENiagaraGPUCountUpdatePhase::PostOpaque : ENiagaraGPUCountUpdatePhase::PreOpaque;
		DrawIndirectArgGenTasks[CountPhase].Add(Info);
		++PoolEntry->UsedEntries[CountPhase];
	}

	return FIndirectArgSlot(DrawIndirectPool[SlotInfo->PoolIndex]->Buffer.Buffer, DrawIndirectPool[SlotInfo->PoolIndex]->Buffer.SRV, SlotInfo->BufferOffset);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FNiagaraGPUInstanceCountManager::UpdateDrawIndirectBuffers(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, FRHICommandList& RHICmdList, ENiagaraGPUCountUpdatePhase::Type CountPhase)
{
	check(IsInRenderingThread());

	// Anything to process?
	TArray<FNiagaraDrawIndirectArgGenTaskInfo>& ArgTasks = DrawIndirectArgGenTasks[CountPhase];
	const bool bClearCounts = (CountPhase == ENiagaraGPUCountUpdatePhase::PreOpaque) && (InstanceCountClearTasks.Num() > 0) && ComputeDispatchInterface->IsFirstViewFamily();
	if ( (ArgTasks.Num() > 0 || bClearCounts) && FNiagaraUtilities::AllowComputeShaders(GShaderPlatformForFeatureLevel[FeatureLevel]) )
	{
		INC_DWORD_STAT_BY(STAT_NiagaraIndirectDraws, ArgTasks.Num());

		SCOPED_DRAW_EVENT(RHICmdList, NiagaraUpdateDrawIndirectBuffers);

		// Allocate task buffer
		FReadBuffer TaskInfosBuffer;
		{
			const uint32 ArgGenSize = ArgTasks.Num() * sizeof(FNiagaraDrawIndirectArgGenTaskInfo);
			const uint32 InstanceCountClearSize = bClearCounts ? InstanceCountClearTasks.Num() * sizeof(uint32) : 0;
			const uint32 TaskBufferSize = ArgGenSize + InstanceCountClearSize;
			TaskInfosBuffer.Initialize(RHICmdList, TEXT("NiagaraTaskInfosBuffer"), sizeof(uint32), TaskBufferSize / sizeof(uint32), EPixelFormat::PF_R32_UINT, BUF_Volatile);

			uint8* TaskBufferData = (uint8*)RHICmdList.LockBuffer(TaskInfosBuffer.Buffer, 0, TaskBufferSize, RLM_WriteOnly);
			FMemory::Memcpy(TaskBufferData, ArgTasks.GetData(), ArgGenSize);
			FMemory::Memcpy(TaskBufferData + ArgGenSize, InstanceCountClearTasks.GetData(), InstanceCountClearSize);
			RHICmdList.UnlockBuffer(TaskInfosBuffer.Buffer);
		}

		FNiagaraEmptyUAVPoolScopedAccess UAVPoolAccessScope(ComputeDispatchInterface->GetEmptyUAVPool());
		TArray<FRHITransitionInfo, TInlineAllocator<10>> Transitions;
		Transitions.Reserve(DrawIndirectPool.Num() + 2);

		FRWBuffer& CurrentCountBuffer = ComputeDispatchInterface->IsFirstViewFamily() || CountPhase == ENiagaraGPUCountUpdatePhase::PostOpaque ? CountBuffer : MultiViewCountBuffer;

		// Get counts buffer
		FUnorderedAccessViewRHIRef CountsUAV = nullptr;
		const bool bCountBufferIsValid = CurrentCountBuffer.UAV.IsValid();
		if (bCountBufferIsValid)
		{
			// treat the incoming UAV as being unknown to be sure a barrier is inserted in the case
			// where the preceding dispatch wrote to the counts buffer
			Transitions.Emplace(CurrentCountBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute);
			CountsUAV = CurrentCountBuffer.UAV;
		}
		else
		{
			// This can happen if there are no InstanceCountClearTasks and all DrawIndirectArgGenTasks_PreOpaque are using culled counts
			CountsUAV = ComputeDispatchInterface->GetEmptyUAVFromPool(RHICmdList, PF_R32_UINT, ENiagaraEmptyUAVType::Buffer);
		}

		// Get culled counts buffer
		FShaderResourceViewRHIRef CulledCountsSRV = nullptr;
		if (CulledCountBuffer.SRV.IsValid())
		{
			if (CulledCountsRHIAccess != ERHIAccess::SRVCompute)
			{
				Transitions.Emplace(CulledCountBuffer.UAV, CulledCountsRHIAccess, ERHIAccess::SRVCompute);
				CulledCountsRHIAccess = ERHIAccess::SRVCompute;
			}
			CulledCountsSRV = CulledCountBuffer.SRV.GetReference();
		}
		else
		{
			CulledCountsSRV = FNiagaraRenderer::GetDummyUIntBuffer();
		}

		// Execute transitions
		for (auto& PoolEntry : DrawIndirectPool)
		{
			Transitions.Emplace(PoolEntry->Buffer.UAV, kIndirectArgsDefaultState, ERHIAccess::UAVCompute);
		}
		RHICmdList.Transition(Transitions);

		// Execute tasks
		FNiagaraDrawIndirectArgsGenCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FNiagaraDrawIndirectArgsGenCS::FSupportsTextureRW>(GRHISupportsRWTextureBuffers ? 1 : 0);
		TShaderMapRef<FNiagaraDrawIndirectArgsGenCS> DrawIndirectArgsGenCS(GetGlobalShaderMap(FeatureLevel), PermutationVector);

		if (bCountBufferIsValid)
		{
			RHICmdList.BeginUAVOverlap(CountsUAV);
		}

		const int32 NumDispatches = FMath::Max(DrawIndirectPool.Num(), 1);
		uint32 ArgGenTaskOffset = 0;
		for (int32 DispatchIdx = 0; DispatchIdx < NumDispatches; ++DispatchIdx)
		{
			// Get draw indirect pool UAV
			// Note: If we have counts to clear but no indirect args we won't have a DrawIndirectPool entry
			FRHIUnorderedAccessView* ArgsUAV = nullptr;
			int32 NumArgGenTasks = 0;
			if (DrawIndirectPool.IsValidIndex(DispatchIdx))
			{
				FIndirectArgsPoolEntryPtr& PoolEntry = DrawIndirectPool[DispatchIdx];
				ArgsUAV = PoolEntry->Buffer.UAV;
				NumArgGenTasks = PoolEntry->UsedEntries[CountPhase];
			}
			else
			{
				ArgsUAV = ComputeDispatchInterface->GetEmptyUAVFromPool(RHICmdList, PF_R32_UINT, ENiagaraEmptyUAVType::Buffer);
			}

			const bool bIsLastDispatch = DispatchIdx == (NumDispatches - 1);
			const int32 NumInstanceCountClearTasks = bIsLastDispatch && bClearCounts ? InstanceCountClearTasks.Num() : 0;

			// Do we have anything to do for this pool?
			if (NumArgGenTasks + NumInstanceCountClearTasks == 0)
			{
				continue;
			}

			FNiagaraDrawIndirectArgsGenCS::FParameters ArgsGenParameters;
			ArgsGenParameters.TaskInfos				= TaskInfosBuffer.SRV;
			ArgsGenParameters.CulledInstanceCounts	= CulledCountsSRV;
			ArgsGenParameters.RWInstanceCounts		= CountsUAV;
			ArgsGenParameters.RWDrawIndirectArgs	= ArgsUAV;
			ArgsGenParameters.TaskCount.X			= ArgGenTaskOffset;
			ArgsGenParameters.TaskCount.Y			= NumArgGenTasks;
			ArgsGenParameters.TaskCount.Z			= NumInstanceCountClearTasks;
			ArgsGenParameters.TaskCount.W			= NumArgGenTasks + NumInstanceCountClearTasks;

			// If the device supports RW Texture buffers then we can use a single compute pass, otherwise we need to split into two passes
			if (GRHISupportsRWTextureBuffers)
			{
				FComputeShaderUtils::Dispatch(RHICmdList, DrawIndirectArgsGenCS, ArgsGenParameters, FIntVector(FMath::DivideAndRoundUp(NumArgGenTasks + NumInstanceCountClearTasks, NIAGARA_DRAW_INDIRECT_ARGS_GEN_THREAD_COUNT), 1, 1));
			}
			else
			{
				if (NumArgGenTasks > 0)
				{
					FComputeShaderUtils::Dispatch(RHICmdList, DrawIndirectArgsGenCS, ArgsGenParameters, FIntVector(FMath::DivideAndRoundUp(NumArgGenTasks, NIAGARA_DRAW_INDIRECT_ARGS_GEN_THREAD_COUNT), 1, 1));
				}

				if (NumInstanceCountClearTasks > 0)
				{
					FNiagaraDrawIndirectResetCountsCS::FParameters ClearCountParameters;
					ClearCountParameters.TaskInfos			= ArgsGenParameters.TaskInfos;
					ClearCountParameters.RWInstanceCounts	= ArgsGenParameters.RWInstanceCounts;
					ClearCountParameters.TaskCount			= ArgsGenParameters.TaskCount;
					ClearCountParameters.TaskCount.X		= 0;

					FNiagaraDrawIndirectResetCountsCS::FPermutationDomain PermutationVectorResetCounts;
					TShaderMapRef<FNiagaraDrawIndirectResetCountsCS> DrawIndirectResetCountsArgsGenCS(GetGlobalShaderMap(FeatureLevel), PermutationVectorResetCounts);
					FComputeShaderUtils::Dispatch(RHICmdList, DrawIndirectResetCountsArgsGenCS, ClearCountParameters, FIntVector(FMath::DivideAndRoundUp(NumInstanceCountClearTasks, NIAGARA_DRAW_INDIRECT_ARGS_GEN_THREAD_COUNT), 1, 1));
				}
			}

			ArgGenTaskOffset += NumArgGenTasks;
		}

		if (bCountBufferIsValid)
		{
			RHICmdList.EndUAVOverlap(CountsUAV);
		}

		// Generate and execute transitions
		Transitions.Reset();
		for (auto& PoolEntry : DrawIndirectPool)
		{
			Transitions.Emplace(PoolEntry->Buffer.UAV, ERHIAccess::UAVCompute, kIndirectArgsDefaultState);
		}
		Transitions.Emplace(CurrentCountBuffer.UAV, ERHIAccess::UAVCompute, kCountBufferDefaultState);
		RHICmdList.Transition(Transitions);
	}

	// Add free counts back to list as we have cleared them
	if ( bClearCounts )
	{
		FreeEntries.Append(InstanceCountClearTasks);
		InstanceCountClearTasks.Empty();
	}

	DrawIndirectArgGenTasks[CountPhase].Empty();

	// Final phase we can do some booking keeping / clearing of data
	if ( CountPhase == ENiagaraGPUCountUpdatePhase::PostOpaque )
	{
		// Clear indirect arg map
		DrawIndirectArgMap.Empty();

		// Release culled count buffers
		bAcquiredCulledCounts = false;
		RequiredCulledCounts = 0;

		// Optionally shrink the indirect arg pool
		if (GNiagaraIndirectArgsPoolAllowShrinking)
		{
			if (DrawIndirectPool.Num() == 1 && DrawIndirectPool[0]->AllocatedEntries > uint32(GNiagaraIndirectArgsPoolMinSize))
			{
				// See if this was a low water mark frame
				FIndirectArgsPoolEntryPtr& PoolEntry = DrawIndirectPool[0];
				const uint32 LowWaterCount = FMath::Max<uint32>(GNiagaraIndirectArgsPoolMinSize, FMath::CeilToInt(float(PoolEntry->AllocatedEntries) * GNiagaraIndirectArgsPoolLowWaterAmount));
				if (PoolEntry->UsedEntriesTotal < LowWaterCount)
				{
					++DrawIndirectLowWaterFrames;
				}
				else
				{
					// We've allocated above the low water amount, reset the timer
					DrawIndirectLowWaterFrames = 0;
				}
			}
			else
			{
				// Either the pool is empty, at the min size, or we had to increase the pool size this frame. Either way, reset the shrink timer
				DrawIndirectLowWaterFrames = 0;
			}
		}

		// Clear indirect args pool counts
		for (auto& Pool : DrawIndirectPool)
		{
			Pool->UsedEntriesTotal = 0;
			for ( uint32& UsedEntry : Pool->UsedEntries )
			{
				UsedEntry = 0;
			}
		}
	}
}

const uint32* FNiagaraGPUInstanceCountManager::GetGPUReadback()
{
	check(IsInRenderingThread());

	if (CountReadback && CountReadbackSize && CountReadback->IsReady())
	{
		SCOPE_CYCLE_COUNTER(STAT_NiagaraGPUReadbackLock);
		return (uint32*)(CountReadback->Lock(CountReadbackSize * sizeof(uint32)));
	}
	else
	{
		return nullptr;
	}
}

void FNiagaraGPUInstanceCountManager::ReleaseGPUReadback()
{
	check(IsInRenderingThread());
	check(CountReadback && CountReadbackSize);
	CountReadback->Unlock();
	// Readback can only ever be done once, to prevent misusage with index lifetime
	CountReadbackSize = 0;
}

void FNiagaraGPUInstanceCountManager::EnqueueGPUReadback(FRHICommandListImmediate& RHICmdList)
{
	check(IsInRenderingThread());
	if (UsedInstanceCounts > 0 && (UsedInstanceCounts != FreeEntries.Num()))
	{
		if (!CountReadback)
		{
			CountReadback = new FRHIGPUBufferReadback(TEXT("Niagara GPU Instance Count Readback"));
		}
		CountReadbackSize = UsedInstanceCounts;

		// No need for a transition, FNiagaraGpuComputeDispatch ensures that the buffer is left in the correct state after the simulation.
		CountReadback->EnqueueCopy(RHICmdList, CountBuffer.Buffer);
	}
}

bool FNiagaraGPUInstanceCountManager::HasPendingGPUReadback() const
{
	check(IsInRenderingThread());
	return CountReadback && CountReadbackSize;
}

void FNiagaraGPUInstanceCountManager::CopyToMultiViewCountBuffer(FRHICommandListImmediate& RHICmdList)
{
	check(IsInRenderingThread());
	if (AllocatedInstanceCounts > 0)
	{
		// Need to copy on all GPUs
		SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::All());

		// Set AllocatedInstanceCounts and copy CountBuffer
		if (MultiViewAllocatedInstanceCounts != AllocatedInstanceCounts)
		{
			MultiViewAllocatedInstanceCounts = AllocatedInstanceCounts;
			MultiViewCountBuffer.Initialize(RHICmdList, TEXT("NiagaraGPUInstanceCounts"), sizeof(uint32), MultiViewAllocatedInstanceCounts, EPixelFormat::PF_R32_UINT, ERHIAccess::UAVCompute, BUF_Static | BUF_SourceCopy);
		}
		else
		{
			RHICmdList.Transition(FRHITransitionInfo(MultiViewCountBuffer.UAV, kCountBufferDefaultState, ERHIAccess::UAVCompute));
		}

		FRHIUnorderedAccessView* UAVs[] = { MultiViewCountBuffer.UAV };
		int32 UsedIndexCounts[] = { MultiViewAllocatedInstanceCounts };
		CopyUIntBufferToTargets(RHICmdList, FeatureLevel, CountBuffer.SRV, UAVs, UsedIndexCounts, 0, UE_ARRAY_COUNT(UAVs));

		RHICmdList.Transition(FRHITransitionInfo(MultiViewCountBuffer.UAV, ERHIAccess::UAVCompute, kCountBufferDefaultState));
	}
}


void FNiagaraGPUInstanceCountManager::ProcessInitInstanceCountTasks(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, FRHICommandList& RHICmdList)
{
	if(InstanceCountInitTasks.NumBytes() == 0)
	{
		return;
	}

	SCOPED_DRAW_EVENT(RHICmdList, NiagaraProcessInitInstanceCountTasks);

	const int32 NumTasks = InstanceCountInitTasks.Num() / 2;

	// Allocate task buffer
	FReadBuffer TaskInfosBuffer;
	{
		const uint32 TaskBufferSize = InstanceCountInitTasks.Num() * sizeof(uint32);
		TaskInfosBuffer.Initialize(RHICmdList, TEXT("NiagaraInitCountsTaskInfosBuffer"), sizeof(uint32), InstanceCountInitTasks.Num(), EPixelFormat::PF_R32_UINT, BUF_Volatile);

		uint8* TaskBufferData = (uint8*)RHICmdList.LockBuffer(TaskInfosBuffer.Buffer, 0, TaskBufferSize, RLM_WriteOnly);
		FMemory::Memcpy(TaskBufferData, InstanceCountInitTasks.GetData(), TaskBufferSize);
		RHICmdList.UnlockBuffer(TaskInfosBuffer.Buffer);
	}

	FNiagaraEmptyUAVPoolScopedAccess UAVPoolAccessScope(ComputeDispatchInterface->GetEmptyUAVPool());
	TArray<FRHITransitionInfo, TInlineAllocator<10>> Transitions;
	Transitions.Reserve(1);
	FRWBuffer& CurrentCountBuffer = CountBuffer;

	// Get counts buffer
	FUnorderedAccessViewRHIRef CountsUAV = nullptr;
	const bool bCountBufferIsValid = CurrentCountBuffer.UAV.IsValid();
	if (bCountBufferIsValid)
	{
		// treat the incoming UAV as being unknown to be sure a barrier is inserted in the case
		// where the preceding dispatch wrote to the counts buffer
		Transitions.Emplace(CurrentCountBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute);
		CountsUAV = CurrentCountBuffer.UAV;
	}
	else
	{
		// This can happen if there are no InstanceCountClearTasks and all DrawIndirectArgGenTasks_PreOpaque are using culled counts
		CountsUAV = ComputeDispatchInterface->GetEmptyUAVFromPool(RHICmdList, PF_R32_UINT, ENiagaraEmptyUAVType::Buffer);
	}

	RHICmdList.Transition(Transitions);


	FNiagaraInstanceCountsInitCS::FParameters InitCountParameters;
	InitCountParameters.TaskInfos = TaskInfosBuffer.SRV;
	InitCountParameters.RWInstanceCounts = CountsUAV;
	InitCountParameters.TaskCount.X = NumTasks;
	

	FNiagaraInstanceCountsInitCS::FPermutationDomain PermutationVectorResetCounts;
	TShaderMapRef<FNiagaraInstanceCountsInitCS> InitCountsCS(GetGlobalShaderMap(FeatureLevel), PermutationVectorResetCounts);
	FComputeShaderUtils::Dispatch(RHICmdList, InitCountsCS, InitCountParameters, FIntVector(FMath::DivideAndRoundUp(NumTasks, NIAGARA_DRAW_INDIRECT_ARGS_GEN_THREAD_COUNT), 1, 1));

	// Generate and execute transitions
	Transitions.Reset();
	Transitions.Emplace(CurrentCountBuffer.UAV, ERHIAccess::UAVCompute, kCountBufferDefaultState);
	RHICmdList.Transition(Transitions);

	InstanceCountInitTasks.Reset();
}

void FNiagaraGPUInstanceCountManager::AddInstanceCountInitTask(uint32 Offset, uint32 Value)
{
 	InstanceCountInitTasks.Emplace(Offset);
 	InstanceCountInitTasks.Emplace(Value);
}