// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGPUProfiler.h"
#include "HAL/IConsoleManager.h"
#include "RHICommandList.h"
#include "NiagaraTrace.h"

bool GNiagaraGpuProfilingEnabled = true;
static FAutoConsoleVariableRef CVarNiagaraGpuProfilingEnabled(
	TEXT("fx.Niagara.GpuProfiling.Enabled"),
	GNiagaraGpuProfilingEnabled,
	TEXT("Primary control to allow Niagara to use GPU profiling or not.\n"),
	ECVF_Default
);

#if WITH_NIAGARA_GPU_PROFILER

thread_local uint32 FNiagaraGPUProfiler::ActiveQueryIndex = INDEX_NONE;

FNiagaraGPUProfiler::FNiagaraGPUProfiler(uintptr_t InOwnerContext)
	: OwnerContext(InOwnerContext)
{
	QueryPool = RHICreateRenderQueryPool(RQT_AbsoluteTime);
}

FNiagaraGPUProfiler::~FNiagaraGPUProfiler()
{
	for (FGpuFrameData& Frame : GpuFrames)
	{
		Frame.EndQuery.ReleaseQuery();

		for (FGpuDispatchTimer& DispatchTimer : Frame.DispatchTimers)
		{
			DispatchTimer.StartQuery.ReleaseQuery();
			DispatchTimer.EndQuery.ReleaseQuery();
		}
	}
}

void FNiagaraGPUProfiler::BeginFrame(FRHICommandList& RHICmdList)
{
	// Process all frames until we run out
	while (FGpuFrameData* ReadFrame = GetReadFrame())
	{
		// Attempt to process, might not be read
		if ( !ProcessFrame(RHICmdList, *ReadFrame) )
		{
			break;
		}

		// Frame was processed
		CurrentReadFrame = (CurrentReadFrame + 1) % NumBufferFrames;
	}

	// If we are not enabled nothing to profile
	static const auto CSVStatsEnabledCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.DetailedCSVStats"));
	const bool bCsvCollecting = CSVStatsEnabledCVar && CSVStatsEnabledCVar->GetBool();
	const bool bInsightsTraceCollecting = FNiagaraInsights::IsChannelActive();
	if ( !GNiagaraGpuProfilingEnabled || ((NumReaders == 0) && !bCsvCollecting && !bInsightsTraceCollecting) )
	{
		return;
	}

	// Get frame to write into
	ActiveWriteFrame = GetWriteFrame();
}

void FNiagaraGPUProfiler::EndFrame(FRHICommandList& RHICmdList)
{
	if (ActiveWriteFrame == nullptr)
	{
		return;
	}

	// Inject end marker so we know if all dispatches are complete
	ActiveWriteFrame->EndQuery = QueryPool->AllocateQuery();
	RHICmdList.EndRenderQuery(ActiveWriteFrame->EndQuery.GetQuery());

	ActiveWriteFrame = nullptr;
	CurrentWriteFrame = (CurrentWriteFrame + 1) % NumBufferFrames;
}

void FNiagaraGPUProfiler::BeginDispatch(FRHICommandList& RHICmdList, const FNiagaraGpuProfileEvent& Event)
{
	if (ActiveWriteFrame == nullptr)
	{
		return;
	}


	UE::TScopeLock Lock(Mutex);
	check(ActiveQueryIndex == INDEX_NONE);
	ActiveQueryIndex = ActiveWriteFrame->DispatchTimers.Num();
	FGpuDispatchTimer& DispatchTimer = ActiveWriteFrame->DispatchTimers.Emplace_GetRef(Event);

	DispatchTimer.StartQuery = QueryPool->AllocateQuery();
	RHICmdList.EndRenderQuery(DispatchTimer.StartQuery.GetQuery());
}

void FNiagaraGPUProfiler::EndDispatch(FRHICommandList& RHICmdList)
{
	if (ActiveWriteFrame == nullptr)
	{
		return;
	}
	
	UE::TScopeLock Lock(Mutex);
	check(ActiveWriteFrame->DispatchTimers.IsValidIndex(ActiveQueryIndex));

	FGpuDispatchTimer& DispatchTimer = ActiveWriteFrame->DispatchTimers[ActiveQueryIndex];
	DispatchTimer.EndQuery = QueryPool->AllocateQuery();
	RHICmdList.EndRenderQuery(DispatchTimer.EndQuery.GetQuery());

	ActiveQueryIndex = INDEX_NONE;
}

bool FNiagaraGPUProfiler::ProcessFrame(FRHICommandList& /*RHICmdList*/, FGpuFrameData& ReadFrame)
{
	// Frame ready to process?
	//-OPT: We can just look at the last write stage end timer here, but that relies on the batcher always executing
	uint64 DummyEndTime;
	if (!RHIGetRenderQueryResult(ReadFrame.EndQuery.GetQuery(), DummyEndTime, false))
	{
		return false;
	}
	ReadFrame.EndQuery.ReleaseQuery();

	//-OPT: Potentially pool these
	FNiagaraGpuFrameResultsPtr FrameResults = MakeShared<FNiagaraGpuFrameResults, ESPMode::ThreadSafe>();
	FrameResults->OwnerContext = OwnerContext;
	FrameResults->TotalDispatches = 0;
	FrameResults->TotalDurationMicroseconds = 0;
	FrameResults->DispatchResults.Reserve(ReadFrame.DispatchTimers.Num());

	// Process results
	for (FGpuDispatchTimer& DispatchTimer : ReadFrame.DispatchTimers)
	{
		auto& DispatchResults = FrameResults->DispatchResults.AddDefaulted_GetRef();
			
		uint64 StartMicroseconds = 0;
		uint64 EndMicroseconds = 0;
		ensure(RHIGetRenderQueryResult(DispatchTimer.StartQuery.GetQuery(), StartMicroseconds, false));
		ensure(RHIGetRenderQueryResult(DispatchTimer.EndQuery.GetQuery(), EndMicroseconds, false));
		DispatchTimer.StartQuery.ReleaseQuery();
		DispatchTimer.EndQuery.ReleaseQuery();

		DispatchResults.bUniqueInstance = DispatchTimer.Event.bUniqueInstance;
		DispatchResults.OwnerComponent = DispatchTimer.Event.OwnerComponent;
		DispatchResults.OwnerEmitter = DispatchTimer.Event.OwnerEmitter;
		DispatchResults.StageName = DispatchTimer.Event.StageName;
		DispatchResults.DurationMicroseconds = EndMicroseconds - StartMicroseconds;

		++FrameResults->TotalDispatches;
		FrameResults->TotalDurationMicroseconds += DispatchResults.DurationMicroseconds;
	}

	ReadFrame.DispatchTimers.Empty();

	// Post Results
	PostResults(FrameResults);

	return true;
}

#endif
