// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "RHIResources.h"
#include "HAL/Platform.h"
#include "Misc/Build.h"
#include "Particles/ParticlePerfStats.h"

#include "NiagaraGPUProfilerInterface.h"

#if WITH_NIAGARA_GPU_PROFILER

/** Helper class to time gpu runtime cost of dispatches */
class FNiagaraGPUProfiler : public FNiagaraGPUProfilerInterface
{
	static constexpr int32 NumBufferFrames = 5;

	struct FGpuDispatchTimer
	{
		explicit FGpuDispatchTimer(const FNiagaraGpuProfileEvent& InEvent) : Event(InEvent) {}

		FNiagaraGpuProfileEvent	Event;
		FRHIPooledRenderQuery	StartQuery;
		FRHIPooledRenderQuery	EndQuery;
	};

	struct FGpuFrameData
	{
		bool CanWrite() const { return EndQuery.GetQuery() == nullptr; }
		bool CanRead() const { return EndQuery.GetQuery() != nullptr; }

		FRHIPooledRenderQuery		EndQuery;
		TArray<FGpuDispatchTimer>	DispatchTimers;
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFrameResults, const FNiagaraGpuFrameResultsPtr&);

public:
	FNiagaraGPUProfiler(uintptr_t InOwnerContext);
	~FNiagaraGPUProfiler();

	void BeginFrame(FRHICommandList& RHICmdList);
	void EndFrame(FRHICommandList& RHICmdList);

	void BeginDispatch(FRHICommandList& RHICmdList, const FNiagaraGpuProfileEvent& Event);
	void EndDispatch(FRHICommandList& RHICmdList);

private:
	FGpuFrameData* GetReadFrame() { check(CurrentReadFrame >= 0 && CurrentReadFrame < UE_ARRAY_COUNT(GpuFrames)); return GpuFrames[CurrentReadFrame].CanRead() ? &GpuFrames[CurrentReadFrame] : nullptr; }
	FGpuFrameData* GetWriteFrame() { check(CurrentWriteFrame >= 0 && CurrentWriteFrame < UE_ARRAY_COUNT(GpuFrames)); return GpuFrames[CurrentWriteFrame].CanWrite() ? &GpuFrames[CurrentWriteFrame] : nullptr; }
	bool ProcessFrame(FRHICommandList& RHICmdList, FGpuFrameData& ReadFrame);

private:
	uintptr_t					OwnerContext = 0;

	int32						CurrentReadFrame = 0;					// Index of the next frame to read from
	int32						CurrentWriteFrame = 0;					// Index of the next frame to write into
	FGpuFrameData				GpuFrames[NumBufferFrames];

	FGpuFrameData*				ActiveWriteFrame = nullptr;				// Not null while we are generating a frame of data, otherwise null
	static thread_local uint32	ActiveQueryIndex;					// We don't support timing dispatches inside one another
	UE::FMutex					Mutex;

	FRenderQueryPoolRHIRef		QueryPool;
};

#endif //WITH_NIAGARA_GPU_PROFILER
