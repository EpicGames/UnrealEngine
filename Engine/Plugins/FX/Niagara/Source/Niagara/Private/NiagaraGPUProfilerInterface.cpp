// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGPUProfilerInterface.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGPUProfiler.h"
#include "NiagaraGPUSystemTick.h"
#include "NiagaraDataInterfaceBase.h"
#include "NiagaraComponent.h"
#include "NiagaraEmitter.h"

#include "Async/Async.h"
#include "NiagaraSimStageData.h"

#if WITH_NIAGARA_GPU_PROFILER

//////////////////////////////////////////////////////////////////////////

FNiagaraGpuProfilerListener::FNiagaraGpuProfilerListener()
{
}

FNiagaraGpuProfilerListener::~FNiagaraGpuProfilerListener()
{
	check(IsInGameThread());
	SetEnabled(false);
	SetHandler(nullptr);
}

void FNiagaraGpuProfilerListener::SetEnabled(bool bInEnabled)
{
	check(IsInGameThread());
	if ( bEnabled != bInEnabled)
	{
		bEnabled = bInEnabled;
		FNiagaraGPUProfilerInterface::NumReaders.fetch_add(bEnabled ? 1 : -1);
	}
}

void FNiagaraGpuProfilerListener::SetHandler(TFunction<void(const FNiagaraGpuFrameResultsPtr&)> Function)
{
	check(IsInGameThread());
	if (GameThreadHandler.IsValid())
	{
		FNiagaraGPUProfilerInterface::GetOnFrameResults_GameThread().Remove(GameThreadHandler);
	}
	if ( Function )
	{
		GameThreadHandler = FNiagaraGPUProfilerInterface::GetOnFrameResults_GameThread().AddLambda(Function);
	}
}

//////////////////////////////////////////////////////////////////////////

FNiagaraGpuProfileEvent::FNiagaraGpuProfileEvent(const FNiagaraComputeInstanceData& InstanceData, const FNiagaraSimStageData& SimStageData, const bool bFirstInstanceData)
{
	bUniqueInstance = SimStageData.bSetDataToRender && bFirstInstanceData;
	OwnerComponent = InstanceData.Context->ProfilingComponentPtr;
	OwnerEmitter = InstanceData.Context->ProfilingEmitterPtr;
	StageName = SimStageData.StageMetaData->SimulationStageName;
}

FNiagaraGpuProfileEvent::FNiagaraGpuProfileEvent(const FNiagaraComputeInstanceData& InstanceData, FName CustomStageName)
{
	bUniqueInstance = false;
	OwnerComponent = InstanceData.Context->ProfilingComponentPtr;
	OwnerEmitter = InstanceData.Context->ProfilingEmitterPtr;
	StageName = CustomStageName;
}

//////////////////////////////////////////////////////////////////////////

FNiagaraGpuProfileScope::FNiagaraGpuProfileScope(FRHICommandList& InRHICmdList, const class FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, const FNiagaraGpuProfileEvent& Event)
	: RHICmdList(InRHICmdList)
	, GPUProfiler(static_cast<FNiagaraGPUProfiler*>(ComputeDispatchInterface->GetGPUProfiler()))
{
	if (GPUProfiler)
	{
		GPUProfiler->BeginDispatch(RHICmdList, Event);
	}
}

FNiagaraGpuProfileScope::~FNiagaraGpuProfileScope()
{
	if (GPUProfiler)
	{
		GPUProfiler->EndDispatch(RHICmdList);
	}
}

//////////////////////////////////////////////////////////////////////////

void SetResultsForParticlePerfStats(const FNiagaraGpuFrameResultsPtr& FrameResults)
{
#if WITH_PARTICLE_PERF_STATS
	auto PostStats =
		[](FParticlePerfStats* Stats, bool bUniqueInstance, uint64 DurationMicroseconds)
		{
			if ( Stats )
			{
				Stats->GetGPUStats().NumInstances += bUniqueInstance ? 1: 0;
				Stats->GetGPUStats().TotalMicroseconds += DurationMicroseconds;
			}
		};

	for (const auto& DispatchResult : FrameResults->DispatchResults)
	{
		UNiagaraComponent* OwnerComponent = Cast<UNiagaraComponent>(DispatchResult.OwnerComponent.Get());
		if ( OwnerComponent == nullptr )
		{
			return;
		}

		FParticlePerfStatsContext StatsContext = OwnerComponent->GetPerfStatsContext();
		PostStats(StatsContext.GetWorldStats(), DispatchResult.bUniqueInstance, DispatchResult.DurationMicroseconds);
		PostStats(StatsContext.GetSystemStats(), DispatchResult.bUniqueInstance, DispatchResult.DurationMicroseconds);
		PostStats(StatsContext.GetComponentStats(), DispatchResult.bUniqueInstance, DispatchResult.DurationMicroseconds);
	}
#endif
}

//////////////////////////////////////////////////////////////////////////
//-TODO: Move out of this file
void SetResultsForEditorStats(const FNiagaraGpuFrameResultsPtr& FrameResults)
{
#if WITH_NIAGARA_GPU_PROFILER_EDITOR
	// Send data for editor stat display
	//-TODO: editor stats should merge with particle perf stats
	TMap<FVersionedNiagaraEmitterData*, TMap<TStatIdData const*, float>> CapturedStats;
	CapturedStats.Reserve(FrameResults->DispatchResults.Num());

	for (const auto& DispatchResult : FrameResults->DispatchResults)
	{
		FVersionedNiagaraEmitterData* EmitterData = DispatchResult.OwnerEmitter.GetEmitterData();
		if ( EmitterData == nullptr )
		{
			continue;
		}

		// Build stat name
		TStringBuilder<128> StatNameBuilder;
		StatNameBuilder.Append(TEXT("GPU_Stage_"));
		DispatchResult.StageName.AppendString(StatNameBuilder);
		const FName StatFName(StatNameBuilder.ToString());

		const TStatId StatId = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_NiagaraDetailed>(StatFName);
		CapturedStats.FindOrAdd(EmitterData).FindOrAdd(StatId.GetRawPointer()) += DispatchResult.DurationMicroseconds;
	}

	for (auto& CapturedStat : CapturedStats)
	{
		CapturedStat.Key->GetStatData().AddStatCapture(TTuple<uint64, ENiagaraScriptUsage>(uint64(FrameResults->OwnerContext), ENiagaraScriptUsage::ParticleGPUComputeScript), CapturedStat.Value);
	}
#endif
}

//////////////////////////////////////////////////////////////////////////

std::atomic<int> FNiagaraGPUProfilerInterface::NumReaders(0);
FNiagaraGPUProfilerInterface::FOnFrameResults FNiagaraGPUProfilerInterface::OnFrameResults_GameThread;
FNiagaraGPUProfilerInterface::FOnFrameResults FNiagaraGPUProfilerInterface::OnFrameResults_RenderThread;

void FNiagaraGPUProfilerInterface::PostResults(const FNiagaraGpuFrameResultsPtr& FrameResults)
{
	// Set data for particle perf stats
	SetResultsForParticlePerfStats(FrameResults);
	OnFrameResults_RenderThread.Broadcast(FrameResults);

	// Send data to GameThread
	AsyncTask(
		ENamedThreads::GameThread,
		[FrameResults_GT=FrameResults]
		{
			SetResultsForEditorStats(FrameResults_GT);
			OnFrameResults_GameThread.Broadcast(FrameResults_GT);
		}
	);
}

#endif //WITH_NIAGARA_GPU_PROFILER
