// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGPUSystemTick.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystem.h"

namespace NiagaraLoggingCVars
{
	static bool bEmitGPUTickGenErrors = true;
	static FAutoConsoleVariableRef CVarEmitGPUTickGenErrors(
		TEXT("fx.Niagara.Logging.EmitGPUGenTickErrors"),
		bEmitGPUTickGenErrors,
		TEXT("If true, we will emit errors to the log when Niagara GPU Ticks have critical errors."),
		ECVF_Default
	);
}

#if NIAGARA_VALIDATE_NDIPROXY_REFS
FNiagaraComputeDataInterfaceInstanceData::~FNiagaraComputeDataInterfaceInstanceData()
{
	for (auto ProxyIt=InterfaceProxiesToOffsets.CreateIterator(); ProxyIt; ++ProxyIt)
	{
		--ProxyIt->Key->ProxyTickRefs;
	}
}
#endif

void FNiagaraGPUSystemTick::Init(FNiagaraSystemInstance* InSystemInstance)
{
	ensure(InSystemInstance != nullptr);
	CA_ASSUME(InSystemInstance != nullptr);
	ensure(!InSystemInstance->IsComplete());
	SystemInstanceID = InSystemInstance->GetId();
	SystemGpuComputeProxy = InSystemInstance->GetSystemGpuComputeProxy();
 
	bHasErrors = false;

	uint32 DataSizeForGPU = InSystemInstance->GPUDataInterfaceInstanceDataSize;

	if (DataSizeForGPU > 0)
	{
		uint32 AllocationSize = DataSizeForGPU;

		DIInstanceData = new FNiagaraComputeDataInterfaceInstanceData;
		DIInstanceData->PerInstanceDataSize = AllocationSize;
		DIInstanceData->PerInstanceDataForRT = FMemory::Malloc(AllocationSize);
		DIInstanceData->Instances = InSystemInstance->DataInterfaceInstanceDataOffsets.Num();

		uint8* InstanceDataBase = (uint8*) DIInstanceData->PerInstanceDataForRT;
		uint32 RunningOffset = 0;

		DIInstanceData->InterfaceProxiesToOffsets.Reserve(InSystemInstance->GPUDataInterfaces.Num());

		for (const auto& Pair : InSystemInstance->GPUDataInterfaces)
		{
			UNiagaraDataInterface* Interface = Pair.Key.Get();
			if (Interface == nullptr)
			{
				continue;
			}

			FNiagaraDataInterfaceProxy* Proxy = Interface->GetProxy();
			const int32 Offset = Pair.Value;

			const int32 RTDataSize = Align(Interface->PerInstanceDataPassedToRenderThreadSize(), 16);
			ensure(RTDataSize > 0);
			check(Proxy);

			void* PerInstanceData = &InSystemInstance->DataInterfaceInstanceData[Offset];

			Interface->ProvidePerInstanceDataForRenderThread(InstanceDataBase, PerInstanceData, SystemInstanceID);

			// @todo rethink this. So ugly.
		#if NIAGARA_VALIDATE_NDIPROXY_REFS
			++Proxy->ProxyTickRefs;
		#endif
			DIInstanceData->InterfaceProxiesToOffsets.Add(Proxy, RunningOffset);

			InstanceDataBase += RTDataSize;
			RunningOffset += RTDataSize;
		}
	}

	check(MAX_uint32 > InSystemInstance->ActiveGPUEmitterCount);

	// Layout our packet.
	const uint32 PackedDispatchesSize = InSystemInstance->ActiveGPUEmitterCount * sizeof(FNiagaraComputeInstanceData);
	// We want the Params after the instance data to be aligned so we can upload to the gpu.
	uint32 PackedDispatchesSizeAligned = Align(PackedDispatchesSize, SHADER_PARAMETER_STRUCT_ALIGNMENT);
	uint32 TotalParamSize = InSystemInstance->TotalGPUParamSize;

	uint32 TotalPackedBufferSize = PackedDispatchesSizeAligned + TotalParamSize;

	InstanceData_ParamData_Packed = (uint8*)FMemory::Malloc(TotalPackedBufferSize);

	FNiagaraComputeInstanceData* Instances = (FNiagaraComputeInstanceData*)(InstanceData_ParamData_Packed);
	uint8* ParamDataBufferPtr = InstanceData_ParamData_Packed + PackedDispatchesSizeAligned;

	// we want to include interpolation parameters (current and previous frame) if any of the emitters in the system
	// require it
	const bool bIncludeInterpolationParameters = InSystemInstance->GPUParamIncludeInterpolation;
	const int32 InterpFactor = bIncludeInterpolationParameters ? 2 : 1;

	GlobalParamData = ParamDataBufferPtr;
	SystemParamData = GlobalParamData + InterpFactor * sizeof(FNiagaraGlobalParameters);
	OwnerParamData = SystemParamData + InterpFactor * sizeof(FNiagaraSystemParameters);

	// actually copy all of the data over, for the system data we only need to do it once (rather than per-emitter)
	FMemory::Memcpy(GlobalParamData, &InSystemInstance->GetGlobalParameters(), sizeof(FNiagaraGlobalParameters));
	FMemory::Memcpy(SystemParamData, &InSystemInstance->GetSystemParameters(), sizeof(FNiagaraSystemParameters));
	FMemory::Memcpy(OwnerParamData, &InSystemInstance->GetOwnerParameters(), sizeof(FNiagaraOwnerParameters));

	if (bIncludeInterpolationParameters)
	{
		FMemory::Memcpy(GlobalParamData + sizeof(FNiagaraGlobalParameters), &InSystemInstance->GetGlobalParameters(true), sizeof(FNiagaraGlobalParameters));
		FMemory::Memcpy(SystemParamData + sizeof(FNiagaraSystemParameters), &InSystemInstance->GetSystemParameters(true), sizeof(FNiagaraSystemParameters));
		FMemory::Memcpy(OwnerParamData + sizeof(FNiagaraOwnerParameters), &InSystemInstance->GetOwnerParameters(true), sizeof(FNiagaraOwnerParameters));
	}

	ParamDataBufferPtr = OwnerParamData + InterpFactor * sizeof(FNiagaraOwnerParameters);

	// Now we will generate instance data for every GPU simulation we want to run on the render thread.
	// This is spawn rate as well as DataInterface per instance data and the ParameterData for the emitter.
	// @todo Ideally we would only update DataInterface and ParameterData bits if they have changed.
	uint32 InstanceIndex = 0;
	bool bStartNewOverlapGroup = false;

	const TConstArrayView<FNiagaraEmitterExecutionIndex> EmitterExecutionOrder = InSystemInstance->GetEmitterExecutionOrder();
	for (const FNiagaraEmitterExecutionIndex& EmiterExecIndex : EmitterExecutionOrder)
	{
		// The dependency resolution code does not consider CPU and GPU emitters separately, so the flag which marks the start of a new overlap group can be set on either
		// a CPU or GPU emitter. We must turn on bStartNewOverlapGroup when we encounter the flag, and reset it when we've actually marked a GPU emitter as starting a new group.
		bStartNewOverlapGroup |= EmiterExecIndex.bStartNewOverlapGroup;

		const uint32 EmitterIdx = EmiterExecIndex.EmitterIndex;
		if (FNiagaraEmitterInstance* EmitterInstance = &InSystemInstance->GetEmitters()[EmitterIdx].Get())
		{
			if ( EmitterInstance->IsComplete() )
			{
				continue;
			}

			FNiagaraEmitterInstanceImpl* EmitterInstanceImpl = EmitterInstance->AsStateful();
			if (!EmitterInstanceImpl)
			{
				continue;
			}

			FNiagaraComputeExecutionContext* GPUContext = EmitterInstance->GetGPUContext();
			if (!EmitterInstance->GetEmitter() || !GPUContext || EmitterInstance->GetSimTarget() != ENiagaraSimTarget::GPUComputeSim)
			{
				continue;
			}

			// Handle edge case where an emitter was set to inactive on the first frame by scalability
			// In which case it will never have ticked so we should not execute a GPU tick for this until it becomes active
			// See FNiagaraSystemInstance::Tick_Concurrent for details
			if (EmitterInstanceImpl->HasTicked() == false)
			{
				ensure((EmitterInstance->GetExecutionState() == ENiagaraExecutionState::Inactive) || (EmitterInstance->GetExecutionState() == ENiagaraExecutionState::InactiveClear));
				continue;
			}

			FNiagaraComputeInstanceData* InstanceData = new (&Instances[InstanceIndex]) FNiagaraComputeInstanceData;
			InstanceIndex++;

			InstanceData->Context = GPUContext;
			check(GPUContext->MainDataSet);

			// Validate that we generated interpolation parameters if we need them for the context
			if (GPUContext->HasInterpolationParameters && !bIncludeInterpolationParameters)
			{
				UE_LOGF(LogNiagara, Warning, "Mismatch between required interpolation parameters in GPUContext(%d) vs calculated(%d) Sim(%ls)", GPUContext->HasInterpolationParameters, bIncludeInterpolationParameters, GPUContext->GetDebugSimName());
			}

			InstanceData->SpawnInfo = GPUContext->GpuSpawnInfo_GT;

			// Consume pending reset
			if ( GPUContext->bResetPending_GT )
			{
				InstanceData->bResetData = GPUContext->bResetPending_GT;
				GPUContext->bResetPending_GT = false;

				GPUContext->ParticleCountReadFence.fetch_add(1, std::memory_order_relaxed);
			}
			InstanceData->ParticleCountFence = GPUContext->ParticleCountReadFence.load(std::memory_order_relaxed);

			InstanceData->EmitterParamData = ParamDataBufferPtr;
			ParamDataBufferPtr += InterpFactor * sizeof(FNiagaraEmitterParameters);

			// actually copy all of the data over
			FMemory::Memcpy(InstanceData->EmitterParamData, &InSystemInstance->GetEmitterParameters(EmitterIdx), sizeof(FNiagaraEmitterParameters));
			if (bIncludeInterpolationParameters)
			{
				FMemory::Memcpy(InstanceData->EmitterParamData + sizeof(FNiagaraEmitterParameters), &InSystemInstance->GetEmitterParameters(EmitterIdx, true), sizeof(FNiagaraEmitterParameters));
			}

			ParamDataBufferPtr = GPUContext->WriteConstantBufferInstanceData(ParamDataBufferPtr, *InstanceData);

			bHasInterpolatedParameters |= GPUContext->HasInterpolationParameters;

			// Calling PostTick will push current -> previous parameters this must be done after copying the parameter data
			GPUContext->PostTick();

			InstanceData->bStartNewOverlapGroup = bStartNewOverlapGroup;
			bStartNewOverlapGroup = false;

			// @todo-threadsafety Think of a better way to do this!
			const TArray<UNiagaraDataInterface*>& DataInterfaces = GPUContext->CombinedParamStore.GetDataInterfaces();
			InstanceData->DataInterfaceProxies.Reserve(DataInterfaces.Num());
			InstanceData->IterationDataInterfaceProxies.Reserve(DataInterfaces.Num());

			for (UNiagaraDataInterface* DI : DataInterfaces)
			{
				FNiagaraDataInterfaceProxy* DIProxy = DI->GetProxy();

				if(DIProxy)
				{
					InstanceData->DataInterfaceProxies.Add(DIProxy);

					if ( FNiagaraDataInterfaceProxyRW* RWProxy = DIProxy->AsIterationProxy() )
					{
						InstanceData->IterationDataInterfaceProxies.Add(RWProxy);
					}
				}	
				else
				{
					bHasErrors = true;

					if(NiagaraLoggingCVars::bEmitGPUTickGenErrors)
					{
						UE_LOGF(LogNiagara, Error, "Error Generating GPU Tick. Null DI Proxy!\nUse cmd \"fx.Niagara.Logging.EmitGPUGenTickErrors false\" to suppress these errors.\nNiagara System: %ls\nData Interface:%ls", *InSystemInstance->GetSystem()->GetPathName(), DI ? *DI->GetPathName() : TEXT("null DI!"));
					}
				}
			}

			// Gather number of iterations for each stage, and if the stage should run or not
			InstanceData->bHasMultipleStages = false;
			InstanceData->PerStageInfo.Reserve(GPUContext->SimStageExecData->SimStageMetaData.Num());		// Note: This presize isn't accurate but in the general none looping case it's correct

			const FNiagaraParameterStore& ParameterStore = EmitterInstance->GetRendererBoundVariables();
			for (const FNiagaraSimStageExecutionLoopData& LoopData : GPUContext->SimStageExecData->ExecutionLoops)
			{
				int32 NumLoops = LoopData.NumLoopsBinding.IsNone() ? LoopData.NumLoops : ParameterStore.GetParameterValueOrDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), LoopData.NumLoopsBinding), LoopData.NumLoops);
				NumLoops = static_cast<int32>(FMath::Clamp(NumLoops, 0, TNumericLimits<uint16>::Max()));
				for (int32 LoopIndex=0; LoopIndex < NumLoops; ++LoopIndex)
				{
					for (int32 SimStageIndex=LoopData.StartStageIndex; SimStageIndex <= LoopData.EndStageIndex; ++SimStageIndex)
					{
						const FSimulationStageMetaData& SimStageMetaData = GPUContext->SimStageExecData->SimStageMetaData[SimStageIndex];

						// Should we run the stage?
						if (!SimStageMetaData.ShouldRunStage(InstanceData->bResetData))
						{
							continue;
						}

						// Is the stage enabled?
						if (!SimStageMetaData.EnabledBinding.IsNone())
						{
							if (const uint8* ParameterData = ParameterStore.GetParameterData(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), SimStageMetaData.EnabledBinding)))
							{
								const FNiagaraBool StageEnabled = *reinterpret_cast<const FNiagaraBool*>(ParameterData);
								if (StageEnabled.GetValue() == false)
								{
									continue;
								}
							}
						}

						// Get number of iterations
						int32 NumIterations = SimStageMetaData.NumIterationsBinding.IsNone() ? SimStageMetaData.NumIterations : ParameterStore.GetParameterValueOrDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.NumIterationsBinding), SimStageMetaData.NumIterations);
						NumIterations = static_cast<int32>(FMath::Clamp(NumIterations, 0, TNumericLimits<uint16>::Max()));
						if ( NumIterations <= 0 )
						{
							continue;
						}

						// Get element count
						FIntVector ElementCountXYZ = FIntVector(1, 1, 1);
						if (SimStageMetaData.IterationSourceType == ENiagaraIterationSource::DirectSet)
						{
							FNiagaraParameterStore& BoundParamStore = EmitterInstance->GetRendererBoundVariables();
							ElementCountXYZ = FIntVector(1, 1, 1);

							// This will cause a PVS warning, if we add something before this enum entry add the code back in
							//if (SimStageMetaData.GpuDispatchType >= ENiagaraGpuDispatchType::OneD)
							{
								ElementCountXYZ.X = SimStageMetaData.ElementCountXBinding.IsNone() ? SimStageMetaData.ElementCount.X : BoundParamStore.GetParameterValueOrDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.ElementCountXBinding), 0);
							}
							if (SimStageMetaData.GpuDispatchType >= ENiagaraGpuDispatchType::TwoD)
							{
								ElementCountXYZ.Y = SimStageMetaData.ElementCountYBinding.IsNone() ? SimStageMetaData.ElementCount.Y : BoundParamStore.GetParameterValueOrDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.ElementCountYBinding), 0);
							}
							if (SimStageMetaData.GpuDispatchType >= ENiagaraGpuDispatchType::ThreeD)
							{
								ElementCountXYZ.Z = SimStageMetaData.ElementCountZBinding.IsNone() ? SimStageMetaData.ElementCount.Z : BoundParamStore.GetParameterValueOrDefault(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.ElementCountZBinding), 0);
							}

							// If any of the element count values are <= 0 we can kill the stage as it won't execute
							if (ElementCountXYZ.GetMin() <= 0)
							{
								continue;
							}
						}

						// Stage is live we can add it
						FNiagaraComputeInstanceData::FPerStageInfo& NewStageInfo = InstanceData->PerStageInfo.AddDefaulted_GetRef();
						NewStageInfo.SimStageIndex		= IntCastChecked<int16>(SimStageIndex);
						NewStageInfo.NumIterations		= IntCastChecked<int16>(NumIterations);
						NewStageInfo.LoopIndex			= IntCastChecked<int16>(LoopIndex);
						NewStageInfo.NumLoops			= IntCastChecked<int16>(NumLoops);
						NewStageInfo.ElementCountXYZ	= ElementCountXYZ;

						InstanceData->bHasMultipleStages = true;
						InstanceData->TotalDispatches += NumIterations;
					}
				}
			}

			TotalDispatches += InstanceData->TotalDispatches;
		}
	}

	check(InSystemInstance->ActiveGPUEmitterCount == InstanceIndex);
	InstanceCount = InstanceIndex;
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	InstanceDataDebuggingOnly = GetInstances();
#endif
}

void FNiagaraGPUSystemTick::Destroy()
{
	for ( FNiagaraComputeInstanceData& Instance : GetInstances() )
	{
		Instance.Context->ParticleCountWriteFence.store(Instance.ParticleCountFence, std::memory_order_relaxed);
		Instance.~FNiagaraComputeInstanceData();
	}

	FMemory::Free(InstanceData_ParamData_Packed);
	if (DIInstanceData)
	{
		FMemory::Free(DIInstanceData->PerInstanceDataForRT);
		delete DIInstanceData;
	}
}

void FNiagaraGPUSystemTick::BuildUniformBuffers()
{
	check(ExternalUnformBuffers_RT.Num() == 0);

	const int32 InterpCount = bHasInterpolatedParameters ? 2 : 1;
	ExternalUnformBuffers_RT.AddDefaulted(InstanceCount * InterpCount);

	TArrayView<FNiagaraComputeInstanceData> Instances = GetInstances();
	for (uint32 iInstance=0; iInstance < InstanceCount; ++iInstance)
	{
		const FNiagaraComputeInstanceData& Instance = Instances[iInstance];

		FNiagaraRHIUniformBufferLayout* ExternalCBufferLayout = Instance.Context->ExternalCBufferLayout;
		const bool bExternalLayoutValid = ExternalCBufferLayout && (ExternalCBufferLayout->Resources.Num() || ExternalCBufferLayout->ConstantBufferSize > 0);
		if ( Instance.Context->GPUScript_RT->IsExternalConstantBufferUsed_RenderThread(0) )
		{
			if ( ensure(ExternalCBufferLayout && bExternalLayoutValid) )
			{
				ExternalUnformBuffers_RT[iInstance] = RHICreateUniformBuffer(Instance.ExternalParamData, ExternalCBufferLayout, Instance.bHasMultipleStages ? EUniformBufferUsage::UniformBuffer_SingleFrame : EUniformBufferUsage::UniformBuffer_SingleDraw);
			}
		}
		if ( Instance.Context->GPUScript_RT->IsExternalConstantBufferUsed_RenderThread(1) )
		{
			if (ensure(ExternalCBufferLayout && bExternalLayoutValid))
			{
				check(ExternalCBufferLayout->ConstantBufferSize + ExternalCBufferLayout->ConstantBufferSize <= Instance.ExternalParamDataSize);
				ExternalUnformBuffers_RT[InstanceCount + iInstance] = RHICreateUniformBuffer(Instance.ExternalParamData + ExternalCBufferLayout->ConstantBufferSize, ExternalCBufferLayout, Instance.bHasMultipleStages ? EUniformBufferUsage::UniformBuffer_SingleFrame : EUniformBufferUsage::UniformBuffer_SingleDraw);
			}
		}
	}
}
