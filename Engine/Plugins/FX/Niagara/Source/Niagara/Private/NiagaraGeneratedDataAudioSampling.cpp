// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGeneratedDataAudioSampling.h"

#include "Algo/RemoveIf.h"
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "DSP/AudioFFT.h"
#include "DSP/ConstantQ.h"
#include "DSP/DeinterleaveView.h"
#include "DSP/FFTAlgorithm.h"
#include "DSP/MultithreadedPatching.h"
#include "DSP/SlidingWindow.h"
#include "ISubmixBufferListener.h"
#include "NiagaraWorldManager.h"
#include "Sound/SoundSubmix.h"

namespace UE::Niagara::GeneratedDataAudioSampling::Private
{

static Audio::FPseudoConstantQKernelSettings GetConstantQSettings(float InMinimumFrequency, float InMaximumFrequency, int32 InNumBands, float InNumBandsPerOctave, float InBandwidthStretch)
{
	Audio::FPseudoConstantQKernelSettings CQTKernelSettings;

	CQTKernelSettings.NumBands = InNumBands;
	CQTKernelSettings.KernelLowestCenterFreq = InMinimumFrequency;
	CQTKernelSettings.NumBandsPerOctave = InNumBandsPerOctave;
	CQTKernelSettings.BandWidthStretch = InBandwidthStretch;
	CQTKernelSettings.Normalization = Audio::EPseudoConstantQNormalization::EqualEnergy;

	return CQTKernelSettings;
}

static Audio::FFFTSettings GetFFTSettings(float InMinimumFrequency, float InSampleRate, float InNumBandsPerOctave)
{
	const int32 MaximumSupportedLog2FFTSize = 13;
	const int32 MinimumSupportedLog2FFTSize = 8;

	InNumBandsPerOctave = FMath::Max(0.01f, InNumBandsPerOctave);

	float MinimumFrequencySpacing = (FMath::Pow(2.f, 1.f / InNumBandsPerOctave) - 1.f) * InMinimumFrequency;

	MinimumFrequencySpacing = FMath::Max(0.01f, MinimumFrequencySpacing);

	int32 DesiredFFTSize = FMath::CeilToInt(3.f * InSampleRate / MinimumFrequencySpacing);

	int32 Log2FFTSize = MinimumSupportedLog2FFTSize;
	while ((DesiredFFTSize > (1 << Log2FFTSize)) && (Log2FFTSize < MaximumSupportedLog2FFTSize))
	{
		Log2FFTSize++;
	}

	Audio::FFFTSettings FFTSettings;

	FFTSettings.Log2Size = Log2FFTSize;
	FFTSettings.bArrays128BitAligned = true;
	FFTSettings.bEnableHardwareAcceleration = true;

	return FFTSettings;
}

class FSubmixListener : public ISubmixBufferListener
{
public:

	/** Construct an FSubmixListener
		*
		* @param InMixer - Input FPatchMixer. Submix audio will be mixed into this patch mixer.
		* @param InNumSamplesToBuffer - Number of samples to hold in patch input. Low values may cause
		*                               data to be overwritten by threads that produce audio. High
		*                               values require more memory.
		*/
	FSubmixListener(Audio::FPatchMixer& InMixer, int32 InMaxChannelCount, int32 InNumSamplesToBuffer, int32 InMinimumSamplesForActivation, Audio::FDeviceId InDeviceId, USoundSubmix* InSoundSubmix)
		: MaxChannelCount(InMaxChannelCount)
		, MixerInput(InMixer.AddNewInput(InNumSamplesToBuffer, 1.0f))
		, AudioDeviceId(InDeviceId)
		, Submix(InSoundSubmix)
	{
		InitializedEvent.Emplace(UE_SOURCE_LOCATION);
		SamplesTillInitialized = InMinimumSamplesForActivation;
	}

	virtual ~FSubmixListener() override
	{
		check(!bIsRegistered);
	}

	FSubmixListener() = delete;
	FSubmixListener(const FSubmixListener& Other) = delete;
	FSubmixListener(FSubmixListener&& Other) = delete;

	void RegisterToSubmix()
	{
		if (FAudioDevice* AudioDevice = FAudioDeviceManager::Get()->GetAudioDeviceRaw(AudioDeviceId))
		{
			bIsRegistered = true;

			USoundSubmix* SubmixToRegister = Submix.IsValid() ? Submix.Get() : &AudioDevice->GetMainSubmixObject();
			AudioDevice->RegisterSubmixBufferListener(AsShared(), *SubmixToRegister);
		}
	}

	void UnregisterFromSubmix()
	{
		if (bIsRegistered)
		{
			bIsRegistered = false;

			if (IsInGameThread())
			{
				if (FAudioDevice* AudioDevice = FAudioDeviceManager::Get()->GetAudioDeviceRaw(AudioDeviceId))
				{
					USoundSubmix* SubmixToUnregister = Submix.IsValid() ? Submix.Get() : &AudioDevice->GetMainSubmixObject();
					AudioDevice->UnregisterSubmixBufferListener(AsShared(), *SubmixToUnregister);
				}
			}
			else
			{
				FAudioThread::RunCommandOnAudioThread([Self = AsShared(), AudioDeviceId = AudioDeviceId, Submix = Submix]()
				{
					if (FAudioDevice* AudioDevice = FAudioDeviceManager::Get()->GetAudioDeviceRaw(AudioDeviceId))
					{
						USoundSubmix* SubmixToUnregister = Submix.IsValid() ? Submix.Get() : &AudioDevice->GetMainSubmixObject();
						AudioDevice->UnregisterSubmixBufferListener(Self, *SubmixToUnregister);
					}
				});
			}
		}
	}

	/** Returns the current sample rate of the current submix. */
	float GetSampleRate() const
	{
		return static_cast<float>(SubmixSampleRate);
	}

	/** Returns the number of channels of the current submix. */
	int32 GetChannelCount() const
	{
		return ActiveChannelCount;
	}

	Audio::FDeviceId GetAudioDeviceId() const
	{
		return AudioDeviceId;
	}

	// Begin ISubmixBufferListener overrides
	virtual const FString& GetListenerName() const override
	{
		static const FString ListenerName = TEXT("NiagaraAudioSamplingListener");
		return ListenerName;
	}

	virtual void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate, double AudioClock) override
	{
		SubmixChannelCount = NumChannels;
		SubmixSampleRate = SampleRate;

		if (MaxChannelCount && MaxChannelCount < SubmixChannelCount)
		{
			// strip out the unneeded audio data based on our limited number of channels
			const int32 FrameCount = NumSamples / NumChannels;
			const int32 StrippedSampleCount = FrameCount * MaxChannelCount;

			TArray<float> StrippedData;
			StrippedData.Reserve(StrippedSampleCount);
			for (int32 FrameIt = 0; FrameIt < FrameCount; ++FrameIt)
			{
				int32 ChannelIt = 0;
				for (; ChannelIt < MaxChannelCount; ++ChannelIt)
				{
					StrippedData.Add(*AudioData);
					++AudioData;
				}

				for (; ChannelIt < NumChannels; ++ChannelIt)
				{
					++AudioData;
				}
			}

			ActiveChannelCount = MaxChannelCount;
			MixerInput.PushAudio(StrippedData.GetData(), StrippedData.Num());
		}
		else
		{
			MixerInput.PushAudio(AudioData, NumSamples);
			ActiveChannelCount = SubmixChannelCount;
		}

		const int32 PrevSamplesRemaining = SamplesTillInitialized.fetch_sub(NumSamples, std::memory_order_relaxed);
		if (PrevSamplesRemaining > 0 && ((PrevSamplesRemaining - NumSamples) <= 0))
		{
			InitializedEvent->Trigger();
		}
	}
	// End ISubmixBufferListener overrides

	using FOptionalEvent = TOptional<UE::Tasks::FTaskEvent>;
	FOptionalEvent GetInitializedEvent()
	{
		return InitializedEvent;
	}
	void ReleaseInitializedEvent()
	{
		InitializedEvent.Reset();
	}

private:
	// properties pulled from the Submix
	int32 SubmixChannelCount = 0;
	int32 SubmixSampleRate = 0;

	// limits to the number of channels we need to worry about specified at creation time (0 implies no limit)
	int32 MaxChannelCount = 0;

	// holds the number of channels that we are actively using (taking account of both SubmixChannelCount and MaxChannelCount)
	int32 ActiveChannelCount = 0;

	Audio::FPatchInput MixerInput;

	Audio::FDeviceId AudioDeviceId = INDEX_NONE;
	TWeakObjectPtr<USoundSubmix> Submix = nullptr;
	bool bIsRegistered = false;

	FOptionalEvent InitializedEvent;
	std::atomic<int32> SamplesTillInitialized;
};

struct FCollectAudioContext
{
	int32 AudioSampleCount = 0;
	int32 SpectrumResolution = 0;
	int32 ResampledAudioResolution = 0;
	int32 ChannelCount = 0;
	float NoiseFloorDb = 0.0f;
	float SampleRate = 0.0f;
	float MinimumFrequency = 0.0f;
	float MaximumFrequency = 0.0f;

	bool bContinuousSampling = false;
	bool bUseLatestAudio = false;
};

class FSpectrumBuilder
{
public:
	using FSlidingBuffer = Audio::TSlidingBuffer<float>;
	using FSlidingWindow = Audio::TAutoSlidingWindow<float, Audio::FAudioBufferAlignedAllocator>;
	using FDeinterleaveView = Audio::TAutoDeinterleaveView<float, Audio::FAudioBufferAlignedAllocator>;
	using FChannel = FDeinterleaveView::TChannel<Audio::FAudioBufferAlignedAllocator>;

	void Build(const FCollectAudioContext& CollectContext, const Audio::FAlignedFloatBuffer& AudioBuffer)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAudioSampling::GenerateSpectrum);

		SetAudioFormat(CollectContext);

		ResetSpectrumBuffers(CollectContext);

		// Set data to zero if we are skipping update due to bad samplerate or channel count.
		if (CollectContext.ChannelCount == 0 || FMath::IsNearlyZero(CollectContext.SampleRate))
		{
			return;
		}

		// If we're in a bad internal state, give up here.	
		if (!FFTAlgorithm.IsValid() || !CQTKernel.IsValid())
		{
			return;
		}

		// Run sliding window over available audio.
		FSlidingWindow SlidingWindow(*SlidingBuffer, AudioBuffer, InterleavedBuffer);

		const int32 NumBands = CollectContext.SpectrumResolution;
		const float NoiseFloorDb = CollectContext.NoiseFloorDb;

		auto MakeChannelBufferView = [&CollectContext](Audio::FAlignedFloatBuffer& PlanarBuffer, int32 ChannelIndex) -> TArrayView<float>
		{
			return MakeArrayView(PlanarBuffer.GetData() + CollectContext.SpectrumResolution * ChannelIndex, CollectContext.SpectrumResolution);
		};

		int32 NumWindows = 0;
		for (Audio::FAlignedFloatBuffer& InterleavedWindow : SlidingWindow)
		{
			NumWindows++;

			// Run deinterleave view over window.
			FDeinterleaveView DeinterleaveView(InterleavedWindow, DeinterleavedBuffer, CollectContext.ChannelCount);
			for (FChannel Channel : DeinterleaveView)
			{
				Audio::ArrayMultiplyInPlace(WindowBuffer, Channel.Values);

				// Perform FFT
				FMemory::Memcpy(FFTInputBuffer.GetData(), Channel.Values.GetData(), sizeof(float) * Channel.Values.Num());

				FFTAlgorithm->ForwardRealToComplex(FFTInputBuffer.GetData(), FFTOutputBuffer.GetData());

				// Transflate FFTOutput to power spectrum
				Audio::ArrayComplexToPower(FFTOutputBuffer, PowerSpectrumBuffer);

				// Take CQT of power spectrum
				CQTKernel->TransformArray(PowerSpectrumBuffer, SpectrumBuffer);

				// Accumulate power spectrum CQT output.
				Audio::ArrayAddInPlace(SpectrumBuffer, MakeChannelBufferView(PlanarChannelSpectrumBuffers, Channel.ChannelIndex));//[Channel.ChannelIndex]);
			}
		}

		if (NumWindows > 0)
		{
			// Two scalings are applied. One for FFT, another for number of windows calculated.
			float LinearScale = FFTScale / static_cast<float>(NumWindows);

			// This scaling moves output to roughly 0 to 1 range.
			float DbScale = 1.f / FMath::Max(1.f, -NoiseFloorDb);

			// Apply scaling for each channel.
			for (int32 ChannelIndex = 0; ChannelIndex < CollectContext.ChannelCount; ChannelIndex++)
			{
				//Audio::FAlignedFloatBuffer& Buffer = ChannelSpectrumBuffers[ChannelIndex];
				TArrayView<float> BufferView = MakeChannelBufferView(PlanarChannelSpectrumBuffers, ChannelIndex);

				Audio::ArrayMultiplyByConstantInPlace(BufferView, LinearScale);

				Audio::ArrayPowerToDecibelInPlace(BufferView, NoiseFloorDb);

				Audio::ArraySubtractByConstantInPlace(BufferView, NoiseFloorDb);

				Audio::ArrayMultiplyByConstantInPlace(BufferView, DbScale);
			}
		}
	}

	// Updates internal objects for num channels and samplerate
	void SetAudioFormat(const FCollectAudioContext& CollectContext)
	{
		if (CollectContext.SampleRate != SampleRate)
		{
			ResizeAudioTransform(CollectContext);
		}

		if (CollectContext.ChannelCount != ChannelCount)
		{
			ResetSpectrumBuffers(CollectContext);
		}

		if ((CollectContext.SampleRate != SampleRate) || (CollectContext.ChannelCount != ChannelCount))
		{
			int32 FFTSize = FFTAlgorithm.IsValid() ? FFTAlgorithm->Size() : 0;

			ResizeWindow(CollectContext.ChannelCount, FFTSize);

			ChannelCount = CollectContext.ChannelCount;
			SampleRate = CollectContext.SampleRate;
		}
	}

	void ResizeAudioTransform(const FCollectAudioContext& CollectContext)
	{
		using namespace UE::Niagara::GeneratedDataAudioSampling::Private;

		FFTAlgorithm.Reset();
		CQTKernel.Reset();
		FFTInputBuffer.Reset();
		FFTOutputBuffer.Reset();
		PowerSpectrumBuffer.Reset();
		SpectrumBuffer.Reset();

		FFTScale = 1.f;

		if (CollectContext.SampleRate <= 0.f)
		{
			return;
		}

		if (CollectContext.SpectrumResolution < 1)
		{
			return;
		}

		float NumOctaves = GetNumOctaves(CollectContext.MinimumFrequency, CollectContext.MaximumFrequency);
		float NumBandsPerOctave = GetNumBandsPerOctave(CollectContext.SpectrumResolution, NumOctaves);

		Audio::FFFTSettings FFTSettings = GetFFTSettings(CollectContext.MinimumFrequency, CollectContext.SampleRate, NumBandsPerOctave);
		if (!Audio::FFFTFactory::AreFFTSettingsSupported(FFTSettings))
		{
			UE_LOGF(LogNiagara, Warning, "FFT settings are not supported");
			return;
		}

		FFTAlgorithm = Audio::FFFTFactory::NewFFTAlgorithm(FFTSettings);

		if (!FFTAlgorithm.IsValid())
		{
			UE_LOGF(LogNiagara, Error, "Failed to create FFT");
			return;
		}

		FFTInputBuffer.AddZeroed(FFTAlgorithm->NumInputFloats());
		FFTOutputBuffer.AddUninitialized(FFTAlgorithm->NumOutputFloats());
		PowerSpectrumBuffer.AddUninitialized(FFTOutputBuffer.Num() / 2);
		SpectrumBuffer.AddUninitialized(CollectContext.SpectrumResolution);

		float FFTSize = static_cast<float>(FFTAlgorithm->Size());
		check(FFTSize > 0.f);

		// We want to have all FFT implementations
		// return the same scaling so that the energy conservatino property of the fourier transform
		// is supported.  This scaling factor is applied to the power spectrum, so we square the 
		// scaling we would have performed on the magnitude spectrum.
		switch (FFTAlgorithm->ForwardScaling())
		{
		case Audio::EFFTScaling::MultipliedByFFTSize:
			FFTScale = 1.f / (FFTSize * FFTSize);
			break;

		case Audio::EFFTScaling::MultipliedBySqrtFFTSize:
			FFTScale = 1.f / FFTSize;
			break;

		case Audio::EFFTScaling::DividedByFFTSize:
			FFTScale = FFTSize * FFTSize;
			break;

		case Audio::EFFTScaling::DividedBySqrtFFTSize:
			FFTScale = FFTSize;
			break;

		default:
			FFTScale = 1.f;
			break;
		}

		float BandwidthStretch = GetBandwidthStretch(CollectContext.SampleRate, FFTSize, NumBandsPerOctave, CollectContext.MinimumFrequency);

		Audio::FPseudoConstantQKernelSettings CQTSettings = GetConstantQSettings(CollectContext.MinimumFrequency, CollectContext.MaximumFrequency, CollectContext.SpectrumResolution, NumBandsPerOctave, BandwidthStretch);
		CQTKernel = Audio::NewPseudoConstantQKernelTransform(CQTSettings, FFTAlgorithm->Size(), CollectContext.SampleRate);

		if (!CQTKernel.IsValid())
		{
			UE_LOGF(LogNiagara, Error, "Failed to create CQT kernel.");
			return;
		}
	}

	void ResetSpectrumBuffers(const FCollectAudioContext& CollectContext)
	{
		PlanarChannelSpectrumBuffers.Reset();
		PlanarChannelSpectrumBuffers.AddZeroed(CollectContext.SpectrumResolution * CollectContext.ChannelCount);
	}

	void ResizeWindow(int32 InNumChannels, int32 InFFTSize)
	{
		int32 NumWindowFrames = FMath::Max(1, InFFTSize);

		NumWindowFrames = FMath::Min(NumWindowFrames, 1024);

		WindowBuffer.Reset();
		WindowBuffer.AddUninitialized(NumWindowFrames);

		Audio::GenerateBlackmanWindow(WindowBuffer.GetData(), NumWindowFrames, 1, true);

		int32 NumWindowSamples = NumWindowFrames;

		if (InNumChannels > 0)
		{
			NumWindowSamples *= InNumChannels;
		}

		// 50% overlap
		int32 NumHopSamples = FMath::Max(1, NumWindowSamples / 2);

		SlidingBuffer = MakeUnique<FSlidingBuffer>(NumWindowSamples, NumHopSamples);

	}

	TConstArrayView<float> GetPlanarSpectrumBuffer() const
	{
		return MakeConstArrayView(PlanarChannelSpectrumBuffers);
	}

	// Get settings 
	static void Clamp(float& InMinimumFrequency, float& InMaximumFrequency, int32& InNumBands)
	{
		const int32 MinimumSupportedNumBands = 1;
		const int32 MaximumSupportedNumBands = 16384;
		const float MinimumSupportedFrequency = 20.f;
		const float MaximumSupportedFrequency = 20000.f;

		InMinimumFrequency = FMath::Max(InMinimumFrequency, MinimumSupportedFrequency);
		InMaximumFrequency = FMath::Min(InMaximumFrequency, MaximumSupportedFrequency);
		InMaximumFrequency = FMath::Max(InMinimumFrequency, InMaximumFrequency);

		InNumBands = FMath::Clamp(InNumBands, MinimumSupportedNumBands, MaximumSupportedNumBands);
	}

	static float GetNumOctaves(float InMinimumFrequency, float InMaximumFrequency)
	{
		const float MinimumSupportedNumOctaves = 0.01f;

		float NumOctaves = FMath::Log2(InMaximumFrequency) - FMath::Log2(InMinimumFrequency);
		NumOctaves = FMath::Max(MinimumSupportedNumOctaves, NumOctaves);

		return NumOctaves;
	}

	static float GetNumBandsPerOctave(int32 InNumBands, float InNumOctaves)
	{
		InNumOctaves = FMath::Max(0.01f, InNumOctaves);
		const float MinimumSupportedNumBandsPerOctave = 0.01f;

		float NumBandsPerOctave = FMath::Max(MinimumSupportedNumBandsPerOctave, static_cast<float>(InNumBands) / InNumOctaves);

		return NumBandsPerOctave;
	}

	static float GetBandwidthStretch(float InSampleRate, float InFFTSize, float InNumBandsPerOctave, float InMinimumFrequency)
	{
		InFFTSize = FMath::Max(1.f, InFFTSize);

		float MinimumFrequencySpacing = (FMath::Pow(2.f, 1.f / InNumBandsPerOctave) - 1.f) * InMinimumFrequency;
		MinimumFrequencySpacing = FMath::Max(0.01f, MinimumFrequencySpacing);

		float FFTBinSpacing = InSampleRate / InFFTSize;
		float BandwidthStretch = FFTBinSpacing / MinimumFrequencySpacing;

		return FMath::Clamp(BandwidthStretch, 0.5f, 2.f);
	}

private:
	TUniquePtr<FSlidingBuffer> SlidingBuffer;
	Audio::FAlignedFloatBuffer PlanarChannelSpectrumBuffers;
	TUniquePtr<Audio::FContiguousSparse2DKernelTransform> CQTKernel;
	TUniquePtr<Audio::IFFTAlgorithm> FFTAlgorithm;
	Audio::FAlignedFloatBuffer InterleavedBuffer;
	Audio::FAlignedFloatBuffer DeinterleavedBuffer;
	Audio::FAlignedFloatBuffer FFTInputBuffer;
	Audio::FAlignedFloatBuffer FFTOutputBuffer;
	Audio::FAlignedFloatBuffer PowerSpectrumBuffer;
	Audio::FAlignedFloatBuffer SpectrumBuffer;
	Audio::FAlignedFloatBuffer WindowBuffer;

	int32 ChannelCount = 0;
	float SampleRate = 0.0f;
	float FFTScale = 1.0f;
};


class FNDIAudio_SharedResourceImpl : public FNDIAudio_SharedResource
{
public:
	FNDIAudio_SharedResourceImpl() = delete;
	FNDIAudio_SharedResourceImpl(const FNDIAudio_SharedResource&) = delete;
	FNDIAudio_SharedResourceImpl(const FResourceKey& InKey)
		: FNDIAudio_SharedResource(InKey)
	{
		AddSubmixListener(InKey.DeviceId);
	}

	virtual ~FNDIAudio_SharedResourceImpl()
	{
		RemoveSubmixListener(ResourceKey.DeviceId);
	}

	virtual int32 GetChannelCount() const override
	{
		int32 ChannelCount = 0;

		if (SubmixListener.IsValid())
		{
			ChannelCount = FMath::Max(ChannelCount, SubmixListener->GetChannelCount());
		}

		return ChannelCount;
	}

	virtual float GetSampleRate() const override
	{
		float SampleRate = 0.0f;

		if (SubmixListener.IsValid())
		{
			SampleRate = FMath::Max(SampleRate, SubmixListener->GetSampleRate());
		}

		return SampleRate;
	}

	virtual TConstArrayView<float> ReadAudioBuffer(int32& ChannelCount, int32& FrameCount) const override
	{
		ChannelCount = GetChannelCount();
		FrameCount = ResourceKey.AudioResamplingResolution;

		const int32 SampleCount = ResampledAudioBuffer.Num();

		if (SampleCount < (ChannelCount * FrameCount))
		{
			ChannelCount = 0;
			FrameCount = 0;
			return TConstArrayView<float>();
		}

		return MakeConstArrayView(ResampledAudioBuffer);
	}

	virtual TConstArrayView<float> ReadSpectrumBuffer(int32& ChannelCount, int32& FrameCount) const override
	{
		ChannelCount = 0;
		FrameCount = 0;

		if (SpectrumBuilder)
		{
			TConstArrayView<float> SpectrumBuffer = SpectrumBuilder->GetPlanarSpectrumBuffer();
			ChannelCount = GetChannelCount();
			FrameCount = ResourceKey.SpectrumSamplingResolution;

			if (SpectrumBuffer.Num() < (ChannelCount * FrameCount))
			{
				ChannelCount = 0;
				FrameCount = 0;
				return TConstArrayView<float>();
			}

			return SpectrumBuffer;
		}

		return TConstArrayView<float>();
	}

	bool ResampleAudioBuffer(int32 TargetResolution, TArray<float>& OutResampledBuffer) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAudioSampling::ResampleAudioBuffer);

		const int32 ChannelCount = GetChannelCount();
		const int32 SourceSampleCount = PopBuffer.Num();

		if (!ChannelCount
			|| !TargetResolution
			|| !SourceSampleCount)
		{
			OutResampledBuffer.Reset();
			return false;
		}

		const float SourceSampleRate = GetSampleRate();
		const int32 TargetSampleCount = TargetResolution * ChannelCount;

		const float SampleRateRatio = (float)(TargetSampleCount) / ((float)SourceSampleCount);
		const float DestinationSampleRate = SourceSampleRate * SampleRateRatio;

		Audio::FResamplingParameters ResampleParameters = {
			Audio::EResamplingMethod::Linear,
			ChannelCount,
			SourceSampleRate,
			DestinationSampleRate,
			const_cast<Audio::FAlignedFloatBuffer&>(PopBuffer)
		};

		int32 ResampledBufferSize = Audio::GetOutputBufferSize(ResampleParameters);

		Audio::FAlignedFloatBuffer ResampledBuffer;
		ResampledBuffer.Reset();
		ResampledBuffer.AddZeroed(ResampledBufferSize);

		Audio::FResamplerResults ResampleResults;
		ResampleResults.OutBuffer = &ResampledBuffer;

		bool bResampleResult = Audio::Resample(ResampleParameters, ResampleResults);
		check(bResampleResult);

		OutResampledBuffer = MoveTemp(ResampledBuffer);
		OutResampledBuffer.SetNumZeroed(TargetSampleCount);

		return true;
	}

	virtual void CollectAudio() override
	{
		constexpr bool bAsyncCollectAudio = true;
		constexpr bool bAsyncGenerateSpectrum = true;

		if (!SubmixListener)
		{
			return;
		}

		{
			FSubmixListener::FOptionalEvent Event = SubmixListener->GetInitializedEvent();
			if (Event.IsSet())
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAudioSampling::EventWait);
				Event->Wait();
				SubmixListener->ReleaseInitializedEvent();
			}
		}

		const bool bIncludeSpectrum = SpectrumCpuRefCount > 0 || SpectrumGpuRefCount > 0;
		const bool bIncludeAudioResample = AudioCpuRefCount > 0 || AudioGpuRefCount > 0;

		FCollectAudioContext CollectContext;
		CollectContext.SpectrumResolution = bIncludeSpectrum ? ResourceKey.SpectrumSamplingResolution : 0;
		CollectContext.ResampledAudioResolution = bIncludeAudioResample ? ResourceKey.AudioResamplingResolution : 0;
		CollectContext.ChannelCount = GetChannelCount();
		CollectContext.NoiseFloorDb = ResourceKey.NoiseFloorDb;
		CollectContext.SampleRate = GetSampleRate();
		CollectContext.MinimumFrequency = ResourceKey.MinimumFrequency;
		CollectContext.MaximumFrequency = ResourceKey.MaximumFrequency;
		CollectContext.bUseLatestAudio = ResourceKey.bUseLatestAudio;
		CollectContext.bContinuousSampling = ResourceKey.bContinuousSampling;
		CollectContext.AudioSampleCount = GetRequiredSampleCount(CollectContext.ChannelCount, CollectContext.SampleRate);

		TerminalAudioTask.Reset();
		if (bAsyncCollectAudio)
		{
			CollectAudioTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, CollectContext]
			{
				CollectAudioInternal(CollectContext);
			});

			TerminalAudioTask.Emplace(CollectAudioTask);
		}
		else
		{
			CollectAudioTask = UE::Tasks::FTask();
			CollectAudioInternal(CollectContext);
		}

		if (CollectContext.SpectrumResolution > 0)
		{
			if (!SpectrumBuilder)
			{
				SpectrumBuilder = MakeUnique<FSpectrumBuilder>();
			}

			if (bAsyncGenerateSpectrum)
			{
				GenerateSpectrumTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, CollectContext]
				{
					SpectrumBuilder->Build(CollectContext, PopBuffer);
				}, UE::Tasks::Prerequisites(CollectAudioTask));

				TerminalAudioTask.Emplace(GenerateSpectrumTask);
			}
			else
			{
				CollectAudioTask.Wait();

				GenerateSpectrumTask = UE::Tasks::FTask();
				SpectrumBuilder->Build(CollectContext, PopBuffer);
			}
		}
		else
		{
			SpectrumBuilder.Reset();
		}
	}

	virtual void WaitForAudio() const override
	{
		if (TerminalAudioTask.IsSet())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAudioSampling::WaitForAudio);
			TerminalAudioTask->Wait();
		}
	}

protected:
	void AddSubmixListener(Audio::FDeviceId InDeviceId)
	{
		if (SubmixListener)
		{
			check(SubmixListener->GetAudioDeviceId() == InDeviceId);
		}
		else
		{
			if (FAudioDevice* DeviceHandle = FAudioDeviceManager::Get()->GetAudioDeviceRaw(InDeviceId))
			{
				const float DeviceSampleRate = DeviceHandle->GetSampleRate();
				const int32 MaxChannelCount = ResourceKey.MaxChannelCount ? ResourceKey.MaxChannelCount : AUDIO_MIXER_MAX_OUTPUT_CHANNELS;
				const int32 MaxBufferSampleCount = GetRequiredSampleCount(MaxChannelCount, DeviceSampleRate);
				const int32 MinSamplesRequired = 1;

				SubmixListener = MakeShared<FSubmixListener>(PatchMixer, MaxChannelCount, MaxBufferSampleCount, MinSamplesRequired, InDeviceId, ResourceKey.Submix.Get());
				SubmixListener->RegisterToSubmix();
			}
		}
	}

	void RemoveSubmixListener(Audio::FDeviceId InDeviceId)
	{
		if (SubmixListener)
		{
			check(SubmixListener->GetAudioDeviceId() == InDeviceId);
			SubmixListener->UnregisterFromSubmix();
			SubmixListener.Reset();
		}
	}

	void CollectAudioInternal(const FCollectAudioContext& CollectContext)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAudioSampling::CollectAudio);

		if (CollectContext.ChannelCount == 0 || CollectContext.AudioSampleCount == 0)
		{
			return;
		}

		int32 AvailableSampleCount = PatchMixer.MaxNumberOfSamplesThatCanBePopped();
		AvailableSampleCount -= (AvailableSampleCount % CollectContext.ChannelCount);

		if (AvailableSampleCount > 0)
		{
			if (CollectContext.bContinuousSampling)
			{
				PopBuffer.Reset();
				PopBuffer.AddUninitialized(AvailableSampleCount);
				PatchMixer.PopAudio(PopBuffer.GetData(), AvailableSampleCount, CollectContext.bUseLatestAudio);
			}
			// fill an entire buffer
			else if (AvailableSampleCount >= CollectContext.AudioSampleCount)
			{
				PopBuffer.SetNumZeroed(CollectContext.AudioSampleCount);
				PatchMixer.PopAudio(PopBuffer.GetData(), CollectContext.AudioSampleCount, CollectContext.bUseLatestAudio);
			}
			// see if we should append to the existing buffer
			else if (AvailableSampleCount + PopBuffer.Num() <= CollectContext.AudioSampleCount)
			{
				const int32 ExistingSampleCount = PopBuffer.Num();
				PopBuffer.SetNumZeroed(ExistingSampleCount + AvailableSampleCount);
				PatchMixer.PopAudio(PopBuffer.GetData() + ExistingSampleCount, AvailableSampleCount, CollectContext.bUseLatestAudio);
			}
			// pop out existing data to make room for the new data coming in
			else
			{
				const int32 SaveCount = FMath::Max(0, CollectContext.AudioSampleCount - AvailableSampleCount);
				TArray<float> SavedSamples(MakeArrayView<float>(PopBuffer.GetData() + PopBuffer.Num() - SaveCount, SaveCount));
				PopBuffer.Reset();
				PopBuffer.Append(SavedSamples);
				PopBuffer.SetNumZeroed(CollectContext.AudioSampleCount);
				PatchMixer.PopAudio(PopBuffer.GetData() + SaveCount, CollectContext.AudioSampleCount - SaveCount, CollectContext.bUseLatestAudio);
			}

			if (CollectContext.ResampledAudioResolution)
			{
				ResampleAudioBuffer(CollectContext.ResampledAudioResolution, ResampledAudioBuffer);
			}
		}

		if (!CollectContext.ResampledAudioResolution)
		{
			ResampledAudioBuffer.Reset();
		}
	}

	int32 GetRequiredSampleCount(int32 ChannelCount, float SampleRate)
	{
		if (ResourceKey.SamplingMethod == ESamplingWindowMethod::ByCount)
		{
			return Align(ResourceKey.SamplingWindowCount, AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);
		}
		else if (ResourceKey.SamplingMethod == ESamplingWindowMethod::ByTime)
		{
			const int32 SamplesPerChannel = FMath::CeilToInt(SampleRate * (ResourceKey.SamplingWindowInMilliseconds / 1000.0f));
			return ChannelCount * Align(SamplesPerChannel, AUDIO_NUM_FLOATS_PER_VECTOR_REGISTER);
		}

		return 0;
	}

	// Mixer for sending audio
	Audio::FPatchMixer PatchMixer;

	Audio::FAlignedFloatBuffer PopBuffer;
	TArray<float> ResampledAudioBuffer;

	TSharedPtr<FSubmixListener> SubmixListener;

	TUniquePtr<FSpectrumBuilder> SpectrumBuilder;

	UE::Tasks::FTask CollectAudioTask;
	UE::Tasks::FTask GenerateSpectrumTask;

	TOptional<UE::Tasks::FTask> TerminalAudioTask;
};

}

const ETickingGroup FNDIAudio_GeneratedData::GeneratedDataTickGroup = NiagaraFirstTickGroup;

FNDI_GeneratedData::TypeHash FNDIAudio_GeneratedData::GetTypeHash()
{
	static const TypeHash Hash = FCrc::Strihash_DEPRECATED(TEXT("FNDIAudio_GeneratedData"));
	return Hash;
}

void FNDIAudio_GeneratedData::Tick(ETickingGroup TickGroup, float DeltaSeconds)
{
	if (TickGroup == GeneratedDataTickGroup)
	{
		FRWScopeLock WriteLock(SharedResourcesLock, SLT_Write);

		for (FSharedResourceArray::TIterator ResourceIt(SharedResources); ResourceIt; ++ResourceIt)
		{
			TSharedPtr<FNDIAudio_SharedResource>& SharedResource = *ResourceIt;
			if (SharedResource.IsValid() && SharedResource->IsUsed())
			{
				SharedResource->CollectAudio();
			}
			else
			{
				SharedResource.Reset();
			}

			// clear out any unused entries in SharedResources
			if (!SharedResource.IsValid())
			{
				ResourceIt.RemoveCurrent();
			}
		}
	}
}

FNDIAudio_SharedResourceHandle FNDIAudio_GeneratedData::GetSharedResource(const FNDIAudio_SharedResourceUsage& Usage, const FResourceDesc& ResourceDesc)
{
	using namespace UE::Niagara::GeneratedDataAudioSampling::Private;

	auto FindExistingResource = [this, &ResourceDesc]() -> int32
	{
		const int32 ResourceCount = SharedResources.Num();
		for (int32 ResourceIt = 0; ResourceIt < ResourceCount; ++ResourceIt)
		{
			const TSharedPtr<FNDIAudio_SharedResource>& SharedResource = SharedResources[ResourceIt];
			if (SharedResource.IsValid())
			{
				if (SharedResource->GetResourceKey().CanSupport(ResourceDesc))
				{
					return ResourceIt;
				}
			}
		}

		return INDEX_NONE;
	};

	// see if any of our existing resources will suffice
	{
		FRWScopeLock ReadLock(SharedResourcesLock, SLT_ReadOnly);

		const int32 ExistingIndex = FindExistingResource();
		if (ExistingIndex != INDEX_NONE)
		{
			return FNDIAudio_SharedResourceHandle(Usage, SharedResources[ExistingIndex], false /*bNeedsDataImmediately*/);
		}
	}

	FRWScopeLock ResourceWriteLock(SharedResourcesLock, SLT_Write);

	{
		const int32 ExistingIndex = FindExistingResource();
		if (ExistingIndex != INDEX_NONE)
		{
			return FNDIAudio_SharedResourceHandle(Usage, SharedResources[ExistingIndex], false /*bNeedsDataImmediately*/);
		}
	}

	TSharedPtr<FNDIAudio_SharedResourceImpl> SharedResource = MakeShared<FNDIAudio_SharedResourceImpl>(ResourceDesc);
	SharedResources.Add(SharedResource);
	FNDIAudio_SharedResourceHandle Handle(Usage, SharedResource, false /*bNeedsDataImmediately*/);

	SharedResource->CollectAudio();

	return Handle;
}

FNDIAudio_SharedResource::FNDIAudio_SharedResource(const FResourceKey& InKey)
	: ResourceKey(InKey)
{
}

bool FNDIAudio_SharedResource::IsUsed() const
{
	return AudioCpuRefCount > 0
		|| AudioGpuRefCount > 0
		|| SpectrumCpuRefCount > 0
		|| SpectrumGpuRefCount > 0;
}

void FNDIAudio_SharedResource::RegisterUser(const FNDIAudio_SharedResourceUsage& Usage, bool bNeedsDataImmediately)
{
	AudioCpuRefCount += FNDIAudio_SharedResourceUsage::CpuUsageFlags(Usage.AudioUsage) ? 1 : 0;
	AudioGpuRefCount += FNDIAudio_SharedResourceUsage::GpuUsageFlags(Usage.AudioUsage) ? 1 : 0;
	SpectrumCpuRefCount += FNDIAudio_SharedResourceUsage::CpuUsageFlags(Usage.SpectrumUsage) ? 1 : 0;
	SpectrumGpuRefCount += FNDIAudio_SharedResourceUsage::GpuUsageFlags(Usage.SpectrumUsage) ? 1 : 0;
}

void FNDIAudio_SharedResource::UnregisterUser(const FNDIAudio_SharedResourceUsage& Usage)
{
	AudioCpuRefCount -= FNDIAudio_SharedResourceUsage::CpuUsageFlags(Usage.AudioUsage) ? 1 : 0;
	AudioGpuRefCount -= FNDIAudio_SharedResourceUsage::GpuUsageFlags(Usage.AudioUsage) ? 1 : 0;
	SpectrumCpuRefCount -= FNDIAudio_SharedResourceUsage::CpuUsageFlags(Usage.SpectrumUsage) ? 1 : 0;
	SpectrumGpuRefCount -= FNDIAudio_SharedResourceUsage::GpuUsageFlags(Usage.SpectrumUsage) ? 1 : 0;
}

bool FNDIAudio_SharedResource::FResourceKey::CanSupport(const FResourceKey& Other) const
{
	// check the settings that have to match
	if ((Submix != Other.Submix)
		|| (bUseLatestAudio != Other.bUseLatestAudio)
		|| (DeviceId != Other.DeviceId)
		|| (MaxChannelCount != Other.MaxChannelCount))
	{
		return false;
	}

	if (SamplingMethod != Other.SamplingMethod)
	{
		return false;
	}
	else if (SamplingMethod == ESamplingWindowMethod::ByCount)
	{
		if (SamplingWindowCount != Other.SamplingWindowCount)
		{
			return false;
		}
	}
	else if (SamplingMethod == ESamplingWindowMethod::ByTime)
	{
		if (SamplingWindowInMilliseconds != Other.SamplingWindowInMilliseconds)
		{
			return false;
		}
	}

	if (bResampleAudio && Other.bResampleAudio)
	{
		if (AudioResamplingResolution != Other.AudioResamplingResolution)
		{
			return false;
		}
	}

	if (bGenerateSpectrum && Other.bGenerateSpectrum)
	{
		if ((MinimumFrequency != Other.MinimumFrequency)
			|| (MaximumFrequency != Other.MaximumFrequency)
			|| (NoiseFloorDb != Other.NoiseFloorDb)
			|| (SpectrumSamplingResolution != Other.SpectrumSamplingResolution))
		{
			return false;
		}
	}

	return true;
}

