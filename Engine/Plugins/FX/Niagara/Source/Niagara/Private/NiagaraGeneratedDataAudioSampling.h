// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "AudioDefines.h"
#include "NiagaraDataInterface.h"
#include "Sound/SoundSubmix.h"

class FNDIAudio_SharedResourceUsage : public FNDI_SharedResourceUsage
{
public:
	enum class EAudioBufferUsage { Unused, CpuOnly, GpuOnly, Both };
	enum class ESpectrumBufferUsage { Unused, CpuOnly, GpuOnly, Both };

	FNDIAudio_SharedResourceUsage(EAudioBufferUsage InAudioUsage = EAudioBufferUsage::Unused, ESpectrumBufferUsage InSpectrumUsage = ESpectrumBufferUsage::Unused)
		: FNDI_SharedResourceUsage(CpuUsageFlags(InAudioUsage) || CpuUsageFlags(InSpectrumUsage), GpuUsageFlags(InAudioUsage) || GpuUsageFlags(InSpectrumUsage))
		, AudioUsage(InAudioUsage)
		, SpectrumUsage(InSpectrumUsage)
	{
	}

	FNDIAudio_SharedResourceUsage(const FNDIAudio_SharedResourceUsage& Other)
		: FNDI_SharedResourceUsage(Other)
		, AudioUsage(Other.AudioUsage)
		, SpectrumUsage(Other.SpectrumUsage)
	{
	}

	EAudioBufferUsage AudioUsage = EAudioBufferUsage::Unused;
	ESpectrumBufferUsage SpectrumUsage = ESpectrumBufferUsage::Unused;

	template<typename T>
	static bool CpuUsageFlags(const T& UsageType)
	{
		return UsageType == T::Both || UsageType == T::CpuOnly;
	}

	template<typename T>
	static bool GpuUsageFlags(const T& UsageType)
	{
		return UsageType == T::Both || UsageType == T::GpuOnly;
	}

	template<typename T>
	static void UpdateUsageFlags(bool bCpuUsage, bool bGpuUsage, T& UsageType)
	{
		if (bCpuUsage && bGpuUsage)
		{
			UsageType = T::Both;
		}
		else if (bCpuUsage)
		{
			UsageType = T::CpuOnly;
		}
		else if (bGpuUsage)
		{
			UsageType = T::GpuOnly;
		}
		else
		{
			UsageType = T::Unused;
		}
	}
};

// SharedResource that can be held by multiple system instances which manages the resources created by this DI
class FNDIAudio_SharedResource
{
public:
	enum class ESamplingWindowMethod
	{
		ByTime, ByCount
	};

	struct FResourceKey
	{
		TWeakObjectPtr<USoundSubmix> Submix = nullptr;

		int32 AudioResamplingResolution = 0;
		int32 SpectrumSamplingResolution = 0;
		int32 MaxChannelCount = 0; // If set memory usage for the collection buffer can be reduced
		float MinimumFrequency = 0.0f;
		float MaximumFrequency = 0.0f;
		float NoiseFloorDb = 0.0f;
		Audio::FDeviceId DeviceId = INDEX_NONE;
		bool bUseLatestAudio = false;
		bool bResampleAudio = false;
		bool bGenerateSpectrum = false;
		bool bContinuousSampling = false;
		ESamplingWindowMethod SamplingMethod = ESamplingWindowMethod::ByCount;
		int32 SamplingWindowCount = 0;
		float SamplingWindowInMilliseconds = 0.0f;

		bool CanSupport(const FResourceKey& Other) const;
	};

	FNDIAudio_SharedResource() = delete;
	FNDIAudio_SharedResource(const FNDIAudio_SharedResource&) = delete;
	FNDIAudio_SharedResource(const FResourceKey& InKey);
	virtual ~FNDIAudio_SharedResource() = default;

	virtual int32 GetChannelCount() const = 0;
	virtual float GetSampleRate() const = 0;

	virtual TConstArrayView<float> ReadAudioBuffer(int32& ChannelCount, int32& FrameCount) const = 0;
	virtual TConstArrayView<float> ReadSpectrumBuffer(int32& ChannelCount, int32& FrameCount) const = 0;

	const FResourceKey& GetResourceKey() const { return ResourceKey; }
	bool IsUsed() const;

	void RegisterUser(const FNDIAudio_SharedResourceUsage& Usage, bool bNeedsDataImmediately);
	void UnregisterUser(const FNDIAudio_SharedResourceUsage& Usage);

	void Release();

	virtual void CollectAudio() = 0;
	virtual void WaitForAudio() const = 0;

protected:

	FResourceKey ResourceKey;

	std::atomic<int32> AudioCpuRefCount = {0};
	std::atomic<int32> AudioGpuRefCount = {0};
	std::atomic<int32> SpectrumCpuRefCount = {0};
	std::atomic<int32> SpectrumGpuRefCount = {0};
};

using FNDIAudio_SharedResourceHandle = FNDI_SharedResourceHandle<FNDIAudio_SharedResource, FNDIAudio_SharedResourceUsage>;

// keyed by the requested Submix (if null, that means the default for each given device)
class FNDIAudio_GeneratedData : public FNDI_GeneratedData
{
public:
	static const ETickingGroup GeneratedDataTickGroup;

	FNDIAudio_GeneratedData() = default;
	virtual ~FNDIAudio_GeneratedData() = default;

	static TypeHash GetTypeHash();

	NIAGARA_API virtual void Tick(ETickingGroup TickGroup, float DeltaSeconds) override;

	using FResourceDesc = FNDIAudio_SharedResource::FResourceKey;
	NIAGARA_API FNDIAudio_SharedResourceHandle GetSharedResource(const FNDIAudio_SharedResourceUsage& Usage, const FResourceDesc& ResourceDesc);

private:
	using FSharedResourceArray = TArray<TSharedPtr<FNDIAudio_SharedResource>>;
	FSharedResourceArray SharedResources;
	FRWLock SharedResourcesLock;
};
