// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGpuComputeDebug.h"
#include "Engine/Engine.h"
#include "NiagaraDebugShaders.h"

#include "CanvasTypes.h"
#include "CommonRenderResources.h"
#include "Engine/Font.h"
#include "RHI.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ScreenPass.h"
#include "SceneView.h"

int32 GNiagaraGpuComputeDebug_MinTextureHeight = 128;
static FAutoConsoleVariableRef CVarNiagaraGpuComputeDebug_MinTextureHeight(
	TEXT("fx.Niagara.GpuComputeDebug.MinTextureHeight"),
	GNiagaraGpuComputeDebug_MinTextureHeight,
	TEXT("The minimum height we will visualize a texture at, smaller textures will be scaled up to match this."),
	ECVF_Default
);

int32 GNiagaraGpuComputeDebug_MaxTextureHeight = 128;
static FAutoConsoleVariableRef CVarNiagaraGpuComputeDebug_MaxTextureHeight(
	TEXT("fx.Niagara.GpuComputeDebug.MaxTextureHeight"),
	GNiagaraGpuComputeDebug_MaxTextureHeight,
	TEXT("The maximum height we will visualize a texture at, this is to avoid things becoming too large on screen."),
	ECVF_Default
);

int32 GNiagaraGpuComputeDebug_MaxLineInstances = 4096;
static FAutoConsoleVariableRef CVarNiagaraGpuComputeDebug_MaxLineInstances(
	TEXT("fx.Niagara.GpuComputeDebug.MaxLineInstances"),
	GNiagaraGpuComputeDebug_MaxLineInstances,
	TEXT("Maximum number of line draw we support in a single frame."),
	ECVF_Default
);

int32 GNiagaraGpuComputeDebug_DrawDebugEnabled = 1;
static FAutoConsoleVariableRef CVarNiagaraGpuComputeDebug_DrawDebugEnabled(
	TEXT("fx.Niagara.GpuComputeDebug.DrawDebugEnabled"),
	GNiagaraGpuComputeDebug_DrawDebugEnabled,
	TEXT("Should we draw any of the debug information or not."),
	ECVF_Default
);

#if NIAGARA_COMPUTEDEBUG_ENABLED

namespace NiagaraGpuComputeDebugLocal
{
	static constexpr uint32 NumUintsPerLine = 7;
}
static_assert(sizeof(FNiagaraSimulationDebugDrawData::FGpuLine) == (NiagaraGpuComputeDebugLocal::NumUintsPerLine * sizeof(uint32)), "Line size does not match expected GPU size");

//////////////////////////////////////////////////////////////////////////

FNiagaraGpuComputeDebug::FNiagaraGpuComputeDebug(ERHIFeatureLevel::Type InFeatureLevel)
	: FeatureLevel(InFeatureLevel)
{
}

void FNiagaraGpuComputeDebug::Tick(FRDGBuilder& GraphBuilder)
{
	for (auto it=DebugDrawBuffers.CreateConstIterator(); it; ++it)
	{
		FNiagaraSimulationDebugDrawData* DebugDrawData = it.Value().Get();
		DebugDrawData->RDGStaticLineBuffer = nullptr;
		DebugDrawData->GpuLineBufferArgs.EndGraphUsage();
		DebugDrawData->GpuLineVertexBuffer.EndGraphUsage();

		if ( !DebugDrawData->bRequiresUpdate )
		{
			continue;
		}

		DebugDrawData->bRequiresUpdate = false;
		if (DebugDrawData->GpuLineBufferArgs.IsValid())
		{
			NiagaraDebugShaders::ClearUAV(GraphBuilder, DebugDrawData->GpuLineBufferArgs.GetOrCreateUAV(GraphBuilder), FUintVector4(2, 0, 0, 0), 4);
		}

		DebugDrawData->StaticLineCount = DebugDrawData->StaticLines.Num();
		if (DebugDrawData->StaticLineCount > 0 )
		{
			const uint32 BufferNumUint32 = NiagaraGpuComputeDebugLocal::NumUintsPerLine * DebugDrawData->StaticLineCount;
			DebugDrawData->RDGStaticLineBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), BufferNumUint32), TEXT("NiagaraGpuComputeDebug::DrawLineVertexBuffer"));
			GraphBuilder.QueueBufferUpload(DebugDrawData->RDGStaticLineBuffer, DebugDrawData->StaticLines.GetData(), BufferNumUint32 * sizeof(uint32));
			GraphBuilder.QueueBufferExtraction(DebugDrawData->RDGStaticLineBuffer, &DebugDrawData->StaticLineBuffer);

			DebugDrawData->StaticLines.Reset();
		}
		else
		{
			DebugDrawData->StaticLineBuffer.SafeRelease();
		}
	}
}

void FNiagaraGpuComputeDebug::AddSystemInstance(FNiagaraSystemInstanceID SystemInstanceID, FString SystemName)
{
	SystemInstancesToWatch.FindOrAdd(SystemInstanceID) = SystemName;
}

void FNiagaraGpuComputeDebug::RemoveSystemInstance(FNiagaraSystemInstanceID SystemInstanceID)
{
	SystemInstancesToWatch.Remove(SystemInstanceID);
	VisualizeTextures.RemoveAll([&SystemInstanceID](const FNiagaraVisualizeTexture& Texture) -> bool { return Texture.SystemInstanceID == SystemInstanceID; });
}

void FNiagaraGpuComputeDebug::OnSystemDeallocated(FNiagaraSystemInstanceID SystemInstanceID)
{
	VisualizeTextures.RemoveAll([&SystemInstanceID](const FNiagaraVisualizeTexture& Texture) -> bool { return Texture.SystemInstanceID == SystemInstanceID; });
	DebugDrawBuffers.Remove(SystemInstanceID);
}

void FNiagaraGpuComputeDebug::AddTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FVector2D PreviewDisplayRange)
{
	AddAttributeTexture(GraphBuilder, SystemInstanceID, SourceName, Texture, FIntPoint::ZeroValue, FIntVector4(INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE), PreviewDisplayRange);
}

void FNiagaraGpuComputeDebug::AddAttributeTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FIntPoint NumTextureAttributes, FIntVector4 AttributeIndices, FVector2D PreviewDisplayRange)
{
	FIntVector4 TextureAttributesInt4 = FIntVector4(NumTextureAttributes.X, NumTextureAttributes.Y, 0, 0);
	AddAttributeTexture(GraphBuilder, SystemInstanceID, SourceName, Texture, TextureAttributesInt4, AttributeIndices, PreviewDisplayRange);
}

void FNiagaraGpuComputeDebug::AddAttributeTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FIntVector4 NumTextureAttributes, FIntVector4 AttributeIndices, FVector2D PreviewDisplayRange)
{
	if (!SystemInstancesToWatch.Contains(SystemInstanceID))
	{
		return;
	}

	if (SourceName.IsNone() || (Texture == nullptr))
	{
		return;
	}

	const FRDGTextureDesc& TextureDesc = Texture->Desc;

	bool bCreateTexture = false;

	FNiagaraVisualizeTexture* VisualizeEntry = VisualizeTextures.FindByPredicate([&SourceName, &SystemInstanceID](const FNiagaraVisualizeTexture& Texture) -> bool { return Texture.SystemInstanceID == SystemInstanceID && Texture.SourceName == SourceName; });
	if (!VisualizeEntry)
	{
		VisualizeEntry = &VisualizeTextures.AddDefaulted_GetRef();
		VisualizeEntry->SystemInstanceID = SystemInstanceID;
		VisualizeEntry->SourceName = SourceName;
		bCreateTexture = true;
	}
	else
	{
		const FPooledRenderTargetDesc& EntryTextureDesc = VisualizeEntry->Texture->GetDesc();
		bCreateTexture = 
			(EntryTextureDesc.GetSize() != TextureDesc.GetSize()) ||
			(EntryTextureDesc.Format != TextureDesc.Format) ||
			(EntryTextureDesc.NumMips != TextureDesc.NumMips) ||
			(EntryTextureDesc.ArraySize != TextureDesc.ArraySize);
	}
	VisualizeEntry->NumTextureAttributes = NumTextureAttributes;
	VisualizeEntry->AttributesToVisualize = AttributeIndices;
	VisualizeEntry->PreviewDisplayRange = PreviewDisplayRange;

	// Do we need to create a texture to copy into?
	if ( bCreateTexture )
	{
		// Create a minimal copy of the Texture's Desc
		const FRHITextureCreateDesc NewTextureDesc =
			FRHITextureCreateDesc(TEXT("FNiagaraGpuComputeDebug"), TextureDesc.Dimension)
			.SetExtent(TextureDesc.Extent)
			.SetNumMips(TextureDesc.NumMips)
			.SetDepth(TextureDesc.Depth)
			.SetArraySize(TextureDesc.ArraySize)
			.SetFormat(TextureDesc.Format)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask);

		VisualizeEntry->Texture = CreateRenderTarget(GraphBuilder.RHICmdList.CreateTexture(NewTextureDesc), TEXT("FNiagaraGpuComputeDebug"));
	}
	check(VisualizeEntry->Texture.IsValid());

	FRHICopyTextureInfo CopyInfo;
	CopyInfo.NumMips = TextureDesc.NumMips;
	CopyInfo.NumSlices = TextureDesc.IsTextureCube() ? 6 : TextureDesc.ArraySize;
	AddCopyTexturePass(GraphBuilder, Texture, GraphBuilder.RegisterExternalTexture(VisualizeEntry->Texture), CopyInfo);
}

FNiagaraSimulationDebugDrawData* FNiagaraGpuComputeDebug::GetSimulationDebugDrawData(FNiagaraSystemInstanceID SystemInstanceID)
{
	check(IsInRenderingThread());

	TUniquePtr<FNiagaraSimulationDebugDrawData>& DebugDrawData = DebugDrawBuffers.FindOrAdd(SystemInstanceID);
	if (!DebugDrawData.IsValid())
	{
		DebugDrawData.Reset(new FNiagaraSimulationDebugDrawData());
	}

	return DebugDrawData.Get();
}

FNiagaraSimulationDebugDrawData* FNiagaraGpuComputeDebug::GetSimulationDebugDrawData(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, uint32 OverrideMaxDebugLines)
{
	check(IsInRenderingThread());

	FNiagaraSimulationDebugDrawData* DebugDrawData = GetSimulationDebugDrawData(SystemInstanceID);
	const uint64 MaxViewDimensionForTypedBuffer = GRHIGlobals.MaxViewDimensionForTypedBuffer;
	const uint32 MaxPossibleLines = uint32(FMath::Min(FMath::DivideAndRoundDown(MaxViewDimensionForTypedBuffer, uint64(NiagaraGpuComputeDebugLocal::NumUintsPerLine)), uint64(TNumericLimits<uint32>::Max() >> 1)));
	const uint32 MaxLineInstancesToUse = FMath::Min(FMath::Max(DebugDrawData->GpuLineMaxInstances, OverrideMaxDebugLines > 0 ? OverrideMaxDebugLines : uint32(GNiagaraGpuComputeDebug_MaxLineInstances)), MaxPossibleLines);
	if (DebugDrawData->GpuLineMaxInstances != MaxLineInstancesToUse)
	{
		DebugDrawData->GpuLineBufferArgs.Release();
		DebugDrawData->GpuLineVertexBuffer.Release();

		if (MaxLineInstancesToUse > 0)
		{
			DebugDrawData->GpuLineMaxInstances = MaxLineInstancesToUse;

			DebugDrawData->GpuLineBufferArgs.Initialize(GraphBuilder, TEXT("NiagaraGpuComputeDebug::DrawLineBufferArgs"), PF_R32_UINT, sizeof(uint32), sizeof(FRHIDrawIndirectParameters) / sizeof(uint32), EBufferUsageFlags::DrawIndirect);
			DebugDrawData->GpuLineVertexBuffer.Initialize(GraphBuilder, TEXT("NiagaraGpuComputeDebug::DrawLineVertexBuffer"), PF_R32_UINT, sizeof(uint32), NiagaraGpuComputeDebugLocal::NumUintsPerLine * DebugDrawData->GpuLineMaxInstances);

			NiagaraDebugShaders::ClearUAV(GraphBuilder, DebugDrawData->GpuLineBufferArgs.GetOrCreateUAV(GraphBuilder), FUintVector4(2, 0, 0, 0), 4);
		}
	}

	return DebugDrawData;
}

void FNiagaraGpuComputeDebug::RemoveSimulationDebugDrawData(FNiagaraSystemInstanceID SystemInstanceID)
{
	DebugDrawBuffers.Remove(SystemInstanceID);
}

bool FNiagaraGpuComputeDebug::ShouldDrawDebug() const
{
	return GNiagaraGpuComputeDebug_DrawDebugEnabled && (VisualizeTextures.Num() > 0);
}

void FNiagaraGpuComputeDebug::DrawDebug(FRDGBuilder& GraphBuilder, const FSceneView& View, const FScreenPassRenderTarget& Output)
{
	if (!GNiagaraGpuComputeDebug_DrawDebugEnabled || (VisualizeTextures.Num() == 0))
	{
		return;
	}

	++TickCounter;

	const UFont* Font = GEngine->GetTinyFont();
	const int32 FontHeight = FMath::CeilToInt(Font->GetMaxCharHeight());

	FIntPoint Location(10, Output.ViewRect.Height() - 10);

	const int32 DisplayMinHeight = GNiagaraGpuComputeDebug_MinTextureHeight > 0 ? GNiagaraGpuComputeDebug_MinTextureHeight : 0;
	const int32 DisplayMaxHeight = GNiagaraGpuComputeDebug_MaxTextureHeight > 0 ? GNiagaraGpuComputeDebug_MaxTextureHeight : TNumericLimits<int32>::Max();

	for (const FNiagaraVisualizeTexture& VisualizeEntry : VisualizeTextures)
	{
		FIntVector TextureSize = VisualizeEntry.Texture->GetDesc().GetSize();
		if ( VisualizeEntry.NumTextureAttributes.X > 0 )
		{
			check(VisualizeEntry.NumTextureAttributes.Y > 0);
			TextureSize.X /= VisualizeEntry.NumTextureAttributes.X;
			TextureSize.Y /= VisualizeEntry.NumTextureAttributes.Y;
		}

		// Get system name
		const FString& SystemName = SystemInstancesToWatch.FindRef(VisualizeEntry.SystemInstanceID);

		const int32 DisplayHeight = FMath::Clamp(TextureSize.Y, DisplayMinHeight, DisplayMaxHeight);

		Location.Y -= DisplayHeight;

		NiagaraDebugShaders::VisualizeTexture(GraphBuilder, View, Output, Location, DisplayHeight, VisualizeEntry.AttributesToVisualize, GraphBuilder.RegisterExternalTexture(VisualizeEntry.Texture), VisualizeEntry.NumTextureAttributes, TickCounter, VisualizeEntry.PreviewDisplayRange);

		Location.Y -= FontHeight;

		AddDrawCanvasPass(GraphBuilder, {}, View, Output,
			[Location, SourceName=VisualizeEntry.SourceName.ToString(), SystemName, Font](FCanvas& Canvas)
			{
				Canvas.DrawShadowedString(Location.X, Location.Y, *FString::Printf(TEXT("DataInterface: %s, System: %s"), *SourceName, *SystemName), Font, FLinearColor(1, 1, 1));
			}
		);

		Location.Y -= 1;
	}
}

void FNiagaraGpuComputeDebug::DrawSceneDebug(FRDGBuilder& GraphBuilder, const FSceneView& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth)
{
	if (!GNiagaraGpuComputeDebug_DrawDebugEnabled)
	{
		return;
	}

	for (auto it = DebugDrawBuffers.CreateConstIterator(); it; ++it)
	{
		FNiagaraSimulationDebugDrawData* DebugDrawData = it.Value().Get();
		if (DebugDrawData->StaticLineCount > 0)
		{
			if (DebugDrawData->RDGStaticLineBuffer == nullptr)
			{
				DebugDrawData->RDGStaticLineBuffer = GraphBuilder.RegisterExternalBuffer(DebugDrawData->StaticLineBuffer);
			}

			NiagaraDebugShaders::DrawDebugLines(
				GraphBuilder, View, SceneColor, SceneDepth,
				DebugDrawData->StaticLineCount,
				DebugDrawData->RDGStaticLineBuffer
			);
		}
		if (DebugDrawData->GpuLineMaxInstances > 0)
		{
			NiagaraDebugShaders::DrawDebugLines(
				GraphBuilder, View, SceneColor, SceneDepth,
				DebugDrawData->GpuLineBufferArgs.GetOrCreateBuffer(GraphBuilder),
				DebugDrawData->GpuLineVertexBuffer.GetOrCreateBuffer(GraphBuilder)
			);
		}
	}
}

#endif //NIAGARA_COMPUTEDEBUG_ENABLED
