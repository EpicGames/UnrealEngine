// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
Class used help realtime debug Gpu Compute simulations
==============================================================================*/
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraRenderGraphUtils.h"
#include "RenderGraphDefinitions.h"

class FSceneView;

#if NIAGARA_COMPUTEDEBUG_ENABLED

struct FNiagaraSimulationDebugDrawData
{
	struct FGpuLine
	{
		FVector3f	Start;
		FVector3f	End;
		uint32		Color;
	};

	bool				bRequiresUpdate = true;
	int32				LastUpdateTickCount = INDEX_NONE;

	TArray<FGpuLine>				StaticLines;
	uint32							StaticLineCount = 0;
	TRefCountPtr<FRDGPooledBuffer>	StaticLineBuffer;
	FRDGBufferRef					RDGStaticLineBuffer = nullptr;

	uint32							GpuLineMaxInstances = 0;
	FNiagaraPooledRWBuffer			GpuLineBufferArgs;
	FNiagaraPooledRWBuffer			GpuLineVertexBuffer;
};

class FNiagaraGpuComputeDebug
{
public:
	struct FNiagaraVisualizeTexture
	{
		FNiagaraSystemInstanceID			SystemInstanceID;
		FName								SourceName;
		TRefCountPtr<IPooledRenderTarget>	Texture;
		FIntVector4							NumTextureAttributes = FIntVector4(0,0,0,0);
		FIntVector4							AttributesToVisualize = FIntVector4(-1, -1, -1, -1);
		FVector2D							PreviewDisplayRange = FVector2D::ZeroVector;
	};

	FNiagaraGpuComputeDebug(ERHIFeatureLevel::Type InFeatureLevel);

	// Called at the start of the frame
	void Tick(FRDGBuilder& GraphBuilder);

	// Enables providing debug information for the system instance
	void AddSystemInstance(FNiagaraSystemInstanceID SystemInstanceID, FString SystemName);
	// Disabled providing debug information for the system instance
	void RemoveSystemInstance(FNiagaraSystemInstanceID SystemInstanceID);

	// Notification from the batcher when a system instance has been removed, i.e. the system was reset
	void OnSystemDeallocated(FNiagaraSystemInstanceID SystemInstanceID);

	// Add a texture to visualize
	void AddTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FVector2D PreviewDisplayRange = FVector2D::ZeroVector);

	// Add a texture to visualize that contains a number of attributes and select which attributes to push into RGBA where -1 means ignore that channel
	// The first -1 in the attribute indices list will also limit the number of attributes we attempt to read.
	// NumTextureAttributes in this version is meant for a 2D atlas
	void AddAttributeTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FIntPoint NumTextureAttributes, FIntVector4 AttributeIndices, FVector2D PreviewDisplayRange = FVector2D::ZeroVector);

	// Add a texture to visualize that contains a number of attributes and select which attributes to push into RGBA where -1 means ignore that channel
	// The first -1 in the attribute indices list will also limit the number of attributes we attempt to read
	// NumTextureAttributes in this version is meant for a 3D atlas
	void AddAttributeTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FIntVector4 NumTextureAttributes, FIntVector4 AttributeIndices, FVector2D PreviewDisplayRange = FVector2D::ZeroVector);

	// Get Debug draw buffers for a system instance
	FNiagaraSimulationDebugDrawData* GetSimulationDebugDrawData(FNiagaraSystemInstanceID SystemInstanceID);
	FNiagaraSimulationDebugDrawData* GetSimulationDebugDrawData(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, uint32 OverrideMaxDebugLines);

	// Force remove debug draw data
	void RemoveSimulationDebugDrawData(FNiagaraSystemInstanceID SystemInstanceID);

	// Do we need DrawDebug to be called?
	bool ShouldDrawDebug() const;

	// Draw all the debug information for the system
	void DrawDebug(FRDGBuilder& GraphBuilder, const FSceneView& View, const struct FScreenPassRenderTarget& Output);

	// Draw debug information that requires rendering into the scene
	void DrawSceneDebug(FRDGBuilder& GraphBuilder, const FSceneView& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth);

private:
	ERHIFeatureLevel::Type FeatureLevel;
	uint32 TickCounter = 0;
	TArray<FNiagaraVisualizeTexture> VisualizeTextures;
	TMap<FNiagaraSystemInstanceID, TUniquePtr<FNiagaraSimulationDebugDrawData>> DebugDrawBuffers;
	TMap<FNiagaraSystemInstanceID, FString> SystemInstancesToWatch;
};

#endif //NIAGARA_COMPUTEDEBUG_ENABLED
