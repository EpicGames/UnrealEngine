// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGpuComputeDebugInterface.h"
#include "NiagaraGpuComputeDebug.h"

#if NIAGARA_COMPUTEDEBUG_ENABLED

void FNiagaraGpuComputeDebugInterface::AddTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FVector2D PreviewDisplayRange)
{
	if (ComputeDebug != nullptr)
	{
		ComputeDebug->AddTexture(GraphBuilder, SystemInstanceID, SourceName, Texture, PreviewDisplayRange);
	}
}

void FNiagaraGpuComputeDebugInterface::AddAttributeTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FIntPoint NumTextureAttributes, FIntVector4 AttributeIndices, FVector2D PreviewDisplayRange)
{
	if (ComputeDebug != nullptr)
	{
		ComputeDebug->AddAttributeTexture(GraphBuilder, SystemInstanceID, SourceName, Texture, NumTextureAttributes, AttributeIndices, PreviewDisplayRange);
	}
}

void FNiagaraGpuComputeDebugInterface::AddAttributeTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FIntVector4 NumTextureAttributes, FIntVector4 AttributeIndices, FVector2D PreviewDisplayRange)
{
	if (ComputeDebug != nullptr )
	{
		ComputeDebug->AddAttributeTexture(GraphBuilder, SystemInstanceID, SourceName, Texture, NumTextureAttributes, AttributeIndices, PreviewDisplayRange);
	}
}

#endif //NIAGARA_COMPUTEDEBUG_ENABLED
