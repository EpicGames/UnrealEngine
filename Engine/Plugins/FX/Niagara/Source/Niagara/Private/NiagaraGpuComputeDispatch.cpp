// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGpuComputeDispatch.h"

#include "Async/Async.h"
#include "CanvasTypes.h"
#include "EngineModule.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "EngineModule.h"
#include "Materials/MaterialRenderProxy.h"
#include "Misc/ScopeExit.h"
#include "NiagaraGPUSortInfo.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include "NiagaraAsyncGpuTraceHelper.h"
#include "NiagaraDataInterfaceRW.h"
#if NIAGARA_COMPUTEDEBUG_ENABLED
#include "NiagaraGpuComputeDebug.h"
#endif
#include "NiagaraGPUProfilerInterface.h"
#include "NiagaraGpuReadbackManager.h"
#include "NiagaraRenderer.h"
#include "NiagaraScript.h"
#include "NiagaraShader.h"
#include "NiagaraShaderParticleID.h"
#include "NiagaraSortingGPU.h"
#include "NiagaraStats.h"
#include "NiagaraSystemGpuComputeProxy.h"
#include "NiagaraWorldManager.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "PipelineStateCache.h"
#include "RHI.h"
#include "SceneInterface.h"
#include "SceneRenderTargetParameters.h"
#include "TextureResource.h"
#include "FXRenderingUtils.h"
#include "Rendering/RenderCommandPipes.h"
#include "GPUSkinCache.h"
#include "SkeletalMeshUpdater.h"

#include "NiagaraDataChannelData.h"

DECLARE_CYCLE_STAT(TEXT("GPU Dispatch Setup [RT]"), STAT_NiagaraGPUDispatchSetup_RT, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("GPU Emitter Dispatch [RT]"), STAT_NiagaraGPUSimTick_RT, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("GPU Data Readback [RT]"), STAT_NiagaraGPUReadback_RT, STATGROUP_Niagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("# GPU Particles"), STAT_NiagaraGPUParticles, STATGROUP_Niagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("# GPU Sorted Particles"), STAT_NiagaraGPUSortedParticles, STATGROUP_Niagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("# GPU Sorted Buffers"), STAT_NiagaraGPUSortedBuffers, STATGROUP_Niagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("Readback latency (frames)"), STAT_NiagaraReadbackLatency, STATGROUP_Niagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("# GPU Dispatches"), STAT_NiagaraGPUDispatches, STATGROUP_Niagara);

DECLARE_GPU_STAT_NAMED(NiagaraGPU, TEXT("Niagara"));
DECLARE_GPU_STAT_NAMED(NiagaraGPUSimulation, TEXT("Niagara GPU Simulation"));
DECLARE_GPU_STAT_NAMED(NiagaraGPUClearIDTables, TEXT("NiagaraGPU Clear ID Tables"));
DECLARE_GPU_STAT_NAMED(NiagaraGPUComputeFreeIDs, TEXT("Niagara GPU Compute All Free IDs"));
DECLARE_GPU_STAT_NAMED(NiagaraGPUComputeFreeIDsEmitter, TEXT("Niagara GPU Compute Emitter Free IDs"));
DECLARE_GPU_STAT_NAMED(NiagaraGPUSorting, TEXT("Niagara GPU sorting"));

CSV_DEFINE_CATEGORY_MODULE(NIAGARA_API, NiagaraGpuCompute, true);

uint32 FNiagaraComputeExecutionContext::TickCounter = 0;

int32 GNiagaraGpuSubmitCommandHint = 0;
static FAutoConsoleVariableRef CVarNiagaraGpuSubmitCommandHint(
	TEXT("fx.NiagaraGpuSubmitCommandHint"),
	GNiagaraGpuSubmitCommandHint,
	TEXT("If greater than zero, we use this value to submit commands after the number of dispatches have been issued."),
	ECVF_Default
);

int32 GNiagaraGpuLowLatencyTranslucencyEnabled = 1;
static FAutoConsoleVariableRef CVarNiagaraGpuLowLatencyTranslucencyEnabled(
	TEXT("fx.NiagaraGpuLowLatencyTranslucencyEnabled"),
	GNiagaraGpuLowLatencyTranslucencyEnabled,
	TEXT("When enabled translucent materials can use the current frames simulation data no matter which tick pass Niagara uses.\n")
	TEXT("This can result in an additional data buffer being required but will reduce any latency when using view uniform buffer / depth buffer / distance fields / etc"),
	ECVF_Default
);

int32 GNiagaraBatcherFreeBufferEarly = 1;
static FAutoConsoleVariableRef CVarNiagaraBatcherFreeBufferEarly(
	TEXT("fx.NiagaraBatcher.FreeBufferEarly"),
	GNiagaraBatcherFreeBufferEarly,
	TEXT("Will take the path to release GPU buffers when possible.\n")
	TEXT("This will reduce memory pressure but can result in more allocations if you buffers ping pong from zero particles to many."),
	ECVF_Default
);

#if WITH_MGPU
int32 GNiagaraOptimizedCrossGPUTransferEnabled = 1;
static FAutoConsoleVariableRef CVarNiagaraOptimizedCrossGPUTransferEnabled(
	TEXT("fx.NiagaraOptimizeCrossGPUTransfer"),
	GNiagaraOptimizedCrossGPUTransferEnabled,
	TEXT("Optimizes fence waits for cross GPU transfers when rendering views on multiple GPUs via nDisplay.  (Default = 1)"),
	ECVF_Default
);
#endif

const FName FNiagaraGpuComputeDispatch::Name(TEXT("FNiagaraGpuComputeDispatch"));

namespace FNiagaraGpuComputeDispatchLocal
{
	int32 GTickFlushMaxQueuedFrames = 3;
	static FAutoConsoleVariableRef CVarNiagaraTickFlushMaxQueuedFrames(
		TEXT("fx.Niagara.Batcher.TickFlush.MaxQueuedFrames"),
		GTickFlushMaxQueuedFrames,
		TEXT("The number of unprocessed frames with queued ticks before we process them.\n")
		TEXT("The larger the number the more data we process in a single frame, this is generally only a concern when the application does not have focus."),
		ECVF_Default
	);

	int32 GTickFlushMaxPendingTicks = 10;
	static FAutoConsoleVariableRef CVarNiagaraTickFlushMaxPendingTicks(
		TEXT("fx.Niagara.Batcher.TickFlush.MaxPendingTicks"),
		GTickFlushMaxPendingTicks,
		TEXT("The maximum number of unprocess ticks before we process them.\n")
		TEXT("The larger the number the more data we process in a single frame."),
		ECVF_Default
	);

	int32 GTickFlushMode = 1;
	static FAutoConsoleVariableRef CVarNiagaraTickFlushMode(
		TEXT("fx.Niagara.Batcher.TickFlush.Mode"),
		GTickFlushMode,
		TEXT("What to do when we go over our max queued frames.\n")
		TEXT("0 = Keep ticks queued, can result in a long pause when gaining focus again.\n")
		TEXT("1 = (Default) Process all queued ticks with dummy view / buffer data, may result in incorrect simulation due to missing depth collisions, etc.\n")
		TEXT("2 = Kill all pending ticks, may result in incorrect simulation due to missing frames of data, i.e. a particle reset.\n"),
		ECVF_Default
	);

	#if !WITH_EDITOR
		constexpr int32 GDebugLogging = 0;
	#else
		static int32 GDebugLogging = 0;
		static FAutoConsoleVariableRef CVarNiagaraDebugLogging(
			TEXT("fx.Niagara.Batcher.DebugLogging"),
			GDebugLogging,
			TEXT("Enables a lot of spew to the log to debug the batcher."),
			ECVF_Default
		);
	#endif

	#if STATS
		static int32 GDumpSortedSimulations = 0;
		static FAutoConsoleVariableRef CVarDumpSortedSimulations(
			TEXT("fx.Niagara.Batcher.DumpSortedSimulations"),
			GDumpSortedSimulations,
			TEXT("Will dump sorted simulations to the log for the provided number of frames."),
			ECVF_Default
		);
	#endif

	template<typename TTransitionArrayType>
	static void AddDataBufferTransitions(TTransitionArrayType& BeforeTransitionArray, TTransitionArrayType& AfterTransitionArray, FNiagaraDataBuffer* DestinationData, ERHIAccess BeforeState = ERHIAccess::SRVMask, ERHIAccess AfterState = ERHIAccess::UAVCompute)
	{
		if (FRHIUnorderedAccessView* FloatUAV = DestinationData->GetGPUBufferFloat().UAV )
		{
			BeforeTransitionArray.Emplace(FloatUAV, BeforeState, AfterState);
			AfterTransitionArray.Emplace(FloatUAV, AfterState, BeforeState);
		}
		if (FRHIUnorderedAccessView* HalfUAV = DestinationData->GetGPUBufferHalf().UAV)
		{
			BeforeTransitionArray.Emplace(HalfUAV, BeforeState, AfterState);
			AfterTransitionArray.Emplace(HalfUAV, AfterState, BeforeState);
		}
		if (FRHIUnorderedAccessView* IntUAV = DestinationData->GetGPUBufferInt().UAV)
		{
			BeforeTransitionArray.Emplace(IntUAV, BeforeState, AfterState);
			AfterTransitionArray.Emplace(IntUAV, AfterState, BeforeState);
		}
	}

	static bool CsvStatsEnabled()
	{
	#if WITH_PARTICLE_PERF_CSV_STATS
		static IConsoleVariable* DetailedCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.DetailedCSVStats"));
		return DetailedCVar && DetailedCVar->GetBool();
	#else
		return false;
	#endif
	}
}

//////////////////////////////////////////////////////////////////////////

FFXSystemInterface* FNiagaraGpuComputeDispatch::GetInterface(const FName& InName)
{
	return InName == Name ? this : nullptr;
}

FNiagaraGpuComputeDispatch::FNiagaraGpuComputeDispatch(ERHIFeatureLevel::Type InFeatureLevel, EShaderPlatform InShaderPlatform, FGPUSortManager* InGPUSortManager)
	: FNiagaraGpuComputeDispatchInterface(InShaderPlatform, InFeatureLevel)
	, GPUSortManager(InGPUSortManager)
{
	// Register the batcher callback in the GPUSortManager.
	// The callback is used to generate the initial keys and values for the GPU sort tasks,
	// the values being the sorted particle indices used by the Niagara renderers.
	// The registration also involves defining the list of flags possibly used in GPUSortManager::AddTask()
	if (GPUSortManager)
	{
		GPUSortManager->Register(
			FGPUSortKeyGenDelegate::CreateLambda(
				[this](FRHICommandListImmediate& RHICmdList, int32 BatchId, int32 NumElementsInBatch, EGPUSortFlags Flags, FRHIUnorderedAccessView* KeysUAV, FRHIUnorderedAccessView* ValuesUAV)
				{
					GenerateSortKeys(RHICmdList, BatchId, NumElementsInBatch, Flags, KeysUAV, ValuesUAV);
				}
			),
			EGPUSortFlags::AnyKeyPrecision | EGPUSortFlags::AnyKeyGenLocation | EGPUSortFlags::AnySortLocation | EGPUSortFlags::ValuesAsInt32,
			Name
		);

		if (FNiagaraUtilities::AllowComputeShaders(GetShaderPlatform()))
		{
			// Because of culled indirect draw args, we have to update the draw indirect buffer after the sort key generation
			GPUSortManager->PostPreRenderEvent.AddLambda(
				[this](FRHICommandListImmediate& RHICmdList)
				{
					GPUInstanceCounterManager.UpdateDrawIndirectBuffers(this, RHICmdList, ENiagaraGPUCountUpdatePhase::PreOpaque);
				}
			);

			GPUSortManager->PostPostRenderEvent.AddLambda
			(
				[this](FRHICommandListImmediate& RHICmdList)
				{
					GPUInstanceCounterManager.UpdateDrawIndirectBuffers(this, RHICmdList, ENiagaraGPUCountUpdatePhase::PostOpaque);
				#if WITH_MGPU
					TransferMultiGPUBuffers(RHICmdList);
				#endif // WITH_MGPU
				}
			);
		}
	}

	AsyncGpuTraceHelper.Reset(new FNiagaraAsyncGpuTraceHelper(InShaderPlatform, InFeatureLevel, this));

#if NIAGARA_COMPUTEDEBUG_ENABLED
	GpuComputeDebugPtr.Reset(new FNiagaraGpuComputeDebug(FeatureLevel));
#endif
#if WITH_NIAGARA_GPU_PROFILER
	GPUProfilerPtr = MakeUnique<FNiagaraGPUProfiler>(uintptr_t(static_cast<FNiagaraGpuComputeDispatchInterface*>(this)));
#endif
	GpuReadbackManagerPtr.Reset(new FNiagaraGpuReadbackManager());
	EmptyUAVPoolPtr.Reset(new FNiagaraEmptyUAVPool());
}

FNiagaraGpuComputeDispatch::~FNiagaraGpuComputeDispatch()
{
	FinishDispatches();

	AsyncGpuTraceHelper->Reset();

	for (auto It = NDCDataProxies.CreateIterator(); It; ++It)
	{
		FNiagaraDataChannelDataProxyPtr& NDCProxy = *It;
		NDCProxy->OnRemovedFromDispatcher(this);
	}
	NDCDataProxies.Empty();
}

void FNiagaraGpuComputeDispatch::AddGpuComputeProxy(FNiagaraSystemGpuComputeProxy* ComputeProxy)
{
	check(ComputeProxy->ComputeDispatchIndex == INDEX_NONE);

	const ENiagaraGpuComputeTickStage::Type TickStage = ComputeProxy->GetComputeTickStage();
	ComputeProxy->ComputeDispatchIndex = ProxiesPerStage[TickStage].Num();
	ProxiesPerStage[TickStage].Add(ComputeProxy);

	NumProxiesThatRequireGlobalDistanceField	+= ComputeProxy->RequiresGlobalDistanceField() ? 1 : 0;
	NumProxiesThatRequireDepthBuffer			+= ComputeProxy->RequiresDepthBuffer() ? 1 : 0;
	NumProxiesThatRequireEarlyViewData			+= ComputeProxy->RequiresEarlyViewData() ? 1 : 0;
	NumProxiesThatRequireRayTracingScene		+= ComputeProxy->RequiresRayTracingScene() ? 1 : 0;
	NumProxiesThatRequireCurrentFrameNDC		+= ComputeProxy->RequiresCurrentFrameNDC() ? 1 : 0;
	ProxyGpuCountBufferEstimate					+= ComputeProxy->GetGpuCountBufferEstimate();
}

void FNiagaraGpuComputeDispatch::RemoveGpuComputeProxy(FNiagaraSystemGpuComputeProxy* ComputeProxy)
{
	check(ComputeProxy->ComputeDispatchIndex != INDEX_NONE);

	const int32 TickStage = int32(ComputeProxy->GetComputeTickStage());
	const int32 ProxyIndex = ComputeProxy->ComputeDispatchIndex;
	check(ProxiesPerStage[TickStage][ProxyIndex] == ComputeProxy);

	ProxiesPerStage[TickStage].RemoveAtSwap(ProxyIndex);
	if (ProxiesPerStage[TickStage].IsValidIndex(ProxyIndex))
	{
		ProxiesPerStage[TickStage][ProxyIndex]->ComputeDispatchIndex = ProxyIndex;
	}
	ComputeProxy->ComputeDispatchIndex = INDEX_NONE;

	NumProxiesThatRequireGlobalDistanceField	-= ComputeProxy->RequiresGlobalDistanceField() ? 1 : 0;
	NumProxiesThatRequireDepthBuffer			-= ComputeProxy->RequiresDepthBuffer() ? 1 : 0;
	NumProxiesThatRequireEarlyViewData			-= ComputeProxy->RequiresEarlyViewData() ? 1 : 0;
	NumProxiesThatRequireRayTracingScene		-= ComputeProxy->RequiresRayTracingScene() ? 1 : 0;
	NumProxiesThatRequireCurrentFrameNDC		-= ComputeProxy->RequiresCurrentFrameNDC() ? 1 : 0;	

	const uint32 GpuCountEstimate = ComputeProxy->GetGpuCountBufferEstimate();
	if (ensure(GpuCountEstimate <= ProxyGpuCountBufferEstimate))
	{
		ProxyGpuCountBufferEstimate				-= GpuCountEstimate;
	}
	else
	{
		ProxyGpuCountBufferEstimate				= 0;
	}

#if NIAGARA_COMPUTEDEBUG_ENABLED
	if (FNiagaraGpuComputeDebug* GpuComputeDebug = GpuComputeDebugPtr.Get())
	{
		GpuComputeDebug->OnSystemDeallocated(ComputeProxy->GetSystemInstanceID());
	}
#endif
#if !UE_BUILD_SHIPPING
	GpuDebugReadbackInfos.RemoveAll(
		[&](const FDebugReadbackInfo& Info)
		{
			// In the unlikely event we have one in the queue make sure it's marked as complete with no data in it
			if ( Info.InstanceID == ComputeProxy->GetSystemInstanceID())
			{
				Info.DebugInfo->Frame.CopyFromGPUReadback(nullptr, nullptr, nullptr, 0, 0, 0, 0, 0);
				Info.DebugInfo->bWritten = true;
			}
			return Info.InstanceID == ComputeProxy->GetSystemInstanceID();
		}
	);
#endif
}

void FNiagaraGpuComputeDispatch::AddNDCDataProxy(FNiagaraDataChannelDataProxyPtr NDCDataProxy)
{
	NDCDataProxy->OnAddedToDispatcher(this);
	NDCDataProxies.Add(NDCDataProxy);
}

void FNiagaraGpuComputeDispatch::RemoveNDCDataProxy(FNiagaraDataChannelDataProxyPtr NDCDataProxy)
{	
	NDCDataProxy->OnRemovedFromDispatcher(this);
	NDCDataProxies.RemoveSwap(NDCDataProxy);
}

void FNiagaraGpuComputeDispatch::Tick(UWorld* World, float DeltaTime)
{
	check(IsInGameThread());
	UE::RenderCommandPipe::FSyncScope Scope(MakeArrayView({ &UE::RenderCommandPipe::SkeletalMesh }));
	ENQUEUE_RENDER_COMMAND(NiagaraPumpBatcher)(
		[RT_NiagaraBatcher=this](FRHICommandListImmediate& RHICmdList)
		{
			RT_NiagaraBatcher->ProcessPendingTicksFlush(RHICmdList, false);
			RT_NiagaraBatcher->GetGPUInstanceCounterManager().FlushIndirectArgsPool(RHICmdList);
		}
	);
}

void FNiagaraGpuComputeDispatch::FlushPendingTicks_GameThread()
{
	check(IsInGameThread());
	UE::RenderCommandPipe::FSyncScope Scope(MakeArrayView({ &UE::RenderCommandPipe::SkeletalMesh }));
	ENQUEUE_RENDER_COMMAND(NiagaraFlushPendingTicks)(
		[RT_NiagaraBatcher=this](FRHICommandListImmediate& RHICmdList)
		{
			RT_NiagaraBatcher->ProcessPendingTicksFlush(RHICmdList, true);
			RT_NiagaraBatcher->GetGPUInstanceCounterManager().FlushIndirectArgsPool(RHICmdList);
		}
	);
}

void FNiagaraGpuComputeDispatch::FlushAndWait_GameThread()
{
	check(IsInGameThread());
	UE::RenderCommandPipe::FSyncScope Scope(MakeArrayView({ &UE::RenderCommandPipe::SkeletalMesh }));
	ENQUEUE_RENDER_COMMAND(NiagaraFlushPendingTicks)(
		[RT_NiagaraBatcher=this](FRHICommandListImmediate& RHICmdList)
		{
			RT_NiagaraBatcher->ProcessPendingTicksFlush(RHICmdList, true);
			RT_NiagaraBatcher->GetGPUInstanceCounterManager().FlushIndirectArgsPool(RHICmdList);
			RT_NiagaraBatcher->GetGpuReadbackManager()->WaitCompletion(RHICmdList);
		}
	);
	FlushRenderingCommands();
}

void FNiagaraGpuComputeDispatch::ProcessPendingTicksFlush(FRHICommandListImmediate& RHICmdList, bool bForceFlush)
{
	// Test to see if we have any proxies, if not we have nothing to do
	bool bHasProxies = false;
	for ( int iTickStage=0; iTickStage < ENiagaraGpuComputeTickStage::Max; ++iTickStage)
	{
		if ( ProxiesPerStage[iTickStage].Num() > 0 )
		{
			bHasProxies = true;
			break;
		}
	}

	if ( !bHasProxies)
	{
		return;
	}

	// Do we need to force a flush because we crossed the frame count threshold?
	++FramesBeforeTickFlush;
	bForceFlush |= FramesBeforeTickFlush >= uint32(FMath::Max(0, FNiagaraGpuComputeDispatchLocal::GTickFlushMaxQueuedFrames));

	// Do we need to force a flush because we crossed the max pending ticks on a single instance threashold?
	//-OPT: Ideally we don't traverse this every frame but it's required if we need to flush as we may need to do so in batches to avoid an RDG limit
	const int32 TickFlushMaxPendingTicks = FMath::Max(FNiagaraGpuComputeDispatchLocal::GTickFlushMaxPendingTicks, 1);
	int32 MaxPendingTicks = 0;
	for (int iTickStage = 0; iTickStage < ENiagaraGpuComputeTickStage::Max; ++iTickStage)
	{
		for (FNiagaraSystemGpuComputeProxy* Proxy : ProxiesPerStage[iTickStage])
		{
			MaxPendingTicks = FMath::Max<int32>(MaxPendingTicks, Proxy->PendingTicks.Num());
		}
	}
	bForceFlush |= MaxPendingTicks >= TickFlushMaxPendingTicks;

	// Do we need to execute the flush?
	if (bForceFlush == false)
	{
		return;
	}

	// A tick flush was request reset the counter and execute according to the mode
	FramesBeforeTickFlush = 0;
	switch (FNiagaraGpuComputeDispatchLocal::GTickFlushMode)
	{
		// Do nothing
		default:
		case 0:
		{
			//UE_LOGF(LogNiagara, Log, "FNiagaraGpuComputeDispatch: Queued ticks (%d) are building up, this may cause a stall when released.", Ticks_RT.Num());
			break;
		}

		// Process all the pending ticks that have built up
		case 1:
		{
			//UE_LOGF(LogNiagara, Log, "FNiagaraGpuComputeDispatch: Queued ticks are being Processed due to not rendering.  This may result in undesirable simulation artifacts.");

			// Early out if we have no pending ticks to process
			if (MaxPendingTicks == 0)
			{
				GpuReadbackManagerPtr->Tick(RHICmdList);
				return;
			}

			// Ensure any deferred updates are flushed out
			FDeferredUpdateResource::UpdateResources(RHICmdList);
			FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions();

			FSkeletalMeshUpdater* SkeletalMeshUpdater = GetSceneInterface()->GetSkeletalMeshUpdater();
			FGPUSkinCache* GPUSkinCache = GetSceneInterface()->GetGPUSkinCache();

			if (SkeletalMeshUpdater || GPUSkinCache)
			{
				const ERHIPipeline GPUSkinCachePipeline = ERHIPipeline::Graphics;

				FRDGBuilder GraphBuilder(RHICmdList);
				if (SkeletalMeshUpdater)
				{
					SkeletalMeshUpdater->Submit(GraphBuilder, GPUSkinCachePipeline);
				}
				if (GPUSkinCache)
				{
					GPUSkinCache->Dispatch(GraphBuilder, {}, GPUSkinCachePipeline);
				}
				GraphBuilder.Execute();
			}

			// Make a temporary ViewInfo
			//-TODO: We could gather some more information here perhaps?
			FSceneViewFamilyContext ViewFamily(
				FSceneViewFamily::ConstructionValues(nullptr, GetSceneInterface(), FEngineShowFlags(ESFIM_Game))
				.SetTime(CachedViewInitOptions.GameTime)
			);

			FSceneViewInitOptions ViewInitOptions;
			ViewInitOptions.ViewFamily = &ViewFamily;
			ViewInitOptions.SetViewRectangle(CachedViewInitOptions.ViewRect);
			ViewInitOptions.ViewOrigin = CachedViewInitOptions.ViewOrigin;
			ViewInitOptions.ViewRotationMatrix = CachedViewInitOptions.ViewRotationMatrix;
			ViewInitOptions.ProjectionMatrix = CachedViewInitOptions.ProjectionMatrix;

			GetRendererModule().CreateAndInitSingleView(RHICmdList, &ViewFamily, &ViewInitOptions);

			// Only one element, don't need to fully stride this array
			TConstStridedView<FSceneView> DummyViews = MakeStridedView<const FSceneView>(0, ViewFamily.Views[0], 1);
			const bool bAllowGPUParticleUpdate = true;

			// Ensure system textures are initialized
			GetRendererModule().InitializeSystemTextures(RHICmdList);

			// Allow downstream logic to detect we are running pending ticks outside the scene renderer
			bIsOutsideSceneRenderer = true;

			// Execute all ticks that we can support without invalid simulations
			MaxTicksToFlush = TickFlushMaxPendingTicks;
			for (int32 iTickBatch = 0; iTickBatch < MaxPendingTicks; iTickBatch+=MaxTicksToFlush)
			{
				FRDGBuilder GraphBuilder(RHICmdList);
				CreateSystemTextures(GraphBuilder);
				PreInitViews(GraphBuilder, bAllowGPUParticleUpdate, TArrayView<const FSceneViewFamily*>(), nullptr);
				AddPass(GraphBuilder, RDG_EVENT_NAME("UpdateDrawIndirectBuffers - PreOpaque"),
					[this](FRHICommandListImmediate& RHICmdList)
					{
						GPUInstanceCounterManager.UpdateDrawIndirectBuffers(this, RHICmdList, ENiagaraGPUCountUpdatePhase::PreOpaque);
					}
				);
				PostInitViews(GraphBuilder, DummyViews, bAllowGPUParticleUpdate);
				FSceneUniformBuffer &SceneUB = UE::FXRenderingUtils::CreateSceneUniformBuffer(GraphBuilder, GetSceneInterface());
				PostRenderOpaque(GraphBuilder, DummyViews, SceneUB, bAllowGPUParticleUpdate);
				AddPass(GraphBuilder, RDG_EVENT_NAME("UpdateDrawIndirectBuffers - PostOpaque"),
					[this](FRHICommandListImmediate& RHICmdList)
					{
						GPUInstanceCounterManager.UpdateDrawIndirectBuffers(this, RHICmdList, ENiagaraGPUCountUpdatePhase::PostOpaque);
					}
				);
				GraphBuilder.Execute();
			}
			MaxTicksToFlush = TNumericLimits<int32>::Max();

			bIsOutsideSceneRenderer = false;
			break;
		}

		// Kill all the pending ticks that have built up
		case 2:
		{
			//UE_LOGF(LogNiagara, Log, "FNiagaraGpuComputeDispatch: Queued ticks are being Destroyed due to not rendering.  This may result in undesirable simulation artifacts.");

			FinishDispatches();
			AsyncGpuTraceHelper->Reset();

			break;
		}
	}
}

void FNiagaraGpuComputeDispatch::FinishDispatches()
{
	check(IsInRenderingThread());

	for (int iTickStage = 0; iTickStage < ENiagaraGpuComputeTickStage::Max; ++iTickStage)
	{
		for (FNiagaraSystemGpuComputeProxy* ComputeProxy : ProxiesPerStage[iTickStage])
		{
			ComputeProxy->ReleaseTicks(MaxTicksToFlush, bIsLastViewFamily);
		}
	}

	for (FNiagaraGpuDispatchList& DispatchList : DispatchListPerStage)
	{
		DispatchList.DispatchGroups.Empty();
		if (DispatchList.CountsToRelease.Num() > 0)
		{
			GPUInstanceCounterManager.FreeEntryArray(DispatchList.CountsToRelease);
			DispatchList.CountsToRelease.Empty();
		}
	}
}

FNiagaraDataInterfaceProxyRW* FNiagaraGpuComputeDispatch::FindIterationInterface(FNiagaraComputeInstanceData* Instance, const uint32 SimulationStageIndex) const
{
	// Determine if the iteration is outputting to a custom data size
	return Instance->FindIterationInterface(SimulationStageIndex);
}

void FNiagaraGpuComputeDispatch::DumpDebugFrame()
{
	// Anything doing?
	bool bHasAnyWork = false;
	for (int iTickStage = 0; iTickStage < ENiagaraGpuComputeTickStage::Max; ++iTickStage)
	{
		bHasAnyWork |= DispatchListPerStage[iTickStage].HasWork();
	}
	if ( !bHasAnyWork )
	{
		return;
	}

	// Dump Frame
	UE_LOGF(LogNiagara, Warning, "====== BatcherFrame(%d)", GFrameNumberRenderThread);

	for (int iTickStage = 0; iTickStage < ENiagaraGpuComputeTickStage::Max; ++iTickStage)
	{
		if (!DispatchListPerStage[iTickStage].HasWork())
		{
			continue;
		}

		FNiagaraGpuDispatchList& DispatchList = DispatchListPerStage[iTickStage];
		UE_LOGF(LogNiagara, Warning, "==== TickStage(%d) TotalGroups(%d)", iTickStage, DispatchList.DispatchGroups.Num());

		for ( int iDispatchGroup=0; iDispatchGroup < DispatchList.DispatchGroups.Num(); ++iDispatchGroup )
		{
			const FNiagaraGpuDispatchGroup& DispatchGroup = DispatchListPerStage[iTickStage].DispatchGroups[iDispatchGroup];

			if (DispatchGroup.TicksWithPerInstanceData.Num() > 0)
			{
				UE_LOGF(LogNiagara, Warning, "====== TicksWithPerInstanceData(%d)", DispatchGroup.TicksWithPerInstanceData.Num());
				for (FNiagaraGPUSystemTick* Tick : DispatchGroup.TicksWithPerInstanceData)
				{
					for (const auto& Pair : Tick->DIInstanceData->InterfaceProxiesToOffsets)
					{
						FNiagaraDataInterfaceProxy* Proxy = Pair.Key;
						UE_LOGF(LogNiagara, Warning, "Proxy(%ls)", *Proxy->GetSourceDIName().ToString());
					}
				}
			}

			UE_LOGF(LogNiagara, Warning, "====== DispatchGroup(%d)", iDispatchGroup);
			for ( const FNiagaraGpuDispatchInstance& DispatchInstance : DispatchGroup.DispatchInstances )
			{
				const FNiagaraSimStageData& SimStageData = DispatchInstance.SimStageData;
				const FNiagaraComputeInstanceData& InstanceData = DispatchInstance.InstanceData;

				TStringBuilder<512> Builder;
				Builder.Appendf(TEXT("Proxy(%p) "), DispatchInstance.Tick.SystemGpuComputeProxy);
				Builder.Appendf(TEXT("ComputeContext(%p) "), InstanceData.Context);
				Builder.Appendf(TEXT("Emitter(%s) "), InstanceData.Context->GetDebugSimName());
				Builder.Appendf(TEXT("Stage(%d | %s) "), SimStageData.StageIndex, *SimStageData.StageMetaData->SimulationStageName.ToString());

				if (InstanceData.bResetData)
				{
					Builder.Append(TEXT("ResetData "));
				}

				if (InstanceData.Context->MainDataSet->RequiresPersistentIDs())
				{
					Builder.Append(TEXT("HasPersistentIDs "));
				}

				if ( DispatchInstance.SimStageData.bFirstStage )
				{
					Builder.Append(TEXT("FirstStage "));
				}

				if ( DispatchInstance.SimStageData.bLastStage )
				{
					Builder.Append(TEXT("LastStage "));
				}

				if (DispatchInstance.SimStageData.bSetDataToRender)
				{
					Builder.Append(TEXT("SetDataToRender "));
				}
					

				if (InstanceData.Context->EmitterInstanceReadback.GPUCountOffset != INDEX_NONE)
				{
					if (InstanceData.Context->EmitterInstanceReadback.GPUCountOffset == SimStageData.SourceCountOffset)
					{
						Builder.Appendf(TEXT("ReadbackSource(%d) "), InstanceData.Context->EmitterInstanceReadback.CPUCount);
					}
				}
				Builder.Appendf(TEXT("Source(%p 0x%08x %d) "), SimStageData.Source, SimStageData.SourceCountOffset, SimStageData.SourceNumInstances);
				Builder.Appendf(TEXT("Destination(%p 0x%08x %d) "), SimStageData.Destination, SimStageData.DestinationCountOffset, SimStageData.DestinationNumInstances);
				Builder.Appendf(TEXT("Iteration(%d | %s) "), SimStageData.IterationIndex, SimStageData.AlternateIterationSource ? *SimStageData.AlternateIterationSource->GetSourceDIName().ToString() : TEXT("Particles"));
				Builder.Appendf(TEXT("DispatchElementCount(%d, %d, %d) "), SimStageData.DispatchArgs.ElementCount.X, SimStageData.DispatchArgs.ElementCount.Y, SimStageData.DispatchArgs.ElementCount.Z);
				UE_LOGF(LogNiagara, Warning, "%ls", Builder.ToString());
			}

			if (DispatchGroup.FreeIDUpdates.Num() > 0)
			{
				UE_LOGF(LogNiagara, Warning, "====== FreeIDUpdates");
				for (const FNiagaraGpuFreeIDUpdate& FreeIDUpdate : DispatchGroup.FreeIDUpdates)
				{
					UE_LOGF(LogNiagara, Warning, "ComputeContext(%p) Emitter(%ls)", FreeIDUpdate.ComputeContext, FreeIDUpdate.ComputeContext->GetDebugSimName());
				}
			}
		}
		if (DispatchList.CountsToRelease.Num() > 0)
		{
			UE_LOGF(LogNiagara, Warning, "====== CountsToRelease");

			const int NumPerLine = 16;

			TStringBuilder<512> StringBuilder;
			for ( int32 i=0; i < DispatchList.CountsToRelease.Num(); ++i )
			{
				const bool bFirst = (i % NumPerLine) == 0;
				const bool bLast = ((i % NumPerLine) == NumPerLine - 1) || (i == DispatchList.CountsToRelease.Num() -1);

				if ( !bFirst )
				{
					StringBuilder.Append(TEXT(", "));
				}
				StringBuilder.Appendf(TEXT("0x%08x"), DispatchList.CountsToRelease[i]);

				if ( bLast )
				{
					UE_LOGF(LogNiagara, Warning, "%ls", StringBuilder.ToString());
					StringBuilder.Reset();
				}
			}
		}
	}
}

void FNiagaraGpuComputeDispatch::UpdateInstanceCountManager(FRHICommandListImmediate& RHICmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraGpuComputeDispatch_UpdateInstanceCountManager);

	// Resize dispatch buffer count
	//-OPT: No need to iterate over all the ticks, we can store this as ticks are queued
	{
		int32 TotalDispatchCount = 0;
		for (int iTickStage = 0; iTickStage < ENiagaraGpuComputeTickStage::Max; ++iTickStage)
		{
			for (FNiagaraSystemGpuComputeProxy* ComputeProxy : ProxiesPerStage[iTickStage])
			{
				const int32 NumTicksToProcess = FMath::Min(ComputeProxy->PendingTicks.Num(), MaxTicksToFlush);
				for (int32 iTickToProcess=0; iTickToProcess < NumTicksToProcess; ++iTickToProcess)
				{
					FNiagaraGPUSystemTick& Tick = ComputeProxy->PendingTicks[iTickToProcess];
					TotalDispatchCount += (int32)Tick.TotalDispatches;

					for ( FNiagaraComputeInstanceData& InstanceData : Tick.GetInstances() )
					{
						if (InstanceData.bResetData)
						{
							InstanceData.Context->EmitterInstanceReadback.GPUCountOffset = INDEX_NONE;
						}
					}
				}
			}
		}

		GPUInstanceCounterManager.ResizeBuffers(RHICmdList, TotalDispatchCount + ProxyGpuCountBufferEstimate);
	}

	// Consume any pending readbacks that are ready
	{
		SCOPE_CYCLE_COUNTER(STAT_NiagaraGPUReadback_RT);
		if ( const uint32* Counts = GPUInstanceCounterManager.GetGPUReadback() )
		{
			if (FNiagaraGpuComputeDispatchLocal::GDebugLogging)
			{
				UE_LOGF(LogNiagara, Warning, "====== BatcherFrame(%d) Readback Complete", GFrameNumberRenderThread);
			}

			for (int iTickStage=0; iTickStage < ENiagaraGpuComputeTickStage::Max; ++iTickStage)
			{
				for (FNiagaraSystemGpuComputeProxy* ComputeProxy : ProxiesPerStage[iTickStage])
				{
					for ( FNiagaraComputeExecutionContext* ComputeContext : ComputeProxy->ComputeContexts )
					{
						if (ComputeContext->EmitterInstanceReadback.GPUCountOffset == INDEX_NONE )
						{
							continue;
						}

						const uint32 DeadInstanceCount = ComputeContext->EmitterInstanceReadback.CPUCount - Counts[ComputeContext->EmitterInstanceReadback.GPUCountOffset];
						const uint32 CurrentInstanceCount = ComputeContext->GetCurrentNumInstances();
						if ( DeadInstanceCount <= CurrentInstanceCount)
						{
							ComputeContext->SetCurrentNumInstances(CurrentInstanceCount - DeadInstanceCount);
						}
						if (FNiagaraGpuComputeDispatchLocal::GDebugLogging)
						{
							UE_LOGF(LogNiagara, Warning, "ComputeContext(%p) Emitter(%ls) DeadInstances(%d) CountReleased(0x%08x)", ComputeContext, ComputeContext->GetDebugSimName(), DeadInstanceCount, ComputeContext->EmitterInstanceReadback.GPUCountOffset);
						}

						// Readback complete
						ComputeContext->EmitterInstanceReadback.GPUCountOffset = INDEX_NONE;
					}
				}
			}

			// Release the readback buffer
			GPUInstanceCounterManager.ReleaseGPUReadback();
		}
	}

	//Run instance count Initialization task. Some instance counts must be initialized with values from the CPU before use in GPU sims.
	{
		GPUInstanceCounterManager.ProcessInitInstanceCountTasks(this, RHICmdList);
	}
}

void FNiagaraGpuComputeDispatch::PrepareTicksForProxy(FRHICommandListImmediate& RHICmdList, FNiagaraSystemGpuComputeProxy* ComputeProxy, FNiagaraGpuDispatchList& GpuDispatchList)
{
	for (FNiagaraComputeExecutionContext* ComputeContext : ComputeProxy->ComputeContexts)
	{
		ComputeContext->CurrentMaxInstances_RT = 0;
		ComputeContext->CurrentMaxAllocateInstances_RT = 0;
		ComputeContext->BufferSwapsThisFrame_RT = 0;
		ComputeContext->FinalDispatchGroup_RT = INDEX_NONE;
		ComputeContext->FinalDispatchGroupInstance_RT = INDEX_NONE;
	}

	if (ComputeProxy->PendingTicks.Num() == 0)
	{
		return;
	}

	const bool bEnqueueCountReadback = !GPUInstanceCounterManager.HasPendingGPUReadback();

	// Set final tick flag
	const int32 NumTicksToProcess = FMath::Min(ComputeProxy->PendingTicks.Num(), MaxTicksToFlush);
	ComputeProxy->PendingTicks[NumTicksToProcess - 1].bIsFinalTick = true;

	// Process ticks
	int32 iTickStartDispatchGroup = 0;

	for ( int32 iTickToProcess=0; iTickToProcess < NumTicksToProcess; ++iTickToProcess )
	{
		FNiagaraGPUSystemTick& Tick = ComputeProxy->PendingTicks[iTickToProcess];

		int32 iInstanceStartDispatchGroup = iTickStartDispatchGroup;
		int32 iInstanceCurrDispatchGroup = iTickStartDispatchGroup;
		bool bHasFreeIDUpdates = false;

		// Track that we need to consume per instance data before executing the ticks
		//if (Tick.DIInstanceData)
		//{
		//	GpuDispatchList.PreAllocateGroups(iTickStartDispatchGroup + 1);
		//	GpuDispatchList.DispatchGroups[iTickStartDispatchGroup].TicksWithPerInstanceData.Add(&Tick);
		//}

		// Iterate over all instances preparing our number of instances
		for (FNiagaraComputeInstanceData& InstanceData : Tick.GetInstances())
		{
			FNiagaraComputeExecutionContext* ComputeContext = InstanceData.Context;

			// Instance requires a reset?
			if (InstanceData.bResetData)
			{
				ComputeContext->SetCurrentNumInstances(0);
				if (ComputeContext->CountOffset_RT != INDEX_NONE)
				{
					GpuDispatchList.CountsToRelease.Add(ComputeContext->CountOffset_RT);
					ComputeContext->CountOffset_RT = INDEX_NONE;
				}
			}

			// If shader is not ready don't do anything
			if (!ComputeContext->GPUScript_RT->IsShaderMapComplete_RenderThread())
			{
				continue;
			}

			// Nothing to dispatch?
			if ( InstanceData.TotalDispatches == 0 )
			{
				continue;
			}

#if WITH_EDITOR
			//-TODO: Validate feature level in the editor as when using the preview mode we can be using the wrong shaders for the renderer type.
			//       i.e. We may attempt to sample the gbuffer / depth using deferred scene textures rather than mobile which will crash.
			if (ComputeContext->GPUScript_RT->GetFeatureLevel() != FeatureLevel)
			{
				if (ComputeProxy->GetComputeTickStage() == ENiagaraGpuComputeTickStage::PostOpaqueRender)
				{
					if (bRaisedWarningThisFrame == false)
					{
						bRaisedWarningThisFrame = true;
						GEngine->AddOnScreenDebugMessage(uint64(this), 1.f, FColor::White, *FString::Printf(TEXT("GPU Simulation(%s) will not show in preview mode, as we may sample from wrong SceneTextures buffer."), ComputeContext->GetDebugSimName()));
					}
					continue;
				}
			}
#endif

			// Determine this instances start dispatch group, in the case of emitter dependencies (i.e. particle reads) we need to continue rather than starting again
			iInstanceStartDispatchGroup = InstanceData.bStartNewOverlapGroup ? iInstanceCurrDispatchGroup : iInstanceStartDispatchGroup;
			iInstanceCurrDispatchGroup = iInstanceStartDispatchGroup;

			// Pre-allocator groups
			GpuDispatchList.PreAllocateGroups(iInstanceCurrDispatchGroup + InstanceData.TotalDispatches);

			// Calculate instance counts
			const uint32 MaxBufferInstances = ComputeContext->MainDataSet->GetMaxInstanceCount();
			const uint32 PrevNumInstances = ComputeContext->GetCurrentNumInstances();

			uint32 NewInstanceCount = PrevNumInstances;
			NewInstanceCount = NewInstanceCount + InstanceData.SpawnInfo.SpawnRateInstances + InstanceData.SpawnInfo.EventSpawnTotal;
			NewInstanceCount = FMath::Min(NewInstanceCount, MaxBufferInstances);

			// Calculate new maximum count
			ComputeContext->CurrentMaxInstances_RT = FMath::Max(ComputeContext->CurrentMaxInstances_RT, NewInstanceCount);

			if (!GNiagaraBatcherFreeBufferEarly || (NewInstanceCount > 0))
			{
				ComputeContext->CurrentMaxAllocateInstances_RT = FMath::Max3(ComputeContext->CurrentMaxAllocateInstances_RT, NewInstanceCount, InstanceData.SpawnInfo.MaxParticleCount);
			}
			else
			{
				ComputeContext->CurrentMaxAllocateInstances_RT = FMath::Max(ComputeContext->CurrentMaxAllocateInstances_RT, NewInstanceCount);
			}

			ComputeContext->SetCurrentNumInstances(NewInstanceCount);

			bHasFreeIDUpdates |= ComputeContext->MainDataSet->RequiresPersistentIDs();

			//-OPT: Do we need this test?  Can remove in favor of MaxUpdateIterations
			bool bFirstStage = true;
			for (const FNiagaraComputeInstanceData::FPerStageInfo& PerStageInfo : InstanceData.PerStageInfo)
			{
				const int32 SimStageIndex = PerStageInfo.SimStageIndex;
				const FSimulationStageMetaData& SimStageMetaData = ComputeContext->SimStageExecData->SimStageMetaData[SimStageIndex];

				FNiagaraDataInterfaceProxyRW* IterationInterface = InstanceData.FindIterationInterface(SimStageIndex);
				const int32 NumIterations = PerStageInfo.NumIterations;
				for ( int32 IterationIndex=0; IterationIndex < NumIterations; ++IterationIndex )
				{
					// Build SimStage data
					FNiagaraGpuDispatchGroup& DispatchGroup = GpuDispatchList.DispatchGroups[iInstanceCurrDispatchGroup++];
					FNiagaraGpuDispatchInstance& DispatchInstance = DispatchGroup.DispatchInstances.Emplace_GetRef(Tick, InstanceData);
					FNiagaraSimStageData& SimStageData = DispatchInstance.SimStageData;
					SimStageData.bFirstStage = bFirstStage;
					SimStageData.StageIndex = IntCastChecked<uint16>(SimStageIndex);
					SimStageData.NumIterations = IntCastChecked<uint16>(NumIterations);
					SimStageData.IterationIndex = IntCastChecked<uint16>(IterationIndex);
					SimStageData.NumLoops = PerStageInfo.NumLoops;
					SimStageData.LoopIndex = PerStageInfo.LoopIndex;
					SimStageData.DispatchArgs.ElementCount = PerStageInfo.ElementCountXYZ;
					SimStageData.StageMetaData = &SimStageMetaData;
					SimStageData.AlternateIterationSource = IterationInterface;

					bFirstStage = false;

					FNiagaraDataBuffer* SourceData = ComputeContext->bHasTickedThisFrame_RT ? ComputeContext->GetPrevDataBuffer() : ComputeContext->MainDataSet->GetCurrentData();

					// This stage does not modify particle data, i.e. read only or not related to particles at all
					if (!SimStageData.StageMetaData->bWritesParticles)
					{
						SimStageData.Source = SourceData;
						SimStageData.SourceCountOffset = ComputeContext->CountOffset_RT;
						SimStageData.SourceNumInstances = NewInstanceCount;
						SimStageData.Destination = nullptr;
						SimStageData.DestinationCountOffset = ComputeContext->CountOffset_RT;
						SimStageData.DestinationNumInstances = NewInstanceCount;
					}
					// This stage writes particles but will not kill any, we can use the buffer as both source and destination
					else if (SimStageData.StageMetaData->bPartialParticleUpdate)
					{
						SimStageData.Source = nullptr;
						SimStageData.SourceCountOffset = ComputeContext->CountOffset_RT;
						SimStageData.SourceNumInstances = NewInstanceCount;
						SimStageData.Destination = SourceData;
						SimStageData.DestinationCountOffset = ComputeContext->CountOffset_RT;
						SimStageData.DestinationNumInstances = NewInstanceCount;
					}
					// This stage may kill particles, we need to allocate a new destination buffer
					else
					{
						SimStageData.Source = SourceData;
						SimStageData.SourceCountOffset = ComputeContext->CountOffset_RT;
						//-TODO: This is a little odd, perhaps we need to change the preallocate
						SimStageData.SourceNumInstances = SimStageIndex == 0 && IterationIndex == 0 ? PrevNumInstances : NewInstanceCount;
						SimStageData.Destination = ComputeContext->GetNextDataBuffer();
						SimStageData.DestinationCountOffset = GPUInstanceCounterManager.AcquireEntry();
						SimStageData.DestinationNumInstances = NewInstanceCount;

						ComputeContext->AdvanceDataBuffer();
						ComputeContext->CountOffset_RT = SimStageData.DestinationCountOffset;
						ComputeContext->bHasTickedThisFrame_RT = true;

						// If we are the last tick then we may want to enqueue for a readback
						// Note: Do not pull count from SimStageData as a reset tick will be INDEX_NONE
						check(SimStageData.SourceCountOffset != INDEX_NONE || SimStageData.SourceNumInstances == 0);
						if (SimStageData.SourceCountOffset != INDEX_NONE)
						{
							if ( bEnqueueCountReadback && Tick.bIsFinalTick && (ComputeContext->EmitterInstanceReadback.GPUCountOffset == INDEX_NONE))
							{
								bRequiresReadback = true;
								ComputeContext->EmitterInstanceReadback.CPUCount = SimStageData.SourceNumInstances;
								ComputeContext->EmitterInstanceReadback.GPUCountOffset = SimStageData.SourceCountOffset;
							}
							GpuDispatchList.CountsToRelease.Add(SimStageData.SourceCountOffset);
						}
					}
				}
			}

			// Set this as the last stage and store the final dispatch group / instance
			FNiagaraGpuDispatchGroup& FinalDispatchGroup = GpuDispatchList.DispatchGroups[iInstanceCurrDispatchGroup - 1];
			FinalDispatchGroup.DispatchInstances.Last().SimStageData.bLastStage = true;

			ComputeContext->FinalDispatchGroup_RT = iInstanceCurrDispatchGroup - 1;
			ComputeContext->FinalDispatchGroupInstance_RT = FinalDispatchGroup.DispatchInstances.Num() - 1;

			// Keep track of where the next set of dispatch should occur
			iTickStartDispatchGroup = FMath::Max(iTickStartDispatchGroup, iInstanceCurrDispatchGroup);
		}

		// Accumulate Free ID updates
		// Note: These must be done at the end of the tick due to the way spawned instances read from the free list
		if (bHasFreeIDUpdates)
		{
			FNiagaraGpuDispatchGroup& FinalDispatchGroup = GpuDispatchList.DispatchGroups[iInstanceCurrDispatchGroup - 1];
			for (FNiagaraComputeInstanceData& InstanceData : Tick.GetInstances())
			{
				FNiagaraComputeExecutionContext* ComputeContext = InstanceData.Context;
				if (!ComputeContext->GPUScript_RT->IsShaderMapComplete_RenderThread())
				{
					continue;
				}

				if ( ComputeContext->MainDataSet->RequiresPersistentIDs() && ComputeContext->bHasTickedThisFrame_RT )
				{
					FinalDispatchGroup.FreeIDUpdates.Emplace(ComputeContext);
					NumRequiredFreeIDListSizes = FMath::Max<uint32>(NumRequiredFreeIDListSizes, FinalDispatchGroup.FreeIDUpdates.Num());
				}
			}
		}

		// Build constant buffers for tick
		Tick.BuildUniformBuffers();
	}

	// Now that all ticks have been processed we can adjust our output buffers to the correct size
	// We will also set the translucent data to render, i.e. this frames data.
	for (FNiagaraComputeExecutionContext* ComputeContext : ComputeProxy->ComputeContexts)
	{
		if (!ComputeContext->bHasTickedThisFrame_RT)
		{
			continue;
		}

		// Ensure we set the data to render as the context may have been dropped during a multi-tick
		check(ComputeContext->FinalDispatchGroup_RT != INDEX_NONE);
		FNiagaraGpuDispatchGroup& FinalDispatchGroup = GpuDispatchList.DispatchGroups[ComputeContext->FinalDispatchGroup_RT];
		FinalDispatchGroup.DispatchInstances[ComputeContext->FinalDispatchGroupInstance_RT].SimStageData.bSetDataToRender = true;

		// We need to store the current data from the main data set as we will be temporarily stomping it during multi-ticking
		ComputeContext->DataSetOriginalBuffer_RT = ComputeContext->MainDataSet->GetCurrentData();

		//-OPT: We should allocate all GPU free IDs together since they require a transition
		if (ComputeContext->MainDataSet->RequiresPersistentIDs())
		{
			ComputeContext->MainDataSet->AllocateGPUFreeIDs(ComputeContext->CurrentMaxAllocateInstances_RT + 1, RHICmdList, FeatureLevel, ComputeContext->GetDebugSimName());
		}

		// Allocate space for the buffers we need to perform ticking.  In cases of multiple ticks or multiple write stages we need 3 buffers (current rendered and two simulation buffers).
		//-OPT: We can batch the allocation of persistent IDs together so the compute shaders overlap
		const uint32 NumBuffersToResize = FMath::Min<uint32>(ComputeContext->BufferSwapsThisFrame_RT, UE_ARRAY_COUNT(ComputeContext->DataBuffers_RT));
		for (uint32 i=0; i < NumBuffersToResize; ++i)
		{
			ComputeContext->DataBuffers_RT[i]->AllocateGPU(RHICmdList, ComputeContext->CurrentMaxAllocateInstances_RT + 1, FeatureLevel, ComputeContext->GetDebugSimName());
		}

		// Ensure we don't keep multi-tick buffers around longer than they are required by releasing them
		for (uint32 i=NumBuffersToResize; i < UE_ARRAY_COUNT(ComputeContext->DataBuffers_RT); ++i)
		{
			ComputeContext->DataBuffers_RT[i]->ReleaseGPU();
		}

		// RDG will defer the Niagara dispatches until the graph is executed.
		// Therefore we need to setup the DataToRender for MeshProcessors & sorting to use the correct data,
		// that is anything that happens before PostRenderOpaque
		if ((ComputeProxy->GetComputeTickStage() == ENiagaraGpuComputeTickStage::PreInitViews) || (ComputeProxy->GetComputeTickStage() == ENiagaraGpuComputeTickStage::PostInitViews))
		{
			FNiagaraDataBuffer* FinalBuffer = ComputeContext->GetPrevDataBuffer();
			FinalBuffer->SetGPUInstanceCountBufferOffset(ComputeContext->CountOffset_RT);
			FinalBuffer->SetNumInstances(ComputeContext->GetCurrentNumInstances());
			FinalBuffer->SetGPUDataReadyStage(ComputeProxy->GetComputeTickStage());
			ComputeContext->SetDataToRender(FinalBuffer);
		}
		// When low latency translucency is enabled we can setup the final buffer / final count here.
		// This will allow our mesh processor commands to pickup the data for the same frame.
		// This allows simulations that use the depth buffer, for example, to execute with no latency
		else if ( GNiagaraGpuLowLatencyTranslucencyEnabled )
		{
			FNiagaraDataBuffer* FinalBuffer = ComputeContext->GetPrevDataBuffer();
			FinalBuffer->SetGPUInstanceCountBufferOffset(ComputeContext->CountOffset_RT);
			FinalBuffer->SetNumInstances(ComputeContext->GetCurrentNumInstances());
			FinalBuffer->SetGPUDataReadyStage(ComputeProxy->GetComputeTickStage());
			ComputeContext->SetTranslucentDataToRender(FinalBuffer);
		}
	}
}

void FNiagaraGpuComputeDispatch::PrepareAllTicks(FRHICommandListImmediate& RHICmdList)
{
	NumRequiredFreeIDListSizes = 0;

	for (int iTickStage=0; iTickStage < ENiagaraGpuComputeTickStage::Max; ++iTickStage)
	{
		for (FNiagaraSystemGpuComputeProxy* ComputeProxy : ProxiesPerStage[iTickStage])
		{
			PrepareTicksForProxy(RHICmdList, ComputeProxy, DispatchListPerStage[iTickStage]);
		}
	}

	if (NumRequiredFreeIDListSizes > NumAllocatedFreeIDListSizes)
	{
		constexpr uint32 ALLOC_CHUNK_SIZE = 128;
		NumAllocatedFreeIDListSizes = Align(NumRequiredFreeIDListSizes, ALLOC_CHUNK_SIZE);
		FreeIDListSizesBuffer.Release();
		FreeIDListSizesBuffer.Initialize(RHICmdList, TEXT("NiagaraFreeIDListSizes"), sizeof(uint32), NumAllocatedFreeIDListSizes, EPixelFormat::PF_R32_SINT, ERHIAccess::UAVCompute, BUF_Static);
	}
}

void FNiagaraGpuComputeDispatch::ExecuteTicks(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, ENiagaraGpuComputeTickStage::Type TickStage)
{
#if WITH_MGPU
	if (TickStage == ENiagaraGpuComputeTickStage::PostOpaqueRender)
	{
		// See if this view is rendering on a GPU that needs to wait on its cross GPU transfers
		uint32 GPUIndex = Views[0].GPUMask.GetFirstIndex();
		if (OptimizedCrossGPUTransferMask & (1u << GPUIndex))
		{
			// Track that we've waited on the buffers for this GPU
			OptimizedCrossGPUTransferMask &= ~(1u << GPUIndex);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("Niagara::ExecuteTicksPre"),
				ERDGPassFlags::None,
				[this, TickStage, NumDispatchGroups = DispatchListPerStage[TickStage].DispatchGroups.Num(), GPUIndex](FRHICommandList& RHICmdList)
				{
					WaitForMultiGPUBuffers(RHICmdList, GPUIndex);
				}
			);
		}

		// Last view family when rendering multiple view families?
		if (bIsLastViewFamily && !bIsFirstViewFamily)
		{
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("Niagara::MultiViewPreviousDataClear"),
				ERDGPassFlags::None,
				[this](FRHICommandList& RHICmdList)
				{
					// Handle bookkeeping for multi-view, clearing MultiViewPreviousDataToRender pointer and running a couple bits of cleanup logic,
					// which we had to skip earlier to keep necessary values valid when generating render data for additional view families.
					for (FNiagaraComputeExecutionContext* ComputeContext : NeedsMultiViewPreviousDataClear)
					{
						ComputeContext->SetMultiViewPreviousDataToRender(nullptr);

						// Mark data as ready for anyone who picks up the buffer on the next frame (see FNiagaraGpuComputeDispatch::ExecuteTicks)
						ComputeContext->GetDataToRender(RHICmdList, false)->SetGPUDataReadyStage(ENiagaraGpuComputeTickStage::First);

						// Clear instance count offsets (see FNiagaraSystemGpuComputeProxy::ReleaseTicks)
						for (int i = 0; i < UE_ARRAY_COUNT(ComputeContext->DataBuffers_RT); ++i)
						{
							ComputeContext->DataBuffers_RT[i]->ClearGPUInstanceCount();
						}
					}

					NeedsMultiViewPreviousDataClear.Reset();
				}
			);
		}
	}
#endif

	// Anything to execute for this stage
	FNiagaraGpuDispatchList& DispatchList = DispatchListPerStage[TickStage];
	if ( !DispatchList.HasWork() )
	{
		return;
	}
	const int32 StageStartTotalDispatches = TotalDispatchesThisFrame;

	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraGpuComputeDispatch_ExecuteTicks);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraGPUSimTick_RT);
	RDG_RHI_EVENT_SCOPE_STAT(GraphBuilder, NiagaraGPUSimulation, NiagaraGpuComputeDispatch);	//-TODO:RDG: Show TickStage

	// Setup Parameters that can be read from data interfaces
	SimulationSceneViews = Views;

	if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Deferred)
	{
		SceneTexturesUniformParams = UE::FXRenderingUtils::GetOrCreateSceneTextureUniformBuffer(GraphBuilder, SimulationSceneViews, FeatureLevel, ESceneTextureSetupMode::SceneVelocity);
	}
	else if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Mobile)
	{
		MobileSceneTexturesUniformParams = UE::FXRenderingUtils::GetOrCreateMobileSceneTextureUniformBuffer(GraphBuilder, SimulationSceneViews, EMobileSceneTextureSetupMode::None);
	}

	SubstratePublicGlobalUniformParams = UE::FXRenderingUtils::GetSubstratePublicGlobalUniformBuffer(GraphBuilder, SimulationSceneViews);
	
	FNiagaraGpuComputeDataManagerTickContext Context(GraphBuilder);
	for (auto& DataManager : GpuDataManagers)
	{
		if (DataManager.Value)
		{
			DataManager.Value->PreTick(Context, TickStage);
		}
	}

	// Loop over dispatches
	for ( const FNiagaraGpuDispatchGroup& DispatchGroup : DispatchList.DispatchGroups )
	{
		const bool bIsFirstGroup = &DispatchGroup == &DispatchList.DispatchGroups[0];
		const bool bIsLastGroup = &DispatchGroup == &DispatchList.DispatchGroups.Last();

		bIsExecutingFirstDispatchGroup = bIsFirstGroup;
		bIsExecutingLastDispatchGroup = bIsLastGroup;

		// Consume per tick data from the game thread
		//-TODO: This does not work currently as some senders assume the data will not be deferred processed
		//for (FNiagaraGPUSystemTick* Tick : DispatchGroup.TicksWithPerInstanceData)
		//{
		//	uint8* BasePointer = reinterpret_cast<uint8*>(Tick->DIInstanceData->PerInstanceDataForRT);
		//
		//	for (const auto& Pair : Tick->DIInstanceData->InterfaceProxiesToOffsets)
		//	{
		//		FNiagaraDataInterfaceProxy* Proxy = Pair.Key;
		//		uint8* InstanceDataPtr = BasePointer + Pair.Value;
		//		Proxy->ConsumePerInstanceDataFromGameThread(InstanceDataPtr, Tick->SystemInstanceID);
		//	}
		//}

		// Generate pre / post stage graph data
		//-OPT: Once we have RenderGraph dependencies between stages a lot of this tracking can go away as the graph will handle it
		TArray<FRHITransitionInfo> PreStageTransitions;
		TArray<FRHITransitionInfo> PostStageTransitions;
		TArray<TPair<FRHIUnorderedAccessView*, int32>> PreStageIDToIndexInit;
		PreStageTransitions.Reserve(DispatchGroup.DispatchInstances.Num() * 3);
		PostStageTransitions.Reserve(DispatchGroup.DispatchInstances.Num() * 3);

		for (const FNiagaraGpuDispatchInstance& DispatchInstance : DispatchGroup.DispatchInstances)
		{
			if (FNiagaraDataBuffer* DestinationBuffer = DispatchInstance.SimStageData.Destination)
			{
				FNiagaraGpuComputeDispatchLocal::AddDataBufferTransitions(PreStageTransitions, PostStageTransitions, DestinationBuffer);
			}

			FNiagaraComputeExecutionContext* ComputeContext = DispatchInstance.InstanceData.Context;
			const bool bRequiresPersistentIDs = ComputeContext->MainDataSet->RequiresPersistentIDs();
			if (bRequiresPersistentIDs)
			{
				if ( FNiagaraDataBuffer* IDToIndexBuffer = DispatchInstance.SimStageData.Destination )
				{
					FRHIUnorderedAccessView* BufferUAV = IDToIndexBuffer->GetGPUIDToIndexTable().UAV;
					const int32 BufferNumEntries = IDToIndexBuffer->GetGPUIDToIndexTable().NumBytes / sizeof(int32);

					PreStageIDToIndexInit.Emplace(BufferUAV, BufferNumEntries);
					PreStageTransitions.Emplace(BufferUAV, ERHIAccess::SRVCompute, ERHIAccess::UAVCompute);
					PostStageTransitions.Emplace(BufferUAV, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute);
				}
			}
		}

		PreStageTransitions.Emplace(GPUInstanceCounterManager.GetInstanceCountBuffer().UAV, bIsFirstGroup ? FNiagaraGPUInstanceCountManager::kCountBufferDefaultState : ERHIAccess::UAVCompute, ERHIAccess::UAVCompute);
		if (bIsLastGroup)
		{
			PostStageTransitions.Emplace(GPUInstanceCounterManager.GetInstanceCountBuffer().UAV, ERHIAccess::UAVCompute, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState);
		}

		if (DispatchGroup.FreeIDUpdates.Num() > 0)
		{
			PostStageTransitions.Reserve(PostStageTransitions.Num() + DispatchGroup.FreeIDUpdates.Num());
			for (const FNiagaraGpuFreeIDUpdate& FreeIDUpdate : DispatchGroup.FreeIDUpdates)
			{
				PostStageTransitions.Emplace(FreeIDUpdate.ComputeContext->MainDataSet->GetGPUFreeIDs().UAV, ERHIAccess::SRVCompute, ERHIAccess::UAVCompute);
			}
		}

		// Pre Stage
		{
			FNDIGpuComputeDispatchArgsGenContext DispatchArgsGenContext(GraphBuilder, *this);

			TSet<FNiagaraDataInterfaceProxy*> ProxiesToFinalize;
			for (const FNiagaraGpuDispatchInstance& DispatchInstance : DispatchGroup.DispatchInstances)
			{
				if (DispatchInstance.SimStageData.AlternateIterationSource)
				{
					DispatchArgsGenContext.SetInstanceData(DispatchInstance.Tick.SystemInstanceID, const_cast<FNiagaraSimStageData&>(DispatchInstance.SimStageData));
					DispatchInstance.SimStageData.AlternateIterationSource->GetDispatchArgs(DispatchArgsGenContext);
				}

				PreStageInterface(GraphBuilder, DispatchInstance.Tick, DispatchInstance.InstanceData, DispatchInstance.SimStageData, ProxiesToFinalize);
			}
			for (FNiagaraDataInterfaceProxy* ProxyToFinalize : ProxiesToFinalize)
			{
				ProxyToFinalize->FinalizePreStage(GraphBuilder, *this);
			}
		}

		// Execute Transitions
 		GraphBuilder.AddPass(
			RDG_EVENT_NAME("Niagara::ExecuteTicks::DispatchGroupPre"),
			ERDGPassFlags::None,
			[this, PreStageTransitions=MoveTemp(PreStageTransitions), PreStageIDToIndexInit=MoveTemp(PreStageIDToIndexInit), DispatchInstances=MakeArrayView(DispatchGroup.DispatchInstances)](FRHICommandList& RHICmdList)
			{
				// Execute Before Transitions
				RHICmdList.Transition(PreStageTransitions);

				// Initialize the IDtoIndex tables
				if (PreStageIDToIndexInit.Num() > 0)
				{
					SCOPED_DRAW_EVENT(RHICmdList, NiagaraGPUComputeClearIDToIndexBuffer);

					TArray<FRHITransitionInfo> IDToIndexTransitions;
					IDToIndexTransitions.Reserve(PreStageIDToIndexInit.Num());

					for (const TPair<FRHIUnorderedAccessView*, int32>& IDtoIndexBuffer : PreStageIDToIndexInit)
					{
						NiagaraFillGPUIntBuffer(RHICmdList, FeatureLevel, IDtoIndexBuffer.Key, IDtoIndexBuffer.Value, INDEX_NONE);
						IDToIndexTransitions.Emplace(IDtoIndexBuffer.Key, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute);
					}
					RHICmdList.Transition(IDToIndexTransitions);
				}
				
				RHICmdList.BeginUAVOverlap(GPUInstanceCounterManager.GetInstanceCountBuffer().UAV);
			}
		);

		// Execute Stage
		for (const FNiagaraGpuDispatchInstance& DispatchInstance : DispatchGroup.DispatchInstances)
		{
			++FNiagaraComputeExecutionContext::TickCounter;
			if (DispatchInstance.InstanceData.bResetData && DispatchInstance.SimStageData.bFirstStage)
			{
				ResetDataInterfaces(GraphBuilder, DispatchInstance.Tick, DispatchInstance.InstanceData);
			}

			DispatchStage(GraphBuilder, DispatchInstance.Tick, DispatchInstance.InstanceData, DispatchInstance.SimStageData);
		}

		// Execute legacy Post Stage
 		GraphBuilder.AddPass(
			RDG_EVENT_NAME("Niagara::ExecuteTicks::DispatchGroupPost"),
			ERDGPassFlags::None,
			[this, PostStageTransitions=MoveTemp(PostStageTransitions), DispatchInstances=MakeArrayView(DispatchGroup.DispatchInstances)](FRHICommandList& RHICmdList)
			{
				RHICmdList.EndUAVOverlap(GPUInstanceCounterManager.GetInstanceCountBuffer().UAV);

				RHICmdList.Transition(PostStageTransitions);
			}
		);

		// Execute PostStage / Post Simulate
		TSet<FNiagaraDataInterfaceProxy*> ProxiesToFinalize;
		for (const FNiagaraGpuDispatchInstance& DispatchInstance : DispatchGroup.DispatchInstances)
		{
			PostStageInterface(GraphBuilder, DispatchInstance.Tick, DispatchInstance.InstanceData, DispatchInstance.SimStageData, ProxiesToFinalize);
			if (DispatchInstance.SimStageData.bLastStage)
			{
				PostSimulateInterface(GraphBuilder, DispatchInstance.Tick, DispatchInstance.InstanceData, DispatchInstance.SimStageData);
			}

			// Update CurrentData with the latest information as things like ParticleReads can use this data
			FNiagaraComputeExecutionContext* ComputeContext = DispatchInstance.InstanceData.Context;
			const FNiagaraSimStageData& FinalSimStageData = DispatchInstance.SimStageData;
			FNiagaraDataBuffer* FinalSimStageDataBuffer = FinalSimStageData.Destination != nullptr ? FinalSimStageData.Destination : FinalSimStageData.Source;
			check(FinalSimStageDataBuffer != nullptr);

			// If we are setting the data to render we need to ensure we switch back to the original CurrentData then swap the GPU buffers into it
			if (DispatchInstance.SimStageData.bSetDataToRender)
			{
				check(ComputeContext->DataSetOriginalBuffer_RT != nullptr);
				FNiagaraDataBuffer* CurrentData = ComputeContext->DataSetOriginalBuffer_RT;
				ComputeContext->DataSetOriginalBuffer_RT = nullptr;

				ComputeContext->MainDataSet->CurrentData = CurrentData;
				CurrentData->SwapGPU(FinalSimStageDataBuffer);

				ComputeContext->SetTranslucentDataToRender(nullptr);
				ComputeContext->SetDataToRender(CurrentData);

			#if WITH_MGPU
				if (bCrossGPUTransferEnabled)
				{
					AddCrossGPUTransfer(GraphBuilder.RHICmdList, CurrentData->GetGPUBufferFloat().Buffer);
					AddCrossGPUTransfer(GraphBuilder.RHICmdList, CurrentData->GetGPUBufferHalf().Buffer);
					AddCrossGPUTransfer(GraphBuilder.RHICmdList, CurrentData->GetGPUBufferInt().Buffer);
				}
			#endif // WITH_MGPU

				if (bIsLastViewFamily)
				{
					// Mark data as ready for anyone who picks up the buffer on the next frame
					CurrentData->SetGPUDataReadyStage(ENiagaraGpuComputeTickStage::First);
				}
				else
				{
				#if WITH_MGPU
					// Track the previous data to render for multi-view, so additional view families can render using consistent data with
					// the first view family, and add it to array marking that we need to clean it up on the last view family.
					ComputeContext->SetMultiViewPreviousDataToRender(FinalSimStageDataBuffer);
					NeedsMultiViewPreviousDataClear.Emplace(ComputeContext);
				#endif //WITH_MGPU

					// Mark CurrentData as not to be used until PostOpaqueRender.  Render data generated before PostOpaqueRender should use
					// previous frame data (MultiViewPreviousDataToRender, set above), instead of CurrentData.  Setting the ready stage back to
					// ENiagaraGpuComputeTickStage::First for the next frame happens later in the Niagara::MultiViewPreviousDataClear Pass.
					CurrentData->SetGPUDataReadyStage(ENiagaraGpuComputeTickStage::PostOpaqueRender);
				}
			}
			// If this is not the final tick of the final stage we need set our temporary buffer for data interfaces, etc, that may snoop from CurrentData
			else
			{
				ComputeContext->MainDataSet->CurrentData = FinalSimStageDataBuffer;
			}
		}

		for (FNiagaraDataInterfaceProxy* ProxyToFinalize : ProxiesToFinalize)
		{
			ProxyToFinalize->FinalizePostStage(GraphBuilder, *this);
		}

		// Free ID updates
		if (DispatchGroup.FreeIDUpdates.Num() > 0)
		{
			// Grab information for the pass
			for (const FNiagaraGpuFreeIDUpdate& FreeIDUpdate : DispatchGroup.FreeIDUpdates)
			{
				FNiagaraDataSet* MainDataSet = FreeIDUpdate.ComputeContext->MainDataSet;
				FNiagaraDataBuffer* CurrentData = FreeIDUpdate.ComputeContext->MainDataSet->GetCurrentData();

				FreeIDUpdate.IDToIndexSRV		= CurrentData->GetGPUIDToIndexTable().SRV;
				FreeIDUpdate.FreeIDsUAV			= MainDataSet->GetGPUFreeIDs().UAV;
				FreeIDUpdate.NumAllocatedIDs	= MainDataSet->GetGPUNumAllocatedIDs();

				check(FreeIDUpdate.IDToIndexSRV != nullptr);
				check(FreeIDUpdate.FreeIDsUAV != nullptr);
			}

 			GraphBuilder.AddPass(
				RDG_EVENT_NAME("Niagara::ExecuteTicks::FreeIDUpdates"),
				ERDGPassFlags::None,
				[this, FreeIDUpdates=MakeArrayView(DispatchGroup.FreeIDUpdates)](FRHICommandList& RHICmdList)
				{
					const uint32 NumFreeIDUpdates = FreeIDUpdates.Num();

					// Initialize the free ID size buffer
					{
						SCOPED_DRAW_EVENT(RHICmdList, NiagaraGPUComputeClearFreeIDListSizes);
						RHICmdList.Transition(FRHITransitionInfo(FreeIDListSizesBuffer.UAV, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute));
						NiagaraFillGPUIntBuffer(RHICmdList, FeatureLevel, FreeIDListSizesBuffer.UAV, FreeIDListSizesBuffer.NumBytes / sizeof(uint32), 0);
					}

					// Update Free IDs
					{
						RHI_BREADCRUMB_EVENT_STAT(RHICmdList, NiagaraGPUComputeFreeIDs, "NiagaraGPUComputeFreeIDs");

						RHICmdList.Transition(FRHITransitionInfo(FreeIDListSizesBuffer.UAV, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute));

						TArray<FRHITransitionInfo> TransitionsFreeIDs;
						TransitionsFreeIDs.Reserve(NumFreeIDUpdates);

						RHICmdList.BeginUAVOverlap(FreeIDListSizesBuffer.UAV);
						for (uint32 iInstance=0; iInstance < NumFreeIDUpdates; ++iInstance)
						{
							const FNiagaraGpuFreeIDUpdate& FreeIDUpdateInfo = FreeIDUpdates[iInstance];
							SCOPED_DRAW_EVENTF(RHICmdList, NiagaraGPUComputeFreeIDsEmitter, TEXT("Update Free ID Buffer - %s"), FreeIDUpdateInfo.ComputeContext->GetDebugSimFName());
							NiagaraComputeGPUFreeIDs(RHICmdList, FeatureLevel, FreeIDUpdateInfo.IDToIndexSRV, FreeIDUpdateInfo.NumAllocatedIDs, FreeIDUpdateInfo.FreeIDsUAV, FreeIDListSizesBuffer.UAV, iInstance);
							TransitionsFreeIDs.Emplace(FreeIDUpdateInfo.FreeIDsUAV, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute);
						}
						RHICmdList.EndUAVOverlap(FreeIDListSizesBuffer.UAV);
						RHICmdList.Transition(TransitionsFreeIDs);
					}
				}
			);
		}
	}

	const int32 StageTotalDispatches = TotalDispatchesThisFrame - StageStartTotalDispatches;
	GraphBuilder.AddPostExecuteCallback([&DispatchList]
	{
		// Clear dispatch groups
		// We do not release the counts as we won't do that until we finish the dispatches
		DispatchList.DispatchGroups.Empty();
	});

	// Tear down for tick pass
	SimulationSceneViews = TConstStridedView<FSceneView>();
	SceneTexturesUniformParams = nullptr;
	MobileSceneTexturesUniformParams = nullptr;
	SubstratePublicGlobalUniformParams = nullptr;
	bIsExecutingFirstDispatchGroup = false;

	CurrentPassExternalAccessQueue.Submit(GraphBuilder);
}

void FNiagaraGpuComputeDispatch::DispatchStage(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData, const FNiagaraSimStageData& SimStageData)
{
#if STATS
	FScopeCycleCounter SystemCounterStat(InstanceData.Context->SystemStatID);
	FScopeCycleCounter EmitterCounterStat(InstanceData.Context->EmitterStatID);
#endif

	// Setup source buffer
	if ( SimStageData.Source != nullptr )
	{
		SimStageData.Source->SetNumInstances(SimStageData.SourceNumInstances);
		SimStageData.Source->SetGPUInstanceCountBufferOffset(SimStageData.SourceCountOffset);
	}

	// Setup destination buffer
	int32 InstancesToSpawn = 0;
	if ( SimStageData.Destination != nullptr )
	{
		SimStageData.Destination->SetNumInstances(SimStageData.DestinationNumInstances);
		SimStageData.Destination->SetGPUInstanceCountBufferOffset(SimStageData.DestinationCountOffset);
		SimStageData.Destination->SetIDAcquireTag(FNiagaraComputeExecutionContext::TickCounter);

		if ( SimStageData.bFirstStage )
		{
			check(SimStageData.DestinationNumInstances >= SimStageData.SourceNumInstances);
			InstancesToSpawn = SimStageData.DestinationNumInstances - SimStageData.SourceNumInstances;
		}
		SimStageData.Destination->SetNumSpawnedInstances(InstancesToSpawn);
	}

	// Get dispatch count
	ENiagaraGpuDispatchType DispatchType = ENiagaraGpuDispatchType::OneD;
	FIntVector DispatchCount = FIntVector(0, 0, 0);
	FIntVector DispatchNumThreads = FIntVector(0, 0, 0);

	switch (SimStageData.StageMetaData->IterationSourceType)
	{
		case ENiagaraIterationSource::Particles:
		{
			DispatchType = ENiagaraGpuDispatchType::OneD;
			DispatchCount = FIntVector(SimStageData.DestinationNumInstances, 1, 1);
			DispatchNumThreads = FNiagaraShader::GetDefaultThreadGroupSize(ENiagaraGpuDispatchType::OneD);
			break;
		}

		case ENiagaraIterationSource::DataInterface:
		{
			if (SimStageData.AlternateIterationSource == nullptr)
			{
				return;
			}

			DispatchType = SimStageData.StageMetaData->GpuDispatchType;
			DispatchNumThreads = SimStageData.StageMetaData->GpuDispatchNumThreads;

			if (SimStageData.StageMetaData->bGpuIndirectDispatch)
			{
				if (SimStageData.DispatchArgs.IndirectBuffer == nullptr)
				{
					return;
				}
			}
			else
			{
				DispatchCount = SimStageData.DispatchArgs.ElementCount;
			
				// Verify the number of elements isn't higher that what we can handle
				checkf(uint64(DispatchCount.X) * uint64(DispatchCount.Y) * uint64(DispatchCount.Z) < uint64(TNumericLimits<int32>::Max()), TEXT("DispatchCount(%d, %d, %d) for IterationInterface(%s) overflows an int32 this is not allowed"), DispatchCount.X, DispatchCount.Y, DispatchCount.Z, *SimStageData.AlternateIterationSource->GetSourceDIName().ToString());

				// Data interfaces such as grids / render targets can choose to dispatch in either the correct dimensionality for the target (i.e. RT2D would choose 2D)
				// or run in linear mode if performance is not beneficial due to increased waves.  It is also possible the we may choose to override on the simulation stage.
				// Therefore we need to special case OneD and convert our element count back to linear.
				if (DispatchType == ENiagaraGpuDispatchType::OneD)
				{
					DispatchCount.X = DispatchCount.X * DispatchCount.Y * DispatchCount.Z;
					DispatchCount.Y = 1;
					DispatchCount.Z = 1;
				}
			}
			break;
		}

		case ENiagaraIterationSource::DirectSet:
		{
			DispatchType = SimStageData.StageMetaData->GpuDispatchType;
			DispatchNumThreads = SimStageData.StageMetaData->GpuDispatchNumThreads;

			switch (DispatchType)
			{
				case ENiagaraGpuDispatchType::OneD:
					DispatchCount = FIntVector(SimStageData.DispatchArgs.ElementCount.X, 1, 1);
					break;
				case ENiagaraGpuDispatchType::TwoD:
					DispatchCount = FIntVector(SimStageData.DispatchArgs.ElementCount.X, SimStageData.DispatchArgs.ElementCount.Y, 1);
					break;
				case ENiagaraGpuDispatchType::ThreeD:
					DispatchCount = SimStageData.DispatchArgs.ElementCount;
					break;
				default:
					UE_LOGF(LogNiagara, Fatal, "FNiagaraGpuComputeDispatch: Unknown DispatchType(%d)", int(DispatchType));
					break;
			}

			// If we are dispatch groups, not threads, then we need to * by the thread group size
			if (SimStageData.StageMetaData->GpuDirectDispatchElementType == ENiagaraDirectDispatchElementType::NumGroups)
			{
				DispatchCount *= DispatchNumThreads;
			}
			break;
		}

		default:
			UE_LOGF(LogNiagara, Fatal, "FNiagaraGpuComputeDispatch: Unknown IterationSouce(%d)", int(SimStageData.StageMetaData->IterationSourceType));
			return;
	}

	const int32 TotalDispatchCount = DispatchCount.X * DispatchCount.Y * DispatchCount.Z;
	if (TotalDispatchCount <= 0 && SimStageData.DispatchArgs.IndirectBuffer == nullptr)
	{
		return;
	}

	checkf(DispatchNumThreads.X * DispatchNumThreads.Y * DispatchNumThreads.Z > 0, TEXT("DispatchNumThreads(%d, %d, %d) is invalid"), DispatchNumThreads.X, DispatchNumThreads.Y, DispatchNumThreads.Z);

	// Setup our RDG UAV pool access
	FNiagaraEmptyRDGUAVPoolScopedAccess RDGUAVPoolAccessScope(GetEmptyUAVPool());

	// Set Parameters
	const FNiagaraShaderScriptParametersMetadata& NiagaraShaderParametersMetadata = InstanceData.Context->GPUScript_RT->GetScriptParametersMetadata_RT();
	const FShaderParametersMetadata* ShaderParametersMetadata = NiagaraShaderParametersMetadata.ShaderParametersMetadata.Get();
	FNiagaraShader::FParameters* DispatchParameters = GraphBuilder.AllocParameters<FNiagaraShader::FParameters>(ShaderParametersMetadata);

	const bool bRequiresPersistentIDs = InstanceData.Context->MainDataSet->RequiresPersistentIDs();

	DispatchParameters->SimStart = InstanceData.bResetData ? 1U : 0U;
	DispatchParameters->EmitterTickCounter = FNiagaraComputeExecutionContext::TickCounter;
	DispatchParameters->NumSpawnedInstances = InstancesToSpawn;
	DispatchParameters->FreeIDList = bRequiresPersistentIDs ? InstanceData.Context->MainDataSet->GetGPUFreeIDs().SRV.GetReference() : FNiagaraRenderer::GetDummyIntBuffer();

	// Set spawn Information
	// This parameter is an array of structs with 2 floats and 2 ints on CPU, but a float4 array on GPU. The shader uses asint() to cast the integer values. To set the parameter,
	// we pass the structure array as a float* to SetShaderValueArray() and specify the number of floats (not float vectors).
	static_assert((sizeof(InstanceData.SpawnInfo.SpawnInfoStartOffsets) % SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT) == 0, "sizeof SpawnInfoStartOffsets should be a multiple of SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT");
	static_assert((sizeof(InstanceData.SpawnInfo.SpawnInfoParams) % SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT) == 0, "sizeof SpawnInfoParams should be a multiple of SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT");
	static_assert(sizeof(DispatchParameters->EmitterSpawnInfoOffsets) == sizeof(InstanceData.SpawnInfo.SpawnInfoStartOffsets), "Mismatch between shader parameter size and SpawnInfoStartOffsets");
	static_assert(sizeof(DispatchParameters->EmitterSpawnInfoParams) == sizeof(InstanceData.SpawnInfo.SpawnInfoParams), "Mismatch between shader parameter size and SpawnInfoParams");
	FMemory::Memcpy(&DispatchParameters->EmitterSpawnInfoOffsets, &InstanceData.SpawnInfo.SpawnInfoStartOffsets, sizeof(InstanceData.SpawnInfo.SpawnInfoStartOffsets));
	FMemory::Memcpy(&DispatchParameters->EmitterSpawnInfoParams, &InstanceData.SpawnInfo.SpawnInfoParams, sizeof(InstanceData.SpawnInfo.SpawnInfoParams));

	// Setup instance counts
	DispatchParameters->RWInstanceCounts			= GPUInstanceCounterManager.GetInstanceCountBuffer().UAV;
	if (SimStageData.StageMetaData->IterationSourceType == ENiagaraIterationSource::Particles)
	{
		DispatchParameters->ReadInstanceCountOffset = SimStageData.SourceCountOffset;
		DispatchParameters->WriteInstanceCountOffset = SimStageData.DestinationCountOffset;
	}
	else
	{
		DispatchParameters->ReadInstanceCountOffset = INDEX_NONE;
		DispatchParameters->WriteInstanceCountOffset = INDEX_NONE;
	}

	// Simulation Stage Information
	// X = Count Buffer Instance Count Offset (INDEX_NONE == Use Instance Count)
	// Y = Instance Count
	// Z = Iteration Index
	// W = Num Iterations
	{
		DispatchParameters->SimulationStageIterationInfo = FUintVector4(INDEX_NONE, 0, 0, 0);
		switch (SimStageData.StageMetaData->IterationSourceType)
		{
			case ENiagaraIterationSource::Particles:
				break;

			case ENiagaraIterationSource::DataInterface:
				if (SimStageData.AlternateIterationSource != nullptr)	
				{
					const uint32 IterationInstanceCountOffset = SimStageData.DispatchArgs.GpuElementCountOffset;
					DispatchParameters->SimulationStageIterationInfo.X = IterationInstanceCountOffset;
					DispatchParameters->SimulationStageIterationInfo.Y = IterationInstanceCountOffset == INDEX_NONE ? TotalDispatchCount : 0;
				}
				break;

			case ENiagaraIterationSource::DirectSet:
				DispatchParameters->SimulationStageIterationInfo.Y = TotalDispatchCount;
				break;
		}

		DispatchParameters->SimulationStageIterationInfo.Z  = uint32(SimStageData.NumIterations) << 16;
		DispatchParameters->SimulationStageIterationInfo.Z |= uint32(SimStageData.IterationIndex);

		DispatchParameters->SimulationStageIterationInfo.W  = uint32(SimStageData.NumLoops) << 16;
		DispatchParameters->SimulationStageIterationInfo.W |= uint32(SimStageData.LoopIndex);
	}

	// Set particle iteration state info
	// Where X = Parameter Binding, YZ = Inclusive Range
	DispatchParameters->ParticleIterationStateInfo.X = SimStageData.StageMetaData->ParticleIterationStateComponentIndex;
	DispatchParameters->ParticleIterationStateInfo.Y = SimStageData.StageMetaData->ParticleIterationStateRange.X;
	DispatchParameters->ParticleIterationStateInfo.Z = SimStageData.StageMetaData->ParticleIterationStateRange.Y;

	// Set static input buffers
	DispatchParameters->StaticInputFloat = Tick.SystemGpuComputeProxy->StaticFloatBuffer;

	// Set Particle Input Buffer
	if ( (SimStageData.Source != nullptr) && (SimStageData.Source->GetNumInstancesAllocated() > 0) )
	{
		DispatchParameters->InputFloat = SimStageData.Source->GetGPUBufferFloat().SRV;
		DispatchParameters->InputHalf = SimStageData.Source->GetGPUBufferHalf().SRV;
		DispatchParameters->InputInt = SimStageData.Source->GetGPUBufferInt().SRV;
		DispatchParameters->ComponentBufferSizeRead = SimStageData.Source->GetFloatStride() / sizeof(float);
	}
	else
	{
		DispatchParameters->InputFloat = FNiagaraRenderer::GetDummyFloatBuffer();
		DispatchParameters->InputHalf = FNiagaraRenderer::GetDummyHalfBuffer();
		DispatchParameters->InputInt = FNiagaraRenderer::GetDummyIntBuffer();
		DispatchParameters->ComponentBufferSizeRead = 0;
	}

	// Set Particle Output Buffer
	if (SimStageData.Destination != nullptr)
	{
		DispatchParameters->RWOutputFloat = SimStageData.Destination->GetGPUBufferFloat().UAV;
		DispatchParameters->RWOutputHalf = SimStageData.Destination->GetGPUBufferHalf().UAV;
		DispatchParameters->RWOutputInt = SimStageData.Destination->GetGPUBufferInt().UAV;
		DispatchParameters->RWIDToIndexTable = SimStageData.Destination->GetGPUIDToIndexTable().UAV;
		DispatchParameters->ComponentBufferSizeWrite = SimStageData.Destination->GetFloatStride() / sizeof(float);
	}
	else
	{
		DispatchParameters->RWOutputFloat = nullptr;
		DispatchParameters->RWOutputHalf = nullptr;
		DispatchParameters->RWOutputInt = nullptr;
		DispatchParameters->RWIDToIndexTable = nullptr;
		DispatchParameters->ComponentBufferSizeWrite = 0;
	}

	// Set Compute Shader
	const TShaderRef<FNiagaraShader> ComputeShader = InstanceData.Context->GPUScript_RT->GetShader(SimStageData.StageIndex);

	// Set data interface parameters
	SetDataInterfaceParameters(GraphBuilder, Tick, InstanceData, ComputeShader, SimStageData, NiagaraShaderParametersMetadata, reinterpret_cast<uint8*>(DispatchParameters));

	// Set tick parameters
	Tick.GetGlobalParameters(InstanceData, &DispatchParameters->GlobalParameters);
	Tick.GetSystemParameters(InstanceData, &DispatchParameters->SystemParameters);
	Tick.GetOwnerParameters(InstanceData, &DispatchParameters->OwnerParameters);
	Tick.GetEmitterParameters(InstanceData, &DispatchParameters->EmitterParameters);

	// Set ViewUniformBuffer if it's required
	if (ComputeShader->bNeedsViewUniformBuffer)
	{
		DispatchParameters->View = SimulationSceneViews[0].ViewUniformBuffer;
	}
	DispatchParameters->SceneTextures.SceneTextures			= SceneTexturesUniformParams;
	DispatchParameters->SceneTextures.MobileSceneTextures	= MobileSceneTexturesUniformParams;
	DispatchParameters->SubstratePublic						= SubstratePublicGlobalUniformParams;

	// Indirect Gpu Dispatch
	if (SimStageData.StageMetaData->bGpuIndirectDispatch)
	{
		DispatchParameters->IndirectDispatchArgsBuffer = SimStageData.DispatchArgs.IndirectBuffer;
		DispatchParameters->IndirectDispatchArgs = GraphBuilder.CreateSRV(SimStageData.DispatchArgs.IndirectBuffer, EPixelFormat::PF_R32G32B32A32_UINT);
		DispatchParameters->IndirectDispatchArgsOffset = (SimStageData.DispatchArgs.IndirectOffset / sizeof(FUintVector4)) + 1;

		GraphBuilder.AddPass(
			RDG_EVENT_NAME(
				"NiagaraGpuSim(%s) Indirect Stage(%s %u) Iteration(%u) NumThreads(%dx%dx%d)",
				InstanceData.Context->GetDebugSimName(),
				*SimStageData.StageMetaData->SimulationStageName.ToString(),
				SimStageData.StageIndex,
				SimStageData.IterationIndex,
				DispatchNumThreads.X, DispatchNumThreads.Y, DispatchNumThreads.Z
			),
			ShaderParametersMetadata,
			DispatchParameters,
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			[this, ShaderParametersMetadata, DispatchParameters, ComputeShader, TickPtr=&Tick, InstanceDataPtr=&InstanceData, SimStageDataPtr=&SimStageData](FRHICommandList& RHICmdList)
			{
				FRHIComputeShader* RHIComputeShader = ComputeShader.GetComputeShader();
				FComputePipelineStateInitializer Initializer(RHIComputeShader, ComputeShader.GetSpecializationConstants());
				SetComputePipelineState(RHICmdList, Initializer);

				FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

				if (ComputeShader->ExternalConstantBufferParam[0].IsBound())
				{
					BatchedParameters.SetShaderUniformBuffer(ComputeShader->ExternalConstantBufferParam[0].GetBaseIndex(), TickPtr->GetExternalUniformBuffer(*InstanceDataPtr, false));
				}
				if (ComputeShader->ExternalConstantBufferParam[1].IsBound())
				{
					check(InstanceDataPtr->Context->HasInterpolationParameters);
					BatchedParameters.SetShaderUniformBuffer(ComputeShader->ExternalConstantBufferParam[1].GetBaseIndex(), TickPtr->GetExternalUniformBuffer(*InstanceDataPtr, true));
				}

				FNiagaraEmptyUAVPoolScopedAccess UAVPoolAccessScope(GetEmptyUAVPool());

				FNiagaraGpuProfileScope GpuProfileDispatchScope(RHICmdList, this, FNiagaraGpuProfileEvent(*InstanceDataPtr, *SimStageDataPtr, InstanceDataPtr == &TickPtr->GetInstances()[0]));

				FComputeShaderUtils::ValidateIndirectArgsBuffer(SimStageDataPtr->DispatchArgs.IndirectBuffer->GetSize(), SimStageDataPtr->DispatchArgs.IndirectOffset);
				SetShaderParameters(BatchedParameters, ComputeShader, ShaderParametersMetadata, *DispatchParameters);

				RHICmdList.SetBatchedShaderParameters(RHIComputeShader, BatchedParameters);

				RHICmdList.DispatchIndirectComputeShader(SimStageDataPtr->DispatchArgs.IndirectBuffer->GetIndirectRHICallBuffer(), SimStageDataPtr->DispatchArgs.IndirectOffset);
				UnsetShaderUAVs<FRHICommandList, FNiagaraShader>(RHICmdList, ComputeShader, RHIComputeShader);
			}
		);
	}
	// Direct Gpu Dispatch
	else
	{
		// Calculate thread groups
		// Note: In the OneD case we can use the Y dimension to get higher particle counts
		if (DispatchType == ENiagaraGpuDispatchType::OneD)
		{
			const int32 GroupCount = FMath::DivideAndRoundUp(DispatchCount.X, DispatchNumThreads.X);
			if (GroupCount > GRHIMaxDispatchThreadGroupsPerDimension.X)
			{
				DispatchCount.Y = FMath::DivideAndRoundUp(GroupCount, GRHIMaxDispatchThreadGroupsPerDimension.X);
				DispatchCount.X = FMath::DivideAndRoundUp(GroupCount, DispatchCount.Y) * DispatchNumThreads.X;
				ensure(DispatchCount.Y <= GRHIMaxDispatchThreadGroupsPerDimension.Y);
			}
		}

		const FIntVector ThreadGroupCount(
			FMath::DivideAndRoundUp(DispatchCount.X, DispatchNumThreads.X),
			FMath::DivideAndRoundUp(DispatchCount.Y, DispatchNumThreads.Y),
			FMath::DivideAndRoundUp(DispatchCount.Z, DispatchNumThreads.Z)
		);

		// Validate we don't overflow ThreadGroupCounts
		if ((ThreadGroupCount.X < 0) || (ThreadGroupCount.X > GRHIMaxDispatchThreadGroupsPerDimension.X) ||
			(ThreadGroupCount.Y < 0) || (ThreadGroupCount.Y > GRHIMaxDispatchThreadGroupsPerDimension.Y) ||
			(ThreadGroupCount.Z < 0) || (ThreadGroupCount.Z > GRHIMaxDispatchThreadGroupsPerDimension.Z) )
		{
			UE_LOGF(LogNiagara, Warning, "FNiagaraGpuComputeDispatch: Invalid ThreadGroupdCount(%d, %d, %d) for ElementCount(%d, %d, %d) Stage (%ls)",
				ThreadGroupCount.X, ThreadGroupCount.Y, ThreadGroupCount.Z,
				DispatchCount.X, DispatchCount.Y, DispatchCount.Z,
				*SimStageData.StageMetaData->SimulationStageName.ToString()
			);
			return;
		}

		DispatchParameters->DispatchThreadIdToLinear	= FUintVector3(1, DispatchCount.X, DispatchCount.X * DispatchCount.Y);
		DispatchParameters->DispatchThreadIdBounds		= FUintVector3(DispatchCount.X, DispatchCount.Y, DispatchCount.Z);

		// Simple case can be this, i.e. no legacy DI's, no External Constants, no profiling
		//FComputeShaderUtils::AddPass(
		//	GraphBuilder,
		//	RDG_EVENT_NAME("NiagaraStage"),
		//	ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		//	ComputeShader,
		//	&ShaderParametersMetadata,
		//	DispatchParameters,
		//	ThreadGroupCount
		//);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME(
				"NiagaraGpuSim(%s) DispatchCount(%dx%dx%d) Stage(%s %u) Iteration(%u) NumThreads(%dx%dx%d)",
				InstanceData.Context->GetDebugSimName(),
				DispatchCount.X, DispatchCount.Y, DispatchCount.Z,
				*SimStageData.StageMetaData->SimulationStageName.ToString(),
				SimStageData.StageIndex,
				SimStageData.IterationIndex,
				DispatchNumThreads.X, DispatchNumThreads.Y, DispatchNumThreads.Z
			),
			ShaderParametersMetadata,
			DispatchParameters,
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			[this, ShaderParametersMetadata, DispatchParameters, ComputeShader, ThreadGroupCount, TickPtr=&Tick, InstanceDataPtr=&InstanceData, SimStageDataPtr=&SimStageData](FRHICommandList& RHICmdList)
			{
				FRHIComputeShader* RHIComputeShader = ComputeShader.GetComputeShader();
				FComputePipelineStateInitializer Initializer(RHIComputeShader, ComputeShader.GetSpecializationConstants());
				SetComputePipelineState(RHICmdList, Initializer);

				FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

				if (ComputeShader->ExternalConstantBufferParam[0].IsBound())
				{
					BatchedParameters.SetShaderUniformBuffer(ComputeShader->ExternalConstantBufferParam[0].GetBaseIndex(), TickPtr->GetExternalUniformBuffer(*InstanceDataPtr, false));
				}
				if (ComputeShader->ExternalConstantBufferParam[1].IsBound())
				{
					check(InstanceDataPtr->Context->HasInterpolationParameters);
					BatchedParameters.SetShaderUniformBuffer(ComputeShader->ExternalConstantBufferParam[1].GetBaseIndex(), TickPtr->GetExternalUniformBuffer(*InstanceDataPtr, true));
				}

				FNiagaraGpuProfileScope GpuProfileDispatchScope(RHICmdList, this, FNiagaraGpuProfileEvent(*InstanceDataPtr, *SimStageDataPtr, InstanceDataPtr == &TickPtr->GetInstances()[0]));

				SetShaderParameters(BatchedParameters, ComputeShader, ShaderParametersMetadata, *DispatchParameters);

				RHICmdList.SetBatchedShaderParameters(RHIComputeShader, BatchedParameters);

				DispatchComputeShader(RHICmdList, ComputeShader, ThreadGroupCount.X, ThreadGroupCount.Y, ThreadGroupCount.Z);
				UnsetShaderUAVs<FRHICommandList, FNiagaraShader>(RHICmdList, ComputeShader, RHIComputeShader);
			}
		);
	}

	INC_DWORD_STAT(STAT_NiagaraGPUDispatches);
#if CSV_PROFILER_STATS && WITH_PER_SYSTEM_PARTICLE_PERF_STATS && WITH_NIAGARA_DEBUG_EMITTER_NAME
	if (FNiagaraGpuComputeDispatchLocal::CsvStatsEnabled())
	{
		if (FCsvProfiler* CSVProfiler = FCsvProfiler::Get())
		{
			CSVProfiler->RecordCustomStat(InstanceData.Context->GetDebugSimFName(), CSV_CATEGORY_INDEX(NiagaraGpuCompute), 1, ECsvCustomStatOp::Accumulate);
		}
	}
#endif

	// Optionally submit commands to the GPU
	// This can be used to avoid accidental TDR detection in the editor especially when issuing multiple ticks in the same frame
	++TotalDispatchesThisFrame;
	//-TODO:RDG: Fix submit commands hint
	//if (GNiagaraGpuSubmitCommandHint > 0)
	//{
	//	if ((TotalDispatchesThisFrame % GNiagaraGpuSubmitCommandHint) == 0)
	//	{
	//		RHICmdList.SubmitCommandsHint();
	//	}
	//}
}

void FNiagaraGpuComputeDispatch::PreInitViews(FRDGBuilder& GraphBuilder, bool bAllowGPUParticleUpdate, const TArrayView<const FSceneViewFamily*>& ViewFamilies, const FSceneViewFamily* CurrentFamily)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraGPUDispatchSetup_RT);
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, Niagara);
	LLM_SCOPE(ELLMTag::Niagara);

	OnPreInitViewsEvent.Broadcast(GraphBuilder);

	bIsFirstViewFamily = CurrentFamily ? CurrentFamily == ViewFamilies[0] : true;
	bIsLastViewFamily = CurrentFamily ? CurrentFamily == ViewFamilies.Last() : true;
	bRequiresReadback = false;
#if WITH_EDITOR
	bRaisedWarningThisFrame = false;
#endif
#if WITH_MGPU
	bCrossGPUTransferEnabled = GNumExplicitGPUsForRendering > 1;

	// We want to optimize GPU transfers from the simulating GPU to other view rendering GPUs, meaning we want to defer fence waits on
	// the other GPUs until the data is needed.  To keep things simple, we only do this optimization if all the families are single GPU,
	// which will be true in practice for multiple families rendered via nDisplay.  Here we compute a mask defining which other GPUs we
	// want to optimize transfers for (ones that render other views).  GPUs which don't do any view rendering in the current frame don't
	// incur any performance cost waiting on the fences immediately (the default behavior of cross GPU transfers), so we don't need to
	// delay the fence wait for those.
	if (bIsFirstViewFamily)
	{
		OptimizedCrossGPUTransferMask = 0;
	}

	if ((ViewFamilies.Num() > 1) && bIsFirstViewFamily && GNiagaraOptimizedCrossGPUTransferEnabled)
	{
		bool bOptimizeTransfers = true;
		FRHIGPUMask SimGPUMask = ViewFamilies[0]->Views[0]->GPUMask;
		FRHIGPUMask AllViewGPUMask = SimGPUMask;

		for (const FSceneViewFamily* Family : ViewFamilies)
		{
			FRHIGPUMask FamilyFirstViewMask = Family->Views[0]->GPUMask;
			AllViewGPUMask |= FamilyFirstViewMask;

			if (!FamilyFirstViewMask.HasSingleIndex())
			{
				bOptimizeTransfers = false;
				break;
			}
			for (const FSceneView* View : Family->Views)
			{
				if (View->GPUMask != FamilyFirstViewMask)
				{
					bOptimizeTransfers = false;
					break;
				}
			}
			if (!bOptimizeTransfers)
			{
				break;
			}
		}

		if (bOptimizeTransfers)
		{
			// Figure out what destination GPUs we should optimize transfers to
			OptimizedCrossGPUTransferMask = AllViewGPUMask.GetNative() & ~SimGPUMask.GetNative();
		}
	}
#endif
	if ((ViewFamilies.Num() > 1) && bIsFirstViewFamily)
	{
		GPUInstanceCounterManager.CopyToMultiViewCountBuffer(GraphBuilder.RHICmdList);
	}

	EmptyUAVPoolPtr->Tick();
	GpuReadbackManagerPtr->Tick(GraphBuilder.RHICmdList);
#if NIAGARA_COMPUTEDEBUG_ENABLED
	if ( FNiagaraGpuComputeDebug* GpuComputeDebug = GpuComputeDebugPtr.Get() )
	{
		GpuComputeDebug->Tick(GraphBuilder);
	}
#endif

	// If we during a scene render ensure remove the GDF cache data	to avoid holding onto GDF textures that could be reallocated
	if (IsOutsideSceneRenderer() == false)
	{
		CachedGDFData = FCachedDistanceFieldData();
	}
	
	TotalDispatchesThisFrame = 0;

	// Add pass to begin the gpu profiler frame
#if WITH_NIAGARA_GPU_PROFILER
	GPUProfilerPtr->BeginFrame(GraphBuilder.RHICmdList);
#endif

#if STATS
	if (FNiagaraGpuComputeDispatchLocal::GDumpSortedSimulations > 0)
	{
		--FNiagaraGpuComputeDispatchLocal::GDumpSortedSimulations;

		UE_LOGF(LogNiagara, Log, "Niagara SimulationsToSort:");
		for (const FNiagaraGPUSortInfo& SortInfo : SimulationsToSort)
		{
			TStringBuilder<256> EnabledFeatures;
			if (SortInfo.SortMode != ENiagaraSortMode::None && SortInfo.SortAttributeOffset != INDEX_NONE)
			{
				EnabledFeatures.Append(TEXT("SortingEnabled "));
			}
			if (SortInfo.bEnableCulling)
			{
				EnabledFeatures.Append(TEXT("CullingEnabled "));
				if (SortInfo.CullPlanes.Num() > 0)
				{
					EnabledFeatures.Append(TEXT("FrustumCulling "));
				}
				if (SortInfo.RendererVisTagAttributeOffset != INDEX_NONE)
				{
					EnabledFeatures.Append(TEXT("RendererVisEnabled "));
				}
				if (SortInfo.MeshIndexAttributeOffset != INDEX_NONE)
				{
					EnabledFeatures.Append(TEXT("MeshIndexEnabled "));
				}
			}
			UE_LOGF(LogNiagara, Log, " - Owner(%ls) ParticleCount(%d) %ls", *SortInfo.OwnerName.ToString(), SortInfo.ParticleCount, EnabledFeatures.ToString());
		}
	}
#endif

	// Reset the list of GPUSort tasks and release any resources they hold on to.
	// It might be worth considering doing so at the end of the render to free the resources immediately.
	// (note that currently there are no callback appropriate to do it)
	SimulationsToSort.Reset();

	if (FNiagaraUtilities::AllowGPUParticles(GetShaderPlatform()))
	{
		if ( bAllowGPUParticleUpdate )
		{
			FramesBeforeTickFlush = 0;

			for (auto It = NDCDataProxies.CreateIterator(); It; ++It)
			{
				FNiagaraDataChannelDataProxyPtr& NDCProxy = *It;
				if (NDCProxy.IsUnique() == false)
				{
					NDCProxy->BeginFrame(this, GraphBuilder.RHICmdList);
				}
				else
				{
					NDCProxy->OnRemovedFromDispatcher(this);
					It.RemoveCurrentSwap();
				}
			}
			
			FNiagaraGpuComputeDataManagerTickContext Context(GraphBuilder);
			for (auto& DataManager : GpuDataManagers)
			{
				if (DataManager.Value)
				{
					DataManager.Value->BeginFrame(Context);
				}
			}

			UpdateInstanceCountManager(GraphBuilder.RHICmdList);
			PrepareAllTicks(GraphBuilder.RHICmdList);

			AsyncGpuTraceHelper->BeginFrame(GraphBuilder.RHICmdList, this);

			if ( FNiagaraGpuComputeDispatchLocal::GDebugLogging)
			{
				DumpDebugFrame();
			}

			ExecuteTicks(GraphBuilder, TConstStridedView<FSceneView>(), ENiagaraGpuComputeTickStage::PreInitViews);
		}
		else
		{
			// We need to update the count buffer with the requested estimate from our proxies as they may still run compute work
			// Not doing this means a system which assumes it has a free count can fail due to the buffer being zero sized for example
			GPUInstanceCounterManager.ResizeBuffers(GraphBuilder.RHICmdList, ProxyGpuCountBufferEstimate);
		}
	}
	else
	{
		GPUInstanceCounterManager.ResizeBuffers(GraphBuilder.RHICmdList, ProxyGpuCountBufferEstimate);
		FinishDispatches();
	}

#if WITH_MGPU
	// If we have any simulations ticking we need to add the counter buffer to the list of buffers to transfer
	// This must be done during graph building rather than when we transfer as the GPU mask will be all nodes
	auto HasAnyWork =
		[&]() -> bool
		{
			for (const FNiagaraGpuDispatchList& DispatchList : DispatchListPerStage)
			{
				if (DispatchList.HasWork())
				{
					return true;
				}
			}
			return false;
		};

	if (bAllowGPUParticleUpdate && ((IsFirstViewFamily() && GPUInstanceCounterManager.HasEntriesPendingFree()) || HasAnyWork()))
	{
		MultiGPUResourceModified(GraphBuilder, GPUInstanceCounterManager.GetInstanceCountBuffer().Buffer, true, true);
	}
#endif
}

void FNiagaraGpuComputeDispatch::PostInitViews(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, bool bAllowGPUParticleUpdate)
{
	LLM_SCOPE(ELLMTag::Niagara);

	bAllowGPUParticleUpdate = bAllowGPUParticleUpdate && Views.Num() > 0 && Views[0].AllowGPUParticleUpdate();

	if (bAllowGPUParticleUpdate && FNiagaraUtilities::AllowGPUParticles(GetShaderPlatform()))
	{
		RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, Niagara);

		ExecuteTicks(GraphBuilder, Views, ENiagaraGpuComputeTickStage::PostInitViews);
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FNiagaraGpuComputeDispatchParameters,)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
END_SHADER_PARAMETER_STRUCT()

static FNiagaraGpuComputeDispatchParameters *MakeParameters(FRDGBuilder& GraphBuilder, TRDGUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRDG)
{
	FNiagaraGpuComputeDispatchParameters *Parameters = GraphBuilder.AllocParameters<FNiagaraGpuComputeDispatchParameters>();
	Parameters->Scene = SceneUniformBufferRDG;
	return Parameters;
}

void FNiagaraGpuComputeDispatch::PostRenderOpaque(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleUpdate)
{
	LLM_SCOPE(ELLMTag::Niagara);

	bAllowGPUParticleUpdate = bAllowGPUParticleUpdate && Views.Num() > 0 && Views[0].AllowGPUParticleUpdate();

	// If we are during a scene render cache the GDF information for any potential tick flushing
	// This fixes issues with distance field collisions (for example) where we may be scrubbing backwards and are force to flush to avoid TDRs / RDG pass limits
	// The alternative to this would be to query the FSceneInterface but we currently do not have a clean way to do this as the GDF clipmaps are built transiently plus the ViewStates do not exist
	if (IsOutsideSceneRenderer() == false)
	{
		if (const FGlobalDistanceFieldParameterData* SceneGDFData = UE::FXRenderingUtils::GetGlobalDistanceFieldParameterData(Views))
		{
			CachedGDFData.bCacheValid			= true;
			CachedGDFData.PageAtlasTexture		= SceneGDFData->PageAtlasTexture;
			CachedGDFData.CoverageAtlasTexture	= SceneGDFData->CoverageAtlasTexture;
			CachedGDFData.PageObjectGridBuffer	= SceneGDFData->PageObjectGridBuffer;
			CachedGDFData.PageTableTexture		= SceneGDFData->PageTableTexture;
			CachedGDFData.MipTexture			= SceneGDFData->MipTexture;
			CachedGDFData.GDFParameterData		= *SceneGDFData;
		}
		else
		{
			CachedGDFData						= FCachedDistanceFieldData();
		}
	}
	TGuardValue<bool> GuardGDFValidForPass(CachedGDFData.bValidForPass, CachedGDFData.bCacheValid);

	// Cache view information which will be used if we have to flush simulation commands in the future
	if ( bAllowGPUParticleUpdate && Views.IsValidIndex(0) )
	{
		if (const FSceneViewFamily* ViewFamily = Views[0].Family)
		{
			CachedViewInitOptions.GameTime			= ViewFamily->Time;
		}

		CachedViewInitOptions.ViewRect				= UE::FXRenderingUtils::GetRawViewRectUnsafe(Views[0]);
		CachedViewInitOptions.ViewOrigin			= Views[0].SceneViewInitOptions.ViewOrigin;
		CachedViewInitOptions.ViewRotationMatrix	= Views[0].SceneViewInitOptions.ViewRotationMatrix;
		CachedViewInitOptions.ProjectionMatrix		= Views[0].SceneViewInitOptions.ProjectionMatrix;
	}

	TRDGUniformBufferRef<FSceneUniformParameters> SceneUniformBufferRDG = UE::FXRenderingUtils::GetSceneUniformBuffer(GraphBuilder, SceneUniformBuffer);

	if (bAllowGPUParticleUpdate && FNiagaraUtilities::AllowGPUParticles(GetShaderPlatform()))
	{
		auto PassParameters = MakeParameters(GraphBuilder, SceneUniformBufferRDG);
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("AsyncGpuTraceHelper::PostRenderOpaque"),
			PassParameters,
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			[this, Views, PassParameters](FRHICommandListImmediate& RHICmdList)
			{
				AsyncGpuTraceHelper->PostRenderOpaque(RHICmdList, this, Views, PassParameters->Scene->GetRHIRef());
			}
		);

		ExecuteTicks(GraphBuilder, Views, ENiagaraGpuComputeTickStage::PostOpaqueRender);
	}
	{
		auto PassParameters = MakeParameters(GraphBuilder, SceneUniformBufferRDG);
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("Niagara::PostRenderFinish"),
			PassParameters,
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			[this, bExecuteReadback=bRequiresReadback, PassParameters, bAllowGPUParticleUpdate](FRHICommandListImmediate& RHICmdList)
			{
				if (bAllowGPUParticleUpdate)
				{
					AsyncGpuTraceHelper->EndFrame(RHICmdList, this, PassParameters->Scene->GetRHIRef());
				}

				ProcessDebugReadbacks(RHICmdList, false);

				if (bExecuteReadback)
				{
					check(!GPUInstanceCounterManager.HasPendingGPUReadback());
					GPUInstanceCounterManager.EnqueueGPUReadback(RHICmdList);
				}
			}
		);
	}

	GraphBuilder.AddPostExecuteCallback(
		[this, &RHICmdList=GraphBuilder.RHICmdList, &GraphBuilder, bAllowGPUParticleUpdate]
		{
		#if WITH_NIAGARA_GPU_PROFILER
			GPUProfilerPtr->EndFrame(RHICmdList);
		#endif
		
			if (bAllowGPUParticleUpdate)
			{
				FinishDispatches();

				for (auto It = NDCDataProxies.CreateIterator(); It; ++It)
				{
					FNiagaraDataChannelDataProxyPtr& NDCProxy = *It;
					if(NDCProxy.IsUnique() == false)
					{
						NDCProxy->EndFrame(this, RHICmdList);
					}
					else
					{
						NDCProxy->OnRemovedFromDispatcher(this);
						It.RemoveCurrentSwap();
					}
				}
			}
			
			FNiagaraGpuComputeDataManagerTickContext Context(GraphBuilder);
			for (auto& DataManager : GpuDataManagers)
			{
				if (DataManager.Value)
				{
					DataManager.Value->EndFrame(Context);
				}
			}
		}
	);

	bRequiresReadback = false;

	OnPostRenderEvent.Broadcast(GraphBuilder);

	if (FNiagaraGpuComputeDispatchLocal::CsvStatsEnabled())
	{
		CSV_CUSTOM_STAT(NiagaraGpuCompute, TotalDispatchesThisFrame, TotalDispatchesThisFrame, ECsvCustomStatOp::Set);
	}
}

void FNiagaraGpuComputeDispatch::ProcessDebugReadbacks(FRHICommandList& RHICmdList, bool bWaitCompletion)
{
#if !UE_BUILD_SHIPPING
	// Execute any pending readbacks as the ticks have now all been processed
	for (const FDebugReadbackInfo& DebugReadback : GpuDebugReadbackInfos)
	{
		FNiagaraDataBuffer* CurrentDataBuffer = DebugReadback.Context->MainDataSet->GetCurrentData();
		if ( CurrentDataBuffer == nullptr )
		{
			// Data is invalid
			DebugReadback.DebugInfo->Frame.CopyFromGPUReadback(nullptr, nullptr, nullptr, 0, 0, 0, 0, 0);
			DebugReadback.DebugInfo->bWritten = true;
			continue;
		}

		const uint32 CountOffset = CurrentDataBuffer->GetGPUInstanceCountBufferOffset();
		if (CountOffset == INDEX_NONE)
		{
			// Data is invalid
			DebugReadback.DebugInfo->Frame.CopyFromGPUReadback(nullptr, nullptr, nullptr, 0, 0, 0, 0, 0);
			DebugReadback.DebugInfo->bWritten = true;
			continue;
		}

		// Execute readback
		constexpr int32 MaxReadbackBuffers = 4;
		TArray<FRHIBuffer*, TInlineAllocator<MaxReadbackBuffers>> ReadbackBuffers;

		const int32 CountBufferIndex = ReadbackBuffers.Add(GPUInstanceCounterManager.GetInstanceCountBuffer().Buffer);
		const int32 FloatBufferIndex = (CurrentDataBuffer->GetGPUBufferFloat().NumBytes == 0) ? INDEX_NONE : ReadbackBuffers.Add(CurrentDataBuffer->GetGPUBufferFloat().Buffer);
		const int32 HalfBufferIndex = (CurrentDataBuffer->GetGPUBufferHalf().NumBytes == 0) ? INDEX_NONE : ReadbackBuffers.Add(CurrentDataBuffer->GetGPUBufferHalf().Buffer);
		const int32 IntBufferIndex = (CurrentDataBuffer->GetGPUBufferInt().NumBytes == 0) ? INDEX_NONE : ReadbackBuffers.Add(CurrentDataBuffer->GetGPUBufferInt().Buffer);

		const int32 FloatBufferStride = CurrentDataBuffer->GetFloatStride();
		const int32 HalfBufferStride = CurrentDataBuffer->GetHalfStride();
		const int32 IntBufferStride = CurrentDataBuffer->GetInt32Stride();

		GpuReadbackManagerPtr->EnqueueReadbacks(
			RHICmdList,
			MakeArrayView(ReadbackBuffers),
			[=, DebugInfo=DebugReadback.DebugInfo](TConstArrayView<TPair<void*, uint32>> BufferData)
			{
				checkf(4 + (CountOffset * 4) <= BufferData[CountBufferIndex].Value, TEXT("CountOffset %d is out of bounds (%d)"), CountOffset, BufferData[CountBufferIndex].Value);
				const int32 InstanceCount = reinterpret_cast<int32*>(BufferData[CountBufferIndex].Key)[CountOffset];
				const float* FloatDataBuffer = FloatBufferIndex == INDEX_NONE ? nullptr : reinterpret_cast<float*>(BufferData[FloatBufferIndex].Key);
				const FFloat16* HalfDataBuffer = HalfBufferIndex == INDEX_NONE ? nullptr : reinterpret_cast<FFloat16*>(BufferData[HalfBufferIndex].Key);
				const int32* IntDataBuffer = IntBufferIndex == INDEX_NONE ? nullptr : reinterpret_cast<int32*>(BufferData[IntBufferIndex].Key);

				DebugInfo->Frame.CopyFromGPUReadback(FloatDataBuffer, IntDataBuffer, HalfDataBuffer, 0, InstanceCount, FloatBufferStride, IntBufferStride, HalfBufferStride);
				DebugInfo->bWritten = true;
			}
		);
	}
	GpuDebugReadbackInfos.Empty();

	if (bWaitCompletion)
	{
		GpuReadbackManagerPtr->WaitCompletion(RHICmdList.GetAsImmediate());
	}
#endif
}

bool FNiagaraGpuComputeDispatch::UsesGlobalDistanceField() const
{
	return NumProxiesThatRequireGlobalDistanceField > 0;
}

bool FNiagaraGpuComputeDispatch::UsesDepthBuffer() const
{
	return NumProxiesThatRequireDepthBuffer > 0;
}

bool FNiagaraGpuComputeDispatch::RequiresEarlyViewUniformBuffer() const
{
	return NumProxiesThatRequireEarlyViewData > 0;
}

bool FNiagaraGpuComputeDispatch::RequiresRayTracingScene() const
{
	return NumProxiesThatRequireRayTracingScene > 0;
}

void FNiagaraGpuComputeDispatch::PreRender(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleUpdate)
{
	OnPreRenderEvent.Broadcast(GraphBuilder);

	if (!FNiagaraUtilities::AllowGPUParticles(GetShaderPlatform()))
	{
		return;
	}

	LLM_SCOPE(ELLMTag::Niagara);
}

void FNiagaraGpuComputeDispatch::OnMarkPendingKill()
{
	FNiagaraWorldManager::OnComputeDispatchInterfaceDestroyed(this);
	FFXSystemInterface::OnMarkPendingKill();
}

bool FNiagaraGpuComputeDispatch::AddSortedGPUSimulation(FRHICommandListBase& RHICmdList, FNiagaraGPUSortInfo& SortInfo)
{
	if (GPUSortManager && GPUSortManager->AddTask(RHICmdList, SortInfo.AllocationInfo, SortInfo.ParticleCount, SortInfo.SortFlags))
	{
		// It's not worth currently to have a map between SortInfo.AllocationInfo.SortBatchId and the relevant indices in SimulationsToSort
		// because the number of batches is expect to be very small (1 or 2). If this change, it might be worth reconsidering.
		UE::TScopeLock Lock(AddSortedGPUSimulationMutex);
		SimulationsToSort.Add(SortInfo);
		return true;
	}
	else
	{
		return false;
	}
}

const FGlobalDistanceFieldParameterData* FNiagaraGpuComputeDispatch::GetGlobalDistanceFieldData() const
{
	return CachedGDFData.bValidForPass ? &CachedGDFData.GDFParameterData : nullptr;
}

void FNiagaraGpuComputeDispatch::GenerateSortKeys(FRHICommandListImmediate& RHICmdList, int32 BatchId, int32 NumElementsInBatch, EGPUSortFlags Flags, FRHIUnorderedAccessView* KeysUAV, FRHIUnorderedAccessView* ValuesUAV)
{
	const bool bHighPrecision = EnumHasAnyFlags(Flags, EGPUSortFlags::HighPrecisionKeys);
	const FGPUSortManager::FKeyGenInfo KeyGenInfo((uint32)NumElementsInBatch, bHighPrecision);

	const bool bUseWaveOps = FNiagaraSortKeyGenCS::UseWaveOps(ShaderPlatform);

	FNiagaraSortKeyGenCS::FPermutationDomain SortPermutationVector;
	SortPermutationVector.Set<FNiagaraSortKeyGenCS::FSortUsingMaxPrecision>(bHighPrecision);
	SortPermutationVector.Set<FNiagaraSortKeyGenCS::FEnableCulling>(false);
	SortPermutationVector.Set<FNiagaraSortKeyGenCS::FUseWaveOps>(bUseWaveOps);

	FNiagaraSortKeyGenCS::FPermutationDomain SortAndCullPermutationVector;
	SortAndCullPermutationVector.Set<FNiagaraSortKeyGenCS::FSortUsingMaxPrecision>(bHighPrecision);
	SortAndCullPermutationVector.Set<FNiagaraSortKeyGenCS::FEnableCulling>(true);
	SortAndCullPermutationVector.Set<FNiagaraSortKeyGenCS::FUseWaveOps>(bUseWaveOps);

	TShaderMapRef<FNiagaraSortKeyGenCS> SortKeyGenCS(GetGlobalShaderMap(FeatureLevel), SortPermutationVector);
	TShaderMapRef<FNiagaraSortKeyGenCS> SortAndCullKeyGenCS(GetGlobalShaderMap(FeatureLevel), SortAndCullPermutationVector);

	FRWBuffer* CulledCountsBuffer = GPUInstanceCounterManager.AcquireCulledCountsBuffer(RHICmdList);

	FNiagaraSortKeyGenCS::FParameters Params;
	Params.SortKeyMask = KeyGenInfo.SortKeyParams.X;
	Params.SortKeyShift = KeyGenInfo.SortKeyParams.Y;
	Params.SortKeySignBit = KeyGenInfo.SortKeyParams.Z;
	Params.OutKeys = KeysUAV;
	Params.OutParticleIndices = ValuesUAV;

	FRHIUnorderedAccessView* OverlapUAVs[3];
	uint32 NumOverlapUAVs = 0;

	OverlapUAVs[NumOverlapUAVs] = KeysUAV;
	++NumOverlapUAVs;
	OverlapUAVs[NumOverlapUAVs] = ValuesUAV;
	++NumOverlapUAVs;

	if (CulledCountsBuffer)
	{
		Params.OutCulledParticleCounts = CulledCountsBuffer->UAV;
		OverlapUAVs[NumOverlapUAVs] = CulledCountsBuffer->UAV;
		++NumOverlapUAVs;
	}
	else
	{
		// Note: We don't care that the buffer will be allowed to be reused
		FNiagaraEmptyUAVPoolScopedAccess UAVPoolAccessScope(GetEmptyUAVPool());
		Params.OutCulledParticleCounts = GetEmptyUAVFromPool(RHICmdList, PF_R32_UINT, ENiagaraEmptyUAVType::Buffer);
	}

	RHICmdList.BeginUAVOverlap(MakeArrayView(OverlapUAVs, NumOverlapUAVs));

	for (const FNiagaraGPUSortInfo& SortInfo : SimulationsToSort)
	{
		if (SortInfo.AllocationInfo.SortBatchId == BatchId)
		{
		#if STATS
			SCOPED_DRAW_EVENTF(RHICmdList, GenerateSortKeys, TEXT("Owner:%s"), SortInfo.OwnerName.ToString());
		#endif

			Params.NiagaraParticleDataFloat = SortInfo.ParticleDataFloatSRV;
			Params.NiagaraParticleDataHalf = SortInfo.ParticleDataHalfSRV;
			Params.NiagaraParticleDataInt = SortInfo.ParticleDataIntSRV;
			Params.GPUParticleCountBuffer = SortInfo.GPUParticleCountSRV;
			Params.FloatDataStride = SortInfo.FloatDataStride;
			Params.HalfDataStride = SortInfo.HalfDataStride;
			Params.IntDataStride = SortInfo.IntDataStride;
			Params.ParticleCount = SortInfo.ParticleCount;
			Params.GPUParticleCountOffset = SortInfo.GPUParticleCountOffset;
			Params.CulledGPUParticleCountOffset = SortInfo.CulledGPUParticleCountOffset;
			Params.EmitterKey = (uint32)SortInfo.AllocationInfo.ElementIndex << KeyGenInfo.ElementKeyShift;
			Params.OutputOffset = SortInfo.AllocationInfo.BufferOffset;
			Params.CameraPosition = (FVector3f)SortInfo.ViewOrigin;
			Params.CameraDirection = (FVector3f)SortInfo.ViewDirection;
			Params.SortMode = (uint32)SortInfo.SortMode;
			Params.SortAttributeOffset = SortInfo.SortAttributeOffset;
			Params.CullPositionAttributeOffset = SortInfo.CullPositionAttributeOffset;
			Params.CullOrientationAttributeOffset = SortInfo.CullOrientationAttributeOffset;
			Params.CullScaleAttributeOffset = SortInfo.CullScaleAttributeOffset;
			Params.RendererVisibility = SortInfo.RendererVisibility;
			Params.RendererVisTagAttributeOffset = SortInfo.RendererVisTagAttributeOffset;
			Params.MeshIndex = SortInfo.MeshIndex;
			Params.MeshIndexAttributeOffset = SortInfo.MeshIndexAttributeOffset;
			Params.CullDistanceRangeSquared = SortInfo.DistanceCullRange * SortInfo.DistanceCullRange;
			Params.LODScreenSize = SortInfo.LODScreenSize;
			Params.LocalBoundingSphere = FVector4f((FVector3f)SortInfo.LocalBSphere.Center, SortInfo.LocalBSphere.W);
			Params.CullingWorldSpaceOffset = (FVector3f)SortInfo.CullingWorldSpaceOffset;

			Params.NumCullPlanes = 0;
			for (const FPlane& Plane : SortInfo.CullPlanes)
			{
				Params.CullPlanes[Params.NumCullPlanes].X = float(Plane.X);
				Params.CullPlanes[Params.NumCullPlanes].Y = float(Plane.Y);
				Params.CullPlanes[Params.NumCullPlanes].Z = float(Plane.Z);
				Params.CullPlanes[Params.NumCullPlanes].W = float(Plane.W);	// LWC Precision loss
				++Params.NumCullPlanes;
			}

			// Choose the shader to bind
			const int32 NumThreadGroups = FMath::DivideAndRoundUp(SortInfo.ParticleCount, NIAGARA_KEY_GEN_THREAD_COUNT);
			FComputeShaderUtils::Dispatch(
				RHICmdList,
				SortInfo.bEnableCulling ? SortAndCullKeyGenCS : SortKeyGenCS,
				Params,
				FComputeShaderUtils::GetGroupCountWrapped(NumThreadGroups)
			);
		}
	}

	RHICmdList.EndUAVOverlap(MakeArrayView(OverlapUAVs, NumOverlapUAVs));
}

FNiagaraAsyncGpuTraceHelper& FNiagaraGpuComputeDispatch::GetAsyncGpuTraceHelper() const
{
	check(AsyncGpuTraceHelper.IsValid());
	return *AsyncGpuTraceHelper.Get();
}

void FNiagaraGpuComputeDispatch::ResetDataInterfaces(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData) 
{
	const int32 NumDataInterfaces = InstanceData.DataInterfaceProxies.Num();
	if (NumDataInterfaces == 0)
	{
		return;
	}

	const FNiagaraShaderRef& ComputeShader = InstanceData.Context->GPUScript_RT->GetShader(0);
	TConstArrayView<FNiagaraDataInterfaceParamRef> DIParameters = ComputeShader->GetDIParameters();

	FNDIGpuComputeResetContext Context(GraphBuilder, *this, Tick.SystemInstanceID);
	for (int32 iDataInterface=0; iDataInterface < NumDataInterfaces; ++iDataInterface)
	{
		const FNiagaraDataInterfaceParamRef& DIParam = DIParameters[iDataInterface];
		if (DIParam.ShaderParametersOffset != INDEX_NONE)
		{
			FNiagaraDataInterfaceProxy* DataInterfaceProxy = InstanceData.DataInterfaceProxies[iDataInterface];
			DataInterfaceProxy->ResetData(Context);
		}
	}
}

void FNiagaraGpuComputeDispatch::SetDataInterfaceParameters(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData, const FNiagaraShaderRef& ComputeShader, const FNiagaraSimStageData& SimStageData, const FNiagaraShaderScriptParametersMetadata& NiagaraShaderParametersMetadata, uint8* ParametersStructure) 
{
	const int32 NumDataInterfaces = InstanceData.DataInterfaceProxies.Num();
	if (NumDataInterfaces == 0)
	{
		return;
	}

	const FNiagaraShaderMapPointerTable& PointerTable = ComputeShader.GetPointerTable();
	TConstArrayView<FNiagaraDataInterfaceParamRef> DIParameters = ComputeShader->GetDIParameters();

	FNiagaraDataInterfaceSetShaderParametersContext Context(GraphBuilder, *this, Tick, InstanceData, SimStageData, ComputeShader, NiagaraShaderParametersMetadata, ParametersStructure);
	for ( int32 iDataInterface=0; iDataInterface < NumDataInterfaces; ++iDataInterface )
	{
		FNiagaraDataInterfaceProxy* DataInterfaceProxy = InstanceData.DataInterfaceProxies[iDataInterface];

		const FNiagaraDataInterfaceParamRef& DIParam = DIParameters[iDataInterface];
		if (DIParam.ShaderParametersOffset != INDEX_NONE)
		{
			Context.SetDataInterface(DataInterfaceProxy, DIParam.ShaderParametersOffset, DIParam.Parameters);
			CastChecked<UNiagaraDataInterface>(DIParam.DIType.Get(PointerTable.DITypes))->SetShaderParameters(Context);
		}
	}
}

void FNiagaraGpuComputeDispatch::PreStageInterface(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData, const FNiagaraSimStageData& SimStageData, TSet<FNiagaraDataInterfaceProxy*>& ProxiesToFinalize) 
{
	const int32 NumDataInterfaces = InstanceData.DataInterfaceProxies.Num();
	if (NumDataInterfaces == 0)
	{
		return;
	}

	const FNiagaraShaderRef& ComputeShader = InstanceData.Context->GPUScript_RT->GetShader(0);
	TConstArrayView<FNiagaraDataInterfaceParamRef> DIParameters = ComputeShader->GetDIParameters();

	FNDIGpuComputePreStageContext Context(GraphBuilder, *this, Tick, InstanceData, SimStageData);
	for (int32 iDataInterface = 0; iDataInterface < NumDataInterfaces; ++iDataInterface)
	{
		const FNiagaraDataInterfaceParamRef& DIParam = DIParameters[iDataInterface];
		if (DIParam.ShaderParametersOffset != INDEX_NONE)
		{
			FNiagaraDataInterfaceProxy* DataInterfaceProxy = InstanceData.DataInterfaceProxies[iDataInterface];
			Context.SetDataInterfaceProxy(DataInterfaceProxy);
			DataInterfaceProxy->PreStage(Context);
			if (DataInterfaceProxy->RequiresPreStageFinalize())
			{
				ProxiesToFinalize.Add(DataInterfaceProxy);
			}
		}
	}
}

void FNiagaraGpuComputeDispatch::PostStageInterface(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData, const FNiagaraSimStageData& SimStageData, TSet<FNiagaraDataInterfaceProxy*>& ProxiesToFinalize) 
{
	const int32 NumDataInterfaces = InstanceData.DataInterfaceProxies.Num();
	if (NumDataInterfaces == 0)
	{
		return;
	}

	const FNiagaraShaderRef& ComputeShader = InstanceData.Context->GPUScript_RT->GetShader(0);
	TConstArrayView<FNiagaraDataInterfaceParamRef> DIParameters = ComputeShader->GetDIParameters();

	FNDIGpuComputePostStageContext Context(GraphBuilder, *this, Tick, InstanceData, SimStageData);
	for (int32 iDataInterface = 0; iDataInterface < NumDataInterfaces; ++iDataInterface)
	{
		const FNiagaraDataInterfaceParamRef& DIParam = DIParameters[iDataInterface];
		if (DIParam.ShaderParametersOffset != INDEX_NONE)
		{
			FNiagaraDataInterfaceProxy* DataInterfaceProxy = InstanceData.DataInterfaceProxies[iDataInterface];
			Context.SetDataInterfaceProxy(DataInterfaceProxy);
			DataInterfaceProxy->PostStage(Context);
			if (DataInterfaceProxy->RequiresPostStageFinalize())
			{
				ProxiesToFinalize.Add(DataInterfaceProxy);
			}
		}
	}
}

void FNiagaraGpuComputeDispatch::PostSimulateInterface(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData, const FNiagaraSimStageData& SimStageData) 
{
	const int32 NumDataInterfaces = InstanceData.DataInterfaceProxies.Num();
	if (NumDataInterfaces == 0)
	{
		return;
	}

	const FNiagaraShaderRef& ComputeShader = InstanceData.Context->GPUScript_RT->GetShader(0);
	TConstArrayView<FNiagaraDataInterfaceParamRef> DIParameters = ComputeShader->GetDIParameters();

	FNDIGpuComputePostSimulateContext Context(GraphBuilder, *this, Tick.SystemInstanceID, SimStageData.bSetDataToRender);
	for (int32 iDataInterface = 0; iDataInterface < NumDataInterfaces; ++iDataInterface)
	{
		const FNiagaraDataInterfaceParamRef& DIParam = DIParameters[iDataInterface];
		if (DIParam.ShaderParametersOffset != INDEX_NONE)
		{
			FNiagaraDataInterfaceProxy* DataInterfaceProxy = InstanceData.DataInterfaceProxies[iDataInterface];
			DataInterfaceProxy->PostSimulate(Context);
		}
	}
}

FGPUSortManager* FNiagaraGpuComputeDispatch::GetGPUSortManager() const
{
	return GPUSortManager;
}

void FNiagaraGpuComputeDispatch::AddDebugReadback(FNiagaraSystemInstanceID InstanceID, TSharedPtr<struct FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> DebugInfo, FNiagaraComputeExecutionContext* Context)
{
	FDebugReadbackInfo& ReadbackInfo = GpuDebugReadbackInfos.AddDefaulted_GetRef();
	ReadbackInfo.InstanceID = InstanceID;
	ReadbackInfo.DebugInfo = DebugInfo;
	ReadbackInfo.Context = Context;
}

bool FNiagaraGpuComputeDispatch::ShouldDebugDraw_RenderThread() const
{
#if NIAGARA_COMPUTEDEBUG_ENABLED
	if (FNiagaraGpuComputeDebug* GpuComputeDebug = GpuComputeDebugPtr.Get())
	{
		return GpuComputeDebug->ShouldDrawDebug();
	}
#endif
	return false;
}

void FNiagaraGpuComputeDispatch::DrawDebug_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const struct FScreenPassRenderTarget& Output)
{
#if NIAGARA_COMPUTEDEBUG_ENABLED
	if (FNiagaraGpuComputeDebug* GpuComputeDebug = GpuComputeDebugPtr.Get())
	{
		GpuComputeDebug->DrawDebug(GraphBuilder, View, Output);
	}
#endif
}

void FNiagaraGpuComputeDispatch::DrawSceneDebug_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth)
{
#if NIAGARA_COMPUTEDEBUG_ENABLED
	if (FNiagaraGpuComputeDebug* GpuComputeDebug = GpuComputeDebugPtr.Get())
	{
		GpuComputeDebug->DrawSceneDebug(GraphBuilder, View, SceneColor, SceneDepth);
	}
#endif
}

#if WITH_MGPU
void FNiagaraGpuComputeDispatch::MultiGPUResourceModified(FRDGBuilder& GraphBuilder, FRHIBuffer* Buffer, bool bRequiredForSimulation, bool bRequiredForRendering) const
{
	MultiGPUResourceModified(GraphBuilder.RHICmdList, Buffer, bRequiredForSimulation, bRequiredForRendering);
}

void FNiagaraGpuComputeDispatch::MultiGPUResourceModified(FRDGBuilder& GraphBuilder, FRHITexture* Texture, bool bRequiredForSimulation, bool bRequiredForRendering) const
{
	MultiGPUResourceModified(GraphBuilder.RHICmdList, Texture, bRequiredForSimulation, bRequiredForRendering);
}

void FNiagaraGpuComputeDispatch::MultiGPUResourceModified(FRHICommandList& RHICmdList, FRHIBuffer* Buffer, bool bRequiredForSimulation, bool bRequiredForRendering) const
{
	if (bCrossGPUTransferEnabled && bRequiredForRendering)
	{
		const_cast<FNiagaraGpuComputeDispatch*>(this)->AddCrossGPUTransfer(RHICmdList, Buffer);
	}
}

void FNiagaraGpuComputeDispatch::MultiGPUResourceModified(FRHICommandList& RHICmdList, FRHITexture* Texture, bool bRequiredForSimulation, bool bRequiredForRendering) const
{
	if (bCrossGPUTransferEnabled && bRequiredForRendering)
	{
		const bool bPullData = false;
		const bool bLockStep = false;

		const FRHIGPUMask GPUMask = RHICmdList.GetGPUMask();
		for (uint32 GPUIndex : FRHIGPUMask::All())
		{
			if (!GPUMask.Contains(GPUIndex))
			{
				if (OptimizedCrossGPUTransferMask & (1u << GPUIndex))
				{
					// Add a sync point for this destination GPU when we're adding the first transfer
					if (OptimizedCrossGPUTransferBuffers[GPUIndex].Num() == 0)
					{
						OptimizedCrossGPUFences[GPUIndex] = RHICreateCrossGPUTransferFence();
					}
					OptimizedCrossGPUTransferBuffers[GPUIndex].Emplace(Texture, GPUMask.GetFirstIndex(), GPUIndex, bPullData, bLockStep);
				}
				else
				{
					CrossGPUTransferBuffers.Emplace(Texture, GPUMask.GetFirstIndex(), GPUIndex, bPullData, bLockStep);
				}
			}
		}
	}
}

void FNiagaraGpuComputeDispatch::AddCrossGPUTransfer(FRHICommandList& RHICmdList, FRHIBuffer* Buffer)
{
	check(bCrossGPUTransferEnabled);
	if (Buffer)
	{
		const bool bPullData = false;
		const bool bLockStep = false;

		const FRHIGPUMask GPUMask = RHICmdList.GetGPUMask();
		for (uint32 GPUIndex : FRHIGPUMask::All())
		{
			if (!GPUMask.Contains(GPUIndex))
			{
				if (OptimizedCrossGPUTransferMask & (1u << GPUIndex))
				{
					// Add a sync point for this destination GPU when we're adding the first transfer
					if (OptimizedCrossGPUTransferBuffers[GPUIndex].Num() == 0)
					{
						OptimizedCrossGPUFences[GPUIndex] = RHICreateCrossGPUTransferFence();
					}
					OptimizedCrossGPUTransferBuffers[GPUIndex].Emplace(Buffer, GPUMask.GetFirstIndex(), GPUIndex, bPullData, bLockStep);
				}
				else
				{
					CrossGPUTransferBuffers.Emplace(Buffer, GPUMask.GetFirstIndex(), GPUIndex, bPullData, bLockStep);
				}
			}
		}
	}
}

void FNiagaraGpuComputeDispatch::TransferMultiGPUBuffers(FRHICommandList& RHICmdList)
{
	// Transfer buffers for cross GPU rendering
	if (CrossGPUTransferBuffers.Num())
	{
		RHICmdList.TransferResources(CrossGPUTransferBuffers);
		CrossGPUTransferBuffers.Reset();
	}

	// Optimized transfer of buffers for cross GPU rendering (deferred fence wait)
	for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
	{
		// Each destination GPU has a separate list of transfers, so we can wait for them on the GPU when it needs them
		if (OptimizedCrossGPUTransferBuffers[GPUIndex].Num())
		{
			// Signal function transitions resources on destination GPU and signals when transition has finished
			TArray<FCrossGPUTransferFence*> PreTransferSyncPoints;
			RHIGenerateCrossGPUPreTransferFences(OptimizedCrossGPUTransferBuffers[GPUIndex], PreTransferSyncPoints);
			RHICmdList.CrossGPUTransferSignal(OptimizedCrossGPUTransferBuffers[GPUIndex], PreTransferSyncPoints);

			RHICmdList.CrossGPUTransfer(OptimizedCrossGPUTransferBuffers[GPUIndex], PreTransferSyncPoints, TArrayView<FCrossGPUTransferFence*>(&OptimizedCrossGPUFences[GPUIndex], 1));
			OptimizedCrossGPUTransferBuffers[GPUIndex].Reset();
		}
	}
}

void FNiagaraGpuComputeDispatch::WaitForMultiGPUBuffers(FRHICommandList& RHICmdList, uint32 GPUIndex)
{
	if (OptimizedCrossGPUFences[GPUIndex])
	{
		RHICmdList.CrossGPUTransferWait(TArrayView<FCrossGPUTransferFence*>(&OptimizedCrossGPUFences[GPUIndex], 1));
		OptimizedCrossGPUFences[GPUIndex] = nullptr;
	}
}
#endif // WITH_MGPU
