// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
FNiagaraGpuComputeDispatch.h: Queueing and batching for Niagara simulation;
use to reduce per-simulation overhead by batching together simulations using
the same VectorVM byte code / compute shader code
==============================================================================*/
#pragma once

#include "FXSystem.h"
#include "GameTime.h"
#include "GlobalDistanceFieldParameters.h"
#include "NiagaraCommon.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraGPUProfiler.h"
#include "NiagaraGPUSystemTick.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSimStageData.h"
#include "RenderGraphUtils.h"
#include "RHIResources.h"
#include "RendererInterface.h"
#include "RendererUtils.h"

enum class EGPUSortFlags : uint32;
struct FNiagaraGPUSortInfo;

class FGPUSortManager;
class FNiagaraAsyncGpuTraceHelper;

//////////////////////////////////////////////////////////////////////////

struct FNiagaraGpuDispatchInstance
{
	FNiagaraGpuDispatchInstance(const FNiagaraGPUSystemTick& InTick, const FNiagaraComputeInstanceData& InInstanceData)
		: Tick(InTick)
		, InstanceData(InInstanceData)
	{
	}

	const FNiagaraGPUSystemTick& Tick;
	const FNiagaraComputeInstanceData& InstanceData;
	FNiagaraSimStageData SimStageData;
};


struct FNiagaraGpuFreeIDUpdate
{
	FNiagaraGpuFreeIDUpdate(FNiagaraComputeExecutionContext* InComputeContext)
		: ComputeContext(InComputeContext)
	{
	}

	FNiagaraComputeExecutionContext*	ComputeContext = nullptr;
	mutable FRHIShaderResourceView*		IDToIndexSRV = nullptr;
	mutable FRHIUnorderedAccessView*	FreeIDsUAV = nullptr;
	mutable uint32						NumAllocatedIDs = 0;
};

struct FNiagaraGpuDispatchGroup
{
	TArray<FNiagaraGPUSystemTick*>				TicksWithPerInstanceData;
	TArray<FNiagaraGpuDispatchInstance>			DispatchInstances;
	TArray<FNiagaraGpuFreeIDUpdate>				FreeIDUpdates;
};

struct FNiagaraGpuDispatchList
{
	void PreAllocateGroups(int32 LastGroup)
	{
		const int32 GroupsToAllocate = LastGroup - DispatchGroups.Num();
		if (GroupsToAllocate > 0)
		{
			DispatchGroups.AddDefaulted(GroupsToAllocate);
		}
	}

	bool HasWork() const { return DispatchGroups.Num() > 0; }

	TArray<uint32>						CountsToRelease;
	TArray<FNiagaraGpuDispatchGroup>	DispatchGroups;
};

//////////////////////////////////////////////////////////////////////////

class FNiagaraGpuComputeDispatch : public FNiagaraGpuComputeDispatchInterface
{
public:
	static const FName Name;

	virtual FFXSystemInterface* GetInterface(const FName& InName) override;

	FNiagaraGpuComputeDispatch(ERHIFeatureLevel::Type InFeatureLevel, EShaderPlatform InShaderPlatform, FGPUSortManager* InGPUSortManager);

	~FNiagaraGpuComputeDispatch();

	/** Add system instance proxy to the batcher for tracking. */
	virtual void AddGpuComputeProxy(FNiagaraSystemGpuComputeProxy* ComputeProxy) override;

	/** Remove system instance proxy from the batcher. */
	virtual void RemoveGpuComputeProxy(FNiagaraSystemGpuComputeProxy* ComputeProxy) override;

	virtual void AddNDCDataProxy(FNiagaraDataChannelDataProxyPtr NDCDataProxy) override;
	virtual void RemoveNDCDataProxy(FNiagaraDataChannelDataProxyPtr NDCDataProxy) override;

#if WITH_EDITOR
	virtual void Suspend() override {}
	virtual void Resume() override {}
#endif // #if WITH_EDITOR

	virtual void DrawDebug(FCanvas* Canvas) override {}
	virtual bool ShouldDebugDraw_RenderThread() const override;
	virtual void DrawDebug_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const struct FScreenPassRenderTarget& Output) override;
	virtual void DrawSceneDebug_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth) override;
	virtual void AddVectorField(UVectorFieldComponent* VectorFieldComponent) override {}
	virtual void RemoveVectorField(UVectorFieldComponent* VectorFieldComponent) override {}
	virtual void UpdateVectorField(UVectorFieldComponent* VectorFieldComponent) override {}
	virtual void PreInitViews(FRDGBuilder& GraphBuilder, bool bAllowGPUParticleUpdate, const TArrayView<const FSceneViewFamily*> &ViewFamilies, const FSceneViewFamily* CurrentFamily) override;
	virtual void PostInitViews(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, bool bAllowGPUParticleUpdate) override;
	virtual bool UsesGlobalDistanceField() const override;
	virtual bool UsesDepthBuffer() const override;
	virtual bool RequiresEarlyViewUniformBuffer() const override;
	virtual bool RequiresRayTracingScene() const override;
	virtual void PreRender(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleUpdate) override;
	virtual void OnMarkPendingKill() override;

	virtual void Tick(UWorld* World, float DeltaTime) override;

	virtual void PostRenderOpaque(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleUpdate) override;

	// FNiagaraGpuComputeDispatchInterface Impl
	virtual void FlushPendingTicks_GameThread() override;
	virtual void FlushAndWait_GameThread() override;
	// FNiagaraGpuComputeDispatchInterface Impl

	/**
	 * Process and respond to a build up of excessive ticks inside the batcher.
	 * In the case of the application not having focus the game thread may continue
	 * to process and send ticks to the render thread but the rendering thread may
	 * never process them.  The World Manager will ensure this is called once per
	 * game frame so we have an opportunity to flush the ticks avoiding a stall
	 * when we gain focus again.
	 */
	void ProcessPendingTicksFlush(FRHICommandListImmediate& RHICmdList, bool bForceFlush);

	/** Processes all pending readbacks */
	virtual void ProcessDebugReadbacks(FRHICommandList& RHICmdList, bool bWaitCompletion) override;

	virtual bool AddSortedGPUSimulation(FRHICommandListBase& RHICmdList, FNiagaraGPUSortInfo& SortInfo) override;
	virtual const FGlobalDistanceFieldParameterData* GetGlobalDistanceFieldData() const override;

	void ResetDataInterfaces(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData);
	void SetDataInterfaceParameters(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData, const FNiagaraShaderRef& ComputeShader, const FNiagaraSimStageData& SimStageData, const FNiagaraShaderScriptParametersMetadata& NiagaraShaderParametersMetadata, uint8* ParametersStructure);
	void PreStageInterface(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData, const FNiagaraSimStageData& SimStageData, TSet<FNiagaraDataInterfaceProxy*>& ProxiesToFinalize);
	void PostStageInterface(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData, const FNiagaraSimStageData& SimStageData, TSet<FNiagaraDataInterfaceProxy*>& ProxiesToFinalize);
	void PostSimulateInterface(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData, const FNiagaraSimStageData& SimStageData);

	/** Given a shader stage index, find the corresponding data interface */
	FNiagaraDataInterfaceProxyRW* FindIterationInterface(FNiagaraComputeInstanceData* Instance, const uint32 SimulationStageIndex) const;

	/** Get the shared SortManager, used in the rendering loop to call FGPUSortManager::OnPreRender() and FGPUSortManager::OnPostRenderOpaque() */
	virtual FGPUSortManager* GetGPUSortManager() const override;

#if WITH_NIAGARA_GPU_PROFILER
	virtual FNiagaraGPUProfilerInterface* GetGPUProfiler() const final { return GPUProfilerPtr.Get(); }
#endif

	/** Allows access to the current passes external access queue. */
	FRDGExternalAccessQueue& GetCurrentPassExternalAccessQueue() const { return CurrentPassExternalAccessQueue; }

	/** Debug only function to readback data. */
	virtual void AddDebugReadback(FNiagaraSystemInstanceID InstanceID, TSharedPtr<struct FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> DebugInfo, FNiagaraComputeExecutionContext* Context) override;

#if WITH_MGPU
	virtual void MultiGPUResourceModified(FRDGBuilder& GraphBuilder, FRHIBuffer* Buffer, bool bRequiredForSimulation, bool bRequiredForRendering) const override;
	virtual void MultiGPUResourceModified(FRDGBuilder& GraphBuilder, FRHITexture* Texture, bool bRequiredForSimulation, bool bRequiredForRendering) const override;

	virtual void MultiGPUResourceModified(FRHICommandList& RHICmdList, FRHIBuffer* Buffer, bool bRequiredForSimulation, bool bRequiredForRendering) const override;
	virtual void MultiGPUResourceModified(FRHICommandList& RHICmdList, FRHITexture* Texture, bool bRequiredForSimulation, bool bRequiredForRendering) const override;
#endif

	virtual FNiagaraAsyncGpuTraceHelper& GetAsyncGpuTraceHelper() const override;

	//-TODO: Temporary while the count buffer is not an RDG resource
	bool IsExecutingFirstDispatchGroup() const { return bIsExecutingFirstDispatchGroup; }
	bool IsExecutingLastDispatchGroup() const { return bIsExecutingLastDispatchGroup; }
	bool bIsExecutingFirstDispatchGroup = false;
	bool bIsExecutingLastDispatchGroup = false;

private:
	void DumpDebugFrame();
	void UpdateInstanceCountManager(FRHICommandListImmediate& RHICmdList);
	void PrepareTicksForProxy(FRHICommandListImmediate& RHICmdList, FNiagaraSystemGpuComputeProxy* ComputeProxy, FNiagaraGpuDispatchList& GpuDispatchList);
	void PrepareAllTicks(FRHICommandListImmediate& RHICmdList);
	void ExecuteTicks(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, ENiagaraGpuComputeTickStage::Type TickStage);
	void DispatchStage(FRDGBuilder& GraphBuilder, const FNiagaraGPUSystemTick& Tick, const FNiagaraComputeInstanceData& InstanceData, const FNiagaraSimStageData& SimStageData);

	/**
	 * Generate all the initial keys and values for a GPUSortManager sort batch.
	 * Sort batches are created when GPU sort tasks are registered in AddSortedGPUSimulation().
	 * Each sort task defines constraints about when the initial sort data can generated and
	 * and when the sorted results are needed (see EGPUSortFlags for details).
	 * Currently, for Niagara, all the sort tasks have the EGPUSortFlags::KeyGenAfterPreRender flag
	 * and so the callback registered in GPUSortManager->Register() only has the EGPUSortFlags::KeyGenAfterPreRender usage.
	 * This garanties that GenerateSortKeys() only gets called after PreRender(), which is a constraint required because
	 * Niagara renders the current state of the GPU emitters, before the are ticked
	 * (Niagara GPU emitters are ticked at InitView and in PostRenderOpaque).
	 * Note that this callback must only initialize the content for the elements that relates to the tasks it has registered in this batch.
	 *
	 * @param RHICmdList - The command list used to initiate the keys and values on GPU.
	 * @param BatchId - The GPUSortManager batch id (regrouping several similar sort tasks).
	 * @param NumElementsInBatch - The number of elements grouped in the batch (each element maps to a sort task)
	 * @param Flags - Details about the key precision (see EGPUSortFlags::AnyKeyPrecision) and the keygen location (see EGPUSortFlags::AnyKeyGenLocation).
	 * @param KeysUAV - The UAV that holds all the initial keys used to sort the values (being the particle indices here).
	 * @param ValuesUAV - The UAV that holds the initial values (particle indices) to be sorted accordingly to the keys.
	 */
	void GenerateSortKeys(FRHICommandListImmediate& RHICmdList, int32 BatchId, int32 NumElementsInBatch, EGPUSortFlags Flags, FRHIUnorderedAccessView* KeysUAV, FRHIUnorderedAccessView* ValuesUAV);

	void FinishDispatches();

	/** The shared GPUSortManager, used to register GPU sort tasks in order to generate sorted particle indices per emitter. */
	TRefCountPtr<FGPUSortManager> GPUSortManager;
	/** All sort tasks registered in AddSortedGPUSimulation(). Holds all the data required in GenerateSortKeys(). */
	TArray<FNiagaraGPUSortInfo> SimulationsToSort;

	TUniquePtr<FNiagaraAsyncGpuTraceHelper> AsyncGpuTraceHelper;

#if WITH_NIAGARA_GPU_PROFILER
	TUniquePtr<FNiagaraGPUProfiler> GPUProfilerPtr;
#endif

	uint32 FramesBeforeTickFlush = 0;

	mutable FRDGExternalAccessQueue CurrentPassExternalAccessQueue;

	/** A buffer of list sizes used by UpdateFreeIDBuffers to allow overlapping several dispatches. */
	FRWBuffer FreeIDListSizesBuffer;
	uint32 NumAllocatedFreeIDListSizes = 0;
	uint32 NumRequiredFreeIDListSizes = 0;

	uint32 NumProxiesThatRequireGlobalDistanceField = 0;
	uint32 NumProxiesThatRequireDepthBuffer = 0;
	uint32 NumProxiesThatRequireEarlyViewData = 0;
	uint32 NumProxiesThatRequireRayTracingScene = 0;
	uint32 NumProxiesThatRequireCurrentFrameNDC = 0;

	uint32 ProxyGpuCountBufferEstimate = 0;

	int32 TotalDispatchesThisFrame = 0;

	int32 MaxTicksToFlush = TNumericLimits<int32>::Max();

	UE::FMutex AddSortedGPUSimulationMutex;

	bool bRequiresReadback = false;
	TArray<FNiagaraSystemGpuComputeProxy*> ProxiesPerStage[ENiagaraGpuComputeTickStage::Max];
	
	TArray<FNiagaraDataChannelDataProxyPtr> NDCDataProxies;

	FNiagaraGpuDispatchList DispatchListPerStage[ENiagaraGpuComputeTickStage::Max];

	struct FDebugReadbackInfo
	{
		FNiagaraSystemInstanceID InstanceID;
		TSharedPtr<struct FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> DebugInfo;
		FNiagaraComputeExecutionContext* Context;
	};
	TArray<FDebugReadbackInfo> GpuDebugReadbackInfos;

#if WITH_MGPU
	// List of items that need MultiViewPreviousDataToRender to be cleaned up on the last view family
	TArray<FNiagaraComputeExecutionContext*> NeedsMultiViewPreviousDataClear;

	// Cross GPU transfer arrays need to be mutable, so they can be written from const MultiGPUResourceModified utility functions
	bool bCrossGPUTransferEnabled = false;
	mutable TArray<FTransferResourceParams> CrossGPUTransferBuffers;

	uint32 OptimizedCrossGPUTransferMask = 0;
	mutable TArray<FTransferResourceParams> OptimizedCrossGPUTransferBuffers[MAX_NUM_GPUS];
	mutable FCrossGPUTransferFence* OptimizedCrossGPUFences[MAX_NUM_GPUS] = { 0 };

	void AddCrossGPUTransfer(FRHICommandList& RHICmdList, FRHIBuffer* Buffer);

	void TransferMultiGPUBuffers(FRHICommandList& RHICmdList);
	void WaitForMultiGPUBuffers(FRHICommandList& RHICmdList, uint32 GPUIndex);
#endif // WITH_MGPU

	// Cached information to build a dummy view info if necessary
	struct FCachedViewInitOptions
	{
		FGameTime	GameTime;
		FIntRect	ViewRect			= FIntRect(0, 0, 64, 64);
		FVector		ViewOrigin			= FVector::ZeroVector;
		FMatrix		ViewRotationMatrix	= FMatrix::Identity;
		FMatrix		ProjectionMatrix	= FMatrix::Identity;
	};
	FCachedViewInitOptions CachedViewInitOptions;

	// Cached info for distance fields
	struct FCachedDistanceFieldData
	{
		bool								bCacheValid = false;
		bool								bValidForPass = false;
		FRHITexture*						PageAtlasTexture = nullptr;
		FRHITexture*						CoverageAtlasTexture = nullptr;
		FRDGPooledBuffer*					PageObjectGridBuffer = nullptr;
		FRHITexture*						PageTableTexture = nullptr;
		FRHITexture*						MipTexture = nullptr;
		FGlobalDistanceFieldParameterData	GDFParameterData;
	};
	FCachedDistanceFieldData			CachedGDFData;

#if WITH_EDITOR
	bool bRaisedWarningThisFrame = false;
#endif

	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesUniformParams = nullptr;
	TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> MobileSceneTexturesUniformParams = nullptr;
	TRDGUniformBufferRef<FSubstratePublicGlobalUniformParameters> SubstratePublicGlobalUniformParams = nullptr;
};
