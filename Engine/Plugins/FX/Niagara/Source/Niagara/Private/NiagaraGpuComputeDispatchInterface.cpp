// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGpuComputeDispatch.h"
#include "NiagaraGpuComputeDebug.h"
#include "NiagaraGpuComputeDebugInterface.h"
#include "NiagaraGpuReadbackManager.h"

#include "Engine/World.h"
#include "FXSystem.h"
#include "RenderGraphBuilder.h"
#include "SceneInterface.h"
#include "SystemTextures.h"

FNiagaraGpuComputeDispatchInterface::FNiagaraGpuComputeDispatchInterface(EShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel)
	: ShaderPlatform(InShaderPlatform)
	, FeatureLevel(InFeatureLevel)
	, GPUInstanceCounterManager(InFeatureLevel)
{
}

FNiagaraGpuComputeDispatchInterface::~FNiagaraGpuComputeDispatchInterface()
{
}

FNiagaraGpuComputeDispatchInterface* FNiagaraGpuComputeDispatchInterface::Get(UWorld* World)
{
	return World ? Get(World->Scene) : nullptr;
}

FNiagaraGpuComputeDispatchInterface* FNiagaraGpuComputeDispatchInterface::Get(FSceneInterface* Scene)
{
	return Scene ? Get(Scene->GetFXSystem()) : nullptr;
}

FNiagaraGpuComputeDispatchInterface* FNiagaraGpuComputeDispatchInterface::Get(FFXSystemInterface* FXSceneInterface)
{
	return FXSceneInterface ? static_cast<FNiagaraGpuComputeDispatchInterface*>(FXSceneInterface->GetInterface(FNiagaraGpuComputeDispatch::Name)) : nullptr;
}

#if NIAGARA_COMPUTEDEBUG_ENABLED
FNiagaraGpuComputeDebugInterface FNiagaraGpuComputeDispatchInterface::GetGpuComputeDebugInterface() const
{
	return FNiagaraGpuComputeDebugInterface(GpuComputeDebugPtr.Get());
}
#endif

FRDGTextureRef FNiagaraGpuComputeDispatchInterface::GetBlackTexture(FRDGBuilder& GraphBuilder, ETextureDimension TextureDimension) const
{
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	switch (TextureDimension)
	{
		case ETextureDimension::Texture2D:			return SystemTextures.Black;
		case ETextureDimension::Texture2DArray:		return SystemTextures.BlackArray;
		case ETextureDimension::Texture3D:			return SystemTextures.VolumetricBlack;
		case ETextureDimension::TextureCube:		return SystemTextures.CubeBlack;
		case ETextureDimension::TextureCubeArray:	return SystemTextures.CubeArrayBlack;
		default: checkNoEntry(); return nullptr;
	}
}

FRDGTextureSRVRef FNiagaraGpuComputeDispatchInterface::GetBlackTextureSRV(FRDGBuilder& GraphBuilder, ETextureDimension TextureDimension) const
{
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	switch (TextureDimension)
	{
		case ETextureDimension::Texture2D:			return GraphBuilder.CreateSRV(SystemTextures.Black);
		case ETextureDimension::Texture2DArray:		return GraphBuilder.CreateSRV(SystemTextures.BlackArray);
		case ETextureDimension::Texture3D:			return GraphBuilder.CreateSRV(SystemTextures.VolumetricBlack);
		case ETextureDimension::TextureCube:		return GraphBuilder.CreateSRV(SystemTextures.CubeBlack);
		case ETextureDimension::TextureCubeArray:	return GraphBuilder.CreateSRV(SystemTextures.CubeArrayBlack);
		default: checkNoEntry(); return nullptr;
	}
}

FRDGTextureUAVRef FNiagaraGpuComputeDispatchInterface::GetEmptyTextureUAV(FRDGBuilder& GraphBuilder, EPixelFormat Format, ETextureDimension TextureDimension) const
{
	return EmptyUAVPoolPtr->GetEmptyRDGUAVFromPool(GraphBuilder, Format, TextureDimension);
}

FRDGBufferUAVRef FNiagaraGpuComputeDispatchInterface::GetEmptyBufferUAV(FRDGBuilder& GraphBuilder, EPixelFormat Format) const
{
	return EmptyUAVPoolPtr->GetEmptyRDGUAVFromPool(GraphBuilder, Format);
}

FRDGBufferSRVRef FNiagaraGpuComputeDispatchInterface::GetEmptyBufferSRV(FRDGBuilder& GraphBuilder, EPixelFormat Format) const
{
	return GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, GPixelFormats[Format].BlockBytes, FUintVector4(0, 0, 0, 0)), Format);
}
