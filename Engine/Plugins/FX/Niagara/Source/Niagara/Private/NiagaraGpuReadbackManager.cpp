// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGpuReadbackManager.h"

#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHICommandList.h"

const FName FNiagaraGpuReadbackManager::FenceName("NiagaraGpuReadback");

FNiagaraGpuReadbackManager::FNiagaraGpuReadbackManager()
{
}

FNiagaraGpuReadbackManager::~FNiagaraGpuReadbackManager()
{
}

void FNiagaraGpuReadbackManager::Tick(FRHICommandListImmediate& RHICmdList)
{
	TickInternal(RHICmdList, false);
}

void FNiagaraGpuReadbackManager::TickInternal(FRHICommandListImmediate& RHICmdList, bool bAssumeGpuIdle)
{
	check(IsInRenderingThread());

	TArray<TPair<void*, uint32>, TInlineAllocator<1>> ReadbackData;
	while (FPendingReadback * Readback = PendingReadbacks.Peek())
	{
		// When we hit the first incomplete readback the rest will also be incomplete as we assume chronological insertion order
		if (!bAssumeGpuIdle && !Readback->Fence->Poll())
		{
			break;
		}

		// Gather data an execute the callback
		for (const TPair<FStagingBufferRHIRef, uint32>& StagingBuffer : Readback->StagingBuffers)
		{
			void* DataPtr = RHICmdList.LockStagingBuffer(StagingBuffer.Key, Readback->Fence.GetReference(), 0, StagingBuffer.Value);
			ensureMsgf(DataPtr || (StagingBuffer.Value == 0), TEXT("NiagaraGpuReadbackManager StagingBuffer returned nullptr and size is %d bytes, bAssumeGpuIdle(%d)"), StagingBuffer.Value, bAssumeGpuIdle);
			ReadbackData.Emplace(DataPtr, StagingBuffer.Value);
		}

		Readback->Callback(MakeArrayView(ReadbackData));

		for (const TPair<FStagingBufferRHIRef, uint32>& StagingBuffer : Readback->StagingBuffers)
		{
			RHICmdList.UnlockStagingBuffer(StagingBuffer.Key);
		}

		ReadbackData.Reset();

		// Remove the readback as it's complete
		PendingReadbacks.Pop();
	}
}

// Wait for all pending readbacks to complete
void FNiagaraGpuReadbackManager::WaitCompletion(FRHICommandListImmediate& RHICmdList)
{
	// Ensure all GPU commands have been executed as we will ignore the fence
	// This is because the fence may be implemented as a simple counter rather than a real fence
	RHICmdList.SubmitAndBlockUntilGPUIdle();

	// Perform a tick which will flush everything
	TickInternal(RHICmdList, true);
}

void FNiagaraGpuReadbackManager::EnqueueReadback(FRDGBuilder& GraphBuilder, FRDGBufferRef Buffer, FCompletionCallback Callback)
{
	check(IsInRenderingThread());
	EnqueueReadback(GraphBuilder, Buffer, 0, Buffer->Desc.GetSize(), Callback);
}

void FNiagaraGpuReadbackManager::EnqueueReadback(FRDGBuilder& GraphBuilder, FRDGBufferRef Buffer, uint32 Offset, uint32 NumBytes, FCompletionCallback Callback)
{
	check(IsInRenderingThread());

	FPendingReadback Readback;
	Readback.StagingBuffers.Emplace(RHICreateStagingBuffer(), NumBytes);
	Readback.Fence = RHICreateGPUFence(FenceName);
	Readback.Callback = Callback;
	Readback.Fence->Clear();

	AddReadbackBufferPass(
		GraphBuilder, RDG_EVENT_NAME("NiagaraReadback"), Buffer,
		[Buffer, ReadbackBuffer=Readback.StagingBuffers[0].Key, Offset, NumBytes, Fence=Readback.Fence](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.CopyToStagingBuffer(Buffer->GetRHI(), ReadbackBuffer, Offset, NumBytes);
			RHICmdList.WriteGPUFence(Fence);
		}
	);

	PendingReadbacks.Enqueue(Readback);
}

void FNiagaraGpuReadbackManager::EnqueueReadbacks(FRDGBuilder& GraphBuilder, TConstArrayView<FRDGBufferRef> Buffers, FCompletionCallback Callback)
{
	check(IsInRenderingThread());

	FPendingReadback Readback;
	Readback.Fence = RHICreateGPUFence(FenceName);
	Readback.Callback = Callback;
	Readback.Fence->Clear();

	for (int32 i=0; i < Buffers.Num(); ++i)
	{
		FRDGBufferRef Buffer = Buffers[i];
		FStagingBufferRHIRef ReadbackBuffer = RHICreateStagingBuffer();
		const uint32 Offset = 0;
		const uint32 NumBytes = Buffer->Desc.GetSize();

		Readback.StagingBuffers.Emplace(ReadbackBuffer, NumBytes);

		const bool bWriteFence = i == Buffers.Num() - 1;
		AddReadbackBufferPass(
			GraphBuilder, RDG_EVENT_NAME("NiagaraReadback"), Buffer,
			[Buffer, ReadbackBuffer, Offset=0, NumBytes, Fence=bWriteFence ? Readback.Fence : nullptr](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.CopyToStagingBuffer(Buffer->GetRHI(), ReadbackBuffer, Offset, NumBytes);
				if (Fence != nullptr)
				{
					RHICmdList.WriteGPUFence(Fence);
				}
			}
		);
	}

	PendingReadbacks.Enqueue(Readback);
}

void FNiagaraGpuReadbackManager::EnqueueReadbacks(FRDGBuilder& GraphBuilder, TConstArrayView<FRDGBufferRequest> BufferRequests, FCompletionCallback Callback)
{
	check(IsInRenderingThread());

	FPendingReadback Readback;
	Readback.Fence = RHICreateGPUFence(FenceName);
	Readback.Callback = Callback;
	Readback.Fence->Clear();

	for (int32 i = 0; i < BufferRequests.Num(); ++i)
	{
		FRDGBufferRef Buffer = BufferRequests[i].Buffer;
		FStagingBufferRHIRef ReadbackBuffer = RHICreateStagingBuffer();
		const uint32 Offset = BufferRequests[i].Offset;
		const uint32 NumBytes = BufferRequests[i].Size;

		check(NumBytes > 0);

		Readback.StagingBuffers.Emplace(ReadbackBuffer, NumBytes);

		const bool bWriteFence = i == BufferRequests.Num() - 1;
		AddReadbackBufferPass(
			GraphBuilder, RDG_EVENT_NAME("NiagaraReadback"), Buffer,
			[Buffer, ReadbackBuffer, Offset = 0, NumBytes, Fence=bWriteFence ? Readback.Fence : nullptr](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.CopyToStagingBuffer(Buffer->GetRHI(), ReadbackBuffer, Offset, NumBytes);
				if (Fence != nullptr)
				{
					RHICmdList.WriteGPUFence(Fence);
				}
			}
		);
	}

	PendingReadbacks.Enqueue(Readback);
}

void FNiagaraGpuReadbackManager::EnqueueReadback(FRHICommandList& RHICmdList, FRHIBuffer* Buffer, FCompletionCallback Callback)
{
	check(IsInRenderingThread());
	
	EnqueueReadback(RHICmdList, Buffer, 0, Buffer->GetSize(), MoveTemp(Callback));
}

void FNiagaraGpuReadbackManager::EnqueueReadback(FRHICommandList& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 NumBytes, FCompletionCallback Callback)
{
	check(IsInRenderingThread());

	FPendingReadback Readback;
	Readback.StagingBuffers.Emplace(RHICreateStagingBuffer(), NumBytes);
	Readback.Fence = RHICreateGPUFence(FenceName);
	Readback.Callback = Callback;

	RHICmdList.CopyToStagingBuffer(Buffer, Readback.StagingBuffers[0].Key, Offset, Readback.StagingBuffers[0].Value);

	Readback.Fence->Clear();
	RHICmdList.WriteGPUFence(Readback.Fence);

	PendingReadbacks.Enqueue(Readback);
}

void FNiagaraGpuReadbackManager::EnqueueReadbacks(FRHICommandList& RHICmdList, TConstArrayView<FRHIBuffer*> Buffers, FCompletionCallback Callback)
{
	check(IsInRenderingThread());

	FPendingReadback Readback;
	for (FRHIBuffer* Buffer : Buffers)
	{
		const auto& ReadbackData = Readback.StagingBuffers.Emplace_GetRef(RHICreateStagingBuffer(), Buffer->GetSize());
		RHICmdList.CopyToStagingBuffer(Buffer, ReadbackData.Key, 0, ReadbackData.Value);
	}
	Readback.Fence = RHICreateGPUFence(FenceName);
	Readback.Callback = Callback;

	Readback.Fence->Clear();
	RHICmdList.WriteGPUFence(Readback.Fence);

	PendingReadbacks.Enqueue(Readback);
}

void FNiagaraGpuReadbackManager::EnqueueReadbacks(FRHICommandList& RHICmdList, TConstArrayView<FBufferRequest> BufferRequests, FCompletionCallback Callback)
{
	check(IsInRenderingThread());

	FPendingReadback Readback;
	for (const FBufferRequest& BufferRequest : BufferRequests)
	{
		check(BufferRequest.Size > 0);
		const auto& ReadbackData = Readback.StagingBuffers.Emplace_GetRef(RHICreateStagingBuffer(), BufferRequest.Size);
		RHICmdList.CopyToStagingBuffer(BufferRequest.Buffer, ReadbackData.Key, BufferRequest.Offset, BufferRequest.Size);
	}
	Readback.Fence = RHICreateGPUFence(FenceName);
	Readback.Callback = Callback;

	Readback.Fence->Clear();
	RHICmdList.WriteGPUFence(Readback.Fence);

	PendingReadbacks.Enqueue(Readback);
}
