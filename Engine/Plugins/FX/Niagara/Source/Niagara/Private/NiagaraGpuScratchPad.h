// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "RHIDefinitions.h"
#include "RHIUtilities.h"
#include "RendererInterface.h"

class FScene;
class FSceneView;
class FRayTracingPipelineState;
class FRHICommandList;
class FRHIRayTracingScene;
class FRHIShaderResourceView;
class FRHIUniformBuffer;
class FRHIUnorderedAccessView;
struct FRWBuffer;
struct FNiagaraDataInterfaceProxy;


//////////////////////////////////////////////////////////////////////////
//TODO: Move scratch pad buffers to core rendering code along side global dynamic read buffers etc.

/** Allocates space from a pool of fixed size static RWBuffers.
*	Allocations are valid between calls to Reset.
*	Useful for transient working data generated in one dispatch and consumed in others.
*	All buffers in the same scratch pad are transitioned as one.
*/
class FNiagaraGpuScratchPad
{
public:
	struct FAllocation
	{
		FRWBuffer* Buffer = nullptr;
		uint32 Offset = 0;

		FORCEINLINE bool IsValid()const { return Buffer != nullptr; }
	};

	struct FScratchBuffer
	{
	public:
		struct FBuffer
		{
			FRWBuffer Buffer;
			uint32 Used = 0;
		};

		FScratchBuffer(uint32 InBucketSize, EPixelFormat InPixelFormat, uint32 InBytesPerElement, EBufferUsageFlags InBufferUsageFlags)
			: MinBucketSize(InBucketSize), PixelFormat(InPixelFormat), BytesPerElement(InBytesPerElement), BufferUsageFlags(InBufferUsageFlags)
		{}

		/** Allocates NumElements from the buffers. Will try to find an existing buffer to fit the allocation but will create a new one if needed. */
		bool Allocate(uint32 NumElements, FString& BufferDebugName, FRHICommandList& RHICmdList, FAllocation& OutAllocation)
		{
			//Look for a buffer to fit this allocation.
			FBuffer* ToUse = nullptr;
			for (FBuffer& Buffer : Buffers)
			{
				uint32 NumBufferElements = Buffer.Buffer.NumBytes / BytesPerElement;
				if (NumElements + Buffer.Used <= NumBufferElements)
				{
					ToUse = &Buffer;
				}
			}

			bool bNew = false;
			if (ToUse == nullptr)
			{
				//Allocate a new buffer that will fit this allocation.
				FBuffer* NewBuffer = new FBuffer();
				Buffers.Add(NewBuffer);

				//Round up the size to pow 2 to ensure a slowly growing user doesn't just keep allocating new buckets. Probably a better guess size here than power of 2.
				uint32 NewBufferSize = FPlatformMath::RoundUpToPowerOfTwo(FMath::Max(MinBucketSize, NumElements));

				NewBuffer->Buffer.Initialize(RHICmdList, *BufferDebugName, BytesPerElement, NewBufferSize, PixelFormat, CurrentAccess, BufferUsageFlags);
			
				ToUse = NewBuffer;

				NumAllocatedBytes += NewBuffer->Buffer.NumBytes;

				bNew = true;
			}

			OutAllocation.Buffer = &ToUse->Buffer;
			OutAllocation.Offset = ToUse->Used;
			ToUse->Used += NumElements;
			NumUsedBytes += NumElements * BytesPerElement;
			return bNew;
		}

		/** Resets the buffer and releases it's internal RWBuffers too. */
		void Release()
		{
			Reset();
			CurrentAccess = ERHIAccess::UAVCompute;
			for (FBuffer& Buffer : Buffers)
			{
				Buffer.Buffer.Release();
				Buffer.Used = 0;
			}
			Buffers.Reset();

			NumAllocatedBytes = 0;
		}

		/** Resets the usage for all buffers to zero. */
		void Reset()
		{
			for (FBuffer& Buffer : Buffers)
			{
				Buffer.Used = 0;
			}

			NumUsedBytes = 0;
		}

		template<typename TFunc>
		void ForEachBuffer(TFunc Func)
		{
			for (FBuffer& Buffer : Buffers)
			{
				Func(Buffer.Buffer);
			}
		}

		/** Transitions all buffers in the scratch pad. */
		void Transition(FRHICommandList& RHICmdList, ERHIAccess InPreviousState, ERHIAccess InNewState, EResourceTransitionFlags InFlags = EResourceTransitionFlags::None)
		{
			//Store off current access state so we can place any new buffers in the same state.
			CurrentAccess = InNewState;

 			TArray<FRHITransitionInfo, SceneRenderingAllocator> Transitions;
 			Transitions.Reserve(Buffers.Num());
 			for (FBuffer& Buffer : Buffers)
 			{
 				Transitions.Emplace(Buffer.Buffer.UAV, InPreviousState, InNewState, InFlags);
 			}
 
 			RHICmdList.Transition(Transitions);
		}

		/** Returns the total bytes allocated for all buffers in the scratch pad. */
		FORCEINLINE uint32 AllocatedBytes()const
		{
			return NumAllocatedBytes;
		}

		/** Returns the number of used bytes allocated for all buffers in the scratch pad. */
		FORCEINLINE uint32 UsedBytes()const
		{
			return NumUsedBytes;
		}

		FORCEINLINE ERHIAccess GetExpectedCurrentAccess()const{return CurrentAccess;}
	private:

		uint32 MinBucketSize = 4096;
		TIndirectArray<FBuffer> Buffers;

		EPixelFormat PixelFormat;
		uint32 BytesPerElement = 0;
		EBufferUsageFlags BufferUsageFlags = EBufferUsageFlags::None;
		ERHIAccess CurrentAccess = ERHIAccess::UAVCompute;

		uint32 NumAllocatedBytes = 0;
		uint32 NumUsedBytes = 0;
	};

	FNiagaraGpuScratchPad(uint32 InBucketSize, const FString& InDebugName, EBufferUsageFlags BufferUsageFlags = BUF_Static)
		: BucketSize(InBucketSize)
		, DebugName(InDebugName)
		, ScratchBufferFloat(BucketSize, PF_R32_FLOAT, sizeof(float), BufferUsageFlags)
		, ScratchBufferUInt(BucketSize, PF_R32_UINT, sizeof(uint32), BufferUsageFlags)
	{
	}

	/** Allocate a number of elements from the scratch pad. This will attempt to allocate from an existing buffer but will create a new one if it can't. */
	template<typename T>
	FORCEINLINE FAllocation Alloc(uint32 Size, FRHICommandList& RHICmdList, bool bClearNew);

	/** Resets the scratch pad and also releases all buffers it holds. */
	void Release()
	{
		ScratchBufferFloat.Release();
		ScratchBufferUInt.Release();
	}

	/** Resets the usage counters for buffers in this scratch pad, invalidating all previous allocations. Does not affect actual allocations. Optionally clears all buffers to zero. */
	void Reset(FRHICommandList& RHICmdList, bool bClear)
	{
		ScratchBufferFloat.Reset();
		ScratchBufferUInt.Reset();

		if (bClear)
		{
			Transition(RHICmdList, ERHIAccess::Unknown, ERHIAccess::UAVCompute);

			auto ClearFloatBuffer = [&](FRWBuffer& Buffer)
			{
				RHICmdList.ClearUAVFloat(Buffer.UAV, FVector4f(0.0f));
			};
			ScratchBufferFloat.ForEachBuffer(ClearFloatBuffer);

			auto ClearUIntBuffer = [&](FRWBuffer& Buffer)
			{
				RHICmdList.ClearUAVUint(Buffer.UAV, FUintVector4(0, 0, 0, 0));
			};
			ScratchBufferUInt.ForEachBuffer(ClearUIntBuffer);

			Transition(RHICmdList, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute);
		}
	}

	/** Transitions all buffers in the scratch pad. */
	void Transition(FRHICommandList& RHICmdList, ERHIAccess InPreviousState, ERHIAccess InNewState, EResourceTransitionFlags InFlags = EResourceTransitionFlags::None)
	{
		ScratchBufferFloat.Transition(RHICmdList, InPreviousState, InNewState, InFlags);
		ScratchBufferUInt.Transition(RHICmdList, InPreviousState, InNewState, InFlags);
	}

	/** Returns the total bytes allocated for all buffers in the scratch pad. */
	FORCEINLINE uint32 AllocatedBytes()const
	{
		return ScratchBufferFloat.AllocatedBytes() + ScratchBufferUInt.AllocatedBytes();
	}

	/** Returns the number of used bytes allocated for all buffers in the scratch pad. */
	FORCEINLINE uint32 UsedBytes()const
	{
		return ScratchBufferFloat.UsedBytes() + ScratchBufferUInt.UsedBytes();
	}

private:

	/** Size of each FRWBuffer in the pool. */
	uint32 BucketSize = 4096;
	FString DebugName;

	FScratchBuffer ScratchBufferFloat;
	FScratchBuffer ScratchBufferUInt;
};

template<>
FORCEINLINE FNiagaraGpuScratchPad::FAllocation FNiagaraGpuScratchPad::Alloc<float>(uint32 Size, FRHICommandList& RHICmdList, bool bClearNew)
{
	FAllocation Allocation;
	bool bNewBuffer = ScratchBufferFloat.Allocate(Size, DebugName, RHICmdList, Allocation);
	if (bNewBuffer && bClearNew)
	{
		RHICmdList.Transition(FRHITransitionInfo(Allocation.Buffer->UAV, ScratchBufferFloat.GetExpectedCurrentAccess(), ERHIAccess::UAVCompute));
		RHICmdList.ClearUAVFloat(Allocation.Buffer->UAV, FVector4f(0.0f));
		RHICmdList.Transition(FRHITransitionInfo(Allocation.Buffer->UAV, ERHIAccess::UAVCompute, ScratchBufferFloat.GetExpectedCurrentAccess()));
	}
	return Allocation;
}

template<>
FORCEINLINE FNiagaraGpuScratchPad::FAllocation FNiagaraGpuScratchPad::Alloc<uint32>(uint32 Size, FRHICommandList& RHICmdList, bool bClearNew)
{
	FAllocation Allocation;
	bool bNewBuffer = ScratchBufferUInt.Allocate(Size, DebugName, RHICmdList, Allocation);
	if (bNewBuffer && bClearNew)
	{
		RHICmdList.Transition(FRHITransitionInfo(Allocation.Buffer->UAV, ScratchBufferUInt.GetExpectedCurrentAccess(), ERHIAccess::UAVCompute));
		RHICmdList.ClearUAVUint(Allocation.Buffer->UAV, FUintVector4(0, 0, 0, 0));
		RHICmdList.Transition(FRHITransitionInfo(Allocation.Buffer->UAV, ERHIAccess::UAVCompute, ScratchBufferUInt.GetExpectedCurrentAccess()));
	}
	return Allocation;
}

//////////////////////////////////////////////////////////////////////////

/** Behaves similar to FNiagaraGpuScratchPad but for use with RWStructuredBuffers
*/
template<typename T>
class FNiagaraGpuScratchPadStructured
{
public:
	struct FAllocation
	{
		FRWBufferStructured* Buffer = nullptr;
		uint32 Offset = 0;

		FORCEINLINE bool IsValid()const { return Buffer != nullptr; }
	};

	struct FBuffer
	{
		FRWBufferStructured Buffer;
		uint32 Used = 0;
	};

	FNiagaraGpuScratchPadStructured(uint32 InBucketSize, const FString& InDebugName, EBufferUsageFlags InBufferUsageFlags = EBufferUsageFlags::Static | EBufferUsageFlags::UnorderedAccess)
		: MinBucketSize(InBucketSize)
		, DebugName(InDebugName)
		, BufferUsageFlags(InBufferUsageFlags)
	{
	}

	/** Allocates NumElements from the scratch pad. Will try to allocate from an existing buffer but will create a new one if needed. */
	FAllocation Alloc(FRHICommandListBase& RHICmdList, uint32 NumElements)
	{
		//Look for a buffer to fit this allocation.
		FBuffer* ToUse = nullptr;
		for (FBuffer& Buffer : Buffers)
		{
			uint32 NumBufferElements = Buffer.Buffer.NumBytes / sizeof(T);
			if (NumElements + Buffer.Used < NumBufferElements)
			{
				ToUse = &Buffer;
			}
		}

		if (ToUse == nullptr)
		{
			//Allocate a new buffer that will fit this allocation.

			FBuffer* NewBuffer = new FBuffer();
			Buffers.Add(NewBuffer);

			//Round up the size to pow 2 to ensure a slowly growing user doesn't just keep allocating new buckets. Probably a better guess size here than power of 2.
			uint32 NewBufferSize = FPlatformMath::RoundUpToPowerOfTwo(FMath::Max(MinBucketSize, NumElements));

			NewBuffer->Buffer.Initialize(
				RHICmdList,
				*DebugName,
				sizeof(T),
				NewBufferSize,
				BufferUsageFlags,
				false /*bUseUavCounter*/,
				false /*bAppendBuffer*/,
				CurrentAccess);

			ToUse = NewBuffer;
			NumAllocatedBytes += NewBuffer->Buffer.NumBytes;
		}

		FAllocation Allocation;
		Allocation.Buffer = &ToUse->Buffer;
		Allocation.Offset = ToUse->Used;
		ToUse->Used += NumElements;
		NumUsedBytes += NumElements * sizeof(T);
		return Allocation;
	}

	/** Resets the scratch pad and releases all buffers it holds. */
	void Release()
	{
		Reset();
		CurrentAccess = ERHIAccess::UAVCompute;
		for (FBuffer& Buffer : Buffers)
		{
			Buffer.Buffer.Release();
			Buffer.Used = 0;
		}
		Buffers.Reset();

		NumAllocatedBytes = 0;
	}

	/** Resets the scratch pad, invalidating all previous allocations. Does not modify actual RWBuffer allocations. */
	void Reset()
	{
		for (FBuffer& Buffer : Buffers)
		{
			Buffer.Used = 0;
		}

		NumUsedBytes = 0;
	}

	/** Transitions all buffers in the scratch pad. */
	void Transition(FRHICommandList& RHICmdList, ERHIAccess InPreviousState, ERHIAccess InNewState, EResourceTransitionFlags InFlags = EResourceTransitionFlags::None)
	{
		//Store off current access state so we can place any new buffers in the same state.
		CurrentAccess = InNewState;

 		TArray<FRHITransitionInfo, SceneRenderingAllocator> Transitions;
 		Transitions.Reserve(Buffers.Num());
 		for (FBuffer& Buffer : Buffers)
 		{
 			Transitions.Emplace(Buffer.Buffer.UAV, InPreviousState, InNewState, InFlags);
 		}

 		RHICmdList.Transition(Transitions);
	}

	/** Returns the total bytes allocated for all buffers in the scratch pad. */
	FORCEINLINE uint32 AllocatedBytes()const
	{
		return NumAllocatedBytes;
	}

	/** Returns the number of used bytes allocated for all buffers in the scratch pad. */
	FORCEINLINE uint32 UsedBytes()const
	{
		return NumUsedBytes;
	}

private:

	/** Size of each FRWBuffer in the pool. */
	uint32 MinBucketSize = 4096;
	TIndirectArray<FBuffer> Buffers;
	FString DebugName;
	EBufferUsageFlags BufferUsageFlags = EBufferUsageFlags::None;
	ERHIAccess CurrentAccess = ERHIAccess::UAVCompute;

	uint32 NumAllocatedBytes = 0;
	uint32 NumUsedBytes = 0;
};

