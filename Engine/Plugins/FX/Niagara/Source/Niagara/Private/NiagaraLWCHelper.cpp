// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraLWCHelper.h"

#include "NiagaraComputeExecutionContext.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraDataSet.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraLWCTileShiftPositions.h"

namespace NiagaraLWCHelper
{
using FGPUEmitterArray = TArray<FNiagaraComputeExecutionContext*, TInlineAllocator<8>>;

void RebaseGPUEmitters(FRHICommandListImmediate& RHICmdList, const FVector3f& TileShift, const FGPUEmitterArray& GPUEmitters, FNiagaraGpuComputeDispatchInterface* ComputeInterface)
{
	TArray<FRHITransitionInfo, TInlineAllocator<8>>			TransitionsBefore;
	TArray<FRHITransitionInfo, TInlineAllocator<8>>			TransitionsAfter;
	TArray<FNiagaraLWCTileShiftPositionsCS::FParameters>	ComputeJobs;

	for (FNiagaraComputeExecutionContext* GPUContext : GPUEmitters)
	{
		FNiagaraDataSet* DataSet = GPUContext->MainDataSet;
		FNiagaraDataBuffer* CurrentBuffer = DataSet->GetCurrentData();
		if (CurrentBuffer == nullptr)
		{
			return;
		}

		const uint32 CountBufferOffset = CurrentBuffer->GetGPUInstanceCountBufferOffset();
		if (CountBufferOffset == INDEX_NONE)
		{
			return;
		}

		const uint32 NumInstances = CurrentBuffer->GetNumInstances();
		if (NumInstances == 0)
		{
			return;
		}

		FRHIUnorderedAccessView* FloatBufferUAV = CurrentBuffer->GetGPUBufferFloat().UAV;

		FNiagaraLWCTileShiftPositionsCS::FParameters ShaderParameters;
		ShaderParameters.FloatBuffer		= FloatBufferUAV;
		ShaderParameters.FloatBufferStride	= CurrentBuffer->GetFloatStride() / sizeof(float);
		ShaderParameters.NumInstances		= NumInstances;
		ShaderParameters.CountBuffer		= ComputeInterface->GetGPUInstanceCounterManager().GetInstanceCountBuffer().SRV;
		ShaderParameters.CountBufferOffset	= CountBufferOffset;
		ShaderParameters.TileShift			= TileShift;

		const TConstArrayView<FNiagaraVariableLayoutInfo> VariableLayouts(DataSet->GetCompiledData().VariableLayouts);
		const TConstArrayView<FNiagaraVariableBase> Variables(DataSet->GetCompiledData().Variables);
		const int32 NumVariables = FMath::Min(VariableLayouts.Num(), Variables.Num());
		uint32 NumPositions = 0;
		bool bNeedsTransitions = false;

		for (int32 iVariable = 0; iVariable < NumVariables; ++iVariable)
		{
			if (Variables[iVariable].GetType() != FNiagaraTypeDefinition::GetPositionDef())
			{
				continue;
			}

			if (ensure(VariableLayouts[iVariable].GetNumFloatComponents() == 3))
			{
				const uint32 FloatOffset = VariableLayouts[iVariable].GetFloatComponentStart();
				GET_SCALAR_ARRAY_ELEMENT(ShaderParameters.PositionComponentOffsets, NumPositions) = FloatOffset;
				++NumPositions;
				if (NumPositions >= FNiagaraLWCTileShiftPositionsCS::MaxPositions)
				{
					ShaderParameters.NumPositions = NumPositions;
					bNeedsTransitions = true;
					ComputeJobs.Add(ShaderParameters);
					NumPositions = 0;
				}
			}
		}

		if (NumPositions > 0)
		{
			ShaderParameters.NumPositions = NumPositions;
			bNeedsTransitions = true;
			ComputeJobs.Add(ShaderParameters);
		}

		if (bNeedsTransitions)
		{
			TransitionsBefore.Emplace(FloatBufferUAV, ERHIAccess::SRVMask, ERHIAccess::UAVCompute);
			TransitionsAfter.Emplace(FloatBufferUAV, ERHIAccess::UAVCompute, ERHIAccess::SRVMask);
		}
	}

	if (ComputeJobs.Num() > 0)
	{
		RHICmdList.Transition(TransitionsBefore);
		for (const FNiagaraLWCTileShiftPositionsCS::FParameters& Parameters : ComputeJobs)
		{
			FNiagaraLWCTileShiftPositionsCS::Execute(RHICmdList, Parameters);
		}
		RHICmdList.Transition(TransitionsAfter);
	}
}

void RebaseEmitters(const FVector3f& TileShift, TArrayView<FNiagaraEmitterInstanceRef> Emitters, FNiagaraGpuComputeDispatchInterface* ComputeInterface)
{
	FGPUEmitterArray GPUEmittersToRebase;

	for (const FNiagaraEmitterInstanceRef& EmitterRef : Emitters)
	{
		FNiagaraEmitterInstanceImpl* EmitterInstance = EmitterRef->AsStateful();
		if (EmitterInstance == nullptr || EmitterInstance->IsLocalSpace())
		{
			return;
		}

		// GPU data to rebase
		if (FNiagaraComputeExecutionContext* GPUContext = EmitterInstance->GetGPUContext())
		{
			if (ComputeInterface && EmitterInstance->GetNumParticles() > 0)
			{
				GPUEmittersToRebase.Add(GPUContext);
			}
		}
		// CPU data to rebase
		else
		{
			RebaseDataSet(TileShift, EmitterInstance->GetParticleData());
		}
	}

	if (GPUEmittersToRebase.Num() > 0)
	{
		ENQUEUE_RENDER_COMMAND(RebaseGPUEmitter)
		(
			[TileShift, GPUEmittersToRebase, ComputeInterface](FRHICommandListImmediate& RHICmdList)
			{
				RebaseGPUEmitters(RHICmdList, TileShift, GPUEmittersToRebase, ComputeInterface);
			}
		);
	}
}

void RebaseDataSet(const FVector3f& TileShift, FNiagaraDataSet& DataSet, uint32 iInstance)
{
	FNiagaraDataBuffer* CurrentData = DataSet.GetCurrentData();
	if (CurrentData == nullptr || iInstance < CurrentData->GetNumInstances())
	{
		return;
	}

	const TConstArrayView<FNiagaraVariableLayoutInfo> VariableLayouts(DataSet.GetCompiledData().VariableLayouts);
	const TConstArrayView<FNiagaraVariableBase> Variables(DataSet.GetCompiledData().Variables);
	const int32 NumVariables = FMath::Min(VariableLayouts.Num(), Variables.Num());
	for (int32 iVariable = 0; iVariable < NumVariables; ++iVariable)
	{
		if (Variables[iVariable].GetType() != FNiagaraTypeDefinition::GetPositionDef())
		{
			continue;
		}

		if (ensure(VariableLayouts[iVariable].GetNumFloatComponents() == 3))
		{
			const uint32 FloatOffset = VariableLayouts[iVariable].GetFloatComponentStart();
			float* XComponent = reinterpret_cast<float*>(CurrentData->GetComponentPtrFloat(FloatOffset + 0));
			float* YComponent = reinterpret_cast<float*>(CurrentData->GetComponentPtrFloat(FloatOffset + 1));
			float* ZComponent = reinterpret_cast<float*>(CurrentData->GetComponentPtrFloat(FloatOffset + 2));
			XComponent[iInstance] += TileShift.X;
			YComponent[iInstance] += TileShift.Y;
			ZComponent[iInstance] += TileShift.Z;
		}
	}
}

void RebaseDataSet(const FVector3f& TileShift, FNiagaraDataSet& DataSet)
{
	FNiagaraDataBuffer* CurrentData = DataSet.GetCurrentData();
	if (CurrentData == nullptr || CurrentData->GetNumInstances() == 0)
	{
		return;
	}

	const TConstArrayView<FNiagaraVariableLayoutInfo> VariableLayouts(DataSet.GetCompiledData().VariableLayouts);
	const TConstArrayView<FNiagaraVariableBase> Variables(DataSet.GetCompiledData().Variables);
	const int32 NumVariables = FMath::Min(VariableLayouts.Num(), Variables.Num());
	for (int32 iVariable = 0; iVariable < NumVariables; ++iVariable)
	{
		if (Variables[iVariable].GetType() != FNiagaraTypeDefinition::GetPositionDef())
		{
			continue;
		}

		if (ensure(VariableLayouts[iVariable].GetNumFloatComponents() == 3))
		{
			const uint32 FloatOffset = VariableLayouts[iVariable].GetFloatComponentStart();
			float* XComponent = reinterpret_cast<float*>(CurrentData->GetComponentPtrFloat(FloatOffset + 0));
			float* YComponent = reinterpret_cast<float*>(CurrentData->GetComponentPtrFloat(FloatOffset + 1));
			float* ZComponent = reinterpret_cast<float*>(CurrentData->GetComponentPtrFloat(FloatOffset + 2));
			for (uint32 iInstance = 0; iInstance < CurrentData->GetNumInstances(); ++iInstance)
			{
				XComponent[iInstance] += TileShift.X;
				YComponent[iInstance] += TileShift.Y;
				ZComponent[iInstance] += TileShift.Z;
			}
		}
	}
}

} //namespace NiagaraLWCHelper
