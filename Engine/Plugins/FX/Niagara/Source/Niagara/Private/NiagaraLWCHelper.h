// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEmitterInstance.h"

class FNiagaraDataSet;
class FNiagaraGpuComputeDispatchInterface;

namespace NiagaraLWCHelper
{
	extern void RebaseEmitters(const FVector3f& TileShift, TArrayView<FNiagaraEmitterInstanceRef> Emitters, FNiagaraGpuComputeDispatchInterface* ComputeInterface);
	extern void RebaseDataSet(const FVector3f& TileShift, FNiagaraDataSet& DataSet, uint32 iInstance);
	extern void RebaseDataSet(const FVector3f& TileShift, FNiagaraDataSet& DataSet);
}
