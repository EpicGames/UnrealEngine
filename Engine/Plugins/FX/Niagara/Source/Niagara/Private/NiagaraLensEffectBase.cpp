// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraLensEffectBase.h"

#include "NiagaraComponent.h"
#include "NiagaraSystem.h"
#include "Camera/PlayerCameraManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraLensEffectBase)


ANiagaraLensEffectBase::ANiagaraLensEffectBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer
			.DoNotCreateDefaultSubobject(TEXT("Sprite"))
			.DoNotCreateDefaultSubobject(TEXT("ArrowComponent0"))
		   )
	, DesiredRelativeTransform(FTransform::Identity)
	, BaseAuthoredFOV(80.f)
	, bAllowMultipleInstances(false)
	, bResetWhenRetriggered(true)
{
	SetDestroyOnSystemFinish(true);

	if (UNiagaraComponent* NSC = GetNiagaraComponent())
	{
		NSC->bOnlyOwnerSee = true;
		NSC->bAutoActivate = false;
	}
}

const FTransform& ANiagaraLensEffectBase::GetRelativeTransform() const
{
	return DesiredRelativeTransform;
}

float ANiagaraLensEffectBase::GetBaseFOV() const
{
	return BaseAuthoredFOV;
}

bool ANiagaraLensEffectBase::ShouldAllowMultipleInstances() const
{
	return bAllowMultipleInstances;
}

bool ANiagaraLensEffectBase::ResetWhenTriggered() const
{
	return bResetWhenRetriggered;
}

bool ANiagaraLensEffectBase::ShouldTreatEmitterAsSame(TSubclassOf<AActor> OtherEmitter) const
{
	return OtherEmitter && (OtherEmitter == GetClass() || EmittersToTreatAsSame.Find(OtherEmitter) != INDEX_NONE);
}

void ANiagaraLensEffectBase::NotifyWillBePooled()
{
	SetDestroyOnSystemFinish(false);
}

void ANiagaraLensEffectBase::AdjustBaseFOV(float NewFOV)
{
	BaseAuthoredFOV = NewFOV;
}

void ANiagaraLensEffectBase::RegisterCamera(APlayerCameraManager* CameraManager)
{
	OwningCameraManager = CameraManager;
}

void ANiagaraLensEffectBase::NotifyRetriggered()
{
	// only play the camera effect on clients
	UWorld const* const World = GetWorld();
	check(World);
	if (!IsNetMode(NM_DedicatedServer))
	{
		if (UNiagaraComponent* NSC = GetNiagaraComponent())
		{
			NSC->Activate(bResetWhenRetriggered);
		}
	}
}

void ANiagaraLensEffectBase::ActivateLensEffect()
{  
	// only play the camera effect on clients
	UWorld const* const World = GetWorld();
	check(World);
	if(!IsNetMode(NM_DedicatedServer))
	{
		if (UNiagaraComponent* NSC = GetNiagaraComponent())
		{
			NSC->Activate();
		}
	}
}

void ANiagaraLensEffectBase::DeactivateLensEffect()
{
	if (UNiagaraComponent* NSC = GetNiagaraComponent())
	{
		NSC->Deactivate();
	}
}

void ANiagaraLensEffectBase::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	if (OwningCameraManager)
	{
		OwningCameraManager->RemoveGenericCameraLensEffect(this);
	}
}

bool ANiagaraLensEffectBase::IsLooping() const
{
	if (UNiagaraSystem* System = GetNiagaraComponent()->GetAsset())
	{
		return System->IsLooping();
	}

	return false;
}

