// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraLightRendererProperties.h"
#include "NiagaraBoundsCalculator.h"
#include "NiagaraRenderer.h"
#include "NiagaraConstants.h"
#include "NiagaraModule.h"
#include "NiagaraRendererLights.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraLightRendererProperties)

#if WITH_EDITOR
#include "Widgets/Images/SImage.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/SWidget.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Text/STextBlock.h"
#endif

#define LOCTEXT_NAMESPACE "UNiagaraLightRendererProperties"

namespace NiagaraLightRendererPropertiesPrivate
{
	static FLinearColor GDefaultColor = FLinearColor::White;
	static float GDefaultRadius = 0.0f;
	static float GDefaultVolumetricScattering = 0.0f;
}

TArray<TWeakObjectPtr<UNiagaraLightRendererProperties>> UNiagaraLightRendererProperties::LightRendererPropertiesToDeferredInit;

namespace NiagaraLightRendererPropertiesPrivate
{
	static const FName NAME_UniqueID("UniqueID");
	static FNiagaraVariableBase GetUniqueIDVariable() { return FNiagaraVariableBase(FNiagaraTypeDefinition::GetIntDef(), NAME_UniqueID); }
}

UNiagaraLightRendererProperties::UNiagaraLightRendererProperties()
	: bUseInverseSquaredFalloff(1)
	, bAffectsTranslucency(0)
	, bAllowMegaLights(0)
	, bMegaLightsCastShadows(0)
	, bAlphaScalesBrightness(0)
	, RadiusScale(1.0f)
	, DefaultExponent(1.0f)
	, ColorAdd(FVector(0.0f, 0.0f, 0.0f))
{
	AttributeBindings =
	{
		&LightRenderingEnabledBinding,
		&LightExponentBinding,
		&PositionBinding,
		&ColorBinding,
		&RadiusBinding,
		&VolumetricScatteringBinding,
		&RendererVisibilityTagBinding,
		&SpecularScaleBinding,
		&DiffuseScaleBinding,
	};
}

void UNiagaraLightRendererProperties::PostLoad()
{
	Super::PostLoad();
	
#if WITH_EDITORONLY_DATA
	SourceMode = ValidateSourceMode(SourceMode);
	ChangeToPositionBinding(PositionBinding);
#endif
	PostLoadBindings(ENiagaraRendererSourceDataMode::Particles);
}

void UNiagaraLightRendererProperties::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		// We can end up hitting PostInitProperties before the Niagara Module has initialized bindings this needs, mark this object for deferred init and early out.
		if (FModuleManager::Get().IsModuleLoaded("Niagara") == false)
		{
			LightRendererPropertiesToDeferredInit.Add(this);
			return;
		}
		else if (!PositionBinding.IsValid())
		{
			PositionBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
			ColorBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_COLOR);
			RadiusBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_RADIUS);
			LightExponentBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_EXPONENT);
			LightRenderingEnabledBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_ENABLED);
			VolumetricScatteringBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING);
			RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
			SpecularScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE);
			DiffuseScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_DIFFUSE_SCALE);
		}
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraLightRendererProperties::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraLightRendererProperties, SourceMode))
	{
		UpdateSourceModeDerivates(SourceMode, true);
	}
}
#endif// WITH_EDITORONLY_DATA

/** The bindings depend on variables that are created during the NiagaraModule startup. However, the CDO's are build prior to this being initialized, so we defer setting these values until later.*/
void UNiagaraLightRendererProperties::InitCDOPropertiesAfterModuleStartup()
{
	UNiagaraLightRendererProperties* CDO = CastChecked<UNiagaraLightRendererProperties>(UNiagaraLightRendererProperties::StaticClass()->GetDefaultObject());
	CDO->InitRendererBaseBindings();
	CDO->PositionBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
	CDO->ColorBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_COLOR);
	CDO->RadiusBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_RADIUS);
	CDO->LightExponentBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_EXPONENT);
	CDO->LightRenderingEnabledBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_ENABLED);
	CDO->VolumetricScatteringBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING);
	CDO->RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
	CDO->SpecularScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE);
	CDO->DiffuseScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_DIFFUSE_SCALE);

	NiagaraLightRendererPropertiesPrivate::GDefaultColor = FNiagaraConstants::GetAttributeWithDefaultValue(SYS_PARAM_PARTICLES_COLOR).GetValue<FLinearColor>();
	NiagaraLightRendererPropertiesPrivate::GDefaultRadius = FNiagaraConstants::GetAttributeWithDefaultValue(SYS_PARAM_PARTICLES_LIGHT_RADIUS).GetValue<float>();
	NiagaraLightRendererPropertiesPrivate::GDefaultVolumetricScattering = FNiagaraConstants::GetAttributeWithDefaultValue(SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING).GetValue<float>();

	for (TWeakObjectPtr<UNiagaraLightRendererProperties>& WeakLightRendererProperties : LightRendererPropertiesToDeferredInit)
	{
		if (UNiagaraLightRendererProperties* RendererProperties = WeakLightRendererProperties.Get())
		{
			if (!RendererProperties->PositionBinding.IsValid())
			{
				RendererProperties->InitRendererBaseBindings();
				RendererProperties->PositionBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
				RendererProperties->ColorBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_COLOR);
				RendererProperties->RadiusBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_RADIUS);
				RendererProperties->LightExponentBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_EXPONENT);
				RendererProperties->LightRenderingEnabledBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_ENABLED);
				RendererProperties->VolumetricScatteringBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING);
				RendererProperties->RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
				RendererProperties->SpecularScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE);
				RendererProperties->DiffuseScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_LIGHT_DIFFUSE_SCALE);
			}
		}
	}
}

FNiagaraRenderer* UNiagaraLightRendererProperties::CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
	FNiagaraRenderer* NewRenderer = new FNiagaraRendererLights(FeatureLevel, this, Emitter);
	NewRenderer->Initialize(this, Emitter, InController);
	return NewRenderer;
}

class FNiagaraBoundsCalculatorLights : public FNiagaraBoundsCalculator
{
public:
	FNiagaraBoundsCalculatorLights(const FNiagaraDataSetAccessor<FNiagaraPosition>& InPositionAccessor, const FNiagaraDataSetAccessor<float>& InRadiusAccessor, float InRadiusScale)
		: PositionAccessor(InPositionAccessor)
		, RadiusAccessor(InRadiusAccessor)
		, RadiusScale(InRadiusScale)
	{}

	FNiagaraBoundsCalculatorLights() = delete;
	virtual ~FNiagaraBoundsCalculatorLights() = default;

	virtual void InitAccessors(const FNiagaraDataSetCompiledData* CompiledData) override {}
	virtual FBox CalculateBounds(const FTransform& SystemTransform, const FNiagaraDataSet& DataSet, const int32 NumInstances) const override
	{
		if (!NumInstances || !PositionAccessor.IsValid())
		{
			return FBox(ForceInit);
		}

		FNiagaraPosition BoundsMin(ForceInitToZero);
		FNiagaraPosition BoundsMax(ForceInitToZero);
		PositionAccessor.GetReader(DataSet).GetMinMax(BoundsMin, BoundsMax);
		FBox Bounds(BoundsMin, BoundsMax);

		float MaxRadius = 0.0f;
		if (RadiusAccessor.IsValid())
		{
			MaxRadius = RadiusAccessor.GetReader(DataSet).GetMax();
		}
		else
		{
			MaxRadius = FNiagaraConstants::GetAttributeWithDefaultValue(SYS_PARAM_PARTICLES_LIGHT_RADIUS).GetValue<float>();
		}

		return Bounds.ExpandBy(MaxRadius * RadiusScale);
	}

protected:
	const FNiagaraDataSetAccessor<FNiagaraPosition>& PositionAccessor;
	const FNiagaraDataSetAccessor<float>& RadiusAccessor;
	const float RadiusScale = 1.0f;
};

FNiagaraBoundsCalculator* UNiagaraLightRendererProperties::CreateBoundsCalculator()
{
	if (GetCurrentSourceMode() == ENiagaraRendererSourceDataMode::Emitter)
	{
		return nullptr;
	}

	return new FNiagaraBoundsCalculatorLights(PositionDataSetAccessor, RadiusDataSetAccessor, RadiusScale);
}

void UNiagaraLightRendererProperties::GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const
{
	//OutMaterials.Add(Material);
	//Material should live here.
}

void UNiagaraLightRendererProperties::CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData)
{
	UpdateSourceModeDerivates(SourceMode);

	LightIdAccessor.Init(CompiledData, NiagaraLightRendererPropertiesPrivate::NAME_UniqueID);

	InitParticleDataSetAccessor(PositionDataSetAccessor, CompiledData, PositionBinding);
	InitParticleDataSetAccessor(ColorDataSetAccessor, CompiledData, ColorBinding);
	InitParticleDataSetAccessor(RadiusDataSetAccessor, CompiledData, RadiusBinding);
	InitParticleDataSetAccessor(ExponentDataSetAccessor, CompiledData, LightExponentBinding);
	InitParticleDataSetAccessor(ScatteringDataSetAccessor, CompiledData, VolumetricScatteringBinding);
	InitParticleDataSetAccessor(EnabledDataSetAccessor, CompiledData, LightRenderingEnabledBinding);
	InitParticleDataSetAccessor(RendererVisibilityTagAccessor, CompiledData, RendererVisibilityTagBinding);
	InitParticleDataSetAccessor(SpecularScaleAccessor, CompiledData, SpecularScaleBinding);
	InitParticleDataSetAccessor(DiffuseScaleAccessor, CompiledData, DiffuseScaleBinding);
}

FLinearColor UNiagaraLightRendererProperties::GetDefaultColor()
{
	return NiagaraLightRendererPropertiesPrivate::GDefaultColor;
}

float UNiagaraLightRendererProperties::GetDefaultRadius()
{
	return NiagaraLightRendererPropertiesPrivate::GDefaultRadius;
}

float UNiagaraLightRendererProperties::GetDefaultVolumetricScattering()
{
	return NiagaraLightRendererPropertiesPrivate::GDefaultVolumetricScattering;
}

void UNiagaraLightRendererProperties::UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit)
{
	Super::UpdateSourceModeDerivates(InSourceMode, bFromPropertyEdit);
}

bool UNiagaraLightRendererProperties::PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore)
{
	bool bAnyAdded = Super::PopulateRequiredBindings(InParameterStore);

	for (const FNiagaraVariableAttributeBinding* Binding : AttributeBindings)
	{
		if (Binding && Binding->CanBindToHostParameterMap())
		{
			InParameterStore.AddParameter(Binding->GetParamMapBindableVariable(), false);
			bAnyAdded = true;
		}
	}

	return bAnyAdded;
}

#if WITH_EDITORONLY_DATA
TArray<FNiagaraVariable> UNiagaraLightRendererProperties::GetBoundAttributes() const
{
	TArray<FNiagaraVariable> BoundAttributes = Super::GetBoundAttributes();

	// UniqueID is needed by MegaLights to track light visibility
	BoundAttributes.AddUnique(NiagaraLightRendererPropertiesPrivate::GetUniqueIDVariable());

	return BoundAttributes;
}

const TArray<FNiagaraVariable>& UNiagaraLightRendererProperties::GetOptionalAttributes()
{
	static TArray<FNiagaraVariable> Attrs;
	if (Attrs.Num() == 0)
	{
		Attrs.Add(SYS_PARAM_PARTICLES_POSITION);
		Attrs.Add(SYS_PARAM_PARTICLES_COLOR);
		Attrs.Add(SYS_PARAM_PARTICLES_LIGHT_RADIUS);
		Attrs.Add(SYS_PARAM_PARTICLES_LIGHT_EXPONENT);
		Attrs.Add(SYS_PARAM_PARTICLES_LIGHT_ENABLED);
		Attrs.Add(SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING);
		Attrs.Add(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
		Attrs.Add(SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE);
	}
	return Attrs;
}

void UNiagaraLightRendererProperties::GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	TSharedRef<SWidget> LightTooltip = SNew(STextBlock)
		.Text(LOCTEXT("LightRenderer", "Light Renderer"));
	OutWidgets.Add(LightTooltip);
}

void UNiagaraLightRendererProperties::GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FText>& OutErrors, TArray<FText>& OutWarnings, TArray<FText>& OutInfo) const
{
	Super::GetRendererFeedback(InEmitter, OutErrors, OutWarnings, OutInfo);
}

#endif // WITH_EDITORONLY_DATA

#undef LOCTEXT_NAMESPACE
