// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMeshRendererProperties.h"
#include "Materials/Material.h"
#include "NiagaraRendererMeshes.h"
#include "NiagaraRenderableMeshArrayInterface.h"
#include "NiagaraConstants.h"
#include "NiagaraBoundsCalculatorHelper.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraGPUSortInfo.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemImpl.h"

#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "Materials/MaterialRenderProxy.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Modules/ModuleManager.h"
#include "MaterialDomain.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraMeshRendererProperties)

#if WITH_EDITOR
#include "AssetThumbnail.h"
#include "NiagaraModule.h"
#include "Styling/SlateIconFinder.h"
#include "Internationalization/Regex.h"
#include "Dialogs/Dialogs.h"
#include "Framework/Notifications/NotificationManager.h"
#include "StaticMeshResources.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#define LOCTEXT_NAMESPACE "UNiagaraMeshRendererProperties"

TArray<TWeakObjectPtr<UNiagaraMeshRendererProperties>> UNiagaraMeshRendererProperties::MeshRendererPropertiesToDeferredInit;


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace NiagaraMeshRendererPropertiesInternal
{
	bool IsStaticMeshSafeToUse(const FNiagaraEmitterInstance* EmitterInstance, UStaticMesh* StaticMesh)
	{
		if (StaticMesh)
		{
			bool bInvalidDueToCompiling = false;
#if WITH_EDITOR
			// During EOF updates we can not test GetRenderData as we will cause a wait / reregister to occur which is invalid while in EOF updates
			// The assumption is that when we do hit this situation we are waiting on a static mesh build from reimport, etc, so we can just skip until
			// we get the post build callback, ideally we would have a better way to handle this by not rendering while the build is in progress but
			// that requires quite a large rework of how we handle reading from the static mesh data, especially around updating MICs in PostLoad.
			if (StaticMesh->IsCompiling())
			{
				FNiagaraSystemInstance* SystemInstance = EmitterInstance ? EmitterInstance->GetParentSystemInstance() : nullptr;
				UWorld* World = SystemInstance ? SystemInstance->GetWorld() : nullptr;
				bInvalidDueToCompiling = World && World->bPostTickComponentUpdate;
			}
			else
#endif
			{
				const FNiagaraSystemInstance* SystemInstance = EmitterInstance ? EmitterInstance->GetParentSystemInstance() : nullptr;
				const EShaderPlatform ShaderPlatform = SystemInstance != nullptr ? GShaderPlatformForFeatureLevel[SystemInstance->GetFeatureLevel()] : GMaxRHIShaderPlatform;

				if (StaticMesh->HasValidNaniteData() && !StaticMesh->HasNaniteFallbackMesh(ShaderPlatform))
				{
					GEngine->AddOnScreenDebugMessage(uint64(EmitterInstance) ^ uint64(StaticMesh), 10.f, FColor::Yellow,
						*FString::Printf(TEXT("NiagaraRenderableMesh does not have a fallback mesh. Enable Nanite fallback mesh to fix this issue. System: %s, Mesh: %s"), *GetFullNameSafe(SystemInstance ? SystemInstance->GetSystem() : nullptr), *GetFullNameSafe(StaticMesh)));

					if (FNiagaraUtilities::LogVerboseWarnings())
					{
						UE_LOGF(LogNiagara, Warning, "NiagaraRenderableMesh does not have a fallback mesh. Enable Nanite fallback mesh to fix this issue. System: %ls, Mesh: %ls", *GetFullNameSafe(SystemInstance ? SystemInstance->GetSystem() : nullptr), *GetFullNameSafe(StaticMesh));
					}

					return false;
				}
			}

			return !bInvalidDueToCompiling && StaticMesh->GetRenderData();
		}
		return false;
	}

	void ResolveRenderableMeshInternal(const FNiagaraMeshRendererMeshProperties& MeshProperties, const FNiagaraEmitterInstance* EmitterInstance, INiagaraRenderableMeshInterface*& OutInterface, UStaticMesh*& OutStaticMesh)
	{
		OutInterface = nullptr;
		OutStaticMesh = nullptr;
		if (EmitterInstance)
		{
			const FNiagaraVariableBase& MeshParameter = MeshProperties.MeshParameterBinding.ResolvedParameter;
			if (MeshParameter.IsValid())
			{
				if (MeshParameter.IsDataInterface())
				{
					OutInterface = Cast<INiagaraRenderableMeshInterface>(EmitterInstance->GetRendererBoundVariables().GetDataInterface(MeshParameter));
					if (OutInterface != nullptr)
					{
						return;
					}
				}

				UStaticMesh* BoundMesh = Cast<UStaticMesh>(EmitterInstance->GetRendererBoundVariables().GetUObject(MeshParameter));
				if (IsStaticMeshSafeToUse(EmitterInstance, BoundMesh))
				{
					OutStaticMesh = BoundMesh;
					return;
				}
			}
		}
		if (IsStaticMeshSafeToUse(EmitterInstance, MeshProperties.Mesh))
		{
			OutStaticMesh = MeshProperties.Mesh;
		}
	}

#if WITH_EDITOR
	struct FMeshChangedHandlers
	{
		FDelegateHandle OnMeshChanged;
		FDelegateHandle OnPreMeshBuild;
		FDelegateHandle OnPostMeshBuild;
	};

	static TMap<FObjectKey, FMeshChangedHandlers> GOnMeshChangesObjects;

	void AddMeshChangedIfNeeded(UStaticMesh* StaticMesh)
	{
		if (GIsEditor && StaticMesh && IsInGameThread() && !GOnMeshChangesObjects.Contains(StaticMesh))
		{
			FMeshChangedHandlers& Handlers = GOnMeshChangesObjects.Add(StaticMesh);
			Handlers.OnMeshChanged = StaticMesh->GetOnMeshChanged().AddLambda([StaticMesh]() { UNiagaraMeshRendererProperties::OnMeshChanged(StaticMesh); });
			Handlers.OnPreMeshBuild = StaticMesh->OnPreMeshBuild().AddLambda([](UStaticMesh* StaticMesh) { UNiagaraMeshRendererProperties::OnMeshChanged(StaticMesh); });
			Handlers.OnPostMeshBuild = StaticMesh->OnPostMeshBuild().AddLambda([](UObject* StaticMesh) { UNiagaraMeshRendererProperties::OnMeshChanged(StaticMesh); });
		}
	}

	void RemoveMeshChangedHandlers()
	{
		if (GIsEditor)
		{
			for (auto It=GOnMeshChangesObjects.CreateIterator(); It; ++It)
			{
				FObjectKey MeshObjectKey = It.Key();
				if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(MeshObjectKey.ResolveObjectPtr()))
				{
					FMeshChangedHandlers& Handlers = It.Value();

					StaticMesh->GetOnMeshChanged().Remove(Handlers.OnMeshChanged);
					StaticMesh->OnPreMeshBuild().Remove(Handlers.OnPreMeshBuild);
					StaticMesh->OnPostMeshBuild().Remove(Handlers.OnPostMeshBuild);
				}
			}
			GOnMeshChangesObjects.Empty();
		}
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FNiagaraRenderableStaticMesh : public INiagaraRenderableMesh
{
public:
	FNiagaraRenderableStaticMesh(const UStaticMesh* StaticMesh)
	{
		WeakStaticMesh	= StaticMesh;
		RenderData		= StaticMesh->GetRenderData();
		MeshMinLOD		= StaticMesh->GetMinLODIdx();
		MinLOD			= StaticMesh->GetMinLODIdx();
		LocalBounds		= StaticMesh->GetExtendedBounds().GetBox();

	#if WITH_EDITOR
		// Const cast as it's just for the editor to simplify the code
		NiagaraMeshRendererPropertiesInternal::AddMeshChangedIfNeeded(const_cast<UStaticMesh*>(StaticMesh));
	#endif
	}

	void SetMinLODBias(int32 MinLODBias) override
	{
		MinLOD = FMath::Max(MeshMinLOD + MinLODBias, 0);
	}

	FBox GetLocalBounds() const override
	{
		return LocalBounds;
	}

	virtual void GetLODModelData(FLODModelData& OutLODModelData, int32 LODLevel) const override
	{
		LODLevel = FMath::Max(MinLOD, LODLevel);
		OutLODModelData.LODIndex = RenderData->GetCurrentFirstLODIdx(LODLevel);
		if (!RenderData->LODResources.IsValidIndex(OutLODModelData.LODIndex))
		{
			OutLODModelData.LODIndex = INDEX_NONE;
			return;
		}

		const FStaticMeshLODResources& LODResources = RenderData->LODResources[OutLODModelData.LODIndex];

		OutLODModelData.NumVertices = LODResources.GetNumVertices();
		OutLODModelData.NumIndices = LODResources.IndexBuffer.GetNumIndices();
		OutLODModelData.Sections = MakeArrayView(LODResources.Sections);
		OutLODModelData.IndexBuffer = &LODResources.IndexBuffer;
		OutLODModelData.VertexFactoryUserData = RenderData->LODVertexFactories.IsValidIndex(OutLODModelData.LODIndex) ? RenderData->LODVertexFactories[OutLODModelData.LODIndex].VertexFactory.GetUniformBuffer() : nullptr;
		OutLODModelData.RayTracingGeometry = nullptr;

		if (LODResources.AdditionalIndexBuffers != nullptr && LODResources.AdditionalIndexBuffers->WireframeIndexBuffer.IsInitialized())
		{
			OutLODModelData.WireframeNumIndices = LODResources.AdditionalIndexBuffers->WireframeIndexBuffer.GetNumIndices();
			OutLODModelData.WireframeIndexBuffer = &LODResources.AdditionalIndexBuffers->WireframeIndexBuffer;
		}
	}

#if RHI_RAYTRACING
	virtual void GetRayTraceLODModelData(FLODModelData& OutLODModelData, int32 LODLevel) const override
	{
		LODLevel = FMath::Max(MinLOD, LODLevel);

		if (RenderData->RayTracingProxy != nullptr)
		{
			FStaticMeshRayTracingProxyLODArray& RayTracingLODs = RenderData->RayTracingProxy->LODs;

			const int32 NumLODs = RayTracingLODs.Num();
			const int32 RayTracingMinLOD = RenderData->RayTracingProxy->bUsingRenderingLODs ? RenderData->GetCurrentFirstLODIdx(LODLevel) : 0;

			for (int32 LODIndex = RayTracingMinLOD; LODIndex < NumLODs; ++LODIndex)
			{
				const FStaticMeshRayTracingProxyLOD& RayTracingLOD = RayTracingLODs[LODIndex];
				const FRayTracingGeometry& RayTracingGeometry = *RayTracingLOD.RayTracingGeometry;

				if (RayTracingGeometry.IsValid() && !RayTracingGeometry.IsEvicted() && !RayTracingGeometry.HasPendingBuildRequest())
				{
					OutLODModelData.LODIndex = LODIndex;
					OutLODModelData.RayTracingGeometry = &RayTracingGeometry;
					OutLODModelData.NumVertices = RayTracingLOD.VertexBuffers->StaticMeshVertexBuffer.GetNumVertices();
					OutLODModelData.NumIndices = RayTracingLOD.IndexBuffer->GetNumIndices();
					OutLODModelData.Sections = *RayTracingLOD.Sections;
					OutLODModelData.IndexBuffer = RayTracingLOD.IndexBuffer;
					OutLODModelData.VertexFactoryUserData = RenderData->RayTracingProxy->LODVertexFactories->IsValidIndex(OutLODModelData.LODIndex) ? (*RenderData->RayTracingProxy->LODVertexFactories)[OutLODModelData.LODIndex].VertexFactory.GetUniformBuffer() : nullptr;

					return;
				}
			}
		}
		else
		{
#if DO_CHECK
			const UStaticMesh* StaticMesh = WeakStaticMesh.Get();
			check(!IsRayTracingAllowed() || StaticMesh == nullptr || !StaticMesh->bSupportRayTracing);
#endif
		}
	}
#endif

	virtual FIntVector2 GetLODRange() const override
	{
		return FIntVector2(MeshMinLOD, RenderData->LODResources.Num());
	}

	virtual FVector3f GetLODScreenSize(int32 LODLevel) const override
	{
		constexpr int32 MaxLODLevel = MAX_STATIC_MESH_LODS - 1;
		LODLevel = FMath::Clamp(LODLevel, 0, MaxLODLevel);
		return FVector3f(
			LODLevel < MaxLODLevel ? RenderData->ScreenSize[LODLevel + 1].GetValue() : 0.0f,
			RenderData->ScreenSize[LODLevel].GetValue(),
			static_cast<float>(RenderData->Bounds.SphereRadius)
		);
	}

	virtual int32 ComputeLOD(const FVector& SphereOrigin, const float SphereRadius, const FSceneView& SceneView, float LODDistanceFactor) override
	{
		return ComputeStaticMeshLOD(RenderData, SphereOrigin, SphereRadius, SceneView, MinLOD, LODDistanceFactor);
	}

	static void InitVertexFactoryComponents(
		const FStaticMeshVertexBuffers& VertexBuffers,
		FNiagaraMeshVertexFactory* VertexFactory,
		FStaticMeshDataType& OutData)
	{
		VertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(VertexFactory, OutData);
		VertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(VertexFactory, OutData);
		VertexBuffers.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(VertexFactory, OutData, MAX_TEXCOORDS);
		VertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(VertexFactory, OutData);
	}

	void SetupVertexFactory(FRHICommandListBase& RHICmdList, FNiagaraMeshVertexFactory& InVertexFactory, const FLODModelData& LODModelData) const override
	{
		FStaticMeshDataType Data;
		const FStaticMeshLODResources& LODResources = RenderData->LODResources[LODModelData.LODIndex];
		InitVertexFactoryComponents(LODResources.VertexBuffers, &InVertexFactory, Data);
		InVertexFactory.SetData(RHICmdList, Data);
	}

#if RHI_RAYTRACING
	void SetupRayTraceVertexFactory(FRHICommandListBase& RHICmdList, class FNiagaraMeshVertexFactory& InVertexFactory, const FLODModelData& LODModelData) const override
	{
		FStaticMeshDataType Data;
		InitVertexFactoryComponents(*RenderData->RayTracingProxy->LODs[LODModelData.LODIndex].VertexBuffers, &InVertexFactory, Data);
		InVertexFactory.SetData(RHICmdList, Data);
	}
#endif

	void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials) const override
	{
		const UStaticMesh* StaticMesh = WeakStaticMesh.Get();
		if (StaticMesh == nullptr)
		{
			UE_LOGF(LogNiagara, Log, "FNiagaraRenderableStaticMesh - StaticMesh is no longer valid");
			return;
		}

		// Retrieve a list of materials whose indices match up with the mesh, and only fill it in with materials that are used by any section of any LOD
		for (const FStaticMeshLODResources& LODModel : RenderData->LODResources)
		{
			for (const FStaticMeshSection& Section : LODModel.Sections)
			{
				if (Section.MaterialIndex >= 0)
				{
					if (Section.MaterialIndex >= OutMaterials.Num())
					{
						OutMaterials.AddZeroed(Section.MaterialIndex - OutMaterials.Num() + 1);
					}
					else if (OutMaterials[Section.MaterialIndex])
					{
						continue;
					}

					UMaterialInterface* Material = StaticMesh->GetMaterial(Section.MaterialIndex);
					if (!Material)
					{
						Material = UMaterial::GetDefaultMaterial(MD_Surface);
					}
					OutMaterials[Section.MaterialIndex] = Material;
				}
			}
		}
	}

	TWeakObjectPtr<const UStaticMesh>	WeakStaticMesh;
	const class FStaticMeshRenderData*	RenderData = nullptr;
	int32								MeshMinLOD = 0;
	int32								MinLOD = 0;
	FBox								LocalBounds = FBox(ForceInitToZero);
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraMeshMaterialOverride::FNiagaraMeshMaterialOverride()
	: ExplicitMat(nullptr)
	, UserParamBinding(FNiagaraTypeDefinition(UMaterialInterface::StaticClass()))
{
}

bool FNiagaraMeshMaterialOverride::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	// We have to handle the fact that UNiagaraMeshRendererProperties OverrideMaterials just used to be an array of UMaterialInterfaces
	if (Tag.Type == NAME_ObjectProperty)
	{
		Slot << ExplicitMat;
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraMeshRendererMeshProperties::FNiagaraMeshRendererMeshProperties()
{
	LODRange = FIntVector2(0, MAX_STATIC_MESH_LODS);
#if WITH_EDITORONLY_DATA
	UserParamBinding_DEPRECATED = FNiagaraUserParameterBinding(FNiagaraTypeDefinition(UStaticMesh::StaticClass()));

	MeshParameterBinding.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
	MeshParameterBinding.SetAllowedInterfaces({UNiagaraRenderableMeshInterface::StaticClass()});
	MeshParameterBinding.SetAllowedObjects({UStaticMesh::StaticClass()});

	LODLevelBinding.SetUsage(ENiagaraParameterBindingUsage::System | ENiagaraParameterBindingUsage::Emitter | ENiagaraParameterBindingUsage::StaticVariable);
	LODLevelBinding.SetAllowedTypeDefinitions({ FNiagaraTypeDefinition::GetIntDef().ToStaticDef() });
	LODLevelBinding.SetDefaultParameter(FNiagaraTypeDefinition::GetIntDef().ToStaticDef(), 0);

	LODBiasBinding.SetUsage(ENiagaraParameterBindingUsage::System | ENiagaraParameterBindingUsage::Emitter | ENiagaraParameterBindingUsage::StaticVariable);
	LODBiasBinding.SetAllowedTypeDefinitions({ FNiagaraTypeDefinition::GetIntDef().ToStaticDef() });
	LODBiasBinding.SetDefaultParameter(FNiagaraTypeDefinition::GetIntDef().ToStaticDef(), 0);
#endif
}

bool FNiagaraMeshRendererMeshProperties::HasValidRenderableMesh() const
{
	return Mesh || MeshParameterBinding.ResolvedParameter.IsValid();
}

FNiagaraRenderableMeshPtr FNiagaraMeshRendererMeshProperties::ResolveRenderableMesh(const FNiagaraEmitterInstance* EmitterInstance) const
{
	INiagaraRenderableMeshInterface* RenderableMeshInterface = nullptr;
	UStaticMesh* StaticMesh = nullptr;
	NiagaraMeshRendererPropertiesInternal::ResolveRenderableMeshInternal(*this, EmitterInstance, RenderableMeshInterface, StaticMesh);

	if (RenderableMeshInterface)
	{
		if ( const FNiagaraSystemInstance* SystemInstance = EmitterInstance->GetParentSystemInstance() )
		{
			return RenderableMeshInterface->GetRenderableMesh(SystemInstance->GetId());
		}
	}

	if (StaticMesh)
	{
		return MakeShared<FNiagaraRenderableStaticMesh>(StaticMesh);
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UNiagaraMeshRendererProperties::UNiagaraMeshRendererProperties()
	: bOverrideMaterials(false)
	, bSortOnlyWhenTranslucent(true)
	, bSubImageBlend(true)
	, bLockedAxisEnable(false)
{
#if WITH_EDITORONLY_DATA
	FlipbookSuffixFormat = TEXT("_{frame_number}");
	FlipbookSuffixNumDigits = 1;
	NumFlipbookFrames = 1;

	MeshesBinding.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
	MeshesBinding.SetAllowedInterfaces({ UNiagaraRenderableMeshArrayInterface::StaticClass() });
#endif

	AttributeBindings =
	{
		&PositionBinding,
		&VelocityBinding,
		&ColorBinding,
		&ScaleBinding,
		&MeshOrientationBinding,
		&MaterialRandomBinding,
		&NormalizedAgeBinding,
		&CustomSortingBinding,
		&SubImageIndexBinding,
		&DynamicMaterialBinding,
		&DynamicMaterial1Binding,
		&DynamicMaterial2Binding,
		&DynamicMaterial3Binding,
		&CameraOffsetBinding,

		// These are associated with attributes in the VF layout only if bGenerateAccurateMotionVectors is true
		&PrevPositionBinding,
		&PrevScaleBinding,
		&PrevMeshOrientationBinding,
		&PrevCameraOffsetBinding,
		&PrevVelocityBinding,

		// The remaining bindings are not associated with attributes in the VF layout
		&RendererVisibilityTagBinding,
		&MeshIndexBinding,
	};
}

FNiagaraRenderer* UNiagaraMeshRendererProperties::CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
	const bool bMeshesBindingValid = MeshesBinding.ResolvedParameter.IsValid();
	bool bShouldCreateRenderer = bMeshesBindingValid;
	if (!bMeshesBindingValid)
	{
		for (const auto& MeshProperties : Meshes)
		{
			if (MeshProperties.HasValidRenderableMesh())
			{
				bShouldCreateRenderer = true;
				break;
			}
		}
	}

	if (bShouldCreateRenderer)
	{
		FNiagaraRendererMeshes* NewRenderer = new FNiagaraRendererMeshes(FeatureLevel, this, Emitter);
		NewRenderer->Initialize(this, Emitter, InController);
		if (bMeshesBindingValid || NewRenderer->HasValidMeshes())
		{
			return NewRenderer;
		}

		// There are cases where we might end up with no meshes to render due to LODs or features not being enabled on that platform
		// so we discard the renderer here, the cost to do this work in HasValidRenderableMesh makes it not worthwhile
		delete NewRenderer;
	}

	return nullptr;
}

FNiagaraBoundsCalculator* UNiagaraMeshRendererProperties::CreateBoundsCalculator()
{
	if (GetCurrentSourceMode() == ENiagaraRendererSourceDataMode::Emitter)
	{
		return nullptr;
	}

	FBox LocalBounds;
	LocalBounds.Init();

	FVector MaxLocalMeshOffset(ForceInitToZero);
	FVector MaxWorldMeshOffset(ForceInitToZero);

	bool bLocalSpace = false;
	if (FVersionedNiagaraEmitterData* EmitterData = GetEmitterData())
	{
		bLocalSpace = EmitterData->bLocalSpace;
	}

	for (const auto& MeshProperties : Meshes)
	{
		if (MeshProperties.Mesh)
		{
			FBox MeshBounds = MeshProperties.Mesh->GetBounds().GetBox();
			MeshBounds.Min *= MeshProperties.Scale;
			MeshBounds.Max *= MeshProperties.Scale;

			switch (MeshProperties.PivotOffsetSpace)
			{
			case ENiagaraMeshPivotOffsetSpace::Mesh:
				// Offset the local bounds
				MeshBounds = MeshBounds.ShiftBy(MeshProperties.PivotOffset);
				break;

			case ENiagaraMeshPivotOffsetSpace::World:
				MaxWorldMeshOffset = MaxWorldMeshOffset.ComponentMax(MeshProperties.PivotOffset.GetAbs());
				break;

			case ENiagaraMeshPivotOffsetSpace::Local:
				MaxLocalMeshOffset = MaxLocalMeshOffset.ComponentMax(MeshProperties.PivotOffset.GetAbs());
				break;

			case ENiagaraMeshPivotOffsetSpace::Simulation:
				{
					FVector& Offset = bLocalSpace ? MaxLocalMeshOffset : MaxWorldMeshOffset;
					Offset = Offset.ComponentMax(MeshProperties.PivotOffset.GetAbs());
				}
				break;
			}

			LocalBounds += MeshBounds;
		}
	}

	//-TODO: This is incorrect when using a binding as we don't know the meshes until later
	if (!LocalBounds.IsValid && MeshesBinding.ResolvedParameter.IsValid())
	{
		LocalBounds = FBox(FVector::OneVector, FVector::OneVector);
	}

	if (LocalBounds.IsValid)
	{
		// Take the bounding center into account with the extents, as it may not be at the origin
		const FVector Extents = LocalBounds.Max.GetAbs().ComponentMax(LocalBounds.Min.GetAbs());
		FNiagaraBoundsCalculatorHelper<false, true, false>* BoundsCalculator
			= new FNiagaraBoundsCalculatorHelper<false, true, false>(Extents, MaxLocalMeshOffset, MaxWorldMeshOffset, bLocalSpace);
		return BoundsCalculator;
	}

	return nullptr;

}

void UNiagaraMeshRendererProperties::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		// We can end up hitting PostInitProperties before the Niagara Module has initialized bindings this needs, mark this object for deferred init and early out.
		if (FModuleManager::Get().IsModuleLoaded("Niagara") == false)
		{
			MeshRendererPropertiesToDeferredInit.Add(this);
			return;
		}
		InitBindings();
	}
}

void UNiagaraMeshRendererProperties::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);
	const int32 NiagaraVersion = Ar.CustomVer(FNiagaraCustomVersion::GUID);

	if (Ar.IsLoading())
	{
		if (NiagaraVersion < FNiagaraCustomVersion::DisableSortingByDefault)
		{
			SortMode = ENiagaraSortMode::ViewDistance;
		}
		if (NiagaraVersion < FNiagaraCustomVersion::SubImageBlendEnabledByDefault)
		{
			bSubImageBlend = false;
		}
		if (NiagaraVersion < FNiagaraCustomVersion::CustomSortingBindingToAge)
		{
			CustomSortingBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_NORMALIZED_AGE);
		}
	}

	Super::Serialize(Ar);
}

void UNiagaraMeshRendererProperties::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(RendererLayoutWithCustomSorting.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(RendererLayoutWithoutCustomSorting.GetAllocatedSize());
}

/** The bindings depend on variables that are created during the NiagaraModule startup. However, the CDO's are build prior to this being initialized, so we defer setting these values until later.*/
void UNiagaraMeshRendererProperties::InitCDOPropertiesAfterModuleStartup()
{
	UNiagaraMeshRendererProperties* CDO = CastChecked<UNiagaraMeshRendererProperties>(UNiagaraMeshRendererProperties::StaticClass()->GetDefaultObject());
	CDO->InitRendererBaseBindings();
	CDO->InitBindings();

	for (TWeakObjectPtr<UNiagaraMeshRendererProperties>& WeakMeshRendererProperties : MeshRendererPropertiesToDeferredInit)
	{
		if (UNiagaraMeshRendererProperties* RendererProperties = WeakMeshRendererProperties.Get())
		{
			RendererProperties->InitRendererBaseBindings();
			RendererProperties->InitBindings();
		}
	}
}

void UNiagaraMeshRendererProperties::InitBindings()
{
	if (!PositionBinding.IsValid())
	{
		PositionBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
		ColorBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_COLOR);
		VelocityBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VELOCITY);
		SubImageIndexBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX);
		DynamicMaterialBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM);
		DynamicMaterial1Binding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1);
		DynamicMaterial2Binding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2);
		DynamicMaterial3Binding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3);
		MeshOrientationBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_MESH_ORIENTATION);
		ScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SCALE);
		MaterialRandomBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_MATERIAL_RANDOM);
		NormalizedAgeBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_NORMALIZED_AGE);
		CameraOffsetBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_CAMERA_OFFSET);
		RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
		MeshIndexBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_MESH_INDEX);

		//Default custom sorting to age
		CustomSortingBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_AGE);

		// Initialize the array with a single, defaulted entry
		Meshes.AddDefaulted();
	}

	SetPreviousBindings(FVersionedNiagaraEmitterBase(), SourceMode);
}

void UNiagaraMeshRendererProperties::SetPreviousBindings(const FVersionedNiagaraEmitterBase& SrcEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	PrevPositionBinding.SetAsPreviousValue(PositionBinding, SrcEmitter, InSourceMode);
	PrevScaleBinding.SetAsPreviousValue(ScaleBinding, SrcEmitter, InSourceMode);
	PrevMeshOrientationBinding.SetAsPreviousValue(MeshOrientationBinding, SrcEmitter, InSourceMode);
	PrevCameraOffsetBinding.SetAsPreviousValue(CameraOffsetBinding, SrcEmitter, InSourceMode);
	PrevVelocityBinding.SetAsPreviousValue(VelocityBinding, SrcEmitter, InSourceMode);
}

void UNiagaraMeshRendererProperties::UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit)
{
	Super::UpdateSourceModeDerivates(InSourceMode, bFromPropertyEdit);

	FVersionedNiagaraEmitterBase SrcEmitter = GetOuterEmitterBase();
	if (SrcEmitter.Emitter)
	{
	#if WITH_EDITORONLY_DATA
		MeshesBinding.OnRenameEmitter(SrcEmitter.Emitter->GetUniqueEmitterName());
	#endif

		for (FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
		{
			MaterialParamBinding.CacheValues(SrcEmitter.Emitter);
		}

		SetPreviousBindings(SrcEmitter, InSourceMode);

#if WITH_EDITORONLY_DATA
		for (FNiagaraMeshRendererMeshProperties& Mesh : Meshes)
		{
			Mesh.MeshParameterBinding.OnRenameEmitter(SrcEmitter.Emitter->GetUniqueEmitterName());
			Mesh.LODLevelBinding.OnRenameEmitter(SrcEmitter.Emitter->GetUniqueEmitterName());
			Mesh.LODBiasBinding.OnRenameEmitter(SrcEmitter.Emitter->GetUniqueEmitterName());
		}
#endif
	}
}

void UNiagaraMeshRendererProperties::CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData)
{
	UpdateSourceModeDerivates(SourceMode);
	UpdateMICs();

	// Initialize layout
	const int32 NumLayoutVars = NeedsPreciseMotionVectors() ? ENiagaraMeshVFLayout::Num_Max : ENiagaraMeshVFLayout::Num_Default;
	RendererLayoutWithCustomSorting.Initialize(NumLayoutVars);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, PositionBinding, ENiagaraMeshVFLayout::Position);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, VelocityBinding, ENiagaraMeshVFLayout::Velocity);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, ColorBinding, ENiagaraMeshVFLayout::Color);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, ScaleBinding, ENiagaraMeshVFLayout::Scale);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, MeshOrientationBinding, ENiagaraMeshVFLayout::Rotation);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, MaterialRandomBinding, ENiagaraMeshVFLayout::MaterialRandom);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, NormalizedAgeBinding, ENiagaraMeshVFLayout::NormalizedAge);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, CustomSortingBinding, ENiagaraMeshVFLayout::CustomSorting);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, SubImageIndexBinding, ENiagaraMeshVFLayout::SubImage);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, CameraOffsetBinding, ENiagaraMeshVFLayout::CameraOffset);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, DynamicMaterialBinding,  ENiagaraMeshVFLayout::DynamicParam0);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, DynamicMaterial1Binding, ENiagaraMeshVFLayout::DynamicParam1);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, DynamicMaterial2Binding, ENiagaraMeshVFLayout::DynamicParam2);
	RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, DynamicMaterial3Binding, ENiagaraMeshVFLayout::DynamicParam3);
	if (NeedsPreciseMotionVectors())
	{
		RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, PrevPositionBinding, ENiagaraMeshVFLayout::PrevPosition);
		RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, PrevScaleBinding, ENiagaraMeshVFLayout::PrevScale);
		RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, PrevMeshOrientationBinding, ENiagaraMeshVFLayout::PrevRotation);
		RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, PrevCameraOffsetBinding, ENiagaraMeshVFLayout::PrevCameraOffset);
		RendererLayoutWithCustomSorting.SetVariableFromBinding(CompiledData, PrevVelocityBinding, ENiagaraMeshVFLayout::PrevVelocity);
	}
	RendererLayoutWithCustomSorting.Finalize();

	RendererLayoutWithoutCustomSorting.Initialize(NumLayoutVars);
	RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, PositionBinding, ENiagaraMeshVFLayout::Position);
	RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, VelocityBinding, ENiagaraMeshVFLayout::Velocity);
	RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, ColorBinding, ENiagaraMeshVFLayout::Color);
	RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, ScaleBinding, ENiagaraMeshVFLayout::Scale);
	RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, MeshOrientationBinding, ENiagaraMeshVFLayout::Rotation);
	RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, MaterialRandomBinding, ENiagaraMeshVFLayout::MaterialRandom);
	RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, NormalizedAgeBinding, ENiagaraMeshVFLayout::NormalizedAge);
	RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, SubImageIndexBinding, ENiagaraMeshVFLayout::SubImage);
	RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, CameraOffsetBinding, ENiagaraMeshVFLayout::CameraOffset);
	const bool bDynamicParam0Valid = RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, DynamicMaterialBinding,  ENiagaraMeshVFLayout::DynamicParam0);
	const bool bDynamicParam1Valid = RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, DynamicMaterial1Binding, ENiagaraMeshVFLayout::DynamicParam1);
	const bool bDynamicParam2Valid = RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, DynamicMaterial2Binding, ENiagaraMeshVFLayout::DynamicParam2);
	const bool bDynamicParam3Valid = RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, DynamicMaterial3Binding, ENiagaraMeshVFLayout::DynamicParam3);
	if (NeedsPreciseMotionVectors())
	{
		RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, PrevPositionBinding, ENiagaraMeshVFLayout::PrevPosition);
		RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, PrevScaleBinding, ENiagaraMeshVFLayout::PrevScale);
		RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, PrevMeshOrientationBinding, ENiagaraMeshVFLayout::PrevRotation);
		RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, PrevCameraOffsetBinding, ENiagaraMeshVFLayout::PrevCameraOffset);
		RendererLayoutWithoutCustomSorting.SetVariableFromBinding(CompiledData, PrevVelocityBinding, ENiagaraMeshVFLayout::PrevVelocity);
	}
	RendererLayoutWithoutCustomSorting.Finalize();

#if WITH_EDITORONLY_DATA
	// Build dynamic parameter mask
	// Serialize in cooked builds
	MaterialParamValidMask = GetDynamicParameterCombinedChannelMask(
		bDynamicParam0Valid ? DynamicMaterialBinding.GetName() : NAME_None,
		bDynamicParam1Valid ? DynamicMaterial1Binding.GetName() : NAME_None,
		bDynamicParam2Valid ? DynamicMaterial2Binding.GetName() : NAME_None,
		bDynamicParam3Valid ? DynamicMaterial3Binding.GetName() : NAME_None
	);

	// Gather LOD information per mesh
	UNiagaraSystem* OwnerSystem = GetTypedOuter<UNiagaraSystem>();
	for (FNiagaraMeshRendererMeshProperties& Mesh : Meshes)
	{
		Mesh.LODLevel = Mesh.LODLevelBinding.GetDefaultValue<int32>();
		Mesh.LODBias = Mesh.LODBiasBinding.GetDefaultValue<int32>();

		if (OwnerSystem && (Mesh.LODLevelBinding.AliasedParameter.IsValid() || Mesh.LODBiasBinding.AliasedParameter.IsValid()))
		{
			OwnerSystem->ForEachScript(
				[&Mesh](const UNiagaraScript* NiagaraScript)
				{
					if (Mesh.LODLevelBinding.AliasedParameter.IsValid())
					{
						TOptional<int32> VariableValue = NiagaraScript->GetCompiledStaticVariableValue<int32>(Mesh.LODLevelBinding.ResolvedParameter);
						if (VariableValue.IsSet())
						{
							Mesh.LODLevel = VariableValue.GetValue();
						}
					}
					if (Mesh.LODBiasBinding.AliasedParameter.IsValid())
					{
						TOptional<int32> VariableValue = NiagaraScript->GetCompiledStaticVariableValue<int32>(Mesh.LODBiasBinding.ResolvedParameter);
						if (VariableValue.IsSet())
						{
							Mesh.LODBias = VariableValue.GetValue();
						}
					}
				}
			);
		}
	}
#endif
}

#if WITH_EDITOR
void UNiagaraMeshRendererProperties::OnMeshChanged()
{
	FNiagaraSystemUpdateContext ReregisterContext;

	if (UNiagaraSystem* NiagaraSystem = GetTypedOuter<UNiagaraSystem>())
	{
		ReregisterContext.Add(NiagaraSystem, true);
	}

	CheckMaterialUsage();
	UpdateMICs();
}

void UNiagaraMeshRendererProperties::OnMeshChanged(UObject* MeshObject)
{
	// We need to guard recursive calls that can come from PSO caching requesting the mesh which can call the post build step
	// This is all very messy as we do not get a consistent callback from any one of the methods for various operations so we have to litter this across various calls
	static bool GRecursionGuard = false;
	if (GRecursionGuard)
	{
		return;
	}

	TGuardValue<bool> RecursionGuardValue(GRecursionGuard, true);

	FNiagaraSystemUpdateContext ReregisterContext;

	TSet<FObjectKey> AddedSystems;
	for (TObjectIterator<UNiagaraMeshRendererProperties> It; It; ++It)
	{
		UNiagaraMeshRendererProperties* Properties = *It;
		UNiagaraSystem* System = ::IsValid(Properties) ? Properties->GetTypedOuter<UNiagaraSystem>() : nullptr;
		if (::IsValid(System) == false)
		{
			continue;
		}

		// Work out if we need add this system, and if we need to update the MICs / material usage
		bool bHasExternalBinding = Properties->MeshesBinding.ResolvedParameter.IsValid();
		bool bHasReference = false;
		for (const FNiagaraMeshRendererMeshProperties& Mesh : Properties->Meshes)
		{
			bHasReference |= Mesh.Mesh == MeshObject;
			bHasExternalBinding |= Mesh.MeshParameterBinding.ResolvedParameter.IsValid();
		}
		if (!bHasReference && !bHasExternalBinding)
		{
			continue;
		}

		if (!AddedSystems.Contains(System))
		{
			AddedSystems.Add(System);
			ReregisterContext.Add(System, true);
		}

		if (bHasReference)
		{
			Properties->CheckMaterialUsage();
			Properties->UpdateMICs();
		}
	}
}

void UNiagaraMeshRendererProperties::RemoveMeshChangedHandlers()
{
	NiagaraMeshRendererPropertiesInternal::RemoveMeshChangedHandlers();
}
#endif //WITH_EDITOR

void UNiagaraMeshRendererProperties::UpdateMICs()
{
#if WITH_EDITORONLY_DATA
	// Grab existing MICs so we can reuse and clear them out so they aren't applied during GetUsedMaterials
	TArray<TObjectPtr<UMaterialInstanceConstant>> MICMaterials;
	MICMaterials.Reserve(MICOverrideMaterials.Num());
	for (const FNiagaraMeshMICOverride& ExistingOverride : MICOverrideMaterials)
	{
		MICMaterials.Add(ExistingOverride.ReplacementMaterial);
	}
	MICOverrideMaterials.Reset(0);

	// Gather materials and generate MICs
	TArray<UMaterialInterface*> Materials;
	GetUsedMaterials(nullptr, Materials);

	UpdateMaterialParametersMIC(MaterialParameters, Materials, MICMaterials);

	// Create Material <-> MIC remap
	for (int i=0; i < MICMaterials.Num(); ++i)
	{
		const FNiagaraMeshMICOverride* ExistingOverride = MICOverrideMaterials.FindByPredicate([FindMaterial = Materials[i]](const FNiagaraMeshMICOverride& ExistingOverride) { return ExistingOverride.OriginalMaterial == FindMaterial; });
		if (ExistingOverride)
		{
			ensureMsgf(ExistingOverride->ReplacementMaterial == MICMaterials[i], TEXT("MIC Material should match replacement material, static bindings will be incorrect.  Please report this issue."));
		}
		else
		{
			FNiagaraMeshMICOverride& NewOverride = MICOverrideMaterials.AddDefaulted_GetRef();
			NewOverride.OriginalMaterial = Materials[i];
			NewOverride.ReplacementMaterial = MICMaterials[i];
		}
	}
#endif
}

void UNiagaraMeshRendererProperties::ApplyMaterialOverrides(const FNiagaraEmitterInstance* EmitterInstance, TArray<UMaterialInterface*>& InOutMaterials) const
{
	if (bOverrideMaterials)
	{
		const int32 NumOverrideMaterials = FMath::Min(OverrideMaterials.Num(), InOutMaterials.Num());
		for (int32 OverrideIndex = 0; OverrideIndex < NumOverrideMaterials; ++OverrideIndex)
		{
			if (!InOutMaterials[OverrideIndex])
			{
				continue;
			}

			UMaterialInterface* OverrideMat = nullptr;

			// UserParamBinding, if mapped to a real value, always wins. Otherwise, use the ExplictMat if it is set. Finally, fall
			// back to the particle mesh material. This allows the user to effectively optionally bind to a Material binding
			// and still have good defaults if it isn't set to anything.
			if (EmitterInstance && OverrideMaterials[OverrideIndex].UserParamBinding.Parameter.IsValid())
			{
				OverrideMat = Cast<UMaterialInterface>(EmitterInstance->FindBinding(OverrideMaterials[OverrideIndex].UserParamBinding.Parameter));
			}

			if (!OverrideMat)
			{
				OverrideMat = OverrideMaterials[OverrideIndex].ExplicitMat;
			}

			if (OverrideMat)
			{
				InOutMaterials[OverrideIndex] = OverrideMat;
			}
		}
	}

	// Apply MIC override materials
	if (MICOverrideMaterials.Num() > 0)
	{
		for (UMaterialInterface*& Material : InOutMaterials)
		{
			if (const FNiagaraMeshMICOverride* Override = MICOverrideMaterials.FindByPredicate([&Material](const FNiagaraMeshMICOverride& MICOverride) { return MICOverride.OriginalMaterial == Material; }))
			{
				Material = Override->ReplacementMaterial;
			}
		}
	}
}

void UNiagaraMeshRendererProperties::ForEachMesh(FNiagaraSystemInstance* SystemInstance, TFunction<void(int32)> NumMeshesDelegate, TFunction<void(const FNiagaraMeshRendererMeshProperties&)> IterateDelegate) const
{
	NumMeshesDelegate(Meshes.Num());
	for (const FNiagaraMeshRendererMeshProperties& MeshProperties : Meshes)
	{
		IterateDelegate(MeshProperties);
	}
}

const INiagaraRenderableMeshArrayInterface* UNiagaraMeshRendererProperties::GetMeshArrayInterface(const FNiagaraEmitterInstance* EmitterInstance) const
{
	INiagaraRenderableMeshArrayInterface* ResolvedInterface = nullptr;
	if (EmitterInstance)
	{
		ResolvedInterface = Cast<INiagaraRenderableMeshArrayInterface>(EmitterInstance->GetRendererBoundVariables().GetDataInterface(MeshesBinding.ResolvedParameter));
	}
	return ResolvedInterface ? ResolvedInterface : this;
}

const FVertexFactoryType* UNiagaraMeshRendererProperties::GetVertexFactoryType() const
{
	return &FNiagaraMeshVertexFactory::StaticType;
}

void UNiagaraMeshRendererProperties::GetUsedMaterials(const FNiagaraEmitterInstance* EmitterInstance, TArray<UMaterialInterface*>& OutMaterials) const
{
	FNiagaraSystemInstance* SystemInstance = EmitterInstance ? EmitterInstance->GetParentSystemInstance() : nullptr;

	TArray<UMaterialInterface*> OrderedMeshMaterials;

	const INiagaraRenderableMeshArrayInterface* MeshArrayInterface = GetMeshArrayInterface(EmitterInstance);
	MeshArrayInterface->ForEachMesh(
		SystemInstance,
		[&](const FNiagaraMeshRendererMeshProperties& MeshProperties)
		{
			OrderedMeshMaterials.Reset(0);

			INiagaraRenderableMeshInterface* RenderableMeshInterface = nullptr;
			UStaticMesh* StaticMesh = nullptr;
			NiagaraMeshRendererPropertiesInternal::ResolveRenderableMeshInternal(MeshProperties, EmitterInstance, RenderableMeshInterface, StaticMesh);

			if (RenderableMeshInterface && SystemInstance)
			{
				RenderableMeshInterface->GetUsedMaterials(SystemInstance->GetId(), OrderedMeshMaterials);
			}
			else if (StaticMesh)
			{
				FNiagaraRenderableStaticMesh(StaticMesh).GetUsedMaterials(OrderedMeshMaterials);
			}

			if (OrderedMeshMaterials.Num() > 0)
			{
				ApplyMaterialOverrides(EmitterInstance, OrderedMeshMaterials);

				OutMaterials.Reserve(OutMaterials.Num() + OrderedMeshMaterials.Num());
				for (TObjectPtr<UMaterialInterface> MaterialInterface : OrderedMeshMaterials)
				{
					if (MaterialInterface)
					{
						OutMaterials.AddUnique(MaterialInterface);
					}
				}
			}
		}
	);
}

void UNiagaraMeshRendererProperties::GetUsedMeshes(const FNiagaraEmitterInstance* InEmitter, TArray<UObject*>& OutMeshes) const
{
	for (const FNiagaraMeshRendererMeshProperties& Property : Meshes)
	{
		INiagaraRenderableMeshInterface* RenderableMeshInterface = nullptr;
		UStaticMesh* StaticMesh = nullptr;
		NiagaraMeshRendererPropertiesInternal::ResolveRenderableMeshInternal(Property, InEmitter, RenderableMeshInterface, StaticMesh);
		if (StaticMesh)
		{
			OutMeshes.AddUnique(StaticMesh);
		}
	}
}

void UNiagaraMeshRendererProperties::CollectPSOPrecacheData(const FNiagaraEmitterInstance* InEmitter, FPSOPrecacheParamsList& OutParams) const
{
	const FVertexFactoryType* VFType = GetVertexFactoryType();
	bool bSupportsManualVertexFetch = VFType->SupportsManualVertexFetch(GMaxRHIFeatureLevel);

	const INiagaraRenderableMeshArrayInterface* MeshArrayInterface = GetMeshArrayInterface(InEmitter);
	MeshArrayInterface->ForEachMesh(
		InEmitter ? InEmitter->GetParentSystemInstance() : nullptr,
		[&](const FNiagaraMeshRendererMeshProperties& MeshProperties)
		{
			INiagaraRenderableMeshInterface* RenderableMeshInterface = nullptr;
			UStaticMesh* StaticMesh = nullptr;
			NiagaraMeshRendererPropertiesInternal::ResolveRenderableMeshInternal(MeshProperties, InEmitter, RenderableMeshInterface, StaticMesh);
			if (StaticMesh)
			{
				TArray<UMaterialInterface*> OrderedMeshMaterials;
				FNiagaraRenderableStaticMesh(StaticMesh).GetUsedMaterials(OrderedMeshMaterials);
				if (OrderedMeshMaterials.Num())
				{
					ApplyMaterialOverrides(nullptr, OrderedMeshMaterials);
					for (UMaterialInterface* MeshMaterial : OrderedMeshMaterials)
					{
						if (MeshMaterial)
						{
							FPSOPrecacheParams& PSOPrecacheParams = OutParams.AddDefaulted_GetRef();
							PSOPrecacheParams.MaterialInterface = MeshMaterial;
							if (!bSupportsManualVertexFetch)
							{
								// Assuming here that all LOD use same vertex decl
								int32 MeshLODIdx = StaticMesh->GetMinLODIdx();
								if (StaticMesh->GetRenderData()->LODResources.IsValidIndex(MeshLODIdx))
								{
									FStaticMeshDataType Data;
									FVertexDeclarationElementList Elements;
									FNiagaraRenderableStaticMesh::InitVertexFactoryComponents(StaticMesh->GetRenderData()->LODResources[MeshLODIdx].VertexBuffers, nullptr, Data);
									FNiagaraMeshVertexFactory::GetVertexElements(GMaxRHIFeatureLevel, bSupportsManualVertexFetch, Data, Elements);
									PSOPrecacheParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(VFType, Elements));
								}
							}
							else
							{
								PSOPrecacheParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(VFType));
							}
						}
					}
				}
			}
		}
	);
}

void UNiagaraMeshRendererProperties::GetStreamingMeshInfo(const FBoxSphereBounds& OwnerBounds, const FNiagaraEmitterInstance* InEmitter, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	for (const FNiagaraMeshRendererMeshProperties& MeshProperties : Meshes)
	{
		INiagaraRenderableMeshInterface* RenderableMeshInterface = nullptr;
		UStaticMesh* StaticMesh = nullptr;
		NiagaraMeshRendererPropertiesInternal::ResolveRenderableMeshInternal(MeshProperties, InEmitter, RenderableMeshInterface, StaticMesh);

		if (StaticMesh && StaticMesh->RenderResourceSupportsStreaming() && StaticMesh->GetRenderAssetType() == EStreamableRenderAssetType::StaticMesh)
		{
			const FBoxSphereBounds MeshBounds = StaticMesh->GetBounds();
			const FBoxSphereBounds StreamingBounds = FBoxSphereBounds(
				OwnerBounds.Origin + MeshBounds.Origin,
				MeshBounds.BoxExtent * MeshProperties.Scale,
				MeshBounds.SphereRadius * MeshProperties.Scale.GetMax());
			const float MeshTexelFactor = float(MeshBounds.SphereRadius * 2.0);

			new (OutStreamingRenderAssets) FStreamingRenderAssetPrimitiveInfo(StaticMesh, StreamingBounds, MeshTexelFactor);
		}
	}
}

#if WITH_EDITORONLY_DATA
TArray<FNiagaraVariable> UNiagaraMeshRendererProperties::GetBoundAttributes() const 
{
	TArray<FNiagaraVariable> BoundAttributes = Super::GetBoundAttributes();
	BoundAttributes.Reserve(BoundAttributes.Num() + MaterialParameters.AttributeBindings.Num());

	for (const FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		BoundAttributes.AddUnique(MaterialParamBinding.GetParamMapBindableVariable());
	}

	if (MeshesBinding.IsSetToDefault() == false)
	{
		BoundAttributes.AddUnique(MeshesBinding.AliasedParameter);
	}

	for (const FNiagaraMeshRendererMeshProperties& Mesh : Meshes)
	{
		if (Mesh.MeshParameterBinding.IsSetToDefault() == false)
		{
			BoundAttributes.AddUnique(Mesh.MeshParameterBinding.AliasedParameter);
		}
	}

	if (bOverrideMaterials)
	{
		for (const FNiagaraMeshMaterialOverride& OverrideMaterial : OverrideMaterials)
		{
			if (OverrideMaterial.UserParamBinding.Parameter.IsValid())
			{
				BoundAttributes.AddUnique(OverrideMaterial.UserParamBinding.Parameter);
			}
		}
	}

	return BoundAttributes;
}
#endif

bool UNiagaraMeshRendererProperties::PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore)
{
	bool bAnyAdded = Super::PopulateRequiredBindings(InParameterStore);

	if (MeshesBinding.ResolvedParameter.IsValid())
	{
		InParameterStore.AddParameter(MeshesBinding.ResolvedParameter, false);
		bAnyAdded = true;
	}

	for (const FNiagaraVariableAttributeBinding* Binding : AttributeBindings)
	{
		if (Binding && Binding->CanBindToHostParameterMap())
		{
			InParameterStore.AddParameter(Binding->GetParamMapBindableVariable(), false);
			bAnyAdded = true;
		}
	}

	for (FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		InParameterStore.AddParameter(MaterialParamBinding.GetParamMapBindableVariable(), false);
		bAnyAdded = true;
	}

	for (FNiagaraMeshRendererMeshProperties& Binding : Meshes)
	{
		if (Binding.MeshParameterBinding.ResolvedParameter.IsValid())
		{
			InParameterStore.AddParameter(Binding.MeshParameterBinding.ResolvedParameter, false);
			bAnyAdded = true;
		}
	}

	return bAnyAdded;
}

void UNiagaraMeshRendererProperties::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	SourceMode = ValidateSourceMode(SourceMode);

	if (Meshes.Num() == 1 && Meshes[0].Mesh == nullptr && ParticleMesh_DEPRECATED != nullptr)
	{
		// Likely predates the mesh array ... just add ParticleMesh to the list of Meshes
		FNiagaraMeshRendererMeshProperties& Mesh = Meshes[0];
		Mesh.Mesh = ParticleMesh_DEPRECATED;
		Mesh.PivotOffset = PivotOffset_DEPRECATED;
		Mesh.PivotOffsetSpace = PivotOffsetSpace_DEPRECATED;
	}
#endif

	for (FNiagaraMeshRendererMeshProperties& MeshProperties : Meshes)
	{
		if (MeshProperties.Mesh)
		{
			MeshProperties.Mesh->ConditionalPostLoad();
		}
#if WITH_EDITORONLY_DATA
		if (MeshProperties.UserParamBinding_DEPRECATED.Parameter.GetName().IsNone() == false)
		{
			MeshProperties.MeshParameterBinding.ResolvedParameter = MeshProperties.UserParamBinding_DEPRECATED.Parameter;
			MeshProperties.MeshParameterBinding.AliasedParameter = MeshProperties.UserParamBinding_DEPRECATED.Parameter;
		}
#endif
	}

#if WITH_EDITORONLY_DATA
	ChangeToPositionBinding(PositionBinding);
	ChangeToPositionBinding(PrevPositionBinding);
#endif
	
	PostLoadBindings(SourceMode);
	
	// Fix up these bindings from their loaded source bindings
	SetPreviousBindings(FVersionedNiagaraEmitterBase(), SourceMode);

	for ( const FNiagaraMeshMaterialOverride& OverrideMaterial : OverrideMaterials )
	{
		if (OverrideMaterial.ExplicitMat )
		{
			OverrideMaterial.ExplicitMat->ConditionalPostLoad();
		}
	}

	for (const FNiagaraMeshMICOverride& MICOverrideMaterial : MICOverrideMaterials)
	{
		if (MICOverrideMaterial.OriginalMaterial)
		{
			MICOverrideMaterial.OriginalMaterial->ConditionalPostLoad();
		}
		if (MICOverrideMaterial.ReplacementMaterial)
		{
			MICOverrideMaterial.ReplacementMaterial->ConditionalPostLoad();
		}
	}

#if WITH_EDITORONLY_DATA
	if (MaterialParameterBindings_DEPRECATED.Num() > 0)
	{
		MaterialParameters.AttributeBindings = MaterialParameterBindings_DEPRECATED;
		MaterialParameterBindings_DEPRECATED.Empty();
	}
#endif
	MaterialParameters.ConditionalPostLoad();
}

#if WITH_EDITORONLY_DATA
const TArray<FNiagaraVariable>& UNiagaraMeshRendererProperties::GetOptionalAttributes()
{
	static TArray<FNiagaraVariable> Attrs;

	if (Attrs.Num() == 0)
	{
		Attrs.Add(SYS_PARAM_PARTICLES_POSITION);
		Attrs.Add(SYS_PARAM_PARTICLES_VELOCITY);
		Attrs.Add(SYS_PARAM_PARTICLES_COLOR);
		Attrs.Add(SYS_PARAM_PARTICLES_NORMALIZED_AGE);
		Attrs.Add(SYS_PARAM_PARTICLES_SCALE);
		Attrs.Add(SYS_PARAM_PARTICLES_MESH_ORIENTATION);
		Attrs.Add(SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX);
		Attrs.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM);
		Attrs.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1);
		Attrs.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2);
		Attrs.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3);
	}

	return Attrs;
}

void UNiagaraMeshRendererProperties::GetAdditionalVariables(TArray<FNiagaraVariableBase>& OutArray) const
{
	if (NeedsPreciseMotionVectors())
	{
		OutArray.Reserve(5);
		OutArray.AddUnique(PrevPositionBinding.GetParamMapBindableVariable());
		OutArray.AddUnique(PrevScaleBinding.GetParamMapBindableVariable());
		OutArray.AddUnique(PrevMeshOrientationBinding.GetParamMapBindableVariable());
		OutArray.AddUnique(PrevCameraOffsetBinding.GetParamMapBindableVariable());
		OutArray.AddUnique(PrevVelocityBinding.GetParamMapBindableVariable());		
	}
}

void UNiagaraMeshRendererProperties::GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{	
	TArray<UObject*> Assets;
	// We create a thumbnail per mesh entry even if it's invalid to highlight that there are unused mesh entries
	for(const FNiagaraMeshRendererMeshProperties& MeshProperties : Meshes)
	{
		UStaticMesh* Mesh = MeshProperties.Mesh;
		if (Mesh && Mesh->HasValidRenderData())
		{
			Assets.Add(Mesh);
		}
		else
		{
			Assets.Add(nullptr);
		}
	}
	
	CreateRendererWidgetsForAssets(Assets, InThumbnailPool, OutWidgets);
}

void UNiagaraMeshRendererProperties::GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	TSharedRef<SWidget> DefaultMeshTooltip = SNew(STextBlock)
			.Text(LOCTEXT("MeshRendererNoMat", "Mesh Renderer (No Mesh Set)"));
	
	TArray<TSharedPtr<SWidget>> RendererWidgets;
	if (Meshes.Num() > 0)
	{
		GetRendererWidgets(InEmitter, RendererWidgets, InThumbnailPool);
	}
	
	for(int32 MeshIndex = 0; MeshIndex < Meshes.Num(); MeshIndex++)
	{
		const FNiagaraMeshRendererMeshProperties& MeshProperties = Meshes[MeshIndex];
		
		TSharedPtr<SWidget> TooltipWidget = DefaultMeshTooltip;

		// we make sure to reuse the mesh widget as a thumbnail if the mesh is valid
		INiagaraRenderableMeshInterface* RenderableMeshInterface = nullptr;
		UStaticMesh* StaticMesh = nullptr;
		NiagaraMeshRendererPropertiesInternal::ResolveRenderableMeshInternal(Meshes[MeshIndex], InEmitter, RenderableMeshInterface, StaticMesh);

		if (StaticMesh)
		{
			TooltipWidget = RendererWidgets[MeshIndex];
		}

		// we override the previous thumbnail tooltip with a text indicating parameter binding, if it exists
		if(MeshProperties.MeshParameterBinding.ResolvedParameter.IsValid())
		{
			TooltipWidget = SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("MeshBoundTooltip", "Mesh slot is bound to parameter {0}"), FText::FromName(MeshProperties.MeshParameterBinding.ResolvedParameter.GetName())));
		}
		
		OutWidgets.Add(TooltipWidget);
	}

	if (Meshes.Num() == 0)
	{
		OutWidgets.Add(DefaultMeshTooltip);
	}
}

void UNiagaraMeshRendererProperties::GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const
{
	Super::GetRendererFeedback(InEmitter, OutErrors, OutWarnings, OutInfo);

	if (MaterialParameters.HasAnyBindings())
	{
		TArray<UMaterialInterface*> Materials;
		GetUsedMaterials(nullptr, Materials);
		MaterialParameters.GetFeedback(Materials, OutWarnings);
	}

	// TODO: Check if Nanite fallback mesh was generated
}

void UNiagaraMeshRendererProperties::GetRendererMaterialUsageFeedback(TArray<FNiagaraRendererFeedback>& OutFeedback) const
{
	Super::GetRendererMaterialUsageFeedback(OutFeedback);
	GetMaterialUsageFeedback(MATUSAGE_NiagaraMeshParticles, { MD_Volume }, OutFeedback);
}

void UNiagaraMeshRendererProperties::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	SubImageSize.X = FMath::Max(SubImageSize.X, 1.0);
	SubImageSize.Y = FMath::Max(SubImageSize.Y, 1.0);

	const bool bIsRedirect = PropertyChangedEvent.ChangeType == EPropertyChangeType::Redirected;
	const bool bRebuildMeshList = ChangeRequiresMeshListRebuild(PropertyChangedEvent.Property);
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (bRebuildMeshList)
	{
		if (!IsRunningCommandlet() &&
			PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraMeshRendererProperties, bEnableMeshFlipbook) &&
			bEnableMeshFlipbook &&
			Meshes.Num() > 0)
		{
			// Give the user a chance to cancel doing something that will be destructive to the current mesh data
			FSuppressableWarningDialog::FSetupInfo Info(
				LOCTEXT("ShowNiagaraMeshRendererFlipbookWarning_Message", "Enabling the Mesh Flipbook option will replace all meshes currently selected for this renderer. Continue?"),
				LOCTEXT("ShowNiagaraMeshRendererFlipbookWarning_Title", "Confirm Enable Flipbook"),
				TEXT("SuppressNiagaraMeshRendererFlipbookWarning")
			);
			Info.ConfirmText = LOCTEXT("ShowNiagaraMeshRendererFlipbookWarning_Confirm", "Yes");
			Info.CancelText = LOCTEXT("ShowNiagaraMeshRendererFlipbookWarning_Cancel", "No");
			FSuppressableWarningDialog MeshRendererFlipbookWarning(Info);

			if (MeshRendererFlipbookWarning.ShowModal() == FSuppressableWarningDialog::EResult::Cancel)
			{
				bEnableMeshFlipbook = false;
			}
			else
			{
				RebuildMeshList();
			}
		}
		else
		{
			RebuildMeshList();
		}
	}

	// If changing the source mode, we may need to update many of our values.
	if (PropertyName == TEXT("SourceMode"))
	{
		UpdateSourceModeDerivates(SourceMode, true);
	}
	else if (FStructProperty* StructProp = CastField<FStructProperty>(PropertyChangedEvent.Property))
	{
		if (StructProp->Struct == FNiagaraVariableAttributeBinding::StaticStruct())
		{
			UpdateSourceModeDerivates(SourceMode, true);
		}
	}
	else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(PropertyChangedEvent.Property))
	{
		if (ArrayProp->Inner)
		{
			FStructProperty* ChildStructProp = CastField<FStructProperty>(ArrayProp->Inner);
			if (ChildStructProp->Struct == FNiagaraMaterialAttributeBinding::StaticStruct())
			{
				UpdateSourceModeDerivates(SourceMode, true);
			}
		}
	}

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraMeshRendererProperties, Meshes))
	{
		for (FNiagaraMeshRendererMeshProperties& MeshProperties : Meshes)
		{
			if (MeshProperties.bUseLODRange)
			{
				MeshProperties.LODRange.X = FMath::Clamp(MeshProperties.LODRange.X, 0, MAX_STATIC_MESH_LODS - 1);
				MeshProperties.LODRange.Y = FMath::Clamp(MeshProperties.LODRange.Y, 1, MAX_STATIC_MESH_LODS);

				MeshProperties.LODRange.X = FMath::Clamp(MeshProperties.LODRange.X, 0, MeshProperties.LODRange.Y - 1);
				MeshProperties.LODRange.Y = FMath::Clamp(MeshProperties.LODRange.Y, MeshProperties.LODRange.X + 1, MAX_STATIC_MESH_LODS);
			}
		}
	}

	// Update our MICs if we change override material / material bindings / meshes
	//-OPT: Could narrow down further to only static materials
	if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraMeshRendererProperties, OverrideMaterials)) ||
		(MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraMeshRendererProperties, Meshes)) ||
		(MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraMeshRendererProperties, MaterialParameters)))
	{
		UpdateMICs();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UNiagaraMeshRendererProperties::RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RenameVariable(OldVariable, NewVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MeshesBinding.OnRenameVariable(OldVariable, NewVariable, InEmitter.Emitter->GetUniqueEmitterName());
	MaterialParameters.RenameVariable(OldVariable, NewVariable, InEmitter, GetCurrentSourceMode());
	for (FNiagaraMeshRendererMeshProperties& Mesh : Meshes)
	{
		Mesh.MeshParameterBinding.OnRenameVariable(OldVariable, NewVariable, InEmitter.Emitter->GetUniqueEmitterName());
		Mesh.LODLevelBinding.OnRenameVariable(OldVariable, NewVariable, InEmitter.Emitter->GetUniqueEmitterName());
		Mesh.LODBiasBinding.OnRenameVariable(OldVariable, NewVariable, InEmitter.Emitter->GetUniqueEmitterName());
	}
#endif
}

void UNiagaraMeshRendererProperties::RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RemoveVariable(OldVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MeshesBinding.OnRemoveVariable(OldVariable, InEmitter.Emitter->GetUniqueEmitterName());
	MaterialParameters.RemoveVariable(OldVariable, InEmitter, GetCurrentSourceMode());
	for (FNiagaraMeshRendererMeshProperties& Mesh : Meshes)
	{
		Mesh.MeshParameterBinding.OnRemoveVariable(OldVariable, InEmitter.Emitter->GetUniqueEmitterName());
		Mesh.LODLevelBinding.OnRemoveVariable(OldVariable, InEmitter.Emitter->GetUniqueEmitterName());
		Mesh.LODBiasBinding.OnRemoveVariable(OldVariable, InEmitter.Emitter->GetUniqueEmitterName());
	}
#endif
}

void UNiagaraMeshRendererProperties::CheckMaterialUsage()
{
	for (const auto& MeshProperties : Meshes)
	{
		const FStaticMeshRenderData* MeshRenderData = MeshProperties.Mesh ? MeshProperties.Mesh->GetRenderData() : nullptr;

		constexpr int32 LodIndexForCheck = 0;
		if (MeshRenderData && MeshRenderData->LODResources.IsValidIndex(LodIndexForCheck))
		{
			const FStaticMeshLODResources& LODModel = MeshRenderData->LODResources[LodIndexForCheck];
			for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
			{
				const FStaticMeshSection& Section = LODModel.Sections[SectionIndex];
				UMaterialInterface *MaterialInterface = MeshProperties.Mesh->GetMaterial(Section.MaterialIndex);
				if (MaterialInterface)
				{
					MaterialInterface->CheckMaterialUsage(MATUSAGE_NiagaraMeshParticles);
				}
			}
		}
	}
}

bool UNiagaraMeshRendererProperties::ChangeRequiresMeshListRebuild(const FProperty* Property)
{
	if (Property == nullptr)
	{
		return false;
	}

	// If any of these are changed, we have to rebuild the mesh list
	static const TArray<FName, TInlineAllocator<6>> RebuildMeshPropertyNames
	{
		GET_MEMBER_NAME_CHECKED(UNiagaraMeshRendererProperties, bEnableMeshFlipbook),
		GET_MEMBER_NAME_CHECKED(UNiagaraMeshRendererProperties, FirstFlipbookFrame),
		GET_MEMBER_NAME_CHECKED(UNiagaraMeshRendererProperties, FlipbookSuffixFormat),
		GET_MEMBER_NAME_CHECKED(UNiagaraMeshRendererProperties, FlipbookSuffixNumDigits),
		GET_MEMBER_NAME_CHECKED(UNiagaraMeshRendererProperties, NumFlipbookFrames),
		GET_MEMBER_NAME_CHECKED(FNiagaraMeshRendererMeshProperties, Mesh),
	};
	return RebuildMeshPropertyNames.Contains(Property->GetFName());
}

void UNiagaraMeshRendererProperties::RebuildMeshList()
{
	if (!bEnableMeshFlipbook)
	{
		// Mesh flipbook has been disabled, so let's just leave the mesh list as it was
		return;
	}

	Meshes.Empty();

	if (!FirstFlipbookFrame)
	{
		// No first page mesh selected
		return;
	}

	Meshes.AddDefaulted_GetRef().Mesh = FirstFlipbookFrame;

	if (NumFlipbookFrames <= 1)
	{
		// No need to build a flipbook list, just add the base mesh and bail
		return;
	}

	auto ShowFlipbookWarningToast = [](const FText& Text)
	{
		FNotificationInfo WarningNotification(Text);
		WarningNotification.ExpireDuration = 5.0f;
		WarningNotification.bFireAndForget = true;
		WarningNotification.bUseLargeFont = false;
		WarningNotification.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
		FSlateNotificationManager::Get().AddNotification(WarningNotification);
		UE_LOGF(LogNiagara, Warning, "%ls", *Text.ToString());
	};

	static const FString FrameNumReplace = TEXT("{frame_number}");
	const int32 NumPosInSuffix = FlipbookSuffixFormat.Find(FrameNumReplace);
	if (NumPosInSuffix == INDEX_NONE)
	{
		ShowFlipbookWarningToast(LOCTEXT("FlipbookSuffixWarningToastMessage", "Error gathering meshes for Mesh Flipbook. Suffix Format is missing \"{frame_number}\""));
		return;
	}

	FSoftObjectPath ParticleMeshPath = FirstFlipbookFrame->GetPathName();
	FString BaseName = ParticleMeshPath.GetAssetName();
	int32 FirstFrameIdx = 0;

	// Build a regex pattern string to use to attempt to find the first frame number in the first frame mesh
	FString MatchString;
	for (int32 CharIdx = 0; CharIdx < FlipbookSuffixFormat.Len(); ++CharIdx)
	{
		if (CharIdx == NumPosInSuffix)
		{
			// Add the number match string and skip past the frame number
			MatchString.Append(TEXT("([0-9][0-9]*)"));
			CharIdx += FlipbookSuffixFormat.Len() - 1;
		}
		else
		{
			TCHAR CurChar = FlipbookSuffixFormat[CharIdx];
			if (CurChar >= TCHAR('#') && CurChar <= TCHAR('}'))
			{
				MatchString.AppendChar(TCHAR('\\'));
			}
			MatchString.AppendChar(CurChar);
		}
	}
	MatchString.AppendChar(TCHAR('$'));

	FRegexPattern Pattern(MatchString);
	FRegexMatcher Matcher(Pattern, BaseName);
	if (Matcher.FindNext())
	{
		// Remove the suffix for the base name and retrieve the first frame index
		int32 SuffixLen = Matcher.GetMatchEnding() - Matcher.GetMatchBeginning();
		BaseName.LeftChopInline(SuffixLen, EAllowShrinking::No);

		FString NumMatch = Matcher.GetCaptureGroup(1);
		FirstFrameIdx = FCString::Atoi(*NumMatch);
	}

	// Get the path to the package
	FString BasePackageLocation = ParticleMeshPath.GetLongPackageName();
	int32 PackageDirEnd;
	if (BasePackageLocation.FindLastChar(TCHAR('/'), PackageDirEnd))
	{
		BasePackageLocation.LeftInline(PackageDirEnd, EAllowShrinking::No);
	}

	// Now retrieve all meshes for the flipbook and add them
	bool bAnyError = false;
	int32 LastFrameIdx = FirstFrameIdx + NumFlipbookFrames - 1;
	for (int32 FrameIdx = FirstFrameIdx + 1; FrameIdx <= LastFrameIdx; ++FrameIdx)
	{
		FString NumString = FString::FromInt(FrameIdx);
		while ((uint32)NumString.Len() < FlipbookSuffixNumDigits)
		{
			NumString.InsertAt(0, TCHAR('0'));
		}

		TMap<FString, FStringFormatArg> Args = {
			{ TEXT("frame_number"), NumString }
		};

		FString FrameName = BaseName + FString::Format(*FlipbookSuffixFormat, Args);
		FSoftObjectPath ObjPath(BasePackageLocation / (FrameName + TCHAR('.') + FrameName));
		UStaticMesh* FrameMesh = Cast<UStaticMesh>(ObjPath.TryLoad());
		if (!FrameMesh)
		{
			UE_LOGF(LogNiagara, Warning, "Failed to load Static Mesh '%ls' while attempting to create mesh flipbook. This frame will be missing from the flipbook.",
				*ObjPath.GetAssetPathString());
			bAnyError = true;
		}

		Meshes.AddDefaulted_GetRef().Mesh = FrameMesh;
	}

	if (bAnyError)
	{
		ShowFlipbookWarningToast(LOCTEXT("FlipbookWarningToastMessage", "Failed to load one or more meshes for Mesh Flipbook. See the Output Log for details."));
	}
}

FNiagaraVariable UNiagaraMeshRendererProperties::GetBoundAttribute(const FNiagaraVariableAttributeBinding* Binding) const
{
	if (!NeedsPreciseMotionVectors())
	{
		if (Binding == &PrevPositionBinding
			|| Binding == &PrevScaleBinding
			|| Binding == &PrevMeshOrientationBinding
			|| Binding == &PrevCameraOffsetBinding
			|| Binding == &PrevVelocityBinding)
		{
			return FNiagaraVariable();
		}
	}

	return Super::GetBoundAttribute(Binding);
}

#endif // WITH_EDITORONLY_DATA

#undef LOCTEXT_NAMESPACE
