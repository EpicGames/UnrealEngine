// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMessageDataBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraMessageDataBase)

#if WITH_EDITORONLY_DATA
bool UNiagaraMessageDataBase::Equals(const UNiagaraMessageDataBase* Other) const
{
	if (GetClass() != Other->GetClass())
	{
		return false;
	}

	for (TFieldIterator<FProperty> PropertyIterator(GetClass()); PropertyIterator; ++PropertyIterator)
	{
		if (PropertyIterator->Identical(
			PropertyIterator->ContainerPtrToValuePtr<void>(this),
			PropertyIterator->ContainerPtrToValuePtr<void>(Other), PPF_DeepComparison) == false)
		{
			return false;
		}
	}

	return true;
}
#endif
