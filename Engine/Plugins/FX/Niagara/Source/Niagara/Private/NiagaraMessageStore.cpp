// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMessageStore.h"
#include "NiagaraMessageDataBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraMessageStore)

#if WITH_EDITORONLY_DATA

const TMap<FGuid, TObjectPtr<UNiagaraMessageDataBase>>& FNiagaraMessageStore::GetMessages() const
{ 
	return MessageKeyToMessageMap;
}

void FNiagaraMessageStore::SetMessages(const TMap<FGuid, TObjectPtr<UNiagaraMessageDataBase>>& InMessageKeyToMessageMap)
{
	MessageKeyToMessageMap = InMessageKeyToMessageMap;
	DismissedMessageKeys.Empty();
}

void FNiagaraMessageStore::AddMessage(const FGuid& MessageKey, UNiagaraMessageDataBase* NewMessage)
{
	TObjectPtr<UNiagaraMessageDataBase>* ExistingMessagePtr = MessageKeyToMessageMap.Find(MessageKey);
	if (ExistingMessagePtr == nullptr || (*ExistingMessagePtr)->Equals(NewMessage) == false)
	{
		MessageKeyToMessageMap.Add(MessageKey, NewMessage);
	}
}

void FNiagaraMessageStore::RemoveMessage(const FGuid& MessageKey) 
{
	MessageKeyToMessageMap.Remove(MessageKey);
	DismissedMessageKeys.Remove(MessageKey);
}

void FNiagaraMessageStore::DismissMessage(const FGuid& MessageKey)
{
	DismissedMessageKeys.AddUnique(MessageKey);
}

bool FNiagaraMessageStore::IsMessageDismissed(const FGuid& MessageKey)
{
	return DismissedMessageKeys.Contains(MessageKey);
}

bool FNiagaraMessageStore::HasDismissedMessages() const
{
	return DismissedMessageKeys.IsEmpty() == false;
}

void FNiagaraMessageStore::ClearDismissedMessages()
{
	DismissedMessageKeys.Empty();
}

FNiagaraMessageSourceAndStore::FNiagaraMessageSourceAndStore(UObject& InSource, FNiagaraMessageStore& InStore)
{
	SourceWeak = &InSource;
	Store = &InStore;
}

UObject* FNiagaraMessageSourceAndStore::GetSource() const
{
	return SourceWeak.Get();
}

FNiagaraMessageStore* FNiagaraMessageSourceAndStore::GetStore() const
{
	return SourceWeak.IsValid() ? Store : nullptr;
}

#endif // WITH_EDITORONLY_DATA
