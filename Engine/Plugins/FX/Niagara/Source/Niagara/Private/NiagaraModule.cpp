// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraModule.h"
#include "GPUSortManager.h"
#include "Materials/MaterialInterface.h"
#include "Modules/ModuleManager.h"
#include "NiagaraComponentSettings.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraEmitter.h"
#include "NiagaraTypes.h"
#include "NiagaraEvents.h"
#include "NiagaraSettings.h"
#include "NiagaraDataChannelManager.h"
#include "NiagaraDataChannelDebugUtilities.h"
#include "NiagaraRuntimeTypeUtilities.h"
#include "NiagaraSimCache.h"
#include "NiagaraSimulationTaskPriority.h"
#include "NiagaraUseOpenVDB.h"
#include "NiagaraSystem.h"
#include "UObject/Class.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/Package.h"
#include "NiagaraWorldManager.h"
#include "UObject/UObjectIterator.h"
#include "VectorVM.h"
#include "NiagaraConstants.h"
#include "NiagaraDecalRendererProperties.h"
#include "NiagaraLightRendererProperties.h"
#include "NiagaraSpriteRendererProperties.h"
#include "NiagaraMeshRendererProperties.h"
#include "NiagaraRibbonRendererProperties.h"
#include "NiagaraComponentRendererProperties.h"
#include "NiagaraVolumeRendererProperties.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraDataChannel.h"
#include "NiagaraShaderModule.h"
#include "NiagaraTrace.h"
#include "NiagaraTypeRegistry.h"
#include "NiagaraGpuComputeDispatch.h"
#include "Misc/CoreDelegates.h"
#include "NiagaraDebuggerClient.h"
#include "Particles/FXBudget.h"
#include "Engine/StaticMesh.h"
#include "UObject/UObjectGlobals.h"
#include "NiagaraAssetTagDefinitions.h"

#include "DataInterface/NiagaraDataInterfaceDataChannelCommon.h"
#include "DataInterface/NiagaraDataInterfaceDataChannelWrite.h"
#include "DataInterface/NiagaraDataInterfaceDataChannelRead.h"
#include "Stateless/NiagaraStatelessCommon.h"

#if UE_USE_OPENVDB
#include "NiagaraOpenVDB.h"
#endif

IMPLEMENT_MODULE(INiagaraModule, Niagara);

#define LOCTEXT_NAMESPACE "NiagaraModule"

int32 FNiagaraCompileHashVisitor::LogCompileIdGeneration = 0;
static FAutoConsoleVariableRef CVarLogCompileIdGeneration(
	TEXT("fx.LogCompileIdGeneration"),
	FNiagaraCompileHashVisitor::LogCompileIdGeneration,
	TEXT("If > 0 all compile id generation will be logged. If 2 or greater, log detailed info. \n"),
	ECVF_Default
);


float INiagaraModule::EngineGlobalSpawnCountScale = 1.0f;
float INiagaraModule::EngineGlobalSystemCountScale = 1.0f;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FNiagaraAssetTagDefinition INiagaraModule::LightweightTagDefinition = {
	LOCTEXT("LightweightDisplayName", "Lightweight")
	, (int32) (ENiagaraAssetLibraryAssetTypes::Systems)
	, FText::GetEmpty()
	, ENiagaraAssetTagDefinitionImportance::Primary
	, FLinearColor::Red
	, FGuid(0xE1737BF2, 0x52264191, 0xB9B9AA5F, 0xFB8C7412)};

const FNiagaraAssetTagDefinition INiagaraModule::TemplateTagDefinition = {
	LOCTEXT("TemplateDisplayName", "Template")
	, (int32) (ENiagaraAssetLibraryAssetTypes::Emitters) | (int32) (ENiagaraAssetLibraryAssetTypes::Systems)
	, FText::GetEmpty()
	, ENiagaraAssetTagDefinitionImportance::Primary
	, FLinearColor::Yellow
	, FGuid(0xFC157AFA, 0x50264263, 0xB9A9E658, 0xFFAC2D22)};

const FNiagaraAssetTagDefinition INiagaraModule::LearningContentTagDefinition = {
	LOCTEXT("LearningContentDisplayName", "Learning Content")
	, (int32) (ENiagaraAssetLibraryAssetTypes::Emitters) | (int32) (ENiagaraAssetLibraryAssetTypes::Systems) | (int32) (ENiagaraAssetLibraryAssetTypes::Scripts)
	, FText::GetEmpty()
	, ENiagaraAssetTagDefinitionImportance::Primary
	, FLinearColor::Green
	, FGuid(0xFCF21AFA, 0x50764BFA, 0xB9B2E618, 0xF2A0CD6F)};

const FNiagaraAssetTagDefinition INiagaraModule::HiddenAssetTagDefinition = {
	LOCTEXT("HiddenDisplayName", "Hidden in Asset Browser")
	, (int32) (ENiagaraAssetLibraryAssetTypes::Emitters) | (int32) (ENiagaraAssetLibraryAssetTypes::Systems) | (int32) (ENiagaraAssetLibraryAssetTypes::Scripts)
	, FText::GetEmpty()
	, ENiagaraAssetTagDefinitionImportance::Internal
	, FLinearColor::Black
	, FGuid(0xC7B2F2A1, 0x71762AFB, 0x51BDE232, 0x8CA4BBA3)};

const FNiagaraAssetTagDefinition INiagaraModule::DeprecatedTagDefinition = {
	LOCTEXT("DeprecatedDisplayName", "Deprecated")
	, (int32) (ENiagaraAssetLibraryAssetTypes::Emitters) | (int32) (ENiagaraAssetLibraryAssetTypes::Systems) | (int32) (ENiagaraAssetLibraryAssetTypes::Scripts)
	, FText::GetEmpty()
	, ENiagaraAssetTagDefinitionImportance::Internal
	, FLinearColor::Black
	, FGuid(0xF1F2F2FB, 0x50C64AF9, 0xB6BCE631, 0x32C4CC61)};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

std::atomic<bool> INiagaraModule::bDataChannelRefreshRequested = false;

int32 GEnableVerboseNiagaraChangeIdLogging = 0;
static FAutoConsoleVariableRef CVarEnableVerboseNiagaraChangeIdLogging(
	TEXT("fx.EnableVerboseNiagaraChangeIdLogging"),
	GEnableVerboseNiagaraChangeIdLogging,
	TEXT("If > 0 Verbose change id logging info will be printed. \n"),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarNiaraGlobalSystemCountScale(
	TEXT("fx.NiagaraGlobalSystemCountScale"),
	INiagaraModule::EngineGlobalSystemCountScale,
	TEXT("A global scale on system count thresholds for culling in Niagara. \n"),
	ECVF_Scalability
);

bool INiagaraModule::bUseGlobalFXBudget = true;
static FAutoConsoleVariableRef CVarUseGlobalFXBudget(
	TEXT("fx.Niagara.UseGlobalFXBudget"),
	INiagaraModule::bUseGlobalFXBudget,
	TEXT("If true, Niagara will track performace data into the global FX budget and feed the global budget values into scalability. \n"),
	FConsoleVariableDelegate::CreateStatic(&INiagaraModule::OnUseGlobalFXBudgetChanged),
	ECVF_Default
);

static float GNiagaraMaxCompilePollTimePerFrame = 0.016f;
static FAutoConsoleVariableRef CVarNiagaraMaxCompilePollTimePerFrame(
	TEXT("fx.Niagara.MaxCompilePollTimePerFrame"),
	GNiagaraMaxCompilePollTimePerFrame,
	TEXT("When a lot of system compile tasks queue up, this is the max time per frame that is used to advance them."),
	ECVF_Default
);

void INiagaraModule::OnUseGlobalFXBudgetChanged(IConsoleVariable* Variable)
{
	//Warn if we're enabling Niagara's use of budgeting but the budgeting itself is disabled.
	if (UseGlobalFXBudget() && FFXBudget::Enabled() == false && !IsRunningCommandlet())
	{
		//FFXBudget::SetEnabled(true);
		UE_LOGF(LogNiagara, Warning, "Niagara has enabled fx.Niagara.UseGlobalFXBudget but fx.Budget.Enabled is false so global FX budgetting will still be disabled. Consider also enabling fx.Budget.Enabled.");
	}
}

bool INiagaraModule::bDataChannelsEnabled = true;

static FAutoConsoleVariableRef CVarDataChannelEnabled(
	TEXT("fx.Niagara.DataChannels.Enabled"),
	INiagaraModule::bDataChannelsEnabled,
	TEXT("If true, Niagara Data Channels will be enabled. \n"),
	FConsoleVariableDelegate::CreateStatic(&INiagaraModule::OnDataChannelsEnabledChanged),
	ECVF_Default
);

void INiagaraModule::OnDataChannelsEnabledChanged(IConsoleVariable* Variable)
{
	if(IsRunningCookCommandlet())
	{
		//Never allow NDCs when cooking.
		INiagaraModule::bDataChannelsEnabled = false;
		return;
	}

	if(DataChannelsEnabled())
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(UNiagaraDataInterfaceDataChannelWrite::StaticClass(), Flags);
		FNiagaraTypeRegistry::Register(UNiagaraDataInterfaceDataChannelRead::StaticClass(), Flags);

		FNiagaraWorldManager::ForAllWorldManagers(
			[](FNiagaraWorldManager& WorldMan)
			{
				WorldMan.GetDataChannelManager().Init();
			});
	}
// 	else TODO: for completeness we could add the ability to unregister types 
// 	{
// 		FNiagaraTypeRegistry::Unregister(UNiagaraDataInterfaceDataChannelWrite::StaticClass());
// 		FNiagaraTypeRegistry::Register(UNiagaraDataInterfaceDataChannelRead::StaticClass());
// 		FNiagaraTypeRegistry::Register(UNiagaraDataInterfaceDataChannelSpawn::StaticClass());
// 	}

	//Re-init everything for now.
	//Can be more targeted if needed.
	FNiagaraSystemUpdateContext Context;
	Context.AddAll(true);
}

FNiagaraVariable INiagaraModule::Engine_WorldDeltaTime;
FNiagaraVariable INiagaraModule::Engine_DeltaTime;
FNiagaraVariable INiagaraModule::Engine_InvDeltaTime;
FNiagaraVariable INiagaraModule::Engine_Time;
FNiagaraVariable INiagaraModule::Engine_RealTime;
FNiagaraVariable INiagaraModule::Engine_QualityLevel;

FNiagaraVariable INiagaraModule::Engine_Owner_Position;
FNiagaraVariable INiagaraModule::Engine_Owner_Velocity;
FNiagaraVariable INiagaraModule::Engine_Owner_XAxis;
FNiagaraVariable INiagaraModule::Engine_Owner_YAxis;
FNiagaraVariable INiagaraModule::Engine_Owner_ZAxis;
FNiagaraVariable INiagaraModule::Engine_Owner_Scale;
FNiagaraVariable INiagaraModule::Engine_Owner_Rotation;
FNiagaraVariable INiagaraModule::Engine_Owner_LWC_Tile;
FNiagaraVariable INiagaraModule::Engine_ExecIndex;

FNiagaraVariable INiagaraModule::Engine_Owner_SystemLocalToWorld;
FNiagaraVariable INiagaraModule::Engine_Owner_SystemWorldToLocal;
FNiagaraVariable INiagaraModule::Engine_Owner_SystemLocalToWorldTransposed;
FNiagaraVariable INiagaraModule::Engine_Owner_SystemWorldToLocalTransposed;
FNiagaraVariable INiagaraModule::Engine_Owner_SystemLocalToWorldNoScale;
FNiagaraVariable INiagaraModule::Engine_Owner_SystemWorldToLocalNoScale;

FNiagaraVariable INiagaraModule::Engine_Owner_TimeSinceRendered;
FNiagaraVariable INiagaraModule::Engine_Owner_LODDistance;
FNiagaraVariable INiagaraModule::Engine_Owner_LODDistanceFraction;

FNiagaraVariable INiagaraModule::Engine_Owner_ExecutionState;

FNiagaraVariable INiagaraModule::Engine_ExecutionCount;
FNiagaraVariable INiagaraModule::Engine_Emitter_NumParticles;
FNiagaraVariable INiagaraModule::Engine_Emitter_SimulationPosition;
FNiagaraVariable INiagaraModule::Engine_Emitter_TotalSpawnedParticles;
FNiagaraVariable INiagaraModule::Engine_Emitter_SpawnCountScale;
FNiagaraVariable INiagaraModule::Engine_Emitter_ID;
FNiagaraVariable INiagaraModule::Engine_System_TickCount;
FNiagaraVariable INiagaraModule::Engine_System_NumEmittersAlive;
FNiagaraVariable INiagaraModule::Engine_System_SignificanceIndex;
FNiagaraVariable INiagaraModule::Engine_System_RandomSeed;
FNiagaraVariable INiagaraModule::Engine_System_CurrentTimeStep;
FNiagaraVariable INiagaraModule::Engine_System_NumTimeSteps;
FNiagaraVariable INiagaraModule::Engine_System_TimeStepFraction;
FNiagaraVariable INiagaraModule::Engine_System_NumParticles;

FNiagaraVariable INiagaraModule::Engine_System_NumEmitters;
FNiagaraVariable INiagaraModule::Engine_NumSystemInstances;

FNiagaraVariable INiagaraModule::Engine_GlobalSpawnCountScale;
FNiagaraVariable INiagaraModule::Engine_GlobalSystemScale;

FNiagaraVariable INiagaraModule::Engine_System_Age;

FNiagaraVariable INiagaraModule::Emitter_Age;
FNiagaraVariable INiagaraModule::Emitter_LocalSpace;
FNiagaraVariable INiagaraModule::Emitter_Determinism;
FNiagaraVariable INiagaraModule::Emitter_InterpolatedSpawn;
FNiagaraVariable INiagaraModule::Emitter_OverrideGlobalSpawnCountScale;
FNiagaraVariable INiagaraModule::Emitter_SimulationTarget;
FNiagaraVariable INiagaraModule::Emitter_RandomSeed;
FNiagaraVariable INiagaraModule::Engine_Emitter_InstanceSeed;
FNiagaraVariable INiagaraModule::Emitter_SpawnRate;
FNiagaraVariable INiagaraModule::Emitter_SpawnInterval;
FNiagaraVariable INiagaraModule::Emitter_InterpSpawnStartDt;
FNiagaraVariable INiagaraModule::Emitter_SpawnGroup;

FNiagaraVariable INiagaraModule::Particles_UniqueID;
FNiagaraVariable INiagaraModule::Particles_ID;
FNiagaraVariable INiagaraModule::Particles_Position;
FNiagaraVariable INiagaraModule::Particles_Velocity;
FNiagaraVariable INiagaraModule::Particles_Color;
FNiagaraVariable INiagaraModule::Particles_SpriteRotation;
FNiagaraVariable INiagaraModule::Particles_Age;
FNiagaraVariable INiagaraModule::Particles_NormalizedAge;
FNiagaraVariable INiagaraModule::Particles_SpriteSize;
FNiagaraVariable INiagaraModule::Particles_SpriteFacing;
FNiagaraVariable INiagaraModule::Particles_SpriteAlignment;
FNiagaraVariable INiagaraModule::Particles_SubImageIndex;
FNiagaraVariable INiagaraModule::Particles_DynamicMaterialParameter;
FNiagaraVariable INiagaraModule::Particles_DynamicMaterialParameter1;
FNiagaraVariable INiagaraModule::Particles_DynamicMaterialParameter2;
FNiagaraVariable INiagaraModule::Particles_DynamicMaterialParameter3;
FNiagaraVariable INiagaraModule::Particles_Scale;
FNiagaraVariable INiagaraModule::Particles_Lifetime;
FNiagaraVariable INiagaraModule::Particles_MeshOrientation;
FNiagaraVariable INiagaraModule::Particles_UVScale;
FNiagaraVariable INiagaraModule::Particles_PivotOffset;
FNiagaraVariable INiagaraModule::Particles_CameraOffset;
FNiagaraVariable INiagaraModule::Particles_MaterialRandom;
FNiagaraVariable INiagaraModule::Particles_LightRadius;
FNiagaraVariable INiagaraModule::Particles_LightExponent;
FNiagaraVariable INiagaraModule::Particles_LightEnabled;
FNiagaraVariable INiagaraModule::Particles_LightVolumetricScattering;
FNiagaraVariable INiagaraModule::Particles_LightSpecularScale;
FNiagaraVariable INiagaraModule::Particles_LightDiffuseScale;
FNiagaraVariable INiagaraModule::Particles_RibbonID;
FNiagaraVariable INiagaraModule::Particles_RibbonWidth;
FNiagaraVariable INiagaraModule::Particles_RibbonTwist;
FNiagaraVariable INiagaraModule::Particles_RibbonFacing;
FNiagaraVariable INiagaraModule::Particles_RibbonLinkOrder;
FNiagaraVariable INiagaraModule::Particles_RibbonU0Override;
FNiagaraVariable INiagaraModule::Particles_RibbonV0RangeOverride;
FNiagaraVariable INiagaraModule::Particles_RibbonU1Override;
FNiagaraVariable INiagaraModule::Particles_RibbonV1RangeOverride;
FNiagaraVariable INiagaraModule::Particles_RibbonUVDistance;
FNiagaraVariable INiagaraModule::Particles_VisibilityTag;
FNiagaraVariable INiagaraModule::Particles_MeshIndex;
FNiagaraVariable INiagaraModule::Particles_ComponentsEnabled;
FNiagaraVariable INiagaraModule::ScriptUsage;
FNiagaraVariable INiagaraModule::ScriptContext;
FNiagaraVariable INiagaraModule::FunctionDebugState;
FNiagaraVariable INiagaraModule::DataInstance_Alive;
FNiagaraVariable INiagaraModule::Translator_BeginDefaults;
FNiagaraVariable INiagaraModule::Translator_CallID;

void INiagaraModule::StartupModule()
{
	FNiagaraTypeRegistry::Init();

	VectorVM::Init();
	FNiagaraTypeHelper::InitLWCTypes();

	LLM_SCOPE(ELLMTag::Niagara);
	FNiagaraTypeDefinition::Init();
	FNiagaraTypeRegistry::ProcessRegistryQueue();

	FNiagaraRuntimeTypeUtilities::Initialize();

	if (IsRunningCookCommandlet())
	{
		//Never allow NDCs when cooking.
		INiagaraModule::bDataChannelsEnabled = false;
	}

#if UE_USE_OPENVDB
	// Global registration of  the vdb types.
	openvdb::initialize();
	if (!Vec4SGrid::isRegistered())
	{
		Vec4SGrid::registerGrid();
	}
	
	if (!Vec4HGrid::isRegistered())
	{
		Vec4HGrid::registerGrid();
	}
#endif

	FNiagaraWorldManager::OnStartup();

	NiagaraSimulationTaskPriority::Initialize();

#if WITH_NIAGARA_DEBUGGER
	DebuggerClient = MakePimpl<FNiagaraDebuggerClient>();
#endif

#if WITH_EDITOR	
	// Loading uncooked data in a game environment, we still need to get some functionality from the NiagaraEditor module.
	// This includes the ability to compile scripts and load WITH_EDITOR_ONLY data.
	// Note that when loading with the Editor, the NiagaraEditor module is loaded based on the plugin description.
	FModuleManager::Get().LoadModule(TEXT("NiagaraEditor"));

	FCoreUObjectDelegates::OnAssetLoaded.AddRaw(this, &INiagaraModule::OnAssetLoaded);
#endif

	//Init commonly used FNiagaraVariables

	Engine_WorldDeltaTime = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.WorldDeltaTime"));
	Engine_DeltaTime = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.DeltaTime"));
	Engine_InvDeltaTime = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.InverseDeltaTime"));
	
	Engine_Time = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.Time"));
	Engine_RealTime = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.RealTime"));
	Engine_QualityLevel = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.QualityLevel"));

	Engine_Owner_Position = FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Engine.Owner.Position"));
	Engine_Owner_Velocity = FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Engine.Owner.Velocity"));
	Engine_Owner_XAxis = FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Engine.Owner.SystemXAxis"));
	Engine_Owner_YAxis = FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Engine.Owner.SystemYAxis"));
	Engine_Owner_ZAxis = FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Engine.Owner.SystemZAxis"));
	Engine_Owner_Scale = FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Engine.Owner.Scale"));
	Engine_Owner_Rotation = FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Engine.Owner.Rotation"));
	Engine_Owner_LWC_Tile = FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Engine.Owner.LWCTile"));
	Engine_ExecIndex = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.ExecIndex"));

	Engine_Owner_SystemLocalToWorld = FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Engine.Owner.SystemLocalToWorld"));
	Engine_Owner_SystemWorldToLocal = FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Engine.Owner.SystemWorldToLocal"));
	Engine_Owner_SystemLocalToWorldTransposed = FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Engine.Owner.SystemLocalToWorldTransposed"));
	Engine_Owner_SystemWorldToLocalTransposed = FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Engine.Owner.SystemWorldToLocalTransposed"));
	Engine_Owner_SystemLocalToWorldNoScale = FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Engine.Owner.SystemLocalToWorldNoScale"));
	Engine_Owner_SystemWorldToLocalNoScale = FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Engine.Owner.SystemWorldToLocalNoScale"));

	Engine_Owner_TimeSinceRendered = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.Owner.TimeSinceRendered"));
	Engine_Owner_LODDistance = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.Owner.LODDistance"));
	Engine_Owner_LODDistanceFraction = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.Owner.LODDistanceFraction"));

	Engine_Owner_ExecutionState = FNiagaraVariable(FNiagaraTypeDefinition::GetExecutionStateEnum(), TEXT("Engine.Owner.ExecutionState"));
	
	Engine_ExecutionCount = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.ExecutionCount"));
	Engine_Emitter_NumParticles = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.Emitter.NumParticles"));
	Engine_Emitter_SimulationPosition = FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Engine.Emitter.SimulationPosition"));
	Engine_Emitter_TotalSpawnedParticles = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.Emitter.TotalSpawnedParticles"));
	Engine_Emitter_SpawnCountScale = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.Emitter.SpawnCountScale"));
	Engine_Emitter_InstanceSeed = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.Emitter.InstanceSeed"));
	Engine_Emitter_ID = FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraEmitterID::StaticStruct()), TEXT("Engine.Emitter.ID"));
	Engine_System_TickCount = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.System.TickCount"));
	Engine_System_NumEmittersAlive = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.System.NumEmittersAlive"));
	Engine_System_SignificanceIndex = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.System.SignificanceIndex"));
	Engine_System_RandomSeed = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.System.RandomSeed"));

	Engine_System_CurrentTimeStep = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.System.CurrentTimeStep"));
	Engine_System_NumTimeSteps = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.System.NumTimeSteps"));
	Engine_System_TimeStepFraction = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.System.TimeStepFraction"));
	Engine_System_NumParticles = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.System.NumParticles"));

	Engine_System_NumEmitters = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.System.NumEmitters"));
	Engine_NumSystemInstances = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Engine.NumSystemInstances"));

	Engine_GlobalSpawnCountScale = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.GlobalSpawnCountScale"));
	Engine_GlobalSystemScale = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.GlobalSystemCountScale"));

	Engine_System_Age = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Engine.System.Age"));
	Emitter_Age = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Emitter.Age"));
	Emitter_LocalSpace = FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.LocalSpace"));
	Emitter_RandomSeed = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Emitter.RandomSeed"));
	Emitter_Determinism = FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.Determinism"));
	Emitter_InterpolatedSpawn = FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.InterpolatedSpawn"));
	Emitter_OverrideGlobalSpawnCountScale = FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.OverrideGlobalSpawnCountScale"));
	Emitter_SimulationTarget = FNiagaraVariable(FNiagaraTypeDefinition::GetSimulationTargetEnum(), TEXT("Emitter.SimulationTarget"));
	Emitter_SpawnRate = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Emitter.SpawnRate"));
	Emitter_SpawnInterval = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Emitter.SpawnInterval"));
	Emitter_InterpSpawnStartDt = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Emitter.InterpSpawnStartDt"));
	Emitter_SpawnGroup = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Emitter.SpawnGroup"));

	Particles_UniqueID = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particles.UniqueID"));
	Particles_ID = FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particles.ID"));
	Particles_Position = FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Particles.Position"));
	Particles_Velocity = FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particles.Velocity"));
	Particles_Color = FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Particles.Color"));
	Particles_SpriteRotation = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.SpriteRotation"));
	Particles_Age = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.Age"));
	Particles_NormalizedAge = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.NormalizedAge"));
	Particles_SpriteSize = FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Particles.SpriteSize"));
	Particles_SpriteFacing = FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particles.SpriteFacing"));
	Particles_SpriteAlignment = FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particles.SpriteAlignment"));
	Particles_SubImageIndex = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.SubImageIndex"));
	Particles_DynamicMaterialParameter = FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Particles.DynamicMaterialParameter"));
	Particles_DynamicMaterialParameter1 = FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Particles.DynamicMaterialParameter1"));
	Particles_DynamicMaterialParameter2 = FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Particles.DynamicMaterialParameter2"));
	Particles_DynamicMaterialParameter3 = FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Particles.DynamicMaterialParameter3"));
	Particles_Scale = FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particles.Scale"));
	Particles_Lifetime = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.Lifetime"));
	Particles_MeshOrientation = FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Particles.MeshOrientation"));
	Particles_UVScale = FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Particles.UVScale"));
	Particles_PivotOffset = FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Particles.PivotOffset"));
	Particles_CameraOffset = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.CameraOffset"));
	Particles_MaterialRandom = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.MaterialRandom"));
	Particles_LightRadius = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.LightRadius"));
	Particles_LightExponent = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.LightExponent"));
	Particles_LightEnabled = FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Particles.LightEnabled"));
	Particles_LightVolumetricScattering = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.LightVolumetricScattering"));
	Particles_LightSpecularScale = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.LightSpecularScale"));
	Particles_LightDiffuseScale = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.LightDiffuseScale"));
	Particles_RibbonID = FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particles.RibbonID"));
	Particles_RibbonWidth = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.RibbonWidth"));
	Particles_RibbonTwist = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.RibbonTwist"));
	Particles_RibbonFacing = FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particles.RibbonFacing"));
	Particles_RibbonLinkOrder = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.RibbonLinkOrder"));
	Particles_RibbonUVDistance = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.RibbonUVDistance"));
	Particles_RibbonU0Override = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.RibbonU0Override"));
	Particles_RibbonV0RangeOverride = FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Particles.RibbonV0RangeOverride"));
	Particles_RibbonU1Override = FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.RibbonU1Override"));
	Particles_RibbonV1RangeOverride = FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Particles.RibbonV1RangeOverride"));
	Particles_VisibilityTag = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particles.VisibilityTag"));
	Particles_MeshIndex = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particles.MeshIndex"));
	Particles_ComponentsEnabled = FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Particles.ComponentsEnabled"));

	ScriptUsage = FNiagaraVariable(FNiagaraTypeDefinition::GetScriptUsageEnum(), TEXT("Script.Usage"));
	ScriptContext = FNiagaraVariable(FNiagaraTypeDefinition::GetScriptContextEnum(), TEXT("Script.Context"));
	DataInstance_Alive = FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("DataInstance.Alive"));

	FunctionDebugState = FNiagaraVariable(FNiagaraTypeDefinition::GetFunctionDebugStateEnum(), TEXT("Function.DebugState"));

	Translator_BeginDefaults = FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("Begin Defaults"));
	Translator_CallID = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Translator.CallID"));

	FNiagaraConstants::Init();
	UNiagaraDecalRendererProperties::InitCDOPropertiesAfterModuleStartup();
	UNiagaraLightRendererProperties::InitCDOPropertiesAfterModuleStartup();
	UNiagaraSpriteRendererProperties::InitCDOPropertiesAfterModuleStartup();
	UNiagaraRibbonRendererProperties::InitCDOPropertiesAfterModuleStartup();
	UNiagaraMeshRendererProperties::InitCDOPropertiesAfterModuleStartup();
	UNiagaraComponentRendererProperties::InitCDOPropertiesAfterModuleStartup();
	UNiagaraVolumeRendererProperties::InitCDOPropertiesAfterModuleStartup();
	NiagaraStatelessCommon::Initialize();

	// Register the data interface CDO finder with the shader module..
	INiagaraShaderModule& NiagaraShaderModule = FModuleManager::LoadModuleChecked<INiagaraShaderModule>("NiagaraShader");
	NiagaraShaderModule.SetOnRequestDefaultDataInterfaceHandler(INiagaraShaderModule::FOnRequestDefaultDataInterface::CreateLambda([](const FString& DIClassName) -> UNiagaraDataInterfaceBase*
	{
		return FNiagaraTypeRegistry::GetDefaultDataInterfaceByName(DIClassName);
	}));

	FFXSystemInterface::RegisterCustomFXSystem(
		FNiagaraGpuComputeDispatch::Name,
		FCreateCustomFXSystemDelegate::CreateLambda([](ERHIFeatureLevel::Type InFeatureLevel, EShaderPlatform InShaderPlatform, FGPUSortManager* InGPUSortManager) -> FFXSystemInterface*
	{
		return new FNiagaraGpuComputeDispatch(InFeatureLevel, InShaderPlatform, InGPUSortManager);
	}));

	// Needed for NiagaraDataInterfaceAudioSpectrum
	FModuleManager::Get().LoadModule(TEXT("SignalProcessing"));

#if NIAGARA_PERF_BASELINES
#if !WITH_EDITOR
	//The editor has it's own path for generating baselines. This is used in non editor builds only.
	UNiagaraEffectType::OnGeneratePerfBaselines().BindRaw(this, &INiagaraModule::GeneratePerfBaselines);
#endif
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RegisterInternalAssetTagDefinitions();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &INiagaraModule::OnPostEngineInit);
	FCoreDelegates::OnPreExit.AddRaw(this, &INiagaraModule::OnPreExit);
	FWorldDelegates::OnWorldBeginTearDown.AddRaw(this, &INiagaraModule::OnWorldBeginTearDown);
	FWorldDelegates::OnWorldTickStart.AddRaw(this, &INiagaraModule::OnWorldTickStart);

	FCoreDelegates::OnBeginFrame.AddRaw(this, &INiagaraModule::OnBeginFrame);
	FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &INiagaraModule::OnPostGarbageCollect);
}

void INiagaraModule::OnPostEngineInit()
{
#if NIAGARA_PERF_BASELINES
	if (BaselineHandler.IsValid() == false)
	{
		BaselineHandler = MakeUnique<FNiagaraPerfBaselineHandler>();
	}
#endif

	IConsoleManager& CMan = IConsoleManager::Get();
	OnCVarUnregisteredHandle = CMan.OnCVarUnregistered().AddLambda(
		[](IConsoleVariable* CVar)
		{
			FNiagaraPlatformSet::OnCVarUnregistered(CVar);
		}
	);

	FNiagaraComponentSettings::UpdateSettings();
	NiagaraStatelessCommon::UpdateSettings();
}

void INiagaraModule::OnPreExit()
{
#if NIAGARA_PERF_BASELINES
	if (BaselineHandler.IsValid())
	{
		BaselineHandler.Reset();
	}
#endif

#if WITH_NIAGARA_DEBUGGER
	DebuggerClient.Reset();
#endif

	IConsoleManager& CMan = IConsoleManager::Get();
	CMan.OnCVarUnregistered().Remove(OnCVarUnregisteredHandle);
}

#if WITH_EDITOR
void PollSystemCompilations()
{
	double Start = FPlatformTime::Seconds();
	for (TObjectIterator<UNiagaraSystem> SysIt; SysIt; ++SysIt)
	{
		UNiagaraSystem* System = *SysIt;
		System->PollForCompilationComplete(false);
		if (FPlatformTime::Seconds() - Start > GNiagaraMaxCompilePollTimePerFrame)
		{
			break;
		}
	}
}
#endif

void INiagaraModule::OnWorldTickStart(UWorld* World, ELevelTick TickType, float DeltaSeconds)
{
#if WITH_EDITOR
	PollSystemCompilations();
#endif

	RefreshDataChannels();
	
#if NIAGARA_PERF_BASELINES
	if (BaselineHandler.IsValid())
	{
		BaselineHandler->Tick(World, DeltaSeconds);
	}
#endif
}

void INiagaraModule::RefreshDataChannels()
{
	if (bDataChannelRefreshRequested)
	{
		bDataChannelRefreshRequested = false;
		FNiagaraWorldManager::ForAllWorldManagers(
			[](FNiagaraWorldManager& WorldMan)
			{
				WorldMan.GetDataChannelManager().RefreshDataChannels();
			});
	}
}

const TArray<const FNiagaraAssetTagDefinition*>& INiagaraModule::GetInternalAssetTagDefinitions() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return InternalAssetTagDefinitions;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void INiagaraModule::OnPostGarbageCollect()
{
	FNiagaraTypeHelper::TickTypeRemap();
}

void INiagaraModule::OnBeginFrame()
{
	FNiagaraInsights::Update();
	FNiagaraComponentSettings::UpdateSettings();
	NiagaraStatelessCommon::UpdateSettings();
	FNiagaraPlatformSet::RefreshScalability();
}

void INiagaraModule::OnWorldBeginTearDown(UWorld* World)
{
#if NIAGARA_PERF_BASELINES
	if (BaselineHandler.IsValid())
	{
		BaselineHandler->OnWorldBeginTearDown(World);
	}
#endif
}

void INiagaraModule::ShutdownRenderingResources()
{
	FFXSystemInterface::UnregisterCustomFXSystem(FNiagaraGpuComputeDispatch::Name);
}

void INiagaraModule::ShutdownModule()
{
#if WITH_EDITOR
	UNiagaraMeshRendererProperties::RemoveMeshChangedHandlers();
#endif

	FWorldDelegates::OnWorldBeginTearDown.RemoveAll(this);
	FWorldDelegates::OnWorldTickStart.RemoveAll(this);
	FCoreDelegates::OnBeginFrame.RemoveAll(this);

	FNiagaraWorldManager::OnShutdown();

	// Clear out the handler when shutting down..
	INiagaraShaderModule& NiagaraShaderModule = FModuleManager::LoadModuleChecked<INiagaraShaderModule>("NiagaraShader");
	NiagaraShaderModule.ResetOnRequestDefaultDataInterfaceHandler();

	ShutdownRenderingResources();

	FNiagaraTypeRegistry::TearDown();

	FNDIDataChannelLayoutManager::TearDown();

#if WITH_EDITOR
	FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);
#endif

#if WITH_NIAGARA_DEBUGGER
	DebuggerClient.Reset();
	FNiagaraDataChannelDebugUtilities::TearDown();
#endif

#if NIAGARA_PERF_BASELINES
	UNiagaraEffectType::OnGeneratePerfBaselines().Unbind();
#endif
}

INiagaraModule& INiagaraModule::Get()
{
	return FModuleManager::LoadModuleChecked<INiagaraModule>("Niagara");
}

#if WITH_EDITOR
const INiagaraMergeManager& INiagaraModule::GetMergeManager() const
{
	checkf(MergeManager.IsValid(), TEXT("Merge manager was never registered, or was unregistered."));
	return *MergeManager.Get();
}

void INiagaraModule::RegisterMergeManager(TSharedRef<INiagaraMergeManager> InMergeManager)
{
	checkf(MergeManager.IsValid() == false, TEXT("Only one merge manager can be registered at a time."));
	MergeManager = InMergeManager;
}

void INiagaraModule::UnregisterMergeManager(TSharedRef<INiagaraMergeManager> InMergeManager)
{
	checkf(MergeManager.IsValid(), TEXT("MergeManager is not registered"));
	checkf(MergeManager == InMergeManager, TEXT("Can only unregister the merge manager which was previously registered."));
	MergeManager.Reset();
}

const INiagaraEditorOnlyDataUtilities& INiagaraModule::GetEditorOnlyDataUtilities() const
{
	checkf(EditorOnlyDataUtilities.IsValid(), TEXT("Editor only data utilities object was never registered, or was unregistered."));
	return *EditorOnlyDataUtilities.Get();
}

void INiagaraModule::RegisterEditorOnlyDataUtilities(TSharedRef<INiagaraEditorOnlyDataUtilities> InEditorOnlyDataUtilities)
{
	checkf(EditorOnlyDataUtilities.IsValid() == false, TEXT("Only one editor only data utilities object can be registered at a time."));
	EditorOnlyDataUtilities = InEditorOnlyDataUtilities;
}

void INiagaraModule::UnregisterEditorOnlyDataUtilities(TSharedRef<INiagaraEditorOnlyDataUtilities> InEditorOnlyDataUtilities)
{
	checkf(EditorOnlyDataUtilities.IsValid(), TEXT("Editor only data utilities object is not registered"));
	checkf(EditorOnlyDataUtilities == InEditorOnlyDataUtilities, TEXT("Can only unregister the editor only data utilities object which was previously registered."));
	EditorOnlyDataUtilities.Reset();
}

const INiagaraVerseReflectionUtilities* INiagaraModule::GetVerseReflectionUtilities() const
{
	return VerseReflectionUtilities.Get();
}

void INiagaraModule::RegisterVerseReflectionUtilities(TSharedRef<INiagaraVerseReflectionUtilities> InUtilities)
{
	checkf(VerseReflectionUtilities.IsValid() == false, TEXT("Verse Reflection Utilities are already registered"));
	VerseReflectionUtilities = InUtilities;
}

void INiagaraModule::UnregisterVerseReflectionUtilities(TSharedRef<INiagaraVerseReflectionUtilities> InUtilities)
{
	checkf(VerseReflectionUtilities == InUtilities, TEXT("Verse Reflection Utilities does not match the unregistered request"));
	VerseReflectionUtilities.Reset();
}

int32 INiagaraModule::StartScriptCompileJob(const FNiagaraCompileRequestDataBase* InCompileData, const FNiagaraCompileRequestDuplicateDataBase* InCompileDuplicateData, const FNiagaraCompileOptions& InCompileOptions)
{
	checkf(ScriptCompilerDelegate.IsBound(), TEXT("Create default script compiler delegate not bound."));
	return ScriptCompilerDelegate.Execute(InCompileData, InCompileDuplicateData, InCompileOptions);
}

TSharedPtr<FNiagaraVMExecutableData> INiagaraModule::GetCompileJobResult(int32 JobID, bool bWait, FNiagaraScriptCompileMetrics& Metrics)
{
	checkf(ScriptCompilerDelegate.IsBound(), TEXT("Script compilation result delegate not bound."));
	return CompilationResultDelegate.Execute(JobID, bWait, Metrics);
}

FDelegateHandle INiagaraModule::RegisterScriptCompiler(FScriptCompiler ScriptCompiler)
{
	checkf(ScriptCompilerDelegate.IsBound() == false, TEXT("Only one handler is allowed for the ScriptCompiler delegate"));
	ScriptCompilerDelegate = ScriptCompiler;
	return ScriptCompilerDelegate.GetHandle();
}

void INiagaraModule::UnregisterScriptCompiler(FDelegateHandle DelegateHandle)
{
	checkf(ScriptCompilerDelegate.IsBound(), TEXT("ScriptCompiler is not registered"));
	checkf(ScriptCompilerDelegate.GetHandle() == DelegateHandle, TEXT("Can only unregister the ScriptCompiler delegate with the handle it was registered with."));
	ScriptCompilerDelegate.Unbind();
}


FDelegateHandle INiagaraModule::RegisterCompileResultDelegate(FCheckCompilationResult ResultDelegate)
{
	checkf(CompilationResultDelegate.IsBound() == false, TEXT("Only one handler is allowed for the CompilationResultDelegate"));
	CompilationResultDelegate = ResultDelegate;
	return CompilationResultDelegate.GetHandle();
}

void INiagaraModule::UnregisterCompileResultDelegate(FDelegateHandle DelegateHandle)
{
	checkf(CompilationResultDelegate.IsBound(), TEXT("CompilationResultDelegate is not registered"));
	checkf(CompilationResultDelegate.GetHandle() == DelegateHandle, TEXT("Can only unregister the CompilationResultDelegate with the handle it was registered with."));
	CompilationResultDelegate.Unbind();
}

TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> INiagaraModule::Precompile(UObject* Obj, FGuid Version)
{
	checkf(PrecompileDelegate.IsBound(), TEXT("Precompile delegate not bound."));
	return PrecompileDelegate.Execute(Obj, Version);
}

TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> INiagaraModule::PrecompileDuplicate(
	const FNiagaraCompileRequestDataBase* OwningSystemRequestData,
	UNiagaraSystem* OwningSystem,
	UNiagaraEmitter* OwningEmitter,
	UNiagaraScript* TargetScript,
	FGuid TargetVersion)
{
	checkf(PrecompileDuplicateDelegate.IsBound(), TEXT("PrecompileDuplicate delegate not bound."));
	return PrecompileDuplicateDelegate.Execute(OwningSystemRequestData, OwningSystem, OwningEmitter, TargetScript, TargetVersion);
}

TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> INiagaraModule::CacheGraphTraversal(const UObject* Obj, FGuid Version)
{
	checkf(GraphTraversalCacheDelegate.IsBound(), TEXT("GraphTraversalCache delegate not bound."));
	return GraphTraversalCacheDelegate.Execute(Obj, Version);
}

FDelegateHandle INiagaraModule::RegisterPrecompiler(FOnPrecompile PreCompiler)
{
	checkf(PrecompileDelegate.IsBound() == false, TEXT("Only one handler is allowed for the Precompiler delegate"));
	PrecompileDelegate = PreCompiler;
	return PrecompileDelegate.GetHandle();
}

void INiagaraModule::UnregisterPrecompiler(FDelegateHandle DelegateHandle)
{
	checkf(PrecompileDelegate.IsBound(), TEXT("ObjectPrecompiler is not registered"));
	checkf(PrecompileDelegate.GetHandle() == DelegateHandle, TEXT("Can only unregister the Precompiler delegate with the handle it was registered with."));
	PrecompileDelegate.Unbind();
}



FDelegateHandle INiagaraModule::RegisterPrecompileDuplicator(FOnPrecompileDuplicate PreCompileDuplicator)
{
	checkf(PrecompileDuplicateDelegate.IsBound() == false, TEXT("Only one handler is allowed for the PrecompileDuplicator delegate"));
	PrecompileDuplicateDelegate = PreCompileDuplicator;
	return PrecompileDuplicateDelegate.GetHandle();
}

void INiagaraModule::UnregisterPrecompileDuplicator(FDelegateHandle DelegateHandle)
{
	checkf(PrecompileDuplicateDelegate.IsBound(), TEXT("PrecompileDuplicator is not registered"));
	checkf(PrecompileDuplicateDelegate.GetHandle() == DelegateHandle, TEXT("Can only unregister the PrecompileDuplicator delegate with the handle it was registered with."));
	PrecompileDuplicateDelegate.Unbind();
}


FDelegateHandle INiagaraModule::RegisterGraphTraversalCacher(FOnCacheGraphTraversal GraphTraversalCacher)
{
	checkf(GraphTraversalCacheDelegate.IsBound() == false, TEXT("Only one handler is allowed for the GraphTraversalCacher delegate"));
	GraphTraversalCacheDelegate = GraphTraversalCacher;
	return GraphTraversalCacheDelegate.GetHandle();
}

void INiagaraModule::UnregisterGraphTraversalCacher(FDelegateHandle DelegateHandle)
{
	checkf(GraphTraversalCacheDelegate.IsBound(), TEXT("ObjectGraphTraversalCacher is not registered"));
	checkf(GraphTraversalCacheDelegate.GetHandle() == DelegateHandle, TEXT("Can only unregister the GraphTraversalCacher delegate with the handle it was registered with."));
	GraphTraversalCacheDelegate.Unbind();
}

FNiagaraCompilationTaskHandle INiagaraModule::RequestCompileSystem(UNiagaraSystem* System, bool bForce, const ITargetPlatform* TargetPlatform)
{
	checkf(RequestCompileSystemDelegate.IsBound(), TEXT("RequestCompileSystemDelegate delegate not bound."));
	return RequestCompileSystemDelegate.Execute(System, bForce, TargetPlatform);
}

FDelegateHandle INiagaraModule::RegisterRequestCompileSystem(FOnRequestCompileSystem RequestCompileSystemCallback)
{
	checkf(RequestCompileSystemDelegate.IsBound() == false, TEXT("Only one handler is allowed for the RequestCompileSystem delegate"));
	RequestCompileSystemDelegate = RequestCompileSystemCallback;
	return RequestCompileSystemDelegate.GetHandle();
}

void INiagaraModule::UnregisterRequestCompileSystem(FDelegateHandle DelegateHandle)
{
	checkf(RequestCompileSystemDelegate.IsBound(), TEXT("RequestCompileSystemDelegate is not registered"));
	checkf(RequestCompileSystemDelegate.GetHandle() == DelegateHandle, TEXT("Can only unregister the RequestCompileSystem delegate with the handle it was registered with."));
	RequestCompileSystemDelegate.Unbind();
}

bool INiagaraModule::PollSystemCompile(FNiagaraCompilationTaskHandle TaskHandle, FNiagaraSystemAsyncCompileResults& Results, bool bWait, bool bPeek)
{
	checkf(PollSystemCompileDelegate.IsBound(), TEXT("PollSystemCompileDelegate delegate not bound."));
	return PollSystemCompileDelegate.Execute(TaskHandle, Results, bWait, bPeek);
}

FDelegateHandle INiagaraModule::RegisterPollSystemCompile(FOnPollSystemCompile PollSystemCompileCallback)
{
	checkf(PollSystemCompileDelegate.IsBound() == false, TEXT("Only one handler is allowed for the PollSystemCompile delegate"));
	PollSystemCompileDelegate = PollSystemCompileCallback;
	return PollSystemCompileDelegate.GetHandle();
}

void INiagaraModule::UnregisterPollSystemCompile(FDelegateHandle DelegateHandle)
{
	checkf(PollSystemCompileDelegate.IsBound(), TEXT("PollSystemCompileDelegate is not registered"));
	checkf(PollSystemCompileDelegate.GetHandle() == DelegateHandle, TEXT("Can only unregister the PollSystemCompile delegate with the handle it was registered with."));
	PollSystemCompileDelegate.Unbind();
}

void INiagaraModule::AbortSystemCompile(FNiagaraCompilationTaskHandle TaskHandle)
{
	checkf(AbortSystemCompileDelegate.IsBound(), TEXT("AbortSystemCompileDelegate delegate not bound."));
	AbortSystemCompileDelegate.Execute(TaskHandle);
}

FDelegateHandle INiagaraModule::RegisterAbortSystemCompile(FOnAbortSystemCompile AbortSystemCompileCallback)
{
	checkf(AbortSystemCompileDelegate.IsBound() == false, TEXT("Only one handler is allowed for the AbortSystemCompile delegate"));
	AbortSystemCompileDelegate = AbortSystemCompileCallback;
	return AbortSystemCompileDelegate.GetHandle();
}

void INiagaraModule::UnregisterAbortSystemCompile(FDelegateHandle DelegateHandle)
{
	checkf(AbortSystemCompileDelegate.IsBound(), TEXT("AbortSystemCompileDelegate is not registered"));
	checkf(AbortSystemCompileDelegate.GetHandle() == DelegateHandle, TEXT("Can only unregister the AbortSystemCompile delegate with the handle it was registered with."));
	AbortSystemCompileDelegate.Unbind();
}

void INiagaraModule::OnAssetLoaded(UObject* Asset)
{
	if (Asset->IsA<UNiagaraSystem>())
	{
		CastChecked<UNiagaraSystem>(Asset)->UpdateSystemAfterLoad();
	}
	else if (Asset->IsA<UNiagaraEmitter>())
	{
		CastChecked<UNiagaraEmitter>(Asset)->UpdateEmitterAfterLoad();
	}
}

#endif

//////////////////////////////////////////////////////////////////////////

UScriptStruct* FNiagaraTypeDefinition::ParameterMapStruct;
UScriptStruct* FNiagaraTypeDefinition::IDStruct;
UScriptStruct* FNiagaraTypeDefinition::RandInfoStruct;
UScriptStruct* FNiagaraTypeDefinition::NumericStruct;
UScriptStruct* FNiagaraTypeDefinition::FloatStruct;
UScriptStruct* FNiagaraTypeDefinition::BoolStruct;
UScriptStruct* FNiagaraTypeDefinition::IntStruct;
UScriptStruct* FNiagaraTypeDefinition::Matrix4Struct;
UScriptStruct* FNiagaraTypeDefinition::Vec4Struct;
UScriptStruct* FNiagaraTypeDefinition::Vec3Struct;
UScriptStruct* FNiagaraTypeDefinition::Vec2Struct;
UScriptStruct* FNiagaraTypeDefinition::ColorStruct;
UScriptStruct* FNiagaraTypeDefinition::QuatStruct;
UScriptStruct* FNiagaraTypeDefinition::WildcardStruct;
UScriptStruct* FNiagaraTypeDefinition::PositionStruct;

UScriptStruct* FNiagaraTypeDefinition::HalfStruct;
UScriptStruct* FNiagaraTypeDefinition::HalfVec2Struct;
UScriptStruct* FNiagaraTypeDefinition::HalfVec3Struct;
UScriptStruct* FNiagaraTypeDefinition::HalfVec4Struct;

UClass* FNiagaraTypeDefinition::UObjectClass;
UClass* FNiagaraTypeDefinition::UMaterialClass;
UClass* FNiagaraTypeDefinition::UTextureClass;
UClass* FNiagaraTypeDefinition::UTextureRenderTargetClass;

UEnum* FNiagaraTypeDefinition::ExecutionStateEnum;
UEnum* FNiagaraTypeDefinition::CoordinateSpaceEnum;
UEnum* FNiagaraTypeDefinition::OrientationAxisEnum;
UEnum* FNiagaraTypeDefinition::SimulationTargetEnum;
UEnum* FNiagaraTypeDefinition::ExecutionStateSourceEnum;
UEnum* FNiagaraTypeDefinition::ScriptUsageEnum;
UEnum* FNiagaraTypeDefinition::ScriptContextEnum;

UEnum* FNiagaraTypeDefinition::ParameterScopeEnum;
UEnum* FNiagaraTypeDefinition::ParameterPanelCategoryEnum;

UEnum* FNiagaraTypeDefinition::FunctionDebugStateEnum;

FNiagaraTypeDefinition FNiagaraTypeDefinition::ParameterMapDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::IDDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::RandInfoDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::NumericDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::FloatDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::BoolDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::IntDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::Matrix4Def;
FNiagaraTypeDefinition FNiagaraTypeDefinition::Vec4Def;
FNiagaraTypeDefinition FNiagaraTypeDefinition::Vec3Def;
FNiagaraTypeDefinition FNiagaraTypeDefinition::Vec2Def;
FNiagaraTypeDefinition FNiagaraTypeDefinition::ColorDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::PositionDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::QuatDef;

FNiagaraTypeDefinition FNiagaraTypeDefinition::HalfDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::HalfVec2Def;
FNiagaraTypeDefinition FNiagaraTypeDefinition::HalfVec3Def;
FNiagaraTypeDefinition FNiagaraTypeDefinition::HalfVec4Def;

FNiagaraTypeDefinition FNiagaraTypeDefinition::UObjectDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::UMaterialDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::UTextureDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::UTextureRenderTargetDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::UStaticMeshDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::USimCacheClassDef;
FNiagaraTypeDefinition FNiagaraTypeDefinition::WildcardDef;

TSet<UScriptStruct*> FNiagaraTypeDefinition::NumericStructs;
TArray<FNiagaraTypeDefinition> FNiagaraTypeDefinition::OrderedNumericTypes;

TSet<UScriptStruct*> FNiagaraTypeDefinition::ScalarStructs;

TSet<UStruct*> FNiagaraTypeDefinition::FloatStructs;
TSet<UStruct*> FNiagaraTypeDefinition::IntStructs;
TSet<UStruct*> FNiagaraTypeDefinition::BoolStructs;

FNiagaraTypeDefinition FNiagaraTypeDefinition::CollisionEventDef;

bool FNiagaraTypeDefinition::IsDataInterface()const
{
	UStruct* ClassStruct = GetStruct();
	return ClassStruct ? ClassStruct->IsChildOf(UNiagaraDataInterface::StaticClass()) : false;
}

void FNiagaraTypeDefinition::Init()
{
	static auto* CoreUObjectPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/CoreUObject"));
	static auto* NiagaraPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/Niagara"));
	FNiagaraTypeDefinition::ParameterMapStruct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraParameterMap"));
	FNiagaraTypeDefinition::IDStruct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraID"));
	FNiagaraTypeDefinition::RandInfoStruct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraRandInfo"));
	FNiagaraTypeDefinition::NumericStruct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraNumeric"));
	FNiagaraTypeDefinition::FloatStruct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraFloat"));
	FNiagaraTypeDefinition::BoolStruct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraBool"));
	FNiagaraTypeDefinition::IntStruct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraInt32"));
	FNiagaraTypeDefinition::Matrix4Struct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraMatrix"));
	FNiagaraTypeDefinition::WildcardStruct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraWildcard"));
	FNiagaraTypeDefinition::PositionStruct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraPosition"));

	FNiagaraTypeDefinition::HalfStruct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraHalf"));
	FNiagaraTypeDefinition::HalfVec2Struct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraHalfVector2"));
	FNiagaraTypeDefinition::HalfVec3Struct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraHalfVector3"));
	FNiagaraTypeDefinition::HalfVec4Struct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraHalfVector4"));

	FNiagaraTypeDefinition::Vec2Struct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Vector2f"));
	FNiagaraTypeDefinition::Vec3Struct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Vector3f"));
	FNiagaraTypeDefinition::Vec4Struct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Vector4f"));
	FNiagaraTypeDefinition::ColorStruct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("LinearColor"));
	FNiagaraTypeDefinition::QuatStruct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Quat4f"));

	FNiagaraTypeDefinition::UObjectClass = UObject::StaticClass();
	FNiagaraTypeDefinition::UMaterialClass = UMaterialInterface::StaticClass();
	FNiagaraTypeDefinition::UTextureClass = UTexture::StaticClass();
	FNiagaraTypeDefinition::UTextureRenderTargetClass = UTextureRenderTarget::StaticClass();
	
	ParameterMapDef = FNiagaraTypeDefinition(ParameterMapStruct);
	IDDef = FNiagaraTypeDefinition(IDStruct);
	RandInfoDef = FNiagaraTypeDefinition(RandInfoStruct);
	NumericDef = FNiagaraTypeDefinition(NumericStruct);
	FloatDef = FNiagaraTypeDefinition(FloatStruct);
	BoolDef = FNiagaraTypeDefinition(BoolStruct);
	IntDef = FNiagaraTypeDefinition(IntStruct);
	Vec2Def = FNiagaraTypeDefinition(Vec2Struct);
	Vec3Def = FNiagaraTypeDefinition(Vec3Struct);
	Vec4Def = FNiagaraTypeDefinition(Vec4Struct);
	ColorDef = FNiagaraTypeDefinition(ColorStruct);
	PositionDef = FNiagaraTypeDefinition(PositionStruct);
	QuatDef = FNiagaraTypeDefinition(QuatStruct);
	Matrix4Def = FNiagaraTypeDefinition(Matrix4Struct);
	WildcardDef = FNiagaraTypeDefinition(WildcardStruct);
	
	HalfDef = FNiagaraTypeDefinition(HalfStruct);
	HalfVec2Def = FNiagaraTypeDefinition(HalfVec2Struct);
	HalfVec3Def = FNiagaraTypeDefinition(HalfVec3Struct);
	HalfVec4Def = FNiagaraTypeDefinition(HalfVec4Struct);

	UObjectDef = FNiagaraTypeDefinition(UObjectClass);
	UMaterialDef = FNiagaraTypeDefinition(UMaterialClass);
	UTextureDef = FNiagaraTypeDefinition(UTextureClass);
	UTextureRenderTargetDef = FNiagaraTypeDefinition(UTextureRenderTargetClass);
	UStaticMeshDef = FNiagaraTypeDefinition(UStaticMesh::StaticClass());
	USimCacheClassDef = FNiagaraTypeDefinition(UNiagaraSimCache::StaticClass());

	CollisionEventDef = FNiagaraTypeDefinition(FNiagaraCollisionEventPayload::StaticStruct());
	NumericStructs.Add(NumericStruct);
	NumericStructs.Add(FloatStruct);
	NumericStructs.Add(IntStruct);
	NumericStructs.Add(Vec2Struct);
	NumericStructs.Add(Vec3Struct);
	NumericStructs.Add(Vec4Struct);
	NumericStructs.Add(ColorStruct);
	NumericStructs.Add(PositionStruct);
	NumericStructs.Add(QuatStruct);
	//Make matrix a numeric type?

	FloatStructs.Add(FloatStruct);
	FloatStructs.Add(Vec2Struct);
	FloatStructs.Add(Vec3Struct);
	FloatStructs.Add(Vec4Struct);
	//FloatStructs.Add(Matrix4Struct)??
	FloatStructs.Add(ColorStruct);
	FloatStructs.Add(PositionStruct);
	FloatStructs.Add(QuatStruct);

	IntStructs.Add(IntStruct);

	BoolStructs.Add(BoolStruct);

	OrderedNumericTypes.Add(IntStruct);
	OrderedNumericTypes.Add(FloatStruct);
	OrderedNumericTypes.Add(Vec2Struct);
	OrderedNumericTypes.Add(Vec3Struct);
	OrderedNumericTypes.Add(Vec4Struct);
	OrderedNumericTypes.Add(ColorStruct);
	OrderedNumericTypes.Add(PositionStruct);
	OrderedNumericTypes.Add(QuatStruct);

	ScalarStructs.Add(BoolStruct);
	ScalarStructs.Add(IntStruct);
	ScalarStructs.Add(FloatStruct);
	ScalarStructs.Add(HalfStruct);

	CoordinateSpaceEnum = StaticEnum<ENiagaraCoordinateSpace>();
	OrientationAxisEnum = StaticEnum<ENiagaraOrientationAxis>();
	ExecutionStateEnum = StaticEnum<ENiagaraExecutionState>();
	ExecutionStateSourceEnum = StaticEnum<ENiagaraExecutionStateSource>();
	SimulationTargetEnum = StaticEnum<ENiagaraSimTarget>();
	ScriptUsageEnum = StaticEnum<ENiagaraCompileUsageStaticSwitch>();
	ScriptContextEnum = StaticEnum<ENiagaraScriptContextStaticSwitch>();

	FunctionDebugStateEnum = StaticEnum<ENiagaraFunctionDebugState>();
	
#if WITH_EDITOR
	RecreateUserDefinedTypeRegistry();
#endif
}

bool FNiagaraTypeDefinition::IsValidNumericInput(const FNiagaraTypeDefinition& TypeDef)
{
	if (NumericStructs.Contains(TypeDef.GetScriptStruct()))
	{
		return true;
	}
	return false;
}


bool FNiagaraTypeDefinition::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
#if WITH_EDITORONLY_DATA
	if (UnderlyingType != UT_None && ClassStructOrEnum == nullptr)
	{
		UE_LOGF(LogNiagara, Error, "Invalid data in niagara type definition, might be due to broken serialization or missing DI implementation");
		return false;
	}

	TStringBuilder<128> PathName;
	if (UEnum* TDEnum = GetEnum())
	{
		// Do we need to enumerate all the enum values and rebuild if that changes or are we ok with just knowing that there are the same  count of enum entries?
		// For now, am just going to be ok with the number of entries. The actual string values don't matter so much.
		PathName.Reset();
		TDEnum->GetPathName(nullptr, PathName);
		InVisitor->UpdateString(TEXT("\tEnumPath"), PathName);
		InVisitor->UpdateString(TEXT("\tEnumCppType"), TDEnum->CppType);
		InVisitor->UpdatePOD(TEXT("\t\tNumEnums"),TDEnum->NumEnums());
	}
	else if (UClass* TDClass = GetClass())
	{
		// For data interfaces, get the default object and the compile version so that we can properly update when code changes.
		check(IsInGameThread());
		UObject* TempObj = TDClass->GetDefaultObject(false);
		check(TempObj);

		PathName.Reset();
		TDClass->GetPathName(nullptr, PathName);
		InVisitor->UpdateString(TEXT("\tClassName"), PathName);

		UNiagaraDataInterface* TempDI = Cast< UNiagaraDataInterface>(TempObj);
		if (TempDI)
		{
			if (!TempDI->AppendCompileHash(InVisitor))
			{
				PathName.Reset();
				TempDI->GetPathName(nullptr, PathName);
				UE_LOGF(LogNiagara, Warning, "Unable to generate AppendCompileHash for DI %ls", PathName.ToString());
			}
		}
	}
	else if (UStruct* TDStruct = GetStruct())
	{
		PathName.Reset();
		TDStruct->GetPathName(nullptr, PathName);
		InVisitor->UpdateString(TEXT("\tStructName"), PathName);
		TStringBuilder<128> NameBuffer;
		// Structs are potentially changed, so we will want to register their actual types and variable names.
		for (TFieldIterator<FProperty> PropertyIt(TDStruct, EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::IncludeDeprecated); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			if (Property->HasMetaData(TEXT("SkipForCompileHash")))
			{
				continue;
			}
			NameBuffer.Reset();
			Property->GetFName().ToString(NameBuffer);
			InVisitor->UpdateString(TEXT("\t\tPropertyName"), NameBuffer);
			NameBuffer.Reset();
			Property->GetClass()->GetFName().ToString(NameBuffer);
			InVisitor->UpdateString(TEXT("\t\tPropertyClass"), NameBuffer);
		}
	}
	else
	{
		FString InvalidStr = TEXT("Invalid");
		InVisitor->UpdateString(TEXT("\tTDName"), InvalidStr);
	}

	InVisitor->UpdatePOD(TEXT("\tFlags"), Flags);

	return true;
#else
	return false;
#endif
}

FNiagaraTypeDefinition FNiagaraTypeDefinition::ToStaticDef() const
{
	FNiagaraTypeDefinition Def(*this);
	Def.Flags |= TF_Static;
	Def.InvalidateCachedIndex(Def.Flags != Flags);
	return Def;
}

FNiagaraTypeDefinition FNiagaraTypeDefinition::RemoveStaticDef() const
{
	FNiagaraTypeDefinition Def(*this);
	Def.Flags &= ~TF_Static;
	Def.InvalidateCachedIndex(Def.Flags != Flags);
	return Def;
}

#if WITH_EDITORONLY_DATA
bool FNiagaraTypeDefinition::IsInternalType() const
{
	if (const UScriptStruct* ScriptStruct = GetScriptStruct())
	{
		return ScriptStruct->GetBoolMetaData(TEXT("NiagaraInternalType"));
	}

	return false;
}
#endif

#if WITH_EDITOR
void FNiagaraTypeDefinition::RecreateUserDefinedTypeRegistry()
{
	static auto* CoreUObjectPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/CoreUObject"));
	static auto* NiagaraPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/Niagara"));

	FNiagaraTypeRegistry::ClearUserDefinedRegistry();

	const ENiagaraTypeRegistryFlags VarFlags = ENiagaraTypeRegistryFlags::AllowAnyVariable;
	const ENiagaraTypeRegistryFlags ParamFlags = VarFlags | ENiagaraTypeRegistryFlags::AllowParameter;
	const ENiagaraTypeRegistryFlags PayloadFlags = VarFlags | ENiagaraTypeRegistryFlags::AllowPayload;

	FNiagaraTypeRegistry::Register(CollisionEventDef, PayloadFlags);

	FNiagaraTypeRegistry::Register(ParameterMapDef, ParamFlags);
	FNiagaraTypeRegistry::Register(IDDef, ParamFlags);
	FNiagaraTypeRegistry::Register(RandInfoDef, ParamFlags);
	FNiagaraTypeRegistry::Register(NumericDef, ParamFlags);
	FNiagaraTypeRegistry::Register(FloatDef, ParamFlags);
	FNiagaraTypeRegistry::Register(HalfDef, ParamFlags);
	FNiagaraTypeRegistry::Register(IntDef, ParamFlags);
	FNiagaraTypeRegistry::Register(BoolDef, ParamFlags);
	FNiagaraTypeRegistry::Register(Vec2Def, ParamFlags);
	FNiagaraTypeRegistry::Register(Vec3Def, ParamFlags);
	FNiagaraTypeRegistry::Register(Vec4Def, ParamFlags);
	FNiagaraTypeRegistry::Register(ColorDef, ParamFlags);
	FNiagaraTypeRegistry::Register(PositionDef, ParamFlags);
	FNiagaraTypeRegistry::Register(QuatDef, ParamFlags);
	FNiagaraTypeRegistry::Register(Matrix4Def, ParamFlags);

	// Register static versions
	ENiagaraTypeRegistryFlags StaticParamFlags = ENiagaraTypeRegistryFlags::AllowSystemVariable | ENiagaraTypeRegistryFlags::AllowEmitterVariable | ENiagaraTypeRegistryFlags::AllowParticleVariable | ENiagaraTypeRegistryFlags::AllowParameter;
	FNiagaraTypeRegistry::Register(IntDef.ToStaticDef(), StaticParamFlags);
	FNiagaraTypeRegistry::Register(BoolDef.ToStaticDef(), StaticParamFlags);

	// @todo wildcard shouldn't be available for parameters etc., so just don't register here?
	//FNiagaraTypeRegistry::Register(WildcardDef, VarFlags);

	FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(ExecutionStateEnum), ParamFlags);
	FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(CoordinateSpaceEnum), ParamFlags);
	FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(OrientationAxisEnum), ParamFlags);
	FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(ExecutionStateSourceEnum), ParamFlags);

	UScriptStruct* SpawnInfoStruct = FindObjectChecked<UScriptStruct>(NiagaraPkg, TEXT("NiagaraSpawnInfo"));
	FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(SpawnInfoStruct), ParamFlags);

	FNiagaraTypeRegistry::Register(UObjectDef, ParamFlags);
	FNiagaraTypeRegistry::Register(UMaterialDef, ParamFlags);
	FNiagaraTypeRegistry::Register(UTextureDef, ParamFlags);
	FNiagaraTypeRegistry::Register(UTextureRenderTargetDef, ParamFlags);
	FNiagaraTypeRegistry::Register(UStaticMeshDef, ParamFlags);
	FNiagaraTypeRegistry::Register(USimCacheClassDef, ParamFlags);
	FNiagaraTypeRegistry::Register(StaticEnum<ENiagaraLegacyTrailWidthMode>(), ParamFlags);


	FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<EPhysicalSurface>()), ParamFlags);

	if (!IsRunningCommandlet())
	{
		TArray<FSoftObjectPath> DenyList;
		DenyList.Emplace(TEXT("/Niagara/Enums/ENiagaraCoordinateSpace.ENiagaraCoordinateSpace"));
		DenyList.Emplace(TEXT("/Niagara/Enums/ENiagaraOrientationAxis.ENiagaraOrientationAxis"));
		
		const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
		check(Settings);
		TArray<FSoftObjectPath> TotalStructAssets;
		for (FSoftObjectPath AssetRef : Settings->AdditionalParameterTypes)
		{
			TotalStructAssets.AddUnique(AssetRef);
		}

		for (FSoftObjectPath AssetRef : Settings->AdditionalPayloadTypes)
		{
			TotalStructAssets.AddUnique(AssetRef);
		}

		for (FSoftObjectPath AssetRef : TotalStructAssets)
		{
			FSoftObjectPath AssetRefPathNamePreResolve = AssetRef;

			if (DenyList.Contains(AssetRefPathNamePreResolve))
			{
				continue;
			}

			UObject* Obj = AssetRef.ResolveObject();
			if (Obj == nullptr)
			{
				Obj = AssetRef.TryLoad();
			}

			if (Obj != nullptr)
			{
				const FSoftObjectPath* ParamRefFound = Settings->AdditionalParameterTypes.FindByPredicate([&](const FSoftObjectPath& Ref) { return Ref.ToString() == AssetRef.ToString(); });
				const FSoftObjectPath* PayloadRefFound = Settings->AdditionalPayloadTypes.FindByPredicate([&](const FSoftObjectPath& Ref) { return Ref.ToString() == AssetRef.ToString(); });
				UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Obj);
				if (ScriptStruct != nullptr)
				{
					ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::IsUserDefined;
					if (ParamRefFound)
					{
						Flags |= ENiagaraTypeRegistryFlags::AllowParameter;
					}
					if (PayloadRefFound)
					{
						Flags |= ENiagaraTypeRegistryFlags::AllowPayload;
					}
					ScriptStruct = FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(ScriptStruct, ENiagaraStructConversion::UserFacing);
					FNiagaraTypeRegistry::Register(ScriptStruct, Flags);
				}

				UObjectRedirector* Redirector = FindObject<UObjectRedirector>(nullptr, *AssetRefPathNamePreResolve.ToString());
				bool bRedirectorFollowed = Redirector && (Redirector->DestinationObject == Obj);
				if (!bRedirectorFollowed && Obj->GetPathName() != AssetRefPathNamePreResolve.ToString())
				{
					UE_LOGF(LogNiagara, Warning, "Additional parameter/payload enum has moved from where it was in settings (this may cause errors at runtime): Was: \"%ls\" Now: \"%ls\"", *AssetRefPathNamePreResolve.ToString(), *Obj->GetPathName());
				}

			}
			else
			{
				UE_LOGF(LogNiagara, Warning, "Could not find additional parameter/payload type: %ls", *AssetRef.ToString());
			}
		}
		
		for (FSoftObjectPath AssetRef : Settings->AdditionalParameterEnums)
		{
			FSoftObjectPath AssetRefPathNamePreResolve = AssetRef;
			UObject* Obj = AssetRef.ResolveObject();

			if (DenyList.Contains(AssetRefPathNamePreResolve))
			{
				continue;
			}

			if (Obj == nullptr)
			{
				Obj = AssetRef.TryLoad();
			}

			if (Obj != nullptr)
			{
				UEnum* Enum = Cast<UEnum>(Obj);
				if (Enum != nullptr)
				{
					ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter | ENiagaraTypeRegistryFlags::IsUserDefined;
					
					FNiagaraTypeDefinition Def(Enum);
					FNiagaraTypeRegistry::Register(Def, Flags);
					FNiagaraTypeRegistry::Register(Def.ToStaticDef(), StaticParamFlags);
				}

				UObjectRedirector* Redirector = FindObject<UObjectRedirector>(nullptr, *AssetRefPathNamePreResolve.ToString());
				bool bRedirectorFollowed = Redirector && (Redirector->DestinationObject == Obj);
				if (!bRedirectorFollowed && Obj->GetPathName() != AssetRefPathNamePreResolve.ToString())
				{
					UE_LOGF(LogNiagara, Warning, "Additional parameter/payload enum has moved from where it was in settings (this may cause errors at runtime): Was: \"%ls\" Now: \"%ls\"", *AssetRefPathNamePreResolve.ToString(), *Obj->GetPathName());
				}
			}
			else
			{
				UE_LOGF(LogNiagara, Warning, "Could not find additional parameter/payload enum: %ls", *AssetRef.ToString());
			}
		}
	}
}
#endif

bool FNiagaraTypeDefinition::IsScalarDefinition(const FNiagaraTypeDefinition& Type)
{
	return ScalarStructs.Contains(Type.GetScriptStruct()) || (Type.GetScriptStruct() == IntStruct && Type.GetEnum() != nullptr);
}

bool FNiagaraTypeDefinition::TypesAreAssignable(const FNiagaraTypeDefinition& InputPinType, const FNiagaraTypeDefinition& OutputPinType, bool bAllowLossyLWCConversions)
{
	if (const UClass* AClass = InputPinType.GetClass())
	{
		if (const UClass* BClass = OutputPinType.GetClass())
		{
			return AClass == BClass;
		}
	}
	
	if (const UClass* BClass = OutputPinType.GetClass())
	{
		return false;
	}

	if (const UClass* AClass = InputPinType.GetClass())
	{
		return false;
	}

	// Enums can only be assigned from enums of the same type.
	if(InputPinType.IsEnum() && InputPinType.GetEnum() != OutputPinType.GetEnum())
	{
		return false;
	}

	//Enums can be assigned to enums of the same type or plain integers.
	if(OutputPinType.IsEnum() && InputPinType.GetEnum() != OutputPinType.GetEnum() && InputPinType != FNiagaraTypeDefinition::GetIntDef())
	{
		return false;
	}

	// Static can go to static or static can go to nonstatic, but nonstatic can't go to static
	if (((!InputPinType.IsStatic() && OutputPinType.IsStatic()) || (InputPinType.IsStatic() && OutputPinType.IsStatic())) && InputPinType.GetStruct() == OutputPinType.GetStruct() && InputPinType.GetEnum() == OutputPinType.GetEnum())
	{
		return true;
	}
	else if ((InputPinType.IsStatic() || OutputPinType.IsStatic()) && InputPinType.GetStruct() == OutputPinType.GetStruct() && InputPinType.GetEnum() == OutputPinType.GetEnum())
	{
		return false;
	}

	if (InputPinType.GetStruct() == OutputPinType.GetStruct())
	{
		return true;
	}

	bool bIsSupportedConversion = false;
	if (IsScalarDefinition(InputPinType) && IsScalarDefinition(OutputPinType))
	{
		bIsSupportedConversion = (InputPinType == IntDef && OutputPinType == FloatDef) || (OutputPinType == IntDef && InputPinType == FloatDef) || (InputPinType == IntDef && OutputPinType == BoolDef) || (OutputPinType == IntDef && InputPinType == BoolDef);
	}
	else
	{
		bIsSupportedConversion = (InputPinType == ColorDef && OutputPinType == Vec4Def) || (OutputPinType == ColorDef && InputPinType == Vec4Def)
			// we only allow direct assignments between vec <-> position if we are allowing lossy conversions
			|| (bAllowLossyLWCConversions && ((OutputPinType == PositionDef && InputPinType == Vec3Def) || (OutputPinType == Vec3Def && InputPinType == PositionDef)));
	}

	if (bIsSupportedConversion)
	{
		return true;
	}


	return	(InputPinType == NumericDef && NumericStructs.Contains(OutputPinType.GetScriptStruct())) ||
		(OutputPinType == NumericDef && NumericStructs.Contains(InputPinType.GetScriptStruct())) ||
		(InputPinType == NumericDef && (OutputPinType.GetStruct() == GetIntStruct()) && OutputPinType.GetEnum() != nullptr) ||
		(OutputPinType == NumericDef && (InputPinType.GetStruct() == GetIntStruct()) && InputPinType.GetEnum() != nullptr);
}

bool FNiagaraTypeDefinition::IsLossyConversion(const FNiagaraTypeDefinition& FromType, const FNiagaraTypeDefinition& ToType)
{
	// @todo this should be unidirectional, same with TypesAreAssignable
	return (FromType.IsSameBaseDefinition(IntDef) && ToType.IsSameBaseDefinition(FloatDef))
		|| (ToType.IsSameBaseDefinition(IntDef) && FromType.IsSameBaseDefinition(FloatDef))
		|| (FromType.IsSameBaseDefinition(BoolDef) && ToType.IsSameBaseDefinition(FloatDef))
		|| (ToType.IsSameBaseDefinition(BoolDef) && FromType.IsSameBaseDefinition(FloatDef))
		|| (FromType == PositionDef && ToType == Vec3Def)
		|| (FromType == Vec3Def && ToType == PositionDef);
}

FNiagaraTypeDefinition FNiagaraTypeDefinition::GetNumericOutputType(TConstArrayView<FNiagaraTypeDefinition> TypeDefinintions, ENiagaraNumericOutputTypeSelectionMode SelectionMode)
{
	checkf(SelectionMode != ENiagaraNumericOutputTypeSelectionMode::None, TEXT("Can not get numeric output type with selection mode none."));
	checkf(SelectionMode != ENiagaraNumericOutputTypeSelectionMode::Custom, TEXT("Can not get numeric output type with selection mode custom."));

	//This may need some work. Should work fine for now.
	if (SelectionMode == ENiagaraNumericOutputTypeSelectionMode::Scalar)
	{
		bool bHasFloats = false;
		bool bHasInts = false;
		bool bHasBools = false;
		for (const FNiagaraTypeDefinition& Type : TypeDefinintions)
		{
			bHasFloats |= FloatStructs.Contains(Type.GetStruct());
			bHasInts |= IntStructs.Contains(Type.GetStruct());
			bHasBools |= BoolStructs.Contains(Type.GetStruct());
		}
		//Not sure what to do if we have multiple different types here.
		//Possibly pick this up ealier and throw a compile error?
		if (bHasFloats) return FNiagaraTypeDefinition::GetFloatDef();
		if (bHasInts) return FNiagaraTypeDefinition::GetIntDef();
		if (bHasBools) return FNiagaraTypeDefinition::GetBoolDef();
	}
	// Always return the numeric type definition if it's included since this isn't a valid use case and we don't want to hide it.
	int32 NumericTypeDefinitionIndex = TypeDefinintions.IndexOfByKey(NumericDef);
	if (NumericTypeDefinitionIndex != INDEX_NONE)
	{
		// TODO: Warning here?
		return NumericDef;
	}

	TArray<FNiagaraTypeDefinition> SortedTypeDefinitions(TypeDefinintions);
	SortedTypeDefinitions.Sort([&](const FNiagaraTypeDefinition& TypeA, const FNiagaraTypeDefinition& TypeB)
		{
			int32 AIndex = OrderedNumericTypes.IndexOfByKey(TypeA);
			int32 BIndex = OrderedNumericTypes.IndexOfByKey(TypeB);
			return AIndex < BIndex;
		});

	if (SelectionMode == ENiagaraNumericOutputTypeSelectionMode::Largest)
	{
		return SortedTypeDefinitions.Last();
	}
	else // if (SelectionMode == ENiagaraNumericOutputTypeSelectionMode::Smallest)
	{
		return SortedTypeDefinitions[0];
	}
}

void FNiagaraTypeDefinition::InvalidateCachedIndex(bool bInvalidate)
{
	RegisteredTypeDefIndex = bInvalidate ? INDEX_NONE : RegisteredTypeDefIndex;
}

bool FNiagaraTypeDefinition::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	//We provide a custom import text only so that we can generate derived data (GeneratePathNameHash()).
	//We call back into the standard ImportText but prevent it from using the custom import and re-entering this function with the last arg.
	StaticStruct()->ImportText(Buffer, this, nullptr, PPF_None, ErrorText, StaticStruct()->GetName(), false);

	//Now we're imported we can generate our derived values.
	GeneratePathNameHash();

	return true;//Outer call to ImportText should exit.
}

bool FNiagaraTypeDefinition::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);

	UScriptStruct* StructToResetOnExit = nullptr;
	ON_SCOPE_EXIT
	{
		if (StructToResetOnExit)
		{
			ClassStructOrEnum = StructToResetOnExit;
			Flags &= ~TF_SerializedAsLWC;
		}
	};

	if (Ar.IsSaving())
	{
		if (UScriptStruct* StructDef = GetScriptStruct())
		{
			// if the struct is a transient package then we look for the source struct that was used
			if (StructDef->GetOutermost() == GetTransientPackage())
			{
				Flags |= TF_SerializedAsLWC;

				StructToResetOnExit = StructDef;
				ClassStructOrEnum = FNiagaraTypeHelper::GetLWCStruct(StructDef);
				check(ClassStructOrEnum);
				check(ClassStructOrEnum.GetPackage() != GetTransientPackage());
			}
		}
	}

	if (Ar.IsLoading() || Ar.IsSaving() || Ar.IsModifyingWeakAndStrongReferences())
	{
		UScriptStruct* Struct = FNiagaraTypeDefinition::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);
	}

	if (Ar.IsLoading())
	{
		InvalidateCachedIndex();

#if WITH_EDITORONLY_DATA
		if (Ar.CustomVer(FNiagaraCustomVersion::GUID) < FNiagaraCustomVersion::MemorySaving)
		{
			if (Enum_DEPRECATED != nullptr)
			{
				UnderlyingType = UT_Enum;
				ClassStructOrEnum = Enum_DEPRECATED;
				Enum_DEPRECATED = nullptr;
			}
			else if (Struct_DEPRECATED != nullptr)
			{
				UnderlyingType = Struct_DEPRECATED->IsA<UClass>() ? UT_Class : UT_Struct;
				ClassStructOrEnum = Struct_DEPRECATED;
				Struct_DEPRECATED = nullptr;
			}
			else
			{
				UnderlyingType = UT_None;
				ClassStructOrEnum = nullptr;
			}
		}
#endif

		if(!IsEnum())
		{
			if (UScriptStruct* StructDef = GetScriptStruct())
			{
				if (Flags & TF_SerializedAsLWC)
				{
					ClassStructOrEnum = FNiagaraTypeHelper::GetSWCStruct(StructDef);

					Flags &= ~TF_SerializedAsLWC;
				}
#if WITH_EDITORONLY_DATA
				else if((Flags & TF_AllowLWC_DEPRECATED) == 0)
				{	
					// If a basic type was serialized with the LWC flag, restore the correct struct
					ClassStructOrEnum = FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(static_cast<UScriptStruct*>(ClassStructOrEnum), ENiagaraStructConversion::UserFacing);
				}
#endif
			}
		}

		GeneratePathNameHash();
	}

	return true;
}

void FNiagaraTypeDefinition::GeneratePathNameHash()
{
	if (ClassStructOrEnum)
	{
		UnderlyingTypePathNameHash = GetTypeHash(ClassStructOrEnum->GetFName());
		const UObject* Parent = ClassStructOrEnum;
		while ((Parent = Parent->GetOuter()) != nullptr)
		{
			UnderlyingTypePathNameHash = HashCombineFast(UnderlyingTypePathNameHash, GetTypeHash(Parent->GetFName()));
		}
	}
	else
	{
		UnderlyingTypePathNameHash = GetTypeHash(NAME_None);
	}
}

//////////////////////////////////////////////////////////////////////////

FDelegateHandle INiagaraModule::SetOnProcessShaderCompilationQueue(FOnProcessQueue InOnProcessQueue)
{
	checkf(OnProcessQueue.IsBound() == false, TEXT("Shader processing queue delegate already set."));
	OnProcessQueue = InOnProcessQueue;
	return OnProcessQueue.GetHandle();
}

void INiagaraModule::ResetOnProcessShaderCompilationQueue(FDelegateHandle DelegateHandle)
{
	checkf(OnProcessQueue.GetHandle() == DelegateHandle, TEXT("Can only reset the process compilation queue delegate with the handle it was created with."));
	OnProcessQueue.Unbind();
}

void INiagaraModule::ProcessShaderCompilationQueue()
{
	checkf(OnProcessQueue.IsBound(), TEXT("Can not process shader queue.  Delegate was never set."));
	return OnProcessQueue.Execute();
}

const FNiagaraTypeDefinition& FNiagaraTypeDefinitionHandle::Resolve() const
{
	return FNiagaraTypeRegistry::Get().GetRegisteredType(RegisteredTypeIndex);

}

int32 FNiagaraTypeDefinitionHandle::Register(const FNiagaraTypeDefinition& TypeDef) const
{
	if (!TypeDef.IsValid())
	{
		return INDEX_NONE;
	}

	return FNiagaraTypeRegistry::Get().RegisterTypeInternal(TypeDef);
}

FArchive& operator<<(FArchive& Ar, FNiagaraTypeDefinitionHandle& Handle)
{
	UScriptStruct* TypeDefStruct = FNiagaraTypeDefinition::StaticStruct();

	if (Ar.IsSaving())
	{
		FNiagaraTypeDefinition TypeDef = *Handle;
		TypeDefStruct->SerializeItem(Ar, &TypeDef, nullptr);
	}
	else if (Ar.IsLoading())
	{
		FNiagaraTypeDefinition TypeDef;
		TypeDefStruct->SerializeItem(Ar, &TypeDef, nullptr);
		Handle = FNiagaraTypeDefinitionHandle(TypeDef);
	}

	return Ar;
}

bool FNiagaraVariableBase::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);
	const int32 NiagaraVersion = Ar.CustomVer(FNiagaraCustomVersion::GUID);

	if (!Ar.IsLoading() || NiagaraVersion >= FNiagaraCustomVersion::VariablesUseTypeDefRegistry)
	{
		Ar << Name;
		Ar << TypeDefHandle;
		return true;
	}

	return false;
}

#if WITH_EDITORONLY_DATA
void FNiagaraVariableBase::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.CustomVer(FNiagaraCustomVersion::GUID) < FNiagaraCustomVersion::VariablesUseTypeDefRegistry)
	{
		TypeDefHandle = FNiagaraTypeDefinitionHandle(TypeDef_DEPRECATED);
	}
}
#endif

bool FNiagaraVariableBase::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	return InVisitor->UpdateName(TEXT("NiagaraVariable.Name"), Name)
		&& TypeDefHandle->AppendCompileHash(InVisitor);
}

bool FNiagaraVariable::Serialize(FArchive& Ar)
{
	FNiagaraVariableBase::Serialize(Ar);

	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);
	const int32 NiagaraVersion = Ar.CustomVer(FNiagaraCustomVersion::GUID);

	if (!Ar.IsLoading() || NiagaraVersion >= FNiagaraCustomVersion::VariablesUseTypeDefRegistry)
	{
		Ar << VarData;
		return true;
	}

	// loading legacy data
	return false;
}

#if WITH_EDITORONLY_DATA
void FNiagaraVariable::PostSerialize(const FArchive& Ar)
{
	FNiagaraVariableBase::PostSerialize(Ar);
}
#endif

#if WITH_EDITOR

namespace NiagaraCBufferHelperImpl
{
	static FSHAHash GenerateHash(TConstArrayView<FNiagaraVariable> Variables)
	{
		FSHA1 HashState;
		FNiagaraCompileHashVisitor Visitor(HashState);
		for (const FNiagaraVariable& Var : Variables)
		{
			Var.AppendCompileHash(&Visitor);
		}
		return HashState.Finalize();
	}

	template<typename TStruct>
	static FSHAHash GetStructHash()
	{
		static TOptional<FSHAHash> StructHash;
		UE_CALL_ONCE([]()
		{
			StructHash.Emplace(NiagaraCBufferHelperImpl::GenerateHash(TStruct::GetVariables()));
		});

		return StructHash.GetValue();
	}
}

const TArray<FNiagaraVariable>& FNiagaraGlobalParameters::GetVariables()
{
	static const FName NAME_NiagaraStructPadding0 = "Engine.PaddingInt32_0";
	static const FName NAME_NiagaraStructPadding1 = "Engine.PaddingInt32_1";

	static const TArray<FNiagaraVariable> Variables =
	{
		SYS_PARAM_ENGINE_WORLD_DELTA_TIME,
		SYS_PARAM_ENGINE_DELTA_TIME,
		SYS_PARAM_ENGINE_INV_DELTA_TIME,
		SYS_PARAM_ENGINE_TIME,
		SYS_PARAM_ENGINE_REAL_TIME,
		SYS_PARAM_ENGINE_QUALITY_LEVEL,
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding0),
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding1),
	};

	return Variables;
}

FSHAHash FNiagaraGlobalParameters::GetStructHash()
{
	return NiagaraCBufferHelperImpl::GetStructHash<FNiagaraGlobalParameters>();
}

const TArray<FNiagaraVariable>& FNiagaraSystemParameters::GetVariables()
{
	static const FName NAME_NiagaraStructPadding0 = "Engine.System.PaddingInt32_0";
	static const FName NAME_NiagaraStructPadding1 = "Engine.System.PaddingInt32_1";

	static const TArray<FNiagaraVariable> Variables =
	{
		SYS_PARAM_ENGINE_TIME_SINCE_RENDERED,
		SYS_PARAM_ENGINE_LOD_DISTANCE,
		SYS_PARAM_ENGINE_LOD_DISTANCE_FRACTION,
		SYS_PARAM_ENGINE_SYSTEM_AGE,
		SYS_PARAM_ENGINE_EXECUTION_STATE,
		SYS_PARAM_ENGINE_SYSTEM_TICK_COUNT,
		SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS,
		SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS_ALIVE,
		SYS_PARAM_ENGINE_SYSTEM_SIGNIFICANCE_INDEX,
		SYS_PARAM_ENGINE_SYSTEM_RANDOM_SEED,
		SYS_PARAM_ENGINE_SYSTEM_CURRENT_TIME_STEP,
		SYS_PARAM_ENGINE_SYSTEM_NUM_TIME_STEPS,
		SYS_PARAM_ENGINE_SYSTEM_TIME_STEP_FRACTION,			
		SYS_PARAM_ENGINE_SYSTEM_NUM_PARTICLES,

		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding0),
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding1),
	};

	return Variables;
}

FSHAHash FNiagaraSystemParameters::GetStructHash()
{
	return NiagaraCBufferHelperImpl::GetStructHash<FNiagaraSystemParameters>();
}

const TArray<FNiagaraVariable>& FNiagaraOwnerParameters::GetVariables()
{
	static const FName NAME_NiagaraStructPadding0 = "Engine.Owner.PaddingInt32_0";
	static const FName NAME_NiagaraStructPadding1 = "Engine.Owner.PaddingInt32_1";
	static const FName NAME_NiagaraStructPadding2 = "Engine.Owner.PaddingInt32_2";
	static const FName NAME_NiagaraStructPadding3 = "Engine.Owner.PaddingInt32_3";
	static const FName NAME_NiagaraStructPadding4 = "Engine.Owner.PaddingInt32_4";
	static const FName NAME_NiagaraStructPadding5 = "Engine.Owner.PaddingInt32_5";
	static const FName NAME_NiagaraStructPadding6 = "Engine.Owner.PaddingInt32_6";

	static const TArray<FNiagaraVariable> Variables =
	{
		SYS_PARAM_ENGINE_LOCAL_TO_WORLD,
		SYS_PARAM_ENGINE_WORLD_TO_LOCAL,
		SYS_PARAM_ENGINE_LOCAL_TO_WORLD_TRANSPOSED,
		SYS_PARAM_ENGINE_WORLD_TO_LOCAL_TRANSPOSED,
		SYS_PARAM_ENGINE_LOCAL_TO_WORLD_NO_SCALE,
		SYS_PARAM_ENGINE_WORLD_TO_LOCAL_NO_SCALE,
		SYS_PARAM_ENGINE_ROTATION,
		SYS_PARAM_ENGINE_POSITION,
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding0),
		SYS_PARAM_ENGINE_VELOCITY,
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding1),
		SYS_PARAM_ENGINE_X_AXIS,
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding2),
		SYS_PARAM_ENGINE_Y_AXIS,
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding3),
		SYS_PARAM_ENGINE_Z_AXIS,
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding4),
		SYS_PARAM_ENGINE_SCALE,
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding5),
		SYS_PARAM_ENGINE_LWC_TILE,
	};

	return Variables;
}

FSHAHash FNiagaraOwnerParameters::GetStructHash()
{
	return NiagaraCBufferHelperImpl::GetStructHash<FNiagaraOwnerParameters>();
}

const TArray<FNiagaraVariable>& FNiagaraEmitterParameters::GetVariables()
{
	static const FName NAME_NiagaraStructPadding0 = "Engine.Emitter.PaddingInt32_0";
	static const FName NAME_NiagaraStructPadding1 = "Engine.Emitter.PaddingInt32_1";

	static const TArray<FNiagaraVariable> Variables =
	{
		SYS_PARAM_ENGINE_EMITTER_NUM_PARTICLES,
		SYS_PARAM_ENGINE_EMITTER_TOTAL_SPAWNED_PARTICLES,
		SYS_PARAM_ENGINE_EMITTER_SPAWN_COUNT_SCALE,
		SYS_PARAM_EMITTER_AGE,

		SYS_PARAM_EMITTER_RANDOM_SEED,
		SYS_PARAM_ENGINE_EMITTER_INSTANCE_SEED,
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding0),
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), NAME_NiagaraStructPadding1),
	};

	return Variables;
}

FSHAHash FNiagaraEmitterParameters::GetStructHash()
{
	return NiagaraCBufferHelperImpl::GetStructHash<FNiagaraEmitterParameters>();
}

#endif

void INiagaraModule::RegisterInternalAssetTagDefinitions()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// All internal tags have to be manually assigned a unique and stable guid so that names can be changed without affecting assigned assets.
	InternalAssetTagDefinitions.Add(&TemplateTagDefinition);
	InternalAssetTagDefinitions.Add(&LightweightTagDefinition);
	InternalAssetTagDefinitions.Add(&LearningContentTagDefinition);
	InternalAssetTagDefinitions.Add(&HiddenAssetTagDefinition);
	InternalAssetTagDefinitions.Add(&DeprecatedTagDefinition);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if NIAGARA_PERF_BASELINES

void INiagaraModule::GeneratePerfBaselines(TArray<UNiagaraEffectType*>& BaselinesToGenerate)
{	
	if (BaselinesToGenerate.Num() == 0)
	{
		return;
	}

	if (BaselineHandler.IsValid())
	{
		BaselineHandler->GenerateBaselines(BaselinesToGenerate);
	}
}

#endif

#undef LOCTEXT_NAMESPACE
