// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterBinding.h"
#include "NiagaraConstants.h"
#include "NiagaraDataInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraParameterBinding)

#if WITH_EDITORONLY_DATA
void FNiagaraParameterBinding::OnRenameEmitter(FStringView EmitterName)
{
	if (AliasedParameter.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString))
	{
		ResolvedParameter = AliasedParameter;
		ResolvedParameter.ReplaceRootNamespace(FNiagaraConstants::EmitterNamespaceString, EmitterName);
	}
	if (DefaultAliasedParameter.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString))
	{
		DefaultResolvedParameter = DefaultAliasedParameter;
		DefaultResolvedParameter.ReplaceRootNamespace(FNiagaraConstants::EmitterNamespaceString, EmitterName);
	}
}

void FNiagaraParameterBinding::OnRenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, FStringView EmitterName)
{
	if (AliasedParameter.GetName() == OldVariable.GetName() && AliasedParameter.GetType() == OldVariable.GetType())
	{
		if (!AliasedParameter.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString) || ResolvedParameter.IsInNameSpace(EmitterName))
		{
			AliasedParameter = NewVariable;
			ResolvedParameter = AliasedParameter;
			ResolvedParameter.ReplaceRootNamespace(FNiagaraConstants::EmitterNamespaceString, EmitterName);
		}
	}
}

void FNiagaraParameterBinding::OnRemoveVariable(const FNiagaraVariableBase& OldVariable, FStringView EmitterName)
{
	if (AliasedParameter.GetName() == OldVariable.GetName() && AliasedParameter.GetType() == OldVariable.GetType())
	{
		if (!AliasedParameter.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString) || ResolvedParameter.IsInNameSpace(EmitterName))
		{
			AliasedParameter = DefaultAliasedParameter;
			ResolvedParameter = DefaultResolvedParameter;
		}
	}
}

void FNiagaraParameterBinding::SetDefaultParameter(const FNiagaraVariable& Variable)
{
	DefaultResolvedParameter = Variable;
	DefaultAliasedParameter = Variable;

	SetToDefault();
}

void FNiagaraParameterBinding::SetToDefault()
{
	AliasedParameter = DefaultAliasedParameter;
	ResolvedParameter = DefaultResolvedParameter;

	if (HasDefaultValueEditorOnly() && DefaultAliasedParameter.IsDataAllocated())
	{
		SetDefaultValueEditorOnly(MakeArrayView(DefaultAliasedParameter.GetData(), DefaultAliasedParameter.GetAllocatedSizeInBytes()));
	}
}

bool FNiagaraParameterBinding::IsSetToDefault() const
{
	if (HasDefaultValueEditorOnly() && DefaultAliasedParameter.IsDataAllocated())
	{
		TConstArrayView<uint8> DefaultValue = GetDefaultValueEditorOnly();
		check(DefaultValue.Num() == DefaultAliasedParameter.GetAllocatedSizeInBytes());
		if ( FMemory::Memcmp(DefaultValue.GetData(), DefaultAliasedParameter.GetData(), DefaultAliasedParameter.GetAllocatedSizeInBytes()) != 0 )
		{
			return false;
		}
	}
	return AliasedParameter == DefaultAliasedParameter;
}

FString FNiagaraParameterBinding::ToString() const
{
	TStringBuilder<128> Builder;
	Builder.Append(TEXT("Type("));
	Builder.Append(ResolvedParameter.GetType().GetName());
	Builder.Append(TEXT(") Name("));
	ResolvedParameter.GetName().ToString(Builder);
	if (HasDefaultValueEditorOnly())
	{
		TConstArrayView<uint8> DefaultValue = GetDefaultValueEditorOnly();
		Builder.Append(TEXT(") Value("));
		Builder.Append(BytesToHex(DefaultValue.GetData(), DefaultValue.Num()));
	}
	Builder.Append(TEXT(")"));
	return Builder.ToString();
}

void FNiagaraParameterBinding::ForEachRenameEmitter(UObject* InObject, FStringView EmitterName)
{
	for (TFieldIterator<FStructProperty> PropIt(InObject->GetClass()); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = *PropIt)
		{
			if (StructProp->Struct && StructProp->Struct->IsChildOf(FNiagaraParameterBinding::StaticStruct()))
			{
				FNiagaraParameterBinding* ParameterBinding = StructProp->ContainerPtrToValuePtr<FNiagaraParameterBinding>(InObject);
				ParameterBinding->OnRenameEmitter(EmitterName);
			}
		}
	}
}

void FNiagaraParameterBinding::ForEachRenameVariable(UObject* InObject, const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, FStringView EmitterName)
{
	for (TFieldIterator<FStructProperty> PropIt(InObject->GetClass()); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = *PropIt)
		{
			if (StructProp->Struct && StructProp->Struct->IsChildOf(FNiagaraParameterBinding::StaticStruct()))
			{
				FNiagaraParameterBinding* ParameterBinding = StructProp->ContainerPtrToValuePtr<FNiagaraParameterBinding>(InObject);
				ParameterBinding->OnRenameVariable(OldVariable, NewVariable, EmitterName);
			}
		}
	}
}

void FNiagaraParameterBinding::ForEachRemoveVariable(UObject* InObject, const FNiagaraVariableBase& OldVariable, FStringView EmitterName)
{
	for (TFieldIterator<FStructProperty> PropIt(InObject->GetClass()); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = *PropIt)
		{
			if (StructProp->Struct && StructProp->Struct->IsChildOf(FNiagaraParameterBinding::StaticStruct()))
			{
				FNiagaraParameterBinding* ParameterBinding = StructProp->ContainerPtrToValuePtr<FNiagaraParameterBinding>(InObject);
				ParameterBinding->OnRemoveVariable(OldVariable, EmitterName);
			}
		}
	}
}

bool FNiagaraParameterBindingWithValue::HasDefaultValueEditorOnly() const
{
	return true;
}

TConstArrayView<uint8> FNiagaraParameterBindingWithValue::GetDefaultValueEditorOnly() const
{
	check(DefaultValue.Num() > 0);
	return DefaultValue;
}

void FNiagaraParameterBindingWithValue::SetDefaultValueEditorOnly(TConstArrayView<uint8> Memory)
{
	DefaultValue = Memory;
}

void FNiagaraParameterBindingWithValue::SetDefaultValueEditorOnly(const uint8* Memory)
{
	check(DefaultValue.Num() > 0);
	FMemory::Memcpy(DefaultValue.GetData(), Memory, DefaultValue.Num());
}
#endif
