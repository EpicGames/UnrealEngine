// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterCollection.h"

#include "Algo/RemoveIf.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/World.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "Misc/SecureHash.h"
#include "Modules/ModuleManager.h"
#include "NiagaraConstants.h"
#include "NiagaraDataInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraParameterCollection)

//////////////////////////////////////////////////////////////////////////

UNiagaraParameterCollectionInstance::UNiagaraParameterCollectionInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, ParametersNeedReset(false)
#endif
{
	ParameterStorage.SetOwner(this);
}

UNiagaraParameterCollectionInstance::~UNiagaraParameterCollectionInstance()
{
}

void UNiagaraParameterCollectionInstance::PostLoad()
{
	Super::PostLoad();

	ParameterStorage.PostLoad(this);

	// Before calling SyncWithCollection we must ensure the collections parameter store is PostLoaded
	// otherwise the parameters may not be sorted correctly.  If we are the default instance of the
	// Collection then we don't need to do this.
	if (Collection && (Collection->GetDefaultInstance() != this))
	{
		Collection->ConditionalPostLoad();
		InitDelegates(Collection);
	}

	//Ensure we're synced up with our collection. TODO: Do conditionally via a version number/guid?
	SyncWithCollection();
}

void UNiagaraParameterCollectionInstance::SetParent(UNiagaraParameterCollection* InParent)
{
	ResetDelegates(Collection);

	Collection = InParent;

	InitDelegates(Collection);

	SyncWithCollection();
}

bool UNiagaraParameterCollectionInstance::IsDefaultInstance()const
{
	return GetParent() && GetParent()->GetDefaultInstance() == this; 
}

bool UNiagaraParameterCollectionInstance::AddParameter(const FNiagaraVariable& Parameter)
{
	Modify();
	return ParameterStorage.AddParameter(Parameter);
}

bool UNiagaraParameterCollectionInstance::RemoveParameter(const FNiagaraVariable& Parameter)
{
	Modify();
	return ParameterStorage.RemoveParameter(Parameter); 
}

void UNiagaraParameterCollectionInstance::RenameParameter(const FNiagaraVariable& Parameter, FName NewName)
{
	Modify();
	ParameterStorage.RenameParameter(Parameter, NewName); 
}

void UNiagaraParameterCollectionInstance::Empty()
{
	Modify();
	ParameterStorage.Empty();
}

void UNiagaraParameterCollectionInstance::GetParameters(TArray<FNiagaraVariable>& OutParameters)
{
	ParameterStorage.GetParameters(OutParameters);
}

static void BuildCollectionValues(
	const UMaterialParameterCollection& OwnerCollection,
	const UMaterialParameterCollectionInstance& OwnerInstance,
	TArray<TPair<FName, float>>& ScalarParameters,
	TArray<TPair<FName, FLinearColor>>& VectorParameters)
{
	// initialize the source parameters from the material collection instance
	const int32 ScalarParameterCount = OwnerCollection.ScalarParameters.Num();
	ScalarParameters.AddUninitialized(ScalarParameterCount);
	for (int32 ScalarIt = 0; ScalarIt < ScalarParameterCount; ++ScalarIt)
	{
		ScalarParameters[ScalarIt].Key = OwnerCollection.ScalarParameters[ScalarIt].ParameterName;
		OwnerInstance.GetScalarParameterValue(OwnerCollection.ScalarParameters[ScalarIt], ScalarParameters[ScalarIt].Value);
	}

	const int32 VectorParameterCount = OwnerCollection.VectorParameters.Num();
	VectorParameters.AddUninitialized(VectorParameterCount);
	for (int32 VectorIt = 0; VectorIt < VectorParameterCount; ++VectorIt)
	{
		VectorParameters[VectorIt].Key = OwnerCollection.VectorParameters[VectorIt].ParameterName;
		OwnerInstance.GetVectorParameterValue(OwnerCollection.VectorParameters[VectorIt], VectorParameters[VectorIt].Value);
	}
}

void UNiagaraParameterCollectionInstance::Bind(UWorld* World)
{
	if (const UMaterialParameterCollection* MaterialCollection = Collection ? Collection->GetSourceCollection() : nullptr)
	{
#if WITH_EDITOR
		MaterialCollectionChangedDelegate = MaterialCollection->OnCollectionChanged().AddUObject(this, &UNiagaraParameterCollectionInstance::DirtyParameters);
#endif

		SourceMaterialCollectionInstance = World->GetParameterCollectionInstance(MaterialCollection);

		if (SourceMaterialCollectionInstance)
		{
			ScalarParamUpdateDelegate = SourceMaterialCollectionInstance->OnScalarParameterUpdated().AddLambda([this](UMaterialParameterCollectionInstance::ScalarParameterUpdate DirtyParameter)
			{
				FRWScopeLock WriteLock(DirtyParameterLock, SLT_Write);
				DirtyScalarParameters.Emplace(DirtyParameter);
			});

			VectorParamUpdateDelegate = SourceMaterialCollectionInstance->OnVectorParameterUpdated().AddLambda([this](UMaterialParameterCollectionInstance::VectorParameterUpdate DirtyParameter)
			{
				FRWScopeLock WriteLock(DirtyParameterLock, SLT_Write);
				DirtyVectorParameters.Emplace(DirtyParameter);
			});

			// initialize the source parameters from the material collection instance
			TArray<TPair<FName, float>> ScalarParameters;
			TArray<TPair<FName, FLinearColor>> VectorParameters;

			BuildCollectionValues(*MaterialCollection, *SourceMaterialCollectionInstance, ScalarParameters, VectorParameters);
			RefreshSourceParameters(World, ScalarParameters, VectorParameters);
		}
	}
}

bool UNiagaraParameterCollectionInstance::IsExternallyDriven(const FNiagaraVariable& Parameter) const
{
	if (Collection)
	{
		if (const UMaterialParameterCollection* MaterialCollection = Collection->GetSourceCollection())
		{
			FName FriendlyName = Collection->FriendlyNameFromParameterName(Parameter.GetName());

			return MaterialCollection->ScalarParameters.ContainsByPredicate([&](const FCollectionParameterBase& InParam) { return InParam.ParameterName == FriendlyName; })
				|| MaterialCollection->VectorParameters.ContainsByPredicate([&](const FCollectionParameterBase& InParam) { return InParam.ParameterName == FriendlyName; });
		}
	}

	return false;
}

void UNiagaraParameterCollectionInstance::Unbind(UWorld* World)
{
	if (const UMaterialParameterCollection* SourceCollection = Collection ? Collection->GetSourceCollection() : nullptr)
	{
#if WITH_EDITOR
		SourceCollection->OnCollectionChanged().Remove(MaterialCollectionChangedDelegate);
#endif

		if (UMaterialParameterCollectionInstance* SourceInstance = World->GetParameterCollectionInstance(SourceCollection))
		{
			SourceInstance->OnScalarParameterUpdated().Remove(ScalarParamUpdateDelegate);
			SourceInstance->OnVectorParameterUpdated().Remove(VectorParamUpdateDelegate);
		}
	}

#if WITH_EDITOR
	MaterialCollectionChangedDelegate.Reset();
#endif
	ScalarParamUpdateDelegate.Reset();
	VectorParamUpdateDelegate.Reset();
}

void UNiagaraParameterCollectionInstance::RefreshSourceParameters(
	UWorld* World,
	const TArray<TPair<FName, float>>& ScalarParameters,
	const TArray<TPair<FName, FLinearColor>>& VectorParameters)
{
	// if the NPC uses any MPC as sources, the make those bindings now
	if (const UMaterialParameterCollection* MaterialCollection = Collection ? Collection->GetSourceCollection() : nullptr)
	{
		// find the appropriate Instance
		if (!SourceMaterialCollectionInstance || SourceMaterialCollectionInstance->GetCollection() != MaterialCollection)
		{
			SourceMaterialCollectionInstance = World->GetParameterCollectionInstance(MaterialCollection);
		}

		if (SourceMaterialCollectionInstance)
		{
			FNameBuilder VariableName;
			VariableName << Collection->GetFullNamespaceName();
			const int32 NamespaceLength = VariableName.Len();

			if (ScalarParameters.Num())
			{
				const FNiagaraTypeDefinition& ScalarDef = FNiagaraTypeDefinition::GetFloatDef();

				for (const TPair<FName, float>& ScalarParameter : ScalarParameters)
				{
					VariableName.RemoveSuffix(VariableName.Len() - NamespaceLength);
					VariableName << ScalarParameter.Key;
					const FName VariableFName = *VariableName;

					ParameterStorage.SetParameterValue(ScalarParameter.Value, FNiagaraVariableBase(ScalarDef, VariableFName));
				}
			}

			if (VectorParameters.Num())
			{
				const FNiagaraTypeDefinition& ColorDef = FNiagaraTypeDefinition::GetColorDef();

				for (const TPair<FName, FLinearColor>& VectorParameter : VectorParameters)
				{
					VariableName.RemoveSuffix(VariableName.Len() - NamespaceLength);
					VariableName << VectorParameter.Key;
					const FName VariableFName = *VariableName;

					ParameterStorage.SetParameterValue(VectorParameter.Value, FNiagaraVariableBase(ColorDef, VariableFName));
				}
			}
		}
	}
}

void UNiagaraParameterCollectionInstance::InitDelegates(UNiagaraParameterCollection* ParentCollection)
{
#if WITH_EDITOR
	if (ParentCollection)
	{
		ParentCollectionChangedDelegate = ParentCollection->OnCollectionChangedDelegate.AddUObject(this, &UNiagaraParameterCollectionInstance::SyncWithCollection);
		MaterialParametersChangedHandle = ParentCollection->OnMaterialParametersChanged.AddUObject(this, &UNiagaraParameterCollectionInstance::SyncWithCollectionParameters);
		if (!IsDefaultInstance())
		{
			ParameterValuesEditedHandle = ParentCollection->OnParameterValuesEdited.AddUObject(this, &UNiagaraParameterCollectionInstance::SyncWithCollectionParameters);
		}
	}
#endif

}

void UNiagaraParameterCollectionInstance::ResetDelegates(UNiagaraParameterCollection* ParentCollection)
{
#if WITH_EDITOR
	if (ParentCollection)
	{
		if (ParentCollectionChangedDelegate.IsValid())
		{
			ParentCollection->OnCollectionChangedDelegate.Remove(ParentCollectionChangedDelegate);
		}

		if (MaterialParametersChangedHandle.IsValid())
		{
			ParentCollection->OnMaterialParametersChanged.Remove(MaterialParametersChangedHandle);
		}

		if (ParameterValuesEditedHandle.IsValid())
		{
			ParentCollection->OnParameterValuesEdited.Remove(ParameterValuesEditedHandle);
		}
	}
#endif
}

void UNiagaraParameterCollectionInstance::Tick(UWorld* World)
{
	{
		FRWScopeLock WriteLock(DirtyParameterLock, SLT_Write);

#if WITH_EDITOR
		if (ParametersNeedReset)
		{
			if (const UMaterialParameterCollection* SourceCollection = Collection ? Collection->GetSourceCollection() : nullptr)
			{
				if (UMaterialParameterCollectionInstance* SourceInstance = World->GetParameterCollectionInstance(SourceCollection))
				{
					// initialize the source parameters from the material collection instance
					TArray<TPair<FName, float>> ScalarParameters;
					TArray<TPair<FName, FLinearColor>> VectorParameters;

					BuildCollectionValues(*SourceCollection, *SourceInstance, ScalarParameters, VectorParameters);
					RefreshSourceParameters(World, ScalarParameters, VectorParameters);
				}
			}

			ParametersNeedReset = false;
		}
		else
#endif
		if (DirtyScalarParameters.Num() || DirtyVectorParameters.Num())
		{
			RefreshSourceParameters(World, DirtyScalarParameters, DirtyVectorParameters);
			DirtyScalarParameters.Empty();
			DirtyVectorParameters.Empty();
		}
	}

	//Push our parameter changes to any bound stores.
	ParameterStorage.Tick();
}

void UNiagaraParameterCollectionInstance::SyncWithCollection()
{
	FNiagaraParameterStore OldStore = ParameterStorage;
	ParameterStorage.Empty(Collection == nullptr);

	if (Collection == nullptr)
	{
		OverridenParameters.Empty();
		return;
	}

	const TArray<FNiagaraVariable>& CollectionParameters = Collection->GetParameters();

	for (const FNiagaraVariable& Param : CollectionParameters)
	{
		int32 Offset = OldStore.IndexOf(Param);
		if (Offset != INDEX_NONE && OverridesParameter(Param))
		{
			//If this parameter is in the old store and we're overriding it, use the existing value in the store.
			int32 ParameterStorageOffset = INDEX_NONE;
			ParameterStorage.AddParameter(Param, false, true, &ParameterStorageOffset);
			if (Param.IsDataInterface())
			{
				ParameterStorage.SetDataInterface(OldStore.GetDataInterface(Offset), Param);
			}
			else if (Param.IsUObject())
			{
				ParameterStorage.SetUObject(OldStore.GetUObject(Offset), Param);
			}
			else
			{
				ParameterStorage.SetParameterData(OldStore.GetParameterData(Offset, Param.GetType()), ParameterStorageOffset, Param.GetSizeInBytes());
			}
		}
		else
		{
			//If the parameter did not exist in the old store or we don't override this parameter, sync it up to the parent collection.
			FNiagaraParameterStore& DefaultStore = Collection->GetDefaultInstance()->GetParameterStore();
			Offset = DefaultStore.IndexOf(Param);

			// if the parameter was not in the parent collection than we can delete the orphaned parameter
			if (Offset != INDEX_NONE)
			{
				int32 ParameterStorageOffset = INDEX_NONE;
				ParameterStorage.AddParameter(Param, false, true, &ParameterStorageOffset);
				if (Param.IsDataInterface())
				{
					ParameterStorage.SetDataInterface(CastChecked<UNiagaraDataInterface>(StaticDuplicateObject(DefaultStore.GetDataInterface(Offset), this)), Param);
				}
				else if (Param.IsUObject())
				{
					ParameterStorage.SetUObject(DefaultStore.GetUObject(Offset), Param);
				}
				else
				{
					ParameterStorage.SetParameterData(DefaultStore.GetParameterData(Offset, Param.GetType()), ParameterStorageOffset, Param.GetSizeInBytes());
				}
			}
		}
	}

	// clean up OverridenParameters as well
	OverridenParameters.SetNum(Algo::RemoveIf(OverridenParameters, [&CollectionParameters](const FNiagaraVariable& OverrideParameter) -> bool
	{
		return !CollectionParameters.Contains(OverrideParameter);
	}));

	ParameterStorage.Rebind();
}

void UNiagaraParameterCollectionInstance::SyncWithCollectionParameters(TConstArrayView<FName> ParametersToRefresh)
{
	if (ParametersToRefresh.IsEmpty())
	{
		return;
	}

	for (FNiagaraVariable& Param : Collection->GetParameters())
	{
		if (!ParametersToRefresh.Contains(Param.GetName()))
		{
			continue;
		}

		// only worry about parameters we're not overriding
		if (OverridesParameter(Param))
		{
			continue;
		}

		const FNiagaraParameterStore& DefaultStore = Collection->GetDefaultInstance()->GetParameterStore();
		const int32 CollectionIndex = DefaultStore.IndexOf(Param);
		const int32 LocalIndex = ParameterStorage.IndexOf(Param);

		// if the parameter was not in the parent collection than we can delete the orphaned parameter
		if (CollectionIndex != INDEX_NONE && LocalIndex != INDEX_NONE)
		{
			if (Param.IsDataInterface())
			{
				ParameterStorage.SetDataInterface(CastChecked<UNiagaraDataInterface>(StaticDuplicateObject(DefaultStore.GetDataInterface(CollectionIndex), this)), Param);
			}
			else if (Param.IsUObject())
			{
				ParameterStorage.SetUObject(DefaultStore.GetUObject(CollectionIndex), Param);
			}
			else
			{
				const int32* LocalOffset = ParameterStorage.FindParameterOffset(Param);
				if (LocalOffset)
				{
					ParameterStorage.SetParameterData(DefaultStore.GetParameterData(CollectionIndex, Param.GetType()), *LocalOffset, Param.GetSizeInBytes());
				}
			}
		}
	}

#if WITH_EDITOR
	OnParameterValuesChanged.Broadcast(ParametersToRefresh);
#endif
}

bool UNiagaraParameterCollectionInstance::OverridesParameter(const FNiagaraVariable& Parameter)const
{ 
	return IsDefaultInstance() || OverridenParameters.Contains(Parameter); 
}

void UNiagaraParameterCollectionInstance::SetOverridesParameter(const FNiagaraVariable& Parameter, bool bOverrides)
{
	if (bOverrides)
	{
		OverridenParameters.AddUnique(Parameter);
	}
	else
	{
		OverridenParameters.Remove(Parameter);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraParameterCollectionInstance::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraParameterCollectionInstance, Collection))
	{
		SetParent(nullptr);
	}
}

void UNiagaraParameterCollectionInstance::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraParameterCollectionInstance, Collection))
	{
		SetParent(Collection);
	}
}
#endif
//Blueprint Accessors
bool UNiagaraParameterCollectionInstance::GetBoolParameter(const FString& InVariableName)
{
	return ParameterStorage.GetParameterValue<int32>(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), Collection->ParameterNameFromFriendlyString(InVariableName))) == FNiagaraBool::True;
}

float UNiagaraParameterCollectionInstance::GetFloatParameter(const FString& InVariableName)
{
	return ParameterStorage.GetParameterValue<float>(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), Collection->ParameterNameFromFriendlyString(InVariableName)));
}

int32 UNiagaraParameterCollectionInstance::GetIntParameter(const FString& InVariableName)
{
	return ParameterStorage.GetParameterValue<int32>(FNiagaraVariable(FNiagaraTypeDefinition::GetIntStruct(), Collection->ParameterNameFromFriendlyString(InVariableName)));
}

FVector2D UNiagaraParameterCollectionInstance::GetVector2DParameter(const FString& InVariableName)
{
	return FVector2D(ParameterStorage.GetParameterValue<FVector2f>(FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), Collection->ParameterNameFromFriendlyString(InVariableName))));
}

FVector UNiagaraParameterCollectionInstance::GetVectorParameter(const FString& InVariableName)
{
	return FVector(ParameterStorage.GetParameterValue<FVector3f>(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), Collection->ParameterNameFromFriendlyString(InVariableName))));
}

FVector4 UNiagaraParameterCollectionInstance::GetVector4Parameter(const FString& InVariableName)
{
	return FVector4(ParameterStorage.GetParameterValue<FVector4f>(FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), Collection->ParameterNameFromFriendlyString(InVariableName))));
}


FQuat UNiagaraParameterCollectionInstance::GetQuatParameter(const FString& InVariableName)
{
	return FQuat(ParameterStorage.GetParameterValue<FQuat4f>(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), Collection->ParameterNameFromFriendlyString(InVariableName))));
}

FLinearColor UNiagaraParameterCollectionInstance::GetColorParameter(const FString& InVariableName)
{
	return ParameterStorage.GetParameterValue<FLinearColor>(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), Collection->ParameterNameFromFriendlyString(InVariableName)));
}

#define NPC_SUPPORT_FUNCTION_LOGGING (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)

template<typename T>
static bool CheckConflictWithSourceMpc(FName ParameterName, const TCHAR* FunctionCall, const T& Value, const UNiagaraParameterCollection* Collection)
{
	if (const UMaterialParameterCollection* SourceCollection = Collection ? Collection->GetSourceCollection() : nullptr)
	{
		if (SourceCollection->GetParameterId(ParameterName).IsValid())
		{
#if NPC_SUPPORT_FUNCTION_LOGGING
			static bool LogWrittenOnce = false;

			if (!LogWrittenOnce)
			{
				UE_LOGF(LogNiagara, Warning, "Skipping attempt to %ls for parameter %ls of %ls because it is driven by MPC %ls",
					*ParameterName.ToString(), FunctionCall, *Collection->GetFullName(), *Collection->GetSourceCollection()->GetFullName());

				LogWrittenOnce = true;
			}
#endif

			return true;
		}
	}

	return false;
}

#if NPC_SUPPORT_FUNCTION_LOGGING
	#define NPC_BUILD_FUNCTION_STRING() (StringCast<TCHAR>(__FUNCTION__).Get())
#else
	#define NPC_BUILD_FUNCTION_STRING() (nullptr)
#endif

void UNiagaraParameterCollectionInstance::SetBoolParameter(const FString& InVariableName, bool InValue)
{
	const FName ParameterName = Collection->ParameterNameFromFriendlyString(InVariableName);
	if (!CheckConflictWithSourceMpc(ParameterName, NPC_BUILD_FUNCTION_STRING(), InValue, Collection))
	{
		ParameterStorage.SetParameterValue(InValue ? FNiagaraBool::True : FNiagaraBool::False, FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), ParameterName));
	}
}

void UNiagaraParameterCollectionInstance::SetFloatParameter(const FString& InVariableName, float InValue)
{
	const FName ParameterName = Collection->ParameterNameFromFriendlyString(InVariableName);
	if (!CheckConflictWithSourceMpc(ParameterName, NPC_BUILD_FUNCTION_STRING(), InValue, Collection))
	{
		ParameterStorage.SetParameterValue(InValue, FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), ParameterName));
	}
}

void UNiagaraParameterCollectionInstance::SetIntParameter(const FString& InVariableName, int32 InValue)
{
	const FName ParameterName = Collection->ParameterNameFromFriendlyString(InVariableName);
	if (!CheckConflictWithSourceMpc(ParameterName, NPC_BUILD_FUNCTION_STRING(), InValue, Collection))
	{
		ParameterStorage.SetParameterValue(InValue, FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), ParameterName));
	}
}

void UNiagaraParameterCollectionInstance::SetVector2DParameter(const FString& InVariableName, FVector2D InValue)
{
	const FName ParameterName = Collection->ParameterNameFromFriendlyString(InVariableName);
	if (!CheckConflictWithSourceMpc(ParameterName, NPC_BUILD_FUNCTION_STRING(), InValue, Collection))
	{
		ParameterStorage.SetParameterValue(FVector2f(InValue), FNiagaraVariable(FNiagaraTypeDefinition::GetVec2Def(), ParameterName));
	}
}

void UNiagaraParameterCollectionInstance::SetVectorParameter(const FString& InVariableName, FVector InValue)
{
	const FName ParameterName = Collection->ParameterNameFromFriendlyString(InVariableName);
	if (!CheckConflictWithSourceMpc(ParameterName, NPC_BUILD_FUNCTION_STRING(), InValue, Collection))
	{
		ParameterStorage.SetParameterValue(FVector3f(InValue), FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), ParameterName));
	}
}

void UNiagaraParameterCollectionInstance::SetVector4Parameter(const FString& InVariableName, const FVector4& InValue)
{
	const FName ParameterName = Collection->ParameterNameFromFriendlyString(InVariableName);
	if (!CheckConflictWithSourceMpc(ParameterName, NPC_BUILD_FUNCTION_STRING(), InValue, Collection))
	{
		ParameterStorage.SetParameterValue(FVector4f(InValue), FNiagaraVariable(FNiagaraTypeDefinition::GetVec4Def(), ParameterName));
	}
}

void UNiagaraParameterCollectionInstance::SetColorParameter(const FString& InVariableName, FLinearColor InValue)
{
	const FName ParameterName = Collection->ParameterNameFromFriendlyString(InVariableName);
	if (!CheckConflictWithSourceMpc(ParameterName, NPC_BUILD_FUNCTION_STRING(), InValue, Collection))
	{
		ParameterStorage.SetParameterValue(InValue, FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), ParameterName));
	}
}

void UNiagaraParameterCollectionInstance::SetQuatParameter(const FString& InVariableName, const FQuat& InValue)
{
	const FName ParameterName = Collection->ParameterNameFromFriendlyString(InVariableName);
	if (!CheckConflictWithSourceMpc(ParameterName, NPC_BUILD_FUNCTION_STRING(), InValue, Collection))
	{
		ParameterStorage.SetParameterValue(FQuat4f(InValue), FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), ParameterName));
	}
}

//////////////////////////////////////////////////////////////////////////

UNiagaraParameterCollection::UNiagaraParameterCollection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Namespace = GetFName();
	BuildFullNamespace();

	DefaultInstance = ObjectInitializer.CreateDefaultSubobject<UNiagaraParameterCollectionInstance>(this, TEXT("Default Instance"));
	DefaultInstance->SetParent(this);
}

#if WITH_EDITORONLY_DATA
void UNiagaraParameterCollection::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

#if WITH_EDITOR
	if (PropertyAboutToChange
		&& PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraParameterCollection, SourceMaterialCollection)
		&& SourceMaterialCollection
		&& OnMaterialCollectionChanged.IsValid())
	{
		SourceMaterialCollection->OnCollectionChanged().Remove(OnMaterialCollectionChanged);
		OnMaterialCollectionChanged.Reset();
	}
#endif // WITH_EDITOR
}

void UNiagaraParameterCollection::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	MakeNamespaceNameUnique();

	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraParameterCollection, SourceMaterialCollection))
		{
#if WITH_EDITOR
			if (SourceMaterialCollection)
			{
				OnMaterialCollectionChanged = SourceMaterialCollection->OnCollectionChanged().AddUObject(this, &UNiagaraParameterCollection::RefreshSourceMaterialCollection);
			}
#endif
			RefreshSourceMaterialCollection();
			OnCollectionChangedDelegate.Broadcast();
		}
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraParameterCollection, Namespace))
		{
			NamespaceChanged();
			OnCollectionChangedDelegate.Broadcast();
		}
	}
}

#if WITH_EDITOR
void UNiagaraParameterCollection::PreEditUndo()
{
	Super::PreEditUndo();

	// managing the editor delegates in Pre/Post EditChange breaks down with undo, so if we hit an undo we clean up the delegate
	if (SourceMaterialCollection && OnMaterialCollectionChanged.IsValid())
	{
		SourceMaterialCollection->OnCollectionChanged().Remove(OnMaterialCollectionChanged);
		OnMaterialCollectionChanged.Reset();
	}
}

void UNiagaraParameterCollection::PostEditUndo()
{
	Super::PostEditUndo();

	// managing the editor delegates in Pre/Post EditChange breaks down with undo, so if we hit an undo, verify that they are setup
	if (SourceMaterialCollection && !OnMaterialCollectionChanged.IsValid())
	{
		OnMaterialCollectionChanged = SourceMaterialCollection->OnCollectionChanged().AddUObject(this, &UNiagaraParameterCollection::RefreshSourceMaterialCollection);
		RefreshSourceMaterialCollection();
		OnCollectionChangedDelegate.Broadcast();
	}
}
#endif


void UNiagaraParameterCollection::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	MakeNamespaceNameUnique();
}

#endif

int32 UNiagaraParameterCollection::IndexOfParameter(const FNiagaraVariable& Var)
{
	return Parameters.IndexOfByPredicate([&](const FNiagaraVariable& Other)
	{
		return Var.IsEquivalent(Other);
	});
}

int32 UNiagaraParameterCollection::AddParameter(const FNiagaraVariable& Parameter)
{
	// go through the existing elements to see if we already have an entry for the supplied parameter
	int32 Idx = IndexOfParameter(Parameter);
	if (Idx == INDEX_NONE)
	{
		Modify();

		Idx = Parameters.Add(Parameter);
		DefaultInstance->AddParameter(Parameter);
	}

	return Idx;
}

int32 UNiagaraParameterCollection::AddParameter(FName Name, FNiagaraTypeDefinition Type)
{
	return AddParameter(FNiagaraVariable(Type, Name));
}

//void UNiagaraParameterCollection::RemoveParameter(int32 ParamIdx)
void UNiagaraParameterCollection::RemoveParameter(const FNiagaraVariable& Parameter)
{
	Modify();
	CompileId = FGuid::NewGuid();  // Any scripts depending on this parameter name will likely need to be changed.
	DefaultInstance->RemoveParameter(Parameter);
	Parameters.Remove(Parameter);
}

void UNiagaraParameterCollection::RenameParameter(FNiagaraVariable& Parameter, FName NewName)
{
	Modify();
	CompileId = FGuid::NewGuid(); // Any scripts depending on this parameter name will likely need to be changed.
	int32 ParamIdx = Parameters.IndexOfByKey(Parameter);
	check(ParamIdx != INDEX_NONE);

	Parameters[ParamIdx].SetName(NewName);
	DefaultInstance->RenameParameter(Parameter, NewName);
}

void UNiagaraParameterCollection::BuildFullNamespace()
{
	FNameBuilder FullNamespaceBuilder;
	FullNamespaceBuilder << PARAM_MAP_NPC_STR;
	FullNamespaceBuilder << Namespace;
	FullNamespaceBuilder << TEXT(".");

	FullNamespace = FName(FullNamespaceBuilder);
}

FNiagaraCompileHash UNiagaraParameterCollection::GetCompileHash() const
{
	// TODO - Implement an actual hash for parameter collections instead of just hashing a change id.
	FSHA1 CompileHash;
	CompileHash.Update((const uint8*)&CompileId, sizeof(FGuid));
	CompileHash.Final();

	TArray<uint8> DataHash;
	DataHash.AddUninitialized(FSHA1::DigestSize);
	CompileHash.GetHash(DataHash.GetData());

	return FNiagaraCompileHash(DataHash);
}

void UNiagaraParameterCollection::RefreshCompileId()
{
	CompileId = FGuid::NewGuid();
}

FNiagaraVariable UNiagaraParameterCollection::CollectionParameterFromFriendlyParameter(const FNiagaraVariable& FriendlyParameter)const
{
	return FNiagaraVariable(FriendlyParameter.GetType(), ConditionalAddFullNamespace(FriendlyParameter.GetName()));
}

FNiagaraVariable UNiagaraParameterCollection::FriendlyParameterFromCollectionParameter(const FNiagaraVariable& CollectionParameter)const
{
	return FNiagaraVariable(CollectionParameter.GetType(), ConditionalAddFullNamespace(CollectionParameter.GetName()));
}

FString UNiagaraParameterCollection::GetFullNamespace() const
{
	FNameBuilder FullNamespaceBuilder(FullNamespace);
	return FullNamespaceBuilder.ToString();
}

// deprecated
FString UNiagaraParameterCollection::FriendlyNameFromParameterName(FString ParameterString) const
{
	FNameBuilder ParameterNameBuilder(*ParameterString);
	return FriendlyNameFromParameterName(FName(ParameterNameBuilder)).ToString();
}

FName UNiagaraParameterCollection::FriendlyNameFromParameterName(FName ParameterName) const
{
	FNameBuilder ParameterNameBuilder(ParameterName);
	FStringView ParameterNameView = ParameterNameBuilder.ToView();

	FNameBuilder FullNamespaceBuilder(FullNamespace);

	if (ParameterNameView.StartsWith(FullNamespaceBuilder))
	{
		ParameterNameView.RemovePrefix(FullNamespaceBuilder.Len());
		return FName(ParameterNameView);
	}

	return ParameterName;
}

FName UNiagaraParameterCollection::ConditionalAddFullNamespace(FName ParameterName) const
{
	FNameBuilder ParameterNameBuilder(ParameterName);
	FNameBuilder FullNamespaceBuilder(FullNamespace);

	if (ParameterNameBuilder.ToView().StartsWith(FullNamespaceBuilder))
	{
		return ParameterName;
	}

	FNameBuilder ResultName;
	ResultName << FullNamespace;
	ResultName << ParameterNameBuilder;

	return FName(ResultName);
}

// deprecated
FString UNiagaraParameterCollection::ParameterNameFromFriendlyName(const FString& FriendlyString) const
{
	return ParameterNameFromFriendlyString(FriendlyString).ToString();
}

FName UNiagaraParameterCollection::ParameterNameFromFriendlyString(const FString& FriendlyString) const
{
	FName FriendlyName(*FriendlyString);

	return ConditionalAddFullNamespace(FriendlyName);
}

#if WITH_EDITORONLY_DATA
void UNiagaraParameterCollection::MakeNamespaceNameUnique()
{
 	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
 	TArray<FAssetData> CollectionAssets;
 	AssetRegistryModule.Get().GetAssetsByClass(UNiagaraParameterCollection::StaticClass()->GetClassPathName(), CollectionAssets);
	TArray<FName> ExistingNames;
 	for (FAssetData& CollectionAsset : CollectionAssets)
 	{
		// skip ourselves - note that ColelctionAsset.GetFullName() uses a fully qualified class name as a prefix in contrast to GetFullName()
		if (CollectionAsset.GetObjectPathString() != GetPathName())
		{
			ExistingNames.Add(CollectionAsset.GetTagValueRef<FName>(GET_MEMBER_NAME_CHECKED(UNiagaraParameterCollection, Namespace)));
		}
	}

	if (ExistingNames.Contains(Namespace))
	{
		// first try to use the name of the newly created asset as the namespace
		FName ObjectName = GetFName();
		if (!ExistingNames.Contains(ObjectName))
		{
			Namespace = ObjectName;
		}
		else
		{
			FString CandidateNameString = ObjectName.ToString();
			FString BaseNameString = CandidateNameString;
			if (CandidateNameString.Len() >= 3 && CandidateNameString.Right(3).IsNumeric())
			{
				BaseNameString = CandidateNameString.Left(CandidateNameString.Len() - 3);
			}

			FName UniqueName = FName(*BaseNameString);
			int32 NameIndex = 1;
			while (ExistingNames.Contains(UniqueName))
			{
				UniqueName = FName(*FString::Printf(TEXT("%s%03i"), *BaseNameString, NameIndex));
				NameIndex++;
			}

			UE_LOGF(LogNiagara, Warning, "Parameter collection namespace conflict found. \"%ls\" is already in use!", *Namespace.ToString());
			Namespace = UniqueName;
		}
		BuildFullNamespace();
		NamespaceChanged();
	}
}

void UNiagaraParameterCollection::RefreshSourceMaterialCollection()
{
#if WITH_EDITOR
	TSet<FName> PreviouslyOwnedParameters = MoveTemp(ParametersOwnedBySourceCollection);
	ParametersOwnedBySourceCollection.Reset();

	bool bParametersAddedOrRemoved = false;
	TArray<FName> UpdatedParameters;
	auto AddOrUpdateParameter = [this, &bParametersAddedOrRemoved, &UpdatedParameters](const FNiagaraVariable& Parameter)
#else
	auto AddOrUpdateParameter = [this](const FNiagaraVariable& Parameter)
#endif
	{
		// go through the existing elements to see if we already have an entry for the supplied parameter
		int32 Idx = IndexOfParameter(Parameter);
		if (Idx == INDEX_NONE)
		{
			Modify();

			Idx = Parameters.Add(Parameter);
			DefaultInstance->AddParameter(Parameter);

#if WITH_EDITOR
			bParametersAddedOrRemoved = true;
#endif
			return;
		}
		else if (Parameter.IsDataAllocated())
		{
			if (!Parameters[Idx].HoldsSameData(Parameter))
			{
				if (const uint8* ParameterData = Parameter.GetData())
				{
					Modify();

					Parameters[Idx].SetData(ParameterData);
					DefaultInstance->GetParameterStore().SetParameterData(ParameterData, Parameter);

#if WITH_EDITOR
					UpdatedParameters.Add(Parameter.GetName());
#endif
				}
			}
		}
	};

	if (SourceMaterialCollection)
	{
		TArray<FName> ScalarParameterNames;
		TArray<FName> VectorParameterNames;

		SourceMaterialCollection->GetParameterNames(ScalarParameterNames, false /* bVectorParameters */);
		SourceMaterialCollection->GetParameterNames(VectorParameterNames, true /* bVectorParameters */);

		const FNiagaraTypeDefinition& ScalarDef = FNiagaraTypeDefinition::GetFloatDef();
		const FNiagaraTypeDefinition& ColorDef = FNiagaraTypeDefinition::GetColorDef();

		for (const FName& ScalarParameterName : ScalarParameterNames)
		{
			FNiagaraVariable ScalarParameter(ScalarDef, ConditionalAddFullNamespace(ScalarParameterName));
			ScalarParameter.SetValue(SourceMaterialCollection->GetScalarParameterByName(ScalarParameterName)->DefaultValue);

			AddOrUpdateParameter(ScalarParameter);
		}

		for (const FName& VectorParameterName : VectorParameterNames)
		{
			FNiagaraVariable VectorParameter(ColorDef, ConditionalAddFullNamespace(VectorParameterName));
			VectorParameter.SetValue(SourceMaterialCollection->GetVectorParameterByName(VectorParameterName)->DefaultValue);

			AddOrUpdateParameter(VectorParameter);
		}

#if WITH_EDITOR
		ParametersOwnedBySourceCollection.Append(ScalarParameterNames);
		ParametersOwnedBySourceCollection.Append(VectorParameterNames);
#endif
	}

#if WITH_EDITOR
	for (FName PreviousParameter : PreviouslyOwnedParameters)
	{
		if (!ParametersOwnedBySourceCollection.Contains(PreviousParameter))
		{
			const FName FullParameterName = ConditionalAddFullNamespace(PreviousParameter);

			auto FindVariableByName = [FullParameterName](const FNiagaraVariable& Var) -> bool
			{
				return Var.GetName() == FullParameterName;
			};

			if (const FNiagaraVariable* FoundParameter = Parameters.FindByPredicate(FindVariableByName))
			{
				FNiagaraVariable ParameterToRemove = *FoundParameter;
				RemoveParameter(ParameterToRemove);
				bParametersAddedOrRemoved = true;
			}
		}
	}

	if (bParametersAddedOrRemoved)
	{
		OnCollectionChangedDelegate.Broadcast();
	}

	if (!UpdatedParameters.IsEmpty())
	{
		OnMaterialParametersChanged.Broadcast(MakeConstArrayView(UpdatedParameters));
	}
#endif
}

void UNiagaraParameterCollection::NamespaceChanged()
{
	TArray<FNiagaraVariable> ParametersToRemove;
	TSet<FName> ExistingVariableNames;

	for (FNiagaraVariable Parameter : Parameters)
	{
		FNameBuilder ParameterName(Parameter.GetName());
		FStringView ParameterNameView = ParameterName.ToView();

		// look for the form NPC.<OldNamespace>.<Name> and replace it with NPC.<Namespace>.<Name>
		if (ParameterNameView.StartsWith(TEXT("NPC.")))
		{
			int32 LastSeparatorIndex = INDEX_NONE;
			if (ParameterNameView.FindLastChar('.', LastSeparatorIndex))
			{
				FStringView VariableNameView = ParameterNameView.Mid(LastSeparatorIndex + 1);
				if (!VariableNameView.IsEmpty())
				{
					FName VariableName(VariableNameView);

					bool DuplicateFound = false;
					ExistingVariableNames.Add(VariableName, &DuplicateFound);
					if (DuplicateFound)
					{
						// mark the parameter as requiring removal
						ParametersToRemove.Add(Parameter);
					}

					FNameBuilder NewParameterName;
					FullNamespace.AppendString(NewParameterName);
					VariableName.AppendString(NewParameterName);
					RenameParameter(Parameter, FName(NewParameterName));
				}
			}
		}
	}

	for (const FNiagaraVariable& Parameter : ParametersToRemove)
	{
		RemoveParameter(Parameter);
	}
}

#endif

void UNiagaraParameterCollection::PostLoad()
{
	Super::PostLoad();

	// after serialization we need to build the transient full namespace
	BuildFullNamespace();

	DefaultInstance->ConditionalPostLoad();

	if (CompileId.IsValid() == false)
	{
		CompileId = FGuid::NewGuid();
	}

	if (SourceMaterialCollection)
	{
#if WITH_EDITOR
		OnMaterialCollectionChanged = SourceMaterialCollection->OnCollectionChanged().AddUObject(this, &UNiagaraParameterCollection::RefreshSourceMaterialCollection);
#endif

		SourceMaterialCollection->ConditionalPostLoad();

#if WITH_EDITORONLY_DATA
		// catch up with any changes that may have been made to the MPC
		RefreshSourceMaterialCollection();
#endif
	}
}

