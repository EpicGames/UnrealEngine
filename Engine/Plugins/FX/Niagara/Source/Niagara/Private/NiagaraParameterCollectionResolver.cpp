// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterCollectionResolver.h"

#include "NiagaraComponent.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraScript.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemSimulation.h"
#include "NiagaraWorldManager.h"

FNiagaraParameterCollectionResolver::FNiagaraParameterCollectionResolver(UNiagaraComponent* InNiagaraComponent, UNiagaraSystem* InNiagaraSystem, UWorld* World)
{
	NiagaraComponent	= InNiagaraComponent;
	NiagaraSystem		= InNiagaraSystem;
	NiagaraWorldManager	= World ? FNiagaraWorldManager::Get(World) : nullptr;
}

FNiagaraParameterCollectionResolver::FNiagaraParameterCollectionResolver(UNiagaraComponent* InNiagaraComponent, UNiagaraSystem* InNiagaraSystem, FNiagaraWorldManager* InNiagaraWorldManager)
{
	NiagaraComponent	= InNiagaraComponent;
	NiagaraSystem		= InNiagaraSystem;
	NiagaraWorldManager	= InNiagaraWorldManager;
}

void FNiagaraParameterCollectionResolver::BindCollections(TConstArrayView<TObjectPtr<UNiagaraParameterCollection>> ParameterCollections, FNiagaraParameterStore& DestinationStore) const
{
	for (UNiagaraParameterCollection* Collection : ParameterCollections)
	{
		if (Collection)
		{
			if (UNiagaraParameterCollectionInstance* CollectionInstance = ResolveInstance(Collection))
			{
				CollectionInstance->GetParameterStore().Bind(&DestinationStore);
			}
			else
			{
				UE_LOGF(LogNiagara, Error, "Attempting to bind system simulation to a null parameter collection instance | Collection: %ls | System: %ls |", *GetPathNameSafe(Collection), *GetPathNameSafe(NiagaraSystem));
			}
		}
		else
		{
			UE_LOGF(LogNiagara, Error, "Attempting to bind system simulation to a null parameter collection | System: %ls |", *GetPathNameSafe(NiagaraSystem));
		}
	}
}

void FNiagaraParameterCollectionResolver::UnbindCollections(TConstArrayView<TObjectPtr<UNiagaraParameterCollection>> ParameterCollections, FNiagaraParameterStore& DestinationStore) const
{
	for (UNiagaraParameterCollection* Collection : ParameterCollections)
	{
		UNiagaraParameterCollectionInstance* CollectionInstance = Collection ? ResolveInstance(Collection) : nullptr;
		if (CollectionInstance)
		{
			CollectionInstance->GetParameterStore().Unbind(&DestinationStore);
		}
	}
}

void FNiagaraParameterCollectionResolver::BindCollections(UNiagaraScript* Script, FNiagaraParameterStore& DestinationStore) const
{
	if (Script)
	{
		BindCollections(Script->GetCachedParameterCollectionReferences(), DestinationStore);
	}
}

void FNiagaraParameterCollectionResolver::UnbindCollections(UNiagaraScript* Script, FNiagaraParameterStore& DestinationStore) const
{
	if (Script)
	{
		UnbindCollections(Script->GetCachedParameterCollectionReferences(), DestinationStore);
	}
}

UNiagaraParameterCollectionInstance* FNiagaraParameterCollectionResolver::ResolveInstance(UNiagaraParameterCollection* Collection) const
{
	if (NiagaraComponent)
	{
		if (UNiagaraParameterCollectionInstance* Instance = NiagaraComponent->GetParameterCollectionOverride(Collection))
		{
			return Instance;
		}
	}

	if (NiagaraSystem)
	{
		if (UNiagaraParameterCollectionInstance* Instance = NiagaraSystem->GetParameterCollectionOverride(Collection))
		{
			return Instance;
		}
	}

	if (NiagaraWorldManager)
	{
		return NiagaraWorldManager->GetParameterCollection(Collection);
	}

	return nullptr;
}
