// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "UObject/ObjectPtr.h"

class UNiagaraComponent;
class UNiagaraParameterCollection;
class UNiagaraParameterCollectionInstance;
struct FNiagaraParameterStore;
class UNiagaraScript;
class UNiagaraSystem;
class FNiagaraSystemSimulation;
class FNiagaraWorldManager;
class UWorld;

struct FNiagaraParameterCollectionResolver
{
public:
	explicit FNiagaraParameterCollectionResolver(UNiagaraComponent* InNiagaraComponent, UNiagaraSystem* InNiagaraSystem, UWorld* World);
	explicit FNiagaraParameterCollectionResolver(UNiagaraComponent* InNiagaraComponent, UNiagaraSystem* InNiagaraSystem, FNiagaraWorldManager* InNiagaraWorldManager);

	void BindCollections(TConstArrayView<TObjectPtr<UNiagaraParameterCollection>> ParameterCollections, FNiagaraParameterStore& DestinationStore) const;
	void UnbindCollections(TConstArrayView<TObjectPtr<UNiagaraParameterCollection>> ParameterCollections, FNiagaraParameterStore& DestinationStore) const;

	void BindCollections(UNiagaraScript* Script, FNiagaraParameterStore& DestinationStore) const;
	void UnbindCollections(UNiagaraScript* Script, FNiagaraParameterStore& DestinationStore) const;

	UNiagaraParameterCollectionInstance* ResolveInstance(UNiagaraParameterCollection* Collection) const;

private:
	UNiagaraComponent*		NiagaraComponent = nullptr;
	UNiagaraSystem*			NiagaraSystem = nullptr;
	FNiagaraWorldManager*	NiagaraWorldManager = nullptr;
};
