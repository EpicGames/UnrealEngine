// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterDefinitionsSubscriber.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleManager.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorDataBase.h"
#include "NiagaraParameterDefinitionsBase.h"
#include "NiagaraScriptSourceBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraParameterDefinitionsSubscriber)


#if WITH_EDITORONLY_DATA

namespace NiagaraParameterDefinitionsSubscriberPrivate
{
	FDelegateHandle OnAssetRegistryFilesLoaded;
	TArray<TPair<TWeakObjectPtr<UObject>, INiagaraParameterDefinitionsSubscriber::FDeferredSyncFunction>> DeferredSyncFunctions;

	void OnFilesLoaded()
	{
		FSynchronizeWithParameterDefinitionsArgs Args;
		Args.bForceGatherDefinitions = true;
		Args.bSubscribeAllNameMatchParameters = true;

		for (const auto& DeferredSync : DeferredSyncFunctions)
		{
			if (UObject* Owner = DeferredSync.Key.Get())
			{
				for (INiagaraParameterDefinitionsSubscriber* Subscriber : DeferredSync.Value(Owner))
				{
					Subscriber->SynchronizeWithParameterDefinitions(Args);
				}
			}
		}
		DeferredSyncFunctions.Empty();

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().OnFilesLoaded().Remove(OnAssetRegistryFilesLoaded);
		OnAssetRegistryFilesLoaded.Reset();
	}

	void AddDeferredSync(TWeakObjectPtr<UObject> WeakOwner, INiagaraParameterDefinitionsSubscriber::FDeferredSyncFunction Function)
	{
		check(IsInGameThread());
		if (OnAssetRegistryFilesLoaded.IsValid() == false)
		{
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
			OnAssetRegistryFilesLoaded = AssetRegistryModule.Get().OnFilesLoaded().AddStatic(&OnFilesLoaded);
		}

		for (const auto& DeferredSync : DeferredSyncFunctions)
		{
			if (DeferredSync.Key == WeakOwner)
			{
				return;
			}
		}

		DeferredSyncFunctions.Emplace(WeakOwner, Function);
	}
}

void INiagaraParameterDefinitionsSubscriber::PostLoadDefinitionsSubscriptions(TWeakObjectPtr<UObject> WeakOwner, TFunction<TArray<INiagaraParameterDefinitionsSubscriber*>(UObject*)> DeferredSync)
{
	// First remove any subscriptions that are pointing to null parameter definitions assets. 
	// The null entries are either definitions assets that have been deleted, or definitions assets in a content plugin that is not mounted.
	// For the second scenario, we will rediscover the definitions asset in the content plugin the next time it is mounted in the bSubscribeAllNameMatchParameters pass.
	TArray<FParameterDefinitionsSubscription>& Subscriptions = GetParameterDefinitionsSubscriptions();
	for (int32 Idx = Subscriptions.Num() - 1; Idx > -1; --Idx)
	{
		if (Subscriptions[Idx].Definitions == nullptr)
		{
			Subscriptions.RemoveAtSwap(Idx);
		}
	}

	auto SyncAllNameMatchParameters = [this]() {
		// When postloading definition subscriptions, we want to synchronize all parameters with all parameter definitions that are matching by name.
		// As such; Set bForceGatherDefinitions so that all NiagaraParameterDefinitions assets are gathered to consider for linking, and;
		// Set bSubscribeAllNameMatchParameters so that name matches are considered for linking parameters to parameter definitions.
		FSynchronizeWithParameterDefinitionsArgs Args;
		Args.bForceGatherDefinitions = true;
		Args.bSubscribeAllNameMatchParameters = true;
		SynchronizeWithParameterDefinitions(Args);
	};

	// Check the asset registry for asset discovery having been completed.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	const bool bAllParameterDefinitionsDiscovered = AssetRegistryModule.Get().IsLoadingAssets() == false;

	// If the asset registry has not discovered all parameter definitions yet;
	// Immediately synchronize parameters we are already linked to so that we can sync changes to default value.
	// Then, wait for the callback asset registry discovery completion to sync all name match parameters.
	if (bAllParameterDefinitionsDiscovered == false)
	{
		SynchronizeWithParameterDefinitions();

		NiagaraParameterDefinitionsSubscriberPrivate::AddDeferredSync(WeakOwner, DeferredSync);
	}
	// Else the asset registry has discovered all parameter definitions assets;
	// Process linking name matched parameters and synchronizing existing definitions now.
	else
	{
		SyncAllNameMatchParameters();
	}
}

TArray<UNiagaraParameterDefinitionsBase*> INiagaraParameterDefinitionsSubscriber::GetSubscribedParameterDefinitions() const
{
	const TArray<FParameterDefinitionsSubscription>& Subscriptions = GetParameterDefinitionsSubscriptions();
	TArray<UNiagaraParameterDefinitionsBase*> SubscribedDefinitions;

	for (const FParameterDefinitionsSubscription& Subscription : Subscriptions)
	{
		if (Subscription.Definitions != nullptr)
		{
			SubscribedDefinitions.Add(Subscription.Definitions);
		}
	}
	return SubscribedDefinitions;
}

bool INiagaraParameterDefinitionsSubscriber::GetIsSubscribedToParameterDefinitions(const UNiagaraParameterDefinitionsBase* Definition) const
{
	const TArray<FParameterDefinitionsSubscription>& Subscriptions = GetParameterDefinitionsSubscriptions();

	for (const FParameterDefinitionsSubscription& Subscription : Subscriptions)
	{
		if (Subscription.Definitions == Definition)
		{
			return true;
		}
	}
	return false;
}

UNiagaraParameterDefinitionsBase* INiagaraParameterDefinitionsSubscriber::FindSubscribedParameterDefinitionsById(const FGuid& DefinitionsId) const
{
	TArray<UNiagaraParameterDefinitionsBase*> SubscribedDefinitions = GetSubscribedParameterDefinitions();

	for (UNiagaraParameterDefinitionsBase* SubscribedDefinition : SubscribedDefinitions)
	{
		if (SubscribedDefinition->GetDefinitionsUniqueId() == DefinitionsId)
		{
			return SubscribedDefinition;
		}
	}
	return nullptr;
}

void INiagaraParameterDefinitionsSubscriber::SubscribeToParameterDefinitions(UNiagaraParameterDefinitionsBase* NewParameterDefinitions, bool bDoNotAssertIfAlreadySubscribed /*= false*/)
{
	TArray<FParameterDefinitionsSubscription>& Subscriptions = GetParameterDefinitionsSubscriptions();
	for (const FParameterDefinitionsSubscription& Subscription : Subscriptions)
	{
		if (Subscription.Definitions == NewParameterDefinitions)
		{
			if (bDoNotAssertIfAlreadySubscribed == false)
			{
				ensureMsgf(false, TEXT("Tried to link to parameter definition that was already linked to!"));
			}
			return;
		}
	}

	FParameterDefinitionsSubscription& NewSubscription = Subscriptions.AddDefaulted_GetRef();
	NewSubscription.Definitions = NewParameterDefinitions;
	NewSubscription.CachedChangeIdHash = NewParameterDefinitions->GetChangeIdHash();

	OnSubscribedParameterDefinitionsChangedDelegate.Broadcast();
}

void INiagaraParameterDefinitionsSubscriber::UnsubscribeFromParameterDefinitions(const FGuid& ParameterDefinitionsToRemoveId)
{
	TArray<FParameterDefinitionsSubscription>& Subscriptions = GetParameterDefinitionsSubscriptions();
	for (int32 Idx = Subscriptions.Num() - 1; Idx > -1; --Idx)
	{
		if (Subscriptions[Idx].Definitions->GetDefinitionsUniqueId() == ParameterDefinitionsToRemoveId)
		{
			Subscriptions.RemoveAtSwap(Idx);
			//Synchronize after removing the subscription to remove the subscribed flag from all parameters that were subscribed to the removed definition.
			SynchronizeWithParameterDefinitions();
			OnSubscribedParameterDefinitionsChangedDelegate.Broadcast();
			return;
		}
	}
	ensureMsgf(false, TEXT("Tried to unlink from parameter definition that was not linked to!"));
}

void INiagaraParameterDefinitionsSubscriber::SynchronizeWithParameterDefinitions(const FSynchronizeWithParameterDefinitionsArgs Args /*= FSynchronizeWithParameterDefinitionsArgs()*/)
{
	struct FDefinitionAndChangeIdHash
	{
		UNiagaraParameterDefinitionsBase* Definition;
		int32 ChangeIdHash;
	};

	// Get all available parameter definitions.
	// Depending on whether asset discovery is complete we will either get all parameter definitions via asset registry or get all in subscriptions, respectively.
	TArray<FParameterDefinitionsSubscription>& Subscriptions = GetParameterDefinitionsSubscriptions();
	TArray<UNiagaraParameterDefinitionsBase*> AllDefinitions;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule.Get().IsLoadingAssets() == false)
	{
		TArray<FAssetData> ParameterDefinitionsAssetData;

		// Since this runs during the cook, we must query for disk-only assets to ensure deterministic results
		// (any asset queries running during cook of a package will affect dependencies)
		FARFilter Filter;
		Filter.ClassPaths.Add(FTopLevelAssetPath(TEXT("/Script/NiagaraEditor"), TEXT("NiagaraParameterDefinitions")));
		Filter.bRecursiveClasses = false;
		Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();
		AssetRegistryModule.GetRegistry().GetAssets(Filter, ParameterDefinitionsAssetData);
		
		for (const FAssetData& ParameterDefinitionsAssetDatum : ParameterDefinitionsAssetData)
		{
			if (!ParameterDefinitionsAssetDatum.IsUAsset() || !ParameterDefinitionsAssetDatum.IsAssetLoaded())
			{
				continue;
			}
			UNiagaraParameterDefinitionsBase* ParameterDefinitions = Cast<UNiagaraParameterDefinitionsBase>(ParameterDefinitionsAssetDatum.GetAsset());
			if (ParameterDefinitions == nullptr)
			{
				UE_LOGF(LogNiagara, Warning, "Failed to load parameter definition %ls from asset registry!", *ParameterDefinitionsAssetDatum.GetSoftObjectPath().ToString());
				continue;
			}
			AllDefinitions.Add(ParameterDefinitions);
		}
	}
	else
	{
		for (const FParameterDefinitionsSubscription& Subscription : Subscriptions)
		{
			AllDefinitions.Add(Subscription.Definitions);
		}
	}

	// Collect the FGuid ParameterIds for every parameter in every definition asset.
	TSet<FGuid> DefinitionParameterIds;
	for (const UNiagaraParameterDefinitionsBase* AllDefinitionsItr : AllDefinitions)
	{
		DefinitionParameterIds.Append(AllDefinitionsItr->GetParameterIds());
	}

	// Filter out Target Definitions that do not have a subscription associated with their unique id.
	TArray<FDefinitionAndChangeIdHash> TargetDefinitionAndChangeIdHashes;
	TArray<UNiagaraParameterDefinitionsBase*> TargetDefinitions;

	for (const FParameterDefinitionsSubscription& Subscription : Subscriptions)
	{
		if (Args.bForceGatherDefinitions)
		{
			FDefinitionAndChangeIdHash& DefinitionAndChangeIdHash = TargetDefinitionAndChangeIdHashes.Emplace_GetRef();
			DefinitionAndChangeIdHash.Definition = Subscription.Definitions;
			DefinitionAndChangeIdHash.ChangeIdHash = Subscription.Definitions->GetChangeIdHash();
			TargetDefinitions.Add(Subscription.Definitions);
		}
		else if (Subscription.CachedChangeIdHash != Subscription.Definitions->GetChangeIdHash())
		{
			FDefinitionAndChangeIdHash& DefinitionAndChangeIdHash = TargetDefinitionAndChangeIdHashes.Emplace_GetRef();
			DefinitionAndChangeIdHash.Definition = Subscription.Definitions;
			DefinitionAndChangeIdHash.ChangeIdHash = Subscription.Definitions->GetChangeIdHash();
			TargetDefinitions.Add(Subscription.Definitions);
		}
	}

	// Filter out only specific definitions from target definitions if specified.
	if (Args.SpecificDefinitionsUniqueIds.Num() > 0)
	{
		TArray<UNiagaraParameterDefinitionsBase*> TempTargetDefinitions = TargetDefinitions.FilterByPredicate([&Args](const UNiagaraParameterDefinitionsBase* TargetDefinition) { return Args.SpecificDefinitionsUniqueIds.Contains(TargetDefinition->GetDefinitionsUniqueId()); });
		TargetDefinitions = TempTargetDefinitions;
	}

	// Add any additional definitions if specified.
	for (UNiagaraParameterDefinitionsBase* AdditionalParameterDefinitionsItr : Args.AdditionalParameterDefinitions)
	{
		FDefinitionAndChangeIdHash& DefinitionAndChangeIdHash = TargetDefinitionAndChangeIdHashes.Emplace_GetRef();
		DefinitionAndChangeIdHash.Definition = AdditionalParameterDefinitionsItr;
		DefinitionAndChangeIdHash.ChangeIdHash = AdditionalParameterDefinitionsItr->GetChangeIdHash();
	}

	// Synchronize source scripts.
	for (UNiagaraScriptSourceBase* SourceScript : GetAllSourceScripts())
	{
		if (ensure(SourceScript))
		{
			SourceScript->SynchronizeGraphParametersWithParameterDefinitions(TargetDefinitions, AllDefinitions, DefinitionParameterIds, this, Args);
		}
	}

	// Synchronize editor only script variables.
	TArray<TTuple<FName, FName>> OldToNewNameArr;
	OldToNewNameArr.Append(Args.AdditionalOldToNewNames);
	for (UNiagaraEditorParametersAdapterBase* ParametersAdapter : GetEditorOnlyParametersAdapters())
	{
		OldToNewNameArr.Append(ParametersAdapter->SynchronizeParametersWithParameterDefinitions(TargetDefinitions, AllDefinitions, DefinitionParameterIds, this, Args));
	}

	// Editor only script variable synchronization may also implicate variables set in the stack through underlying source script UNiagaraNodeAssignments and UNiagaraNodeMapGets; synchronize those here.
	for (const TTuple<FName, FName>& OldToNewName : OldToNewNameArr)
	{
		for (UNiagaraScriptSourceBase* SourceScript : GetAllSourceScripts())
		{
			SourceScript->RenameGraphAssignmentAndSetNodePins(OldToNewName.Key, OldToNewName.Value);
		}
	}

	// Only mark the parameter definitions synchronized if every parameter definition was evaluated for synchronization.
	if (Args.SpecificDestScriptVarIds.Num() == 0)
	{
		MarkParameterDefinitionSubscriptionsSynchronized(Args.SpecificDefinitionsUniqueIds);
	}

	// Synchronize owned subscribers with the owning subscribers definitions.
	for (INiagaraParameterDefinitionsSubscriber* OwnedSubscriber : GetOwnedParameterDefinitionsSubscribers())
	{
		FSynchronizeWithParameterDefinitionsArgs SubArgs = Args;
		SubArgs.AdditionalParameterDefinitions = TargetDefinitions;
		SubArgs.AdditionalOldToNewNames = OldToNewNameArr;
		OwnedSubscriber->SynchronizeWithParameterDefinitions(SubArgs);
	}

	OnSubscribedParameterDefinitionsChangedDelegate.Broadcast();
}

void INiagaraParameterDefinitionsSubscriber::MarkParameterDefinitionSubscriptionsSynchronized(TArray<FGuid> SynchronizedParameterDefinitionsIds /*= TArray<FGuid>()*/)
{
	TArray<FParameterDefinitionsSubscription>& Subscriptions = GetParameterDefinitionsSubscriptions();

	for (FParameterDefinitionsSubscription& Subscription : Subscriptions)
	{
		if (SynchronizedParameterDefinitionsIds.Num() > 0 && SynchronizedParameterDefinitionsIds.Contains(Subscription.Definitions->GetDefinitionsUniqueId()) == false)
		{
			continue;
		}
		else
		{
			Subscription.CachedChangeIdHash = Subscription.Definitions->GetChangeIdHash();
		}
	}
}
#endif

