// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameters.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraParameters)

//////////////////////////////////////////////////////////////////////////

#if WITH_EDITORONLY_DATA

void FNiagaraParameters::Empty()
{
	Parameters.Empty();
}

void FNiagaraParameters::DumpParameters()
{
	for (FNiagaraVariable& Var : Parameters)
	{
		UE_LOGF(LogNiagara, Log, "Param: %ls Type : %ls", *Var.ToString(), *Var.GetType().GetName());
	}
}

void FNiagaraParameters::AppendToConstantsTable(uint8* ConstantsTable, const FNiagaraParameters& Externals)const
{
	uint8* Curr = ConstantsTable;
	for (const FNiagaraVariable &Uni : Parameters)
	{
		const FNiagaraVariable *ExternalUni = Externals.FindParameter(Uni);
		if (ExternalUni)
		{
			ExternalUni->CopyTo(Curr);
		}
		else
		{
			Uni.CopyTo(Curr);
		}

		Curr += Uni.GetSizeInBytes();
	}
}

void FNiagaraParameters::AppendToConstantsTable(uint8* ConstantsTable)const
{
	uint8* Curr = ConstantsTable;
	for (const FNiagaraVariable &Uni : Parameters)
	{
		Uni.CopyTo(Curr);
		Curr += Uni.GetSizeInBytes();
	}
}

FNiagaraVariable* FNiagaraParameters::SetOrAdd(const FNiagaraVariable& InParameter)
{
	int32 Idx = Parameters.IndexOfByPredicate([&](const FNiagaraVariable& C) { return InParameter == C; });
	FNiagaraVariable* Param = Idx != INDEX_NONE ? &Parameters[Idx] : nullptr;
	if (!Param)
	{
		Idx = Parameters.AddDefaulted();
		Param = &Parameters[Idx];
	}

	*Param = InParameter;
	return Param;
}



FNiagaraVariable* FNiagaraParameters::FindParameter(FNiagaraVariable InParam)
{
	auto* C = Parameters.FindByPredicate([&](const FNiagaraVariable& Param) { return Param == InParam; });
	return C;
}

const FNiagaraVariable* FNiagaraParameters::FindParameter(FNiagaraVariable InParam)const
{
	auto* C = Parameters.FindByPredicate([&](const FNiagaraVariable& Param) { return Param == InParam; });
	return C;
}


#endif // WITH_EDITORONLY_DATA

//////////////////////////////////////////////////////////////////////////

