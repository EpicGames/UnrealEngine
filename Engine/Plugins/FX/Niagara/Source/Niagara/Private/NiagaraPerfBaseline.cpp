// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraPerfBaseline.h"
#include "Engine/Engine.h"
#include "Misc/Paths.h"
#include "NiagaraComponent.h"
#include "NiagaraEffectType.h"
#include "NiagaraFunctionLibrary.h"
#include "Components/TextRenderComponent.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "NiagaraSystem.h"
#include "RenderingThread.h"
#include "HAL/IConsoleManager.h"
#include "HAL/FileManager.h"
#include "CanvasTypes.h"
#include "HAL/FileManager.h"
#include "UnrealClient.h"
#include "UObject/UObjectIterator.h" // IWYU pragma: keep

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraPerfBaseline)

#define LOCTEXT_NAMESPACE "NiagaraPerformanceBaselines"

#if NIAGARA_PERF_BASELINES

float GPerfBaselineThreshold_Poor = 2.0f;
static FAutoConsoleVariableRef CVarPerfBaselineThreshold_Poor(
	TEXT("fx.PerfBaselineThreshold_Poor"),
	GPerfBaselineThreshold_Poor,
	TEXT("Ratio to the baseline perf that we consider a system to have poor perf and warn about it. \n"),
	ECVF_Default
);

float GPerfBaselineThreshold_Bad = 5.0f;
static FAutoConsoleVariableRef CVarPerfBaselineThreshold_Bad(
	TEXT("fx.PerfBaselineThreshold_Bad"),
	GPerfBaselineThreshold_Bad,
	TEXT("Ratio to the baseline perf that we consider a system to have bad perf and warn strongly about it. \n"),
	ECVF_Default
);

int32 GbNiagaraPerfReporting = 0;
static FAutoConsoleVariableRef CVarNiagaraPerfReporting(
	TEXT("fx.NiagaraPerfReporting"),
	GbNiagaraPerfReporting,
	TEXT("0 = Disabled \n1 = Text Perf Report on world Transitions. \n2 = Text Report for every test with poor or bad perf.\n3 = As 2 but screenshots are also generated for each bad test."),
	ECVF_Default
);

int32 GbNiagaraRegenerateBaselinesOnWorldChange = 1;
static FAutoConsoleVariableRef CVarNiagaraRegenerateBaselinesOnWorldChange(
	TEXT("fx.NiagaraRegenBaselinesOnWorldChange"),
	GbNiagaraRegenerateBaselinesOnWorldChange,
	TEXT("If > 0 performance baselines for Niagara will be regenerated on every level change. \n"),
	ECVF_Default
);

float GBaselineGenerationDelay = 5.0f;
static FAutoConsoleVariableRef CVarBaselineGenerationDelay(
	TEXT("fx.Niagara.BaselineGenerationDelay"),
	GBaselineGenerationDelay,
	TEXT("Time we delay before match start for generating niagara perfoamnce baselines in a cooked game. \n"),
	ECVF_Default
);

int32 GPerfTestFrames = 240;
static FAutoConsoleVariableRef CVarPerfTestFrames(
	TEXT("fx.Niagara.PerfTestFrames"),
	GPerfTestFrames,
	TEXT("How many frames to gather in each performance test. \n"),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////

FNiagaraPerfBaselineStats::FNiagaraPerfBaselineStats(FAccumulatedParticlePerfStats& Stats, bool bSyncRT)
{
	//If you don't sync then safety with the RT is on you.
	if(bSyncRT)
	{
		FlushRenderingCommands();
	}

	//Grab the median max index to hopefully avoid any really spikey frames. Especially on PC & Dev builds etc.
	//However this will likely mean we miss high activation times.
	//We should add some other mechanism for tracking activation times more explicitly.
	int32 GTMaxIndex = ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES / 2;
	int32 RTMaxIndex = ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES / 2;
	if (Stats.GetGameThreadStats().NumFrames < ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES)
	{
		GTMaxIndex = Stats.GetGameThreadStats().NumFrames / 2;
	}	
	if (Stats.GetRenderThreadStats().NumFrames < ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES)
	{
		RTMaxIndex = Stats.GetRenderThreadStats().NumFrames / 2;
	}

	PerInstanceAvg_GT = float(FPlatformTime::ToMilliseconds64(Stats.GetGameThreadStats().GetPerInstanceAvgCycles()) * 1000.0);
	PerInstanceMax_GT = float(FPlatformTime::ToMilliseconds64(Stats.GetGameThreadStats().GetPerInstanceMaxCycles(GTMaxIndex)) * 1000.0);

	PerInstanceAvg_RT = float(FPlatformTime::ToMilliseconds64(Stats.GetRenderThreadStats().GetPerInstanceAvgCycles()) * 1000.0);
	PerInstanceMax_RT = float(FPlatformTime::ToMilliseconds64(Stats.GetRenderThreadStats().GetPerInstanceMaxCycles(RTMaxIndex)) * 1000.0);
}

FNiagaraPerfBaselineStats::EComparisonResult FNiagaraPerfBaselineStats::Compare(float SystemStats, float Baseline, float& OutRatio)
{
	if (SystemStats > 0.0f && Baseline > 0.0f)
	{
		OutRatio = SystemStats / Baseline;

		if (OutRatio >= GPerfBaselineThreshold_Bad)
		{
			return EComparisonResult::Bad;
		}
		else if (OutRatio >= GPerfBaselineThreshold_Poor)
		{
			return EComparisonResult::Poor;
		}

		return EComparisonResult::Good;
	}
	else
	{
		OutRatio = 0.0f;
		return EComparisonResult::Unknown;
	}
}

FNiagaraPerfBaselineStats::EComparisonResult FNiagaraPerfBaselineStats::Compare(const FNiagaraPerfBaselineStats& SystemStats, const FNiagaraPerfBaselineStats& Baseline, FNiagaraPerfBaselineStats& OutRatio, 
	EComparisonResult& OutGTAvgResult, EComparisonResult& OutGTMaxResult, EComparisonResult& OutRTAvgResult, EComparisonResult& OutRTMaxResult)
{
	if (SystemStats.IsValid() && Baseline.IsValid())
	{
		OutGTAvgResult = Compare(SystemStats.PerInstanceAvg_GT, Baseline.PerInstanceAvg_GT, OutRatio.PerInstanceAvg_GT);
		OutGTMaxResult = Compare(SystemStats.PerInstanceMax_GT, Baseline.PerInstanceMax_GT, OutRatio.PerInstanceMax_GT);
		OutRTAvgResult = Compare(SystemStats.PerInstanceAvg_RT, Baseline.PerInstanceAvg_RT, OutRatio.PerInstanceAvg_RT);
		OutRTMaxResult = Compare(SystemStats.PerInstanceMax_RT, Baseline.PerInstanceMax_RT, OutRatio.PerInstanceMax_RT);

		EComparisonResult GTResult = (EComparisonResult)FMath::Max((int32)OutGTAvgResult, (int32)OutGTMaxResult);
		EComparisonResult RTResult = (EComparisonResult)FMath::Max((int32)OutRTAvgResult, (int32)OutRTMaxResult);
		return (EComparisonResult)FMath::Max((int32)GTResult, (int32)RTResult);
	}
	else
	{
		OutRatio = FNiagaraPerfBaselineStats();
		OutGTAvgResult = EComparisonResult::Unknown;
		OutGTMaxResult = EComparisonResult::Unknown;
		OutRTAvgResult = EComparisonResult::Unknown;
		OutRTMaxResult = EComparisonResult::Unknown;
		return EComparisonResult::Unknown;
	}
}

FNiagaraPerfBaselineStats::EComparisonResult FNiagaraPerfBaselineStats::Compare(const FNiagaraPerfBaselineStats& SystemStats, const FNiagaraPerfBaselineStats& Baseline, FNiagaraPerfBaselineStats& OutRatio)
{
	EComparisonResult Dummy;
	return Compare(SystemStats, Baseline, OutRatio, Dummy, Dummy, Dummy, Dummy);
}

FLinearColor FNiagaraPerfBaselineStats::GetComparisonResultColor(EComparisonResult Result)
{
	switch (Result)
	{
	case EComparisonResult::Good: return FLinearColor::Green;
	case EComparisonResult::Poor: return FLinearColor::Yellow;
	case EComparisonResult::Bad: return FLinearColor::Red;
	case EComparisonResult::Unknown: return FLinearColor::Blue;
	}
	checkf(false, TEXT("Invalid result in GetComparisonResultColor()"));
	return FLinearColor::Black;
}

FText FNiagaraPerfBaselineStats::GetComparisonResultText(EComparisonResult Result)
{
	switch (Result)
	{
	case EComparisonResult::Good: return LOCTEXT("BaselineResultName_Good", "Good");
	case EComparisonResult::Poor: return LOCTEXT("BaselineResultName_Poor", "Poor");
	case EComparisonResult::Bad: return LOCTEXT("BaselineResultName_Bad", "Bad");
	case EComparisonResult::Unknown: return LOCTEXT("BaselineResultName_Unknown", "Unknown");
	}
	checkf(false, TEXT("Invalid result in GetComparisonResultText()"));
	return LOCTEXT("BaselineResultName_Invalid", "Invalid");
}

//////////////////////////////////////////////////////////////////////////

ANiagaraPerfBaselineActor::ANiagaraPerfBaselineActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Root component
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent"));

	Label = CreateDefaultSubobject<UTextRenderComponent>(TEXT("LabelText"));
	Label->SetText(LOCTEXT("BaselineActorLabel", "Generating Niagara Baseline Stats"));
	Label->SetWorldSize(60.0f);
	Label->SetRelativeRotation(FRotator(0.0f, 180.0f , 0.0f));
	Label->SetRelativeLocation(FVector(-250.0f, 0.0f, -100.0f));
	Label->SetHorizontalAlignment(EHTA_Center);
	//Label->SetTextRenderColor(FColor::Black);
	//Label->SetRelativeLocation(FVector(-300.0, 0.0f, 0.0f));

	//Ideally I'd have this just render atop everything else but this doesn't seem to work.
	//Not important enough to sink time into.
 	//Label->bUseEditorCompositing = true;

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;
	PrimaryActorTick.TickGroup = ETickingGroup::TG_PrePhysics;
}

void ANiagaraPerfBaselineActor::BeginPlay()
{
	Super::BeginPlay();

	//Create the listener for the given system. It will destroy itself and callback into OnEndTest when complete. 
	FParticlePerfStatsListenerPtr Listener = MakeShared<FNiagaraPerfBaselineStatsListener, ESPMode::ThreadSafe>(Controller);
	FParticlePerfStatsManager::AddListener(Listener);
}

void ANiagaraPerfBaselineActor::TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction)
{
	if (Controller)
	{
		Controller->OnOwnerTick(DeltaTime);
	}

	Super::TickActor(DeltaTime, TickType, ThisTickFunction);
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraBaselineController::OnBeginTest_Implementation()
{
}

bool UNiagaraBaselineController::OnTickTest_Implementation()
{
	if (ensure(Owner) && ensure(System) && ensure(EffectType))
	{
		if (Owner->GetGameTimeSinceCreation() >= TestDuration)
		{
			return false;
		}
		FString Message = FString::Printf(TEXT("Generating Niagara Performance Baselines: %s"), *System->GetName());
		GEngine->AddOnScreenDebugMessage((int32)GetUniqueID(), 4.0f, FColor::White, Message);
		return true;
	}
	return false;
}

void UNiagaraBaselineController::OnEndTest_Implementation(FNiagaraPerfBaselineStats Stats)
{
	GEngine->RemoveOnScreenDebugMessage((int32)GetUniqueID());

	if (ensure(System) && ensure(EffectType))
	{
		EffectType->UpdatePerfBaselineStats(Stats);
	}

	if (ensure(Owner))
	{
		Owner->Destroy();
	}
}

void UNiagaraBaselineController::OnOwnerTick_Implementation(float DeltaTime)
{
}

UNiagaraSystem* UNiagaraBaselineController::GetSystem()
{
	return System.LoadSynchronous();
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraBaselineController_Basic::OnBeginTest_Implementation()
{
	Super::OnBeginTest_Implementation();
	if(NumInstances > 0)
	{
		SpawnedComponents.SetNumZeroed(NumInstances);
	}
}

bool UNiagaraBaselineController_Basic::OnTickTest_Implementation()
{
	return Super::OnTickTest_Implementation();
}

void UNiagaraBaselineController_Basic::OnEndTest_Implementation(FNiagaraPerfBaselineStats Stats)
{
	Super::OnEndTest_Implementation(Stats);

	for (auto& Comp : SpawnedComponents)
	{
		if (Comp)
		{
			Comp->DestroyComponent();
		}
	}
}

void UNiagaraBaselineController_Basic::OnOwnerTick_Implementation(float DeltaTime)
{
	Super::OnOwnerTick_Implementation(DeltaTime);
	if (GetSystem())
	{
		if (ensure(Owner))
		{
			FVector OwnerLoc = Owner->GetActorLocation();
			for (int32 i = 0; i < NumInstances; ++i)
			{
				if (SpawnedComponents[i] == nullptr || !SpawnedComponents[i]->IsActive())
				{
					SpawnedComponents[i] = UNiagaraFunctionLibrary::SpawnSystemAtLocation(GetWorld(), GetSystem(), OwnerLoc, FRotator::ZeroRotator, FVector::OneVector, false, false, ENCPoolMethod::None, false);
					SpawnedComponents[i]->InitForPerformanceBaseline();
					SpawnedComponents[i]->Activate();
				}
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////


FNiagaraPerfBaselineStatsListener::FNiagaraPerfBaselineStatsListener(UNiagaraBaselineController* OwnerBaseline)
: Baseline(OwnerBaseline)
{

}

void FNiagaraPerfBaselineStatsListener::Begin()
{
	if (UNiagaraBaselineController* BaselinePtr = Baseline.Get())
	{
		Baseline->OnBeginTest();
	}
}

void FNiagaraPerfBaselineStatsListener::End()
{
	//Copy the accumulated stats into the baseline. Sync with RT to ensure complete data.
	FNiagaraPerfBaselineStats BaselineStats;
	BaselineStats = FNiagaraPerfBaselineStats(AccumulatedStats, true);

	if (UNiagaraBaselineController* BaselinePtr = Baseline.Get())
	{
		BaselinePtr->OnEndTest(BaselineStats);
	}
}

bool FNiagaraPerfBaselineStatsListener::Tick()
{
	if (UNiagaraBaselineController* BaselinePtr = Baseline.Get())
	{
		if (FParticlePerfStats* Stats = FParticlePerfStatsManager::GetSystemPerfStats(BaselinePtr->GetSystem()))
		{
			AccumulatedStats.Tick(*Stats);
		}

		return BaselinePtr->OnTickTest();
	}
	return false;
}

void FNiagaraPerfBaselineStatsListener::TickRT()
{
	if (UNiagaraBaselineController* BaselinePtr = Baseline.Get())
	{
		if (FParticlePerfStats* Stats = FParticlePerfStatsManager::GetSystemPerfStats(BaselinePtr->GetSystem()))
		{
			AccumulatedStats.TickRT(*Stats);
		}
	}
}

//////////////////////////////////////////////////////////////////////////
const int32 FParticlePerfStatsListener_NiagaraPerformanceReporter::TestDebugMessageID = FCrc::Strihash_DEPRECATED(TEXT("NiagaraPerfReporterMessageID"));

void FParticlePerfStatsListener_NiagaraPerformanceReporter::ReportToLog()
{
	//Ensure test results are complete.
	FlushRenderingCommands();
	HandleTestResults();

	struct FBaselineReportItem
	{
		const UNiagaraSystem* System;
		FStoredStatsInfo* Stats;

		bool operator<(const FBaselineReportItem& Other)const
		{
			if(Stats && Other.Stats && Stats->BadTestHistory.Num() < Other.Stats->BadTestHistory.Num())
			{
				return true;
			}
			return System->GetName() < Other.System->GetName();
		}
	};

	TArray<FBaselineReportItem> BadPerfSystems;
	TArray<FBaselineReportItem> NoFXTypeSystems;
	TArray<FBaselineReportItem> InvalidBaselineSystems;

	for (auto& SysStatPair : StoredStats)
	{
		FStoredStatsInfo& StatsInfo = SysStatPair.Value;
		if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(SysStatPair.Key.Get()))
		{
			UNiagaraEffectType* FXType = System->GetEffectType();

			if (StatsInfo.BadTestHistory.Num())
			{
				if (FXType && FXType->IsPerfBaselineValid())
				{
					if (FXType->IsPerfBaselineValid())
					{						
						if(StatsInfo.BadTestHistory.Num() > 0)
						{
							FBaselineReportItem& Item = BadPerfSystems.AddDefaulted_GetRef();
							Item.System = System;
							Item.Stats = &StatsInfo;
						}
					}
					else
					{
						FBaselineReportItem& ReportItem = InvalidBaselineSystems.AddDefaulted_GetRef();
						ReportItem.System = System;
						ReportItem.Stats = nullptr;
					}
				}
				else
				{
					FBaselineReportItem& ReportItem = NoFXTypeSystems.AddDefaulted_GetRef();
					ReportItem.System = System;
					ReportItem.Stats = nullptr;
				}
			}
		}
	};

	auto PrintHeader = [](FString HeaderText, bool bWithStatHeadings=true)
	{
		UE_LOGF(LogNiagara, Log, "|----------------------------------------------------------|");
		UE_LOGF(LogNiagara, Log, "| %ls |", *HeaderText);
		UE_LOGF(LogNiagara, Log, "|----------------------------------------------------------|");
	};

	UE_LOGF(LogNiagara, Log, "|================================================================|");
	UE_LOGF(LogNiagara, Log, "| Niagara Performance Report |");
	UE_LOGF(LogNiagara, Log, "| Stats for each system are gathered in short tests of %u frames. |", GPerfTestFrames);
	UE_LOGF(LogNiagara, Log, "| Total Tests: %u |", TotalTests);
	UE_LOGF(LogNiagara, Log, "|================================================================|");

	if(BadPerfSystems.Num() > 0)
	{
		auto PrintBadTest = [](FStatTestInfo& TestInfo, FNiagaraPerfBaselineStats& Baseline)
		{
			UE_LOGF(LogNiagara, Log, "| Test %u | Start Time: %g | End Time: %g ", TestInfo.TestIndex, TestInfo.StartTime, TestInfo.EndTime); 
			
			FNiagaraPerfBaselineStats TestBaselineStats(TestInfo.AccumulatedStats,false);
			FNiagaraPerfBaselineStats Ratio;
			FNiagaraPerfBaselineStats::Compare(TestBaselineStats, Baseline, Ratio);

			FAccumulatedParticlePerfStats_GT& GT = TestInfo.AccumulatedStats.GetGameThreadStats();
			if(GT.AccumulatedStats.NumInstances)
			{			
				//Grab the median max index to hopefully avoid any really spikey frames. Especially on PC & Dev builds etc.
				//However this will likely mean we miss high activation times.
				//We should add some other mechanism for tracking activation times more explicitly.
				int32 GTMaxIndex = ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES / 2;
				if (GT.NumFrames < ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES)
				{
					GTMaxIndex = GT.NumFrames / 2;
				}

				const uint32 TickGameThreadTime = uint32(FPlatformTime::ToMilliseconds64(GT.AccumulatedStats.TickGameThreadCycles) * 1000.0);
				const uint32 TickConcurrentTime = uint32(FPlatformTime::ToMilliseconds64(GT.AccumulatedStats.TickConcurrentCycles) * 1000.0);
				const uint32 FinalizeTime = uint32(FPlatformTime::ToMilliseconds64(GT.AccumulatedStats.FinalizeCycles) * 1000.0);
				const uint32 EndOfFrameTime = uint32(FPlatformTime::ToMilliseconds64(GT.AccumulatedStats.EndOfFrameCycles) * 1000.0);
				
				UE_LOGF(LogNiagara, Log, "| Game Thread | Num: %4llu | Avg: %6u | Max: %6u | AvgRatio: %g | Max Ratio: %g |", GT.AccumulatedStats.NumInstances, (uint32)GT.GetPerInstanceAvg(), (uint32)GT.GetPerInstanceMax(GTMaxIndex), Ratio.PerInstanceAvg_GT, Ratio.PerInstanceMax_GT);
				UE_LOGF(LogNiagara, Log, "| | Tick            | Total: %5u | Avg: %5llu |", TickGameThreadTime, TickGameThreadTime / GT.AccumulatedStats.NumInstances);
				UE_LOGF(LogNiagara, Log, "| | Tick Concurrent | Total: %5u | Avg: %5llu |", TickConcurrentTime, TickConcurrentTime / GT.AccumulatedStats.NumInstances);
				UE_LOGF(LogNiagara, Log, "| | Finalize        | Total: %5u | Avg: %5llu |", FinalizeTime, FinalizeTime / GT.AccumulatedStats.NumInstances);
				UE_LOGF(LogNiagara, Log, "| | EOF Update      | Total: %5u | Avg: %5llu |", EndOfFrameTime, EndOfFrameTime / GT.AccumulatedStats.NumInstances);
			}
			FAccumulatedParticlePerfStats_RT& RT = TestInfo.AccumulatedStats.GetRenderThreadStats();
			if (RT.AccumulatedStats.NumInstances)
			{		
				//Grab the median max index to hopefully avoid any really spikey frames. Especially on PC & Dev builds etc.
				//However this will likely mean we miss high activation times.
				//We should add some other mechanism for tracking activation times more explicitly.
				int32 RTMaxIndex = ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES / 2;
				if (RT.NumFrames < ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES)
				{
					RTMaxIndex = RT.NumFrames / 2;
				}
				const uint32 RTAvg = uint32(RT.GetPerInstanceAvg());
				const uint32 RTMax = uint32(RT.GetPerInstanceMax(RTMaxIndex));
				const uint32 RenderUpdateTime = uint32(FPlatformTime::ToMilliseconds64(RT.AccumulatedStats.RenderUpdateCycles) * 1000.0);
				const uint32 GDMETime = uint32(FPlatformTime::ToMilliseconds64(RT.AccumulatedStats.GetDynamicMeshElementsCycles) * 1000.0);
				UE_LOGF(LogNiagara, Log, "| Render Thread | Num: %4llu | Avg: %6u | Max: %6u | AvgRatio: %g | Max Ratio: %g |", RT.AccumulatedStats.NumInstances, RTAvg, RTMax, Ratio.PerInstanceAvg_RT, Ratio.PerInstanceMax_RT);
				UE_LOGF(LogNiagara, Log, "| | Update | Total: %5u | Avg: %5llu |", RenderUpdateTime, RenderUpdateTime / RT.AccumulatedStats.NumInstances);
				UE_LOGF(LogNiagara, Log, "| | GDME   | Total: %5u | Avg: %5llu |", GDMETime, GDMETime / RT.AccumulatedStats.NumInstances);
			}
		};
		PrintHeader(TEXT("Systems with bad performance relative to their baseline."));
		for (FBaselineReportItem& Item : BadPerfSystems)
		{
			UE_LOGF(LogNiagara, Log, "|================================================================|");
			UE_LOGF(LogNiagara, Log, "| System: %ls | Total Bad Tests: %u", *Item.System->GetName(), Item.Stats->BadTestHistory.Num());
			FNiagaraPerfBaselineStats& Baseline = Item.System->GetEffectType()->GetPerfBaselineStats();
			UNiagaraBaselineController* Controller = Item.System->GetEffectType()->GetPerfBaselineController();
			UE_LOGF(LogNiagara, Log, "| Baseline: %ls | GT Avg: %5u | GT Max: %5u | RT Avg: %5u | RT Max: %5u |", *Controller->GetSystem()->GetName(), (uint32)Baseline.PerInstanceAvg_GT, (uint32)Baseline.PerInstanceMax_GT, (uint32)Baseline.PerInstanceAvg_RT, (uint32)Baseline.PerInstanceMax_RT);
			UE_LOGF(LogNiagara, Log, "|----------------------------------------------------------------|");
			for(auto& Test : Item.Stats->BadTestHistory)
			{
				PrintBadTest(Test, Baseline);
				UE_LOGF(LogNiagara, Log, "|----------------------------------------------------------------|");
			}
			UE_LOGF(LogNiagara, Log, "|================================================================|");
		}
	}
	
	if (InvalidBaselineSystems.Num() > 0)
	{
		PrintHeader(TEXT("Systems without valid baseline data."));
		for (FBaselineReportItem& Item : InvalidBaselineSystems)
		{
			UE_LOGF(LogNiagara, Log, "| System: %ls | FXType: %ls |", *Item.System->GetName(), *Item.System->GetEffectType()->GetName());
		}
	}

	if (NoFXTypeSystems.Num() > 0)
	{
		PrintHeader(TEXT("Systems without an Effect Type."));
		for (FBaselineReportItem& Item : NoFXTypeSystems)
		{
			UE_LOGF(LogNiagara, Log, "| System: %ls |", *Item.System->GetName());
		}
	}
	UE_LOGF(LogNiagara, Log, "|================================================================|");
}

// void FParticlePerfStatsListener_NiagaraPerformanceReporter::ReportToFile()
// {
// 	//TODO: Any tests with poor perf should dump the bad system stats
// 	//Also dump a screen shot for refrence.
// 
// // 	FHighResScreenshotConfig& Config = GetHighResScreenshotConfig();
// // 	FIntPoint ScreenshotViewportSize = UAutomationBlueprintFunctionLibrary::GetAutomationScreenshotSize(ScreenshotOptions);
// // 	if (Config.SetResolution(ScreenshotViewportSize.X, ScreenshotViewportSize.Y, 1.0f))
// // 	{
// // 		GEngine->GameViewport->GetGameViewport()->TakeHighResScreenShot();
// // 	}
// }

void FParticlePerfStatsListener_NiagaraPerformanceReporter::ReportToScreen()
{
	bool bHasBadPerfReports = false;
	for (auto& SysStatPair : StoredStats)
	{
		if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(SysStatPair.Key.Get()))
		{
			if (UNiagaraEffectType* FXType = System->GetEffectType())
			{
				if (FXType->IsPerfBaselineValid())
				{
					FStoredStatsInfo& StatsInfo = SysStatPair.Value;
					FNiagaraPerfBaselineStats& Baseline = FXType->GetPerfBaselineStats();
					FNiagaraPerfBaselineStats Current(StatsInfo.Current, false);
					FNiagaraPerfBaselineStats StatRatio;
					FNiagaraPerfBaselineStats::EComparisonResult ComparisonResult = FNiagaraPerfBaselineStats::Compare(Current, Baseline, StatRatio);

					if (ComparisonResult == FNiagaraPerfBaselineStats::EComparisonResult::Poor || ComparisonResult == FNiagaraPerfBaselineStats::EComparisonResult::Bad)
					{
						FLinearColor TextColor = FNiagaraPerfBaselineStats::GetComparisonResultColor(ComparisonResult);

						FString Message = FString::Printf(TEXT("| %s | Test %u | Niagara Perf Warning! See Niagara Performance Report for details. |"), *System->GetName(), TotalTests);

						GEngine->AddOnScreenDebugMessage((int32)System->GetUniqueID(), 5.0f, TextColor.ToFColor(false), Message);
						UE_LOGF(LogNiagara, Warning, "%ls", *Message);
						
						bHasBadPerfReports = true;
					}
				}
			}
		}

		if (bHasBadPerfReports)
		{
			GEngine->RemoveOnScreenDebugMessage(TestDebugMessageID);
		}
		else
		{
			//if we have no bad perf then add an everything is OK message just so we know we're gathering perf data.
			FString Message = FString::Printf(TEXT("| Gathering Niagara Perf Test %u |"), TotalTests + 1);
			GEngine->AddOnScreenDebugMessage(TestDebugMessageID, 5.0f, FColor::White, *Message);
		}
	}
}

FParticlePerfStatsListener_NiagaraPerformanceReporter::FParticlePerfStatsListener_NiagaraPerformanceReporter(UWorld* InWorld)
: FParticlePerfStatsListener_GatherAll(false, true, false)
, World(InWorld)
{
	CurrentWorldTime = World->GetTimeSeconds();
	CurrentFrameNumber = GFrameNumber;
	TestNameString = FDateTime::Now().ToString(TEXT("%d-%H.%M.%S"));
}

void FParticlePerfStatsListener_NiagaraPerformanceReporter::HandleTestResults()
{
	//Only handle results if we have an outstanding test to process and the RT has finished writing it's results.
	if (bResultsTrigger == false || ResultsFence.IsFenceComplete() == false)
	{
		return;
	}	

	FScopeLock Lock(&AccumulatedStatsGuard);
	bResultsTrigger = false;

	struct FTestReport
	{
		const UNiagaraSystem* System;
		FNiagaraPerfBaselineStats::EComparisonResult OverallResult;
		FNiagaraPerfBaselineStats::EComparisonResult GTAvgResult;
		FNiagaraPerfBaselineStats::EComparisonResult GTMaxResult;
		FNiagaraPerfBaselineStats::EComparisonResult RTAvgResult;
		FNiagaraPerfBaselineStats::EComparisonResult RTMaxResult;
		FAccumulatedParticlePerfStats* Stats;
		FNiagaraPerfBaselineStats Ratio;
	};
	TArray<FTestReport> TestReports;

	const double TestStartTime = CurrentWorldTime;
	const int32 TestStartFrame = CurrentFrameNumber;
	CurrentWorldTime = World->GetTimeSeconds();
	CurrentFrameNumber = GFrameNumber;

	for (auto& SysStatPair : StoredStats)
	{
		if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(SysStatPair.Key.Get()))
		{
			if (const UNiagaraEffectType* FXType = System->GetEffectType())
			{
				if (FXType->IsPerfBaselineValid())
				{
					FStoredStatsInfo& StatsInfo = SysStatPair.Value;					

					FNiagaraPerfBaselineStats& Baseline = FXType->GetPerfBaselineStats();

					//Add any poorly performing tests to the history
					FNiagaraPerfBaselineStats StatRatio;
					FNiagaraPerfBaselineStats Current(StatsInfo.Current, false);
					
					FTestReport Report;

					Report.OverallResult = FNiagaraPerfBaselineStats::Compare(Current, Baseline, Report.Ratio, Report.GTAvgResult, Report.GTMaxResult, Report.RTAvgResult, Report.RTMaxResult);
					if (Report.OverallResult == FNiagaraPerfBaselineStats::EComparisonResult::Poor || Report.OverallResult == FNiagaraPerfBaselineStats::EComparisonResult::Bad)
					{
						//Finish the report and add to those that are being dumped to the file.
						if(GbNiagaraPerfReporting > 1)
						{
							Report.System = System;
							Report.Stats = &StatsInfo.Current;
							TestReports.Add(Report);
						}
						

						FStatTestInfo& NewBadTest = StatsInfo.BadTestHistory.AddDefaulted_GetRef();
						NewBadTest.TestIndex = TotalTests;
						NewBadTest.StartTime = TestStartTime;
						NewBadTest.EndTime = CurrentWorldTime;
						NewBadTest.AccumulatedStats = StatsInfo.Current;
					}
				}
			}
		}
	}

	if (TestReports.Num() > 0 && GbNiagaraPerfReporting > 1)
	{
		const FString PathName = FPaths::ProfilingDir() + TEXT("ParticlePerf/NiagaraPerfReports-") + TestNameString;
		IFileManager::Get().MakeDirectory(*PathName);
		
		const FString Filename = FString::Printf(TEXT("Test %u [%u-%u].txt"), TotalTests, TestStartFrame, CurrentFrameNumber);
		const FString FilePath = PathName / Filename;

		//Write the output location to the screen and logs.
		FString Message = FString::Printf(TEXT("Writing Report for Perf Test %u to %s..."), TotalTests, *FilePath);
		GEngine->AddOnScreenDebugMessage(TestDebugMessageID, 2.0f, FColor::White, Message);

		if (FArchive* FileAr = IFileManager::Get().CreateDebugFileWriter(*FilePath))
		{
			TUniquePtr<FOutputDeviceArchiveWrapper> FileArWrapper(new FOutputDeviceArchiveWrapper(FileAr));

			FOutputDevice& Ar = *FileArWrapper.Get();

			Ar.Logf(TEXT("|===========================================================================|"));
			Ar.Logf(TEXT("|================================ Test %u ======================================|"), TotalTests);
			Ar.Logf(TEXT("| Start Frame: %u | End Frame: %u | Start Time: %g | End Time: %g |"), TestStartFrame, CurrentFrameNumber, TestStartTime, CurrentWorldTime);
			Ar.Logf(TEXT("|===========================================================================|"));
			Ar.Logf(TEXT("|===========================================================================|"));

			auto DumpReportToDevice = [](FOutputDevice& Ar, FTestReport& Report)
			{
				FNiagaraPerfBaselineStats& Baseline = Report.System->GetEffectType()->GetPerfBaselineStats();
				Ar.Logf(TEXT("|===========================================================================|"));
				Ar.Logf(TEXT("| System | %s |"), *Report.System->GetPathName());
				Ar.Logf(TEXT("| Baseline | %s | GT Avg: %5u | GT Max: %5u | RT Avg: %5u | RT Max: %5u |"), *Report.System->GetEffectType()->GetPathName(), (uint32)Baseline.PerInstanceAvg_GT, (uint32)Baseline.PerInstanceMax_GT, (uint32)Baseline.PerInstanceAvg_RT, (uint32)Baseline.PerInstanceMax_RT);
				Ar.Logf(TEXT("| Results | Overall: %s | GT Avg: %s | GT Max: %s | RT Avg: %s | RT Max: %s |"), 
					*FNiagaraPerfBaselineStats::GetComparisonResultText(Report.OverallResult).ToString(),
					*FNiagaraPerfBaselineStats::GetComparisonResultText(Report.GTAvgResult).ToString(),
					*FNiagaraPerfBaselineStats::GetComparisonResultText(Report.GTMaxResult).ToString(),
					*FNiagaraPerfBaselineStats::GetComparisonResultText(Report.RTAvgResult).ToString(),
					*FNiagaraPerfBaselineStats::GetComparisonResultText(Report.RTMaxResult).ToString());

				FAccumulatedParticlePerfStats_GT& GT = Report.Stats->GetGameThreadStats();
				if (GT.AccumulatedStats.NumInstances)
				{
					//Grab the median max index to hopefully avoid any really spikey frames. Especially on PC & Dev builds etc.
					//However this will likely mean we miss high activation times.
					//We should add some other mechanism for tracking activation times more explicitly.
					int32 GTMaxIndex = ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES / 2;
					if (GT.NumFrames < ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES)
					{
						GTMaxIndex = GT.NumFrames / 2;
					}

					const uint32 TickGameThreadTime = uint32(FPlatformTime::ToMilliseconds64(GT.AccumulatedStats.TickGameThreadCycles) * 1000.0);
					const uint32 TickConcurrentTime = uint32(FPlatformTime::ToMilliseconds64(GT.AccumulatedStats.TickConcurrentCycles) * 1000.0);
					const uint32 FinalizeTime = uint32(FPlatformTime::ToMilliseconds64(GT.AccumulatedStats.FinalizeCycles) * 1000.0);
					const uint32 EndOfFrameTime = uint32(FPlatformTime::ToMilliseconds64(GT.AccumulatedStats.EndOfFrameCycles) * 1000.0);

					Ar.Logf(TEXT("| Game Thread | Num: %4u | Avg: %6u | Max: %6u | AvgRatio: %g | Max Ratio: %g |"), GT.AccumulatedStats.NumInstances, (uint32)GT.GetPerInstanceAvg(), (uint32)GT.GetPerInstanceMax(GTMaxIndex), Report.Ratio.PerInstanceAvg_GT, Report.Ratio.PerInstanceMax_GT);
					Ar.Logf(TEXT("| | Tick            | Total: %5u | Avg: %5u |"), TickGameThreadTime, TickGameThreadTime / GT.AccumulatedStats.NumInstances);
					Ar.Logf(TEXT("| | Tick Concurrent | Total: %5u | Avg: %5u |"), TickConcurrentTime, TickConcurrentTime / GT.AccumulatedStats.NumInstances);
					Ar.Logf(TEXT("| | Finalize        | Total: %5u | Avg: %5u |"), FinalizeTime, FinalizeTime / GT.AccumulatedStats.NumInstances);
					Ar.Logf(TEXT("| | EOF Update      | Total: %5u | Avg: %5u |"), EndOfFrameTime, EndOfFrameTime / GT.AccumulatedStats.NumInstances);
				}
				FAccumulatedParticlePerfStats_RT& RT = Report.Stats->GetRenderThreadStats();
				if (RT.AccumulatedStats.NumInstances)
				{
					//Grab the median max index to hopefully avoid any really spikey frames. Especially on PC & Dev builds etc.
					//However this will likely mean we miss high activation times.
					//We should add some other mechanism for tracking activation times more explicitly.
					int32 RTMaxIndex = ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES / 2;
					if (RT.NumFrames < ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES)
					{
						RTMaxIndex = RT.NumFrames / 2;
					}
					const uint32 RTAvg = uint32(RT.GetPerInstanceAvg());
					const uint32 RTMax = uint32(RT.GetPerInstanceMax(RTMaxIndex));
					const uint32 RenderUpdateTime = uint32(FPlatformTime::ToMilliseconds64(RT.AccumulatedStats.RenderUpdateCycles) * 1000.0);
					const uint32 GDMETime = uint32(FPlatformTime::ToMilliseconds64(RT.AccumulatedStats.GetDynamicMeshElementsCycles) * 1000.0);
					Ar.Logf(TEXT("| Render Thread | Num: %4u | Avg: %6u | Max: %6u | AvgRatio: %g | Max Ratio: %g |"), RT.AccumulatedStats.NumInstances, RTAvg, RTMax, Report.Ratio.PerInstanceAvg_RT, Report.Ratio.PerInstanceMax_RT);
					Ar.Logf(TEXT("| | Update | Total: %5u | Avg: %5u |"), RenderUpdateTime, RenderUpdateTime / RT.AccumulatedStats.NumInstances);
					Ar.Logf(TEXT("| | GDME   | Total: %5u | Avg: %5u |"), GDMETime, GDMETime / RT.AccumulatedStats.NumInstances);
				}
			};

			for (FTestReport& Report : TestReports)
			{
				DumpReportToDevice(Ar, Report);
			}
			delete FileAr;
		}

		//Also take a Screenshot
		if (GbNiagaraPerfReporting > 2)
		{
			const FString ScreenShotFilename = PathName / FString::Printf(TEXT("Test %u [%u-%u]"), TotalTests, TestStartFrame, CurrentFrameNumber);
			const bool bShowUI = true;
			FScreenshotRequest::RequestScreenshot(ScreenShotFilename, true, false);
		}
	}

	ReportToScreen();
	++TotalTests;
}

bool FParticlePerfStatsListener_NiagaraPerformanceReporter::Tick()
{
	bool bRet = FParticlePerfStatsListener_GatherAll::Tick();

	//Handle the most recent completed test once the RT has finished writing it's results.
	HandleTestResults();

	if (++NumFrames > GPerfTestFrames)
	{
		NumFrames = 0;
		FScopeLock Lock(&AccumulatedStatsGuard);
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
		for (auto& SysStatPair : AccumulatedSystemStats)
		{
			if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(SysStatPair.Key.Get()))
			{
				FAccumulatedParticlePerfStats* Stats = SysStatPair.Value.Get();
				check(Stats);

				if (UNiagaraEffectType* FXType = System->GetEffectType())
				{
					if (FXType->IsPerfBaselineValid())
					{
						FStoredStatsInfo& StatsInfo = StoredStats.FindOrAdd(System);

						StatsInfo.Current.GetGameThreadStats() = Stats->GetGameThreadStats();						
					}
				}

				Stats->ResetGT();
			}
		}
#endif

		//Trigger handling of the new results.
		//Must be delayed until the RT has written the RT stats for the current test.
		ResultsFence.BeginFence();
		bResultsTrigger = true;
	}

	return bRet;
}

void FParticlePerfStatsListener_NiagaraPerformanceReporter::TickRT()
{
	FParticlePerfStatsListener_GatherAll::TickRT();

	if (++NumFramesRT > GPerfTestFrames)
	{
		TArray<UNiagaraSystem*, TInlineAllocator<16>> ToRemove;
		NumFramesRT = 0;
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
		FScopeLock Lock(&AccumulatedStatsGuard);
		for (auto& SysStatPair : AccumulatedSystemStats)
		{
			if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(SysStatPair.Key.Get()))
			{
				FAccumulatedParticlePerfStats* Stats = SysStatPair.Value.Get();
				check(Stats);

				if (UNiagaraEffectType* FXType = System->GetEffectType())
				{
					if (FXType->IsPerfBaselineValid())
					{
						FStoredStatsInfo& StatsInfo = StoredStats.FindOrAdd(System);

						StatsInfo.Current.GetRenderThreadStats() = Stats->GetRenderThreadStats();
					}
				}

				Stats->ResetRT();
			}
		}
#endif
	}
}

//////////////////////////////////////////////////////////////////////////
FParticlePerfStatsListener_NiagaraBaselineComparisonRender::FParticlePerfStatsListener_NiagaraBaselineComparisonRender()
: FParticlePerfStatsListener_GatherAll(false, true, false)
{

}

bool FParticlePerfStatsListener_NiagaraBaselineComparisonRender::Tick()
{
	bool bRet = FParticlePerfStatsListener_GatherAll::Tick();

	//Every so often we reset our accumulated stats and sort the worst to the top.
	//We grab the GT stats here and clear them the RT does the same for RT stats.
	//TODO: Sort the stats at a lower frequency than the update.
	if (++NumFrames > GPerfTestFrames)
	{
		NumFrames = 0;
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
		FScopeLock Lock(&AccumulatedStatsGuard);
		for (auto& SysStatPair : AccumulatedSystemStats)
		{
			if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(SysStatPair.Key.Get()))
			{
				FAccumulatedParticlePerfStats* Stats = SysStatPair.Value.Get();
				check(Stats);

				FNiagaraPerfBaselineStats& Curr = CurrentStats.FindOrAdd(System);
				Curr.PerInstanceAvg_GT = Stats->GetGameThreadStats().GetPerInstanceAvg();

				//Grab the median max index to hopefully avoid any really spikey frames. Especially on PC & Dev builds etc.
				//However this will likely mean we miss high activation times.
				//We should add some other mechanism for tracking activation times more explicitly.
				int32 GTMaxIndex = ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES / 2;
				if (Stats->GetGameThreadStats().NumFrames < ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES)
				{
					GTMaxIndex = Stats->GetGameThreadStats().NumFrames / 2;
				}
				
				Curr.PerInstanceMax_GT = Stats->GetGameThreadStats().GetPerInstanceMax(GTMaxIndex);
				Stats->ResetGT();
			}
		}
#endif
	}

	return bRet;
}

void FParticlePerfStatsListener_NiagaraBaselineComparisonRender::TickRT()
{
	FParticlePerfStatsListener_GatherAll::TickRT();

	if (++NumFramesRT > GPerfTestFrames)
	{
		TArray<UNiagaraSystem*, TInlineAllocator<16>> ToRemove;
		NumFramesRT = 0;
#if WITH_PER_SYSTEM_PARTICLE_PERF_STATS
		FScopeLock Lock(&AccumulatedStatsGuard);
		for (auto& SysStatPair : AccumulatedSystemStats)
		{
			if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(SysStatPair.Key.Get()))
			{
				FAccumulatedParticlePerfStats* Stats = SysStatPair.Value.Get();
				check(Stats);

				FNiagaraPerfBaselineStats& Curr = CurrentStats.FindChecked(System);
				Curr.PerInstanceAvg_RT = Stats->GetRenderThreadStats().GetPerInstanceAvg();

				//Grab the median max index to hopefully avoid any really spikey frames. Especially on PC & Dev builds etc.
				//However this will likely mean we miss high activation times.
				//We should add some other mechanism for tracking activation times more explicitly.
				int32 RTMaxIndex = ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES / 2;
				if (Stats->GetRenderThreadStats().NumFrames < ACCUMULATED_PARTICLE_PERF_STAT_MAX_SAMPLES)
				{
					RTMaxIndex = Stats->GetRenderThreadStats().NumFrames / 2;
				}
				Curr.PerInstanceMax_RT = Stats->GetRenderThreadStats().GetPerInstanceMax(RTMaxIndex);
				Stats->ResetRT();
			}
		}
#endif
	}
}

#include "Engine/Font.h"

int32 FParticlePerfStatsListener_NiagaraBaselineComparisonRender::RenderStats(class UWorld* World, class FViewport* Viewport, class FCanvas* Canvas, int32 /*X*/, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	FScopeLock Lock(&AccumulatedStatsGuard);

	struct FBaselineReportItem
	{
		const UNiagaraSystem* System;
		FNiagaraPerfBaselineStats PerfStats;
		FNiagaraPerfBaselineStats Ratio;
		FNiagaraPerfBaselineStats::EComparisonResult OverallResult = FNiagaraPerfBaselineStats::EComparisonResult::Good;
		FNiagaraPerfBaselineStats::EComparisonResult GTAvgResult = FNiagaraPerfBaselineStats::EComparisonResult::Good;
		FNiagaraPerfBaselineStats::EComparisonResult GTMaxResult = FNiagaraPerfBaselineStats::EComparisonResult::Good;
		FNiagaraPerfBaselineStats::EComparisonResult RTAvgResult = FNiagaraPerfBaselineStats::EComparisonResult::Good;
		FNiagaraPerfBaselineStats::EComparisonResult RTMaxResult = FNiagaraPerfBaselineStats::EComparisonResult::Good;

		bool operator<(const FBaselineReportItem& Other)const
		{
			if ((int32)OverallResult == (int32)Other.OverallResult)
			{
				return Ratio.PerInstanceAvg_GT > Other.Ratio.PerInstanceAvg_GT;
			}

			return (int32)OverallResult > (int32)Other.OverallResult;
		}
	};

	TArray<FBaselineReportItem> Items;
	Items.Reserve(CurrentStats.Num());

	int32 NumRows = 0;
	for (auto it = CurrentStats.CreateIterator(); it; ++it)
	{
		if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(it.Key().Get()))
		{
			FNiagaraPerfBaselineStats& PerfStats = it.Value();
			UNiagaraEffectType* FXType = System->GetEffectType();

			if (PerfStats.IsValid())
			{
				++NumRows;
				FBaselineReportItem& ReportItem = Items.AddDefaulted_GetRef();
				ReportItem.System = System;
				ReportItem.PerfStats = PerfStats;
				if (FXType)
				{
					const FNiagaraPerfBaselineStats& Baseline = FXType->GetPerfBaselineStats();
					ReportItem.OverallResult = FNiagaraPerfBaselineStats::Compare(PerfStats, Baseline, ReportItem.Ratio, ReportItem.GTAvgResult, ReportItem.GTMaxResult, ReportItem.RTAvgResult, ReportItem.RTMaxResult);
				}
			}
		}
	};

	Items.Sort();


	UFont* Font = GEngine->GetSmallFont();
	check(Font != nullptr);

	float CharWidth = 0.0f;
	float CharHeight = 0.0f;
	Font->GetCharSize('W', CharWidth, CharHeight);
	const int32 NameWidth = FMath::RoundToInt(32.0f * CharWidth);
	const int32 ColumnWidth = FMath::RoundToInt(6.0f * CharWidth);
	const int32 FontHeight = FMath::TruncToInt(Font->GetMaxCharHeight() + 2.0f);

	const int32 BaseX = 50;
	int32 X = BaseX + NameWidth;

	// Draw background
	static FLinearColor HeaderBackground = FLinearColor(0.2f, 0.2f, 0.2f, 0.5f);
	static FLinearColor BackgroundColors[] = { FLinearColor(0.6f, 0.6f, 0.6f, 0.7f), FLinearColor(0.4f, 0.4f, 0.4f, 0.7f) };

	FLinearColor HeaderColor = FLinearColor::White;

	// Display Header
	Canvas->DrawTile(BaseX - 2, Y - 1, (NameWidth + ColumnWidth * 4) + 4, FontHeight, 0.0f, 0.0f, 1.0f, 1.0f, HeaderBackground);
	Canvas->DrawShadowedString(BaseX, Y, TEXT("System Name"), Font, HeaderColor);
	Canvas->DrawShadowedString(X + ColumnWidth * 0, Y, TEXT("GT Avg"), Font, HeaderColor);
	Canvas->DrawShadowedString(X + ColumnWidth * 1, Y, TEXT("GT Max"), Font, HeaderColor);
	Canvas->DrawShadowedString(X + ColumnWidth * 2, Y, TEXT("RT Avg"), Font, HeaderColor);
	Canvas->DrawShadowedString(X + ColumnWidth * 3, Y, TEXT("RT Max"), Font, HeaderColor);
	Y += FontHeight;

	FString tempString;
	int32 RowNum = 0;
	for (FBaselineReportItem& ReportItem : Items)
	{
		FNiagaraPerfBaselineStats& PerfStats = ReportItem.PerfStats;
		const UNiagaraSystem* System = ReportItem.System;
		UNiagaraEffectType* FXType = System->GetEffectType();

		FLinearColor OverallColor = FNiagaraPerfBaselineStats::GetComparisonResultColor(ReportItem.OverallResult);

		// Background
		++RowNum;
		Canvas->DrawTile(BaseX - 2, Y - 1, (NameWidth + ColumnWidth * 4) + 4, FontHeight, 0.0f, 0.0f, 1.0f, 1.0f, BackgroundColors[RowNum & 1]);

		// Baseline
		if (FXType)
		{
		// System Name
		FString SystemName = System->GetFName().ToString();
		Canvas->DrawShadowedString(BaseX, Y, *SystemName, Font, OverallColor);

			if(FXType->IsPerfBaselineValid())			
			{
				FLinearColor ResultColor = FNiagaraPerfBaselineStats::GetComparisonResultColor(ReportItem.GTAvgResult);
				FText ResultText = FNiagaraPerfBaselineStats::GetComparisonResultText(ReportItem.GTAvgResult);
				Canvas->DrawShadowedString(X + ColumnWidth * 0, Y, *ResultText.ToString(), Font, ResultColor);

				ResultColor = FNiagaraPerfBaselineStats::GetComparisonResultColor(ReportItem.GTMaxResult);
				ResultText = FNiagaraPerfBaselineStats::GetComparisonResultText(ReportItem.GTMaxResult);
				Canvas->DrawShadowedString(X + ColumnWidth * 1, Y, *ResultText.ToString(), Font, ResultColor);

				ResultColor = FNiagaraPerfBaselineStats::GetComparisonResultColor(ReportItem.RTAvgResult);
				ResultText = FNiagaraPerfBaselineStats::GetComparisonResultText(ReportItem.RTAvgResult);
				Canvas->DrawShadowedString(X + ColumnWidth * 2, Y, *ResultText.ToString(), Font, ResultColor);

				ResultColor = FNiagaraPerfBaselineStats::GetComparisonResultColor(ReportItem.RTMaxResult);
				ResultText = FNiagaraPerfBaselineStats::GetComparisonResultText(ReportItem.RTMaxResult);
				Canvas->DrawShadowedString(X + ColumnWidth * 3, Y, *ResultText.ToString(), Font, ResultColor);

				if (UNiagaraBaselineController* Controller = FXType->GetPerfBaselineController())
				{
					if (UNiagaraSystem* BaseSystem = Controller->GetSystem())
					{
						if (BaseSystem->bFixedBounds == false)
						{
							Canvas->DrawShadowedString(X + ColumnWidth * 4, Y, TEXT("Base System Dynamic Bounds!"), Font, FColor::Yellow);
						}
					}
					else
					{
						Canvas->DrawShadowedString(X + ColumnWidth * 4, Y, TEXT("Missing Baseline System!"), Font, FColor::Yellow);
					}
				}
			}
			else
			{
				tempString.Reset();
				if (FXType->GetPerfBaselineController())
				{
					tempString.Append(TEXT("Baseline is invalid."));
				}
				else
				{
					tempString.Appendf(TEXT("No perf baseline controller for FXType: %s"), *FXType->GetName());
				}
				Canvas->DrawShadowedString(X + ColumnWidth * 0, Y, *tempString, Font, OverallColor);
			}
		}
		else
		{
			// System Name
			FString SystemName = System->GetFName().ToString();
			Canvas->DrawShadowedString(BaseX, Y, *SystemName, Font, FColor::Blue);

			tempString.Reset();
			tempString.Appendf(TEXT("System has no Effect Type!"));
			Canvas->DrawShadowedString(X + ColumnWidth * 0, Y, *tempString, Font, FColor::Blue);
		}

		Y += FontHeight;
	}

	return Y;
}

//////////////////////////////////////////////////////////////////////////

FNiagaraPerfBaselineHandler::FNiagaraPerfBaselineHandler()
{
	if (GEngine)
	{
		GEngine->AddEngineStat(TEXT("STAT_NiagaraBaselines"), TEXT("STATCAT_Niagara"), LOCTEXT("NiagaraPerfBaselineStatDesc", "Displays stats on how currently running systems compare to their performance baselines."),
			UEngine::FEngineStatRender::CreateRaw(this, &FNiagaraPerfBaselineHandler::RenderStatPerfBaselines), UEngine::FEngineStatToggle::CreateRaw(this, &FNiagaraPerfBaselineHandler::ToggleStatPerfBaselines), false);
	}
}

FNiagaraPerfBaselineHandler::~FNiagaraPerfBaselineHandler()
{
	if (PerfBaselineListener.IsValid())
	{
		FParticlePerfStatsManager::RemoveListener(PerfBaselineListener);
		PerfBaselineListener.Reset();
	}
	if (DebugRenderListener.IsValid())
	{
		FParticlePerfStatsManager::RemoveListener(DebugRenderListener);
		DebugRenderListener.Reset();
	}
}

void FNiagaraPerfBaselineHandler::OnWorldBeginTearDown(UWorld* World)
{
	if (PerfBaselineListener.IsValid())
	{
		PerfBaselineListener->ReportToLog();
		FParticlePerfStatsManager::RemoveListener(PerfBaselineListener);
		PerfBaselineListener.Reset();
	}

	//In cooked games we invalidate and force regeneration of perf baselines on all level transitions.
#if !WITH_EDITOR
	if (GbNiagaraRegenerateBaselinesOnWorldChange)
	{
		for (TObjectIterator<UNiagaraEffectType> It; It; ++It)
		{
			It->InvalidatePerfBaseline();
		}
	}
#endif
}

void FNiagaraPerfBaselineHandler::Tick(UWorld* World, float DeltaSeconds)
{
	//Add/Remove the perf baseline listener where needed.
	bool bCanGenBaseliensForWorld = false;
	if (World->IsGameWorld() && World->HasBegunPlay() && World->bMatchStarted)
	{
		bCanGenBaseliensForWorld = true;
		if (GbNiagaraPerfReporting > 0)
		{
			if (PerfBaselineListener.IsValid() == false)
			{
				PerfBaselineListener = MakeShared<FParticlePerfStatsListener_NiagaraPerformanceReporter, ESPMode::ThreadSafe>(World);
				FParticlePerfStatsManager::AddListener(PerfBaselineListener);
			}
		}
		else
		{
			if (PerfBaselineListener.IsValid())
			{
				FParticlePerfStatsManager::RemoveListener(PerfBaselineListener);
				PerfBaselineListener.Reset();
			}
		}
	}

	if (PerfBaselineListener.IsValid() || DebugRenderListener.IsValid())
	{
		//Ensure any perf baselines are generated if they're needed.
		//Bit of a hack but this can either call into the NiagaraModule in cooked games which will trigger the below gen code.
		//Or in editor builds this will call into the NiagaraEditor module and generate the baselines in their own window.
		UNiagaraEffectType::GeneratePerfBaselines();
	}

	if (BaselinesToGenerate.Num() > 0 && bCanGenBaseliensForWorld)
	{
		if (BaselineGenerationState == EBaselineGenState::Begin)
		{
			BaselineGenerationState = EBaselineGenState::WaitingToGenerate;
			WorldTimeToGenerate = World->GetTimeSeconds() + GBaselineGenerationDelay;
			BaselineGenWorld = World;			
		}
		
		if (BaselineGenWorld.IsValid())
		{
			if (World == BaselineGenWorld)
			{
				if (BaselineGenerationState == EBaselineGenState::WaitingToGenerate)
				{
					static const int32 GenBaselineMessageID = FCrc::Strihash_DEPRECATED(TEXT("NiagaraGenBaselineMessageID"));
					if (World->GetTimeSeconds() >= WorldTimeToGenerate)
					{
						BaselineGenerationState = EBaselineGenState::Generating;

						FString Message = TEXT("Generating Niagara Perf Baselines...");
						GEngine->AddOnScreenDebugMessage(GenBaselineMessageID, 4.0f, FColor::White, Message);
					}
					else
					{
						FString Message = FString::Printf(TEXT("Generating Niagara Perf Baselines in %g seconds"), WorldTimeToGenerate - World->GetTimeSeconds());
						GEngine->AddOnScreenDebugMessage(GenBaselineMessageID, .5f, FColor::White, Message);
					}
				}

				if (BaselineGenerationState == EBaselineGenState::Generating)
				{
					for (TWeakObjectPtr<UNiagaraEffectType>& WeakFXType : BaselinesToGenerate)
					{
						if (UNiagaraEffectType* FXType = WeakFXType.Get())
						{
							if (FXType->IsPerfBaselineValid() == false)
							{
								FXType->SpawnBaselineActor(World);
							}
						}
					}

					BaselineGenerationState = EBaselineGenState::Complete;
					BaselinesToGenerate.Reset();
					BaselineGenWorld.Reset();
				}
			}
		}
		else
		{
			BaselineGenerationState = EBaselineGenState::Complete;
			BaselinesToGenerate.Reset();
			BaselineGenWorld.Reset();
		}
	}
}

void FNiagaraPerfBaselineHandler::GenerateBaselines(TArray<UNiagaraEffectType*>& InBaselinesToGenerate)
{
	if (InBaselinesToGenerate.Num() > 0)
	{
		BaselinesToGenerate.Reserve(InBaselinesToGenerate.Num());
		for (auto Item : InBaselinesToGenerate)
		{
			BaselinesToGenerate.AddUnique(Item);
		}

		if (BaselineGenerationState == EBaselineGenState::Complete)
		{
			BaselineGenerationState = EBaselineGenState::Begin;
		}
	}
}

bool FNiagaraPerfBaselineHandler::ToggleStatPerfBaselines(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	if (DebugRenderListener.IsValid())
	{
		FParticlePerfStatsManager::RemoveListener(DebugRenderListener);
		DebugRenderListener.Reset();
	}
	else if (DebugRenderListener.IsValid() == false)
	{
		DebugRenderListener = MakeShared<FParticlePerfStatsListener_NiagaraBaselineComparisonRender, ESPMode::ThreadSafe>();
		FParticlePerfStatsManager::AddListener(DebugRenderListener);

		//Ensure perf baselines are generated.
		UNiagaraEffectType::GeneratePerfBaselines();
	}
	return false;
}


int32 FNiagaraPerfBaselineHandler::RenderStatPerfBaselines(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	if (DebugRenderListener.IsValid() == false)
	{
		DebugRenderListener = MakeShared<FParticlePerfStatsListener_NiagaraBaselineComparisonRender, ESPMode::ThreadSafe>();
		FParticlePerfStatsManager::AddListener(DebugRenderListener);

		//Ensure perf baselines are generated.
		UNiagaraEffectType::GeneratePerfBaselines();
	}

	if (ensure(DebugRenderListener.IsValid()))
	{
		Y = DebugRenderListener->RenderStats(World, Viewport, Canvas, X, Y, ViewLocation, ViewRotation);
	}
	return Y;
}


#else//NIAGARA_PERF_BASELINES

ANiagaraPerfBaselineActor::ANiagaraPerfBaselineActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

void UNiagaraBaselineController::OnBeginTest_Implementation(){}
bool UNiagaraBaselineController::OnTickTest_Implementation(){ return false; }
void UNiagaraBaselineController::OnEndTest_Implementation(FNiagaraPerfBaselineStats Stats){}
void UNiagaraBaselineController::OnOwnerTick_Implementation(float DeltaTime){}
UNiagaraSystem* UNiagaraBaselineController::GetSystem(){ return nullptr; }

void UNiagaraBaselineController_Basic::OnBeginTest_Implementation(){}
bool UNiagaraBaselineController_Basic::OnTickTest_Implementation(){	return false; }
void UNiagaraBaselineController_Basic::OnEndTest_Implementation(FNiagaraPerfBaselineStats Stats){}
void UNiagaraBaselineController_Basic::OnOwnerTick_Implementation(float DeltaTime){}


#endif//NIAGARA_PERF_BASELINES


#undef LOCTEXT_NAMESPACE

