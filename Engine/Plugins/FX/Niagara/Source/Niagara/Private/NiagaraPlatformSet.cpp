// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraPlatformSet.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "NiagaraEmitter.h"
#include "Scalability.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigUtilities.h"
#include "NiagaraSystemImpl.h"
#include "NiagaraSettings.h"
#include "UObject/UObjectIterator.h"
#include "NiagaraScalabilitySettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraPlatformSet)

#if WITH_EDITOR
#endif

#define LOCTEXT_NAMESPACE "NiagaraPlatformSet"

//Array of CVars that have been changed this frame.
//We do a deferred refresh of systems that use these CVars for scalability.
TArray<FName> FNiagaraPlatformSet::ChangedCVars;

TMap<FName, FNiagaraPlatformSet::FCachedCVarInfo> FNiagaraPlatformSet::CachedCVarInfo;
FCriticalSection FNiagaraPlatformSet::CachedCVarInfoCritSec;

/**
Whether a platform can change it's scalability settings at runtime.
Defaults to false for all platforms and is explicitly enabled for desktop platforms.
*/
const TCHAR* CanChangeEQCVarName = TEXT("fx.NiagaraAllowRuntimeScalabilityChanges");
int32 GbNiagaraAllowRuntimeScalabilityChanges = 0;
static FAutoConsoleVariableRef CVarNiagaraAllowRuntimeScalabilityChanges(
	CanChangeEQCVarName,
	GbNiagaraAllowRuntimeScalabilityChanges,
	TEXT("If > 0 this platform allows niagara scalability settings changes at runtime. \n"),
	ECVF_Scalability
);

const TCHAR* PruneEmittersOnCookName = TEXT("fx.Niagara.PruneEmittersOnCook");
int32 GbPruneEmittersOnCook = 1;
static FAutoConsoleVariableRef CVarPruneEmittersOnCook(
	PruneEmittersOnCookName,
	GbPruneEmittersOnCook,
	TEXT("If > 0 this platform will prune disabled emitters during cook. \n"),
	ECVF_Scalability
);

static const int32 DefaultQualityLevel = 3;
const TCHAR* NiagaraQualityLevelName = TEXT("fx.Niagara.QualityLevel");
int32 GNiagaraQualityLevel = DefaultQualityLevel;

static FAutoConsoleCommand GCmdSetNiagaraQualityLevelOverride(
	TEXT("fx.Niagara.SetOverrideQualityLevel"),
	TEXT("Sets which quality level we should override with, no args means clear the override and return to non overriden quality level). Valid levels are 0-4 (Low-Cinematic)"),
	FConsoleCommandWithArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args)
		{
			if(Args.Num() == 0)
			{
				FNiagaraPlatformSet::ClearNiagaraQualityLevelOverride();
				UE_LOGF(LogNiagara, Display, "Clearing the Niagara Quality Level override and returning quality level to %ls", *FNiagaraPlatformSet::GetQualityLevelText(FNiagaraPlatformSet::GetQualityLevel()).ToString());
			}
			else if(Args.Num() == 1)
			{
				if(FCString::IsNumeric(*Args[0]))
				{
					int32 NewQualityLevel = FCString::Atoi(*Args[0]);
					if(ensure(NewQualityLevel >= 0 && NewQualityLevel <= 4))
					{
						FNiagaraPlatformSet::SetNiagaraQualityLevelOverride(NewQualityLevel);
						UE_LOGF(LogNiagara, Display, "Set the global Niagara quality level to %ls", *FNiagaraPlatformSet::GetQualityLevelText(FNiagaraPlatformSet::GetQualityLevel()).ToString());
					}
				}
			}
		}
	)
); 

static FAutoConsoleVariableRef CVarNiagaraQualityLevel(
	NiagaraQualityLevelName,
	GNiagaraQualityLevel,
	TEXT("The quality level for Niagara Effects. \n"),
	ECVF_Scalability
);

void FNiagaraPlatformSet::SetNiagaraQualityLevelOverride(int32 Override) 
{
	if(Override != QualityLevelOverride)
	{
		QualityLevelOverride = Override; 
		InvalidateCachedData();
		RefreshScalability();
	}
}

void FNiagaraPlatformSet::ClearNiagaraQualityLevelOverride() 
{
	SetNiagaraQualityLevelOverride(INDEX_NONE);
};

// Sync function for checking quality level hasn't changed
// This must be done at the end of all console variable changes otherwise we can encounter multiple changes to the Q level resulting in some systems being left deactivated
FAutoConsoleVariableSink CVarNiagaraQualityLevelSync(FConsoleCommandDelegate::CreateStatic(&FNiagaraPlatformSet::RefreshScalability));

/** Minimum quality level available for a platform. Defaults to -1, indicating there is no minimum. */
static const int32 DefaultMinQualityLevel = INDEX_NONE;
const TCHAR* NiagaraMinQualityLevelName = TEXT("fx.Niagara.QualityLevel.Min");
int32 GNiagaraMinQualityLevel = DefaultMinQualityLevel;
static FAutoConsoleVariableRef CVarNiagaraMinQualityLevel(
	NiagaraMinQualityLevelName,
	GNiagaraMinQualityLevel,
	TEXT("The minimum quality level for Niagara Effects. \n"),
	ECVF_ReadOnly
);

/** Maximum quality level available for a platform. Defaults to -1, indicating there is no maximum. */
static const int32 DefaultMaxQualityLevel = INDEX_NONE;
const TCHAR* NiagaraMaxQualityLevelName = TEXT("fx.Niagara.QualityLevel.Max");
int32 GNiagaraMaxQualityLevel = DefaultMaxQualityLevel;
static FAutoConsoleVariableRef CVarNiagaraMaxQualityLevel(
	NiagaraMaxQualityLevelName,
	GNiagaraMaxQualityLevel,
	TEXT("The Maximum quality level for Niagara Effects. \n"),
	ECVF_ReadOnly
);

int32 GbAllowAllDeviceProfiles = 0;
static FAutoConsoleVariableRef CVarAllowAllDeviceProfiles(
	TEXT("fx.Niagara.AllowAllDeviceProfiles"),
	GbAllowAllDeviceProfiles,
	TEXT(" \n"),
	ECVF_Default
);

/** 
This is a special case CVar that allows us to use CVar conditions to maintain behavior with legacy device profiles.
Do not use directly for new content.
Legacy device profiles can be given a specific value for this CVar and then CVar conditions used to enable/disable as appropriate to match with legacy assets with enabled/disabled content based on them.
*/
int32 GNiagaraLegacyDeviceProfile = INDEX_NONE;
static FAutoConsoleVariableRef CVarNiagaraLegacyDeviceProfile(
	TEXT("fx.Niagara.LegacyDeviceProfile"),
	GNiagaraLegacyDeviceProfile,
	TEXT("This is a special case CVar that allows us to use CVar conditions to maintain behavior with legacy device profiles.\nDo not use directly for new content.\nLegacy device profiles can be given a specific value for this CVarand then CVar conditions used to enable / disable as appropriate to match with legacy assets with enabled / disabled content based on them. \n"),
	ECVF_Scalability
);

// Override platform device profile
// In editor all profiles will be available
// On cooked builds only the profiles for that cooked platform will be available
TWeakObjectPtr<UDeviceProfile> GNiagaraPlatformOverride;

static int32 GNiagaraBackupQualityLevel = INDEX_NONE;

static FAutoConsoleCommand GCmdSetNiagaraPlatformOverride(
	TEXT("fx.Niagara.SetOverridePlatformName"),
	TEXT("Sets which platform we should override with, no args means reset to default"),
	FConsoleCommandWithArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args)
		{
			GNiagaraPlatformOverride.Reset();
			if (Args.Num() == 0)
			{
				if (GNiagaraBackupQualityLevel != INDEX_NONE)
				{
					UE::ConfigUtilities::OnSetCVarFromIniEntry(*GDeviceProfilesIni, NiagaraQualityLevelName, *LexToString(GNiagaraBackupQualityLevel), ECVF_SetByMask);
				}
				GNiagaraBackupQualityLevel = INDEX_NONE;
				UE_LOGF(LogNiagara, Warning, "Niagara Clearing Override DeviceProfile");
			}
			else
			{
				for (UObject* DeviceProfileObj : UDeviceProfileManager::Get().Profiles)
				{
					if (UDeviceProfile* Profile = Cast<UDeviceProfile>(DeviceProfileObj))
					{
						if (Profile->GetName() == Args[0])
						{
							SetGNiagaraDeviceProfile(Profile);
							break;
						}
					}
				}

				if (GNiagaraPlatformOverride.IsValid())
				{
					UE_LOGF(LogNiagara, Warning, "Niagara Setting Override DeviceProfile '%ls'", *Args[0]);
				}
				else
				{
					UE_LOGF(LogNiagara, Warning, "Niagara Failed to Find Override DeviceProfile '%ls'", *Args[0]);
				}
			}
		}
	)
);

static UDeviceProfile* NiagaraGetActiveDeviceProfile()
{
	UDeviceProfileManager& DPMan = UDeviceProfileManager::Get();
	if (UDeviceProfile* DPPreview = DPMan.GetPreviewDeviceProfile())
	{
		return DPPreview;
	}

	UDeviceProfile* ActiveProfile = GNiagaraPlatformOverride.Get(); 
	if (ActiveProfile == nullptr)
	{
		ActiveProfile = UDeviceProfileManager::Get().GetActiveProfile();
	}
	return ActiveProfile;
}

int32 FNiagaraPlatformSet::CachedQualityLevel = INDEX_NONE;
int32 FNiagaraPlatformSet::CachedAvailableQualityLevelMask = INDEX_NONE;
int32 FNiagaraPlatformSet::QualityLevelOverride = INDEX_NONE;

int32 FNiagaraPlatformSet::GetQualityLevel()
{
	if(QualityLevelOverride != INDEX_NONE)
	{
		return QualityLevelOverride;
	}

	if (CachedQualityLevel == INDEX_NONE)
	{
		//Return GNiagaraQualityLevel so that we have a valid level but do not update cached quality level. 
		//This must be kept in order to trigger a proper refresh of scalability state.
		//It is possible that InvalidateCachedData to be called and we reach here before we hit our own CVar Sinks and refresh handling.
		return GNiagaraQualityLevel;
	}

	return CachedQualityLevel;
}

int32 FNiagaraPlatformSet::GetMinQualityLevel()
{
	return GNiagaraMinQualityLevel;
}

int32 FNiagaraPlatformSet::GetMaxQualityLevel()
{
	return GNiagaraMaxQualityLevel;
}

int32 FNiagaraPlatformSet::GetAvailableQualityLevelMask()
{
	if (CachedAvailableQualityLevelMask == INDEX_NONE)
	{
		if (CanChangeScalabilityAtRuntime())
		{
			CachedAvailableQualityLevelMask = INDEX_NONE;
		}
		else
		{
			CachedAvailableQualityLevelMask = 0;
			int32 Min = GetMinQualityLevel();
			int32 Max = GetMaxQualityLevel();

			if (Min == INDEX_NONE && Max == INDEX_NONE)
			{
				CachedAvailableQualityLevelMask = 0xFFFFFFFF;
			}
			else
			{
				Min = Min == INDEX_NONE ? 0 : Min;
				Max = Max == INDEX_NONE ? 31 : Max;
				
				for (int32 QL = Min; QL <= Max; ++QL)
				{
					CachedAvailableQualityLevelMask |= (1 << QL);
				}
			}
		}
	}
	return CachedAvailableQualityLevelMask;
}

int32 FNiagaraPlatformSet::GetFullQualityLevelMask(int32 NumQualityLevels)
{
	int32 QualityLevelMask = 0;
	
	for (int32 QL = 0; QL < NumQualityLevels; ++QL)
	{
		QualityLevelMask |= (1 << QL);
	}

	return QualityLevelMask;
}

uint32 FNiagaraPlatformSet::LastDirtiedFrame = 0;
#if WITH_EDITOR
FCriticalSection FNiagaraPlatformSet::CachedPlatformIniCritSec;
TMap<const UDeviceProfile*, int32> FNiagaraPlatformSet::CachedQLMasksPerDeviceProfile;
TMap<FName, FNiagaraPlatformSet::FPlatformIniSettings> FNiagaraPlatformSet::CachedPlatformIniSettings;
#endif

FText FNiagaraPlatformSet::GetQualityLevelText(int32 QualityLevel)
{
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);

	if (Settings->QualityLevels.IsValidIndex(QualityLevel))
	{
		return Settings->QualityLevels[QualityLevel];
	}	
	else
	{
		return FText::AsNumber(QualityLevel);
	}
}

FText FNiagaraPlatformSet::GetQualityLevelMaskText(int32 QualityLevelMask)
{
	if (QualityLevelMask == INDEX_NONE)
	{
		return LOCTEXT("QualityLevelAll", "All");
	}
	else if (QualityLevelMask == 0)
	{
		return LOCTEXT("QualityLevelNone", "None");
	}
	else
	{
		return GetQualityLevelText(QualityLevelFromMask(QualityLevelMask));
	}
}

FNiagaraPlatformSet::FNiagaraPlatformSet(int32 QLMask)
	: QualityLevelMask(QLMask)
	, bEnabledForCurrentProfileAndEffectQuality(false)
	, LastBuiltFrame(0)
{
	IsActive();
}

FNiagaraPlatformSet::FNiagaraPlatformSet()
	: QualityLevelMask(INDEX_NONE)
	, bEnabledForCurrentProfileAndEffectQuality(false)
	, LastBuiltFrame(0)
{
}

bool FNiagaraPlatformSet::operator==(const FNiagaraPlatformSet& Other)const
{
	return QualityLevelMask == Other.QualityLevelMask && DeviceProfileStates == Other.DeviceProfileStates;
}

bool FNiagaraPlatformSet::IsActive()const
{
	if (LastBuiltFrame <= LastDirtiedFrame || LastBuiltFrame == 0)
	{
		UDeviceProfile* ActiveProfile = NiagaraGetActiveDeviceProfile();
		int32 QualityLevel = GetQualityLevel();
#if WITH_EDITOR
		if(OnOverrideQualityLevelDelegate.IsBound())
		{
			QualityLevel = OnOverrideQualityLevelDelegate.Execute();
		}
		if(OnOverrideActiveDeviceProfileDelegate.IsBound())
		{
			TOptional<TObjectPtr<UDeviceProfile>> OverrideProfile = OnOverrideActiveDeviceProfileDelegate.Execute();
			if(OverrideProfile.IsSet())
			{
				ActiveProfile = OverrideProfile.GetValue();
			}
		}
#endif
		FNiagaraPlatformSetEnabledState EnabledState = IsEnabled(ActiveProfile, QualityLevel, true);
		bEnabledForCurrentProfileAndEffectQuality = EnabledState.bIsActive;
		LastBuiltFrame = GFrameNumber;
	}
	return bEnabledForCurrentProfileAndEffectQuality;
}

int32 FNiagaraPlatformSet::GetEnabledMaskForDeviceProfile(const UDeviceProfile* DeviceProfile)const
{
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);

	int32 RetQLMask = 0;
	for (int32 i = 0; i < Settings->QualityLevels.Num(); ++i)
	{
		FNiagaraPlatformSetEnabledState EnabledState = IsEnabled(DeviceProfile, i, false);
		if (EnabledState.bCanBeActive)
		{
			RetQLMask |= CreateQualityLevelMask(i);
		}
	}

	return RetQLMask;
}

int32 FNiagaraPlatformSet::IsEnabledForDeviceProfile(const UDeviceProfile* DeviceProfile)const
{
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);

	int32 RetQLMask = 0;
	for (int32 i = 0; i < Settings->QualityLevels.Num(); ++i)
	{
		FNiagaraPlatformSetEnabledState Enabled = IsEnabled(DeviceProfile, i, false);
		if (Enabled.bCanBeActive)
		{
			RetQLMask |= CreateQualityLevelMask(i);
		}
	}

	return RetQLMask;
}

bool FNiagaraPlatformSet::IsEnabledForQualityLevel(int32 QualityLevel)const
{
	for (UObject* DeviceProfileObj : UDeviceProfileManager::Get().Profiles)
	{
		if (UDeviceProfile* Profile = Cast<UDeviceProfile>(DeviceProfileObj))
		{
			FNiagaraPlatformSetEnabledState Enabled = IsEnabled(Profile, QualityLevel, false);
			if (Enabled.bCanBeActive)
			{
				return true;
			}
		}
	}

	return false;
}

void FNiagaraPlatformSet::GetOverridenDeviceProfiles(int32 QualityLevel, TArray<UDeviceProfile*>& OutEnabledProfiles, TArray<UDeviceProfile*>& OutDisabledProfiles)const
{
	int32 QLMask = CreateQualityLevelMask(QualityLevel);
	for (const FNiagaraDeviceProfileStateEntry& Entry : DeviceProfileStates)
	{
		if (TObjectPtr<UDeviceProfile>* DeviceProfile = UDeviceProfileManager::Get().Profiles.FindByPredicate([&](UObject* CheckProfile) {return CheckProfile->GetFName() == Entry.ProfileName;}))
		{
			UDeviceProfile* Profile = *DeviceProfile;
			if (CanConsiderDeviceProfile(Profile))
			{
				//If this platform cannot change at runtime then we store all EQs in the state so that the device is still overridden if someone changes it's EQ CVar.
				//So here we must also check that this QualityLevel is the right one for the platforms current setting.
				int32 ProfileQLMask = GetAvailableQualityMaskForDeviceProfile(Profile);
				if (ProfileQLMask == INDEX_NONE || (QLMask & ProfileQLMask) != 0)
				{
					ENiagaraPlatformSelectionState State = Entry.GetState(QualityLevel);
					if (State == ENiagaraPlatformSelectionState::Enabled)
					{
						OutEnabledProfiles.Add(Profile);
					}
					else if (State == ENiagaraPlatformSelectionState::Disabled)
					{
						OutDisabledProfiles.Add(Profile);
					}
				}
			}
		}
	}
}

bool FNiagaraPlatformSet::CanChangeScalabilityAtRuntime()
{
	//For the current platform we can just read direct as this CVar is readonly.
	return GbNiagaraAllowRuntimeScalabilityChanges != 0;
}

/** Returns the currenlty(or default) active quality mask for a profile. */
int32 FNiagaraPlatformSet::GetActiveQualityMaskForDeviceProfile(const UDeviceProfile* Profile)
{
#if WITH_EDITOR
	FScopeLock Lock(&CachedPlatformIniCritSec);

	if (int32* CachedQLForProfile = CachedQLMasksPerDeviceProfile.Find(Profile))
	{
		return *CachedQLForProfile;
	}

	if (!ensureMsgf(IsInGameThread(), TEXT("GetActiveQualityMaskForDeviceProfile is attempting to access the config system outside of the game thread, results may be incorrect.")))
	{
		return INDEX_NONE;
	}

	FPlatformIniSettings& PlatformSettings = GetPlatformIniSettings(Profile->DeviceType);
	int32 QLMask = PlatformSettings.QualityLevelMask;

	if (PlatformSettings.bCanChangeScalabilitySettingsAtRuntime)
	{
		QLMask = INDEX_NONE;
	}
	else
	{
		//Check if the DPs set Niagara quality directly.
		int32 QualityLevel = INDEX_NONE;
		if (!Profile->GetConsolidatedCVarValue(NiagaraQualityLevelName, QualityLevel))
		{
			QualityLevel = DefaultQualityLevel;
			//If not, grab it from the effects quality setting.
			int32 EffectsQuality = INDEX_NONE;
			//See if this profile overrides effects quality.
			if (!Profile->GetConsolidatedCVarValue(TEXT("sg.EffectsQuality"), EffectsQuality))
			{
				EffectsQuality = PlatformSettings.EffectsQuality;
			}
			check(EffectsQuality != INDEX_NONE);

			QualityLevel = PlatformSettings.QualityLevelsPerEffectsQuality[EffectsQuality];
		}
		check(QualityLevel != INDEX_NONE);
		QLMask = CreateQualityLevelMask(QualityLevel);
	}

	//UE_LOGF(LogNiagara, Warning, "%ls - DeviceProfile(%d)", *Profile->GetName(), QLMask);

	CachedQLMasksPerDeviceProfile.Add(Profile) = QLMask;
	return QLMask;

#else

	//When not in editor we can assume we're asking about the current platform.
	check(Profile == NiagaraGetActiveDeviceProfile());
	bool bCanChangeEQAtRuntime = CanChangeScalabilityAtRuntime();

	return CreateQualityLevelMask(bCanChangeEQAtRuntime ? INDEX_NONE : GetQualityLevel());

#endif
}

/** Returns the mask of all available quality levels for a device profile. */
int32 FNiagaraPlatformSet::GetAvailableQualityMaskForDeviceProfile(const UDeviceProfile* Profile)
{
#if WITH_EDITOR
	
	//We can no longer rely on trawling the device profiles and ini files for all the quality levels a platform can be at
	//So we rely on manual min/max values from the inis
	FPlatformIniSettings& PlatformSettings = GetPlatformIniSettings(Profile->DeviceType);
	return PlatformSettings.QualityLevelMask;

#else
	
	//In non editor builds we can assume we mean the current platform.
	return GetAvailableQualityLevelMask();

#endif
}

FNiagaraPlatformSetEnabledState FNiagaraPlatformSet::IsEnabled(const UDeviceProfile* Profile, int32 QualityLevel, bool bCheckCurrentStateOnly, FNiagaraPlatformSetEnabledStateDetails* OutDetails)const
{
	checkSlow(Profile);

	FNiagaraPlatformSetEnabledState Ret;

	bool bCanChangeScalabilityAtRuntime = CanChangeScalabilityAtRuntime(Profile);

	//First get the base settings from the Quality Levels.
	int32 ActiveProfileMask = GetActiveQualityMaskForDeviceProfile(Profile);
	int32 ActiveProfileQuality = QualityLevelFromMask(ActiveProfileMask);
	int32 AvailableProfileMask = GetAvailableQualityMaskForDeviceProfile(Profile);
	int32 CurrentQLMask = CreateQualityLevelMask(QualityLevel);

	/** Is the current QL enabled for the platform set? */
	bool bQLIsEnabled = CurrentQLMask & QualityLevelMask;

	/** Is this QL the active or default for this profile. */
	bool bProfileActiveQL = (CurrentQLMask & ActiveProfileMask) != 0;

	/** Is this QL available for this profile? */
	bool bProfileAvailableQL = (CurrentQLMask & AvailableProfileMask) != 0;

#if WITH_EDITOR
	if (OutDetails)
	{
		FPlatformIniSettings& PlatformSettings = GetPlatformIniSettings(Profile->DeviceType);

		OutDetails->DefaultQualityMask = ActiveProfileMask;
		OutDetails->AvailableQualityMask = AvailableProfileMask;
	}

	//Does this platform set match the passed in current quality level?
	FText QLText = GetQualityLevelText(QualityLevel);
	FText ActiveQLText = GetQualityLevelText(ActiveProfileQuality);
#endif

	if (bProfileAvailableQL)
	{
		if (bQLIsEnabled)//Is the whole quality level disabled?
		{
			if (bProfileActiveQL || bCheckCurrentStateOnly)//The quality level is enabled but this profile is not active for this QL
			{
				Ret.bIsActive = true;
				Ret.bCanBeActive = true;
			}
			else
			{
				Ret.bIsActive = false;
				Ret.bCanBeActive = true;

#if WITH_EDITOR
				if (OutDetails)
				{
					OutDetails->ReasonsForInActive.Emplace(FText::Format(LOCTEXT("QualityLevelNotActiveReasonFmt", "{0} is enabled at {1} but defaults to {2}."), FText::FromString(Profile->GetName()), QLText, ActiveQLText));
				}
#endif
			}
		}
		else
		{
			Ret.bIsActive = false;
			Ret.bCanBeActive = false;

#if WITH_EDITOR
			if (OutDetails)
			{
				OutDetails->ReasonsForDisabled.Emplace(FText::Format(LOCTEXT("QualityLevelDisabledReasonFmt", "{0} Quality is disabled."), QLText));
			}
#endif
		}
	}
	else
	{
		Ret.bIsActive = false;
		Ret.bCanBeActive = false;

#if WITH_EDITOR
		if (OutDetails)
		{
			FPlatformIniSettings& PlatformSettings = GetPlatformIniSettings(Profile->DeviceType);
			OutDetails->ReasonsForDisabled.Emplace(FText::Format(LOCTEXT("PlatformSetDisabledFmt", "{1} is outside the Min/Max quality for {0}."), QLText, FText::FromString(Profile->GetName())));
		}
#endif
	}

	//Next see if this device profile is specifically overridden.
	if (DeviceProfileStates.Num() > 0)
	{
		//Walk up the parent hierarchy to see if we have an explicit state for this profile set.
		const UDeviceProfile* CurrProfile = Profile;
		while (CurrProfile)
		{
			//if (CanConsiderDeviceProfile(CurrProfile))
			{
				if (const FNiagaraDeviceProfileStateEntry* StateEntry = DeviceProfileStates.FindByPredicate([&](const FNiagaraDeviceProfileStateEntry& ProfileState) {return ProfileState.ProfileName == CurrProfile->GetFName(); }))
				{
					ENiagaraPlatformSelectionState SelectionState = StateEntry->GetState(QualityLevel);
					if (SelectionState != ENiagaraPlatformSelectionState::Default)
					{
						bool bProfileEnabled = SelectionState == ENiagaraPlatformSelectionState::Enabled;
						if (bProfileEnabled)
						{
							if (bProfileAvailableQL)
							{
								Ret.bIsActive = true;
								Ret.bCanBeActive = true;

#if WITH_EDITOR
								if (OutDetails)
								{
									//Clear existing failure details
									OutDetails->ReasonsForInActive.Reset();
									OutDetails->ReasonsForDisabled.Reset();
								}
#endif
							}
						}
						else
						{
							Ret.bIsActive = false;
							Ret.bCanBeActive = false;

							#if WITH_EDITOR
							if (OutDetails)
							{
								if (Profile == CurrProfile)
								{
									if (AvailableProfileMask == INDEX_NONE)
									{
										OutDetails->ReasonsForDisabled.Emplace(FText::Format(LOCTEXT("PlatformSetDPDisabledFmt", "{0} is disabled at {1} Quality."), FText::FromString(CurrProfile->GetName()), QLText));
									}
									else
									{
										OutDetails->ReasonsForDisabled.Emplace(FText::Format(LOCTEXT("PlatformSetDPDisabledFmt2", "{0} is disabled."), FText::FromString(CurrProfile->GetName())));
									}
								}
								else
								{
									if (AvailableProfileMask == INDEX_NONE)
									{
										OutDetails->ReasonsForDisabled.Emplace(FText::Format(LOCTEXT("PlatformSetParentDPDisabledFmt", "{0} is Disabeld at {1} Quality."), FText::FromString(CurrProfile->GetName()), QLText));
									}
									else
									{
										OutDetails->ReasonsForDisabled.Emplace(FText::Format(LOCTEXT("PlatformSetParentDPDisabledFmt2", "{0} is Disabled."), FText::FromString(CurrProfile->GetName())));
									}
								}
							}
							#endif
						}

						break;
					}
				}
			}
			CurrProfile = Cast<UDeviceProfile>(CurrProfile->Parent);
		}
	}

	//Finally, check any CVar conditions placed on this platform set.
	for (const FNiagaraPlatformSetCVarCondition& CVarCondition : CVarConditions)
	{
		//Bail if any cvar condition isn't met.
		bool bCanEverPass = true;
		bool bCanEverFail = false;
		//Get all quality levels where this condition will pass.
		int32 ConditionPassedMask = INDEX_NONE;
		bool bConditionPassed = CVarCondition.Evaluate(Profile, QualityLevel, bCheckCurrentStateOnly, bCanEverPass, bCanEverFail);

		ensure(bCanEverPass || bCanEverFail);

		bool bCanBeActiveDisabled = false;
		bool bIsActiveDistabled = false;
		if(CVarCondition.PassResponse == ENiagaraCVarConditionResponse::Enable && bProfileAvailableQL)
		{
			if (bConditionPassed)
			{
				Ret.bIsActive = true;				
			}
			if (bCanEverPass)
			{
				Ret.bCanBeActive = true;
			}
		}
		else if (CVarCondition.PassResponse == ENiagaraCVarConditionResponse::Disable)
		{
			if (bConditionPassed)
			{
				Ret.bIsActive = false;
				bIsActiveDistabled = true;
			}
			if (bCanEverFail == false)//If this condition will always pass then we'll always be disabled.
			{
				Ret.bCanBeActive = false;
				bCanBeActiveDisabled = true;
			}
		}

		if (CVarCondition.FailResponse == ENiagaraCVarConditionResponse::Enable && bProfileAvailableQL)
		{
			if (bConditionPassed == false)
			{
				Ret.bIsActive = true;
			}
			if (bCanEverFail)
			{
				Ret.bCanBeActive = true;
			}
		}
		else if (CVarCondition.FailResponse == ENiagaraCVarConditionResponse::Disable)
		{
			if (bConditionPassed == false)
			{
				Ret.bIsActive = false;
				bIsActiveDistabled = true;
			}
			if (bCanEverPass == false)//If we'll always fail than this platform set will never be active.
			{
				Ret.bCanBeActive = false;
				bCanBeActiveDisabled = true;
			}
		}

#if WITH_EDITOR
		if(OutDetails)
		{
			if (bIsActiveDistabled)
			{
				FText ReasonTest = FText::Format(LOCTEXT("CVarConditionDisabledFmt","{0} is inactive due to a condition on CVar {1}"), FText::FromString(Profile->GetName()), FText::FromName(CVarCondition.CVarName));
				OutDetails->ReasonsForInActive.Emplace(ReasonTest);
			}
			if (bCanBeActiveDisabled)
			{
				FText ReasonTest = FText::Format(LOCTEXT("CVarConditionDisabledFmt2", "{0} is disabled due to a condition on CVar {1}"), FText::FromString(Profile->GetName()), FText::FromName(CVarCondition.CVarName));
				OutDetails->ReasonsForDisabled.Emplace(ReasonTest);
			}
		}
#endif
	}

	return Ret;
}

bool FNiagaraPlatformSet::ApplyRedirects()
{
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);

	bool bSuccess = true;
	for (const FNiagaraPlatformSetRedirect& Redirect : Settings->PlatformSetRedirects)
	{
		if (Redirect.Mode == ENiagaraDeviceProfileRedirectMode::CVar)
		{
#if WITH_EDITOR
			//Do some additional validation in editor.
			IConsoleVariable* CVar = FNiagaraPlatformSet::GetCVar(*Redirect.CVarConditionEnabled.CVarName.ToString());
			if (!CVar)
			{
				UE_LOGF(LogNiagara, Warning, "Could not find CVar %ls when trying to apply PlatformSet Redirects.", *Redirect.CVarConditionEnabled.CVarName.ToString());
				bSuccess  = false;
				continue;
			}
			CVar = FNiagaraPlatformSet::GetCVar(*Redirect.CVarConditionDisabled.CVarName.ToString());
			if (!CVar)
			{
				UE_LOGF(LogNiagara, Warning, "Could not find CVar %ls when trying to apply PlatformSet Redirects.", *Redirect.CVarConditionDisabled.CVarName.ToString());
				bSuccess = false;
				continue;
			}
#endif

			bool bAddCVarEnabled = false;
			bool bAddCVarDisabled = false;

			for (auto DPEntryIt = DeviceProfileStates.CreateIterator(); DPEntryIt; ++DPEntryIt)
			{
				FNiagaraDeviceProfileStateEntry& DPEntry = *DPEntryIt;

				//Do we want to redirect this profile?
				if (Redirect.ProfileNames.Contains(DPEntry.ProfileName) == false)
				{
					continue;
				}
				
// 				if (DPEntry.SetQualityLevelMask != INDEX_NONE)
// 				{
// 					UE_LOGF(LogNiagara, Warning, "Cannot redirect entry for device profile %ls with a CVar as it is not set for all quality levels.", *DPEntry.ProfileName.ToString());
// 					continue;
// 				}
// 
// 				if (DPEntry.QualityLevelMask != INDEX_NONE && DPEntry.QualityLevelMask != 0)
// 				{
// 					UE_LOGF(LogNiagara, Warning, "Cannot redirect entry for device profile %ls with a CVar as it is not set for all quality levels.", *DPEntry.ProfileName.ToString());
// 					continue;
// 				}

				bool bEnabled = DPEntry.QualityLevelMask != 0;

				if (bEnabled)
				{
					bAddCVarEnabled = true;
				}
				else
				{
					bAddCVarDisabled = true;
				}			

				DPEntryIt.RemoveCurrent();
			}

			if (bAddCVarEnabled)
			{
				CVarConditions.Add(Redirect.CVarConditionEnabled);
			}

			if (bAddCVarDisabled)
			{
				CVarConditions.Add(Redirect.CVarConditionDisabled);
			}
		}
		else if (Redirect.Mode == ENiagaraDeviceProfileRedirectMode::DeviceProfile)
		{
			//Switch to new device profile.

			bool bRedirect = false;
			uint32 SetMask = 0xFFFFFFFF;
			uint32 QLMask = 0xFFFFFFFF;
			for (auto It = DeviceProfileStates.CreateIterator(); It; It++)
			{
				FNiagaraDeviceProfileStateEntry& Entry = *It;
				if(Redirect.ProfileNames.Contains(Entry.ProfileName))
				{
					if (bRedirect)
					{
						//we've already applied this redirect for another check the state matches.
						bool bMatch = QLMask == Entry.QualityLevelMask && SetMask == Entry.SetQualityLevelMask;
						if (!bMatch)
						{
							UE_LOGF(LogNiagara, Warning, "Found conflicting entries when trying to apply DP redirect from %ls to %ls", *Entry.ProfileName.ToString(), *Redirect.RedirectProfileName.ToString());
							bSuccess = false;
							break;
						}
						It.RemoveCurrent();
					}
					else
					{
						//First time we've applied this redirect. Swap the DP name and note the masks.
						Entry.ProfileName = Redirect.RedirectProfileName;
						QLMask = Entry.QualityLevelMask;
						SetMask = Entry.SetQualityLevelMask;
						bRedirect = true;
					}					
				}
			}
		}
		else
		{
			UE_LOGF(LogNiagara, Warning, "Bad Mode for Platform Set Redirect!");
			bSuccess = false;
		}
	}

	//Remove any dupes left in the DP array
	for (int32 i = 0; i < DeviceProfileStates.Num(); ++i)
	{
		FNiagaraDeviceProfileStateEntry& Entry = DeviceProfileStates[i];

		for (int32 j = i + 1; j < DeviceProfileStates.Num(); )
		{
			FNiagaraDeviceProfileStateEntry& Other = DeviceProfileStates[j];
			
			check(i!=j);

			if (Entry == Other)
			{
				DeviceProfileStates.RemoveAtSwap(j);
			}
			else
			{
				++j;//If we remove an entry we want to recheck the current j.
			}
		}
	}

	return bSuccess;
}

void FNiagaraPlatformSet::OnCVarChanged(IConsoleVariable* CVar)
{
	InvalidateCachedData();

	FScopeLock Lock(&CachedCVarInfoCritSec);

	//Iterate on our cached cvars as there'll be a v small subset of CVars Niagara cares about.
	for (auto It = CachedCVarInfo.CreateIterator(); It; ++It)
	{
		FName CVarName = It->Key;
		FCachedCVarInfo& Info = It->Value;
		if (Info.CVar == CVar)
		{
			ChangedCVars.AddUnique(CVarName);
			It.RemoveCurrent();
		}
	}
}

void FNiagaraPlatformSet::OnCVarUnregistered(IConsoleVariable* CVar)
{
	InvalidateCachedData();

	//We also unregister the changed delegate when it's unregistered.
	FScopeLock Lock(&CachedCVarInfoCritSec);

	//Iterate on our cached cvars as there'll be a v small subset of CVars Niagara cares about.
	for (auto It = CachedCVarInfo.CreateIterator(); It; ++It)
	{
		FName CVarName = It->Key;
		FCachedCVarInfo& Info = It->Value;
		if (Info.CVar == CVar)
		{
			check(CVar == Info.CVar);
			CVar->OnChangedDelegate().Remove(Info.ChangedHandle);
			ChangedCVars.AddUnique(CVarName);
			It.RemoveCurrent();
		}
	}
}

IConsoleVariable* FNiagaraPlatformSet::GetCVar(FName CVarName)
{
	FScopeLock Lock(&CachedCVarInfoCritSec);
	
	if (FCachedCVarInfo* CachedInfo = CachedCVarInfo.Find(CVarName))
	{
		return CachedInfo->CVar;
	}

	IConsoleManager& CMan = IConsoleManager::Get();
	if (IConsoleVariable* CVar = CMan.FindConsoleVariable(*CVarName.ToString()))
	{
		FCachedCVarInfo* CachedInfo = &CachedCVarInfo.Add(CVarName);
		CachedInfo->CVar = CVar;
		CachedInfo->ChangedHandle = CachedInfo->CVar->OnChangedDelegate().AddStatic(FNiagaraPlatformSet::OnCVarChanged);
		return CVar;
	}

	return nullptr;
}

void FNiagaraPlatformSet::RefreshScalability()
{
	//If the main quality level has changed we do a full refresh, other wise we can do a more targeted refresh just based on the changed CVars.
	if (GNiagaraQualityLevel != CachedQualityLevel)
	{
		FNiagaraPlatformSet::InvalidateCachedData();
		CachedQualityLevel = GNiagaraQualityLevel;

		for (TObjectIterator<UNiagaraSystem> It; It; ++It)
		{
			UNiagaraSystem* System = *It;
			check(System);
			System->UpdateScalability();
		}

		//We've done a full scalability so clear these out as we wont need to do a targeted refresh.
		ChangedCVars.Reset();

		//Always refresh our scalability settings. Can we be more targeted here?
		for (TObjectIterator<UNiagaraSettingsProfileSet> It; It; ++It)
		{
			UNiagaraSettingsProfileSet* Settings = *It;
			check(Settings);
			Settings->OnScalabilityUpdate();
		}
	}

	bool bNeedsRefresh = ChangedCVars.Num() > 0;//Other reasons to refresh deferred like this?

	if (bNeedsRefresh)
	{
		InvalidateCachedData();

		//Always refresh our scalability settings. Can we be more targeted here?
		for (TObjectIterator<UNiagaraSettingsProfileSet> It; It; ++It)
		{
			UNiagaraSettingsProfileSet* Settings = *It;
			check(Settings);
			Settings->OnScalabilityUpdate();
		}

		//Find any systems relying on a changed CVar and refresh it's scalability settings.
		for (TObjectIterator<UNiagaraSystem> It; It; ++It)
		{
			UNiagaraSystem* System = *It;
			check(System);

			bool bNeedsUpdate = false;
			auto NeedsScalabilityUpdate = [&](FNiagaraPlatformSet& PSet)
			{
				for (FNiagaraPlatformSetCVarCondition& CVarCondition : PSet.CVarConditions)
				{
					if (ChangedCVars.Contains(CVarCondition.CVarName))
					{
						bNeedsUpdate = true;
					}
				}
			};

			System->ForEachPlatformSet(NeedsScalabilityUpdate);
			if (UNiagaraEffectType* FXType = System->GetEffectType())
			{
				FXType->ForEachPlatformSet(NeedsScalabilityUpdate);
			}

			if (bNeedsUpdate)
			{
				System->UpdateScalability();
			}
		}
	}

	ChangedCVars.Reset();
}

bool FNiagaraPlatformSet::CanConsiderDeviceProfile(const UDeviceProfile* Profile)const
{
	return GbAllowAllDeviceProfiles != 0 || Profile->IsVisibleForAssets();
}

void FNiagaraPlatformSet::InvalidateCachedData()
{
#if WITH_EDITOR
	{
		FScopeLock Lock(&CachedPlatformIniCritSec);
		CachedQLMasksPerDeviceProfile.Empty();
		CachedPlatformIniSettings.Empty();
	}
	FDeviceProfileValueCache::Empty();
#endif

	CachedQualityLevel = INDEX_NONE;
	LastDirtiedFrame = GFrameNumber;
}

bool FNiagaraPlatformSet::IsEnabledForPlatform(const FString& PlatformName)const
{
	for (const UObject* ProfileObj : UDeviceProfileManager::Get().Profiles)
	{
		if (const UDeviceProfile* Profile = Cast<const UDeviceProfile>(ProfileObj))
		{
			if (Profile->DeviceType == PlatformName)
			{
				if(IsEnabledForDeviceProfile(Profile))
				{
					return true;//At least one profile for this platform is enabled.
				}
			}
		}
	}

	//No enabled profiles for this platform.
	return false;
}

bool FNiagaraPlatformSet::ShouldPruneEmittersOnCook(const FString& PlatformName)
{
#if WITH_EDITOR
	FPlatformIniSettings& Settings = GetPlatformIniSettings(PlatformName);
	return Settings.bPruneEmittersOnCook != 0;
#else
	return GbPruneEmittersOnCook != 0;
#endif
}

bool FNiagaraPlatformSet::CanChangeScalabilityAtRuntime(const UDeviceProfile* DeviceProfile)
{
	if (ensure(DeviceProfile))
	{
#if WITH_EDITOR
		FPlatformIniSettings& PlatformSettings = GetPlatformIniSettings(DeviceProfile->DeviceType);
		return PlatformSettings.bCanChangeScalabilitySettingsAtRuntime != 0;
#else
		return CanChangeScalabilityAtRuntime();
#endif
	}
	return true;//Assuming true if we fail to find the platform seems safest.
}

void SetGNiagaraDeviceProfile(UDeviceProfile* Profile)
{
	GNiagaraPlatformOverride = Profile;

	//Save the previous QL state the first time we enter a preview.
	if (GNiagaraBackupQualityLevel == INDEX_NONE)
	{
		GNiagaraBackupQualityLevel = GNiagaraQualityLevel;
	}

	UDeviceProfile* OverrideDP = GNiagaraPlatformOverride.Get();
	check(OverrideDP);
	int32 DPQL = FNiagaraPlatformSet::QualityLevelFromMask(FNiagaraPlatformSet::GetActiveQualityMaskForDeviceProfile(OverrideDP));
					
	UE::ConfigUtilities::OnSetCVarFromIniEntry(*GDeviceProfilesIni, NiagaraQualityLevelName, *LexToString(DPQL), ECVF_SetByMask);
}

#if WITH_EDITOR

bool FNiagaraPlatformSet::IsEffectQualityEnabled(int32 EffectQuality)const
{
	return ((1 << EffectQuality) & QualityLevelMask) != 0;
}

void FNiagaraPlatformSet::SetEnabledForEffectQuality(int32 EffectQuality, bool bEnabled)
{
	int32 EQBit = (1 << EffectQuality);
	if (bEnabled)
	{
		QualityLevelMask |= EQBit;
	}
	else
	{
		QualityLevelMask &= ~EQBit;
	}
	OnChanged();
}

void FNiagaraPlatformSet::SetDeviceProfileState(UDeviceProfile* Profile, int32 QualityLevel, ENiagaraPlatformSelectionState NewState)
{
	int32 Index = INDEX_NONE;
	Index = DeviceProfileStates.IndexOfByPredicate([&](const FNiagaraDeviceProfileStateEntry& Entry) { return Entry.ProfileName == Profile->GetFName(); });

	int32 ProfileQLMask = GetAvailableQualityMaskForDeviceProfile(Profile);
	if (ProfileQLMask != INDEX_NONE)
	{
		//For platforms that cannot change EQ at runtime we mark all state bits when setting state here so that if someone changes their EQ setting in the future, the state will be preserved.
		QualityLevel = INDEX_NONE;
	}

	if(Index == INDEX_NONE)
	{
		if (NewState != ENiagaraPlatformSelectionState::Default)
		{
			FNiagaraDeviceProfileStateEntry& NewEntry = DeviceProfileStates.AddDefaulted_GetRef();
			NewEntry.ProfileName = Profile->GetFName();
			NewEntry.SetState(QualityLevel, NewState);
		}
	}
	else
	{
		DeviceProfileStates[Index].SetState(QualityLevel, NewState);

		if (DeviceProfileStates[Index].AllDefaulted())
		{
			DeviceProfileStates.RemoveAtSwap(Index);//We don't need to store the default state. It's implied by no entry.
		}
	}
	OnChanged();
}

ENiagaraPlatformSelectionState FNiagaraPlatformSet::GetDeviceProfileState(UDeviceProfile* Profile, int32 QualityLevel)const
{
	if (const FNiagaraDeviceProfileStateEntry* ExistingEntry = DeviceProfileStates.FindByPredicate([&](const FNiagaraDeviceProfileStateEntry& Entry) { return Entry.ProfileName == Profile->GetFName(); }))
	{
		int32 ProfileQLMask = GetAvailableQualityMaskForDeviceProfile(Profile);
		if (ProfileQLMask == INDEX_NONE || ProfileQLMask & CreateQualityLevelMask(QualityLevel))
		{
			//For profiles that cannot change scalability at runtime we store all flags in their state so that if anyone ever changes their EQ Cvar, the state setting remains valid.
			//This just means we also have to ensure this is the correct EQ here.

			return ExistingEntry->GetState(QualityLevel);
		}
	}
	return ENiagaraPlatformSelectionState::Default;
}

void FNiagaraPlatformSet::OnChanged()
{
	LastBuiltFrame = 0;
	InvalidateCachedData();
}

bool FNiagaraPlatformSet::GatherConflicts(TConstArrayView<const FNiagaraPlatformSet*> PlatformSets, TArray<FNiagaraPlatformSetConflictInfo>& OutConflicts)
{
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);
	int32 NumLevels = Settings->QualityLevels.Num();

	FNiagaraPlatformSetConflictInfo* CurrentConflict = nullptr;
	for (int32 A = 0; A < PlatformSets.Num(); ++A)
	{
		for (int32 B = A + 1; B < PlatformSets.Num(); ++B)
		{
			check(A != B);

			if (PlatformSets[A] == nullptr || PlatformSets[B] == nullptr) continue;

			const FNiagaraPlatformSet& SetA = *PlatformSets[A];
			const FNiagaraPlatformSet& SetB = *PlatformSets[B];

			CurrentConflict = nullptr;
			for (UObject* DPObj : UDeviceProfileManager::Get().Profiles)
			{
				UDeviceProfile* Profile = CastChecked<UDeviceProfile>(DPObj);
				if (SetA.CanConsiderDeviceProfile(Profile) == false && SetB.CanConsiderDeviceProfile(Profile) == false)
				{
					continue;
				}
				int32 AEnabledMask = SetA.GetEnabledMaskForDeviceProfile(Profile);
				int32 BEnabledMask = SetB.GetEnabledMaskForDeviceProfile(Profile);
				int32 ConflictMask = AEnabledMask & BEnabledMask;

				if (ConflictMask != 0)
				{
					//We have a conflict so add it to the output.
					if (CurrentConflict == nullptr)
					{
						CurrentConflict = &OutConflicts.AddDefaulted_GetRef();
						CurrentConflict->SetAIndex = A;
						CurrentConflict->SetBIndex = B;
					}
					FNiagaraPlatformSetConflictEntry& ConflictEntry = CurrentConflict->Conflicts.AddDefaulted_GetRef();
					ConflictEntry.ProfileName = Profile->GetFName();
					ConflictEntry.QualityLevelMask = ConflictMask;
				}
			}
		}
	}

	return OutConflicts.Num() > 0;
}

FNiagaraPlatformSet::FPlatformIniSettings& FNiagaraPlatformSet::GetPlatformIniSettings(const FString& PlatformName)
{
	FScopeLock Lock(&CachedPlatformIniCritSec);

	if (FPlatformIniSettings* CachedSettings = CachedPlatformIniSettings.Find(*PlatformName))
	{
		return *CachedSettings;
	}

	//Load config files in which we can reasonable expect to find fx.Niagara.QualityLevel and may be set.
	FConfigFile LocalEngineIni;
	FConfigFile* LoadedEngineIni = FConfigCacheIni::FindOrLoadPlatformConfig(LocalEngineIni, TEXT("Engine"), *PlatformName); //Should use BaseProfileName? Are either of these ensured to be correct? I worry this is brittle.

	FConfigFile LocalGameIni;
	FConfigFile* LoadedGameIni = FConfigCacheIni::FindOrLoadPlatformConfig(LocalGameIni, TEXT("Game"), *PlatformName); //Should use BaseProfileName? Are either of these ensured to be correct? I worry this is brittle.
	
	FConfigFile LocalScalabilityIni;
	FConfigFile* LoadedScalabilityIni = FConfigCacheIni::FindOrLoadPlatformConfig(LocalScalabilityIni, TEXT("Scalability"), *PlatformName); //Should use BaseProfileName? Are either of these ensured to be correct? I worry this is brittle.

	auto FindCVarValue = [&](const TCHAR* Section, const TCHAR* CVarName, int32& OutVal)
	{
		bool bFound = true;
		if (!LoadedScalabilityIni->GetInt(Section, CVarName, OutVal))
		{
			if (!LoadedGameIni->GetInt(Section, CVarName, OutVal))
			{
				if (!LoadedEngineIni->GetInt(Section, CVarName, OutVal))
				{
					bFound = false;
				}
			}
		}
		return bFound;
	};

	//The effect quality for this platform.
	int32 EffectsQuality = Scalability::DefaultQualityLevel;

	//If this platform can change scalability settings at runtime then we return the full mask.
	int32 CanChangeScalabiltiySettings = 0;
	int32 MinQualityLevel = DefaultMinQualityLevel;
	int32 MaxQualityLevel = DefaultMaxQualityLevel;

	FindCVarValue(TEXT("SystemSettings"), CanChangeEQCVarName, CanChangeScalabiltiySettings);

	FindCVarValue(TEXT("SystemSettings"), NiagaraMinQualityLevelName, MinQualityLevel);

	FindCVarValue(TEXT("SystemSettings"), NiagaraMaxQualityLevelName, MaxQualityLevel);

	if (!FindCVarValue(TEXT("ScalabilityGroups"), TEXT("sg.EffectsQuality"), EffectsQuality))
	{
		FindCVarValue(TEXT("SystemSettings"), TEXT("sg.EffectsQuality"), EffectsQuality);
	}

	//Get the platforms default quality level setting.
	//This can be overridden directly in a device profile or indirectly by overriding effects quality.

	int32 PruneEmittersOnCook = GbPruneEmittersOnCook;
	FindCVarValue(TEXT("SystemSettings"), PruneEmittersOnCookName, PruneEmittersOnCook);

	FPlatformIniSettings& NewSetting = CachedPlatformIniSettings.Add(*PlatformName);
	NewSetting = FPlatformIniSettings(CanChangeScalabiltiySettings, PruneEmittersOnCook, EffectsQuality, MinQualityLevel, MaxQualityLevel);

	//Find the Niagara Quality Levels set for each EffectsQuality Level for this platform.
	int32 NumEffectsQualities = Scalability::GetQualityLevelCounts().EffectsQuality;
	FString EQStr;
	for (int32 EQ = 0; EQ < NumEffectsQualities; ++EQ)
	{
		FString SectionName = Scalability::GetScalabilitySectionString(TEXT("EffectsQuality"), EQ, NumEffectsQualities);
		int32 NiagaraQualityLevelForEQ = DefaultQualityLevel;
		LoadedScalabilityIni->GetInt(*SectionName, NiagaraQualityLevelName, NiagaraQualityLevelForEQ);
		NewSetting.QualityLevelsPerEffectsQuality.Add(NiagaraQualityLevelForEQ);
		EQStr += FString::Printf(TEXT("EQ:%d = NQL:%d\n"), EQ, NiagaraQualityLevelForEQ);
	}

	//UE_LOGF(LogNiagara, Warning, "\n=====================================================\n%ls - PlatformSettings(%d, %d, %d)\n%ls\n=========================================", *PlatformName, CanChangeEffectQuality, PruneEmittersOnCook, EffectsQuality, *EQStr);
	return NewSetting;
}

int32 FNiagaraPlatformSet::GetEffectQualityMaskForPlatform(const FString& PlatformName)
{
	FPlatformIniSettings& PlatformSettings = GetPlatformIniSettings(PlatformName);
	return PlatformSettings.QualityLevelMask;
}

FNiagaraPlatformSet::FPlatformIniSettings::FPlatformIniSettings()
	:bCanChangeScalabilitySettingsAtRuntime(0), bPruneEmittersOnCook(false), EffectsQuality(0), MinQualityLevel(INDEX_NONE), MaxQualityLevel(INDEX_NONE), QualityLevelMask(INDEX_NONE)
{}
FNiagaraPlatformSet::FPlatformIniSettings::FPlatformIniSettings(int32 InbCanChangeScalabilitySettingsAtRuntime, int32 InbPruneEmittersOnCook, int32 InEffectsQuality, int32 InMinQualityLevel, int32 InMaxQualityLevel)
	: bCanChangeScalabilitySettingsAtRuntime(InbCanChangeScalabilitySettingsAtRuntime), bPruneEmittersOnCook(InbPruneEmittersOnCook)
	, EffectsQuality(InEffectsQuality), MinQualityLevel(InMinQualityLevel), MaxQualityLevel(InMaxQualityLevel)
{
	if (bCanChangeScalabilitySettingsAtRuntime)
	{
		QualityLevelMask = INDEX_NONE;
	}
	else
	{
		if (InMinQualityLevel == INDEX_NONE && MaxQualityLevel == INDEX_NONE)
		{
			QualityLevelMask = INDEX_NONE;
		}
		else
		{
			QualityLevelMask = 0;
			MinQualityLevel = MinQualityLevel == INDEX_NONE ? 0 : MinQualityLevel;
			MaxQualityLevel = MaxQualityLevel == INDEX_NONE ? 31 : MaxQualityLevel;
			for (int32 QL = MinQualityLevel; QL <= MaxQualityLevel; ++QL)
			{
				QualityLevelMask |= (1 << QL);
			}
		}
	}
}

#endif
//////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
TMap<const UDeviceProfile*, FDeviceProfileValueCache::FCVarValueMap> FDeviceProfileValueCache::CachedDeviceProfileValues;

const FNiagaraCVarValues& FDeviceProfileValueCache::GetValues(const UDeviceProfile* DeviceProfile, IConsoleVariable* CVar, FName CVarName)
{
	check(CVar);

	//First look if we've asked for this CVar for this device profile before.
	FCVarValueMap& CVarMap = CachedDeviceProfileValues.FindOrAdd(DeviceProfile);

	if (FNiagaraCVarValues* CVarValues = CVarMap.Find(CVar))
	{
		return *CVarValues;
	}

	//If not we'll need to look for it.	
	FNiagaraCVarValues& NewCVarValues = CVarMap.Add(CVar);
	
	const FString& PlatformName = DeviceProfile->DeviceType;

	FConfigFile EngineSettings;
	FConfigCacheIni::LoadLocalIniFile(EngineSettings, TEXT("Engine"), true, *PlatformName);

	FConfigFile GameSettings;
	FConfigCacheIni::LoadLocalIniFile(GameSettings, TEXT("Game"), true, *PlatformName);

	FConfigFile ScalabilitySettings;
	FConfigCacheIni::LoadLocalIniFile(ScalabilitySettings, TEXT("Scalability"), true, *PlatformName);

	int32 NumEffectsQualities = Scalability::GetQualityLevelCounts().EffectsQuality;

	FString DefaultStr;

	//First see if the device profile has it explicitly set.
	//If we found the cvar in the device profile then use that as a base, otherwise look through all the inis.
	if (DeviceProfile->GetConsolidatedCVarValue(*CVarName.ToString(), DefaultStr, false) == false)
	{
		//Try to grab a default from our initial scalability group.
		FNiagaraPlatformSet::FPlatformIniSettings& PlatformIniSettings = FNiagaraPlatformSet::GetPlatformIniSettings(PlatformName);
		
		int32 EffectsQuality = INDEX_NONE;
		//See if this profile overrides effects quality.
		if (!DeviceProfile->GetConsolidatedCVarValue(TEXT("sg.EffectsQuality"), EffectsQuality))
		{
			EffectsQuality = PlatformIniSettings.EffectsQuality;
		}
		check(EffectsQuality != INDEX_NONE && EffectsQuality < NumEffectsQualities);
		FString DefaultEffectsQualitySectionName = Scalability::GetScalabilitySectionString(TEXT("EffectsQuality"), EffectsQuality, NumEffectsQualities);

		if (ScalabilitySettings.GetString(*DefaultEffectsQualitySectionName, *CVarName.ToString(), DefaultStr) == false)
		{
			//Otherwise try to grab from the main inis.
			auto FindCVarValue = [&](const TCHAR* Section, const TCHAR* CVarName, FString& OutVal)
			{
				bool bFound = true;
				if (!ScalabilitySettings.GetString(Section, CVarName, OutVal))
				{
					if (!GameSettings.GetString(Section, CVarName, OutVal))
					{
						if (!EngineSettings.GetString(Section, CVarName, OutVal))
						{
							bFound = false;
						}
					}
				}
				return bFound;
			};
				
			if (FindCVarValue(TEXT("SystemSettings"), *CVarName.ToString(), DefaultStr) == false)
			{
				if (IConsoleVariable* PlatformCVar = CVar->GetPlatformValueVariable(*PlatformName).Get())//Possible this could do all the above work for us.
				{
					DefaultStr = PlatformCVar->GetString();
				}
				else
				{
					DefaultStr = CVar->GetDefaultValue();
				}
			}
		}
	}

	check(DefaultStr.IsEmpty() == false);

	NewCVarValues.Default.Set(DefaultStr, CVar);
	NewCVarValues.PerQualityLevelValues.SetNum(NumEffectsQualities);

	//Also gather possible values from other scalability groups.
	//Falling back to the overall default if that EQ doesn't set it.
	FString EQStr;
	for (int32 EQ = 0; EQ < NumEffectsQualities; ++EQ)
	{		
		FString SectionName = Scalability::GetScalabilitySectionString(TEXT("EffectsQuality"), EQ, NumEffectsQualities);
		if (ScalabilitySettings.GetString(*SectionName, *CVarName.ToString(), EQStr))
		{
			NewCVarValues.PerQualityLevelValues[EQ].Set(EQStr, CVar);
		}
		else
		{
			NewCVarValues.PerQualityLevelValues[EQ].Set(DefaultStr, CVar);
		}
	}

	return NewCVarValues;
}

void FDeviceProfileValueCache::Empty()
{
	CachedDeviceProfileValues.Empty();
}

#endif

//////////////////////////////////////////////////////////////////////////

FNiagaraPlatformSetCVarCondition::FNiagaraPlatformSetCVarCondition()
	: bUseMinInt(true)
	, bUseMaxInt(false)
	, bUseMinFloat(true)
	, bUseMaxFloat(false)
{

}

IConsoleVariable* FNiagaraPlatformSetCVarCondition::GetCVar()const
{
	if (CachedCVar == nullptr || FNiagaraPlatformSet::GetLastDirtiedFrame() >= LastCachedFrame || LastCachedFrame == INDEX_NONE)
	{
		LastCachedFrame = GFrameNumber;
		CachedCVar = FNiagaraPlatformSet::GetCVar(CVarName);
	}
	return CachedCVar;
}

void FNiagaraPlatformSetCVarCondition::SetCVar(FName InCVarName)
{
	CVarName = InCVarName;
	CachedCVar = nullptr;
}

template<> bool FNiagaraPlatformSetCVarCondition::GetCVarValue<bool>(IConsoleVariable* CVar)const { return CVar->GetBool(); }
template<> int32 FNiagaraPlatformSetCVarCondition::GetCVarValue<int32>(IConsoleVariable* CVar)const { return CVar->GetInt(); }
template<> float FNiagaraPlatformSetCVarCondition::GetCVarValue<float>(IConsoleVariable* CVar)const { return CVar->GetFloat(); }

template<typename T>
bool FNiagaraPlatformSetCVarCondition::Evaluate_Internal(const UDeviceProfile* DeviceProfile, int32 QualityLevel, bool bCheckCurrentStateOnly, bool& bOutCanEverPass, bool& bOutCanEverFail)const
{		
	bool bConditionPassed = false;

	IConsoleVariable* CVar = GetCVar();

	if(!CVar)
	{
		return false;
	}

	bool bCanChangeAtRuntime = FNiagaraPlatformSet::CanChangeScalabilityAtRuntime(DeviceProfile) && (CVar->GetFlags() & ECVF_ReadOnly) == 0;

#if WITH_EDITOR
	if (bCheckCurrentStateOnly == false)
	{
		const FNiagaraCVarValues& PossibleValues = FDeviceProfileValueCache::GetValues(DeviceProfile, CVar, CVarName);

		T CVarValue;
		if (PossibleValues.PerQualityLevelValues.IsValidIndex(QualityLevel))
		{
			CVarValue = PossibleValues.PerQualityLevelValues[QualityLevel].Get<T>();
		}
		else
		{
			CVarValue = PossibleValues.Default.Get<T>();
		}

		bConditionPassed = CheckValue(CVarValue);

		bOutCanEverPass = bConditionPassed;
		bOutCanEverFail = !bConditionPassed;

		if (bCanChangeAtRuntime)
		{
			bOutCanEverFail = true;
			bOutCanEverPass = true;
		}
		else
		{
			for (int32 i = 0; i < PossibleValues.PerQualityLevelValues.Num(); ++i)
			{
				T Val = PossibleValues.PerQualityLevelValues[i].Get<T>();
				bool bPass = CheckValue(Val);
				if (bPass)
				{
					bOutCanEverPass = true;
				}
				else
				{
					bOutCanEverFail = true;
				}
			}
		}

		return bConditionPassed;
	}
#endif
	
	bConditionPassed = CheckValue(GetCVarValue<T>(CVar));

	if (bCanChangeAtRuntime)
	{
		bOutCanEverPass = true;
		bOutCanEverFail = true;
	}
	else
	{
		bOutCanEverPass = bConditionPassed;
		bOutCanEverFail = !bConditionPassed;
	}

	return bConditionPassed;
}

bool FNiagaraPlatformSetCVarCondition::Evaluate(const UDeviceProfile* DeviceProfile, int32 QualityLevel, bool bCheckCurrentStateOnly, bool& bOutCanEverPass, bool& bOutCanEverFail)const
{
	if (IConsoleVariable* CVar = GetCVar())
	{
		if (CVar->IsVariableBool())
		{
			return Evaluate_Internal<bool>(DeviceProfile, QualityLevel, bCheckCurrentStateOnly, bOutCanEverPass, bOutCanEverFail);
		}
		else if (CVar->IsVariableInt())
		{
			return Evaluate_Internal<int32>(DeviceProfile, QualityLevel, bCheckCurrentStateOnly, bOutCanEverPass, bOutCanEverFail);
		}
		else if (CVar->IsVariableFloat())
		{
			return Evaluate_Internal<float>(DeviceProfile, QualityLevel, bCheckCurrentStateOnly, bOutCanEverPass, bOutCanEverFail);
		}
		else
		{
			ensureMsgf(false, TEXT("CVar %s is of an unsupported type for FNiagaraPlatformSetCVarCondition. Supported types are Bool, Int and Float. This should not be possible unless the CVar's type has been chagned."), *CVarName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogNiagara, Warning, "Niagara Platform Set is trying to use a CVar that doesn't exist. %ls", *CVarName.ToString());
	}

	return false;
}

FNiagaraPlatformSetRedirect::FNiagaraPlatformSetRedirect()
	: Mode(ENiagaraDeviceProfileRedirectMode::CVar)
{}

#undef LOCTEXT_NAMESPACE

