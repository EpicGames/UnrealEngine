// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraPreviewGrid.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Texture2D.h"
#include "NiagaraComponent.h"
#include "NiagaraSystem.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraPreviewGrid)

#if WITH_EDITOR
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#endif

#if WITH_EDITOR
void UNiagaraPreviewAxis::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	//Trigger a regeneration of preview grid.
	if (ANiagaraPreviewGrid* PreviewGrid = GetTypedOuter<ANiagaraPreviewGrid>())
	{
		PreviewGrid->PostEditChange();
	}
}
#endif

//////////////////////////////////////////////////////////////////////////

ANiagaraPreviewGrid::ANiagaraPreviewGrid(const FObjectInitializer& ObjectInitializer)
	: System(nullptr)
	, ResetMode(ENiagaraPreviewGridResetMode::Never)
	, PreviewAxisX(nullptr)
	, PreviewAxisY(nullptr)
	, PreviewClass(nullptr)
	, SpacingX(250.0f)
	, SpacingY(250.0f)
	, NumX(0)
	, NumY(0)
#if WITH_EDITORONLY_DATA
	, SpriteComponent(nullptr)
	, ArrowComponent(nullptr)
#endif
	, bPreviewDirty(true)
	, bPreviewActive(false)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;

// 	static ConstructorHelpers::FClassFinder<ANiagaraPreviewBase> DefaultPreviewClassBP(TEXT("/Niagara/Blueprints/NiagaraPreview"));
// 	if (DefaultPreviewClassBP.Class != NULL)
// 	{
// 		PreviewClass = DefaultPreviewClassBP.Class;
// 	}

	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	check(RootComponent);
#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("ArrowComponent0"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTextureObject;
			FName ID_Effects;
			FText NAME_Effects;
			FConstructorStatics()
				: SpriteTextureObject(TEXT("/Niagara/Icons/S_ParticleSystem"))
				, ID_Effects(TEXT("Effects"))
				, NAME_Effects(NSLOCTEXT("SpriteCategory", "Effects", "Effects"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;


		if (SpriteComponent)
		{
			SpriteComponent->Sprite = ConstructorStatics.SpriteTextureObject.Get();
			SpriteComponent->SetRelativeScale3D_Direct(FVector(0.5f, 0.5f, 0.5f));
			SpriteComponent->bHiddenInGame = false;
			SpriteComponent->bIsScreenSizeScaled = true;
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Effects;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Effects;
			SpriteComponent->SetupAttachment(RootComponent);
			SpriteComponent->bReceivesDecals = false;
		}

		if (ArrowComponent)
		{
			ArrowComponent->ArrowColor = FColor(0, 255, 128);

			ArrowComponent->ArrowSize = 1.5f;
			ArrowComponent->bHiddenInGame = false;
			ArrowComponent->bTreatAsASprite = true;
			ArrowComponent->bIsScreenSizeScaled = true;
			ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Effects;
			ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Effects;
			ArrowComponent->SetupAttachment(RootComponent);
			ArrowComponent->SetUsingAbsoluteScale(true);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void ANiagaraPreviewGrid::PostLoad()
{
	Super::PostLoad();

	//Fixup old data that incorrectly did not have this component.
	if (RootComponent == nullptr)
	{
		RootComponent = NewObject<USceneComponent>(this, TEXT("SceneComp"));
	}
}

void ANiagaraPreviewGrid::BeginDestroy()
{
	Super::BeginDestroy();
	DestroyPreviews();
}

#if WITH_EDITOR
void ANiagaraPreviewGrid::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bPreviewDirty = true;
}
#endif

void ANiagaraPreviewGrid::ActivatePreviews(bool bReset)
{
	if (bPreviewDirty)
	{
		bPreviewDirty = false;
		GeneratePreviews();
	}

	FString XLabel;
	FString YLabel;

	for (int32 X = 0; X < NumX; ++X)
	{
		const float XLocation = X / float(NumX - 1);

		for (int32 Y = 0; Y < NumY; ++Y)
		{
			const float YLocation = Y / float(NumY - 1);

			int32 PreviewIdx = PreviewIndex(X, Y);
			UChildActorComponent* PreviewComp = PreviewComponents[PreviewIdx];
			ANiagaraPreviewBase* PreviewActor = CastChecked<ANiagaraPreviewBase>(PreviewComp->GetChildActor());

			TArray<UNiagaraComponent*, TInlineAllocator<4>> NiagaraComponents;
			PreviewActor->GetComponents(NiagaraComponents);
			for (UNiagaraComponent* Component : NiagaraComponents)
			{
				PreviewAxisX->ApplyToPreview(Component, X, true, XLabel);
				PreviewAxisY->ApplyToPreview(Component, Y, false, YLabel);
				
				Component->Activate(true);
			}
		}
	}
}

void ANiagaraPreviewGrid::DeactivatePreviews()
{
	for (int32 X = 0; X < NumX; ++X)
	{
		const float XLocation = X / float(NumX - 1);

		for (int32 Y = 0; Y < NumY; ++Y)
		{
			const float YLocation = Y / float(NumY - 1);

			int32 PreviewIdx = PreviewIndex(X, Y);
			UChildActorComponent* PreviewComp = PreviewComponents[PreviewIdx];
			ANiagaraPreviewBase* PreviewActor = CastChecked<ANiagaraPreviewBase>(PreviewComp->GetChildActor());

			TArray<UNiagaraComponent*, TInlineAllocator<4>> NiagaraComponents;
			PreviewActor->GetComponents(NiagaraComponents);
			for (UNiagaraComponent* Component : NiagaraComponents)
			{
				Component->Deactivate();
			}
		}
	}
}

void ANiagaraPreviewGrid::SetPaused(bool bPaused)
{
	for (int32 X = 0; X < NumX; ++X)
	{
		const float XLocation = X / float(NumX - 1);

		for (int32 Y = 0; Y < NumY; ++Y)
		{
			const float YLocation = Y / float(NumY - 1);

			int32 PreviewIdx = PreviewIndex(X, Y);
			UChildActorComponent* PreviewComp = PreviewComponents[PreviewIdx];
			ANiagaraPreviewBase* PreviewActor = CastChecked<ANiagaraPreviewBase>(PreviewComp->GetChildActor());

			TArray<UNiagaraComponent*, TInlineAllocator<4>> NiagaraComponents;
			PreviewActor->GetComponents(NiagaraComponents);
			for (UNiagaraComponent* Component : NiagaraComponents)
			{
				Component->SetPaused(bPaused);
			}
		}
	}

	SetActorTickEnabled(!bPaused);
}

void ANiagaraPreviewGrid::GetPreviews(TArray<UNiagaraComponent*>& OutPreviews)
{
	for (int32 X = 0; X < NumX; ++X)
	{
		const float XLocation = X / float(NumX - 1);

		for (int32 Y = 0; Y < NumY; ++Y)
		{
			const float YLocation = Y / float(NumY - 1);

			int32 PreviewIdx = PreviewIndex(X, Y);
			UChildActorComponent* PreviewComp = PreviewComponents[PreviewIdx];
			ANiagaraPreviewBase* PreviewActor = CastChecked<ANiagaraPreviewBase>(PreviewComp->GetChildActor());

			TArray<UNiagaraComponent*, TInlineAllocator<4>> NiagaraComponents;
			PreviewActor->GetComponents(NiagaraComponents);
			for (UNiagaraComponent* Component : NiagaraComponents)
			{
				OutPreviews.Add(Component);
			}
		}
	}
}

void ANiagaraPreviewGrid::TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction)
{
	Super::TickActor(DeltaTime, TickType, ThisTickFunction);

	if (bPreviewDirty)
	{
		bPreviewDirty = false;
		GeneratePreviews();
	}
	TickPreviews();
}

void ANiagaraPreviewGrid::DestroyPreviews()
{
	for (UChildActorComponent* Preview : PreviewComponents)
	{
		if (Preview)
		{
			Preview->DestroyComponent();
		}
	}
	PreviewComponents.Reset();
	NumX = 0;
	NumY = 0;
}

void ANiagaraPreviewGrid::GeneratePreviews()
{
	DestroyPreviews();

	if (System && PreviewAxisX && PreviewAxisY && PreviewClass)
	{
		System->ConditionalPostLoad();
		PreviewClass->ConditionalPostLoad();

		NumX = PreviewAxisX->Num();
		NumY = PreviewAxisY->Num();

		PreviewComponents.SetNum(NumX * NumY);

		if (NumX > 0 && NumY > 0)
		{
			for (int32 X = 0; X < NumX; ++X)
			{
				const float XLocation = X / float(NumX - 1);

				for (int32 Y = 0; Y < NumY; ++Y)
				{
					int32 Index = PreviewIndex(X, Y);

					UChildActorComponent* PreviewComp = NewObject<UChildActorComponent>(this, *FString::Printf(TEXT("%s[%d][%d]"), *GetName(), X, Y), RF_Transient);
					PreviewComponents[Index] = PreviewComp;
					PreviewComp->SetChildActorClass(PreviewClass);
					PreviewComp->bAutoActivate = false;
					PreviewComp->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepRelativeTransform);
					PreviewComp->RegisterComponentWithWorld(RootComponent->GetWorld());
					ANiagaraPreviewBase* PreviewActor = CastChecked<ANiagaraPreviewBase>(PreviewComp->GetChildActor());

					FAttachmentTransformRules AttachRules(EAttachmentRule::SnapToTarget, EAttachmentRule::SnapToTarget, EAttachmentRule::KeepRelative, false);
					PreviewActor->AttachToComponent(PreviewComp, AttachRules);
					PreviewActor->SetActorRelativeLocation(FVector(SpacingX * X, SpacingY * Y, 0.0f));
					PreviewActor->PrimaryActorTick.bCanEverTick = true;
					PreviewActor->PrimaryActorTick.bStartWithTickEnabled = true;
					PreviewActor->SetSystem(System);

					TArray<UNiagaraComponent*, TInlineAllocator<4>> NiagaraComponents;
					PreviewActor->GetComponents(NiagaraComponents);
					for (UNiagaraComponent* Component : NiagaraComponents)
					{
						Component->Activate(true);
					}
				}
			}
		}
	}
}

void ANiagaraPreviewGrid::TickPreviews()
{
	if (System && PreviewAxisX && PreviewAxisY)
	{
		bool bAllInactive = true;
		for (int32 X = 0; X < NumX; ++X)
		{
			const float XLocation = X / float(NumX - 1);

			for (int32 Y = 0; Y < NumY; ++Y)
			{
				const float YLocation = Y / float(NumY - 1);

				int32 PreviewIdx = PreviewIndex(X, Y);

				UChildActorComponent* PreviewComp = PreviewComponents[PreviewIdx];
				ANiagaraPreviewBase* PreviewActor = CastChecked<ANiagaraPreviewBase>(PreviewComp->GetChildActor());

				TArray<FStringFormatArg> Args;
				Args.Add(FStringFormatArg(PreviewAxisX->GetClass()->GetName()));
				Args.Add(FStringFormatArg(X));
				FString XLabel = FString::Format(TEXT("{1} | X = {0}"), Args);

				Args.Reset();
				Args.Add(FStringFormatArg(PreviewAxisY->GetClass()->GetName()));
				Args.Add(FStringFormatArg(Y));
				FString YLabel = FString::Format(TEXT("{1} | Y = {0}"), Args);

				TArray<UNiagaraComponent*, TInlineAllocator<4>> NiagaraComponents;
				PreviewActor->GetComponents(NiagaraComponents);
				for (UNiagaraComponent* Component : NiagaraComponents)
				{
					//Ensure we're active. Will re-trigger one shot effects.
					if (!Component->IsActive())
					{
						if (ResetMode == ENiagaraPreviewGridResetMode::Individual)
						{
							Component->Activate(true);
						}
					}
					else
					{
						bAllInactive = false;
					}

					PreviewAxisX->ApplyToPreview(Component, X, true, XLabel);
					PreviewAxisY->ApplyToPreview(Component, Y, false, YLabel);
				}

				PreviewActor->SetLabelText(FText::FromString(XLabel), FText::FromString(YLabel));
			}
		}

		if (bAllInactive && ResetMode == ENiagaraPreviewGridResetMode::All)
		{
			ActivatePreviews(true);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraPreviewAxis_InterpParamInt32::ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText)
{
	check(Count > 0);
	check(PreviewComponent);
	float Interp = Count > 1 ? (float)PreviewIndex / (Count - 1) : 1.0f;
	int32 Val = FMath::Lerp(Min, Max, Interp);
	PreviewComponent->SetVariableInt(Param, Val);
	OutLabelText = FString::Printf(TEXT("%s = %d"), *Param.ToString(), Val);
}
void UNiagaraPreviewAxis_InterpParamFloat::ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText)
{
	check(Count > 0);
	check(PreviewComponent);
	float Interp = Count > 1 ? (float)PreviewIndex / (Count - 1) : 1.0f;
	float Val = FMath::Lerp(Min, Max, Interp);
	PreviewComponent->SetVariableFloat(Param, Val);
	OutLabelText = FString::Printf(TEXT("%s = %g"), *Param.ToString(), Val);
}
void UNiagaraPreviewAxis_InterpParamVector2D::ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText)
{
	check(Count > 0);
	check(PreviewComponent);
	float Interp = Count > 1 ? (float)PreviewIndex / (Count - 1) : 1.0f;
	FVector2D Val = FMath::Lerp(Min, Max, Interp);
	PreviewComponent->SetVariableVec2(Param, Val);
	OutLabelText = FString::Printf(TEXT("%s = {%g, %g}"), *Param.ToString(), Val.X, Val.Y);
}
void UNiagaraPreviewAxis_InterpParamVector::ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText)
{
	check(Count > 0);
	check(PreviewComponent);
	float Interp = Count > 1 ? (float)PreviewIndex / (Count - 1) : 1.0f;
	FVector Val = FMath::Lerp(Min, Max, Interp);
	PreviewComponent->SetVariableVec3(Param, Val);
	OutLabelText = FString::Printf(TEXT("%s = {%g, %g, %g}"), *Param.ToString(), Val.X, Val.Y, Val.Z);
}
void UNiagaraPreviewAxis_InterpParamVector4::ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText)
{
	check(Count > 0);
	check(PreviewComponent);
	float Interp = Count > 1 ? (float)PreviewIndex / (Count - 1) : 1.0f;
	FVector4 Val = FMath::Lerp(Min, Max, Interp);
	PreviewComponent->SetVariableVec4(Param, Val);
	OutLabelText = FString::Printf(TEXT("%s = {%g, %g, %g, %g}"), *Param.ToString(), Val.X, Val.Y, Val.Z, Val.W);
}
void UNiagaraPreviewAxis_InterpParamLinearColor::ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText)
{
	check(Count > 0);
	check(PreviewComponent);
	float Interp = Count > 1 ? (float)PreviewIndex / (Count - 1) : 1.0f;
	FLinearColor Val = FMath::Lerp(Min, Max, Interp);
	PreviewComponent->SetVariableLinearColor(Param, Val);
	OutLabelText = FString::Printf(TEXT("%s = {%g, %g, %g, %g}"), *Param.ToString(), Val.R, Val.G, Val.B, Val.A);
}
