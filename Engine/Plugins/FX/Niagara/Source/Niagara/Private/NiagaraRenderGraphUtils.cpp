// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraRenderGraphUtils.h"
#include "NiagaraStats.h"

#include "RenderGraphUtils.h"

//////////////////////////////////////////////////////////////////////////

void FNiagaraPooledRWBuffer::InitializeInternal(FRDGBuilder& GraphBuilder, const TCHAR* ResourceName, EPixelFormat InPixelFormat, const FRDGBufferDesc& BufferDesc)
{
	Release();

	TransientRDGBuffer = GraphBuilder.CreateBuffer(BufferDesc, ResourceName);
	Buffer = GraphBuilder.ConvertToExternalBuffer(TransientRDGBuffer);
	PixelFormat = InPixelFormat;

#if STATS
	NumBytes = BufferDesc.GetSize();
	INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, NumBytes);
#endif
}

void FNiagaraPooledRWBuffer::InitializeInternal(FRDGBuilder& GraphBuilder, const TCHAR* ResourceName, EPixelFormat InPixelFormat, const uint32 BytesPerElemenet, const uint32 NumElements, EBufferUsageFlags UsageFlags)
{
	FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateBufferDesc(BytesPerElemenet, NumElements);
	BufferDesc.Usage |= UsageFlags;
	InitializeInternal(GraphBuilder, ResourceName, InPixelFormat, BufferDesc);
}

void FNiagaraPooledRWBuffer::Release()
{
#if STATS
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, NumBytes);
	NumBytes = 0;
#endif

	Buffer.SafeRelease();
	PixelFormat = PF_Unknown;
}

FRDGBufferRef FNiagaraPooledRWBuffer::GetOrCreateBuffer(FRDGBuilder& GraphBuilder)
{
	check(IsValid());
	if (TransientRDGBuffer == nullptr)
	{
		TransientRDGBuffer = GraphBuilder.RegisterExternalBuffer(Buffer);
	}
	else
	{
		check(GraphBuilder.FindExternalBuffer(Buffer) != nullptr);
	}
	return TransientRDGBuffer;
}

FRDGBufferSRVRef FNiagaraPooledRWBuffer::GetOrCreateSRV(FRDGBuilder& GraphBuilder)
{
	check(IsValid());
	if (TransientRDGSRV == nullptr)
	{
		TransientRDGSRV = GraphBuilder.CreateSRV(GetOrCreateBuffer(GraphBuilder), PixelFormat);
	}
	else
	{
		check(GraphBuilder.FindExternalBuffer(Buffer) != nullptr);
	}
	return TransientRDGSRV;
}

FRDGBufferUAVRef FNiagaraPooledRWBuffer::GetOrCreateUAV(FRDGBuilder& GraphBuilder)
{
	check(IsValid());
	if (TransientRDGUAV == nullptr)
	{
		TransientRDGUAV = GraphBuilder.CreateUAV(GetOrCreateBuffer(GraphBuilder), PixelFormat);
	}
	else
	{
		check(GraphBuilder.FindExternalBuffer(Buffer) != nullptr);
	}
	return TransientRDGUAV;
}

void FNiagaraPooledRWBuffer::EndGraphUsage()
{
	TransientRDGBuffer = nullptr;
	TransientRDGSRV = nullptr;
	TransientRDGUAV = nullptr;
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraPooledRWTexture::InitializeInternal(FRDGBuilder& GraphBuilder, const TCHAR* ResourceName, const FRDGTextureDesc& TextureDesc)
{
	Release();

	TransientRDGTexture = GraphBuilder.CreateTexture(TextureDesc, ResourceName);
	Texture = GraphBuilder.ConvertToExternalTexture(TransientRDGTexture);

#if STATS
	NumBytes = RHIComputeMemorySize(Texture->GetRHI());
	INC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, NumBytes);
#endif
}

void FNiagaraPooledRWTexture::Release()
{
#if STATS
	DEC_MEMORY_STAT_BY(STAT_NiagaraGPUDataInterfaceMemory, NumBytes);
	NumBytes = 0;
#endif
	Texture.SafeRelease();
}

FRDGTextureRef FNiagaraPooledRWTexture::GetOrCreateTexture(FRDGBuilder& GraphBuilder)
{
	if (TransientRDGTexture == nullptr)
	{
		TransientRDGTexture = GraphBuilder.RegisterExternalTexture(Texture);
	}
	else
	{
		check(GraphBuilder.FindExternalTexture(Texture) != nullptr);
	}
	return TransientRDGTexture;
}

FRDGTextureSRVRef FNiagaraPooledRWTexture::GetOrCreateSRV(FRDGBuilder& GraphBuilder)
{
	check(IsValid());
	if (TransientRDGSRV == nullptr)
	{
		TransientRDGSRV = GraphBuilder.CreateSRV(GetOrCreateTexture(GraphBuilder));
	}
	else
	{
		check(GraphBuilder.FindExternalTexture(Texture) != nullptr);
	}
	return TransientRDGSRV;
}

FRDGTextureUAVRef FNiagaraPooledRWTexture::GetOrCreateUAV(FRDGBuilder& GraphBuilder)
{
	check(IsValid());
	if (TransientRDGUAV == nullptr)
	{
		TransientRDGUAV = GraphBuilder.CreateUAV(GetOrCreateTexture(GraphBuilder));
	}
	else
	{
		check(GraphBuilder.FindExternalTexture(Texture) != nullptr);
	}
	return TransientRDGUAV;
}

void FNiagaraPooledRWTexture::CopyToTexture(FRDGBuilder& GraphBuilder, FRHITexture* DestinationTextureRHI, const TCHAR* NameIfNotRegistered)
{
	const FPooledRenderTargetDesc& TextureDesc = Texture->GetDesc();
	const FRHITextureDesc& DestTextureDesc = DestinationTextureRHI->GetDesc();
	if (!ensureMsgf(
			(DestTextureDesc.Dimension == ETextureDimension::Texture3D) &&
			(DestTextureDesc.Format == TextureDesc.Format) &&
			(DestTextureDesc.Extent == TextureDesc.Extent) &&
			(DestTextureDesc.Depth == TextureDesc.Depth),
			TEXT("FNiagaraPooledRWTexture::CopyToTexture - Destination does not match copy will fail")
		)
	)
	{
		return;
	}

	FRDGTexture* SourceTexture = GetOrCreateTexture(GraphBuilder);
	FRDGTexture* DestinationTexture = GraphBuilder.FindExternalTexture(DestinationTextureRHI);
	if (DestinationTexture == nullptr)
	{
		DestinationTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DestinationTextureRHI, NameIfNotRegistered));
	}

	FRHICopyTextureInfo CopyInfo;
	CopyInfo.NumMips = SourceTexture->Desc.NumMips;
	CopyInfo.NumSlices = SourceTexture->Desc.ArraySize;
	AddCopyTexturePass(GraphBuilder, SourceTexture, DestinationTexture, CopyInfo);
}

void FNiagaraPooledRWTexture::EndGraphUsage()
{
	TransientRDGTexture = nullptr;
	TransientRDGSRV = nullptr;
	TransientRDGUAV = nullptr;
}
