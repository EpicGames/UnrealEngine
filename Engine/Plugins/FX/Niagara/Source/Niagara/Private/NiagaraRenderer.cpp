// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRenderer.h"
#include "Engine/Texture.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "NiagaraDataSet.h"
#include "NiagaraSceneProxy.h"
#include "NiagaraStats.h"
#include "NiagaraShader.h"
#include "DynamicBufferAllocator.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraGPUSortInfo.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraSystemInstanceController.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "PrimitiveViewRelevance.h"

DECLARE_CYCLE_STAT(TEXT("Sort Particles"), STAT_NiagaraSortParticles, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Global Float Alloc - All"), STAT_NiagaraAllocateGlobalFloatAll, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Global Float Alloc - InsideLock"), STAT_NiagaraAllocateGlobalFloatInsideLock, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Global Float Alloc - Alloc New Buffer"), STAT_NiagaraAllocateGlobalFloatAllocNew, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Global Float Alloc - Map Buffer"), STAT_NiagaraAllocateGlobalFloatMapBuffer, STATGROUP_Niagara);

int32 GNiagaraRadixSortThreshold = 400;
static FAutoConsoleVariableRef CVarNiagaraRadixSortThreshold(
	TEXT("Niagara.RadixSortThreshold"),
	GNiagaraRadixSortThreshold,
	TEXT("Instance count at which radix sort gets used instead of introspective sort.\n")
	TEXT("Set to  -1 to never use radixsort. (default=400)"),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////

bool UNiagaraRendererProperties::GetIsActive()const
{
	return GetIsEnabled() && Platforms.IsActive();
}

//////////////////////////////////////////////////////////////////////////

class FNiagaraEmptyBufferSRV : public FRenderResource
{
public:
	FNiagaraEmptyBufferSRV(EPixelFormat InPixelFormat, const FString& InDebugName, uint32 InDefaultValue = 0) : PixelFormat(InPixelFormat), DebugName(InDebugName), DefaultValue(InDefaultValue) {}
	EPixelFormat PixelFormat;
	FString DebugName;
	FBufferRHIRef Buffer;
	FShaderResourceViewRHIRef SRV;
	uint32 DefaultValue = 0;


	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// Create a buffer with one element.
		uint32 NumBytes = GPixelFormats[PixelFormat].BlockBytes;

		FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(*DebugName, NumBytes)
			.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Static)
			.DetermineInitialState();

		if (PixelFormat == PF_R8G8B8A8)
		{
			FRHIBufferInitializer Initializer = RHICmdList.CreateBufferInitializer(CreateDesc.SetInitActionInitializer());
			Initializer.WriteData(&DefaultValue, sizeof(DefaultValue));
			Buffer = Initializer.Finalize();
		}
		else
		{
			Buffer = RHICmdList.CreateBuffer(CreateDesc.SetInitActionZeroData());
		}

		SRV = RHICmdList.CreateShaderResourceView(
			Buffer, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PixelFormat));
	}

	virtual void ReleaseRHI() override
	{
		SRV.SafeRelease();
		Buffer.SafeRelease();
	}
};

class FNiagaraEmptyTextureSRV : public FRenderResource
{
public:
	FNiagaraEmptyTextureSRV(EPixelFormat InPixelFormat, const FString& InDebugName, ETextureDimension InDimension)
		: PixelFormat(InPixelFormat)
		, DebugName(InDebugName)
		, Dimension(InDimension)
	{}

	EPixelFormat PixelFormat;
	FString DebugName;
	ETextureDimension Dimension;
	FTextureRHIRef Texture;
	FShaderResourceViewRHIRef SRV;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FRHITextureCreateDesc Desc = FRHITextureCreateDesc(*DebugName, Dimension)
			.SetExtent(1, 1)
			.SetDepth(1)
			.SetArraySize(1)
			.SetFormat(PixelFormat)
			.SetFlags(ETextureCreateFlags::ShaderResource);

		// Create a 1x1 texture.

		switch (Dimension)
		{
			case ETextureDimension::Texture2D:
			case ETextureDimension::Texture2DArray:
			{
				FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc.SetInitActionInitializer());
				FRHITextureSubresourceInitializer Subresource = Initializer.GetSubresource(FRHITextureInitializer::FSubresourceIndex());
				FMemory::Memzero(Subresource.Data, Subresource.Size);
				Texture = Initializer.Finalize();
				break;
			}

			case ETextureDimension::Texture3D:
			{
				FTextureRHIRef Texture3D = RHICmdList.CreateTexture(Desc);
				Texture = Texture3D;

				const FPixelFormatInfo& Info = GPixelFormats[PixelFormat];
				TArray<uint8, TInlineAllocator<16>> Data;
				Data.AddZeroed(Info.BlockBytes);
				RHICmdList.UpdateTexture3D(Texture3D, 0, FUpdateTextureRegion3D(0, 0, 0, 0, 0, 0, 1, 1, 1), Info.BlockBytes, Info.BlockBytes, Data.GetData());
				break;
			}

			default:
				checkNoEntry();
				return;
		}

		SRV = RHICmdList.CreateShaderResourceView(Texture, FRHIViewDesc::CreateTextureSRV().SetDimensionFromTexture(Texture));
	}

	virtual void ReleaseRHI() override
	{
		SRV.SafeRelease();
		Texture.SafeRelease();
	}
};

class FNiagaraEmptyByteAddressBufferSRV : public FRenderResource
{
public:
	FNiagaraEmptyByteAddressBufferSRV(const FString& InDebugName) : DebugName(InDebugName) {}

	FString DebugName;
	FBufferRHIRef Buffer;
	FShaderResourceViewRHIRef SRV;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		constexpr int32 NumBytes = sizeof(FUintVector4);

		FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateByteAddress(*DebugName, NumBytes, 0)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetInitActionZeroData();

		Buffer = RHICmdList.CreateBuffer(CreateDesc);

		SRV = RHICmdList.CreateShaderResourceView(
			Buffer,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Raw)
		);
	}

	virtual void ReleaseRHI() override
	{
		SRV.SafeRelease();
		Buffer.SafeRelease();
	}
};

static TGlobalResource<FNiagaraEmptyBufferSRV> DummyFloatBuffer(PF_R32_FLOAT, TEXT("NiagaraRenderer::DummyFloat"));
static TGlobalResource<FNiagaraEmptyBufferSRV> DummyFloat2Buffer(PF_G16R16F, TEXT("NiagaraRenderer::DummyFloat2"));
static TGlobalResource<FNiagaraEmptyBufferSRV> DummyFloat4Buffer(PF_A32B32G32R32F, TEXT("NiagaraRenderer::DummyFloat4"));
static TGlobalResource<FNiagaraEmptyBufferSRV> DummyWhiteColorBuffer(PF_R8G8B8A8, TEXT("NiagaraRenderer::DummyWhiteColorBuffer"), FColor::White.ToPackedRGBA());
static TGlobalResource<FNiagaraEmptyBufferSRV> DummyIntBuffer(PF_R32_SINT, TEXT("NiagaraRenderer::DummyInt"));
static TGlobalResource<FNiagaraEmptyBufferSRV> DummyUIntBuffer(PF_R32_UINT, TEXT("NiagaraRenderer::DummyUInt"));
static TGlobalResource<FNiagaraEmptyBufferSRV> DummyUInt2Buffer(PF_R32G32_UINT, TEXT("NiagaraRenderer::DummyUInt2"));
static TGlobalResource<FNiagaraEmptyBufferSRV> DummyUInt4Buffer(PF_R32G32B32A32_UINT, TEXT("NiagaraRenderer::DummyUInt4"));
static TGlobalResource<FNiagaraEmptyBufferSRV> DummyHalfBuffer(PF_R16F, TEXT("NiagaraRenderer::DummyHalf"));
static TGlobalResource<FNiagaraEmptyByteAddressBufferSRV> DummyByteAddressBuffer(TEXT("NiagaraRenderer::DummyByteAddressBuffer"));

FRHIShaderResourceView* FNiagaraRenderer::GetDummyFloatBuffer()
{
	return DummyFloatBuffer.SRV;
}

FRHIShaderResourceView* FNiagaraRenderer::GetDummyFloat2Buffer()
{
	return DummyFloat2Buffer.SRV;
}

FRHIShaderResourceView* FNiagaraRenderer::GetDummyFloat4Buffer()
{
	return DummyFloat4Buffer.SRV;
}

FRHIShaderResourceView* FNiagaraRenderer::GetDummyWhiteColorBuffer()
{
	return DummyWhiteColorBuffer.SRV;
}

FRHIShaderResourceView* FNiagaraRenderer::GetDummyIntBuffer()
{;
	return DummyIntBuffer.SRV;
}

FRHIShaderResourceView* FNiagaraRenderer::GetDummyUIntBuffer()
{
	return DummyUIntBuffer.SRV;
}

FRHIShaderResourceView* FNiagaraRenderer::GetDummyUInt2Buffer()
{
	return DummyUInt2Buffer.SRV;
}

FRHIShaderResourceView* FNiagaraRenderer::GetDummyUInt4Buffer()
{
	return DummyUInt4Buffer.SRV;
}

FRHIShaderResourceView* FNiagaraRenderer::GetDummyHalfBuffer()
{
	return DummyHalfBuffer.SRV;
}

FRHIShaderResourceView* FNiagaraRenderer::GetDummyByteAddressBuffer()
{
	return DummyByteAddressBuffer.SRV;
}

FParticleRenderData FNiagaraRenderer::TransferDataToGPU(FRHICommandListBase& RHICmdList, FGlobalDynamicReadBuffer& DynamicReadBuffer, const FNiagaraRendererLayout* RendererLayout, TConstArrayView<uint32> IntComponents, const FNiagaraDataBuffer* SrcData)
{
	const int32 NumInstances = SrcData->GetNumInstances();
	const int32 TotalFloatSize = RendererLayout->GetTotalFloatComponents_RenderThread() * NumInstances;
	const int32 TotalHalfSize = RendererLayout->GetTotalHalfComponents_RenderThread() * NumInstances;
	const int32 TotalIntSize = IntComponents.Num() * NumInstances;

	FParticleRenderData Allocation;
	Allocation.FloatData = TotalFloatSize ? DynamicReadBuffer.AllocateFloat(TotalFloatSize) : FGlobalDynamicReadBuffer::FAllocation();
	Allocation.HalfData = TotalHalfSize ? DynamicReadBuffer.AllocateHalf(TotalHalfSize) : FGlobalDynamicReadBuffer::FAllocation();
	Allocation.IntData = TotalIntSize ? DynamicReadBuffer.AllocateInt32(TotalIntSize) : FGlobalDynamicReadBuffer::FAllocation();

	Allocation.FloatStride = TotalFloatSize ? NumInstances * sizeof(float) : 0;
	Allocation.HalfStride = TotalHalfSize ? NumInstances * sizeof(FFloat16) : 0;
	Allocation.IntStride = TotalIntSize ? NumInstances * sizeof(int32) : 0;

	for (const FNiagaraRendererVariableInfo& VarInfo : RendererLayout->GetVFVariables_RenderThread())
	{
		const int32 GpuOffset = VarInfo.GetRawGPUOffset();
		if (GpuOffset != INDEX_NONE && VarInfo.ShouldUpload())
		{
			if (VarInfo.IsHalfType())
			{
				for (int32 CompIdx = 0; CompIdx < VarInfo.GetNumComponents(); ++CompIdx)
				{
					const uint8* SrcComponent = SrcData->GetComponentPtrHalf(VarInfo.GetRawDatasetOffset() + CompIdx);
					void* Dest = Allocation.HalfData.Buffer + Allocation.HalfStride * (GpuOffset + CompIdx);
					FMemory::Memcpy(Dest, SrcComponent, Allocation.HalfStride);
				}
			}
			else
			{
				for (int32 CompIdx = 0; CompIdx < VarInfo.GetNumComponents(); ++CompIdx)
				{
					const uint8* SrcComponent = SrcData->GetComponentPtrFloat(VarInfo.GetRawDatasetOffset() + CompIdx);
					void* Dest = Allocation.FloatData.Buffer + Allocation.FloatStride * (GpuOffset + CompIdx);
					FMemory::Memcpy(Dest, SrcComponent, Allocation.FloatStride);
				}
			}
		}
	}

	if (TotalIntSize > 0)
	{
		for (int i=0; i < IntComponents.Num(); ++i )
		{
			uint8* Dst = Allocation.IntData.Buffer + Allocation.IntStride * i;
			const uint8* Src = SrcData->GetComponentPtrInt32(IntComponents[i]);
			FMemory::Memcpy(Dst, Src, Allocation.IntStride);
		}
	}

	return Allocation;
}

//////////////////////////////////////////////////////////////////////////

FNiagaraDynamicDataBase::FNiagaraDynamicDataBase(const FNiagaraEmitterInstance* InEmitter)
{
	check(InEmitter);

	SystemInstanceID = InEmitter->GetParentSystemInstance()->GetId();

	// CPU simulations we take a reference to the most recent data
	if (InEmitter->GetSimTarget() == ENiagaraSimTarget::CPUSim)
	{
		CPUParticleData = &InEmitter->GetParticleData().GetCurrentDataChecked();
	}
	// GPU simulations we get a callback which will give us the correct data
	else
	{
		ComputeDataBufferInterface = InEmitter->GetComputeDataBufferInterface();
	}
}

FNiagaraDynamicDataBase::~FNiagaraDynamicDataBase()
{
}

bool FNiagaraDynamicDataBase::IsGpuLowLatencyTranslucencyEnabled() const
{
	return ComputeDataBufferInterface ? ComputeDataBufferInterface->HasTranslucentDataToRender() : false;
}

FNiagaraDataBuffer* FNiagaraDynamicDataBase::GetParticleDataToRender(FRHICommandListBase& RHICmdList, bool bIsLowLatencyTranslucent) const
{
	FNiagaraDataBuffer* Ret = ComputeDataBufferInterface ? ComputeDataBufferInterface->GetDataToRender(RHICmdList, bIsLowLatencyTranslucent) : CPUParticleData.GetReference();
	checkSlow(Ret == nullptr || Ret->IsBeingRead());
	return Ret;
}

//////////////////////////////////////////////////////////////////////////
FNiagaraRenderer::FNiagaraRenderer(ERHIFeatureLevel::Type InFeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter)
	: DynamicDataRender(nullptr)
	, bLocalSpace(Emitter->IsLocalSpace())
	, bHasLights(false)
	, bMotionBlurEnabled(InProps ? InProps->MotionVectorSetting != ENiagaraRendererMotionVectorSetting::Disable : false)
	, SimTarget(Emitter->GetSimTarget())
	, FeatureLevel(InFeatureLevel)
#if STATS
	, EmitterStatID(Emitter->GetEmitterStatID(false, false))
#endif
{
}

void FNiagaraRenderer::Initialize(const UNiagaraRendererProperties* InProps, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
	//Get our list of valid base materials. Fall back to default material if they're not valid.
	BaseMaterials_GT.Empty();
	InProps->GetUsedMaterials(Emitter, BaseMaterials_GT);
	bool bCreateMidsForUsedMaterials = InProps->NeedsMIDsForMaterials();

	bRendersInSecondaryDepthPass = false;

	// Let check if the GPU simulation shader script needed to use the partial depth texture for depth queries.
	const bool bNeedsPartialDepthTexture = Emitter->NeedsPartialDepthTexture();

	uint32 Index = 0;
	for (UMaterialInterface*& Mat : BaseMaterials_GT)
	{
		if (!IsMaterialValid(Mat))
		{
			Mat = UMaterial::GetDefaultMaterial(MD_Surface);
		}
		else if (Mat && bCreateMidsForUsedMaterials && !Mat->IsA<UMaterialInstanceDynamic>())
		{
			if (UMaterialInterface* Override = InController.GetMaterialOverride(InProps, Index))
			{
				Mat = Override;
			}
		}

		Index ++;
		if (Mat)
		{
			BaseMaterialRelevance_GT |= Mat->GetRelevance_Concurrent(GetFeatureLevelShaderPlatform_Checked(FeatureLevel));

			// If partial depth was sampled, we need to make sure this emitter does not end up in it but only in this full depth texture.
			// For exemple, this is to avoid self collision.
			bRendersInSecondaryDepthPass |= IsOpaqueOrMaskedBlendMode(Mat->GetBlendMode()) && bNeedsPartialDepthTexture;
		}
	}
}

FNiagaraRenderer::~FNiagaraRenderer()
{
	ReleaseRenderThreadResources();
	SetDynamicData_RenderThread(nullptr);
}

FPrimitiveViewRelevance FNiagaraRenderer::GetViewRelevance(const FSceneView* View, const FNiagaraSceneProxy *SceneProxy)const
{
	FPrimitiveViewRelevance Result;
	if (HasDynamicData())
	{
		Result.bOpaque = View->Family->EngineShowFlags.Bounds;
		Result.bRenderInSecondStageDepthPass = bRendersInSecondaryDepthPass;
		DynamicDataRender->GetMaterialRelevance().SetPrimitiveViewRelevance(Result);
	}

	return Result;
}

void FNiagaraRenderer::SetDynamicData_RenderThread(FNiagaraDynamicDataBase* NewDynamicData)
{
	if (DynamicDataRender)
	{
		delete DynamicDataRender;
		DynamicDataRender = NULL;
	}
	DynamicDataRender = NewDynamicData;
}

struct FParticleOrderAsUint
{
	uint32 OrderAsUint;
	int32 Index;

	template <bool bStrictlyPositive, bool bAscending>
	FORCEINLINE void SetAsUint(int32 InIndex, float InOrder)
	{
		const uint32 SortKeySignBit = 0x80000000;
		uint32 InOrderAsUint = reinterpret_cast<uint32&>(InOrder);
		InOrderAsUint = (bStrictlyPositive || InOrder >= 0) ? (InOrderAsUint | SortKeySignBit) : ~InOrderAsUint;
		OrderAsUint = bAscending ? InOrderAsUint : ~InOrderAsUint;

		Index = InIndex;
	}

	template <bool bStrictlyPositive, bool bAscending>
	FORCEINLINE_DEBUGGABLE void SetAsUint(int32 InIndex, FFloat16 InOrder)
	{
		const uint32 SortKeySignBit = 0x8000;
		uint32 InOrderAsUint = InOrder.Encoded;
		InOrderAsUint = (bStrictlyPositive || InOrder.IsNegative()) ? (InOrderAsUint | SortKeySignBit) : ~InOrderAsUint;
		OrderAsUint = bAscending ? InOrderAsUint : ~InOrderAsUint;
		OrderAsUint &= 0xFFFF;
		Index = InIndex;
	}

	FORCEINLINE operator uint32() const { return OrderAsUint; }
};

bool FNiagaraRenderer::IsRendererEnabled(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const
{
	if (InProperties->RendererEnabledBinding.GetParamMapBindableVariable().IsValid())
	{
		const FNiagaraParameterStore& BoundParamStore = Emitter->GetRendererBoundVariables();
		if (const uint8* ParameterData = BoundParamStore.GetParameterData(InProperties->RendererEnabledBinding.GetParamMapBindableVariable()))
		{
			const FNiagaraBool RendererEnabled = *reinterpret_cast<const FNiagaraBool*>(ParameterData);
			if (RendererEnabled.GetValue() == false)
			{
				return false;
			}
		}
	}
	if (Emitter->IsWaitingForPSOCaching())
	{
		return false;
	}
	return true;
}

bool FNiagaraRenderer::UseLocalSpace(const FNiagaraSceneProxy* Proxy)const
{
	//Force local space if we're using a cull proxy. Cull proxies are simulated at the origin so their world space particles can be transformed to this system as though they were local space.
	return bLocalSpace || Proxy->GetProxyDynamicData().bUseCullProxy;
}

bool FNiagaraRenderer::ViewFamilySupportLowLatencyTranslucency(const FSceneViewFamily& ViewFamily)
{
	return
		// Disable low latency when anything other than the lit view mode is active
		// Some of the other view modes short circuit rendering so the data may not be ready in time resulting in corruption / GPU hangs
		ViewFamily.ViewMode == VMI_Lit &&
		// when underwater, single layer water will render translucency before opaque
		!ViewFamily.Views.ContainsByPredicate([](const FSceneView* SceneView) { return SceneView->IsUnderwater(); });
}

void FNiagaraRenderer::ProcessMaterialParameterBindings(const FNiagaraRendererMaterialParameters& MaterialParameters, const FNiagaraEmitterInstance* InEmitter, TConstArrayView<UMaterialInterface*> InMaterials) const
{
	if (MaterialParameters.HasAnyBindings() == false || !InEmitter)
	{
		return;
	}

	FNiagaraSystemInstance* SystemInstance = InEmitter->GetParentSystemInstance();
	if (SystemInstance)
	{
		auto SystemSim = SystemInstance->GetSystemSimulation();

		if (SystemSim.IsValid())
		{
			for (UMaterialInterface* Mat : InMaterials)
			{
				UMaterialInstanceDynamic* MatDyn = Cast<UMaterialInstanceDynamic>(Mat);
				if (MatDyn)
				{
					for (const FNiagaraMaterialAttributeBinding& Binding : MaterialParameters.AttributeBindings)
					{
						if (Binding.GetParamMapBindableVariable().GetType() == FNiagaraTypeDefinition::GetVec4Def() ||
							(Binding.GetParamMapBindableVariable().GetType().IsDataInterface() && Binding.NiagaraChildVariable.GetType() == FNiagaraTypeDefinition::GetVec4Def()))
						{
							FLinearColor Var(1.0f, 1.0f, 1.0f, 1.0f);
							InEmitter->GetBoundRendererValue_GT(Binding.GetParamMapBindableVariable(), Binding.NiagaraChildVariable, &Var);
							MatDyn->SetVectorParameterValue(Binding.MaterialParameterName, Var);
						}
						else if (Binding.GetParamMapBindableVariable().GetType() == FNiagaraTypeDefinition::GetColorDef() ||
							(Binding.GetParamMapBindableVariable().GetType().IsDataInterface() && Binding.NiagaraChildVariable.GetType() == FNiagaraTypeDefinition::GetColorDef()))
						{
							FLinearColor Var(1.0f, 1.0f, 1.0f, 1.0f);
							InEmitter->GetBoundRendererValue_GT(Binding.GetParamMapBindableVariable(), Binding.NiagaraChildVariable, &Var);
							MatDyn->SetVectorParameterValue(Binding.MaterialParameterName, Var);
						}
						else if (Binding.GetParamMapBindableVariable().GetType() == FNiagaraTypeDefinition::GetVec3Def() ||
							(Binding.GetParamMapBindableVariable().GetType().IsDataInterface() && Binding.NiagaraChildVariable.GetType() == FNiagaraTypeDefinition::GetVec3Def()))
						{
							FLinearColor Var(1.0f, 1.0f, 1.0f, 1.0f);
							InEmitter->GetBoundRendererValue_GT(Binding.GetParamMapBindableVariable(), Binding.NiagaraChildVariable, &Var);
							MatDyn->SetVectorParameterValue(Binding.MaterialParameterName, Var);
						}
						else if (Binding.GetParamMapBindableVariable().GetType() == FNiagaraTypeDefinition::GetPositionDef() ||
							(Binding.GetParamMapBindableVariable().GetType().IsDataInterface() && Binding.NiagaraChildVariable.GetType() == FNiagaraTypeDefinition::GetPositionDef()))
						{
							FNiagaraPosition Var(ForceInitToZero);
							InEmitter->GetBoundRendererValue_GT(Binding.GetParamMapBindableVariable(), Binding.NiagaraChildVariable, &Var);
							const FNiagaraLWCConverter LwcConverter = SystemInstance->GetLWCConverter(false);
							const FVector WorldPos = LwcConverter.ConvertSimulationPositionToWorld(Var);

							MatDyn->SetDoubleVectorParameterValue(Binding.MaterialParameterName, WorldPos);
							MatDyn->SetVectorParameterValue(Binding.MaterialParameterName, WorldPos);
						}
						else if (Binding.GetParamMapBindableVariable().GetType() == FNiagaraTypeDefinition::GetVec2Def() ||
							(Binding.GetParamMapBindableVariable().GetType().IsDataInterface() && Binding.NiagaraChildVariable.GetType() == FNiagaraTypeDefinition::GetVec2Def()))
						{
							FLinearColor Var(1.0f, 1.0f, 1.0f, 1.0f);
							InEmitter->GetBoundRendererValue_GT(Binding.GetParamMapBindableVariable(), Binding.NiagaraChildVariable, &Var);
							MatDyn->SetVectorParameterValue(Binding.MaterialParameterName, Var);
						}
						else if (Binding.GetParamMapBindableVariable().GetType() == FNiagaraTypeDefinition::GetFloatDef() ||
							(Binding.GetParamMapBindableVariable().GetType().IsDataInterface() && Binding.NiagaraChildVariable.GetType() == FNiagaraTypeDefinition::GetFloatDef()))
						{
							float Var = 1.0f;
							InEmitter->GetBoundRendererValue_GT(Binding.GetParamMapBindableVariable(), Binding.NiagaraChildVariable, &Var);
							MatDyn->SetScalarParameterValue(Binding.MaterialParameterName, Var);
						}
						else if (
							Binding.GetParamMapBindableVariable().GetType() == FNiagaraTypeDefinition::GetUObjectDef() ||
							Binding.GetParamMapBindableVariable().GetType() == FNiagaraTypeDefinition::GetUTextureDef() ||
							Binding.GetParamMapBindableVariable().GetType() == FNiagaraTypeDefinition::GetUTextureRenderTargetDef() ||
							(Binding.GetParamMapBindableVariable().GetType().IsDataInterface() && Binding.NiagaraChildVariable.GetType() == FNiagaraTypeDefinition::GetUTextureDef())
							)
						{
							UObject* Var = nullptr;
							InEmitter->GetBoundRendererValue_GT(Binding.GetParamMapBindableVariable(), Binding.NiagaraChildVariable, &Var);
							UTexture* Tex = Cast<UTexture>(Var);
							if (Tex && Tex->GetResource())
							{
								MatDyn->SetTextureParameterValue(Binding.MaterialParameterName, Tex);
							}
							// Not a valid texture?  Setting nullptr will not remove the override and set to default.
							// We have to do this manually getting the parent material default.
							else if ( const UMaterial* ParentMaterial = MatDyn->GetMaterial() )
							{
								ParentMaterial->GetTextureParameterValue(Binding.MaterialParameterName, Tex, false);
								MatDyn->SetTextureParameterValue(Binding.MaterialParameterName, Tex);
							}
						}
					}

					for (const FNiagaraRendererMaterialScalarParameter& ScalarParameter : MaterialParameters.ScalarParameters)
					{
						MatDyn->SetScalarParameterValue(ScalarParameter.MaterialParameterName, ScalarParameter.Value);
					}

					for (const FNiagaraRendererMaterialVectorParameter& VectorParameter : MaterialParameters.VectorParameters)
					{
						MatDyn->SetVectorParameterValue(VectorParameter.MaterialParameterName, VectorParameter.Value);
					}

					for (const FNiagaraRendererMaterialTextureParameter& TextureParameter : MaterialParameters.TextureParameters)
					{
						if (IsValid(TextureParameter.Texture))
						{
							MatDyn->SetTextureParameterValue(TextureParameter.MaterialParameterName, TextureParameter.Texture);
						}
					}
				}
			}
		}
	}
}

bool FNiagaraRenderer::IsViewRenderingOpaqueOnly(const FSceneView* View, bool bCanCastTranslucentShadows)
{
	const bool bShadowView = View->GetDynamicMeshElementsShadowCullFrustum() != nullptr;
	return View->CustomRenderPass || (bShadowView && !bCanCastTranslucentShadows);
}

bool FNiagaraRenderer::AreViewsRenderingOpaqueOnly(const TArray<const FSceneView*>& Views, int32 ViewVisibilityMask, bool bCanCastTranslucentShadows)
{
	for ( int32 i=0; i < Views.Num(); ++i )
	{
		if (((ViewVisibilityMask & (1 << i)) != 0) && !IsViewRenderingOpaqueOnly(Views[i], bCanCastTranslucentShadows))
		{
			return false;
		}
	}
	return true;
}

void FNiagaraRenderer::SortIndices(const FNiagaraGPUSortInfo& SortInfo, const FNiagaraRendererVariableInfo& SortVariable, const FNiagaraDataBuffer& Buffer, FGlobalDynamicReadBuffer::FAllocation& OutIndices)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSortParticles);

	uint32 NumInstances = Buffer.GetNumInstances();
	check(OutIndices.ReadBuffer->NumBytes >= (OutIndices.Buffer - OutIndices.ReadBuffer->MappedBuffer) + NumInstances * sizeof(int32));
	check(SortInfo.SortMode != ENiagaraSortMode::None);
	check(SortInfo.SortAttributeOffset != INDEX_NONE);

	const bool bUseRadixSort = GNiagaraRadixSortThreshold != -1 && (int32)NumInstances  > GNiagaraRadixSortThreshold;
	const bool bSortVarIsHalf = SortVariable.IsHalfType();

	int32* RESTRICT IndexBuffer = (int32*)(OutIndices.Buffer);

	FMemMark Mark(FMemStack::Get());
	FParticleOrderAsUint* RESTRICT ParticleOrder = (FParticleOrderAsUint*)FMemStack::Get().Alloc(sizeof(FParticleOrderAsUint) * NumInstances, alignof(FParticleOrderAsUint));

	if (SortInfo.SortMode == ENiagaraSortMode::ViewDepth || SortInfo.SortMode == ENiagaraSortMode::ViewDistance)
	{
		if (bSortVarIsHalf)
		{
			const int32 BaseCompOffset = SortVariable.GetRawDatasetOffset();
			FFloat16* RESTRICT PositionX = (FFloat16*)Buffer.GetComponentPtrHalf(BaseCompOffset);
			FFloat16* RESTRICT PositionY = (FFloat16*)Buffer.GetComponentPtrHalf(BaseCompOffset + 1);
			FFloat16* RESTRICT PositionZ = (FFloat16*)Buffer.GetComponentPtrHalf(BaseCompOffset + 2);
			auto GetPos = [&PositionX, &PositionY, &PositionZ](int32 Idx)
			{
				return FVector(PositionX[Idx], PositionY[Idx], PositionZ[Idx]);
			};

			if (SortInfo.SortMode == ENiagaraSortMode::ViewDepth)
			{
				for (uint32 i = 0; i < NumInstances; ++i)
				{
					ParticleOrder[i].SetAsUint<true, false>(i, float(FVector::DotProduct(GetPos(i) - SortInfo.ViewOrigin, SortInfo.ViewDirection)));
				}
			}
			else
			{
				for (uint32 i = 0; i < NumInstances; ++i)
				{
					ParticleOrder[i].SetAsUint<true, false>(i, float((GetPos(i) - SortInfo.ViewOrigin).SizeSquared()));
				}
			}
		}
		else
		{
			const int32 BaseCompOffset = SortVariable.GetRawDatasetOffset();
			float* RESTRICT PositionX = (float*)Buffer.GetComponentPtrFloat(BaseCompOffset);
			float* RESTRICT PositionY = (float*)Buffer.GetComponentPtrFloat(BaseCompOffset + 1);
			float* RESTRICT PositionZ = (float*)Buffer.GetComponentPtrFloat(BaseCompOffset + 2);
			auto GetPos = [&PositionX, &PositionY, &PositionZ](int32 Idx)
			{
				return FVector(PositionX[Idx], PositionY[Idx], PositionZ[Idx]);
			};

			if (SortInfo.SortMode == ENiagaraSortMode::ViewDepth)
			{
				for (uint32 i = 0; i < NumInstances; ++i)
				{
					ParticleOrder[i].SetAsUint<true, false>(i, float(FVector::DotProduct(GetPos(i) - SortInfo.ViewOrigin, SortInfo.ViewDirection)));
				}
			}
			else
			{
				for (uint32 i = 0; i < NumInstances; ++i)
				{
					ParticleOrder[i].SetAsUint<true, false>(i, float((GetPos(i) - SortInfo.ViewOrigin).SizeSquared()));
				}
			}
		}
	}
	else
	{
		if (bSortVarIsHalf)
		{
			FFloat16* RESTRICT CustomSorting = (FFloat16*)Buffer.GetComponentPtrHalf(SortVariable.GetRawDatasetOffset());
			if (SortInfo.SortMode == ENiagaraSortMode::CustomAscending)
			{
				for (uint32 i = 0; i < NumInstances; ++i)
				{
					ParticleOrder[i].SetAsUint<false, true>(i, CustomSorting[i]);
				}
			}
			else // ENiagaraSortMode::CustomDecending
			{
				for (uint32 i = 0; i < NumInstances; ++i)
				{
					ParticleOrder[i].SetAsUint<false, false>(i, CustomSorting[i]);
				}
			}
		}
		else
		{
			float* RESTRICT CustomSorting = (float*)Buffer.GetComponentPtrFloat(SortVariable.GetRawDatasetOffset());
			if (SortInfo.SortMode == ENiagaraSortMode::CustomAscending)
			{
				for (uint32 i = 0; i < NumInstances; ++i)
				{
					ParticleOrder[i].SetAsUint<false, true>(i, CustomSorting[i]);
				}
			}
			else // ENiagaraSortMode::CustomDecending
			{
				for (uint32 i = 0; i < NumInstances; ++i)
				{
					ParticleOrder[i].SetAsUint<false, false>(i, CustomSorting[i]);
				}
			}
		}
	}

	if (!bUseRadixSort)
	{
		Algo::SortBy(MakeArrayView(ParticleOrder, NumInstances), &FParticleOrderAsUint::OrderAsUint);
		//Now transfer to the real index buffer.
		for (uint32 i = 0; i < NumInstances; ++i)
		{
			IndexBuffer[i] = ParticleOrder[i].Index;
		}
	}
	else
	{
		FParticleOrderAsUint* RESTRICT ParticleOrderResult = bUseRadixSort ? (FParticleOrderAsUint*)FMemStack::Get().Alloc(sizeof(FParticleOrderAsUint) * NumInstances, alignof(FParticleOrderAsUint)) : nullptr;
		RadixSort32(ParticleOrderResult, ParticleOrder, NumInstances);
		//Now transfer to the real index buffer.
		for (uint32 i = 0; i < NumInstances; ++i)
		{
			IndexBuffer[i] = ParticleOrderResult[i].Index;
		}
	}
}

//-TODO: We don't perform distance / frustum culling in here yet
template<bool bWithInstanceCull>
struct FNiagaraSortCullHelper
{
private:
	FNiagaraSortCullHelper(const FNiagaraGPUSortInfo& InSortInfo, const FNiagaraDataBuffer& Buffer)
		: SortInfo(InSortInfo)
	{
		if (bWithInstanceCull)
		{
			VisibilityTag = SortInfo.RendererVisTagAttributeOffset == INDEX_NONE ? nullptr : (int32*)Buffer.GetComponentPtrInt32(SortInfo.RendererVisTagAttributeOffset);
			VisibilityValue = SortInfo.RendererVisibility;

			MeshIndexTag = SortInfo.MeshIndexAttributeOffset == INDEX_NONE ? nullptr : (int32*)Buffer.GetComponentPtrInt32(SortInfo.MeshIndexAttributeOffset);
			MeshIndexValue = SortInfo.MeshIndex;
		}
	}

	bool IsVisibile(int32 i) const
	{
		if (bWithInstanceCull)
		{
			if (VisibilityTag && (VisibilityTag[i] != VisibilityValue))
			{
				return false;
			}
			if (MeshIndexTag && (MeshIndexTag[i] != MeshIndexValue))
			{
				return false;
			}
		}
		return true;
	}

	template<bool bStrictlyPositive, bool bAscending, typename TSortKey>
	int32 BuildParticleOrder_Inner2(const int32 NumInstances, FParticleOrderAsUint* RESTRICT ParticleOrder, TSortKey GetSortKey)
	{
		int32 OutInstances = 0;
		for ( int32 i=0; i < NumInstances; ++i )
		{
			if (IsVisibile(i))
			{
				ParticleOrder[OutInstances++].SetAsUint<bStrictlyPositive, bAscending>(i, float(GetSortKey(i)));
			}
		}
		return OutInstances;
	}

	template<typename TComponentType, typename TGetComponent>
	int32 BuildParticleOrder_Inner1(const int32 NumInstances, const uint32 SortVariableOffset, TGetComponent GetComponent, FParticleOrderAsUint* RESTRICT ParticleOrder)
	{
		switch (SortInfo.SortMode)
		{
			case ENiagaraSortMode::ViewDepth:
			{
				TComponentType* RESTRICT PositionX = GetComponent(SortVariableOffset + 0);
				TComponentType* RESTRICT PositionY = GetComponent(SortVariableOffset + 1);
				TComponentType* RESTRICT PositionZ = GetComponent(SortVariableOffset + 2);
				return BuildParticleOrder_Inner2<true, false>(NumInstances, ParticleOrder, [&](int32 i) { return FVector::DotProduct(FVector(PositionX[i], PositionY[i], PositionZ[i]) - SortInfo.ViewOrigin, SortInfo.ViewDirection); });
			}
			case ENiagaraSortMode::ViewDistance:
			{
				TComponentType* RESTRICT PositionX = GetComponent(SortVariableOffset + 0);
				TComponentType* RESTRICT PositionY = GetComponent(SortVariableOffset + 1);
				TComponentType* RESTRICT PositionZ = GetComponent(SortVariableOffset + 2);
				return BuildParticleOrder_Inner2<true, false>(NumInstances, ParticleOrder, [&](int32 i) { return (FVector(PositionX[i], PositionY[i], PositionZ[i]) - SortInfo.ViewOrigin).SizeSquared(); });
			}
			case ENiagaraSortMode::CustomAscending:
			{
				TComponentType* RESTRICT CustomSorting = GetComponent(SortVariableOffset);
				return BuildParticleOrder_Inner2<false, true>(NumInstances, ParticleOrder, [&](int32 i) { return CustomSorting[i]; });
			}
			case ENiagaraSortMode::CustomDecending:
			{
				TComponentType* RESTRICT CustomSorting = GetComponent(SortVariableOffset);
				return BuildParticleOrder_Inner2<false, false>(NumInstances, ParticleOrder, [&](int32 i) { return CustomSorting[i]; });
			}
		}
		checkf(false, TEXT("Unknown sort mode"));
		return 0;
	}

	int32 CullInstances_Inner(int32 NumInstances, int32* RESTRICT OutIndexBuffer)
	{
		int32 OutInstances = 0;
		for ( int32 i=0; i < NumInstances; ++i )
		{
			if (IsVisibile(i))
			{
				OutIndexBuffer[OutInstances++] = i;
			}
		}
		return OutInstances;
	}

public:
	static int32 BuildParticleOrder(const FNiagaraGPUSortInfo& SortInfo, const FNiagaraDataBuffer& Buffer, FParticleOrderAsUint* RESTRICT ParticleOrder)
	{
		const uint32 SortVariableOffset = SortInfo.SortAttributeOffset & ~(1 << 31);
		const bool bSortIsHalf = SortVariableOffset != SortInfo.SortAttributeOffset;
		const int32 NumInstances = Buffer.GetNumInstances();

		if (bSortIsHalf)
		{
			return FNiagaraSortCullHelper(SortInfo, Buffer).BuildParticleOrder_Inner1<FFloat16>(NumInstances, SortVariableOffset, [&](int i) { return (FFloat16*)Buffer.GetComponentPtrHalf(i); }, ParticleOrder);
		}
		else
		{
			return FNiagaraSortCullHelper(SortInfo, Buffer).BuildParticleOrder_Inner1<float>(NumInstances, SortVariableOffset, [&](int i) { return (float*)Buffer.GetComponentPtrFloat(i); }, ParticleOrder);
		}
	}

	static int32 CullInstances(const FNiagaraGPUSortInfo& SortInfo, const FNiagaraDataBuffer& Buffer, int32* RESTRICT OutIndexBuffer)
	{
		return FNiagaraSortCullHelper(SortInfo, Buffer).CullInstances_Inner(Buffer.GetNumInstances(), OutIndexBuffer);
	}

private:
	const FNiagaraGPUSortInfo& SortInfo;

	int32* RESTRICT	VisibilityTag = nullptr;
	int32			VisibilityValue = 0;

	int32* RESTRICT	MeshIndexTag = nullptr;
	int32			MeshIndexValue = 0;
};

int32 FNiagaraRenderer::SortAndCullIndices(const FNiagaraGPUSortInfo& SortInfo, const FNiagaraDataBuffer& Buffer, FGlobalDynamicReadBuffer::FAllocation& OutIndices)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSortParticles);

	int32 OutNumInstances = 0;
	int32* RESTRICT IndexBuffer = (int32*)(OutIndices.Buffer);

	if ((SortInfo.SortMode != ENiagaraSortMode::None) && (SortInfo.SortAttributeOffset != INDEX_NONE))
	{
		const int32 SrcNumInstances = Buffer.GetNumInstances();

		FMemMark Mark(FMemStack::Get());
		FParticleOrderAsUint* RESTRICT ParticleOrder = (FParticleOrderAsUint*)FMemStack::Get().Alloc(sizeof(FParticleOrderAsUint) * SrcNumInstances, alignof(FParticleOrderAsUint));

		// Cull and prepare for sort
		if ( SortInfo.bEnableCulling )
		{
			OutNumInstances = FNiagaraSortCullHelper<true>::BuildParticleOrder(SortInfo, Buffer, ParticleOrder);
		}
		else
		{
			OutNumInstances = FNiagaraSortCullHelper<false>::BuildParticleOrder(SortInfo, Buffer, ParticleOrder);
		}

		// Perform the sort
		const bool bUseRadixSort = GNiagaraRadixSortThreshold != -1 && (int32)OutNumInstances > GNiagaraRadixSortThreshold;
		if (!bUseRadixSort)
		{
			Algo::SortBy(MakeArrayView(ParticleOrder, OutNumInstances), &FParticleOrderAsUint::OrderAsUint);

			for (int32 i = 0; i < OutNumInstances; ++i)
			{
				IndexBuffer[i] = ParticleOrder[i].Index;
			}
		}
		else
		{
			FParticleOrderAsUint* RESTRICT ParticleOrderResult = (FParticleOrderAsUint*)FMemStack::Get().Alloc(sizeof(FParticleOrderAsUint) * OutNumInstances, alignof(FParticleOrderAsUint));
			RadixSort32(ParticleOrderResult, ParticleOrder, OutNumInstances);

			for (int32 i = 0; i < OutNumInstances; ++i)
			{
				IndexBuffer[i] = ParticleOrderResult[i].Index;
			}
		}
	}
	else if ( SortInfo.bEnableCulling )
	{
		OutNumInstances = FNiagaraSortCullHelper<true>::CullInstances(SortInfo, Buffer, IndexBuffer);
	}
	else
	{
		checkf(false, TEXT("Either sorting or culling must be enabled or we don't generate output buffers"));
	}
	return OutNumInstances;
}

FVector4f FNiagaraRenderer::CalcMacroUVParameters(const FSceneView& View, FVector MacroUVPosition, float MacroUVRadius)
{
	FVector4f MacroUVParameters = FVector4f(0.0f, 0.0f, 1.0f, 1.0f);
	if (MacroUVRadius > 0.0f)
	{
		const FMatrix& ViewProjMatrix = View.ViewMatrices.GetWorldToClip();
		const FMatrix& ViewMatrix = View.ViewMatrices.GetTranslatedViewMatrix();

		const FVector4 ObjectPostProjectionPositionWithW = ViewProjMatrix.TransformPosition(MacroUVPosition);
		const FVector2D ObjectNDCPosition = FVector2D(ObjectPostProjectionPositionWithW / FMath::Max(ObjectPostProjectionPositionWithW.W, 0.00001f));
		const FVector4 RightPostProjectionPosition = ViewProjMatrix.TransformPosition(MacroUVPosition + MacroUVRadius * ViewMatrix.GetColumn(0));
		const FVector4 UpPostProjectionPosition = ViewProjMatrix.TransformPosition(MacroUVPosition + MacroUVRadius * ViewMatrix.GetColumn(1));

		const FVector4::FReal RightNDCPosX = RightPostProjectionPosition.X / FMath::Max(RightPostProjectionPosition.W, 0.0001f);
		const FVector4::FReal UpNDCPosY = UpPostProjectionPosition.Y / FMath::Max(UpPostProjectionPosition.W, 0.0001f);
		const FVector4::FReal DX = FMath::Min(RightNDCPosX - ObjectNDCPosition.X, WORLD_MAX);
		const FVector4::FReal DY = FMath::Min(UpNDCPosY - ObjectNDCPosition.Y, WORLD_MAX);

		MacroUVParameters.X = float(ObjectNDCPosition.X);	// LWC_TODO: Precision loss?
		MacroUVParameters.Y = float(ObjectNDCPosition.Y);
		if (DX != 0.0f && DY != 0.0f && !FMath::IsNaN(DX) && FMath::IsFinite(DX) && !FMath::IsNaN(DY) && FMath::IsFinite(DY))
		{
			MacroUVParameters.Z = 1.0f / float(DX);
			MacroUVParameters.W = -1.0f / float(DY);
		}
	}
	return MacroUVParameters;
}

