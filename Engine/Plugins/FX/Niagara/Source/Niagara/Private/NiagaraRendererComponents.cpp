// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRendererComponents.h"
#include "Engine/World.h"
#include "Components/PrimitiveComponent.h"
#include "NiagaraDataSet.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraStats.h"
#include "NiagaraComponentRendererProperties.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstance.h"
#include "String/ParseTokens.h"
#include "Async/Async.h"

#if WITH_EDITOR
	#include "Editor.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Component renderer update bindings"), STAT_NiagaraComponentRendererUpdateBindings, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Component renderer spawning [GT]"), STAT_NiagaraComponentRendererSpawning, STATGROUP_Niagara);

static int32 GNiagaraWarnComponentRenderCount = 50;
static FAutoConsoleVariableRef CVarNiagaraWarnComponentRenderCount(
	TEXT("fx.Niagara.WarnComponentRenderCount"),
	GNiagaraWarnComponentRenderCount,
	TEXT("The max number of components that a single system can spawn before a log warning is shown."),
	ECVF_Default
);

static float GNiagaraComponentRenderPoolInactiveTimeLimit = 5;
static FAutoConsoleVariableRef CVarNiagaraComponentRenderPoolInactiveTimeLimit(
	TEXT("fx.Niagara.ComponentRenderPoolInactiveTimeLimit"),
	GNiagaraComponentRenderPoolInactiveTimeLimit,
	TEXT("The time in seconds an inactive component can linger in the pool before being destroyed."),
	ECVF_Default
);

namespace NiagaraRendererComponentsImpl
{
	static const FName NAME_UniqueID("UniqueID");
}

//////////////////////////////////////////////////////////////////////////

template<typename T>
void SetValueWithAccessor(FNiagaraVariable& DataVariable, const FNiagaraDataSet& Data, int ParticleIndex)
{
	DataVariable.SetValue<T>(FNiagaraDataSetAccessor<T>::CreateReader(Data, DataVariable.GetName()).Get(ParticleIndex));
}

void SetVariableByType(FNiagaraVariable& DataVariable, const FNiagaraDataSet& Data, int ParticleIndex)
{
	const FNiagaraTypeDefinition& VarType = DataVariable.GetType();
	if (VarType == FNiagaraTypeDefinition::GetFloatDef()) { SetValueWithAccessor<float>(DataVariable, Data, ParticleIndex); }
	else if (VarType == FNiagaraTypeDefinition::GetIntDef()) { SetValueWithAccessor<int32>(DataVariable, Data, ParticleIndex); }
	else if (VarType == FNiagaraTypeDefinition::GetBoolDef()) { SetValueWithAccessor<FNiagaraBool>(DataVariable, Data, ParticleIndex); }
	else if (VarType == FNiagaraTypeDefinition::GetVec2Def()) { SetValueWithAccessor<FVector2f>(DataVariable, Data, ParticleIndex); }
	else if (VarType == FNiagaraTypeDefinition::GetVec3Def()) { SetValueWithAccessor<FVector3f>(DataVariable, Data, ParticleIndex); }
	else if (VarType == FNiagaraTypeDefinition::GetVec4Def()) { SetValueWithAccessor<FVector4f>(DataVariable, Data, ParticleIndex); }
	else if (VarType == FNiagaraTypeDefinition::GetColorDef()) { SetValueWithAccessor<FLinearColor>(DataVariable, Data, ParticleIndex); }
	else if (VarType == FNiagaraTypeDefinition::GetQuatDef()) { SetValueWithAccessor<FQuat4f>(DataVariable, Data, ParticleIndex); }
	else if (VarType == FNiagaraTypeDefinition::GetPositionDef()) {	SetValueWithAccessor<FNiagaraPosition>(DataVariable, Data, ParticleIndex); }
}

void ConvertVariableToType(const FNiagaraVariable& SourceVariable, const FNiagaraTypeDefinition& TargetType, TArray<uint8, TInlineAllocator<32>>& TargetData, const FNiagaraLWCConverter& LwcConverter)
{
	FNiagaraTypeDefinition SourceType = SourceVariable.GetType();

	if (SourceType == FNiagaraTypeDefinition::GetVec3Def() && TargetType == UNiagaraComponentRendererProperties::GetFColorDef())
	{
		const FVector3f Data = SourceVariable.GetValue<FVector3f>();
		TargetData.AddUninitialized(sizeof(FColor));
		*reinterpret_cast<FColor*>(TargetData.GetData()) = FColor(
			(uint8)FMath::Clamp<int32>(FMath::TruncToInt(Data.X * 255.f), 0, 255),
			(uint8)FMath::Clamp<int32>(FMath::TruncToInt(Data.Y * 255.f), 0, 255),
			(uint8)FMath::Clamp<int32>(FMath::TruncToInt(Data.Z * 255.f), 0, 255));
	}
	else if (SourceType == FNiagaraTypeDefinition::GetVec4Def() && TargetType == UNiagaraComponentRendererProperties::GetFColorDef())
	{
		const FVector4f Data = SourceVariable.GetValue<FVector4f>();
		TargetData.AddUninitialized(sizeof(FColor));
		*reinterpret_cast<FColor*>(TargetData.GetData()) = FColor(
			(uint8)FMath::Clamp<int32>(FMath::TruncToInt(Data.X * 255.f), 0, 255),
			(uint8)FMath::Clamp<int32>(FMath::TruncToInt(Data.Y * 255.f), 0, 255),
			(uint8)FMath::Clamp<int32>(FMath::TruncToInt(Data.Z * 255.f), 0, 255),
			(uint8)FMath::Clamp<int32>(FMath::TruncToInt(Data.W * 255.f), 0, 255));
	}
	else if (SourceType == FNiagaraTypeDefinition::GetColorDef() && TargetType == UNiagaraComponentRendererProperties::GetFColorDef())
	{
		const FLinearColor Data = SourceVariable.GetValue<FLinearColor>();
		TargetData.AddUninitialized(sizeof(FColor));
		*reinterpret_cast<FColor*>(TargetData.GetData()) = Data.QuantizeRound();
	}
	else if (SourceType == FNiagaraTypeDefinition::GetVec3Def() && TargetType == UNiagaraComponentRendererProperties::GetFRotatorDef())
	{
		const FVector3f Data = SourceVariable.GetValue<FVector3f>();
		TargetData.AddUninitialized(sizeof(FRotator));
		*reinterpret_cast<FRotator*>(TargetData.GetData()) = FRotator(Data.X, Data.Y, Data.Z);
	}
	else if (SourceType == FNiagaraTypeDefinition::GetVec2Def() && TargetType == UNiagaraComponentRendererProperties::GetFVector2DDef())
	{
		const FVector2f Data = SourceVariable.GetValue<FVector2f>();
		TargetData.AddUninitialized(sizeof(FVector2D));
		*reinterpret_cast<FVector2D*>(TargetData.GetData()) = FVector2D(Data);
	}	
	else if (SourceType == FNiagaraTypeDefinition::GetVec3Def() && TargetType == UNiagaraComponentRendererProperties::GetFVectorDef())
	{
		const FVector3f Data = SourceVariable.GetValue<FVector3f>();
		TargetData.AddUninitialized(sizeof(FVector));
		*reinterpret_cast<FVector*>(TargetData.GetData()) = FVector(Data);
	}
	else if (SourceType == FNiagaraTypeDefinition::GetVec4Def() && TargetType == UNiagaraComponentRendererProperties::GetFVector4Def())
	{
		const FVector4f Data = SourceVariable.GetValue<FVector4f>();
		TargetData.AddUninitialized(sizeof(FVector4));
		*reinterpret_cast<FVector4*>(TargetData.GetData()) = FVector4(Data);
	}
	else if (SourceType == FNiagaraTypeDefinition::GetPositionDef() && TargetType == UNiagaraComponentRendererProperties::GetFVectorDef())
	{
		const FVector Data = LwcConverter.ConvertSimulationPositionToWorld(SourceVariable.GetValue<FNiagaraPosition>());
		TargetData.AddUninitialized(sizeof(FVector));
		*reinterpret_cast<FVector*>(TargetData.GetData()) = Data;
	}
	else if (SourceType == FNiagaraTypeDefinition::GetPositionDef() && TargetType == UNiagaraComponentRendererProperties::GetFVector3fDef())
	{
		const FVector3f Data = FVector3f(LwcConverter.ConvertSimulationPositionToWorld(SourceVariable.GetValue<FNiagaraPosition>()));
		TargetData.AddUninitialized(sizeof(FVector3f));
		*reinterpret_cast<FVector3f*>(TargetData.GetData()) = Data;
	}
	else if (SourceType == FNiagaraTypeDefinition::GetQuatDef() && TargetType == UNiagaraComponentRendererProperties::GetFQuatDef())
	{
		const FQuat4f Data = SourceVariable.GetValue<FQuat4f>();
		TargetData.AddUninitialized(sizeof(FQuat));
		*reinterpret_cast<FQuat*>(TargetData.GetData()) = FQuat(Data);
	}
	else if (SourceType == FNiagaraTypeDefinition::GetQuatDef() && TargetType == UNiagaraComponentRendererProperties::GetFRotatorDef())
	{
		const FQuat4f Data = SourceVariable.GetValue<FQuat4f>();
		TargetData.AddUninitialized(sizeof(FRotator));
		*reinterpret_cast<FRotator*>(TargetData.GetData()) = FRotator(Data.Rotator());
	}
	else if (SourceType == FNiagaraTypeDefinition::GetFloatDef() && TargetType == UNiagaraComponentRendererProperties::GetDoubleDef())
	{
		const FNiagaraFloat Data = SourceVariable.GetValue<FNiagaraFloat>();
		TargetData.AddUninitialized(sizeof(FNiagaraDouble));
		reinterpret_cast<FNiagaraDouble*>(TargetData.GetData())->Value = Data.Value;
	}
}

void InvokeSetterFunction(UObject* InRuntimeObject, UFunction* Setter, const uint8* InData, int32 DataSize, const TMap<FString, FString>& SetterDefaultValues)
{
	int32 ParmsSize = Setter->ParmsSize;
	check(InRuntimeObject && Setter && ParmsSize > 0);

	uint8* Params = const_cast<uint8*>(InData);
	if (Setter->ReturnValueOffset != MAX_uint16 || Setter->NumParms > 1)
	{
		// Function has a return value or multiple parameters, we need to initialize memory for the entire parameter pack
		// We use alloca here (as in UObject::ProcessEvent) to avoid a heap allocation. Alloca memory survives the current function's stack frame.
		Params = reinterpret_cast<uint8*>(FMemory_Alloca(ParmsSize));

		bool bFirstProperty = true;
		for (FProperty* Property = Setter->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			// Initialize the parameter pack with any param properties that reside in the container
			if (Property->IsInContainer(ParmsSize))
			{
				// Check for a default value
				FNameBuilder PropertyName(Property->GetFName());
				FStringView PropertyNameView = PropertyName.ToView();
				uint32 PropertyNameHash = GetTypeHash(PropertyNameView);

				const FString* DefaultValue = SetterDefaultValues.FindByHash(PropertyNameHash, PropertyNameView);
				if (DefaultValue)
				{
					Property->ImportText_Direct(**DefaultValue, Property->ContainerPtrToValuePtr<uint8>(Params), InRuntimeObject, PPF_None);
				}
				else
				{
					Property->InitializeValue_InContainer(Params);
				}

				// The first encountered property is assumed to be the input value so initialize this with the user-specified value from InPropertyValue
				if (Property->HasAnyPropertyFlags(CPF_Parm) && !Property->HasAnyPropertyFlags(CPF_ReturnParm) && bFirstProperty)
				{
					// LWC backwards compatibility for old bindings
					UScriptStruct* ScriptStruct = ((FStructProperty*)Property)->Struct;
					if (DataSize == sizeof(FVector3f) && Property->GetElementSize() == sizeof(FVector3d) && ScriptStruct && (ScriptStruct->GetFName() == NAME_Vector || ScriptStruct->GetFName() == NAME_Vector3d))
					{
						FVector3f* Vec3f = (FVector3f*)InData;
						FVector Vec3d(*Vec3f);
						Property->CopyCompleteValue(Property->ContainerPtrToValuePtr<void>(Params), &Vec3d);
					}
					else
					{
						const bool bIsValid = ensureMsgf(DataSize == Property->GetElementSize(), TEXT("Property type does not match for setter function %s::%s (%ibytes != %ibytes"), *InRuntimeObject->GetName(), *Setter->GetName(), DataSize, Property->GetElementSize());
						if (bIsValid)
						{
							Property->CopyCompleteValue(Property->ContainerPtrToValuePtr<void>(Params), InData);
						}
						else
						{
							return;
						}
					}
				}
				bFirstProperty = false;
			}
		}
	}

	// Now we have the parameters set up correctly, call the function
	InRuntimeObject->ProcessEvent(Setter, Params);
}

//////////////////////////////////////////////////////////////////////////

#if WITH_EDITORONLY_DATA
struct FNiagaraRendererComponentsOnObjectsReplacedHelper
{
	FNiagaraRendererComponentsOnObjectsReplacedHelper(FNiagaraRendererComponents* InOwner)
		: Owner(InOwner)
	{
		if ( GEditor )
		{
			if ( IsInGameThread() )
			{
				FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FNiagaraRendererComponentsOnObjectsReplacedHelper::OnObjectsReplacedCallback);
			}
			else
			{
				AsyncTask(ENamedThreads::GameThread, [Object = this]() { FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(Object, &FNiagaraRendererComponentsOnObjectsReplacedHelper::OnObjectsReplacedCallback); });
			}
		}
	}

	~FNiagaraRendererComponentsOnObjectsReplacedHelper()
	{
		if ( GEditor )
		{
			check(IsInGameThread());
			FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
		}
	}

	void OnObjectsReplacedCallback(const TMap<UObject*, UObject*>& ReplacementsMap)
	{
		FScopeLock ThreadGuard(&CallbackLock);
		if ( Owner != nullptr )
		{
			Owner->OnObjectsReplacedCallback(ReplacementsMap);
		}
	}

	void Release()
	{
		FScopeLock ThreadGuard(&CallbackLock);
		Owner = nullptr;
		AsyncTask(ENamedThreads::GameThread, [Object=this]() { delete Object; });
	}

	FCriticalSection CallbackLock;
	FNiagaraRendererComponents* Owner = nullptr;
};
#endif

//////////////////////////////////////////////////////////////////////////

FNiagaraRendererComponents::FNiagaraRendererComponents(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter)
	: FNiagaraRenderer(FeatureLevel, InProps, Emitter)
{
	const UNiagaraComponentRendererProperties* Properties = CastChecked<const UNiagaraComponentRendererProperties>(InProps);
	TemplateKey = TObjectKey<USceneComponent>(Properties->TemplateComponent);

	ComponentPool.Reserve(Properties->ComponentCountLimit);

#if WITH_EDITORONLY_DATA
	if (GEditor)
	{
		OnObjectsReplacedHandler = new FNiagaraRendererComponentsOnObjectsReplacedHelper(this);
	}
#endif
}

FNiagaraRendererComponents::~FNiagaraRendererComponents()
{
	// These should have been freed in DestroyRenderState_Concurrent
	check(ComponentPool.Num() == 0);

#if WITH_EDITORONLY_DATA
	if ( OnObjectsReplacedHandler )
	{
		OnObjectsReplacedHandler->Release();
		OnObjectsReplacedHandler = nullptr;
	}
#endif
}

void FNiagaraRendererComponents::DestroyRenderState_Concurrent()
{
#if WITH_EDITORONLY_DATA
	// Release replace handler
	if (OnObjectsReplacedHandler)
	{
		OnObjectsReplacedHandler->Release();
		OnObjectsReplacedHandler = nullptr;
	}
#endif

	// If we are on the GameThread immediately execute, but only if not inside EOF updates
	// Failure to do so can cause issue with construction scripts rerunning as the components can linger when they should not
	bool bRequiresAyncTask = true;
	if (IsInGameThread())
	{
		UWorld* FoundWorld = nullptr;
		if (AActor* OwnerActor = SpawnedOwner.Get())
		{
			FoundWorld = OwnerActor->GetWorld();
			if (FoundWorld != nullptr)
			{
				bRequiresAyncTask = FoundWorld->bPostTickComponentUpdate;
			}
		}

		if ( FoundWorld == nullptr )
		{
			for (auto& PoolEntry : ComponentPool)
			{
				if (USceneComponent* Component = PoolEntry.Component.Get())
				{
					FoundWorld = Component->GetWorld();
					if (FoundWorld != nullptr)
					{
						bRequiresAyncTask = FoundWorld->bPostTickComponentUpdate;
						break;
					}
				}
			}
		}
	}

	if (bRequiresAyncTask)
	{
		// Rendering resources are being destroyed, but the component pool and their owner actor must be destroyed on the game thread
		AsyncTask(
			ENamedThreads::GameThread,
			[Pool_GT=MoveTemp(ComponentPool), Owner_GT=MoveTemp(SpawnedOwner)]()
			{
				// we do not reset ParticlesWithComponents here because it's possible the render state is destroyed without destroying the renderer. In this case we want to know which particles
				// had spawned some components previously
				for (auto& PoolEntry : Pool_GT)
				{
					if (USceneComponent* Component = PoolEntry.Component.Get())
					{
						Component->DestroyComponent();
					}
				}

				if (AActor* OwnerActor = Owner_GT.Get())
				{
					OwnerActor->Destroy();
				}
			}
		);
	}
	else
	{
		if (AActor* OwnerActor = SpawnedOwner.Get())
		{
			OwnerActor->Destroy();
		}

		for (auto& PoolEntry : ComponentPool)
		{
			if (USceneComponent* Component = PoolEntry.Component.Get())
			{
				Component->DestroyComponent();
			}
		}
	}
	ComponentPool.Empty();
	SpawnedOwner.Reset();
}

/** Update render data buffer from attributes */
void FNiagaraRendererComponents::PostSystemTick_GameThread(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraComponentRendererUpdateBindings);

	FNiagaraSystemInstance* SystemInstance = Emitter->GetParentSystemInstance();

	//Bail if we don't have the required attributes to render this emitter.
	const UNiagaraComponentRendererProperties* Properties = CastChecked<const UNiagaraComponentRendererProperties>(InProperties);
	if (!SystemInstance || !Properties || !Properties->TemplateComponent || SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	if (Emitter->IsDisabledFromIsolation())
	{
		ResetComponentPool(true);
		return;
	}
#endif

	USceneComponent* AttachComponent = SystemInstance->GetAttachComponent();
	if (!AttachComponent)
	{
		// we can't attach the components anywhere, so just bail
		return;
	}


	const FNiagaraEmitterInstanceImpl* StatefulEmitter = Emitter->AsStateful();
	if (StatefulEmitter == nullptr)
	{
		return;
	}

	const double CurrentTime = AttachComponent->GetWorld()->GetRealTimeSeconds();
	const FNiagaraDataSet& Data = Emitter->GetParticleData();
	const FNiagaraDataBuffer& ParticleData = Data.GetCurrentDataChecked();
	FNiagaraDataSetReaderInt32<FNiagaraBool> EnabledAccessor = FNiagaraDataSetAccessor<FNiagaraBool>::CreateReader(Data, Properties->EnabledBinding.GetDataSetBindableVariable().GetName());
	FNiagaraDataSetReaderInt32<int32> VisTagAccessor = FNiagaraDataSetAccessor<int32>::CreateReader(Data, Properties->RendererVisibilityTagBinding.GetDataSetBindableVariable().GetName());
	FNiagaraDataSetReaderInt32<int32> UniqueIDAccessor = FNiagaraDataSetAccessor<int32>::CreateReader(Data, NiagaraRendererComponentsImpl::NAME_UniqueID);

	const bool bIsRendererEnabled = IsRendererEnabled(Properties, Emitter);

	auto IsParticleEnabled = [&bIsRendererEnabled, &EnabledAccessor, &VisTagAccessor, Properties](int32 ParticleIndex)
	{
		if (bIsRendererEnabled && EnabledAccessor.GetSafe(ParticleIndex, true))
		{
			if (VisTagAccessor.IsValid())
			{
				return VisTagAccessor.GetSafe(ParticleIndex, 0) == Properties->RendererVisibility;
			}
			return true;
		}
		return false;
	};

	TMap<int32, int32> ParticlesWithComponents;
	TArray<int32> FreeList;
	if (Properties->bAssignComponentsOnParticleID && ComponentPool.Num() > 0)
	{
		FreeList.Reserve(ComponentPool.Num());

		// Determine the slots that were assigned to particles last frame
		TMap<int32, int32> UsedSlots;
		UsedSlots.Reserve(ComponentPool.Num());
		for (int32 EntryIndex = 0; EntryIndex < ComponentPool.Num(); ++EntryIndex)
		{
			FComponentPoolEntry& Entry = ComponentPool[EntryIndex];
			if (Entry.LastAssignedToParticleID >= 0)
			{
				UsedSlots.Emplace(Entry.LastAssignedToParticleID, EntryIndex);
			}
			else
			{
				FreeList.Add(EntryIndex);
			}
		}

		// Ensure the final list only contains particles that are alive and enabled
		ParticlesWithComponents.Reserve(UsedSlots.Num());
		for (uint32 ParticleIndex = 0; ParticleIndex < ParticleData.GetNumInstances(); ParticleIndex++)
		{
			int32 ParticleID = UniqueIDAccessor.GetSafe(ParticleIndex, -1);
			int32 PoolIndex;
			if (UsedSlots.RemoveAndCopyValue(ParticleID, PoolIndex))
			{
				if (IsParticleEnabled(ParticleIndex))
				{
					ParticlesWithComponents.Emplace(ParticleID, PoolIndex);
				}
				else
				{
					// Particle has disabled components since last tick, ensure the component for this entry gets deactivated before re-use
					USceneComponent* Component = ComponentPool[PoolIndex].Component.Get();
					if (Component && Component->IsActive())
					{
						Component->Deactivate();
						Component->SetVisibility(false, true);
					}
					FreeList.Add(PoolIndex);
					ComponentPool[PoolIndex].LastAssignedToParticleID = -1;
				}
			}
		}

		// Any remaining in the used slots are now free to be reclaimed, due to their particles either dying or having their component disabled
		for (TPair<int32, int32> UsedSlot : UsedSlots)
		{
			// Particle has died since last tick, ensure the component for this entry gets deactivated before re-use
			USceneComponent* Component = ComponentPool[UsedSlot.Value].Component.Get();
			if (Component && Component->IsActive())
			{
				Component->Deactivate();
				Component->SetVisibility(false, true);
			}
			FreeList.Add(UsedSlot.Value);
			ComponentPool[UsedSlot.Value].LastAssignedToParticleID = -1;
		}
	}

	FObjectKey ComponentKey(Properties->TemplateComponent);
	const int32 MaxComponents = Properties->ComponentCountLimit;
	int32 ComponentCount = 0;
	for (uint32 ParticleIndex = 0; ParticleIndex < ParticleData.GetNumInstances(); ParticleIndex++)
	{
		if (!IsParticleEnabled(ParticleIndex))
		{
			// Skip particles that don't want a component
			continue;
		}

		int32 ParticleID = -1;
		int32 PoolIndex = -1;
		if (Properties->bAssignComponentsOnParticleID)
		{
			// Get the particle ID and see if we have any components already assigned to the particle
			ParticleID = UniqueIDAccessor.GetSafe(ParticleIndex, -1);
			ParticlesWithComponents.RemoveAndCopyValue(ParticleID, PoolIndex);

			if (PoolIndex == -1 && Properties->bCreateComponentFirstParticleFrame)
			{
				// Don't allow this particle to acquire a component unless it was just spawned or had a component assigned to it previously
				bool bIsNewlySpawnedParticle = StatefulEmitter->IsParticleComponentActive(ComponentKey, ParticleID) || ParticleIndex >= ParticleData.GetNumInstances() - ParticleData.GetNumSpawnedInstances();
				if (!bIsNewlySpawnedParticle)
				{
					continue;
				}
			}
		}

		if (PoolIndex == -1 && ComponentCount + ParticlesWithComponents.Num() >= MaxComponents)
		{
			// The pool is full and there aren't any unused slots to claim
			continue;
		}

		// Acquire a component for this particle
		bool bNewlyAcquiredComponent = false;
		USceneComponent* SceneComponent = nullptr;
		if (PoolIndex == -1)
		{
			// Start by trying to pull from the pool
			if (!Properties->bAssignComponentsOnParticleID)
			{
				// We can just take the next slot
				PoolIndex = ComponentCount < ComponentPool.Num() ? ComponentCount : -1;
			}
			else if (FreeList.Num())
			{
				PoolIndex = FreeList.Pop(EAllowShrinking::No);
			}
			bNewlyAcquiredComponent = true;
		}

		if (PoolIndex >= 0)
		{
			SceneComponent = ComponentPool[PoolIndex].Component.Get();
		}

		if (!SceneComponent || SceneComponent->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
		{
			SCOPE_CYCLE_COUNTER(STAT_NiagaraComponentRendererSpawning);

			// Determine the owner actor or spawn one
			AActor* OwnerActor = SpawnedOwner.Get();
			if (OwnerActor == nullptr)
			{
				OwnerActor = AttachComponent->GetOwner();
				if (OwnerActor == nullptr)
				{
					// NOTE: This can happen with spawned systems
					OwnerActor = AttachComponent->GetWorld()->SpawnActor<AActor>();
					OwnerActor->SetFlags(RF_Transient);
					SpawnedOwner = OwnerActor;
				}
			}

			// if we don't have a pooled component we create a new one from the template
			SceneComponent = DuplicateObject<USceneComponent>(Properties->TemplateComponent, OwnerActor);
			SceneComponent->ClearFlags(RF_ArchetypeObject);
			SceneComponent->SetFlags(RF_Transient);
#if WITH_EDITORONLY_DATA
			SceneComponent->bVisualizeComponent = Properties->bVisualizeComponents;
#endif
			SceneComponent->SetupAttachment(AttachComponent);
			SceneComponent->RegisterComponent();
			SceneComponent->AddTickPrerequisiteComponent(AttachComponent);

			if (PoolIndex >= 0)
			{
				// This should only happen if the component was destroyed externally
				ComponentPool[PoolIndex].Component = SceneComponent;
			}
			else
			{
				// Add a new pool entry
				PoolIndex = ComponentPool.Num();
				ComponentPool.AddDefaulted_GetRef().Component = SceneComponent;
			}
		}

		FComponentPoolEntry& PoolEntry = ComponentPool[PoolIndex];
		FNiagaraLWCConverter LwcConverter = SystemInstance->GetLWCConverter(Emitter->IsLocalSpace());
		TickPropertyBindings(Properties, SceneComponent, Data, ParticleIndex, PoolEntry, LwcConverter);

		// Activate the component.
		// If components are assigned by ID then we can optionally do this only on the first frame the component is acquired by a particle.
		if(!Properties->bAssignComponentsOnParticleID || bNewlyAcquiredComponent || !Properties->bOnlyActivateNewlyAquiredComponents)
		{
			if (!SceneComponent->IsActive())
			{
				SceneComponent->SetVisibility(true, true);
				SceneComponent->Activate(false);
				if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(SceneComponent))
				{
					// prevent motion blur when reusing components
					PrimitiveComponent->ResetSceneVelocity();
				}
			}
		}

		PoolEntry.LastAssignedToParticleID = ParticleID;
		PoolEntry.LastActiveTime = CurrentTime;
		if (Properties->bCreateComponentFirstParticleFrame)
		{
			StatefulEmitter->SetParticleComponentActive(ComponentKey, ParticleID);
		}

		++ComponentCount;
		if (ComponentCount > GNiagaraWarnComponentRenderCount)
		{
			// This warning logspam can be pretty hindering to performance if left to it's own devices. We'll let it warn a bunch at first,
			// then suppress it. That way it's noticeable, but not crippling.
			static int32 MaxWarnings = 50;
			if (MaxWarnings > 0)
			{
				UE_LOGF(LogNiagara, Warning, "System %ls has over %i active components spawned from the effect. Either adjust the effect's component renderer or change the warning limit with fx.Niagara.WarnComponentRenderCount.",
					*SystemInstance->GetSystem()->GetName(), GNiagaraWarnComponentRenderCount);
				--MaxWarnings;
			}
		}

		if (ComponentCount >= MaxComponents)
		{
			// We've hit our prescribed limit
			break;
		}
	}

	if (ComponentCount < ComponentPool.Num())
	{
		// go over the pooled components we didn't need this tick to see if we can destroy some and deactivate the rest
		for (int32 PoolIndex = 0; PoolIndex < ComponentPool.Num(); ++PoolIndex)
		{
			FComponentPoolEntry& PoolEntry = ComponentPool[PoolIndex];
			if (Properties->bAssignComponentsOnParticleID)
			{
				if (PoolEntry.LastAssignedToParticleID >= 0)
				{
					// This one's in use
					continue;
				}
			}
			else if (PoolIndex < ComponentCount)
			{
				continue;
			}

			USceneComponent* Component = PoolEntry.Component.Get();
			if (!Component || (CurrentTime - PoolEntry.LastActiveTime) >= GNiagaraComponentRenderPoolInactiveTimeLimit)
			{
				if (Component)
				{
					Component->DestroyComponent();
				}

				// destroy the component pool slot
				ComponentPool.RemoveAtSwap(PoolIndex, EAllowShrinking::No);
				--PoolIndex;
				continue;
			}
			else if (Component->IsActive())
			{
				Component->Deactivate();
				Component->SetVisibility(false, true);
			}
		}
	}
}

void FNiagaraRendererComponents::OnSystemComplete_GameThread(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter)
{
	ResetComponentPool(true);
}

void FNiagaraRendererComponents::TickPropertyBindings(
	const UNiagaraComponentRendererProperties* Properties,
	USceneComponent* Component,
	const FNiagaraDataSet& Data,
	int32 ParticleIndex,
	FComponentPoolEntry& PoolEntry,
	const FNiagaraLWCConverter& LwcConverter)
{
	for (const FNiagaraComponentPropertyBinding& PropertyBinding : Properties->PropertyBindings)
	{
		const FNiagaraPropertySetter* PropertySetter = Properties->SetterFunctionMapping.Find(PropertyBinding.PropertyName);
		if (!PropertySetter)
		{
			// it's possible that Initialize wasn't called or the bindings changed in the meantime
			continue;
		}

		FNiagaraVariable DataVariable(PropertyBinding.AttributeBinding.GetDataSetBindableVariable());
		if (!DataVariable.IsValid() || !Data.HasVariable(DataVariable))
		{
			continue;
		}

		SetVariableByType(DataVariable, Data, ParticleIndex);

		FNiagaraTypeDefinition PropertyType = PropertyBinding.PropertyType;
		// Get property type from property setter function
		if(!PropertyType.IsValid() && PropertySetter->Function)
		{
			for (FProperty* Property = PropertySetter->Function->PropertyLink; Property; Property = Property->PropertyLinkNext)
			{
				if (Property->HasAnyPropertyFlags(CPF_Parm) && !Property->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					if (FStructProperty* StructProp = CastField<FStructProperty>(Property))
					{
						PropertyType = FNiagaraTypeDefinition(StructProp->Struct, FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Allow);
						break;
					}

					// we can't determine the struct type, but since it's the same size we assume it's the same type as the source var
					FNiagaraTypeDefinition VarType = DataVariable.GetType();
					if (Property->GetSize() == VarType.GetSize())
					{
						break;
					}

					if (Property->GetClass()->IsChildOf(FDoubleProperty::StaticClass()))
					{
						PropertyType = UNiagaraComponentRendererProperties::GetDoubleDef();
						break;
					}

					// the property is missing the struct info and has a different size. In case of LWC types we try to guess the type if it fits the size.
					bool bDoubleSize = Property->GetSize() == VarType.GetSize() * 2;
					if (VarType == FNiagaraTypeDefinition::GetVec2Def() && bDoubleSize)
					{
						PropertyType = UNiagaraComponentRendererProperties::GetFVector2DDef();
					}
					else if ((VarType == FNiagaraTypeDefinition::GetVec3Def() || VarType == FNiagaraTypeDefinition::GetPositionDef()) && bDoubleSize)
					{
						PropertyType = UNiagaraComponentRendererProperties::GetFVectorDef();
					}
					else if (VarType == FNiagaraTypeDefinition::GetVec4Def() && bDoubleSize)
					{
						PropertyType = UNiagaraComponentRendererProperties::GetFVector4Def();
					}
					else if (VarType == FNiagaraTypeDefinition::GetQuatDef() && bDoubleSize)
					{
						PropertyType = UNiagaraComponentRendererProperties::GetFQuatDef();
					}
					else
					{
						// unable to resolve the type, invalidate everything to skip the tick
						PropertyType = FNiagaraTypeDefinition();
						DataVariable = FNiagaraVariable();
					}
				}
			}
		}

		bool bForceStructConversion = PropertyType.GetScriptStruct() && !FNiagaraTypeHelper::IsNiagaraFriendlyTopLevelStruct(PropertyType.GetScriptStruct(), ENiagaraStructConversion::UserFacing);

		TArrayView<uint8> TargetData;
		TArray<uint8, TInlineAllocator<32>> ConvertedData;
		if (PropertyType.IsValid() && DataVariable.GetType() != PropertyType && (!PropertySetter->bIgnoreConversion || bForceStructConversion))
		{
			ConvertVariableToType(DataVariable, PropertyType, ConvertedData, LwcConverter);
			TargetData = MakeArrayView(ConvertedData);
		}
		else
		{
			TargetData = MakeArrayView(DataVariable.GetData(), DataVariable.GetSizeInBytes());
		}

		if (!DataVariable.IsDataAllocated())
		{
			continue;
		}

		// set the values from the particle bindings
		if (PropertySetter->Function && PropertySetter->Function->NumParms >= 1)
		{
			// if we have a setter function we invoke it instead of setting the property directly, because then the object gets a chance to react to the new value
			InvokeSetterFunction(Component, PropertySetter->Function, TargetData.GetData(), TargetData.Num(), PropertyBinding.PropertySetterParameterDefaults);
		}
		else
		{
			// no setter found, just slam the value in the object memory and hope for the best
			if (!PoolEntry.PropertyAddressMapping.Contains(PropertyBinding.PropertyName))
			{
				PoolEntry.PropertyAddressMapping.Add(PropertyBinding.PropertyName, FNiagaraRendererComponents::FindProperty(*Component, PropertyBinding.PropertyName.ToString()));
			}
			FComponentPropertyAddress PropertyAddress = PoolEntry.PropertyAddressMapping[PropertyBinding.PropertyName];
			if (!PropertyAddress.GetProperty())
			{
				continue;
			}
			uint8* Dest = (uint8*)Component + PropertyAddress.GetProperty()->GetOffset_ForInternal();
			FMemory::Memcpy(Dest, TargetData.GetData(), TargetData.Num());
		}
	}
}

FNiagaraRendererComponents::FComponentPropertyAddress FNiagaraRendererComponents::FindPropertyRecursive(void* BasePointer, UStruct* InStruct, const TArray<FName>& InPropertyNames, uint32 Index)
{
	FComponentPropertyAddress NewAddress;
	FProperty* Property = FindFProperty<FProperty>(InStruct, InPropertyNames[Index]);

	if (FStructProperty* StructProp = CastField<FStructProperty>(Property))
	{
		NewAddress.Property = StructProp;
		NewAddress.Address = BasePointer;

		if (InPropertyNames.IsValidIndex(Index + 1))
		{
			void* StructContainer = StructProp->ContainerPtrToValuePtr<void>(BasePointer);
			return FindPropertyRecursive(StructContainer, StructProp->Struct, InPropertyNames, Index + 1);
		}
		else
		{
			check(StructProp->GetName() == InPropertyNames[Index]);
		}
	}
	else if (Property)
	{
		NewAddress.Property = Property;
		NewAddress.Address = BasePointer;
	}

	return NewAddress;
}

FNiagaraRendererComponents::FComponentPropertyAddress FNiagaraRendererComponents::FindProperty(const UObject& InObject, const FString& InPropertyPath)
{
	TArray<FName> PropertyNames;
	UE::String::ParseTokens(InPropertyPath, TEXT('.'), [&PropertyNames](FStringView Token) { PropertyNames.Emplace(Token); });

	if (IsValid(&InObject) && PropertyNames.Num() > 0)
	{
		return FindPropertyRecursive((void*)&InObject, InObject.GetClass(), PropertyNames, 0);
	}
	return FComponentPropertyAddress();
}

#if WITH_EDITOR

void FNiagaraRendererComponents::OnObjectsReplacedCallback(const TMap<UObject*, UObject*>& ReplacementsMap)
{
	TArray<UObject*> Keys;
	ReplacementsMap.GetKeys(Keys);

	for (UObject* OldObject : Keys)
	{
		TObjectKey<USceneComponent> OldObjectKey(Cast<USceneComponent>(OldObject));
		if (OldObjectKey == TemplateKey)
		{
			ResetComponentPool(false);
			break;
		}
	}
}

#endif

void FNiagaraRendererComponents::ResetComponentPool(bool bResetOwner)
{
	for (FComponentPoolEntry& PoolEntry : ComponentPool)
	{
		if (PoolEntry.Component.IsValid())
		{
			PoolEntry.Component->DestroyComponent();
		}
	}
	ComponentPool.SetNum(0, EAllowShrinking::No);

	if (bResetOwner)
	{
		if (AActor* OwnerActor = SpawnedOwner.Get())
		{
			SpawnedOwner.Reset();
			OwnerActor->Destroy();
		}
	}
}
