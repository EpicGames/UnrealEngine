// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRendererDecals.h"

#include "NiagaraCullProxyComponent.h"
#include "NiagaraDecalRendererProperties.h"
#include "NiagaraEmitter.h"
#include "NiagaraSceneProxy.h"
#include "NiagaraSystemInstance.h"

#include "Async/Async.h"
#include "Components/LineBatchComponent.h"
#include "Engine/World.h"
#include "SceneInterface.h"
#include "SceneProxies/DeferredDecalProxy.h"

namespace NiagaraRendererDecalsLocal
{
	static bool GRendererEnabled = true;
	static FAutoConsoleVariableRef CVarRendererEnabled(
		TEXT("fx.Niagara.DecalRenderer.Enabled"),
		GRendererEnabled,
		TEXT("If == 0, Niagara Decal Renderers are disabled."),
		ECVF_Default
	);

#define NIAGARARENDERERDECALS_DEBUG_ENABLED !UE_BUILD_SHIPPING

#if WITH_NIAGARA_RENDERER_DEBUGDRAW
	void DrawDebugDecal(UWorld* World, TConstArrayView<FDeferredDecalUpdateParams> AllUpdateParams, double WorldTime)
	{
		if (World == nullptr || AllUpdateParams.Num() == 0)
		{
			return;
		}

		// Line Batcher is not thread safe and we could be on any thread, therefore send a task to draw the information safely
		AsyncTask(
			ENamedThreads::GameThread,
			[WeakWorld=MakeWeakObjectPtr(World), AllUpdateParams_GT=TArray<FDeferredDecalUpdateParams>(AllUpdateParams), WorldTime]
			{
				UWorld* World = WeakWorld.Get();
				ULineBatchComponent* LineBatcher = World ? World->GetLineBatcher(UWorld::ELineBatcherType::World) : nullptr;
				if (LineBatcher == nullptr)
				{
					return;
				}
				
				for (const FDeferredDecalUpdateParams& UpdateParams : AllUpdateParams_GT)
				{
					if ( UpdateParams.OperationType == FDeferredDecalUpdateParams::EOperationType::RemoveFromSceneAndDelete )
					{
						continue;
					}

					//const float Fade = (WorldTime - UpdateParams.FadeStartDelay) / UpdateParams.FadeDuration;
					//const uint8 IntFade = FMath::Clamp(uint8(255.0f * Fade), 0, 255);

					const FMatrix DecalMatrix = UpdateParams.Transform.ToMatrixWithScale();
					const FBox BoundsBox(-FVector(1), FVector(1));
					const FColor DecalColor = UpdateParams.DecalColor.ToFColor(false);
					LineBatcher->DrawBox(BoundsBox, DecalMatrix, DecalColor, 0, 0.0f);

					const FVector StartLocation = UpdateParams.Transform.TransformPosition(FVector(-1.0f, 0.0f, 0.0f));
					const FVector MidLocation = UpdateParams.Transform.TransformPosition(FVector(0.0f, 0.0f, 0.0f));
					const float ArrowSize = FMath::Min(UpdateParams.Transform.GetScale3D().Y, UpdateParams.Transform.GetScale3D().Z) * 0.5f;
					
					LineBatcher->DrawDirectionalArrow(StartLocation, MidLocation, ArrowSize * ArrowSize, FLinearColor::Red, 0.0f, 0, 1.0f);
				}
			}
		);
	}
#endif
}

FNiagaraRendererDecals::FNiagaraRendererDecals(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter)
	: FNiagaraRenderer(FeatureLevel, InProps, Emitter)
{
}

FNiagaraRendererDecals::~FNiagaraRendererDecals()
{
	check(ActiveDecalProxies.Num() == 0);
}

void FNiagaraRendererDecals::ReleaseAllDecals() const
{
	if (ActiveDecalProxies.Num() > 0)
	{
		USceneComponent* OwnerComponent = WeakOwnerComponent.Get();
		UWorld* World = OwnerComponent ? OwnerComponent->GetWorld() : nullptr;
		if (World && World->Scene)
		{
			TArray<FDeferredDecalUpdateParams> DecalUpdates;
			DecalUpdates.AddDefaulted(ActiveDecalProxies.Num());
			for (int i = 0; i < ActiveDecalProxies.Num(); ++i)
			{
				DecalUpdates[i].OperationType = FDeferredDecalUpdateParams::EOperationType::RemoveFromSceneAndDelete;
				DecalUpdates[i].DecalProxy = ActiveDecalProxies[i];
			}

			World->Scene->BatchUpdateDecals(MoveTemp(DecalUpdates));
			ActiveDecalProxies.Empty();
		}
	}
}

void FNiagaraRendererDecals::DestroyRenderState_Concurrent()
{
	ReleaseAllDecals();

	ensureMsgf(ActiveDecalProxies.Num() == 0, TEXT("ActiveDecalProxies have been leaked?"));
	ActiveDecalProxies.Empty();
}

FPrimitiveViewRelevance FNiagaraRendererDecals::GetViewRelevance(const FSceneView* View, const FNiagaraSceneProxy *SceneProxy)const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = false;
	Result.bShadowRelevance = false;
	Result.bDynamicRelevance = false;
	Result.bOpaque = false;
	Result.bHasSimpleLights = false;

	return Result;
}

FNiagaraDynamicDataBase* FNiagaraRendererDecals::GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const
{
	using namespace NiagaraRendererDecalsLocal;

	// Get DataToRender
	const FNiagaraDataSet& DataSet = Emitter->GetParticleData();
	const FNiagaraDataBuffer* DataToRender = DataSet.GetCurrentData();
	FNiagaraSystemInstance* SystemInstance = Emitter->GetParentSystemInstance();
	if (DataToRender == nullptr || SystemInstance == nullptr || !IsRendererEnabled(InProperties, Emitter))
	{
		ReleaseAllDecals();
		return nullptr;
	}

	UWorld* World = SystemInstance->GetWorld();
	check(World && World->Scene);

	USceneComponent* OwnerComponent = SystemInstance->GetAttachComponent();
	check(OwnerComponent);

	WeakOwnerComponent = OwnerComponent;
	const UNiagaraDecalRendererProperties* RendererProperties = CastChecked<const UNiagaraDecalRendererProperties>(InProperties);

	// Note: We use the OwnerComponent for local space simulations as disabling requires current frame data means the one inside the simulation is old and causes a mismatch with sprite / mesh renderers
	const bool bUseLocalSpace = UseLocalSpace(Proxy);
	const FTransform SimToWorld = bUseLocalSpace ? OwnerComponent->GetComponentTransform() : SystemInstance->GetLWCSimToWorld(false);
	const FNiagaraPosition DefaultPos = FVector3f::ZeroVector;
	const FQuat4f DefaultRot = UNiagaraDecalRendererProperties::GetDefaultOrientation();
	const FVector3f DefaultSize = UNiagaraDecalRendererProperties::GetDefaultDecalSize();
	const FNiagaraBool DefaultVisible = UNiagaraDecalRendererProperties::GetDefaultDecalVisible();
	const float DefaultFade = UNiagaraDecalRendererProperties::GetDefaultDecalFade();
	const int32 DefaultSortOrder = UNiagaraDecalRendererProperties::GetDefaultDecalSortOrder();

	// Check for Material Update
	UMaterialInterface* Material = BaseMaterials_GT.Num() ? BaseMaterials_GT[0] : nullptr;
	if (Material != WeakMaterial.Get())
	{
		WeakMaterial = Material;
		ReleaseAllDecals();
	}

	if (RendererProperties->MaterialParameters.HasAnyBindings())
	{
		ProcessMaterialParameterBindings(RendererProperties->MaterialParameters, Emitter, MakeArrayView(BaseMaterials_GT));
	}

	// Generate list of decals to update
	int NumActiveDecalProxies = 0;

	TArray<FDeferredDecalUpdateParams> DecalUpdates;

	// Get default values which might be none particle parameters
	const FNiagaraParameterStore& ParameterStore = Emitter->GetRendererBoundVariables();
	const bool LocalDefaultVisible = ParameterStore.GetParameterValueOrDefault(RendererProperties->DecalVisibleBinding.GetParamMapBindableVariable(), DefaultVisible).GetValue();
	const int32 LocalDefaultVisTag = ParameterStore.GetParameterValueOrDefault(RendererProperties->RendererVisibilityTagBinding.GetParamMapBindableVariable(), RendererProperties->RendererVisibility);
	const FVector3f LocalDefaultSimPos = ParameterStore.GetParameterValueOrDefault(RendererProperties->PositionBinding.GetParamMapBindableVariable(), DefaultPos);
	const FQuat4f LocalDefaultSimRot = ParameterStore.GetParameterValueOrDefault(RendererProperties->DecalOrientationBinding.GetParamMapBindableVariable(), DefaultRot).GetNormalized();
	const FVector3f LocalDefaultSize = ParameterStore.GetParameterValueOrDefault(RendererProperties->DecalSizeBinding.GetParamMapBindableVariable(), DefaultSize);
	const FLinearColor LocalDefaultDecalColor = ParameterStore.GetParameterValueOrDefault(RendererProperties->DecalColorBinding.GetParamMapBindableVariable(), FLinearColor::White);
	const float LocalDefaultFade = ParameterStore.GetParameterValueOrDefault(RendererProperties->DecalFadeBinding.GetParamMapBindableVariable(), DefaultFade);
	const int32 LocalDefaultSortOrder = ParameterStore.GetParameterValueOrDefault(RendererProperties->DecalSortOrderBinding.GetParamMapBindableVariable(), DefaultSortOrder);

	// Particles Source mode?
	if (RendererProperties->SourceMode == ENiagaraRendererSourceDataMode::Particles)
	{
		DecalUpdates.Reserve(DataToRender->GetNumInstances());

		// Create all our data readers
		const auto PositionReader = RendererProperties->PositionDataSetAccessor.GetReader(DataSet);
		const auto RotationReader = RendererProperties->DecalOrientationDataSetAccessor.GetReader(DataSet);
		const auto SizeReader = RendererProperties->DecalSizeDataSetAccessor.GetReader(DataSet);
		const auto ColorReader = RendererProperties->DecalColorDataSetAccessor.GetReader(DataSet);
		const auto FadeReader = RendererProperties->DecalFadeDataSetAccessor.GetReader(DataSet);
		const auto SortOrderReader = RendererProperties->DecalSortOrderDataSetAccessor.GetReader(DataSet);
		const auto VisibleReader = RendererProperties->DecalVisibleAccessor.GetReader(DataSet);
		const auto VisTagReader = RendererProperties->RendererVisibilityTagAccessor.GetReader(DataSet);

		for (uint32 ParticleIndex = 0; ParticleIndex < DataToRender->GetNumInstances(); ++ParticleIndex)
		{
			// Check if the decal is visible
			const bool bVisible = VisibleReader.GetSafe(ParticleIndex, LocalDefaultVisible).GetValue();
			const int32 VisTag = VisTagReader.GetSafe(ParticleIndex, LocalDefaultVisTag);
			if (!bVisible || VisTag != RendererProperties->RendererVisibility)
			{
				continue;
			}

			// Grab Decal Attributes
			const FVector3f SimPos = PositionReader.GetSafe(ParticleIndex, LocalDefaultSimPos);
			const FQuat4f SimRot = RotationReader.GetSafe(ParticleIndex, LocalDefaultSimRot).GetNormalized();
			const FVector Position = SimToWorld.TransformPosition(FVector(SimPos));
			const FQuat Rotation = SimToWorld.TransformRotation(FQuat(SimRot));
			const FVector Size = FVector(SizeReader.GetSafe(ParticleIndex, LocalDefaultSize) * 0.5f);
			const FLinearColor DecalColor = ColorReader.GetSafe(ParticleIndex, LocalDefaultDecalColor);
			const float Fade = FadeReader.GetSafe(ParticleIndex, LocalDefaultFade);
			const int32 SortOrder = SortOrderReader.GetSafe(ParticleIndex, LocalDefaultSortOrder);

			// Create Update Parameters
			FDeferredDecalUpdateParams& UpdateParams = DecalUpdates.AddDefaulted_GetRef();
			if (NumActiveDecalProxies < ActiveDecalProxies.Num())
			{
				UpdateParams.DecalProxy = ActiveDecalProxies[NumActiveDecalProxies];
				UpdateParams.OperationType = FDeferredDecalUpdateParams::EOperationType::Update;
			}
			else
			{
				UpdateParams.DecalProxy = new FDeferredDecalProxy(OwnerComponent, Material);
				UpdateParams.OperationType = FDeferredDecalUpdateParams::EOperationType::AddToSceneAndUpdate;
				ActiveDecalProxies.Add(UpdateParams.DecalProxy);
			}
			++NumActiveDecalProxies;

			UpdateParams.Transform = FTransform(Rotation, Position, FVector(Size));
			UpdateParams.Bounds = FBoxSphereBounds(FSphere(Position, Size.GetAbsMax() * 2.0));
			UpdateParams.AbsSpawnTime = World->TimeSeconds - FMath::Clamp(1.0f - Fade, 0.0f, 1.0f);
			UpdateParams.FadeStartDelay = 0.0f;
			UpdateParams.FadeDuration = 1.0f;
			UpdateParams.FadeScreenSize = RendererProperties->DecalScreenSizeFade;
			UpdateParams.DecalColor = DecalColor;
			UpdateParams.SortOrder = SortOrder;
		}
	}
	// Emitter source mode
	else
	{
		//-OPT: Emitter mode we should be able to cache the parameter offset rather than search each frame
		if (LocalDefaultVisible && LocalDefaultVisTag == RendererProperties->RendererVisibility)
		{
			const FVector3f SimPos = LocalDefaultSimPos;
			const FQuat4f SimRot = LocalDefaultSimRot;
			const FVector Position = SimToWorld.TransformPosition(FVector(SimPos));
			const FQuat Rotation = SimToWorld.TransformRotation(FQuat(SimRot));
			const FVector Size = FVector(LocalDefaultSize * 0.5f);
			const FLinearColor DecalColor = LocalDefaultDecalColor;
			const float Fade = LocalDefaultFade;
			const int32 SortOrder = LocalDefaultSortOrder;

			// Create Update Parameters
			FDeferredDecalUpdateParams& UpdateParams = DecalUpdates.AddDefaulted_GetRef();
			if (NumActiveDecalProxies < ActiveDecalProxies.Num())
			{
				UpdateParams.DecalProxy = ActiveDecalProxies[NumActiveDecalProxies];
				UpdateParams.OperationType = FDeferredDecalUpdateParams::EOperationType::Update;
			}
			else
			{
				UpdateParams.DecalProxy = new FDeferredDecalProxy(OwnerComponent, Material);
				UpdateParams.OperationType = FDeferredDecalUpdateParams::EOperationType::AddToSceneAndUpdate;
				ActiveDecalProxies.Add(UpdateParams.DecalProxy);
			}
			++NumActiveDecalProxies;

			UpdateParams.Transform = FTransform(Rotation, Position, FVector(Size));
			UpdateParams.Bounds = FBoxSphereBounds(FSphere(Position, Size.GetAbsMax() * 2.0));
			UpdateParams.AbsSpawnTime = World->TimeSeconds - FMath::Clamp(1.0f - Fade, 0.0f, 1.0f);
			UpdateParams.FadeStartDelay = 0.0f;
			UpdateParams.FadeDuration = 1.0f;
			UpdateParams.FadeScreenSize = RendererProperties->DecalScreenSizeFade;
			UpdateParams.DecalColor = DecalColor;
			UpdateParams.SortOrder = SortOrder;
		}
	}

	// Remove any unused decals
	while (ActiveDecalProxies.Num() > NumActiveDecalProxies)
	{
		FDeferredDecalUpdateParams& UpdateParams = DecalUpdates.AddDefaulted_GetRef();
		UpdateParams.OperationType = FDeferredDecalUpdateParams::EOperationType::RemoveFromSceneAndDelete;
		UpdateParams.DecalProxy = ActiveDecalProxies.Pop();
	}

	// Send updates to RT
	if (DecalUpdates.Num() > 0)
	{
	#if WITH_NIAGARA_RENDERER_DEBUGDRAW
		if (RendererProperties->IsDebugDrawEnabled())
		{
			DrawDebugDecal(World, DecalUpdates, World->TimeSeconds);
		}
	#endif
		World->Scene->BatchUpdateDecals(MoveTemp(DecalUpdates));
	}

	return nullptr;
}
