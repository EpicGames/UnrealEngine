// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRendererLights.h"
#include "NiagaraDataSet.h"
#include "NiagaraDataSetAccessor.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraSceneProxy.h"
#include "NiagaraSettings.h"
#include "NiagaraStats.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraComponent.h"

#include "NiagaraLightRendererProperties.h"
#include "NiagaraRendererLights.h"
#include "NiagaraCullProxyComponent.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneInfo.h"
#include "SceneInterface.h"

#include "Async/Async.h"
#include "Components/LineBatchComponent.h"
#include "Engine/World.h"
#include "SceneView.h"

DECLARE_CYCLE_STAT(TEXT("Generate Particle Lights"), STAT_NiagaraGenLights, STATGROUP_Niagara);

namespace NiagaraRendererLightsPrivate
{
	static bool GbRendererEnabled = true;
	static FAutoConsoleVariableRef CVarGbRendererEnabled(
		TEXT("fx.EnableNiagaraLightRendering"),
		GbRendererEnabled,
		TEXT("If false Niagara Light Renderers are disabled."),
		ECVF_Default
	);

#if WITH_NIAGARA_RENDERER_DEBUGDRAW 
	void DebugDraw(FNiagaraSystemInstance* SystemInstance, TArrayView<FNiagaraRendererLights::SimpleLightData> InLightData)
	{
		UWorld* World = SystemInstance->GetWorld();
		if (World == nullptr || InLightData.Num() == 0)
		{
			return;
		}

		AsyncTask(
			ENamedThreads::GameThread,
			[WeakWorld=MakeWeakObjectPtr(World), LightDataArray=TArray<FNiagaraRendererLights::SimpleLightData>(InLightData)]
			{
				UWorld* World = WeakWorld.Get();
				ULineBatchComponent* LineBatcher = World ? World->GetLineBatcher(UWorld::ELineBatcherType::World) : nullptr;
				if (LineBatcher == nullptr)
				{
					return;
				}

				for (const FNiagaraRendererLights::SimpleLightData& LightData : LightDataArray)
				{
					const FVector LightLocation = LightData.PerViewEntry.Position;
					const float LightRadius = LightData.LightEntry.Radius;
					const FColor LightColor = FLinearColor(LightData.LightEntry.Color.X, LightData.LightEntry.Color.Y, LightData.LightEntry.Color.Z).ToFColor(true);

					LineBatcher->DrawCircle(LightLocation, FVector::XAxisVector, FVector::YAxisVector, LightColor, LightRadius, 16, 0);
					LineBatcher->DrawCircle(LightLocation, FVector::XAxisVector, FVector::ZAxisVector, LightColor, LightRadius, 16, 0);
					LineBatcher->DrawCircle(LightLocation, FVector::YAxisVector, FVector::ZAxisVector, LightColor, LightRadius, 16, 0);
				}
			}
		);
	}
#endif //WITH_NIAGARA_RENDERER_DEBUGDRAW 
}

struct FNiagaraDynamicDataLights : public FNiagaraDynamicDataBase
{
	FNiagaraDynamicDataLights(const FNiagaraEmitterInstance* InEmitter)
		: FNiagaraDynamicDataBase(InEmitter)
	{
	}

	TArray<FNiagaraRendererLights::SimpleLightData> LightArray;
};

//////////////////////////////////////////////////////////////////////////


FNiagaraRendererLights::FNiagaraRendererLights(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter)
	: FNiagaraRenderer(FeatureLevel, InProps, Emitter)
{
	// todo - for platforms where we know we can't support deferred shading we can just set this to false
	bHasLights = true;
}

FPrimitiveViewRelevance FNiagaraRendererLights::GetViewRelevance(const FSceneView* View, const FNiagaraSceneProxy *SceneProxy)const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = bHasLights && SceneProxy->IsShown(View) && View->Family->EngineShowFlags.Particles && View->Family->EngineShowFlags.Niagara;
	Result.bShadowRelevance = false;
	Result.bDynamicRelevance = false;
	Result.bOpaque = false;
	Result.bHasSimpleLights = bHasLights;

	return Result;
}

/** Update render data buffer from attributes */
FNiagaraDynamicDataBase* FNiagaraRendererLights::GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const
{
	// particle (simple) lights are only supported with deferred shading
	if (!bHasLights)
	{
		return nullptr;
	}
	if (Proxy->GetScene().GetShadingPath() != EShadingPath::Deferred)
	{
		const EShaderPlatform ShaderPlatform = Proxy->GetScene().GetShaderPlatform();
		if (!IsMobileDeferredShadingEnabled(ShaderPlatform) && !MobileForwardEnableParticleLights(ShaderPlatform))
		{
			return nullptr;
		}
	}

	if (!IsRendererEnabled(InProperties, Emitter))
	{
		return nullptr;
	}

	SCOPE_CYCLE_COUNTER(STAT_NiagaraGenLights);

	//Bail if we don't have the required attributes to render this emitter.
	const UNiagaraLightRendererProperties* Properties = CastChecked<const UNiagaraLightRendererProperties>(InProperties);
	const FNiagaraDataSet& Data = Emitter->GetParticleData();
	const FNiagaraDataBuffer* DataToRender = Data.GetCurrentData();
	if (DataToRender == nullptr || Emitter->GetParentSystemInstance() == nullptr)
	{
		return nullptr;
	}

	if (Properties->bAllowInCullProxies == false)
	{
		check(Emitter);

		FNiagaraSystemInstance* Inst = Emitter->GetParentSystemInstance();
		check(Emitter->GetParentSystemInstance());

		//TODO: Probably should push some state into the system instance for this?
		bool bIsCullProxy = Cast<UNiagaraCullProxyComponent>(Inst->GetAttachComponent()) != nullptr;
		if (bIsCullProxy)
		{
			return nullptr;
		}
	}

	FNiagaraSystemInstance* SystemInstance = Emitter->GetParentSystemInstance();
	FNiagaraDynamicDataLights* DynamicData = new FNiagaraDynamicDataLights(Emitter);

	USceneComponent* OwnerComponent = SystemInstance->GetAttachComponent();
	check(OwnerComponent);

	// Note: We use the OwnerComponent for local space simulations as disabling requires current frame data means the one inside the simulation is old and causes a mismatch with sprite / mesh renderers
	const bool bUseLocalSpace = UseLocalSpace(Proxy);
	const FTransform SimToWorld = bUseLocalSpace ? OwnerComponent->GetComponentTransform() : SystemInstance->GetLWCSimToWorld(false);
	const FVector3f DefaultSimPos = FVector3f::ZeroVector;

	const FNiagaraParameterStore& ParameterStore = Emitter->GetRendererBoundVariables();
	const FVector3f DefaultPos = ParameterStore.GetParameterValueOrDefault(Properties->PositionBinding.GetParamMapBindableVariable(), DefaultSimPos);
	const FLinearColor DefaultColor = ParameterStore.GetParameterValueOrDefault(Properties->ColorBinding.GetParamMapBindableVariable(), UNiagaraLightRendererProperties::GetDefaultColor());
	const float DefaultRadius = ParameterStore.GetParameterValueOrDefault(Properties->RadiusBinding.GetParamMapBindableVariable(), UNiagaraLightRendererProperties::GetDefaultRadius());
	const float DefaultScattering = ParameterStore.GetParameterValueOrDefault(Properties->VolumetricScatteringBinding.GetParamMapBindableVariable(), UNiagaraLightRendererProperties::GetDefaultVolumetricScattering());
	const FNiagaraBool DefaultEnabled = ParameterStore.GetParameterValueOrDefault(Properties->LightRenderingEnabledBinding.GetParamMapBindableVariable(), FNiagaraBool(true));
	const int32 DefaultVisibilityTag = ParameterStore.GetParameterValueOrDefault(Properties->RendererVisibilityTagBinding.GetParamMapBindableVariable(), Properties->RendererVisibility);
	const float DefaultExponent = ParameterStore.GetParameterValueOrDefault(Properties->LightExponentBinding.GetParamMapBindableVariable(), Properties->DefaultExponent);
	const float DefaultSpecularScale = ParameterStore.GetParameterValueOrDefault(Properties->SpecularScaleBinding.GetParamMapBindableVariable(), Properties->SpecularScale);
	const float DefaultDiffuseScale = ParameterStore.GetParameterValueOrDefault(Properties->DiffuseScaleBinding.GetParamMapBindableVariable(), Properties->DiffuseScale);

	const float InverseExposureBlend = Properties->bOverrideInverseExposureBlend ? Properties->InverseExposureBlend : GetDefault<UNiagaraSettings>()->DefaultLightInverseExposureBlend;
	const int32 SourcePrimitiveId = Proxy->GetPrimitiveSceneInfo()->GetPersistentIndex().Index;

	// Particles Source mode?
	if (Properties->SourceMode == ENiagaraRendererSourceDataMode::Particles)
	{
		//I'm not a great fan of pulling scalar components out to a structured vert buffer like this.
		//TODO: Experiment with a new VF that reads the data directly from the scalar layout.
		const auto PositionReader = Properties->PositionDataSetAccessor.GetReader(Data);
		const auto ColorReader = Properties->ColorDataSetAccessor.GetReader(Data);
		const auto RadiusReader = Properties->RadiusDataSetAccessor.GetReader(Data);
		const auto ExponentReader = Properties->ExponentDataSetAccessor.GetReader(Data);
		const auto ScatteringReader = Properties->ScatteringDataSetAccessor.GetReader(Data);
		const auto EnabledReader = Properties->EnabledDataSetAccessor.GetReader(Data);
		const auto VisTagReader = Properties->RendererVisibilityTagAccessor.GetReader(Data);
		const auto SpecularScaleReader = Properties->SpecularScaleAccessor.GetReader(Data);
		const auto DiffuseScaleReader = Properties->DiffuseScaleAccessor.GetReader(Data);
		const auto LightIdReader = Properties->LightIdAccessor.GetReader(Data);

		for (uint32 ParticleIndex = 0; ParticleIndex < DataToRender->GetNumInstances(); ParticleIndex++)
		{
			const int32 VisTag = VisTagReader.GetSafe(ParticleIndex, DefaultVisibilityTag);
			const bool bShouldRenderParticleLight = EnabledReader.GetSafe(ParticleIndex, DefaultEnabled).GetValue() && (VisTag == Properties->RendererVisibility);
			const float LightRadius = RadiusReader.GetSafe(ParticleIndex, DefaultRadius) * Properties->RadiusScale;
			if (bShouldRenderParticleLight && (LightRadius > 0.0f))
			{
				SimpleLightData& LightData = DynamicData->LightArray.AddDefaulted_GetRef();

				const FLinearColor Color = ColorReader.GetSafe(ParticleIndex, DefaultColor);
				const float Brightness = Properties->bAlphaScalesBrightness ? Color.A : 1.0f;
				const FVector3f SimPos = PositionReader.GetSafe(ParticleIndex, DefaultPos);

				LightData.LightEntry.Radius = LightRadius;
				LightData.LightEntry.Color = FVector3f(Color) * Brightness + Properties->ColorAdd;
				LightData.LightEntry.Exponent = Properties->bUseInverseSquaredFalloff ? 0 : ExponentReader.GetSafe(ParticleIndex, DefaultExponent);
				LightData.LightEntry.InverseExposureBlend = InverseExposureBlend;
				LightData.LightEntry.bAffectTranslucency = Properties->bAffectsTranslucency;
				LightData.LightEntry.bAllowMegaLights = Properties->bAllowMegaLights;
				LightData.LightEntry.bMegaLightsCastShadows = Properties->bMegaLightsCastShadows;
				LightData.LightEntry.VolumetricScatteringIntensity = ScatteringReader.GetSafe(ParticleIndex, DefaultScattering);
				LightData.LightEntry.SpecularScale = SpecularScaleReader.GetSafe(ParticleIndex, DefaultSpecularScale);
				LightData.LightEntry.DiffuseScale = DiffuseScaleReader.GetSafe(ParticleIndex, DefaultDiffuseScale);
				LightData.LightEntry.LightId = FSimpleLightId(LightIdReader.GetSafe(ParticleIndex, INDEX_NONE), SourcePrimitiveId);
				LightData.PerViewEntry.Position = SimToWorld.TransformPosition(FVector(SimPos));
			}
		}
	}
	else
	{
		const bool bEnabled = DefaultEnabled.GetValue();
		const int32 VisTag = DefaultVisibilityTag;
		const float LightRadius = DefaultRadius * Properties->RadiusScale;
		if (bEnabled && VisTag == Properties->RendererVisibility && LightRadius > 0.0f)
		{
			const FVector3f SimPos = DefaultPos;
			const FLinearColor LightColor = DefaultColor;
			const float Brightness = Properties->bAlphaScalesBrightness ? LightColor.A : 1.0f;

			SimpleLightData& LightData = DynamicData->LightArray.AddDefaulted_GetRef();
			LightData.LightEntry.Radius = LightRadius;
			LightData.LightEntry.Color = FVector3f(LightColor) * Brightness + Properties->ColorAdd;
			LightData.LightEntry.Exponent = Properties->bUseInverseSquaredFalloff ? 0 : DefaultExponent;
			LightData.LightEntry.InverseExposureBlend = InverseExposureBlend;
			LightData.LightEntry.bAffectTranslucency = Properties->bAffectsTranslucency;
			LightData.LightEntry.bAllowMegaLights = Properties->bAllowMegaLights;
			LightData.LightEntry.bMegaLightsCastShadows = Properties->bMegaLightsCastShadows;
			LightData.LightEntry.VolumetricScatteringIntensity = DefaultScattering;
			LightData.LightEntry.SpecularScale = DefaultSpecularScale;
			LightData.LightEntry.DiffuseScale = DefaultDiffuseScale;
			LightData.LightEntry.LightId = FSimpleLightId(0, SourcePrimitiveId);

			LightData.PerViewEntry.Position = SimToWorld.TransformPosition(FVector(SimPos));
		}
	}

#if WITH_NIAGARA_RENDERER_DEBUGDRAW
	if (Properties->IsDebugDrawEnabled())
	{
		NiagaraRendererLightsPrivate::DebugDraw(Emitter->GetParentSystemInstance(), DynamicData->LightArray);
	}
#endif

	return DynamicData;
}

void FNiagaraRendererLights::GatherSimpleLights(FSimpleLightArray& OutParticleLights)const
{
	if (NiagaraRendererLightsPrivate::GbRendererEnabled == false)
	{
		return;
	}

	if (const FNiagaraDynamicDataLights* DynamicData = static_cast<const FNiagaraDynamicDataLights*>(DynamicDataRender))
	{
		const int32 LightCount = DynamicData->LightArray.Num();

		OutParticleLights.InstanceData.Reserve(OutParticleLights.InstanceData.Num() + LightCount);
		OutParticleLights.PerViewData.Reserve(OutParticleLights.PerViewData.Num() + LightCount);

		for (const FNiagaraRendererLights::SimpleLightData &LightData : DynamicData->LightArray)
		{
			// When not using camera-offset, output one position for all views to share.
			OutParticleLights.PerViewData.Add(LightData.PerViewEntry);

			// Add an entry for the light instance.
			OutParticleLights.InstanceData.Add(LightData.LightEntry);
		}
	}
}
