// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRendererMeshes.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/StaticMesh.h"
#include "FXRenderingUtils.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialRenderProxy.h"
#include "NiagaraComponent.h"
#include "NiagaraCullProxyComponent.h"
#include "NiagaraDataSet.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraGPURayTracingTransformsShader.h"
#include "NiagaraGPUSortInfo.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraRenderableMeshArrayInterface.h"
#include "NiagaraRenderableMeshInterface.h"
#include "NiagaraSceneProxy.h"
#include "NiagaraSortingGPU.h"
#include "NiagaraStats.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraSystemInstance.h"
#include "PipelineStateCache.h"
#include "RayTracingInstance.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "SceneRendererInterface.h"
#include "StaticMeshResources.h"

DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Generate Mesh Vertex Data [GT]"), STAT_NiagaraGenMeshVertexData, STATGROUP_Niagara, EStatFlags::Verbose);

DECLARE_DWORD_COUNTER_STAT(TEXT("NumMeshesRenderer"), STAT_NiagaraNumMeshes, STATGROUP_Niagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumMeshVerts"), STAT_NiagaraNumMeshVerts, STATGROUP_Niagara);

static bool GbEnableNiagaraMeshRendering = true;
static FAutoConsoleVariableRef CVarEnableNiagaraMeshRendering(
	TEXT("fx.EnableNiagaraMeshRendering"),
	GbEnableNiagaraMeshRendering,
	TEXT("If false Niagara Mesh Renderers are disabled."),
	ECVF_Default
);

static int32 GNiagaraMeshRendererCalcMeshUsedParticleCount = 64;
static FAutoConsoleVariableRef CVarNiagaraMeshRendererCalcMeshUsedParticleCount(
	TEXT("fx.Niagara.MeshRenderer.CalcMeshUsedParticleCount"),
	GNiagaraMeshRendererCalcMeshUsedParticleCount,
	TEXT("Optimization which will inspect renderer vis / mesh index to determine which set of meshes we can potential render with."),
	ECVF_Default
);

#if RHI_RAYTRACING
static TAutoConsoleVariable<int32> CVarRayTracingNiagaraMeshes(
	TEXT("r.RayTracing.Geometry.NiagaraMeshes"),
	1,
	TEXT("Include Niagara meshes in ray tracing effects (default = 1 (Niagara meshes enabled in ray tracing))"));
#endif

struct FNiagaraDynamicDataMesh : public FNiagaraDynamicDataBase
{
	FNiagaraDynamicDataMesh(const FNiagaraEmitterInstance* InEmitter)
		: FNiagaraDynamicDataBase(InEmitter)
	{
	}

	virtual void ApplyMaterialOverride(int32 MaterialIndex, UMaterialInterface* MaterialOverride) override
	{
		if (Materials.IsValidIndex(MaterialIndex) && MaterialOverride)
		{
			Materials[MaterialIndex] = MaterialOverride->GetRenderProxy();
		}
	}

	TArray<FMaterialRenderProxy*, TInlineAllocator<8>> Materials;
	TArray<UNiagaraDataInterface*> DataInterfacesBound;
	TArray<UObject*> ObjectsBound;
	TArray<uint8> ParameterDataBound;
};

//////////////////////////////////////////////////////////////////////////

static FBox ScaleFBox(const FBox InBox, const FVector3f InMeshScale, const FVector3f InBoundsScale)
{
	const FVector MeshScale = FVector(InMeshScale);
	const FVector BoundsScale = FVector(InBoundsScale);
	const FVector MeshCenter = InBox.GetCenter() * MeshScale;
	const FVector MeshExtent = InBox.GetExtent() * MeshScale;
	return FBox(MeshCenter - (MeshExtent * BoundsScale), MeshCenter + (MeshExtent * BoundsScale));
}

//////////////////////////////////////////////////////////////////////////

FNiagaraRendererMeshes::FNiagaraRendererMeshes(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties* Props, const FNiagaraEmitterInstance* Emitter)
	: FNiagaraRenderer(FeatureLevel, Props, Emitter)
	, MaterialParamValidMask(0)
{
	check(Emitter);
	check(Props);

	const UNiagaraMeshRendererProperties* Properties = CastChecked<const UNiagaraMeshRendererProperties>(Props);
	SourceMode = Properties->SourceMode;
	FacingMode = Properties->FacingMode;
	MeshBoundsScale = FVector3f(Properties->MeshBoundsScale);
	bLockedAxisEnable = Properties->bLockedAxisEnable;
	LockedAxis = FVector3f(Properties->LockedAxis);
	LockedAxisSpace = Properties->LockedAxisSpace;
	SortMode = Properties->SortMode;
	bSortHighPrecision = UNiagaraRendererProperties::IsSortHighPrecision(Properties->SortPrecision);
	bSortOnlyWhenTranslucent = Properties->bSortOnlyWhenTranslucent;
	bGpuLowLatencyTranslucency = UNiagaraRendererProperties::IsGpuTranslucentThisFrame(FeatureLevel, Properties->GpuTranslucentLatency);
	bOverrideMaterials = Properties->bOverrideMaterials;
	SubImageSize = FVector2f(Properties->SubImageSize);	// LWC_TODO: Precision loss
	bSubImageBlend = Properties->bSubImageBlend;
	bEnableFrustumCulling = Properties->bEnableFrustumCulling;
	bEnableLODCulling = false;
	bEnableCulling = bEnableFrustumCulling;
	DistanceCullRange = FVector2f(0, FLT_MAX);
	DistanceCullRangeSquared = FVector2f(0, FLT_MAX);
	RendererVisibility = Properties->RendererVisibility;
	bAccurateMotionVectors = Properties->NeedsPreciseMotionVectors();
	bCastShadows = Properties->bCastShadows;
#if WITH_EDITORONLY_DATA
	bIncludeInHitProxy = Properties->bIncludeInHitProxy;
#endif

	if (Properties->bEnableCameraDistanceCulling)
	{
		DistanceCullRange = FVector2f(Properties->MinCameraDistance, Properties->MaxCameraDistance);
		DistanceCullRangeSquared = DistanceCullRange* DistanceCullRange;
		bEnableCulling |= true;
	}

	// Ensure valid value for the locked axis
	if (!LockedAxis.Normalize())
	{
		LockedAxis.Set(0.0f, 0.0f, 1.0f);
	}

	const FNiagaraDataSet& Data = Emitter->GetParticleData();

	int32 FloatOffset;
	int32 HalfOffset;

	ParticleRendererVisTagOffset = INDEX_NONE;
	EmitterRendererVisTagOffset = INDEX_NONE;
	if (Properties->RendererVisibilityTagBinding.CanBindToHostParameterMap())
	{
		EmitterRendererVisTagOffset = Emitter->GetRendererBoundVariables().IndexOf(Properties->RendererVisibilityTagBinding.GetParamMapBindableVariable());
	}
	else if (Data.GetVariableComponentOffsets(Properties->RendererVisibilityTagBinding.GetDataSetBindableVariable(), FloatOffset, ParticleRendererVisTagOffset, HalfOffset))
	{
		bEnableCulling |= ParticleRendererVisTagOffset != INDEX_NONE;
	}

	ParticleMeshIndexOffset = INDEX_NONE;
	EmitterMeshIndexOffset = INDEX_NONE;
	if (Properties->MeshIndexBinding.CanBindToHostParameterMap())
	{
		EmitterMeshIndexOffset = Emitter->GetRendererBoundVariables().IndexOf(Properties->MeshIndexBinding.GetParamMapBindableVariable());
	}
	else if (Data.GetVariableComponentOffsets(Properties->MeshIndexBinding.GetDataSetBindableVariable(), FloatOffset, ParticleMeshIndexOffset, HalfOffset))
	{
		bEnableCulling |= ParticleMeshIndexOffset != INDEX_NONE;
	}

	MaterialParamValidMask = Properties->MaterialParamValidMask;

	RendererLayoutWithCustomSorting = &Properties->RendererLayoutWithCustomSorting;
	RendererLayoutWithoutCustomSorting = &Properties->RendererLayoutWithoutCustomSorting;

	bSetAnyBoundVars = false;
	if (Emitter->GetRendererBoundVariables().IsEmpty() == false)
	{
		const TArray< const FNiagaraVariableAttributeBinding*>& VFBindings = Properties->GetAttributeBindings();
		const int32 NumBindings = bAccurateMotionVectors ? ENiagaraMeshVFLayout::Type::Num_Max : ENiagaraMeshVFLayout::Type::Num_Default;
		check(VFBindings.Num() >= ENiagaraMeshVFLayout::Type::Num_Max);
		for (int32 i = 0; i < ENiagaraMeshVFLayout::Type::Num_Max; i++)
		{
			VFBoundOffsetsInParamStore[i] = INDEX_NONE;
			if (i < NumBindings && VFBindings[i] && VFBindings[i]->CanBindToHostParameterMap())
			{
				VFBoundOffsetsInParamStore[i] = Emitter->GetRendererBoundVariables().IndexOf(VFBindings[i]->GetParamMapBindableVariable());
				if (VFBoundOffsetsInParamStore[i] != INDEX_NONE)
					bSetAnyBoundVars = true;
			}
		}
	}
	else
	{
		for (int32 i = 0; i < ENiagaraMeshVFLayout::Type::Num_Max; i++)
		{
			VFBoundOffsetsInParamStore[i] = INDEX_NONE;
		}
	}
}

FNiagaraRendererMeshes::~FNiagaraRendererMeshes()
{
}

void FNiagaraRendererMeshes::Initialize(const UNiagaraRendererProperties* InProps, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
	FNiagaraRenderer::Initialize(InProps, Emitter, InController);

	check(Emitter);
	check(InProps);

	const UNiagaraMeshRendererProperties* Properties = CastChecked<const UNiagaraMeshRendererProperties>(InProps);

	// Initialize the valid mesh slots, and prep them with the data for every mesh, LOD, and section we'll be needing over the lifetime of the renderer
	const INiagaraRenderableMeshArrayInterface* MeshArrayInterface = Properties->GetMeshArrayInterface(Emitter);

	MeshUsedMax = 0;

	FNiagaraSystemInstance* SystemInstance = Emitter ? Emitter->GetParentSystemInstance() : nullptr;

	uint32 SourceMeshIndex = 0;
	MeshArrayInterface->ForEachMesh(
		SystemInstance,
		[this](int32 NumMeshes)
		{
			Meshes.Empty(NumMeshes);
		},
		[this, &SourceMeshIndex, Emitter, Properties](const FNiagaraMeshRendererMeshProperties& MeshProperties)
		{
			FNiagaraRenderableMeshPtr RenderableMesh = MeshProperties.ResolveRenderableMesh(Emitter);
			if (RenderableMesh != nullptr)
			{
				MeshUsedMax = SourceMeshIndex + 1;

				FIntVector2 LODRange = FIntVector2(0, 1);
				if (MeshProperties.LODMode == ENiagaraMeshLODMode::LODLevel)
				{
					LODRange.X = MeshProperties.LODLevel;
					LODRange.Y = LODRange.X + 1;
				}
				else if (MeshProperties.LODMode == ENiagaraMeshLODMode::PerParticle)
				{
					LODRange = RenderableMesh->GetLODRange();
					if (MeshProperties.bUseLODRange)
					{
						LODRange.X = FMath::Max(MeshProperties.LODRange.X, LODRange.X);
						LODRange.Y = FMath::Max(MeshProperties.LODRange.Y, LODRange.Y);
					}
					bEnableLODCulling |= LODRange.Y - LODRange.X > 1;
				}

				for (int32 LOD=LODRange.X; LOD < LODRange.Y; ++LOD)
				{
					// We have a valid mesh fill in the details
					FMeshData& MeshData = Meshes.AddDefaulted_GetRef();
					MeshData.RenderableMesh = RenderableMesh;
					MeshData.SourceMeshIndex = SourceMeshIndex;
					MeshData.LODMode = MeshProperties.LODMode;
					MeshData.LODLevel = LOD;
					MeshData.PivotOffset = FVector3f(MeshProperties.PivotOffset);
					MeshData.PivotOffsetSpace = MeshProperties.PivotOffsetSpace;
					MeshData.Scale = FVector3f(MeshProperties.Scale);
					MeshData.Rotation = FQuat4f(MeshProperties.Rotation.Quaternion());

					if (MeshProperties.LODMode == ENiagaraMeshLODMode::LODBias)
					{
						RenderableMesh->SetMinLODBias(MeshProperties.LODBias);
					}
					else if (MeshProperties.LODMode == ENiagaraMeshLODMode::ByComponentBounds || MeshProperties.LODMode == ENiagaraMeshLODMode::ComponentOrigin || MeshProperties.LODMode == ENiagaraMeshLODMode::PerParticle)
					{
						MeshData.LODDistanceFactor = MeshProperties.LODDistanceFactor;
						if (MeshProperties.LODMode == ENiagaraMeshLODMode::PerParticle)
						{
							MeshData.LODScreenSize = RenderableMesh->GetLODScreenSize(LOD);
							MeshData.LODScreenSize.X = LOD < LODRange.Y - 1 ? MeshData.LODScreenSize.X : 0.0f;
							MeshData.LODScreenSize.Y = LOD > LODRange.X ? MeshData.LODScreenSize.Y : -1.0f;
						}
					}

					// Get materials and remap them into the base material list
					TArray<UMaterialInterface*> UsedMaterials;
					RenderableMesh->GetUsedMaterials(UsedMaterials);
					Properties->ApplyMaterialOverrides(Emitter, UsedMaterials);

					for (UMaterialInterface* UsedMaterial : UsedMaterials)
					{
						MeshData.MaterialRemapTable.Add(
							BaseMaterials_GT.IndexOfByPredicate(
								[&](UMaterialInterface* LookMat)
								{
									if (LookMat == UsedMaterial)
									{
										return true;
									}
									if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(LookMat))
									{
										return UsedMaterial == MID->Parent;
									}
									return false;
								}
							)
						);
					}
				}
			}

			// Increment the source mesh index
			++SourceMeshIndex;
		}
	);
}

void FNiagaraRendererMeshes::ReleaseRenderThreadResources()
{
}

void FNiagaraRendererMeshes::PrepareParticleMeshRenderData(FRHICommandListBase& RHICmdList, FParticleMeshRenderData& ParticleMeshRenderData, const FSceneViewFamily& ViewFamily, FMeshElementCollector& Collector, FNiagaraDynamicDataBase* InDynamicData, const FNiagaraSceneProxy* SceneProxy, bool bRayTracing, ENiagaraGpuComputeTickStage::Type GpuReadyTickStage) const
{
	ParticleMeshRenderData.Collector = &Collector;

	ParticleMeshRenderData.bUseGPUScene = FNiagaraMeshVertexFactory::IsGPUSceneEnabled(SceneProxy->GetScene().GetShaderPlatform(), FeatureLevel);

	// Anything to render?
	ParticleMeshRenderData.DynamicDataMesh = static_cast<FNiagaraDynamicDataMesh*>(InDynamicData);
	if (!ParticleMeshRenderData.DynamicDataMesh || !SceneProxy->GetComputeDispatchInterface())
	{
		return;
	}

	// Early out if we have no data or instances, this must be done before we read the material
	FNiagaraDataBuffer* CurrentParticleData = ParticleMeshRenderData.DynamicDataMesh->GetParticleDataToRender(RHICmdList, bGpuLowLatencyTranslucency);
	if (!CurrentParticleData || (SourceMode == ENiagaraRendererSourceDataMode::Particles && CurrentParticleData->GetNumInstances() == 0) || !Meshes.Num())
	{
		return;
	}

	// Check if any materials are translucent and if we can pickup the low latency data
	// If these conditions change please update the DebugHUD display also to reflect it
	// Note: SceneCaptures will use latent data as GpuReadyTickStage < CurrentParticleData->GetGPUDataReadyStage()
	//       For main pass scene captures we exclude the batches if they are translucent
	const bool bIsExcludedViewMode = AllowDebugViewmodes() && (ViewFamily.EngineShowFlags.Wireframe || ViewFamily.EngineShowFlags.ShaderComplexity || ViewFamily.EngineShowFlags.ShaderComplexityWithQuadOverdraw);

	ParticleMeshRenderData.bIsGpuLowLatencyTranslucency =
		bGpuLowLatencyTranslucency &&
		GpuReadyTickStage >= CurrentParticleData->GetGPUDataReadyStage() &&
		!bIsExcludedViewMode &&
		!SceneProxy->CastsVolumetricTranslucentShadow() &&
		ParticleMeshRenderData.DynamicDataMesh->Materials.Num() > 0 &&
		ParticleMeshRenderData.DynamicDataMesh->IsGpuLowLatencyTranslucencyEnabled() &&
		ViewFamilySupportLowLatencyTranslucency(ViewFamily);

	ParticleMeshRenderData.bHasTranslucentMaterials = false;
	ParticleMeshRenderData.bAllTranslucentMaterials = ParticleMeshRenderData.DynamicDataMesh->Materials.Num() > 0;
	ParticleMeshRenderData.bCanCastTranslucentShadows = false;

	for (FMaterialRenderProxy* MaterialProxy : ParticleMeshRenderData.DynamicDataMesh->Materials)
	{
		check(MaterialProxy);
		const FMaterial& Material = MaterialProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
		const bool bTranslucent = IsTranslucentBlendMode(Material);

		ParticleMeshRenderData.bHasTranslucentMaterials |= bTranslucent;
		ParticleMeshRenderData.bAllTranslucentMaterials &= bTranslucent;
		ParticleMeshRenderData.bCanCastTranslucentShadows |= SceneProxy->CastsVolumetricTranslucentShadow() || Material.GetCastDynamicShadowAsMasked();

		// If even one material can cause the mesh to render before FFXSystemInterface::PostRenderOpaque, we cannot use low latency data
		ParticleMeshRenderData.bIsGpuLowLatencyTranslucency =
			ParticleMeshRenderData.bIsGpuLowLatencyTranslucency &&
			!UE::FXRenderingUtils::CanMaterialRenderBeforeFXPostOpaque(ViewFamily, *SceneProxy, Material);
	}
	ParticleMeshRenderData.bIsGpuLowLatencyTranslucency = ParticleMeshRenderData.bIsGpuLowLatencyTranslucency && !ParticleMeshRenderData.bCanCastTranslucentShadows;

	ParticleMeshRenderData.SourceParticleData = ParticleMeshRenderData.DynamicDataMesh->GetParticleDataToRender(RHICmdList, ParticleMeshRenderData.bIsGpuLowLatencyTranslucency);
	
	// Anything to render?
	if ((ParticleMeshRenderData.SourceParticleData == nullptr) ||
		(SourceMode == ENiagaraRendererSourceDataMode::Particles && ParticleMeshRenderData.SourceParticleData->GetNumInstances() == 0) ||
		(Meshes.Num() == 0)
	)
	{
		ParticleMeshRenderData.SourceParticleData = nullptr;
		return;
	}

	// If the visibility tag comes from a parameter map, so we can evaluate it here and just early out if it doesn't match up
	if (EmitterRendererVisTagOffset != INDEX_NONE && ParticleMeshRenderData.DynamicDataMesh->ParameterDataBound.IsValidIndex(EmitterRendererVisTagOffset))
	{
		int32 VisTag = 0;
		FMemory::Memcpy(&VisTag, ParticleMeshRenderData.DynamicDataMesh->ParameterDataBound.GetData() + EmitterRendererVisTagOffset, sizeof(int32));
		if (RendererVisibility != VisTag)
		{
			ParticleMeshRenderData.SourceParticleData = nullptr;
			return;
		}
	}

	// Particle source mode
	if (SourceMode == ENiagaraRendererSourceDataMode::Particles)
	{
		const EShaderPlatform ShaderPlatform = SceneProxy->GetComputeDispatchInterface()->GetShaderPlatform();

		// Determine if we need sorting
		ParticleMeshRenderData.bNeedsSort = SortMode != ENiagaraSortMode::None && (ParticleMeshRenderData.bHasTranslucentMaterials || !bSortOnlyWhenTranslucent);
		const bool bNeedCustomSort = ParticleMeshRenderData.bNeedsSort && (SortMode == ENiagaraSortMode::CustomAscending || SortMode == ENiagaraSortMode::CustomDecending);
		ParticleMeshRenderData.RendererLayout = bNeedCustomSort ? RendererLayoutWithCustomSorting : RendererLayoutWithoutCustomSorting;
		ParticleMeshRenderData.SortVariable = bNeedCustomSort ? ENiagaraMeshVFLayout::CustomSorting : ENiagaraMeshVFLayout::Position;
		if (ParticleMeshRenderData.bNeedsSort)
		{
			TConstArrayView<FNiagaraRendererVariableInfo> VFVariables = ParticleMeshRenderData.RendererLayout->GetVFVariables_RenderThread();
			const FNiagaraRendererVariableInfo& SortVariable = VFVariables[ParticleMeshRenderData.SortVariable];
			ParticleMeshRenderData.bNeedsSort = SortVariable.GetGPUOffset() != INDEX_NONE;
		}

		// Do we need culling? (When using GPU Scene, we don't cull in the sort passes)
		ParticleMeshRenderData.bNeedsCull = (bEnableCulling || bEnableLODCulling) && !ParticleMeshRenderData.bUseGPUScene;
		ParticleMeshRenderData.bAllowPerParticleMeshLODs = bEnableLODCulling;
		ParticleMeshRenderData.bSortCullOnGpu = (ParticleMeshRenderData.bNeedsSort && FNiagaraUtilities::AllowGPUSorting(ShaderPlatform)) || (ParticleMeshRenderData.bNeedsCull && FNiagaraUtilities::AllowGPUCulling(ShaderPlatform));

		// Validate what we setup
		if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			if (!ensureMsgf(!ParticleMeshRenderData.bNeedsCull || ParticleMeshRenderData.bSortCullOnGpu, TEXT("Culling is requested on GPU but sorting is disabled, this will result in incorrect rendering. Asset(%s)."), *SceneProxy->GetResourceName().ToString()))
			{
				ParticleMeshRenderData.bNeedsCull = false;
				ParticleMeshRenderData.bAllowPerParticleMeshLODs = false;
				ParticleMeshRenderData.bAllowFirstMeshOnly = true;
			}
			ParticleMeshRenderData.bNeedsSort &= ParticleMeshRenderData.bSortCullOnGpu;

			//-TODO: Culling and sorting from InitViewsAfterPrePass can not be respected if the culled entries have already been acquired
			if ((ParticleMeshRenderData.bNeedsSort || ParticleMeshRenderData.bNeedsCull) && !SceneProxy->GetComputeDispatchInterface()->GetGPUInstanceCounterManager().CanAcquireCulledEntry())
			{
				//ensureMsgf(false, TEXT("Culling & sorting is not supported once the culled counts have been acquired, sorting & culling will be disabled for these draws."));
				ParticleMeshRenderData.bNeedsSort = false;
				ParticleMeshRenderData.bNeedsCull = false;
				ParticleMeshRenderData.bAllowPerParticleMeshLODs = false;
				ParticleMeshRenderData.bAllowFirstMeshOnly = true;
			}
		}
		else
		{
			//-TODO: Culling and sorting from InitViewsAfterPrePass can not be respected if the culled entries have already been acquired
			if (ParticleMeshRenderData.bSortCullOnGpu)
			{
				//ensureMsgf(false, TEXT("Culling & sorting is not supported once the culled counts have been acquired, sorting & culling will be disabled for these draws."));
				ParticleMeshRenderData.bSortCullOnGpu &= SceneProxy->GetComputeDispatchInterface()->GetGPUInstanceCounterManager().CanAcquireCulledEntry();
			}

			// For CPU sims, decide if we should sort / cull on the GPU or not
			if ( ParticleMeshRenderData.bSortCullOnGpu && !ParticleMeshRenderData.bAllowPerParticleMeshLODs)
			{
				const int32 NumInstances = ParticleMeshRenderData.SourceParticleData->GetNumInstances();

				const int32 SortThreshold = GNiagaraGPUSortingCPUToGPUThreshold;
				const bool bSortMoveToGpu = (SortThreshold >= 0) && (NumInstances >= SortThreshold);

				const int32 CullThreshold = GNiagaraGPUCullingCPUToGPUThreshold;
				const bool bCullMoveToGpu = (CullThreshold >= 0) && (NumInstances >= CullThreshold);

				ParticleMeshRenderData.bSortCullOnGpu = bSortMoveToGpu || bCullMoveToGpu;
			}

			// Update if we support per particle LODs, we either need to be able to run the sort & cull process or use GPU scene
			ParticleMeshRenderData.bAllowPerParticleMeshLODs &= ParticleMeshRenderData.bSortCullOnGpu || ParticleMeshRenderData.bUseGPUScene;
		}

		// Update layout as it could have changed
		ParticleMeshRenderData.RendererLayout = bNeedCustomSort ? RendererLayoutWithCustomSorting : RendererLayoutWithoutCustomSorting;
	}
}

bool FNiagaraRendererMeshes::CalculateMeshUsed(FRHICommandListBase& RHICmdList, FParticleMeshRenderData& ParticleMeshRenderData) const
{
	//-OPT: Should we handle modes where the tags are inside none-particle data?  i.e. EmitterRendererVisTagOffset | EmitterMeshIndexOffset
	if (ParticleRendererVisTagOffset == INDEX_NONE && ParticleMeshIndexOffset == INDEX_NONE)
	{
		return true;
	}

	if (SimTarget != ENiagaraSimTarget::CPUSim)
	{
		return true;
	}

	const int32 NumInstances = ParticleMeshRenderData.SourceParticleData->GetNumInstances();
	if ( NumInstances > GNiagaraMeshRendererCalcMeshUsedParticleCount )
	{
		return true;
	}

	if (SourceMode != ENiagaraRendererSourceDataMode::Particles)
	{
		//-OPT: Should we handle emitter mode?
		return true;
	}

	const FNiagaraDataBuffer* DataToRender = ParticleMeshRenderData.DynamicDataMesh->GetParticleDataToRender(RHICmdList);
	if (ParticleRendererVisTagOffset != INDEX_NONE)
	{
		const int32* RendererVisValues = reinterpret_cast<const int32*>(DataToRender->GetComponentPtrInt32(ParticleRendererVisTagOffset));
		int32 FoundIndex = 0;
		while (FoundIndex < NumInstances)
		{
			if (RendererVisValues[FoundIndex] == RendererVisibility)
			{
				break;
			}
			++FoundIndex;
		}
		if (FoundIndex == NumInstances)
		{
			return false;
		}
	}

	if (ParticleMeshIndexOffset == INDEX_NONE)
	{
		return true;
	}

	ParticleMeshRenderData.MeshUsed.AddUninitialized(MeshUsedMax);
	ParticleMeshRenderData.MeshUsed.SetRange(0, MeshUsedMax, false);
	bool bAnyVisible = false;

	const int32* MeshIndexValues = reinterpret_cast<const int32*>(DataToRender->GetComponentPtrInt32(ParticleMeshIndexOffset));
	for (int32 i=0; i < NumInstances; ++i)
	{
		const int32 MeshIndex = MeshIndexValues[i];
		if (MeshIndex >= 0 && MeshIndex < MeshUsedMax)
		{
			ParticleMeshRenderData.MeshUsed[MeshIndex] = true;
			bAnyVisible = true;
		}
	}

	return bAnyVisible;
}

void FNiagaraRendererMeshes::PrepareParticleRenderBuffers(FRHICommandListBase& RHICmdList, FParticleMeshRenderData& ParticleMeshRenderData, FGlobalDynamicReadBuffer& DynamicReadBuffer) const
{
	if ( SourceMode == ENiagaraRendererSourceDataMode::Particles )
	{
		if ( SimTarget == ENiagaraSimTarget::CPUSim )
		{
			// Determine what integer attributes we need to copy to the GPU for culling
			TArray<uint32, TInlineAllocator<2>> IntParamsToCopy;
			if (ParticleMeshRenderData.bNeedsCull || ParticleMeshRenderData.bUseGPUScene)
			{
				if (ParticleMeshRenderData.bSortCullOnGpu || ParticleMeshRenderData.bUseGPUScene)
				{
					if (ParticleRendererVisTagOffset != INDEX_NONE)
					{
						ParticleMeshRenderData.RendererVisTagOffset = IntParamsToCopy.Add(ParticleRendererVisTagOffset);
					}
					if (ParticleMeshIndexOffset != INDEX_NONE)
					{
						ParticleMeshRenderData.MeshIndexOffset = IntParamsToCopy.Add(ParticleMeshIndexOffset);
					}
				}
				else
				{
					ParticleMeshRenderData.RendererVisTagOffset = ParticleRendererVisTagOffset;
					ParticleMeshRenderData.MeshIndexOffset = ParticleMeshIndexOffset;
				}
			}

			FParticleRenderData ParticleRenderData = TransferDataToGPU(RHICmdList, DynamicReadBuffer, ParticleMeshRenderData.RendererLayout, IntParamsToCopy, ParticleMeshRenderData.SourceParticleData);
			const uint32 NumInstances = ParticleMeshRenderData.SourceParticleData->GetNumInstances();

			ParticleMeshRenderData.ParticleFloatSRV = GetSrvOrDefaultFloat(ParticleRenderData.FloatData);
			ParticleMeshRenderData.ParticleHalfSRV = GetSrvOrDefaultHalf(ParticleRenderData.HalfData);
			ParticleMeshRenderData.ParticleIntSRV = GetSrvOrDefaultInt(ParticleRenderData.IntData);
			ParticleMeshRenderData.ParticleFloatDataStride = ParticleRenderData.FloatStride / sizeof(float);
			ParticleMeshRenderData.ParticleHalfDataStride = ParticleRenderData.HalfStride / sizeof(FFloat16);
			ParticleMeshRenderData.ParticleIntDataStride = ParticleRenderData.IntStride / sizeof(int32);
		}
		else
		{
			ParticleMeshRenderData.ParticleFloatSRV = GetSrvOrDefaultFloat(ParticleMeshRenderData.SourceParticleData->GetGPUBufferFloat());
			ParticleMeshRenderData.ParticleHalfSRV = GetSrvOrDefaultHalf(ParticleMeshRenderData.SourceParticleData->GetGPUBufferHalf());
			ParticleMeshRenderData.ParticleIntSRV = GetSrvOrDefaultInt(ParticleMeshRenderData.SourceParticleData->GetGPUBufferInt());
			ParticleMeshRenderData.ParticleFloatDataStride = ParticleMeshRenderData.SourceParticleData->GetFloatStride() / sizeof(float);
			ParticleMeshRenderData.ParticleHalfDataStride = ParticleMeshRenderData.SourceParticleData->GetHalfStride() / sizeof(FFloat16);
			ParticleMeshRenderData.ParticleIntDataStride = ParticleMeshRenderData.SourceParticleData->GetInt32Stride() / sizeof(int32);

			ParticleMeshRenderData.RendererVisTagOffset = ParticleRendererVisTagOffset;
			ParticleMeshRenderData.MeshIndexOffset = ParticleMeshIndexOffset;
		}
	}
	else
	{
		ParticleMeshRenderData.ParticleFloatSRV = FNiagaraRenderer::GetDummyFloatBuffer();
		ParticleMeshRenderData.ParticleHalfSRV = FNiagaraRenderer::GetDummyHalfBuffer();
		ParticleMeshRenderData.ParticleIntSRV = FNiagaraRenderer::GetDummyIntBuffer();
		ParticleMeshRenderData.ParticleFloatDataStride = 0;
		ParticleMeshRenderData.ParticleHalfDataStride = 0;
		ParticleMeshRenderData.ParticleIntDataStride = 0;
	}
}

void FNiagaraRendererMeshes::InitializeSortInfo(const FParticleMeshRenderData& ParticleMeshRenderData, const FNiagaraSceneProxy& SceneProxy, const FSceneView& View, int32 ViewIndex, bool bIsInstancedStereo, FNiagaraGPUSortInfo& OutSortInfo) const
{
	TConstArrayView<FNiagaraRendererVariableInfo> VFVariables = ParticleMeshRenderData.RendererLayout->GetVFVariables_RenderThread();

#if STATS
	OutSortInfo.OwnerName = EmitterStatID.GetName();
#endif
	OutSortInfo.ParticleCount = ParticleMeshRenderData.SourceParticleData->GetNumInstances();
	OutSortInfo.SortMode = SortMode;
	OutSortInfo.SetSortFlags(bSortHighPrecision, ParticleMeshRenderData.SourceParticleData->GetGPUDataReadyStage());
	OutSortInfo.bEnableCulling = ParticleMeshRenderData.bNeedsCull;
	OutSortInfo.RendererVisTagAttributeOffset = ParticleMeshRenderData.RendererVisTagOffset;
	OutSortInfo.MeshIndexAttributeOffset = ParticleMeshRenderData.MeshIndexOffset;
	OutSortInfo.RendererVisibility = RendererVisibility;
	OutSortInfo.DistanceCullRange = DistanceCullRange;

	if (ParticleMeshRenderData.bSortCullOnGpu)
	{
		FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = SceneProxy.GetComputeDispatchInterface();

		OutSortInfo.ParticleDataFloatSRV = ParticleMeshRenderData.ParticleFloatSRV;
		OutSortInfo.ParticleDataHalfSRV = ParticleMeshRenderData.ParticleHalfSRV;
		OutSortInfo.ParticleDataIntSRV = ParticleMeshRenderData.ParticleIntSRV;
		OutSortInfo.FloatDataStride = ParticleMeshRenderData.ParticleFloatDataStride;
		OutSortInfo.HalfDataStride = ParticleMeshRenderData.ParticleHalfDataStride;
		OutSortInfo.IntDataStride = ParticleMeshRenderData.ParticleIntDataStride;
		OutSortInfo.GPUParticleCountSRV = GetSrvOrDefaultUInt(ComputeDispatchInterface->GetGPUInstanceCounterManager().GetInstanceCountBuffer());
		OutSortInfo.GPUParticleCountOffset = ParticleMeshRenderData.SourceParticleData->GetGPUInstanceCountBufferOffset();
	}

	if (ParticleMeshRenderData.SortVariable != INDEX_NONE)
	{
		const FNiagaraRendererVariableInfo& SortVariable = VFVariables[ParticleMeshRenderData.SortVariable];
		OutSortInfo.SortAttributeOffset = ParticleMeshRenderData.bSortCullOnGpu ? SortVariable.GetGPUOffset() : SortVariable.GetEncodedDatasetOffset();
	}

	auto GetViewMatrices = [](const FSceneView& View, FVector& OutViewOrigin) -> const FViewMatrices&
	{
		OutViewOrigin = View.ViewMatrices.GetViewOrigin();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (const FViewMatrices* CachedViewMatrices = View.State ? View.State->GetFrozenViewMatrices() : nullptr)
		{
			// Use the frozen view for culling so we can test that it's working
			OutViewOrigin = CachedViewMatrices->GetViewOrigin();

			// Don't retrieve the cached matrices for shadow views
			bool bIsShadowView = View.GetDynamicMeshElementsShadowCullFrustum() != nullptr;
			if (!bIsShadowView)
			{
				return *CachedViewMatrices;
			}
		}
#endif

		return View.ViewMatrices;
	};

	const TArray<const FSceneView*>& AllViewsInFamily = View.Family->Views;
	const FViewMatrices& ViewMatrices = GetViewMatrices(View, OutSortInfo.ViewOrigin);
	OutSortInfo.ViewDirection = ViewMatrices.GetWorldToView().GetColumn(2);

	if (bIsInstancedStereo)
	{
		// For VR, do distance culling and sorting from a central eye position to prevent differences between views
		OutSortInfo.ViewOrigin = View.CullingOrigin;
	}

	if (bEnableFrustumCulling)
	{
		if (const FConvexVolume* ShadowFrustum = View.GetDynamicMeshElementsShadowCullFrustum())
		{
			// Ensure we don't break the maximum number of planes here
			// (For an accurate shadow frustum, a tight hull is formed from the silhouette and back-facing planes of the view frustum)
			if (ensure(ShadowFrustum->Planes.Num() <= FNiagaraGPUSortInfo::MaxCullPlanes))
			{
				OutSortInfo.CullPlanes = ShadowFrustum->Planes;

				// Remove pre-shadow translation to get the planes in world space
				const FVector PreShadowTranslation = View.GetPreShadowTranslation();
				for (FPlane& Plane : OutSortInfo.CullPlanes)
				{
					Plane.W -= FVector::DotProduct(FVector(Plane), PreShadowTranslation);
				}
			}
		}
		else
		{
			if (bIsInstancedStereo)
			{
				// For Instanced Stereo, cull using an extended frustum that encompasses both eyes
				OutSortInfo.CullPlanes = View.GetCullingFrustum().Planes;
			}
			else
			{
				OutSortInfo.CullPlanes = View.ViewFrustum.Planes;
			}
		}
	}

	if (UseLocalSpace(&SceneProxy))
	{
		OutSortInfo.ViewOrigin = SceneProxy.GetLocalToWorldInverse().TransformPosition(OutSortInfo.ViewOrigin);
		OutSortInfo.ViewDirection = SceneProxy.GetLocalToWorld().GetTransposed().TransformVector(OutSortInfo.ViewDirection);
		if (bEnableFrustumCulling)
		{
			for (FPlane& Plane : OutSortInfo.CullPlanes)
			{
				Plane = Plane.TransformBy(SceneProxy.GetLocalToWorldInverse());
			}
		}
	}
	else
	{
		const FVector LWCTileOffset = FVector(SceneProxy.GetLWCRenderTile()) * FLargeWorldRenderScalar::GetTileSize();
		OutSortInfo.ViewOrigin -= LWCTileOffset;
		if (bEnableFrustumCulling)
		{
			for (FPlane& Plane : OutSortInfo.CullPlanes)
			{
				Plane = Plane.TranslateBy(-LWCTileOffset);
			}
		}
	}

	if (ParticleMeshRenderData.bNeedsCull)
	{
		if ( ParticleMeshRenderData.bSortCullOnGpu )
		{
			OutSortInfo.CullPositionAttributeOffset = VFVariables[ENiagaraMeshVFLayout::Position].GetGPUOffset();
			OutSortInfo.CullOrientationAttributeOffset = VFVariables[ENiagaraMeshVFLayout::Rotation].GetGPUOffset();
			OutSortInfo.CullScaleAttributeOffset = VFVariables[ENiagaraMeshVFLayout::Scale].GetGPUOffset();
		}
		else
		{
			OutSortInfo.CullPositionAttributeOffset = VFVariables[ENiagaraMeshVFLayout::Position].GetEncodedDatasetOffset();
			OutSortInfo.CullOrientationAttributeOffset = VFVariables[ENiagaraMeshVFLayout::Rotation].GetEncodedDatasetOffset();
			OutSortInfo.CullScaleAttributeOffset = VFVariables[ENiagaraMeshVFLayout::Scale].GetEncodedDatasetOffset();
		}
	}
}

void FNiagaraRendererMeshes::PreparePerMeshData(FParticleMeshRenderData& ParticleMeshRenderData, const FNiagaraMeshVertexFactory& VertexFactory, const FNiagaraSceneProxy& SceneProxy, const FMeshData& MeshData) const
{
	// Calculate pivot offset / culling sphere
	const FBox MeshLocalBounds = ScaleFBox(MeshData.RenderableMesh->GetLocalBounds(), MeshData.Scale, MeshBoundsScale);
	ParticleMeshRenderData.CullingSphere.Center = MeshLocalBounds.GetCenter();
	ParticleMeshRenderData.CullingSphere.W = MeshLocalBounds.GetExtent().Length();

	if (MeshData.PivotOffsetSpace == ENiagaraMeshPivotOffsetSpace::Mesh)
	{
		ParticleMeshRenderData.WorldSpacePivotOffset = FVector::ZeroVector;
		ParticleMeshRenderData.CullingSphere.Center += (FVector)MeshData.PivotOffset;
	}
	else
	{
		ParticleMeshRenderData.WorldSpacePivotOffset = (FVector)MeshData.PivotOffset;
		if (MeshData.PivotOffsetSpace == ENiagaraMeshPivotOffsetSpace::Local ||
			(bLocalSpace && MeshData.PivotOffsetSpace == ENiagaraMeshPivotOffsetSpace::Simulation))
		{
			// The offset is in local space, transform it to world
			ParticleMeshRenderData.WorldSpacePivotOffset = SceneProxy.GetLocalToWorld().TransformVector(ParticleMeshRenderData.WorldSpacePivotOffset);
		}
	}

	const int32 NumTexCoords = VertexFactory.GetNumTexcoords();
	const int32 ColorIndexMask = VertexFactory.GetColorIndexMask();
	ParticleMeshRenderData.VertexFetch_Parameters = { ColorIndexMask, NumTexCoords, INDEX_NONE, INDEX_NONE };

	if (RHISupportsManualVertexFetch(GShaderPlatformForFeatureLevel[FeatureLevel]))
	{
		ParticleMeshRenderData.TexCoordBufferSrv = VertexFactory.GetTextureCoordinatesSRV();
		ParticleMeshRenderData.PackedTangentsBufferSrv = VertexFactory.GetTangentsSRV();
		ParticleMeshRenderData.ColorComponentsBufferSrv = VertexFactory.GetColorComponentsSRV();
	}
	else
	{
		ParticleMeshRenderData.TexCoordBufferSrv = FNiagaraRenderer::GetDummyFloat2Buffer();
		ParticleMeshRenderData.PackedTangentsBufferSrv = FNiagaraRenderer::GetDummyFloat4Buffer();
		ParticleMeshRenderData.ColorComponentsBufferSrv = FNiagaraRenderer::GetDummyFloat4Buffer();
	}
}

uint32 FNiagaraRendererMeshes::PerformSortAndCull(FRHICommandListBase& RHICmdList, FParticleMeshRenderData& ParticleMeshRenderData, FGlobalDynamicReadBuffer& ReadBuffer, FNiagaraGPUSortInfo& SortInfo, FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, const FSceneView& View, const FMeshData& MeshData) const
{
	// Emitter mode culls earlier on
	if (SourceMode == ENiagaraRendererSourceDataMode::Emitter)
	{
		ParticleMeshRenderData.ParticleSortedIndicesSRV = GFNiagaraNullSortedIndicesVertexBuffer.VertexBufferSRV.GetReference();
		ParticleMeshRenderData.ParticleSortedIndicesOffset = 0xffffffff;
		return 1;
	}

	uint32 NumInstances = ParticleMeshRenderData.SourceParticleData->GetNumInstances();
	if (ParticleMeshRenderData.bNeedsCull || ParticleMeshRenderData.bNeedsSort)
	{
		SortInfo.LocalBSphere = ParticleMeshRenderData.CullingSphere;
		SortInfo.CullingWorldSpaceOffset = ParticleMeshRenderData.WorldSpacePivotOffset;
		SortInfo.MeshIndex = MeshData.SourceMeshIndex;
		SortInfo.LODScreenSize = FNiagaraRendererMeshes::GetShaderLODScreenSize(View, MeshData);

		if (ParticleMeshRenderData.bSortCullOnGpu)
		{
			SortInfo.CulledGPUParticleCountOffset = ParticleMeshRenderData.bNeedsCull ? ComputeDispatchInterface->GetGPUInstanceCounterManager().AcquireCulledEntry() : INDEX_NONE;
			if (ComputeDispatchInterface->AddSortedGPUSimulation(RHICmdList, SortInfo))
			{
				ParticleMeshRenderData.ParticleSortedIndicesSRV = SortInfo.AllocationInfo.BufferSRV;
				ParticleMeshRenderData.ParticleSortedIndicesOffset = SortInfo.AllocationInfo.BufferOffset;
			}
		}
		else
		{
			FGlobalDynamicReadBuffer::FAllocation SortedIndices;
			SortedIndices = ReadBuffer.AllocateUInt32(NumInstances);
			NumInstances = SortAndCullIndices(SortInfo, *ParticleMeshRenderData.SourceParticleData, SortedIndices);
			ParticleMeshRenderData.ParticleSortedIndicesSRV = SortedIndices.SRV;
			ParticleMeshRenderData.ParticleSortedIndicesOffset = 0;
		}
	}
	else
	{
		ParticleMeshRenderData.ParticleSortedIndicesSRV = GFNiagaraNullSortedIndicesVertexBuffer.VertexBufferSRV.GetReference();
		ParticleMeshRenderData.ParticleSortedIndicesOffset = 0xffffffff;
	}
	return NumInstances;
}

FNiagaraMeshCommonParameters FNiagaraRendererMeshes::CreateCommonShaderParams(const FParticleMeshRenderData& ParticleMeshRenderData, const FSceneView& View, const FMeshData& MeshData, const FNiagaraSceneProxy& SceneProxy) const
{
	const bool bUseLocalSpace = UseLocalSpace(&SceneProxy);

	FNiagaraMeshCommonParameters Params;
	Params.NiagaraParticleDataFloat	= ParticleMeshRenderData.ParticleFloatSRV;
	Params.NiagaraParticleDataHalf 	= ParticleMeshRenderData.ParticleHalfSRV;
	Params.NiagaraParticleDataInt 	= ParticleMeshRenderData.ParticleIntSRV;
	Params.NiagaraFloatDataStride 	= FMath::Max(ParticleMeshRenderData.ParticleFloatDataStride, ParticleMeshRenderData.ParticleHalfDataStride);
	Params.NiagaraIntDataStride 	= ParticleMeshRenderData.ParticleIntDataStride;
	
	Params.SortedIndices 			= ParticleMeshRenderData.ParticleSortedIndicesSRV;
	Params.SortedIndicesOffset 		= ParticleMeshRenderData.ParticleSortedIndicesOffset;	

	Params.SystemLWCTile			= SceneProxy.GetLWCRenderTile();
	Params.bLocalSpace 				= bUseLocalSpace;
	Params.AccurateMotionVectors	= bAccurateMotionVectors;
	Params.DeltaSeconds 			= View.Family->Time.GetDeltaWorldTimeSeconds();
	Params.FacingMode 				= (uint32)FacingMode;	

	Params.MeshScale				= MeshData.Scale;	
	Params.MeshRotation				= MeshData.Rotation;
	if (MeshData.PivotOffsetSpace == ENiagaraMeshPivotOffsetSpace::Mesh)
	{
		Params.MeshOffset 				= MeshData.PivotOffset;
		Params.bMeshOffsetIsWorldSpace	= false;
	}
	else
	{
		Params.MeshOffset 				= (FVector3f)ParticleMeshRenderData.WorldSpacePivotOffset;
		Params.bMeshOffsetIsWorldSpace	= true;
	}

	Params.bLockedAxisEnable			= bLockedAxisEnable;
	Params.LockedAxis 					= LockedAxis;
	Params.LockedAxisSpace				= (uint32)LockedAxisSpace;

	if (bUseLocalSpace)
	{
		Params.DefaultPosition = FVector4f(0.0f, 0.0f, 0.0f, 1.0f);
	}
	else
	{
		Params.DefaultPosition = FVector3f(SceneProxy.GetLocalToWorld().GetOrigin() - FVector(SceneProxy.GetLWCRenderTile()) * FLargeWorldRenderScalar::GetTileSize());
	}

	Params.DefaultPrevPosition			= Params.DefaultPosition;
	Params.DefaultVelocity 				= FVector3f(0.f, 0.0f, 0.0f);
	Params.DefaultPrevVelocity			= Params.DefaultVelocity;	
	Params.DefaultScale					= FVector3f(1.0f, 1.0f, 1.0f);
	Params.DefaultPrevScale 			= Params.DefaultScale;
	Params.DefaultRotation 				= FQuat4f(0.0f, 0.0f, 0.0f, 1.0f);
	Params.DefaultPrevRotation 			= Params.DefaultRotation;
	Params.DefaultCameraOffset 			= 0.0f;
	Params.DefaultPrevCameraOffset 		= Params.DefaultCameraOffset;

	Params.PrevScaleDataOffset 			= INDEX_NONE;
	Params.PrevRotationDataOffset 		= INDEX_NONE;
	Params.PrevPositionDataOffset 		= INDEX_NONE;
	Params.PrevVelocityDataOffset 		= INDEX_NONE;
	Params.PrevCameraOffsetDataOffset	= INDEX_NONE;

	if (SourceMode == ENiagaraRendererSourceDataMode::Particles)
	{
		TConstArrayView<FNiagaraRendererVariableInfo> VFVariables = ParticleMeshRenderData.RendererLayout->GetVFVariables_RenderThread();
		Params.ScaleDataOffset 			= VFVariables[ENiagaraMeshVFLayout::Scale].GetGPUOffset();
		Params.RotationDataOffset 		= VFVariables[ENiagaraMeshVFLayout::Rotation].GetGPUOffset();
		Params.PositionDataOffset 		= VFVariables[ENiagaraMeshVFLayout::Position].GetGPUOffset();
		Params.VelocityDataOffset 		= VFVariables[ENiagaraMeshVFLayout::Velocity].GetGPUOffset();
		Params.CameraOffsetDataOffset 	= VFVariables[ENiagaraMeshVFLayout::CameraOffset].GetGPUOffset();
		
		if (bAccurateMotionVectors)
		{
			Params.PrevScaleDataOffset			= SelectVFComponent(VFVariables, ENiagaraMeshVFLayout::PrevScale, ENiagaraMeshVFLayout::Scale);
			Params.PrevRotationDataOffset 		= SelectVFComponent(VFVariables, ENiagaraMeshVFLayout::PrevRotation, ENiagaraMeshVFLayout::Rotation);
			Params.PrevPositionDataOffset 		= SelectVFComponent(VFVariables, ENiagaraMeshVFLayout::PrevPosition, ENiagaraMeshVFLayout::Position);
			Params.PrevVelocityDataOffset 		= SelectVFComponent(VFVariables, ENiagaraMeshVFLayout::PrevVelocity, ENiagaraMeshVFLayout::Velocity);
			Params.PrevCameraOffsetDataOffset 	= SelectVFComponent(VFVariables, ENiagaraMeshVFLayout::PrevCameraOffset, ENiagaraMeshVFLayout::CameraOffset);
		}
	}
	else if (SourceMode == ENiagaraRendererSourceDataMode::Emitter) // Clear all these out because we will be using the defaults to specify them
	{
		Params.ScaleDataOffset 			= INDEX_NONE;
		Params.RotationDataOffset 		= INDEX_NONE;
		Params.PositionDataOffset 		= INDEX_NONE;
		Params.VelocityDataOffset 		= INDEX_NONE;
		Params.CameraOffsetDataOffset 	= INDEX_NONE;
	}
	else
	{
		// Unsupported source data mode detected
		check(SourceMode <= ENiagaraRendererSourceDataMode::Emitter);
	}

	if (bSetAnyBoundVars)
	{
		const FNiagaraDynamicDataMesh* DynamicDataMesh = ParticleMeshRenderData.DynamicDataMesh;
		const uint8* ParameterBoundData = DynamicDataMesh->ParameterDataBound.GetData();

		const int32 NumVFOffsets = bAccurateMotionVectors ? ENiagaraMeshVFLayout::Type::Num_Max : ENiagaraMeshVFLayout::Type::Num_Default;
		for (int32 i = 0; i < NumVFOffsets; i++)
		{
			if (VFBoundOffsetsInParamStore[i] != INDEX_NONE && DynamicDataMesh->ParameterDataBound.IsValidIndex(VFBoundOffsetsInParamStore[i]))
			{
				switch (i)
				{
				case ENiagaraMeshVFLayout::Type::Position:
					FMemory::Memcpy(&Params.DefaultPosition, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraMeshVFLayout::Type::Velocity:
					FMemory::Memcpy(&Params.DefaultVelocity, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraMeshVFLayout::Type::Scale:
					FMemory::Memcpy(&Params.DefaultScale, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraMeshVFLayout::Type::Rotation:
					FMemory::Memcpy(&Params.DefaultRotation, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FQuat4f));
					break;
				case ENiagaraMeshVFLayout::Type::CameraOffset:
					FMemory::Memcpy(&Params.DefaultCameraOffset, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;
				case ENiagaraMeshVFLayout::Type::PrevPosition:
					FMemory::Memcpy(&Params.DefaultPrevPosition, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraMeshVFLayout::Type::PrevScale:
					FMemory::Memcpy(&Params.DefaultPrevScale, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraMeshVFLayout::Type::PrevRotation:
					FMemory::Memcpy(&Params.DefaultPrevRotation, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FQuat4f));
					break;
				case ENiagaraMeshVFLayout::Type::PrevCameraOffset:
					FMemory::Memcpy(&Params.DefaultPrevCameraOffset, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;
				default:
					break;
				}
			}
			else
			{
				// If these prev values aren't bound to the host parameters, but their current values are, copy them
				switch (i)
				{
				case ENiagaraMeshVFLayout::Type::PrevPosition:
					Params.DefaultPrevPosition = Params.DefaultPosition;
					break;
				case ENiagaraMeshVFLayout::Type::PrevScale:
					Params.DefaultPrevScale = Params.DefaultScale;
					break;
				case ENiagaraMeshVFLayout::Type::PrevRotation:
					Params.DefaultPrevRotation = Params.DefaultRotation;
					break;
				case ENiagaraMeshVFLayout::Type::PrevCameraOffset:
					Params.DefaultPrevCameraOffset = Params.DefaultCameraOffset;
					break;
				default:
					break;
				}
			}
		}
	}

	return Params;
}

FNiagaraMeshUniformBufferRef FNiagaraRendererMeshes::CreateVFUniformBuffer(const FParticleMeshRenderData& ParticleMeshRenderData, const FNiagaraMeshCommonParameters& CommonParameters) const
{
	// Compute the vertex factory uniform buffer.
	FNiagaraMeshUniformParameters Params;
	FMemory::Memzero(&Params, sizeof(Params)); // Clear unset bytes

	Params.Common 								= CommonParameters;

	Params.MaterialParamValidMask 				= MaterialParamValidMask;
	Params.SubImageSize 						= FVector4f(SubImageSize.X, SubImageSize.Y, 1.0f / SubImageSize.X, 1.0f / SubImageSize.Y);
	Params.SubImageBlendMode 					= bSubImageBlend;

	Params.DefaultNormAge 						= 0.0f;
	Params.DefaultSubImage 						= 0.0f;
	Params.DefaultMatRandom 					= 0.0f;
	Params.DefaultColor							= FVector4f(1.0f, 1.0f, 1.0f, 1.0f);				
	Params.DefaultDynamicMaterialParameter0 	= FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
	Params.DefaultDynamicMaterialParameter1 	= FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
	Params.DefaultDynamicMaterialParameter2 	= FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
	Params.DefaultDynamicMaterialParameter3 	= FVector4f(1.0f, 1.0f, 1.0f, 1.0f);

	if (SourceMode == ENiagaraRendererSourceDataMode::Particles)
	{
		TConstArrayView<FNiagaraRendererVariableInfo> VFVariables = ParticleMeshRenderData.RendererLayout->GetVFVariables_RenderThread();
		Params.ColorDataOffset 			= VFVariables[ENiagaraMeshVFLayout::Color].GetGPUOffset();
		Params.MaterialRandomDataOffset = VFVariables[ENiagaraMeshVFLayout::MaterialRandom].GetGPUOffset();
		Params.NormalizedAgeDataOffset 	= VFVariables[ENiagaraMeshVFLayout::NormalizedAge].GetGPUOffset();
		Params.SubImageDataOffset 		= VFVariables[ENiagaraMeshVFLayout::SubImage].GetGPUOffset();
		Params.MaterialParamDataOffset 	= VFVariables[ENiagaraMeshVFLayout::DynamicParam0].GetGPUOffset();
		Params.MaterialParam1DataOffset	= VFVariables[ENiagaraMeshVFLayout::DynamicParam1].GetGPUOffset();
		Params.MaterialParam2DataOffset	= VFVariables[ENiagaraMeshVFLayout::DynamicParam2].GetGPUOffset();
		Params.MaterialParam3DataOffset	= VFVariables[ENiagaraMeshVFLayout::DynamicParam3].GetGPUOffset();
	}	
	else if (SourceMode == ENiagaraRendererSourceDataMode::Emitter) // Clear all these out because we will be using the defaults to specify them
	{
		Params.ColorDataOffset 			= INDEX_NONE;
		Params.MaterialRandomDataOffset = INDEX_NONE;
		Params.NormalizedAgeDataOffset 	= INDEX_NONE;
		Params.SubImageDataOffset 		= INDEX_NONE;
		Params.MaterialParamDataOffset 	= INDEX_NONE;
		Params.MaterialParam1DataOffset = INDEX_NONE;
		Params.MaterialParam2DataOffset = INDEX_NONE;
		Params.MaterialParam3DataOffset = INDEX_NONE;
	}
	else
	{
		// Unsupported source data mode detected
		check(SourceMode <= ENiagaraRendererSourceDataMode::Emitter);
	}

	Params.VertexFetch_Parameters = ParticleMeshRenderData.VertexFetch_Parameters;
	Params.VertexFetch_TexCoordBuffer = ParticleMeshRenderData.TexCoordBufferSrv;
	Params.VertexFetch_PackedTangentsBuffer = ParticleMeshRenderData.PackedTangentsBufferSrv;
	Params.VertexFetch_ColorComponentsBuffer = ParticleMeshRenderData.ColorComponentsBufferSrv;

	if (bSetAnyBoundVars)
	{
		const FNiagaraDynamicDataMesh* DynamicDataMesh = ParticleMeshRenderData.DynamicDataMesh;
		const uint8* ParameterBoundData = DynamicDataMesh->ParameterDataBound.GetData();

		const int32 NumVFOffsets = bAccurateMotionVectors ? ENiagaraMeshVFLayout::Type::Num_Max : ENiagaraMeshVFLayout::Type::Num_Default;
		for (int32 i = 0; i < NumVFOffsets; i++)
		{
			if (VFBoundOffsetsInParamStore[i] != INDEX_NONE && DynamicDataMesh->ParameterDataBound.IsValidIndex(VFBoundOffsetsInParamStore[i]))
			{
				switch (i)
				{
				case ENiagaraMeshVFLayout::Type::NormalizedAge:
					FMemory::Memcpy(&Params.DefaultNormAge, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;				
				case ENiagaraMeshVFLayout::Type::SubImage:
					FMemory::Memcpy(&Params.DefaultSubImage, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;
				case ENiagaraMeshVFLayout::Type::Color:
					FMemory::Memcpy(&Params.DefaultColor, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FLinearColor));
					break;
				case ENiagaraMeshVFLayout::Type::DynamicParam0:
					FMemory::Memcpy(&Params.DefaultDynamicMaterialParameter0, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FVector4f));
					Params.MaterialParamValidMask |= 0x000f;
					break;
				case ENiagaraMeshVFLayout::Type::DynamicParam1:
					FMemory::Memcpy(&Params.DefaultDynamicMaterialParameter1, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FVector4f));
					Params.MaterialParamValidMask |= 0x00f0;
					break;
				case ENiagaraMeshVFLayout::Type::DynamicParam2:
					FMemory::Memcpy(&Params.DefaultDynamicMaterialParameter2, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FVector4f));
					Params.MaterialParamValidMask |= 0x0f00;
					break;
				case ENiagaraMeshVFLayout::Type::DynamicParam3:
					FMemory::Memcpy(&Params.DefaultDynamicMaterialParameter3, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(FVector4f));
					Params.MaterialParamValidMask |= 0xf000;
					break;
				case ENiagaraMeshVFLayout::Type::MaterialRandom:
					FMemory::Memcpy(&Params.DefaultMatRandom, ParameterBoundData + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;
				case ENiagaraMeshVFLayout::Type::CustomSorting:
					// unsupported for now...
					break;
				default:
					break;
				}
			}
		}
	}

	return FNiagaraMeshUniformBufferRef::CreateUniformBufferImmediate(Params, UniformBuffer_SingleFrame);
}

FVector4f FNiagaraRendererMeshes::GetShaderLODScreenSize(const FSceneView& View, const FMeshData& MeshData)
{
	const FSceneView& LODView = GetLODView(View);
	const FMatrix& ProjMatrix = LODView.ViewMatrices.GetViewToClip();
	const float LODSize = MeshData.LODScreenSize.Z * FMath::Max(ProjMatrix.M[0][0], ProjMatrix.M[1][1]) * 0.5f;

	return FVector4f(
		FMath::Square(MeshData.LODScreenSize.X * 0.5f),
		MeshData.LODScreenSize.Y >= 0.0f ? FMath::Square(MeshData.LODScreenSize.Y * 0.5f) : UE_FLOAT_HUGE_DISTANCE,
		FMath::Square(LODSize),
		ProjMatrix.M[2][3] * LODView.LODDistanceFactor * MeshData.LODDistanceFactor
	);
}

void FNiagaraRendererMeshes::SetupElementForGPUScene(
	const FParticleMeshRenderData& ParticleMeshRenderData,
	const FNiagaraMeshCommonParameters& CommonParameters,
	const FNiagaraSceneProxy& SceneProxy,
	const FMeshData& MeshData,
	const FSceneView& View,
	uint32 NumInstances,
	bool bNeedsPrevTransform,
	FMeshBatchElement& OutMeshBatchElement
) const
{
	FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = SceneProxy.GetComputeDispatchInterface();
	check(ComputeDispatchInterface);

	FGPUSceneUpdateResource& GPUSceneRes = ParticleMeshRenderData.Collector->AllocateOneFrameResource<FGPUSceneUpdateResource>(FeatureLevel, bAccurateMotionVectors != 0);

	GPUSceneRes.DynamicPrimitiveData.EnableInstanceDynamicData(bNeedsPrevTransform);
	GPUSceneRes.DynamicPrimitiveData.SetNumInstanceCustomDataFloats(1);

	// Take the GPU write path if we are particles or we have options set that would involve duplicating a lot of code from shader into here
	const bool bWriteOnGpu = 
		SourceMode == ENiagaraRendererSourceDataMode::Particles ||
		FacingMode != ENiagaraMeshFacingMode::Default ||
		bLockedAxisEnable;
	if (bWriteOnGpu)
	{
		FMemory::Memzero(&GPUSceneRes.GPUWriteParams, sizeof(GPUSceneRes.GPUWriteParams));

		const FBox LocalBounds = ScaleFBox(MeshData.RenderableMesh->GetLocalBounds(), MeshData.Scale, MeshBoundsScale);

		GPUSceneRes.GPUWriteParams.Common 					= CommonParameters;
		GPUSceneRes.GPUWriteParams.ParticleCount 			= NumInstances;
		GPUSceneRes.GPUWriteParams.GPUParticleCountBuffer 	= GetSrvOrDefaultUInt(ComputeDispatchInterface->GetGPUInstanceCounterManager().GetInstanceCountBuffer());
		GPUSceneRes.GPUWriteParams.GPUParticleCountOffset 	= ParticleMeshRenderData.SourceParticleData->GetGPUInstanceCountBufferOffset();
		GPUSceneRes.GPUWriteParams.MeshIndex 				= MeshData.SourceMeshIndex;
		GPUSceneRes.GPUWriteParams.MeshIndexDataOffset 		= ParticleMeshRenderData.MeshIndexOffset;
		GPUSceneRes.GPUWriteParams.RendererVisibility 		= RendererVisibility;
		GPUSceneRes.GPUWriteParams.VisibilityTagDataOffset 	= ParticleMeshRenderData.RendererVisTagOffset;
		GPUSceneRes.GPUWriteParams.LocalBoundingCenter		= (FVector3f)LocalBounds.GetCenter();
		GPUSceneRes.GPUWriteParams.DistanceCullRangeSquared = DistanceCullRangeSquared;
		GPUSceneRes.GPUWriteParams.LODScreenSize			= GetShaderLODScreenSize(View, MeshData);
		GPUSceneRes.GPUWriteParams.bNeedsPrevTransform		= bNeedsPrevTransform ? 1 : 0;

		// We need to set this flag to force the system to always cull individual instances, because we may need to discard instances that are:
		// * Not tied to any live particles
		// * Are distance culled
		// * Are culled because of mismatched MeshIndex
		// * Are culled because of mismatched VisibilityTag
		OutMeshBatchElement.bForceInstanceCulling = true;

		// Force it to preserve instance order if we are sorted so that GPU Scene's instance culling doesn't scramble them
		OutMeshBatchElement.bPreserveInstanceOrder = ParticleMeshRenderData.bNeedsSort 
			// NOTE: For now, this feature is not supported for mobile platforms
			&& (FeatureLevel > ERHIFeatureLevel::ES3_1);

		GPUSceneRes.DynamicPrimitiveData.DataWriterGPU = FGPUSceneWriteDelegate::CreateLambda(		
			ParticleMeshRenderData.Collector,
			[&GPUSceneRes,ResouceName=SceneProxy.GetResourceName()](FRDGBuilder& GraphBuilder, const FGPUSceneWriteDelegateParams& Params)
			{
			#if !UE_BUILD_SHIPPING
				//-TEMP: We have a nullptr for the view in a very low repro situation this is to trap the system name to hopefully give some clues.
				if (Params.View == nullptr)
				{
					UE_LOGF(LogUObjectGlobals, Fatal, "Null view pointer in GPU scene update - %ls", *ResouceName.ToString());
				}
			#endif

				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				GPUSceneRes.GPUWriteParams.GPUSceneWriterParameters	= Params.GPUWriteParams;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				GPUSceneRes.GPUWriteParams.View						= Params.View->ViewUniformBuffer; // NOTE: Set here, not outside lambda
				GPUSceneRes.GPUWriteParams.Scene					= GetSceneUniformBufferRef(GraphBuilder, *Params.View);
				GPUSceneRes.GPUWriteParams.PrimitiveId 				= Params.PersistentPrimitiveId;
		
				FNiagaraGPUSceneUtils::AddUpdateMeshParticleInstancesPass(
					GraphBuilder,
					GPUSceneRes.GPUWriteParams,
					GPUSceneRes.FeatureLevel,
					GPUSceneRes.bPreciseMotionVectors
				);
			}
		);

		if (ParticleMeshRenderData.bIsGpuLowLatencyTranslucency)
		{
			GPUSceneRes.DynamicPrimitiveData.DataWriterGPUPass = EGPUSceneGPUWritePass::PostOpaqueRendering;
		}
	}
	else
	{
		// No point in writing the data for a single instance on the GPU - we can just set this info here
		const FVector OriginOffset = FVector(SceneProxy.GetLWCRenderTile()) * FLargeWorldRenderScalar::GetTileSize();
		
		FEmitterSourceInstanceData& EmitterSourceData = GPUSceneRes.EmitterSourceData;
		EmitterSourceData.CustomData = 0.0f;		

		FMatrix LocalToPrimitive = FTransform(
			FQuat(CommonParameters.DefaultRotation.GetNormalized() * CommonParameters.MeshRotation),
			FVector(CommonParameters.DefaultPosition + CommonParameters.MeshOffset),
			FVector(CommonParameters.DefaultScale * CommonParameters.MeshScale)
		).ToMatrixWithScale();

		if (!bLocalSpace)
		{
			// The instance transforms are expected to be in primitive local space, so we have to transform them from world space
			LocalToPrimitive.SetOrigin(LocalToPrimitive.GetOrigin() + OriginOffset);
			LocalToPrimitive *= SceneProxy.GetLocalToWorldInverse();				
		}

		EmitterSourceData.InstanceSceneData.LocalToPrimitive = LocalToPrimitive;

		GPUSceneRes.DynamicPrimitiveData.InstanceSceneData = TConstArrayView<FInstanceSceneData>(&EmitterSourceData.InstanceSceneData, 1);
		GPUSceneRes.DynamicPrimitiveData.InstanceCustomData = TConstArrayView<float>(&EmitterSourceData.CustomData, 1);
		
		if (bNeedsPrevTransform)
		{
			FMatrix PrevLocalToPrimitive;
			if (CommonParameters.AccurateMotionVectors)
			{
				PrevLocalToPrimitive = FTransform(
					FQuat(CommonParameters.DefaultPrevRotation.GetNormalized() * CommonParameters.MeshRotation),
					FVector(CommonParameters.DefaultPrevPosition + CommonParameters.MeshOffset),
					FVector(CommonParameters.DefaultPrevScale * CommonParameters.MeshScale)
				).ToMatrixWithScale();
			}
			else
			{
				PrevLocalToPrimitive = LocalToPrimitive;
			}

			if (!bLocalSpace)
			{
				// The instance transforms are expected to be in primitive local space, so we have to transform them from world space
				PrevLocalToPrimitive.SetOrigin(PrevLocalToPrimitive.GetOrigin() + OriginOffset);

				bool bHasPrecomputedVolumetricLightmap;
				FMatrix PreviousPrimitiveToWorld;
				int32 SingleCaptureIndex;
				bool bOutputVelocity;
				FPrimitiveSceneInfo* PrimitiveSceneInfo = SceneProxy.GetPrimitiveSceneInfo();
				SceneProxy.GetScene().GetPrimitiveUniformShaderParameters_RenderThread(PrimitiveSceneInfo, bHasPrecomputedVolumetricLightmap, PreviousPrimitiveToWorld, SingleCaptureIndex, bOutputVelocity);

				PrevLocalToPrimitive *= PreviousPrimitiveToWorld.Inverse();
			}

			EmitterSourceData.InstanceDynamicData.PrevLocalToPrimitive = PrevLocalToPrimitive;
			
			GPUSceneRes.DynamicPrimitiveData.InstanceDynamicData = TConstArrayView<FInstanceDynamicData>(&EmitterSourceData.InstanceDynamicData, 1);
		}
	}

	OutMeshBatchElement.DynamicPrimitiveData = &GPUSceneRes.DynamicPrimitiveData;
}

void FNiagaraRendererMeshes::CreateMeshBatchForSection(
	FRHICommandListBase& RHICmdList,
	const FParticleMeshRenderData& ParticleMeshRenderData,
	const FNiagaraMeshCommonParameters& CommonParameters,
	FMeshBatch& MeshBatch,
	FVertexFactory& VertexFactory,
	FMaterialRenderProxy& MaterialProxy,
	const FNiagaraSceneProxy& SceneProxy,
	const FMeshData& MeshData,
	const INiagaraRenderableMesh::FLODModelData& LODModel,
	const FStaticMeshSection& Section,
	const FSceneView& View,
	int32 ViewIndex,
	uint32 NumInstances,
	uint32 GPUCountBufferOffset,
	bool bIsWireframe,
	bool bIsInstancedStereo,
	bool bNeedsPrevTransform
) const
{
	MeshBatch.VertexFactory = &VertexFactory;
	MeshBatch.LCI = NULL;
	MeshBatch.ReverseCulling = SceneProxy.IsLocalToWorldDeterminantNegative();
	MeshBatch.CastShadow = SceneProxy.CastsDynamicShadow() && bCastShadows && Section.bCastShadow;
#if RHI_RAYTRACING
	MeshBatch.CastRayTracedShadow = SceneProxy.CastsDynamicShadow() && bCastShadows && Section.bCastShadow;
#endif
	MeshBatch.DepthPriorityGroup = (ESceneDepthPriorityGroup)SceneProxy.GetDepthPriorityGroup(&View);
#if WITH_EDITORONLY_DATA
	if (bIncludeInHitProxy == false)
	{
		MeshBatch.BatchHitProxyId = FHitProxyId::InvisibleHitProxyId;
	}
#endif

	const FBox LocalBounds = ScaleFBox(MeshData.RenderableMesh->GetLocalBounds(), FVector3f::OneVector, MeshBoundsScale);

	FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
	if (ParticleMeshRenderData.bUseGPUScene)
	{
		BatchElement.PrimitiveUniformBufferResource = SceneProxy.GetCustomUniformBufferResource(RHICmdList, IsMotionBlurEnabled(), LocalBounds);
	}
	else
	{
		BatchElement.PrimitiveUniformBuffer = SceneProxy.GetCustomUniformBuffer(RHICmdList, IsMotionBlurEnabled(), LocalBounds);
	}
	BatchElement.FirstIndex = 0;
	BatchElement.MinVertexIndex = 0;
	BatchElement.MaxVertexIndex = 0;
	BatchElement.NumInstances = NumInstances;

	if (ParticleMeshRenderData.bUseGPUScene)
	{
		SetupElementForGPUScene(
			ParticleMeshRenderData,
			CommonParameters,
			SceneProxy,
			MeshData,
			View,
			NumInstances,
			bNeedsPrevTransform,
			BatchElement
		);		
	}

	if (bIsWireframe)
	{
		if (LODModel.WireframeIndexBuffer != nullptr)
		{
			MeshBatch.Type = PT_LineList;
			MeshBatch.MaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			BatchElement.FirstIndex = 0;
			BatchElement.IndexBuffer = LODModel.WireframeIndexBuffer;
			BatchElement.NumPrimitives = LODModel.WireframeNumIndices / 2;
		}
		else
		{
			MeshBatch.Type = PT_TriangleList;
			MeshBatch.MaterialRenderProxy = &MaterialProxy;
			MeshBatch.bWireframe = true;
			BatchElement.FirstIndex = 0;
			BatchElement.IndexBuffer = LODModel.IndexBuffer;
			BatchElement.NumPrimitives = LODModel.NumIndices / 3;
		}
	}
	else
	{
		MeshBatch.Type = PT_TriangleList;
		MeshBatch.MaterialRenderProxy = &MaterialProxy;
		BatchElement.IndexBuffer = LODModel.IndexBuffer;
		BatchElement.FirstIndex = Section.FirstIndex;
		BatchElement.NumPrimitives = Section.NumTriangles;
	}

	if ((SourceMode == ENiagaraRendererSourceDataMode::Particles) && (GPUCountBufferOffset != INDEX_NONE))
	{
		// We need to use indirect draw args, because the number of actual instances is coming from the GPU
		auto ComputeDispatchInterface = SceneProxy.GetComputeDispatchInterface();
		check(ComputeDispatchInterface);

		auto& CountManager = ComputeDispatchInterface->GetGPUInstanceCounterManager();
		auto IndirectDraw = CountManager.AddDrawIndirect(
			RHICmdList,
			GPUCountBufferOffset,
			Section.NumTriangles * 3,
			Section.FirstIndex,
			bIsInstancedStereo,
			ParticleMeshRenderData.bNeedsCull && ParticleMeshRenderData.bSortCullOnGpu,
			ParticleMeshRenderData.SourceParticleData->GetGPUDataReadyStage()
		);

		BatchElement.NumPrimitives = 0;
		BatchElement.IndirectArgsBuffer = IndirectDraw.Buffer;
		BatchElement.IndirectArgsOffset = IndirectDraw.Offset;
	}
	else
	{
		check(BatchElement.NumPrimitives > 0);
	}

	MeshBatch.bCanApplyViewModeOverrides = true;
	MeshBatch.bUseWireframeSelectionColoring = SceneProxy.IsSelected();
}

void FNiagaraRendererMeshes::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector, const FNiagaraSceneProxy* SceneProxy) const
{
	check(SceneProxy);
	PARTICLE_PERF_STAT_CYCLES_RT(SceneProxy->GetProxyDynamicData().PerfStatsContext, GetDynamicMeshElements);
	FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();

	// Prepare our particle render data
	// This will also determine if we have anything to render
	// ENiagaraGpuComputeTickStage::Last is used as the GPU ready stage as we can support reading translucent data after PostRenderOpaque sims have run
	FParticleMeshRenderData ParticleMeshRenderData;
	PrepareParticleMeshRenderData(Collector.GetRHICommandList(), ParticleMeshRenderData, ViewFamily, Collector, DynamicDataRender, SceneProxy, false, ENiagaraGpuComputeTickStage::Last);

	if (ParticleMeshRenderData.SourceParticleData == nullptr || GbEnableNiagaraMeshRendering == false)
	{
		return;
	}

	if (ParticleMeshRenderData.bAllTranslucentMaterials && AreViewsRenderingOpaqueOnly(Views, VisibilityMap, ParticleMeshRenderData.bCanCastTranslucentShadows))
	{
		return;
	}

#if STATS
	FScopeCycleCounter EmitterStatsCounter(EmitterStatID);
#endif

	if (!CalculateMeshUsed(Collector.GetRHICommandList(), ParticleMeshRenderData))
	{
		return;
	}

	PrepareParticleRenderBuffers(RHICmdList, ParticleMeshRenderData, Collector.GetDynamicReadBuffer());

	// If mesh index comes from the parameter store grab the information now
	int32 EmitterModeMeshIndex = INDEX_NONE;
	if (EmitterMeshIndexOffset != INDEX_NONE && ParticleMeshRenderData.DynamicDataMesh->ParameterDataBound.IsValidIndex(EmitterMeshIndexOffset))
	{
		FMemory::Memcpy(&EmitterModeMeshIndex, ParticleMeshRenderData.DynamicDataMesh->ParameterDataBound.GetData() + EmitterMeshIndexOffset, sizeof(int32));
	}

	// Generate mesh batches per view
	const int32 NumViews = Views.Num();
	for (int32 ViewIndex = 0; ViewIndex < NumViews; ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];
			const bool bIsInstancedStereo = View->bIsInstancedStereoEnabled && IStereoRendering::IsStereoEyeView(*View);
			const bool bIsShadowView = View->GetDynamicMeshElementsShadowCullFrustum() != nullptr;

			if (bIsInstancedStereo && !IStereoRendering::IsAPrimaryView(*View))
			{
				// One eye renders everything, so we can skip non-primaries
				continue;
			}				

			if (SourceMode == ENiagaraRendererSourceDataMode::Emitter && bEnableCulling)
			{
				FVector ViewOrigin = View->ViewMatrices.GetViewOrigin();
				FVector RefPosition = SceneProxy->GetLocalToWorld().GetOrigin();

				double DistSquared = SceneProxy->GetProxyDynamicData().LODDistanceOverride >= 0.0f ? FMath::Square(SceneProxy->GetProxyDynamicData().LODDistanceOverride) : FVector::DistSquared(RefPosition, ViewOrigin);
				if (DistSquared < DistanceCullRange.X * DistanceCullRange.X || DistSquared > DistanceCullRange.Y * DistanceCullRange.Y)
				{
					// Distance cull the whole emitter
					continue;
				}
			}

			// Initialize sort parameters that are mesh/section invariant
			FNiagaraGPUSortInfo SortInfo;
			if (ParticleMeshRenderData.bNeedsSort || ParticleMeshRenderData.bNeedsCull)
			{
				InitializeSortInfo(ParticleMeshRenderData, *SceneProxy, *View, ViewIndex, bIsInstancedStereo, SortInfo);
			}

			FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = SceneProxy->GetComputeDispatchInterface();
			for (int32 MeshIndex = 0; MeshIndex < Meshes.Num(); ++MeshIndex)
			{
				const FMeshData& MeshData = Meshes[MeshIndex];
				if (MeshData.SourceMeshIndex > 0 && ParticleMeshRenderData.bAllowFirstMeshOnly)
				{
					break;
				}

				// Emitter mode mesh index binding
				if (EmitterModeMeshIndex != INDEX_NONE && MeshIndex != EmitterModeMeshIndex)
				{
					continue;
				}

				if (ParticleMeshRenderData.MeshUsed.Num() > 0 && !ParticleMeshRenderData.MeshUsed[Meshes[MeshIndex].SourceMeshIndex])
				{
					continue;
				}

				// If we can no perform sorting / culling on the GPU we need to skip meshes which are LODs
				if (MeshData.LODMode == ENiagaraMeshLODMode::PerParticle && !ParticleMeshRenderData.bAllowPerParticleMeshLODs)
				{
					if (MeshIndex != 0 && Meshes[MeshIndex - 1].SourceMeshIndex == Meshes[MeshIndex].SourceMeshIndex)
					{
						continue;
					}
				}

				INiagaraRenderableMesh::FLODModelData LODModel;
				if (MeshData.LODMode == ENiagaraMeshLODMode::ByComponentBounds)
				{
					const FBoxSphereBounds ProxyBounds = SceneProxy->GetBounds();
					const int32 LODLevel = MeshData.RenderableMesh->ComputeLOD(ProxyBounds.Origin, ProxyBounds.SphereRadius, *View, MeshData.LODDistanceFactor);
					MeshData.RenderableMesh->GetLODModelData(LODModel, LODLevel);
				}
				else if (MeshData.LODMode == ENiagaraMeshLODMode::ComponentOrigin)
				{
					const FBox MeshLocalBounds = ScaleFBox(MeshData.RenderableMesh->GetLocalBounds(), MeshData.Scale, MeshBoundsScale);
					const FBox MeshWorldBounds = MeshLocalBounds.TransformBy(SceneProxy->GetLocalToWorld());
					const int32 LODLevel = MeshData.RenderableMesh->ComputeLOD(MeshWorldBounds.GetCenter(), float(MeshWorldBounds.GetExtent().Size()), *View, MeshData.LODDistanceFactor);
					MeshData.RenderableMesh->GetLODModelData(LODModel, LODLevel);
				}
				else
				{
					MeshData.RenderableMesh->GetLODModelData(LODModel, MeshData.LODLevel);
				}

				if (LODModel.LODIndex == INDEX_NONE)
				{
					continue;
				}

				FMeshCollectorResources* CollectorResources = &Collector.AllocateOneFrameResource<FMeshCollectorResources>();

				// Get the next vertex factory to use
				// TODO: Find a way to safely pool these such that they won't be concurrently accessed by multiple views
				FNiagaraMeshVertexFactory& VertexFactory = CollectorResources->VertexFactory;
				if (ViewFamily.Scene != nullptr)
				{
					VertexFactory.SetNiagaraMeshFeatureLevel(ViewFamily.Scene->GetFeatureLevel());
				}
				VertexFactory.EnablePrimitiveIDElement(ParticleMeshRenderData.bUseGPUScene);
				VertexFactory.InitResource(RHICmdList);
				MeshData.RenderableMesh->SetupVertexFactory(RHICmdList, VertexFactory, LODModel);

				PreparePerMeshData(ParticleMeshRenderData, VertexFactory, *SceneProxy, MeshData);

				// Sort/Cull particles if needed.
				const uint32 NumInstances = PerformSortAndCull(RHICmdList, ParticleMeshRenderData, Collector.GetDynamicReadBuffer(), SortInfo, ComputeDispatchInterface, *View, MeshData);
				if ( NumInstances > 0 )
				{
					// Increment stats
					INC_DWORD_STAT_BY(STAT_NiagaraNumMeshVerts, NumInstances * LODModel.NumVertices);
					INC_DWORD_STAT_BY(STAT_NiagaraNumMeshes, NumInstances);

					FNiagaraMeshCommonParameters CommonParameters = CreateCommonShaderParams(ParticleMeshRenderData, *View, MeshData, *SceneProxy);
					FNiagaraMeshUniformBufferRef VFUniformBuffer = CreateVFUniformBuffer(ParticleMeshRenderData, CommonParameters);
					VertexFactory.SetUniformBuffer(VFUniformBuffer);
					CollectorResources->UniformBuffer = VFUniformBuffer;

					const bool bIsWireframe = AllowDebugViewmodes() && ViewFamily.EngineShowFlags.Wireframe;
					for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
					{
						const FStaticMeshSection& Section = LODModel.Sections[SectionIndex];
						const uint32 RemappedMaterialIndex = MeshData.MaterialRemapTable.IsValidIndex(Section.MaterialIndex) ? MeshData.MaterialRemapTable[Section.MaterialIndex] : INDEX_NONE;
						FMaterialRenderProxy* MaterialProxy = ParticleMeshRenderData.DynamicDataMesh->Materials.IsValidIndex(RemappedMaterialIndex) ? ParticleMeshRenderData.DynamicDataMesh->Materials[RemappedMaterialIndex] : UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
						if (Section.NumTriangles == 0 || MaterialProxy == nullptr)
						{
							//@todo. This should never occur, but it does occasionally.
							continue;
						}

						const FMaterial& Material = MaterialProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
						const bool bTranslucent = IsTranslucentBlendMode(Material);
						const bool bNeedsPrevTransform = !bTranslucent || Material.IsTranslucencyWritingVelocity();
						if (bTranslucent)
						{
							const bool bCanCastTranslucentShadows = SceneProxy->CastsVolumetricTranslucentShadow() || Material.GetCastDynamicShadowAsMasked();

							// If we are rendering opaque only we can skip this batch
							//-OPT: If we only have opaque materials we can skip earlier however due to RemappedMaterialIndex potentially being invalid this is tricky
							if (IsViewRenderingOpaqueOnly(View, bCanCastTranslucentShadows))
							{
								continue;
							}

							// This should never occur as the GPU data is considered not ready so low latency is disabled for regular scene captures
							if (!ensureMsgf(!View->bIsSceneCapture || !ParticleMeshRenderData.bUseGPUScene || !ParticleMeshRenderData.bIsGpuLowLatencyTranslucency, TEXT("Attemping to render translucent mesh particles with low latency into a scenecapture this is not supported.")))
							{
								continue;
							}
						}

						// When using GPU scene, the indirect draw is managed, so prevent creating indirect draw args
						uint32 GPUCountBufferOffset = INDEX_NONE;
						if (!ParticleMeshRenderData.bUseGPUScene)
						{
							GPUCountBufferOffset = SortInfo.CulledGPUParticleCountOffset != INDEX_NONE ? SortInfo.CulledGPUParticleCountOffset : ParticleMeshRenderData.SourceParticleData->GetGPUInstanceCountBufferOffset();
						}

						FMeshBatch& MeshBatch = Collector.AllocateMesh();
						CreateMeshBatchForSection(
							RHICmdList,
							ParticleMeshRenderData,
							CommonParameters,
							MeshBatch,
							VertexFactory,
							*MaterialProxy,
							*SceneProxy,
							MeshData,
							LODModel,
							Section,
							*View,
							ViewIndex,
							NumInstances,
							GPUCountBufferOffset,
							bIsWireframe,
							bIsInstancedStereo,
							bNeedsPrevTransform
						);

						Collector.AddMesh(ViewIndex, MeshBatch);
					}
				}
			}
		}
	}
}

#if RHI_RAYTRACING

void FNiagaraRendererMeshes::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FNiagaraSceneProxy* SceneProxy)
{
	if (!CVarRayTracingNiagaraMeshes.GetValueOnRenderThread())
	{
		return;
	}

	check(SceneProxy);

	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	const FSceneViewFamily* ViewFamily = nullptr;
	{
		const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
		checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

		const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

		ViewFamily = FirstActiveView->Family;
	}
	check(ViewFamily);

	FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();

	// Prepare our particle render data
	// This will also determine if we have anything to render
	// ENiagaraGpuComputeTickStage::PostInitViews is used as we need the data one InitViews is complete as the HWRT BVH will be generated before other sims have run
	FParticleMeshRenderData ParticleMeshRenderData;
	PrepareParticleMeshRenderData(RHICmdList, ParticleMeshRenderData, *ViewFamily, Collector, DynamicDataRender, SceneProxy, true, ENiagaraGpuComputeTickStage::PostInitViews);

	if (ParticleMeshRenderData.SourceParticleData == nullptr || Meshes.Num() == 0)
	{
		return;
	}

	// Disable sorting and culling as we manage this ourself
	ParticleMeshRenderData.bNeedsSort = false;
	ParticleMeshRenderData.bNeedsCull = false;
	ParticleMeshRenderData.bSortCullOnGpu = false;

	PrepareParticleRenderBuffers(RHICmdList, ParticleMeshRenderData, Collector.GetDynamicReadBuffer());

	FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = SceneProxy->GetComputeDispatchInterface();

	for (int32 MeshIndex = 0; MeshIndex < Meshes.Num(); ++MeshIndex)
	{
		// No binding for mesh index we only render the first mesh not all of them
		if ( (MeshIndex > 0) && (ParticleMeshIndexOffset == INDEX_NONE) )
		{
			break;
		}

		const FMeshData& MeshData = Meshes[MeshIndex];

		INiagaraRenderableMesh::FLODModelData LODModel;
		MeshData.RenderableMesh->GetRayTraceLODModelData(LODModel, MeshData.LODLevel);
		if (LODModel.LODIndex == INDEX_NONE || LODModel.RayTracingGeometry == nullptr)
		{
			continue;
		}

		// Build per-view FRayTracingInstances.
		// The static mesh RayTracingGeometry can be shared across views, but each view MUST have its own FMeshBatches and per-section FGPUSceneUpdateResource.
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if ((VisibilityMap & (1 << ViewIndex)) == 0)
			{
				continue;
			}

			const FSceneView* View = Views[ViewIndex];
			const bool bIsInstancedStereo = View->bIsInstancedStereoEnabled && IStereoRendering::IsStereoEyeView(*View);

			FRayTracingInstance RayTracingInstance;
			RayTracingInstance.Geometry = LODModel.RayTracingGeometry;

			FMeshCollectorResources* CollectorResources = &Collector.AllocateOneFrameResource<FMeshCollectorResources>();

			// Get the next vertex factory to use
			// TODO: Find a way to safely pool these such that they won't be concurrently accessed by multiple views
			FNiagaraMeshVertexFactory& VertexFactory = CollectorResources->VertexFactory;
			if (View->Family->Scene != nullptr)
			{
				VertexFactory.SetNiagaraMeshFeatureLevel(View->Family->Scene->GetFeatureLevel());
			}
			VertexFactory.EnablePrimitiveIDElement(ParticleMeshRenderData.bUseGPUScene);
			VertexFactory.InitResource(RHICmdList);
			MeshData.RenderableMesh->SetupRayTraceVertexFactory(RHICmdList, VertexFactory, LODModel);

			PreparePerMeshData(ParticleMeshRenderData, VertexFactory, *SceneProxy, MeshData);

			// Sort/Cull particles if needed. Sort & cull are forced off above so don't need to call InitializeSortInfo and this just sets the null sorted-indices SRV and returns NumInstances.
			FNiagaraGPUSortInfo SortInfo;
			const uint32 NumInstances = PerformSortAndCull(RHICmdList, ParticleMeshRenderData, Collector.GetDynamicReadBuffer(), SortInfo, ComputeDispatchInterface, *View, MeshData);
			if ( NumInstances == 0 )
			{
				continue;
			}

			FNiagaraMeshCommonParameters CommonParameters = CreateCommonShaderParams(ParticleMeshRenderData, *View, MeshData, *SceneProxy);
			FNiagaraMeshUniformBufferRef VFUniformBuffer = CreateVFUniformBuffer(ParticleMeshRenderData, CommonParameters);
			VertexFactory.SetUniformBuffer(VFUniformBuffer);
			CollectorResources->UniformBuffer = VFUniformBuffer;

			RayTracingInstance.Materials.Reserve(LODModel.Sections.Num());

			for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
			{
				const FStaticMeshSection& Section = LODModel.Sections[SectionIndex];
				if (Section.NumTriangles == 0)
				{
					continue;
				}

				const uint32 RemappedMaterialIndex = MeshData.MaterialRemapTable.IsValidIndex(Section.MaterialIndex) ? MeshData.MaterialRemapTable[Section.MaterialIndex] : INDEX_NONE;
				if (!ParticleMeshRenderData.DynamicDataMesh->Materials.IsValidIndex(RemappedMaterialIndex))
				{
					// This should never occur. Otherwise, the section data changed since initialization
					continue;
				}

				FMaterialRenderProxy* MaterialProxy = ParticleMeshRenderData.DynamicDataMesh->Materials[RemappedMaterialIndex];
				if (MaterialProxy == nullptr)
				{
					continue;
				}

				const FMaterialRenderProxy* ConstMaterialProxy = ParticleMeshRenderData.DynamicDataMesh->Materials[RemappedMaterialIndex];
				const FMaterial& Material = ConstMaterialProxy->GetMaterialWithFallback(FeatureLevel, ConstMaterialProxy);

				FMeshBatch MeshBatch;
				CreateMeshBatchForSection(
					RHICmdList,
					ParticleMeshRenderData,
					CommonParameters,
					MeshBatch,
					VertexFactory,
					*MaterialProxy,
					*SceneProxy,
					MeshData,
					LODModel,
					Section,
					*View,
					ViewIndex,
					NumInstances,
					INDEX_NONE,
					false, // bIsWireframe
					bIsInstancedStereo,
					false // bNeedsPrevTransform
				);
				MeshBatch.SegmentIndex = SectionIndex;
				MeshBatch.LODIndex = uint8(LODModel.LODIndex);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				MeshBatch.VisualizeLODIndex = uint8(LODModel.LODIndex);
#endif

				MeshBatch.bCanApplyViewModeOverrides = false;

				MeshBatch.Elements[0].VertexFactoryUserData = LODModel.VertexFactoryUserData;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				MeshBatch.Elements[0].VisualizeElementIndex = SectionIndex;
#endif

				RayTracingInstance.Materials.Add(MoveTemp(MeshBatch));
			}

			if (RayTracingInstance.Materials.Num() == 0 || LODModel.Sections.Num() != RayTracingInstance.Materials.Num())
			{
				// Material/section mismatch is data-driven and view-independent; subsequent views would fail the same check.
				continue;
			}

			// Emitter source mode? (View-independent, but kept inside the per-view loop for locality - the work is cheap.)
			const FMatrix LocalTransform(SceneProxy->GetLocalToWorld());
			const bool bUseLocalSpace = UseLocalSpace(SceneProxy);
			if (SourceMode == ENiagaraRendererSourceDataMode::Emitter)
			{
				// TODO: Get instance transform from GPU Scene when using ENiagaraRendererSourceDataMode::Emitter instead of using the logic below

				FVector Pos = bUseLocalSpace ? FVector::ZeroVector : LocalTransform.GetOrigin();
				FVector3f Scale{ 1.0f, 1.0f, 1.0f };
				FQuat Rot = FQuat::Identity;

				if (bSetAnyBoundVars)
				{
					const uint8* ParameterBoundData = ParticleMeshRenderData.DynamicDataMesh->ParameterDataBound.GetData();
					if (VFBoundOffsetsInParamStore[ENiagaraMeshVFLayout::Type::Position] != INDEX_NONE
						&& ParticleMeshRenderData.DynamicDataMesh->ParameterDataBound.IsValidIndex(VFBoundOffsetsInParamStore[ENiagaraMeshVFLayout::Type::Position]))
					{
						FVector3f TempPos;
						FMemory::Memcpy(&TempPos, ParameterBoundData + VFBoundOffsetsInParamStore[ENiagaraMeshVFLayout::Type::Position], sizeof(FVector3f));
						Pos = (FVector)TempPos;
						if (!bUseLocalSpace)
						{
							// Handle LWC
							Pos += FVector(SceneProxy->GetLWCRenderTile()) * FLargeWorldRenderScalar::GetTileSize();
						}
					}
					if (VFBoundOffsetsInParamStore[ENiagaraMeshVFLayout::Type::Scale] != INDEX_NONE
						&& ParticleMeshRenderData.DynamicDataMesh->ParameterDataBound.IsValidIndex(VFBoundOffsetsInParamStore[ENiagaraMeshVFLayout::Type::Scale]))
					{
						FMemory::Memcpy(&Scale, ParameterBoundData + VFBoundOffsetsInParamStore[ENiagaraMeshVFLayout::Type::Scale], sizeof(FVector3f));
					}
					if (VFBoundOffsetsInParamStore[ENiagaraMeshVFLayout::Type::Rotation] != INDEX_NONE
						&& ParticleMeshRenderData.DynamicDataMesh->ParameterDataBound.IsValidIndex(VFBoundOffsetsInParamStore[ENiagaraMeshVFLayout::Type::Rotation]))
					{
						FMemory::Memcpy(&Rot, ParameterBoundData + VFBoundOffsetsInParamStore[ENiagaraMeshVFLayout::Type::Rotation], sizeof(FVector4f));
					}
				}

				FVector4 Transform1 = FVector4(1.0f, 0.0f, 0.0f, Pos.X);
				FVector4 Transform2 = FVector4(0.0f, 1.0f, 0.0f, Pos.Y);
				FVector4 Transform3 = FVector4(0.0f, 0.0f, 1.0f, Pos.Z);

				FTransform RotationTransform(Rot.GetNormalized());
				FMatrix RotationMatrix = RotationTransform.ToMatrixWithScale();

				Transform1.X = RotationMatrix.M[0][0];
				Transform1.Y = RotationMatrix.M[0][1];
				Transform1.Z = RotationMatrix.M[0][2];

				Transform2.X = RotationMatrix.M[1][0];
				Transform2.Y = RotationMatrix.M[1][1];
				Transform2.Z = RotationMatrix.M[1][2];

				Transform3.X = RotationMatrix.M[2][0];
				Transform3.Y = RotationMatrix.M[2][1];
				Transform3.Z = RotationMatrix.M[2][2];

				FMatrix ScaleMatrix(FMatrix::Identity);
				ScaleMatrix.M[0][0] *= Scale.X;
				ScaleMatrix.M[1][1] *= Scale.Y;
				ScaleMatrix.M[2][2] *= Scale.Z;

				FMatrix InstanceTransform = FMatrix(FPlane(Transform1), FPlane(Transform2), FPlane(Transform3), FPlane(0.0, 0.0, 0.0, 1.0));
				InstanceTransform = InstanceTransform * ScaleMatrix;
				InstanceTransform = InstanceTransform.GetTransposed();

				if (bUseLocalSpace)
				{
					InstanceTransform = InstanceTransform * LocalTransform;
				}

				RayTracingInstance.InstanceTransforms.Add(InstanceTransform);
				RayTracingInstance.NumTransforms = 1;
			}
			else
			{
				// only set number of transforms so actual transform data is copied from GPU Scene
				RayTracingInstance.NumTransforms = NumInstances;
			}

			// Add MeshBatches to FMeshElementCollector so they're added to GPUScene if necessary
			// TODO: should probably do it automatically in Collector.AddRayTracingInstance(...)
			for (FMeshBatch& MeshBatch : RayTracingInstance.Materials)
			{
				Collector.AddMesh(ViewIndex, MeshBatch);
			}

			// Need to use MoveTemp(...) to avoid reallocs that would invalidate the mesh batch pointers cached above by AddMesh
			Collector.AddRayTracingInstance(ViewIndex, MoveTemp(RayTracingInstance));
		}
	}
}
#endif


FNiagaraDynamicDataBase* FNiagaraRendererMeshes::GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraGenMeshVertexData);

	const UNiagaraMeshRendererProperties* Properties = CastChecked<const UNiagaraMeshRendererProperties>(InProperties);
	
	if (!IsRendererEnabled(Properties, Emitter))
	{
		return nullptr;
	}

	if (Properties->bAllowInCullProxies == false)
	{
		check(Emitter);

		FNiagaraSystemInstance* Inst = Emitter->GetParentSystemInstance();
		check(Emitter->GetParentSystemInstance());

		//TODO: Probably should push some state into the system instance for this?
		const bool bIsCullProxy = Cast<UNiagaraCullProxyComponent>(Inst->GetAttachComponent()) != nullptr;
		if (bIsCullProxy)
		{
			return nullptr;
		}
	}

	
	FNiagaraDataBuffer* DataToRender = Emitter->GetParticleData().GetCurrentData();
	if (!DataToRender || 
		Meshes.Num() == 0 ||
		(SourceMode == ENiagaraRendererSourceDataMode::Particles && SimTarget != ENiagaraSimTarget::GPUComputeSim && DataToRender->GetNumInstances() == 0))
	{
		return nullptr;
	}

	// Bail if we have cached mesh render data for any meshes that are no longer valid
	for (const auto& MeshData : Meshes)
	{
		if (MeshData.RenderableMesh == nullptr)
		{
			return nullptr;
		}
	}

	FNiagaraDynamicDataMesh* DynamicData = new FNiagaraDynamicDataMesh(Emitter);
	DynamicData->SetMaterialRelevance(BaseMaterialRelevance_GT);

	DynamicData->Materials.Reset(BaseMaterials_GT.Num());
	for (UMaterialInterface* Mat : BaseMaterials_GT)
	{
		//In preparation for a material override feature, we pass our material(s) and relevance in via dynamic data.
		//The renderer ensures we have the correct usage and relevance for materials in BaseMaterials_GT.
		//Any override feature must also do the same for materials that are set.
		check(Mat->CheckMaterialUsage_Concurrent(MATUSAGE_NiagaraMeshParticles));
		DynamicData->Materials.Add(Mat->GetRenderProxy());
	}

	if (DynamicData)
	{
		const FNiagaraParameterStore& ParameterData = Emitter->GetRendererBoundVariables();
		DynamicData->DataInterfacesBound = ParameterData.GetDataInterfaces();
		DynamicData->ObjectsBound = ParameterData.GetUObjects();
		DynamicData->ParameterDataBound = ParameterData.GetParameterDataArray();
	}

	if (DynamicData && Properties->MaterialParameters.HasAnyBindings())
	{
		ProcessMaterialParameterBindings(Properties->MaterialParameters, Emitter, MakeArrayView(BaseMaterials_GT));
	}

	return DynamicData;
}

int FNiagaraRendererMeshes::GetDynamicDataSize()const
{
	uint32 Size = sizeof(FNiagaraDynamicDataMesh);
	return Size;
}

bool FNiagaraRendererMeshes::IsMaterialValid(const UMaterialInterface* Mat)const
{
	return Mat && Mat->CheckMaterialUsage_Concurrent(MATUSAGE_NiagaraMeshParticles);
}
