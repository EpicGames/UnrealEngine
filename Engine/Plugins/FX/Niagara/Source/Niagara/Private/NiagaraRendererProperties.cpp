// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraRendererProperties.h"
#include "NiagaraTypes.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSet.h"
#include "NiagaraConstants.h"
#include "NiagaraEmitter.h"
#include "NiagaraParameterBinding.h"
#include "NiagaraScriptSourceBase.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemImpl.h"

#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/Modules/NiagaraStatelessModule_DynamicMaterialParameters.h"

#include "Algo/Copy.h"
#include "Interfaces/ITargetPlatform.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialDomain.h"
#include "SceneInterface.h"
#include "SceneUtils.h"
#include "Styling/SlateIconFinder.h"

#if WITH_EDITORONLY_DATA
#include "Widgets/Images/SImage.h"
#include "AssetThumbnail.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraRendererProperties)

#define LOCTEXT_NAMESPACE "UNiagaraRendererProperties"

static int GNiagaraPSOPrecacheReverseCulling = 0;
static FAutoConsoleVariableRef CVarPSOPrecacheProjectedShadows(
	TEXT("fx.Niagara.PSOPrecache.ReverseCulling"),
	GNiagaraPSOPrecacheReverseCulling,
	TEXT("Also Precache PSOs with with reverse culling set when not 2 sided. (default 1)"),
	ECVF_Default
);

#if WITH_EDITORONLY_DATA
int32 GNiagaraRendererCookOutStaticEnabledBinding = 1;
static FAutoConsoleVariableRef CVarNiagaraRendererCookOutStaticEnabledBinding(
	TEXT("fx.Niagara.Renderer.CookOutStaticEnabledBinding"),
	GNiagaraRendererCookOutStaticEnabledBinding,
	TEXT("If none zero renderers with static variables used for enabled binding will cook out if they are not enabled."),
	ECVF_Scalability
);
#endif

namespace NiagaraRendererPropertiesPrivate
{

#if WITH_EDITORONLY_DATA

// Attempts to resolve a static variable across all the scripts
// If the state is undetermined (i.e. does not exist or is inconsistent across scripts) we will not return a value
static TOptional<bool> TryResolveStaticVariableBool(const UNiagaraEmitter* NiagaraEmitter, FNiagaraVariableBase BoundVariable)
{
	BoundVariable.SetType(BoundVariable.GetType().ToStaticDef());

	TOptional<bool> StaticValue;
	bool bHasConflictingValues = false;
	auto FindStaticValue =
		[&](UNiagaraScript* NiagaraScript)
		{
			if (NiagaraScript == nullptr || StaticValue.IsSet())
			{
				return;
			}

			for (const FNiagaraVariable& StaticVariable : NiagaraScript->GetVMExecutableData().StaticVariablesWritten)
			{
				if (static_cast<const FNiagaraVariableBase&>(StaticVariable) != BoundVariable)
				{
					continue;
				}
				if (StaticValue.IsSet())
				{
					bHasConflictingValues = StaticValue.GetValue() != StaticVariable.GetValue<bool>();
				}
				else
				{
					StaticValue = StaticVariable.GetValue<bool>();
				}
				break;
			}
		};

	NiagaraEmitter->ForEachVersionData(
		[&](const FVersionedNiagaraEmitterData& EmitterData)
		{
			EmitterData.ForEachScript(FindStaticValue);
		}
	);

	if (UNiagaraSystem* NiagaraSystem = NiagaraEmitter->GetTypedOuter<UNiagaraSystem>())
	{
		FindStaticValue(NiagaraSystem->GetSystemSpawnScript());
		FindStaticValue(NiagaraSystem->GetSystemUpdateScript());
	}

	if (bHasConflictingValues)
	{
		StaticValue.Reset();
	}

	return StaticValue;
}

#endif

void MarkAndRenameMaterialForGarbage(UMaterialInstanceConstant* MIC)
{
	if (MIC)
	{
		const ERenameFlags RenameFlags = REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors;
		UPackage* TransientPackage = GetTransientPackage();

		MIC->MarkAsGarbage();
		MIC->Rename(nullptr, TransientPackage, RenameFlags);

#if WITH_EDITORONLY_DATA
		if (UMaterialInterfaceEditorOnlyData* EditorOnlyData = MIC->GetEditorOnlyData())
		{
			EditorOnlyData->MarkAsGarbage();
			EditorOnlyData->Rename(nullptr, TransientPackage, RenameFlags);
		}
#endif
	}
}

} // NiagaraRendererPropertiesPrivate

FNiagaraRendererVariableInfo::FNiagaraRendererVariableInfo(const FNiagaraDataSetCompiledData* CompiledData, const FNiagaraVariableBase& Variable)
{
	const int32 VariableIndex = CompiledData->Variables.IndexOfByPredicate(
		[&Variable](const FNiagaraVariable& InVariable)
		{
			return InVariable.GetName() == Variable.GetName();
		}
	);

	if (VariableIndex != INDEX_NONE)
	{
		const FNiagaraVariableLayoutInfo& LayoutInfo = CompiledData->VariableLayouts[VariableIndex];
		if (LayoutInfo.GetNumFloatComponents() > 0)
		{
			check(LayoutInfo.GetNumHalfComponents() + LayoutInfo.GetNumInt32Components() == 0)
			DatasetOffset	= LayoutInfo.GetFloatComponentStart();
			GPUBufferOffset	= LayoutInfo.GetFloatComponentStart();
			NumComponents	= LayoutInfo.GetNumFloatComponents();
			bUpload			= false;
			bHalfType		= false;
		}
		else if (LayoutInfo.GetNumHalfComponents() > 0)
		{
			check(LayoutInfo.GetNumFloatComponents() + LayoutInfo.GetNumInt32Components() == 0)
			DatasetOffset	= LayoutInfo.GetHalfComponentStart();
			GPUBufferOffset	= LayoutInfo.GetHalfComponentStart();
			NumComponents	= LayoutInfo.GetNumHalfComponents();
			bUpload			= false;
			bHalfType		= true;
		}
		else if (LayoutInfo.GetNumInt32Components() > 0)
		{
			check(LayoutInfo.GetNumInt32Components() + LayoutInfo.GetNumHalfComponents() == 0)
			DatasetOffset	= LayoutInfo.GetInt32ComponentStart();
			GPUBufferOffset	= LayoutInfo.GetInt32ComponentStart();
			NumComponents	= LayoutInfo.GetNumInt32Components();
			bUpload			= false;
			bHalfType		= false;
		}
	}
}

void FNiagaraRendererLayout::Initialize(int32 NumVariables)
{
	VFVariables_GT.Reset(NumVariables);
	VFVariables_GT.AddDefaulted(NumVariables);
	TotalFloatComponents_GT = 0;
	TotalHalfComponents_GT = 0;
}

bool FNiagaraRendererLayout::SetVariable(const FNiagaraDataSetCompiledData* CompiledData, const FNiagaraVariableBase& Variable, int32 VFVarOffset)
{
	// No compiled data, nothing to bind
	if (CompiledData == nullptr)
	{
		return false;
	}

	// use the DataSetVariable to figure out the information about the data that we'll be sending to the renderer
	const int32 VariableIndex = CompiledData->Variables.IndexOfByPredicate(
		[&](const FNiagaraVariable& InVariable)
		{
			return InVariable.GetName() == Variable.GetName();
		}
	);
	if (VariableIndex == INDEX_NONE)
	{
		VFVariables_GT[VFVarOffset] = FNiagaraRendererVariableInfo();
		return false;
	}

	const FNiagaraVariable& DataSetVariable = CompiledData->Variables[VariableIndex];
	const FNiagaraTypeDefinition& VarType = DataSetVariable.GetType();

	const bool bHalfVariable = VarType == FNiagaraTypeDefinition::GetHalfDef()
		|| VarType == FNiagaraTypeDefinition::GetHalfVec2Def()
		|| VarType == FNiagaraTypeDefinition::GetHalfVec3Def()
		|| VarType == FNiagaraTypeDefinition::GetHalfVec4Def();


	const FNiagaraVariableLayoutInfo& DataSetVariableLayout = CompiledData->VariableLayouts[VariableIndex];
	const int32 VarSize = bHalfVariable ? sizeof(FFloat16) : sizeof(float);
	const int32 NumComponents = DataSetVariable.GetSizeInBytes() / VarSize;
	const int32 Offset = bHalfVariable ? DataSetVariableLayout.GetHalfComponentStart() : DataSetVariableLayout.GetFloatComponentStart();
	uint16& TotalVFComponents = bHalfVariable ? TotalHalfComponents_GT : TotalFloatComponents_GT;

	int32 GPULocation = INDEX_NONE;
	bool bUpload = true;
	if (Offset != INDEX_NONE)
	{
		if (FNiagaraRendererVariableInfo* ExistingVarInfo = VFVariables_GT.FindByPredicate([&](const FNiagaraRendererVariableInfo& VarInfo) { return VarInfo.DatasetOffset == Offset && VarInfo.bHalfType == bHalfVariable; }))
		{
			//Don't need to upload this var again if it's already been uploaded for another var info. Just point to that.
			//E.g. when custom sorting uses age.
			GPULocation = ExistingVarInfo->GPUBufferOffset;
			bUpload = false;
		}
		else
		{
			//For CPU Sims we pack just the required data tightly in a GPU buffer we upload. For GPU sims the data is there already so we just provide the real data location.
			GPULocation = CompiledData->SimTarget == ENiagaraSimTarget::CPUSim ? TotalVFComponents : Offset;
			check(static_cast<int32>(TotalVFComponents) + NumComponents <= TNumericLimits<uint16>::Max());
			TotalVFComponents += static_cast<uint16>(NumComponents);
		}
	}

	VFVariables_GT[VFVarOffset] = FNiagaraRendererVariableInfo(Offset, GPULocation, NumComponents, bUpload, bHalfVariable);

	return Offset != INDEX_NONE;
}


bool FNiagaraRendererLayout::SetVariableFromBinding(const FNiagaraDataSetCompiledData* CompiledData, const FNiagaraVariableAttributeBinding& VariableBinding, int32 VFVarOffset)
{
	if (VariableBinding.IsParticleBinding())
	{
		return SetVariable(CompiledData, VariableBinding.GetDataSetBindableVariable(), VFVarOffset);
	}
	return false;
}

void FNiagaraRendererLayout::Finalize()
{
	ENQUEUE_RENDER_COMMAND(NiagaraFinalizeLayout)
	(
		[this, VFVariables=VFVariables_GT,TotalFloatComponents=TotalFloatComponents_GT, TotalHalfComponents=TotalHalfComponents_GT](FRHICommandListImmediate& RHICmdList) mutable
		{
			VFVariables_RT = MoveTemp(VFVariables);
			TotalFloatComponents_RT = TotalFloatComponents;
			TotalHalfComponents_RT = TotalHalfComponents;
		}
	);
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraRendererMaterialParameters::ConditionalPostLoad()
{
	for (const FNiagaraRendererMaterialTextureParameter& TextureParameter : TextureParameters)
	{
		if (TextureParameter.Texture)
		{
			TextureParameter.Texture->ConditionalPostLoad();
		}
	}
}

#if WITH_EDITORONLY_DATA
void FNiagaraRendererMaterialParameters::RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode SourceMode)
{
	RenameVariable(OldVariable, NewVariable, FVersionedNiagaraEmitterBase(InEmitter.Emitter, InEmitter.Version), SourceMode);
}

void FNiagaraRendererMaterialParameters::RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode SourceMode)
{
	RemoveVariable(OldVariable, FVersionedNiagaraEmitterBase(InEmitter.Emitter, InEmitter.Version), SourceMode);
}

void FNiagaraRendererMaterialParameters::RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode SourceMode)
{
	for (FNiagaraMaterialAttributeBinding& Binding : AttributeBindings)
	{
		Binding.RenameVariableIfMatching(OldVariable, NewVariable, InEmitter.Emitter, SourceMode);
	}

	if ( OldVariable.GetType() == FNiagaraTypeDefinition::GetBoolDef().ToStaticDef() )
	{
		for (FNiagaraRendererMaterialStaticBoolParameter& Binding : StaticBoolParameters)
		{
			if (Binding.StaticVariableName == OldVariable.GetName())
			{
				Binding.StaticVariableName = NewVariable.GetName();
			}
		}
	}
}

void FNiagaraRendererMaterialParameters::RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode SourceMode)
{
	for (FNiagaraMaterialAttributeBinding& Binding : AttributeBindings)
	{
		if (Binding.Matches(OldVariable, InEmitter.Emitter, SourceMode))
		{
			Binding.NiagaraVariable = FNiagaraVariable();
			Binding.CacheValues(InEmitter.Emitter);
		}
	}

	if (OldVariable.GetType() == FNiagaraTypeDefinition::GetBoolDef().ToStaticDef())
	{
		for (FNiagaraRendererMaterialStaticBoolParameter& Binding : StaticBoolParameters)
		{
			if (Binding.StaticVariableName == OldVariable.GetName())
			{
				Binding.StaticVariableName = FName();
			}
		}
	}
}

void FNiagaraRendererMaterialParameters::GetFeedback(TArrayView<UMaterialInterface*> Materials, TArray<FNiagaraRendererFeedback>& OutWarnings) const
{
	TArray<bool> AttributeBindingsValid;
	TArray<bool> ScalarParametersValid;
	TArray<bool> VectorParametersValid;
	TArray<bool> TextureParametersValid;
	AttributeBindingsValid.AddDefaulted(AttributeBindings.Num());
	ScalarParametersValid.AddDefaulted(ScalarParameters.Num());
	VectorParametersValid.AddDefaulted(VectorParameters.Num());
	TextureParametersValid.AddDefaulted(TextureParameters.Num());

	TArray<FMaterialParameterInfo> TempParameterInfo;
	TArray<FGuid> TempParameterIds;
	auto ContainsParameter =
		[&TempParameterInfo](FName InName)
		{
			for ( const FMaterialParameterInfo& Parameter : TempParameterInfo )
			{
				if (Parameter.Name == InName)
				{
					return true;
				}
			}
			return false;
		};

	for (UMaterialInterface* Material : Materials)
	{
		if (Material == nullptr)
		{
			continue;
		}
		
		if (AttributeBindingsValid.Num() > 0 || ScalarParametersValid.Num() > 0)
		{
			Material->GetAllScalarParameterInfo(TempParameterInfo, TempParameterIds);
			for (int32 i = 0; i < AttributeBindings.Num(); ++i)
			{
				AttributeBindingsValid[i] |= ContainsParameter(AttributeBindings[i].MaterialParameterName);
			}
			for (int32 i = 0; i < ScalarParameters.Num(); ++i)
			{
				ScalarParametersValid[i] |= ContainsParameter(ScalarParameters[i].MaterialParameterName);
			}
		}
		if (AttributeBindingsValid.Num() > 0 || VectorParametersValid.Num() > 0)
		{
			Material->GetAllVectorParameterInfo(TempParameterInfo, TempParameterIds);
			for (int32 i = 0; i < AttributeBindings.Num(); ++i)
			{
				AttributeBindingsValid[i] |= ContainsParameter(AttributeBindings[i].MaterialParameterName);
			}
			for (int32 i = 0; i < VectorParameters.Num(); ++i)
			{
				VectorParametersValid[i] |= ContainsParameter(VectorParameters[i].MaterialParameterName);
			}
		}
		if (AttributeBindingsValid.Num() > 0)
		{
			Material->GetAllDoubleVectorParameterInfo(TempParameterInfo, TempParameterIds);
			for (int32 i = 0; i < AttributeBindings.Num(); ++i)
			{
				AttributeBindingsValid[i] |= ContainsParameter(AttributeBindings[i].MaterialParameterName);
			}
		}
		if (AttributeBindingsValid.Num() > 0 || TextureParametersValid.Num() > 0)
		{
			Material->GetAllTextureParameterInfo(TempParameterInfo, TempParameterIds);
			for (int32 i=0; i < AttributeBindings.Num(); ++i)
			{
				AttributeBindingsValid[i] |= ContainsParameter(AttributeBindings[i].MaterialParameterName);
			}
			for (int32 i = 0; i < TextureParameters.Num(); ++i)
			{
				TextureParametersValid[i] |= ContainsParameter(TextureParameters[i].MaterialParameterName);
			}
		}
	}

	for (int32 i=0; i < AttributeBindingsValid.Num(); ++i)
	{
		if (AttributeBindingsValid[i] == false)
		{
			OutWarnings.Emplace(
				FText::Format(LOCTEXT("AttributeBindingMissingDesc", "AttributeBinding '{0}' could not be found in the renderer materials.  We will still create the MID which may be unnecessary."), FText::FromName(AttributeBindings[i].MaterialParameterName)),
				FText::Format(LOCTEXT("AttributeBindingMissing", "AttributeBinding '{0}' not found on materials."), FText::FromName(AttributeBindings[i].MaterialParameterName))
			);
		}
	}
	for (int32 i = 0; i < ScalarParametersValid.Num(); ++i)
	{
		if (ScalarParametersValid[i] == false)
		{
			OutWarnings.Emplace(
				FText::Format(LOCTEXT("ScalarParameterMissingDesc", "ScalarParameter '{0}' could not be found in the renderer materials.  We will still create the MID which may be unnecessary."), FText::FromName(ScalarParameters[i].MaterialParameterName)),
				FText::Format(LOCTEXT("ScalarParameterMissing", "ScalarParameter '{0}' not found on materials."), FText::FromName(ScalarParameters[i].MaterialParameterName))
			);
		}
	}
	for (int32 i = 0; i < VectorParametersValid.Num(); ++i)
	{
		if (VectorParametersValid[i] == false)
		{
			OutWarnings.Emplace(
				FText::Format(LOCTEXT("VectorParameterMissingDesc", "VectorParameter '{0}' could not be found in the renderer materials.  We will still create the MID which may be unnecessary."), FText::FromName(VectorParameters[i].MaterialParameterName)),
				FText::Format(LOCTEXT("VectorParameterMissing", "VectorParameter '{0}' not found on materials."), FText::FromName(VectorParameters[i].MaterialParameterName))
			);
		}
	}
	for (int32 i = 0; i < TextureParametersValid.Num(); ++i)
	{
		if (TextureParametersValid[i] == false)
		{
			OutWarnings.Emplace(
				FText::Format(LOCTEXT("TextureParameterMissingDesc", "TextureParameter '{0}' could not be found in the renderer materials.  We will still create the MID which may be unnecessary."), FText::FromName(TextureParameters[i].MaterialParameterName)),
				FText::Format(LOCTEXT("TextureParameterMissing", "TextureParameter '{0}' not found on materials."), FText::FromName(TextureParameters[i].MaterialParameterName))
			);
		}
	}
}
#endif //WITH_EDITORONLY_DATA

//////////////////////////////////////////////////////////////////////////

#if WITH_EDITORONLY_DATA
bool UNiagaraRendererProperties::IsSupportedVariableForBinding(const FNiagaraVariableBase& InSourceForBinding, const FName& InTargetBindingName) const
{
	if (InTargetBindingName == GET_MEMBER_NAME_CHECKED(UNiagaraRendererProperties, RendererEnabledBinding))
	{
		return
			InSourceForBinding.IsInNameSpace(FNiagaraConstants::UserNamespace) ||
			InSourceForBinding.IsInNameSpace(FNiagaraConstants::SystemNamespace) ||
			InSourceForBinding.IsInNameSpace(FNiagaraConstants::EmitterNamespace);
	}

	const ENiagaraRendererSourceDataMode CurrentSourceMode = GetCurrentSourceMode();
	return
		(CurrentSourceMode == ENiagaraRendererSourceDataMode::Particles && InSourceForBinding.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString)) ||
		InSourceForBinding.IsInNameSpace(FNiagaraConstants::UserNamespaceString) ||
		InSourceForBinding.IsInNameSpace(FNiagaraConstants::SystemNamespaceString) ||
		InSourceForBinding.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString);
}

void UNiagaraRendererProperties::RenameEmitter(const FName& InOldName, const UNiagaraEmitter* InRenamedEmitter)
{
	const ENiagaraRendererSourceDataMode SourceMode = GetCurrentSourceMode();
	UpdateSourceModeDerivates(SourceMode);
	if (InRenamedEmitter)
	{
		FNiagaraParameterBinding::ForEachRenameEmitter(this, InRenamedEmitter->GetUniqueEmitterName());
	}
}

TArray<FNiagaraVariable> UNiagaraRendererProperties::GetBoundAttributes() const
{
	TArray<FNiagaraVariable> BoundAttributes;
	BoundAttributes.Reserve(AttributeBindings.Num() + 1);

	for (const FNiagaraVariableAttributeBinding* AttributeBinding : AttributeBindings)
	{
		FNiagaraVariable BoundAttribute = GetBoundAttribute(AttributeBinding);
		if (BoundAttribute.IsValid())
		{
			BoundAttributes.Add(BoundAttribute);
		}
	}

	if (RendererEnabledBinding.IsValid())
	{
		BoundAttributes.AddUnique(RendererEnabledBinding.GetParamMapBindableVariable());
	}

	return BoundAttributes;
}

void UNiagaraRendererProperties::ChangeToPositionBinding(FNiagaraVariableAttributeBinding& Binding)
{
	if (Binding.GetType() == FNiagaraTypeDefinition::GetVec3Def())
	{
		FNiagaraVariable NewVarType(FNiagaraTypeDefinition::GetPositionDef(), Binding.GetParamMapBindableVariable().GetName());
		Binding = FNiagaraConstants::GetAttributeDefaultBinding(NewVarType);
	}
}

bool UNiagaraRendererProperties::BuildMaterialStaticParameterSet(const FNiagaraRendererMaterialParameters& MaterialParameters, const UMaterialInterface* Material, FStaticParameterSet& StaticParameterSet) const
{
	StaticParameterSet.Empty();

	UNiagaraSystem* NiagaraSystem = GetTypedOuter<UNiagaraSystem>();
	if (NiagaraSystem == nullptr)
	{
		return false;
	}
	UNiagaraEmitter* NiagaraEmitter = GetTypedOuter<UNiagaraEmitter>();

	TArray<FMaterialParameterInfo> AllStaticSwitchParameterInfos;
	{
		TArray<FGuid> ParameterGuids;
		Material->GetAllStaticSwitchParameterInfo(AllStaticSwitchParameterInfos, ParameterGuids);
	}

	bool bModified = false;

	auto SetMaterialStaticParameter =
		[&](FName ParameterName, bool bParameterValue)
		{
			for (const FMaterialParameterInfo& ParameterInfo : AllStaticSwitchParameterInfos)
			{
				if (ParameterInfo.Name != ParameterName)
				{
					continue;
				}

				FStaticSwitchParameter* StaticParameter = StaticParameterSet.StaticSwitchParameters.FindByPredicate(
					[ParameterInfo](const FStaticSwitchParameter& StaticParameter)
					{
						return StaticParameter.ParameterInfo == ParameterInfo;
					}
				);

				if (StaticParameter == nullptr)
				{
					FGuid ParameterGuid;
					bool bDefaultValue = false;
					if (Material->GetStaticSwitchParameterDefaultValue(ParameterInfo, bDefaultValue, ParameterGuid))
					{
						if (bDefaultValue != bParameterValue)
						{
							FMaterialParameterMetadata ParameterMetadata(bParameterValue);
							ParameterMetadata.ExpressionGuid = ParameterGuid;

							StaticParameterSet.SetParameterValue(ParameterInfo, ParameterMetadata, EMaterialSetParameterValueFlags::None);
							bModified = true;
						}
					}
				}
				else
				{
					if (StaticParameter && StaticParameter->Value != bParameterValue)
					{
						StaticParameter->Value = bParameterValue;
						bModified = true;
					}
				}
			}
		};

	for (const FNiagaraRendererMaterialStaticBoolParameter& ParameterBinding : MaterialParameters.StaticBoolParameters)
	{
		if (ParameterBinding.StaticValue.IsSet())
		{
			SetMaterialStaticParameter(ParameterBinding.MaterialParameterName, ParameterBinding.StaticValue.GetValue());
		}
		else
		{
			NiagaraSystem->ForEachScript(
				[&](UNiagaraScript* NiagaraScript)
				{
					for (const FNiagaraVariable& StaticVariable : NiagaraScript->GetVMExecutableData().StaticVariablesWritten)
					{
						if (StaticVariable.GetType() != FNiagaraTypeDefinition::GetBoolDef().ToStaticDef())
						{
							continue;
						}

						FNiagaraVariableBase ResolvedStaticVariable = StaticVariable;
						if (NiagaraEmitter)
						{
							ResolvedStaticVariable.ReplaceRootNamespace(NiagaraEmitter->GetUniqueEmitterName(), FNiagaraConstants::EmitterNamespaceString);
						}
						if (ResolvedStaticVariable.GetName() != ParameterBinding.StaticVariableName)
						{
							continue;
						}

						SetMaterialStaticParameter(ParameterBinding.MaterialParameterName, StaticVariable.GetValue<bool>());
					}
				}
			);
		}
	}

	return bModified;
}

bool UNiagaraRendererProperties::UpdateMaterialStaticParameters(const FNiagaraRendererMaterialParameters& MaterialParameters, UMaterialInstanceConstant* MIC)
{
	FStaticParameterSet StaticParameterSet;
	if (BuildMaterialStaticParameterSet(MaterialParameters, MIC, StaticParameterSet))
	{
		MIC->UpdateStaticPermutation(StaticParameterSet);
		return true;
	}

	return false;
}

void UNiagaraRendererProperties::UpdateMaterialParametersMIC(const FNiagaraRendererMaterialParameters& MaterialParameters, TObjectPtr<UMaterialInterface>& InOutMaterial, TObjectPtr<UMaterialInstanceConstant>& InOutMIC, FMaterialUpdateContext* OptionalMaterialUpdateContext)
{
	TArray<UMaterialInterface*> LocalMaterials = { InOutMaterial };
	TArray<TObjectPtr<UMaterialInstanceConstant>> LocalMICs = { InOutMIC };

	UpdateMaterialParametersMIC(MaterialParameters, LocalMaterials, LocalMICs, OptionalMaterialUpdateContext);

	InOutMIC = LocalMICs.IsEmpty() ? nullptr : LocalMICs[0];
}

void UNiagaraRendererProperties::UpdateMaterialParametersMIC(const FNiagaraRendererMaterialParameters& MaterialParameters, TArrayView<UMaterialInterface*> Materials, TArray<TObjectPtr<UMaterialInstanceConstant>>& InOutMICs, FMaterialUpdateContext* OptionalMaterialUpdateContext)
{
	// Create a pool of unique MICs that we can potentially reuse
	TArray<UMaterialInstanceConstant*> MICPool;
	MICPool.Reserve(InOutMICs.Num());
	for (UMaterialInstanceConstant* MIC : InOutMICs)
	{
		if (MIC != nullptr)
		{
			MICPool.AddUnique(MIC);
		}
	}
	InOutMICs.Reset(0);

	ON_SCOPE_EXIT
	{
		// Garabge the MICs we are no longer going to use
		for (int i = 0; i < MICPool.Num(); ++i)
		{
			NiagaraRendererPropertiesPrivate::MarkAndRenameMaterialForGarbage(MICPool[i]);
		}
	};

	if (Materials.Num() == 0 || MaterialParameters.StaticBoolParameters.Num() == 0)
	{
		return;
	}

	FMaterialUpdateContext DefaultMaterialUpdateContext(FMaterialUpdateContext::EOptions::SkipRecacheStaticPermutations | FMaterialUpdateContext::EOptions::SkipUpdateStaticDrawLists);

	// Loop over each material to see if we need to generate a MIC for it
	for ( int i=0; i < Materials.Num(); ++i )
	{
		UMaterialInterface* Material = Materials[i];
		if ( Material == nullptr )
		{
			continue;
		}

		FStaticParameterSet MaterialParameterSet;
		if (BuildMaterialStaticParameterSet(MaterialParameters, Material, MaterialParameterSet))
		{
			//-OPT: We should be able to reuse rather than create
			FNameBuilder NameBuilder;
			Material->GetFName().ToString(NameBuilder);
			NameBuilder.Append(TEXT("_MIC"));

			UMaterialInstanceConstant* MIC = nullptr;
			if (MICPool.Num() > 0)
			{
				FName MICName(NameBuilder);
				auto MatchesNameAndMaterial = [&MICName, Material](UMaterialInstanceConstant* InMIC) -> bool
				{
					return InMIC->GetFName() == MICName && InMIC->Parent == Material;
				};

				const int32 ExistingIndex = MICPool.IndexOfByPredicate(MatchesNameAndMaterial);
				if (ExistingIndex != INDEX_NONE)
				{
					MIC = MICPool[ExistingIndex];
					MICPool.RemoveAtSwap(ExistingIndex, EAllowShrinking::No);
				}
			}
			if (MIC == nullptr)
			{
				if (UObject* ExistingObject = StaticFindObject(UMaterialInstanceConstant::StaticClass(), this, *NameBuilder))
				{
					const FString FullName = ExistingObject->GetFullName();
					const EInternalObjectFlags InternalFlags = ExistingObject->GetInternalFlags();
					const EObjectFlags Flags = ExistingObject->GetFlags();

					UE_LOGF(LogNiagara, Log, "While trying to allocate %ls UNiagaraRendererProperties::UpdateMaterialParametersMIC() found a pre-existing object.  This will result in a re-allocation!  FullName: %ls | InternalFlags: %x | Flags: %x",
						*NameBuilder, *FullName, EnumToUnderlyingType(InternalFlags), EnumToUnderlyingType(Flags));

					if (UMaterialInstanceConstant* ExistingMIC = Cast<UMaterialInstanceConstant>(ExistingObject))
					{
						NiagaraRendererPropertiesPrivate::MarkAndRenameMaterialForGarbage(ExistingMIC);
					}
				}

				MIC = NewObject<UMaterialInstanceConstant>(this, FName(NameBuilder));
				MIC->SetParentEditorOnly(Material);
			}

			MIC->UpdateStaticPermutation(MaterialParameterSet, OptionalMaterialUpdateContext ? OptionalMaterialUpdateContext : &DefaultMaterialUpdateContext);
			MIC->PostEditChange();

			InOutMICs.SetNum(i + 1);
			InOutMICs[i] = MIC;
		}
	}
}

int32 UNiagaraRendererProperties::GetDynamicParameterChannelMask(const FVersionedNiagaraEmitterData* EmitterData, FName BindingName, int32 DefaultChannelMask) const
{
	if (EmitterData == nullptr || BindingName.IsNone())
	{
		return 0;
	}

	TOptional<int32> ChannelMask;

	// We store the mask per script type to avoid static variable name collisions so we need to search by Particles.*.DynamicParameterChannelMask
	FNameBuilder BindingNameSearch;
	BindingName.ToString(BindingNameSearch);
	int32 NamespaceLocation = INDEX_NONE;
	if (!BindingNameSearch.ToView().FindChar('.', NamespaceLocation))
	{
		return DefaultChannelMask;
	}
	BindingNameSearch.Append(TEXT("ChannelMask"));

	FStringView BindingNamePrefix = BindingNameSearch.ToView().Mid(0, NamespaceLocation + 1);
	FStringView BindingNamePostfix = BindingNameSearch.ToView().Mid(NamespaceLocation, BindingNameSearch.ToView().Len() - NamespaceLocation);

	EmitterData->ForEachScript(
		[&ChannelMask, &BindingNamePrefix, &BindingNamePostfix](const UNiagaraScript* NiagaraScript)
		{
			const FNiagaraTypeDefinition VariableTypeDef = FNiagaraTypeDefinition::GetIntDef().ToStaticDef();

			const FNiagaraVMExecutableData& VMExecData = NiagaraScript->GetVMExecutableData();
			for (const FNiagaraVariable& StaticVariable : VMExecData.StaticVariablesWritten)
			{
				if (StaticVariable.GetType() != VariableTypeDef || !StaticVariable.IsDataAllocated())
				{
					continue;
				}

				FNameBuilder VariableName;
				StaticVariable.GetName().ToString(VariableName);
				if (VariableName.ToView().StartsWith(BindingNamePrefix) && VariableName.ToView().EndsWith(BindingNamePostfix))
				{
					ChannelMask = ChannelMask.Get(0) | StaticVariable.GetValue<int32>();
				}
			}
		}
	);
	return ChannelMask.Get(DefaultChannelMask);
}

int32 UNiagaraRendererProperties::GetDynamicParameterCombinedChannelMask(FName Parameter0Name, FName Parameter1Name, FName Parameter2Name, FName Parameter3Name) const
{
	const FVersionedNiagaraEmitterData* EmitterData = GetEmitterData();
	int32 CombinedChannelMask = 0;
	if (EmitterData == nullptr)
	{
		// This is a bit clunky but we no relationship to do this in a more agnostic way at the moment
		// We could pass down the owner emitter handle to CacheFromCompiledData.
		if (const UNiagaraStatelessEmitter* StatelessEmitter = GetTypedOuter<UNiagaraStatelessEmitter>())
		{
			if (const UNiagaraStatelessModule_DynamicMaterialParameters* DynamicParameterModule = StatelessEmitter->GetModule<UNiagaraStatelessModule_DynamicMaterialParameters>())
			{
				CombinedChannelMask = DynamicParameterModule->GetRendererChannelMask();
			}
		}
	}
	else
	{
		CombinedChannelMask |= GetDynamicParameterChannelMask(EmitterData, Parameter0Name, 0xf) << 0;
		CombinedChannelMask |= GetDynamicParameterChannelMask(EmitterData, Parameter1Name, 0xf) << 4;
		CombinedChannelMask |= GetDynamicParameterChannelMask(EmitterData, Parameter2Name, 0xf) << 8;
		CombinedChannelMask |= GetDynamicParameterChannelMask(EmitterData, Parameter3Name, 0xf) << 12;
	}
	return CombinedChannelMask;
}

FNiagaraVariable UNiagaraRendererProperties::GetBoundAttribute(const FNiagaraVariableAttributeBinding* Binding) const
{
	if (Binding->GetParamMapBindableVariable().IsValid())
	{
		return Binding->GetParamMapBindableVariable();
	}
	/*
	else if (AttributeBinding->DataSetVariable.IsValid())
	{
		return AttributeBinding->DataSetVariable;
	}
	else
	{
		return AttributeBinding->DefaultValueIfNonExistent;
	}*/

	return FNiagaraVariable();
}

void UNiagaraRendererProperties::CreateDefaultRendererWidget(TArray<TSharedPtr<SWidget>>& OutWidgets) const
{
	TSharedRef<SWidget> Widget = SNew(SImage)
		.Image(GetStackIcon());
	OutWidgets.Add(Widget);
}

void UNiagaraRendererProperties::CreateRendererWidgetsForAssets(TConstArrayView<UObject*> InAssets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool, TArray<TSharedPtr<SWidget>>& OutWidgets) const
{
	constexpr int32 ThumbnailSize = 32;
	
	for (UObject* Asset : InAssets)
	{
		if (Asset == nullptr)
		{
			CreateDefaultRendererWidget(OutWidgets);
			continue;
		}

		TSharedPtr<FAssetThumbnail> AssetThumbnail = MakeShareable(new FAssetThumbnail(Asset, ThumbnailSize, ThumbnailSize, InThumbnailPool));

		TSharedRef<SWidget> ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget();
		OutWidgets.Add(ThumbnailWidget);
	}
}

void UNiagaraRendererProperties::CreateRendererWidgetsForAssets(TConstArrayView<UMaterialInterface*> InMaterials, TSharedPtr<FAssetThumbnailPool> InThumbnailPool, TArray<TSharedPtr<SWidget>>& OutWidgets) const
{
	TArray<UObject*> Assets;
	Assets.Reserve(InMaterials.Num());
	Algo::Copy(InMaterials, Assets);
	CreateRendererWidgetsForAssets(Assets, InThumbnailPool, OutWidgets);
}

void UNiagaraRendererProperties::GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	CreateDefaultRendererWidget(OutWidgets);
}

void UNiagaraRendererProperties::GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings,	TArray<FNiagaraRendererFeedback>& OutInfo) const
{
	TArray<FText> Errors;
	TArray<FText> Warnings;
	TArray<FText> Infos;
	GetRendererFeedback(InEmitter, Errors, Warnings, Infos);
	for (FText ErrorText : Errors)
	{
		OutErrors.Add(FNiagaraRendererFeedback( ErrorText));
	}
	for (FText WarningText : Warnings)
	{
		OutWarnings.Add(FNiagaraRendererFeedback( WarningText));
	}
	for (FText InfoText : Infos)
	{
		OutInfo.Add(FNiagaraRendererFeedback(InfoText));
	}

	GetRendererMaterialUsageFeedback(OutWarnings);
}

void UNiagaraRendererProperties::GetMaterialUsageFeedback(EMaterialUsage Usage, TConstArrayView<EMaterialDomain> InvalidMaterialDomains, TArray<FNiagaraRendererFeedback>& OutFeedback) const
{
	TArray<UMaterialInterface*> Materials;
	GetUsedMaterials(nullptr, Materials);

	auto AppendToString =
		[](FString& InString, const FString& InItem)
		{
			if ( InString.Len() > 0 )
			{
				InString.Append(TEXT(", "));
			}
			InString.Append(InItem);
		};

	FString FailedUsageNames;
	FString FailedDomainNames;
	for (UMaterialInterface* Material : Materials)
	{
		UMaterial* BaseMaterial = Material ? Material->GetMaterial() : nullptr;
		if (!BaseMaterial)
		{
			continue;
		}

		bool bOutHasUsage = false;
		Material->NeedsSetMaterialUsage_Concurrent(bOutHasUsage, Usage);
		if (bOutHasUsage == false)
		{
			AppendToString(FailedUsageNames, Material->GetName());
		}

		if (InvalidMaterialDomains.Contains(BaseMaterial->MaterialDomain))
		{
			AppendToString(FailedDomainNames, Material->GetName());
		}
	}

	if (FailedUsageNames.Len() > 0)
	{
		OutFeedback.Emplace(
			FText::Format(LOCTEXT("InvalidMaterialUsage_Desc", "Some materials '{0}' do not have the correct usage flags set, and will use the default material."), FText::FromString(FailedUsageNames)),
			LOCTEXT("InvalidMaterialUsage_Summary", "Materials might not render correctly.")
		);
	}

	if (FailedDomainNames.Len() > 0)
	{
		const UEnum* MaterialDomainEnum = StaticEnum<EMaterialDomain>();

		FString InvalidMaterialDomainsString;
		for (EMaterialDomain MaterialDomain : InvalidMaterialDomains)
		{
			AppendToString(InvalidMaterialDomainsString, MaterialDomainEnum->GetDisplayNameTextByIndex(MaterialDomain).ToString());
		}

		OutFeedback.Emplace(
			FText::Format(LOCTEXT("InvalidMaterialDomain_Desc", "Some materials '{0}' use material domains '{1}' which are not supported so may not render correctly."), FText::FromString(FailedDomainNames), FText::FromString(InvalidMaterialDomainsString)),
			LOCTEXT("InvalidMaterialDomain_Summary", "Materials use unsupported material domain.")
		);
	}
}

void UNiagaraRendererProperties::GetMaterialUsageFeedback(EMaterialUsage Usage, TArray<FNiagaraRendererFeedback>& OutFeedback) const
{
	GetMaterialUsageFeedback(Usage, MakeArrayView<EMaterialDomain>(nullptr, 0), OutFeedback);
}

const FSlateBrush* UNiagaraRendererProperties::GetStackIcon() const
{
	return FSlateIconFinder::FindIconBrushForClass(GetClass());
}

FText UNiagaraRendererProperties::GetWidgetDisplayName() const
{
	return GetClass()->GetDisplayNameText();
}

bool UNiagaraRendererProperties::SupportsEmitterMode() const
{
	// LWE don't support emitter mode
	// Doing the reverse test does not work as the outer might be the clipboard
	return GetTypedOuter<UNiagaraStatelessEmitter>() == nullptr;
}

ENiagaraRendererSourceDataMode UNiagaraRendererProperties::ValidateSourceMode(ENiagaraRendererSourceDataMode InSourceMode) const
{
	return SupportsEmitterMode() ? InSourceMode : ENiagaraRendererSourceDataMode::Particles;
}

#if WITH_NIAGARA_RENDERER_DEBUGDRAW
bool UNiagaraRendererProperties::SupportsDebugDraw() const
{
	return false;
}

TOptional<FText> UNiagaraRendererProperties::GetDebugDrawTooltip() const
{
	return {};
}

bool UNiagaraRendererProperties::IsDebugDrawEnabled() const
{
	if (SupportsDebugDraw() && bDebugDrawEnabled)
	{
		if (UNiagaraSystem* NiagaraSystem = GetTypedOuter<UNiagaraSystem>())
		{
			return NiagaraSystem->ShouldDisableDebugSwitches() == false;
		}
	}
	return false;
}
void UNiagaraRendererProperties::SetDebugDrawEnabled(bool bInEnabled)
{
	bDebugDrawEnabled = bInEnabled;
}
#endif

void UNiagaraRendererProperties::RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitter& InEmitter)
{
	RenameVariable(OldVariable, NewVariable, FVersionedNiagaraEmitterBase(InEmitter.Emitter, InEmitter.Version));
}

void UNiagaraRendererProperties::RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitter& InEmitter)
{
	RemoveVariable(OldVariable, FVersionedNiagaraEmitterBase(InEmitter.Emitter, InEmitter.Version));
}

void UNiagaraRendererProperties::RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	for (TFieldIterator<FStructProperty> PropIt(GetClass()); PropIt; ++PropIt)
	{
		FStructProperty* StructProp = *PropIt;
		if (!StructProp || !StructProp->Struct || !StructProp->Struct->IsChildOf(FNiagaraVariableAttributeBinding::StaticStruct()))
		{
			continue;
		}

		if (FNiagaraVariableAttributeBinding* Binding = StructProp->ContainerPtrToValuePtr<FNiagaraVariableAttributeBinding>(this))
		{
			Binding->RenameVariableIfMatching(OldVariable, NewVariable, InEmitter, GetCurrentSourceMode());
		}
	}

	if (InEmitter.Emitter)
	{
		FNiagaraParameterBinding::ForEachRenameVariable(this, OldVariable, NewVariable, InEmitter.Emitter->GetUniqueEmitterName());
	}
}

void UNiagaraRendererProperties::RemoveVariable(const FNiagaraVariableBase& OldVariable,const FVersionedNiagaraEmitterBase& InEmitter)
{
	for (TFieldIterator<FStructProperty> PropIt(GetClass()); PropIt; ++PropIt)
	{
		FStructProperty* StructProp = *PropIt;
		if (!StructProp || !StructProp->Struct || !StructProp->Struct->IsChildOf(FNiagaraVariableAttributeBinding::StaticStruct()))
		{
			continue;
		}

		FNiagaraVariableAttributeBinding* Binding = StructProp->ContainerPtrToValuePtr<FNiagaraVariableAttributeBinding>(this);
		if (!Binding || !Binding->Matches(OldVariable, InEmitter, GetCurrentSourceMode()))
		{
			continue;
		}

		if ( const FNiagaraVariableAttributeBinding* DefaultBinding = StructProp->ContainerPtrToValuePtr<FNiagaraVariableAttributeBinding>(GetClass()->GetDefaultObject()) )
		{
			Binding->ResetToDefault(*DefaultBinding, InEmitter, GetCurrentSourceMode());
		}
	}

	if (InEmitter.Emitter)
	{
		FNiagaraParameterBinding::ForEachRemoveVariable(this, OldVariable, InEmitter.Emitter->GetUniqueEmitterName());
	}
}

#endif

uint32 UNiagaraRendererProperties::ComputeMaxUsedComponents(const FNiagaraDataSetCompiledData* CompiledDataSetData) const
{
	enum BaseType
	{
		BaseType_Int,
		BaseType_Float,
		BaseType_Half,
		BaseType_NUM
	};

	TArray<int32, TInlineAllocator<32>> SeenOffsets[BaseType_NUM];
	uint32 NumComponents[BaseType_NUM] = { 0 };

	auto AccumulateUniqueComponents = [&](BaseType Type, uint32 ComponentCount, int32 ComponentOffset)
	{
		if (!SeenOffsets[Type].Contains(ComponentOffset))
		{
			SeenOffsets[Type].Add(ComponentOffset);
			NumComponents[Type] += ComponentCount;
		}
	};

	for (const FNiagaraVariableAttributeBinding* Binding : AttributeBindings)
	{
		const FNiagaraVariable& Var = Binding->GetDataSetBindableVariable();

		const int32 VariableIndex = CompiledDataSetData->Variables.IndexOfByKey(Var);
		if ( VariableIndex != INDEX_NONE )
		{
			const FNiagaraVariableLayoutInfo& DataSetVarLayout = CompiledDataSetData->VariableLayouts[VariableIndex];

			if (const uint32 FloatCount = DataSetVarLayout.GetNumFloatComponents())
			{
				AccumulateUniqueComponents(BaseType_Float, FloatCount, DataSetVarLayout.GetFloatComponentStart());
			}

			if (const uint32 IntCount = DataSetVarLayout.GetNumInt32Components())
			{
				AccumulateUniqueComponents(BaseType_Int, IntCount, DataSetVarLayout.GetInt32ComponentStart());
			}

			if (const uint32 HalfCount = DataSetVarLayout.GetNumHalfComponents())
			{
				AccumulateUniqueComponents(BaseType_Half, HalfCount, DataSetVarLayout.GetHalfComponentStart());
			}
		}
	}

	uint32 MaxNumComponents = 0;

	for (uint32 ComponentCount : NumComponents)
	{
		MaxNumComponents = FMath::Max(MaxNumComponents, ComponentCount);
	}

	return MaxNumComponents;
}

void UNiagaraRendererProperties::GetAssetTagsForContext(const UObject* InAsset, FGuid AssetVersion, const TArray<const UNiagaraRendererProperties*>& InProperties, TMap<FName, uint32>& NumericKeys, TMap<FName, FString>& StringKeys) const
{
	UClass* Class = GetClass();

	// Default count up how many instances there are of this class and report to content browser
	if (Class)
	{
		uint32 NumInstances = 0;
		for (const UNiagaraRendererProperties* Prop : InProperties)
		{
			if (Prop && Prop->IsA(Class))
			{
				NumInstances++;
			}
		}

		// Note that in order for these tags to be registered, we always have to put them in place for the CDO of the object, but 
		// for readability's sake, we leave them out of non-CDO assets.
		if (NumInstances > 0 || (InAsset && InAsset->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject)))
		{
			FString Key = Class->GetName();
			Key.ReplaceInline(TEXT("Niagara"), TEXT(""));
			Key.ReplaceInline(TEXT("Properties"), TEXT(""));
			NumericKeys.Add(FName(Key)) = NumInstances;
		}
	}
}

bool UNiagaraRendererProperties::PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore)
{
	bool bAnyAdded = false;
	if (RendererEnabledBinding.GetParamMapBindableVariable().IsValid())
	{
		bAnyAdded |= InParameterStore.AddParameter(RendererEnabledBinding.GetParamMapBindableVariable(), false);
	}
	return bAnyAdded;
}

void UNiagaraRendererProperties::CollectPSOPrecacheData(FNiagaraEmitterInstance* EmitterInstance, FMaterialInterfacePSOPrecacheParamsList& MaterialInterfacePSOPrecacheParamsList) const
{
	const FVertexFactoryType* VFType = GetVertexFactoryType();
	if (VFType == nullptr)
	{
		return;
	}

	FMaterialInterfacePSOPrecacheParams NewEntry;
	NewEntry.PSOPrecacheParams.SetMobility(EComponentMobility::Movable);
	NewEntry.PSOPrecacheParams.bDisableBackFaceCulling = IsBackfaceCullingDisabled();

	UNiagaraRendererProperties::FPSOPrecacheParamsList PSOPrecacheParamsList;
	CollectPSOPrecacheData(EmitterInstance, PSOPrecacheParamsList);

	for (UNiagaraRendererProperties::FPSOPrecacheParams& PSOPrecacheParams : PSOPrecacheParamsList)
	{
		NewEntry.MaterialInterface = PSOPrecacheParams.MaterialInterface;
		NewEntry.VertexFactoryDataList = PSOPrecacheParams.VertexFactoryDataList;

		NewEntry.PSOPrecacheParams.bReverseCulling = false;
		AddMaterialInterfacePSOPrecacheParamsToList(NewEntry, MaterialInterfacePSOPrecacheParamsList);

		// Also precache with reverse culling if not two sided because we don't know of the component using the asset will have negative determinant
		if (!NewEntry.PSOPrecacheParams.bDisableBackFaceCulling && GNiagaraPSOPrecacheReverseCulling > 0)
		{
			NewEntry.PSOPrecacheParams.bReverseCulling = true;
			AddMaterialInterfacePSOPrecacheParamsToList(NewEntry, MaterialInterfacePSOPrecacheParamsList);
		}
	}
}


bool UNiagaraRendererProperties::NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const
{
	// only keep enabled renderers that are parented to valid emitters
	if (const UNiagaraEmitter* OwnerEmitter = GetTypedOuter<const UNiagaraEmitter>())
	{
		if (OwnerEmitter->NeedsLoadForTargetPlatform(TargetPlatform))
		{
			if (bIsEnabled && Platforms.IsEnabledForPlatform(TargetPlatform->IniPlatformName()))
			{
			#if WITH_EDITORONLY_DATA
				if (GNiagaraRendererCookOutStaticEnabledBinding && RendererEnabledBinding.IsValid())
				{
					TOptional<bool> ResolvedStaticValue = NiagaraRendererPropertiesPrivate::TryResolveStaticVariableBool(OwnerEmitter, RendererEnabledBinding.GetParamMapBindableVariable());
					return ResolvedStaticValue.Get(true);
				}
			#endif
				return true;
			}
		}
	}
	//-TODO:Stateless: We need a base emitter type
	else if (const UNiagaraStatelessEmitter* OwnerStatelessEmitter = GetTypedOuter<UNiagaraStatelessEmitter>())
	{
		if (OwnerStatelessEmitter->NeedsLoadForTargetPlatform(TargetPlatform))
		{
			if (bIsEnabled && Platforms.IsEnabledForPlatform(TargetPlatform->IniPlatformName()))
			{
//#if WITH_EDITORONLY_DATA
//				if (GNiagaraRendererCookOutStaticEnabledBinding && RendererEnabledBinding.IsValid())
//				{
//					TOptional<bool> ResolvedStaticValue = NiagaraRendererPropertiesPrivate::TryResolveStaticVariableBool(OwnerEmitter, RendererEnabledBinding.GetParamMapBindableVariable());
//					return ResolvedStaticValue.Get(true);
//				}
//#endif
				return true;
			}
		}
	}
	//-TODO:Stateless: We need a base emitter type

	return false;
}

void UNiagaraRendererProperties::PostLoadBindings(ENiagaraRendererSourceDataMode InSourceMode)
{
	RendererEnabledBinding.PostLoad(ENiagaraRendererSourceDataMode::Emitter);
	for (int32 i = 0; i < AttributeBindings.Num(); i++)
	{
		FNiagaraVariableAttributeBinding* Binding = const_cast<FNiagaraVariableAttributeBinding*>(AttributeBindings[i]);
		Binding->PostLoad(InSourceMode);
	}
}

void UNiagaraRendererProperties::PostInitProperties()
{
	Super::PostInitProperties();
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		SetFlags(RF_Transactional);

		InitRendererBaseBindings();
	}
#endif
}

void UNiagaraRendererProperties::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (bMotionBlurEnabled_DEPRECATED == false)
	{
		MotionVectorSetting = ENiagaraRendererMotionVectorSetting::Disable;
	}
#endif
}

#if WITH_EDITOR
bool UNiagaraRendererProperties::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty != nullptr)
	{
		static const FName NAME_SourceMode("SourceMode");
		if (InProperty->GetFName() == NAME_SourceMode)
		{
			const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(InProperty);
			if (EnumProperty && EnumProperty->GetEnum() == StaticEnum<ENiagaraRendererSourceDataMode>())
			{
				return SupportsEmitterMode();
			}
		}
	}

	return true;
}
#endif //WITH_EDITOR

#if WITH_EDITORONLY_DATA

void UNiagaraRendererProperties::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (FVersionedNiagaraEmitterData* EmitterData = GetEmitterData())
	{
		// Check for properties changing that invalidate the current script compilation for the emitter
		bool bNeedsRecompile = false;
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UNiagaraRendererProperties, MotionVectorSetting))
		{
			if (EmitterData->GraphSource)
			{
				EmitterData->GraphSource->MarkNotSynchronized(TEXT("Renderer MotionVectorSetting changed"));
			}
			bNeedsRecompile = true;
		}

		if (bNeedsRecompile)
		{
			UNiagaraSystem::RequestCompileForEmitter(GetOuterEmitter());
		}

		// Just in case we changed something that needs static params, refresh that cached list.
		EmitterData->RebuildRendererBindings(*GetOuterEmitter().Emitter);
	}

}

#endif

void UNiagaraRendererProperties::SetIsEnabled(bool bInIsEnabled)
{
	if (bIsEnabled != bInIsEnabled)
	{
		FNiagaraSystemUpdateContext UpdateContext;
	#if WITH_EDITORONLY_DATA
		UpdateContext.Add(GetOuterEmitter(), true);
	#else
		// Shouldn't really be called at runtime, but let's ensure we handle the case
		UpdateContext.Add(GetTypedOuter<UNiagaraSystem>(), true);
	#endif

		bIsEnabled = bInIsEnabled;
	}
}

void UNiagaraRendererProperties::UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit)
{
	FVersionedNiagaraEmitterBase SrcEmitter = GetOuterEmitterBase();
	if (SrcEmitter.Emitter)
	{
		for (const FNiagaraVariableAttributeBinding* Binding : AttributeBindings)
		{
			((FNiagaraVariableAttributeBinding*)Binding)->CacheValues(SrcEmitter, InSourceMode);
		}

		RendererEnabledBinding.CacheValues(SrcEmitter, InSourceMode);

#if WITH_EDITORONLY_DATA
		// If we added or removed any valid bindings to a non-particle source during editing, we need to reset to prevent hazards and
		// to ensure new ones get bound by the simulation
		if (bFromPropertyEdit)
		{
			// We may need to refresh internal variables because this may be the first binding to it, so request a recompile as that will pull data 
			// into the right place.
			UNiagaraSystem::RequestCompileForEmitter(GetOuterEmitter());
			FNiagaraSystemUpdateContext Context(SrcEmitter, true);
		}
#endif
	}
}

FVersionedNiagaraEmitterData* UNiagaraRendererProperties::GetEmitterData() const
{
	if (UNiagaraEmitter* SrcEmitter = GetTypedOuter<UNiagaraEmitter>())
	{
		return SrcEmitter->GetEmitterData(OuterEmitterVersion);
	}
	return nullptr;
}

FVersionedNiagaraEmitter UNiagaraRendererProperties::GetOuterEmitter() const
{
	if (UNiagaraEmitter* SrcEmitter = GetTypedOuter<UNiagaraEmitter>())
	{
		return FVersionedNiagaraEmitter(SrcEmitter, OuterEmitterVersion);
	}
	return FVersionedNiagaraEmitter();
}

FVersionedNiagaraEmitterBase UNiagaraRendererProperties::GetOuterEmitterBase() const
{
	if (UNiagaraEmitterBase* SrcEmitter = GetTypedOuter<UNiagaraEmitterBase>())
	{
		return FVersionedNiagaraEmitterBase(SrcEmitter, OuterEmitterVersion);
	}
	return FVersionedNiagaraEmitterBase();
}

#if WITH_EDITORONLY_DATA
bool UNiagaraRendererProperties::IsContainedInEmitterRendererArray() const
{
	FVersionedNiagaraEmitterBase VersionedEmitter = GetOuterEmitterBase();
	if (VersionedEmitter.Emitter)
	{
		if (UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(VersionedEmitter.Emitter))
		{
			if (FVersionedNiagaraEmitterData* EmitterData = Emitter->GetEmitterData(VersionedEmitter.Version))
			{
				return EmitterData->GetRenderers().Contains(this);
			}
		}
		else if (UNiagaraStatelessEmitter* StatelessEmitter = Cast<UNiagaraStatelessEmitter>(VersionedEmitter.Emitter))
		{
			return StatelessEmitter->GetRenderers().Contains(this);
		}
	}
	return false;
}
#endif

void UNiagaraRendererProperties::InitRendererBaseBindings()
{
	if (!RendererEnabledBinding.IsValid())
	{
		FNiagaraVariableBase EnabledDefaultVariable(FNiagaraTypeDefinition::GetBoolDef(), NAME_None);
		RendererEnabledBinding.Setup(EnabledDefaultVariable, EnabledDefaultVariable, ENiagaraRendererSourceDataMode::Emitter);
	}
}

bool UNiagaraRendererProperties::NeedsPreciseMotionVectors() const
{
	if (MotionVectorSetting == ENiagaraRendererMotionVectorSetting::AutoDetect)
	{
		// TODO - We could get even smarter here and early return with false if we know that the material can absolutely not be overridden by the user and
		// it doesn't need to render velocity
		return GetDefault<UNiagaraSettings>()->DefaultRendererMotionVectorSetting == ENiagaraDefaultRendererMotionVectorSetting::Precise;
	}
	
	return MotionVectorSetting == ENiagaraRendererMotionVectorSetting::Precise;
}

bool UNiagaraRendererProperties::IsSortHighPrecision(ENiagaraRendererSortPrecision SortPrecision)
{
	if (SortPrecision == ENiagaraRendererSortPrecision::Default)
	{
		return GetDefault<UNiagaraSettings>()->DefaultSortPrecision == ENiagaraDefaultSortPrecision::High;
	}
	return SortPrecision == ENiagaraRendererSortPrecision::High;
}

bool UNiagaraRendererProperties::ShouldGpuTranslucentThisFrame(ENiagaraRendererGpuTranslucentLatency Latency)
{
	if (Latency == ENiagaraRendererGpuTranslucentLatency::ProjectDefault)
	{
		return GetDefault<UNiagaraSettings>()->DefaultGpuTranslucentLatency == ENiagaraDefaultGpuTranslucentLatency::Immediate;
	}
	return Latency == ENiagaraRendererGpuTranslucentLatency::Immediate;
}

bool UNiagaraRendererProperties::IsGpuTranslucentThisFrame(ERHIFeatureLevel::Type FeatureLevel, ENiagaraRendererGpuTranslucentLatency Latency)
{
	// We can not support low latency on the mobile renderer path as it calls PostRenderOpaque after translucent in some paths
	if (GetFeatureLevelShadingPath(FeatureLevel) != EShadingPath::Deferred)
	{
		return false;
	}

	return ShouldGpuTranslucentThisFrame(Latency);
}

#undef LOCTEXT_NAMESPACE

