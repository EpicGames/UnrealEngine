// Copyright Epic Games, Inc. All Rights Reserved.

//-TODO: Add sprite cutout support
//-TODO: Add way to specify local / world and how we should handle LWC
//-TODO: Validate cooked doesn't contain permutations + validate can run

#include "NiagaraRendererReadback.h"
#include "NiagaraComponent.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGpuComputeDataManager.h"
#include "NiagaraGpuReadbackManager.h"
#include "NiagaraSceneProxy.h"
#include "NiagaraVertexFactoryExport.h"

#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "Async/Async.h"
#include "Components.h"
#include "MeshMaterialShader.h"
#include "MeshPassProcessor.h"
#include "RenderGraphUtils.h"
#include "SceneInterface.h"
#include "SceneTexturesConfig.h"
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraRendererReadback)

#if WITH_NIAGARA_RENDERER_READBACK
namespace NiagaraRendererReadback
{
	uint32 GIsCapturing = 0;

	BEGIN_SHADER_PARAMETER_STRUCT(FReadbackPassParams, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer,				RWVertexBuffer)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters,			View)
		//SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters,	Scene)
	END_SHADER_PARAMETER_STRUCT()

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FReadbackMeshBatch
	{
		uint32						OutputVertexOffset = 0;
		uint32						SectionIndex = 0;
		const FSceneView*			View = nullptr;
		const FPrimitiveSceneProxy* SceneProxy = nullptr;
		FMeshBatch					MeshBatch;					// We might be able to use a pointer here since it's all within the same frame
		uint32						NumInstances = 0;
		uint32						NumVerticesPerInstance = 0;
	};

	struct FRendererReadbackRequest
	{
		bool													bExportMaterials = true;
		bool													bApplyWPO = false;
		bool													bWorldSpace = true;
		bool													bCaptureAllPrimitives = false;
		TOptional<int>											ViewIndexToCapture;
		FNiagaraRendererReadbackResult							Result;

		UE::FMutex												MeshBatchesMutex;
		TArray<FReadbackMeshBatch>								MeshBatches;

		FNiagaraRendererReadbackComplete						CompleteCallback;
		TArray<FPrimitiveComponentId, TInlineAllocator<1>>		PrimitivesToCapture;
	};

	using FRendererReadbackRequestPtr = TSharedPtr<FRendererReadbackRequest, ESPMode::ThreadSafe>;

	void ExecuteCompleteCallback(FRendererReadbackRequestPtr Request)
	{
		AsyncTask(
			ENamedThreads::GameThread,
			[Request]()
			{
				Request->CompleteCallback(Request->Result);
			}
		);
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void ExportMeshBatch(FRHICommandList& RHICmdList, FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FReadbackMeshBatch& ReadbackMeshBatch, bool bApplyWPO, bool bWorldSpace, const FNiagaraRendererReadbackResult& VertexLayout, FRDGBufferUAV* VertexBufferUAV)
	{
		const FMeshBatch& MeshBatch = ReadbackMeshBatch.MeshBatch;

		for (const FMaterialRenderProxy* MaterialRenderProxy=MeshBatch.MaterialRenderProxy; MaterialRenderProxy; MaterialRenderProxy=MaterialRenderProxy->GetFallback(FeatureLevel))
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (!Material || !Material->GetRenderingThreadShaderMap())
			{
				continue;
			}

			FMaterialShaderTypes ShaderTypes;
			ShaderTypes.AddShaderType<FNiagaraVertexFactoryExportCS>();

			FMaterialShaders MaterialShaders;
			if (Material->TryGetShaders(ShaderTypes, MeshBatch.VertexFactory->GetType(), MaterialShaders) == false)
			{
				continue;
			}

			const FSceneView* View = ReadbackMeshBatch.View;
			const FPrimitiveSceneProxy* SceneProxy = ReadbackMeshBatch.SceneProxy;

			TShaderRef<FNiagaraVertexFactoryExportCS> Shader;
			MaterialShaders.TryGetShader(SF_Compute, Shader);
							
			FMeshProcessorShaders MeshProcessorShaders;
			MeshProcessorShaders.ComputeShader = Shader;

			FMeshDrawShaderBindings ShaderBindings;
			ShaderBindings.Initialize(MeshProcessorShaders);

			FMeshMaterialShaderElementData ShaderElementData;
			ShaderElementData.InitializeMeshMaterialData(View, SceneProxy, MeshBatch, -1, false);

			FMeshDrawSingleShaderBindings SingleShaderBindings = ShaderBindings.GetSingleShaderBindings(SF_Compute);
			Shader->GetShaderBindings(Scene, FeatureLevel, SceneProxy, *MaterialRenderProxy, *Material, ShaderElementData, SingleShaderBindings);

			const FMeshBatchElement& MeshBatchElement = MeshBatch.Elements[0];

			FVertexInputStreamArray DummyArray;
			FMeshMaterialShader::GetElementShaderBindings(Shader, Scene, View, MeshBatch.VertexFactory, EVertexInputStreamType::Default, FeatureLevel, SceneProxy, MeshBatch, MeshBatchElement, ShaderElementData, SingleShaderBindings, DummyArray);

			const uint32 SectionInfoOffset = (VertexLayout.NumVertices * VertexLayout.VertexStride) + (ReadbackMeshBatch.SectionIndex * 4);

			SingleShaderBindings.Add(Shader->IsIndirectDraw,			MeshBatchElement.IndirectArgsBuffer == nullptr ? 0 : 1);
			SingleShaderBindings.Add(Shader->NumInstances,				ReadbackMeshBatch.NumInstances);
			SingleShaderBindings.Add(Shader->NumVerticesPerInstance,	ReadbackMeshBatch.NumVerticesPerInstance);
			SingleShaderBindings.Add(Shader->bApplyWPO,					bApplyWPO ? 1 : 0);
			SingleShaderBindings.Add(Shader->bWorldSpace,				bWorldSpace ? 1 : 0);

			SingleShaderBindings.Add(Shader->VertexStride,				VertexLayout.VertexStride);
			SingleShaderBindings.Add(Shader->VertexPositionOffset,		VertexLayout.VertexPositionOffset);
			SingleShaderBindings.Add(Shader->VertexColorOffset,			VertexLayout.VertexColorOffset);
			SingleShaderBindings.Add(Shader->VertexTangentBasisOffset,	VertexLayout.VertexTangentBasisOffset);
			SingleShaderBindings.Add(Shader->VertexTexCoordOffset,		VertexLayout.VertexTexCoordOffset);
			SingleShaderBindings.Add(Shader->VertexTexCoordNum,			VertexLayout.VertexTexCoordNum);
			SingleShaderBindings.Add(Shader->VertexTexCoordBindings,	VertexLayout.VertexTexCoordBindings);
			SingleShaderBindings.Add(Shader->VertexOutputOffset,		ReadbackMeshBatch.OutputVertexOffset);
			SingleShaderBindings.Add(Shader->SectionInfoOutputOffset,	SectionInfoOffset);

		#if MESH_DRAW_COMMAND_DEBUG_DATA
			ShaderBindings.Finalize(&MeshProcessorShaders);
		#endif
								
			FRHIComputeShader* ComputeShader = Shader.GetComputeShader();
			SetComputePipelineState(RHICmdList, ComputeShader);

			FShaderBindingState ShaderBindingState;
			FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
			SetUAVParameter(BatchedParameters, Shader->RWVertexData, VertexBufferUAV->GetRHI());
			ShaderBindings.SetParameters(BatchedParameters, &ShaderBindingState);
			RHICmdList.SetBatchedShaderParameters(ComputeShader, BatchedParameters);

			const uint32 MaxOrNumVertices = ReadbackMeshBatch.NumInstances * ReadbackMeshBatch.NumVerticesPerInstance;
			const FIntVector NumWrappedThreadGroups = FComputeShaderUtils::GetGroupCountWrapped(MaxOrNumVertices, FNiagaraVertexFactoryExportCS::ThreadGroupSize);
			RHICmdList.DispatchComputeShader(NumWrappedThreadGroups.X, NumWrappedThreadGroups.Y, NumWrappedThreadGroups.Z);

			return;
		}
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FRendererReadbackComputeManager final : public FNiagaraGpuComputeDataManager
	{
	public:
		FRendererReadbackComputeManager(FNiagaraGpuComputeDispatchInterface* InOwnerInterface)
			: FNiagaraGpuComputeDataManager(InOwnerInterface)
		{
			InOwnerInterface->GetOnPreInitViewsEvent().AddRaw(this, &FRendererReadbackComputeManager::OnPreRender);
			InOwnerInterface->GetOnPostRenderEvent().AddRaw(this, &FRendererReadbackComputeManager::OnPostRender);
		}

		virtual ~FRendererReadbackComputeManager()
		{
		}

		static FName GetManagerName()
		{
			static FName ManagerName("FRendererReadbackComputeManager");
			return ManagerName;
		}

		void OnPreRender(FRDGBuilder& GraphBuilder)
		{
			if (ReadbackRequests.Num() == 0)
			{
				return;
			}
			++NiagaraRendererReadback::GIsCapturing;
		}

		void OnPostRender(FRDGBuilder& GraphBuilder)
		{
			if (ReadbackRequests.Num() == 0)
			{
				return;
			}

			check(NiagaraRendererReadback::GIsCapturing > 0);
			--NiagaraRendererReadback::GIsCapturing;

			FScene* Scene = GetOwnerInterface()->GetScene();
			const ERHIFeatureLevel::Type FeatureLevel = GetOwnerInterface()->GetSceneInterface()->GetFeatureLevel();

			for (const FRendererReadbackRequestPtr& Request : ReadbackRequests)
			{
				for (FReadbackMeshBatch& ReadbackMeshBatch : Request->MeshBatches)
				{
					ReadbackMeshBatch.OutputVertexOffset = Request->Result.NumVertices;
					ReadbackMeshBatch.SectionIndex = Request->Result.Sections.Num();

					const FMeshBatch& MeshBatch = ReadbackMeshBatch.MeshBatch;
					const FMeshBatchElement& MeshBatchElement = MeshBatch.Elements[0];

					const uint32 MaxOrNumVertices = ReadbackMeshBatch.NumInstances * ReadbackMeshBatch.NumVerticesPerInstance;

					FNiagaraRendererReadbackResult::FSection& Section = Request->Result.Sections.AddDefaulted_GetRef();
					if (Request->bExportMaterials)
					{
						Section.WeakMaterialInterface	= MeshBatch.MaterialRenderProxy->GetMaterialInterface();
					}
					Section.FirstTriangle	= Request->Result.NumVertices / 3;
					Section.NumTriangles	= MaxOrNumVertices / 3;
					Request->Result.NumVertices += MaxOrNumVertices;
				}

				if (Request->Result.NumVertices == 0)
				{
					ExecuteCompleteCallback(Request);
					continue;
				}

				// Allocate output buffer
				const uint32 VertexSize = Request->Result.NumVertices * Request->Result.VertexStride;
				const uint32 SectionSize = Request->Result.Sections.Num() * sizeof(uint32);

				FRDGBufferDesc VertexBufferDesc = FRDGBufferDesc::CreateByteAddressDesc(VertexSize + SectionSize);
				VertexBufferDesc.Usage |= EBufferUsageFlags::SourceCopy;
				FRDGBufferRef VertexBuffer = GraphBuilder.CreateBuffer(VertexBufferDesc, TEXT("NiagaraRendererReadback"), ERDGBufferFlags::None);
				FRDGBufferUAVRef VertexBufferUAV = GraphBuilder.CreateUAV(VertexBuffer, PF_Unknown);

				// Make sure we clear the UAV so the section data we write out is valid even when we can't find an appropriate export shader
				AddClearUAVPass(GraphBuilder, VertexBufferUAV, 0);

				// Capture data
				{
					const FSceneView* SceneView = Request->MeshBatches[0].View;

					FReadbackPassParams* PassParameters = GraphBuilder.AllocParameters<FReadbackPassParams>();
					PassParameters->RWVertexBuffer	= VertexBufferUAV;
					PassParameters->View			= SceneView->ViewUniformBuffer;

					GraphBuilder.AddPass(
						RDG_EVENT_NAME("NiagaraRendererReadback"),
						PassParameters,
						ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
						[Request, PassParameters, Scene, FeatureLevel, SceneView](/*FRDGAsyncTask, */FRHICommandListImmediate& RHICmdList)
						{
							FMemMark Mark(FMemStack::Get());

							for (const FReadbackMeshBatch& ReadbackMeshBatch : Request->MeshBatches)
							{
								ExportMeshBatch(RHICmdList, Scene, FeatureLevel, ReadbackMeshBatch, Request->bApplyWPO, Request->bWorldSpace, Request->Result, PassParameters->RWVertexBuffer);
							}
						}
					);
				}

				// Queue readback of the buffer
				FNiagaraGpuReadbackManager* ReadbackManager = GetOwnerInterface()->GetGpuReadbackManager();
				ReadbackManager->EnqueueReadback(
					GraphBuilder,
					VertexBuffer,
					[Request, VertexSize, SectionSize](TConstArrayView<TPair<void*, uint32>> ReadbackData)
					{
						check(ReadbackData.Num() == 1);
						check(ReadbackData[0].Value == VertexSize + SectionSize);

						const uint8* VertexData = static_cast<const uint8*>(ReadbackData[0].Key);
						const uint32* SectionInfo = reinterpret_cast<const uint32*>(VertexData + VertexSize);

						// Get Vertex Count and Allocate Space
						uint32 VertexCount = 0;
						for ( int32 i=0; i < Request->Result.Sections.Num(); ++i )
						{
							VertexCount += SectionInfo[i];
						}
						Request->Result.NumVertices = VertexCount;

						Request->Result.VertexData.AddUninitialized(VertexCount * Request->Result.VertexStride);
						uint8* OutputVertexData = Request->Result.VertexData.GetData();

						// Copy over the vertex data section by section
						VertexCount = 0;
						for (int32 i=0; i < Request->Result.Sections.Num(); ++i)
						{
							const uint32 SectionVertexCount = SectionInfo[i];
							FMemory::Memcpy(OutputVertexData, VertexData, SectionVertexCount * Request->Result.VertexStride);

							FNiagaraRendererReadbackResult::FSection& Section = Request->Result.Sections[i];
							OutputVertexData += SectionVertexCount * Request->Result.VertexStride;
							VertexData += Section.NumTriangles * 3 * Request->Result.VertexStride;

							Section.FirstTriangle = VertexCount / 3;
							Section.NumTriangles = SectionVertexCount / 3;

							VertexCount += SectionVertexCount;
						}

						ExecuteCompleteCallback(Request);
					}
				);
			}

		#if 1
			ReadbackRequests.Empty();
		#else
			// For debugging forces contant readback once requested
			TArray<FRendererReadbackRequestPtr> TempRequests;
			Swap(TempRequests, ReadbackRequests);

			for (const FRendererReadbackRequestPtr& TempRequest : TempRequests)
			{
				FRendererReadbackRequestPtr NewRequest		= MakeShared<FRendererReadbackRequest>();
				NewRequest->CompleteCallback				= TempRequest->CompleteCallback;
				NewRequest->PrimitivesToCapture				= TempRequest->PrimitivesToCapture;
				NewRequest->Result.VertexStride				= TempRequest->Result.VertexStride;
				NewRequest->Result.VertexPositionOffset		= TempRequest->Result.VertexPositionOffset;
				NewRequest->Result.VertexColorOffset		= TempRequest->Result.VertexColorOffset;
				NewRequest->Result.VertexTangentBasisOffset	= TempRequest->Result.VertexTangentBasisOffset;
				NewRequest->Result.VertexTexCoordOffset		= TempRequest->Result.VertexTexCoordOffset;
				NewRequest->Result.VertexTexCoordNum		= TempRequest->Result.VertexTexCoordNum;
				ReadbackRequests.Emplace(NewRequest);
			}
		#endif
		}

		void AddMeshBatch(const FSceneView* View, const FPrimitiveSceneProxy* SceneProxy, const FMeshBatch& MeshBatch, uint32 NumInstances, uint32 NumVerticesPerInstance)
		{
			for (const FRendererReadbackRequestPtr& Request : ReadbackRequests)
			{
				if (Request->bCaptureAllPrimitives == false && !Request->PrimitivesToCapture.Contains(SceneProxy->GetPrimitiveComponentId()))
				{
					continue;
				}

				if (MeshBatch.Elements.Num() != 1)
				{
					Request->Result.Errors.AddUnique(TEXT("Skipped mesh batch as only a single element is supported."));
					return;
				}

				if (!ensureMsgf(MeshBatch.Type == PT_TriangleList, TEXT("Only PT_TriangleList are supported for renderer readback")))
				{
					Request->Result.Errors.AddUnique(TEXT("Skipped mesh batch as only PT_TriangleList is supported."));
					return;
				}

				if (NumInstances * NumVerticesPerInstance == 0)
				{
					return;
				}

				if (!FNiagaraVertexFactoryExportCS::SupportsVertexFactoryType(MeshBatch.VertexFactory->GetType()))
				{
					Request->Result.Errors.AddUnique(FString::Printf(TEXT("Skipped mesh batch due to unsupported vertex factory '%s'"), MeshBatch.VertexFactory->GetType()->GetName()));
					return;
				}

				if (Request->ViewIndexToCapture.IsSet())
				{
					const int32 CaptureViewIndex = Request->ViewIndexToCapture.GetValue();
					const FSceneView* CaptureView = View && View->Family && View->Family->Views.IsValidIndex(CaptureViewIndex) ? View->Family->Views[CaptureViewIndex] : nullptr;
					if (View != CaptureView)
					{
						continue;
					}
				}

				UE::TScopeLock ScopeLock(Request->MeshBatchesMutex);
				FReadbackMeshBatch& ReadbackMeshBatch = Request->MeshBatches.AddDefaulted_GetRef();
				ReadbackMeshBatch.View						= View;
				ReadbackMeshBatch.SceneProxy				= SceneProxy;
				ReadbackMeshBatch.MeshBatch					= MeshBatch;
				ReadbackMeshBatch.NumInstances				= NumInstances;
				ReadbackMeshBatch.NumVerticesPerInstance	= NumVerticesPerInstance;
			}
		}

		TArray<FRendererReadbackRequestPtr> ReadbackRequests;
	};

	template<typename T>
	T GetVertexValue(const FNiagaraRendererReadbackResult& Result, uint32 Vertex, uint32 ComponentOffset, uint32 ElementOffset, const T& DefaultValue)
	{
		T Value = DefaultValue;
		if (Vertex < Result.NumVertices && ComponentOffset != INDEX_NONE)
		{
			const uint32 Offset = (Result.VertexStride * Vertex) + ComponentOffset + ElementOffset;
			FMemory::Memcpy(&Value, Result.VertexData.GetData() + Offset, sizeof(T));
		}
		return Value;
	}

	template<typename T>
	T GetVertexValue(const FNiagaraRendererReadbackResult& Result, uint32 Vertex, uint32 ComponentOffset, const T& DefaultValue)
	{
		return GetVertexValue(Result, Vertex, ComponentOffset, 0, DefaultValue);
	}

	void InternalEnqueueReadback(FNiagaraGpuComputeDispatchInterface* ComputeInterface, FRendererReadbackRequestPtr ReadbackRequest, const FNiagaraRendererReadbackParameters& Parameters)
	{
		using namespace NiagaraRendererReadback;

		// Copy over parameters
		ReadbackRequest->bExportMaterials	= Parameters.bExportMaterials;
		ReadbackRequest->bApplyWPO			= Parameters.bApplyWPO;
		ReadbackRequest->bWorldSpace		= Parameters.CoordinateSpace == ENiagaraRendererReadbackCoordinateSpace::World;
		ReadbackRequest->ViewIndexToCapture	= Parameters.ViewIndexToCapture;

		// Build Vertex Output
		{
			FNiagaraRendererReadbackResult& Result = ReadbackRequest->Result;
			if (Parameters.bExportPosition)
			{
				Result.VertexPositionOffset = Result.VertexStride;
				Result.VertexStride += sizeof(FVector3f);
			}
			if (Parameters.bExportTangentBasis)
			{
				Result.VertexTangentBasisOffset = Result.VertexStride;
				Result.VertexStride += sizeof(FVector3f) * 3;
			}
			if (Parameters.bExportColor)
			{
				Result.VertexColorOffset = Result.VertexStride;
				Result.VertexStride += sizeof(FVector4f);
			}
			if (Parameters.ExportNumTexCoords > 0)
			{
				const int32 NumTexCoords = FMath::Min<int32>(Parameters.ExportNumTexCoords, MAX_STATIC_TEXCOORDS);
				Result.VertexTexCoordNum = NumTexCoords;
				Result.VertexTexCoordOffset = Result.VertexStride;
				Result.VertexStride += sizeof(FVector2f) * NumTexCoords;
				for (int32 i=0; i < FMath::Min<int32>(Parameters.ExportTexCoordBindings.Num(), UE_ARRAY_COUNT(Result.VertexTexCoordBindings)); ++i )
				{
					Result.VertexTexCoordBindings[i] = uint32(Parameters.ExportTexCoordBindings[i]);
				}
			}
		}

		ENQUEUE_RENDER_COMMAND(EnqueueRendererReadback)(
			[ComputeInterface, ReadbackRequest](FRHICommandListImmediate&)
			{
				FRendererReadbackComputeManager& ReadbackManager = ComputeInterface->GetOrCreateDataManager<FRendererReadbackComputeManager>();
				ReadbackManager.ReadbackRequests.Emplace(ReadbackRequest);
			}
		);
	}

	void EnqueueReadbackForComponents(TConstArrayView<UNiagaraComponent*> Components, FNiagaraRendererReadbackComplete Callback, const FNiagaraRendererReadbackParameters& Parameters)
	{
		FNiagaraGpuComputeDispatchInterface* ComputeInterface = nullptr;
		FRendererReadbackRequestPtr ReadbackRequest = MakeShared<FRendererReadbackRequest>();
		ReadbackRequest->CompleteCallback = MoveTemp(Callback);

		ReadbackRequest->bCaptureAllPrimitives = false;

		for (UNiagaraComponent* Component : Components)
		{
			UWorld* ComponentWorld = ::IsValid(Component) ? Component->GetWorld() : nullptr;
			if (ComponentWorld == nullptr)
			{
				continue;
			}

			FNiagaraGpuComputeDispatchInterface* ComponentComputeInterface = FNiagaraGpuComputeDispatchInterface::Get(ComponentWorld);
			ComputeInterface = ComputeInterface == nullptr ? ComponentComputeInterface : ComputeInterface;

			if (ComponentComputeInterface == nullptr)
			{
				ReadbackRequest->Result.Errors.AddUnique(TEXT("Request failed due to no compute interface."));
				ExecuteCompleteCallback(ReadbackRequest);
				return;
			}
			else if (ComputeInterface != ComponentComputeInterface)
			{
				ReadbackRequest->Result.Errors.AddUnique(TEXT("Request failed due to components with mismatching compute interface."));
				ExecuteCompleteCallback(ReadbackRequest);
				return;
			}

			ReadbackRequest->PrimitivesToCapture.Add(Component->GetPrimitiveSceneId());
		}

		// Nothing added then there is nothing to do complete
		if (ReadbackRequest->PrimitivesToCapture.Num() == 0)
		{
			ReadbackRequest->Result.Errors.AddUnique(TEXT("No primitives found to capture."));
			ExecuteCompleteCallback(ReadbackRequest);
			return;
		}

		InternalEnqueueReadback(ComputeInterface, ReadbackRequest, Parameters);
	}

	void EnqueueReadbackForWorld(UWorld* World, FNiagaraRendererReadbackComplete Callback, const FNiagaraRendererReadbackParameters& Parameters)
	{
		FRendererReadbackRequestPtr ReadbackRequest = MakeShared<FRendererReadbackRequest>();
		ReadbackRequest->CompleteCallback = MoveTemp(Callback);
		ReadbackRequest->bCaptureAllPrimitives = true;

		FNiagaraGpuComputeDispatchInterface* ComputeInterface = FNiagaraGpuComputeDispatchInterface::Get(World);
		if (ComputeInterface == nullptr)
		{
			ReadbackRequest->Result.Errors.AddUnique(TEXT("Request failed due to no compute interface."));
			ExecuteCompleteCallback(ReadbackRequest);
			return;
		}

		InternalEnqueueReadback(ComputeInterface, ReadbackRequest, Parameters);
	}
} //namespace NiagaraRendererReadback

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FVector3f FNiagaraRendererReadbackResult::GetPosition(uint32 Vertex) const
{
	return NiagaraRendererReadback::GetVertexValue(*this, Vertex, VertexPositionOffset, FVector3f::ZeroVector);
}

FLinearColor FNiagaraRendererReadbackResult::GetColor(uint32 Vertex) const
{
	return NiagaraRendererReadback::GetVertexValue(*this, Vertex, VertexColorOffset, FLinearColor::Black);
}

FVector3f FNiagaraRendererReadbackResult::GetTangentX(uint32 Vertex) const
{
	return NiagaraRendererReadback::GetVertexValue(*this, Vertex, VertexTangentBasisOffset, FVector3f::XAxisVector);
}

FVector3f FNiagaraRendererReadbackResult::GetTangentY(uint32 Vertex) const
{
	return NiagaraRendererReadback::GetVertexValue(*this, Vertex, VertexTangentBasisOffset, sizeof(FVector3f), FVector3f::YAxisVector);
}

FVector3f FNiagaraRendererReadbackResult::GetTangentZ(uint32 Vertex) const
{
	return NiagaraRendererReadback::GetVertexValue(*this, Vertex, VertexTangentBasisOffset, sizeof(FVector3f) * 2, FVector3f::ZAxisVector);
}

FVector2f FNiagaraRendererReadbackResult::GetTexCoord(uint32 Vertex, uint32 TexCoordIndex) const
{
	if (TexCoordIndex >= VertexTexCoordNum)
	{
		return FVector2f::ZeroVector;
	}
	return NiagaraRendererReadback::GetVertexValue(*this, Vertex, VertexTexCoordOffset, sizeof(FVector2f) * TexCoordIndex, FVector2f::ZeroVector);
}

void NiagaraRendererReadback::EnqueueReadback(TConstArrayView<UNiagaraComponent*> Components, FNiagaraRendererReadbackComplete Callback, const FNiagaraRendererReadbackParameters& Parameters)
{
	EnqueueReadbackForComponents(Components, MoveTemp(Callback), Parameters);
}

void NiagaraRendererReadback::EnqueueReadback(UNiagaraComponent* Component, TFunction<void(const FNiagaraRendererReadbackResult& Result)> Callback, const FNiagaraRendererReadbackParameters& Parameters)
{
	EnqueueReadbackForComponents(MakeArrayView(&Component, 1), MoveTemp(Callback), Parameters);
}

void NiagaraRendererReadback::EnqueueReadback(UWorld* World, FNiagaraRendererReadbackComplete Callback, const FNiagaraRendererReadbackParameters& Parameters)
{
	EnqueueReadbackForWorld(World, MoveTemp(Callback), Parameters);
}

void NiagaraRendererReadback::CaptureMeshBatch(const FSceneView* View, const FNiagaraSceneProxy* SceneProxy, const FMeshBatch& MeshBatch, uint32 NumInstances, uint32 NumVerticesPerInstance)
{
	using namespace NiagaraRendererReadback;

	FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = SceneProxy->GetComputeDispatchInterface();
	if (ComputeDispatchInterface == nullptr)
	{
		return;
	}

	FRendererReadbackComputeManager& ReadbackManager = ComputeDispatchInterface->GetOrCreateDataManager<FRendererReadbackComputeManager>();
	ReadbackManager.AddMeshBatch(View, SceneProxy, MeshBatch, NumInstances, NumVerticesPerInstance);
}
#endif //WITH_NIAGARA_RENDERER_READBACK
