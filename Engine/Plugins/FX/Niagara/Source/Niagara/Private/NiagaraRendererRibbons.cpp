// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRendererRibbons.h"

#include "GlobalRenderResources.h"
#include "GPUSortManager.h"
#include "NiagaraRibbonVertexFactory.h"
#include "NiagaraRibbonShared.h"
#include "NiagaraDataSet.h"
#include "NiagaraDataSetAccessor.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraSceneProxy.h"
#include "NiagaraGpuComputeDataManager.h"
#include "NiagaraRendererReadback.h"
#include "NiagaraStats.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraComponent.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RayTracingInstance.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "Math/NumericLimits.h"
#include "NiagaraCullProxyComponent.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraRibbonCompute.h"
#include "RenderGraphUtils.h"
#include "RenderUtils.h"
#include "PrimitiveDrawingUtils.h"

DECLARE_CYCLE_STAT(TEXT("Generate Ribbon Vertex Data [GT]"), STAT_NiagaraGenRibbonVertexData, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Render Ribbons [RT]"), STAT_NiagaraRenderRibbons, STATGROUP_Niagara);

DECLARE_CYCLE_STAT(TEXT("Generate Indices CPU [GT]"), STAT_NiagaraRenderRibbonsGenIndiciesCPU, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Generate Indices GPU [RT]"), STAT_NiagaraRenderRibbonsGenIndiciesGPU, STATGROUP_Niagara);

DECLARE_CYCLE_STAT(TEXT("Generate Vertices CPU [GT]"), STAT_NiagaraRenderRibbonsGenVerticesCPU, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU [RT]"), STAT_NiagaraRenderRibbonsGenVerticesGPU, STATGROUP_Niagara);

DECLARE_STATS_GROUP(TEXT("NiagaraRibbons"), STATGROUP_NiagaraRibbons, STATCAT_Niagara);

DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU - Sort [RT]"), STAT_NiagaraRenderRibbonsGenVerticesSortGPU, STATGROUP_NiagaraRibbons);
DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU - InitialSort [RT]"), STAT_NiagaraRenderRibbonsGenVerticesInitialSortGPU, STATGROUP_NiagaraRibbons);
DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU - FinalSort [RT]"), STAT_NiagaraRenderRibbonsGenVerticesFinalSortGPU, STATGROUP_NiagaraRibbons);
DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU - Reduction Phase 1 [RT]"), STAT_NiagaraRenderRibbonsGenVerticesReductionPhase1GPU, STATGROUP_NiagaraRibbons);
DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU - Reduction Init [RT]"), STAT_NiagaraRenderRibbonsGenVerticesReductionInitGPU, STATGROUP_NiagaraRibbons);
DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU - Reduction Propagate [RT]"), STAT_NiagaraRenderRibbonsGenVerticesReductionPropagateGPU, STATGROUP_NiagaraRibbons);
DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU - Reduction Tessellation [RT]"), STAT_NiagaraRenderRibbonsGenVerticesReductionTessellationGPU, STATGROUP_NiagaraRibbons);
DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU - Reduction Phase 2 [RT]"), STAT_NiagaraRenderRibbonsGenVerticesReductionPhase2GPU, STATGROUP_NiagaraRibbons);

DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU - Reduction Finalize [RT]"), STAT_NiagaraRenderRibbonsGenVerticesReductionFinalizeGPU, STATGROUP_NiagaraRibbons);
DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU - MultiRibbon Init [RT]"), STAT_NiagaraRenderRibbonsGenVerticesMultiRibbonInitGPU, STATGROUP_NiagaraRibbons);
DECLARE_CYCLE_STAT(TEXT("Generate Vertices GPU - MultiRibbon Init Compute [RT]"), STAT_NiagaraRenderRibbonsGenVerticesMultiRibbonInitComputeGPU, STATGROUP_NiagaraRibbons);

DECLARE_GPU_STAT_NAMED(NiagaraGPURibbons, TEXT("Niagara GPU Ribbons"));

int32 GNiagaraRibbonTessellationEnabled = 1;
static FAutoConsoleVariableRef CVarNiagaraRibbonTessellationEnabled(
	TEXT("Niagara.Ribbon.Tessellation.Enabled"),
	GNiagaraRibbonTessellationEnabled,
	TEXT("Determine if we allow tesellation on this platform or not."),
	ECVF_Scalability
);

float GNiagaraRibbonTessellationAngle = 15.f * (2.f * PI) / 360.f; // Every 15 degrees
static FAutoConsoleVariableRef CVarNiagaraRibbonTessellationAngle(
	TEXT("Niagara.Ribbon.Tessellation.MinAngle"),
	GNiagaraRibbonTessellationAngle,
	TEXT("Ribbon segment angle to tesselate in radian. (default=15 degrees)"),
	ECVF_Scalability
);

int32 GNiagaraRibbonMaxTessellation = 16;
static FAutoConsoleVariableRef CVarNiagaraRibbonMaxTessellation(
	TEXT("Niagara.Ribbon.Tessellation.MaxInterp"),
	GNiagaraRibbonMaxTessellation,
	TEXT("When TessellationAngle is > 0, this is the maximum tesselation factor. \n")
	TEXT("Higher values allow more evenly divided tesselation. \n")
	TEXT("When TessellationAngle is 0, this is the actually tesselation factor (default=16)."),
	ECVF_Scalability
);

float GNiagaraRibbonTessellationScreenPercentage = 0.002f;
static FAutoConsoleVariableRef CVarNiagaraRibbonTessellationScreenPercentage(
	TEXT("Niagara.Ribbon.Tessellation.MaxErrorScreenPercentage"),
	GNiagaraRibbonTessellationScreenPercentage,
	TEXT("Screen percentage used to compute the tessellation factor. \n")
	TEXT("Smaller values will generate more tessellation, up to max tesselltion. (default=0.002)"),
	ECVF_Scalability
);

float GNiagaraRibbonTessellationMinDisplacementError = 0.5f;
static FAutoConsoleVariableRef CVarNiagaraRibbonTessellationMinDisplacementError(
	TEXT("Niagara.Ribbon.Tessellation.MinAbsoluteError"),
	GNiagaraRibbonTessellationMinDisplacementError,
	TEXT("Minimum absolute world size error when tessellating. \n")
	TEXT("Prevent over tessellating when distance gets really small. (default=0.5)"),
	ECVF_Scalability
);

float GNiagaraRibbonMinSegmentLength = 1.f;
static FAutoConsoleVariableRef CVarNiagaraRibbonMinSegmentLength(
	TEXT("Niagara.Ribbon.MinSegmentLength"),
	GNiagaraRibbonMinSegmentLength,
	TEXT("Min length of niagara ribbon segments. (default=1)"),
	ECVF_Scalability
);

static bool GbEnableNiagaraRibbonRendering = true;
static FAutoConsoleVariableRef CVarEnableNiagaraRibbonRendering(
	TEXT("fx.EnableNiagaraRibbonRendering"),
	GbEnableNiagaraRibbonRendering,
	TEXT("If false Niagara Ribbon Renderers are disabled."),
	ECVF_Default
);

static int32 GNiagaraRibbonGpuEnabled = 1;
static FAutoConsoleVariableRef CVarNiagaraRibbonGpuEnabled(
	TEXT("Niagara.Ribbon.GpuEnabled"),
	GNiagaraRibbonGpuEnabled,
	TEXT("Enable any GPU ribbon related code (including GPU init)."),
	ECVF_Scalability
);

static int32 GNiagaraRibbonGpuInitMode = 0;
static FAutoConsoleVariableRef CVarNiagaraRibbonGpuInitMode(
	TEXT("Niagara.Ribbon.GpuInitMode"),
	GNiagaraRibbonGpuInitMode,
	TEXT("Modifies the GPU initialization mode used, i.e. offloading CPU calculations to the GPU.\n")
	TEXT("0 = Respect bUseGPUInit from properties (Default)\n")
	TEXT("1 = Force enabled\n")
	TEXT("2 = Force disabled"),
	ECVF_Scalability
);

static int32 GNiagaraRibbonGpuBufferCachePurgeCounter = 0;
static FAutoConsoleVariableRef CVarNiagaraRibbonGpuBufferCachePurgeCounter(
	TEXT("Niagara.Ribbon.GpuBufferCachePurgeCounter"),
	GNiagaraRibbonGpuBufferCachePurgeCounter,
	TEXT("The number of frames we hold onto ribbon buffer for.")
	TEXT("Where 0 (Default) we purge them if not used next frame.")
	TEXT("Negative values will purge the buffers the same frame, essentially zero reusing."),
	ECVF_Default
);

static int32 GNiagaraRibbonGpuAllocateMaxCount = 1;
static FAutoConsoleVariableRef CVarNiagaraRibbonGpuAllocateMaxCount(
	TEXT("Niagara.Ribbon.GpuAllocateMaxCount"),
	GNiagaraRibbonGpuAllocateMaxCount,
	TEXT("When enabled (default) we allocate the maximum number of required elements.")
	TEXT("This can result in memory bloat if the count is highly variable but will be more stable performance wise"),
	ECVF_Default
);

static int32 GNiagaraRibbonGpuBufferAlign = 512;
static FAutoConsoleVariableRef CVarNiagaraRibbonGpuBufferAlign(
	TEXT("Niagara.Ribbon.GpuBufferAlign"),
	GNiagaraRibbonGpuBufferAlign,
	TEXT("When not allocating the maximum number of required elements we align up the request elements to this size to improve buffer reuse."),
	ECVF_Default
);

static bool GNiagaraRibbonShareGeneratedData = true;
static FAutoConsoleVariableRef CVarNiagaraRibbonShareGeneratedData(
	TEXT("Niagara.Ribbon.ShareGeneratedData"),
	GNiagaraRibbonShareGeneratedData,
	TEXT("Allow ribbons to share the generate data where possible."),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarRayTracingNiagaraRibbons(
	TEXT("r.RayTracing.Geometry.NiagaraRibbons"),
	1,
	TEXT("Include Niagara ribbons in ray tracing effects (default = 1 (Niagara ribbons enabled in ray tracing))"));

static TAutoConsoleVariable<int32> CVarRayTracingNiagaraRibbonsGPU(
	TEXT("r.RayTracing.Geometry.NiagaraRibbons.GPU"),
	1,
	TEXT("If we allow GPU ribbon raytracing"));

static bool GNiagaraRibbonForceIndex32 = false;
static FAutoConsoleVariableRef CNiagaraRibbonForceIndex32(
	TEXT("Niagara.Ribbon.ForceIndex32"),
	GNiagaraRibbonForceIndex32,
	TEXT("Force creating 32 bits index buffers for the ribbons."),
	ECVF_Default
);

namespace NiagaraRendererRibbonsPrivate
{
	// max absolute error 9.0x10^-3
	// Eberly's polynomial degree 1 - respect bounds
	// input [-1, 1] and output [0, PI]
	static float AcosFast(float InX)
	{
		float X = FMath::Abs(InX);
		float Res = -0.156583f * X + (0.5f * PI);
		Res *= FMath::Sqrt(FMath::Max(0.f, 1.0f - X));
		return (InX >= 0) ? Res : PI - Res;
	}

	// Calculates the number of bits needed to store a maximum value
	static uint32 CalculateBitsForRange(uint32 Range)
	{
		return FMath::CeilToInt(FMath::Loge(static_cast<float>(Range)) / FMath::Loge(static_cast<float>(2)));
	}

	// Generates the mask to remove unecessary bits after a range of bits
	static uint32 CalculateBitMask(uint32 NumBits)
	{
		return static_cast<uint32>(static_cast<uint64>(0xFFFFFFFF) >> (32 - NumBits));
	}

	static ENiagaraGpuTessellationMode::Type GetGpuTessellationMode(const FNiagaraRibbonGenerationConfig& GenerationConfig)
	{
		if (GenerationConfig.WantsAutomaticTessellation())
		{
			return ENiagaraGpuTessellationMode::Automatic;
		}
		else if (GenerationConfig.WantsConstantTessellation())
		{
			return ENiagaraGpuTessellationMode::Constant;
		}
		return ENiagaraGpuTessellationMode::Disabled;
	}
} //namespace NiagaraRendererRibbonsPrivate

// This data must match INDEX_GEN_INDIRECT_ARGS_STRIDE in NiagaraRibbonCommon.ush
// Be careful if we ever allocate more than 1 of these as ExecuteIndirect arguments have boundary restrictions on some platforms
struct FNiagaraRibbonIndirectDrawBufferLayout
{
	FRHIDispatchIndirectParametersNoPadding	IndexGenIndirectArgs;			//  0 - 3 uints
	FRHIDrawIndexedIndirectParameters		DrawIndirectParameters;			//  3 - 5 uints
	FRHIDrawIndexedIndirectParameters		StereoDrawIndirectParameters;	//  8 - 5 uints
	
	uint32	TessellationFactor;												// 13 - 1 uint
	uint32	NumSegments;													// 14 - 1 uint
	uint32	NumSubSegments;													// 15 - 1 uint
	float	OneOverSubSegmentCount;											// 16 - 1 uint

	static constexpr int32 DrawIndirectParametersByteOffset			= 3 * sizeof(uint32);
	static constexpr int32 StereoDrawIndirectParametersByteOffset	= 8 * sizeof(uint32);
	static constexpr int32 NumElements								= 17;
};

struct FNiagaraRibbonIndexBuffer final : FIndexBuffer
{
	FNiagaraRibbonIndexBuffer() {}

	virtual ~FNiagaraRibbonIndexBuffer() override
	{
		FRenderResource::ReleaseResource();
	}

	// CPU allocation path
	void Initialize(FRHICommandListBase& RHICmdList, FGlobalDynamicIndexBuffer::FAllocationEx& IndexAllocation)
	{
		InitResource(RHICmdList);

		IndexBufferRHI = IndexAllocation.IndexBuffer->IndexBufferRHI;
		FirstIndex = IndexAllocation.FirstIndex;
	#if RHI_RAYTRACING
		if (IsRayTracingAllowed())
		{
			const uint32 IndexStride = IndexBufferRHI->GetDesc().Stride;
			SRV = RHICmdList.CreateShaderResourceView(
				IndexBufferRHI, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(IndexStride == 2 ? PF_R16_UINT : PF_R32_UINT));
		}
	#endif
	}

	// GPU allocation path assumes 32 bit indicies
	void Initialize(FRHICommandListBase& RHICmdList, const uint32 NumElements)
	{
		InitResource(RHICmdList);

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateIndex<uint32>(TEXT("NiagaraRibbonIndexBuffer"), NumElements)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask | ERHIAccess::VertexOrIndexBuffer);

		IndexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	#if RHI_RAYTRACING
		SRV = RHICmdList.CreateShaderResourceView(
			IndexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
	#endif
		UAV = RHICmdList.CreateUnorderedAccessView(
			IndexBufferRHI, 
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
	}

	virtual void ReleaseRHI() override
	{
		UAV.SafeRelease();
	#if RHI_RAYTRACING
		SRV.SafeRelease();
	#endif
		FIndexBuffer::ReleaseRHI();
	}

	uint32						FirstIndex = 0;
#if RHI_RAYTRACING
	FShaderResourceViewRHIRef	SRV;
#endif
	FUnorderedAccessViewRHIRef	UAV;
};


struct FNiagaraDynamicDataRibbon : public FNiagaraDynamicDataBase
{
	FNiagaraDynamicDataRibbon(const FNiagaraEmitterInstance* InEmitter)
		: FNiagaraDynamicDataBase(InEmitter)
	{
	}

	virtual void ApplyMaterialOverride(int32 MaterialIndex, UMaterialInterface* MaterialOverride) override
	{
		if (MaterialIndex == 0 && MaterialOverride)
		{
			Material = MaterialOverride->GetRenderProxy();
		}
	}

	
	FMaterialRenderProxy* Material = nullptr;		// MaterialProxy used on the renderer
	uint32 MaxAllocationCount = 0;					// Maximum allocation count allowed (i.e. max we can fit in a buffer)
	uint32 MaxAllocatedCountEstimate = 0;			// Maximum allocated count ever seen / estimate, since this is updated on the GT might be lower than actual particle count

	bool bUseGPUInit = false;
	bool bIsGPUSystem = false;
	
	TSharedPtr<FNiagaraRibbonCPUGeneratedVertexData> GenerationOutput;
	
	int32 GetAllocatedSize()const
	{
		return GenerationOutput.IsValid()? GenerationOutput->GetAllocatedSize() : 0;
	}
};


struct FNiagaraRibbonRenderingFrameViewResources
{
	FNiagaraRibbonVertexFactory				VertexFactory;
	FNiagaraRibbonUniformBufferRef			UniformBuffer;
	TSharedPtr<FNiagaraRibbonIndexBuffer>	IndexBuffer;
	TSharedPtr<FRWBuffer>					IndirectDrawBuffer;
	FNiagaraIndexGenerationInput			IndexGenerationSettings;
	bool									bNeedsIndexBufferGeneration = true;

	~FNiagaraRibbonRenderingFrameViewResources()
	{
		UniformBuffer.SafeRelease();
		VertexFactory.ReleaseResource();
	}
};

struct FNiagaraRibbonRenderingFrameResources
{
	TArray<TSharedPtr<FNiagaraRibbonRenderingFrameViewResources>> ViewResources;

	FParticleRenderData ParticleData;
		
	FRHIShaderResourceView*	ParticleFloatSRV;
	FRHIShaderResourceView* ParticleHalfSRV;
	FRHIShaderResourceView* ParticleIntSRV;
	
	int32 ParticleFloatDataStride = INDEX_NONE;
	int32 ParticleHalfDataStride = INDEX_NONE;
	int32 ParticleIntDataStride = INDEX_NONE;
	
	int32 RibbonIdParamOffset = INDEX_NONE;
	
	~FNiagaraRibbonRenderingFrameResources()
	{
		ViewResources.Empty();

		ParticleFloatSRV = nullptr;
		ParticleHalfSRV = nullptr;
		ParticleIntSRV = nullptr;

		ParticleFloatDataStride = INDEX_NONE;
		ParticleHalfDataStride = INDEX_NONE;
		ParticleIntDataStride = INDEX_NONE;
		
		RibbonIdParamOffset = INDEX_NONE;
	}	
};

struct FNiagaraRibbonGPUInitParameters
{
	FNiagaraRibbonGPUInitParameters(const FNiagaraRendererRibbons* InRenderer, const FNiagaraDataBuffer* InSourceParticleData, const TSharedPtr<FNiagaraRibbonRenderingFrameResources>& InRenderingResources)
		: Renderer(InRenderer)
		, NumInstances(InSourceParticleData->GetNumInstances())
		, GPUInstanceCountBufferOffset(InSourceParticleData->GetGPUInstanceCountBufferOffset())
		, RenderingResources(InRenderingResources)
	{
	}

	const FNiagaraRendererRibbons*	Renderer = nullptr;
	const uint32					NumInstances = 0;
	const uint32					GPUInstanceCountBufferOffset = 0;
	TWeakPtr<FNiagaraRibbonRenderingFrameResources> RenderingResources;
};

class FNiagaraRibbonMeshCollectorResources : public FOneFrameResource
{
public:
	TSharedRef<FNiagaraRibbonRenderingFrameResources> RibbonResources;

	FNiagaraRibbonMeshCollectorResources()
		: RibbonResources(new FNiagaraRibbonRenderingFrameResources())
	{
		
	}
};

bool FNiagaraRibbonGpuBuffer::Allocate(FRHICommandListBase& RHICmdList, uint32 NumElements, uint32 MaxElements, ERHIAccess InResourceState, bool bGpuReadOnly, EBufferUsageFlags AdditionalBufferUsage)
{
	if (NumElements == 0)
	{
		Release();
		return false;
	}
	else
	{
		static constexpr float UpsizeMultipler = 1.1f;
		static constexpr float DownsizeMultiplier = 1.2f;

		check(NumElements <= MaxElements);

		const bool bGpuUsageChanged = bGpuReadOnly ? UAV != nullptr : UAV == nullptr;

		const uint32 CurrentElements = NumBytes / ElementBytes;
		if (bGpuUsageChanged || CurrentElements < NumElements || CurrentElements > uint32(FMath::CeilToInt32(NumElements * DownsizeMultiplier)))
		{
			NumElements = FMath::Min(MaxElements, uint32(FMath::RoundToInt32(NumElements * UpsizeMultipler)));
			NumBytes = ElementBytes * NumElements;

			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(DebugName, NumBytes)
				.AddUsage(AdditionalBufferUsage | EBufferUsageFlags::ShaderResource | (bGpuReadOnly ? EBufferUsageFlags::Volatile : EBufferUsageFlags::Static | EBufferUsageFlags::UnorderedAccess))
				.SetInitialState(InResourceState);

			Buffer = RHICmdList.CreateBuffer(CreateDesc);
			SRV = RHICmdList.CreateShaderResourceView(
				Buffer, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PixelFormat));
			UAV = bGpuReadOnly ? nullptr : RHICmdList.CreateUnorderedAccessView(
				Buffer, 
				FRHIViewDesc::CreateBufferUAV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PixelFormat));
			return true;
		}
	}
	return false;
}

void FNiagaraRibbonGpuBuffer::Release()
{
	NumBytes = 0;
	Buffer.SafeRelease();
	UAV.SafeRelease();
	SRV.SafeRelease();
}

FNiagaraRibbonVertexBuffers::FNiagaraRibbonVertexBuffers()
	: SortedIndicesBuffer(TEXT("RibbonSortedIndices"), EPixelFormat::PF_R32_UINT, sizeof(uint32))
	, TangentsAndDistancesBuffer(TEXT("TangentsAndDistancesBuffer"), EPixelFormat::PF_R32_FLOAT, sizeof(float))
	, MultiRibbonIndicesBuffer(TEXT("MultiRibbonIndicesBuffer"), EPixelFormat::PF_R32_UINT, sizeof(uint32))
	, RibbonLookupTableBuffer(TEXT("RibbonLookupTableBuffer"), EPixelFormat::PF_R32_UINT, sizeof(uint32))
	, SegmentsBuffer(TEXT("SegmentsBuffer"), EPixelFormat::PF_R32_UINT, sizeof(uint32))
	, GPUComputeCommandBuffer(TEXT("GPUComputeCommandBuffer"), EPixelFormat::PF_R32_UINT, sizeof(uint32))
{
}

void FNiagaraRibbonVertexBuffers::InitializeOrUpdateBuffers(FRHICommandListBase& RHICmdList, const FNiagaraRibbonGenerationConfig& GenerationConfig, const TSharedPtr<FNiagaraRibbonCPUGeneratedVertexData>& GeneratedGeometryData, const FNiagaraDataBuffer* SourceParticleData, int32 MaxAllocationCount, bool bIsUsingGPUInit)
{	
	constexpr ERHIAccess InitialBufferAccessFlags = ERHIAccess::SRVMask;

	if (bIsUsingGPUInit)
	{
		const uint32 TotalParticles = SourceParticleData->GetNumInstances();
		const uint32 MaxAllocatedRibbons = GenerationConfig.HasRibbonIDs() ? (GenerationConfig.GetMaxNumRibbons() > 0 ? GenerationConfig.GetMaxNumRibbons() : TotalParticles) : 1;

		//-OPT:  We should be able to assume 2 particles per ribbon, however the compute pass does not cull our single particle ribbons therefore we need to allocate
		//       enough space to assume each particle will be the start of a unique ribbon to avoid running OOB on the buffers.
		const int32 TotalRibbons = FMath::Clamp<int32>(TotalParticles, 1, MaxAllocatedRibbons);

		SortedIndicesBuffer.Allocate(RHICmdList, TotalParticles, MaxAllocationCount, InitialBufferAccessFlags | ERHIAccess::VertexOrIndexBuffer, false, EBufferUsageFlags::None);
		TangentsAndDistancesBuffer.Allocate(RHICmdList, TotalParticles * 4, MaxAllocationCount * 4, InitialBufferAccessFlags, false);
		MultiRibbonIndicesBuffer.Allocate(RHICmdList, GenerationConfig.HasRibbonIDs() ? TotalParticles : 0, MaxAllocationCount, InitialBufferAccessFlags, false);
		RibbonLookupTableBuffer.Allocate(RHICmdList, TotalRibbons * FRibbonMultiRibbonInfoBufferEntry::NumElements, MaxAllocatedRibbons * FRibbonMultiRibbonInfoBufferEntry::NumElements, InitialBufferAccessFlags, false);
		SegmentsBuffer.Allocate(RHICmdList, TotalParticles, MaxAllocationCount, InitialBufferAccessFlags, false);
		bJustCreatedCommandBuffer |= GPUComputeCommandBuffer.Allocate(RHICmdList, VERTEX_GEN_OUTPUT_DATA_AUTO_TESSELLATION_NUM_ELEMENTS, VERTEX_GEN_OUTPUT_DATA_AUTO_TESSELLATION_NUM_ELEMENTS, InitialBufferAccessFlags | ERHIAccess::IndirectArgs, false, EBufferUsageFlags::DrawIndirect);
	}
	else
	{		
		check(GeneratedGeometryData.IsValid());

		SortedIndicesBuffer.Allocate(RHICmdList, GeneratedGeometryData->SortedIndices.Num(), MaxAllocationCount, InitialBufferAccessFlags | ERHIAccess::VertexOrIndexBuffer, true, EBufferUsageFlags::None);
		TangentsAndDistancesBuffer.Allocate(RHICmdList, GeneratedGeometryData->TangentAndDistances.Num() * 4, MaxAllocationCount * 4, InitialBufferAccessFlags, true);
		MultiRibbonIndicesBuffer.Allocate(RHICmdList, GenerationConfig.HasRibbonIDs() ? GeneratedGeometryData->MultiRibbonIndices.Num() : 0, MaxAllocationCount, InitialBufferAccessFlags, true);
		RibbonLookupTableBuffer.Allocate(RHICmdList, GeneratedGeometryData->RibbonInfoLookup.Num() * FRibbonMultiRibbonInfoBufferEntry::NumElements, MaxAllocationCount * FRibbonMultiRibbonInfoBufferEntry::NumElements, InitialBufferAccessFlags, true);
		SegmentsBuffer.Release();
		GPUComputeCommandBuffer.Release();
		bJustCreatedCommandBuffer = false;
	}
}

struct FNiagaraRibbonGPUInitComputeBuffers
{
	FRWBuffer SortBuffer;
	FRWBuffer TransientTessellationStats;
	FRWBufferStructured TransientAccumulation[2];
	
	FNiagaraRibbonGPUInitComputeBuffers() { }

	void InitOrUpdateBuffers(FRHICommandListBase& RHICmdList, int32 NeededSize, bool bWantsMultiRibbon, bool bWantsTessellation, bool bWantsTessellationTwist)
	{
		using namespace NiagaraRendererRibbonsPrivate;

		// TODO: Downsize these when we haven't needed the size for a bit
		constexpr ERHIAccess InitialAccess = ERHIAccess::SRVMask;
		
		if (SortBuffer.NumBytes < NeededSize * sizeof(int32))
		{
			SortBuffer.Initialize(RHICmdList, TEXT("NiagarGPUInit-SortedIndices"), sizeof(uint32), NeededSize, EPixelFormat::PF_R32_UINT, InitialAccess | ERHIAccess::VertexOrIndexBuffer);
		}

		const uint32 TessellationBufferSize = NeededSize * (bWantsTessellation ? (bWantsTessellationTwist ? NIAGARA_RIBBON_TESSELLATION_FULL_STATS_NUM_ELEMENTS : NIAGARA_RIBBON_TESSELLATION_NO_TWIST_STATS_NUM_ELEMENTS) : 0);
		if (TransientTessellationStats.NumBytes < TessellationBufferSize * sizeof(float))
		{
			TransientTessellationStats.Initialize(RHICmdList, TEXT("NiagaraGPUInit-Tessellation-0"), sizeof(float), TessellationBufferSize, EPixelFormat::PF_R32_FLOAT, InitialAccess, BUF_Static);
		}

		const uint32 AccumulationBufferStructSize = sizeof(FRibbonAccumulationValues);
		if (TransientAccumulation[0].NumBytes < (AccumulationBufferStructSize * NeededSize))
		{
			TransientAccumulation[0].Initialize(RHICmdList, TEXT("NiagaraGPUInit-Accumulation-0"), AccumulationBufferStructSize, NeededSize, BUF_Static, false, false, InitialAccess);
			TransientAccumulation[1].Initialize(RHICmdList, TEXT("NiagaraGPUInit-Accumulation-1"), AccumulationBufferStructSize, NeededSize, BUF_Static, false, false, InitialAccess);
		}
	}	
};

class FNiagaraGpuRibbonsDataManager final : public FNiagaraGpuComputeDataManager
{
	struct FIndirectDrawBufferEntry
	{
		uint64					FrameUsed = 0;
		TSharedPtr<FRWBuffer>	Buffer;
	};

	struct FIndexBufferEntry
	{
		uint64									FrameUsed = 0;
		int32									NumIndices = 0;
		TSharedPtr<FNiagaraRibbonIndexBuffer>	Buffer;
	};

public:
	FNiagaraGpuRibbonsDataManager(FNiagaraGpuComputeDispatchInterface* InOwnerInterface)
		: FNiagaraGpuComputeDataManager(InOwnerInterface)
	{
		FGPUSortManager* SortManager = InOwnerInterface->GetGPUSortManager();
		SortManager->PostPreRenderEvent.AddRaw(this, &FNiagaraGpuRibbonsDataManager::OnPostPreRender);
		SortManager->PostPostRenderEvent.AddRaw(this, &FNiagaraGpuRibbonsDataManager::OnPostPostRender);
	}

	virtual ~FNiagaraGpuRibbonsDataManager()
	{
	}

	static FName GetManagerName()
	{
		static FName ManagerName("FNiagaraGpuRibbonsDataManager");
		return ManagerName;
	}

	void RegisterRenderer(const FNiagaraRendererRibbons* InRenderer, const FNiagaraDataBuffer* InSourceParticleData, const TSharedPtr<FNiagaraRibbonRenderingFrameResources>& InRenderingResources)
	{
		if (InRenderingResources->ViewResources.Num() > 0)
		{
			UE::TScopeLock ScopeLock(AllocateGuard);

			const int32 GenerateIndex = InSourceParticleData->GetGPUDataReadyStage() == ENiagaraGpuComputeTickStage::PostOpaqueRender ? 1 : 0;
			RenderersToGeneratePerStage[GenerateIndex].Emplace(InRenderer, InSourceParticleData, InRenderingResources);
		}
	}

	FNiagaraRibbonRenderingFrameViewResources* FindExistingRendererViewData(const FNiagaraRendererRibbons* InRenderer, const FNiagaraDataBuffer* InSourceParticleData)
	{
		UE::TScopeLock ScopeLock(AllocateGuard);

		const int32 GenerateIndex = InSourceParticleData->GetGPUDataReadyStage() == ENiagaraGpuComputeTickStage::PostOpaqueRender ? 1 : 0;
		for (FNiagaraRibbonGPUInitParameters& ExistingData : RenderersToGeneratePerStage[GenerateIndex])
		{
			if (ExistingData.Renderer == InRenderer)
			{
				TSharedPtr<FNiagaraRibbonRenderingFrameResources> RenderingResources = ExistingData.RenderingResources.Pin();
				return RenderingResources && RenderingResources->ViewResources.Num() > 0 ? RenderingResources->ViewResources[0].Get() : nullptr;
			}
		}
		return nullptr;
	}

	//-OPT: These caches should be more central and are as a simple solution to reduce memory thrashing / poor performance for ribbons
	TSharedPtr<FRWBuffer> GetOrAllocateIndirectDrawBuffer(FRHICommandListBase& RHICmdList)
	{
		UE::TScopeLock ScopeLock(AllocateGuard);

		FIndirectDrawBufferEntry* BufferEntry = IndirectDrawBufferCache.FindByPredicate(
			[&](const FIndirectDrawBufferEntry& ExistingBuffer)
			{
				return ExistingBuffer.FrameUsed != FrameCounter;
			}
		);
		if (BufferEntry == nullptr)
		{
			BufferEntry = &IndirectDrawBufferCache.AddDefaulted_GetRef();
			BufferEntry->Buffer = MakeShared<FRWBuffer>();
			BufferEntry->Buffer->Initialize(RHICmdList, TEXT("RibbonIndirectDrawBuffer"), sizeof(uint32), FNiagaraRibbonIndirectDrawBufferLayout::NumElements, EPixelFormat::PF_R32_UINT, ERHIAccess::IndirectArgs | ERHIAccess::SRVMask, BUF_Static | BUF_DrawIndirect);
		}
		BufferEntry->FrameUsed = FrameCounter;
		return BufferEntry->Buffer;
	}

	TSharedPtr<FNiagaraRibbonIndexBuffer> GetOrAllocateIndexBuffer(FRHICommandListBase& RHICmdList, int32 NumIndices, int32 MaxIndicesEstimate)
	{
		UE::TScopeLock ScopeLock(AllocateGuard);

		if (GNiagaraRibbonGpuBufferCachePurgeCounter >= 0)
		{
			NumIndices = GNiagaraRibbonGpuAllocateMaxCount == 0 ? Align(NumIndices, GNiagaraRibbonGpuBufferAlign) : MaxIndicesEstimate;
		}

		FIndexBufferEntry* BufferEntry = Index32BufferCache.FindByPredicate(
			[&](const FIndexBufferEntry& ExistingBuffer)
			{
				return ExistingBuffer.FrameUsed != FrameCounter && ExistingBuffer.NumIndices == NumIndices;
			}
		);
		if (BufferEntry == nullptr)
		{
			BufferEntry = &Index32BufferCache.AddDefaulted_GetRef();
			BufferEntry->NumIndices = NumIndices;
			BufferEntry->Buffer = MakeShared<FNiagaraRibbonIndexBuffer>();
			BufferEntry->Buffer->Initialize(RHICmdList, NumIndices);
		}

		BufferEntry->FrameUsed = FrameCounter;
		return BufferEntry->Buffer;
	}

private:
	TArray<FNiagaraRibbonGPUInitParameters>	RenderersToGeneratePerStage[2];
	FNiagaraRibbonGPUInitComputeBuffers		ComputeBuffers;

	TArray<FIndirectDrawBufferEntry>		IndirectDrawBufferCache;
	TArray<FIndexBufferEntry>				Index32BufferCache;

	uint64									FrameCounter = 0;

	UE::FMutex								AllocateGuard;

	void OnPostPreRender(FRHICommandListImmediate& RHICmdList)
	{
		if (GNiagaraRibbonGpuBufferCachePurgeCounter < 0)
		{
			IndirectDrawBufferCache.Empty();
			Index32BufferCache.Empty();
		}
		else
		{
			const uint64 PurgeCounter = GNiagaraRibbonGpuBufferCachePurgeCounter;
			IndirectDrawBufferCache.RemoveAll([&](const FIndirectDrawBufferEntry& InBuffer) { return FrameCounter - InBuffer.FrameUsed > PurgeCounter; });
			Index32BufferCache.RemoveAll([&](const FIndexBufferEntry& InBuffer) { return FrameCounter - InBuffer.FrameUsed > PurgeCounter; });
			++FrameCounter;
		}

		GenerateAllGPUData(RHICmdList, RenderersToGeneratePerStage[0]);
	}

	void OnPostPostRender(FRHICommandListImmediate& RHICmdList)
	{
		GenerateAllGPUData(RHICmdList, RenderersToGeneratePerStage[1]);
	}

	void GenerateAllGPUData(FRHICommandListImmediate& RHICmdList, TArray<FNiagaraRibbonGPUInitParameters>& RenderersToGenerate);
};

struct FNiagaraGenerationInputDataCPUAccessors
{
	FNiagaraGenerationInputDataCPUAccessors(const UNiagaraRibbonRendererProperties * Properties, const FNiagaraDataSet & Data)
		: TotalNumParticles(Data.GetCurrentDataChecked().GetNumInstances())
		, RibbonLinkOrderFloatData(Properties->RibbonLinkOrderFloatAccessor.GetReader(Data))
		, RibbonLinkOrderInt32Data(Properties->RibbonLinkOrderInt32Accessor.GetReader(Data))
		, SimpleRibbonIDData(Properties->RibbonIdDataSetAccessor.GetReader(Data))
		, FullRibbonIDData(Properties->RibbonFullIDDataSetAccessor.GetReader(Data))
		, PosData(Properties->PositionDataSetAccessor.GetReader(Data))
		, AgeData(Properties->NormalizedAgeAccessor.GetReader(Data))
		, SizeData(Properties->SizeDataSetAccessor.GetReader(Data))
		, TwistData(Properties->TwistDataSetAccessor.GetReader(Data))
	{
	}

	template<typename TContainer>
	void RibbonLinkOrderSort(TContainer& Container) const
	{
		if (RibbonLinkOrderFloatData.IsValid())
		{
			Container.Sort([this](const uint32& A, const uint32& B) { return RibbonLinkOrderFloatData[A] < RibbonLinkOrderFloatData[B]; });
		}
		else
		{
			Container.Sort([this](const uint32& A, const uint32& B) { return RibbonLinkOrderInt32Data[A] > RibbonLinkOrderInt32Data[B]; });
		}
	}

	bool HasRibbonLinkOrder() const { return RibbonLinkOrderFloatData.IsValid() || RibbonLinkOrderInt32Data.IsValid(); }

	const uint32 TotalNumParticles;

	const FNiagaraDataSetReaderFloat<float>	RibbonLinkOrderFloatData;
	const FNiagaraDataSetReaderInt32<int32>	RibbonLinkOrderInt32Data;

	const FNiagaraDataSetReaderInt32<int> SimpleRibbonIDData;
	const FNiagaraDataSetReaderStruct<FNiagaraID> FullRibbonIDData;

	const FNiagaraDataSetReaderFloat<FNiagaraPosition> PosData;
	const FNiagaraDataSetReaderFloat<float> AgeData;
	const FNiagaraDataSetReaderFloat<float> SizeData;
	const FNiagaraDataSetReaderFloat<float> TwistData;
};

FNiagaraRendererRibbons::FNiagaraRendererRibbons(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter)
	: FNiagaraRenderer(FeatureLevel, InProps, Emitter)
	, GenerationConfig(CastChecked<const UNiagaraRibbonRendererProperties>(InProps))
	, FacingMode(ENiagaraRibbonFacingMode::Screen)
{
	const UNiagaraRibbonRendererProperties* Properties = CastChecked<const UNiagaraRibbonRendererProperties>(InProps);

	int32 IgnoredFloatOffset, IgnoredHalfOffset;
	Emitter->GetParticleData().GetVariableComponentOffsets(Properties->RibbonIdBinding.GetDataSetBindableVariable(), IgnoredFloatOffset, RibbonIDParamDataSetOffset, IgnoredHalfOffset);

	// Check we actually have ribbon id if we claim we do
	check(!GenerationConfig.HasRibbonIDs() || RibbonIDParamDataSetOffset != INDEX_NONE);
	
	UV0Settings = Properties->UV0Settings;
	UV1Settings = Properties->UV1Settings;
	FacingMode = Properties->FacingMode;
	DrawDirection = Properties->DrawDirection;
	RendererLayout = &Properties->RendererLayout;
	bCastShadows = Properties->bCastShadows;
#if WITH_EDITORONLY_DATA
	bIncludeInHitProxy = Properties->bIncludeInHitProxy;
#endif
	bUseGeometryNormals = Properties->bUseGeometryNormals;
	bGpuRibbonLinkIsFloat = Properties->bGpuRibbonLinkIsFloat;
	GpuRibbonLinkOrderOffset = Properties->GpuRibbonLinkOrderOffset;

	InitializeShape(Properties);
	InitializeTessellation(Properties);	
}

FNiagaraRendererRibbons::~FNiagaraRendererRibbons()
{
}

// FPrimitiveSceneProxy interface.
void FNiagaraRendererRibbons::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	FNiagaraRenderer::CreateRenderThreadResources(RHICmdList);

	{
		// Initialize the shape vertex buffer. This doesn't change frame-to-frame, so we can set it up once
		const int32 NumElements = ShapeState.SliceTriangleToVertexIds.Num();
		ShapeState.SliceTriangleToVertexIdsBuffer.InitializeWithData(RHICmdList, TEXT("SliceTriangleToVertexIdsBuffer"), sizeof(uint32), NumElements, PF_R32_UINT, BUF_Static,
			[this](FRHIBufferInitializer& Initializer)
			{
				Initializer.WriteData(ShapeState.SliceTriangleToVertexIds.GetData(), Initializer.GetWritableDataSize());
			});
	}

	{
		// Initialize the shape vertex buffer. This doesn't change frame-to-frame, so we can set it up once
		const int32 NumElements = ShapeState.SliceVertexData.Num() * FNiagaraRibbonShapeGeometryData::FVertex::NumElements;
		ShapeState.SliceVertexDataBuffer.InitializeWithData(RHICmdList, TEXT("NiagaraShapeVertexDataBuffer"), sizeof(float), NumElements, PF_R32_FLOAT, BUF_Static,
			[this](FRHIBufferInitializer& Initializer)
			{
				Initializer.WriteData(ShapeState.SliceVertexData.GetData(), Initializer.GetWritableDataSize());
			});
	}
	
	
#if RHI_RAYTRACING
	if (IsRayTracingAllowed())
	{
		FRayTracingGeometryInitializer Initializer;
		static const FName DebugName("FNiagaraRendererRibbons");
		static int32 DebugNumber = 0;
		Initializer.DebugName = FDebugName(DebugName, DebugNumber++);
		Initializer.IndexBuffer = nullptr;
		Initializer.TotalPrimitiveCount = 0;
		Initializer.GeometryType = RTGT_Triangles;
		Initializer.bFastBuild = true;
		Initializer.bAllowUpdate = false;
		RayTracingGeometry.SetInitializer(Initializer);
		RayTracingGeometry.InitResource(RHICmdList);
	}
#endif
}

void FNiagaraRendererRibbons::ReleaseRenderThreadResources()
{
	FNiagaraRenderer::ReleaseRenderThreadResources();

	ShapeState.SliceTriangleToVertexIdsBuffer.Release();
	ShapeState.SliceVertexDataBuffer.Release();
	
#if RHI_RAYTRACING
	if (IsRayTracingAllowed())
	{
		RayTracingGeometry.ReleaseResource();
		RayTracingDynamicVertexBuffer.Release();
	}
#endif
}

void FNiagaraRendererRibbons::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector, const FNiagaraSceneProxy *SceneProxy) const
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraRenderRibbons);
	PARTICLE_PERF_STAT_CYCLES_RT(SceneProxy->GetProxyDynamicData().PerfStatsContext, GetDynamicMeshElements);

	FNiagaraDynamicDataRibbon* DynamicData = static_cast<FNiagaraDynamicDataRibbon*>(DynamicDataRender);
	if (!DynamicData)
	{
		return;
	}

	FNiagaraDataBuffer* SourceParticleData = DynamicData->GetParticleDataToRender(Collector.GetRHICommandList());

	if (GbEnableNiagaraRibbonRendering == false || SourceParticleData == nullptr)
	{
		return;
	}

	if (DynamicData->bIsGPUSystem)
	{
		// Bail if we don't have enough particle data to have a valid ribbon
		// or if somehow the sim targets don't match
		if (SimTarget != ENiagaraSimTarget::GPUComputeSim || SourceParticleData->GetNumInstances() < 2)
		{
			return;
		}
	}
	else
	{
		check(SimTarget == ENiagaraSimTarget::CPUSim);

		if (SourceParticleData->GetNumInstances() < 2)
		{
			// Bail if we don't have enough particle data to have a valid ribbon
			return;
		}

		if (!DynamicData->bUseGPUInit && (
			!DynamicData->GenerationOutput.IsValid() ||
			DynamicData->GenerationOutput->SegmentData.Num() <= 0))
		{
			return;
		}
	}

	// Material should never be nullptr, it will point to the Default Material if it was null from the game thread
	check(DynamicData->Material);
	const FMaterial& Material = DynamicData->Material->GetIncompleteMaterialWithFallback(FeatureLevel);

	const bool bTranslucentMaterial = IsTranslucentBlendMode(Material);
	const bool bCanCastTranslucentShadows = SceneProxy->CastsVolumetricTranslucentShadow() || Material.GetCastDynamicShadowAsMasked();
	if (bTranslucentMaterial && AreViewsRenderingOpaqueOnly(Views, VisibilityMap, bCanCastTranslucentShadows))
	{
		return;
	}

	FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();

#if STATS
	FScopeCycleCounter EmitterStatsCounter(EmitterStatID);
#endif
	
	const FNiagaraRibbonMeshCollectorResources& RenderingResources = Collector.AllocateOneFrameResource<FNiagaraRibbonMeshCollectorResources>();
		
	InitializeVertexBuffersResources(RHICmdList, DynamicData, SourceParticleData, Collector.GetDynamicReadBuffer(), RenderingResources.RibbonResources, DynamicData->bUseGPUInit);

	FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = SceneProxy->GetComputeDispatchInterface();
	FNiagaraGpuRibbonsDataManager& GpuRibbonDataManager = ComputeDispatchInterface->GetOrCreateDataManager<FNiagaraGpuRibbonsDataManager>();

	// Can we share the generated data across different view families for this frame?
	// Note: We only handle this for GPU currently
	const bool bShareAcrossViewFamilies = GNiagaraRibbonShareGeneratedData && (DynamicData->bUseGPUInit || DynamicData->bIsGPUSystem);

	// Do we need to generate a per view buffer (i.e. split screen data)
	// Note: GPU can always share as we don't generate anything per view dependent and CPU can only share if not using multi ribbons or opaque
	const bool bNeedsBufferPerView =
		!GNiagaraRibbonShareGeneratedData ||
		(
			!DynamicData->bUseGPUInit && !DynamicData->bIsGPUSystem &&
			bTranslucentMaterial &&
			(DynamicData->GenerationOutput && DynamicData->GenerationOutput->RibbonInfoLookup.Num() > 0)
		);

	bool bNeedsVertexIndexGeneration = false;

	// Compute the per-view uniform buffers.
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];
			check(View);

			if (View->bIsInstancedStereoEnabled && IStereoRendering::IsStereoEyeView(*View) && !IStereoRendering::IsAPrimaryView(*View))
			{
				// We don't have to generate batches for non-primary views in stereo instance rendering
				continue;
			}

			// If we are rendering opaque only we can skip this batch
			//-OPT: If we only have opaque materials we can skip earlier however due to RemappedMaterialIndex potentially being invalid this is tricky
			if (bTranslucentMaterial && IsViewRenderingOpaqueOnly(View, bCanCastTranslucentShadows))
			{
				continue;
			}

			FMeshBatch& MeshBatch = Collector.AllocateMesh();
		#if WITH_EDITORONLY_DATA
			if (bIncludeInHitProxy == false)
			{
				MeshBatch.BatchHitProxyId = FHitProxyId::InvisibleHitProxyId;
			}
		#endif

			const FVector ViewOriginForDistanceCulling = View->ViewMatrices.GetViewOrigin();

			FNiagaraRibbonRenderingFrameViewResources* RenderingViewResources = RenderingResources.RibbonResources->ViewResources.Add_GetRef(MakeShared<FNiagaraRibbonRenderingFrameViewResources>()).Get();
			FNiagaraRibbonRenderingFrameViewResources* SharedRenderingViewResources = nullptr;

			if (bShareAcrossViewFamilies)
			{
				SharedRenderingViewResources = GpuRibbonDataManager.FindExistingRendererViewData(this, SourceParticleData);
			}
			else if (!bNeedsBufferPerView && RenderingResources.RibbonResources->ViewResources.Num() > 1)
			{
				SharedRenderingViewResources = RenderingResources.RibbonResources->ViewResources[0].Get();
			}

			if (SharedRenderingViewResources)
			{
				RenderingViewResources->IndexBuffer = SharedRenderingViewResources->IndexBuffer;
				RenderingViewResources->IndirectDrawBuffer = SharedRenderingViewResources->IndirectDrawBuffer;
				RenderingViewResources->IndexGenerationSettings = SharedRenderingViewResources->IndexGenerationSettings;
				RenderingViewResources->bNeedsIndexBufferGeneration = false;
			}
			else
			{
				bNeedsVertexIndexGeneration = true;
				RenderingViewResources->IndexGenerationSettings = CalculateIndexBufferConfiguration(DynamicData->GenerationOutput, SourceParticleData, SceneProxy, View, ViewOriginForDistanceCulling, DynamicData->bUseGPUInit, DynamicData->bIsGPUSystem);
				GenerateIndexBufferForView(RHICmdList, GpuRibbonDataManager, Collector, RenderingViewResources->IndexGenerationSettings, DynamicData, RenderingViewResources, View, ViewOriginForDistanceCulling);
			}

			SetupPerViewUniformBuffer(RenderingViewResources->IndexGenerationSettings, View, ViewFamily, SceneProxy, RenderingViewResources->UniformBuffer);

			SetupMeshBatchAndCollectorResourceForView(RHICmdList, RenderingViewResources->IndexGenerationSettings, DynamicData, SourceParticleData, View, ViewFamily, SceneProxy, RenderingResources.RibbonResources, RenderingViewResources, MeshBatch, DynamicData->bUseGPUInit);

			Collector.AddMesh(ViewIndex, MeshBatch);

		#if WITH_NIAGARA_RENDERER_READBACK
			if (NiagaraRendererReadback::IsCapturing())
			{
				const uint32 NumVertices = DynamicData->bUseGPUInit ? RenderingViewResources->IndexGenerationSettings.TotalNumIndices : RenderingViewResources->IndexGenerationSettings.CPUTriangleCount * 3;
				NiagaraRendererReadback::CaptureMeshBatch(View, SceneProxy, MeshBatch, 1, NumVertices);
			}
		#endif //WITH_NIAGARA_RENDERER_READBACK
		}
	}

	// Register this renderer for generation this frame if we're a gpu system or using gpu init
	if (bNeedsVertexIndexGeneration && (DynamicData->bUseGPUInit || DynamicData->bIsGPUSystem))
	{
		GpuRibbonDataManager.RegisterRenderer(this, SourceParticleData, RenderingResources.RibbonResources);
	}
}

FNiagaraDynamicDataBase* FNiagaraRendererRibbons::GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter)const
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraGenRibbonVertexData);
	check(Emitter && Emitter->GetParentSystemInstance());

	FNiagaraDynamicDataRibbon* DynamicData = nullptr;
	const UNiagaraRibbonRendererProperties* Properties = CastChecked<const UNiagaraRibbonRendererProperties>(InProperties);

	if (Properties)
	{
		if (!IsRendererEnabled(Properties, Emitter))
		{
			return nullptr;
		}

		if (InProperties->bAllowInCullProxies == false &&
			Cast<UNiagaraCullProxyComponent>(Emitter->GetParentSystemInstance()->GetAttachComponent()) != nullptr)
		{
			return nullptr;
		}

		if ((SimTarget == ENiagaraSimTarget::GPUComputeSim) && GNiagaraRibbonGpuEnabled == 0)
		{
			return nullptr;
		}

		FNiagaraDataBuffer* DataToRender = Emitter->GetParticleData().GetCurrentData();
		if(SimTarget == ENiagaraSimTarget::GPUComputeSim || (DataToRender != nullptr && DataToRender->GetNumInstances() > 1))
		{
			DynamicData = new FNiagaraDynamicDataRibbon(Emitter);
	
			//In preparation for a material override feature, we pass our material(s) and relevance in via dynamic data.
			//The renderer ensures we have the correct usage and relevance for materials in BaseMaterials_GT.
			//Any override feature must also do the same for materials that are set.
			check(BaseMaterials_GT.Num() == 1);
			check(BaseMaterials_GT[0]->CheckMaterialUsage_Concurrent(MATUSAGE_NiagaraRibbons));
			DynamicData->Material = BaseMaterials_GT[0]->GetRenderProxy();
			DynamicData->SetMaterialRelevance(BaseMaterialRelevance_GT);
		}

		if (DynamicData)
		{		
			// We always run GPU init when we're a GPU system
			const bool bIsGPUSystem = SimTarget == ENiagaraSimTarget::GPUComputeSim;
			
			// We disable compute initialization when compute isn't available or they're CVar'd off
			const bool bCanUseComputeGenForCPUSystems = FNiagaraUtilities::AllowComputeShaders(GShaderPlatformForFeatureLevel[FeatureLevel]) && (GNiagaraRibbonGpuInitMode != 2) && (GNiagaraRibbonGpuEnabled != 0);
			const bool bWantsGPUInit = bCanUseComputeGenForCPUSystems && (Properties->bUseGPUInit || (GNiagaraRibbonGpuInitMode == 1));
			
			DynamicData->bUseGPUInit = bIsGPUSystem || bWantsGPUInit;
			DynamicData->bIsGPUSystem = bIsGPUSystem;
			DynamicData->MaxAllocationCount = Emitter->GetParticleData().GetMaxAllocationCount();
			DynamicData->MaxAllocatedCountEstimate = 0;
			if (FVersionedNiagaraEmitterData* EmitterData = Emitter->GetVersionedEmitter().GetEmitterData())
			{
				DynamicData->MaxAllocatedCountEstimate = FMath::Min<uint32>(EmitterData->GetMaxParticleCountEstimate(), DynamicData->MaxAllocationCount);
			}
			
			if (!DynamicData->bUseGPUInit)
			{
				const FNiagaraGenerationInputDataCPUAccessors CPUData(Properties, Emitter->GetParticleData());

				if (CPUData.PosData.IsValid() && CPUData.HasRibbonLinkOrder() && CPUData.TotalNumParticles >= 2)
				{
					DynamicData->GenerationOutput = MakeShared<FNiagaraRibbonCPUGeneratedVertexData>();
					GenerateVertexBufferCPU(CPUData, *DynamicData->GenerationOutput);
				}
				else
				{
					// We don't have valid data so remove the dynamic data
					delete DynamicData;
					DynamicData = nullptr;
				}
			}
		}

		if (DynamicData && Properties->MaterialParameters.HasAnyBindings())
		{
			ProcessMaterialParameterBindings(Properties->MaterialParameters, Emitter, MakeArrayView(BaseMaterials_GT));
		}
	}
	
	return DynamicData;
}

int FNiagaraRendererRibbons::GetDynamicDataSize()const
{
	uint32 Size = sizeof(FNiagaraDynamicDataRibbon);

	Size += uint32(ShapeState.SliceVertexData.GetAllocatedSize());

	return Size;
}

bool FNiagaraRendererRibbons::IsMaterialValid(const UMaterialInterface* Mat)const
{
	return Mat && Mat->CheckMaterialUsage_Concurrent(MATUSAGE_NiagaraRibbons);
}

#if RHI_RAYTRACING
void FNiagaraRendererRibbons::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FNiagaraSceneProxy* SceneProxy)
{
	if (!CVarRayTracingNiagaraRibbons.GetValueOnRenderThread())
	{
		return;
	}
	
	SCOPE_CYCLE_COUNTER(STAT_NiagaraRenderRibbons);
	check(SceneProxy);

	FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();
	FNiagaraDynamicDataRibbon *DynamicDataRibbon = static_cast<FNiagaraDynamicDataRibbon*>(DynamicDataRender);
	FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = SceneProxy->GetComputeDispatchInterface();
	
	if (!ComputeDispatchInterface || !DynamicDataRibbon)
	{
		return;
	}

	FNiagaraDataBuffer* SourceParticleData = DynamicDataRibbon->GetParticleDataToRender(RHICmdList);

	if (GbEnableNiagaraRibbonRendering == 0 || SourceParticleData == nullptr)
	{
		return;
	}

	if (SourceParticleData->GetNumInstances() < 2)
	{
		// Bail if we don't have enough particle data to have a valid ribbon
		return;
	}

	if (!DynamicDataRibbon->bUseGPUInit && (!DynamicDataRibbon->GenerationOutput.IsValid() ||DynamicDataRibbon->GenerationOutput->SegmentData.Num() <= 0))
	{
		return;
	}

	const bool bDataGeneratedOnGpu = DynamicDataRibbon->bUseGPUInit || DynamicDataRibbon->bIsGPUSystem;
	if ( !CVarRayTracingNiagaraRibbonsGPU.GetValueOnRenderThread() && bDataGeneratedOnGpu )
	{
		return;
	}

	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	// Pick the visible view that demands the highest tessellation since the BLAS is shared by all views.
	// Only meaningful on the CPU-init path; the GPU-init path uses a view-independent fixed tessellation factor (see CalculateIndexBufferConfiguration).
	int32 BLASViewIndex = FMath::CountTrailingZeros(VisibilityMap);
	if (!bDataGeneratedOnGpu && Views.Num() > 1)
	{
		int32 HighestTessellation = 1;
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			if ((VisibilityMap & (1 << ViewIndex)) == 0)
			{
				continue;
			}
			const FSceneView* CandidateView = Views[ViewIndex];
			const int32 Tess = CalculateTessellationFactor(SceneProxy, CandidateView, CandidateView->ViewMatrices.GetViewOrigin());
			if (Tess > HighestTessellation)
			{
				HighestTessellation = Tess;
				BLASViewIndex = ViewIndex;
			}
		}
	}

	checkf(Views.IsValidIndex(BLASViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));
	const FSceneView* BLASView = Views[BLASViewIndex];
	const FVector BLASViewOrigin = BLASView->ViewMatrices.GetViewOrigin();

	FNiagaraRibbonMeshCollectorResources& RenderingResources = Collector.AllocateOneFrameResource<FNiagaraRibbonMeshCollectorResources>();
	FNiagaraRibbonRenderingFrameViewResources* RenderingViewResources = RenderingResources.RibbonResources->ViewResources.Add_GetRef(MakeShared<FNiagaraRibbonRenderingFrameViewResources>()).Get();
	RenderingViewResources->IndexGenerationSettings = CalculateIndexBufferConfiguration(DynamicDataRibbon->GenerationOutput, SourceParticleData, SceneProxy, BLASView, BLASViewOrigin, DynamicDataRibbon->bUseGPUInit, DynamicDataRibbon->bIsGPUSystem);

	if (!RenderingViewResources->VertexFactory.GetType()->SupportsRayTracingDynamicGeometry())
	{
		return;
	}

	FNiagaraGpuRibbonsDataManager& GpuRibbonDataManager = ComputeDispatchInterface->GetOrCreateDataManager<FNiagaraGpuRibbonsDataManager>();

	InitializeVertexBuffersResources(RHICmdList, DynamicDataRibbon, SourceParticleData, Collector.GetDynamicReadBuffer(), RenderingResources.RibbonResources, DynamicDataRibbon->bUseGPUInit);

	GenerateIndexBufferForView(RHICmdList, GpuRibbonDataManager, Collector, RenderingViewResources->IndexGenerationSettings, DynamicDataRibbon, RenderingViewResources, BLASView, BLASViewOrigin);

	SetupPerViewUniformBuffer(RenderingViewResources->IndexGenerationSettings, BLASView, *BLASView->Family, SceneProxy, RenderingViewResources->UniformBuffer);

	if (RenderingViewResources->IndexGenerationSettings.TotalNumIndices <= 0)
	{
		return;
	}

	FMeshBatch MeshBatch;
	SetupMeshBatchAndCollectorResourceForView(RHICmdList, RenderingViewResources->IndexGenerationSettings, DynamicDataRibbon, SourceParticleData, BLASView, *BLASView->Family, SceneProxy, RenderingResources.RibbonResources, RenderingViewResources, MeshBatch, DynamicDataRibbon->bUseGPUInit);

	// Get the Vertex / Triangle counts, this is known for CPU but unknown for GPU
	const uint32 VertexCount = bDataGeneratedOnGpu ? RenderingViewResources->IndexGenerationSettings.TotalNumIndices : MeshBatch.Elements[0].NumPrimitives * 3;
	const uint32 TriangleCount = bDataGeneratedOnGpu ? RenderingViewResources->IndexGenerationSettings.TotalNumIndices / 3 : MeshBatch.Elements[0].NumPrimitives;
	if (TriangleCount == 0)
	{
		return;
	}

	TArray<FMeshBatch>& CollectorMeshBatches = Collector.AllocateMeshBatchArray();
	CollectorMeshBatches.Add(MeshBatch);

	FRayTracingInstance RayTracingInstance;
	RayTracingInstance.Geometry = &RayTracingGeometry;
	RayTracingInstance.InstanceTransforms.Add(FMatrix::Identity);
	if (bDataGeneratedOnGpu == false)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// TODO: Shouldn't directly modify FRayTracingGeometry initializer
		RayTracingGeometry.Initializer.IndexBuffer = RenderingViewResources->IndexBuffer->IndexBufferRHI;
		RayTracingGeometry.Initializer.IndexBufferOffset = RenderingViewResources->IndexBuffer->FirstIndex * RenderingViewResources->IndexBuffer->IndexBufferRHI->GetStride();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	RayTracingInstance.MaterialsView = CollectorMeshBatches;

	// Use the internal vertex buffer only when initialized otherwise used the shared vertex buffer - needs to be updated every frame
	FRWBuffer* VertexBuffer = RayTracingDynamicVertexBuffer.NumBytes > 0 ? &RayTracingDynamicVertexBuffer : nullptr;

	if (bDataGeneratedOnGpu)
	{
		//-TODO: We can optimize this to potentially share the data
		GpuRibbonDataManager.RegisterRenderer(this, SourceParticleData, RenderingResources.RibbonResources);
	}

	// A single ray tracing geometry is reused by all views
	// TODO: Support per-view ray tracing geometry when using ENiagaraRibbonFacingMode::Screen
	{
		FRayTracingDynamicGeometryUpdateParams UpdateParams;
		UpdateParams.MeshBatchesView = CollectorMeshBatches;
		UpdateParams.bUsingIndirectDraw = bDataGeneratedOnGpu;
		UpdateParams.NumVertices = VertexCount;
		UpdateParams.VertexBufferSize = VertexCount * static_cast<uint32>(sizeof(FVector3f));
		UpdateParams.NumTriangles = TriangleCount;
		UpdateParams.Geometry = &RayTracingGeometry;
		UpdateParams.Buffer = VertexBuffer;
		UpdateParams.bApplyWorldPositionOffset = true;

		Collector.AddRayTracingGeometryUpdate(BLASViewIndex, MoveTemp(UpdateParams));
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if ((VisibilityMap & (1 << ViewIndex)) == 0)
		{
			continue;
		}

		Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
	}
}
#endif

void FNiagaraRendererRibbons::GenerateShapeStateMultiPlane(FNiagaraRibbonShapeGeometryData& State, int32 MultiPlaneCount, int32 WidthSegmentationCount, bool bEnableAccurateGeometry, bool bUseMaterialBackfaceCulling)
{
	using namespace NiagaraRendererRibbonsPrivate;

	State.Shape = ENiagaraRibbonShapeMode::MultiPlane;
	State.bDisableBackfaceCulling = !bEnableAccurateGeometry && !bUseMaterialBackfaceCulling;
	State.bShouldFlipNormalToView = !bEnableAccurateGeometry;
	State.TrianglesPerSegment = 2 * MultiPlaneCount * WidthSegmentationCount * (bEnableAccurateGeometry? 2 : 1);
	State.NumVerticesInSlice = MultiPlaneCount * (WidthSegmentationCount + 1) * (bEnableAccurateGeometry ? 2 : 1);
	State.BitsNeededForShape = CalculateBitsForRange(State.NumVerticesInSlice);
	State.BitMaskForShape = CalculateBitMask(State.BitsNeededForShape);
	
	const float NormalizeVertexId = float(WidthSegmentationCount > 0 ? WidthSegmentationCount : 1);
	for (int32 PlaneIndex = 0; PlaneIndex < MultiPlaneCount; PlaneIndex++)
	{
		const float RotationAngle = (static_cast<float>(PlaneIndex) / MultiPlaneCount) * 180.0f;

		for (int32 VertexId = 0; VertexId <= WidthSegmentationCount; VertexId++)
		{
			const FVector2f Position = FVector2f((static_cast<float>(VertexId) / NormalizeVertexId) - 0.5f, 0).GetRotated(RotationAngle);
			const FVector2f Normal = FVector2f(0, 1).GetRotated(RotationAngle);
			const float TextureV = static_cast<float>(VertexId) / NormalizeVertexId;

			State.SliceVertexData.Emplace(Position, Normal, TextureV);
		}
	}

	if (bEnableAccurateGeometry)
	{
		for (int32 PlaneIndex = 0; PlaneIndex < MultiPlaneCount; PlaneIndex++)
		{
			const float RotationAngle = (static_cast<float>(PlaneIndex) / MultiPlaneCount) * 180.0f;

			for (int32 VertexId = 0; VertexId <= WidthSegmentationCount; VertexId++)
			{
				const FVector2f Position = FVector2f((static_cast<float>(VertexId) / NormalizeVertexId) - 0.5f, 0).GetRotated(RotationAngle);
				const FVector2f Normal = FVector2f(0, -1).GetRotated(RotationAngle);
				const float TextureV = static_cast<float>(VertexId) / NormalizeVertexId;

				State.SliceVertexData.Emplace(Position, Normal, TextureV);
			}
		}
	}


	const int32 FrontFaceVertexCount = MultiPlaneCount * (WidthSegmentationCount + 1);

	State.SliceTriangleToVertexIds.Reserve(WidthSegmentationCount * MultiPlaneCount * (bEnableAccurateGeometry ? 2 : 1));
	for (int32 PlaneIndex = 0; PlaneIndex < MultiPlaneCount; PlaneIndex++)
	{
		const int32 BaseVertexId = (PlaneIndex * (WidthSegmentationCount + 1));

		for (int32 VertexIdx = 0; VertexIdx < WidthSegmentationCount; VertexIdx++)
		{
			State.SliceTriangleToVertexIds.Add(BaseVertexId + VertexIdx);
			State.SliceTriangleToVertexIds.Add(BaseVertexId + VertexIdx + 1);
		}

		if (bEnableAccurateGeometry)
		{
			for (int32 VertexIdx = 0; VertexIdx < WidthSegmentationCount; VertexIdx++)
			{
				State.SliceTriangleToVertexIds.Add(FrontFaceVertexCount + BaseVertexId + VertexIdx + 1);
				State.SliceTriangleToVertexIds.Add(FrontFaceVertexCount + BaseVertexId + VertexIdx);
			}
		}
	}
}

void FNiagaraRendererRibbons::GenerateShapeStateTube(FNiagaraRibbonShapeGeometryData& State, int32 TubeSubdivisions, bool bUseMaterialBackfaceCulling)
{
	using namespace NiagaraRendererRibbonsPrivate;

	State.Shape = ENiagaraRibbonShapeMode::Tube;
	State.bDisableBackfaceCulling = !bUseMaterialBackfaceCulling;
	State.bShouldFlipNormalToView = false;
	State.TrianglesPerSegment = 2 * TubeSubdivisions;
	State.NumVerticesInSlice = TubeSubdivisions + 1;
	State.BitsNeededForShape = CalculateBitsForRange(State.NumVerticesInSlice);
	State.BitMaskForShape = CalculateBitMask(State.BitsNeededForShape);
	
	State.SliceVertexData.Reserve(TubeSubdivisions + 1);

	const float NormalizeVertexId = float(TubeSubdivisions > 0 ? TubeSubdivisions : 1);
	for (int32 VertexId = 0; VertexId <= TubeSubdivisions; VertexId++)
	{
		const float RotationAngle = (static_cast<float>(VertexId) / NormalizeVertexId) * -360.0f;
		const FVector2f Position = FVector2f(-0.5f, 0.0f).GetRotated(RotationAngle);
		const FVector2f Normal = FVector2f(-1, 0).GetRotated(RotationAngle);
		const float TextureV = static_cast<float>(VertexId) / NormalizeVertexId;

		State.SliceVertexData.Emplace(Position, Normal, TextureV);
	}
	
	State.SliceTriangleToVertexIds.Reserve(TubeSubdivisions * 2);
	for (int32 VertexIdx = 0; VertexIdx < TubeSubdivisions; VertexIdx++)
	{
		State.SliceTriangleToVertexIds.Add(VertexIdx);
		State.SliceTriangleToVertexIds.Add(VertexIdx + 1);
	}
}

void FNiagaraRendererRibbons::GenerateShapeStateCustom(FNiagaraRibbonShapeGeometryData& State, const TArray<FNiagaraRibbonShapeCustomVertex>& CustomVertices, bool bUseMaterialBackfaceCulling)
{
	using namespace NiagaraRendererRibbonsPrivate;

	State.Shape = ENiagaraRibbonShapeMode::Custom;
	State.bDisableBackfaceCulling = !bUseMaterialBackfaceCulling;
	State.bShouldFlipNormalToView = false;
	State.TrianglesPerSegment = 2 * CustomVertices.Num();
	State.NumVerticesInSlice = CustomVertices.Num() + 1;
	State.BitsNeededForShape = CalculateBitsForRange(State.NumVerticesInSlice);
	State.BitMaskForShape = CalculateBitMask(State.BitsNeededForShape);
	
	bool bHasCustomUVs = false;
	for (int32 VertexId = 0; VertexId < CustomVertices.Num(); VertexId++)
	{
		if (!FMath::IsNearlyZero(CustomVertices[VertexId].TextureV))
		{
			bHasCustomUVs = true;
			break;
		}
	}

	for (int32 VertexId = 0; VertexId <= CustomVertices.Num(); VertexId++)
	{
		const auto& CustomVert = CustomVertices[VertexId % CustomVertices.Num()];

		const FVector2f Position = CustomVert.Position;
		const FVector2f Normal = CustomVert.Normal.IsNearlyZero() ? Position.GetSafeNormal() : CustomVert.Normal;
		const float TextureV = bHasCustomUVs ? CustomVert.TextureV : static_cast<float>(VertexId) / CustomVertices.Num();

		State.SliceVertexData.Emplace(Position, Normal, TextureV);
	}

	State.SliceTriangleToVertexIds.Reserve(CustomVertices.Num() * 2);
	for (int32 VertexIdx = 0; VertexIdx < CustomVertices.Num(); VertexIdx++)
	{
		State.SliceTriangleToVertexIds.Add(VertexIdx);
		State.SliceTriangleToVertexIds.Add(VertexIdx + 1);
	}
}

void FNiagaraRendererRibbons::GenerateShapeStatePlane(FNiagaraRibbonShapeGeometryData& State, int32 WidthSegmentationCount, bool bUseMaterialBackfaceCulling)
{
	using namespace NiagaraRendererRibbonsPrivate;

	State.Shape = ENiagaraRibbonShapeMode::Plane;
	State.bDisableBackfaceCulling = !bUseMaterialBackfaceCulling;
	State.bShouldFlipNormalToView = false;
	State.TrianglesPerSegment = 2 * WidthSegmentationCount;
	State.NumVerticesInSlice = WidthSegmentationCount + 1;
	State.BitsNeededForShape = CalculateBitsForRange(State.NumVerticesInSlice);
	State.BitMaskForShape = CalculateBitMask(State.BitsNeededForShape);
	
	State.SliceVertexData.Reserve(WidthSegmentationCount + 1);

	const float NormalizeVertexId = float(WidthSegmentationCount > 0 ? WidthSegmentationCount : 1);
	for (int32 VertexId = 0; VertexId <= WidthSegmentationCount; VertexId++)
	{
		const FVector2f Position = FVector2f((static_cast<float>(VertexId) / NormalizeVertexId) - 0.5f, 0);
		const FVector2f Normal = FVector2f(0, 1);
		const float TextureV = static_cast<float>(VertexId) / NormalizeVertexId;

		State.SliceVertexData.Emplace(Position, Normal, TextureV);
	}
	
	State.SliceTriangleToVertexIds.Reserve(WidthSegmentationCount * 2);
	for (int32 VertexIdx = 0; VertexIdx < WidthSegmentationCount; VertexIdx++)
	{
		State.SliceTriangleToVertexIds.Add(VertexIdx);
		State.SliceTriangleToVertexIds.Add(VertexIdx + 1);
	}
}

void FNiagaraRendererRibbons::InitializeShape(const UNiagaraRibbonRendererProperties* Properties)
{
	if (Properties->Shape == ENiagaraRibbonShapeMode::Custom && Properties->CustomVertices.Num() > 2)
	{
		GenerateShapeStateCustom(ShapeState, Properties->CustomVertices, Properties->bUseMaterialBackfaceCulling);
	}
	else if (Properties->Shape == ENiagaraRibbonShapeMode::Tube && Properties->TubeSubdivisions > 2 && Properties->TubeSubdivisions <= 16)
	{
		GenerateShapeStateTube(ShapeState, Properties->TubeSubdivisions, Properties->bUseMaterialBackfaceCulling);
	}
	else if (Properties->Shape == ENiagaraRibbonShapeMode::MultiPlane && Properties->MultiPlaneCount > 1 && Properties->MultiPlaneCount <= 16)
	{
		GenerateShapeStateMultiPlane(ShapeState, Properties->MultiPlaneCount, Properties->GetWidthSegmentationCount(), Properties->bEnableAccurateGeometry, Properties->bUseMaterialBackfaceCulling);
	}
	else
	{
		GenerateShapeStatePlane(ShapeState, Properties->GetWidthSegmentationCount(), Properties->bUseMaterialBackfaceCulling);
	}	
}

void FNiagaraRendererRibbons::InitializeTessellation(const UNiagaraRibbonRendererProperties* Properties)
{
	TessellationConfig.TessellationMode = Properties->TessellationMode;
	TessellationConfig.CustomTessellationFactor = Properties->TessellationFactor;
	TessellationConfig.bCustomUseConstantFactor = Properties->bUseConstantFactor;
	TessellationConfig.CustomTessellationMinAngle = Properties->TessellationAngle > 0.f && Properties->TessellationAngle < 1.f ? 1.f : Properties->TessellationAngle;
	TessellationConfig.CustomTessellationMinAngle *= PI / 180.f;
	TessellationConfig.bCustomUseScreenSpace = Properties->bScreenSpaceTessellation;
}

template<typename IntType>
void FNiagaraRendererRibbons::CalculateUVScaleAndOffsets(const FNiagaraRibbonUVSettings& UVSettings, const TArray<IntType>& RibbonIndices, const TArray<FVector4f>& RibbonTangentsAndDistances, const FNiagaraDataSetReaderFloat<float>& NormalizedAgeReader,
                                                         int32 StartIndex, int32 EndIndex, int32 NumSegments, float TotalLength, float& OutUScale, float& OutUOffset, float& OutUDistributionScaler)
{
	float NormalizedLeadingSegmentOffset;
	if (UVSettings.LeadingEdgeMode == ENiagaraRibbonUVEdgeMode::SmoothTransition)
	{
		const float FirstAge = NormalizedAgeReader.GetSafe(RibbonIndices[StartIndex], 0.0f);
		const float SecondAge = NormalizedAgeReader.GetSafe(RibbonIndices[StartIndex + 1], 0.0f);

		const float StartTimeStep = SecondAge - FirstAge;
		const float StartTimeOffset = FirstAge < StartTimeStep ? StartTimeStep - FirstAge : 0;

		NormalizedLeadingSegmentOffset = StartTimeStep > 0 ? StartTimeOffset / StartTimeStep : 0.0f;
	}
	else if (UVSettings.LeadingEdgeMode == ENiagaraRibbonUVEdgeMode::Locked)
	{
		NormalizedLeadingSegmentOffset = 0;
	}
	else
	{
		NormalizedLeadingSegmentOffset = 0;
		checkf(false, TEXT("Unsupported ribbon uv edge mode"));
	}

	float NormalizedTrailingSegmentOffset;
	if (UVSettings.TrailingEdgeMode == ENiagaraRibbonUVEdgeMode::SmoothTransition)
	{
		const float SecondToLastAge = NormalizedAgeReader.GetSafe(RibbonIndices[EndIndex - 1], 0.0f);
		const float LastAge = NormalizedAgeReader.GetSafe(RibbonIndices[EndIndex], 0.0f);

		const float EndTimeStep = LastAge - SecondToLastAge;
		const float EndTimeOffset = 1 - LastAge < EndTimeStep ? EndTimeStep - (1 - LastAge) : 0;

		NormalizedTrailingSegmentOffset = EndTimeStep > 0 ? EndTimeOffset / EndTimeStep : 0.0f;
	}
	else if (UVSettings.TrailingEdgeMode == ENiagaraRibbonUVEdgeMode::Locked)
	{
		NormalizedTrailingSegmentOffset = 0;
	}
	else
	{
		NormalizedTrailingSegmentOffset = 0;
		checkf(false, TEXT("Unsupported ribbon uv edge mode"));
	}

	float CalculatedUOffset;
	float CalculatedUScale;
	if (UVSettings.DistributionMode == ENiagaraRibbonUVDistributionMode::ScaledUniformly)
	{
		const float AvailableSegments = NumSegments - (NormalizedLeadingSegmentOffset + NormalizedTrailingSegmentOffset);
		CalculatedUScale = NumSegments / AvailableSegments;
		CalculatedUOffset = -((NormalizedLeadingSegmentOffset / NumSegments) * CalculatedUScale);
		OutUDistributionScaler = 1.0f / NumSegments;
	}
	else if (UVSettings.DistributionMode == ENiagaraRibbonUVDistributionMode::ScaledUsingRibbonSegmentLength)
	{
		const float SecondDistance = RibbonTangentsAndDistances[StartIndex + 1].W;
		const float LeadingDistanceOffset = SecondDistance * NormalizedLeadingSegmentOffset;

		const float SecondToLastDistance = RibbonTangentsAndDistances[EndIndex - 1].W;
		const float LastDistance = RibbonTangentsAndDistances[EndIndex].W;
		const float TrailingDistanceOffset = (LastDistance - SecondToLastDistance) * NormalizedTrailingSegmentOffset;

		const float AvailableLength = TotalLength - (LeadingDistanceOffset + TrailingDistanceOffset);

		CalculatedUScale = TotalLength / AvailableLength;
		CalculatedUOffset = -((LeadingDistanceOffset / TotalLength) * CalculatedUScale);
		OutUDistributionScaler = 1.0f / TotalLength;
	}
	else if (UVSettings.DistributionMode == ENiagaraRibbonUVDistributionMode::TiledOverRibbonLength)
	{
		const float SecondDistance = RibbonTangentsAndDistances[StartIndex + 1].W;
		const float LeadingDistanceOffset = SecondDistance * NormalizedLeadingSegmentOffset;

		CalculatedUScale = TotalLength / UVSettings.TilingLength;
		CalculatedUOffset = -(LeadingDistanceOffset / UVSettings.TilingLength);
		OutUDistributionScaler = 1.0f / TotalLength;
	}
	else if (UVSettings.DistributionMode == ENiagaraRibbonUVDistributionMode::TiledFromStartOverRibbonLength)
	{
		CalculatedUScale = TotalLength / UVSettings.TilingLength;
		CalculatedUOffset = 0;
		OutUDistributionScaler = 1.0f / TotalLength;
	}
	else
	{
		CalculatedUScale = 1;
		CalculatedUOffset = 0;
		checkf(false, TEXT("Unsupported ribbon distribution mode"));
	}

	OutUScale = float(CalculatedUScale * UVSettings.Scale.X);
	OutUOffset = float((CalculatedUOffset * UVSettings.Scale.X) + UVSettings.Offset.X);
}


template<bool bWantsTessellation, bool bHasTwist, bool bWantsMultiRibbon>
void FNiagaraRendererRibbons::GenerateVertexBufferForRibbonPart(const FNiagaraGenerationInputDataCPUAccessors& CPUData, TConstArrayView<uint32> RibbonIndices, uint32 RibbonIndex, FNiagaraRibbonCPUGeneratedVertexData& OutputData) const
{
	using namespace NiagaraRendererRibbonsPrivate;

	TArray<uint32>& SegmentData = OutputData.SegmentData;
	
	const FNiagaraDataSetReaderFloat<FNiagaraPosition>& PosData = CPUData.PosData;	
	const FNiagaraDataSetReaderFloat<float>& AgeData = CPUData.AgeData;
	const FNiagaraDataSetReaderFloat<float>& SizeData = CPUData.SizeData;
	const FNiagaraDataSetReaderFloat<float>& TwistData = CPUData.TwistData;
	
	const int32 StartIndex = OutputData.SortedIndices.Num();

	const FVector3f FirstPos = PosData[RibbonIndices[0]];
	FVector3f CurrPos = FirstPos;
	FVector3f LastToCurrVec = FVector3f::ZeroVector;
	float LastToCurrSize = 0;	
	float LastTwist = 0;
	float LastWidth = 0;
	float TotalDistance = 0.0f;

	// Find the first position with enough distance.
	int32 CurrentIndex = 1;
	while (CurrentIndex < RibbonIndices.Num())
	{
		const int32 CurrentDataIndex = RibbonIndices[CurrentIndex];
		CurrPos = PosData[CurrentDataIndex];
		LastToCurrVec = CurrPos - FirstPos;
		LastToCurrSize = LastToCurrVec.Size();
		if constexpr (bHasTwist)
		{
			LastTwist = TwistData[CurrentDataIndex];
			LastWidth = SizeData[CurrentDataIndex];
		}

		// Find the first segment, or unique segment
		if (LastToCurrSize > GNiagaraRibbonMinSegmentLength)
		{
			// Normalize LastToCurrVec
			LastToCurrVec *= 1.f / LastToCurrSize;

			// Add the first point. Tangent follows first segment.
			OutputData.SortedIndices.Add(RibbonIndices[0]);
			OutputData.TangentAndDistances.Add(FVector4f(LastToCurrVec.X, LastToCurrVec.Y, LastToCurrVec.Z, 0));
			if constexpr (bWantsMultiRibbon)
			{
				OutputData.MultiRibbonIndices.Add(RibbonIndex);
			}
			break;
		}
		else
		{
			LastToCurrSize = 0; // Ensure that the segment gets ignored if too small
			++CurrentIndex;
		}
	}

	// Now iterate on all other points, to proceed each particle connected to 2 segments.
	int32 NextIndex = CurrentIndex + 1;
	while (NextIndex < RibbonIndices.Num())
	{
		const int32 NextDataIndex = RibbonIndices[NextIndex];
		const FVector3f NextPos = PosData[NextDataIndex];
		FVector3f CurrToNextVec = NextPos - CurrPos;
		const float CurrToNextSize = CurrToNextVec.Size();

		float NextTwist = 0;
		float NextWidth = 0;
		if constexpr (bHasTwist)
		{
			NextTwist = TwistData[NextDataIndex];
			NextWidth = SizeData[NextDataIndex];
		}

		// It the next is far enough, or the last element
		if (CurrToNextSize > GNiagaraRibbonMinSegmentLength || NextIndex == RibbonIndices.Num() - 1)
		{
			// Normalize CurrToNextVec
			CurrToNextVec *= 1.f / FMath::Max(GNiagaraRibbonMinSegmentLength, CurrToNextSize);
			const FVector3f Tangent = (1.f - GenerationConfig.GetCurveTension()) * (LastToCurrVec + CurrToNextVec).GetSafeNormal();

			// Update the distance for CurrentIndex.
			TotalDistance += LastToCurrSize;

			// Add the current point, which tangent is computed from neighbors
			OutputData.SortedIndices.Add(RibbonIndices[CurrentIndex]);
			OutputData.TangentAndDistances.Add(FVector4f(Tangent.X, Tangent.Y, Tangent.Z, TotalDistance));

			if constexpr (bWantsMultiRibbon)
			{
				OutputData.MultiRibbonIndices.Add(RibbonIndex);
			}

			// Assumed equal to dot(Tangent, CurrToNextVec)
			OutputData.TotalSegmentLength += CurrToNextSize;
			
			if constexpr (bWantsTessellation)
			{
				OutputData.AverageSegmentLength += CurrToNextSize * CurrToNextSize;
				OutputData.AverageSegmentAngle += CurrToNextSize * AcosFast(FVector3f::DotProduct(LastToCurrVec, CurrToNextVec));
				if constexpr (bHasTwist)
				{
					OutputData.AverageTwistAngle += CurrToNextSize * FMath::Abs(NextTwist - LastTwist);
					OutputData.AverageWidth += CurrToNextSize * LastWidth;
				}
			}

			// Move to next segment.
			CurrentIndex = NextIndex;
			CurrPos = NextPos;
			LastToCurrVec = CurrToNextVec;
			LastToCurrSize = CurrToNextSize;
			LastTwist = NextTwist;
			LastWidth = NextWidth;
		}

		// Try next if there is one.
		++NextIndex;
	}

	// Close the last point and segment if there was at least 2.
	if (LastToCurrSize > 0)
	{
		// Update the distance for CurrentIndex.
		TotalDistance += LastToCurrSize;

		// Add the last point, which tangent follows the last segment.
		OutputData.SortedIndices.Add(RibbonIndices[CurrentIndex]);
		OutputData.TangentAndDistances.Add(FVector4f(LastToCurrVec.X, LastToCurrVec.Y, LastToCurrVec.Z, TotalDistance));
		if constexpr (bWantsMultiRibbon)
		{
			OutputData.MultiRibbonIndices.Add(RibbonIndex);
		}
	}

	const int32 EndIndex = OutputData.SortedIndices.Num() - 1;
	const int32 NumSegments = EndIndex - StartIndex;

	if (NumSegments > 0)
	{
		FRibbonMultiRibbonInfo& MultiRibbonInfo = OutputData.RibbonInfoLookup[RibbonIndex];
		MultiRibbonInfo.StartPos = (FVector)PosData[RibbonIndices[0]];
		MultiRibbonInfo.EndPos = (FVector)PosData[RibbonIndices.Last()];
		MultiRibbonInfo.BaseSegmentDataIndex = SegmentData.Num();
		MultiRibbonInfo.NumSegmentDataIndices = NumSegments;

		// Update the tangents for the first and last vertex, apply a reflect vector logic so that the initial and final curvature is continuous.
		if (NumSegments > 1)
		{
			FVector3f& FirstTangent =  reinterpret_cast<FVector3f&>(OutputData.TangentAndDistances[StartIndex]);
			FVector3f& NextToFirstTangent = reinterpret_cast<FVector3f&>(OutputData.TangentAndDistances[StartIndex + 1]);
			FirstTangent = (2.f * FVector3f::DotProduct(FirstTangent, NextToFirstTangent)) * FirstTangent - NextToFirstTangent;

			FVector3f& LastTangent = reinterpret_cast<FVector3f&>(OutputData.TangentAndDistances[EndIndex]);
			FVector3f& PrevToLastTangent = reinterpret_cast<FVector3f&>(OutputData.TangentAndDistances[EndIndex - 1]);
			LastTangent = (2.f * FVector3f::DotProduct(LastTangent, PrevToLastTangent)) * LastTangent - PrevToLastTangent;
		}

		// Add segment data
		for (int32 SegmentIndex = StartIndex; SegmentIndex < EndIndex; ++SegmentIndex)
		{
			SegmentData.Add(SegmentIndex);
		}

		float U0Offset;
		float U0Scale;
		float U0DistributionScaler;
		if(GenerationConfig.HasCustomU0Data())
		{
			U0Offset = 0;
			U0Scale = 1.0f;
			U0DistributionScaler = 1;
		}
		else
		{
			CalculateUVScaleAndOffsets(
				UV0Settings, OutputData.SortedIndices, OutputData.TangentAndDistances,
				AgeData,
				StartIndex, EndIndex,
				NumSegments, TotalDistance,
				U0Scale, U0Offset, U0DistributionScaler);
		}

		float U1Offset;
		float U1Scale;
		float U1DistributionScaler;
		if (GenerationConfig.HasCustomU1Data())
		{
			U1Offset = 0;
			U1Scale = 1.0f;
			U1DistributionScaler = 1;
		}
		else
		{
			CalculateUVScaleAndOffsets(
				UV1Settings, OutputData.SortedIndices, OutputData.TangentAndDistances,
				AgeData,
				StartIndex, EndIndex,
				NumSegments, TotalDistance,
				U1Scale, U1Offset, U1DistributionScaler);
		}

		MultiRibbonInfo.BufferEntry.U0Scale = U0Scale;
		MultiRibbonInfo.BufferEntry.U0Offset = U0Offset;
		MultiRibbonInfo.BufferEntry.U0DistributionScaler = U0DistributionScaler;
		MultiRibbonInfo.BufferEntry.U1Scale = U1Scale;
		MultiRibbonInfo.BufferEntry.U1Offset = U1Offset;
		MultiRibbonInfo.BufferEntry.U1DistributionScaler = U1DistributionScaler;
		MultiRibbonInfo.BufferEntry.FirstParticleId = StartIndex;
		MultiRibbonInfo.BufferEntry.LastParticleId = EndIndex;
	}
}

template<typename IDType, typename ReaderType, bool bWantsTessellation, bool bHasTwist>
void FNiagaraRendererRibbons::GenerateVertexBufferForMultiRibbonInternal(const FNiagaraGenerationInputDataCPUAccessors& CPUData, const ReaderType& IDReader, FNiagaraRibbonCPUGeneratedVertexData& OutputData) const
{
	using FIndexArray = TArray<uint32, TMemStackAllocator<>>;
	TMap<IDType, FIndexArray> MultiRibbonSortedIndices;

	for (uint32 i=0; i < CPUData.TotalNumParticles; ++i)
	{
		FIndexArray& Indices = MultiRibbonSortedIndices.FindOrAdd(IDReader[i]);
		Indices.Add(i);
	}

	const int32 NumRibbons = MultiRibbonSortedIndices.Num();
	OutputData.RibbonInfoLookup.AddZeroed(MultiRibbonSortedIndices.Num());
	OutputData.SortedIndices.Reserve(OutputData.SortedIndices.Num() + CPUData.TotalNumParticles + NumRibbons);
	OutputData.TangentAndDistances.Reserve(OutputData.TangentAndDistances.Num() + CPUData.TotalNumParticles + NumRibbons);
	OutputData.MultiRibbonIndices.Reserve(OutputData.MultiRibbonIndices.Num() + CPUData.TotalNumParticles + NumRibbons);
	OutputData.SegmentData.Reserve(OutputData.SegmentData.Num() + CPUData.TotalNumParticles + NumRibbons);

	// Sort the ribbons by ID so that the draw order stays consistent.
	MultiRibbonSortedIndices.KeySort(TLess<IDType>());

	uint32 RibbonIndex = 0;
	for (TPair<IDType, FIndexArray>& Pair : MultiRibbonSortedIndices)
	{
		FIndexArray& SortedIndices = Pair.Value;
		CPUData.RibbonLinkOrderSort(SortedIndices);
		GenerateVertexBufferForRibbonPart<bWantsTessellation, bHasTwist, true>(CPUData, SortedIndices, RibbonIndex, OutputData);
		RibbonIndex++;
	}
}

template<typename IDType, typename ReaderType>
void FNiagaraRendererRibbons::GenerateVertexBufferForMultiRibbon(const FNiagaraGenerationInputDataCPUAccessors& CPUData, const ReaderType& IDReader, FNiagaraRibbonCPUGeneratedVertexData& OutputData) const
{
	if (GenerationConfig.WantsAutomaticTessellation())
	{
		if (GenerationConfig.HasTwist())
		{
			GenerateVertexBufferForMultiRibbonInternal<IDType, ReaderType, true, true>(CPUData, IDReader, OutputData);				
		}
		else
		{
			GenerateVertexBufferForMultiRibbonInternal<IDType, ReaderType, true, false>(CPUData, IDReader, OutputData);				
		}
	}
	else
	{
		GenerateVertexBufferForMultiRibbonInternal<IDType, ReaderType, false, false>(CPUData, IDReader, OutputData);		
	}
}

void FNiagaraRendererRibbons::GenerateVertexBufferCPU(const FNiagaraGenerationInputDataCPUAccessors& CPUData, FNiagaraRibbonCPUGeneratedVertexData& OutputData) const
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraRenderRibbonsGenVerticesCPU);
	
	check(CPUData.PosData.IsValid() && CPUData.HasRibbonLinkOrder());

	// TODO: Move sorting to share code with sprite and mesh sorting and support the custom sorting key.
	FMemMark Mark(FMemStack::Get());
	if (GenerationConfig.HasRibbonIDs())
	{
		if (GenerationConfig.HasFullRibbonIDs())
		{
			GenerateVertexBufferForMultiRibbon<FNiagaraID>(CPUData, CPUData.FullRibbonIDData, OutputData);
		}
		else
		{
			// TODO: Remove simple ID path
			check(GenerationConfig.HasSimpleRibbonIDs());

			GenerateVertexBufferForMultiRibbon<uint32>(CPUData, CPUData.SimpleRibbonIDData, OutputData);
		}		
	}
	else
	{
		TArray<uint32, TMemStackAllocator<>> SortedIndices;
		SortedIndices.Reserve(CPUData.TotalNumParticles + 1);
		for (uint32 i = 0; i < CPUData.TotalNumParticles; ++i)
		{
			SortedIndices.Add(i);
		}
		OutputData.RibbonInfoLookup.AddZeroed(1);

		CPUData.RibbonLinkOrderSort(SortedIndices);

		OutputData.SortedIndices.Reserve(OutputData.SortedIndices.Num() + CPUData.TotalNumParticles + 1);
		OutputData.TangentAndDistances.Reserve(OutputData.TangentAndDistances.Num() + CPUData.TotalNumParticles + 1);
		OutputData.SegmentData.Reserve(OutputData.SegmentData.Num() + CPUData.TotalNumParticles + 1);

		if (GenerationConfig.WantsAutomaticTessellation())
		{
			if (GenerationConfig.HasTwist())
			{
				GenerateVertexBufferForRibbonPart<true, true, false>(CPUData, SortedIndices, 0/*RibbonIndex*/, OutputData);			
			}
			else
			{
				GenerateVertexBufferForRibbonPart<true, false, false>(CPUData, SortedIndices, 0/*RibbonIndex*/, OutputData);			
			}
		}
		else
		{
			GenerateVertexBufferForRibbonPart<false, false, false>(CPUData, SortedIndices, 0/*RibbonIndex*/, OutputData);		
		}
	}
	
	if (OutputData.TotalSegmentLength > 0.0)
	{
		const float& TotalSegmentLength = OutputData.TotalSegmentLength;
	
		// weighted sum based on the segment length :
		float& AverageSegmentLength = OutputData.AverageSegmentLength;
		float& AverageSegmentAngle = OutputData.AverageSegmentAngle;
		float& AverageTwistAngle = OutputData.AverageTwistAngle;
		float& AverageWidth = OutputData.AverageWidth;
		
		// Blend the result between the last frame tessellation factors and the current frame base on the total length of all segments.
		// This is only used to increase the tessellation value of the current frame data to prevent glitches where tessellation is significantly changin between frames.
		const float OneOverTotalSegmentLength = 1.f / FMath::Max(1.f, TotalSegmentLength);
		const float AveragingFactor = TessellationSmoothingData.TessellationTotalSegmentLength / (TotalSegmentLength + TessellationSmoothingData.TessellationTotalSegmentLength);
		TessellationSmoothingData.TessellationTotalSegmentLength = TotalSegmentLength;

		AverageSegmentAngle *= OneOverTotalSegmentLength;
		AverageSegmentLength *= OneOverTotalSegmentLength;
		const float AverageSegmentCurvature = AverageSegmentLength / (FMath::Max(SMALL_NUMBER, FMath::Abs(FMath::Sin(AverageSegmentAngle))));

		TessellationSmoothingData.TessellationAngle = FMath::Lerp<float>(AverageSegmentAngle, FMath::Max(TessellationSmoothingData.TessellationAngle, AverageSegmentAngle), AveragingFactor);
		TessellationSmoothingData.TessellationCurvature = FMath::Lerp<float>(AverageSegmentCurvature, FMath::Max(TessellationSmoothingData.TessellationCurvature, AverageSegmentCurvature), AveragingFactor);

		if (GenerationConfig.HasTwist())
		{
			AverageTwistAngle *= OneOverTotalSegmentLength;
			AverageWidth *= OneOverTotalSegmentLength;

			TessellationSmoothingData.TessellationTwistAngle = FMath::Lerp<float>(AverageTwistAngle, FMath::Max(TessellationSmoothingData.TessellationTwistAngle, AverageTwistAngle), AveragingFactor);
			TessellationSmoothingData.TessellationTwistCurvature = FMath::Lerp<float>(AverageWidth, FMath::Max(TessellationSmoothingData.TessellationTwistCurvature, AverageWidth), AveragingFactor);
		}
	}
	else // Reset the metrics when the ribbons are reset.
	{
		TessellationSmoothingData.TessellationAngle = 0;
		TessellationSmoothingData.TessellationCurvature = 0;
		TessellationSmoothingData.TessellationTwistAngle = 0;
		TessellationSmoothingData.TessellationTwistCurvature = 0;
		TessellationSmoothingData.TessellationTotalSegmentLength = 0;
	}
}

int32 FNiagaraRendererRibbons::CalculateTessellationFactor(const FNiagaraSceneProxy* SceneProxy, const FSceneView* View, const FVector& ViewOriginForDistanceCulling) const
{
	using namespace NiagaraRendererRibbonsPrivate;

	bool bUseConstantFactor = false;
	int32 TessellationFactor = GNiagaraRibbonMaxTessellation;
	float TessellationMinAngle = GNiagaraRibbonTessellationAngle;
	float ScreenPercentage = GNiagaraRibbonTessellationScreenPercentage;
	switch (TessellationConfig.TessellationMode)
	{
	case ENiagaraRibbonTessellationMode::Automatic:
		break;
	case ENiagaraRibbonTessellationMode::Custom:
		TessellationFactor = FMath::Min<int32>(TessellationFactor, TessellationConfig.CustomTessellationFactor); // Don't allow factors bigger than the platform limit.
		bUseConstantFactor = TessellationConfig.bCustomUseConstantFactor;
		TessellationMinAngle = TessellationConfig.CustomTessellationMinAngle;
		ScreenPercentage = TessellationConfig.bCustomUseScreenSpace && !bUseConstantFactor ? GNiagaraRibbonTessellationScreenPercentage : 0.f;
		break;
	case ENiagaraRibbonTessellationMode::Disabled:
		TessellationFactor = 1;
		break;
	default:
		break;
	}

	if (bUseConstantFactor)
	{
		return TessellationFactor;
	}
	
	int32 SegmentTessellation = 1;
	
	if (GNiagaraRibbonTessellationEnabled && TessellationFactor > 1 && TessellationSmoothingData.TessellationCurvature > SMALL_NUMBER)
	{
		const float MinTesselation = (TessellationMinAngle == 0.f || bUseConstantFactor)?
			                             static_cast<float>(TessellationFactor)	:
			                             FMath::Max<float>(1.f, FMath::Max(TessellationSmoothingData.TessellationTwistAngle, TessellationSmoothingData.TessellationAngle) / FMath::Max<float>(SMALL_NUMBER, TessellationMinAngle));

		constexpr float MAX_CURVATURE_FACTOR = 0.002f; // This will clamp the curvature to around 2.5 km and avoid numerical issues.
		const float ViewDistance = SceneProxy->GetProxyDynamicData().LODDistanceOverride >= 0.0f ? SceneProxy->GetProxyDynamicData().LODDistanceOverride : float(SceneProxy->GetBounds().ComputeSquaredDistanceFromBoxToPoint(ViewOriginForDistanceCulling));
		const float MaxDisplacementError = FMath::Max(GNiagaraRibbonTessellationMinDisplacementError, ScreenPercentage * FMath::Sqrt(ViewDistance) / View->LODDistanceFactor);
		float Tess = TessellationSmoothingData.TessellationAngle / FMath::Max(MAX_CURVATURE_FACTOR, AcosFast(TessellationSmoothingData.TessellationCurvature / (TessellationSmoothingData.TessellationCurvature + MaxDisplacementError)));
		// FMath::RoundUpToPowerOfTwo ? This could avoid vertices moving around as tesselation increases

		if (TessellationSmoothingData.TessellationTwistAngle > 0 && TessellationSmoothingData.TessellationTwistCurvature > 0)
		{
			const float TwistTess = TessellationSmoothingData.TessellationTwistAngle / FMath::Max(MAX_CURVATURE_FACTOR, AcosFast(TessellationSmoothingData.TessellationTwistCurvature / (TessellationSmoothingData.TessellationTwistCurvature + MaxDisplacementError)));
			Tess = FMath::Max(TwistTess, Tess);
		}
		SegmentTessellation = FMath::Clamp<int32>(FMath::RoundToInt(Tess), FMath::RoundToInt(MinTesselation), TessellationFactor);
	}

	return SegmentTessellation;
}


FNiagaraIndexGenerationInput FNiagaraRendererRibbons::CalculateIndexBufferConfiguration(const TSharedPtr<FNiagaraRibbonCPUGeneratedVertexData>& GeneratedVertices, const FNiagaraDataBuffer* SourceParticleData,
	const FNiagaraSceneProxy* SceneProxy, const FSceneView* View, const FVector& ViewOriginForDistanceCulling, bool bShouldUseGPUInitIndices, bool bIsGPUSim) const
{
	using namespace NiagaraRendererRibbonsPrivate;

	FNiagaraIndexGenerationInput IndexGenInput;

	IndexGenInput.ViewDistance = SceneProxy->GetProxyDynamicData().LODDistanceOverride >= 0.0f ? SceneProxy->GetProxyDynamicData().LODDistanceOverride : float(SceneProxy->GetBounds().ComputeSquaredDistanceFromBoxToPoint(ViewOriginForDistanceCulling));
	IndexGenInput.LODDistanceFactor = View->LODDistanceFactor;

	if (bShouldUseGPUInitIndices)
	{
		// NumInstances is precise for GPU init from CPU but may be > number of alive particles for GPU simulations
		IndexGenInput.MaxSegmentCount = SourceParticleData->GetNumInstances();
	}
	else
	{
		IndexGenInput.MaxSegmentCount = GeneratedVertices->SortedIndices.Num();
	}
	
	

	IndexGenInput.SubSegmentCount = 1;
	if (GenerationConfig.WantsAutomaticTessellation() || GenerationConfig.WantsConstantTessellation())
	{
		if (bShouldUseGPUInitIndices)
		{
			// if we have a constant factor, use it, if not set it to the max allowed since we won't know what we need exactly until later on.
			IndexGenInput.SubSegmentCount = (TessellationConfig.TessellationMode == ENiagaraRibbonTessellationMode::Custom && TessellationConfig.bCustomUseConstantFactor)?
				TessellationConfig.CustomTessellationFactor : GNiagaraRibbonMaxTessellation;
		}
		else
		{
			IndexGenInput.SubSegmentCount = CalculateTessellationFactor(SceneProxy, View, ViewOriginForDistanceCulling);
		}
	}	
	const uint32 NumSegmentBits = CalculateBitsForRange(IndexGenInput.MaxSegmentCount);
	const uint32 NumSubSegmentBits = CalculateBitsForRange(IndexGenInput.SubSegmentCount);
	
	IndexGenInput.SegmentBitShift = NumSubSegmentBits + ShapeState.BitsNeededForShape;
	IndexGenInput.SubSegmentBitShift = ShapeState.BitsNeededForShape;

	IndexGenInput.SegmentBitMask = CalculateBitMask(NumSegmentBits);
	IndexGenInput.SubSegmentBitMask = CalculateBitMask(NumSubSegmentBits);

	IndexGenInput.ShapeBitMask = ShapeState.BitMaskForShape;
	
	IndexGenInput.TotalBitCount = NumSegmentBits + NumSubSegmentBits + ShapeState.BitsNeededForShape;
	IndexGenInput.TotalNumIndices = IndexGenInput.MaxSegmentCount * IndexGenInput.SubSegmentCount * ShapeState.TrianglesPerSegment * 3;
	IndexGenInput.CPUTriangleCount = 0;

	return IndexGenInput;
}

void FNiagaraRendererRibbons::GenerateIndexBufferForView(
	FRHICommandListBase& RHICmdList,
	FNiagaraGpuRibbonsDataManager& GpuRibbonsDataManager,
	FMeshElementCollector& Collector,
	FNiagaraIndexGenerationInput& GeneratedData, FNiagaraDynamicDataRibbon* DynamicDataRibbon,
    FNiagaraRibbonRenderingFrameViewResources* RenderingViewResources, const FSceneView* View,
    const FVector& ViewOriginForDistanceCulling
) const
{
	if (GeneratedData.MaxSegmentCount > 0)
	{
		if (DynamicDataRibbon->bUseGPUInit)
		{
			RenderingViewResources->IndirectDrawBuffer = GpuRibbonsDataManager.GetOrAllocateIndirectDrawBuffer(RHICmdList);
			RenderingViewResources->IndexBuffer = GpuRibbonsDataManager.GetOrAllocateIndexBuffer(RHICmdList, GeneratedData.TotalNumIndices, FMath::Max(GeneratedData.TotalNumIndices, DynamicDataRibbon->MaxAllocatedCountEstimate));
		}
		else
		{
			RenderingViewResources->IndexBuffer = MakeShared<FNiagaraRibbonIndexBuffer>();
			if (GeneratedData.TotalBitCount <= 16 && !GNiagaraRibbonForceIndex32)
			{
				FGlobalDynamicIndexBuffer::FAllocationEx IndexAllocation = Collector.GetDynamicIndexBuffer().Allocate<uint16>(GeneratedData.TotalNumIndices);
				RenderingViewResources->IndexBuffer->Initialize(RHICmdList, IndexAllocation);
				GenerateIndexBufferCPU<uint16>(GeneratedData, DynamicDataRibbon, ShapeState, reinterpret_cast<uint16*>(IndexAllocation.Buffer), View, ViewOriginForDistanceCulling, FeatureLevel, DrawDirection);
			}
			else
			{
				FGlobalDynamicIndexBuffer::FAllocationEx IndexAllocation = Collector.GetDynamicIndexBuffer().Allocate<uint32>(GeneratedData.TotalNumIndices);
				RenderingViewResources->IndexBuffer->Initialize(RHICmdList, IndexAllocation);
				GenerateIndexBufferCPU<uint32>(GeneratedData, DynamicDataRibbon, ShapeState, reinterpret_cast<uint32*>(IndexAllocation.Buffer), View, ViewOriginForDistanceCulling, FeatureLevel, DrawDirection);
			}
		}
	}
}

template <typename TValue>
void FNiagaraRendererRibbons::GenerateIndexBufferCPU(FNiagaraIndexGenerationInput& GeneratedData, FNiagaraDynamicDataRibbon* DynamicDataRibbon, const FNiagaraRibbonShapeGeometryData& ShapeState,
    TValue* StartIndexBuffer, const FSceneView* View, const FVector& ViewOriginForDistanceCulling, ERHIFeatureLevel::Type FeatureLevel, ENiagaraRibbonDrawDirection DrawDirection)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraRenderRibbonsGenIndiciesCPU);

	FMaterialRenderProxy* MaterialRenderProxy = DynamicDataRibbon->Material;
	check(MaterialRenderProxy);
	const bool bIsTranslucent = IsTranslucentBlendMode(MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel));
	
	const TSharedPtr<FNiagaraRibbonCPUGeneratedVertexData>& GeneratedGeometryData = DynamicDataRibbon->GenerationOutput;

	TValue* CurrentIndexBuffer = StartIndexBuffer;
	if (bIsTranslucent && GeneratedGeometryData->RibbonInfoLookup.Num())
	{
		for (const FRibbonMultiRibbonInfo& MultiRibbonInfo : GeneratedGeometryData->RibbonInfoLookup)
		{
			const TArrayView<uint32> CurrentSegmentData(GeneratedGeometryData->SegmentData.GetData() + MultiRibbonInfo.BaseSegmentDataIndex, MultiRibbonInfo.NumSegmentDataIndices);
			CurrentIndexBuffer = AppendToIndexBufferCPU<TValue>(CurrentIndexBuffer, GeneratedData, ShapeState, CurrentSegmentData, MultiRibbonInfo.UseInvertOrder(View->GetViewDirection(), ViewOriginForDistanceCulling, DrawDirection));
		}
	}
	else // Otherwise ignore multi ribbon ordering.
	{
		const TArrayView<uint32> CurrentSegmentData(GeneratedGeometryData->SegmentData.GetData(), GeneratedGeometryData->SegmentData.Num());
		CurrentIndexBuffer = AppendToIndexBufferCPU<TValue>(CurrentIndexBuffer, GeneratedData, ShapeState, CurrentSegmentData, false);
	}
	GeneratedData.CPUTriangleCount = uint32(CurrentIndexBuffer - StartIndexBuffer) / 3;
	check(CurrentIndexBuffer <= StartIndexBuffer + GeneratedData.TotalNumIndices);
}

template <typename TValue>
FORCEINLINE TValue* FNiagaraRendererRibbons::AppendToIndexBufferCPU(TValue* OutIndices, const FNiagaraIndexGenerationInput& GeneratedData, const FNiagaraRibbonShapeGeometryData& ShapeState, const TArrayView<uint32>& SegmentData, bool bInvertOrder)
{
	if (SegmentData.Num() == 0)
	{
		return OutIndices;
	}

	const uint32 FirstSegmentDataIndex = bInvertOrder ? SegmentData.Num() - 1 : 0;
	const uint32 LastSegmentDataIndex = bInvertOrder ? -1 : SegmentData.Num();
	const uint32 SegmentDataIndexInc = bInvertOrder ? -1 : 1;
	const uint32 FlipGeometryIndex = FMath::Min(FMath::Max(ShapeState.SliceTriangleToVertexIds.Num() / 2, 2), ShapeState.SliceTriangleToVertexIds.Num());
	
	for (uint32 SegmentDataIndex = FirstSegmentDataIndex; SegmentDataIndex != LastSegmentDataIndex; SegmentDataIndex += SegmentDataIndexInc)
	{
		const uint32 SegmentIndex = SegmentData[SegmentDataIndex];
		for (uint32 SubSegmentIndex = 0; SubSegmentIndex < GeneratedData.SubSegmentCount; ++SubSegmentIndex)
		{
			const bool bIsFinalInterp = SubSegmentIndex == GeneratedData.SubSegmentCount - 1;

			const uint32 ThisSegmentOffset = SegmentIndex << GeneratedData.SegmentBitShift;
			const uint32 NextSegmentOffset = (SegmentIndex + (bIsFinalInterp ? 1 : 0)) << GeneratedData.SegmentBitShift;

			const uint32 ThisSubSegmentOffset = SubSegmentIndex << GeneratedData.SubSegmentBitShift;
			const uint32 NextSubSegmentOffset = (bIsFinalInterp ? 0 : SubSegmentIndex + 1) << GeneratedData.SubSegmentBitShift;

			const uint32 CurrSegment = ThisSegmentOffset | ThisSubSegmentOffset;
			const uint32 NextSegment = NextSegmentOffset | NextSubSegmentOffset;

			uint32 TriangleId = 0;
			
			for (; TriangleId < FlipGeometryIndex; TriangleId += 2)
			{
				const int32 FirstIndex = ShapeState.SliceTriangleToVertexIds[TriangleId];
				const int32 SecondIndex = ShapeState.SliceTriangleToVertexIds[TriangleId + 1];
				
				OutIndices[0] = TValue(CurrSegment | FirstIndex);
				OutIndices[1] = TValue(CurrSegment | SecondIndex);
				OutIndices[2] = TValue(NextSegment | FirstIndex);
				OutIndices[3] = OutIndices[1];
				OutIndices[4] = TValue(NextSegment | SecondIndex);
				OutIndices[5] = OutIndices[2];

				OutIndices += 6;
			}
			for (; TriangleId < uint32(ShapeState.SliceTriangleToVertexIds.Num()); TriangleId += 2)
			{
				const uint32 FirstIndex = ShapeState.SliceTriangleToVertexIds[TriangleId];
				const uint32 SecondIndex = ShapeState.SliceTriangleToVertexIds[TriangleId + 1];

				OutIndices[0] = TValue(CurrSegment | FirstIndex);
				OutIndices[1] = TValue(CurrSegment | SecondIndex);
				OutIndices[2] = TValue(NextSegment | SecondIndex);
				OutIndices[3] = OutIndices[0];
				OutIndices[4] = OutIndices[2];
				OutIndices[5] = TValue(NextSegment | FirstIndex);

				OutIndices += 6;
			}			
		}		
	}

	return OutIndices;
}

void FNiagaraRendererRibbons::SetupPerViewUniformBuffer(FNiagaraIndexGenerationInput& GeneratedData, const FSceneView* View,
	const FSceneViewFamily& ViewFamily, const FNiagaraSceneProxy* SceneProxy, FNiagaraRibbonUniformBufferRef& OutUniformBuffer) const
{	
	FNiagaraRibbonUniformParameters PerViewUniformParameters;
	FMemory::Memzero(&PerViewUniformParameters,sizeof(PerViewUniformParameters)); // Clear unset bytes

	bool bUseLocalSpace = UseLocalSpace(SceneProxy);
	PerViewUniformParameters.bLocalSpace = bUseLocalSpace;
	PerViewUniformParameters.DeltaSeconds = ViewFamily.Time.GetDeltaWorldTimeSeconds();
	PerViewUniformParameters.SystemLWCTile = SceneProxy->GetLWCRenderTile();
	PerViewUniformParameters.CameraUp = static_cast<FVector3f>(View->GetViewUp()); // FVector4(0.0f, 0.0f, 1.0f, 0.0f);
	PerViewUniformParameters.CameraRight = static_cast<FVector3f>(View->GetViewRight());//	FVector4(1.0f, 0.0f, 0.0f, 0.0f);
	PerViewUniformParameters.ScreenAlignment = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
	PerViewUniformParameters.InterpCount = GeneratedData.SubSegmentCount;
	PerViewUniformParameters.OneOverInterpCount = 1.f / static_cast<float>(GeneratedData.SubSegmentCount);
	PerViewUniformParameters.ParticleIdShift = GeneratedData.SegmentBitShift;
	PerViewUniformParameters.ParticleIdMask = GeneratedData.SegmentBitMask;
	PerViewUniformParameters.InterpIdShift = GeneratedData.SubSegmentBitShift;
	PerViewUniformParameters.InterpIdMask = GeneratedData.SubSegmentBitMask;
	PerViewUniformParameters.SliceVertexIdMask = ShapeState.BitMaskForShape;
	PerViewUniformParameters.ShouldFlipNormalToView = ShapeState.bShouldFlipNormalToView;
	PerViewUniformParameters.ShouldUseMultiRibbon = GenerationConfig.HasRibbonIDs()? 1 : 0;

	TConstArrayView<FNiagaraRendererVariableInfo> VFVariables = RendererLayout->GetVFVariables_RenderThread();
	PerViewUniformParameters.PositionDataOffset = VFVariables[ENiagaraRibbonVFLayout::Position].GetGPUOffset();
	PerViewUniformParameters.PrevPositionDataOffset = SelectVFComponent(VFVariables, ENiagaraRibbonVFLayout::PrevPosition, ENiagaraRibbonVFLayout::Position);
	PerViewUniformParameters.VelocityDataOffset = VFVariables[ENiagaraRibbonVFLayout::Velocity].GetGPUOffset();
	PerViewUniformParameters.ColorDataOffset = VFVariables[ENiagaraRibbonVFLayout::Color].GetGPUOffset();
	PerViewUniformParameters.WidthDataOffset = VFVariables[ENiagaraRibbonVFLayout::Width].GetGPUOffset();
	PerViewUniformParameters.PrevWidthDataOffset = SelectVFComponent(VFVariables, ENiagaraRibbonVFLayout::PrevRibbonWidth, ENiagaraRibbonVFLayout::Width);
	PerViewUniformParameters.TwistDataOffset = GenerationConfig.HasTwist() ? VFVariables[ENiagaraRibbonVFLayout::Twist].GetGPUOffset() : -1;
	PerViewUniformParameters.PrevTwistDataOffset = SelectVFComponent(VFVariables, ENiagaraRibbonVFLayout::PrevRibbonTwist, ENiagaraRibbonVFLayout::PrevRibbonTwist);
	PerViewUniformParameters.NormalizedAgeDataOffset = VFVariables[ENiagaraRibbonVFLayout::NormalizedAge].GetGPUOffset();
	PerViewUniformParameters.MaterialRandomDataOffset = VFVariables[ENiagaraRibbonVFLayout::MaterialRandom].GetGPUOffset();
	PerViewUniformParameters.MaterialParamDataOffset = VFVariables[ENiagaraRibbonVFLayout::MaterialParam0].GetGPUOffset();
	PerViewUniformParameters.MaterialParam1DataOffset = VFVariables[ENiagaraRibbonVFLayout::MaterialParam1].GetGPUOffset();
	PerViewUniformParameters.MaterialParam2DataOffset = VFVariables[ENiagaraRibbonVFLayout::MaterialParam2].GetGPUOffset();
	PerViewUniformParameters.MaterialParam3DataOffset = VFVariables[ENiagaraRibbonVFLayout::MaterialParam3].GetGPUOffset();
	PerViewUniformParameters.DistanceFromStartOffset =
		(UV0Settings.DistributionMode == ENiagaraRibbonUVDistributionMode::TiledFromStartOverRibbonLength ||
		UV1Settings.DistributionMode == ENiagaraRibbonUVDistributionMode::TiledFromStartOverRibbonLength)?
		VFVariables[ENiagaraRibbonVFLayout::DistanceFromStart].GetGPUOffset() : -1;
	PerViewUniformParameters.U0OverrideDataOffset = UV0Settings.bEnablePerParticleUOverride ? VFVariables[ENiagaraRibbonVFLayout::U0Override].GetGPUOffset() : -1;
	PerViewUniformParameters.V0RangeOverrideDataOffset = UV0Settings.bEnablePerParticleVRangeOverride ? VFVariables[ENiagaraRibbonVFLayout::V0RangeOverride].GetGPUOffset() : -1;
	PerViewUniformParameters.U1OverrideDataOffset = UV1Settings.bEnablePerParticleUOverride ? VFVariables[ENiagaraRibbonVFLayout::U1Override].GetGPUOffset() : -1;
	PerViewUniformParameters.V1RangeOverrideDataOffset = UV1Settings.bEnablePerParticleVRangeOverride ? VFVariables[ENiagaraRibbonVFLayout::V1RangeOverride].GetGPUOffset() : -1;

	PerViewUniformParameters.MaterialParamValidMask = GenerationConfig.GetMaterialParamValidMask();

	bool bShouldDoFacing = FacingMode == ENiagaraRibbonFacingMode::Custom || FacingMode == ENiagaraRibbonFacingMode::CustomSideVector;
	PerViewUniformParameters.FacingDataOffset = bShouldDoFacing ? VFVariables[ENiagaraRibbonVFLayout::Facing].GetGPUOffset() : -1;
	PerViewUniformParameters.PrevFacingDataOffset = bShouldDoFacing ? SelectVFComponent(VFVariables, ENiagaraRibbonVFLayout::PrevRibbonFacing, ENiagaraRibbonVFLayout::Facing) : -1;

	PerViewUniformParameters.U0DistributionMode = static_cast<int32>(UV0Settings.DistributionMode);
	PerViewUniformParameters.U1DistributionMode = static_cast<int32>(UV1Settings.DistributionMode);
	PerViewUniformParameters.PackedVData.X = float(UV0Settings.Scale.Y);
	PerViewUniformParameters.PackedVData.Y = float(UV0Settings.Offset.Y);
	PerViewUniformParameters.PackedVData.Z = float(UV1Settings.Scale.Y);
	PerViewUniformParameters.PackedVData.W = float(UV1Settings.Offset.Y);

	OutUniformBuffer = FNiagaraRibbonUniformBufferRef::CreateUniformBufferImmediate(PerViewUniformParameters, UniformBuffer_SingleFrame);
}

inline void FNiagaraRendererRibbons::SetupMeshBatchAndCollectorResourceForView(FRHICommandListBase& RHICmdList, const FNiagaraIndexGenerationInput& GeneratedData, FNiagaraDynamicDataRibbon* DynamicDataRibbon, const FNiagaraDataBuffer* SourceParticleData, const FSceneView* View,
    const FSceneViewFamily& ViewFamily, const FNiagaraSceneProxy* SceneProxy, const TSharedPtr<FNiagaraRibbonRenderingFrameResources>& RenderingResources, FNiagaraRibbonRenderingFrameViewResources* RenderingViewResources,
    FMeshBatch& OutMeshBatch, bool bShouldUseGPUInitIndices) const
{
	const bool bIsWireframe = ViewFamily.EngineShowFlags.Wireframe;
	FMaterialRenderProxy* MaterialRenderProxy = DynamicDataRibbon->Material;
	check(MaterialRenderProxy);
	
	// Set common data on vertex factory
	FNiagaraRibbonVFLooseParameters VFLooseParams;
#if RHI_RAYTRACING
	VFLooseParams.IndexBuffer = GetSrvOrDefaultUInt(RenderingViewResources->IndexBuffer->SRV);
	VFLooseParams.UseIndexBufferForRayTracing = bShouldUseGPUInitIndices;
#else
	VFLooseParams.IndexBuffer = GetDummyUIntBuffer();
	VFLooseParams.UseIndexBufferForRayTracing = false;
#endif
	VFLooseParams.SortedIndices = VertexBuffers.SortedIndicesBuffer.SRV;
	VFLooseParams.TangentsAndDistances = VertexBuffers.TangentsAndDistancesBuffer.SRV;
	VFLooseParams.MultiRibbonIndices = GetSrvOrDefaultUInt(VertexBuffers.MultiRibbonIndicesBuffer.SRV);
	VFLooseParams.PackedPerRibbonDataByIndex = VertexBuffers.RibbonLookupTableBuffer.SRV;
	VFLooseParams.SliceVertexData = ShapeState.SliceVertexDataBuffer.SRV;
	VFLooseParams.NiagaraParticleDataFloat = RenderingResources->ParticleFloatSRV;
	VFLooseParams.NiagaraParticleDataHalf = RenderingResources->ParticleHalfSRV;
	VFLooseParams.NiagaraFloatDataStride = FMath::Max(RenderingResources->ParticleFloatDataStride, RenderingResources->ParticleHalfDataStride);
	VFLooseParams.FacingMode = static_cast<uint32>(FacingMode);
	VFLooseParams.Shape = static_cast<uint32>(ShapeState.Shape);
	VFLooseParams.NeedsPreciseMotionVectors = GenerationConfig.NeedsPreciseMotionVectors();
	VFLooseParams.UseGeometryNormals = (ShapeState.Shape != ENiagaraRibbonShapeMode::Plane || bUseGeometryNormals) ? 1 : 0;

	VFLooseParams.IndirectDrawOutput = bShouldUseGPUInitIndices ? (FRHIShaderResourceView*)RenderingViewResources->IndirectDrawBuffer->SRV : GetDummyUIntBuffer();
	VFLooseParams.IndirectDrawOutputOffset = bShouldUseGPUInitIndices ? 0 : -1;

	// Collector.AllocateOneFrameResource uses default ctor, initialize the vertex factory
	RenderingViewResources->VertexFactory.LooseParameterUniformBuffer = FNiagaraRibbonVFLooseParametersRef::CreateUniformBufferImmediate(VFLooseParams, UniformBuffer_SingleFrame);
	RenderingViewResources->VertexFactory.InitResource(RHICmdList);
	RenderingViewResources->VertexFactory.SetRibbonUniformBuffer(RenderingViewResources->UniformBuffer);

	OutMeshBatch.VertexFactory = &RenderingViewResources->VertexFactory;
	OutMeshBatch.CastShadow = SceneProxy->CastsDynamicShadow() && bCastShadows;
#if RHI_RAYTRACING
	OutMeshBatch.CastRayTracedShadow = SceneProxy->CastsDynamicShadow() && bCastShadows;
#endif
	OutMeshBatch.bUseAsOccluder = false;
	OutMeshBatch.ReverseCulling = SceneProxy->IsLocalToWorldDeterminantNegative();
	OutMeshBatch.bDisableBackfaceCulling = ShapeState.bDisableBackfaceCulling;
	OutMeshBatch.Type = PT_TriangleList;
	OutMeshBatch.DepthPriorityGroup = SceneProxy->GetDepthPriorityGroup(View);
	OutMeshBatch.bCanApplyViewModeOverrides = true;
	OutMeshBatch.bUseWireframeSelectionColoring = SceneProxy->IsSelected();
	OutMeshBatch.SegmentIndex = 0;
	OutMeshBatch.MaterialRenderProxy = bIsWireframe? UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy() : MaterialRenderProxy;
	
	FMeshBatchElement& MeshElement = OutMeshBatch.Elements[0];
	MeshElement.IndexBuffer = RenderingViewResources->IndexBuffer.Get();
	MeshElement.FirstIndex = RenderingViewResources->IndexBuffer->FirstIndex;
	MeshElement.NumInstances = 1;
	MeshElement.MinVertexIndex = 0;
	MeshElement.MaxVertexIndex = 0;

	if (bShouldUseGPUInitIndices)
	{
		MeshElement.NumPrimitives = 0;
		MeshElement.IndirectArgsBuffer = RenderingViewResources->IndirectDrawBuffer->Buffer;
		MeshElement.IndirectArgsOffset = View->IsInstancedStereo() ? FNiagaraRibbonIndirectDrawBufferLayout::StereoDrawIndirectParametersByteOffset : FNiagaraRibbonIndirectDrawBufferLayout::DrawIndirectParametersByteOffset;
	}
	else
	{
		MeshElement.NumPrimitives = GeneratedData.CPUTriangleCount;
		check(MeshElement.NumPrimitives > 0);
	}	
	
	// TODO: MotionVector/Velocity? Probably need to look into this?
	MeshElement.PrimitiveUniformBuffer = SceneProxy->GetCustomUniformBuffer(RHICmdList, false);	// Note: Ribbons don't generate accurate velocities so disabling	
}

void FNiagaraRendererRibbons::InitializeViewIndexBuffersGPU(FRHICommandListImmediate& RHICmdList, FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, const FNiagaraRibbonGPUInitParameters& GpuInitParameters, const FNiagaraRibbonRenderingFrameViewResources* RenderingViewResources) const
{
	using namespace NiagaraRendererRibbonsPrivate;

	SCOPE_CYCLE_COUNTER(STAT_NiagaraRenderRibbonsGenIndiciesGPU);
	
	if (!RenderingViewResources->IndirectDrawBuffer->Buffer.IsValid())
	{
		return;
	}

	const uint32 NumInstances = GpuInitParameters.NumInstances;

	SCOPED_DRAW_EVENT(RHICmdList, NiagaraRenderRibbonsGenIndiciesGPU);
	{
		TShaderMapRef<FNiagaraRibbonCreateIndexBufferParamsCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		FNiagaraRibbonInitializeIndices Params;
		FMemory::Memzero(Params);

		Params.IndirectDrawOutput = RenderingViewResources->IndirectDrawBuffer->UAV;
		Params.VertexGenerationResults = VertexBuffers.GPUComputeCommandBuffer.SRV;

		// Total particle Count
		Params.TotalNumParticlesDirect = NumInstances;

		// Indirect particle Count
		Params.EmitterParticleCountsBuffer = GetSrvOrDefaultUInt(ComputeDispatchInterface->GetGPUInstanceCounterManager().GetInstanceCountBuffer());
		Params.EmitterParticleCountsBufferOffset = GpuInitParameters.GPUInstanceCountBufferOffset;

		Params.IndirectDrawOutputIndex = 0;
		Params.VertexGenerationResultsIndex = 0; /*Offset into command buffer*/
		Params.IndexGenThreadSize = FNiagaraRibbonComputeCommon::IndexGenThreadSize;
		Params.TrianglesPerSegment = ShapeState.TrianglesPerSegment;

		Params.RibbonTessellationMode = GetGpuTessellationMode(GenerationConfig);
		Params.ViewDistance = RenderingViewResources->IndexGenerationSettings.ViewDistance;
		Params.LODDistanceFactor = RenderingViewResources->IndexGenerationSettings.LODDistanceFactor;
		Params.TessellationMode = static_cast<uint32>(TessellationConfig.TessellationMode);
		Params.bCustomUseConstantFactor = TessellationConfig.bCustomUseConstantFactor ? 1 : 0;
		Params.CustomTessellationFactor = TessellationConfig.CustomTessellationFactor;
		Params.CustomTessellationMinAngle = TessellationConfig.CustomTessellationMinAngle;
		Params.bCustomUseScreenSpace = TessellationConfig.bCustomUseScreenSpace ? 1 : 0;
		Params.GNiagaraRibbonMaxTessellation = GNiagaraRibbonMaxTessellation;
		Params.GNiagaraRibbonTessellationAngle = GNiagaraRibbonTessellationAngle;
		Params.GNiagaraRibbonTessellationScreenPercentage = GNiagaraRibbonTessellationScreenPercentage;
		Params.GNiagaraRibbonTessellationEnabled = GNiagaraRibbonTessellationEnabled ? 1 : 0;
		Params.GNiagaraRibbonTessellationMinDisplacementError = GNiagaraRibbonTessellationMinDisplacementError;

		RHICmdList.Transition(FRHITransitionInfo(RenderingViewResources->IndirectDrawBuffer->UAV, ERHIAccess::SRVMask | ERHIAccess::IndirectArgs, ERHIAccess::UAVCompute));
		FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Params, FIntVector(1, 1, 1));
		RHICmdList.Transition(FRHITransitionInfo(RenderingViewResources->IndirectDrawBuffer->UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVMask | ERHIAccess::IndirectArgs));
	}
	
	// Not possible to have a valid ribbon with less than 2 particles, abort!
	// but we do need to write out the indirect draw so it will behave correctly.
	// So the initialize call above sets up the indirect draw, but we'll skip the actual index gen below.
	if (NumInstances < 2)
	{
		return;
	}
		
	{
		FNiagaraRibbonCreateIndexBufferCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRibbonHasFullRibbonID>(GenerationConfig.HasFullRibbonIDs());
		PermutationVector.Set<FRibbonHasRibbonID>(GenerationConfig.HasSimpleRibbonIDs());

		TShaderMapRef<FNiagaraRibbonCreateIndexBufferCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);
		// const int TotalNumInvocations = Params.TotalNumParticles * IndicesConfig.SubSegmentCount;
		// const uint32 NumThreadGroups = FMath::DivideAndRoundUp<uint32>(TotalNumInvocations, FNiagaraRibbonComputeCommon::IndexGenThreadSize);

		constexpr uint32 IndirectDispatchArgsOffset = 0;

		FNiagaraRibbonGenerateIndices Params;
		FMemory::Memzero(Params);
		
		Params.GeneratedIndicesBuffer = RenderingViewResources->IndexBuffer->UAV;
		Params.SortedIndices = VertexBuffers.SortedIndicesBuffer.SRV;
		Params.MultiRibbonIndices = VertexBuffers.MultiRibbonIndicesBuffer.SRV;
		Params.Segments = VertexBuffers.SegmentsBuffer.SRV;

		Params.IndirectDrawInfo = RenderingViewResources->IndirectDrawBuffer->SRV;
		Params.TriangleToVertexIds = ShapeState.SliceTriangleToVertexIdsBuffer.SRV;

		// Total particle Count
		Params.TotalNumParticlesDirect = GpuInitParameters.NumInstances;

		// Indirect particle Count
		Params.EmitterParticleCountsBuffer = GetSrvOrDefaultUInt(ComputeDispatchInterface->GetGPUInstanceCounterManager().GetInstanceCountBuffer());
		Params.EmitterParticleCountsBufferOffset = GpuInitParameters.GPUInstanceCountBufferOffset;

		Params.IndexBufferOffset = 0;
		Params.IndirectDrawInfoIndex = 0;
		Params.TriangleToVertexIdsCount = ShapeState.SliceTriangleToVertexIds.Num();

		Params.TrianglesPerSegment = ShapeState.TrianglesPerSegment;
		Params.NumVerticesInSlice = ShapeState.NumVerticesInSlice;
		Params.BitsNeededForShape = ShapeState.BitsNeededForShape;
		Params.BitMaskForShape = ShapeState.BitMaskForShape;
		Params.SegmentBitShift = RenderingViewResources->IndexGenerationSettings.SegmentBitShift;
		Params.SegmentBitMask = RenderingViewResources->IndexGenerationSettings.SegmentBitMask;
		Params.SubSegmentBitShift = RenderingViewResources->IndexGenerationSettings.SubSegmentBitShift;
		Params.SubSegmentBitMask = RenderingViewResources->IndexGenerationSettings.SubSegmentBitMask;
		
		RHICmdList.Transition(FRHITransitionInfo(RenderingViewResources->IndexBuffer->UAV, ERHIAccess::SRVMask | ERHIAccess::VertexOrIndexBuffer, ERHIAccess::UAVCompute));
		FComputeShaderUtils::DispatchIndirect(RHICmdList, ComputeShader, Params, RenderingViewResources->IndirectDrawBuffer->Buffer, IndirectDispatchArgsOffset);
		RHICmdList.Transition(FRHITransitionInfo(RenderingViewResources->IndexBuffer->UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVMask | ERHIAccess::VertexOrIndexBuffer));
	}
}

void FNiagaraRendererRibbons::InitializeVertexBuffersResources(FRHICommandListBase& RHICmdList, const FNiagaraDynamicDataRibbon* DynamicDataRibbon, FNiagaraDataBuffer* SourceParticleData,
                                                               FGlobalDynamicReadBuffer& DynamicReadBuffer, const TSharedPtr<FNiagaraRibbonRenderingFrameResources>& RenderingResources, bool bShouldUseGPUInit) const
{

	{
		//-OPT: We only need to update this data once for all GDME passes
		UE::TScopeLock VertexBuffersLock(VertexBuffersGuard);

		// Make sure our ribbon data buffers are setup
		VertexBuffers.InitializeOrUpdateBuffers(RHICmdList, GenerationConfig, DynamicDataRibbon->GenerationOutput, SourceParticleData, DynamicDataRibbon->MaxAllocationCount, bShouldUseGPUInit);
	}
	
	// Now we need to bind the source particle data, copying it to the gpu if necessary
	if (DynamicDataRibbon->bIsGPUSystem)
	{		
		RenderingResources->ParticleFloatSRV = GetSrvOrDefaultFloat(SourceParticleData->GetGPUBufferFloat());
		RenderingResources->ParticleHalfSRV = GetSrvOrDefaultHalf(SourceParticleData->GetGPUBufferHalf());
		RenderingResources->ParticleIntSRV = GetSrvOrDefaultInt(SourceParticleData->GetGPUBufferInt());
		
		RenderingResources->ParticleFloatDataStride = SourceParticleData->GetFloatStride() / sizeof(float);
		RenderingResources->ParticleHalfDataStride = SourceParticleData->GetHalfStride() / sizeof(FFloat16);
		RenderingResources->ParticleIntDataStride = SourceParticleData->GetInt32Stride() / sizeof(int32);
		
		RenderingResources->RibbonIdParamOffset = RibbonIDParamDataSetOffset;
	}
	else 
	{
		TArray<uint32, TInlineAllocator<2>> IntParamsToCopy;
		if (bShouldUseGPUInit && GenerationConfig.HasRibbonIDs())
		{
			RenderingResources->RibbonIdParamOffset = IntParamsToCopy.Add(RibbonIDParamDataSetOffset);

			// Also add acquire index if we're running full sized ids.
			if (GenerationConfig.HasFullRibbonIDs())
			{
				IntParamsToCopy.Add(RibbonIDParamDataSetOffset + 1);
			}		
		}
		
		RenderingResources->ParticleData = TransferDataToGPU(RHICmdList, DynamicReadBuffer, RendererLayout, IntParamsToCopy, SourceParticleData);

		RenderingResources->ParticleFloatSRV = GetSrvOrDefaultFloat(RenderingResources->ParticleData.FloatData);
		RenderingResources->ParticleHalfSRV = GetSrvOrDefaultHalf(RenderingResources->ParticleData.HalfData);
		RenderingResources->ParticleIntSRV = GetSrvOrDefaultInt(RenderingResources->ParticleData.IntData);
		
		RenderingResources->ParticleFloatDataStride = RenderingResources->ParticleData.FloatStride / sizeof(float);
		RenderingResources->ParticleHalfDataStride = RenderingResources->ParticleData.HalfStride / sizeof(FFloat16);
		RenderingResources->ParticleIntDataStride = RenderingResources->ParticleData.IntStride / sizeof(int32);
	}

	// If the data was generated sync it here, otherwise we rely on the generation step later to populate it
	if (DynamicDataRibbon->GenerationOutput.IsValid() && DynamicDataRibbon->GenerationOutput->SegmentData.Num() > 0)
	{
		//-OPT: We only need to update this data once for all GDME passes
		UE::TScopeLock VertexBuffersLock(VertexBuffersGuard);

		const FNiagaraRibbonCPUGeneratedVertexData& GeneratedGeometryData = *DynamicDataRibbon->GenerationOutput;

		void *IndexPtr = RHICmdList.LockBuffer(VertexBuffers.SortedIndicesBuffer.Buffer, 0, GeneratedGeometryData.SortedIndices.Num() * sizeof(int32), RLM_WriteOnly);
		FMemory::Memcpy(IndexPtr, GeneratedGeometryData.SortedIndices.GetData(), GeneratedGeometryData.SortedIndices.Num() * sizeof(int32));
		RHICmdList.UnlockBuffer(VertexBuffers.SortedIndicesBuffer.Buffer);

		// pass in the CPU generated total segment distance (for tiling distance modes); needs to be a buffer so we can fetch them in the correct order based on Draw Direction (front->back or back->front)
		//	otherwise UVs will pop when draw direction changes based on camera view point
		void *TangentsAndDistancesPtr = RHICmdList.LockBuffer(VertexBuffers.TangentsAndDistancesBuffer.Buffer, 0, GeneratedGeometryData.TangentAndDistances.Num() * sizeof(FVector4f), RLM_WriteOnly);
		FMemory::Memcpy(TangentsAndDistancesPtr, GeneratedGeometryData.TangentAndDistances.GetData(), GeneratedGeometryData.TangentAndDistances.Num() * sizeof(FVector4f));
		RHICmdList.UnlockBuffer(VertexBuffers.TangentsAndDistancesBuffer.Buffer);
		
		// Copy a buffer which has the per particle multi ribbon index.
		if (GenerationConfig.HasRibbonIDs())
		{
			void* MultiRibbonIndexPtr = RHICmdList.LockBuffer(VertexBuffers.MultiRibbonIndicesBuffer.Buffer, 0, GeneratedGeometryData.MultiRibbonIndices.Num() * sizeof(uint32), RLM_WriteOnly);
			FMemory::Memcpy(MultiRibbonIndexPtr, GeneratedGeometryData.MultiRibbonIndices.GetData(), GeneratedGeometryData.MultiRibbonIndices.Num() * sizeof(uint32));
			RHICmdList.UnlockBuffer(VertexBuffers.MultiRibbonIndicesBuffer.Buffer);
		}
		
		// Copy the packed u data for stable age based uv generation.
		//-OPT: Remove copy, push straight into GPU Memory
		TArray<uint32> PackedRibbonLookupTable;
		PackedRibbonLookupTable.Reserve(GeneratedGeometryData.RibbonInfoLookup.Num() * FRibbonMultiRibbonInfoBufferEntry::NumElements);
		for (int32 Index = 0; Index < GeneratedGeometryData.RibbonInfoLookup.Num(); Index++)
		{
			GeneratedGeometryData.RibbonInfoLookup[Index].PackElementsToLookupTableBuffer(PackedRibbonLookupTable);
		}
		
		void *PackedPerRibbonDataByIndexPtr = RHICmdList.LockBuffer(VertexBuffers.RibbonLookupTableBuffer.Buffer, 0, PackedRibbonLookupTable.Num() * sizeof(uint32), RLM_WriteOnly);
		FMemory::Memcpy(PackedPerRibbonDataByIndexPtr, PackedRibbonLookupTable.GetData(), PackedRibbonLookupTable.Num() * sizeof(uint32));
		RHICmdList.UnlockBuffer(VertexBuffers.RibbonLookupTableBuffer.Buffer);		
	}
}

FRibbonComputeUniformParameters FNiagaraRendererRibbons::SetupComputeVertexGenParams(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, const TSharedPtr<FNiagaraRibbonRenderingFrameResources>& RenderingResources, const FNiagaraRibbonGPUInitParameters& GpuInitParameters) const
{
	using namespace NiagaraRendererRibbonsPrivate;

	FRibbonComputeUniformParameters CommonParams;
	FMemory::Memzero(CommonParams);

	// Total particle Count
	CommonParams.TotalNumParticlesDirect = GpuInitParameters.NumInstances;

	// Indirect particle Count
	CommonParams.EmitterParticleCountsBuffer = GetSrvOrDefaultUInt(ComputeDispatchInterface->GetGPUInstanceCounterManager().GetInstanceCountBuffer());
	CommonParams.EmitterParticleCountsBufferOffset = GpuInitParameters.GPUInstanceCountBufferOffset;

	// Niagara sim data
	CommonParams.NiagaraParticleDataFloat = GetSrvOrDefaultFloat(RenderingResources->ParticleFloatSRV);
	CommonParams.NiagaraParticleDataHalf = RenderingResources->ParticleHalfSRV? RenderingResources->ParticleHalfSRV : GetDummyHalfBuffer();
	CommonParams.NiagaraParticleDataInt = GetSrvOrDefaultInt(RenderingResources->ParticleIntSRV);
	CommonParams.NiagaraFloatDataStride = RenderingResources->ParticleFloatDataStride;
	CommonParams.NiagaraIntDataStride = RenderingResources->ParticleIntDataStride;

	
	// Int bindings
	CommonParams.RibbonIdDataOffset = RenderingResources->RibbonIdParamOffset;

	// Float bindings
	const TConstArrayView<FNiagaraRendererVariableInfo> VFVariables = RendererLayout->GetVFVariables_RenderThread();
	CommonParams.PositionDataOffset = VFVariables[ENiagaraRibbonVFLayout::Position].GetGPUOffset();
	CommonParams.PrevPositionDataOffset = VFVariables[ENiagaraRibbonVFLayout::PrevPosition].GetGPUOffset();
	CommonParams.VelocityDataOffset = VFVariables[ENiagaraRibbonVFLayout::Velocity].GetGPUOffset();
	CommonParams.ColorDataOffset = VFVariables[ENiagaraRibbonVFLayout::Color].GetGPUOffset();
	CommonParams.WidthDataOffset = VFVariables[ENiagaraRibbonVFLayout::Width].GetGPUOffset();
	CommonParams.PrevWidthDataOffset = VFVariables[ENiagaraRibbonVFLayout::PrevRibbonWidth].GetGPUOffset();
	CommonParams.TwistDataOffset = VFVariables[ENiagaraRibbonVFLayout::Twist].GetGPUOffset();
	CommonParams.PrevTwistDataOffset = VFVariables[ENiagaraRibbonVFLayout::PrevRibbonTwist].GetGPUOffset();
	CommonParams.NormalizedAgeDataOffset = VFVariables[ENiagaraRibbonVFLayout::NormalizedAge].GetGPUOffset();
	CommonParams.MaterialRandomDataOffset = VFVariables[ENiagaraRibbonVFLayout::MaterialRandom].GetGPUOffset();
	CommonParams.MaterialParamDataOffset = VFVariables[ENiagaraRibbonVFLayout::MaterialParam0].GetGPUOffset();
	CommonParams.MaterialParam1DataOffset = VFVariables[ENiagaraRibbonVFLayout::MaterialParam1].GetGPUOffset();
	CommonParams.MaterialParam2DataOffset = VFVariables[ENiagaraRibbonVFLayout::MaterialParam2].GetGPUOffset();
	CommonParams.MaterialParam3DataOffset = VFVariables[ENiagaraRibbonVFLayout::MaterialParam3].GetGPUOffset();

	const bool bShouldLinkDistanceFromStart = (UV0Settings.DistributionMode == ENiagaraRibbonUVDistributionMode::TiledFromStartOverRibbonLength ||
		UV1Settings.DistributionMode == ENiagaraRibbonUVDistributionMode::TiledFromStartOverRibbonLength);
	
	CommonParams.DistanceFromStartOffset = bShouldLinkDistanceFromStart ? VFVariables[ENiagaraRibbonVFLayout::DistanceFromStart].GetGPUOffset() : -1;
	CommonParams.U0OverrideDataOffset = UV0Settings.bEnablePerParticleUOverride ? VFVariables[ENiagaraRibbonVFLayout::U0Override].GetGPUOffset() : -1;
	CommonParams.V0RangeOverrideDataOffset = UV0Settings.bEnablePerParticleVRangeOverride ? VFVariables[ENiagaraRibbonVFLayout::V0RangeOverride].GetGPUOffset() : -1;
	CommonParams.U1OverrideDataOffset = UV1Settings.bEnablePerParticleUOverride ? VFVariables[ENiagaraRibbonVFLayout::U1Override].GetGPUOffset() : -1;
	CommonParams.V1RangeOverrideDataOffset = UV1Settings.bEnablePerParticleVRangeOverride ? VFVariables[ENiagaraRibbonVFLayout::V1RangeOverride].GetGPUOffset() : -1;

	CommonParams.MaterialParamValidMask = GenerationConfig.GetMaterialParamValidMask();

	const bool bShouldDoFacing = FacingMode == ENiagaraRibbonFacingMode::Custom || FacingMode == ENiagaraRibbonFacingMode::CustomSideVector;
	CommonParams.FacingDataOffset = bShouldDoFacing ? VFVariables[ENiagaraRibbonVFLayout::Facing].GetGPUOffset() : -1;
	CommonParams.PrevFacingDataOffset = bShouldDoFacing ? VFVariables[ENiagaraRibbonVFLayout::PrevRibbonFacing].GetGPUOffset() : -1;

	CommonParams.RibbonLinkOrderIsFloat = bGpuRibbonLinkIsFloat;
	CommonParams.RibbonLinkOrderDataOffset = GpuRibbonLinkOrderOffset;

	CommonParams.RibbonTessellationMode = GetGpuTessellationMode(GenerationConfig);

	CommonParams.U0DistributionMode = static_cast<int32>(UV0Settings.DistributionMode);
	CommonParams.U1DistributionMode = static_cast<int32>(UV1Settings.DistributionMode);

	return CommonParams;
}

void FNiagaraRendererRibbons::InitializeVertexBuffersGPU(FRHICommandListImmediate& RHICmdList, FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, const FNiagaraRibbonGPUInitParameters& GpuInitParameters,
	FNiagaraRibbonGPUInitComputeBuffers& TempBuffers, const TSharedPtr<FNiagaraRibbonRenderingFrameResources>& RenderingResources) const
{	
	SCOPE_CYCLE_COUNTER(STAT_NiagaraRenderRibbonsGenVerticesGPU);

	FRibbonComputeUniformParameters CommonParams = SetupComputeVertexGenParams(ComputeDispatchInterface, RenderingResources, GpuInitParameters);

	const int32 NumExecutableInstances = GpuInitParameters.NumInstances;

	const bool bCanRun = NumExecutableInstances >= 2;
	
	// Clear the command buffer if we just initialized it, or if the sim doesn't have enough data to run
	if ((!bCanRun || VertexBuffers.bJustCreatedCommandBuffer) && VertexBuffers.GPUComputeCommandBuffer.NumBytes > 0)
	{
		RHICmdList.Transition(FRHITransitionInfo(VertexBuffers.GPUComputeCommandBuffer.Buffer, ERHIAccess::SRVMask | ERHIAccess::IndirectArgs, ERHIAccess::UAVCompute));
		RHICmdList.ClearUAVUint(VertexBuffers.GPUComputeCommandBuffer.UAV, FUintVector4(0));
		RHICmdList.Transition(FRHITransitionInfo(VertexBuffers.GPUComputeCommandBuffer.Buffer, ERHIAccess::UAVCompute, ERHIAccess::SRVMask | ERHIAccess::IndirectArgs));
		VertexBuffers.bJustCreatedCommandBuffer = false;
	}
	
	// Not possible to have a valid ribbon with less than 2 particles, so the remaining work is unnecessary as there's nothing needed here
	if (!bCanRun)
	{
		return;
	}

	{
		SCOPED_DRAW_EVENT(RHICmdList, NiagaraRenderRibbonsGenVerticesSortGPU);
		SCOPE_CYCLE_COUNTER(STAT_NiagaraRenderRibbonsGenVerticesSortGPU);
		
		FNiagaraRibbonSortPhase1CS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRibbonHasFullRibbonID>(GenerationConfig.HasFullRibbonIDs());
		PermutationVector.Set<FRibbonHasRibbonID>(GenerationConfig.HasSimpleRibbonIDs());
		
		TShaderMapRef<FNiagaraRibbonSortPhase1CS> BubbleSortShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);
		TShaderMapRef<FNiagaraRibbonSortPhase2CS> MergeSortShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);
			
		FRibbonOrderSortParameters SortParams;
		FMemory::Memzero(SortParams);
		SortParams.Common = CommonParams;
		SortParams.DestinationSortedIndices = VertexBuffers.SortedIndicesBuffer.UAV;
		SortParams.SortedIndices = GetSrvOrDefaultUInt(TempBuffers.SortBuffer.SRV);
		
		int CurrentBufferOrientation = 0;		
		const auto SwapBuffers = [&]()
		{
			CurrentBufferOrientation ^= 0x1;	
			const bool bComputeOnOutputBuffer = CurrentBufferOrientation == 0;
			
			SortParams.DestinationSortedIndices = bComputeOnOutputBuffer ? VertexBuffers.SortedIndicesBuffer.UAV : TempBuffers.SortBuffer.UAV;
			SortParams.SortedIndices = bComputeOnOutputBuffer ? TempBuffers.SortBuffer.SRV : VertexBuffers.SortedIndicesBuffer.SRV;			
		};
	
		const uint32 NumInitialThreadGroups = FMath::DivideAndRoundUp<uint32>(NumExecutableInstances, FNiagaraRibbonSortPhase1CS::BubbleSortGroupWidth);	
		const uint32 NumMergeSortThreadGroups = FMath::DivideAndRoundUp<uint32>(NumExecutableInstances, FNiagaraRibbonSortPhase2CS::ThreadGroupSize);
		const uint32 MergeSortPasses = FMath::CeilLogTwo(NumInitialThreadGroups);
		
		// If should do an initial flip so we start with the temp buffer to end in the correct buffer
		if (MergeSortPasses % 2 != 0)
		{
			SwapBuffers();
		}

		{			
			SCOPED_DRAW_EVENT(RHICmdList, NiagaraRenderRibbonsGenVerticesInitialSortGPU);
			SCOPE_CYCLE_COUNTER(STAT_NiagaraRenderRibbonsGenVerticesInitialSortGPU);
			
			// Initial sort, sets up the buffer, and runs a parallel bubble sort to create groups of BubbleSortGroupWidth size
			RHICmdList.Transition(FRHITransitionInfo(SortParams.DestinationSortedIndices, ERHIAccess::SRVMask | ERHIAccess::VertexOrIndexBuffer, ERHIAccess::UAVCompute));
			FComputeShaderUtils::Dispatch(RHICmdList, BubbleSortShader, SortParams, FIntVector(NumInitialThreadGroups, 1, 1));
			RHICmdList.Transition(FRHITransitionInfo(SortParams.DestinationSortedIndices, ERHIAccess::UAVCompute, ERHIAccess::SRVMask | ERHIAccess::VertexOrIndexBuffer));
		}
		
		{
			SCOPED_DRAW_EVENT(RHICmdList, NiagaraRenderRibbonsGenVerticesFinalSortGPU);
			SCOPE_CYCLE_COUNTER(STAT_NiagaraRenderRibbonsGenVerticesFinalSortGPU);
			
			// Repeatedly runs a scatter based merge sort until we have the final buffer
			uint32 SortGroupSize = FNiagaraRibbonSortPhase1CS::BubbleSortGroupWidth;
			for (uint32 Idx = 0; Idx < MergeSortPasses; Idx++)
			{
				SortParams.MergeSortSourceBlockSize = SortGroupSize;
				SortParams.MergeSortDestinationBlockSize = SortGroupSize * 2;
		
				SwapBuffers();
			
				RHICmdList.Transition(FRHITransitionInfo(SortParams.DestinationSortedIndices, ERHIAccess::SRVMask | ERHIAccess::VertexOrIndexBuffer, ERHIAccess::UAVCompute));
				FComputeShaderUtils::Dispatch(RHICmdList, MergeSortShader, SortParams, FIntVector(NumInitialThreadGroups, 1, 1));
				RHICmdList.Transition(FRHITransitionInfo(SortParams.DestinationSortedIndices, ERHIAccess::UAVCompute, ERHIAccess::SRVMask | ERHIAccess::VertexOrIndexBuffer));
				
				SortGroupSize *= 2;
			}			
		}		
	}
	
	{
		SCOPED_DRAW_EVENT(RHICmdList, NiagaraRenderRibbonsAggregation);

		FNiagaraRibbonVertexReductionInitializationCS::FPermutationDomain InitPermutationVector;
		InitPermutationVector.Set<FRibbonHasFullRibbonID>(GenerationConfig.HasFullRibbonIDs());
		InitPermutationVector.Set<FRibbonHasRibbonID>(GenerationConfig.HasSimpleRibbonIDs());
		TShaderMapRef<FNiagaraRibbonVertexReductionInitializationCS> ReductionInitializationShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), InitPermutationVector);

		FNiagaraRibbonAggregationStepCS::FPermutationDomain StepPermutationVector;
		StepPermutationVector.Set<FRibbonHasFullRibbonID>(GenerationConfig.HasFullRibbonIDs());
		StepPermutationVector.Set<FRibbonHasRibbonID>(GenerationConfig.HasSimpleRibbonIDs());
		TShaderMapRef<FNiagaraRibbonAggregationStepCS> AggregationStepShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), StepPermutationVector);

		FNiagaraRibbonAggregationApplyCS::FPermutationDomain ApplyPermutationVector;
		ApplyPermutationVector.Set<FRibbonHasFullRibbonID>(GenerationConfig.HasFullRibbonIDs());
		ApplyPermutationVector.Set<FRibbonHasRibbonID>(GenerationConfig.HasSimpleRibbonIDs());
		TShaderMapRef<FNiagaraRibbonAggregationApplyCS> AggregationApplyShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ApplyPermutationVector);

		// repeatedly run the step shader to perform the aggregation
		FUnorderedAccessViewRHIRef CurrentAccumulationUAV;
		FShaderResourceViewRHIRef CurrentAccumulationSRV;

		// Setup buffers
		const int32 NumPrefixScanPasses = FMath::CeilLogTwo(NumExecutableInstances);

		uint32 CurrentBufferOrientation = 0x0;
		const auto SwapAccumulationBuffers = [&]()
		{
			CurrentBufferOrientation ^= 0x1;

			if (CurrentBufferOrientation)
			{
				CurrentAccumulationSRV = TempBuffers.TransientAccumulation[0].SRV;
				CurrentAccumulationUAV = TempBuffers.TransientAccumulation[1].UAV;
			}
			else
			{
				CurrentAccumulationSRV = TempBuffers.TransientAccumulation[1].SRV;
				CurrentAccumulationUAV = TempBuffers.TransientAccumulation[0].UAV;
			}
		};

		// init CurrentAccumulationBufferSRV/UAV
		SwapAccumulationBuffers();

		// aggregation involves calculating a per-particle value for a number of properties and then accumulating these values over all
		// segments in the ribbons:
		// Segment indices are initialized with a 1 for a valid segment, 0 otherwise.  Inclusive prefix sum accumulates the data
		//	-note that we mark the OutputSegments field as -1 if it's an invalid segment (like if there's only one particle)
		// Ribbon indices are initialized with a 1 for the start of a new ribbon, 0 otherwise.  Inclusive prefix sum accumulates the data

		// initialize the output buffers and queue up data to be aggregated
		{
			FNiagaraRibbonVertexReductionParameters InitParams;
			FMemory::Memzero(InitParams);
			InitParams.Common = CommonParams;
			InitParams.SortedIndices = VertexBuffers.SortedIndicesBuffer.SRV;
			InitParams.CurveTension = GenerationConfig.GetCurveTension();
			InitParams.OutputTangentsAndDistances = VertexBuffers.TangentsAndDistancesBuffer.UAV;
			InitParams.OutputMultiRibbonIndices = VertexBuffers.MultiRibbonIndicesBuffer.UAV;
			InitParams.OutputSegments = VertexBuffers.SegmentsBuffer.UAV;
			InitParams.OutputAccumulationBuffer = CurrentAccumulationUAV;

			RHICmdList.Transition(FRHITransitionInfo(InitParams.OutputTangentsAndDistances, ERHIAccess::SRVMask, ERHIAccess::UAVMask));
			RHICmdList.Transition(FRHITransitionInfo(InitParams.OutputMultiRibbonIndices, ERHIAccess::SRVMask, ERHIAccess::UAVMask));
			RHICmdList.Transition(FRHITransitionInfo(InitParams.OutputSegments, ERHIAccess::SRVMask, ERHIAccess::UAVMask));
			RHICmdList.Transition(FRHITransitionInfo(InitParams.OutputAccumulationBuffer, ERHIAccess::SRVMask, ERHIAccess::UAVMask));

			const uint32 NumThreadGroupsInitialization = FMath::DivideAndRoundUp<uint32>(NumExecutableInstances, FNiagaraRibbonComputeCommon::VertexGenReductionInitializationThreadSize);
			FComputeShaderUtils::Dispatch(RHICmdList, ReductionInitializationShader, InitParams, FIntVector(NumThreadGroupsInitialization, 1, 1));

			RHICmdList.Transition(FRHITransitionInfo(InitParams.OutputTangentsAndDistances, ERHIAccess::UAVMask, ERHIAccess::SRVMask));
			RHICmdList.Transition(FRHITransitionInfo(InitParams.OutputMultiRibbonIndices, ERHIAccess::UAVMask, ERHIAccess::SRVMask));
			RHICmdList.Transition(FRHITransitionInfo(InitParams.OutputSegments, ERHIAccess::UAVMask, ERHIAccess::SRVMask));
			RHICmdList.Transition(FRHITransitionInfo(InitParams.OutputAccumulationBuffer, ERHIAccess::UAVMask, ERHIAccess::SRVMask));
		}

		{
			FNiagaraRibbonAggregationStepParameters StepParams;
			FMemory::Memzero(StepParams);
			StepParams.Common = CommonParams;
			StepParams.SortedIndices = VertexBuffers.SortedIndicesBuffer.SRV;

			const uint32 NumThreadGroups = FMath::DivideAndRoundUp<uint32>(NumExecutableInstances, FNiagaraRibbonComputeCommon::VertexGenReductionPropagationThreadSize);

			for (StepParams.PrefixScanStride = 1; StepParams.PrefixScanStride < static_cast<uint32>(NumExecutableInstances); StepParams.PrefixScanStride *= 2)
			{
				SwapAccumulationBuffers();

				StepParams.InputAccumulation = CurrentAccumulationSRV;
				StepParams.OutputAccumulation = CurrentAccumulationUAV;

				RHICmdList.Transition(FRHITransitionInfo(StepParams.OutputAccumulation, ERHIAccess::SRVMask, ERHIAccess::UAVCompute));
				FComputeShaderUtils::Dispatch(RHICmdList, AggregationStepShader, StepParams, FIntVector(NumThreadGroups, 1, 1));
				RHICmdList.Transition(FRHITransitionInfo(StepParams.OutputAccumulation, ERHIAccess::UAVCompute, ERHIAccess::SRVMask));
			}
		}

		// apply the aggregate data onto the output buffers
		{
			SwapAccumulationBuffers();

			FNiagaraEmptyUAVPoolScopedAccess UAVPoolAccessScope(ComputeDispatchInterface->GetEmptyUAVPool());
			TArray<FRHITransitionInfo, TInlineAllocator<4>> Transitions;

			FNiagaraRibbonAggregationApplyParameters ApplyParams;
			FMemory::Memzero(ApplyParams);
			ApplyParams.Common = CommonParams;
			ApplyParams.SortedIndices = VertexBuffers.SortedIndicesBuffer.SRV;
			ApplyParams.InputAccumulation = CurrentAccumulationSRV;
			ApplyParams.OutputTangentsAndDistances = VertexBuffers.TangentsAndDistancesBuffer.UAV;
			ApplyParams.OutputMultiRibbonIndices = VertexBuffers.MultiRibbonIndicesBuffer.UAV;
			if (ApplyParams.OutputMultiRibbonIndices)
			{
				Transitions.Emplace(ApplyParams.OutputMultiRibbonIndices, ERHIAccess::SRVMask, ERHIAccess::UAVMask);
			}
			else
			{
				ApplyParams.OutputMultiRibbonIndices = ComputeDispatchInterface->GetEmptyUAVFromPool(RHICmdList, VertexBuffers.MultiRibbonIndicesBuffer.PixelFormat, ENiagaraEmptyUAVType::Buffer);
			}
			ApplyParams.OutputTessellationStats = TempBuffers.TransientTessellationStats.UAV;
			if (ApplyParams.OutputTessellationStats)
			{
				Transitions.Emplace(ApplyParams.OutputTessellationStats, ERHIAccess::SRVMask, ERHIAccess::UAVMask);
			}
			else
			{
				ApplyParams.OutputTessellationStats = ComputeDispatchInterface->GetEmptyUAVFromPool(RHICmdList, EPixelFormat::PF_R32_FLOAT, ENiagaraEmptyUAVType::Buffer);
			}
			ApplyParams.OutputSegments = VertexBuffers.SegmentsBuffer.UAV;

			Transitions.Emplace(ApplyParams.OutputTangentsAndDistances, ERHIAccess::SRVMask, ERHIAccess::UAVMask);
			Transitions.Emplace(ApplyParams.OutputSegments, ERHIAccess::SRVMask, ERHIAccess::UAVMask);

			RHICmdList.Transition(Transitions);

			const uint32 NumThreadGroups = FMath::DivideAndRoundUp<uint32>(NumExecutableInstances, FNiagaraRibbonComputeCommon::VertexGenReductionPropagationThreadSize);
			FComputeShaderUtils::Dispatch(RHICmdList, AggregationApplyShader, ApplyParams, FIntVector(NumThreadGroups, 1, 1));

			for (FRHITransitionInfo& Transition : Transitions)
			{
				Swap(Transition.AccessBefore, Transition.AccessAfter);
			}
			RHICmdList.Transition(Transitions);
		}

		static constexpr int32 CommandBufferOffset = 0;
		
		{
			SCOPED_DRAW_EVENT(RHICmdList, NiagaraRenderRibbonsGenVerticesReductionPhase2GPU);
		
			FNiagaraRibbonVertexReductionFinalizationParameters FinalizationParams;
			FMemory::Memzero(FinalizationParams);
			FinalizationParams.Common = CommonParams;
			FinalizationParams.SortedIndices = VertexBuffers.SortedIndicesBuffer.SRV;
			FinalizationParams.TangentsAndDistances = VertexBuffers.TangentsAndDistancesBuffer.SRV;
			FinalizationParams.MultiRibbonIndices = FNiagaraRenderer::GetSrvOrDefaultUInt(VertexBuffers.MultiRibbonIndicesBuffer.SRV);
			FinalizationParams.TessellationStats = FNiagaraRenderer::GetSrvOrDefaultFloat(TempBuffers.TransientTessellationStats.SRV);
			FinalizationParams.Segments = VertexBuffers.SegmentsBuffer.SRV;
			FinalizationParams.AccumulationBuffer = CurrentAccumulationSRV;
			FinalizationParams.PackedPerRibbonData = VertexBuffers.RibbonLookupTableBuffer.UAV;
			FinalizationParams.OutputCommandBuffer = VertexBuffers.GPUComputeCommandBuffer.UAV;
			FinalizationParams.OutputCommandBufferIndex= CommandBufferOffset;
			FinalizationParams.FinalizationThreadBlockSize = FNiagaraRibbonComputeCommon::VertexGenFinalizationThreadSize;
			
			FNiagaraRibbonVertexReductionFinalizeCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FRibbonHasFullRibbonID>(GenerationConfig.HasFullRibbonIDs());
			PermutationVector.Set<FRibbonHasRibbonID>(GenerationConfig.HasSimpleRibbonIDs());
			
			TShaderMapRef<FNiagaraRibbonVertexReductionFinalizeCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			// We only run a single threadgroup when we're not running multi-ribbon since we assume start/end is the first/last particle
			const uint32 NumThreadGroups = GenerationConfig.HasRibbonIDs() ?
				FMath::DivideAndRoundUp<uint32>(NumExecutableInstances, FNiagaraRibbonComputeCommon::VertexGenReductionFinalizationThreadSize) :
				1;
			RHICmdList.Transition({
				FRHITransitionInfo(VertexBuffers.RibbonLookupTableBuffer.Buffer, ERHIAccess::SRVMask, ERHIAccess::UAVCompute),
				FRHITransitionInfo(VertexBuffers.GPUComputeCommandBuffer.Buffer, ERHIAccess::SRVMask | ERHIAccess::IndirectArgs, ERHIAccess::UAVCompute)});

			FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, FinalizationParams, FIntVector(NumThreadGroups, 1, 1));
			
			// We don't need to transition RibbonLookupTableBuffer as it's still needed for the next shader
			RHICmdList.Transition({
				FRHITransitionInfo(VertexBuffers.RibbonLookupTableBuffer.Buffer, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute),
				FRHITransitionInfo(VertexBuffers.GPUComputeCommandBuffer.Buffer, ERHIAccess::UAVCompute, ERHIAccess::SRVMask | ERHIAccess::IndirectArgs)});		
		}
		
		{
			SCOPED_DRAW_EVENT(RHICmdList, NiagaraRenderRibbonsGenVerticesMultiRibbonInitGPU);
			SCOPE_CYCLE_COUNTER(STAT_NiagaraRenderRibbonsGenVerticesMultiRibbonInitGPU);
		
			FNiagaraRibbonVertexFinalizationParameters FinalizeParams;
			FMemory::Memzero(FinalizeParams);
			FinalizeParams.Common = CommonParams;
			FinalizeParams.SortedIndices = VertexBuffers.SortedIndicesBuffer.SRV;
			FinalizeParams.TangentsAndDistances = VertexBuffers.TangentsAndDistancesBuffer.UAV;
			FinalizeParams.PackedPerRibbonData = VertexBuffers.RibbonLookupTableBuffer.UAV;
			FinalizeParams.CommandBuffer = VertexBuffers.GPUComputeCommandBuffer.SRV;
			FinalizeParams.CommandBufferOffset = CommandBufferOffset;
		
			constexpr auto AddUVChannelParams = [](const FNiagaraRibbonUVSettings& Input, FNiagaraRibbonUVSettingsParams& Output)
			{
				Output.Offset = FVector2f(Input.Offset);
				Output.Scale = FVector2f(Input.Scale);
				Output.TilingLength = Input.TilingLength;
				Output.DistributionMode = static_cast<int32>(Input.DistributionMode);
				Output.LeadingEdgeMode = static_cast<int32>(Input.LeadingEdgeMode);
				Output.TrailingEdgeMode = static_cast<int32>(Input.TrailingEdgeMode);
				Output.bEnablePerParticleUOverride = Input.bEnablePerParticleUOverride ? 1 : 0;
				Output.bEnablePerParticleVRangeOverride = Input.bEnablePerParticleVRangeOverride ? 1 : 0;			
			};
	
			AddUVChannelParams(UV0Settings, FinalizeParams.UV0Settings);
			AddUVChannelParams(UV1Settings, FinalizeParams.UV1Settings);
	
			{
				SCOPE_CYCLE_COUNTER(STAT_NiagaraRenderRibbonsGenVerticesMultiRibbonInitComputeGPU);
			
				FNiagaraRibbonUVParamCalculationCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FRibbonHasFullRibbonID>(GenerationConfig.HasFullRibbonIDs());
				PermutationVector.Set<FRibbonHasRibbonID>(GenerationConfig.HasSimpleRibbonIDs());

				TShaderMapRef<FNiagaraRibbonUVParamCalculationCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

				// We don't need to transition RibbonLookupTableBuffer as it's still setup for UAV from the last shader
				RHICmdList.Transition({
					FRHITransitionInfo(VertexBuffers.RibbonLookupTableBuffer.Buffer, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute),
					FRHITransitionInfo(VertexBuffers.TangentsAndDistancesBuffer.Buffer, ERHIAccess::SRVMask, ERHIAccess::UAVCompute)});

				FComputeShaderUtils::DispatchIndirect(RHICmdList, ComputeShader, FinalizeParams, VertexBuffers.GPUComputeCommandBuffer.Buffer, CommandBufferOffset * VERTEX_GEN_OUTPUT_DATA_AUTO_TESSELLATION_NUM_ELEMENTS);

				RHICmdList.Transition({
					FRHITransitionInfo(VertexBuffers.TangentsAndDistancesBuffer.Buffer, ERHIAccess::UAVCompute, ERHIAccess::SRVMask),
					FRHITransitionInfo(VertexBuffers.RibbonLookupTableBuffer.Buffer, ERHIAccess::UAVCompute, ERHIAccess::SRVMask)});
			}		
		}
	}
}

void FNiagaraGpuRibbonsDataManager::GenerateAllGPUData(FRHICommandListImmediate& RHICmdList, TArray<FNiagaraRibbonGPUInitParameters>& RenderersToGenerate)
{
	if (RenderersToGenerate.Num() == 0)
	{
		return;
	}

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, NiagaraGPURibbons, "Niagara GPU Ribbons");

	FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = GetOwnerInterface();

	// Handle all vertex gens first
	for (const FNiagaraRibbonGPUInitParameters& RendererToGen : RenderersToGenerate)
	{
		const TSharedPtr<FNiagaraRibbonRenderingFrameResources> RenderingResources = RendererToGen.RenderingResources.Pin();
		if (RenderingResources.IsValid())
		{
			ComputeBuffers.InitOrUpdateBuffers(
				RHICmdList,
				RendererToGen.NumInstances,
				RendererToGen.Renderer->GenerationConfig.HasRibbonIDs(),
				RendererToGen.Renderer->GenerationConfig.WantsAutomaticTessellation(),
				RendererToGen.Renderer->GenerationConfig.HasTwist()
			);

			RendererToGen.Renderer->InitializeVertexBuffersGPU(RHICmdList, ComputeDispatchInterface, RendererToGen, ComputeBuffers, RenderingResources);
		}
	}

	// Now handle all index gens
	for (const FNiagaraRibbonGPUInitParameters& RendererToGen : RenderersToGenerate)
	{
		const TSharedPtr<FNiagaraRibbonRenderingFrameResources> RenderingResources = RendererToGen.RenderingResources.Pin();
		if (RenderingResources.IsValid())
		{
			for (const TSharedPtr<FNiagaraRibbonRenderingFrameViewResources>& RenderingResourcesView : RenderingResources->ViewResources)
			{
				if (RenderingResourcesView->bNeedsIndexBufferGeneration)
				{
					RendererToGen.Renderer->InitializeViewIndexBuffersGPU(RHICmdList, ComputeDispatchInterface, RendererToGen, RenderingResourcesView.Get());
				}
			}
		}
	}
	
	RenderersToGenerate.Empty();
}
