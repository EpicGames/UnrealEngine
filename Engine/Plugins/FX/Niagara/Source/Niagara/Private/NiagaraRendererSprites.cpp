// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRendererSprites.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "NiagaraComponent.h"
#include "NiagaraCullProxyComponent.h"
#include "NiagaraCutoutVertexBuffer.h"
#include "NiagaraDataSet.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGPUSortInfo.h"
#include "NiagaraRendererReadback.h"
#include "NiagaraSceneProxy.h"
#include "NiagaraSettings.h"
#include "NiagaraSortingGPU.h"
#include "NiagaraStats.h"
#include "NiagaraSystemInstance.h"
#include "ParticleResources.h"
#include "RayTracingInstance.h"
#include "RenderGraphBuilder.h"

DECLARE_DWORD_COUNTER_STAT(TEXT("NumSprites"), STAT_NiagaraNumSprites, STATGROUP_Niagara);

static bool GbEnableNiagaraSpriteRendering = true;
static FAutoConsoleVariableRef CVarEnableNiagaraSpriteRendering(
	TEXT("fx.EnableNiagaraSpriteRendering"),
	GbEnableNiagaraSpriteRendering,
	TEXT("If false Niagara Sprite Renderers are disabled."),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarRayTracingNiagaraSprites(
	TEXT("r.RayTracing.Geometry.NiagaraSprites"),
	1,
	TEXT("Include Niagara sprites in ray tracing effects (default = 1 (Niagara sprites enabled in ray tracing))"));


/** Dynamic data for sprite renderers. */
struct FNiagaraDynamicDataSprites : public FNiagaraDynamicDataBase
{
	FNiagaraDynamicDataSprites(const FNiagaraEmitterInstance* InEmitter)
		: FNiagaraDynamicDataBase(InEmitter)
	{
	}

	virtual void ApplyMaterialOverride(int32 MaterialIndex, UMaterialInterface* MaterialOverride) override
	{
		if (MaterialIndex == 0 && MaterialOverride)
		{
			Material = MaterialOverride->GetRenderProxy();
		}
	}

	FMaterialRenderProxy* Material = nullptr;
	TArray<UNiagaraDataInterface*> DataInterfacesBound;
	TArray<UObject*> ObjectsBound;
	TArray<uint8> ParameterDataBound;
};

//////////////////////////////////////////////////////////////////////////

FNiagaraRendererSprites::FNiagaraRendererSprites(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties* InProps, const FNiagaraEmitterInstance* Emitter)
	: FNiagaraRenderer(FeatureLevel, InProps, Emitter)
	, Alignment(ENiagaraSpriteAlignment::Unaligned)
	, FacingMode(ENiagaraSpriteFacingMode::FaceCamera)
	, SortMode(ENiagaraSortMode::ViewDistance)
	, PivotInUVSpace(0.5f, 0.5f)
	, MacroUVRadius(0.0f)
	, SubImageSize(1.0f, 1.0f)
	, NumIndicesPerInstance(0)
	, bSubImageBlend(false)
	, bRemoveHMDRollInVR(false)
	, bSortHighPrecision(false)
	, bSortOnlyWhenTranslucent(true)
	, bGpuLowLatencyTranslucency(true)
	, bEnableDistanceCulling(false)
	, MinFacingCameraBlendDistance(0.0f)
	, MaxFacingCameraBlendDistance(0.0f)
	, DistanceCullRange(0.0f, FLT_MAX)
	, MaterialParamValidMask(0)
	, RendererVisTagOffset(INDEX_NONE)
	, RendererVisibility(0)
{
	check(InProps && Emitter);

	const UNiagaraSpriteRendererProperties* Properties = CastChecked<const UNiagaraSpriteRendererProperties>(InProps);
	SourceMode = Properties->SourceMode;
	Alignment = Properties->Alignment;
	FacingMode = Properties->FacingMode;
	PivotInUVSpace = FVector2f(Properties->PivotInUVSpace);	// LWC_TODO: Precision loss
	MacroUVRadius = Properties->MacroUVRadius;
	SortMode = Properties->SortMode;
	SubImageSize = FVector2f(Properties->SubImageSize);	// LWC_TODO: Precision loss
	NumIndicesPerInstance = Properties->GetNumIndicesPerInstance();
	bSubImageBlend = Properties->bSubImageBlend;
	bRemoveHMDRollInVR = Properties->bRemoveHMDRollInVR;
	bSortHighPrecision = UNiagaraRendererProperties::IsSortHighPrecision(Properties->SortPrecision);
	bSortOnlyWhenTranslucent = Properties->bSortOnlyWhenTranslucent;
	bGpuLowLatencyTranslucency = UNiagaraRendererProperties::IsGpuTranslucentThisFrame(FeatureLevel, Properties->GpuTranslucentLatency);
	MinFacingCameraBlendDistance = Properties->MinFacingCameraBlendDistance;
	MaxFacingCameraBlendDistance = Properties->MaxFacingCameraBlendDistance;
	RendererVisibility = Properties->RendererVisibility;
	bAccurateMotionVectors = Properties->NeedsPreciseMotionVectors();
	bCastShadows = Properties->bCastShadows;
#if WITH_EDITORONLY_DATA
	bIncludeInHitProxy = Properties->bIncludeInHitProxy;
#endif

	PixelCoverageMode = Properties->PixelCoverageMode;
	if (PixelCoverageMode == ENiagaraRendererPixelCoverageMode::Automatic)
	{
		if ( GetDefault<UNiagaraSettings>()->DefaultPixelCoverageMode != ENiagaraDefaultRendererPixelCoverageMode::Enabled )
		{
			PixelCoverageMode = ENiagaraRendererPixelCoverageMode::Disabled;
		}
	}
	PixelCoverageBlend = FMath::Clamp(Properties->PixelCoverageBlend, 0.0f, 1.0f);

	bEnableDistanceCulling = Properties->bEnableCameraDistanceCulling;
	if (Properties->bEnableCameraDistanceCulling)
	{
		DistanceCullRange = FVector2f(Properties->MinCameraDistance, Properties->MaxCameraDistance);
	}

	// Get the offset of visibility tag in either particle data or parameter store
	RendererVisTagOffset = INDEX_NONE;
	bEnableCulling = bEnableDistanceCulling;
	if (Properties->RendererVisibilityTagBinding.CanBindToHostParameterMap())
	{
		RendererVisTagOffset = Emitter->GetRendererBoundVariables().IndexOf(Properties->RendererVisibilityTagBinding.GetParamMapBindableVariable());
		bVisTagInParamStore = true;
	}
	else
	{
		int32 FloatOffset, HalfOffset;
		const FNiagaraDataSet& Data = Emitter->GetParticleData();
		Data.GetVariableComponentOffsets(Properties->RendererVisibilityTagBinding.GetDataSetBindableVariable(), FloatOffset, RendererVisTagOffset, HalfOffset);
		bVisTagInParamStore = false;
		bEnableCulling |= RendererVisTagOffset != INDEX_NONE;
	}

	NumCutoutVertexPerSubImage = Properties->GetNumCutoutVertexPerSubimage();
	CutoutVertexBuffer.Data = Properties->GetCutoutData();

	MaterialParamValidMask = Properties->MaterialParamValidMask;

	RendererLayoutWithCustomSort = &Properties->RendererLayoutWithCustomSort;
	RendererLayoutWithoutCustomSort = &Properties->RendererLayoutWithoutCustomSort;

	bSetAnyBoundVars = false;
	if (Emitter->GetRendererBoundVariables().IsEmpty() == false)
	{
		const TArray< const FNiagaraVariableAttributeBinding*>& VFBindings = Properties->GetAttributeBindings();
		const int32 NumBindings = bAccurateMotionVectors ? ENiagaraSpriteVFLayout::Num_Max : ENiagaraSpriteVFLayout::Num_Default;
		check(VFBindings.Num() >= ENiagaraSpriteVFLayout::Type::Num_Max);

		for (int32 i = 0; i < ENiagaraSpriteVFLayout::Type::Num_Max; i++)
		{
			VFBoundOffsetsInParamStore[i] = INDEX_NONE;
			if (i < NumBindings && VFBindings[i] && VFBindings[i]->CanBindToHostParameterMap())
			{
				VFBoundOffsetsInParamStore[i] = Emitter->GetRendererBoundVariables().IndexOf(VFBindings[i]->GetParamMapBindableVariable());
				if (VFBoundOffsetsInParamStore[i] != INDEX_NONE)
					bSetAnyBoundVars = true;
			}
		}
	}
	else
	{
		for (int32 i = 0; i < ENiagaraSpriteVFLayout::Type::Num_Max; i++)
		{
			VFBoundOffsetsInParamStore[i] = INDEX_NONE;
		}
	}

	TConstArrayView<FNiagaraRendererVariableInfo> VFVariables = RendererLayoutWithoutCustomSort->GetVFVariables_GameThread();
	if (Alignment == ENiagaraSpriteAlignment::Automatic)
	{
		const int32 RegisterIndex = SourceMode == ENiagaraRendererSourceDataMode::Particles ? VFVariables[ENiagaraSpriteVFLayout::Alignment].GetGPUOffset() : VFBoundOffsetsInParamStore[ENiagaraSpriteVFLayout::Alignment];
		Alignment = RegisterIndex == INDEX_NONE ? ENiagaraSpriteAlignment::Unaligned : ENiagaraSpriteAlignment::CustomAlignment;
	}
	if (FacingMode == ENiagaraSpriteFacingMode::Automatic)
	{
		const int32 RegisterIndex = SourceMode == ENiagaraRendererSourceDataMode::Particles ? VFVariables[ENiagaraSpriteVFLayout::Facing].GetGPUOffset() : VFBoundOffsetsInParamStore[ENiagaraSpriteVFLayout::Facing];
		FacingMode = RegisterIndex == INDEX_NONE ? ENiagaraSpriteFacingMode::FaceCamera : ENiagaraSpriteFacingMode::CustomFacingVector;
	}
}

FNiagaraRendererSprites::~FNiagaraRendererSprites()
{
}

void FNiagaraRendererSprites::ReleaseRenderThreadResources()
{
	FNiagaraRenderer::ReleaseRenderThreadResources();

	CutoutVertexBuffer.ReleaseResource();
#if RHI_RAYTRACING
	if (IsRayTracingAllowed())
	{
		RayTracingGeometry.ReleaseResource();
		RayTracingDynamicVertexBuffer.Release();
	}
#endif
}

void FNiagaraRendererSprites::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	FNiagaraRenderer::CreateRenderThreadResources(RHICmdList);
	CutoutVertexBuffer.InitResource(RHICmdList);

#if RHI_RAYTRACING
	if (IsRayTracingAllowed())
	{
		FRayTracingGeometryInitializer Initializer;
		static const FName DebugName("FNiagaraRendererSprites");
		static int32 DebugNumber = 0;
		Initializer.DebugName = FDebugName(DebugName, DebugNumber++);
		Initializer.IndexBuffer = nullptr;
		Initializer.GeometryType = RTGT_Triangles;
		Initializer.bFastBuild = true;
		Initializer.bAllowUpdate = false;
		RayTracingGeometry.SetInitializer(Initializer);
		RayTracingGeometry.InitResource(RHICmdList);
	}
#endif
}

void FNiagaraRendererSprites::PrepareParticleSpriteRenderData(FRHICommandListBase& RHICmdList, FParticleSpriteRenderData& ParticleSpriteRenderData, const FSceneViewFamily& ViewFamily, FNiagaraDynamicDataBase* InDynamicData, const FNiagaraSceneProxy* SceneProxy, ENiagaraGpuComputeTickStage::Type GpuReadyTickStage) const
{
	ParticleSpriteRenderData.DynamicDataSprites = static_cast<FNiagaraDynamicDataSprites*>(InDynamicData);
	if (!ParticleSpriteRenderData.DynamicDataSprites || !SceneProxy->GetComputeDispatchInterface())
	{
		ParticleSpriteRenderData.SourceParticleData = nullptr;
		return;
	}

	// Early out if we have no data or instances, this must be done before we read the material
	FNiagaraDataBuffer* CurrentParticleData = ParticleSpriteRenderData.DynamicDataSprites->GetParticleDataToRender(RHICmdList, bGpuLowLatencyTranslucency);
	if (!CurrentParticleData || (SourceMode == ENiagaraRendererSourceDataMode::Particles && CurrentParticleData->GetNumInstances() == 0) || (GbEnableNiagaraSpriteRendering == false))
	{
		return;
	}

	FMaterialRenderProxy* MaterialRenderProxy = ParticleSpriteRenderData.DynamicDataSprites->Material;
	check(MaterialRenderProxy);

	// Do we have anything to render?
	const FMaterial& Material = MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
	ParticleSpriteRenderData.BlendMode = Material.GetBlendMode();
	ParticleSpriteRenderData.bHasTranslucentMaterials = IsTranslucentBlendMode(Material);
	ParticleSpriteRenderData.bCanCastTranslucentShadows = SceneProxy->CastsVolumetricTranslucentShadow() || Material.GetCastDynamicShadowAsMasked();

	// If these conditions change please update the DebugHUD display also to reflect it
	bool bLowLatencyTranslucencyEnabled =
		ParticleSpriteRenderData.bHasTranslucentMaterials &&
		bGpuLowLatencyTranslucency &&
		Material.GetMaterialDomain() == MD_Surface &&
		GpuReadyTickStage >= CurrentParticleData->GetGPUDataReadyStage() &&
		!ParticleSpriteRenderData.bCanCastTranslucentShadows &&
		ViewFamilySupportLowLatencyTranslucency(ViewFamily);

	if (bLowLatencyTranslucencyEnabled && SceneProxy->ShouldRenderCustomDepth())
	{
		bLowLatencyTranslucencyEnabled &= !Material.IsTranslucencyWritingCustomDepth();
	}

	ParticleSpriteRenderData.SourceParticleData = ParticleSpriteRenderData.DynamicDataSprites->GetParticleDataToRender(RHICmdList, bLowLatencyTranslucencyEnabled);
	if ( !ParticleSpriteRenderData.SourceParticleData || (SourceMode == ENiagaraRendererSourceDataMode::Particles && ParticleSpriteRenderData.SourceParticleData->GetNumInstances() == 0) )
	{
		ParticleSpriteRenderData.SourceParticleData = nullptr;
		return;
	}

	// If the visibility tag comes from a parameter map, so we can evaluate it here and just early out if it doesn't match up
	if (bVisTagInParamStore && ParticleSpriteRenderData.DynamicDataSprites->ParameterDataBound.IsValidIndex(RendererVisTagOffset))
	{
		int32 VisTag = 0;
		FMemory::Memcpy(&VisTag, ParticleSpriteRenderData.DynamicDataSprites->ParameterDataBound.GetData() + RendererVisTagOffset, sizeof(int32));
		if (RendererVisibility != VisTag)
		{
			ParticleSpriteRenderData.SourceParticleData = nullptr;
			return;
		}
	}

	// Particle source mode
	if (SourceMode == ENiagaraRendererSourceDataMode::Particles)
	{
		const EShaderPlatform ShaderPlatform = SceneProxy->GetComputeDispatchInterface()->GetShaderPlatform();

		// Determine if we need sorting
		ParticleSpriteRenderData.bNeedsSort = SortMode != ENiagaraSortMode::None && (IsAlphaCompositeBlendMode(Material) || IsAlphaHoldoutBlendMode(Material) || IsTranslucentOnlyBlendMode(Material) || !bSortOnlyWhenTranslucent);
		const bool bNeedCustomSort = ParticleSpriteRenderData.bNeedsSort && (SortMode == ENiagaraSortMode::CustomAscending || SortMode == ENiagaraSortMode::CustomDecending);
		ParticleSpriteRenderData.RendererLayout = bNeedCustomSort ? RendererLayoutWithCustomSort : RendererLayoutWithoutCustomSort;
		ParticleSpriteRenderData.SortVariable = bNeedCustomSort ? ENiagaraSpriteVFLayout::CustomSorting : ENiagaraSpriteVFLayout::Position;
		if (ParticleSpriteRenderData.bNeedsSort)
		{
			TConstArrayView<FNiagaraRendererVariableInfo> VFVariables = ParticleSpriteRenderData.RendererLayout->GetVFVariables_RenderThread();
			const FNiagaraRendererVariableInfo& SortVariable = VFVariables[ParticleSpriteRenderData.SortVariable];
			ParticleSpriteRenderData.bNeedsSort = SortVariable.GetGPUOffset() != INDEX_NONE;
		}

		// Do we need culling?
		ParticleSpriteRenderData.bNeedsCull = bEnableCulling;
		ParticleSpriteRenderData.bSortCullOnGpu = (ParticleSpriteRenderData.bNeedsSort && FNiagaraUtilities::AllowGPUSorting(ShaderPlatform)) || (ParticleSpriteRenderData.bNeedsCull && FNiagaraUtilities::AllowGPUCulling(ShaderPlatform));

		// Validate what we setup
		if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			if (!ensureMsgf(!ParticleSpriteRenderData.bNeedsCull || ParticleSpriteRenderData.bSortCullOnGpu, TEXT("Culling is requested on GPU but sorting is disabled, this will result in incorrect rendering. Asset(%s)."), *SceneProxy->GetResourceName().ToString()))
			{
				ParticleSpriteRenderData.bNeedsCull = false;
			}
			ParticleSpriteRenderData.bNeedsSort &= ParticleSpriteRenderData.bSortCullOnGpu;

			//-TODO: Culling and sorting from InitViewsAfterPrePass can not be respected if the culled entries have already been acquired
			if ((ParticleSpriteRenderData.bNeedsSort || ParticleSpriteRenderData.bNeedsCull) && !SceneProxy->GetComputeDispatchInterface()->GetGPUInstanceCounterManager().CanAcquireCulledEntry())
			{
				//ensureMsgf(false, TEXT("Culling & sorting is not supported once the culled counts have been acquired, sorting & culling will be disabled for these draws."));
				ParticleSpriteRenderData.bNeedsSort = false;
				ParticleSpriteRenderData.bNeedsCull = false;
			}
		}
		else
		{
			//-TODO: Culling and sorting from InitViewsAfterPrePass can not be respected if the culled entries have already been acquired
			if (ParticleSpriteRenderData.bSortCullOnGpu)
			{
				//ensureMsgf(false, TEXT("Culling & sorting is not supported once the culled counts have been acquired, sorting & culling will be disabled for these draws."));
				ParticleSpriteRenderData.bSortCullOnGpu &= SceneProxy->GetComputeDispatchInterface()->GetGPUInstanceCounterManager().CanAcquireCulledEntry();
			}

			// Should we GPU sort for CPU systems?
			if ( ParticleSpriteRenderData.bSortCullOnGpu )
			{
				const int32 NumInstances = ParticleSpriteRenderData.SourceParticleData->GetNumInstances();

				const int32 SortThreshold = GNiagaraGPUSortingCPUToGPUThreshold;
				const bool bSortMoveToGpu = (SortThreshold >= 0) && (NumInstances >= SortThreshold);

				const int32 CullThreshold = GNiagaraGPUCullingCPUToGPUThreshold;
				const bool bCullMoveToGpu = (CullThreshold >= 0) && (NumInstances >= CullThreshold);

				ParticleSpriteRenderData.bSortCullOnGpu = bSortMoveToGpu || bCullMoveToGpu;
			}
		}

		// Update layout as it could have changed
		ParticleSpriteRenderData.RendererLayout = bNeedCustomSort ? RendererLayoutWithCustomSort : RendererLayoutWithoutCustomSort;
	}
}

void FNiagaraRendererSprites::PrepareParticleRenderBuffers(FRHICommandListBase& RHICmdList, FParticleSpriteRenderData& ParticleSpriteRenderData, FGlobalDynamicReadBuffer& DynamicReadBuffer) const
{
	if ( SourceMode == ENiagaraRendererSourceDataMode::Particles )
	{
		if ( SimTarget == ENiagaraSimTarget::CPUSim )
		{
			// For CPU simulations we do not gather int parameters inside TransferDataToGPU currently so we need to copy off
			// integrate attributes if we are culling on the GPU.
			TArray<uint32, TInlineAllocator<1>> IntParamsToCopy;
			if (ParticleSpriteRenderData.bNeedsCull)
			{
				if (ParticleSpriteRenderData.bSortCullOnGpu)
				{
					if (RendererVisTagOffset != INDEX_NONE)
					{
						ParticleSpriteRenderData.RendererVisTagOffset = IntParamsToCopy.Add(RendererVisTagOffset);
					}
				}
				else
				{
					ParticleSpriteRenderData.RendererVisTagOffset = RendererVisTagOffset;
				}
			}

			FParticleRenderData ParticleRenderData = TransferDataToGPU(RHICmdList, DynamicReadBuffer, ParticleSpriteRenderData.RendererLayout, IntParamsToCopy, ParticleSpriteRenderData.SourceParticleData);
			const uint32 NumInstances = ParticleSpriteRenderData.SourceParticleData->GetNumInstances();

			ParticleSpriteRenderData.ParticleFloatSRV = GetSrvOrDefaultFloat(ParticleRenderData.FloatData);
			ParticleSpriteRenderData.ParticleHalfSRV = GetSrvOrDefaultHalf(ParticleRenderData.HalfData);
			ParticleSpriteRenderData.ParticleIntSRV = GetSrvOrDefaultInt(ParticleRenderData.IntData);
			ParticleSpriteRenderData.ParticleFloatDataStride = ParticleRenderData.FloatStride / sizeof(float);
			ParticleSpriteRenderData.ParticleHalfDataStride = ParticleRenderData.HalfStride / sizeof(FFloat16);
			ParticleSpriteRenderData.ParticleIntDataStride = ParticleRenderData.IntStride / sizeof(int32);
		}
		else
		{
			ParticleSpriteRenderData.ParticleFloatSRV = GetSrvOrDefaultFloat(ParticleSpriteRenderData.SourceParticleData->GetGPUBufferFloat());
			ParticleSpriteRenderData.ParticleHalfSRV = GetSrvOrDefaultHalf(ParticleSpriteRenderData.SourceParticleData->GetGPUBufferHalf());
			ParticleSpriteRenderData.ParticleIntSRV = GetSrvOrDefaultInt(ParticleSpriteRenderData.SourceParticleData->GetGPUBufferInt());
			ParticleSpriteRenderData.ParticleFloatDataStride = ParticleSpriteRenderData.SourceParticleData->GetFloatStride() / sizeof(float);
			ParticleSpriteRenderData.ParticleHalfDataStride = ParticleSpriteRenderData.SourceParticleData->GetHalfStride() / sizeof(FFloat16);
			ParticleSpriteRenderData.ParticleIntDataStride = ParticleSpriteRenderData.SourceParticleData->GetInt32Stride() / sizeof(int32);

			ParticleSpriteRenderData.RendererVisTagOffset = RendererVisTagOffset;
		}
	}
	else
	{
		ParticleSpriteRenderData.ParticleFloatSRV = FNiagaraRenderer::GetDummyFloatBuffer();
		ParticleSpriteRenderData.ParticleHalfSRV = FNiagaraRenderer::GetDummyHalfBuffer();
		ParticleSpriteRenderData.ParticleIntSRV = FNiagaraRenderer::GetDummyIntBuffer();
		ParticleSpriteRenderData.ParticleFloatDataStride = 0;
		ParticleSpriteRenderData.ParticleHalfDataStride = 0;
		ParticleSpriteRenderData.ParticleIntDataStride = 0;
	}
}

void FNiagaraRendererSprites::InitializeSortInfo(FParticleSpriteRenderData& ParticleSpriteRenderData, const FNiagaraSceneProxy& SceneProxy, const FSceneView& View, int32 ViewIndex, FNiagaraGPUSortInfo& OutSortInfo) const
{
	TConstArrayView<FNiagaraRendererVariableInfo> VFVariables = ParticleSpriteRenderData.RendererLayout->GetVFVariables_RenderThread();

#if STATS
	OutSortInfo.OwnerName = EmitterStatID.GetName();
#endif
	OutSortInfo.ParticleCount = ParticleSpriteRenderData.SourceParticleData->GetNumInstances();
	OutSortInfo.SortMode = SortMode;
	OutSortInfo.SetSortFlags(bSortHighPrecision, ParticleSpriteRenderData.SourceParticleData->GetGPUDataReadyStage());
	OutSortInfo.bEnableCulling = ParticleSpriteRenderData.bNeedsCull;
	OutSortInfo.RendererVisTagAttributeOffset = ParticleSpriteRenderData.RendererVisTagOffset;
	OutSortInfo.RendererVisibility = RendererVisibility;
	OutSortInfo.DistanceCullRange = DistanceCullRange;

	if ( bEnableDistanceCulling )
	{
		OutSortInfo.CullPositionAttributeOffset = ParticleSpriteRenderData.bSortCullOnGpu ? VFVariables[ENiagaraSpriteVFLayout::Position].GetGPUOffset() : VFVariables[ENiagaraSpriteVFLayout::Position].GetEncodedDatasetOffset();
	}

	auto GetViewMatrices =
		[](const FSceneView& View) -> const FViewMatrices&
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (const FViewMatrices* ViewMatrices = View.State ? View.State->GetFrozenViewMatrices() : nullptr)
			{
				// Don't retrieve the cached matrices for shadow views
				bool bIsShadow = View.GetDynamicMeshElementsShadowCullFrustum() != nullptr;
				if (!bIsShadow)
				{
					return *ViewMatrices;
				}
			}
#endif

			return View.ViewMatrices;
		};

	const FViewMatrices& ViewMatrices = GetViewMatrices(View);
	OutSortInfo.ViewOrigin = ViewMatrices.GetViewOrigin();
	OutSortInfo.ViewDirection = ViewMatrices.GetWorldToView().GetColumn(2);

	if (UseLocalSpace(&SceneProxy))
	{
		OutSortInfo.ViewOrigin = SceneProxy.GetLocalToWorldInverse().TransformPosition(OutSortInfo.ViewOrigin);
		OutSortInfo.ViewDirection = SceneProxy.GetLocalToWorld().GetTransposed().TransformVector(OutSortInfo.ViewDirection);
	}
	else
	{
		const FVector LWCTileOffset = FVector(SceneProxy.GetLWCRenderTile()) * FLargeWorldRenderScalar::GetTileSize();
		OutSortInfo.ViewOrigin -= LWCTileOffset;
	}

	if (ParticleSpriteRenderData.bSortCullOnGpu)
	{
		FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = SceneProxy.GetComputeDispatchInterface();

		OutSortInfo.ParticleDataFloatSRV = ParticleSpriteRenderData.ParticleFloatSRV;
		OutSortInfo.ParticleDataHalfSRV = ParticleSpriteRenderData.ParticleHalfSRV;
		OutSortInfo.ParticleDataIntSRV = ParticleSpriteRenderData.ParticleIntSRV;
		OutSortInfo.FloatDataStride = ParticleSpriteRenderData.ParticleFloatDataStride;
		OutSortInfo.HalfDataStride = ParticleSpriteRenderData.ParticleHalfDataStride;
		OutSortInfo.IntDataStride = ParticleSpriteRenderData.ParticleIntDataStride;
		OutSortInfo.GPUParticleCountSRV = GetSrvOrDefaultUInt(ComputeDispatchInterface->GetGPUInstanceCounterManager().GetInstanceCountBuffer());
		OutSortInfo.GPUParticleCountOffset = ParticleSpriteRenderData.SourceParticleData->GetGPUInstanceCountBufferOffset();
	}

	if (ParticleSpriteRenderData.SortVariable != INDEX_NONE)
	{
		const FNiagaraRendererVariableInfo& SortVariable = VFVariables[ParticleSpriteRenderData.SortVariable];
		OutSortInfo.SortAttributeOffset = ParticleSpriteRenderData.bSortCullOnGpu ? SortVariable.GetGPUOffset() : SortVariable.GetEncodedDatasetOffset();
	}
}

void FNiagaraRendererSprites::SetupVertexFactory(FRHICommandListBase& RHICmdList, FParticleSpriteRenderData& ParticleSpriteRenderData, FNiagaraSpriteVertexFactory& VertexFactory) const
{
	// Set facing / alignment
	{
		ENiagaraSpriteFacingMode ActualFacingMode = FacingMode;
		ENiagaraSpriteAlignment ActualAlignmentMode = Alignment;

		int32 FacingVarOffset = INDEX_NONE;
		int32 AlignmentVarOffset = INDEX_NONE;
		if (SourceMode == ENiagaraRendererSourceDataMode::Particles)
		{
			TConstArrayView<FNiagaraRendererVariableInfo> VFVariables = ParticleSpriteRenderData.RendererLayout->GetVFVariables_RenderThread();
			FacingVarOffset = VFVariables[ENiagaraSpriteVFLayout::Facing].GetGPUOffset();
			AlignmentVarOffset = VFVariables[ENiagaraSpriteVFLayout::Alignment].GetGPUOffset();
		}

		if ((FacingVarOffset == INDEX_NONE) && (VFBoundOffsetsInParamStore[ENiagaraSpriteVFLayout::Facing] == INDEX_NONE) && (ActualFacingMode == ENiagaraSpriteFacingMode::CustomFacingVector))
		{
			ActualFacingMode = ENiagaraSpriteFacingMode::FaceCamera;
		}

		if ((AlignmentVarOffset == INDEX_NONE) && (VFBoundOffsetsInParamStore[ENiagaraSpriteVFLayout::Alignment] == INDEX_NONE) && (ActualAlignmentMode == ENiagaraSpriteAlignment::CustomAlignment))
		{
			ActualAlignmentMode = ENiagaraSpriteAlignment::Unaligned;
		}

		VertexFactory.SetAlignmentMode((uint32)ActualAlignmentMode);
		VertexFactory.SetFacingMode((uint32)ActualFacingMode);
	}

	// Cutout geometry.
	const bool bUseSubImage = SubImageSize.X != 1 || SubImageSize.Y != 1;
	const bool bUseCutout = CutoutVertexBuffer.VertexBufferRHI.IsValid();
	if (bUseCutout)
	{
		VertexFactory.SetCutoutParameters(bUseSubImage, NumCutoutVertexPerSubImage);
		VertexFactory.SetCutoutGeometry(CutoutVertexBuffer.VertexBufferSRV);
		if (bUseSubImage == false)
		{
			// Replace the vertex buffer used for cutouts which do not use sub image animation, this is more optimal in the shader
			VertexFactory.SetVertexBufferOverride(&CutoutVertexBuffer);
		}
	}
	
	// The InitResource needs to happen at the end here as SetVertexBufferOverride will set the UV buffers.
	VertexFactory.InitResource(RHICmdList);
}

FNiagaraSpriteUniformBufferRef FNiagaraRendererSprites::CreateViewUniformBuffer(FParticleSpriteRenderData& ParticleSpriteRenderData, const FSceneView& View, const FSceneViewFamily& ViewFamily, const FNiagaraSceneProxy& SceneProxy, FNiagaraSpriteVertexFactory& VertexFactory) const
{
	FNiagaraSpriteUniformParameters PerViewUniformParameters;
	FMemory::Memzero(&PerViewUniformParameters, sizeof(PerViewUniformParameters)); // Clear unset bytes

	const bool bUseLocalSpace = UseLocalSpace(&SceneProxy);
	PerViewUniformParameters.bLocalSpace = bUseLocalSpace;
	PerViewUniformParameters.RotationBias = 0.0f;
	PerViewUniformParameters.RotationScale = 1.0f;
	PerViewUniformParameters.TangentSelector = FVector4f(0.0f, 0.0f, 0.0f, 1.0f);
	PerViewUniformParameters.DeltaSeconds = ViewFamily.Time.GetDeltaWorldTimeSeconds();
	PerViewUniformParameters.NormalsType = 0.0f;
	PerViewUniformParameters.NormalsSphereCenter = FVector4f(0.0f, 0.0f, 0.0f, 1.0f);
	PerViewUniformParameters.NormalsCylinderUnitDirection = FVector4f(0.0f, 0.0f, 1.0f, 0.0f);
	PerViewUniformParameters.MacroUVParameters = CalcMacroUVParameters(View, SceneProxy.GetActorPosition(), MacroUVRadius);
	PerViewUniformParameters.CameraFacingBlend = FVector4f(0.0f, 0.0f, 0.0f, 1.0f);
	PerViewUniformParameters.RemoveHMDRoll = bRemoveHMDRollInVR ? 0.0f : 1.0f;
	PerViewUniformParameters.SubImageSize = FVector4f(SubImageSize.X, SubImageSize.Y, 1.0f / SubImageSize.X, 1.0f / SubImageSize.Y);

	if (bUseLocalSpace)
	{
		PerViewUniformParameters.DefaultPos = FVector4f(0.0f, 0.0f, 0.0f, 1.0f);
	}
	else
	{
		PerViewUniformParameters.DefaultPos = FVector3f(SceneProxy.GetLocalToWorld().GetOrigin() - FVector(SceneProxy.GetLWCRenderTile()) * FLargeWorldRenderScalar::GetTileSize());  // LWC_TODO: precision loss
	}
	PerViewUniformParameters.DefaultPrevPos = PerViewUniformParameters.DefaultPos;
	PerViewUniformParameters.DefaultSize = FVector2f(50.f, 50.0f);
	PerViewUniformParameters.DefaultPrevSize = PerViewUniformParameters.DefaultSize;
	PerViewUniformParameters.DefaultUVScale = FVector2f(1.0f, 1.0f);
	PerViewUniformParameters.DefaultPivotOffset = PivotInUVSpace;
	PerViewUniformParameters.DefaultPrevPivotOffset = PerViewUniformParameters.DefaultPivotOffset;
	PerViewUniformParameters.DefaultVelocity = FVector3f(0.f, 0.0f, 0.0f);
	PerViewUniformParameters.DefaultPrevVelocity = PerViewUniformParameters.DefaultVelocity;
	PerViewUniformParameters.SystemLWCTile = SceneProxy.GetLWCRenderTile();
	PerViewUniformParameters.DefaultRotation = 0.0f;
	PerViewUniformParameters.DefaultPrevRotation = PerViewUniformParameters.DefaultRotation;
	PerViewUniformParameters.DefaultColor = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
	PerViewUniformParameters.DefaultMatRandom = 0.0f;
	PerViewUniformParameters.DefaultCamOffset = 0.0f;
	PerViewUniformParameters.DefaultPrevCamOffset = PerViewUniformParameters.DefaultCamOffset;
	PerViewUniformParameters.DefaultNormAge = 0.0f;
	PerViewUniformParameters.DefaultSubImage = 0.0f;
	PerViewUniformParameters.DefaultFacing = FVector4f(1.0f, 0.0f, 0.0f, 0.0f);
	PerViewUniformParameters.DefaultPrevFacing = PerViewUniformParameters.DefaultFacing;
	PerViewUniformParameters.DefaultAlignment = FVector4f(1.0f, 0.0f, 0.0f, 0.0f);
	PerViewUniformParameters.DefaultPrevAlignment = PerViewUniformParameters.DefaultAlignment;
	PerViewUniformParameters.DefaultDynamicMaterialParameter0 = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
	PerViewUniformParameters.DefaultDynamicMaterialParameter1 = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
	PerViewUniformParameters.DefaultDynamicMaterialParameter2 = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
	PerViewUniformParameters.DefaultDynamicMaterialParameter3 = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);

	PerViewUniformParameters.PrevPositionDataOffset = INDEX_NONE;
	PerViewUniformParameters.PrevVelocityDataOffset = INDEX_NONE;
	PerViewUniformParameters.PrevRotationDataOffset = INDEX_NONE;
	PerViewUniformParameters.PrevSizeDataOffset = INDEX_NONE;
	PerViewUniformParameters.PrevFacingDataOffset = INDEX_NONE;
	PerViewUniformParameters.PrevAlignmentDataOffset = INDEX_NONE;
	PerViewUniformParameters.PrevCameraOffsetDataOffset = INDEX_NONE;
	PerViewUniformParameters.PrevPivotOffsetDataOffset = INDEX_NONE;

	// Determine pixel coverage settings
	const bool PixelCoverageEnabled = View.IsPerspectiveProjection() && (PixelCoverageMode != ENiagaraRendererPixelCoverageMode::Disabled);
	PerViewUniformParameters.PixelCoverageEnabled = PixelCoverageEnabled;
	PerViewUniformParameters.PixelCoverageColorBlend = FVector4f::Zero();
	if (PixelCoverageEnabled)
	{
		if ( PixelCoverageMode == ENiagaraRendererPixelCoverageMode::Automatic )
		{
			PerViewUniformParameters.PixelCoverageEnabled = ParticleSpriteRenderData.bHasTranslucentMaterials;
			if (PerViewUniformParameters.PixelCoverageEnabled)
			{
				if (IsTranslucentOnlyBlendMode(ParticleSpriteRenderData.BlendMode))
				{
					PerViewUniformParameters.PixelCoverageColorBlend = FVector4f(0.0f, 0.0f, 0.0f, PixelCoverageBlend);
				}
				else if (IsAdditiveBlendMode(ParticleSpriteRenderData.BlendMode))
				{
					PerViewUniformParameters.PixelCoverageColorBlend = FVector4f(PixelCoverageBlend, PixelCoverageBlend, PixelCoverageBlend, PixelCoverageBlend);
				}
				else
				{
					//-TODO: Support these blend modes
					//BLEND_Modulate
					//BLEND_AlphaComposite
					//BLEND_AlphaHoldout
					PerViewUniformParameters.PixelCoverageEnabled = false;
				}
			}
		}
		else
		{
			PerViewUniformParameters.PixelCoverageEnabled = true;
			switch (PixelCoverageMode)
			{
				case ENiagaraRendererPixelCoverageMode::Enabled_RGBA:	PerViewUniformParameters.PixelCoverageColorBlend = FVector4f(PixelCoverageBlend, PixelCoverageBlend, PixelCoverageBlend, PixelCoverageBlend); break;
				case ENiagaraRendererPixelCoverageMode::Enabled_RGB:	PerViewUniformParameters.PixelCoverageColorBlend = FVector4f(PixelCoverageBlend, PixelCoverageBlend, PixelCoverageBlend, 0.0f); break;
				case ENiagaraRendererPixelCoverageMode::Enabled_A:		PerViewUniformParameters.PixelCoverageColorBlend = FVector4f(0.0f, 0.0f, 0.0f, PixelCoverageBlend); break;
				default: break;
			}
		}
	}

	PerViewUniformParameters.AccurateMotionVectors = false;
	if (SourceMode == ENiagaraRendererSourceDataMode::Particles)
	{
		TConstArrayView<FNiagaraRendererVariableInfo> VFVariables = ParticleSpriteRenderData.RendererLayout->GetVFVariables_RenderThread();
		PerViewUniformParameters.PositionDataOffset = VFVariables[ENiagaraSpriteVFLayout::Position].GetGPUOffset();
		PerViewUniformParameters.VelocityDataOffset = VFVariables[ENiagaraSpriteVFLayout::Velocity].GetGPUOffset();
		PerViewUniformParameters.RotationDataOffset = VFVariables[ENiagaraSpriteVFLayout::Rotation].GetGPUOffset();
		PerViewUniformParameters.SizeDataOffset = VFVariables[ENiagaraSpriteVFLayout::Size].GetGPUOffset();
		PerViewUniformParameters.ColorDataOffset = VFVariables[ENiagaraSpriteVFLayout::Color].GetGPUOffset();
		PerViewUniformParameters.MaterialParamDataOffset = VFVariables[ENiagaraSpriteVFLayout::MaterialParam0].GetGPUOffset();
		PerViewUniformParameters.MaterialParam1DataOffset = VFVariables[ENiagaraSpriteVFLayout::MaterialParam1].GetGPUOffset();
		PerViewUniformParameters.MaterialParam2DataOffset = VFVariables[ENiagaraSpriteVFLayout::MaterialParam2].GetGPUOffset();
		PerViewUniformParameters.MaterialParam3DataOffset = VFVariables[ENiagaraSpriteVFLayout::MaterialParam3].GetGPUOffset();
		PerViewUniformParameters.SubimageDataOffset = VFVariables[ENiagaraSpriteVFLayout::SubImage].GetGPUOffset();
		PerViewUniformParameters.FacingDataOffset = VFVariables[ENiagaraSpriteVFLayout::Facing].GetGPUOffset();
		PerViewUniformParameters.AlignmentDataOffset = VFVariables[ENiagaraSpriteVFLayout::Alignment].GetGPUOffset();
		PerViewUniformParameters.CameraOffsetDataOffset = VFVariables[ENiagaraSpriteVFLayout::CameraOffset].GetGPUOffset();
		PerViewUniformParameters.UVScaleDataOffset = VFVariables[ENiagaraSpriteVFLayout::UVScale].GetGPUOffset();
		PerViewUniformParameters.PivotOffsetDataOffset = VFVariables[ENiagaraSpriteVFLayout::PivotOffset].GetGPUOffset();
		PerViewUniformParameters.NormalizedAgeDataOffset = VFVariables[ENiagaraSpriteVFLayout::NormalizedAge].GetGPUOffset();
		PerViewUniformParameters.MaterialRandomDataOffset = VFVariables[ENiagaraSpriteVFLayout::MaterialRandom].GetGPUOffset();
		if (bAccurateMotionVectors)
		{
			PerViewUniformParameters.AccurateMotionVectors = true;
			PerViewUniformParameters.PrevPositionDataOffset = SelectVFComponent(VFVariables, ENiagaraSpriteVFLayout::PrevPosition, ENiagaraSpriteVFLayout::Position);
			PerViewUniformParameters.PrevVelocityDataOffset = SelectVFComponent(VFVariables, ENiagaraSpriteVFLayout::PrevVelocity, ENiagaraSpriteVFLayout::Velocity);
			PerViewUniformParameters.PrevRotationDataOffset = SelectVFComponent(VFVariables, ENiagaraSpriteVFLayout::PrevRotation, ENiagaraSpriteVFLayout::Rotation);
			PerViewUniformParameters.PrevSizeDataOffset = SelectVFComponent(VFVariables, ENiagaraSpriteVFLayout::PrevSize, ENiagaraSpriteVFLayout::Size);
			PerViewUniformParameters.PrevFacingDataOffset = SelectVFComponent(VFVariables, ENiagaraSpriteVFLayout::PrevFacing, ENiagaraSpriteVFLayout::Facing);
			PerViewUniformParameters.PrevAlignmentDataOffset = SelectVFComponent(VFVariables, ENiagaraSpriteVFLayout::PrevAlignment, ENiagaraSpriteVFLayout::Alignment);
			PerViewUniformParameters.PrevCameraOffsetDataOffset = SelectVFComponent(VFVariables, ENiagaraSpriteVFLayout::PrevCameraOffset, ENiagaraSpriteVFLayout::CameraOffset);
			PerViewUniformParameters.PrevPivotOffsetDataOffset = SelectVFComponent(VFVariables, ENiagaraSpriteVFLayout::PrevPivotOffset, ENiagaraSpriteVFLayout::PivotOffset);
		}
	}
	else if (SourceMode == ENiagaraRendererSourceDataMode::Emitter) // Clear all these out because we will be using the defaults to specify them
	{
		PerViewUniformParameters.PositionDataOffset = INDEX_NONE;
		PerViewUniformParameters.VelocityDataOffset = INDEX_NONE;
		PerViewUniformParameters.RotationDataOffset = INDEX_NONE;
		PerViewUniformParameters.SizeDataOffset = INDEX_NONE;
		PerViewUniformParameters.ColorDataOffset = INDEX_NONE;
		PerViewUniformParameters.MaterialParamDataOffset = INDEX_NONE;
		PerViewUniformParameters.MaterialParam1DataOffset = INDEX_NONE;
		PerViewUniformParameters.MaterialParam2DataOffset = INDEX_NONE;
		PerViewUniformParameters.MaterialParam3DataOffset = INDEX_NONE;
		PerViewUniformParameters.SubimageDataOffset = INDEX_NONE;
		PerViewUniformParameters.FacingDataOffset = INDEX_NONE;
		PerViewUniformParameters.AlignmentDataOffset = INDEX_NONE;
		PerViewUniformParameters.CameraOffsetDataOffset = INDEX_NONE;
		PerViewUniformParameters.UVScaleDataOffset = INDEX_NONE;
		PerViewUniformParameters.PivotOffsetDataOffset = INDEX_NONE;
		PerViewUniformParameters.NormalizedAgeDataOffset = INDEX_NONE;
		PerViewUniformParameters.MaterialRandomDataOffset = INDEX_NONE;
	}
	else
	{
		// Unsupported source data mode detected
		check(SourceMode <= ENiagaraRendererSourceDataMode::Emitter);
	}

	PerViewUniformParameters.MaterialParamValidMask = MaterialParamValidMask;

	if (bSetAnyBoundVars)
	{
		const FNiagaraDynamicDataSprites* DynamicDataSprites = ParticleSpriteRenderData.DynamicDataSprites;
		const int32 NumLayoutVars = bAccurateMotionVectors ? ENiagaraSpriteVFLayout::Num_Max : ENiagaraSpriteVFLayout::Num_Default;
		for (int32 i = 0; i < NumLayoutVars; i++)
		{
			if (VFBoundOffsetsInParamStore[i] != INDEX_NONE && DynamicDataSprites->ParameterDataBound.IsValidIndex(VFBoundOffsetsInParamStore[i]))
			{
				switch (i)
				{
				case ENiagaraSpriteVFLayout::Type::Position:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultPos, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraSpriteVFLayout::Type::Color:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultColor, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FLinearColor));
					break;
				case ENiagaraSpriteVFLayout::Type::Velocity:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultVelocity, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraSpriteVFLayout::Type::Rotation:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultRotation, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;
				case ENiagaraSpriteVFLayout::Type::Size:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultSize, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector2f));
					break;
				case ENiagaraSpriteVFLayout::Type::Facing:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultFacing, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraSpriteVFLayout::Type::Alignment:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultAlignment, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraSpriteVFLayout::Type::SubImage:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultSubImage, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;
				case ENiagaraSpriteVFLayout::Type::MaterialParam0:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultDynamicMaterialParameter0, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector4f));
					PerViewUniformParameters.MaterialParamValidMask |= 0x000f;
					break;
				case ENiagaraSpriteVFLayout::Type::MaterialParam1:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultDynamicMaterialParameter1, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector4f));
					PerViewUniformParameters.MaterialParamValidMask |= 0x00f0;
					break;
				case ENiagaraSpriteVFLayout::Type::MaterialParam2:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultDynamicMaterialParameter2, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector4f));
					PerViewUniformParameters.MaterialParamValidMask |= 0x0f00;
					break;
				case ENiagaraSpriteVFLayout::Type::MaterialParam3:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultDynamicMaterialParameter3, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector4f));
					PerViewUniformParameters.MaterialParamValidMask |= 0xf000;
					break;
				case ENiagaraSpriteVFLayout::Type::CameraOffset:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultCamOffset, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;
				case ENiagaraSpriteVFLayout::Type::UVScale:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultUVScale, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector2f));
					break;
				case ENiagaraSpriteVFLayout::Type::PivotOffset:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultPivotOffset, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector2f));
					break;
				case ENiagaraSpriteVFLayout::Type::MaterialRandom:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultMatRandom, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;
				case ENiagaraSpriteVFLayout::Type::CustomSorting:
					// unsupport for now...
					break;
				case ENiagaraSpriteVFLayout::Type::NormalizedAge:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultNormAge, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;
				case ENiagaraSpriteVFLayout::Type::PrevPosition:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultPrevPos, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraSpriteVFLayout::Type::PrevVelocity:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultPrevVelocity, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraSpriteVFLayout::Type::PrevRotation:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultPrevRotation, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;
				case ENiagaraSpriteVFLayout::Type::PrevSize:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultPrevSize, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector2f));
					break;
				case ENiagaraSpriteVFLayout::Type::PrevFacing:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultPrevFacing, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraSpriteVFLayout::Type::PrevAlignment:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultPrevAlignment, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector3f));
					break;
				case ENiagaraSpriteVFLayout::Type::PrevCameraOffset:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultPrevCamOffset, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(float));
					break;
				case ENiagaraSpriteVFLayout::Type::PrevPivotOffset:
					FMemory::Memcpy(&PerViewUniformParameters.DefaultPrevPivotOffset, DynamicDataSprites->ParameterDataBound.GetData() + VFBoundOffsetsInParamStore[i], sizeof(FVector2f));
					break;
				}
			}
			else
			{
				switch (i)
				{
				case ENiagaraSpriteVFLayout::Type::PrevPosition:
					PerViewUniformParameters.DefaultPrevPos = PerViewUniformParameters.DefaultPos;
					break;
				case ENiagaraSpriteVFLayout::Type::PrevVelocity:
					PerViewUniformParameters.DefaultPrevVelocity = PerViewUniformParameters.DefaultVelocity;
					break;
				case ENiagaraSpriteVFLayout::Type::PrevRotation:
					PerViewUniformParameters.DefaultPrevRotation = PerViewUniformParameters.DefaultRotation;
					break;
				case ENiagaraSpriteVFLayout::Type::PrevSize:
					PerViewUniformParameters.DefaultPrevSize = PerViewUniformParameters.DefaultSize;
					break;
				case ENiagaraSpriteVFLayout::Type::PrevFacing:
					PerViewUniformParameters.DefaultPrevFacing = PerViewUniformParameters.DefaultFacing;
					break;
				case ENiagaraSpriteVFLayout::Type::PrevAlignment:
					PerViewUniformParameters.DefaultPrevAlignment = PerViewUniformParameters.DefaultAlignment;
					break;
				case ENiagaraSpriteVFLayout::Type::PrevCameraOffset:
					PerViewUniformParameters.DefaultPrevCamOffset = PerViewUniformParameters.DefaultCamOffset;
					break;
				case ENiagaraSpriteVFLayout::Type::PrevPivotOffset:
					PerViewUniformParameters.DefaultPrevPivotOffset = PerViewUniformParameters.DefaultPivotOffset;
					break;
				default:
					break;
				}
			}
		}
	}

	PerViewUniformParameters.SubImageBlendMode = bSubImageBlend;

	if (VertexFactory.GetFacingMode() == uint32(ENiagaraSpriteFacingMode::FaceCameraDistanceBlend))
	{
		float DistanceBlendMinSq = MinFacingCameraBlendDistance * MinFacingCameraBlendDistance;
		float DistanceBlendMaxSq = MaxFacingCameraBlendDistance * MaxFacingCameraBlendDistance;
		float InvBlendRange = 1.0f / FMath::Max(DistanceBlendMaxSq - DistanceBlendMinSq, 1.0f);
		float BlendScaledMinDistance = DistanceBlendMinSq * InvBlendRange;

		PerViewUniformParameters.CameraFacingBlend.X = 1.0f;
		PerViewUniformParameters.CameraFacingBlend.Y = InvBlendRange;
		PerViewUniformParameters.CameraFacingBlend.Z = BlendScaledMinDistance;
	}

	if (VertexFactory.GetAlignmentMode() == uint32(ENiagaraSpriteAlignment::VelocityAligned))
	{
		// velocity aligned
		PerViewUniformParameters.RotationScale = 0.0f;
		PerViewUniformParameters.TangentSelector = FVector4f(0.0f, 1.0f, 0.0f, 0.0f);
	}

	return FNiagaraSpriteUniformBufferRef::CreateUniformBufferImmediate(PerViewUniformParameters, UniformBuffer_SingleFrame);
}

void FNiagaraRendererSprites::CreateMeshBatchForView(
	FRHICommandListBase& RHICmdList,
	FParticleSpriteRenderData& ParticleSpriteRenderData,
	FMeshBatch& MeshBatch,
	const FSceneView& View,
	const FNiagaraSceneProxy& SceneProxy,
	FNiagaraSpriteVertexFactory& VertexFactory,
	uint32 NumInstances,
	uint32 GPUCountBufferOffset,
	bool bDoGPUCulling
) const
{
	FNiagaraSpriteVFLooseParameters VFLooseParams;
	VFLooseParams.NiagaraParticleDataFloat = ParticleSpriteRenderData.ParticleFloatSRV;
	VFLooseParams.NiagaraParticleDataHalf = ParticleSpriteRenderData.ParticleHalfSRV;
	VFLooseParams.NiagaraFloatDataStride = FMath::Max(ParticleSpriteRenderData.ParticleFloatDataStride, ParticleSpriteRenderData.ParticleHalfDataStride);

	FMaterialRenderProxy* MaterialRenderProxy = ParticleSpriteRenderData.DynamicDataSprites->Material;
	check(MaterialRenderProxy);

	VFLooseParams.CutoutParameters = VertexFactory.GetCutoutParameters();
	VFLooseParams.CutoutGeometry = VertexFactory.GetCutoutGeometrySRV() ? VertexFactory.GetCutoutGeometrySRV() : GFNiagaraNullCutoutVertexBuffer.VertexBufferSRV.GetReference();
	VFLooseParams.ParticleAlignmentMode = VertexFactory.GetAlignmentMode();
	VFLooseParams.ParticleFacingMode = VertexFactory.GetFacingMode();
	VFLooseParams.SortedIndices = VertexFactory.GetSortedIndicesSRV() ? VertexFactory.GetSortedIndicesSRV() : GFNiagaraNullSortedIndicesVertexBuffer.VertexBufferSRV.GetReference();
	VFLooseParams.SortedIndicesOffset = VertexFactory.GetSortedIndicesOffset();

	FNiagaraGPUInstanceCountManager::FIndirectArgSlot IndirectDraw;
	if ((SourceMode == ENiagaraRendererSourceDataMode::Particles) && (GPUCountBufferOffset != INDEX_NONE))
	{
		FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = SceneProxy.GetComputeDispatchInterface();
		check(ComputeDispatchInterface);

		IndirectDraw = ComputeDispatchInterface->GetGPUInstanceCounterManager().AddDrawIndirect(
			RHICmdList,
			GPUCountBufferOffset,
			NumIndicesPerInstance,
			0,
			View.IsInstancedStereo(),
			bDoGPUCulling,
			ParticleSpriteRenderData.SourceParticleData->GetGPUDataReadyStage()
		);
	}

	if (IndirectDraw.IsValid())
	{
		VFLooseParams.IndirectArgsBuffer = IndirectDraw.SRV;
		VFLooseParams.IndirectArgsOffset = IndirectDraw.Offset / sizeof(uint32);
	}
	else
	{
		VFLooseParams.IndirectArgsBuffer = GFNiagaraNullSortedIndicesVertexBuffer.VertexBufferSRV;
		VFLooseParams.IndirectArgsOffset = 0;
	}

	VertexFactory.SetLooseParameterUniformBuffer(FNiagaraSpriteVFLooseParametersRef::CreateUniformBufferImmediate(VFLooseParams, UniformBuffer_SingleFrame));

	MeshBatch.VertexFactory = &VertexFactory;
	MeshBatch.CastShadow = SceneProxy.CastsDynamicShadow() && bCastShadows;
#if RHI_RAYTRACING
	MeshBatch.CastRayTracedShadow = SceneProxy.CastsDynamicShadow() && bCastShadows;
#endif
	MeshBatch.bUseAsOccluder = false;
	MeshBatch.ReverseCulling = SceneProxy.IsLocalToWorldDeterminantNegative();
	MeshBatch.Type = PT_TriangleList;
	MeshBatch.DepthPriorityGroup = SceneProxy.GetDepthPriorityGroup(&View);
	MeshBatch.bCanApplyViewModeOverrides = true;
	MeshBatch.bUseWireframeSelectionColoring = SceneProxy.IsSelected();
	MeshBatch.SegmentIndex = 0;

#if WITH_EDITORONLY_DATA
	if (bIncludeInHitProxy == false)
	{
		MeshBatch.BatchHitProxyId = FHitProxyId::InvisibleHitProxyId;
	}
#endif

	const bool bIsWireframe = View.Family->EngineShowFlags.Wireframe;
	if (bIsWireframe)
	{
		MeshBatch.MaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
	}
	else
	{
		MeshBatch.MaterialRenderProxy = MaterialRenderProxy;
	}

	FMeshBatchElement& MeshElement = MeshBatch.Elements[0];
	MeshElement.IndexBuffer = &GParticleIndexBuffer;
	MeshElement.FirstIndex = 0;
	MeshElement.NumPrimitives = NumIndicesPerInstance / 3;
	MeshElement.NumInstances = FMath::Max(0u, NumInstances);
	MeshElement.MinVertexIndex = 0;
	MeshElement.MaxVertexIndex = 0;
	MeshElement.PrimitiveUniformBuffer = SceneProxy.GetCustomUniformBuffer(RHICmdList, IsMotionBlurEnabled());
	if (IndirectDraw.IsValid())
	{
		MeshElement.IndirectArgsBuffer = IndirectDraw.Buffer;
		MeshElement.IndirectArgsOffset = IndirectDraw.Offset;
		MeshElement.NumPrimitives = 0;
	}

	if (NumCutoutVertexPerSubImage == 8)
	{
		MeshElement.IndexBuffer = &GSixTriangleParticleIndexBuffer;
	}

	INC_DWORD_STAT_BY(STAT_NiagaraNumSprites, NumInstances);
}

void FNiagaraRendererSprites::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector, const FNiagaraSceneProxy *SceneProxy) const
{
	check(SceneProxy);
	PARTICLE_PERF_STAT_CYCLES_RT(SceneProxy->GetProxyDynamicData().PerfStatsContext, GetDynamicMeshElements);

	FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();

	// Prepare our particle render data
	// This will also determine if we have anything to render
	// ENiagaraGpuComputeTickStage::Last is used as the GPU ready stage as we can support reading translucent data after PostRenderOpaque sims have run
	FParticleSpriteRenderData ParticleSpriteRenderData;
	PrepareParticleSpriteRenderData(Collector.GetRHICommandList(), ParticleSpriteRenderData, ViewFamily, DynamicDataRender, SceneProxy, ENiagaraGpuComputeTickStage::Last);

	if (ParticleSpriteRenderData.SourceParticleData == nullptr)
	{
		return;
	}

	if (ParticleSpriteRenderData.bHasTranslucentMaterials && AreViewsRenderingOpaqueOnly(Views, VisibilityMap, ParticleSpriteRenderData.bCanCastTranslucentShadows))
	{
		return;
	}

#if STATS
	FScopeCycleCounter EmitterStatsCounter(EmitterStatID);
#endif

	PrepareParticleRenderBuffers(RHICmdList, ParticleSpriteRenderData, Collector.GetDynamicReadBuffer());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];
			if (View->bIsInstancedStereoEnabled && IStereoRendering::IsStereoEyeView(*View) && !IStereoRendering::IsAPrimaryView(*View))
			{
				// We don't have to generate batches for non-primary views in stereo instance rendering
				continue;
			}

			// Scene captures that are rendered in with the regular views will run depth only so we can skip building the batches
			if (ParticleSpriteRenderData.bHasTranslucentMaterials && IsViewRenderingOpaqueOnly(View, ParticleSpriteRenderData.bCanCastTranslucentShadows))
			{
				continue;
			}

			if (SourceMode == ENiagaraRendererSourceDataMode::Emitter && bEnableDistanceCulling)
			{
				const FVector ViewOrigin = View->ViewMatrices.GetViewOrigin();
				FVector RefPosition = SceneProxy->GetLocalToWorld().GetOrigin();
				const int32 BoundPosOffset = VFBoundOffsetsInParamStore[ENiagaraSpriteVFLayout::Type::Position];
				if (BoundPosOffset != INDEX_NONE && ParticleSpriteRenderData.DynamicDataSprites->ParameterDataBound.IsValidIndex(BoundPosOffset))
				{
					// retrieve the reference position from the parameter store
					FVector3f RefPositionFloat(RefPosition);	// LWC_TODO: Precision loss
					FMemory::Memcpy(&RefPositionFloat, ParticleSpriteRenderData.DynamicDataSprites->ParameterDataBound.GetData() + BoundPosOffset, sizeof(FVector3f));
					if (UseLocalSpace(SceneProxy))
					{
						RefPosition = SceneProxy->GetLocalToWorld().TransformPosition(RefPosition);
					}
				}

				const float DistSquared = SceneProxy->GetProxyDynamicData().LODDistanceOverride >= 0.0f ? FMath::Square(SceneProxy->GetProxyDynamicData().LODDistanceOverride) : float(FVector::DistSquared(RefPosition, ViewOrigin));
				if (DistSquared < DistanceCullRange.X * DistanceCullRange.X || DistSquared > DistanceCullRange.Y * DistanceCullRange.Y)
				{
					// Distance cull the whole emitter
					continue;
				}
			}

			FNiagaraGPUSortInfo SortInfo;
			if (ParticleSpriteRenderData.bNeedsSort || ParticleSpriteRenderData.bNeedsCull)
			{
				InitializeSortInfo(ParticleSpriteRenderData, *SceneProxy, *View, ViewIndex, SortInfo);
			}

			FMeshCollectorResources* CollectorResources = &Collector.AllocateOneFrameResource<FMeshCollectorResources>();

			// Get the next vertex factory to use
			// TODO: Find a way to safely pool these such that they won't be concurrently accessed by multiple views
			FNiagaraSpriteVertexFactory& VertexFactory = CollectorResources->VertexFactory;

			// Sort/Cull particles if needed.
			uint32 NumInstances = SourceMode == ENiagaraRendererSourceDataMode::Particles ? ParticleSpriteRenderData.SourceParticleData->GetNumInstances() : 1;

			VertexFactory.SetSortedIndices(nullptr, 0xFFFFFFFF);
			FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = SceneProxy->GetComputeDispatchInterface();
			if (ParticleSpriteRenderData.bNeedsCull || ParticleSpriteRenderData.bNeedsSort)
			{
				if (ParticleSpriteRenderData.bSortCullOnGpu)
				{
					SortInfo.CulledGPUParticleCountOffset = ParticleSpriteRenderData.bNeedsCull ? ComputeDispatchInterface->GetGPUInstanceCounterManager().AcquireCulledEntry() : INDEX_NONE;
					if (ComputeDispatchInterface->AddSortedGPUSimulation(RHICmdList, SortInfo))
					{
						VertexFactory.SetSortedIndices(SortInfo.AllocationInfo.BufferSRV, SortInfo.AllocationInfo.BufferOffset);
					}
				}
				else
				{
					FGlobalDynamicReadBuffer::FAllocation SortedIndices;
					SortedIndices = Collector.GetDynamicReadBuffer().AllocateUInt32(NumInstances);
					NumInstances = SortAndCullIndices(SortInfo, *ParticleSpriteRenderData.SourceParticleData, SortedIndices);
					VertexFactory.SetSortedIndices(SortedIndices.SRV, 0);
				}
			}

			if (NumInstances > 0)
			{
				SetupVertexFactory(RHICmdList, ParticleSpriteRenderData, VertexFactory);
				CollectorResources->UniformBuffer = CreateViewUniformBuffer(ParticleSpriteRenderData, *View, ViewFamily, *SceneProxy, VertexFactory);
				VertexFactory.SetSpriteUniformBuffer(CollectorResources->UniformBuffer);

				const uint32 GPUCountBufferOffset = SortInfo.CulledGPUParticleCountOffset != INDEX_NONE ? SortInfo.CulledGPUParticleCountOffset : ParticleSpriteRenderData.SourceParticleData->GetGPUInstanceCountBufferOffset();
				FMeshBatch& MeshBatch = Collector.AllocateMesh();
				CreateMeshBatchForView(RHICmdList, ParticleSpriteRenderData, MeshBatch, *View, *SceneProxy, VertexFactory, NumInstances, GPUCountBufferOffset, ParticleSpriteRenderData.bNeedsCull);
				Collector.AddMesh(ViewIndex, MeshBatch);

			#if WITH_NIAGARA_RENDERER_READBACK
				if (NiagaraRendererReadback::IsCapturing())
				{
					NiagaraRendererReadback::CaptureMeshBatch(View, SceneProxy, MeshBatch, NumInstances, NumIndicesPerInstance);
				}
			#endif //WITH_NIAGARA_RENDERER_READBACK
			}
		}
	}
}

#if RHI_RAYTRACING
void FNiagaraRendererSprites::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FNiagaraSceneProxy* SceneProxy)
{
	if (!CVarRayTracingNiagaraSprites.GetValueOnRenderThread())
	{
		return;
	}

	check(SceneProxy);

	FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();

	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	// RT geometry will be generated based on first active view and then reused for all other views
	// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
	const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
	checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

	const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

	// Prepare our particle render data
	// This will also determine if we have anything to render
	// ENiagaraGpuComputeTickStage::PostInitViews is used as we need the data one InitViews is complete as the HWRT BVH will be generated before other sims have run
	FParticleSpriteRenderData ParticleSpriteRenderData;
	PrepareParticleSpriteRenderData(RHICmdList, ParticleSpriteRenderData, *FirstActiveView->Family, DynamicDataRender, SceneProxy, ENiagaraGpuComputeTickStage::PostInitViews);

	if (ParticleSpriteRenderData.SourceParticleData == nullptr)
	{
		return;
	}

#if STATS
	FScopeCycleCounter EmitterStatsCounter(EmitterStatID);
#endif

	FGlobalDynamicReadBuffer& DynamicReadBuffer = Collector.GetDynamicReadBuffer();
	PrepareParticleRenderBuffers(RHICmdList, ParticleSpriteRenderData, DynamicReadBuffer);
	
	FNiagaraGPUSortInfo SortInfo;
	if (ParticleSpriteRenderData.bNeedsSort || ParticleSpriteRenderData.bNeedsCull)
	{
		InitializeSortInfo(ParticleSpriteRenderData, *SceneProxy, *FirstActiveView, 0, SortInfo);
	}

	if (!FNiagaraSpriteVertexFactory::StaticType.SupportsRayTracingDynamicGeometry())
	{
		return;
	}

	FMeshCollectorResources* CollectorResources = &Collector.AllocateOneFrameResource<FMeshCollectorResources>();
	FNiagaraSpriteVertexFactory& VertexFactory = CollectorResources->VertexFactory;

	// Sort/Cull particles if needed.
	uint32 NumInstances = SourceMode == ENiagaraRendererSourceDataMode::Particles ? ParticleSpriteRenderData.SourceParticleData->GetNumInstances() : 1;

	VertexFactory.SetSortedIndices(nullptr, 0xFFFFFFFF);
	FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = SceneProxy->GetComputeDispatchInterface();
	if (ParticleSpriteRenderData.bNeedsCull || ParticleSpriteRenderData.bNeedsSort)
	{
		if (ParticleSpriteRenderData.bSortCullOnGpu)
		{
			SortInfo.CulledGPUParticleCountOffset = ParticleSpriteRenderData.bNeedsCull ? ComputeDispatchInterface->GetGPUInstanceCounterManager().AcquireCulledEntry() : INDEX_NONE;
			if (ComputeDispatchInterface->AddSortedGPUSimulation(RHICmdList, SortInfo))
			{
				VertexFactory.SetSortedIndices(SortInfo.AllocationInfo.BufferSRV, SortInfo.AllocationInfo.BufferOffset);
			}
		}
		else
		{
			FGlobalDynamicReadBuffer::FAllocation SortedIndices;
			SortedIndices = DynamicReadBuffer.AllocateUInt32(NumInstances);
			NumInstances = SortAndCullIndices(SortInfo, *ParticleSpriteRenderData.SourceParticleData, SortedIndices);
			VertexFactory.SetSortedIndices(SortedIndices.SRV, 0);
		}
	}

	if (NumInstances > 0)
	{
		SetupVertexFactory(RHICmdList, ParticleSpriteRenderData, VertexFactory);
		CollectorResources->UniformBuffer = CreateViewUniformBuffer(ParticleSpriteRenderData, *FirstActiveView, *FirstActiveView->Family, *SceneProxy, VertexFactory);
		VertexFactory.SetSpriteUniformBuffer(CollectorResources->UniformBuffer);

		const uint32 GPUCountBufferOffset = SortInfo.CulledGPUParticleCountOffset != INDEX_NONE ? SortInfo.CulledGPUParticleCountOffset : ParticleSpriteRenderData.SourceParticleData->GetGPUInstanceCountBufferOffset();

		FMeshBatch MeshBatch;
		CreateMeshBatchForView(RHICmdList, ParticleSpriteRenderData, MeshBatch, *FirstActiveView, *SceneProxy, VertexFactory, NumInstances, GPUCountBufferOffset, ParticleSpriteRenderData.bNeedsCull);

		TArray<FMeshBatch>& CollectorMeshBatches = Collector.AllocateMeshBatchArray();
		CollectorMeshBatches.Add(MeshBatch);

		FRayTracingInstance RayTracingInstance;
		RayTracingInstance.Geometry = &RayTracingGeometry;
		RayTracingInstance.InstanceTransforms.Add(FMatrix::Identity);
		RayTracingInstance.MaterialsView = CollectorMeshBatches;

		// Use the internal vertex buffer only when initialized otherwise used the shared vertex buffer - needs to be updated every frame
		FRWBuffer* VertexBuffer = RayTracingDynamicVertexBuffer.NumBytes > 0 ? &RayTracingDynamicVertexBuffer : nullptr;

		// Different numbers of cutout vertices correspond to different index buffers
		// For 8 verts, use GSixTriangleParticleIndexBuffer
		// For 4 verts cutout geometry and normal particle geometry, use the typical 6 indices
		const int32 NumVerticesPerInstance = NumCutoutVertexPerSubImage == 8 ? 18 : 6;
		const int32 NumTrianglesPerInstance = NumCutoutVertexPerSubImage == 8 ? 6 : 2;

		{
			FRayTracingDynamicGeometryUpdateParams UpdateParams;
			UpdateParams.MeshBatchesView = CollectorMeshBatches;
			UpdateParams.bUsingIndirectDraw = MeshBatch.Elements[0].NumPrimitives == 0;
			UpdateParams.NumVertices = NumVerticesPerInstance * NumInstances;
			UpdateParams.VertexBufferSize = NumVerticesPerInstance * NumInstances * (uint32)sizeof(FVector3f);
			UpdateParams.NumTriangles = NumTrianglesPerInstance * NumInstances;
			UpdateParams.Geometry = &RayTracingGeometry;
			UpdateParams.Buffer = VertexBuffer;
			UpdateParams.bApplyWorldPositionOffset = true;

			Collector.AddRayTracingGeometryUpdate(FirstActiveViewIndex, MoveTemp(UpdateParams));
		}

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if ((VisibilityMap & (1 << ViewIndex)) == 0)
			{
				continue;
			}

			Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
		}
	}
}
#endif

/** Update render data buffer from attributes */
FNiagaraDynamicDataBase *FNiagaraRendererSprites::GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const
{
	FNiagaraDynamicDataSprites *DynamicData = nullptr;
	const UNiagaraSpriteRendererProperties* Properties = CastChecked<const UNiagaraSpriteRendererProperties>(InProperties);

	if (Properties)
	{
		if ( !IsRendererEnabled(Properties, Emitter) )
		{
			return nullptr;
		}

		if (Properties->bAllowInCullProxies == false)
		{
			check(Emitter);

			FNiagaraSystemInstance* Inst = Emitter->GetParentSystemInstance();
			check(Emitter->GetParentSystemInstance());

			//TODO: Probably should push some state into the system instance for this?
			const bool bIsCullProxy = Cast<UNiagaraCullProxyComponent>(Inst->GetAttachComponent()) != nullptr;
			if (bIsCullProxy)
			{
				return nullptr;
			}
		}

		FNiagaraDataBuffer* DataToRender = Emitter->GetParticleData().GetCurrentData();
		if(SimTarget == ENiagaraSimTarget::GPUComputeSim || (DataToRender != nullptr &&  (SourceMode == ENiagaraRendererSourceDataMode::Emitter || (SourceMode == ENiagaraRendererSourceDataMode::Particles && DataToRender->GetNumInstances() > 0))))
		{
			DynamicData = new FNiagaraDynamicDataSprites(Emitter);

			//In preparation for a material override feature, we pass our material(s) and relevance in via dynamic data.
			//The renderer ensures we have the correct usage and relevance for materials in BaseMaterials_GT.
			//Any override feature must also do the same for materials that are set.
			check(BaseMaterials_GT.Num() == 1);
			check(BaseMaterials_GT[0]->CheckMaterialUsage_Concurrent(MATUSAGE_NiagaraSprites));
			DynamicData->Material = BaseMaterials_GT[0]->GetRenderProxy();
			DynamicData->SetMaterialRelevance(BaseMaterialRelevance_GT);
		}

		if (DynamicData)
		{
			const FNiagaraParameterStore& ParameterData = Emitter->GetRendererBoundVariables();
			DynamicData->DataInterfacesBound = ParameterData.GetDataInterfaces();
			DynamicData->ObjectsBound = ParameterData.GetUObjects();
			DynamicData->ParameterDataBound = ParameterData.GetParameterDataArray();
		}

		if (DynamicData && Properties->MaterialParameters.HasAnyBindings())
		{
			ProcessMaterialParameterBindings(Properties->MaterialParameters, Emitter, MakeArrayView(BaseMaterials_GT));
		}
	}

	return DynamicData;  // for VF that can fetch from particle data directly
}

int FNiagaraRendererSprites::GetDynamicDataSize()const
{
	uint32 Size = sizeof(FNiagaraDynamicDataSprites);
	return Size;
}

bool FNiagaraRendererSprites::IsMaterialValid(const UMaterialInterface* Mat)const
{
	return Mat && Mat->CheckMaterialUsage_Concurrent(MATUSAGE_NiagaraSprites);
}
