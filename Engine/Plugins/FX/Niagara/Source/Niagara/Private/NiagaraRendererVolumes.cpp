// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRendererVolumes.h"

#include "NiagaraCullProxyComponent.h"
#include "NiagaraVolumeRendererProperties.h"
#include "NiagaraEmitter.h"
#include "NiagaraSceneProxy.h"
#include "NiagaraSystemInstance.h"

#include "Async/Async.h"
#include "Components/LineBatchComponent.h"
#include "Engine/World.h"
#include "SceneInterface.h"

#include "SceneView.h"

namespace NiagaraRendererVolumesLocal
{
	static bool GRendererEnabled = true;
	static FAutoConsoleVariableRef CVarRendererEnabled(
		TEXT("fx.Niagara.VolumeRenderer.Enabled"),
		GRendererEnabled,
		TEXT("If == 0, Niagara Volume Renderers are disabled."),
		ECVF_Default
	);

	struct FVolumeDynamicData : public FNiagaraDynamicDataBase
	{
		FVolumeDynamicData(const FNiagaraEmitterInstance* InEmitter)
			: FNiagaraDynamicDataBase(InEmitter)
		{
		}

		virtual void ApplyMaterialOverride(int32 MaterialIndex, UMaterialInterface* MaterialOverride) override
		{
			if (MaterialIndex == 0 && MaterialOverride)
			{
				Material = MaterialOverride->GetRenderProxy();
			}
		}

		FMaterialRenderProxy*	Material = nullptr;
		FNiagaraPosition		DefaultPosition = FVector3f::ZeroVector;
		FQuat4f					DefaultRotation = UNiagaraVolumeRendererProperties::GetDefaultVolumeRotation();
		FVector3f				DefaultScale = UNiagaraVolumeRendererProperties::GetDefaultVolumeScale();
		int32					DefaultRendererVisibilityTag = 0;
		int32					DefaultVolumeResolutionMaxAxis = UNiagaraVolumeRendererProperties::GetDefaultVolumeResolutionMaxAxis();
		FVector3f				DefaultVolumeWorldSpaceSize = UNiagaraVolumeRendererProperties::GetDefaultVolumeWorldSpaceSize();

		int32					VolumeResolutionMaxAxis = UNiagaraVolumeRendererProperties::GetDefaultVolumeResolutionMaxAxis();
		FVector3f				VolumeWorldSpaceSize = UNiagaraVolumeRendererProperties::GetDefaultVolumeWorldSpaceSize();
		float					StepFactor = UNiagaraVolumeRendererProperties::GetDefaultStepFactor();
		float					ShadowStepFactor = UNiagaraVolumeRendererProperties::GetDefaultShadowStepFactor();
		float					ShadowBiasFactor = UNiagaraVolumeRendererProperties::GetDefaultShadowBiasFactor();
		float					LightingDownsampleFactor = UNiagaraVolumeRendererProperties::GetDefaultLightingDownsampleFactor();
	};

	struct FHeterogeneousVolumeInstances
	{
		struct FInstance
		{
			FBoxSphereBounds	LocalBounds;
			FMatrix				LocalToWorld;
		};

		TArray<FInstance>	Instances;
	};
}

FNiagaraRendererVolumes::FNiagaraRendererVolumes(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter)
	: FNiagaraRenderer(FeatureLevel, InProperties, Emitter)
	, VertexFactory(FeatureLevel, "FNiagaraRendererVolumes")
{
	const UNiagaraVolumeRendererProperties* Properties = CastChecked<const UNiagaraVolumeRendererProperties>(InProperties);
	check(Properties);

	SourceMode						= Properties->SourceMode;
	RendererVisibilityTag			= Properties->RendererVisibility;
	PositionDataSetAccessor			= Properties->PositionDataSetAccessor;
	RotationDataSetAccessor			= Properties->RotationDataSetAccessor;
	ScaleDataSetAccessor			= Properties->ScaleDataSetAccessor;
	RendererVisibilityTagAccessor	= Properties->RendererVisibilityTagAccessor;
	VolumeResolutionMaxAxisAccessor = Properties->VolumeResolutionMaxAxisAccessor;
	VolumeWorldSpaceSizeAccessor	= Properties->VolumeWorldSpaceSizeAccessor;

	// Generate Attribute -> Renderer Parameter Store Bindings
	const TArray<const FNiagaraVariableAttributeBinding*>& VFAttributeBindings = Properties->GetAttributeBindings();

	check(UE_ARRAY_COUNT(VFBoundOffsetsInParamStore) == VFAttributeBindings.Num());
	for ( int32 i=0; i < UE_ARRAY_COUNT(VFBoundOffsetsInParamStore); ++i )
	{
		VFBoundOffsetsInParamStore[i] = INDEX_NONE;
	
		if (VFAttributeBindings[i] && VFAttributeBindings[i]->CanBindToHostParameterMap())
		{
			VFBoundOffsetsInParamStore[i] = Emitter->GetRendererBoundVariables().IndexOf(VFAttributeBindings[i]->GetParamMapBindableVariable());
			bAnyVFBoundOffsets |= VFBoundOffsetsInParamStore[i] != INDEX_NONE;
		}
	}
}

FNiagaraRendererVolumes::~FNiagaraRendererVolumes()
{
}

void FNiagaraRendererVolumes::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	FNiagaraRenderer::CreateRenderThreadResources(RHICmdList);

	FLocalVertexFactory::FDataType VFData;
	VFData.PositionComponent = FVertexStreamComponent(&GNullVertexBuffer, 0, 0, VET_Float3);
	VFData.PositionComponentSRV = GNullVertexBuffer.VertexBufferSRV;

	VFData.TangentBasisComponents[0] = FVertexStreamComponent(&GNullVertexBuffer, 0, 0, VET_PackedNormal, EVertexStreamUsage::ManualFetch);
	VFData.TangentBasisComponents[1] = FVertexStreamComponent(&GNullVertexBuffer, 0, 0, VET_PackedNormal, EVertexStreamUsage::ManualFetch);
	VFData.TangentsSRV = GNullVertexBuffer.VertexBufferSRV;

	VFData.NumTexCoords = 0;
	VFData.TextureCoordinatesSRV = GNullVertexBuffer.VertexBufferSRV;

	VFData.ColorIndexMask = 0;
	VFData.ColorComponent = FVertexStreamComponent(&GNullColorVertexBuffer, 0, 0, VET_Color, EVertexStreamUsage::ManualFetch);
	VFData.ColorComponentsSRV = GNullColorVertexBuffer.VertexBufferSRV;

	VertexFactory.SetData(RHICmdList, VFData);
	VertexFactory.InitResource(RHICmdList);
}

void FNiagaraRendererVolumes::ReleaseRenderThreadResources()
{
	FNiagaraRenderer::ReleaseRenderThreadResources();

	VertexFactory.ReleaseResource();
}

bool FNiagaraRendererVolumes::IsMaterialValid(const UMaterialInterface* Material) const
{	
	return Material ? Material->CheckMaterialUsage_Concurrent(MATUSAGE_HeterogeneousVolumes) : false;
}

FNiagaraDynamicDataBase* FNiagaraRendererVolumes::GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const
{
	using namespace NiagaraRendererVolumesLocal;

	const UNiagaraVolumeRendererProperties* Properties = CastChecked<const UNiagaraVolumeRendererProperties>(InProperties);
	if (!Properties || !IsRendererEnabled(Properties, Emitter))
	{
		return nullptr;
	}

	if (SourceMode != ENiagaraRendererSourceDataMode::Emitter)
	{
		if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			return nullptr;
		}

		FNiagaraDataBuffer* DataToRender = Emitter->GetParticleData().GetCurrentData();
		if (!DataToRender || DataToRender->GetNumInstances() == 0)
		{
			return nullptr;
		}
	}

	check(BaseMaterials_GT.Num() == 1);
	check(BaseMaterials_GT[0]->CheckMaterialUsage_Concurrent(MATUSAGE_HeterogeneousVolumes));

	// Setup Dynamic Data
	FVolumeDynamicData* VolumeDynamicData = new FVolumeDynamicData(Emitter);
	VolumeDynamicData->Material = BaseMaterials_GT[0]->GetRenderProxy();
	VolumeDynamicData->SetMaterialRelevance(BaseMaterialRelevance_GT);
	VolumeDynamicData->DefaultRendererVisibilityTag = Properties->RendererVisibility;

	// Pull Default Values from Renderer Bounds Parameters
	if (bAnyVFBoundOffsets)
	{
		const FNiagaraParameterStore& ParameterData = Emitter->GetRendererBoundVariables();

		if (VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::Position] != INDEX_NONE)
		{
			VolumeDynamicData->DefaultPosition = ParameterData.GetParameterValueFromOffset<FVector3f>(VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::Position]);
		}
		if (VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::Rotation] != INDEX_NONE)
		{
			VolumeDynamicData->DefaultRotation = ParameterData.GetParameterValueFromOffset<FQuat4f>(VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::Rotation]);
		}
		if (VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::Scale] != INDEX_NONE)
		{
			VolumeDynamicData->DefaultScale = ParameterData.GetParameterValueFromOffset<FVector3f>(VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::Scale]);
		}
		if (VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::RendererVisibilityTag] != INDEX_NONE)
		{
			VolumeDynamicData->DefaultRendererVisibilityTag = ParameterData.GetParameterValueFromOffset<int32>(VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::RendererVisibilityTag]);
		}
		if (VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::VolumeResolutionMaxAxis] != INDEX_NONE)
		{
			VolumeDynamicData->DefaultVolumeResolutionMaxAxis = ParameterData.GetParameterValueFromOffset<int32>(VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::VolumeResolutionMaxAxis]);
		}
		if (VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::VolumeWorldSpaceSize] != INDEX_NONE)
		{
			VolumeDynamicData->DefaultVolumeWorldSpaceSize = ParameterData.GetParameterValueFromOffset<FVector3f>(VFBoundOffsetsInParamStore[ENiagaraVolumeVFLayout::VolumeWorldSpaceSize]);
		}
	}

	// If we are in emitter mode and the vis tag doesn't match we won't send any dynamic data
	//-OPT: Perhaps remove the new / delete?  Path seems unlikely vs cluttering the code
	if (SourceMode == ENiagaraRendererSourceDataMode::Emitter && VolumeDynamicData->DefaultRendererVisibilityTag != RendererVisibilityTag)
	{
		delete VolumeDynamicData;
		return nullptr;
	}

	// Process Material Bindings
	if (Properties->MaterialParameters.HasAnyBindings())
	{
		ProcessMaterialParameterBindings(Properties->MaterialParameters, Emitter, MakeArrayView(BaseMaterials_GT));
	}

	const UNiagaraVolumeRendererProperties* RendererProperties = CastChecked<const UNiagaraVolumeRendererProperties>(InProperties);
	const int32 DefaultVolumeResolutionMaxAxis = UNiagaraVolumeRendererProperties::GetDefaultVolumeResolutionMaxAxis();
	const FVector3f DefaultVolumeWorldSpaceSize = UNiagaraVolumeRendererProperties::GetDefaultVolumeWorldSpaceSize();

	const FNiagaraParameterStore& ParameterStore = Emitter->GetRendererBoundVariables();
	VolumeDynamicData->VolumeResolutionMaxAxis = ParameterStore.GetParameterValueOrDefault(RendererProperties->VolumeResolutionMaxAxisBinding.GetParamMapBindableVariable(), DefaultVolumeResolutionMaxAxis);
	VolumeDynamicData->VolumeWorldSpaceSize = ParameterStore.GetParameterValueOrDefault(RendererProperties->VolumeWorldSpaceSizeBinding.GetParamMapBindableVariable(), DefaultVolumeWorldSpaceSize);
	VolumeDynamicData->StepFactor = RendererProperties->StepFactor;
	VolumeDynamicData->ShadowStepFactor = RendererProperties->ShadowStepFactor;
	VolumeDynamicData->ShadowBiasFactor = RendererProperties->ShadowBiasFactor;
	VolumeDynamicData->LightingDownsampleFactor = RendererProperties->LightingDownsampleFactor;

	return VolumeDynamicData;
}

int FNiagaraRendererVolumes::GetDynamicDataSize() const
{
	using namespace NiagaraRendererVolumesLocal;
	return sizeof(FVolumeDynamicData);
}

void FNiagaraRendererVolumes::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector, const FNiagaraSceneProxy* SceneProxy) const
{
	using namespace NiagaraRendererVolumesLocal;
	
	const FVolumeDynamicData* VolumeDynamicData = static_cast<const FVolumeDynamicData*>(DynamicDataRender);
	if (VolumeDynamicData == nullptr)
	{
		return;
	}

	FNiagaraDataBuffer* ParticleDataBuffer = VolumeDynamicData->GetParticleDataToRender(Collector.GetRHICommandList());
	if (!ParticleDataBuffer || (SourceMode == ENiagaraRendererSourceDataMode::Particles && ParticleDataBuffer->GetNumInstances() == 0))
	{
		return;
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];
			if (View->bIsInstancedStereoEnabled && IStereoRendering::IsStereoEyeView(*View) && !IStereoRendering::IsAPrimaryView(*View))
			{
				// We don't have to generate batches for non-primary views in stereo instance rendering
				continue;
			}

			FHeterogeneousVolumeInstances* VolumeInstances = &Collector.AllocateOneFrameResource<FHeterogeneousVolumeInstances>();
			if (SourceMode == ENiagaraRendererSourceDataMode::Particles)
			{
				const FNiagaraDataSet* DataSet = ParticleDataBuffer->GetOwner();

				FNiagaraDataSetReaderFloat<FNiagaraPosition>	PositionReader = PositionDataSetAccessor.GetReader(*DataSet);
				FNiagaraDataSetReaderFloat<FQuat4f>				RotationReader = RotationDataSetAccessor.GetReader(*DataSet);
				FNiagaraDataSetReaderFloat<FVector3f>			ScaleReader = ScaleDataSetAccessor.GetReader(*DataSet);
				FNiagaraDataSetReaderInt32<int32>				RendererVisibilityTagReader = RendererVisibilityTagAccessor.GetReader(*DataSet);
				FNiagaraDataSetReaderInt32<int32>				VolumeResolutionMaxAxisReader = VolumeResolutionMaxAxisAccessor.GetReader(*DataSet);
				FNiagaraDataSetReaderFloat<FVector3f>			VolumeWorldSpaceSizeReader = VolumeWorldSpaceSizeAccessor.GetReader(*DataSet);

				const bool bUseLocalSpace = UseLocalSpace(SceneProxy);
				const FMatrix LocalToWorld = SceneProxy->GetLocalToWorld();
				const FVector LWCTileOffset = FVector(SceneProxy->GetLWCRenderTile()) * FLargeWorldRenderScalar::GetTileSize();

				VolumeInstances->Instances.Reserve(ParticleDataBuffer->GetNumInstances());
				for (uint32 i=0; i < ParticleDataBuffer->GetNumInstances(); ++i)
				{
					const FVector	Position = FVector(PositionReader.GetSafe(i, VolumeDynamicData->DefaultPosition));
					const FQuat		Rotation = FQuat(RotationReader.GetSafe(i, VolumeDynamicData->DefaultRotation).GetNormalized());
					const FVector	Scale = FVector(ScaleReader.GetSafe(i, VolumeDynamicData->DefaultScale));
					const int32		VisibilityTag = RendererVisibilityTagReader.GetSafe(i, VolumeDynamicData->DefaultRendererVisibilityTag);
					const int32		ResolutionMaxAxis = VolumeResolutionMaxAxisReader.GetSafe(i, VolumeDynamicData->DefaultVolumeResolutionMaxAxis);
					const FVector3f	WorldSpaceSize = VolumeWorldSpaceSizeReader.GetSafe(i, VolumeDynamicData->DefaultVolumeWorldSpaceSize);

					if (VisibilityTag != RendererVisibilityTag || FMath::IsNearlyZero(Scale.SquaredLength()))
					{
						continue;
					}
					
					FMatrix InstanceToWorld = FScaleMatrix::Make(Scale) * FQuatRotationTranslationMatrix::Make(Rotation, Position);
					if ( bUseLocalSpace )
					{
						InstanceToWorld = LocalToWorld * InstanceToWorld;
					}
					else
					{
						InstanceToWorld.SetOrigin(InstanceToWorld.GetOrigin() + LWCTileOffset);
					}

					//-TODO: Push per instance data in here
					FHeterogeneousVolumeInstances::FInstance& VolumeInstance = VolumeInstances->Instances.AddDefaulted_GetRef();
					VolumeInstance.LocalBounds	= SceneProxy->GetLocalBounds();
					VolumeInstance.LocalToWorld	= InstanceToWorld;
				}
			}
			else
			{
				//-TODO: Matches current behavior but should pull emitter bounds / position binding location
				FHeterogeneousVolumeInstances::FInstance& VolumeInstance = VolumeInstances->Instances.AddDefaulted_GetRef();
				VolumeInstance.LocalBounds	= SceneProxy->GetLocalBounds();
				VolumeInstance.LocalToWorld	= SceneProxy->GetLocalToWorld();
			}

			if (VolumeInstances->Instances.Num() == 0)
			{
				continue;
			}

			FMeshBatch& Mesh				= Collector.AllocateMesh();
			Mesh.VertexFactory				= &VertexFactory;
			Mesh.MaterialRenderProxy		= VolumeDynamicData->Material;
			Mesh.LCI						= nullptr;
			Mesh.ReverseCulling				= SceneProxy->IsLocalToWorldDeterminantNegative() ? true : false;
			Mesh.CastShadow					= false;
			Mesh.Type						= PT_TriangleStrip;
			Mesh.bDisableBackfaceCulling	= true;
			Mesh.bCanApplyViewModeOverrides = true;
			Mesh.bUseWireframeSelectionColoring = SceneProxy->IsSelected();

			FMeshBatchElement& BatchElement	= Mesh.Elements[0];
			BatchElement.IndexBuffer		= nullptr;
			BatchElement.FirstIndex			= 0;
			BatchElement.MinVertexIndex		= 0;
			BatchElement.MaxVertexIndex		= 3;
			BatchElement.NumPrimitives		= 2;
			BatchElement.BaseVertexIndex	= 0;
			
			FHeterogeneousVolumeData* HeterogeneousVolumeData = &Collector.AllocateOneFrameResource<FHeterogeneousVolumeData>(SceneProxy);

			FVector3f WorldSpaceSize = VolumeDynamicData->VolumeWorldSpaceSize;
			float WorldSpaceSizeMaxInv = WorldSpaceSize.GetMax() > 0.0 ? 1.0 / WorldSpaceSize.GetMax() : 0.0;
			FVector3f ResolutionFactor = WorldSpaceSize * WorldSpaceSizeMaxInv;

			FVector3f VolumeResolutionV3f = ResolutionFactor * VolumeDynamicData->VolumeResolutionMaxAxis;

			FIntVector VolumeResolution = FIntVector(
				FMath::CeilToInt(VolumeResolutionV3f.X),
				FMath::CeilToInt(VolumeResolutionV3f.Y),
				FMath::CeilToInt(VolumeResolutionV3f.Z));

			HeterogeneousVolumeData->VoxelResolution = VolumeResolution;
			HeterogeneousVolumeData->StepFactor = VolumeDynamicData->StepFactor;
			HeterogeneousVolumeData->ShadowStepFactor = VolumeDynamicData->ShadowStepFactor;
			HeterogeneousVolumeData->ShadowBiasFactor = VolumeDynamicData->ShadowBiasFactor;
			HeterogeneousVolumeData->LightingDownsampleFactor = VolumeDynamicData->LightingDownsampleFactor;
			BatchElement.UserData = HeterogeneousVolumeData;

			Collector.AddMesh(ViewIndex, Mesh);
		}
	}
}

#if RHI_RAYTRACING
void FNiagaraRendererVolumes::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FNiagaraSceneProxy* SceneProxy)
{
	//-TODO: Add support for raytracing
}
#endif //RHI_RAYTRACING
