// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraResolveDIHelpers.h"

#include "Internationalization/Internationalization.h"
#include "NiagaraCommon.h"
#include "NiagaraConstants.h"
#include "NiagaraEmitter.h"
#include "NiagaraScript.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraSystem.h"

#if WITH_EDITORONLY_DATA

#define LOCTEXT_NAMESPACE "NiagaraResolveDIHelpers"

namespace FNiagaraResolveDIHelpers
{
	struct FDataInterfaceSourceEmitterNamePair
	{
		UNiagaraDataInterface* DataInterface;
		FName SourceName;
		FString SourceEmitterName;
	};

	void CollectDIReadsAssignmentsAndBindingsForScript(
		const UNiagaraScript* TargetScript,
		const UNiagaraSystem* System,
		const FString& EmitterName,
		TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair>& OutVariableReadMap,
		TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair>& OutVariableAssignmentMap,
		TMap<FNiagaraVariableBase, FNiagaraVariableBase>& OutVariableBindingMap,
		TArray<FText>& OutErrorMessages)
	{
		if (TargetScript == nullptr)
		{
			return;
		}

		for (const FNiagaraScriptDataInterfaceInfo& CachedDefaultDataInterface : TargetScript->GetCachedDefaultDataInterfaces())
		{
			if (CachedDefaultDataInterface.RegisteredParameterMapWrite != NAME_None)
			{
				FNiagaraVariable WriteVariable(CachedDefaultDataInterface.Type, CachedDefaultDataInterface.RegisteredParameterMapWrite);
				if (EmitterName.IsEmpty() == false)
				{
					WriteVariable = FNiagaraUtilities::ResolveAliases(WriteVariable, FNiagaraAliasContext().ChangeEmitterNameToEmitter(EmitterName));
				}

				// Handle reads and bindings.
				if (CachedDefaultDataInterface.RegisteredParameterMapRead != NAME_None)
				{
					FNiagaraVariable ReadVariable(CachedDefaultDataInterface.Type, CachedDefaultDataInterface.RegisteredParameterMapRead);
					if (EmitterName.IsEmpty() == false)
					{
						ReadVariable = FNiagaraUtilities::ResolveAliases(ReadVariable, FNiagaraAliasContext().ChangeEmitterToEmitterName(EmitterName));
					}

					FDataInterfaceSourceEmitterNamePair* CurrentRead = OutVariableReadMap.Find(ReadVariable);
					if (CurrentRead == nullptr)
					{
						FDataInterfaceSourceEmitterNamePair& NewRead = OutVariableReadMap.Add(ReadVariable);
						NewRead.DataInterface = CachedDefaultDataInterface.DataInterface;
						NewRead.SourceName = CachedDefaultDataInterface.Name;
						NewRead.SourceEmitterName = CachedDefaultDataInterface.SourceEmitterName;
					}

					if (ReadVariable != WriteVariable)
					{
						FNiagaraVariableBase* CurrentBinding = OutVariableBindingMap.Find(WriteVariable);
						if (CurrentBinding != nullptr)
						{
							if (*CurrentBinding != ReadVariable)
							{
								OutErrorMessages.Add(FText::Format(
									LOCTEXT("MultipleBindingsFormat", "A data interface parameter was the target of multiple bindings in a single system.  The data interface used in the simulation may be incorrect.  Target Parameter: {0} First Read: {1} Current Read: {2}"),
									FText::FromName(WriteVariable.GetName()),
									FText::FromName(CurrentBinding->GetName()),
									FText::FromName(ReadVariable.GetName())));
							}
						}
						else
						{
							{
								OutVariableBindingMap.Add(WriteVariable, ReadVariable);
							}
						}
					}
				}
		
				// Handle assignments
				FDataInterfaceSourceEmitterNamePair* CurrentAssignment = OutVariableAssignmentMap.Find(WriteVariable);
				if (CurrentAssignment != nullptr)
				{
					if (CurrentAssignment->DataInterface != CachedDefaultDataInterface.DataInterface)
					{
						OutErrorMessages.Add(FText::Format(
							LOCTEXT("MultipleAssignmentsFormat", "A data interface parameter was the target of an assignment multiple times in a single system.  The data interface used in the simulation may be incorrect.  Target Parameter: {0} First Assignment: {1} Current Assignment: {2}"),
							FText::FromName(WriteVariable.GetName()),
							FText::FromName(CurrentAssignment->SourceName),
							FText::FromName(CachedDefaultDataInterface.Name)));
					}
				}
				else
				{
					FDataInterfaceSourceEmitterNamePair& NewAssignment = OutVariableAssignmentMap.Add(WriteVariable);
					NewAssignment.DataInterface = CachedDefaultDataInterface.DataInterface;
					NewAssignment.SourceName = CachedDefaultDataInterface.Name;
					NewAssignment.SourceEmitterName = CachedDefaultDataInterface.SourceEmitterName;
				}
			}
		}
	}

	void CollectDIReadsAssignmentsAndBindings(
		const UNiagaraSystem* System,
		TMap<FGuid, TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair>>& OutEmitterIdToVariableReadsMap,
		TMap<FGuid, TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair>>& OutEmitterIdToVariableAssignmentsMap,
		TMap<FGuid, TMap<FNiagaraVariableBase, FNiagaraVariableBase>>& OutEmitterIdToVariableBindingsMap,
		TArray<FText>& OutErrorMessages)
	{
		TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair> VariableReadMap;
		TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair> VariableAssignmentMap;
		TMap<FNiagaraVariableBase, FNiagaraVariableBase> VariableBindingMap;

		// Add user parameters as assignments.
		const FNiagaraParameterStore& InstanceParameters = System->GetSystemCompiledData().InstanceParamStore;
		for (const FNiagaraVariableWithOffset& UserParameterWithOffset : InstanceParameters.ReadParameterVariables())
		{
			if (UserParameterWithOffset.IsDataInterface())
			{
				UNiagaraDataInterface* DataInterface = InstanceParameters.GetDataInterface(UserParameterWithOffset.Offset);
				if (DataInterface != nullptr)
				{
					FDataInterfaceSourceEmitterNamePair& NewAssignment = VariableAssignmentMap.Add(UserParameterWithOffset);
					NewAssignment.DataInterface = DataInterface;
					NewAssignment.SourceEmitterName = FString();
				}
			}
		}

		// Collect system and emitter scripts.
		CollectDIReadsAssignmentsAndBindingsForScript(System->GetSystemSpawnScript(), System, FString(), VariableReadMap, VariableAssignmentMap, VariableBindingMap, OutErrorMessages);
		CollectDIReadsAssignmentsAndBindingsForScript(System->GetSystemUpdateScript(), System, FString(), VariableReadMap, VariableAssignmentMap, VariableBindingMap, OutErrorMessages);
		OutEmitterIdToVariableReadsMap.Add(FGuid(), VariableReadMap);
		OutEmitterIdToVariableAssignmentsMap.Add(FGuid(), VariableAssignmentMap);
		OutEmitterIdToVariableBindingsMap.Add(FGuid(), VariableBindingMap);

		// Collect emitter scripts.
		for (const FNiagaraEmitterHandle& EmitterHandle : System->GetEmitterHandles())
		{
			if (EmitterHandle.GetIsEnabled() == false)
			{
				continue;
			}

			FVersionedNiagaraEmitterData* VersionedNiagaraEmitterData = EmitterHandle.GetEmitterData();
			if (VersionedNiagaraEmitterData == nullptr)
			{
				continue;
			}

			TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair> EmitterVariableReadMap = VariableReadMap;
			TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair> EmitterVariableAssignmentMap = VariableAssignmentMap;
			TMap<FNiagaraVariableBase, FNiagaraVariableBase> EmitterVariableBindingMap = VariableBindingMap;

			CollectDIReadsAssignmentsAndBindingsForScript(VersionedNiagaraEmitterData->SpawnScriptProps.Script, System, EmitterHandle.GetUniqueInstanceName(), EmitterVariableReadMap, EmitterVariableAssignmentMap, EmitterVariableBindingMap, OutErrorMessages);
			CollectDIReadsAssignmentsAndBindingsForScript(VersionedNiagaraEmitterData->UpdateScriptProps.Script, System, EmitterHandle.GetUniqueInstanceName(), EmitterVariableReadMap, EmitterVariableAssignmentMap, EmitterVariableBindingMap, OutErrorMessages);

			for (const FNiagaraEventScriptProperties& EventHandler : VersionedNiagaraEmitterData->GetEventHandlers())
			{
				CollectDIReadsAssignmentsAndBindingsForScript(EventHandler.Script, System, EmitterHandle.GetUniqueInstanceName(), EmitterVariableReadMap, EmitterVariableAssignmentMap, EmitterVariableBindingMap, OutErrorMessages);
			}

			for (const UNiagaraSimulationStageBase* SimulationStage : VersionedNiagaraEmitterData->GetSimulationStages())
			{
				if (SimulationStage == nullptr || SimulationStage->bEnabled == false)
				{
					continue;
				}
				CollectDIReadsAssignmentsAndBindingsForScript(SimulationStage->Script, System, EmitterHandle.GetUniqueInstanceName(), EmitterVariableReadMap, EmitterVariableAssignmentMap, EmitterVariableBindingMap, OutErrorMessages);
			}

			OutEmitterIdToVariableReadsMap.Add(EmitterHandle.GetId(), EmitterVariableReadMap);
			OutEmitterIdToVariableAssignmentsMap.Add(EmitterHandle.GetId(), EmitterVariableAssignmentMap);
			OutEmitterIdToVariableBindingsMap.Add(EmitterHandle.GetId(), EmitterVariableBindingMap);
		}
	}

	void CompactBindings(const TMap<FNiagaraVariableBase, FNiagaraVariableBase>& InVariableBindingMap, TMap<FNiagaraVariableBase, FNiagaraVariableBase>& OutCompactedVariableBindingMap, TArray<FText>& OutErrorMessages)
	{
		TArray<FNiagaraVariableBase> SourceVariables;
		InVariableBindingMap.GetKeys(SourceVariables);
		TMap<FNiagaraVariableBase, FNiagaraVariableBase> CompactedVariableBindingMap;
		for (const FNiagaraVariableBase& SourceVariable : SourceVariables)
		{
			FNiagaraVariableBase BoundVariable;
			const FNiagaraVariableBase* CurrentBoundVariable = &SourceVariable;
			TSet<FNiagaraVariableBase> SeenBoundVariables;
			bool bCircularReferenceDetected = false;
			while (CurrentBoundVariable != nullptr && bCircularReferenceDetected == false)
			{
				SeenBoundVariables.Add(*CurrentBoundVariable);
				BoundVariable = *CurrentBoundVariable;
				CurrentBoundVariable = InVariableBindingMap.Find(BoundVariable);
				if (CurrentBoundVariable != nullptr && SeenBoundVariables.Contains(*CurrentBoundVariable))
				{
					bCircularReferenceDetected = true;
					OutErrorMessages.Add(FText::Format(
						LOCTEXT("CircularDependencyWarningFormat", "A data interface parameter circular dependency was found.  The data interface used in the simulation may be incorrect.  Target Parameter: {0} Resolved Parameter: {1}"),
						FText::FromName(SourceVariable.GetName()),
						FText::FromName(CurrentBoundVariable->GetName())));
				}
			}
			OutCompactedVariableBindingMap.Add(SourceVariable, BoundVariable);
		}
	}

	void ResolveDIsForScript(
		const UNiagaraSystem* System,
		UNiagaraScript* TargetScript,
		const FString& EmitterName,
		const TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair>& VariableReadMap,
		const TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair>& VariableAssignmentMap,
		const TMap<FNiagaraVariableBase, FNiagaraVariableBase>& CompactedVariableBindingMap,
		FNiagaraScriptRuntimeCompiledData& OutScriptRuntimeCompiledData,
		TArray<FText>& OutErrorMessages)
	{
		if (TargetScript == nullptr)
		{
			return;
		}
		TArray<FNiagaraScriptResolvedDataInterfaceInfo> ResolvedDataInterfaces;
		ResolvedDataInterfaces.Reserve(TargetScript->GetCachedDefaultDataInterfaces().Num());
		TArray<FNiagaraResolvedUserDataInterfaceBinding> UserDataInterfaceBindings;
		int32 ResolvedDataInterfaceIndex = 0;
		TSet<FNiagaraVariableBase> ParameterStoreVariables;
		for (const FNiagaraScriptDataInterfaceInfo& CachedDefaultDataInterface : TargetScript->GetCachedDefaultDataInterfaces())
		{
			FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDataInterface = ResolvedDataInterfaces.AddDefaulted_GetRef();
			ResolvedDataInterface.Name = CachedDefaultDataInterface.Name;
			ResolvedDataInterface.CompileName = CachedDefaultDataInterface.CompileName;
			ResolvedDataInterface.UserPtrIdx = CachedDefaultDataInterface.UserPtrIdx;

			if (CachedDefaultDataInterface.RegisteredParameterMapRead != NAME_None)
			{
				// If this DI is read from a parameter try to resolve it through the binding and assignment or read maps.
				FNiagaraVariable ReadVariable(CachedDefaultDataInterface.Type, CachedDefaultDataInterface.RegisteredParameterMapRead);
				if (EmitterName.IsEmpty() == false)
				{
					ReadVariable = FNiagaraUtilities::ResolveAliases(ReadVariable, FNiagaraAliasContext().ChangeEmitterToEmitterName(EmitterName));
				}

				const FNiagaraVariableBase* BoundVariable = CompactedVariableBindingMap.Find(ReadVariable);
				if (BoundVariable == nullptr)
				{
					BoundVariable = &ReadVariable;
				}

				const FDataInterfaceSourceEmitterNamePair* BoundDataInterface = VariableAssignmentMap.Find(*BoundVariable);
				if (BoundDataInterface == nullptr)
				{
					// If the data interface was not written to explicitly, then it needs to use the default which was cached in the first read.
					BoundDataInterface = VariableReadMap.Find(*BoundVariable);
				}

				/*
				if (BoundDataInterface == nullptr)
				{
					OutErrorMessages.Add(FText::Format(
						LOCTEXT("MissingReadDataInterface", "A data interface was read from the parameter map, but its default could not be found.  The data interface used in the simulation may be incorrect.  Read Parameter: {0} Bound Parameter: {1}"),
						FText::FromName(ReadVariable.GetName()),
						FText::FromName(BoundVariable->GetName())));
				}
				*/

				if (BoundDataInterface != nullptr && ParameterStoreVariables.Contains(ReadVariable) == false)
				{
					ResolvedDataInterface.ResolvedVariable = *BoundVariable;
					ResolvedDataInterface.ParameterStoreVariable = ReadVariable;
					ResolvedDataInterface.bIsInternal = false;
					ResolvedDataInterface.ResolvedDataInterface = BoundDataInterface->DataInterface;
					ResolvedDataInterface.ResolvedSourceEmitterName = BoundDataInterface->SourceEmitterName;
					ParameterStoreVariables.Add(ResolvedDataInterface.ParameterStoreVariable);
				}

				if (FNiagaraUserRedirectionParameterStore::IsUserParameter(*BoundVariable))
				{
					const FNiagaraParameterStore& InstanceParameters = System->GetSystemCompiledData().InstanceParamStore;
					const int32* UserParameterIndex = InstanceParameters.FindParameterOffset(*BoundVariable);
					if (UserParameterIndex != nullptr && *UserParameterIndex != INDEX_NONE)
					{
						UserDataInterfaceBindings.Add(FNiagaraResolvedUserDataInterfaceBinding(*UserParameterIndex, ResolvedDataInterfaceIndex));
					}
				}
			}
			else if (CachedDefaultDataInterface.RegisteredParameterMapWrite != NAME_None)
			{
				FNiagaraVariable WriteVariable(CachedDefaultDataInterface.Type, CachedDefaultDataInterface.RegisteredParameterMapWrite);
				const FDataInterfaceSourceEmitterNamePair* DataInterfaceAssignment = VariableAssignmentMap.Find(WriteVariable);
				if (DataInterfaceAssignment != nullptr && ParameterStoreVariables.Contains(WriteVariable) == false)
				{
					ResolvedDataInterface.ResolvedVariable = WriteVariable;
					ResolvedDataInterface.ParameterStoreVariable = WriteVariable;
					ResolvedDataInterface.bIsInternal = false;
					ResolvedDataInterface.ResolvedDataInterface = DataInterfaceAssignment->DataInterface;
					ResolvedDataInterface.ResolvedSourceEmitterName = DataInterfaceAssignment->SourceEmitterName;
					ParameterStoreVariables.Add(ResolvedDataInterface.ParameterStoreVariable);
				}
			}

			if (ResolvedDataInterface.ResolvedDataInterface == nullptr)
			{
				// NPC variables will fail to resolve but we want to main that it's both external & not append the internal namespace
				if ( FNiagaraVariable::IsInNameSpace(FNiagaraConstants::ParameterCollectionNamespaceString, ResolvedDataInterface.Name) )
				{
					FNiagaraVariable NPCVariable(CachedDefaultDataInterface.Type, ResolvedDataInterface.Name);
					ResolvedDataInterface.ResolvedVariable = NPCVariable;
					ResolvedDataInterface.ParameterStoreVariable = NPCVariable;
					ResolvedDataInterface.bIsInternal = false;
					ResolvedDataInterface.ResolvedDataInterface = CachedDefaultDataInterface.DataInterface;
					ResolvedDataInterface.ResolvedSourceEmitterName = CachedDefaultDataInterface.SourceEmitterName;
				}
				else
				{
					// If the DI was not read from a parameter or couldn't be found, use the one cached during compilation, and give it an internal
					// name to prevent it from being bound incorrectly.
					FNameBuilder NameBuilder;
					NameBuilder.Append(FNiagaraConstants::InternalNamespaceString);
					NameBuilder.AppendChar(TEXT('.'));
					ResolvedDataInterface.Name.AppendString(NameBuilder);

					FNiagaraVariable InternalVariable(CachedDefaultDataInterface.Type, FName(NameBuilder.ToString()));
					ResolvedDataInterface.ResolvedVariable = InternalVariable;
					ResolvedDataInterface.ParameterStoreVariable = InternalVariable;
					ResolvedDataInterface.bIsInternal = true;
					ResolvedDataInterface.ResolvedDataInterface = CachedDefaultDataInterface.DataInterface;
					ResolvedDataInterface.ResolvedSourceEmitterName = CachedDefaultDataInterface.SourceEmitterName;
				}
			}

			ResolvedDataInterfaceIndex++;
		}
		OutScriptRuntimeCompiledData.ResolvedDataInterfaces = ResolvedDataInterfaces;
		OutScriptRuntimeCompiledData.ResolvedUserDataInterfaceBindings = UserDataInterfaceBindings;
	}

	void SynchronizeMatchingInternalResolvedDataInterfaces(TArrayView<const FNiagaraScriptResolvedDataInterfaceInfo> SourceResolvedDataInterfaces, TArray<FNiagaraScriptResolvedDataInterfaceInfo>& TargetResolvedDataInterfaces)
	{
		for (const FNiagaraScriptResolvedDataInterfaceInfo& SourceResolvedDataInterface : SourceResolvedDataInterfaces)
		{
			for (FNiagaraScriptResolvedDataInterfaceInfo& TargetResolvedDataInterface : TargetResolvedDataInterfaces)
			{
				if (TargetResolvedDataInterface.bIsInternal && TargetResolvedDataInterface.Name == SourceResolvedDataInterface.Name)
				{
					TargetResolvedDataInterface.ResolvedDataInterface = SourceResolvedDataInterface.ResolvedDataInterface;
					TargetResolvedDataInterface.ResolvedSourceEmitterName = SourceResolvedDataInterface.ResolvedSourceEmitterName;
					break;
				}
			}
		}
	}

	/** Handles the special case where internal data interfaces defined in certain scripts need to be synchronized with other scripts that contain them. */
	void ResolveInternalDataInterfaces(TArray<FNiagaraScriptRuntimeCompiledData*> SourceScriptsRuntimeCompiledData, FNiagaraScriptRuntimeCompiledData& TargetScriptRuntimeCompiledData)
	{
		TArray<FNiagaraScriptResolvedDataInterfaceInfo> ResolvedTargetDataInterfaces;
		ResolvedTargetDataInterfaces.Append(TargetScriptRuntimeCompiledData.ResolvedDataInterfaces);
		for (FNiagaraScriptRuntimeCompiledData* SourceScriptRuntimeCompiledData : SourceScriptsRuntimeCompiledData)
		{
			TArrayView<const FNiagaraScriptResolvedDataInterfaceInfo> ResolvedSourceParticleDataInterfaces = MakeConstArrayView(SourceScriptRuntimeCompiledData->ResolvedDataInterfaces);
			SynchronizeMatchingInternalResolvedDataInterfaces(ResolvedSourceParticleDataInterfaces, ResolvedTargetDataInterfaces);
		}
		TargetScriptRuntimeCompiledData.ResolvedDataInterfaces = ResolvedTargetDataInterfaces;
	}

	void ResolveDIsInternal(
		UNiagaraSystem* System,
		const TMap<FGuid, TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair>>& EmitterIdToVariableReadsMap,
		const TMap<FGuid, TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair>>& EmitterIdToVariableAssignmentsMap,
		const TMap<FGuid, TMap<FNiagaraVariableBase, FNiagaraVariableBase>>& EmitterIdToVariableBindingsMap,
		TMap<FNiagaraScriptDataKey, FNiagaraScriptRuntimeCompiledData>& OutScriptRuntimeCompiledDataMap,
		TArray<FText>& OutErrorMessages)
	{
		TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair> VariableReadMap = EmitterIdToVariableReadsMap[FGuid()];
		TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair> VariableAssignmentMap = EmitterIdToVariableAssignmentsMap[FGuid()];
		TMap<FNiagaraVariableBase, FNiagaraVariableBase> VariableBindingMap = EmitterIdToVariableBindingsMap[FGuid()];
		TMap<FNiagaraVariableBase, FNiagaraVariableBase> CompactedVariableBindingMap;
		CompactBindings(VariableBindingMap, CompactedVariableBindingMap, OutErrorMessages);

		UNiagaraScript* SystemSpawnScript = System->GetSystemSpawnScript();
		UNiagaraScript* SystemUpdateScript = System->GetSystemUpdateScript();

		FNiagaraScriptRuntimeCompiledData SystemSpawnRuntimeCompiledData;
		FNiagaraScriptRuntimeCompiledData SystemUpdateRuntimeCompiledData;

		ResolveDIsForScript(System, SystemSpawnScript, FString(), VariableReadMap, VariableAssignmentMap, CompactedVariableBindingMap, SystemSpawnRuntimeCompiledData, OutErrorMessages);
		ResolveDIsForScript(System, SystemUpdateScript, FString(), VariableReadMap, VariableAssignmentMap, CompactedVariableBindingMap, SystemUpdateRuntimeCompiledData, OutErrorMessages);

		OutScriptRuntimeCompiledDataMap.Add(FNiagaraScriptDataKey(FGuid(), SystemSpawnScript->GetUsage(), SystemSpawnScript->GetUsageId()), SystemSpawnRuntimeCompiledData);
		OutScriptRuntimeCompiledDataMap.Add(FNiagaraScriptDataKey(FGuid(), SystemUpdateScript->GetUsage(), SystemUpdateScript->GetUsageId()), SystemUpdateRuntimeCompiledData);

		for (const FNiagaraEmitterHandle& EmitterHandle : System->GetEmitterHandles())
		{
			if (EmitterHandle.GetIsEnabled() == false)
			{
				continue;
			}

			FVersionedNiagaraEmitterData* VersionedNiagaraEmitterData = EmitterHandle.GetEmitterData();
			if (VersionedNiagaraEmitterData == nullptr)
			{
				continue;
			}

			TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair> EmitterVariableReadMap = EmitterIdToVariableReadsMap[EmitterHandle.GetId()];
			TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair> EmitterVariableAssignmentMap = EmitterIdToVariableAssignmentsMap[EmitterHandle.GetId()];
			TMap<FNiagaraVariableBase, FNiagaraVariableBase> EmitterVariableBindingMap = EmitterIdToVariableBindingsMap[EmitterHandle.GetId()];
			TMap<FNiagaraVariableBase, FNiagaraVariableBase> EmitterCompactedVariableBindingMap;
			CompactBindings(EmitterVariableBindingMap, EmitterCompactedVariableBindingMap, OutErrorMessages);

			UNiagaraScript* SpawnScript = VersionedNiagaraEmitterData->SpawnScriptProps.Script;
			UNiagaraScript* UpdateScript = VersionedNiagaraEmitterData->UpdateScriptProps.Script;
			UNiagaraScript* GpuScript = VersionedNiagaraEmitterData->GetGPUComputeScript();

			FNiagaraScriptRuntimeCompiledData SpawnRuntimeCompiledData;
			FNiagaraScriptRuntimeCompiledData UpdateRuntimeCompiledData;

			ResolveDIsForScript(System, SpawnScript, EmitterHandle.GetUniqueInstanceName(), EmitterVariableReadMap, EmitterVariableAssignmentMap, EmitterCompactedVariableBindingMap, SpawnRuntimeCompiledData, OutErrorMessages);
			ResolveDIsForScript(System, UpdateScript, EmitterHandle.GetUniqueInstanceName(), EmitterVariableReadMap, EmitterVariableAssignmentMap, EmitterCompactedVariableBindingMap, UpdateRuntimeCompiledData, OutErrorMessages);

			if (VersionedNiagaraEmitterData->UsesInterpolatedSpawning())
			{
				TArray<FNiagaraScriptRuntimeCompiledData*> SourceScriptRuntimeCompiledData = { &UpdateRuntimeCompiledData };
				ResolveInternalDataInterfaces(SourceScriptRuntimeCompiledData, SpawnRuntimeCompiledData);
			}

			OutScriptRuntimeCompiledDataMap.Add(FNiagaraScriptDataKey(EmitterHandle.GetId(), SpawnScript->GetUsage(), SpawnScript->GetUsageId()), SpawnRuntimeCompiledData);
			OutScriptRuntimeCompiledDataMap.Add(FNiagaraScriptDataKey(EmitterHandle.GetId(), UpdateScript->GetUsage(), UpdateScript->GetUsageId()), UpdateRuntimeCompiledData);

			for (const FNiagaraEventScriptProperties& EventHandler : VersionedNiagaraEmitterData->GetEventHandlers())
			{
				UNiagaraScript* EventScript = EventHandler.Script;
				FNiagaraScriptRuntimeCompiledData EventRuntimeCompiledData;

				ResolveDIsForScript(System, EventScript, EmitterHandle.GetUniqueInstanceName(), EmitterVariableReadMap, EmitterVariableAssignmentMap, EmitterCompactedVariableBindingMap, EventRuntimeCompiledData, OutErrorMessages);
				OutScriptRuntimeCompiledDataMap.Add(FNiagaraScriptDataKey(EmitterHandle.GetId(), EventScript->GetUsage(), EventScript->GetUsageId()), EventRuntimeCompiledData);
			}

			if (VersionedNiagaraEmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim && GpuScript != nullptr)
			{
				FNiagaraScriptRuntimeCompiledData GpuRuntimeCompiledData;
				ResolveDIsForScript(System, GpuScript, EmitterHandle.GetUniqueInstanceName(), EmitterVariableReadMap, EmitterVariableAssignmentMap, EmitterCompactedVariableBindingMap, GpuRuntimeCompiledData, OutErrorMessages);

				TArray<FNiagaraScriptRuntimeCompiledData> SimulationStageScriptRuntimeCompiledDataItems;
				for (const UNiagaraSimulationStageBase* SimulationStage : VersionedNiagaraEmitterData->GetSimulationStages())
				{
					if (SimulationStage == nullptr || SimulationStage->bEnabled == false)
					{
						continue;
					}
					UNiagaraScript* SimulationStageScript = SimulationStage->Script;
					FNiagaraScriptRuntimeCompiledData& SimulationStageRuntimeCompiledData = SimulationStageScriptRuntimeCompiledDataItems.AddDefaulted_GetRef();

					ResolveDIsForScript(System, SimulationStageScript, EmitterHandle.GetUniqueInstanceName(), EmitterVariableReadMap, EmitterVariableAssignmentMap, EmitterCompactedVariableBindingMap, SimulationStageRuntimeCompiledData, OutErrorMessages);
				}

				TArray<FNiagaraScriptRuntimeCompiledData*> SourceScriptRuntimeCompiledData;
				SourceScriptRuntimeCompiledData.Add(&SpawnRuntimeCompiledData);
				SourceScriptRuntimeCompiledData.Add(&UpdateRuntimeCompiledData);
				for (FNiagaraScriptRuntimeCompiledData& SimulationStageScriptRuntimeCompiledData : SimulationStageScriptRuntimeCompiledDataItems)
				{
					SourceScriptRuntimeCompiledData.Add(&SimulationStageScriptRuntimeCompiledData);
				}
				ResolveInternalDataInterfaces(SourceScriptRuntimeCompiledData, GpuRuntimeCompiledData);

				GpuRuntimeCompiledData.SimTarget = ENiagaraSimTarget::GPUComputeSim;
				OutScriptRuntimeCompiledDataMap.Add(FNiagaraScriptDataKey(EmitterHandle.GetId(), GpuScript->GetUsage(), GpuScript->GetUsageId()), GpuRuntimeCompiledData);
			}
		}
	}

	void ResolveDIs(UNiagaraSystem* System, TMap<FNiagaraScriptDataKey, FNiagaraScriptRuntimeCompiledData>& OutScriptRuntimeCompiledDataMap, TArray<FText>& OutErrorMessages)
	{
		TMap<FGuid, TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair>> EmitterIdToVariableReadsMap;
		TMap<FGuid, TMap<FNiagaraVariableBase, FDataInterfaceSourceEmitterNamePair>> EmitterIdToVariableAssignmentsMap;
		TMap<FGuid, TMap<FNiagaraVariableBase, FNiagaraVariableBase>> EmitterIdToVariableBindingsMap;
		FNiagaraResolveDIHelpers::CollectDIReadsAssignmentsAndBindings(System, EmitterIdToVariableReadsMap, EmitterIdToVariableAssignmentsMap, EmitterIdToVariableBindingsMap, OutErrorMessages);
		FNiagaraResolveDIHelpers::ResolveDIsInternal(System, EmitterIdToVariableReadsMap, EmitterIdToVariableAssignmentsMap, EmitterIdToVariableBindingsMap, OutScriptRuntimeCompiledDataMap, OutErrorMessages);
	}
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITORONLY_DATA