// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"

#include "NiagaraCommon.h"
#include "NiagaraScriptRuntimeCompiledData.h"
#include "NiagaraTypes.h"

class UNiagaraDataInterface;
class UNiagaraSystem;

#if WITH_EDITORONLY_DATA

namespace FNiagaraResolveDIHelpers
{
	void ResolveDIs(UNiagaraSystem* System, TMap<FNiagaraScriptDataKey, FNiagaraScriptRuntimeCompiledData>& OutScriptRuntimeCompiledDataMap, TArray<FText>& OutErrorMessages);
}

#endif