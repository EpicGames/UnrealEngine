// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraResourceArrayWriter.h"

FNiagaraResourceArrayWriter::FNiagaraResourceArrayWriter(TResourceArray<uint8>& InBytes, bool bIsPersistent, bool bSetOffset, const FName InArchiveName)
	: FMemoryArchive()
	, Bytes(InBytes)
	, ArchiveName(InArchiveName)
{
	this->SetIsSaving(true);
	this->SetIsPersistent(bIsPersistent);
	if (bSetOffset)
	{
		Offset = InBytes.Num();
	}
}

void FNiagaraResourceArrayWriter::Serialize(void* Data, int64 Num)
{
	const int64 NumBytesToAdd = Offset + Num - Bytes.Num();
	if (NumBytesToAdd > 0)
	{
		const int64 NewArrayCount = Bytes.Num() + NumBytesToAdd;
		if (NewArrayCount >= MAX_int32)
		{
			UE_LOGF(LogSerialization, Fatal, "FMemoryWriter does not support data larger than 2GB. Archive name: %ls.", *ArchiveName.ToString());
		}

		Bytes.AddUninitialized((int32)NumBytesToAdd);
	}

	check((Offset + Num) <= Bytes.Num());

	if (Num)
	{
		FMemory::Memcpy(&Bytes[(int32)Offset], Data, Num);
		Offset += Num;
	}
}

int64 FNiagaraResourceArrayWriter::TotalSize()
{
	return Bytes.Num();
}
