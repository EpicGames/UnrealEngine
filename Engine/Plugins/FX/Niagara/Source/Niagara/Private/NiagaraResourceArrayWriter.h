// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Serialization/MemoryArchive.h"
#include "Containers/DynamicRHIResourceArray.h"

class FNiagaraResourceArrayWriter : public FMemoryArchive
{
public:
	FNiagaraResourceArrayWriter(TResourceArray<uint8>& InBytes, bool bIsPersistent = false, bool bSetOffset = false, const FName InArchiveName = NAME_None);
	virtual void Serialize(void* Data, int64 Num) override;

	/**
	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const override { return TEXT("FNiagaraResourceArrayWriter"); }

	int64 TotalSize() override;

protected:
	TResourceArray<uint8>& Bytes;

	/** Archive name, used to debugging, by default set to NAME_None. */
	const FName ArchiveName;
};
