// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRibbonRendererProperties.h"
#include "NiagaraModule.h"
#include "NiagaraRendererRibbons.h"
#include "NiagaraConstants.h"
#include "NiagaraBoundsCalculatorHelper.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraSystem.h"

#include "Materials/MaterialInstanceConstant.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraRibbonRendererProperties)

#if WITH_EDITOR
#include "Widgets/Images/SImage.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/SWidget.h"
#include "AssetThumbnail.h"
#include "Widgets/Text/STextBlock.h"
#endif

#define LOCTEXT_NAMESPACE "UNiagaraRibbonRendererProperties"

namespace NiagaraRibbonRendererPropertiesPrivate
{
	static const FName NAME_UniqueID("UniqueID");
	static FNiagaraVariableBase GetUniqueIDVariable() { return FNiagaraVariableBase(FNiagaraTypeDefinition::GetIntDef(), NAME_UniqueID); }

	TArray<TWeakObjectPtr<UNiagaraRibbonRendererProperties>> RibbonRendererPropertiesToDeferredInit;
}

FNiagaraRibbonShapeCustomVertex::FNiagaraRibbonShapeCustomVertex()
	: Position(0.0f)
	, Normal(0.0f)
	, TextureV(0.0f)
{
}

FNiagaraRibbonUVSettings::FNiagaraRibbonUVSettings()
	: DistributionMode(ENiagaraRibbonUVDistributionMode::ScaledUsingRibbonSegmentLength)
	, LeadingEdgeMode(ENiagaraRibbonUVEdgeMode::Locked)
	, TrailingEdgeMode(ENiagaraRibbonUVEdgeMode::Locked)
	, bEnablePerParticleUOverride(false)
	, bEnablePerParticleVRangeOverride(false)
	, TilingLength(100.0f)
	, Offset(FVector2D(0.0f, 0.0f))
	, Scale(FVector2D(1.0f, 1.0f))
{
}

UNiagaraRibbonRendererProperties::UNiagaraRibbonRendererProperties()
	: Material(nullptr)
	, MaterialUserParamBinding(FNiagaraTypeDefinition(UMaterialInterface::StaticClass()))
#if WITH_EDITORONLY_DATA
	, UV0TilingDistance_DEPRECATED(0.0f)
	, UV0Scale_DEPRECATED(FVector2D(1.0f, 1.0f))
	, UV0AgeOffsetMode_DEPRECATED(ENiagaraRibbonAgeOffsetMode::Scale)
	, UV1TilingDistance_DEPRECATED(0.0f)
	, UV1Scale_DEPRECATED(FVector2D(1.0f, 1.0f))
	, UV1AgeOffsetMode_DEPRECATED(ENiagaraRibbonAgeOffsetMode::Scale)
#endif
	, MaxNumRibbons(0)
	, Shape(ENiagaraRibbonShapeMode::Plane)
	, bEnableAccurateGeometry(false)
	, bUseMaterialBackfaceCulling(false)
	, bUseGeometryNormals(true)
	, bUseGPUInit(false)
	, bUseConstantFactor(false)
	, bScreenSpaceTessellation(true)
	, bLinkOrderUseUniqueID(true)
	, WidthSegmentationCount(1)
	, MultiPlaneCount(2)
	, TubeSubdivisions(3)
	, TessellationMode(ENiagaraRibbonTessellationMode::Automatic)
	, CurveTension(0.f)
	, TessellationFactor(16)
	, TessellationAngle(15)
{
	AttributeBindings =
	{
		&PositionBinding,
		&ColorBinding,
		&VelocityBinding,
		&NormalizedAgeBinding,
		&RibbonTwistBinding,
		&RibbonWidthBinding,
		&RibbonFacingBinding,
		&RibbonIdBinding,
		&RibbonLinkOrderBinding,
	
		&MaterialRandomBinding,
		&DynamicMaterialBinding,
		&DynamicMaterial1Binding,
		&DynamicMaterial2Binding,
		&DynamicMaterial3Binding,
		&RibbonUVDistance,
		&U0OverrideBinding,
		&V0RangeOverrideBinding,
		&U1OverrideBinding,
		&V1RangeOverrideBinding,

		&PrevPositionBinding,
		&PrevRibbonWidthBinding,
		&PrevRibbonFacingBinding,
		&PrevRibbonTwistBinding,
	};
}

FNiagaraRenderer* UNiagaraRibbonRendererProperties::CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
	FNiagaraRenderer* NewRenderer = new FNiagaraRendererRibbons(FeatureLevel, this, Emitter);
	NewRenderer->Initialize(this, Emitter, InController);
	return NewRenderer;
}

#if WITH_EDITORONLY_DATA
void UpgradeUVSettings(FNiagaraRibbonUVSettings& UVSettings, float TilingDistance, const FVector2D& Offset, const FVector2D& Scale)
{
	if (TilingDistance == 0)
	{
		UVSettings.LeadingEdgeMode = ENiagaraRibbonUVEdgeMode::SmoothTransition;
		UVSettings.TrailingEdgeMode = ENiagaraRibbonUVEdgeMode::SmoothTransition;
		UVSettings.DistributionMode = ENiagaraRibbonUVDistributionMode::ScaledUniformly;
	}
	else
	{
		UVSettings.LeadingEdgeMode = ENiagaraRibbonUVEdgeMode::Locked;
		UVSettings.TrailingEdgeMode = ENiagaraRibbonUVEdgeMode::Locked;
		UVSettings.DistributionMode = ENiagaraRibbonUVDistributionMode::TiledOverRibbonLength;
		UVSettings.TilingLength = TilingDistance;
	}
	UVSettings.Offset = Offset;
	UVSettings.Scale = Scale;
}
#endif

void UNiagaraRibbonRendererProperties::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (MaterialUserParamBinding.Parameter.GetType().GetClass() != UMaterialInterface::StaticClass())
	{
		FNiagaraTypeDefinition MaterialDef(UMaterialInterface::StaticClass());
		MaterialUserParamBinding.Parameter.SetType(MaterialDef);
	}

	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
	if (NiagaraVer < FNiagaraCustomVersion::RibbonRendererUVRefactor)
	{
		UpgradeUVSettings(UV0Settings, UV0TilingDistance_DEPRECATED, UV0Offset_DEPRECATED, UV0Scale_DEPRECATED);
		UpgradeUVSettings(UV1Settings, UV1TilingDistance_DEPRECATED, UV1Offset_DEPRECATED, UV1Scale_DEPRECATED);
	}

	if (NiagaraVer < FNiagaraCustomVersion::RibbonRendererLinkOrderDefaultIsUniqueID)
	{
		bLinkOrderUseUniqueID = false;
	}

	if (NiagaraVer < FNiagaraCustomVersion::RibbonPlaneUseGeometryNormals)
	{
		bUseGeometryNormals = false;
	}

	ChangeToPositionBinding(PositionBinding);
#endif
	
	PostLoadBindings(ENiagaraRendererSourceDataMode::Particles);

	// Fix up these bindings from their loaded source bindings
	SetPreviousBindings(FVersionedNiagaraEmitterBase());

	if ( Material )
	{
		Material->ConditionalPostLoad();
	}

#if WITH_EDITORONLY_DATA
	if (MaterialParameterBindings_DEPRECATED.Num() > 0)
	{
		MaterialParameters.AttributeBindings = MaterialParameterBindings_DEPRECATED;
		MaterialParameterBindings_DEPRECATED.Empty();
	}
#endif
	MaterialParameters.ConditionalPostLoad();
}

FNiagaraBoundsCalculator* UNiagaraRibbonRendererProperties::CreateBoundsCalculator()
{
	return new FNiagaraBoundsCalculatorHelper<false, false, true>();
}

void UNiagaraRibbonRendererProperties::GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const
{
	UMaterialInterface* MaterialInterface = nullptr;
	if (InEmitter != nullptr)
	{
		MaterialInterface = Cast<UMaterialInterface>(InEmitter->FindBinding(MaterialUserParamBinding.Parameter));
	}

#if WITH_EDITORONLY_DATA
	MaterialInterface = MaterialInterface ? MaterialInterface : ToRawPtr(MICMaterial);
#endif

	OutMaterials.Add(MaterialInterface ? MaterialInterface : ToRawPtr(Material));
}

void UNiagaraRibbonRendererProperties::CollectPSOPrecacheData(const FNiagaraEmitterInstance* InEmitter, FPSOPrecacheParamsList& OutParams) const
{
	const FVertexFactoryType* VFType = GetVertexFactoryType();
	UMaterialInterface* MaterialInterface = ToRawPtr(Material);

	if (MaterialInterface)
	{
		FPSOPrecacheParams& PSOPrecacheParams = OutParams.AddDefaulted_GetRef();
		PSOPrecacheParams.MaterialInterface = MaterialInterface;
		// Ribbon VF is the same for MVF and non-MVF cases
		PSOPrecacheParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(VFType));
	}
}

const FVertexFactoryType* UNiagaraRibbonRendererProperties::GetVertexFactoryType() const
{
	return &FNiagaraRibbonVertexFactory::StaticType;
}

bool UNiagaraRibbonRendererProperties::IsBackfaceCullingDisabled() const
{
	if (bUseMaterialBackfaceCulling && Material)
	{
		return Material->IsTwoSided();
	}
	else if (Shape == ENiagaraRibbonShapeMode::MultiPlane)
	{
		return !bEnableAccurateGeometry;
	}
	else
	{
		return true;
	}
}

#if WITH_EDITORONLY_DATA
TArray<FNiagaraVariable> UNiagaraRibbonRendererProperties::GetBoundAttributes() const 
{
	TArray<FNiagaraVariable> BoundAttributes = Super::GetBoundAttributes();
	BoundAttributes.Reserve(BoundAttributes.Num() + MaterialParameters.AttributeBindings.Num() + 2);

	if (bLinkOrderUseUniqueID)
	{
		BoundAttributes.AddUnique(NiagaraRibbonRendererPropertiesPrivate::GetUniqueIDVariable());
	}

	for (const FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		BoundAttributes.AddUnique(MaterialParamBinding.GetParamMapBindableVariable());
	}

	if (MaterialUserParamBinding.Parameter.IsValid())
	{
		BoundAttributes.AddUnique(MaterialUserParamBinding.Parameter);
	}

	return BoundAttributes;
}
#endif

bool UNiagaraRibbonRendererProperties::PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore)
{
	bool bAnyAdded = Super::PopulateRequiredBindings(InParameterStore);

	for (const FNiagaraVariableAttributeBinding* Binding : AttributeBindings)
	{
		if (Binding && Binding->CanBindToHostParameterMap())
		{
			InParameterStore.AddParameter(Binding->GetParamMapBindableVariable(), false);
			bAnyAdded = true;
		}
	}

	for (FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		InParameterStore.AddParameter(MaterialParamBinding.GetParamMapBindableVariable(), false);
		bAnyAdded = true;
	}

	return bAnyAdded;
}

void UNiagaraRibbonRendererProperties::UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit)
{
	UNiagaraEmitterBase* SrcEmitter = GetTypedOuter<UNiagaraEmitterBase>();
	if (SrcEmitter)
	{
		for (FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
		{
			MaterialParamBinding.CacheValues(SrcEmitter);
		}
	}

	Super::UpdateSourceModeDerivates(InSourceMode, bFromPropertyEdit);
}

void UNiagaraRibbonRendererProperties::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		// We can end up hitting PostInitProperties before the Niagara Module has initialized bindings this needs, mark this object for deferred init and early out.
		if (FModuleManager::Get().IsModuleLoaded("Niagara") == false)
		{
			NiagaraRibbonRendererPropertiesPrivate::RibbonRendererPropertiesToDeferredInit.Add(this);
			return;
		}
		InitBindings();
	}
}

void UNiagaraRibbonRendererProperties::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& Ar = Record.GetUnderlyingArchive();

	// MIC will replace the main material during serialize
	// Be careful if adding code that looks at the material to make sure you get the correct one
	{
#if WITH_EDITORONLY_DATA
		TOptional<TGuardValue<TObjectPtr<UMaterialInterface>>> MICGuard;
		if (Ar.IsSaving() && Ar.IsCooking() && MICMaterial)
		{
			MICGuard.Emplace(Material, MICMaterial);
		}
#endif
		Super::Serialize(Record);
	}
}

void UNiagaraRibbonRendererProperties::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(RendererLayout.GetAllocatedSize());
}

/** The bindings depend on variables that are created during the NiagaraModule startup. However, the CDO's are build prior to this being initialized, so we defer setting these values until later.*/
void UNiagaraRibbonRendererProperties::InitCDOPropertiesAfterModuleStartup()
{
	UNiagaraRibbonRendererProperties* CDO = CastChecked<UNiagaraRibbonRendererProperties>(UNiagaraRibbonRendererProperties::StaticClass()->GetDefaultObject());
	CDO->InitBindings();

	for (TWeakObjectPtr<UNiagaraRibbonRendererProperties>& WeakRibbonRendererProperties : NiagaraRibbonRendererPropertiesPrivate::RibbonRendererPropertiesToDeferredInit)
	{
		if (WeakRibbonRendererProperties.Get())
		{
			WeakRibbonRendererProperties->InitBindings();
		}
	}
}

void UNiagaraRibbonRendererProperties::InitBindings()
{
	if (!PositionBinding.IsValid())
	{
		PositionBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
		ColorBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_COLOR);
		VelocityBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VELOCITY);
		DynamicMaterialBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM);
		DynamicMaterial1Binding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1);
		DynamicMaterial2Binding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2);
		DynamicMaterial3Binding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3);
		NormalizedAgeBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_NORMALIZED_AGE);
		RibbonTwistBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_RIBBONTWIST);
		RibbonWidthBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_RIBBONWIDTH);
		RibbonFacingBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_RIBBONFACING);
		RibbonIdBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_RIBBONID);		
		RibbonLinkOrderBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_RIBBONLINKORDER);		
		MaterialRandomBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_MATERIAL_RANDOM);
		RibbonUVDistance = FNiagaraConstants::GetAttributeDefaultBinding(RIBBONUVDISTANCE);
		U0OverrideBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_RIBBONU0OVERRIDE);
		V0RangeOverrideBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_RIBBONV0RANGEOVERRIDE);
		U1OverrideBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_RIBBONU1OVERRIDE);
		V1RangeOverrideBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_RIBBONV1RANGEOVERRIDE);
	}

	SetPreviousBindings(FVersionedNiagaraEmitterBase());
}

void UNiagaraRibbonRendererProperties::SetPreviousBindings(const FVersionedNiagaraEmitterBase& SrcEmitter)
{
	PrevPositionBinding.SetAsPreviousValue(PositionBinding, SrcEmitter, ENiagaraRendererSourceDataMode::Particles);
	PrevRibbonWidthBinding.SetAsPreviousValue(RibbonWidthBinding, SrcEmitter, ENiagaraRendererSourceDataMode::Particles);
	PrevRibbonFacingBinding.SetAsPreviousValue(RibbonFacingBinding, SrcEmitter, ENiagaraRendererSourceDataMode::Particles);
	PrevRibbonTwistBinding.SetAsPreviousValue(RibbonTwistBinding, SrcEmitter, ENiagaraRendererSourceDataMode::Particles);
}

#if WITH_EDITORONLY_DATA
bool UNiagaraRibbonRendererProperties::IsSupportedVariableForBinding(const FNiagaraVariableBase& InSourceForBinding, const FName& InTargetBindingName) const
{
	if (InTargetBindingName == GET_MEMBER_NAME_CHECKED(UNiagaraRendererProperties, RendererEnabledBinding))
	{
		return
			InSourceForBinding.IsInNameSpace(FNiagaraConstants::UserNamespace) ||
			InSourceForBinding.IsInNameSpace(FNiagaraConstants::SystemNamespace) ||
			InSourceForBinding.IsInNameSpace(FNiagaraConstants::EmitterNamespace);
	}

	return InSourceForBinding.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString);
}
#endif

void UNiagaraRibbonRendererProperties::CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData)
{
	UpdateMICs();

	// Initialize accessors
	RibbonLinkOrderFloatAccessor.Init(CompiledData, RibbonLinkOrderBinding.GetDataSetBindableVariable().GetName());
	RibbonLinkOrderInt32Accessor.Init(nullptr, NAME_None);
	if (!RibbonLinkOrderFloatAccessor.IsValid())
	{
		if (bLinkOrderUseUniqueID)
		{
			RibbonLinkOrderInt32Accessor.Init(CompiledData, NiagaraRibbonRendererPropertiesPrivate::NAME_UniqueID);
		}
		if (!RibbonLinkOrderInt32Accessor.IsValid())
		{
			RibbonLinkOrderFloatAccessor.Init(CompiledData, NormalizedAgeBinding.GetDataSetBindableVariable().GetName());
		}
	}

	NormalizedAgeAccessor.Init(CompiledData, NormalizedAgeBinding.GetDataSetBindableVariable().GetName());

	PositionDataSetAccessor.Init(CompiledData, PositionBinding.GetDataSetBindableVariable().GetName());
	SizeDataSetAccessor.Init(CompiledData, RibbonWidthBinding.GetDataSetBindableVariable().GetName());
	TwistDataSetAccessor.Init(CompiledData, RibbonTwistBinding.GetDataSetBindableVariable().GetName());
	FacingDataSetAccessor.Init(CompiledData, RibbonFacingBinding.GetDataSetBindableVariable().GetName());

	MaterialParam0DataSetAccessor.Init(CompiledData, DynamicMaterialBinding.GetDataSetBindableVariable().GetName());
	MaterialParam1DataSetAccessor.Init(CompiledData, DynamicMaterial1Binding.GetDataSetBindableVariable().GetName());
	MaterialParam2DataSetAccessor.Init(CompiledData, DynamicMaterial2Binding.GetDataSetBindableVariable().GetName());
	MaterialParam3DataSetAccessor.Init(CompiledData, DynamicMaterial3Binding.GetDataSetBindableVariable().GetName());

	FNiagaraDataSetAccessor<float> RibbonUVDistanceAccessor;
	RibbonUVDistanceAccessor.Init(CompiledData, RibbonUVDistance.GetDataSetBindableVariable().GetName());
	DistanceFromStartIsBound = RibbonUVDistanceAccessor.IsValid();

	FNiagaraDataSetAccessor<float> U0OverrideDataSetAccessor;
	U0OverrideDataSetAccessor.Init(CompiledData, U0OverrideBinding.GetDataSetBindableVariable().GetName());
	U0OverrideIsBound = U0OverrideDataSetAccessor.IsValid();
	FNiagaraDataSetAccessor<float> U1OverrideDataSetAccessor;
	U1OverrideDataSetAccessor.Init(CompiledData, U1OverrideBinding.GetDataSetBindableVariable().GetName());
	U1OverrideIsBound = U1OverrideDataSetAccessor.IsValid();

	if (RibbonIdBinding.GetDataSetBindableVariable().GetType() == FNiagaraTypeDefinition::GetIDDef())
	{
		RibbonFullIDDataSetAccessor.Init(CompiledData, RibbonIdBinding.GetDataSetBindableVariable().GetName());
	}
	else
	{
		RibbonIdDataSetAccessor.Init(CompiledData, RibbonIdBinding.GetDataSetBindableVariable().GetName());
	}

	const bool bShouldDoFacing = FacingMode == ENiagaraRibbonFacingMode::Custom || FacingMode == ENiagaraRibbonFacingMode::CustomSideVector;

	// Initialize layout
	RendererLayout.Initialize(ENiagaraRibbonVFLayout::Num);
	RendererLayout.SetVariableFromBinding(CompiledData, PositionBinding, ENiagaraRibbonVFLayout::Position);
	RendererLayout.SetVariableFromBinding(CompiledData, VelocityBinding, ENiagaraRibbonVFLayout::Velocity);
	RendererLayout.SetVariableFromBinding(CompiledData, ColorBinding, ENiagaraRibbonVFLayout::Color);
	RendererLayout.SetVariableFromBinding(CompiledData, RibbonWidthBinding, ENiagaraRibbonVFLayout::Width);
	RendererLayout.SetVariableFromBinding(CompiledData, RibbonTwistBinding, ENiagaraRibbonVFLayout::Twist);
	if (bShouldDoFacing)
	{
		RendererLayout.SetVariableFromBinding(CompiledData, RibbonFacingBinding, ENiagaraRibbonVFLayout::Facing);
	}
	RendererLayout.SetVariableFromBinding(CompiledData, NormalizedAgeBinding, ENiagaraRibbonVFLayout::NormalizedAge);
	RendererLayout.SetVariableFromBinding(CompiledData, MaterialRandomBinding, ENiagaraRibbonVFLayout::MaterialRandom);
	RendererLayout.SetVariableFromBinding(CompiledData, RibbonUVDistance, ENiagaraRibbonVFLayout::DistanceFromStart);
	RendererLayout.SetVariableFromBinding(CompiledData, U0OverrideBinding, ENiagaraRibbonVFLayout::U0Override);
	RendererLayout.SetVariableFromBinding(CompiledData, V0RangeOverrideBinding, ENiagaraRibbonVFLayout::V0RangeOverride);
	RendererLayout.SetVariableFromBinding(CompiledData, U1OverrideBinding, ENiagaraRibbonVFLayout::U1Override);
	RendererLayout.SetVariableFromBinding(CompiledData, V1RangeOverrideBinding, ENiagaraRibbonVFLayout::V1RangeOverride);
	const bool bDynamicParam0Valid = RendererLayout.SetVariableFromBinding(CompiledData, DynamicMaterialBinding,  ENiagaraRibbonVFLayout::MaterialParam0);
	const bool bDynamicParam1Valid = RendererLayout.SetVariableFromBinding(CompiledData, DynamicMaterial1Binding, ENiagaraRibbonVFLayout::MaterialParam1);
	const bool bDynamicParam2Valid = RendererLayout.SetVariableFromBinding(CompiledData, DynamicMaterial2Binding, ENiagaraRibbonVFLayout::MaterialParam2);
	const bool bDynamicParam3Valid = RendererLayout.SetVariableFromBinding(CompiledData, DynamicMaterial3Binding, ENiagaraRibbonVFLayout::MaterialParam3);

	if (NeedsPreciseMotionVectors())
	{
		RendererLayout.SetVariableFromBinding(CompiledData, PrevPositionBinding, ENiagaraRibbonVFLayout::PrevPosition);
		RendererLayout.SetVariableFromBinding(CompiledData, PrevRibbonWidthBinding, ENiagaraRibbonVFLayout::PrevRibbonWidth);
		RendererLayout.SetVariableFromBinding(CompiledData, PrevRibbonFacingBinding, ENiagaraRibbonVFLayout::PrevRibbonFacing);
		RendererLayout.SetVariableFromBinding(CompiledData, PrevRibbonTwistBinding, ENiagaraRibbonVFLayout::PrevRibbonTwist);
	}

	RendererLayout.Finalize();

	// Find Ribbon link order for GPU.  VF bindings don't support Int's at the moment, and there's no point any CPU emitter caring about this data.
	bGpuRibbonLinkIsFloat = false;
	GpuRibbonLinkOrderOffset = INDEX_NONE;
	if (CompiledData)
	{
		if (const FNiagaraVariableLayoutInfo* BindingLinkOrderInfo = CompiledData->FindVariableLayoutInfo(RibbonLinkOrderBinding.GetDataSetBindableVariable()))
		{
			bGpuRibbonLinkIsFloat = true;
			GpuRibbonLinkOrderOffset = BindingLinkOrderInfo->GetFloatComponentStart();
		}
		else if (bLinkOrderUseUniqueID)
		{
			if (const FNiagaraVariableLayoutInfo* UniqueIDLinkOrderInfo = CompiledData->FindVariableLayoutInfo(NiagaraRibbonRendererPropertiesPrivate::GetUniqueIDVariable()))
			{
				bGpuRibbonLinkIsFloat = false;
				GpuRibbonLinkOrderOffset = UniqueIDLinkOrderInfo->GetInt32ComponentStart();
			}
		}
		if (GpuRibbonLinkOrderOffset == INDEX_NONE)
		{
			if (const FNiagaraVariableLayoutInfo* NormAgeLinkOrderInfo = CompiledData->FindVariableLayoutInfo(NormalizedAgeBinding.GetDataSetBindableVariable()))
			{
				bGpuRibbonLinkIsFloat = true;
				GpuRibbonLinkOrderOffset = NormAgeLinkOrderInfo->GetFloatComponentStart();
			}
		}
	}

#if WITH_EDITORONLY_DATA
	// Build dynamic parameter mask
	// Serialized in cooked builds
	MaterialParamValidMask = GetDynamicParameterCombinedChannelMask(
		bDynamicParam0Valid ? DynamicMaterialBinding.GetName() : NAME_None,
		bDynamicParam1Valid ? DynamicMaterial1Binding.GetName() : NAME_None,
		bDynamicParam2Valid ? DynamicMaterial2Binding.GetName() : NAME_None,
		bDynamicParam3Valid ? DynamicMaterial3Binding.GetName() : NAME_None
	);
#endif
}

void UNiagaraRibbonRendererProperties::UpdateMICs()
{
#if WITH_EDITORONLY_DATA
	UpdateMaterialParametersMIC(MaterialParameters, Material, MICMaterial);
#endif
}

#if WITH_EDITORONLY_DATA

void UNiagaraRibbonRendererProperties::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraRibbonRendererProperties, TessellationAngle))
	{
		if (TessellationAngle > 0.f && TessellationAngle < 1.f)
		{
			TessellationAngle = 1.f;
		}
	}

	// Update our MICs if we change material / material bindings
	//-OPT: Could narrow down further to only static materials
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraRibbonRendererProperties, Material)) ||
		(MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraRibbonRendererProperties, MaterialParameters)))
	{
		UpdateMICs();
	}
}

const TArray<FNiagaraVariable>& UNiagaraRibbonRendererProperties::GetOptionalAttributes()
{
	static TArray<FNiagaraVariable> Attrs;

	if (Attrs.Num() == 0)
	{
		Attrs.Add(SYS_PARAM_PARTICLES_POSITION);
		Attrs.Add(SYS_PARAM_PARTICLES_NORMALIZED_AGE);
		Attrs.Add(SYS_PARAM_PARTICLES_COLOR);
		Attrs.Add(SYS_PARAM_PARTICLES_RIBBONID);
		Attrs.Add(SYS_PARAM_PARTICLES_RIBBONTWIST);
		Attrs.Add(SYS_PARAM_PARTICLES_RIBBONWIDTH);
		Attrs.Add(SYS_PARAM_PARTICLES_RIBBONFACING);
		Attrs.Add(SYS_PARAM_PARTICLES_RIBBONLINKORDER);
		Attrs.Add(RIBBONUVDISTANCE);
		Attrs.Add(SYS_PARAM_PARTICLES_RIBBONU0OVERRIDE);
		Attrs.Add(SYS_PARAM_PARTICLES_RIBBONV0RANGEOVERRIDE);
		Attrs.Add(SYS_PARAM_PARTICLES_RIBBONU1OVERRIDE);
		Attrs.Add(SYS_PARAM_PARTICLES_RIBBONV1RANGEOVERRIDE);
	}

	return Attrs;
}

void UNiagaraRibbonRendererProperties::GetAdditionalVariables(TArray<FNiagaraVariableBase>& OutArray) const
{
	if (NeedsPreciseMotionVectors())
	{
		OutArray.Append({
				PrevPositionBinding.GetParamMapBindableVariable(),
				PrevRibbonWidthBinding.GetParamMapBindableVariable(),
				PrevRibbonFacingBinding.GetParamMapBindableVariable(),
				PrevRibbonTwistBinding.GetParamMapBindableVariable()
		});
	}
}

FNiagaraVariable UNiagaraRibbonRendererProperties::GetBoundAttribute(const FNiagaraVariableAttributeBinding* Binding) const
{
	if (!NeedsPreciseMotionVectors())
	{
		if ((Binding == &PrevPositionBinding)
			|| (Binding == &PrevRibbonWidthBinding)
			|| (Binding == &PrevRibbonFacingBinding)
			|| (Binding == &PrevRibbonTwistBinding))
		{
			return FNiagaraVariable();
		}
	}

	return Super::GetBoundAttribute(Binding);
}

void UNiagaraRibbonRendererProperties::GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	TArray<UMaterialInterface*> Materials;
	GetUsedMaterials(InEmitter, Materials);

	CreateRendererWidgetsForAssets(Materials, InThumbnailPool, OutWidgets);
}

void UNiagaraRibbonRendererProperties::GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	TArray<UMaterialInterface*> Materials;
	GetUsedMaterials(InEmitter, Materials);
	if (Materials.Num() > 0)
	{
		GetRendererWidgets(InEmitter, OutWidgets, InThumbnailPool);
	}
	else
	{
		TSharedRef<SWidget> RibbonTooltip = SNew(STextBlock)
			.Text(LOCTEXT("RibbonRendererNoMat", "Ribbon Renderer (No Material Set)"));
		OutWidgets.Add(RibbonTooltip);
	}
}


void UNiagaraRibbonRendererProperties::GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const
{
	Super::GetRendererFeedback(InEmitter, OutErrors, OutWarnings, OutInfo);

	const FVersionedNiagaraEmitterData* EmitterData = InEmitter.GetEmitterData();

	// If this is a renderer made before we used UniqueID give the user an option to switch into the new mode
	if (EmitterData && bLinkOrderUseUniqueID == false)
	{
		const FNiagaraVMExecutableData& ExecData = EmitterData->SpawnScriptProps.Script->GetVMExecutableData();
		if (!ExecData.Attributes.Contains(RibbonLinkOrderBinding.GetDataSetBindableVariable()))
		{
			OutInfo.Emplace(
				LOCTEXT("RibbonLinkOrderUsesNormalizedAgeSummary", "RibbonLinkOrder will use normalized age if the link order binding does not exist.  This can produce unpredictable results with burst modules, new renderers will use Particles.UniqueID to fix this issue."),
				LOCTEXT("RibbonLinkOrderUsesNormalizedAgeDesc", "RibbonLinkOrder will fallback to normalized age"),
				LOCTEXT("RibbonLinkOrderUsesNormalizedAgeFix", "Change fallback to use Particles.UniqueID"),
				FNiagaraRendererFeedbackFix::CreateLambda([Renderer = const_cast<UNiagaraRibbonRendererProperties*>(this)]() { Renderer->bLinkOrderUseUniqueID = true; }),
				true
			);
		}
	}

	// If we're in a gpu sim, then uv mode uniform by segment can cause some visual oddity due to non-existent
	// culling of near particles like the cpu initialization pipeline runs
	if (EmitterData && EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		const auto CheckUVSettingsForChannel = [&](const FNiagaraRibbonUVSettings& UVSettings, int32 Index)
		{
			if (UVSettings.DistributionMode == ENiagaraRibbonUVDistributionMode::ScaledUniformly)
			{
				const FText ErrorDescription = FText::Format(LOCTEXT("NiagaraRibbonRendererUVBySegmentGPUDesc", "The specified UV Distribution for Channel {0} on GPU may result in different visual look than a CPU sim due to increased particle density in GPU sim."), FText::AsNumber(Index));
				const FText ErrorSummary = FText::Format(LOCTEXT("NiagaraRibbonRendererUVBySegmentGPUSummary", "The specified UV Settings on Channel {0} on GPU may result in undesirable look."), FText::AsNumber(Index));
				OutWarnings.Add(FNiagaraRendererFeedback(ErrorDescription, ErrorSummary, FText(), FNiagaraRendererFeedbackFix(), true));				
			}
		};

		CheckUVSettingsForChannel(UV0Settings, 0);
		CheckUVSettingsForChannel(UV1Settings, 1);

		if (DrawDirection != ENiagaraRibbonDrawDirection::FrontToBack)
		{
			OutWarnings.Emplace(
				LOCTEXT("GpuDrawDirectionNoSupportDesc", "Gpu ribbons only support the default Draw Direction for 'Front To Back'"),
				LOCTEXT("GpuDrawDirectionNoSupportSummary", "Gpu ribbons do not support this Draw Direction mode it will be ignored"),
				FText(),
				FNiagaraRendererFeedbackFix(),
				true
			);
		}
	}

	// If we're in multiplane shape, and multiplane count is even while we're in camera facing mode then one
	// slice out of the set will be invisible because the camera will be coplanar to it
	if (FacingMode == ENiagaraRibbonFacingMode::Screen && Shape == ENiagaraRibbonShapeMode::MultiPlane && MultiPlaneCount % 2 == 0)
	{
		const FText ErrorDescription = LOCTEXT("NiagaraRibbonRendererMultiPlaneInvisibleFaceDesc", "The specified MultiPlaneCount (Even Count) with ScreenFacing will result in a hidden face due to the camera being coplanar to one face.");
		const FText ErrorSummary = LOCTEXT("NiagaraRibbonRendererMultiPlaneInvisibleFaceSummary", "The specified MultiPlaneCount+ScreenFacing will result in a hidden face.");
		const FText ErrorFix = LOCTEXT("NiagaraRibbonRendererMultiPlaneInvisibleFaceFix", "Fix by decreasing MultiPlane count by 1.");
		const FNiagaraRendererFeedbackFix MultiPlaneFix = FNiagaraRendererFeedbackFix::CreateLambda([this]() { const_cast<UNiagaraRibbonRendererProperties*>(this)->MultiPlaneCount = FMath::Clamp(this->MultiPlaneCount - 1, 1, 16); });
		OutWarnings.Add(FNiagaraRendererFeedback(ErrorDescription, ErrorSummary, ErrorFix, MultiPlaneFix, true));	
	}	

	if (MaterialParameters.HasAnyBindings())
	{
		TArray<UMaterialInterface*> Materials;
		GetUsedMaterials(nullptr, Materials);
		MaterialParameters.GetFeedback(Materials, OutWarnings);
	}
}

void UNiagaraRibbonRendererProperties::GetRendererMaterialUsageFeedback(TArray<FNiagaraRendererFeedback>& OutFeedback) const
{
	Super::GetRendererMaterialUsageFeedback(OutFeedback);
	GetMaterialUsageFeedback(MATUSAGE_NiagaraRibbons, OutFeedback);
}

void UNiagaraRibbonRendererProperties::RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RenameVariable(OldVariable, NewVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MaterialParameters.RenameVariable(OldVariable, NewVariable, InEmitter, GetCurrentSourceMode());
#endif
}

void UNiagaraRibbonRendererProperties::RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RemoveVariable(OldVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MaterialParameters.RemoveVariable(OldVariable, InEmitter, GetCurrentSourceMode());
#endif
}

#endif // WITH_EDITORONLY_DATA
#undef LOCTEXT_NAMESPACE

