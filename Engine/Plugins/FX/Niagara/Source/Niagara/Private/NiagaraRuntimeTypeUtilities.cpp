// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRuntimeTypeUtilities.h"
#include "UObject/ObjectKey.h"

namespace FNiagaraRuntimeTypeUtilities
{
	static FTypeConverter					IntConverter;
	static FTypeConverter					FloatConverter;
	static FTypeConverter					DoubleConverter;
	static TMap<FObjectKey, FTypeConverter> StructTypeConverters;

	template<typename TTypeFrom, typename TTypeTo>
	void CopyValue(const void* FromPointer, void* ToPointer)
	{
		TTypeFrom FromValue;
		FMemory::Memcpy(&FromValue, FromPointer, sizeof(TTypeFrom));

		const TTypeTo ToValue = TTypeTo(FromValue);
		FMemory::Memcpy(ToPointer, &ToValue, sizeof(TTypeTo));
	}

	template<typename TType, typename TNiagaraType>
	void RegisterStructTypeConverterBase(FNiagaraTypeDefinition TypeDef)
	{
		RegisterStructTypeConverter(TBaseStructure<TType>::Get(), FTypeConverter(TypeDef, CopyValue<TType, TNiagaraType>, CopyValue<TNiagaraType, TType>));
	}

	void Initialize()
	{
		IntConverter = FTypeConverter(FNiagaraTypeDefinition::GetIntDef(), CopyValue<int, int>, CopyValue<int, int>);
		FloatConverter = FTypeConverter(FNiagaraTypeDefinition::GetFloatDef(), CopyValue<float, float>, CopyValue<float, float>);
		DoubleConverter = FTypeConverter(FNiagaraTypeDefinition::GetFloatDef(), CopyValue<double, float>, CopyValue<float, double>);

		RegisterStructTypeConverterBase<FVector2D,		FVector2f>(FNiagaraTypeDefinition::GetVec2Def());
		RegisterStructTypeConverterBase<FVector,		FVector3f>(FNiagaraTypeDefinition::GetVec3Def());
		RegisterStructTypeConverterBase<FVector4,		FVector4f>(FNiagaraTypeDefinition::GetVec4Def());
		RegisterStructTypeConverterBase<FQuat,			FQuat4f>(FNiagaraTypeDefinition::GetQuatDef());
		RegisterStructTypeConverterBase<FLinearColor,	FLinearColor>(FNiagaraTypeDefinition::GetColorDef());
	}

	void RegisterStructTypeConverter(const UScriptStruct* Struct, FTypeConverter Converter)
	{
		check(StructTypeConverters.Contains(Struct) == false);
		StructTypeConverters.Emplace(Struct, MoveTemp(Converter));
	}

	void UnregisterStructTypeConverter(const UScriptStruct* Struct)
	{
		check(StructTypeConverters.Contains(Struct) == true);
		StructTypeConverters.Remove(Struct);
	}

	const FTypeConverter* FindTypeConverterForProperty(const FProperty* Property)
	{
		if (Property->IsA<const FIntProperty>())
		{
			return &IntConverter;
		}
		else if (Property->IsA<const FFloatProperty>())
		{
			return &FloatConverter;
		}
		else if (Property->IsA<const FDoubleProperty>())
		{
			return &DoubleConverter;
		}
		else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
		{
			return StructTypeConverters.Find(StructProperty->Struct);
		}
		return nullptr;
	}
}
