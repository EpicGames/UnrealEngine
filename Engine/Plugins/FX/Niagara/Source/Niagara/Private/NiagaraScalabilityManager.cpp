// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraScalabilityManager.h"
#include "NiagaraWorldManager.h"
#include "NiagaraComponent.h"
#include "NiagaraSystemInstanceController.h"
#include "Particles/FXBudget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScalabilityManager)

static float GScalabilityUpdateTime_Low = 1.0f;
static float GScalabilityUpdateTime_Medium = 0.5f;
static float GScalabilityUpdateTime_High = 0.25f;
static FAutoConsoleVariableRef CVarScalabilityUpdateTime_Low(TEXT("fx.NiagaraScalabilityUpdateTime_Low"), GScalabilityUpdateTime_Low, TEXT("Time in seconds between updates to scalability states for Niagara systems set to update at Low frequency. \n"), ECVF_Default);
static FAutoConsoleVariableRef CVarScalabilityUpdateTime_Medium(TEXT("fx.NiagaraScalabilityUpdateTime_Medium"), GScalabilityUpdateTime_Medium, TEXT("Time in seconds between updates to scalability states for Niagara systems set to update at Medium frequency. \n"), ECVF_Default);
static FAutoConsoleVariableRef CVarScalabilityUpdateTime_High(TEXT("fx.NiagaraScalabilityUpdateTime_High"), GScalabilityUpdateTime_High, TEXT("Time in seconds between updates to scalability states for Niagara systems set to update at High frequency. \n"), ECVF_Default);

static int32 GScalabilityManParallelThreshold = 50;
static FAutoConsoleVariableRef CVarScalabilityManParallelThreshold(TEXT("fx.ScalabilityManParallelThreshold"), GScalabilityManParallelThreshold, TEXT("Number of instances required for a niagara significance manger to go parallel for it's update. \n"), ECVF_Default);

static int32 GScalabilityMaxUpdatesPerFrame = 50;
static FAutoConsoleVariableRef CVarScalabilityMaxUpdatesPerFrame(TEXT("fx.ScalabilityMaxUpdatesPerFrame"), GScalabilityMaxUpdatesPerFrame, TEXT("Number of instances that can be processed per frame when updating scalability state. -1 for all of them. \n"), ECVF_Default);

static int32 GApplyInstanceCountsRigidly = 0;
static FAutoConsoleVariableRef CVarApplyInstanceCountsRigidly(TEXT("fx.Niagara.Scalability.ApplyInstanceCountsRigidly"), GApplyInstanceCountsRigidly, TEXT("If true we'll apply instance counts more rigidly. Un-culled Systems wil not be able to activate until overall systems fall below the instance count. \n"), ECVF_Default);

static float GetScalabilityUpdatePeriod(ENiagaraScalabilityUpdateFrequency Frequency)
{
	switch (Frequency)
	{
	case ENiagaraScalabilityUpdateFrequency::High: return GScalabilityUpdateTime_High;
	case ENiagaraScalabilityUpdateFrequency::Medium: return GScalabilityUpdateTime_Medium;
	case ENiagaraScalabilityUpdateFrequency::Low: return GScalabilityUpdateTime_Low;
	}

	return 0.0f;
}

static int32 GetMaxUpdatesPerFrame(const UNiagaraEffectType* EffectType, int32 ItemsRemaining, float UpdatePeriod, float DeltaSeconds)
{
	if (GScalabilityMaxUpdatesPerFrame > 0 && EffectType->UpdateFrequency != ENiagaraScalabilityUpdateFrequency::Continuous)
	{
		int32 UpdateCount = ItemsRemaining;

		if ((UpdatePeriod > SMALL_NUMBER) && (DeltaSeconds < UpdatePeriod))
		{
			UpdateCount = FMath::Min(FMath::CeilToInt(((float)ItemsRemaining) * DeltaSeconds / UpdatePeriod), ItemsRemaining);
		}

		if (UpdateCount > GScalabilityMaxUpdatesPerFrame)
		{
#if !NO_LOGGING
			if (FNiagaraUtilities::LogVerboseWarnings())
			{
				static TSet<const void*> MessagedEffectTypeSet;

				bool AlreadyAdded = false;
				MessagedEffectTypeSet.Add(EffectType, &AlreadyAdded);

				if (!AlreadyAdded)
				{
					UE_LOGF(LogNiagara, Verbose, "NiagaraScalabilityManager needs to process %d updates (will be clamped to %d) for EffectType - %ls - (%d items, %f period (s), %f delta (s)",
						UpdateCount,
						GScalabilityMaxUpdatesPerFrame,
						*EffectType->GetName(),
						ItemsRemaining,
						UpdatePeriod,
						DeltaSeconds);
				}
			}
#endif // !NO_LOGGING
			UpdateCount = GScalabilityMaxUpdatesPerFrame;
		}
		return UpdateCount;
	}

	return ItemsRemaining;
}

FNiagaraScalabilityManager::FNiagaraScalabilityManager(FNiagaraWorldManager* InOwner)
	: WorldMan(InOwner)
	, EffectType(nullptr)
	, LastUpdateTime(0.0)
	, bRefreshCachedSystemData(false)
{

}

FNiagaraScalabilityManager::FNiagaraScalabilityManager()
	: WorldMan(nullptr)
	, EffectType(nullptr)
	, LastUpdateTime(0.0)
	, bRefreshCachedSystemData(false)
{

}

FNiagaraScalabilityManager::~FNiagaraScalabilityManager()
{
	for (UNiagaraComponent* Component : ManagedComponents)
	{
		if (Component)
		{
			Component->ScalabilityManagerHandle = INDEX_NONE;
		}
	}
	ManagedComponents.Empty();
}

void FNiagaraScalabilityManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(EffectType);
	Collector.AddReferencedObjects(ManagedComponents);
}


void FNiagaraScalabilityManager::PreGarbageCollectBeginDestroy()
{
	//After the GC has potentially nulled out references to the components we were tracking we clear them out here.
	//This should only be in the case where MarkAsGarbage() is called directly. Typical component destruction will unregister in OnComponentDestroyed() or OnUnregister().
	//Components then just clear their handle in BeginDestroy knowing they've already been removed from the manager.
	//I would prefer some pre BeginDestroy() callback into the component in which I could cleanly unregister with the manager in all cases but I don't think that's possible.
	int32 CompIdx = ManagedComponents.Num();
	while (--CompIdx >= 0)
	{
		UNiagaraComponent* Comp = ManagedComponents[CompIdx];
		if (Comp == nullptr)
		{
			//UE_LOGF(LogNiagara, Warning, "Unregister from PreGCBeginDestroy @%d/%d - %ls", CompIdx, ManagedComponents.Num(), *EffectType->GetName());
			UnregisterAt(CompIdx);
		}
		else if (!IsValidChecked(Comp) || Comp->IsUnreachable())
		{
			Unregister(Comp);
		}
	}
}

void FNiagaraScalabilityManager::OnRefreshOwnerAllowsScalability()
{
	//We trigger a full refresh of scalability state when the view target changes etc to ensure we're not culling something we now shouldn't.
	bRefreshOwnerAllowsScalability = true;
}

void FNiagaraScalabilityManager::Register(UNiagaraComponent* Component)
{
	check(Component->ScalabilityManagerHandle == INDEX_NONE);
	check(ManagedComponents.Num() == State.Num());

	Component->ScalabilityManagerHandle = ManagedComponents.Add(Component);
	State.AddDefaulted();
	
#if WITH_NIAGARA_DEBUGGER
	Component->DebugCachedScalabilityState = State.Last();
#endif	

	if (HasPendingUpdates())
	{
		DefaultContext.ComponentRequiresUpdate.Add(true);
	}

	//Init System Data.
	GetSystemData(Component->ScalabilityManagerHandle);

	//UE_LOGF(LogNiagara, Warning, "Registered Component %p at index %d", Component, Component->ScalabilityManagerHandle);
}

void FNiagaraScalabilityManager::Unregister(UNiagaraComponent* Component)
{
	checkf(Component->ScalabilityManagerHandle != INDEX_NONE, TEXT("Component(%s) is not in the scalability manager but is being unregistered.  Asset(%s)."), *GetFullNameSafe(Component), *GetFullNameSafe(Component->GetAsset()));

	int32 IndexToRemove = Component->ScalabilityManagerHandle;
	Component->ScalabilityManagerHandle = INDEX_NONE;
	UnregisterAt(IndexToRemove);
}

void FNiagaraScalabilityManager::UnregisterAt(int32 IndexToRemove)
{
	//UE_LOGF(LogNiagara, Warning, "Unregistering Component %p at index %d (Replaced with %p)", ManagedComponents[IndexToRemove], IndexToRemove, ManagedComponents.Num() > 1 ? ManagedComponents.Last() : nullptr);

	check(ManagedComponents.Num() == State.Num());
	if (ManagedComponents.IsValidIndex(IndexToRemove))
	{
		ManagedComponents.RemoveAtSwap(IndexToRemove);
		State.RemoveAtSwap(IndexToRemove);

		if (HasPendingUpdates())
		{
			DefaultContext.ComponentRequiresUpdate.RemoveAtSwap(IndexToRemove);
		}
	}
	else
	{
		UE_LOGF(LogNiagara, Warning, "Attempting to unregister an invalid index from the Scalability Manager. Index: %d - Num: %d", IndexToRemove, ManagedComponents.Num());
	}

	//Redirect the component that is now at IndexToRemove to the correct index.
	if (ManagedComponents.IsValidIndex(IndexToRemove))
	{
		if ((ManagedComponents[IndexToRemove] != nullptr))//Possible this has been GCd. It will be removed later if so.
		{
			ManagedComponents[IndexToRemove]->ScalabilityManagerHandle = IndexToRemove;
		}
	}
}

// Note that this function may unregister elements in the ManagedComponents array (if the Component or System are no longer valid).
// Returns false if there was a problem evaluating the specified index and something had to be unregistered
bool FNiagaraScalabilityManager::EvaluateCullState(FComponentIterationContext& Context, int32 ComponentIndex, int32& UpdateCounter)
{
	check(ManagedComponents.IsValidIndex(ComponentIndex));
	UNiagaraComponent* Component = ManagedComponents[ComponentIndex];

	if (!Component)
	{
		UnregisterAt(ComponentIndex);
		return false;
	}
	//Belt and braces GC safety. If someone calls MarkAsGarbage() directly and we get here before we clear these out in the post GC callback.
	else if (!IsValid(Component))
	{
		Unregister(Component);
		return false;
	}

	FNiagaraScalabilityState& CompState = State[ComponentIndex];

	//Don't update if we're doing new systems only and this is not new.
	//Saves the potential cost of reavaluating every effect in every tick group something new is added.
	//Though this does mean the sorted significance values will be using out of date distances etc.
	//I'm somewhat on the fence currently as to whether it's better to pay this cost for correctness.
	const bool UpdateScalability = Component->ScalabilityManagerHandle == ComponentIndex
		&& (!Context.bNewOnly || CompState.bNewlyRegistered);

	if (UpdateScalability)
	{
		UNiagaraSystem* System = Component->GetAsset();
		if (System == nullptr)
		{
			UE_LOGF(LogNiagara, Warning, "Niagara System has been destroyed with components still registered to the scalability manager. Unregistering this component.\nComponent: 0x%p - %ls\nEffectType: 0x%p - %ls",
				Component, *Component->GetName(), EffectType.Get(), *EffectType->GetName());
			Unregister(Component);
			return false;
		}
		CompState.bNewlyRegisteredDirty = CompState.bNewlyRegistered;
		CompState.bNewlyRegistered = false;

		const FNiagaraSystemScalabilitySettings& Scalability = System->GetScalabilitySettings();

		const FNiagaraScalabilitySystemData& SysData = GetSystemData(ComponentIndex);

#if DEBUG_SCALABILITY_STATE
		CompState.bCulledByInstanceCount = false;
		CompState.bCulledByDistance = false;
		CompState.bCulledByVisibility = false;
#endif
		WorldMan->CalculateScalabilityState(System, Scalability, EffectType, Component, false, Context.WorstGlobalBudgetUse, CompState);

		// components that are not dirty and are culled can be safely skipped because we don't care about their
		// significance.  We also don't care about the significance of those components that are dirty and culled
		// but we do have to make sure to reset their significance index
		// though some systems do require full significance info for culled instances.
		bool bNeedSortedSignificane = (SysData.bNeedsSignificanceForActiveOrDirty && (!CompState.bCulled || CompState.IsDirty())) || SysData.bNeedsSignificanceForCulled;
		if (bNeedSortedSignificane)
		{
			Context.bRequiresGlobalSignificancePass |= System->NeedsSortedSignificanceCull();
		}

		// we may find that this is a false positive because our CompState may get reset in ProcessSignificance
		// but that shouldn't really cost us much
		Context.bHasDirtyState |= CompState.IsDirty();

		++UpdateCounter;
	}

	return true;
}

void FNiagaraScalabilityManager::ProcessSignificance(UNiagaraSignificanceHandler* SignificanceHandler, FComponentIterationContext& Context)
{
	// it would be good to get a better estimate for how many indices we're going to need to process
	Context.SignificanceIndices.Reset(ManagedComponents.Num());

	SignificanceHandler->CalculateSignificance(ManagedComponents, State, SystemData, Context.SignificanceIndices);

	auto ComparePredicate = [&](const FNiagaraScalabilityState& A, const FNiagaraScalabilityState& B)
	{
		return A.Significance > B.Significance;
	};

	Context.SignificanceIndices.Sort([&](int32 A, int32 B) { return ComparePredicate(State[A], State[B]); });

	//clear out working variables for the system data.
	for (FNiagaraScalabilitySystemData& SysData : SystemData)
	{
		SysData.InstanceCount = 0;
		SysData.CullProxyCount = 0;
	}

	int32 EffectTypeActiveInstances = 0;

	struct FCullProxyCreateDestroyTask
	{
		UNiagaraComponent* Component = nullptr;
		bool bCreate = false;
	};

	TArray<FCullProxyCreateDestroyTask, TInlineAllocator<64>> CullProxyTasks;
	for (int32 SortedIt = 0; SortedIt < Context.SignificanceIndices.Num(); ++SortedIt)
	{
		int32 SortedIdx = Context.SignificanceIndices[SortedIt];
		UNiagaraComponent* Component = ManagedComponents[SortedIdx];
		FNiagaraScalabilityState& CompState = State[SortedIdx];
		UNiagaraSystem* System = Component->GetAsset();
		const FNiagaraSystemScalabilitySettings& ScalabilitySettings = System->GetScalabilitySettings();

		FNiagaraScalabilitySystemData& SysData = GetSystemData(SortedIdx);
		if (CompState.bCulled)
		{
			if (CompState.IsDirty())
			{
				Component->SetSystemSignificanceIndex(INDEX_NONE);
			}
		}
		else
		{
			bool bOldCulled = CompState.bCulled;

			WorldMan->SortedSignificanceCull(EffectType, Component, ScalabilitySettings, CompState.Significance, EffectTypeActiveInstances, SysData.InstanceCount, CompState);

			//Inform the component how significant it is so emitters internally can scale based on that information.
			//e.g. expensive emitters can turn off for all but the N most significant systems.
			int32 SignificanceIndex = CompState.bCulled ? INDEX_NONE : SysData.InstanceCount - 1;
			Component->SetSystemSignificanceIndex(SignificanceIndex);

			Context.bHasDirtyState |= CompState.IsDirty();
		}

		//Gather list of cull proxies to create/destroy. Move outside of this loop as in some code paths we can have components register/unregister and invalidate the SortedIdx.
		if (ScalabilitySettings.CullProxyMode != ENiagaraCullProxyMode::None)
		{
			FCullProxyCreateDestroyTask& NewCullProxyTask = CullProxyTasks.AddDefaulted_GetRef();			
			NewCullProxyTask.Component = Component;			
			NewCullProxyTask.bCreate = false;
			if (CompState.bCulled)
			{
				bool bEnableCullProxy = SysData.CullProxyCount < ScalabilitySettings.MaxSystemProxies
										&& CompState.bCulledByVisibility == false;//Don't allow proxies on invisible systems. Keep them all for those culled by distance and budget etc.
				
				if (bEnableCullProxy)
				{
					++SysData.CullProxyCount;
					NewCullProxyTask.bCreate = true;
				}
			}
		}
	}

	//Process any cull proxy creation / destruction here so as to not invalidate the ManagedComponents/SortedIdx as we process.
	for(FCullProxyCreateDestroyTask& CullProxyTask : CullProxyTasks)
	{
		if(IsValid(CullProxyTask.Component))
		{
			if (CullProxyTask.bCreate)
			{
				CullProxyTask.Component->CreateCullProxy(true);
			}
			else
			{
				CullProxyTask.Component->DestroyCullProxy();
			}
		}
	}
}

extern int32 GEnableNiagaraGlobalBudgetCulling;

bool FNiagaraScalabilityManager::ApplyScalabilityState(int32 ComponentIndex, ENiagaraCullReaction CullReaction, bool bNewOnly)
{
	FNiagaraScalabilityState& CompState = State[ComponentIndex];

	if (!CompState.IsDirty())
	{
		return true;
	}

	// If we are only processing new only do not update the state for existing component as it will cause a stall on the game thread
	if (bNewOnly && !CompState.bNewlyRegisteredDirty)
	{
		return true;
	}

	bool bContinueIteration = true;
	if (UNiagaraComponent* Component = ManagedComponents[ComponentIndex])
	{
		//If we're using cull proxies then override all deactivates to be immediate. No need to keep state around that won't be used.
		//TODO: Make this pause instead.
		FNiagaraScalabilitySystemData& SysData = GetSystemData(ComponentIndex);
		if (SysData.bNeedsSignificanceForCulled)
		{
			if (CullReaction == ENiagaraCullReaction::Deactivate)
			{
				CullReaction = ENiagaraCullReaction::DeactivateImmediate;
			}
			else if (CullReaction == ENiagaraCullReaction::DeactivateResume)
			{
				CullReaction = ENiagaraCullReaction::DeactivateImmediateResume;
			}
		}

#if WITH_NIAGARA_DEBUGGER
		//Tell the debugger about our scalability state.
		//Unfortunately cannot have the debugger just read the manager state data as components are removed.
		//To avoid extra copy in future we can have the manager keep another list of recently removed component states which is cleaned up on component destruction.
		Component->DebugCachedScalabilityState = CompState;
#endif

		UNiagaraSystem* System = Component->GetAsset();
		check(System);

		if (CompState.bCulled)
		{
			CompState.Apply();

			bool bKilled = false;
			switch (CullReaction)
			{
			case ENiagaraCullReaction::Deactivate:					Component->DeactivateInternal(false); bKilled = true; break;//We don't increment CompIdx here as this call will remove an entry from ManagedObjects;
			case ENiagaraCullReaction::DeactivateImmediate:			Component->DeactivateImmediateInternal(false); bKilled = true;  break; //We don't increment CompIdx here as this call will remove an entry from ManagedObjects;
			case ENiagaraCullReaction::DeactivateResume:			Component->DeactivateInternal(true); break;
			case ENiagaraCullReaction::DeactivateImmediateResume:	Component->DeactivateImmediateInternal(true); break;
			case ENiagaraCullReaction::PauseResume:					Component->SetPausedInternal(true,true); break;
			};

			if (bKilled)
			{
				bContinueIteration = false;
#if WITH_NIAGARA_DEBUGGER
				WorldMan->GetSystemScalabilityKillCounts(System).Add(Component->DebugCachedScalabilityState);
#endif
			}
		}
		else
		{
			int32 SystemInstanceMax = 0;
			int32 EffectTypeInstanceMax = 0;

			bool bBudgetCullEnabled = GEnableNiagaraGlobalBudgetCulling && FFXBudget::Enabled() && INiagaraModule::UseGlobalFXBudget();
			Component->GetAsset()->GetMaxInstanceCounts(SystemInstanceMax, EffectTypeInstanceMax, bBudgetCullEnabled);

			//We do not allow new activations if it would blow our instance count limits.
			//Though we do allow currently running but inactive components to (!IsComplete()) to reactivate
			const FNiagaraSystemScalabilitySettings& ScalabilitySettings = System->GetScalabilitySettings();
			bool bAllow = GApplyInstanceCountsRigidly == 0;
			bAllow = bAllow || Component->IsComplete() == false;
			bAllow = bAllow || (!ScalabilitySettings.bCullMaxInstanceCount || EffectType->NumInstances < EffectTypeInstanceMax);
			bAllow = bAllow || (!ScalabilitySettings.bCullPerSystemMaxInstanceCount || System->GetActiveInstancesCount() < SystemInstanceMax);

			if(bAllow)
			{
				CompState.Apply();
				if (CullReaction == ENiagaraCullReaction::Deactivate || CullReaction == ENiagaraCullReaction::DeactivateImmediate)
				{
					UE_LOGF(LogNiagara, Error, "Niagara Component is incorrectly still registered with the scalability manager. %d - %ls ", (int32)CullReaction, *Component->GetAsset()->GetFullName());
				}
				else if (CullReaction == ENiagaraCullReaction::PauseResume)
				{
					Component->SetPausedInternal(false, true);
				}
				else
				{
					Component->ActivateInternal(false, true);
				}
			}
		}

		//TODO: Beyond culling by hard limits here we could progressively scale down fx by biasing detail levels they use. Could also introduce some budgeting here like N at lvl 0, M at lvl 1 etc.
		//TODO: Possibly also limiting the rate at which their instances can tick. Ofc system sims still need to run but instances can skip ticks.
	}

	return bContinueIteration;
}

void FNiagaraScalabilityManager::UpdateInternal(FComponentIterationContext& Context)
{
	int32 UpdateCount = 0;
	if (Context.bProcessAllComponents)
	{
		for (int32 ComponentIt = 0; ComponentIt < ManagedComponents.Num();)
		{
			if (EvaluateCullState(Context, ComponentIt, UpdateCount))
			{
				++ComponentIt;
			}
		}
	}
	else
	{
		for (TConstSetBitIterator<> ComponentIt(Context.ComponentRequiresUpdate); UpdateCount < Context.MaxUpdateCount && ComponentIt;)
		{
			Context.ComponentRequiresUpdate[ComponentIt.GetIndex()] = false;

			if (EvaluateCullState(Context, ComponentIt.GetIndex(), UpdateCount))
			{
				++ComponentIt;
			}
		}
	}

	if (Context.bProcessAllComponents || !Context.ComponentRequiresUpdate.Contains(true))
	{
		bool bProcessSignificance = Context.bRequiresGlobalSignificancePass && EffectType->SignificanceHandler;
		if (bProcessSignificance)
		{
			ProcessSignificance(EffectType->SignificanceHandler, Context);
		}

		if (Context.bHasDirtyState)
		{
			const ENiagaraCullReaction CullReaction = EffectType->CullReaction;

			//As we'll be activating and deactivating here, this must be done on the game thread.
			
			//Do one pass to apply in significance order so that any re-activations are done on the most significant items first.
			int32 CompIdx = 0;
			if(bProcessSignificance && GApplyInstanceCountsRigidly != 0)
			{
				while (CompIdx < Context.SignificanceIndices.Num())
				{
					int32 SortedIdx = Context.SignificanceIndices[CompIdx++];
					if(!ApplyScalabilityState(SortedIdx, CullReaction, Context.bNewOnly))
					{
						//If we deactivate a component then we'll have to bail on this pass as the sorted index list is no longer valid. 
						//By this point we should have processed all the activations we want in order anyway.
						break;
					}
				}
			}

			//We have to do another pass if we've not processed everything yet. 
			//Might want to move to having a separate index list for dirty components
			if(CompIdx < ManagedComponents.Num())
			{
				CompIdx = 0;
				while (CompIdx < ManagedComponents.Num())
				{
					if (ApplyScalabilityState(CompIdx, CullReaction, Context.bNewOnly))
					{
						++CompIdx;
					}
				}
			}

			Context.bHasDirtyState = false;
		}

		Context.ComponentRequiresUpdate.Reset();
	}
}

void FNiagaraScalabilityManager::Update(float DeltaSeconds, bool bNewOnly)
{
	if (bRefreshCachedSystemData)
	{	
		bRefreshCachedSystemData = false;

		//Clear and refresh all cached system data.
		SystemDataIndexMap.Reset();
		SystemData.Reset();

		for (int32 CompIdx = 0; CompIdx < ManagedComponents.Num(); ++CompIdx)
		{
			GetSystemData(CompIdx, true);
		}
	}

	bool bShutdown = EffectType == nullptr || FNiagaraWorldManager::GetScalabilityCullingMode() == ENiagaraScalabilityCullingMode::Disabled;
	if (bShutdown)
	{
		while (ManagedComponents.Num())
		{
			ManagedComponents.Last()->UnregisterWithScalabilityManager();
		}

		check(ManagedComponents.Num() == 0);
		check(State.Num() == 0);
		LastUpdateTime = 0.0;
		return;
	}

	float WorstGlobalBudgetUse = FFXBudget::GetWorstAdjustedUsage();

	if (bNewOnly)
	{
		// if we're focused on new instances, but there aren't any, then just exit early
		if (!EffectType->bNewSystemsSinceLastScalabilityUpdate)
		{
			return;
		}

		FComponentIterationContext NewComponentContext;
		NewComponentContext.bNewOnly = true;
		NewComponentContext.bProcessAllComponents = true;
		EffectType->bNewSystemsSinceLastScalabilityUpdate = false;
		NewComponentContext.WorstGlobalBudgetUse = WorstGlobalBudgetUse;

		UpdateInternal(NewComponentContext);
		return;
	}
	else if (EffectType->UpdateFrequency == ENiagaraScalabilityUpdateFrequency::SpawnOnly)
	{
		return;
	}

	const double CurrentTime = WorldMan->GetWorld()->GetTimeSeconds();
	const float TimeSinceUpdate = float(CurrentTime - LastUpdateTime);
	const float UpdatePeriod = GetScalabilityUpdatePeriod(EffectType->UpdateFrequency);

	const bool bResetUpdate = bRefreshOwnerAllowsScalability || EffectType->UpdateFrequency == ENiagaraScalabilityUpdateFrequency::Continuous
		|| ((TimeSinceUpdate >= UpdatePeriod) && !DefaultContext.ComponentRequiresUpdate.Contains(true));

	const int32 ComponentCount = ManagedComponents.Num();

	if (bResetUpdate)
	{
		LastUpdateTime = CurrentTime;

		DefaultContext.bHasDirtyState = false;
		DefaultContext.bNewOnly = false;
		DefaultContext.bRequiresGlobalSignificancePass = false;

		DefaultContext.MaxUpdateCount = GetMaxUpdatesPerFrame(EffectType, ComponentCount, UpdatePeriod, DeltaSeconds);
		DefaultContext.bProcessAllComponents = bRefreshOwnerAllowsScalability || DefaultContext.MaxUpdateCount == ComponentCount;

		if (DefaultContext.bProcessAllComponents)
		{
			DefaultContext.ComponentRequiresUpdate.Reset();
		}
		else
		{
			DefaultContext.ComponentRequiresUpdate.Init(true, ComponentCount);
		}
	}
	// if we're doing a partial update, then define how much we need to process this iteration
	else if (DefaultContext.ComponentRequiresUpdate.Num())
	{
		DefaultContext.MaxUpdateCount = GetMaxUpdatesPerFrame(EffectType, DefaultContext.ComponentRequiresUpdate.CountSetBits(), UpdatePeriod, DeltaSeconds);

		if (DefaultContext.MaxUpdateCount == ComponentCount)
		{
			DefaultContext.bProcessAllComponents = true;
			DefaultContext.ComponentRequiresUpdate.Reset();
		}
	}
	else
	{
		DefaultContext.MaxUpdateCount = 0;
	}

	// early out if we have nothing to process
	if (!DefaultContext.MaxUpdateCount)
	{
		return;
	}

	DefaultContext.WorstGlobalBudgetUse = WorstGlobalBudgetUse;

	UpdateInternal(DefaultContext);
	
	bRefreshOwnerAllowsScalability = false;
}

FNiagaraScalabilitySystemData& FNiagaraScalabilityManager::GetSystemData(int32 ComponentIndex, bool bForceRefresh)
{
	FNiagaraScalabilityState& CompState = State[ComponentIndex];

	if (CompState.SystemDataIndex == INDEX_NONE || bForceRefresh)
	{
		UNiagaraComponent* Component = ManagedComponents[ComponentIndex];
		UNiagaraSystem* System = Component->GetAsset();
		int32* SysDataIndex = SystemDataIndexMap.Find(System);
		if (SysDataIndex == nullptr)
		{
			ensure(SystemData.Num() <= TNumericLimits<decltype(CompState.SystemDataIndex)>::Max());
			CompState.SystemDataIndex = int16(SystemData.Num());
			SystemDataIndexMap.Add(System) = CompState.SystemDataIndex;			

			FNiagaraScalabilitySystemData& NewSystemData = SystemData.AddDefaulted_GetRef();
			NewSystemData.bNeedsSignificanceForActiveOrDirty = System->NeedsSortedSignificanceCull();
			NewSystemData.bNeedsSignificanceForCulled = System->NeedsSortedSignificanceCull() && System->GetScalabilitySettings().CullProxyMode != ENiagaraCullProxyMode::None;
		}
		else
		{
			const int32 SysDataIndexValue = *SysDataIndex;
			ensure(SysDataIndexValue <= TNumericLimits<int16>::Max());
			CompState.SystemDataIndex = int16(SysDataIndexValue);
		}
	}

	return SystemData[CompState.SystemDataIndex];
}

#if WITH_EDITOR
void FNiagaraScalabilityManager::OnSystemPostChange(UNiagaraSystem* System)
{
	InvalidateCachedSystemData();
}
#endif//WITH_EDITOR

void FNiagaraScalabilityManager::InvalidateCachedSystemData()
{
	bRefreshCachedSystemData = true;
}

#if DEBUG_SCALABILITY_STATE

void FNiagaraScalabilityManager::Dump()
{
	FString DetailString;

	struct FSummary
	{
		FSummary()
			: NumCulled(0)
			, NumCulledByDistance(0)
			, NumCulledByInstanceCount(0)
			, NumCulledByVisibility(0)
		{}

		int32 NumCulled;
		int32 NumCulledByDistance;
		int32 NumCulledByInstanceCount;
		int32 NumCulledByVisibility;
	}Summary;

	for (int32 i = 0; i < ManagedComponents.Num(); ++i)
	{
		UNiagaraComponent* Comp = ManagedComponents[i];
		FNiagaraScalabilityState& CompState = State[i];

		FString CulledStr = TEXT("Active:");
		if (CompState.bCulled)
		{
			CulledStr = TEXT("Culled:");
			++Summary.NumCulled;
		}
		if (CompState.bCulledByDistance)
		{
			CulledStr += TEXT("-Distance-");
			++Summary.NumCulledByDistance;
		}
		if (CompState.bCulledByInstanceCount)
		{
			CulledStr += TEXT("-Inst Count-");
			++Summary.NumCulledByInstanceCount;
		}
		if (CompState.bCulledByVisibility)
		{
			CulledStr += TEXT("-Visibility-");
			++Summary.NumCulledByVisibility;
		}

		DetailString += FString::Printf(TEXT("| %s | Sig: %2.4f | %p | %s | %s |\n"), *CulledStr, CompState.Significance, Comp, *Comp->GetAsset()->GetPathName(), *Comp->GetPathName());
	}

	UE_LOGF(LogNiagara, Display, "-------------------------------------------------------------------------------");
	UE_LOGF(LogNiagara, Display, "Effect Type: %ls", *EffectType->GetPathName());
	UE_LOGF(LogNiagara, Display, "-------------------------------------------------------------------------------");
	UE_LOGF(LogNiagara, Display, "| Summary for managed systems of this effect type. Does NOT inclued all possible Niagara FX in scene. |");
	UE_LOGF(LogNiagara, Display, "| Num Managed Components: %d |", ManagedComponents.Num());
	UE_LOGF(LogNiagara, Display, "| Num Active: %d |", ManagedComponents.Num() - Summary.NumCulled);
	UE_LOGF(LogNiagara, Display, "| Num Culled: %d |", Summary.NumCulled);
	UE_LOGF(LogNiagara, Display, "| Num Culled By Distance: %d |", Summary.NumCulledByDistance);
	UE_LOGF(LogNiagara, Display, "| Num Culled By Instance Count: %d |", Summary.NumCulledByInstanceCount);
	UE_LOGF(LogNiagara, Display, "| Num Culled By Visibility: %d |", Summary.NumCulledByVisibility);
	UE_LOGF(LogNiagara, Display, "-------------------------------------------------------------------------------");
	UE_LOGF(LogNiagara, Display, "| Details |");
	UE_LOGF(LogNiagara, Display, "-------------------------------------------------------------------------------\n%ls", *DetailString);
}

#endif

#if WITH_PARTICLE_PERF_CSV_STATS

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Particles);

void FNiagaraScalabilityManager::CSVProfilerUpdate(FCsvProfiler* CSVProfiler)
{
	check(CSVProfiler && CSVProfiler->IsCapturing());
	if (!FParticlePerfStats::GetCSVStatsEnabled())
	{
		return;
	}

	static const FName Total(TEXT("NiagaraCulled/Total"));
	static const FName Distance(TEXT("NiagaraCulled/Distance"));
	static const FName InstCounts(TEXT("NiagaraCulled/InstCounts"));
	static const FName Visibility(TEXT("NiagaraCulled/Visibility"));
	static const FName Budget(TEXT("NiagaraCulled/Budgets"));

	int32 NumCulled = 0;
	int32 NumByDistance = 0;
	int32 NumByInstCounts = 0;
	int32 NumByVisibility = 0;
	int32 NumByBudget = 0;

	for (int32 i = 0; i < ManagedComponents.Num(); ++i)
	{
		UNiagaraComponent* Comp = ManagedComponents[i];
		FNiagaraScalabilityState& CompState = State[i];

		NumCulled += CompState.bCulled ? 1 : 0;
		NumByDistance += CompState.bCulledByDistance ? 1 : 0;
		NumByInstCounts += CompState.bCulledByInstanceCount ? 1 : 0;
		NumByVisibility += CompState.bCulledByVisibility ? 1 : 0;
		NumByBudget += CompState.bCulledByGlobalBudget ? 1 : 0;

		if(CompState.bCulled)
		{
			UNiagaraSystem* System = Comp->GetAsset();
			FParticlePerfStats* Stats = FParticlePerfStatsManager::GetSystemPerfStats(System);
			CSVProfiler->RecordCustomStat(Stats->CSVStat_Culled, CSV_CATEGORY_INDEX(Particles), 1, ECsvCustomStatOp::Accumulate);
		}
	}
	CSVProfiler->RecordCustomStat(Total, CSV_CATEGORY_INDEX(Particles), NumCulled, ECsvCustomStatOp::Accumulate);
	CSVProfiler->RecordCustomStat(Distance, CSV_CATEGORY_INDEX(Particles), NumByDistance, ECsvCustomStatOp::Accumulate);
	CSVProfiler->RecordCustomStat(InstCounts, CSV_CATEGORY_INDEX(Particles), NumByInstCounts, ECsvCustomStatOp::Accumulate);
	CSVProfiler->RecordCustomStat(Visibility, CSV_CATEGORY_INDEX(Particles), NumByVisibility, ECsvCustomStatOp::Accumulate);
	CSVProfiler->RecordCustomStat(Budget, CSV_CATEGORY_INDEX(Particles), NumByBudget, ECsvCustomStatOp::Accumulate);
}
#endif

