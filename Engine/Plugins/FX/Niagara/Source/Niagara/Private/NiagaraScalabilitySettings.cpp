// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScalabilitySettings.h"
#include "NiagaraSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScalabilitySettings)

UE_DEFINE_GAMEPLAY_TAG_COMMENT(TAG_Niagara_ScalabilityProfile_Critical, "Niagara.ScalabilityProfile.Critical", "A critical element for Niagara FX that should have the absolute highest fidelity settings in a given context.")
UE_DEFINE_GAMEPLAY_TAG_COMMENT(TAG_Niagara_ScalabilityProfile_High, "Niagara.ScalabilityProfile.High", "A high importance element for Niagara FX that should favour fidelity over performance in a given context.")
UE_DEFINE_GAMEPLAY_TAG_COMMENT(TAG_Niagara_ScalabilityProfile_Medium, "Niagara.ScalabilityProfile.Medium", "A medium importance element for Niagara FX that should balance performance and fidelity in a given context.")
UE_DEFINE_GAMEPLAY_TAG_COMMENT(TAG_Niagara_ScalabilityProfile_Low, "Niagara.ScalabilityProfile.Low", "A low importance element for Niagara FX that should favour performance over fidelity in a given context.")

namespace NSCVars
{
	static bool GLimitPSOPrecachePriorityBoost = false;
	static FAutoConsoleVariableRef CVarNiagaraLimitPSOPrecachePriorityBoost(
		TEXT("fx.Niagara.PSOPrecache.LimitPriorityBoost"),
		GLimitPSOPrecachePriorityBoost,
		TEXT("Limit Niagara to issuing only high priority PSO precache boosts.\n")
		TEXT("false - Niagara can issue highest priority PSO precache boosts. (default)\n")
		TEXT("true  - Niagara will issue only high priority PSO precache boosts.")
		,
		ECVF_Default
	);
}

#if WITH_EDITOR

void UNiagaraSystemReadinessSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraSystemReadinessSettings, Parent))
	{
		//Check parent chain for loops.
		TSet<TObjectPtr<UNiagaraSystemReadinessSettings>> Seen;
		
		UNiagaraSystemReadinessSettings* Curr = this;
		while(IsValid(Curr))
		{
			if (Seen.Contains(Curr))
			{
				UE_LOGF(LogNiagara, Warning, "Cannot set Parent! Found circular chain of parent settings assets.");
				Parent = nullptr;
				break;
			}
			Seen.Add(Curr);
			Curr = Curr->Parent;
		}
	}
}

#endif 

void UNiagaraSystemReadinessSettings::OnScalabilityUpdate()
{
	for (auto& Pair : Profiles)
	{
		Pair.Value.OnScalabilityUpdate();
	}
}

const FNiagaraReadinessSettingsProfile* UNiagaraSystemReadinessSettings::Find(FGameplayTag Profile)const
{
	if(const FNiagaraReadinessSettingsProfile* Found = Profiles.Find(Profile))
	{
		return Found;
	}

	if (IsValid(Parent))
	{
		// this recursive call would be problematic if users were to set a circular asset chain, but this is checked in UNiagaraSystemReadinessSettings::PostEditChangeProperty
		return Parent->Find(Profile);
	}

	return nullptr;
}

EPSOPrecachePriority FNiagaraSystemPSOPrecacheSettings::GetPriorityBoost()const
{
	if (PreCachePriorityBoost == ENiagaraSystemPSOPrecachingPriorityBoost::Highest) return  NSCVars::GLimitPSOPrecachePriorityBoost ? EPSOPrecachePriority::High : EPSOPrecachePriority::Highest;
	else if (PreCachePriorityBoost == ENiagaraSystemPSOPrecachingPriorityBoost::High) return EPSOPrecachePriority::High;
	else return EPSOPrecachePriority::Medium;
}