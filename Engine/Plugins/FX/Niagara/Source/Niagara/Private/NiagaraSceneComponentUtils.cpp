// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSceneComponentUtils.h"
#include "NiagaraComponent.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSceneComponentSocketUtils.h"
#include "NiagaraActorSceneComponentSocketUtils.h"

#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/StaticMesh.h"
#include "StaticMeshComponentLODInfo.h"

namespace NiagaraSceneComponentUtilsPrivate
{
	UStaticMeshComponent* FindStaticMeshComponent(AActor* Actor, bool bRecurseParents)
	{
		if (AStaticMeshActor* MeshActor = Cast<AStaticMeshActor>(Actor))
		{
			UStaticMeshComponent* Comp = MeshActor->GetStaticMeshComponent();
			if (::IsValid(Comp))
			{
				return Comp;
			}
		}

		// Fall back on any valid component on the actor
		while (Actor)
		{
			for (UActorComponent* ActorComp : Actor->GetComponents())
			{
				UStaticMeshComponent* Comp = Cast<UStaticMeshComponent>(ActorComp);
				if (::IsValid(Comp) && Comp->GetStaticMesh() != nullptr)
				{
					return Comp;
				}
			}

			if (bRecurseParents)
			{
				Actor = Actor->GetParentActor();
			}
			else
			{
				break;
			}
		}

		return nullptr;
	};
}

FNiagaraActorSceneComponentUtils::FNiagaraActorSceneComponentUtils(UNiagaraComponent* OwnerComponent)
	: WeakOwnerComponent(OwnerComponent)
{
}

UObject* FNiagaraActorSceneComponentUtils::GetOwner() const
{
	return WeakOwnerComponent.Get();
}

UClass* FNiagaraActorSceneComponentUtils::FindPropertyInterface(UObject* SourceObject, FName InterfaceFullPath) const
{
	return nullptr;
}

void FNiagaraActorSceneComponentUtils::ResolveStaticMesh(bool bRecurseParents, UObject*& OutComponent, UStaticMesh*& OutStaticMesh) const
{
	USceneComponent* OwnerComponent = WeakOwnerComponent.Get();
	if (OwnerComponent == nullptr)
	{
		return;
	}

	for (USceneComponent* Curr = OwnerComponent; Curr; Curr = Curr->GetAttachParent())
	{
		UStaticMeshComponent* ParentComp = Cast<UStaticMeshComponent>(Curr);
		if (::IsValid(ParentComp))
		{
			OutComponent = ParentComp;
			OutStaticMesh = ParentComp->GetStaticMesh();
			return;
		}
	}

	// Next, try to find one in our outer chain
	UStaticMeshComponent* OuterComp = OwnerComponent->GetTypedOuter<UStaticMeshComponent>();
	if (::IsValid(OuterComp))
	{
		OutComponent = OuterComp;
		OutStaticMesh = OuterComp->GetStaticMesh();
		return;
	}

	if (AActor* Actor = OwnerComponent->GetAttachmentRootActor())
	{
		if (UStaticMeshComponent* StaticMeshComponent = NiagaraSceneComponentUtilsPrivate::FindStaticMeshComponent(Actor, bRecurseParents))
		{
			OutComponent = StaticMeshComponent;
			OutStaticMesh = StaticMeshComponent->GetStaticMesh();
		}
	}
}

void FNiagaraActorSceneComponentUtils::ResolveStaticMesh(UObject* ObjectFrom, bool bRecurseParents, UObject*& OutComponent, UStaticMesh*& OutStaticMesh) const
{
	if (AActor* Actor = Cast<AActor>(ObjectFrom))
	{
		if (UStaticMeshComponent* StaticMeshComponent = NiagaraSceneComponentUtilsPrivate::FindStaticMeshComponent(Actor, bRecurseParents))
		{
			OutComponent = StaticMeshComponent;
			OutStaticMesh = StaticMeshComponent->GetStaticMesh();
		}
	}
	else if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(ObjectFrom))
	{
		OutComponent = StaticMeshComponent;
		OutStaticMesh = StaticMeshComponent->GetStaticMesh();
	}
	else
	{
		OutComponent = nullptr;
		OutStaticMesh = Cast<UStaticMesh>(ObjectFrom);
	}
}

bool FNiagaraActorSceneComponentUtils::GetStaticMeshTransforms(UObject* Component, FTransform& OutComponentTransform, TArray<FTransform>& OutInstanceTransforms) const
{
	USceneComponent* SceneComponent = Cast<USceneComponent>(Component);
	if (!SceneComponent)
	{
		return false;
	}

	OutComponentTransform = SceneComponent->GetComponentToWorld();
	if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(SceneComponent))
	{
		const int32 NumInstances = ISMComponent->PerInstanceSMData.Num();
		OutInstanceTransforms.SetNum(NumInstances);
		for (int32 i = 0; i < NumInstances; ++i)
		{
			ISMComponent->GetInstanceTransform(i, OutInstanceTransforms[i], true);
		}
	}
	else
	{
		OutInstanceTransforms.Empty();
	}

	return true;
}

FColorVertexBuffer* FNiagaraActorSceneComponentUtils::GetStaticMeshOverrideColors(UObject* Component, int32 LODIndex) const
{
	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
	{
		if (StaticMeshComponent->LODData.IsValidIndex(LODIndex))
		{
			return StaticMeshComponent->LODData[LODIndex].OverrideVertexColors;
		}
	}

	return nullptr;
}

FPrimitiveComponentId FNiagaraActorSceneComponentUtils::GetPrimitiveSceneId(UObject* Component) const
{
	UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);
	return PrimitiveComponent ? PrimitiveComponent->GetPrimitiveSceneId() : FPrimitiveComponentId();
}

FVector FNiagaraActorSceneComponentUtils::GetPhysicsLinearVelocity(UObject* Component) const
{
	UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);
	return PrimitiveComponent ? PrimitiveComponent->GetPhysicsLinearVelocity() : FVector::ZeroVector;
}

FNiagaraSceneComponentSocketUtils* FNiagaraActorSceneComponentUtils::CreateSocketUtils() const
{
	return new FNiagaraActorSceneComponentSocketUtils(WeakOwnerComponent);
}
