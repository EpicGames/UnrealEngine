// Copyright Epic Games, Inc. All Rights Reserved.


#include "NiagaraScratchPadContainer.h"
#include "NiagaraScript.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScratchPadContainer)

void UNiagaraScratchPadContainer::PostLoad()
{
	CheckConsistency();

	Super::PostLoad();
}

void UNiagaraScratchPadContainer::CheckConsistency()
{
#if WITH_EDITORONLY_DATA
	for (const TObjectPtr<UNiagaraScript>& Script : Scripts)
	{
		if (Script && Script->GetOuter() != this)
		{
			// Ensure the object has finished loading before renaming since we are changing the outer which will invalidate the linker
			Script->ConditionalPostLoad();
			Script->Rename(*Script->GetName(), this, REN_NonTransactional);
		}
	}
#endif
}

void UNiagaraScratchPadContainer::SetScripts(const TArray<TObjectPtr<UNiagaraScript>>& InScripts)
{
#if WITH_EDITORONLY_DATA
	Scripts = InScripts;
	CheckConsistency();
#endif
}

void UNiagaraScratchPadContainer::AppendScripts(const TArray<TObjectPtr<UNiagaraScript>>& InScripts)
{
#if WITH_EDITORONLY_DATA
	Scripts.Append(InScripts);
	CheckConsistency();
#endif
}

void UNiagaraScratchPadContainer::AppendScripts(TObjectPtr<UNiagaraScratchPadContainer> InScripts)
{
#if WITH_EDITORONLY_DATA
	if (InScripts)
	{
		for (const TObjectPtr<UNiagaraScript>& Script : InScripts->Scripts)
		{
			if (Script)
			{
				FName UniqueName = MakeUniqueObjectName(this, Script->GetClass(), Script->GetFName());
				Script->Rename(*UniqueName.ToString(), this, REN_NonTransactional);
				Scripts.Add(Script);
			}
		}
		InScripts->Scripts.Empty();
	}
#endif
}
int32 UNiagaraScratchPadContainer::FindIndexForScript(UNiagaraScript* InScript) const
{
#if WITH_EDITORONLY_DATA
	for (int32 i = 0; i < Scripts.Num(); i++)
	{
		if (Scripts[i] == InScript)
		{
			return i;
		}
	}
#endif
	return INDEX_NONE;
}

UNiagaraScript* UNiagaraScratchPadContainer::FindScriptAtIndex(int32 Index) const
{
#if WITH_EDITORONLY_DATA
	if (Scripts.IsValidIndex(Index))
	{
		return Scripts[Index];
	}
#endif
	return nullptr;
}


