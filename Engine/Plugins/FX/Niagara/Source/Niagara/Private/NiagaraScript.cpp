// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScript.h"

#include "Algo/RemoveIf.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DataInterface/NiagaraDataInterfaceStaticMesh.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/Compression.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/FileHelper.h"
#include "Misc/SecureHash.h"
#include "Modules/ModuleManager.h"
#include "NiagaraAnalytics.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraDataInterfaceSkeletalMesh.h"
#include "NiagaraEmitter.h"
#include "NiagaraModule.h"
#include "NiagaraScriptSourceBase.h"
#include "NiagaraSettings.h"
#include "NiagaraShader.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraStackSection.h"
#include "NiagaraSystem.h"
#include "ProfilingDebugging/CookStats.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Stats/Stats.h"
#include "String/ParseTokens.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/RenderingObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectThreadContext.h"
#include "VectorVM.h"
#include "ShaderCompilerCore.h"

#if WITH_EDITOR
	#include "DerivedDataCacheInterface.h"
	#include "DerivedDataCacheKey.h"
	#include "INiagaraEditorOnlyDataUtlities.h"
	#include "Interfaces/ITargetPlatform.h"
	#include "Internationalization/Regex.h"
	#include "ShaderCodeLibrary.h"

	// This is a version string that mimics the old versioning scheme. In case of merge conflicts with DDC versions,
	// you MUST generate a new GUID and set this new version. If you want to bump this version, generate a new guid
	// using VS->Tools->Create GUID
	#define NIAGARASCRIPT_DERIVEDDATA_VER		TEXT("AB7397ACFEFD46158A87743735E3C773")
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScript)

#if ENABLE_COOK_STATS
namespace NiagaraScriptCookStats
{
	FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("NiagaraScript.Usage"), TEXT(""));
	});
}
#endif

int32 GNiagaraDumpKeyGen = 0;
static FAutoConsoleVariableRef CVarNiagaraDumpKeyGen(
	TEXT("fx.DumpGraphKeyGen"),
	GNiagaraDumpKeyGen,
	TEXT("If > 0 the key generation will be dumped to the log. \n"),
	ECVF_Default
);

int32 GNiagaraForceSafeScriptAttributeTrim = 0;
static FAutoConsoleVariableRef CVarNiagaraForceSsafeScriptAttributeTrim(
	TEXT("fx.ForceSafeScriptAttributeTrim"),
	GNiagaraForceSafeScriptAttributeTrim,
	TEXT("If > 0 attribute trimming will use a less aggressive algorithm for removing script attributes. \n"),
	ECVF_Default
);

bool GNiagaraCompressScriptByteCode = false;
static FAutoConsoleVariableRef CVarNiagaraCompressScriptByteCode(
	TEXT("fx.Niagara.CompressScriptByteCode"),
	GNiagaraCompressScriptByteCode,
	TEXT("Should we compress script bytecode to save memory. Will be uncompressed on demand."),
	ECVF_Default
);

int32 UNiagaraScript::LogCompileStaticVars =0;
static FAutoConsoleVariableRef CVarLogCompileStaticVars(
	TEXT("fx.LogCompileStaticVars"),
	UNiagaraScript::LogCompileStaticVars,
	TEXT("If > 0 all compile id generation dealing with static variables will be logged.  \n"),
	ECVF_Default
);

static int32 GbNiagaraEventSpawnsUpdateInitialAttributeValues = 1;
static FAutoConsoleVariableRef CVarNiagaraEventSpawnsUpdateInitialAttributeValues(
	TEXT("fx.Niagara.EventSpawnsUpdateAttributeInitialValues"),
	GbNiagaraEventSpawnsUpdateInitialAttributeValues,
	TEXT("If > 0 Niagara Event Spawn Scripts will update the Initial.* values for particle attributes. \n"),
	ECVF_Default
);

static int32 GNiagaraCompileHashAllDataInterfaces = 1;
static FAutoConsoleVariableRef CVarNiagaraCompileHashAllDataInterfaces(
	TEXT("fx.Niagara.CompileHashAllDataInterfaces"),
	GNiagaraCompileHashAllDataInterfaces,
	TEXT("Forces Niagara to include all data interfaces as part of compile hash generation.\n")
	TEXT("0 = Disabled.\n")
	TEXT("1 = Enabled, but hash once, better for perf.\n")
	TEXT("2 = Enabled, generated hash always good for iteration on Niagara work."),
	ECVF_Default
);

static int32 GNiagaraStripByteCodeOverride = 0;
static FAutoConsoleVariableRef CVarNiagaraStripByteCodeOverride(
	TEXT("fx.Niagara.StripByteCodeOverride"),
	GNiagaraStripByteCodeOverride,
	TEXT("Overrides project settings to strip byte code from the script on load..\n")
	TEXT("-1 = Enabled, strips Experimental VM.\n")
	TEXT("0 = Disabled.\n")
	TEXT("1 = Enabled, strips original VM.\n"),
	ECVF_Default
);

static bool GNiagaraGpuScriptsCompiledBySystem = true;
static FAutoConsoleVariableRef CVarNiagaraGpuScriptsCompiledBySystem(
	TEXT("fx.Niagara.GpuScriptsCompiledBySystem"),
	GNiagaraGpuScriptsCompiledBySystem,
	TEXT("If true GPU shaders for Niagara scripts will be compiled along with the NiagaraSystem (when using AsyncTask compilation mode). \n"),
	ECVF_Default
);

static bool GNiagaraCompileGpuSupportsMultiPlatformCook = false;
static FAutoConsoleVariableRef CVarSystemGpuSupportsMultiPlatformCook(
	TEXT("fx.Niagara.SystemGpuSupportsMultiPlatformCook"),
	GNiagaraCompileGpuSupportsMultiPlatformCook,
	TEXT("If false GPU shaders for Niagara scripts will not be compiled by the system when doing a multiplatform cook. \n"),
	ECVF_Default
);

namespace NiagaraScriptInternal
{
#if WITH_EDITORONLY_DATA
	TOptional<ERHIFeatureLevel::Type> PreviewFeatureLevel;

	static FNiagaraCompileHash& CompileHashAllDataInterfaces()
	{
		static TOptional<FNiagaraCompileHash> AllDataInterfacesHash;
		if (AllDataInterfacesHash.IsSet() == false || GNiagaraCompileHashAllDataInterfaces > 1)
		{
			UE_CALL_ONCE(
				[]()
				{
					FSHA1 HashState;

					// only try to access the shader source if we are allowed to compile shaders
					if (AllowShaderCompiling())
					{
						TArray<UClass*> DataInterfaceClasses;
						for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
						{
							UClass* Class = *ClassIterator;
							if (Class == nullptr ||
								Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists) ||
								!ClassIterator->IsChildOf(UNiagaraDataInterface::StaticClass()))
							{
								continue;
							}
							DataInterfaceClasses.Add(Class);
						}
						Algo::Sort(
							DataInterfaceClasses,
							[](UClass* Lhs, UClass* Rhs)
							{
								return Lhs->GetName() < Rhs->GetName();
							}
						);

						FNiagaraCompileHashVisitor Visitor(HashState);
						for (UClass* DIClass : DataInterfaceClasses)
						{
							UNiagaraDataInterface* CDODataInterface = DIClass->GetDefaultObject<UNiagaraDataInterface>();
							CDODataInterface->AppendCompileHash(&Visitor);
						}
					}
					else
					{
						const FGuid DISourcePlaceholder = FGuid(0x17B53667, 0x0A08411A, 0xBDD134D5, 0x039F91C2);
						HashState.Update((const uint8*) &DISourcePlaceholder, sizeof(DISourcePlaceholder));
					}

					HashState.Final();

					TArray<uint8> DataHash;
					DataHash.AddUninitialized(FSHA1::DigestSize);
					HashState.GetHash(DataHash.GetData());

					AllDataInterfacesHash.Emplace(DataHash);
				}
			);
		}
		return AllDataInterfacesHash.GetValue();
	}
#endif

}

FNiagaraScriptDebuggerInfo::FNiagaraScriptDebuggerInfo() : bWaitForGPU(false), Usage(ENiagaraScriptUsage::Function), FrameLastWriteId(-1), bWritten(false)
{
}

FNiagaraScriptDebuggerInfo::FNiagaraScriptDebuggerInfo(FName InName, ENiagaraScriptUsage InUsage, const FGuid& InUsageId) : HandleName(InName), Usage(InUsage), UsageId(InUsageId), FrameLastWriteId(-1), bWritten(false)
{
	if (InUsage == ENiagaraScriptUsage::ParticleGPUComputeScript)
	{
		bWaitForGPU = true;
	}
	else
	{
		bWaitForGPU = false;
	}
}

#if WITH_EDITORONLY_DATA
FNiagaraScriptHashCollector::FNiagaraScriptHashCollector(bool bCollectHashSources)
	: bCollectSources(bCollectHashSources)
{
}

void FNiagaraScriptHashCollector::AddHash(const FNiagaraCompileHash& CompileHash, FStringView CompileHashSource)
{
	ReferencedHashes.AddUnique(CompileHash);
	if (bCollectSources)
	{
		ReferencedHashSources.FindOrAdd(CompileHash).Emplace(CompileHashSource);
	}
}

FString FNiagaraScriptHashCollector::BuildCompileHashSourceString(const FNiagaraCompileHash& CompileHash) const
{
	FString SourceString;

	for (const FString& HashSource : ReferencedHashSources.FindRef(CompileHash))
	{
		if (!SourceString.IsEmpty())
		{
			SourceString.Append(TEXT(", "));
		}

		SourceString.Append(HashSource);
	}

	return SourceString;
}
#endif

UNiagaraScriptSourceBase::UNiagaraScriptSourceBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


bool FNiagaraVMExecutableByteCode::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_ArrayProperty)
	{
		Slot << Data;
		return true;
	}

	return false;
}

void FNiagaraVMExecutableByteCode::SetData(const TArray<uint8>& InData)
{
	Data = InData;
	UncompressedSize = INDEX_NONE;
}

void FNiagaraVMExecutableByteCode::SetData(TArray<uint8>&& InData)
{
	Data = MoveTemp(InData);
	UncompressedSize = INDEX_NONE;
}

bool FNiagaraVMExecutableByteCode::HasByteCode() const
{
	return Data.Num() > 0;
}

bool FNiagaraVMExecutableByteCode::IsCompressed() const
{
	return HasByteCode() && UncompressedSize != INDEX_NONE;
}

bool FNiagaraVMExecutableByteCode::Compress()
{
	if (!IsCompressed())
	{
		int32 OriginalSize = Data.Num();

		// It is possible for compression to actually increase the size of the data, so we over allocate here to handle that.
		int32 CompressedSize = OriginalSize * 4 / 3;

		TArray<uint8> CompressedData;
		CompressedData.SetNumUninitialized(CompressedSize);

		if (FCompression::CompressMemory(NAME_Zlib, CompressedData.GetData(), CompressedSize, Data.GetData(), Data.Num(), COMPRESS_BiasMemory))
		{
			// In the case that compressing it actually increases the size, we leave it uncompressed
			if (CompressedSize < OriginalSize)
			{
				CompressedData.SetNum(CompressedSize);
				Data = MoveTemp(CompressedData);
				Data.Shrink();
				UncompressedSize = OriginalSize;
			}
		}
	}

	return true;
}

bool FNiagaraVMExecutableByteCode::Uncompress()
{
	if (IsCompressed())
	{
		TArray<uint8> UncompressedData;
		UncompressedData.SetNumUninitialized(UncompressedSize);

		if (FCompression::UncompressMemory(NAME_Zlib, UncompressedData.GetData(), UncompressedSize, Data.GetData(), Data.Num()))
		{
			Data = MoveTemp(UncompressedData);
			UncompressedSize = INDEX_NONE;
		}
	}
	return true;
}

void FNiagaraVMExecutableByteCode::Reset()
{
	Data.Empty();
	UncompressedSize = INDEX_NONE;
}

FNiagaraVMExecutableData::FNiagaraVMExecutableData()
	: NumTempRegisters(0)
	, NumUserPtrs(0)
#if WITH_EDITORONLY_DATA
	, LastOpCount(0)
#endif
	, LastCompileStatus(ENiagaraScriptCompileStatus::NCS_Unknown)
#if WITH_EDITORONLY_DATA
	, bReadsAttributeData(false)
#endif
	, bReadsSignificanceIndex(false)
	, bNeedsGPUContextInit(false)
{
}

bool FNiagaraVMExecutableData::IsValid() const
{
	return LastCompileStatus != ENiagaraScriptCompileStatus::NCS_Unknown;
}

void FNiagaraVMExecutableData::Reset()
{
	*this = FNiagaraVMExecutableData();
}

void FNiagaraVMExecutableData::SerializeData(FArchive& Ar, bool bDDCData)
{
	UScriptStruct* FNiagaraVMExecutableDataType = FNiagaraVMExecutableData::StaticStruct();
	FNiagaraVMExecutableDataType->SerializeTaggedProperties(Ar, (uint8*)this, FNiagaraVMExecutableDataType, nullptr);
}

#if WITH_EDITORONLY_DATA
void FNiagaraVMExecutableData::BakeScriptLiterals(TArray<uint8>& OutLiterals) const
{
	OutLiterals.Empty();

	const auto& Variables = InternalParameters.Parameters;
	const int32 VariableCount = Variables.Num();

	int32 TotalSize = 0;
	for (int32 Index = 0; Index < VariableCount; ++Index)
	{
		TotalSize += Variables[Index].GetAllocatedSizeInBytes();
	}

	OutLiterals.AddZeroed(TotalSize);

	uint8* LiteralData = OutLiterals.GetData();

	for (int32 Index = 0; Index < VariableCount; ++Index)
	{
		const FNiagaraVariable& Variable = Variables[Index];
		const int32 VariableSize = Variable.GetAllocatedSizeInBytes();

		FMemory::Memcpy(LiteralData, Variable.GetData(), VariableSize);
		LiteralData += VariableSize;
	}
}

FVersionedNiagaraScriptData::FVersionedNiagaraScriptData()
	: ModuleUsageBitmask( (1 << (int32)ENiagaraScriptUsage::ParticleSpawnScript) | (1 << (int32)ENiagaraScriptUsage::ParticleSpawnScriptInterpolated) | (1 << (int32)ENiagaraScriptUsage::ParticleUpdateScript) | (1 << (int32)ENiagaraScriptUsage::ParticleEventScript) | (1 << (int32)ENiagaraScriptUsage::ParticleSimulationStageScript))
	, bDeprecated(false)
	, DeprecationRecommendation(nullptr)
	, bExperimental(false)
	, LibraryVisibility(ENiagaraScriptLibraryVisibility::Unexposed)
	, NumericOutputTypeSelectionMode(ENiagaraNumericOutputTypeSelectionMode::Largest)
	, bCanBeUsedForTypeConversions(false)
{
}
#endif

bool FNiagaraVMExecutableData::HasByteCode() const
{
	return !ExperimentalContextData.IsEmpty();
}

UNiagaraScript::UNiagaraScript()
{
}

#if WITH_EDITORONLY_DATA
FVersionedNiagaraScriptData* UNiagaraScript::GetLatestScriptData()
{
	return const_cast<FVersionedNiagaraScriptData*>(const_cast<const UNiagaraScript*>(this)->GetLatestScriptData());
}

const FVersionedNiagaraScriptData* UNiagaraScript::GetLatestScriptData() const
{
	if (VersionData.Num() == 0)
	{
		return nullptr;
	}
	if (!bVersioningEnabled)
	{
		return &VersionData[0];
	}
	const FVersionedNiagaraScriptData* VersionedNiagaraScriptData = GetScriptData(ExposedVersion);
	ensureMsgf(VersionedNiagaraScriptData, TEXT("Invalid exposed version for Niagara script %s, asset might be corrupted!"), *this->GetFriendlyName());
	return VersionedNiagaraScriptData;
}

FVersionedNiagaraScriptData* UNiagaraScript::GetScriptData(const FGuid& VersionGuid)
{
	return const_cast<FVersionedNiagaraScriptData*>(const_cast<const UNiagaraScript*>(this)->GetScriptData(VersionGuid));
}

const FVersionedNiagaraScriptData* UNiagaraScript::GetScriptData(const FGuid& VersionGuid) const
{
	if (VersionData.Num() == 0)
	{
		return nullptr;
	}
	
	// check if we even need to support different versions
	if (!bVersioningEnabled)
    {
    	return &VersionData[0];
    }

	if (!VersionGuid.IsValid())
	{
		for (const FVersionedNiagaraScriptData& Data : VersionData)
		{
			if (Data.Version.VersionGuid == ExposedVersion)
			{
				return &Data;
			}
		}
		ensureMsgf(false, TEXT("Invalid exposed version for Niagara script %s, asset might be corrupted!"), *this->GetFriendlyName());
		return nullptr;
	}
	for (const FVersionedNiagaraScriptData& Data : VersionData)
	{
		if (Data.Version.VersionGuid == VersionGuid)
		{
			return &Data;
		}
	}
	return nullptr;
}

TArray<FNiagaraAssetVersion> UNiagaraScript::GetAllAvailableVersions() const
{
	TArray<FNiagaraAssetVersion> Versions;
	for (const FVersionedNiagaraScriptData& Data : VersionData)
	{
		Versions.Add(Data.Version);
	}
	return Versions;
}

FNiagaraAssetVersion UNiagaraScript::GetExposedVersion() const
{
	const FVersionedNiagaraScriptData* ScriptData = GetLatestScriptData();
	return ScriptData ? ScriptData->Version : FNiagaraAssetVersion();
}

FNiagaraAssetVersion const* UNiagaraScript::FindVersionData(const FGuid& VersionGuid) const
{
	for (const FVersionedNiagaraScriptData& Data : VersionData)
	{
		if (Data.Version.VersionGuid == VersionGuid)
		{
			return &Data.Version;
		}
	}
	return nullptr;
}

FGuid UNiagaraScript::AddNewVersion(int32 MajorVersion, int32 MinorVersion)
{
	// check preconditions
	check(MajorVersion >= 1);
	check(MajorVersion != 1 || MinorVersion != 0);
	Modify();

	FVersionedNiagaraScriptData NewVersionData;
	for (int i = VersionData.Num() - 1; i >= 0; i--)
	{
		FVersionedNiagaraScriptData& Data = VersionData[i];
		check(Data.Version.MajorVersion != MajorVersion || Data.Version.MinorVersion != MinorVersion); // the version should not already exist

		if (Data.Version.MajorVersion < MajorVersion || (Data.Version.MajorVersion == MajorVersion && Data.Version.MinorVersion < MinorVersion))
		{
			// copy the data
			NewVersionData = Data;

			FObjectDuplicationParameters ObjParameters(NewVersionData.Source, this);
			ObjParameters.DestClass = NewVersionData.Source->GetClass();
			NewVersionData.Source = Cast<UNiagaraScriptSourceBase>(StaticDuplicateObjectEx(ObjParameters));
			break;
		}
	}

	NewVersionData.VersionChangeDescription = FText();
	NewVersionData.Version = { MajorVersion, MinorVersion, FGuid::NewGuid() };

	VersionData.Add(NewVersionData);
	VersionData.Sort([](const FVersionedNiagaraScriptData& A, const FVersionedNiagaraScriptData& B) { return A.Version < B.Version; });

	return NewVersionData.Version.VersionGuid;
}

void UNiagaraScript::DeleteVersion(const FGuid& VersionGuid)
{
	check(VersionGuid != ExposedVersion);

	for (int i = 0; i < VersionData.Num(); i++)
	{
		FNiagaraAssetVersion& AssetVersion = VersionData[i].Version;
		if (AssetVersion.VersionGuid == VersionGuid)
		{
			check(AssetVersion.MajorVersion != 1 || AssetVersion.MinorVersion != 0);
			Modify();
			VersionData.RemoveAt(i);
			return;
		}
	}
}

void UNiagaraScript::ExposeVersion(const FGuid& VersionGuid)
{
	// check if the requested version exists in the data store
	for (FVersionedNiagaraScriptData& Data : VersionData)
	{
		if (Data.Version.VersionGuid == VersionGuid)
		{
			Modify();
			ExposedVersion = VersionGuid;
			Data.Version.bIsVisibleInVersionSelector = true;
			return;
		}
	}
}

FString NiagaraScriptUsageToString(const ENiagaraScriptUsage& Usage)
{
	switch (Usage)
	{
	case ENiagaraScriptUsage::Module:
		return TEXT("Module");
	case ENiagaraScriptUsage::Function:
		return TEXT("Function");
	case ENiagaraScriptUsage::DynamicInput:
		return TEXT("DynamicInput");
	default:
		return TEXT("Other");
	}
}

void UNiagaraScript::EnableVersioning()
{
	if (bVersioningEnabled)
	{
		return;
	}
	Modify();
	ensure(VersionData.Num() == 1);
	bVersioningEnabled = true;	
	ExposedVersion = VersionData[0].Version.VersionGuid;

	NiagaraAnalytics::RecordEvent(TEXT("Versioning.ScriptEnabled"), TEXT("Usage"), NiagaraScriptUsageToString(Usage));
}

void UNiagaraScript::DisableVersioning(const FGuid& VersionGuidToUse)
{
	CheckVersionDataAvailable();
	bVersioningEnabled = false;

	FVersionedNiagaraScriptData DataToUse = VersionData[0];
	if (VersionGuidToUse.IsValid())
	{
		for (const FVersionedNiagaraScriptData& Data : VersionData)
		{
			if (Data.Version.VersionGuid == VersionGuidToUse)
			{
				DataToUse = Data;
				break;
			}
		}
	}
	DataToUse.Version = FNiagaraAssetVersion(); // reset and create new guid
	VersionData.Empty();
	VersionData.Add(DataToUse);
}

struct FNiagaraScriptVersionDataAccessor final : FNiagaraVersionDataAccessor
{
	virtual ~FNiagaraScriptVersionDataAccessor() override = default;
	explicit FNiagaraScriptVersionDataAccessor(FVersionedNiagaraScriptData* InScriptData) : ScriptData(InScriptData) {}

	virtual FNiagaraAssetVersion& GetObjectVersion() override { return ScriptData->Version; }
	virtual FText& GetVersionChangeDescription() override { return ScriptData->VersionChangeDescription; }
	virtual bool& IsDeprecated() override { return ScriptData->bDeprecated; }
	virtual FText& GetDeprecationMessage() override { return ScriptData->DeprecationMessage; }
	virtual ENiagaraPythonUpdateScriptReference& GetUpdateScriptExecutionType() override { return ScriptData->UpdateScriptExecution; }
	virtual FString& GetPythonUpdateScript() override { return ScriptData->PythonUpdateScript; }
	virtual FFilePath& GetScriptAsset() override { return ScriptData->ScriptAsset; }
	virtual TSubclassOf<UNiagaraVersionUpgradeBase>& GetUpdateUtilityClass() override { return ScriptData->UpdateUtilityClass; }
	
	FVersionedNiagaraScriptData* ScriptData;
};

TSharedPtr<FNiagaraVersionDataAccessor> UNiagaraScript::GetVersionDataAccessor(const FGuid& Version)
{
	if (FVersionedNiagaraScriptData* ScriptData = GetScriptData(Version))
	{
		return MakeShared<FNiagaraScriptVersionDataAccessor>(ScriptData);
	}
	return TSharedPtr<FNiagaraVersionDataAccessor>();
}

void UNiagaraScript::CheckVersionDataAvailable()
{
	if (VersionData.Num() > 0) {
		return;
	}

	// copy over existing data of assets that were created pre-versioning
	FVersionedNiagaraScriptData& Data = VersionData.AddDefaulted_GetRef();
	Data.Source = Source_DEPRECATED;
	Data.Keywords = Keywords_DEPRECATED;
	Data.Category = Category_DEPRECATED;
	Data.Description = Description_DEPRECATED;
	Data.bDeprecated = bDeprecated_DEPRECATED;
	Data.NoteMessage = NoteMessage_DEPRECATED;
	Data.bExperimental = bExperimental_DEPRECATED;
	Data.ScriptMetaData = ScriptMetaData_DEPRECATED;
	Data.LibraryVisibility = LibraryVisibility_DEPRECATED;
	Data.ConversionUtility = ConversionUtility_DEPRECATED;
	Data.ModuleUsageBitmask = ModuleUsageBitmask_DEPRECATED;
	Data.DeprecationMessage = DeprecationMessage_DEPRECATED;
	Data.ExperimentalMessage = ExperimentalMessage_DEPRECATED;
	Data.CollapsedViewFormat = CollapsedViewFormat_DEPRECATED;
	Data.ProvidedDependencies = ProvidedDependencies_DEPRECATED;
	Data.RequiredDependencies = RequiredDependencies_DEPRECATED;
	Data.DeprecationRecommendation = DeprecationRecommendation_DEPRECATED;
	Data.NumericOutputTypeSelectionMode = NumericOutputTypeSelectionMode_DEPRECATED;

	// create a stable initial version guid for our versioned data based on the UNiagaraScript
	Data.Version.VersionGuid = FNiagaraAssetVersion::CreateStableVersionGuid(this);

	ExposedVersion = Data.Version.VersionGuid;
}

UNiagaraScript* UNiagaraScript::CreateCompilationCopy()
{
	UNiagaraScript* Result = NewObject<UNiagaraScript>();

	// create a shallow copy
	for (TFieldIterator<FProperty> PropertyIt(GetClass(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		const uint8* SourceAddr = Property->ContainerPtrToValuePtr<uint8>(this);
		uint8* DestinationAddr = Property->ContainerPtrToValuePtr<uint8>(Result);

		Property->CopyCompleteValue(DestinationAddr, SourceAddr);
	}
	
	return Result;
}

#endif

FVersionedNiagaraEmitter UNiagaraScript::GetOuterEmitter() const
{
	UNiagaraEmitter* Emitter = GetTypedOuter<UNiagaraEmitter>();
	if (Emitter == nullptr)
	{
		return FVersionedNiagaraEmitter();
	}
	for (FNiagaraAssetVersion& Version : Emitter->GetAllAvailableVersions())
	{
		FVersionedNiagaraEmitterData* EmitterData = Emitter->GetEmitterData(Version.VersionGuid);
		TArray<UNiagaraScript*> OutScripts;
		EmitterData->GetScripts(OutScripts, false);
		if (OutScripts.Contains(this))
		{
			return FVersionedNiagaraEmitter(Emitter, Version.VersionGuid);
		}
	}
	return FVersionedNiagaraEmitter();
}

bool FNiagaraVMExecutableDataId::IsValid() const
{
	return CompilerVersionID.IsValid();
}

void FNiagaraVMExecutableDataId::Invalidate()
{
	*this = FNiagaraVMExecutableDataId();
}

#if WITH_EDITORONLY_DATA

TArray<FString> FNiagaraVMExecutableDataId::GetAdditionalVariableStrings() const
{
	TArray<FString> Vars;
	for (const FNiagaraVariableBase& Var : AdditionalVariables)
	{
		Vars.Emplace(Var.GetName().ToString() + TEXT(" ") + Var.GetType().GetName());
	}

	return Vars;
}

#endif

bool FNiagaraVMExecutableDataId::HasInterpolatedParameters() const
{
	return InterpolatedSpawnMode != ENiagaraInterpolatedSpawnMode::NoInterpolation;
}

/**
* Tests this set against another for equality, disregarding override settings.
*
* @param ReferenceSet	The set to compare against
* @return				true if the sets are equal
*/
bool FNiagaraVMExecutableDataId::operator==(const FNiagaraVMExecutableDataId& ReferenceSet) const
{
	if (CompilerVersionID != ReferenceSet.CompilerVersionID ||
#if WITH_EDITORONLY_DATA
		ScriptUsageTypeID != ReferenceSet.ScriptUsageTypeID ||
		ScriptUsageType != ReferenceSet.ScriptUsageType ||
		BaseScriptCompileHash != ReferenceSet.BaseScriptCompileHash ||
		bDisableDebugSwitches != ReferenceSet.bDisableDebugSwitches ||
		bRequiresPersistentIDs != ReferenceSet.bRequiresPersistentIDs ||
		bUsesRapidIterationParams != ReferenceSet.bUsesRapidIterationParams ||
		ScriptVersionID != ReferenceSet.ScriptVersionID ||
#endif
		InterpolatedSpawnMode != ReferenceSet.InterpolatedSpawnMode)
	{
		return false;
	}

#if WITH_EDITORONLY_DATA
	if (ReferencedCompileHashes.Num() != ReferenceSet.ReferencedCompileHashes.Num())
	{
		return false;
	}

	for (int32 ReferencedHashIndex = 0; ReferencedHashIndex < ReferencedCompileHashes.Num(); ReferencedHashIndex++)
	{
		if (ReferencedCompileHashes[ReferencedHashIndex] != ReferenceSet.ReferencedCompileHashes[ReferencedHashIndex])
		{
			return false;
		}
	}

	if (AdditionalDefines.Num() != ReferenceSet.AdditionalDefines.Num())
	{
		return false;
	}


	for (int32 Idx = 0; Idx < ReferenceSet.AdditionalDefines.Num(); Idx++)
	{
		const FString& ReferenceStr = ReferenceSet.AdditionalDefines[Idx];

		if (AdditionalDefines[Idx] != ReferenceStr)
		{
			return false;
		}
	}

	if (AdditionalVariables.Num() != ReferenceSet.AdditionalVariables.Num())
	{
		return false;
	}

	for (int32 Idx = 0; Idx < ReferenceSet.AdditionalVariables.Num(); Idx++)
	{
		const FNiagaraVariableBase& ReferenceVar = ReferenceSet.AdditionalVariables[Idx];

		if (AdditionalVariables[Idx] != ReferenceVar)
		{
			return false;
		}
	}
#endif


	return true;
}

#if WITH_EDITORONLY_DATA
void FNiagaraVMExecutableDataId::AppendKeyString(FString& KeyString, const FString& Delimiter, bool bAppendObjectForDebugging, const FNiagaraScriptHashCollector* HashCollector) const
{
	KeyString += FString::Printf(TEXT("%d%s"), (int32)ScriptUsageType, *Delimiter);
	KeyString += ScriptUsageTypeID.ToString();
	if (bAppendObjectForDebugging)
	{
		KeyString += TEXT(" [ScriptUsageType]");
	}
	KeyString += Delimiter;

	KeyString += CompilerVersionID.ToString();
	if (bAppendObjectForDebugging)
	{
		KeyString += TEXT(" [CompilerVersionID]");
	}
	KeyString += Delimiter;

	KeyString += BaseScriptCompileHash.ToString();
	if (bAppendObjectForDebugging)
	{
		KeyString += TEXT(" [BaseScriptCompileHash]");
	}
	KeyString += Delimiter;

	if (bUsesRapidIterationParams)
	{
		KeyString += TEXT("USESRI") + Delimiter;
	}
	else
	{
		KeyString += TEXT("NORI") + Delimiter;
	}

	if (bDisableDebugSwitches)
	{
		KeyString += TEXT("DISBALEDEBUGSWITCH") + Delimiter;
	}
	else
	{
		KeyString += TEXT("ALLOWDEBUGSWITCH") + Delimiter;
	}

	if (bAppendObjectForDebugging)
	{
		KeyString += TEXT("[AdditionalDefines]") + Delimiter;
	}

	for (int32 Idx = 0; Idx < AdditionalDefines.Num(); Idx++)
	{
		KeyString += AdditionalDefines[Idx];
		KeyString += Delimiter;
	}

	if (bAppendObjectForDebugging)
	{
		KeyString += TEXT("[AdditionalVariables]") + Delimiter;
	}

	for (int32 Idx = 0; Idx < AdditionalVariables.Num(); Idx++)
	{
		KeyString += AdditionalVariables[Idx].GetName().ToString();
		KeyString += Delimiter;
		KeyString += AdditionalVariables[Idx].GetType().GetName();
		KeyString += Delimiter;
	}

	// Add any referenced script compile hashes to the key so that we will recompile when they are changed
	for (int32 HashIndex = 0; HashIndex < ReferencedCompileHashes.Num(); HashIndex++)
	{
		KeyString += ReferencedCompileHashes[HashIndex].ToString();

		if (bAppendObjectForDebugging && HashCollector)
		{
			KeyString += TEXT(" [") + HashCollector->BuildCompileHashSourceString(ReferencedCompileHashes[HashIndex]) + TEXT("]");
		}

		if (HashIndex < ReferencedCompileHashes.Num() - 1)
		{
			KeyString += Delimiter;
		}
	}
}

#endif

#if WITH_EDITORONLY_DATA
const FName UNiagaraScript::NiagaraCustomVersionTagName("NiagaraCustomVersion");
#endif

UNiagaraScript::UNiagaraScript(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Usage(ENiagaraScriptUsage::Function)
#if WITH_EDITORONLY_DATA
	, UsageIndex_DEPRECATED(0)
	, ModuleUsageBitmask_DEPRECATED( (1 << (int32)ENiagaraScriptUsage::ParticleSpawnScript) | (1 << (int32)ENiagaraScriptUsage::ParticleSpawnScriptInterpolated) | (1 << (int32)ENiagaraScriptUsage::ParticleUpdateScript) | (1 << (int32)ENiagaraScriptUsage::ParticleEventScript) | (1 << (int32)ENiagaraScriptUsage::ParticleSimulationStageScript))
	, LibraryVisibility_DEPRECATED(ENiagaraScriptLibraryVisibility::Unexposed)
	, NumericOutputTypeSelectionMode_DEPRECATED(ENiagaraNumericOutputTypeSelectionMode::Largest)
	, IsCooked(false)
#endif
{
#if WITH_EDITORONLY_DATA
	ScriptResource = MakeShared<FNiagaraShaderScript>();
	ScriptResource->OnCompilationComplete().AddUniqueDynamic(this, &UNiagaraScript::RaiseOnGPUCompilationComplete);

	RapidIterationParameters.DebugName = *GetFullName();
#endif
}

#if WITH_EDITORONLY_DATA
class UNiagaraSystem* UNiagaraScript::FindRootSystem()
{
	UObject* Obj = GetOuter();
	if (UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(Obj))
	{
		Obj = Emitter->GetOuter();
	}

	if (UNiagaraSystem* Sys = Cast<UNiagaraSystem>(Obj))
	{
		return Sys;
	}

	return nullptr;
}

bool UNiagaraScript::HasIdsRequiredForShaderCaching() const
{
	return CachedScriptVMId.CompilerVersionID.IsValid() && CachedScriptVMId.BaseScriptCompileHash.IsValid();
}

FNiagaraVMExecutableDataId& UNiagaraScript::GetLastGeneratedVMId(const FGuid& VersionGuid) const
{
	if (IsVersioningEnabled())
	{
		const FVersionedNiagaraScriptData* ScriptData = GetScriptData(VersionGuid.IsValid() ? VersionGuid : ExposedVersion);
		if (ScriptData)
		{
			return ScriptData->LastGeneratedVMId;
		}
	}
	return VersionData[0].LastGeneratedVMId;
}

// while it would be nice to not have to include the script path into the DDC key (so that two scripts on different
// could share the same DDC data) we currently have a number of elements of the system/emitter scripts that get
// incorporated into the data stored in the DDC that make it difficult to have scripts owned by different emitters
// generated deterministic results.  Current limitations include:
//	-stat scopes and emitter functions incorporating identifiers into the generated hlsl
//	-graph specific information (node guids) used in serialized errors/warnings for backtracking to the source in the UI
FString UNiagaraScript::BuildNiagaraDDCKeyString(const FNiagaraVMExecutableDataId& CompileId, const FString& ScriptPath)
{
	enum { UE_NIAGARA_COMPILATION_DERIVEDDATA_VER = 3 };

	FString KeyString;
	KeyString.Reserve(1024);

	KeyString.Appendf(TEXT("%i_%i"),
		(int32)UE_NIAGARA_COMPILATION_DERIVEDDATA_VER, GNiagaraSkipVectorVMBackendOptimizations);

	KeyString.AppendChar(TCHAR('_'));
	KeyString.Append(ScriptPath);
	KeyString.AppendChar(TCHAR('_'));

	CompileId.AppendKeyString(KeyString);

	static UE::DerivedData::FCacheBucket LegacyBucket(TEXTVIEW("LegacyNiagaraScriptDerivedData"), TEXTVIEW("NiagaraScript"));
	return FDerivedDataCacheInterface::BuildCacheKey(TEXT("NiagaraScriptDerivedData"), NIAGARASCRIPT_DERIVEDDATA_VER, *KeyString);
}

FString UNiagaraScript::GetNiagaraDDCKeyString(const FGuid& ScriptVersion, const FString& ScriptPath)
{
	return BuildNiagaraDDCKeyString(GetLastGeneratedVMId(ScriptVersion), ScriptPath);
}

void UNiagaraScript::ComputeVMCompilationId(FNiagaraVMExecutableDataId& Id, const FGuid& VersionGuid, FNiagaraScriptHashCollector* OutHashCollector) const
{
	Id = FNiagaraVMExecutableDataId();

	Id.bUsesRapidIterationParams = true;
	Id.bDisableDebugSwitches = false;
	Id.bRequiresPersistentIDs = false;
	Id.ScriptVersionID = IsVersioningEnabled() ? (VersionGuid.IsValid() ? VersionGuid : ExposedVersion) : FGuid();
	
	const UNiagaraSettings* NiagaraSettings = GetDefault<UNiagaraSettings>();
	Id.AdditionalDefines.Add(TEXT("InvalidNamespaceWrite=") + FString::FormatAsNumber(static_cast<int32>(NiagaraSettings->InvalidNamespaceWriteSeverity)));
	if (NiagaraSettings->bAccurateQuatInterpolation)
	{
		Id.AdditionalDefines.Add(FNiagaraCompileOptions::AccurateQuatInterpolation);
	}
	
	TArray<FNiagaraVariableBase> ReferencedStaticVars;
	TArray<const uint8*> ReferencedStaticVarValues;
	ENiagaraSimTarget SimTargetToBuild = ENiagaraSimTarget::CPUSim;
	// Ideally we wouldn't want to do this but rather than push the data down
	// from the emitter.  Checking all outers here to pick up simulation stages too.
	FVersionedNiagaraEmitter Outer = GetOuterEmitter();
	FVersionedNiagaraEmitterData* EmitterData = Outer.GetEmitterData();
	TArray<UNiagaraScript*> Scripts;

	FNiagaraScriptHashCollector HashCollector(GNiagaraDumpKeyGen == 1 || (OutHashCollector ? OutHashCollector->bCollectSources : false));

	if (EmitterData != nullptr)
	{
		UNiagaraEmitter* Emitter = Outer.Emitter;
		if (UNiagaraSystem* EmitterOwner = Cast<UNiagaraSystem>(Emitter->GetOuter()))
		{
			Scripts.Add(EmitterOwner->GetSystemSpawnScript());
			Scripts.Add(EmitterOwner->GetSystemUpdateScript());

			Id.bUsesRapidIterationParams = EmitterOwner->ShouldUseRapidIterationParameters();
			Id.bDisableDebugSwitches = EmitterOwner->ShouldDisableDebugSwitches();
			if (EmitterOwner->ShouldCompressAttributes())
			{
				Id.AdditionalDefines.Add(TEXT("CompressAttributes"));
			}
			if (EmitterOwner->ShouldIgnoreParticleReadsForAttributeTrim())
			{
				Id.AdditionalDefines.Add(TEXT("IgnoreParticleReadsForAttributeTrim"));
			}

			bool TrimAttributes = EmitterOwner->ShouldTrimAttributes();
			if (TrimAttributes)
			{
				auto TrimAttributesSupported = [=](const FVersionedNiagaraEmitter& OtherEmitter)
				{
					TArray<const UNiagaraDataInterfaceBase*> DataInterfaces;
					if (OtherEmitter.GetEmitterData() && OtherEmitter.GetEmitterData()->GraphSource)
					{
						OtherEmitter.GetEmitterData()->GraphSource->CollectDataInterfaces(DataInterfaces);

						for (const UNiagaraDataInterfaceBase* DataInterface : DataInterfaces)
						{
							if (DataInterface->HasInternalAttributeReads(OtherEmitter.Emitter, Emitter))
							{
								return false;
							}
						}
						return true;
					}
					else
					{
						return false;
					}
				};

				// if this emitter is being referenced by another emitter (PartilceRead) then don't worry about trimming attributes
				if (!EmitterOwner->ShouldIgnoreParticleReadsForAttributeTrim())
				{
					for (const FNiagaraEmitterHandle& EmitterHandle : EmitterOwner->GetEmitterHandles())
					{
						if (EmitterHandle.GetIsEnabled() && !TrimAttributesSupported(EmitterHandle.GetInstance()))
						{
							TrimAttributes = false;
							break;
						}
					}
				}
			}

			if (TrimAttributes)
			{
				Id.AdditionalDefines.Add(GNiagaraForceSafeScriptAttributeTrim ? TEXT("TrimAttributesSafe") : TEXT("TrimAttributes"));

				TArray<FName> AttributesToPreserve;
				auto AddAttributeToPreserve = [&](FName AttributeName) -> void
				{
					if (AttributeName != NAME_None)
					{
						AttributesToPreserve.Add(AttributeName);
					}
				};

				// preserve the attributes that have been defined on the emitter directly
				for (const FString& Attribute : EmitterData->AttributesToPreserve)
				{
					AddAttributeToPreserve(*Attribute);
				}

				// Now preserve the attributes that have been defined on the renderers in use
				for (UNiagaraRendererProperties* RendererProperty : EmitterData->GetRenderers())
				{
					if (RendererProperty->bIsEnabled)
					{
						for (const FNiagaraVariable& BoundAttribute : RendererProperty->GetBoundAttributes())
						{
							AddAttributeToPreserve(BoundAttribute.GetName());
						}
					}
				}

				for (const UNiagaraSimulationStageBase* SimStageBase : EmitterData->GetSimulationStages())
				{
					if (const UNiagaraSimulationStageGeneric* SimStageGeneric = Cast<const UNiagaraSimulationStageGeneric>(SimStageBase))
					{
						AddAttributeToPreserve(SimStageGeneric->EnabledBinding.GetParamMapBindableVariable().GetName());

						if (SimStageGeneric->IterationSource == ENiagaraIterationSource::DirectSet)
						{
							AddAttributeToPreserve(SimStageGeneric->ElementCountX.ResolvedParameter.GetName());
							AddAttributeToPreserve(SimStageGeneric->ElementCountY.ResolvedParameter.GetName());
							AddAttributeToPreserve(SimStageGeneric->ElementCountZ.ResolvedParameter.GetName());
						}
						AddAttributeToPreserve(SimStageGeneric->NumIterations.ResolvedParameter.GetName());
						if ( SimStageGeneric->bOverrideGpuDispatchNumThreads )
						{
							AddAttributeToPreserve(SimStageGeneric->OverrideGpuDispatchNumThreadsX.ResolvedParameter.GetName());
							AddAttributeToPreserve(SimStageGeneric->OverrideGpuDispatchNumThreadsY.ResolvedParameter.GetName());
							AddAttributeToPreserve(SimStageGeneric->OverrideGpuDispatchNumThreadsZ.ResolvedParameter.GetName());
						}
					}
				}

				// We sort the keys so that it doesn't matter what order they were defined in.
				if (!AttributesToPreserve.IsEmpty())
				{
					AttributesToPreserve.Sort(FNameLexicalLess());
					AttributesToPreserve.SetNum(Algo::Unique(AttributesToPreserve));

					Id.AdditionalDefines.Reserve(Id.AdditionalDefines.Num() + AttributesToPreserve.Num());
					for (FName AttributeName : AttributesToPreserve)
					{
						TStringBuilder<256> PreserveAttributeDefine;
						PreserveAttributeDefine.Append(TEXT("PreserveAttribute="));
						AttributeName.AppendString(PreserveAttributeDefine);
						Id.AdditionalDefines.Add(PreserveAttributeDefine.ToString());
					}
				}
			}

			ComputeVMCompilationId_EmitterShared(Id, Outer, ENiagaraRendererSourceDataMode::Particles);
		}

		// Gather up the scripts we will use to iterate for static rapid iteration vars later...
		EmitterData->GetScripts(Scripts, false, true);
		bool bUsesInterpolatedSpawning = EmitterData->UsesInterpolatedSpawning();
		
		// Warn against drift between script and emitter interpolation mode
		if (bUsesInterpolatedSpawning && Usage == ENiagaraScriptUsage::ParticleSpawnScript)
		{
			UE_LOGF(LogNiagara, Warning, "Emitter InterpolatedSpawnMode is on but spawn script Usage is ParticleSpawnScript (expected ParticleSpawnScriptInterpolated). Compile/runtime constant-buffer drift may occur. Script: %ls", *GetFullName());
		}

		if ((bUsesInterpolatedSpawning && Usage == ENiagaraScriptUsage::ParticleGPUComputeScript) ||
			(bUsesInterpolatedSpawning && Usage == ENiagaraScriptUsage::ParticleSpawnScript) ||
			Usage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated)
		{
			Id.InterpolatedSpawnMode = EmitterData->InterpolatedSpawnMode;
			Id.AdditionalDefines.Add(TEXT("InterpolatedSpawn"));
		}
		if (bUsesInterpolatedSpawning)
		{
			Id.AdditionalDefines.Add(TEXT("Emitter.InterpolatedSpawn"));
			if (EmitterData->InterpolatedSpawnMode == ENiagaraInterpolatedSpawnMode::RunUpdateScript)
			{
				Id.AdditionalDefines.Add(TEXT("RunUpdateWithoutInterpolation"));
			}
		}
		if (bUsesInterpolatedSpawning == false && EmitterData->bGpuAlwaysRunParticleUpdateScript == true)
		{
			Id.AdditionalDefines.Add(TEXT("GpuAlwaysRunParticleUpdateScript"));
		}
		if (IsParticleScript(Usage) || IsParticleStageScript(Usage))
		{
			SimTargetToBuild = EmitterData->SimTarget;
		}
		if (EmitterData->RequiresPersistentIDs())
		{
			Id.bRequiresPersistentIDs = true;
			Id.AdditionalDefines.Add(TEXT("RequiresPersistentIDs"));
		}
		if (EmitterData->bLocalSpace)
		{
			Id.AdditionalDefines.Add(TEXT("Emitter.Localspace"));
		}
		if (EmitterData->bDeterminism)
		{
			Id.AdditionalDefines.Add(TEXT("Emitter.Determinism"));
		}

		// Append all Shader Parameter compile hashes
		if (GNiagaraCompileHashAllDataInterfaces)
		{
			HashCollector.AddHash(NiagaraScriptInternal::CompileHashAllDataInterfaces(), TEXT("AllDataInterfaceHashes"));
		}

		// Has simulation stages
		{

			FSHA1 HashState;
			FNiagaraCompileHashVisitor Visitor(HashState);
			for (UNiagaraSimulationStageBase* Base : EmitterData->GetSimulationStages())
			{
				// bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const;
				if (Base && Base->bEnabled)
				{
					Base->AppendCompileHash(&Visitor);
				}
			}
			HashState.Final();

			TArray<uint8> DataHash;
			DataHash.AddUninitialized(FSHA1::DigestSize);
			HashState.GetHash(DataHash.GetData());

			FNiagaraCompileHash Hash(DataHash);
			HashCollector.AddHash(FNiagaraCompileHash(DataHash), TEXT("SimulationStageHeaders"));
		}

		if (IsParticleEventScript(Usage))
		{
			if (const FNiagaraEventScriptProperties* EventScriptProps = EmitterData->GetEventHandlerByIdUnsafe(UsageId))
			{
				if (EventScriptProps->ExecutionMode == EScriptExecutionMode::SpawnedParticles)
				{
					Id.AdditionalDefines.AddUnique(FNiagaraCompileOptions::EventSpawnDefine);

					if (EventScriptProps->UpdateAttributeInitialValues && GbNiagaraEventSpawnsUpdateInitialAttributeValues)
					{
						Id.AdditionalDefines.AddUnique(FNiagaraCompileOptions::EventSpawnInitialAttribWritesDefine);
					}
				}
			}
		}
	}

	UObject* Obj = GetOuter();
	if (UNiagaraSystem* System = Cast<UNiagaraSystem>(Obj))
	{
		Scripts.Add(System->GetSystemSpawnScript());
		Scripts.Add(System->GetSystemUpdateScript());

		Id.bUsesRapidIterationParams = System->ShouldUseRapidIterationParameters();
		Id.bDisableDebugSwitches = System->ShouldDisableDebugSwitches();
		if (System->ShouldCompressAttributes())
		{
			Id.AdditionalDefines.Add(TEXT("CompressAttributes"));
		}

		for (const FNiagaraEmitterHandle& EmitterHandle: System->GetEmitterHandles())
		{
			FVersionedNiagaraEmitterData* HandleData = EmitterHandle.GetEmitterData();
			if (HandleData && EmitterHandle.GetIsEnabled())
			{
				if (HandleData->bLocalSpace)
				{
					Id.AdditionalDefines.Add(EmitterHandle.GetUniqueInstanceName() + TEXT(".Localspace"));
				}
				if (HandleData->bDeterminism)
				{
					Id.AdditionalDefines.Add(EmitterHandle.GetUniqueInstanceName() + TEXT(".Determinism"));
				}
				if (HandleData->UsesInterpolatedSpawning())
				{
					Id.AdditionalDefines.Add(EmitterHandle.GetUniqueInstanceName() + TEXT(".InterpolatedSpawn"));
				}

				ComputeVMCompilationId_EmitterShared(Id, EmitterHandle.GetInstance(), ENiagaraRendererSourceDataMode::Emitter);
			}
		}
	}

	switch (SimTargetToBuild)
	{
	case ENiagaraSimTarget::CPUSim:
		Id.AdditionalDefines.AddUnique(FNiagaraCompileOptions::CpuScriptDefine);
		break;
	case ENiagaraSimTarget::GPUComputeSim:
		Id.AdditionalDefines.AddUnique(FNiagaraCompileOptions::GpuScriptDefine);
		break;
	default:
		checkf(false, TEXT("Unknown sim target type!"));
	}

	for (int32 ScriptIdx = 0; ScriptIdx < Scripts.Num(); ScriptIdx++)
	{
		UNiagaraScript* Script = Scripts[ScriptIdx];
		const bool bIsThisEmitterUsage = UNiagaraScript::IsEmitterScript(Usage);
		const bool bIsThisParticleUsage = UNiagaraScript::IsParticleScript(Usage);
	
		const bool bIsOtherSystemUsage = UNiagaraScript::IsSystemScript(Script->Usage);
		const bool bIsOtherParticleUsage = UNiagaraScript::IsParticleScript(Script->Usage);
		const bool bIsOtherEmitterUsage = UNiagaraScript::IsEmitterScript(Script->Usage);

		// Emitter scripts don't depend on static variables from particle scripts.
		if (bIsThisEmitterUsage && bIsOtherParticleUsage)
			continue;
		
		// We assume that latest source is ok because these are all top-level scripts in either systems or emitters.
		if (Script->GetLatestSource())
		{
			Script->GetLatestSource()->RegisterVMCompilationIdDependencies(HashCollector, Script->Usage, Script->UsageId);
		}

		for (const FNiagaraVariableWithOffset& Var : Script->RapidIterationParameters.ReadParameterVariables())
		{
			if (Var.GetType().IsStatic())
			{
				if ((bIsThisEmitterUsage || bIsThisParticleUsage) && bIsOtherSystemUsage)
				{
					FNameBuilder VarNameBuilder(Var.GetName());
					TArray<FStringView> SplitName;
					UE::String::ParseTokens(VarNameBuilder.ToView(), TEXT("."), SplitName);

					// Only include System based rapid iteration static variables if we're an emitter/particle script as system scripts can contain the amalgam of 
					// it's own and any child emitter script rapid iteration variables.
					if (SplitName.Num() == 3)
					{
						//Constants.Module.Variable input, this would match up for system inputs to modules that could produce new outputs that are static.
					}
					else if (SplitName.Num() > 3 && SplitName[SplitName.Num()-2] == TEXT("System"))
					{
						// Example: Constants.SetVariables_C872846946BD27D8655B60892D599F11.System.New bool
					}
					else
					{
						continue;
					}
				}
				ReferencedStaticVars.Add(Var);
				ReferencedStaticVarValues.Add(Script->RapidIterationParameters.GetParameterData(Var));
			}
		}
	}

	// If we aren't using rapid iteration parameters, we need to bake them into the hashstate for the compile id. This
	// makes their values part of the lookup.
	
	{
		FSHA1 HashState;
		auto UpdateHashState = [&](FName InName) -> void
		{
			FNameBuilder NameBuilder(InName);
			FStringView NameBuilderView = NameBuilder.ToView();
			HashState.UpdateWithString(NameBuilderView.GetData(), NameBuilderView.Len());
		};

		if (false == Id.bUsesRapidIterationParams)
		{
			//UE_LOGF(LogNiagara, Display, "AreScriptAndSourceSynchronized %ls ======================== ", *GetFullName());
			for (const FNiagaraVariableWithOffset& Var : RapidIterationParameters.ReadParameterVariables())
			{
				if (Var.IsValid() == false || Var.IsDataInterface() || Var.IsUObject())
				{
					// Skip these types as they're invalid or don't bake out, just normal parameters get baked.
				}
				else
				{
					// Hash the name, type, and value of each parameter..
					UpdateHashState(Var.GetName());
					UpdateHashState(Var.GetType().GetFName());
					HashState.Update(RapidIterationParameters.GetParameterData(Var), Var.GetSizeInBytes());
				}
			}
		}

		for (int32 i = 0; i < ReferencedStaticVars.Num(); i++)
		{
			if (i == 0 && UNiagaraScript::LogCompileStaticVars > 0)
			{
				UE_LOGF(LogNiagara, Display, "***** Referenced Static Vars %d %ls", (int32)Usage, *GetPathName());
			}

			const FNiagaraVariableBase& StaticVar = ReferencedStaticVars[i];

			// Hash the name, type, and value of each parameter..
			UpdateHashState(StaticVar.GetName());
			UpdateHashState(StaticVar.GetType().GetFName());
			if (const uint8* VarData = ReferencedStaticVarValues[i])
			{
				if (UNiagaraScript::LogCompileStaticVars > 0)
				{
					UE_LOGF(LogNiagara, Display, "Param %ls %ls %d", *StaticVar.GetType().GetName(), *StaticVar.GetName().ToString(), (uint32)VarData[0]);
				}
				HashState.Update(VarData, StaticVar.GetSizeInBytes());
			}
		}
		HashState.Final();

		TArray<uint8> DataHash;
		DataHash.AddUninitialized(FSHA1::DigestSize);
		HashState.GetHash(DataHash.GetData());

		HashCollector.AddHash(FNiagaraCompileHash(DataHash), TEXT("RIParams"));
	}

	if (const FVersionedNiagaraScriptData* ScriptData = GetScriptData(Id.ScriptVersionID))
	{
		if (ScriptData->Source)
		{
			ScriptData->Source->ComputeVMCompilationId(Id, HashCollector, Usage, UsageId);
		}
	}

	// Append the state of the "Fail If Not Set severity" cvar, as it may affect the LastCompileEvents which is a member of the script's VMExecutableData.
	Id.AdditionalDefines.Add(FString::Printf(TEXT("FailIfNotSetMessageSeverity: %d"), GNiagaraTranslatorFailIfNotSetSeverity));

	Id.AdditionalDefines.Add(FString::Printf(TEXT("EnableTraversalCache: %d"), GbNiagaraEnableTraversalCache));
	Id.ReferencedCompileHashes = MoveTemp(HashCollector.ReferencedHashes);

	FNiagaraVMExecutableDataId& LastGeneratedVMId = GetLastGeneratedVMId(VersionGuid);

	// normalize the order
	Id.AdditionalDefines.Sort();
	Id.ReferencedCompileHashes.Sort();
	Id.AdditionalVariables.Sort([](const FNiagaraVariableBase& Lhs, const FNiagaraVariableBase& Rhs) { return Lhs.GetName().LexicalLess(Rhs.GetName()); });

	if (GNiagaraDumpKeyGen == 1 && Id != LastGeneratedVMId)
	{
		TArray<FString> OutputByLines;
		FString StrDump;
		Id.AppendKeyString(StrDump, TEXT("\n"), true, &HashCollector);
		StrDump.ParseIntoArrayLines(OutputByLines, false);

		UE_LOGF(LogNiagara, Display, "KeyGen %ls\n==================\n", *GetPathName());
		for (int32 i = 0; i < OutputByLines.Num(); i++)
		{
			UE_LOGF(LogNiagara, Display, "/*%04d*/\t\t%ls", i + 1, *OutputByLines[i]);
		}
	}

	if (OutHashCollector && OutHashCollector->bCollectSources)
	{
		OutHashCollector->ReferencedHashSources = MoveTemp(HashCollector.ReferencedHashSources);
	}

	LastGeneratedVMId = Id;
}

void UNiagaraScript::ComputeVMCompilationId_EmitterShared(FNiagaraVMExecutableDataId& Id, const FVersionedNiagaraEmitter& Emitter, ENiagaraRendererSourceDataMode InSourceMode) const
{
	// Gather additional variables from renderers
	for (UNiagaraRendererProperties* RendererProperty : Emitter.GetEmitterData()->GetRenderers())
	{
		// Because we currently generate the CompilationId during PostLoad (as a part of AreScriptAndSourceSynchronized)
		// we need to ensure that the RendererProperties have done their PostLoad.
		RendererProperty->ConditionalPostLoad();

		if (RendererProperty->GetCurrentSourceMode() != InSourceMode)
			continue;

		TArray<FNiagaraVariableBase> AdditionalVariables;
		RendererProperty->GetAdditionalVariables(AdditionalVariables);
		for (const FNiagaraVariableBase& AdditionalVariable : AdditionalVariables)
		{
			if (AdditionalVariable.IsValid())
			{
				Id.AdditionalVariables.AddUnique(AdditionalVariable);
			}
		}
	}

	// Sort the additional variables by name lexically so they are always in the same order
	Id.AdditionalVariables.Sort([](const FNiagaraVariableBase& A, const FNiagaraVariableBase& B) { return A.GetName().LexicalLess(B.GetName()); });
}

void UNiagaraScript::InvalidateAndEmptyDefaultDataInterfaces(int32 SlackCount, EDataInterfaceInvalidationMode InvalidationMode)
{
	DataInterfacesPendingInvalidation.Reserve(DataInterfacesPendingInvalidation.Num() + CachedDefaultDataInterfaces.Num());

	for (FNiagaraScriptDataInterfaceInfo& DefaultDataInterface : CachedDefaultDataInterfaces)
	{
		if (DefaultDataInterface.DataInterface)
		{
			if (DefaultDataInterface.DataInterface->GetOuter() == this)
			{
				DataInterfacesPendingInvalidation.Add(DefaultDataInterface.DataInterface);
			}
		}
	}

	CachedDefaultDataInterfaces.Empty(SlackCount);

	if (InvalidationMode == EDataInterfaceInvalidationMode::ProcessInvaliation)
	{
		const ERenameFlags RenameFlags = REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors;

		for (UNiagaraDataInterface* DataInterface : DataInterfacesPendingInvalidation)
		{
			// The data interface could have been GCed before we process invalidations
			if (DataInterface == nullptr)
			{
				continue;
			}

			const FName OrigName = DataInterface->GetFName();

			FNameBuilder InvalidatedName;
			InvalidatedName << TEXT("Invalidated_");
			OrigName.AppendString(InvalidatedName);

			FName NewName(InvalidatedName.ToView());

			DataInterface->ClearFlags(RF_Public);
			while (!DataInterface->Rename(*NewName.ToString(), nullptr, (RenameFlags | REN_Test)))
			{
				NewName.SetNumber(NewName.GetNumber() + 1);
			}

			DataInterface->Rename(*NewName.ToString(), nullptr, RenameFlags);
		}

		DataInterfacesPendingInvalidation.Reset();
	}
}

#endif

bool UNiagaraScript::ContainsUsage(ENiagaraScriptUsage InUsage) const
{
	if (IsEquivalentUsage(InUsage))
	{
		return true;
	}

	if (Usage == ENiagaraScriptUsage::ParticleGPUComputeScript && IsParticleScript(InUsage))
	{
		return true;
	}

	if (Usage == ENiagaraScriptUsage::ParticleGPUComputeScript && InUsage == ENiagaraScriptUsage::ParticleSimulationStageScript)
	{
		return true;
	}

	if (InUsage == ENiagaraScriptUsage::ParticleUpdateScript && Usage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated)
	{
		return true;
	}

	if (InUsage == ENiagaraScriptUsage::EmitterSpawnScript && Usage == ENiagaraScriptUsage::SystemSpawnScript)
	{
		return true;
	}

	if (InUsage == ENiagaraScriptUsage::EmitterUpdateScript && Usage == ENiagaraScriptUsage::SystemUpdateScript)
	{
		return true;
	}

	return false;
}

TOptional<ENiagaraSimTarget> UNiagaraScript::GetSimTarget() const
{
	switch (Usage)
	{
	case ENiagaraScriptUsage::ParticleSpawnScript:
	case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
	case ENiagaraScriptUsage::ParticleUpdateScript:
	case ENiagaraScriptUsage::ParticleEventScript:
	case ENiagaraScriptUsage::ParticleSimulationStageScript:
	case ENiagaraScriptUsage::ParticleGPUComputeScript:
		if (FVersionedNiagaraEmitterData* EmitterData = GetOuterEmitter().GetEmitterData())
		{
			if (EmitterData->SimTarget != ENiagaraSimTarget::CPUSim || CachedScriptVM.IsValid())
			{
				return EmitterData->SimTarget;
			}
		}
		break;
	case ENiagaraScriptUsage::EmitterSpawnScript:
	case ENiagaraScriptUsage::EmitterUpdateScript:
	case ENiagaraScriptUsage::SystemSpawnScript:
	case ENiagaraScriptUsage::SystemUpdateScript:
		if (CachedScriptVM.IsValid())
		{
			return ENiagaraSimTarget::CPUSim;
		}
		break;
	default:
		break;
	};
	return TOptional<ENiagaraSimTarget>();
}

void UNiagaraScript::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

#if WITH_EDITORONLY_DATA
	// Pre-save can happen in any order for objects in the package and since this is now used to cache data for execution we need to make sure that the system compilation
	// is complete before caching the executable data.
	UNiagaraSystem* SystemOwner = FindRootSystem();
	if (SystemOwner)
	{
		SystemOwner->EnsureFullyLoaded();
		SystemOwner->WaitForCompilationComplete();
	}

	ResolveParameterCollectionReferences();
#endif
}

void UNiagaraScript::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);		// only changes version if not loading
	const int32 NiagaraVer = Ar.CustomVer(FNiagaraCustomVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.IsCooking())
	{
		// Cook optimization can temporarily modify the rapid iteration parameters for the serialize call.  We don't want this to broadcast
		// on changed messages because it will modify the change id on owning objects so we suppress those messages starting here.
		FNiagaraParameterStore::FScopedSuppressOnChanged SuppressOnChanged(RapidIterationParameters);

		UNiagaraSystem* System = GetTypedOuter<UNiagaraSystem>();
		bool bUsesRapidIterationParams = System != nullptr && System->ShouldUseRapidIterationParameters();
		if (!bUsesRapidIterationParams)
		{
			// Cache the unmodified rapid iteration parameters in an editor only property so that they can be restored in a cooked editor
			// and so they can be restored after this optimization.
			RapidIterationParametersCookedEditorCache = RapidIterationParameters;

			// Get the active parameters
			// Remove all parameters that aren't data interfaces or uobjects
			int32 NumRemoved = 0;
			auto ParameterVariables = RapidIterationParametersCookedEditorCache.ReadParameterVariables();
			for (const FNiagaraVariableBase& Var : ParameterVariables)
			{
				if (Var.IsDataInterface() || Var.IsUObject())
				{
					continue;
				}
				RapidIterationParameters.RemoveParameter(Var);
				NumRemoved++;
			}

			UE_LOGF(LogNiagara, Verbose, "Pruned %d/%d parameters from script %ls", NumRemoved, ParameterVariables.Num(), *GetFullName());
		}

		// when we are cooking we want to strip out some debug only data.  While this data is already intended for editor only it will
		// bloat the size of packages intended for the cooked editor
		TOptional<FNiagaraVMExecutableData> LocalStrippedDebugData;
		auto SwapDebugData = [](FNiagaraVMExecutableData& SourceData, FNiagaraVMExecutableData& TargetData) -> void
		{
			Swap(TargetData.LastHlslTranslation, SourceData.LastHlslTranslation);

			// We must preserve the LastTranslationGPU as it is used to compile the GPUComputeScript when using the standard compilation mode
			// Swap(TargetData.LastHlslTranslationGPU, SourceData.LastHlslTranslationGPU);

			Swap(TargetData.LastAssemblyTranslation, SourceData.LastAssemblyTranslation);
			Swap(TargetData.CompileTagsEditorOnly, SourceData.CompileTagsEditorOnly);
		};

		if (Ar.IsSaving())
		{
			auto& ExecutableData = GetVMExecutableData();

			if (Usage != ENiagaraScriptUsage::ParticleGPUComputeScript)
			{
				ExecutableData.BakeScriptLiterals(ExecutableData.ScriptLiterals);
			}
			else
			{
				ExecutableData.ScriptLiterals.Empty();
			}

			if (GNiagaraCompressScriptByteCode)
			{
				ExecutableData.ByteCode.Compress();
			}

			LocalStrippedDebugData.Emplace();
			SwapDebugData(*LocalStrippedDebugData, CachedScriptVM);
		}

		Super::Serialize(Ar);

		if (LocalStrippedDebugData.IsSet())
		{
			SwapDebugData(*LocalStrippedDebugData, CachedScriptVM);
		}

		// If we cached the rapid iteration parameters before serialize, restore them here.
		if (RapidIterationParametersCookedEditorCache.Num() != 0)
		{
			RapidIterationParameters = RapidIterationParametersCookedEditorCache;
			RapidIterationParametersCookedEditorCache.Empty();
		}
	}
	else if (Ar.IsLoading() && GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
	{
		Super::Serialize(Ar);
		if (RapidIterationParametersCookedEditorCache.Num() > 0)
		{
			// Restore the rapid iteration parameters from the cache for cooked editor builds.
			RapidIterationParameters = RapidIterationParametersCookedEditorCache;
			RapidIterationParametersCookedEditorCache.Empty();
		}
		IsCooked = true;
	}
	else
#endif
	{
		Super::Serialize(Ar);
	}

	bool IsValidShaderScript = false;
	if (NiagaraVer < FNiagaraCustomVersion::DontCompileGPUWhenNotNeeded)
	{
		IsValidShaderScript = Usage != ENiagaraScriptUsage::Module && Usage != ENiagaraScriptUsage::Function && Usage != ENiagaraScriptUsage::DynamicInput
			&& (NiagaraVer < FNiagaraCustomVersion::NiagaraShaderMapCooking2 || (Usage != ENiagaraScriptUsage::SystemSpawnScript && Usage != ENiagaraScriptUsage::SystemUpdateScript))
			&& (NiagaraVer < FNiagaraCustomVersion::NiagaraCombinedGPUSpawnUpdate || (Usage != ENiagaraScriptUsage::ParticleUpdateScript && Usage != ENiagaraScriptUsage::EmitterSpawnScript && Usage != ENiagaraScriptUsage::EmitterUpdateScript));
	}
	else if (NiagaraVer < FNiagaraCustomVersion::MovedToDerivedDataCache)
	{
		IsValidShaderScript = LegacyCanBeRunOnGpu();
	}
	else
	{
		IsValidShaderScript = CanBeRunOnGpu();
	}

#if WITH_EDITOR
	if (IsValidShaderScript)
	{
		if (NiagaraVer < FNiagaraCustomVersion::UseHashesToIdentifyCompileStateOfTopLevelScripts)
		{
			// In some rare cases a GPU script could have been saved in an error state in a version where skeletal mesh or static mesh data interfaces didn't work properly on GPU.
			// This would fail in the current regime.
			for (const FNiagaraScriptDataInterfaceCompileInfo& InterfaceInfo : CachedScriptVM.DataInterfaceInfo)
			{
				if (InterfaceInfo.Type.GetClass() == UNiagaraDataInterfaceSkeletalMesh::StaticClass() ||
					InterfaceInfo.Type.GetClass() == UNiagaraDataInterfaceStaticMesh::StaticClass())
				{
					IsValidShaderScript = false;
				}
			}
		}
	}

	if (Ar.IsSaving() && Ar.IsCooking() && Ar.IsPersistent() && !Ar.IsObjectReferenceCollector() && FShaderLibraryCooker::NeedsShaderStableKeys(EShaderPlatform::SP_NumPlatforms))
	{
		SaveShaderStableKeys(Ar.CookingTarget());
	}

	if (Ar.IsLoading())
	{
		bMigrateParameterDataToHierarchyRoot = NiagaraVer < FNiagaraCustomVersion::HierarchyEditorScriptSupport;
	}
#endif

	SerializeNiagaraShaderMaps(Ar, NiagaraVer, IsValidShaderScript);
}

#if WITH_EDITOR
void UNiagaraScript::SaveShaderStableKeys(const class ITargetPlatform* TP)
{
	FStableShaderKeyAndValue SaveKeyVal;
	SaveKeyVal.ClassNameAndObjectPath.SetCompactFullNameFromObject(this);
	static FName FName_Niagara(TEXT("Niagara"));
	SaveKeyVal.MaterialDomain = FName_Niagara;
	const TArray<TUniquePtr<FNiagaraShaderScript>>* ScriptResourcesToSavePtr = CachedScriptResourcesForCooking.Find(TP);
	if (ScriptResourcesToSavePtr != nullptr)
	{
		for (const TUniquePtr<FNiagaraShaderScript>& Resource : *ScriptResourcesToSavePtr)
		{
			if (Resource)
			{
				Resource->SaveShaderStableKeys(EShaderPlatform::SP_NumPlatforms, SaveKeyVal);
			}
		}
	}
}

bool FNiagaraModuleDependency::HasValidVersionDependency() const
{
	CheckVersionCache();
	return VersionDependencyCache.bValid;
}

bool FNiagaraModuleDependency::IsVersionAllowed(const FNiagaraAssetVersion& Version) const
{
	if (HasValidVersionDependency())
	{
		FNiagaraAssetVersion MinVersion;
		FNiagaraAssetVersion MaxVersion;
		MinVersion.MajorVersion = VersionDependencyCache.MinMajorVersion;
		MinVersion.MinorVersion = VersionDependencyCache.MinMinorVersion;
		MaxVersion.MajorVersion = VersionDependencyCache.MaxMajorVersion;
		MaxVersion.MinorVersion = VersionDependencyCache.MaxMinorVersion;
		return Version <= MaxVersion && MinVersion <= Version;
	}
	return true;
}

void FNiagaraModuleDependency::CheckVersionCache() const
{
	if (VersionDependencyCache.SourceProperty != RequiredVersion)
	{
		VersionDependencyCache.SourceProperty = RequiredVersion;
		
		FString PatternString(TEXT("^\\s*(\\d+)\\.(\\d+)\\s*(?:(\\+)|(-)\\s*(\\d+)\\.(\\d+))?\\s*$"));
		FRegexPattern Pattern(PatternString);
		FRegexMatcher Matcher(Pattern, RequiredVersion);
		if (Matcher.FindNext())
		{
			VersionDependencyCache.bValid = true;
			VersionDependencyCache.MinMajorVersion = FCString::Atoi(*Matcher.GetCaptureGroup(1));
			VersionDependencyCache.MinMinorVersion = FCString::Atoi(*Matcher.GetCaptureGroup(2));
			if (Matcher.GetCaptureGroup(3) == "+")
			{
				VersionDependencyCache.MaxMajorVersion = MAX_int32;
				VersionDependencyCache.MaxMinorVersion = MAX_int32;
			}
			else if (Matcher.GetCaptureGroup(4) == "-")
			{
				VersionDependencyCache.MaxMajorVersion = FCString::Atoi(*Matcher.GetCaptureGroup(5));
				VersionDependencyCache.MaxMinorVersion = FCString::Atoi(*Matcher.GetCaptureGroup(6));
			}
			else
			{
				VersionDependencyCache.MaxMajorVersion = VersionDependencyCache.MinMajorVersion;
				VersionDependencyCache.MaxMinorVersion = VersionDependencyCache.MinMinorVersion;
			}
		}
		else
		{
			VersionDependencyCache.bValid = false;
		}
	}
}
#endif

FNiagaraCompilerTag* FNiagaraCompilerTag::FindTag(TArray< FNiagaraCompilerTag>& InTags, const FNiagaraVariableBase& InSearchVar)
{
	for (FNiagaraCompilerTag& Tag : InTags)
	{
		if (static_cast<const FNiagaraVariableBase&>(Tag.Variable) == InSearchVar)
			return &Tag;
	}
	return nullptr;
}


const FNiagaraCompilerTag* FNiagaraCompilerTag::FindTag(const TArray< FNiagaraCompilerTag>& InTags, const FNiagaraVariableBase& InSearchVar)
{
	for (const FNiagaraCompilerTag& Tag : InTags)
	{
		if (static_cast<const FNiagaraVariableBase&>(Tag.Variable) == InSearchVar)
			return &Tag;
	}
	return nullptr;
}

/** Is usage A dependent on Usage B?*/
bool UNiagaraScript::IsUsageDependentOn(ENiagaraScriptUsage InUsageA, ENiagaraScriptUsage InUsageB)
{
	if (InUsageA == InUsageB)
	{
		return false;
	}

	// Usages of the same phase are interdependent because we copy the attributes from one to the other and if those got
	// out of sync, there could be problems.

	if ((InUsageA == ENiagaraScriptUsage::ParticleSpawnScript || InUsageA == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated || InUsageA == ENiagaraScriptUsage::ParticleUpdateScript || InUsageA == ENiagaraScriptUsage::ParticleEventScript)
		&& (InUsageB == ENiagaraScriptUsage::ParticleSpawnScript || InUsageB == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated || InUsageB == ENiagaraScriptUsage::ParticleUpdateScript || InUsageB == ENiagaraScriptUsage::ParticleEventScript))
	{
		return true;
	}

	// The GPU compute script is always dependent on the other particle scripts.
	if ((InUsageA == ENiagaraScriptUsage::ParticleGPUComputeScript)
		&& (InUsageB == ENiagaraScriptUsage::ParticleSpawnScript || InUsageB == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated || InUsageB == ENiagaraScriptUsage::ParticleUpdateScript || InUsageB == ENiagaraScriptUsage::ParticleEventScript || InUsageB == ENiagaraScriptUsage::ParticleSimulationStageScript))
	{
		return true;
	}

	if ((InUsageA == ENiagaraScriptUsage::EmitterSpawnScript || InUsageA == ENiagaraScriptUsage::EmitterUpdateScript)
		&& (InUsageB == ENiagaraScriptUsage::EmitterSpawnScript || InUsageB == ENiagaraScriptUsage::EmitterUpdateScript))
	{
		return true;
	}

	if ((InUsageA == ENiagaraScriptUsage::SystemSpawnScript || InUsageA == ENiagaraScriptUsage::SystemUpdateScript)
		&& (InUsageB == ENiagaraScriptUsage::SystemSpawnScript || InUsageB == ENiagaraScriptUsage::SystemUpdateScript))
	{
		return true;
	}

	return false;
}

bool UNiagaraScript::ConvertUsageToGroup(ENiagaraScriptUsage InUsage, ENiagaraScriptGroup& OutGroup)
{
	if (IsParticleScript(InUsage) || IsStandaloneScript(InUsage))
	{
		OutGroup = ENiagaraScriptGroup::Particle;
		return true;
	}
	else if (IsEmitterSpawnScript(InUsage) || IsEmitterUpdateScript(InUsage))
	{
		OutGroup = ENiagaraScriptGroup::Emitter;
		return true;
	}
	else if (IsSystemSpawnScript(InUsage) || IsSystemUpdateScript(InUsage))
	{
		OutGroup = ENiagaraScriptGroup::System;
		return true;
	}

	return false;
}

void UNiagaraScript::PostLoad()
{
	Super::PostLoad();

	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);

#if WITH_EDITORONLY_DATA
	CheckVersionDataAvailable();
#endif
	
	RapidIterationParameters.PostLoad(this);

#if WITH_EDITORONLY_DATA
	// Because we might be using these cached data interfaces, we need to make sure that they are properly postloaded.
	for (FNiagaraScriptDataInterfaceInfo& Info : CachedDefaultDataInterfaces)
	{
		if (Info.DataInterface)
		{
			Info.DataInterface->ConditionalPostLoad();
		}
	}

	for (FNiagaraScriptUObjectCompileInfo& DefaultUObject : CachedDefaultUObjects)
	{
		if (DefaultUObject.Object)
		{
			DefaultUObject.Object->ConditionalPostLoad();
		}
	}

	FVersionedNiagaraScriptData* ScriptData = GetLatestScriptData();
	ensure(ScriptData);
	if (NiagaraVer < FNiagaraCustomVersion::AddSimulationStageUsageEnum)
	{
		uint8 SimulationStageIndex = (uint8)ENiagaraScriptUsage::ParticleSimulationStageScript;
		uint8 MaxIndex = (uint8)ENiagaraScriptUsage::SystemUpdateScript;
		int32& UsageBitmask = ScriptData->ModuleUsageBitmask;
		// Start at the end and shift the bits down to account for the new shader stage bit.
		for (uint8 CurrentIndex = MaxIndex; CurrentIndex > SimulationStageIndex; CurrentIndex--)
		{
			uint8 OldIndex = CurrentIndex - 1;
			if ((UsageBitmask & (1 << OldIndex)) != 0)
			{
				UsageBitmask |= 1 << CurrentIndex;
			}
			else
			{
				UsageBitmask &= ~(1 << CurrentIndex);
			}
		}
		// Clear the simulation stage bit.
		UsageBitmask &= ~(1 << SimulationStageIndex);
	}

	if (NiagaraVer < FNiagaraCustomVersion::SimulationStageInUsageBitmask)
	{
		int32& UsageBitmask = ScriptData->ModuleUsageBitmask;
		const TArray<ENiagaraScriptUsage> SupportedUsages = GetSupportedUsageContextsForBitmask(UsageBitmask);
		if (SupportedUsages.Contains(ENiagaraScriptUsage::ParticleUpdateScript))
		{
			// Set the simulation stage bit by default to true for old assets if particle update is enabled as well
			uint8 SimulationStageIndex = (uint8)ENiagaraScriptUsage::ParticleSimulationStageScript;
			UsageBitmask |= (1 << SimulationStageIndex);
		}
	}

	const bool bRequestHierarchyRootMigration = bMigrateParameterDataToHierarchyRoot;
	bMigrateParameterDataToHierarchyRoot = false;

	VersionedScriptAdapters.Reserve(VersionData.Num());
	for (FVersionedNiagaraScriptData& Data : VersionData)
	{		
		UNiagaraScriptSourceBase* Source = Data.Source;
		if (Source == nullptr)
		{
			continue;
		}

		Source->ConditionalPostLoad();

		if (bRequestHierarchyRootMigration)
		{
			Source->MigrateParameterDataToHierarchyRoot(Data);
		}

		// Synchronize with Definitions after source scripts have been postloaded.
		FVersionedNiagaraScript& VersionedScriptAdapter = VersionedScriptAdapters.Emplace_GetRef(this, Data.Version.VersionGuid);
		VersionedScriptAdapter.PostLoadDefinitionsSubscriptions(
			MakeWeakObjectPtr(this),
			[](UObject* ObjectOwner) -> TArray<INiagaraParameterDefinitionsSubscriber*>
			{
				TArray<INiagaraParameterDefinitionsSubscriber*> OutArray;
				if (UNiagaraScript* Owner = Cast<UNiagaraScript>(ObjectOwner))
				{
					OutArray.Reserve(Owner->VersionedScriptAdapters.Num());
					for (FVersionedNiagaraScript& Adapter : Owner->VersionedScriptAdapters)
					{
						OutArray.Add(&Adapter);
					}
				}
				return OutArray;
			}
		);

		bool bScriptVMNeedsRebuild = false;
		FString RebuildReason;
		if (NiagaraVer < FNiagaraCustomVersion::UseHashesToIdentifyCompileStateOfTopLevelScripts && CachedScriptVMId.CompilerVersionID.IsValid())
		{
			FGuid BaseId = Source->GetCompileBaseId(Usage, UsageId);
			if (BaseId.IsValid() == false)
			{
				UE_LOGF(LogNiagara, Warning,
					"Invalidating compile ids for script %ls because it doesn't have a valid base id.  The owning asset will continue to compile on load until it is resaved.",
					*GetPathName());
				InvalidateCompileResults(TEXT("Script didn't have a valid base id."));
				Source->ForceGraphToRecompileOnNextCheck();
			}
			else
			{
				FNiagaraCompileHash CompileHash = Source->GetCompileHash(Usage, UsageId);
				if (CompileHash.IsValid())
				{
					CachedScriptVMId.BaseScriptCompileHash = CompileHash;
				}
				else
				{
					// If the compile hash isn't valid, the vm id needs to be recalculated and the cached vm needs to be invalidated.
					bScriptVMNeedsRebuild = true;
					RebuildReason = TEXT("Script did not have a valid compile hash.");
				}
			}
		}

		if (CachedScriptVMId.CompilerVersionID.IsValid() && CachedScriptVMId.CompilerVersionID != FNiagaraCustomVersion::GetLatestScriptCompileVersion())
		{
			bScriptVMNeedsRebuild = true;
			RebuildReason = TEXT("Niagara compiler version changed since the last time the script was compiled.");
		}

		if (bScriptVMNeedsRebuild)
		{
        		// We modify here in post load so that it will cause the owning asset to resave when running the resave commandlet.
			Modify();
			InvalidateCompileResults(RebuildReason);

			// Older versions of graphs require the script to drive the CachedCompileIds being fully refreshed when the script version changes.
			// In order to bridge the gap without requiring a new package version we trigger the refresh here
			Source->RefreshGraphCompileId();
		}

		// Convert visibility of old assets
		if (NiagaraVer < FNiagaraCustomVersion::AddLibraryAssetProperty || (NiagaraVer < FNiagaraCustomVersion::AddLibraryVisibilityProperty && bExposeToLibrary_DEPRECATED))
		{
			ScriptData->LibraryVisibility = ENiagaraScriptLibraryVisibility::Library;
		}
	}
#endif

	ProcessSerializedShaderMaps();

	GenerateStatIDs();
}

#if WITH_EDITORONLY_DATA
void UNiagaraScript::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(TEXT("/Script/NiagaraEditor.NiagaraHierarchyScriptParameter"))); // cannot use NiagaraHierarchyScriptParameter::StaticClass() here as this type is in an editor module
}
#endif

bool UNiagaraScript::IsReadyToRun(ENiagaraSimTarget SimTarget) const
{
	if (SimTarget == ENiagaraSimTarget::CPUSim)
	{
		if (CachedScriptVM.IsValid())
		{
			return true;
		}
	}
	else if (SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		return CanBeRunOnGpu();
	}

	return false;
}

#if WITH_EDITORONLY_DATA
void UNiagaraScript::SetPreviewFeatureLevel(ERHIFeatureLevel::Type InPreviewFeatureLevel)
{
	// Anything to do?
	if ( NiagaraScriptInternal::PreviewFeatureLevel.Get(GMaxRHIFeatureLevel) == InPreviewFeatureLevel )
	{
		return;
	}

	// Either Set of Clear the Preview Feature Level
	if ( InPreviewFeatureLevel == GMaxRHIFeatureLevel )
	{
		NiagaraScriptInternal::PreviewFeatureLevel.Reset();
	}
	else
	{
		NiagaraScriptInternal::PreviewFeatureLevel = InPreviewFeatureLevel;
	}

	// We need to invalid IsCooked if we are changing preview level as the scripts can get out of sync
	for (TObjectIterator<UNiagaraScript> It; It; ++It)
	{
		if (UNiagaraScript* Script = *It)
		{
			Script->IsCooked = false;
		}
	}

	// Force scripts to recache that can run on the GPU and already have a script resource
	if (!AreGpuScriptsCompiledBySystem())
	{
		for (TObjectIterator<UNiagaraScript> It; It; ++It)
		{
			UNiagaraScript* Script = *It;
			if ( Script == nullptr || !Script->OwnerCanBeRunOnGpu() || Script->ScriptResource == nullptr )
			{
				continue;
			}

			for (TSharedPtr<FNiagaraShaderScript>& ExistingScript : Script->ScriptResourcesByFeatureLevel )
			{
				if (ExistingScript == Script->ScriptResource)
				{
					Script->CacheResourceShadersForRendering(true, false);
					break;
				}
			}
		}
	}
	else
	{
		for (TObjectIterator<UNiagaraSystem> It; It; ++It)
		{
			if (UNiagaraSystem* System = *It)
			{
				if (System->bFullyLoaded && System->HasAnyGPUEmitters())
				{
					System->RequestCompile(false);
				}
			}
		}
	}
}

ERHIFeatureLevel::Type UNiagaraScript::GetPreviewFeatureLevel()
{
	return NiagaraScriptInternal::PreviewFeatureLevel.Get(GMaxRHIFeatureLevel);
}

#endif

void UNiagaraScript::GenerateStatIDs()
{
#if STATS
	StatScopesIDs.Empty();
	if (IsReadyToRun(ENiagaraSimTarget::CPUSim))
	{
		StatScopesIDs.Reserve(CachedScriptVM.StatScopes.Num());
		for (FNiagaraStatScope& StatScope : CachedScriptVM.StatScopes)
		{
			StatScopesIDs.Add(FDynamicStats::CreateStatId<FStatGroup_STATGROUP_NiagaraDetailed>(StatScope.FriendlyName.ToString()));
		}
	}
#elif ENABLE_STATNAMEDEVENTS
	StatNamedEvents.Empty();
#endif
}

#if WITH_EDITOR

void UNiagaraScript::PostEditUndo()
{
	Super::PostEditUndo();

	FNiagaraSystemUpdateContext UpdateContext;
	UpdateContext.SetDestroyOnAdd(true);
	UpdateContext.Add(this, true);

	InvalidateCompileResults(TEXT("Post Undo"));
}

void UNiagaraScript::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (!AreGpuScriptsCompiledBySystem())
	{
		CacheResourceShadersForRendering(true);
	}

	CustomAssetRegistryTagCache.Reset();
	OnPropertyChangedDelegate.Broadcast(PropertyChangedEvent);
}

void UNiagaraScript::PostEditChangeVersionedProperty(FPropertyChangedEvent& PropertyChangedEvent, const FGuid& Version)
{
	FName PropertyName = PropertyChangedEvent.Property->GetFName();

	if (UNiagaraScriptSourceBase* Source = GetSource(Version))
	{
		FString UnsynchronizedMessage = TEXT("Script property changed");
		if (PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, bDeprecated) || 
         PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, DeprecationMessage) ||
         PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, DeprecationRecommendation))
		{
			UnsynchronizedMessage = TEXT("Deprecation changed.");
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, bExperimental) || 
            PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, ExperimentalMessage))
		{
			UnsynchronizedMessage = TEXT("Experimental changed.");
		}
	
		if (PropertyName == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, NoteMessage))
		{
			UnsynchronizedMessage = TEXT("Note changed.");
		}

		Source->MarkNotSynchronized(UnsynchronizedMessage);
	}

	PostEditChangeProperty(PropertyChangedEvent);
}

#endif

#if WITH_EDITORONLY_DATA
UNiagaraScriptSourceBase* UNiagaraScript::GetSource(const FGuid& VersionGuid)
{
	if (VersionData.Num() == 0)
	{
		// this should only happen when loading old assets where our PostLoad method was not yet called 
		return Source_DEPRECATED;
	}
	FVersionedNiagaraScriptData* ScriptData = GetScriptData(VersionGuid);
	ensure(ScriptData);
	return ScriptData ? ScriptData->Source : nullptr;
}

const UNiagaraScriptSourceBase* UNiagaraScript::GetSource(const FGuid& VersionGuid) const
{
	return const_cast<UNiagaraScript*>(this)->GetSource(VersionGuid);
}

UNiagaraScriptSourceBase* UNiagaraScript::GetLatestSource()
{
	return GetSource(FGuid());
}

const UNiagaraScriptSourceBase* UNiagaraScript::GetLatestSource() const
{
	return GetSource(FGuid());
}

void UNiagaraScript::SetLatestSource(UNiagaraScriptSourceBase* InSource)
{
	SetSource(InSource, FGuid());
}

void UNiagaraScript::SetSource(UNiagaraScriptSourceBase* InSource, const FGuid& VersionGuid)
{
	CheckVersionDataAvailable();
	GetScriptData(VersionGuid)->Source = InSource;
}

bool UNiagaraScript::AreScriptAndSourceSynchronized(const FGuid& VersionGuid) const
{
	if (!AllowShaderCompiling())
	{
		return false;
	}

	const FVersionedNiagaraScriptData* ScriptData = GetScriptData(VersionGuid);
	if (ScriptData && ScriptData->Source)
	{
		FNiagaraVMExecutableDataId NewId;
		FNiagaraScriptHashCollector HashCollector(GEnableVerboseNiagaraChangeIdLogging > 0);
		ComputeVMCompilationId(NewId, VersionGuid, &HashCollector);

		bool bSynchronized = (NewId.IsValid() && NewId == CachedScriptVMId);
		if (!bSynchronized && NewId.IsValid() && CachedScriptVMId.IsValid() && CachedScriptVM.IsValid())
		{
			if (NewId != LastReportedVMId)
			{
				if (GEnableVerboseNiagaraChangeIdLogging)
				{
					if (NewId.BaseScriptCompileHash != CachedScriptVMId.BaseScriptCompileHash)
					{
						UE_LOGF(LogNiagara, Log, "AreScriptAndSourceSynchronized base script compile hashes don't match. %ls != %ls, script %ls",
							*NewId.BaseScriptCompileHash.ToString(), *CachedScriptVMId.BaseScriptCompileHash.ToString(), *GetPathName());
					}

					if (NewId.ReferencedCompileHashes.Num() != CachedScriptVMId.ReferencedCompileHashes.Num())
					{
						UE_LOGF(LogNiagara, Log, "AreScriptAndSourceSynchronized num referenced compile hashes don't match. %d != %d, script %ls",
							NewId.ReferencedCompileHashes.Num(), CachedScriptVMId.ReferencedCompileHashes.Num(), *GetPathName());
					}
					else
					{
						for (int32 i = 0; i < NewId.ReferencedCompileHashes.Num(); i++)
						{
							if (NewId.ReferencedCompileHashes[i] != CachedScriptVMId.ReferencedCompileHashes[i])
							{
								UE_LOGF(LogNiagara, Log, "AreScriptAndSourceSynchronized referenced compile hash %d doesn't match. %ls != %ls, script %ls, source %ls",
									i, *NewId.ReferencedCompileHashes[i].ToString(), *CachedScriptVMId.ReferencedCompileHashes[i].ToString(), *GetPathName(),
									*HashCollector.BuildCompileHashSourceString(NewId.ReferencedCompileHashes[i]));
							}
						}
					}
				}
				LastReportedVMId = NewId;
			}
		}

		return bSynchronized;
	}
	return false;
}

void UNiagaraScript::MarkScriptAndSourceDesynchronized(FString Reason, const FGuid& VersionGuid)
{
	if (UNiagaraScriptSourceBase* Source = GetSource(VersionGuid))
	{
		Source->MarkNotSynchronized(Reason);
	}
}

bool UNiagaraScript::IsScriptShaderSynchronized(const FGuid& VersionGuid) const
{
	if (!AllowShaderCompiling())
	{
		return false;
	}

	if (!IsGPUScript())
	{
		return true;
	}

	if (!ScriptResource.IsValid() || !ScriptResource->IsShaderMapComplete())
	{
		return false;
	}

	return true;
}

bool UNiagaraScript::HandleVariableRenames(const TMap<FNiagaraVariable, FNiagaraVariable>& OldToNewVars, const FString& UniqueEmitterName)
{
	bool bConvertedAnything = false;
	auto Iter = OldToNewVars.CreateConstIterator();
	while (Iter)
	{
		// Sometimes the script is under the generic name, other times it has been converted to the unique emitter name. Handle both cases below...
		FNiagaraVariable RISrcVarA = FNiagaraUtilities::ConvertVariableToRapidIterationConstantName(Iter->Key, !UniqueEmitterName.IsEmpty() ? TEXT("Emitter") : nullptr , GetUsage());
		FNiagaraVariable RISrcVarB = FNiagaraUtilities::ConvertVariableToRapidIterationConstantName(Iter->Key, !UniqueEmitterName.IsEmpty() ? *UniqueEmitterName : nullptr, GetUsage());
		FNiagaraVariable RIDestVarA = FNiagaraUtilities::ConvertVariableToRapidIterationConstantName(Iter->Value, !UniqueEmitterName.IsEmpty() ? TEXT("Emitter") : nullptr, GetUsage());
		FNiagaraVariable RIDestVarB = FNiagaraUtilities::ConvertVariableToRapidIterationConstantName(Iter->Value, !UniqueEmitterName.IsEmpty() ? *UniqueEmitterName : nullptr, GetUsage());

		{
			if (nullptr != RapidIterationParameters.FindParameterOffset(RISrcVarA))
			{
				RapidIterationParameters.RenameParameter(RISrcVarA, RIDestVarA.GetName());
				UE_LOGF(LogNiagara, Log, "Converted RI variable \"%ls\" to \"%ls\" in Script \"%ls\"", *RISrcVarA.GetName().ToString(), *RIDestVarA.GetName().ToString(), *GetFullName());
				bConvertedAnything = true;
			}
			else if (nullptr != RapidIterationParameters.FindParameterOffset(RISrcVarB))
			{
				RapidIterationParameters.RenameParameter(RISrcVarB, RIDestVarB.GetName());
				UE_LOGF(LogNiagara, Log, "Converted RI variable \"%ls\" to \"%ls\" in Script \"%ls\"", *RISrcVarB.GetName().ToString(), *RIDestVarB.GetName().ToString(), *GetFullName());
				bConvertedAnything = true;
			}
		}

		{
			// Go ahead and convert the stored VM executable data too. I'm not 100% sure why this is necessary, since we should be recompiling.
			int32 VarIdx = GetVMExecutableData().Parameters.Parameters.IndexOfByKey(RISrcVarA);
			if (VarIdx != INDEX_NONE)
			{
				GetVMExecutableData().Parameters.Parameters[VarIdx].SetName(RIDestVarA.GetName());
				UE_LOGF(LogNiagara, Log, "Converted exec param variable \"%ls\" to \"%ls\" in Script \"%ls\"", *RISrcVarA.GetName().ToString(), *RIDestVarA.GetName().ToString(), *GetFullName());
				bConvertedAnything = true;
			}

			VarIdx = GetVMExecutableData().Parameters.Parameters.IndexOfByKey(RISrcVarB);
			if (VarIdx != INDEX_NONE)
			{
				GetVMExecutableData().Parameters.Parameters[VarIdx].SetName(RIDestVarB.GetName());
				UE_LOGF(LogNiagara, Log, "Converted exec param  variable \"%ls\" to \"%ls\" in Script \"%ls\"", *RISrcVarB.GetName().ToString(), *RIDestVarB.GetName().ToString(), *GetFullName());
				bConvertedAnything = true;
			}
		}

		{
			// Also handle any data set mappings...
			auto DS2PIterator = GetVMExecutableData().DataSetToParameters.CreateIterator();
			while (DS2PIterator)
			{
				for (int32 i = 0; i < DS2PIterator.Value().Parameters.Num(); i++)
				{
					FNiagaraVariable Var = DS2PIterator.Value().Parameters[i];
					if (Var == RISrcVarA)
					{
						DS2PIterator.Value().Parameters[i].SetName(RIDestVarA.GetName());
						bConvertedAnything = true;
					}
					else if (Var == RISrcVarB)
					{
						DS2PIterator.Value().Parameters[i].SetName(RIDestVarB.GetName());
						bConvertedAnything = true;
					}
				}
				++DS2PIterator;
			}
		}
		++Iter;
	}

	return bConvertedAnything;
}

static bool ValidateExecData(const UNiagaraScript* Script, const FNiagaraVMExecutableData& ExecData, FString& ErrorString)
{
	bool IsValid = true;

	for (const auto& Attribute : ExecData.Attributes)
	{
		if (!Attribute.IsValid())
		{
			ErrorString.Appendf(TEXT("Failure - %s - Attribute [%s] is invalid!\n"), Script ? *Script->GetFullName() : TEXT("<unknown>"), *Attribute.GetName().ToString());
			IsValid = false;
		}
	}

	for (const auto& Parameter : ExecData.Parameters.Parameters)
	{
		if (!Parameter.IsValid())
		{
			ErrorString.Appendf(TEXT("Failure - %s - Parameter [%s] is invalid!\n"), Script ? *Script->GetFullName() : TEXT("<unknown>"), *Parameter.GetName().ToString());
			IsValid = false;
		}
	}

	return IsValid;
}

bool UNiagaraScript::ShouldUseDDC()
{
	static const bool bNoNiagaraDDC = FParse::Param(FCommandLine::Get(), TEXT("noniagaraddc"));

	return !bNoNiagaraDDC;
}

bool UNiagaraScript::BinaryToExecData(const UNiagaraScript* Script, const TArray<uint8>& InBinaryData, FNiagaraVMExecutableData& OutExecData)
{
	check(IsInGameThread());
	if (InBinaryData.Num() == 0)
	{
		return false;
	}

	if (!ShouldUseDDC())
	{
		return false;
	}

	FMemoryReader Ar(InBinaryData, true);

	// Read the archive version from the header of the payload so we can setup our reader with the right version
	FPackageFileVersion ArchiveVersion;
	Ar << ArchiveVersion;

	if (!ArchiveVersion.IsCompatible(GOldestLoadablePackageFileUEVersion))
	{
		UE_LOGF(LogNiagara, Display, "Failed to validate FNiagaraVMExecutableData received from DDC, rejecting!  Reasons:\nDeprecated object version");
		return false;
	}

	FObjectAndNameAsStringProxyArchive SafeAr(Ar, false);
	SafeAr.SetUEVer(ArchiveVersion);

	OutExecData.SerializeData(SafeAr, true);

	FString ValidationErrors;
	if (!ValidateExecData(Script, OutExecData, ValidationErrors))
	{
		UE_LOGF(LogNiagara, Display, "Failed to validate FNiagaraVMExecutableData received from DDC, rejecting!  Reasons:\n%ls", *ValidationErrors);
		return false;
	}

	return !SafeAr.IsError();
}

bool UNiagaraScript::ExecToBinaryData(const UNiagaraScript* Script, TArray<uint8>& OutBinaryData, FNiagaraVMExecutableData& InExecData)
{
	check(IsInGameThread());

	if (!ShouldUseDDC())
	{
		return false;
	}

	FString ValidationErrors;
	if (!ValidateExecData(Script, InExecData, ValidationErrors))
	{
		UE_LOGF(LogNiagara, Error, "Failed to validate FNiagaraVMExecutableData being pushed to DDC, rejecting!  Errors:\n%ls", *ValidationErrors);
		return false;
	}

	FMemoryWriter Ar(OutBinaryData, true);

	// include the archive version into the payload since we're using struct serialization for the ExecData
	FPackageFileVersion ArchiveVersion = GPackageFileUEVersion;
	Ar << ArchiveVersion;

	FObjectAndNameAsStringProxyArchive SafeAr(Ar, false);
	InExecData.SerializeData(SafeAr, true);

	return OutBinaryData.Num() != 0 && !SafeAr.IsError();
}

void WriteTextFileToDisk(FString SaveDirectory, FString FileName, FString TextToSave, bool bAllowOverwriting)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	// CreateDirectoryTree returns true if the destination
	// directory existed prior to call or has been created
	// during the call.
	if (PlatformFile.CreateDirectoryTree(*SaveDirectory))
	{
		// Get absolute file path
		FString AbsoluteFilePath = SaveDirectory + "/" + FileName;

		// Allow overwriting or file doesn't already exist
		if (bAllowOverwriting || !PlatformFile.FileExists(*AbsoluteFilePath))
		{
			if (FFileHelper::SaveStringToFile(TextToSave, *AbsoluteFilePath))
			{
				UE_LOGF(LogNiagara, Log, "Wrote file to %ls", *AbsoluteFilePath);
				return;
			}

		}
	}
}

UNiagaraDataInterface* UNiagaraScript::CopyDataInterface(UNiagaraDataInterface* Src, UObject* Owner)
{
	if (Src)
	{
		const EObjectFlags DataInterfaceObjectFlags = UNiagaraDataInterface::BuildObjectFlagsForOwner(Owner, RF_Transactional);
		UNiagaraDataInterface* DI = NewObject<UNiagaraDataInterface>(Owner, const_cast<UClass*>(Src->GetClass()), NAME_None, DataInterfaceObjectFlags);
		Src->CopyTo(DI);
		return DI;
	}
	return nullptr;
}

#if WITH_EDITORONLY_DATA
const FVersionedNiagaraScript FVersionedNiagaraScriptWeakPtr::Pin() const
{
	if (Script.IsValid())
	{
		return FVersionedNiagaraScript(Script.Get(), Version);
	}
	return FVersionedNiagaraScript();
}

FVersionedNiagaraScript FVersionedNiagaraScriptWeakPtr::Pin()
{
	if (Script.IsValid())
	{
		FVersionedNiagaraScript PinnedVersionedNiagaraScript = FVersionedNiagaraScript(Script.Get(), Version);
		return PinnedVersionedNiagaraScript;
	}
	return FVersionedNiagaraScript();
}

TArray<UNiagaraScriptSourceBase*> FVersionedNiagaraScriptWeakPtr::GetAllSourceScripts()
{
	if (Script.IsValid())
	{
		return { Script.Get()->GetSource(Version) };
	}
	return TArray<UNiagaraScriptSourceBase*>();
}

FString FVersionedNiagaraScriptWeakPtr::GetSourceObjectPathName() const
{
	return Script.IsValid() ? Script.Get()->GetPathName() : FString();
}

TArray<UNiagaraScriptSourceBase*> FVersionedNiagaraScript::GetAllSourceScripts()
{
	if (Script != nullptr)
	{
		return { Script->GetSource(Version) };
	}
	return TArray<UNiagaraScriptSourceBase*>();
}

FString FVersionedNiagaraScript::GetSourceObjectPathName() const
{
	return Script ? Script->GetPathName() : FString();
}

FVersionedNiagaraScriptWeakPtr FVersionedNiagaraScript::ToWeakPtr()
{
	FVersionedNiagaraScriptWeakPtr WeakVersionedNiagaraScript = FVersionedNiagaraScriptWeakPtr(Script, Version);
	return WeakVersionedNiagaraScript;
}

FVersionedNiagaraScriptData* FVersionedNiagaraScript::GetScriptData() const
{
	return Script ? Script->GetScriptData(Version) : nullptr;
}
#endif

UNiagaraDataInterface* ResolveDataInterface(const TMap<FName, UNiagaraDataInterface*>& ObjectNameMap, FName VariableName)
{
	UNiagaraDataInterface* const* FoundDI = ObjectNameMap.Find(VariableName);
	if (FoundDI && *(FoundDI))
	{
		return *FoundDI;
	}
	return nullptr;
}

void DumpNameMap(const TMap<FName, UNiagaraDataInterface*>& ObjectNameMap)
{
	for (const TPair<FName, UNiagaraDataInterface*>& Pair : ObjectNameMap)
	{
		UE_LOGF(LogNiagara, Log, "%ls -> %ls", *Pair.Key.ToString(), *GetPathNameSafe(Pair.Value));
	}
}

FName ResolveEmitterAlias(const FName& InName, const FString& InAlias)
{
	// If the alias is empty than the name can't be resolved.
	if (InAlias.IsEmpty())
	{
		return InName;
	}

	FNiagaraVariable Var(FNiagaraTypeDefinition::GetFloatDef(), InName);
	FNiagaraAliasContext ResolveAliasesContext(FNiagaraAliasContext::ERapidIterationParameterMode::EmitterOrParticleScript);
	ResolveAliasesContext.ChangeEmitterToEmitterName(InAlias);
	Var = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext);
	return Var.GetName();
}

void UNiagaraScript::SetVMCompilationResults(const FNiagaraVMExecutableDataId& InCompileId, FNiagaraVMExecutableData& InScriptVM, const FString& EmitterUniqueName, const TMap<FName, UNiagaraDataInterface*>& ObjectNameMap, bool bApplyRapidIterationParameters)
{
	CachedScriptVMId = InCompileId;
	CachedScriptVM = InScriptVM;
	CachedParameterCollectionReferences.Empty();
	// Proactively clear out the script resource, because it might be stale now.
	ScriptResource->Invalidate();

	if (CachedScriptVM.LastCompileStatus == ENiagaraScriptCompileStatus::NCS_Error)
	{
		// Compiler errors for Niagara will have a strong UI impact but the game should still function properly, there
		// will just be oddities in the visuals. It should be acted upon, but in no way should the game be blocked from
		// a successful cook because of it. Therefore, we do a warning.
		if (!CachedScriptVM.ErrorMsg.IsEmpty())
		{
			UE_ASSET_LOG(LogNiagara, Warning, this, TEXT("%s"), *CachedScriptVM.ErrorMsg);
		}
	}

	ResolveParameterCollectionReferences();
	InvalidateAndEmptyDefaultDataInterfaces(CachedScriptVM.DataInterfaceInfo.Num(), EDataInterfaceInvalidationMode::ProcessInvaliation);
	for (const FNiagaraScriptDataInterfaceCompileInfo& Info : CachedScriptVM.DataInterfaceInfo)
	{
		int32 Idx = CachedDefaultDataInterfaces.AddDefaulted();
		CachedDefaultDataInterfaces[Idx].UserPtrIdx = Info.UserPtrIdx;
		CachedDefaultDataInterfaces[Idx].Name = ResolveEmitterAlias(Info.Name, EmitterUniqueName);
		CachedDefaultDataInterfaces[Idx].CompileName = Info.Name;
		CachedDefaultDataInterfaces[Idx].Type = Info.Type;
		CachedDefaultDataInterfaces[Idx].SourceEmitterName = Info.SourceEmitterName;
		CachedDefaultDataInterfaces[Idx].RegisteredParameterMapRead = ResolveEmitterAlias(Info.RegisteredParameterMapRead, EmitterUniqueName);
		CachedDefaultDataInterfaces[Idx].RegisteredParameterMapWrite = ResolveEmitterAlias(Info.RegisteredParameterMapWrite, EmitterUniqueName);

		// We compiled it just a bit ago, so we should be able to resolve it from the table that we passed in.
		UNiagaraDataInterface* FindDIById = ResolveDataInterface(ObjectNameMap, CachedDefaultDataInterfaces[Idx].Name);
		if (FindDIById != nullptr )
		{
			CachedDefaultDataInterfaces[Idx].DataInterface = CopyDataInterface(FindDIById, this);
			check(CachedDefaultDataInterfaces[Idx].DataInterface != nullptr);
		}

		if (CachedDefaultDataInterfaces[Idx].DataInterface == nullptr)
		{
			if ( Info.Type.IsValid() )
			{
				// Use the CDO since we didn't have a default..
				UObject* Obj = Info.Type.GetClass()->GetDefaultObject(true);
				CachedDefaultDataInterfaces[Idx].DataInterface = Cast<UNiagaraDataInterface>(CopyDataInterface(CastChecked<UNiagaraDataInterface>(Obj), this));

				if (Info.bIsPlaceholder == false)
				{
					UE_LOGF(LogNiagara, Display, "We somehow ended up with a data interface that we couldn't match post compile. This shouldn't happen. Creating a dummy to prevent crashes. DataInterfaceInfoName:%ls Object:%ls", *Info.Name.ToString(), *GetPathNameSafe(this));
					UE_LOGF(LogNiagara, Log, "Object to Name map contents:");
					DumpNameMap(ObjectNameMap);
				}
				check(CachedDefaultDataInterfaces[Idx].DataInterface != nullptr);
			}
			else
			{
				UE_LOGF(LogNiagara, Log, "Data Interface has no class, was it deleted? DataInterfaceInfoName:%ls Object:%ls", *Info.Name.ToString(), *GetPathNameSafe(this));
			}
		}
	}

	CachedDefaultUObjects = CachedScriptVM.UObjectInfos;

	// if the compile process generated soft object paths instead of hard references than resolve them here
	for (FNiagaraScriptUObjectCompileInfo& DefaultUObjectInfo : CachedDefaultUObjects)
	{
		if (DefaultUObjectInfo.ObjectPath.IsValid() && !DefaultUObjectInfo.Object)
		{
			DefaultUObjectInfo.Object = DefaultUObjectInfo.ObjectPath.ResolveObject();
		}
	}

	if (bApplyRapidIterationParameters)
	{
		// if we are applying rapid iteration parameters then the data supplied by InScriptVM, which could have
		// come from the DDC, could invalidate the CompileID that we had previous generated so generate it again
		// now that we've updated everything
		ComputeVMCompilationId(CachedScriptVMId, FGuid());
	}

	GenerateStatIDs();

	// Now go ahead and trigger the GPU script compile now that we have a compiled GPU hlsl script.
	if (!AreGpuScriptsCompiledBySystem())
	{
		if (Usage == ENiagaraScriptUsage::ParticleGPUComputeScript)
		{
			if (CachedScriptVMId.CompilerVersionID.IsValid() && CachedScriptVMId.BaseScriptCompileHash.IsValid())
			{
				CacheResourceShadersForRendering(false, true);
			}
			else
			{
				UE_LOGF(LogNiagara, Warning,
					"Could not cache resource shaders for rendering for script %ls because it had an invalid cached script id. This should be fixed by force recompiling the owning asset using the 'Full Rebuild' option and then saving the asset.",
					*GetPathName());
			}
		}
	}

	OnVMScriptCompiled().Broadcast(this, InCompileId.ScriptVersionID);
}

void UNiagaraScript::ProcessCompilationErrors(bool bCompileSucceeded, TConstArrayView<FShaderCompilerError> CompilationErrors, TArray<FString>& ProcessedErrors)
{
	ProcessedErrors.Reserve(CompilationErrors.Num());

	if (CompilationErrors.IsEmpty())
	{
		return;
	}

	for (const FShaderCompilerError& CompilationError : CompilationErrors)
	{
		ProcessedErrors.Add(CompilationError.GetErrorString().Replace(TEXT("Error"), TEXT("Err0r")));
	}

	static const IConsoleVariable* ShowShaderWarningsVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("niagara.ShowShaderCompilerWarnings"));

	if (!bCompileSucceeded || (ShowShaderWarningsVariable && ShowShaderWarningsVariable->GetBool()))
	{
		UE_LOGF(LogNiagara, Display, "There were issues for compilation of script \"%ls\"", *GetFriendlyName());

		if (!LogNiagara.IsSuppressed(ELogVerbosity::Verbose))
		{
			TArray<FString> OutputByLines;
			CachedScriptVM.LastHlslTranslationGPU.ParseIntoArrayLines(OutputByLines, false);

			UE_LOGF(LogNiagara, Verbose, "Compile output as text:");
			UE_LOGF(LogNiagara, Verbose, "===================================================================================");
			for (int32 i = 0; i < OutputByLines.Num(); i++)
			{
				UE_LOGF(LogNiagara, Verbose, "/*%04d*/\t\t%ls", i + 1, *OutputByLines[i]);
			}
			UE_LOGF(LogNiagara, Verbose, "===================================================================================");
		}

		for (const FShaderCompilerError& Error : CompilationErrors)
		{
			UE_LOGF(LogNiagara, Warning, "%ls", *Error.GetErrorString())
		}
	}
}

void UNiagaraScript::SetComputeCompilationResults(
	const ITargetPlatform* TargetPlatform,
	EShaderPlatform ShaderPlatform,
	ERHIFeatureLevel::Type FeatureLevel,
	const FNiagaraShaderScriptParametersMetadata& ShaderParameters,
	const FNiagaraShaderMapRef& ShaderMap,
	TConstArrayView<FShaderCompilerError> CompilationErrors)
{
	const bool bCompilationSuccessful = ShaderMap.IsValid() && ShaderMap->IsValid();

	TArray<FString> ProcessedErrors;
	ProcessCompilationErrors(bCompilationSuccessful, CompilationErrors, ProcessedErrors);

	// for now we will use the CachedScriptResourcesForCooking for storing all compilation results and will update ScriptResource
	// if it matches with the supplied parameters
	auto ScriptMatches = [FeatureLevel, ShaderPlatform](const TUniquePtr<FNiagaraShaderScript>& ShaderScript) -> bool
	{
		return ShaderScript->GetFeatureLevel() == FeatureLevel && ShaderScript->GetShaderPlatform() == ShaderPlatform;
	};

	TArray<TUniquePtr<FNiagaraShaderScript>>& CachedScripts = CachedScriptResourcesForCooking.FindOrAdd(TargetPlatform);
	FNiagaraShaderScript* TargetShaderScript = nullptr;

	if (TUniquePtr<FNiagaraShaderScript>* ExistingScript = CachedScripts.FindByPredicate(ScriptMatches))
	{
		TargetShaderScript = ExistingScript->Get();
	}
	else
	{
		TargetShaderScript = CachedScripts.Emplace_GetRef(MakeUnique<FNiagaraShaderScript>()).Get();
	}

	auto UpdateShaderScript = [this, &ProcessedErrors, &ShaderParameters, FeatureLevel, ShaderPlatform](FNiagaraShaderScript* ScriptToUpdate, FNiagaraShaderMap* InShaderMap) -> void
	{
		ScriptToUpdate->SetCompileErrors(ProcessedErrors);
		ScriptToUpdate->BuildScriptParametersMetadata(ShaderParameters);
		ScriptToUpdate->SetGameThreadShaderMap(InShaderMap);
		ScriptToUpdate->SetScript(this, FeatureLevel, ShaderPlatform, CachedScriptVMId.CompilerVersionID, CachedScriptVMId.AdditionalDefines,
			CachedScriptVMId.GetAdditionalVariableStrings(),
			CachedScriptVMId.BaseScriptCompileHash, CachedScriptVMId.ReferencedCompileHashes,
			CachedScriptVMId.bUsesRapidIterationParams, GetFriendlyName());
	};

	if (ensure(TargetShaderScript))
	{
		FNiagaraShaderMap* ShaderMapToApply = bCompilationSuccessful ? ShaderMap.GetReference() : nullptr;

		UpdateShaderScript(TargetShaderScript, ShaderMapToApply);

		// next see if we need to update the currently active ShaderScript (and update it's renderthread data)
		{
			const ERHIFeatureLevel::Type ActiveFeatureLevel = NiagaraScriptInternal::PreviewFeatureLevel.Get(GMaxRHIFeatureLevel);
			const EShaderPlatform ActiveShaderPlatform = GShaderPlatformForFeatureLevel[ActiveFeatureLevel];

			if (ActiveFeatureLevel == FeatureLevel && ActiveShaderPlatform == ShaderPlatform)
			{
				if (!ScriptResource.IsValid())
				{
					ScriptResource = MakeShared<FNiagaraShaderScript>(*TargetShaderScript);
				}
				else
				{
					UpdateShaderScript(ScriptResource.Get(), ShaderMapToApply);
				}

				ENQUEUE_RENDER_COMMAND(FSetShaderMapOnScriptResources)(
					[RT_Script = ScriptResource, RT_ShaderMap = ShaderMapToApply](FRHICommandListImmediate& RHICmdList)
				{
					RT_Script->SetRenderingThreadShaderMap(RT_ShaderMap);
				});
			}
		}
	}
}

bool UNiagaraScript::IsShaderMapCached(const ITargetPlatform* TargetPlatform, const FNiagaraShaderMapId& ShaderMapId) const
{
	if (TargetPlatform)
	{
		if (const TArray<TUniquePtr<FNiagaraShaderScript>>* CachedShaders = CachedScriptResourcesForCooking.Find(TargetPlatform))
		{
			for (const TUniquePtr<FNiagaraShaderScript>& CachedShader : *CachedShaders)
			{
				if (CachedShader.IsValid() && CachedShader->IsSame(ShaderMapId))
				{
					return CachedShader->IsShaderMapComplete();
				}
			}
		}
	}
	else
	{
		if (ScriptResource.IsValid())
		{
			return ScriptResource->IsSame(ShaderMapId) && ScriptResource->IsShaderMapComplete();
		}
	}

	return false;
}

void UNiagaraScript::AssignRapidIterationParameters(const FNiagaraVMExecutableData& InScriptVM)
{
	const bool bClearBindings = false;
	RapidIterationParameters.Empty(bClearBindings);
	for (const FNiagaraVariable& Parameter : InScriptVM.BakedRapidIterationParameters)
	{
		const bool bInitialize = false;
		const bool bTriggerRebind = false;
		RapidIterationParameters.AddParameter(Parameter, bInitialize, bTriggerRebind);
	}
	RapidIterationParameters.TriggerOnLayoutChanged();
}

bool UNiagaraScript::ApplyRapidIterationParameters(TConstArrayView<FNiagaraVariable> InParameters, bool bAllowRemoval)
{
	bool bStoreDirty = false;

	{
		FNiagaraParameterStore::FScopedSuppressOnChanged SuppressScope(RapidIterationParameters);

		// go through and remove any parameters that aren't in the supplied parameter list
		if (bAllowRemoval)
		{
			TConstArrayView<FNiagaraVariableWithOffset> ExistingParameters = RapidIterationParameters.ReadParameterVariables();
			TArray<FNiagaraVariableBase> ParametersToRemove;
	
			for (const FNiagaraVariableWithOffset& ExistingParameter : ExistingParameters)
			{
				const bool bStaleParameter = !InParameters.ContainsByPredicate([&ExistingParameter](const FNiagaraVariable& Variable) -> bool
				{
					return Variable == ExistingParameter;
				});

				if (bStaleParameter)
				{
					ParametersToRemove.Add(ExistingParameter);
					bStoreDirty = true;
				}
			}

			for (const FNiagaraVariableBase& ParameterToRemove : ParametersToRemove)
			{
				RapidIterationParameters.RemoveParameter(ParameterToRemove);
			}
		}

		// try to add 
		for (const FNiagaraVariable& Parameter : InParameters)
		{
			constexpr bool bInitInterfaces = false;
			constexpr bool bTriggerRebind = false;

			if (RapidIterationParameters.AddParameter(Parameter, bInitInterfaces, bTriggerRebind, nullptr))
			{
				bStoreDirty = true;
			}
		}
	}

	if (bStoreDirty)
	{
		RapidIterationParameters.Rebind();
	}

	return bStoreDirty;
}

void UNiagaraScript::RequestCompile(const FGuid& ScriptVersion, bool bForceCompile)
{
	FVersionedNiagaraScriptData* ScriptData = GetScriptData(ScriptVersion);
	if (ScriptData && (!AreScriptAndSourceSynchronized(ScriptVersion) || bForceCompile))
	{
		check(IsInGameThread());

		FNiagaraVMExecutableDataId& LastGeneratedVMId = GetLastGeneratedVMId(ScriptVersion);
		if (IsCompilable() == false)
		{
			CachedScriptVM.LastCompileStatus = ENiagaraScriptCompileStatus::NCS_Unknown;
			CachedScriptVMId = LastGeneratedVMId;
			return;
		}

		COOK_STAT(auto Timer = NiagaraScriptCookStats::UsageStats.TimeSyncWork());

		CachedScriptVM.LastCompileStatus = ENiagaraScriptCompileStatus::NCS_BeingCreated;

		TArray<TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe>> DependentRequests;
		TArray<uint8> OutData;
		INiagaraModule& NiagaraModule = FModuleManager::Get().LoadModuleChecked<INiagaraModule>(TEXT("Niagara"));

		TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> RequestData = NiagaraModule.Precompile(this, ScriptVersion);
		if (RequestData.IsValid() == false)
		{
			COOK_STAT(Timer.TrackCyclesOnly());
			UE_LOGF(LogNiagara, Error, "Failed to precompile %ls.  This is due to unexpected invalid or broken data.  Additional details should be in the log.", *GetPathName());
			return;
		}

		TArray<FNiagaraVariable> PrecompiledVariables;
		RequestData->GatherPreCompiledVariables(FString(), PrecompiledVariables);
		TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> RequestDuplicateData = NiagaraModule.PrecompileDuplicate(RequestData.Get(), nullptr, nullptr, this, ScriptVersion);
		if (RequestDuplicateData.IsValid() == false)
		{
			COOK_STAT(Timer.TrackCyclesOnly());
			UE_LOGF(LogNiagara, Error, "Failed to precompile duplicate %ls.  This is due to unexpected invalid or broken data.  Additional details should be in the log.", *GetPathName());
			return;
		}

		const FString ScriptPathName = GetPathName();

		// check the ddc first
		if (!bForceCompile && GetDerivedDataCacheRef().GetSynchronous(*GetNiagaraDDCKeyString(ScriptVersion, ScriptPathName), OutData, ScriptPathName))
		{
			FNiagaraVMExecutableData ExeData;
			if (BinaryToExecData(this, OutData, ExeData))
			{
				COOK_STAT(Timer.AddHit(OutData.Num()));
				SetVMCompilationResults(LastGeneratedVMId, ExeData, FString(), RequestDuplicateData->GetObjectNameMap(), false);
				return;
			}
		}

		ActiveCompileRoots.Empty();
		RequestDuplicateData->GetDuplicatedObjects(MutableView(ActiveCompileRoots));

		FNiagaraCompileOptions Options(GetUsage(), GetUsageId(), ScriptData->ModuleUsageBitmask, ScriptPathName, GetFullName(), GetName());
		int32 JobHandle = NiagaraModule.StartScriptCompileJob(RequestData.Get(), RequestDuplicateData.Get(), Options);
		FNiagaraScriptCompileMetrics ScriptMetrics;
		TSharedPtr<FNiagaraVMExecutableData> ExeData = NiagaraModule.GetCompileJobResult(JobHandle, true, ScriptMetrics);
		if (ExeData)
		{
			SetVMCompilationResults(LastGeneratedVMId, *ExeData, FString(), RequestDuplicateData->GetObjectNameMap(), false);
			// save result to the ddc
			if (ExecToBinaryData(this, OutData, *ExeData))
			{
				COOK_STAT(Timer.AddMiss(OutData.Num()));
				GetDerivedDataCacheRef().Put(*GetNiagaraDDCKeyString(ScriptVersion, ScriptPathName), OutData, ScriptPathName);
			}
		}
		ActiveCompileRoots.Empty();
	}
	else
	{
		UE_LOGF(LogNiagara, Verbose, "Script '%ls' is in-sync skipping compile..", *GetFullName());
	}
}

bool UNiagaraScript::RequestExternallyManagedAsyncCompile(const TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe>& RequestData, const TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe>& RequestDuplicateData, FNiagaraVMExecutableDataId& OutCompileId, uint32& OutAsyncHandle)
{
	COOK_STAT(auto Timer = NiagaraScriptCookStats::UsageStats.TimeSyncWork());
	COOK_STAT(Timer.TrackCyclesOnly());

	FNiagaraVMExecutableDataId& LastGeneratedVMId = GetLastGeneratedVMId();
	OutCompileId = LastGeneratedVMId;

	FVersionedNiagaraScriptData* ScriptData = GetScriptData(LastGeneratedVMId.ScriptVersionID);
	if (ensure(ScriptData) && !AreScriptAndSourceSynchronized(LastGeneratedVMId.ScriptVersionID))
	{
		if (IsCompilable() == false)
		{
			OutAsyncHandle = (uint32)INDEX_NONE;
			CachedScriptVM.LastCompileStatus = ENiagaraScriptCompileStatus::NCS_Unknown;
			CachedScriptVMId = LastGeneratedVMId;
			UE_LOGF(LogNiagara, Warning, "Requested compile of script '%ls', which cannot be compiled!", *GetFullName());
			return false;
		}

		INiagaraModule& NiagaraModule = FModuleManager::Get().LoadModuleChecked<INiagaraModule>(TEXT("Niagara"));
		CachedScriptVM.LastCompileStatus = ENiagaraScriptCompileStatus::NCS_BeingCreated;

		FNiagaraCompileOptions Options(GetUsage(), GetUsageId(), ScriptData->ModuleUsageBitmask, GetPathName(), GetFullName(), GetName());
		Options.AdditionalDefines = LastGeneratedVMId.AdditionalDefines;
		Options.AdditionalVariables = LastGeneratedVMId.AdditionalVariables;
		OutAsyncHandle = NiagaraModule.StartScriptCompileJob(RequestData.Get(), RequestDuplicateData.Get(), Options);
		UE_LOGF(LogNiagara, Verbose, "Script '%ls' is requesting compile..", *GetFullName());
		return true;
	}
	else
	{
		OutAsyncHandle = (uint32)INDEX_NONE;
		UE_LOGF(LogNiagara, Verbose, "Script '%ls' is in-sync skipping compile..", *GetFullName());
		return false;
	}
}
#endif

void UNiagaraScript::RaiseOnGPUCompilationComplete()
{
#if WITH_EDITORONLY_DATA
	OnGPUScriptCompiled().Broadcast(this, FGuid());
	FNiagaraSystemUpdateContext(this, true);

	if (FVersionedNiagaraEmitterData* EmitterData = GetOuterEmitter().GetEmitterData())
	{
		EmitterData->CacheFromShaderCompiled();
	}
#endif
}

void UNiagaraScript::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UNiagaraScript::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
#if WITH_EDITORONLY_DATA

	if (CustomAssetRegistryTagCache.IsSet() == false)
	{
		CustomAssetRegistryTagCache = TMap<FName, FString>();
	}

	const FVersionedNiagaraScriptData* ScriptData = GetLatestScriptData();

	// Dependencies
	TArray<FName> ProvidedDependencies = ScriptData ? ScriptData->ProvidedDependencies : ProvidedDependencies_DEPRECATED; 
	if (ProvidedDependencies.Num() > 0)
	{
		FName ProvidedDependenciesName = GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, ProvidedDependencies);
		FString* ProvidedDependenciesTags = CustomAssetRegistryTagCache->Find(ProvidedDependenciesName);
		if(ProvidedDependenciesTags == nullptr)
		{
			ProvidedDependenciesTags = &CustomAssetRegistryTagCache->Add(ProvidedDependenciesName);
			for (FName ProvidedDependency : ProvidedDependencies)
			{
				ProvidedDependenciesTags->Append(ProvidedDependency.ToString() + ",");
			}
		}

		Context.AddTag(FAssetRegistryTag(ProvidedDependenciesName, *ProvidedDependenciesTags, FAssetRegistryTag::TT_Hidden));
	}

	// Category
	FText CategoryText = ScriptData ? ScriptData->Category : Category_DEPRECATED;
	if (!CategoryText.IsEmpty())
	{
		FName CategoryName = GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, Category);
		FString& CategoryTag = CustomAssetRegistryTagCache->FindOrAdd(CategoryName);
		CategoryTag = CategoryText.ToString();

		Context.AddTag(FAssetRegistryTag(CategoryName, CategoryTag, FAssetRegistryTag::TT_Alphabetical));
	}

	// Description
	FText DescriptionText = ScriptData ? ScriptData->Description : Description_DEPRECATED;
	if (!DescriptionText.IsEmpty())
	{
		FName DescriptionName = GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, Description);
		FString& DescriptionTag = CustomAssetRegistryTagCache->FindOrAdd(DescriptionName);
		DescriptionTag = DescriptionText.ToString();

		Context.AddTag(FAssetRegistryTag(DescriptionName, DescriptionTag, FAssetRegistryTag::TT_Alphabetical));
	}

	// Keywords
	FText KeywordsText = ScriptData ? ScriptData->Keywords : Keywords_DEPRECATED;
	if (!KeywordsText.IsEmpty())
	{
		FName KeywordsName = GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, Keywords);
		FString& KeywordsTag = CustomAssetRegistryTagCache->FindOrAdd(KeywordsName);
		KeywordsTag = KeywordsText.ToString();

		Context.AddTag(FAssetRegistryTag(KeywordsName, KeywordsTag, FAssetRegistryTag::TT_Alphabetical));
	}

	// Visibility
	ENiagaraScriptLibraryVisibility Visibility = ScriptData ? ScriptData->LibraryVisibility : LibraryVisibility_DEPRECATED;
	FName VisibilityName = GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, LibraryVisibility);
	FString& VisibilityTag = CustomAssetRegistryTagCache->FindOrAdd(VisibilityName);
	UEnum* VisibilityEnum = StaticEnum<ENiagaraScriptLibraryVisibility>();
	VisibilityTag = VisibilityEnum->GetNameStringByValue((int64)Visibility);
	Context.AddTag(FAssetRegistryTag(VisibilityName, VisibilityTag, FAssetRegistryTag::TT_Alphabetical));

	// Usage bitmask
	int32 UsageBitmask = ScriptData ? ScriptData->ModuleUsageBitmask : ModuleUsageBitmask_DEPRECATED;
	FName UsageBitmaskName = GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, ModuleUsageBitmask);
	FString& UsageBitmaskTag = CustomAssetRegistryTagCache->FindOrAdd(UsageBitmaskName);
	UsageBitmaskTag = FString::FromInt(UsageBitmask);
	Context.AddTag(FAssetRegistryTag(UsageBitmaskName, UsageBitmaskTag, FAssetRegistryTag::TT_Hidden));

	// Deprecation
	bool bDeprecated = ScriptData ? ScriptData->bDeprecated : bDeprecated_DEPRECATED;
	FName DeprecatedName = GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, bDeprecated);
	FString& DeprecatedTag = CustomAssetRegistryTagCache->FindOrAdd(DeprecatedName);
	DeprecatedTag = FString::FromInt(bDeprecated);
	Context.AddTag(FAssetRegistryTag(DeprecatedName, DeprecatedTag, FAssetRegistryTag::TT_Hidden));

	// Suggested
	bool bSuggested = ScriptData ? ScriptData->bSuggested : false;
	FName SuggestedName = GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, bSuggested);
	FString& SuggestedTag = CustomAssetRegistryTagCache->FindOrAdd(SuggestedName);
	SuggestedTag = FString::FromInt(bSuggested);
	Context.AddTag(FAssetRegistryTag(SuggestedName, SuggestedTag, FAssetRegistryTag::TT_Hidden));
	
	// Add the current custom version to the tags so that tags can be fixed up in the future without having to load
	// the whole asset.
	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
	Context.AddTag(FAssetRegistryTag(NiagaraCustomVersionTagName, FString::FromInt(NiagaraVer), FAssetRegistryTag::TT_Hidden));

	INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>("Niagara");
	Context.AddTag(NiagaraModule.GetEditorOnlyDataUtilities().CreateClassUsageAssetRegistryTag(this));
#endif
}

void UNiagaraScript::BeginDestroy()
{
	Super::BeginDestroy();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (ScriptResource == nullptr || !ScriptResource->QueueForRelease(ReleasedByRT))
		{
			// if there was nothing to release, then we don't need to wait for anything
			ReleasedByRT = true;
		}
	}
	else
	{
		ReleasedByRT = true;
	}
}

bool UNiagaraScript::IsReadyForFinishDestroy()
{
	const bool bIsReady = Super::IsReadyForFinishDestroy();

	return bIsReady && ReleasedByRT;
}

bool UNiagaraScript::IsEditorOnly() const
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return false;
	}

	UNiagaraSystem* OuterSystem = GetTypedOuter<UNiagaraSystem>();
	bool bIsSystemScript = OuterSystem && (OuterSystem->GetSystemSpawnScript() == this || OuterSystem->GetSystemUpdateScript() == this);
	if (!bIsSystemScript && !GetOuterEmitter().GetEmitterData())
	{
		return true;
	}
#endif
	return Super::IsEditorOnly();
}

bool UNiagaraScript::ShouldCompile(EShaderPlatform Platform) const
{
#if WITH_EDITOR

	// We can not guarantee half float support on all platforms on the GPU, therefore we skip compilation in these cases
	TOptional<ENiagaraSimTarget> ActualSimTarget = GetSimTarget();
	if (ActualSimTarget.IsSet() && ActualSimTarget.GetValue() == ENiagaraSimTarget::GPUComputeSim)
	{
		UNiagaraSystem* OuterSystem = GetTypedOuter<UNiagaraSystem>();
		if (OuterSystem && OuterSystem->ShouldCompressAttributes())
		{
			if ( !FDataDrivenShaderPlatformInfo::GetSupportsUnrestrictedHalfFloatBuffers(Platform) )
			{
				return false;
			}
		}
	}

	// Add all data interfaces
	TSet<UClass*> DIUniqueClasses;
	for (const FNiagaraScriptDataInterfaceInfo& DataInterfaceInfo : CachedDefaultDataInterfaces)
	{
		if (UNiagaraDataInterface* DataInterface = DataInterfaceInfo.DataInterface)
		{
			DIUniqueClasses.Add(DataInterface->GetClass());
		}
	}

	// For each data interface allow them to modify the compilation environment
	for (UClass* DIClass : DIUniqueClasses)
	{
		if (UNiagaraDataInterface* DICDO = CastChecked<UNiagaraDataInterface>(DIClass->GetDefaultObject(true)))
		{
			if (!DICDO->ShouldCompile(Platform))
			{
				return false;
			}
		}
	}
	return true;
#else
	return false;
#endif
}

void UNiagaraScript::ModifyCompilationEnvironment(EShaderPlatform Platform, struct FShaderCompilerEnvironment& OutEnvironment) const
{
#if WITH_EDITOR
	// Add all data interfaces
	TSet<UClass*> DIUniqueClasses;
	for ( const FNiagaraScriptDataInterfaceInfo& DataInterfaceInfo : CachedDefaultDataInterfaces )
	{
		if ( UNiagaraDataInterface* DataInterface = DataInterfaceInfo.DataInterface )
		{
			DIUniqueClasses.Add(DataInterface->GetClass());
		}
	}

	// For each data interface allow them to modify the compilation environment
	for ( UClass* DIClass : DIUniqueClasses)
	{
		if ( UNiagaraDataInterface* DICDO = CastChecked<UNiagaraDataInterface>(DIClass->GetDefaultObject(true)) )
		{
			DICDO->ModifyCompilationEnvironment(Platform, OutEnvironment);
		}
	}
#endif
}

#if WITH_EDITOR

TArray<FName> UNiagaraScript::FindShaderFormatsForCooking(const ITargetPlatform* TargetPlatform) const
{
	TArray<FName> DesiredShaderFormats;

	if (CanBeRunOnGpu())
	{
		// currently only scripts that are associated with a system are appropriate for caching.  Standalone scripts or scripts
		// associated with standalone emitters are not required on cooked platforms.
		const bool bStandaloneScript = GetTypedOuter<UNiagaraSystem>() == nullptr;

		FVersionedNiagaraEmitter OuterEmitter = GetOuterEmitter();
		FVersionedNiagaraEmitterData* EmitterData = OuterEmitter.GetEmitterData();
		const bool bHasValidEmitter = (EmitterData != nullptr
			&& EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim
			&& OuterEmitter.Emitter->NeedsLoadForTargetPlatform(TargetPlatform));

		if (!bStandaloneScript && bHasValidEmitter)
		{
			TargetPlatform->GetAllTargetedShaderFormats(DesiredShaderFormats);

			// remove any formats that aren't supported
			DesiredShaderFormats.SetNum(Algo::RemoveIf(DesiredShaderFormats, [this](const FName& ShaderFormat)
			{
				const EShaderPlatform LegacyShaderPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormat);
				return !ShouldCompile(LegacyShaderPlatform);
			}));
		}
	}

	return DesiredShaderFormats;
}

void UNiagaraScript::BeginCacheForCookedPlatformData(const ITargetPlatform *TargetPlatform)
{
	// nothing to be done for cooking non-gpu compute scripts; their work is handled by the owning NiagaraSystem
	if (Usage != ENiagaraScriptUsage::ParticleGPUComputeScript)
	{
		return;
	}

	if (AreGpuScriptsCompiledBySystem())
	{
		// if the system is responsible for generating the GPU scripts, then we don't need to do anything here
		return;
	}

	// Commandlets like DerivedDataCacheCommandlet call BeginCacheForCookedPlatformData directly on objects. This may mean that
	// we have not properly gotten the HLSL script generated by the time that we get here. This does the awkward work of
	// waiting on the parent system to finish generating the HLSL before we can begin compiling it for the GPU.
	if (UNiagaraSystem* SystemOwner = FindRootSystem())
	{
		SystemOwner->WaitForCompilationComplete();
	}

	TArray<FName> ShaderFormatsForCooking = FindShaderFormatsForCooking(TargetPlatform);

	if (!ShaderFormatsForCooking.IsEmpty())
	{
		if (HasIdsRequiredForShaderCaching() == false)
		{
			UE_LOGF(LogNiagara, Warning,
				"Could not cache cooked shader for script %ls because it had an invalid cached script id.  This should be fixed by running the console command fx.PreventSystemRecompile with the owning system asset path as the argument and then resaving the assets.",
				*GetPathName());
			return;
		}

		TArray<TUniquePtr<FNiagaraShaderScript>>& CachedScriptResourcesForPlatform = CachedScriptResourcesForCooking.FindOrAdd(TargetPlatform);

		// Cache for all the shader formats that the cooking target requires
		for (const FName& ShaderFormat : ShaderFormatsForCooking)
		{
			const EShaderPlatform LegacyShaderPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormat);
			CacheResourceShadersForCooking(LegacyShaderPlatform, CachedScriptResourcesForPlatform, TargetPlatform);
		}
	}
}

bool UNiagaraScript::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	if (!HasIdsRequiredForShaderCaching())
	{
		return true;
	}

	if (AreGpuScriptsCompiledBySystem())
	{
		if (UNiagaraSystem* SystemOwner = FindRootSystem())
		{
			if (SystemOwner->HasOutstandingCompilationRequests(true))
			{
				return false;
			}
		}

		return true;
	}

	const TArray<FName> ShaderFormats = FindShaderFormatsForCooking(TargetPlatform);

	if (!ShaderFormats.IsEmpty())
	{
		bool bHasOutstandingCompilationRequests = false;
		if (UNiagaraSystem* SystemOwner = FindRootSystem())
		{
			bHasOutstandingCompilationRequests = SystemOwner->HasOutstandingCompilationRequests();
		}

		if (!bHasOutstandingCompilationRequests)
		{
			const TArray<TUniquePtr<FNiagaraShaderScript>>* CachedScriptResourcesForPlatform = CachedScriptResourcesForCooking.Find(TargetPlatform);
			if (CachedScriptResourcesForPlatform)
			{
				for (const TUniquePtr<FNiagaraShaderScript>& MaterialResource : *CachedScriptResourcesForPlatform)
				{
					if (MaterialResource->IsCompilationFinished() == false)
					{
						// For now, finish compilation here until we can make sure compilation is finished in the cook commandlet asynchronously before serialize
						MaterialResource->FinishCompilation();

						if (MaterialResource->IsCompilationFinished() == false)
						{
							return false;
						}
					}
				}

				return true;
			}
		}

		return false;
	}

	return true;
}

// utility function answering whether it's the NiagaraSystme or the NiagaraScript that will be responsible
// for the generation of the FNiagaraShaderMap and general compilation for GPU scripts
bool UNiagaraScript::AreGpuScriptsCompiledBySystem()
{
	if (!GNiagaraGpuScriptsCompiledBySystem)
	{
		return false;
	}

	const ENiagaraCompilationMode CompilationMode = GetDefault<UNiagaraSettings>()->CompilationMode;
	if (CompilationMode != ENiagaraCompilationMode::AsyncTasks)
	{
		return false;
	}

	if (!GNiagaraCompileGpuSupportsMultiPlatformCook && IsRunningCookCommandlet())
	{
		if (GetTargetPlatformManagerRef().GetActiveTargetPlatforms().Num() > 1)
		{
			return false;
		}
	}

	return true;
}

void UNiagaraScript::CacheResourceShadersForCooking(EShaderPlatform ShaderPlatform, TArray<TUniquePtr<FNiagaraShaderScript>>& InOutCachedResources, const ITargetPlatform* TargetPlatform)
{
	if (CanBeRunOnGpu())
	{
		// spawn and update are combined on GPU, so we only compile spawn scripts
		if (Usage == ENiagaraScriptUsage::ParticleGPUComputeScript)
		{
			ERHIFeatureLevel::Type TargetFeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);
			const auto FindExistingScriptPredicate = [&](const TUniquePtr<FNiagaraShaderScript>& ExistingScript)
			{
				return ExistingScript->MatchesScript(TargetFeatureLevel, ShaderPlatform, CachedScriptVMId);
			};

			// see if the script has already been added before adding a new version
			if (InOutCachedResources.ContainsByPredicate(FindExistingScriptPredicate))
			{
				return;
			}

			TUniquePtr<FNiagaraShaderScript> NewResource = MakeUnique<FNiagaraShaderScript>();
			check(CachedScriptVMId.CompilerVersionID.IsValid());
			check(CachedScriptVMId.BaseScriptCompileHash.IsValid());

			NewResource->SetScript(this, TargetFeatureLevel, ShaderPlatform, CachedScriptVMId.CompilerVersionID, CachedScriptVMId.AdditionalDefines,
				CachedScriptVMId.GetAdditionalVariableStrings(),
				CachedScriptVMId.BaseScriptCompileHash,	CachedScriptVMId.ReferencedCompileHashes,
				CachedScriptVMId.bUsesRapidIterationParams, GetFriendlyName());

			CacheShadersForResources(NewResource.Get(), false, false, true, TargetPlatform);

			INiagaraModule& NiagaraModule = FModuleManager::GetModuleChecked<INiagaraModule>(TEXT("Niagara"));
			NiagaraModule.ProcessShaderCompilationQueue();

			InOutCachedResources.Emplace(MoveTemp(NewResource));
		}
	}
}



void UNiagaraScript::CacheShadersForResources(FNiagaraShaderScript* ResourceToCache, bool bApplyCompletedShaderMapForRendering, bool bForceRecompile, bool bCooking, const ITargetPlatform* TargetPlatform)
{
	// if we're running the cook commandlet and this script isn't being processed as part of the cooking process, then let's not worry
	// about kicking off a compile for it.  As things currently stand we don't process the results (including storing them in the DDC)
	// so we'll just be kicking off the compile each time for no reason.
	if (IsRunningCookCommandlet() && !bCooking)
	{
		return;
	}

	if (CanBeRunOnGpu())
	{
		// When not running in the editor, the shaders are created in-sync (in the postload) to avoid update issues.
		const bool bSync = bCooking || !GIsEditor || GIsAutomationTesting;
		const bool bSuccess = ResourceToCache->CacheShaders(bApplyCompletedShaderMapForRendering, bForceRecompile, bSync, TargetPlatform);

#if defined(NIAGARA_SCRIPT_COMPILE_LOGGING_MEDIUM)
		if (!bSuccess)
		{
			UE_LOGF(LogNiagara, Warning, "Failed to compile Niagara shader %ls for platform %ls.",
				*GetPathName(),
				*LegacyShaderPlatformToShaderFormat(ResourceToCache->GetShaderPlatform()).ToString());

			const TArray<FString>& CompileErrors = ResourceToCache->GetCompileErrors();
			for (int32 ErrorIndex = 0; ErrorIndex < CompileErrors.Num(); ErrorIndex++)
			{
				UE_ASSET_LOG(LogNiagara, Warning, this, TEXT("	%s"), *CompileErrors[ErrorIndex]);
			}
		}
#endif
	}
}

void UNiagaraScript::CacheResourceShadersForRendering(bool bRegenerateId, bool bForceRecompile)
{
	if (bRegenerateId)
	{
		// Regenerate this script's Id if requested
		for (int32 Idx = 0; Idx < ERHIFeatureLevel::Num; Idx++)
		{
			ScriptResourcesByFeatureLevel[Idx] = nullptr;
		}
	}

	//UpdateResourceAllocations();

	if (CanBeRunOnGpu())
	{
		// Need to make sure the owner supports GPU scripts, otherwise this is a wasted compile.
		UNiagaraScriptSourceBase* Source = GetLatestScriptData()->Source;
		if (Source && OwnerCanBeRunOnGpu())
		{
		#if WITH_EDITORONLY_DATA
			const ERHIFeatureLevel::Type CacheFeatureLevel = NiagaraScriptInternal::PreviewFeatureLevel.Get(GMaxRHIFeatureLevel);
		#else
			const ERHIFeatureLevel::Type CacheFeatureLevel = GMaxRHIFeatureLevel;
		#endif
			const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[CacheFeatureLevel];

			ScriptResource->SetScript(this, CacheFeatureLevel, ShaderPlatform, CachedScriptVMId.CompilerVersionID, CachedScriptVMId.AdditionalDefines,
				CachedScriptVMId.GetAdditionalVariableStrings(),
				CachedScriptVMId.BaseScriptCompileHash, CachedScriptVMId.ReferencedCompileHashes,
				CachedScriptVMId.bUsesRapidIterationParams, GetFriendlyName());

			ScriptResource->BuildScriptParametersMetadata(CachedScriptVM.ShaderScriptParametersMetadata);

			CacheShadersForResources(ScriptResource.Get(), true);
			ScriptResourcesByFeatureLevel[CacheFeatureLevel] = ScriptResource;
		}
		else
		{
			ScriptResource->Invalidate();
		}
	}
}

FString UNiagaraScript::GetFriendlyName() const
{
	UEnum* ENiagaraScriptUsageEnum = StaticEnum<ENiagaraScriptUsage>();

	UNiagaraEmitter* EmitterObject = GetTypedOuter<UNiagaraEmitter>();
	UObject* SystemObject = EmitterObject != nullptr ? EmitterObject->GetOuter() : nullptr;
	FString FriendlyName = FString::Printf(TEXT("%s/%s/%s"),
		SystemObject ? *FPaths::MakeValidFileName(SystemObject->GetName()) : TEXT("UnknownSystem"),
		EmitterObject ? *FPaths::MakeValidFileName(EmitterObject->GetUniqueEmitterName()) : TEXT("UnknownEmitter"),
		ENiagaraScriptUsageEnum ? *FPaths::MakeValidFileName(ENiagaraScriptUsageEnum->GetNameStringByValue((int64)Usage)) : TEXT("UnknownEnum")
	);

	return FriendlyName;
}

void UNiagaraScript::SyncAliases(const FNiagaraAliasContext& ResolveAliasesContext)
{
	// First handle any rapid iteration parameters...
	{
		TArray<FNiagaraVariable> Params;
		RapidIterationParameters.GetParameters(Params);
		for (FNiagaraVariable Var : Params)
		{

			FNiagaraVariable NewVar = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext);
			if (NewVar.GetName() != Var.GetName())
			{
				RapidIterationParameters.RenameParameter(Var, NewVar.GetName());
			}
		}
	}

	// Now handle any compile tags overall..
	{
		for (int32 i = 0; i < GetVMExecutableData().CompileTags.Num(); i++)
		{
			const FString& Name = GetVMExecutableData().CompileTags[i].StringValue;
			if (Name.Len())
			{
				FNiagaraVariable NewVar = FNiagaraUtilities::ResolveAliases(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), *Name), ResolveAliasesContext);
				if (NewVar.GetName() != *Name)
				{
					GetVMExecutableData().CompileTags[i].StringValue = NewVar.GetName().ToString();
				}
			}
		}
	#if WITH_EDITORONLY_DATA
		for (int32 i = 0; i < GetVMExecutableData().CompileTagsEditorOnly.Num(); i++)
		{
			const FString& Name = GetVMExecutableData().CompileTagsEditorOnly[i].StringValue;
			if (Name.Len())
			{
				FNiagaraVariable NewVar = FNiagaraUtilities::ResolveAliases(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), *Name), ResolveAliasesContext);
				if (NewVar.GetName() != *Name)
				{
					GetVMExecutableData().CompileTagsEditorOnly[i].StringValue = NewVar.GetName().ToString();
				}
			}
		}
	#endif
	}

	// Now handle any Parameters overall..
	for (int32 i = 0; i < GetVMExecutableData().Parameters.Parameters.Num(); i++)
	{
		if (GetVMExecutableData().Parameters.Parameters[i].IsValid() == false)
		{
			const FNiagaraVariable& InvalidParameter = GetVMExecutableData().Parameters.Parameters[i];
			UE_LOGF(LogNiagara, Error, "Invalid parameter found while syncing script aliases.  Script: %ls Parameter Name: %ls Parameter Type: %ls",
				*GetPathName(), *InvalidParameter.GetName().ToString(), InvalidParameter.GetType().IsValid() ? *InvalidParameter.GetType().GetName() : TEXT("Unknown"));
			continue;
		}

		FNiagaraVariable Var = GetVMExecutableData().Parameters.Parameters[i];
		FNiagaraVariable NewVar = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext);
		if (NewVar.GetName() != Var.GetName())
		{
			GetVMExecutableData().Parameters.Parameters[i] = NewVar;
		}
	}

	// Sync up any simulation stage name references.
	for (FSimulationStageMetaData& SimStageMetaData : GetVMExecutableData().SimulationStageMetaData)
	{
		if ( !SimStageMetaData.IterationDataInterface.IsNone() )
		{
			FNiagaraVariable Var(FNiagaraTypeDefinition(UNiagaraDataInterface::StaticClass()), SimStageMetaData.IterationDataInterface);
			SimStageMetaData.IterationDataInterface = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext).GetName();
		}

		if (!SimStageMetaData.EnabledBinding.IsNone())
		{
			FNiagaraVariable Var(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.EnabledBinding);
			SimStageMetaData.EnabledBinding = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext).GetName();
		}

		if (SimStageMetaData.IterationSourceType == ENiagaraIterationSource::DirectSet)
		{
			if (!SimStageMetaData.ElementCountXBinding.IsNone())
			{
				FNiagaraVariable Var(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.ElementCountXBinding);
				SimStageMetaData.ElementCountXBinding = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext).GetName();
			}
			if (!SimStageMetaData.ElementCountYBinding.IsNone())
			{
				FNiagaraVariable Var(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.ElementCountYBinding);
				SimStageMetaData.ElementCountYBinding = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext).GetName();
			}
			if (!SimStageMetaData.ElementCountZBinding.IsNone())
			{
				FNiagaraVariable Var(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.ElementCountZBinding);
				SimStageMetaData.ElementCountZBinding = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext).GetName();
			}
		}

		if (!SimStageMetaData.NumIterationsBinding.IsNone())
		{
			FNiagaraVariable Var(FNiagaraTypeDefinition::GetIntDef(), SimStageMetaData.NumIterationsBinding);
			SimStageMetaData.NumIterationsBinding = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext).GetName();
		}

		for ( FName& OutputDestination : SimStageMetaData.OutputDestinations )
		{
			if ( !OutputDestination.IsNone() )
			{
				FNiagaraVariable Var(FNiagaraTypeDefinition(UNiagaraDataInterface::StaticClass()), OutputDestination);
				OutputDestination = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext).GetName();
			}
		}

		for (FName& InputDI : SimStageMetaData.InputDataInterfaces)
		{
			if (!InputDI.IsNone())
			{
				FNiagaraVariable Var(FNiagaraTypeDefinition(UNiagaraDataInterface::StaticClass()), InputDI);
				InputDI = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext).GetName();
			}
		}
	}

	// Also handle any data set mappings...
	auto Iterator = GetVMExecutableData().DataSetToParameters.CreateIterator();
	while (Iterator)
	{
		for (int32 i = 0; i < Iterator.Value().Parameters.Num(); i++)
		{
			FNiagaraVariable Var = Iterator.Value().Parameters[i];
			FNiagaraVariable NewVar = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext);
			if (NewVar.GetName() != Var.GetName())
			{
				Iterator.Value().Parameters[i] = NewVar;
			}
		}
		++Iterator;
	}

	// and UObjects
	for (FNiagaraScriptUObjectCompileInfo& Info : CachedDefaultUObjects)
	{
		const FNiagaraVariableBase TempNamedVariable(FNiagaraTypeDefinition::GetFloatDef(), Info.Variable.GetName());
		const FNiagaraVariableBase ResolvedVariable = FNiagaraUtilities::ResolveAliases(TempNamedVariable, ResolveAliasesContext);
		Info.Variable.SetName(ResolvedVariable.GetName());

		auto UpdateName =
			[&](FName& NameVariable)
			{
				if (NameVariable == TempNamedVariable.GetName())
				{
					NameVariable = ResolvedVariable.GetName();
				}
			};

		// also update the MapRead/MapWrite member variables, they seem to typically match the Name parameter, but may be None
		UpdateName(Info.RegisteredParameterMapRead);
		for (FName& VariableName : Info.RegisteredParameterMapWrites)
		{
			UpdateName(VariableName);
		}
	}
}

bool UNiagaraScript::SynchronizeExecutablesWithCompilation(const UNiagaraScript* Script, const TMap<FString, FString>& RenameMap)
{
	check(IsInGameThread());

	if (Script->CachedScriptVM.IsValid() == false)
	{
		// If the master script has no valid VM then there's nothing to synchronize.
		return false;
	}

	if (!AllowShaderCompiling())
	{
		// if we're not allowed to compile shaders than don't even bother evaluating the id
		return false;
	}

	FNiagaraVMExecutableDataId Id;
	ComputeVMCompilationId(Id, FGuid());

	if (Id == Script->GetVMExecutableDataCompilationId())
	{
		CachedScriptVM.Reset();
		ScriptResource->Invalidate();

		CachedScriptVM = Script->CachedScriptVM;
		CachedScriptVMId = Script->CachedScriptVMId;
		CachedParameterCollectionReferences = Script->CachedParameterCollectionReferences;
		CachedDefaultUObjects = Script->CachedDefaultUObjects;
		InvalidateAndEmptyDefaultDataInterfaces(Script->CachedDefaultDataInterfaces.Num(), EDataInterfaceInvalidationMode::ProcessInvaliation);
		for (const FNiagaraScriptDataInterfaceInfo& Info : Script->CachedDefaultDataInterfaces)
		{
			FNiagaraScriptDataInterfaceInfo AddInfo;
			AddInfo = Info;
			AddInfo.DataInterface = CopyDataInterface(Info.DataInterface, this);
			CachedDefaultDataInterfaces.Add(AddInfo);
		}

		GenerateStatIDs();

		//SyncAliases(RenameMap);

		// Now go ahead and trigger the GPU script compile now that we have a compiled GPU hlsl script.
		if (!AreGpuScriptsCompiledBySystem())
		{
			if (Usage == ENiagaraScriptUsage::ParticleGPUComputeScript)
			{
				CacheResourceShadersForRendering(false, true);
			}
		}

		OnVMScriptCompiled().Broadcast(this, FGuid()); // TODO what version id to use here?
		return true;
	}

	return false;
}

void UNiagaraScript::InvalidateCompileResults(const FString& Reason)
{
	UE_LOGF(LogNiagara, Verbose, "InvalidateCompileResults Script:%ls Reason:%ls", *GetPathName(), *Reason);
	CachedScriptVM.Reset();
	ScriptResource->Invalidate();
	CachedScriptVMId.Invalidate();
	if (VersionData.Num() > 0)
	{
		GetLastGeneratedVMId().Invalidate();
	}
	CachedDefaultUObjects.Reset();
	InvalidateAndEmptyDefaultDataInterfaces(0, EDataInterfaceInvalidationMode::QueueInvaliation);
}

FText UNiagaraScript::GetDescription(const FGuid& VersionGuid)
{
	FVersionedNiagaraScriptData* ScriptData = GetScriptData(VersionGuid);
	return ScriptData == nullptr || ScriptData->Description.IsEmpty() ? FText::FromString(GetName()) : ScriptData->Description;
}


UNiagaraScript::FOnScriptCompiled& UNiagaraScript::OnVMScriptCompiled()
{
	return OnVMScriptCompiledDelegate;
}

UNiagaraScript::FOnScriptCompiled& UNiagaraScript::OnGPUScriptCompiled()
{
	return OnGPUScriptCompiledDelegate;
}

UNiagaraScript::FOnPropertyChanged& UNiagaraScript::OnPropertyChanged()
{
	return OnPropertyChangedDelegate;
}

void UNiagaraScript::ResolveParameterCollectionReferences()
{
	const int32 CollectionCount = CachedScriptVM.ParameterCollectionPaths.Num();

	if (CollectionCount)
	{
		const bool RoutingPostLoad = FUObjectThreadContext::Get().IsRoutingPostLoad;

		for (int32 CollectionIt = CollectionCount - 1; CollectionIt >= 0; --CollectionIt)
		{
			FSoftObjectPath SoftPath(CachedScriptVM.ParameterCollectionPaths[CollectionIt]);

			// try to find the object if it's already loaded
			UNiagaraParameterCollection* ParamCollection = Cast<UNiagaraParameterCollection>(SoftPath.ResolveObject());

			if (!ParamCollection && !RoutingPostLoad)
			{
				// if we're not in a PostLoad then we should be able to try to directly load the object
				ParamCollection = Cast<UNiagaraParameterCollection>(SoftPath.TryLoad());
			}

			if (ParamCollection)
			{
				CachedParameterCollectionReferences.AddUnique(ParamCollection);
				CachedScriptVM.ParameterCollectionPaths.RemoveAtSwap(CollectionIt);
			}
		}
	}
}

#endif

#if WITH_EDITORONLY_DATA
void UNiagaraScript::BuildClassSchema(FAppendToClassSchemaContext& Context)
{
	// Used by iterative cooking.  This will provide additional context for if things have changed such that a cook will
	// be required.  This is focused on global settings rather than the usual dependencies between objects.

	// the compiler version
	const FGuid CompilerVersion = FNiagaraCustomVersion::GetLatestScriptCompileVersion();
	Context.Update(&CompilerVersion, sizeof(FGuid));

	// the DI source hash
	if (GNiagaraCompileHashAllDataInterfaces)
	{
		Context.Update(NiagaraScriptInternal::CompileHashAllDataInterfaces().GetData(), FNiagaraCompileHash::HashSize);
	}

	// settings/configurations
	Context.Update(&GNiagaraTranslatorFailIfNotSetSeverity, sizeof(GNiagaraTranslatorFailIfNotSetSeverity));
	Context.Update(&GbNiagaraEnableTraversalCache, sizeof(GbNiagaraEnableTraversalCache));

	const UNiagaraSettings* NiagaraSettings = GetDefault<UNiagaraSettings>();
	checkf(NiagaraSettings, TEXT("Failed to acquire default UNiagaraSettings during AppendToClassSchema"));

	if (NiagaraSettings)
	{
		Context.Update(&NiagaraSettings->InvalidNamespaceWriteSeverity, sizeof(NiagaraSettings->InvalidNamespaceWriteSeverity));
	}

	FNiagaraShader::BuildClassSchema(Context);
}
#endif


TArray<TObjectPtr<UNiagaraParameterCollection>>& UNiagaraScript::GetCachedParameterCollectionReferences()
{
#if WITH_EDITORONLY_DATA
	ResolveParameterCollectionReferences();
#endif

	return CachedParameterCollectionReferences;
}

NIAGARA_API bool UNiagaraScript::IsScriptCompilationPending(bool bGPUScript) const
{
	if (bGPUScript)
	{
		if (ScriptResource.IsValid())
		{
			if (ScriptResource->IsShaderMapComplete())
			{
				return false;
			}

			return !ScriptResource->IsCompilationFinished();
		}
	}
	else if (CachedScriptVM.IsValid())
	{
		if (CachedScriptVM.LastCompileStatus == ENiagaraScriptCompileStatus::NCS_BeingCreated || CachedScriptVM.LastCompileStatus == ENiagaraScriptCompileStatus::NCS_Unknown)
		{
			return !CachedScriptVM.HasByteCode();
		}
	}
	return false;
}

NIAGARA_API bool UNiagaraScript::DidScriptCompilationSucceed(bool bGPUScript) const
{
	if (bGPUScript)
	{
		if (ScriptResource.IsValid())
		{
			if (ScriptResource->IsShaderMapComplete())
			{
				return true;
			}

			if (ScriptResource->IsCompilationFinished())
			{
				// If we failed compilation, it would be finished and Shader would be null.
				return false;
			}
		}

		// If we are on a cooked platform and we have no shader we need to check if we disabled compute shader compilation
		// in which case we lie and say the compilation was ok otherwise the rest of the system will be disabled.
		//-TODO: Strip these emitters on cook instead
		if (FPlatformProperties::RequiresCookedData() && !FNiagaraUtilities::AllowComputeShaders(GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel]))
		{
			return true;
		}
	}
	else if (CachedScriptVM.IsValid())
	{
		return (CachedScriptVM.HasByteCode());
	}

	return false;
}

#if WITH_EDITORONLY_DATA
void UNiagaraScript::GatherScriptStaticVariables(TArray<FNiagaraVariable>& StaticVariables) const
{
	for (const FNiagaraVariableWithOffset& Param : RapidIterationParameters.ReadParameterVariables())
	{
		if (Param.GetType().IsStatic() && Param.Offset != INDEX_NONE)
		{
			FNiagaraVariable StaticVariable = Param;
			StaticVariable.SetData(RapidIterationParameters.GetParameterData(Param));

			StaticVariables.AddUnique(StaticVariable);
		}
	}
}

bool UNiagaraScript::HasScriptStaticVariables() const
{
	for (const FNiagaraVariableWithOffset& Param : RapidIterationParameters.ReadParameterVariables())
	{
		if (Param.GetType().IsStatic() && Param.Offset != INDEX_NONE)
		{
			return true;
		}
	}

	return false;
}
#endif

void UNiagaraScript::SerializeNiagaraShaderMaps(FArchive& Ar, int32 NiagaraVer, bool IsValidShaderScript)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsSaving() && IsValidShaderScript)
	{
		Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
		Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
		Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
		Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

		int32 NumResourcesToSave = 0;
		TArray<TUniquePtr<FNiagaraShaderScript>>* ScriptResourcesToSavePtr = nullptr;

		if (Ar.IsCooking())
		{
			ScriptResourcesToSavePtr = CachedScriptResourcesForCooking.Find(Ar.CookingTarget());
			if (ScriptResourcesToSavePtr != nullptr)
			{
				NumResourcesToSave = ScriptResourcesToSavePtr->Num();
			}
		}

		Ar << NumResourcesToSave;

		if (ScriptResourcesToSavePtr != nullptr)
		{
			for (TUniquePtr<FNiagaraShaderScript>& ScriptResourceToSave : (*ScriptResourcesToSavePtr))
			{
				checkf(ScriptResourceToSave != nullptr, TEXT("Invalid script resource was cached"));
				ScriptResourceToSave->SerializeShaderMap(Ar);
			}
		}
	}
#endif

	if (Ar.IsLoading())
	{
		IsValidShaderScript = (NiagaraVer >= FNiagaraCustomVersion::NiagaraShaderMaps) && (NiagaraVer < FNiagaraCustomVersion::NiagaraShaderMapCooking || IsValidShaderScript);

		if (!IsValidShaderScript)
		{
			return;
		}

#if WITH_EDITORONLY_DATA
		const bool HasEditorData = !Ar.IsFilterEditorOnly();
		if (HasEditorData)
		{
			int32 NumLoadedResources = 0;
			Ar << NumLoadedResources;
			for (int32 i = 0; i < NumLoadedResources; i++)
			{
				FNiagaraShaderScript LoadedResource;
				LoadedResource.SerializeShaderMap(Ar);
				LoadedScriptResources.Add(LoadedResource);
			}
		}
		else
#endif
		{
			check(NiagaraVer >= FNiagaraCustomVersion::NiagaraShaderMaps);
			int32 ResourceCount = 0;
			Ar << ResourceCount;

			for (int32 ResourceIt = 0; ResourceIt < ResourceCount; ++ResourceIt)
			{
				FNiagaraShaderScript Resource;
				Resource.SerializeShaderMap(Ar);

				if (!ScriptResource)
				{
					if (FNiagaraShaderMap* ShaderMap = Resource.GetGameThreadShaderMap())
					{
						if (GMaxRHIShaderPlatform == ShaderMap->GetShaderPlatform())
						{
							ScriptResource = MakeShared<FNiagaraShaderScript>(Resource);
						}
					}
				}
			}
		}
	}
}

void UNiagaraScript::ProcessSerializedShaderMaps()
{
	check(IsInGameThread());

	bool HasScriptResource = false;

#if WITH_EDITORONLY_DATA
	for (FNiagaraShaderScript& LoadedResource : LoadedScriptResources)
	{
		FNiagaraShaderMap* LoadedShaderMap = LoadedResource.GetGameThreadShaderMap();
		if (LoadedShaderMap && LoadedShaderMap->GetShaderPlatform() == GMaxRHIShaderPlatform)
		{
			HasScriptResource = true;
			ScriptResource = MakeShared<FNiagaraShaderScript>(LoadedResource);
			ScriptResource->OnCompilationComplete().AddUniqueDynamic(this, &UNiagaraScript::RaiseOnGPUCompilationComplete);

			ERHIFeatureLevel::Type LoadedFeatureLevel = LoadedShaderMap->GetShaderMapId().FeatureLevel;
			if (!ScriptResourcesByFeatureLevel[LoadedFeatureLevel])
			{
				ScriptResourcesByFeatureLevel[LoadedFeatureLevel] = MakeShared<FNiagaraShaderScript>();
			}

			ScriptResourcesByFeatureLevel[LoadedFeatureLevel]->SetShaderMap(LoadedShaderMap);
			break;
		}
	}
#else
	HasScriptResource = ScriptResource.IsValid();
#endif

	if (HasScriptResource)
	{
		ScriptResource->BuildScriptParametersMetadata(CachedScriptVM.ShaderScriptParametersMetadata);
	}
}

#if WITH_EDITORONLY_DATA
TArray<ENiagaraScriptUsage> FVersionedNiagaraScriptData::GetSupportedUsageContexts() const
{
	return UNiagaraScript::GetSupportedUsageContextsForBitmask(ModuleUsageBitmask);
}

TArray<ENiagaraScriptUsage> UNiagaraScript::GetSupportedUsageContextsForBitmask(int32 InModuleUsageBitmask, bool bIncludeHiddenUsages)
{
	TArray<ENiagaraScriptUsage> Supported;
	UEnum* UsageEnum = StaticEnum<ENiagaraScriptUsage>();
	for (int32 i = 0; i <= (int32)ENiagaraScriptUsage::SystemUpdateScript; i++)
	{
		int32 TargetBit = (InModuleUsageBitmask >> (int32)i) & 1;
		if (TargetBit == 1 && (bIncludeHiddenUsages || UsageEnum->HasMetaData(TEXT("Hidden"), i) == false))
		{
			Supported.Add((ENiagaraScriptUsage)i);
		}
	}
	return Supported;
}

bool UNiagaraScript::IsSupportedUsageContextForBitmask(int32 InModuleUsageBitmask, ENiagaraScriptUsage InUsageContext, bool bIncludeHiddenUsages)
{
	TArray<ENiagaraScriptUsage> SupportedUsages = GetSupportedUsageContextsForBitmask(InModuleUsageBitmask, bIncludeHiddenUsages);
	for (ENiagaraScriptUsage SupportedUsage : SupportedUsages)
	{
		if (UNiagaraScript::IsEquivalentUsage(InUsageContext, SupportedUsage))
		{
			return true;
		}
	}
	return false;
}

bool UNiagaraScript::ContainsEquivilentUsage(const TArray<ENiagaraScriptUsage>& Usages, ENiagaraScriptUsage InUsage)
{
	for (ENiagaraScriptUsage Usage : Usages)
	{
		if (IsEquivalentUsage(Usage, InUsage))
		{
			return true;
		}
	}
	return false;
}

#endif

bool UNiagaraScript::CanBeRunOnGpu()const
{

	if (Usage != ENiagaraScriptUsage::ParticleGPUComputeScript)
	{
		return false;
	}
	if (!CachedScriptVM.IsValid())
	{
		return false;
	}
#if WITH_EDITORONLY_DATA
	for (const FNiagaraScriptDataInterfaceCompileInfo& InterfaceInfo : CachedScriptVM.DataInterfaceInfo)
	{
		const UNiagaraDataInterface* DefaultDataInterface = InterfaceInfo.GetDefaultDataInterface();
		if (DefaultDataInterface == nullptr || DefaultDataInterface->CanExecuteOnTarget(ENiagaraSimTarget::GPUComputeSim) == false)
		{
			return false;
		}
	}
#endif
	return true;
}

bool UNiagaraScript::OwnerCanBeRunOnGpu() const
{
	if (FVersionedNiagaraEmitterData* EmitterData = GetOuterEmitter().GetEmitterData())
	{
		if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			return true;
		}
	}

	return false;
}

bool UNiagaraScript::LegacyCanBeRunOnGpu() const
{
	if (FVersionedNiagaraEmitterData* EmitterData = GetOuterEmitter().GetEmitterData())
	{
		if (EmitterData->SimTarget == ENiagaraSimTarget::CPUSim)
		{
			return false;
		}

		if (!IsParticleSpawnScript())
		{
			return false;
		}

		return true;
	}
	return false;
}


#if WITH_EDITORONLY_DATA
FGuid UNiagaraScript::GetBaseChangeID(const FGuid& VersionGuid) const
{
	const FVersionedNiagaraScriptData* ScriptData = GetScriptData(VersionGuid);
	return ScriptData && ScriptData->Source ? ScriptData->Source->GetChangeID() : FGuid();
}

ENiagaraScriptCompileStatus UNiagaraScript::GetLastCompileStatus() const
{
	bool bHasWarnings = false;
	if (Usage == ENiagaraScriptUsage::ParticleGPUComputeScript)
	{
		const TArray<FString>& Errors = ScriptResource->GetCompileErrors();
		if (Errors.Num() > 0)
		{
			for (const FString& ErrorStr : Errors)
			{
				if (ErrorStr.Contains(TEXT("err0r")))
				{
					return ENiagaraScriptCompileStatus::NCS_Error;
				}
			}
			bHasWarnings = true;
		}	
	}
	if (CachedScriptVM.IsValid())
	{
		if (CachedScriptVM.LastCompileStatus == ENiagaraScriptCompileStatus::NCS_UpToDate && bHasWarnings)
		{
			return ENiagaraScriptCompileStatus::NCS_ComputeUpToDateWithWarnings;
		}
		return CachedScriptVM.LastCompileStatus;
	}
	return ENiagaraScriptCompileStatus::NCS_Unknown;
}
#endif

bool UNiagaraScript::UsesCollection(const UNiagaraParameterCollection* Collection)const
{
	if (CachedScriptVM.IsValid())
	{
		if (CachedParameterCollectionReferences.Contains(Collection))
		{
			return true;
		}
#if WITH_EDITORONLY_DATA
		FSoftObjectPath SoftPath(Collection);

		if (CachedScriptVM.ParameterCollectionPaths.ContainsByPredicate([&](const FString& CollectionPath) { return SoftPath == FSoftObjectPath(CollectionPath); }))
		{
			return true;
		}
#endif
	}
	return false;
}
