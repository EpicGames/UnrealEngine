// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptExecutionParameterStore.h"
#include "NiagaraConstants.h"
#include "NiagaraDataInterface.h"
#include "NiagaraScript.h"
#include "NiagaraScriptRuntimeData.h"
#include "NiagaraSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScriptExecutionParameterStore)

FNiagaraScriptExecutionParameterStore::FNiagaraScriptExecutionParameterStore()
	: FNiagaraParameterStore()
	, ParameterSize(0)
	, bInitialized(false)
{}

FNiagaraScriptExecutionParameterStore::FNiagaraScriptExecutionParameterStore(const FNiagaraParameterStore& Other)
{
	*this = Other;
}

FNiagaraScriptExecutionParameterStore& FNiagaraScriptExecutionParameterStore::operator=(const FNiagaraParameterStore& Other)
{
	FNiagaraParameterStore::operator=(Other);
	return *this;
}

#if WITH_EDITORONLY_DATA
uint32 OffsetAlign(uint32 SrcOffset, uint32 Size)
{
	uint32 OffsetRemaining = SHADER_PARAMETER_STRUCT_ALIGNMENT - (SrcOffset % SHADER_PARAMETER_STRUCT_ALIGNMENT);
	if (Size <= OffsetRemaining)
	{
		return SrcOffset;
	}
	else
	{
		return Align(SrcOffset, SHADER_PARAMETER_STRUCT_ALIGNMENT);
	}
}

uint32 FNiagaraScriptExecutionParameterStore::GenerateLayoutInfoInternal(TArray<FNiagaraScriptExecutionPaddingInfo>& Members, uint32& NextMemberOffset, const UStruct* InSrcStruct, uint32 InSrcOffset)
{
	uint32 VectorPaddedSize = (TShaderParameterTypeInfo<FVector4f>::NumRows * TShaderParameterTypeInfo<FVector4f>::NumColumns) * sizeof(float);

	// Now insert an appropriate data member into the mix...
	if (InSrcStruct == FNiagaraTypeDefinition::GetBoolStruct() || InSrcStruct == FNiagaraTypeDefinition::GetIntStruct())
	{
		uint32 IntSize = (TShaderParameterTypeInfo<uint32>::NumRows * TShaderParameterTypeInfo<uint32>::NumColumns) * sizeof(uint32);
		Members.Emplace(InSrcOffset, Align(NextMemberOffset, TShaderParameterTypeInfo<uint32>::Alignment), IntSize, IntSize);
		InSrcOffset += sizeof(uint32);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct == FNiagaraTypeDefinition::GetFloatStruct())
	{
		uint32 FloatSize = (TShaderParameterTypeInfo<float>::NumRows * TShaderParameterTypeInfo<float>::NumColumns) * sizeof(float);
		Members.Emplace(InSrcOffset, Align(NextMemberOffset, TShaderParameterTypeInfo<float>::Alignment), FloatSize, FloatSize);
		InSrcOffset += sizeof(float);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct == FNiagaraTypeDefinition::GetVec2Struct())
	{
		uint32 StructFinalSize = (TShaderParameterTypeInfo<FVector2f>::NumRows * TShaderParameterTypeInfo<FVector2f>::NumColumns) * sizeof(float);
		Members.Emplace(InSrcOffset, OffsetAlign(NextMemberOffset, VectorPaddedSize), StructFinalSize, VectorPaddedSize);
		InSrcOffset += sizeof(FVector2f);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct && (InSrcStruct->GetFName() == NAME_Vector2D || InSrcStruct->GetFName() == NAME_Vector2d))
	{
		uint32 StructFinalSize = (TShaderParameterTypeInfo<FVector2f>::NumRows * TShaderParameterTypeInfo<FVector2f>::NumColumns) * sizeof(float);
		Members.Emplace(InSrcOffset, OffsetAlign(NextMemberOffset, VectorPaddedSize), StructFinalSize, VectorPaddedSize);
		InSrcOffset += sizeof(FVector2f);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}	
	else if (InSrcStruct == FNiagaraTypeDefinition::GetVec3Struct() || InSrcStruct == FNiagaraTypeDefinition::GetPositionStruct())
	{
		uint32 StructFinalSize = (TShaderParameterTypeInfo<FVector3f>::NumRows * TShaderParameterTypeInfo<FVector3f>::NumColumns) * sizeof(float);
		Members.Emplace(InSrcOffset, OffsetAlign(NextMemberOffset, VectorPaddedSize), StructFinalSize, VectorPaddedSize);
		InSrcOffset += sizeof(FVector3f);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct && (InSrcStruct->GetFName() == NAME_Vector || InSrcStruct->GetFName() == NAME_Vector3d))
	{
		uint32 StructFinalSize = (TShaderParameterTypeInfo<FVector3f>::NumRows * TShaderParameterTypeInfo<FVector3f>::NumColumns) * sizeof(float);
		Members.Emplace(InSrcOffset, OffsetAlign(NextMemberOffset, VectorPaddedSize), StructFinalSize, VectorPaddedSize);
		InSrcOffset += sizeof(FVector3f);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct && (InSrcStruct->GetFName() == NAME_Vector4 || InSrcStruct->GetFName() == NAME_Vector4d))
	{
		uint32 StructFinalSize = (TShaderParameterTypeInfo<FVector4f>::NumRows * TShaderParameterTypeInfo<FVector4f>::NumColumns) * sizeof(float);
		Members.Emplace(InSrcOffset, OffsetAlign(NextMemberOffset, VectorPaddedSize), StructFinalSize, VectorPaddedSize);
		InSrcOffset += sizeof(FVector4f);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct && (InSrcStruct->GetFName() == NAME_Quat || InSrcStruct->GetFName() == NAME_Quat4d))
	{
		uint32 StructFinalSize = (TShaderParameterTypeInfo<FQuat4f>::NumRows * TShaderParameterTypeInfo<FQuat4f>::NumColumns) * sizeof(float);
		Members.Emplace(InSrcOffset, OffsetAlign(NextMemberOffset, VectorPaddedSize), StructFinalSize, VectorPaddedSize);
		InSrcOffset += sizeof(FQuat4f);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct == FNiagaraTypeDefinition::GetVec4Struct() || InSrcStruct == FNiagaraTypeDefinition::GetColorStruct() || InSrcStruct == FNiagaraTypeDefinition::GetQuatStruct())
	{
		uint32 StructFinalSize = (TShaderParameterTypeInfo<FVector4f>::NumRows * TShaderParameterTypeInfo<FVector4f>::NumColumns) * sizeof(float);
		Members.Emplace(InSrcOffset, Align(NextMemberOffset, TShaderParameterTypeInfo<FVector4f>::Alignment), StructFinalSize, StructFinalSize);
		InSrcOffset += sizeof(FVector4f);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct == FNiagaraTypeDefinition::GetMatrix4Struct())
	{
		uint32 StructFinalSize = (TShaderParameterTypeInfo<FMatrix44f>::NumRows * TShaderParameterTypeInfo<FMatrix44f>::NumColumns) * sizeof(float);
		Members.Emplace(InSrcOffset, Align(NextMemberOffset, TShaderParameterTypeInfo<FMatrix44f>::Alignment), StructFinalSize, StructFinalSize);
		InSrcOffset += sizeof(FMatrix44f);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct == FNiagaraTypeDefinition::GetHalfStruct())
	{
		uint32 HalfSize = (/*NumRows*/1 * /*NumColumns*/ 1) * sizeof(FFloat16);
		Members.Emplace(InSrcOffset, Align(NextMemberOffset, TShaderParameterTypeInfo<FFloat16>::Alignment), HalfSize, HalfSize);
		InSrcOffset += sizeof(FFloat16);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct == FNiagaraTypeDefinition::GetHalfVec2Struct())
	{
		uint32 StructFinalSize = (/*NumRows*/2 * /*NumColumns*/1) * sizeof(FFloat16);
		Members.Emplace(InSrcOffset, OffsetAlign(NextMemberOffset, VectorPaddedSize), StructFinalSize, VectorPaddedSize);
		InSrcOffset += sizeof(FFloat16[2]);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct == FNiagaraTypeDefinition::GetHalfVec3Struct())
	{
		uint32 StructFinalSize = (/*NumRows*/3 * /*NumColumns*/1) * sizeof(FFloat16);
		Members.Emplace(InSrcOffset, OffsetAlign(NextMemberOffset, VectorPaddedSize), StructFinalSize, VectorPaddedSize);
		InSrcOffset += sizeof(FFloat16[3]);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else if (InSrcStruct == FNiagaraTypeDefinition::GetHalfVec4Struct())
	{
		uint32 StructFinalSize = (/*NumRows*/4 * /*NumColumns*/1) * sizeof(FFloat16);
		Members.Emplace(InSrcOffset, OffsetAlign(NextMemberOffset, VectorPaddedSize), StructFinalSize, VectorPaddedSize);
		InSrcOffset += sizeof(FFloat16[4]);
		NextMemberOffset = Members[Members.Num() - 1].DestOffset + Members[Members.Num() - 1].DestSize;
	}
	else
	{
		NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_STRUCT_ALIGNMENT); // New structs should be aligned to the head..

		for (TFieldIterator<FProperty> PropertyIt(InSrcStruct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			const UStruct* Struct = nullptr;

			// First determine what struct type we're dealing with...
			if (Property->IsA(FFloatProperty::StaticClass()))
			{
				Struct = FNiagaraTypeDefinition::GetFloatStruct();
			}
			else if (Property->IsA(FIntProperty::StaticClass()))
			{
				Struct = FNiagaraTypeDefinition::GetIntStruct();
			}
			else if (Property->IsA(FBoolProperty::StaticClass()))
			{
				Struct = FNiagaraTypeDefinition::GetBoolStruct();
			}
			else if (FStructProperty* StructProp = CastFieldChecked<FStructProperty>(Property))
			{
				Struct = FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProp->Struct, ENiagaraStructConversion::Simulation);
			}
			else
			{
				check(false);
			}

			InSrcOffset = GenerateLayoutInfoInternal(Members, NextMemberOffset, Struct, InSrcOffset);
		}
	}

	return InSrcOffset;
}

void FNiagaraScriptExecutionParameterStore::InitFromOwningScript(UNiagaraScript* Script, const FNiagaraScriptRuntimeCompiledData* ScriptRuntimeCompiledData, bool bNotifyAsDirty)
{
	//TEMPORARTY:
	//We should replace the storage on the script with an FNiagaraParameterStore also so we can just copy that over here. Though that is an even bigger refactor job so this is a convenient place to break that work up.

	Empty();

	if (Script != nullptr && ScriptRuntimeCompiledData != nullptr)
	{
		AddScriptParams(Script, ScriptRuntimeCompiledData);

		Script->RapidIterationParameters.Bind(this);

		if (bNotifyAsDirty)
		{
			MarkParametersDirty();
			MarkInterfacesDirty();
			OnLayoutChange();
		}
	}
	bInitialized = true;
}

void FNiagaraScriptExecutionParameterStore::AddScriptParams(UNiagaraScript* Script, const FNiagaraScriptRuntimeCompiledData* ScriptRuntimeCompiledData)
{
	if (Script == nullptr || ScriptRuntimeCompiledData == nullptr)
	{
		return;
	}

	const FNiagaraVMExecutableData& ExecutableData = Script->GetVMExecutableData();

	//Here we add the current frame parameters.
	bool bAdded = false;
	for (const FNiagaraVariable& Param : ExecutableData.Parameters.Parameters)
	{
		bAdded |= AddParameter(Param, false, false);
	}

	DebugName = FString::Printf(TEXT("ScriptExecParamStore %s %p"), *Script->GetFullName(), this);

	ParameterSize = GetParameterDataArray().Num();

	//Add previous frame values if we're interpolated spawn.
	bool bIsInterpolatedSpawn = Script->GetVMExecutableDataCompilationId().HasInterpolatedParameters();

	if (bIsInterpolatedSpawn)
	{
		for (const FNiagaraVariable& Param : ExecutableData.Parameters.Parameters)
		{
			FNiagaraVariable PrevParam(Param.GetType(), FName(*(INTERPOLATED_PARAMETER_PREFIX + Param.GetName().ToString())));
			bAdded |= AddParameter(PrevParam, false, false);
		}
	}

	// for VM scripts we need to build the script literals; in cooked builds this is already in the cached VM data
	if (ScriptRuntimeCompiledData->SimTarget != ENiagaraSimTarget::GPUComputeSim)
	{
		ExecutableData.BakeScriptLiterals(CachedScriptLiterals);
	}

	check(Script->GetVMExecutableData().DataInterfaceInfo.Num() == ScriptRuntimeCompiledData->ResolvedDataInterfaces.Num());
	for (const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDataInterface : ScriptRuntimeCompiledData->ResolvedDataInterfaces)
	{
		int32 VarOffset = INDEX_NONE;
		bAdded |= AddParameter(ResolvedDataInterface.ParameterStoreVariable, false, false, &VarOffset);
		if (GetDataInterfaces().IsValidIndex(VarOffset))
		{
			SetDataInterface(ResolvedDataInterface.ResolvedDataInterface, VarOffset);
		}
	}

	for (const FNiagaraResolvedUObjectInfo& ResolvedUObject : ScriptRuntimeCompiledData->ResolvedUObjects)
	{
		int32 VarOffset = INDEX_NONE;
		bAdded |= AddParameter(ResolvedUObject.ResolvedVariable, false, false, &VarOffset);
		if (GetUObjects().IsValidIndex(VarOffset))
		{
			SetUObject(ResolvedUObject.Object, VarOffset);
		}
	}
}
#endif

//////////////////////////////////////////////////////////////////////////
/// FNiagaraScriptInstanceParameterStore
//////////////////////////////////////////////////////////////////////////

FNiagaraScriptInstanceParameterStore::FNiagaraScriptInstanceParameterStore()
	: FNiagaraParameterStore()
	, bInitialized(false)
{}

void FNiagaraScriptInstanceParameterStore::InitFromOwningContext(UNiagaraScript* Script, const FNiagaraScriptRuntimeData* InScriptRuntimeData, bool bNotifyAsDirty)
{
	const FNiagaraScriptExecutionParameterStore* SrcStore = InScriptRuntimeData != nullptr 
		? InScriptRuntimeData->GetExecutionReadyParameterStore() : nullptr;

#if WITH_EDITORONLY_DATA
	DebugName = Script ? FString::Printf(TEXT("ScriptExecParamStore %s %p"), *Script->GetFullName(), this) : TEXT("");
#endif

	if (SrcStore)
	{
		Empty(false);

		SetParameterDataArray(SrcStore->GetParameterDataArray(), false);
		SetDataInterfaces(SrcStore->GetDataInterfaces(), false);
		SetUObjects(SrcStore->GetUObjects(), false);

		if (bNotifyAsDirty)
		{
			MarkParametersDirty();
			MarkInterfacesDirty();
			MarkUObjectsDirty();
			OnLayoutChange();
		}

		ScriptParameterStore.Init(SrcStore);
	}
	else
	{
		Empty();
	}

	bInitialized = true;
}

void FNiagaraScriptInstanceParameterStore::CopyCurrToPrev()
{
	if (const auto* ScriptStore = ScriptParameterStore.Get())
	{
		const int32 ParamStart = ScriptStore->ParameterSize;
		FMemory::Memcpy(GetMutableParameterData_Internal(ParamStart), GetParameterData_Internal(0), ParamStart);
	}
}

uint32 FNiagaraScriptInstanceParameterStore::GetExternalParameterSize() const
{
	if (const auto* ScriptStore = ScriptParameterStore.Get())
	{
		return ScriptStore->ParameterSize;
	}
	return 0;
}

TArrayView<const FNiagaraVariableWithOffset> FNiagaraScriptInstanceParameterStore::ReadParameterVariables() const
{
	if (const auto* ScriptStore = ScriptParameterStore.Get())
	{
		return ScriptStore->ReadParameterVariables();
	}
	else
	{
		return TArrayView<FNiagaraVariableWithOffset>();
	}
}

#if WITH_EDITORONLY_DATA
TArrayView<const uint8> FNiagaraScriptInstanceParameterStore::GetScriptLiterals() const
{
	if (const auto* ScriptStore = ScriptParameterStore.Get())
	{
		return MakeArrayView(ScriptStore->CachedScriptLiterals);
	}
	return MakeArrayView<const uint8>(nullptr, 0);
}
#endif
