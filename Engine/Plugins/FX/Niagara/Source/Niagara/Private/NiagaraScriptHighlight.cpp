// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptHighlight.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScriptHighlight)

#if WITH_EDITORONLY_DATA
#include "JsonObjectConverter.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#endif

FNiagaraScriptHighlight::FNiagaraScriptHighlight()
	: Color(FLinearColor::Transparent)
{
}

bool FNiagaraScriptHighlight::IsValid() const
{
	return Color != FLinearColor::Transparent && DisplayName.IsEmpty() == false;
}

bool FNiagaraScriptHighlight::operator==(const FNiagaraScriptHighlight& Other) const
{
	return Color == Other.Color && 
		(DisplayName.IdenticalTo(Other.DisplayName) || DisplayName.CompareTo(Other.DisplayName) == 0);
}

#if WITH_EDITORONLY_DATA
void FNiagaraScriptHighlight::ArrayToJson(const TArray<FNiagaraScriptHighlight>& InHighlights, FString& OutJson)
{
	TArray<TSharedPtr<FJsonValue>> HighlightValues;
	for (const FNiagaraScriptHighlight& Highlight : InHighlights)
	{
		TSharedPtr<FJsonObject> HighlightObject = FJsonObjectConverter::UStructToJsonObject(Highlight);
		if (HighlightObject.IsValid())
		{
			HighlightValues.Add(MakeShared<FJsonValueObject>(HighlightObject));
		}
	}
	TSharedRef<FJsonValueArray> HighlightValuesArrayValue = MakeShared<FJsonValueArray>(HighlightValues);
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&OutJson);
	FJsonSerializer::Serialize(HighlightValuesArrayValue, FString(), JsonWriter);
}

void FNiagaraScriptHighlight::JsonToArray(const FString& InJson, TArray<FNiagaraScriptHighlight>& OutHighlights)
{
	FJsonObjectConverter::JsonArrayStringToUStruct(InJson, &OutHighlights, 0, 0);
}
#endif
