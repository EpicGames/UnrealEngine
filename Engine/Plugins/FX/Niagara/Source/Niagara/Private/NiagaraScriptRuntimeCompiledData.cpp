// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptRuntimeCompiledData.h"

#include "NiagaraDataInterface.h"
#include "NiagaraScript.h"

#if WITH_EDITORONLY_DATA

bool FNiagaraScriptRuntimeCompiledData::ValidateWithScript(UNiagaraScript& InScript) const
{
	const FNiagaraVMExecutableData& CachedScriptVM = InScript.GetVMExecutableData();
	if (CachedScriptVM.DataInterfaceInfo.Num() != ResolvedDataInterfaces.Num())
	{
		UE_LOGF(LogNiagara, Warning, "Data interface count mismatch during script presave. Invaliding compile results (see full log for details).  Script: %ls", *InScript.GetPathName());
		UE_LOGF(LogNiagara, Log, "Compiled DataInterfaceInfos:");
		for (const FNiagaraScriptDataInterfaceCompileInfo& DataInterfaceCompileInfo : CachedScriptVM.DataInterfaceInfo)
		{
			UE_LOGF(LogNiagara, Log, "Name:%ls, Type: %ls", *DataInterfaceCompileInfo.Name.ToString(), *DataInterfaceCompileInfo.Type.GetName());
		}
		UE_LOGF(LogNiagara, Log, "Resolved DataInterfaceInfos:");
		for (const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDataInterfaceData : ResolvedDataInterfaces)
		{
			UE_LOGF(LogNiagara, Log, "Name:%ls, Type: %ls, Path:%ls",
				*ResolvedDataInterfaceData.CompileName.ToString(), *ResolvedDataInterfaceData.ParameterStoreVariable.GetType().GetName(),
				ResolvedDataInterfaceData.ResolvedDataInterface != nullptr
				? *ResolvedDataInterfaceData.ResolvedDataInterface->GetPathName()
				: TEXT("None"));
		}
		return false;
	}

	if (CachedScriptVM.UObjectInfos.Num() > ResolvedUObjects.Num())
	{
		// Resolved UObjects can be written to more than one parameter so there's only an issue if there are more object infos than there are resolved objects.
		UE_LOGF(LogNiagara, Warning, "UObject count mismatch during script presave. Invaliding compile results (see full log for details).  Script: %ls", *InScript.GetPathName());
		UE_LOGF(LogNiagara, Log, "Compiled UObjects:");
		for (const FNiagaraScriptUObjectCompileInfo& Info : CachedScriptVM.UObjectInfos)
		{
			UE_LOGF(LogNiagara, Log, "Name:%ls, Type: %ls", *Info.Variable.GetName().ToString(), *Info.Variable.GetType().GetName());
		}
		UE_LOGF(LogNiagara, Log, "Cached UObjects:");
		for (const FNiagaraResolvedUObjectInfo& ResolvedUObject : ResolvedUObjects)
		{
			UE_LOGF(LogNiagara, Log, "Name:%ls, Type: %ls", *ResolvedUObject.ResolvedVariable.GetName().ToString(), *ResolvedUObject.ResolvedVariable.GetType().GetName());
		}
		return false;
	}
	return true;
}

#endif // WITH_EDITORONLY_DATA