// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptRuntimeCookedData.h"

#include "NiagaraScript.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScriptRuntimeCookedData)

#if WITH_EDITORONLY_DATA
void FNiagaraScriptRuntimeCookedData::Init(UNiagaraScript& InScript, const FNiagaraScriptRuntimeCompiledData& InScriptRuntimeCompiledData)
{
	CookedScriptRuntimeCompiledData = InScriptRuntimeCompiledData;
	CookedExecutionParameterStore.Empty();
	CookedExecutionParameterStore.AddScriptParams(&InScript, &InScriptRuntimeCompiledData);
	CookedRapidIterationBindings.Empty();
	FNiagaraParameterStoreBinding::GetBindingData(&CookedExecutionParameterStore, &InScript.RapidIterationParameters, CookedRapidIterationBindings);
}
#endif //WITH_EDITORONLY_DATA

bool FNiagaraScriptRuntimeCookedData::ValidateWithScript(UNiagaraScript& InScript) const
{
	const int32 RapidIterationParameterSize = InScript.RapidIterationParameters.GetParameterDataArray().Num();
	const int32 ScriptExecutionParameterSize = CookedExecutionParameterStore.GetParameterDataArray().Num();

	for (const FNiagaraBoundParameter& Binding : CookedRapidIterationBindings)
	{
		const int32 ParameterSize = Binding.Parameter.GetSizeInBytes();
		if (((Binding.SrcOffset + ParameterSize) > RapidIterationParameterSize) ||
			((Binding.DestOffset + ParameterSize) > ScriptExecutionParameterSize))
		{
#if WITH_EDITOR && IS_MONOLITHIC
			UE_LOGF(LogNiagara, Warning, "Mismatch between binding between RapidIterationParamters and script CookedExecutionParameters for script %ls", *InScript.GetPathName());
#else
			UE_LOGF(LogNiagara, Error, "Mismatch between binding between RapidIterationParamters and script CookedExecutionParameters for script %ls", *InScript.GetPathName());
#endif
			return false;
		}
	}
	return true;
}
