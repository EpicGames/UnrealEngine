// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptRuntimeTransientData.h"

#include "NiagaraDataInterface.h"
#include "NiagaraScript.h"
#include "NiagaraScriptRuntimeCompiledData.h"

#if WITH_EDITOR
FNiagaraScriptRuntimeTransientData::FNiagaraScriptRuntimeTransientData(UNiagaraScript& Script, const FNiagaraScriptRuntimeCompiledData& ScriptRuntimeCompiledData)
	: ExecutionReadyParameterStore(new FNiagaraScriptExecutionParameterStore())
	, bOwnsParameterStore(true)
{
	ExecutionReadyParameterStore->InitFromOwningScript(&Script, &ScriptRuntimeCompiledData, true);
	GenerateDefaultFunctionBindings(Script, ScriptRuntimeCompiledData.ResolvedDataInterfaces);
}
#endif

FNiagaraScriptRuntimeTransientData::FNiagaraScriptRuntimeTransientData(UNiagaraScript& Script, const FNiagaraScriptRuntimeCompiledData& ScriptRuntimeCompiledData, FNiagaraScriptExecutionParameterStore& CookedExecutionReadyParameterStore)
	: ExecutionReadyParameterStore(&CookedExecutionReadyParameterStore)
	, bOwnsParameterStore(false)
{
	GenerateDefaultFunctionBindings(Script, ScriptRuntimeCompiledData.ResolvedDataInterfaces);
}

FNiagaraScriptRuntimeTransientData::~FNiagaraScriptRuntimeTransientData()
{
	if (bOwnsParameterStore)
	{
		ExecutionReadyParameterStore->UnbindFromSourceStores();
		delete ExecutionReadyParameterStore;
	}
}

void FNiagaraScriptRuntimeTransientData::GenerateDefaultFunctionBindings(const UNiagaraScript& Script, const TArray<FNiagaraScriptResolvedDataInterfaceInfo>& ResolvedDataInterfaces)
{
	const FNiagaraVMExecutableData& ScriptExecutableData = Script.GetVMExecutableData();
	const int32 ExternalFunctionCount = ScriptExecutableData.CalledVMExternalFunctions.Num();
	CalledVMExternalFunctionBindings.Empty(ExternalFunctionCount);
	if (ExternalFunctionCount > 0)
	{
		const TArray<UNiagaraDataInterface*>& ScriptDataInterfaces = ExecutionReadyParameterStore->GetDataInterfaces();
		const int32 DataInterfaceCount = FMath::Min(ResolvedDataInterfaces.Num(), ScriptDataInterfaces.Num());

		// If we are transacting do not perform the ensures here as they can result in false positives UE-201266
		// For example, if we were to undo / redo these ensures will execute and ScriptDataInterfaces could be out of date resulting in a false positive, we will recache the script data post action and get the correct data
#if WITH_EDITORONLY_DATA
		if (!GIsTransacting)
#endif
		{
			ensureMsgf(DataInterfaceCount == ResolvedDataInterfaces.Num(), TEXT("DataInterface count does not match VM this is likely caused by missing data interface classes"));
			ensureMsgf(DataInterfaceCount == ScriptDataInterfaces.Num(), TEXT("DataInterface count does not match script data interfaces this is likely caused by missing data interface classes"));
		}

		for (const FVMExternalFunctionBindingInfo& BindingInfo : ScriptExecutableData.CalledVMExternalFunctions)
		{
			FVMExternalFunction& FuncBind = CalledVMExternalFunctionBindings.AddDefaulted_GetRef();
			for (int32 DataInterfaceIt = 0; DataInterfaceIt < DataInterfaceCount; ++DataInterfaceIt)
			{
				const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDataInterfaceData = ResolvedDataInterfaces[DataInterfaceIt];
				if (ResolvedDataInterfaceData.UserPtrIdx == INDEX_NONE && ResolvedDataInterfaceData.CompileName == BindingInfo.OwnerName)
				{
					UNiagaraDataInterface* ScriptDataInterface = ScriptDataInterfaces[DataInterfaceIt];
					if (ensureMsgf(ScriptDataInterface, TEXT("Script(%s) Data Interface Slot %d is nullptr"), *GetFullNameSafe(&Script), DataInterfaceIt))
					{
						ScriptDataInterfaces[DataInterfaceIt]->GetVMExternalFunction(BindingInfo, nullptr, FuncBind);
					}
				}
			}
		}
	}
}