// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSettings.h"
#include "NiagaraEffectType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSettings)

UNiagaraSettings::UNiagaraSettings(const FObjectInitializer& ObjectInitlaizer)
	: Super(ObjectInitlaizer)
	, NDISkelMesh_GpuMaxInfluences(ENDISkelMesh_GpuMaxInfluences::Unlimited)
	, NDISkelMesh_GpuUniformSamplingFormat(ENDISkelMesh_GpuUniformSamplingFormat::Full)
	, NDISkelMesh_AdjacencyTriangleIndexFormat(ENDISkelMesh_AdjacencyTriangleIndexFormat::Full)
{
	PositionPinTypeColor = FLinearColor(1.0f, 0.3f, 1.0f, 1.0f);

	NDICollisionQuery_AsyncGpuTraceProviderOrder.Add(ENDICollisionQuery_AsyncGpuTraceProvider::Type::HWRT);
	NDICollisionQuery_AsyncGpuTraceProviderOrder.Add(ENDICollisionQuery_AsyncGpuTraceProvider::Type::GSDF);

	SimCacheAuxiliaryFileBasePath = "{project_dir}/Saved/NiagaraSimCache";
	SimCacheMaxCPUMemoryVolumetrics = 5000;
}

FName UNiagaraSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR

void UNiagaraSettings::AddEnumParameterType(UEnum* Enum)
{
	if(!AdditionalParameterEnums.Contains(Enum))
	{
		AdditionalParameterEnums.Add(Enum);
		FNiagaraTypeDefinition::RecreateUserDefinedTypeRegistry();
	}
}

FText UNiagaraSettings::GetSectionText() const
{
	return NSLOCTEXT("NiagaraPlugin", "NiagaraSettingsSection", "Niagara");
}

void UNiagaraSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		SettingsChangedDelegate.Broadcast(PropertyChangedEvent.Property->GetFName(), this);
	}
}

UNiagaraSettings::FOnNiagaraSettingsChanged& UNiagaraSettings::OnSettingsChanged()
{
	return SettingsChangedDelegate;
}

UNiagaraSettings::FOnNiagaraSettingsChanged UNiagaraSettings::SettingsChangedDelegate;

UNiagaraEffectType* UNiagaraSettings::GetDefaultEffectType() const
{
	return Cast<UNiagaraEffectType>(DefaultEffectType.TryLoad());
}

#endif

void UNiagaraSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.Niagara.Shader.EnableHLSL2021")))
	{
		CVar->Set(bEnableHLSL2021);
	}
#endif
}
