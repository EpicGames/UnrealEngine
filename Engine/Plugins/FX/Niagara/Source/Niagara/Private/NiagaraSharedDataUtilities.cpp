// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSharedDataUtilities.h"

#include "NiagaraDataManager_StaticMesh.h"
#include "NiagaraWorldManagerImpl.h"

namespace NiagaraSharedDataUtilities
{
	FNiagaraSharedDataPtr GetOrCreateSharedData_StaticMesh(FNiagaraWorldManager* WorldMan, UStaticMeshComponent* StaticMeshComponent, TArrayView<UNiagaraComponent*> ReferencingComponents)
	{
		UNiagaraDataManager_StaticMesh& MeshDataMan = WorldMan->GetOrCreateDataManager<UNiagaraDataManager_StaticMesh>();
		FNiagaraSharedData_StaticMeshPtr SharedMeshData = MeshDataMan.GetOrCreateSharedMeshData(StaticMeshComponent);
		
		for(UNiagaraComponent* ReferencingComp : ReferencingComponents)
		{
			MeshDataMan.RegisterReferencingSystem(ReferencingComp, SharedMeshData);
		}

		//Return the shared data pointer associated with this static mesh.
		//This is valid as long as there are things referencing SharedMeshData
		return SharedMeshData;
	}
}
