// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimCache.h"

#include "EngineAnalytics.h"
#include "NiagaraAnalytics.h"
#include "Engine/World.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "NiagaraConstants.h"
#include "NiagaraComponent.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraDataSetReadback.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSimCacheAttributeReaderHelper.h"
#include "NiagaraSimCacheCustomStorageInterface.h"
#include "NiagaraSimCacheDebugData.h"
#include "NiagaraSimCacheHelper.h"
#include "NiagaraSystemImpl.h"
#include "NiagaraSystemInstance.h"
#include "Stateless/NiagaraStatelessEmitterInstance.h"

#if WITH_EDITORONLY_DATA
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSimCache)

#define LOCTEXT_NAMESPACE "NiagaraSimCache"

UNiagaraSimCache::FOnCacheBeginWrite	UNiagaraSimCache::OnCacheBeginWrite;
UNiagaraSimCache::FOnCacheEndWrite		UNiagaraSimCache::OnCacheEndWrite;

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraSimCacheCreateParameters FNiagaraSimCacheCreateParameters::CreateForDebugging()
{
	FNiagaraSimCacheCreateParameters CreateParameters;
	CreateParameters.bIncludeDebugData = true;
	return CreateParameters;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool FNiagaraSimCacheVariable::operator==(const FNiagaraSimCacheVariable& Other) const
{
	return
		Variable	== Other.Variable &&
		FloatOffset == Other.FloatOffset &&
		FloatCount	== Other.FloatCount &&
		HalfOffset	== Other.HalfOffset &&
		HalfCount	== Other.HalfCount &&
		Int32Offset	== Other.Int32Offset &&
		Int32Count	== Other.Int32Count;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraSimCacheFeedbackContext::~FNiagaraSimCacheFeedbackContext()
{
	if (bAutoLogIssues)
	{
		for (const FString& Warning : Warnings)
		{
			UE_LOGF(LogNiagara, Warning, "SimCache Warning: %ls", *Warning);
		}

		for (const FString& Error : Errors)
		{
			UE_LOGF(LogNiagara, Warning, "SimCache Error: %ls", *Error);
		}
	}
}


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FNiagaraSimCacheDataBuffersLayout::IndexOfCacheVariable(const FNiagaraVariableBase& InVariable) const
{
	return Variables.IndexOfByPredicate(
		[InVariable](const FNiagaraSimCacheVariable& CacheVariable)
		{
			return CacheVariable.Variable == InVariable;
		}
	);
}

const FNiagaraSimCacheVariable* FNiagaraSimCacheDataBuffersLayout::FindCacheVariable(const FNiagaraVariableBase& InVariable) const
{
	const int32 Index = IndexOfCacheVariable(InVariable);
	return Index != INDEX_NONE ? &Variables[Index] : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FNiagaraSimCacheBuffersSetup
{
	explicit FNiagaraSimCacheBuffersSetup(const FNiagaraSimCacheDataBuffersLayout& CacheLayout, const FNiagaraSimCacheDataBuffers& DataBuffer)
	{
		FloatDataNum			= CacheLayout.FloatCount * DataBuffer.NumInstances;
		HalfDataNum				= CacheLayout.HalfCount * DataBuffer.NumInstances;
		Int32DataNum			= CacheLayout.Int32Count * DataBuffer.NumInstances;
		IDToIndexTableNum		= DataBuffer.IDToIndexTableElements;
		InterpMappingNum		= CacheLayout.bAllowInterpolation ? DataBuffer.NumInstances : 0;

		FloatDataOffset			= 0;
		HalfDataOffset			= Align(FloatDataOffset + (FloatDataNum * sizeof(float)), sizeof(FFloat16));
		Int32DataOffset			= Align(HalfDataOffset + (HalfDataNum * sizeof(FFloat16)), sizeof(int32));
		IDToIndexTableOffset	= Align(Int32DataOffset + (Int32DataNum * sizeof(int32)), sizeof(int32));
		InterpMappingOffset		= Align(IDToIndexTableOffset + (IDToIndexTableNum * sizeof(int32)), sizeof(uint32));
		BufferSize				= InterpMappingOffset + (InterpMappingNum * sizeof(int32));
	}

	bool SetArrayViews(FNiagaraSimCacheDataBuffers& DataBuffer) const
	{
		const TArrayView64<uint8> DataBufferView = DataBuffer.DataBuffer.GetView();
		if (DataBufferView.Num() != BufferSize)
		{
			DataBuffer.NumInstances		= 0;
			DataBuffer.FloatData		= TArrayView<uint8>();
			DataBuffer.HalfData			= TArrayView<uint8>();
			DataBuffer.Int32Data		= TArrayView<uint8>();
			DataBuffer.IDToIndexTable	= TArrayView<int32>();
			DataBuffer.InterpMapping	= TArrayView<uint32>();
			return false;
		}
		else
		{
			uint8* BaseData				= DataBufferView.GetData();
			DataBuffer.FloatData		= TArrayView<uint8>(BaseData + FloatDataOffset, FloatDataNum * sizeof(float));
			DataBuffer.HalfData			= TArrayView<uint8>(BaseData + HalfDataOffset, HalfDataNum * sizeof(FFloat16));
			DataBuffer.Int32Data		= TArrayView<uint8>(BaseData + Int32DataOffset, Int32DataNum * sizeof(int32));
			DataBuffer.IDToIndexTable	= TArrayView<int32>(reinterpret_cast<int32*>(BaseData + IDToIndexTableOffset), static_cast<int32>(IDToIndexTableNum));
			DataBuffer.InterpMapping	= TArrayView<uint32>(reinterpret_cast<uint32*>(BaseData + InterpMappingOffset), static_cast<int32>(InterpMappingNum));
			return true;
		}
	}

	int64 FloatDataNum = 0;
	int64 HalfDataNum = 0;
	int64 Int32DataNum = 0;
	int64 IDToIndexTableNum = 0;
	int64 InterpMappingNum = 0;

	int64 FloatDataOffset = 0;
	int64 HalfDataOffset = 0;
	int64 Int32DataOffset = 0;
	int64 IDToIndexTableOffset = 0;
	int64 InterpMappingOffset = 0;
	int64 BufferSize = 0;
};

void FNiagaraSimCacheDataBuffers::SetupForWrite(const FNiagaraSimCacheDataBuffersLayout& CacheLayout)
{
	const FNiagaraSimCacheBuffersSetup BufferSetup(CacheLayout, *this);

	BulkData.RemoveBulkData();
	DataBuffer.Reset(static_cast<uint8*>(FMemory::Malloc(BufferSetup.BufferSize)), BufferSetup.BufferSize);
	const bool bSuccess = BufferSetup.SetArrayViews(*this);
	checkf(bSuccess == true, TEXT("Cache Write setup can not fail."));
}

bool FNiagaraSimCacheDataBuffers::SetupForRead(const FNiagaraSimCacheDataBuffersLayout& CacheLayout)
{
	const FNiagaraSimCacheBuffersSetup BufferSetup(CacheLayout, *this);

	if (DataBuffer.GetView().Num() != BufferSetup.BufferSize)
	{
		DataBuffer = BulkData.GetCopyAsBuffer(0, true);
		BulkData.UnloadBulkData();
	}

	return BufferSetup.SetArrayViews(*this);
}

bool FNiagaraSimCacheDataBuffers::SerializeAsArray(FArchive& Ar, UObject* OwnerObject, const FNiagaraSimCacheDataBuffersLayout& CacheLayout)
{
	TArray<uint8> TransientArray;
	TransientArray = DataBuffer.GetView();

	Ar << TransientArray;

	if (Ar.IsLoading())
	{
		uint8* DataBufferMemory = nullptr;
		if (TransientArray.Num() > 0)
		{
			DataBufferMemory = static_cast<uint8*>(FMemory::Malloc(TransientArray.Num()));
			FMemory::Memcpy(DataBufferMemory, TransientArray.GetData(), TransientArray.Num());
		}
		DataBuffer.Reset(DataBufferMemory, TransientArray.Num());
	}

	return SetupForRead(CacheLayout);
}

bool FNiagaraSimCacheDataBuffers::SerializeAsBulkData(FArchive& Ar, UObject* OwnerObject, const FNiagaraSimCacheDataBuffersLayout& CacheLayout)
{
	if (Ar.IsSaving())
	{
		TArrayView64<uint8> DataBufferView = DataBuffer.GetView();
		if (DataBufferView.Num() > 0)
		{
			BulkData.Lock(LOCK_READ_WRITE);
			{
				uint8* NewBulkData = BulkData.Realloc(DataBufferView.Num());
				FMemory::Memcpy(NewBulkData, DataBufferView.GetData(), DataBufferView.Num());
			}
			BulkData.Unlock();
		}
		else
		{
			BulkData.RemoveBulkData();
		}
	}

	BulkData.Serialize(Ar, OwnerObject);

	return SetupForRead(CacheLayout);
}

bool FNiagaraSimCacheDataBuffers::operator==(const FNiagaraSimCacheDataBuffers& Other) const
{
	bool bMatches;
	bMatches = NumInstances == Other.NumInstances;
	bMatches &= IDAcquireTag == Other.IDAcquireTag;
	bMatches &= FloatData.Num() == Other.FloatData.Num();
	bMatches &= HalfData.Num() == Other.HalfData.Num();
	bMatches &= Int32Data.Num() == Other.Int32Data.Num();
	bMatches &= IDToIndexTable.Num() == Other.IDToIndexTable.Num();
	bMatches &= InterpMapping.Num() == Other.InterpMapping.Num();

	TConstArrayView<uint8> ThisDataBuffer = DataBuffer.GetView();
	TConstArrayView<uint8> OtherDataBuffer = Other.DataBuffer.GetView();
	bMatches &= ThisDataBuffer.Num() == OtherDataBuffer.Num();
	if (bMatches)
	{
		bMatches &= FMemory::Memcmp(ThisDataBuffer.GetData(), OtherDataBuffer.GetData(), ThisDataBuffer.Num()) == 0;
	}

	return bMatches;
}

bool FNiagaraSimCacheDataBuffers::operator!=(const FNiagaraSimCacheDataBuffers& Other) const
{
	return !(*this == Other);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UNiagaraSimCache::UNiagaraSimCache(const FObjectInitializer& ObjectInitializer)
{
}

void UNiagaraSimCache::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);

	Super::Serialize(Ar);

	// Early out if we aren't using bulk data, these caches will no longer be supported
	const int32 NiagaraVersion = Ar.CustomVer(FNiagaraCustomVersion::GUID);
	if (NiagaraVersion < FNiagaraCustomVersion::SimCache_BulkDataVersion1)
	{
		UE_LOGF(LogNiagara, Warning, "SimCache needs to be regenerated as this version is no longer supported. %ls (system %ls)", *GetPathName(), *GetPathNameSafe(SoftNiagaraSystem.Get()));
		SoftNiagaraSystem.Reset();
		return;
	}

	bool bIsCacheValid = true;
	if ( CreateParameters.bAllowSerializeLargeCache )
	{
		for (FNiagaraSimCacheFrame& CacheFrame : CacheFrames)
		{
			bIsCacheValid &= CacheFrame.SystemData.SystemDataBuffers.SerializeAsBulkData(Ar, this, CacheLayout.SystemLayout);
			for (int32 iEmitter = 0; iEmitter < CacheLayout.EmitterLayouts.Num(); ++iEmitter)
			{
				FNiagaraSimCacheEmitterFrame& EmitterData = CacheFrame.EmitterData[iEmitter];
				bIsCacheValid &= EmitterData.ParticleDataBuffers.SerializeAsBulkData(Ar, this, CacheLayout.EmitterLayouts[iEmitter]);
			}
		}
	}
	else
	{
		for (FNiagaraSimCacheFrame& CacheFrame : CacheFrames)
		{
			bIsCacheValid &= CacheFrame.SystemData.SystemDataBuffers.SerializeAsArray(Ar, this, CacheLayout.SystemLayout);
			for (int32 iEmitter = 0; iEmitter < CacheLayout.EmitterLayouts.Num(); ++iEmitter)
			{
				FNiagaraSimCacheEmitterFrame& EmitterData = CacheFrame.EmitterData[iEmitter];
				bIsCacheValid &= EmitterData.ParticleDataBuffers.SerializeAsArray(Ar, this, CacheLayout.EmitterLayouts[iEmitter]);
			}
		}
	}

	if (!bIsCacheValid)
	{
		UE_LOGF(LogNiagara, Warning, "SimCache buffer serialization failed, likely due to bulk data not being supported, cache is invalid. %ls", *GetPathName());
		SoftNiagaraSystem.Reset();
	}
}

bool UNiagaraSimCache::IsReadyForFinishDestroy()
{
	return PendingCommandsInFlight == 0;
}

bool UNiagaraSimCache::BeginWrite(FNiagaraSimCacheCreateParameters InCreateParameters, UNiagaraComponent* NiagaraComponent)
{
	FNiagaraSimCacheFeedbackContext FeedbackContext;
	return BeginWrite(InCreateParameters, NiagaraComponent, FeedbackContext);
}

bool UNiagaraSimCache::BeginWrite(FNiagaraSimCacheCreateParameters InCreateParameters, UNiagaraComponent* NiagaraComponent, FNiagaraSimCacheFeedbackContext& FeedbackContext)
{
	UE::TWriteScopeLock RWGuard(RWLock);

	check(PendingCommandsInFlight == 0);

	OnCacheBeginWrite.Broadcast(this);

	FNiagaraSimCacheHelper Helper(NiagaraComponent);
	if (Helper.HasValidSimulation() == false)
	{
		FeedbackContext.Errors.Emplace(FString::Printf(TEXT("No valid simulation data for %s"), *GetPathNameSafe(Helper.NiagaraSystem)));
		return false;
	}

	if (CacheGuid.IsValid() == false)
	{
		CacheGuid = FGuid::NewGuid();
	}

	Modify();
#if WITH_EDITOR
	PostEditChange();
#endif

	// Reset to defaults
	SoftNiagaraSystem = Helper.NiagaraSystem;
	CreateParameters = InCreateParameters;
	StartSeconds = 0.0f;
	DurationSeconds = 0.0f;
	CacheLayout = FNiagaraSimCacheLayout();
	CacheFrames.Empty();
	CaptureTickCount = INDEX_NONE;
	CaptureStartTime = FPlatformTime::Seconds();

	for ( auto it=DataInterfaceStorage.CreateIterator(); it; ++it )
	{
		if (it->Value)
		{
			it->Value->MarkAsGarbage();
		}
	}
	DataInterfaceStorage.Empty();

	// Create our debug data (if required)
	if (CreateParameters.bIncludeDebugData)
	{
		DebugData = NewObject<UNiagaraSimCacheDebugData>(this);
		DebugData->SystemStateData = Helper.NiagaraSystem->GetSystemStateData();
	}
	else
	{
		DebugData = nullptr;
	}

	// Not explicit capture mode?  Empty the list as internally we reuse this list when in rendering only mode
	if (CreateParameters.AttributeCaptureMode == ENiagaraSimCacheAttributeCaptureMode::ExplicitAttributes)
	{
		// Invalid as the user specified nothing to actual capture
		if (CreateParameters.ExplicitCaptureAttributes.IsEmpty())
		{
			FeedbackContext.Errors.Emplace(FString::Printf(TEXT("Missing attributes to capture when in explicit capture mode %s"), *GetPathNameSafe(Helper.NiagaraSystem)));
			SoftNiagaraSystem.Reset();
			return false;
		}
	}
	else
	{
		CreateParameters.ExplicitCaptureAttributes.Empty();
	}

	// When in rendering only mode ask all renderers which attributes we need to capture
	if ( CreateParameters.AttributeCaptureMode == ENiagaraSimCacheAttributeCaptureMode::RenderingOnly )
	{
		FNameBuilder NameBuilder;

	#if WITH_EDITORONLY_DATA
		for ( const FNiagaraEmitterHandle& EmitterHandle : Helper.NiagaraSystem->GetEmitterHandles() )
		{
			//-TODO:Stateless
			FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetInstance().GetEmitterData();
			if (EmitterData == nullptr)
			{
				continue;
			}

			EmitterData->ForEachEnabledRenderer(
				[&](UNiagaraRendererProperties* RenderProperties)
				{
					for (FNiagaraVariableBase BoundAttribute : RenderProperties->GetBoundAttributes())
					{
						NameBuilder.Reset();
						NameBuilder.Append(EmitterHandle.GetUniqueInstanceName());
						NameBuilder.AppendChar(TEXT('.'));
						BoundAttribute.GetName().AppendString(NameBuilder);

						CreateParameters.ExplicitCaptureAttributes.AddUnique(NameBuilder.ToString());
					}
				}
			);
		}
	#endif
		static const FName NAME_ExecutionState("ExecutionState");
		static const FName NAME_SystemExecutionState("System.ExecutionState");
		CreateParameters.ExplicitCaptureAttributes.AddUnique(NAME_SystemExecutionState);

		for (const FNiagaraEmitterInstanceRef& EmitterInstance : Helper.SystemInstance->GetEmitters())
		{
			const FNiagaraParameterStore& RendererBindings = EmitterInstance->GetRendererBoundVariables();
			for ( const FNiagaraVariableWithOffset& Variable : RendererBindings.ReadParameterVariables() )
			{
				CreateParameters.ExplicitCaptureAttributes.AddUnique(Variable.GetName());
			}

			NameBuilder.Reset();
			NameBuilder.Append(EmitterInstance->GetEmitterHandle().GetUniqueInstanceName());
			NameBuilder.AppendChar(TEXT('.'));
			NAME_ExecutionState.AppendString(NameBuilder);
			CreateParameters.ExplicitCaptureAttributes.AddUnique(NameBuilder.ToString());
		}

		FNiagaraDataInterfaceUtilities::ForEachDataInterface(
			Helper.SystemInstance,
			[&](const FNiagaraDataInterfaceUtilities::FDataInterfaceUsageContext& UsageContext)
			{
				if (INiagaraSimCacheCustomStorageInterface* SimCacheCustomStorageInterface = Cast<INiagaraSimCacheCustomStorageInterface>(UsageContext.DataInterface))
				{
					for (const FNiagaraVariableBase& PreserveAttribute : SimCacheCustomStorageInterface->GetSimCacheRendererAttributes(UsageContext.OwnerObject))
					{
						CreateParameters.ExplicitCaptureAttributes.AddUnique(PreserveAttribute.GetName());
					}
				}
				return true;
			}
		);
	}

	// Build new layout for system / emitters
	Helper.BuildCacheLayoutForSystem(CreateParameters, CacheLayout.SystemLayout);

	const int32 NumEmitters = Helper.NiagaraSystem->GetEmitterHandles().Num();
	CacheLayout.EmitterLayouts.AddDefaulted(NumEmitters);
	for ( int32 i=0; i < NumEmitters; ++i )
	{
		Helper.BuildCacheLayoutForEmitter(CreateParameters, CacheLayout.EmitterLayouts[i], i);
	}

	// Find data interfaces we may want to cache
	if ( CreateParameters.bAllowDataInterfaceCaching )
	{
		TSet<FNiagaraVariableBase> VisitedDataInterfaces;
		FNiagaraDataInterfaceUtilities::ForEachDataInterface(
			Helper.SystemInstance,
			[&](const FNiagaraVariableBase& Variable, UNiagaraDataInterface* DataInterface)
			{
				// Are we capturing this data interface?
				if ((CreateParameters.ExplicitCaptureAttributes.Num() != 0) && !CreateParameters.ExplicitCaptureAttributes.Contains(Variable.GetName()))
				{
					return true;
				}

				if (INiagaraSimCacheCustomStorageInterface* SimCacheCustomStorageInterface = Cast<INiagaraSimCacheCustomStorageInterface>(DataInterface))
				{
					if (VisitedDataInterfaces.Contains(Variable))
					{
						return true;
					}
					VisitedDataInterfaces.Add(Variable);

					const void* PerInstanceData = Helper.SystemInstance->FindDataInterfaceInstanceData(DataInterface);
					if (UObject* DICacheStorage = SimCacheCustomStorageInterface->SimCacheBeginWrite(this, Helper.SystemInstance, PerInstanceData, FeedbackContext))
					{
						DataInterfaceStorage.FindOrAdd(Variable) = DICacheStorage;
					}
				}
				return true;
			}
		);
	}

#if WITH_EDITORONLY_DATA
	// Check all our bindings are ok
	for ( const FNiagaraEmitterHandle& EmitterHandle : Helper.NiagaraSystem->GetEmitterHandles() )
	{
		EmitterHandle.GetInstance().GetEmitterData()->ForEachEnabledRenderer(
			[&](UNiagaraRendererProperties* RenderProperties)
			{
				for (FNiagaraVariableBase BoundAttribute : RenderProperties->GetBoundAttributes())
				{
					if (!BoundAttribute.IsDataInterface())
					{
						continue;
					}

					UNiagaraDataInterface* DataInterfaceCDO = CastChecked<UNiagaraDataInterface>(BoundAttribute.GetType().GetClass()->GetDefaultObject());
					if (DataInterfaceCDO->PerInstanceDataSize() == 0)
					{
						continue;
					}

					if (!DataInterfaceStorage.Contains(BoundAttribute))
					{
						FeedbackContext.Warnings.Emplace(FString::Printf(TEXT("Material Data Interface Binding '%s : %s' did not store any data or does not support sim caching, baked result may be incorrect."), *DataInterfaceCDO->GetClass()->GetName(), *BoundAttribute.GetName().ToString()));
					}
				}
			}
		);
	}
#endif

	return true;
}

bool UNiagaraSimCache::BeginAppend(FNiagaraSimCacheCreateParameters InCreateParameters, UNiagaraComponent* NiagaraComponent)
{
	FNiagaraSimCacheFeedbackContext FeedbackContext;
	return BeginAppend(InCreateParameters, NiagaraComponent, FeedbackContext);
}

bool UNiagaraSimCache::BeginAppend(FNiagaraSimCacheCreateParameters InCreateParameters, UNiagaraComponent* NiagaraComponent, FNiagaraSimCacheFeedbackContext& FeedbackContext)
{
	UE::TWriteScopeLock RWGuard(RWLock);

	check(PendingCommandsInFlight == 0);

	OnCacheBeginWrite.Broadcast(this);

	FNiagaraSimCacheHelper Helper(NiagaraComponent);
	if (Helper.HasValidSimulation() == false)
	{
		//FeedbackContext.Errors.Emplace(FString::Printf(TEXT("No valid simulation data for %s"), *GetPathNameSafe(Helper.NiagaraSystem)));
		return false;
	}

	if (!CacheGuid.IsValid() || SoftNiagaraSystem.Get() != Helper.NiagaraSystem)
	{
		return BeginWrite(InCreateParameters, NiagaraComponent);
	}

	// Are we clearing out all frames
	const float SimulationAge = Helper.SystemInstance->GetAge();
	const int32 SimulationTickCount = Helper.SystemInstance->GetTickCount();
	if (SimulationAge <= StartSeconds || SimulationTickCount == 0)
	{
		CacheFrames.Empty();
	}
	// Cull any frames that have a later start time or tick count
	else if (SimulationAge <= StartSeconds + DurationSeconds)
	{
		int32 CullFrame = CacheFrames.Num();
		while (CullFrame > 0)
		{
			const FNiagaraSimCacheFrame& Frame = CacheFrames[CullFrame - 1];
			if (Frame.SimulationAge < SimulationAge && Frame.SimulationTickCount < SimulationTickCount)
			{
				break;
			}
			--CullFrame;
		}

		if (CullFrame != CacheFrames.Num())
		{
			CacheFrames.SetNum(CullFrame);
			if (CacheFrames.Num() > 0)
			{
				DurationSeconds = CacheFrames.Last().SimulationAge - StartSeconds;
				CaptureTickCount = CacheFrames.Last().SimulationTickCount;
			}
		}
	}

	// We don't support debug caches when appending
	DebugData = nullptr;

	if (CacheFrames.Num() == 0)
	{
		StartSeconds = 0.0f;
		DurationSeconds = 0.0f;
		CaptureTickCount = INDEX_NONE;
	}
	else
	{
		CaptureTickCount = SimulationTickCount - 1;
	}
	CaptureStartTime = FPlatformTime::Seconds();

	return true;
}

bool UNiagaraSimCache::WriteFrame(UNiagaraComponent* NiagaraComponent)
{
	FNiagaraSimCacheFeedbackContext FeedbackContext;
	return WriteFrame(NiagaraComponent, FeedbackContext);
}

bool UNiagaraSimCache::WriteFrame(UNiagaraComponent* NiagaraComponent, FNiagaraSimCacheFeedbackContext& FeedbackContext)
{
	UE::TWriteScopeLock RWGuard(RWLock);

	FNiagaraSimCacheHelper Helper(NiagaraComponent);
	if (Helper.HasValidSimulationData() == false)
	{
		SoftNiagaraSystem.Reset();
		FeedbackContext.Errors.Emplace(FString::Printf(TEXT("No valid simulation data for %s"), *GetPathNameSafe(Helper.NiagaraSystem)));
		return false;
	}

	if ( SoftNiagaraSystem.Get() != Helper.NiagaraSystem )
	{
		FeedbackContext.Errors.Emplace(FString::Printf(TEXT("System %s != System %s"), *GetPathNameSafe(SoftNiagaraSystem.Get()), *GetPathNameSafe(Helper.NiagaraSystem)));
		SoftNiagaraSystem.Reset();
		return false;
	}

	// Simulation is complete nothing to cache
	if ( Helper.SystemInstance->IsComplete() )
	{
		FeedbackContext.Errors.Emplace(FString::Printf(TEXT("System already completed its simulation. %s"), *Helper.SystemInstance->GetCrashReporterTag()));
		return false;
	}

	// Is the simulation running?  If not nothing to cache yet
	if ( Helper.SystemInstance->SystemInstanceState != ENiagaraSystemInstanceState::Running )
	{
		// This warning will trigger way to often and isn't that useful
		//	FeedbackContext.Errors.Emplace(FString::Printf(TEXT("System is not running, so there is no data to cache. %s"), *Helper.SystemInstance->GetCrashReporterTag()));
		return false;
	}

	// Is this the first frame?  If so capture the cache start time
	if (CaptureTickCount == INDEX_NONE)
	{
		StartSeconds = Helper.SystemInstance->GetAge();
	}
	else
	{
		// If our tick counter hasn't moved then we won't capture the frame as there's nothing new to process
		if (CaptureTickCount == Helper.SystemInstance->GetTickCount())
		{
			//FeedbackContext.Errors.Emplace(FString::Printf(TEXT("System was not ticked since the last capture. %s"), *Helper.SystemInstance->GetCrashReporterTag()));
			return false;
		}

		// If the tick counter is lower than the previous value then the system was reset so we won't capture
		if (Helper.SystemInstance->GetTickCount() < CaptureTickCount)
		{
			FeedbackContext.Errors.Emplace(FString::Printf(TEXT("System was was reset since the last capture, skipping further cache writes. %s"), *Helper.SystemInstance->GetCrashReporterTag()));
			SoftNiagaraSystem.Reset();
			return false;
		}
	}

	CaptureTickCount = Helper.SystemInstance->GetTickCount();
	DurationSeconds = Helper.SystemInstance->GetAge() - StartSeconds;

	// Cache frame
	FNiagaraSimCacheFrame& CacheFrame = CacheFrames.AddDefaulted_GetRef();
	CacheFrame.LocalToWorld = Helper.SystemInstance->GatheredInstanceParameters.ComponentTrans;
	CacheFrame.LWCTile = Helper.SystemInstance->GetLWCTile();
	CacheFrame.SimulationAge = FMath::RoundToFloat(Helper.SystemInstance->GetAge() * CacheAgeResolution) / CacheAgeResolution;
#if WITH_EDITORONLY_DATA
	CacheFrame.SimulationTickCount = Helper.SystemInstance->GetTickCount();
#endif

	CacheFrame.SystemData.LocalBounds = Helper.SystemInstance->GetLocalBounds();

	const int32 NumEmitters = CacheLayout.EmitterLayouts.Num();
	CacheFrame.EmitterData.AddDefaulted(NumEmitters);

	Helper.WriteDataBuffer(*Helper.SystemSimulationDataBuffer, CacheLayout.SystemLayout, CacheFrame.SystemData.SystemDataBuffers, Helper.SystemInstance->GetSystemInstanceIndex(), 1);

	bool bNeedsGpuTickFlush = true;
	for ( int32 i=0; i < NumEmitters; ++i )
	{
		FNiagaraSimCacheEmitterFrame& CacheEmitterFrame = CacheFrame.EmitterData[i];
		FNiagaraEmitterInstance& EmitterInstance = Helper.SystemInstance->GetEmitters()[i].Get();
		FNiagaraDataBuffer* EmitterCurrentData = EmitterInstance.GetParticleData().GetCurrentData();
		if (EmitterInstance.IsComplete() || !EmitterCurrentData)
		{
			continue;
		}

		CacheEmitterFrame.LocalBounds = EmitterInstance.GetBounds();
		CacheEmitterFrame.TotalSpawnedParticles = EmitterInstance.GetTotalSpawnedParticles();
		if (CacheLayout.EmitterLayouts[i].SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			if (FNiagaraStatelessEmitterInstance* StatelessEmitterInstance = EmitterInstance.AsStateless())
			{
				FNiagaraDataBufferRef DataBuffer = new FNiagaraDataBuffer(EmitterCurrentData->GetOwner());
				StatelessEmitterInstance->CaptureForDebugging(DataBuffer);
				Helper.WriteDataBuffer(*DataBuffer, CacheLayout.EmitterLayouts[i], CacheEmitterFrame.ParticleDataBuffers, 0, DataBuffer->GetNumInstances());
				continue;
			}

			// First time we encounter a GPU emitter we need to process we also need to flush EOF updates / the compute process to ensure we get latest data
			if (bNeedsGpuTickFlush)
			{
				UWorld* World = NiagaraComponent->GetWorld();
				if (ensure(World))
				{
					World->SendAllEndOfFrameUpdates();

					FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = FNiagaraGpuComputeDispatchInterface::Get(World);
					if (ensureMsgf(ComputeDispatchInterface, TEXT("Compute dispatch interface was invalid when flushing commands")))
					{
						ComputeDispatchInterface->FlushPendingTicks_GameThread();
					}
				}

				bNeedsGpuTickFlush = false;
			}

			Helper.WriteDataBufferGPU(EmitterInstance, *EmitterCurrentData, CacheLayout.EmitterLayouts[i], CacheEmitterFrame.ParticleDataBuffers);
		}
		else
		{
			Helper.WriteDataBuffer(*EmitterCurrentData, CacheLayout.EmitterLayouts[i], CacheEmitterFrame.ParticleDataBuffers, 0, EmitterCurrentData->GetNumInstances());
		}
	}

	// If we have debug data time to capture that also
	if (DebugData)
	{
		DebugData->CaptureFrame(Helper, CacheFrames.Num() - 1);
	}

	// Store data interface data
	//-OPT: We shouldn't need to search all the time here
	if (DataInterfaceStorage.IsEmpty() == false)
	{
		const int FrameIndex = CacheFrames.Num() - 1;
		bool bDataInterfacesSucess = true;
		FString DataInterfaceName;
		TSet<FNiagaraVariableBase> VisitedDataInterfaces;

		FNiagaraDataInterfaceUtilities::ForEachDataInterface(
			Helper.SystemInstance,
			[&](const FNiagaraVariableBase& Variable, UNiagaraDataInterface* DataInterface)
			{
				if ( UObject* StorageObject = DataInterfaceStorage.FindRef(Variable) )
				{
					if (VisitedDataInterfaces.Contains(Variable))
					{
						return true;
					}
					VisitedDataInterfaces.Add(Variable);

					INiagaraSimCacheCustomStorageInterface* SimCacheCustomStorageInterface = CastChecked<INiagaraSimCacheCustomStorageInterface>(DataInterface);
					const void* PerInstanceData = Helper.SystemInstance->FindDataInterfaceInstanceData(DataInterface);
					if (!SimCacheCustomStorageInterface->SimCacheWriteFrame(StorageObject, FrameIndex, Helper.SystemInstance, PerInstanceData, FeedbackContext))
					{
						bDataInterfacesSucess = false;
						DataInterfaceName += FString::Printf(TEXT("%s "), *DataInterface->GetName());
					}
				}
				return true;
			}
		);

		// A data interface failed to write information
		if (bDataInterfacesSucess == false)
		{
			FeedbackContext.Errors.Emplace(FString::Printf(TEXT("Data interface(s) %s failed to write information for system %s"), *DataInterfaceName, *Helper.SystemInstance->GetCrashReporterTag()));
			SoftNiagaraSystem.Reset();
			return false;
		}
	}
	return true;
}

bool UNiagaraSimCache::EndWrite(bool bAllowAnalytics)
{
	UE::TWriteScopeLock RWGuard(RWLock);

	check(PendingCommandsInFlight == 0);
	if ( CacheFrames.Num() == 0 )
	{
		SoftNiagaraSystem.Reset();
	}

	if (DataInterfaceStorage.IsEmpty() == false)
	{
		bool bDataInterfacesSucess = true;
		for ( auto it=DataInterfaceStorage.CreateIterator(); it; ++it )
		{
			UClass* DataInterfaceClass = it.Key().GetType().GetClass();
			check(DataInterfaceClass != nullptr);
			INiagaraSimCacheCustomStorageInterface* SimCacheCustomStorageInterface = CastChecked<INiagaraSimCacheCustomStorageInterface>(DataInterfaceClass->GetDefaultObject());
			bDataInterfacesSucess &= SimCacheCustomStorageInterface->SimCacheEndWrite(it.Value());
		}

		if (bDataInterfacesSucess == false)
		{
			SoftNiagaraSystem.Reset();
		}
	}

	// If we allow interpolation we need to build our mapping table from Current -> Next
	if (CreateParameters.bAllowInterpolation && CacheFrames.Num() > 0)
	{
		const auto& CreateInterpMapping =
			[this](const FNiagaraSimCacheDataBuffersLayout& BufferLayout, FNiagaraSimCacheDataBuffers& BufferA, const FNiagaraSimCacheDataBuffers& BufferB)
			{
				if (BufferLayout.bAllowInterpolation == false)
				{
					return;
				}

				TMap<int32, int32> BufferIndexMap;
				BufferIndexMap.Reserve(BufferB.InterpMapping.Num());
				for (int32 i = 0; i < BufferB.InterpMapping.Num(); ++i)
				{
					BufferIndexMap.Add(BufferB.InterpMapping[i], i);
				}
				
				for (int32 i = 0; i < BufferA.InterpMapping.Num(); ++i)
				{
					int32* Mapping = BufferIndexMap.Find(BufferA.InterpMapping[i]);
					BufferA.InterpMapping[i] = Mapping ? *Mapping : INDEX_NONE;
				}
			};

		const auto& ClearInterpMapping =
			[this](const FNiagaraSimCacheDataBuffersLayout& BufferLayout, FNiagaraSimCacheDataBuffers& BufferA)
			{
				if (BufferLayout.bAllowInterpolation == false)
				{
					return;
				}

				for (int32 i = 0; i < BufferA.InterpMapping.Num(); ++i)
				{
					BufferA.InterpMapping[i] = INDEX_NONE;
				}
			};

		// Do all frames that can Interp first
		for (int32 iFrame=0; iFrame < CacheFrames.Num() - 1; ++iFrame)
		{
			FNiagaraSimCacheFrame& CacheFrameA = CacheFrames[iFrame];
			const FNiagaraSimCacheFrame& CacheFrameB = CacheFrames[iFrame + 1];

			CreateInterpMapping(CacheLayout.SystemLayout, CacheFrameA.SystemData.SystemDataBuffers, CacheFrameB.SystemData.SystemDataBuffers);
			for (int32 iEmitter=0; iEmitter < CacheLayout.EmitterLayouts.Num(); ++iEmitter)
			{
				CreateInterpMapping(CacheLayout.EmitterLayouts[iEmitter], CacheFrameA.EmitterData[iEmitter].ParticleDataBuffers, CacheFrameB.EmitterData[iEmitter].ParticleDataBuffers);
			}
		}

		// Final frame is special as we won't Interp
		ClearInterpMapping(CacheLayout.SystemLayout, CacheFrames.Last().SystemData.SystemDataBuffers);
		for (int32 iEmitter = 0; iEmitter < CacheLayout.EmitterLayouts.Num(); ++iEmitter)
		{
			ClearInterpMapping(CacheLayout.EmitterLayouts[iEmitter], CacheFrames.Last().EmitterData[iEmitter].ParticleDataBuffers);
		}
	}

#if WITH_EDITORONLY_DATA
	if (FEngineAnalytics::IsAvailable() && bAllowAnalytics)
	{		
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Emplace(TEXT("DurationSeconds"), FPlatformTime::Seconds() - CaptureStartTime);
		Attributes.Emplace(TEXT("ValidResult"), IsCacheValid());
		Attributes.Emplace(TEXT("NumFrames"), GetNumFrames());
		Attributes.Emplace(TEXT("NumEmitters"), GetNumEmitters());
		Attributes.Emplace(TEXT("NumDataInterfaces"), GetStoredDataInterfaces().Num());
		int64 StoredBytes = 0;
		for (const FNiagaraSimCacheFrame& Frame : CacheFrames)
		{
			StoredBytes += Frame.SystemData.SystemDataBuffers.DataBuffer.GetView().Num();
			for (const FNiagaraSimCacheEmitterFrame& EmitterFrame : Frame.EmitterData)
			{
				StoredBytes += EmitterFrame.ParticleDataBuffers.DataBuffer.GetView().Num();
			}
		}
		Attributes.Emplace(TEXT("ParticleStorageBytes"), StoredBytes);
		NiagaraAnalytics::RecordEvent("SimCache.Capture", Attributes);
	}
#endif

	OnCacheEndWrite.Broadcast(this);
	return IsCacheValid();
}

bool UNiagaraSimCache::CanRead(UNiagaraSystem* NiagaraSystem)
{
	check(IsInGameThread());
	
	if ( !NiagaraSystem )
    {
    	return false;
    }

	if ( NiagaraSystem != SoftNiagaraSystem.Get() )
	{
		return false;
	}

	if ( NiagaraSystem->IsReadyToRun() == false )
	{
		return false;
	}

	// Uncooked platforms can recompile the system so we need to detect if a recache is required
	//-OPT: This should use the changed notification delegate to avoid checks
#if WITH_EDITORONLY_DATA
	if ( !bNeedsReadComponentMappingRecache )
	{
		uint32 CacheVMIndex = 0;
		NiagaraSystem->ForEachScript(
			[&](UNiagaraScript* Script)
			{
				if (CachedScriptVMIds.IsValidIndex(CacheVMIndex))
				{
					bNeedsReadComponentMappingRecache |= CachedScriptVMIds[CacheVMIndex] != Script->GetVMExecutableDataCompilationId();
				}
				else
				{
					bNeedsReadComponentMappingRecache = true;
				}
				++CacheVMIndex;
			}
		);
	}
#endif

	if (bNeedsReadComponentMappingRecache)
	{
		// This modifies the component mappings that concurrent reading could be using
		// We should never really hit any concurrent readers while entering here, this is for safety
		UE::TWriteScopeLock RWGuard(RWLock);

		const int32 NumEmitters = NiagaraSystem->GetEmitterHandles().Num();
		if (NumEmitters != CacheLayout.EmitterLayouts.Num())
		{
			return false;
		}

		bool bCacheValid = true;
		bCacheValid &= FNiagaraSimCacheHelper::BuildCacheReadMappings(CacheLayout.SystemLayout, NiagaraSystem->GetSystemCompiledData().DataSetCompiledData);

		for ( int32 i=0; i < NumEmitters; ++i )
		{
			const FNiagaraEmitterCompiledData& EmitterCompiledData = NiagaraSystem->GetEmitterCompiledData()[i].Get();
			FNiagaraSimCacheDataBuffersLayout& EmitterLayout = CacheLayout.EmitterLayouts[i];
			bCacheValid &= EmitterLayout.IsLayoutValid() == NiagaraSystem->GetEmitterHandle(i).GetIsEnabled();
			if (EmitterLayout.IsLayoutValid())
			{
				bCacheValid &= FNiagaraSimCacheHelper::BuildCacheReadMappings(EmitterLayout, EmitterCompiledData.DataSetCompiledData);
				bCacheValid &= EmitterLayout.SimTarget == EmitterCompiledData.DataSetCompiledData.SimTarget;
			}
		}

		if (bCacheValid == false)
		{
			return false;
		}

#if WITH_EDITORONLY_DATA
		// Gather all the CachedScriptVMIds
		CachedScriptVMIds.Empty();
		NiagaraSystem->ForEachScript(
			[&](UNiagaraScript* Script)
			{
				CachedScriptVMIds.Add(Script->GetVMExecutableDataCompilationId());
			}
		);
		CachedScriptVMIds.Shrink();
#endif
		bNeedsReadComponentMappingRecache = false;

		// Determine if we can run async or not
		bCanRunAsync = !HasStoredDataInterfaces();
		if (bCanRunAsync)
		{
			bool bHasAnyPostReadDataInterface = false;
			FNiagaraDataInterfaceUtilities::ForEachDataInterface(
				NiagaraSystem,
				[&bHasAnyPostReadDataInterface](const FNiagaraDataInterfaceUtilities::FDataInterfaceUsageContext& DIContext)->bool
				{
					if (Cast<INiagaraSimCacheCustomStorageInterface>(DIContext.DataInterface) != nullptr)
					{
						bHasAnyPostReadDataInterface = true;
					}
					return !bHasAnyPostReadDataInterface;
				}
			);
			if (bHasAnyPostReadDataInterface)
			{
				bCanRunAsync = false;
			}
		}
	}

	return true;
}

bool UNiagaraSimCache::Read(float TimeSeconds, FNiagaraSystemInstance* SystemInstance) const
{
	UE::TReadScopeLock RWGuard(RWLock);

	// Adjust time to match our allow resolution
	TimeSeconds = FMath::RoundToFloat(TimeSeconds * CacheAgeResolution) / CacheAgeResolution;

	const float RelativeTime = FMath::Max(TimeSeconds - StartSeconds, 0.0f);
	if ( RelativeTime > DurationSeconds || CacheFrames.Num() == 0 )
	{
		// Complete
		return false;
	}

	//const float FrameTime		= DurationSeconds > 0.0f ? (RelativeTime / DurationSeconds) * float(CacheFrames.Num() - 1) : 0.0f;
	//const int32 FrameIndex		= FMath::FloorToInt(FrameTime);
	//const float FrameFraction	= FrameTime - float(FrameIndex);

	// Take a guess at the frame then iterate to find the actual frame
	// We have to do this because not all sources are recording the cache at a consistent rate
	// This could also be replaced with a binary search but in most cases our guess will be very close to the actual
	const int32 NumFramesMinusOne = CacheFrames.Num() - 1;

	int32 FrameIndex = 0;
	float FrameFraction = 0.0f;
	if (DurationSeconds > 0.0f)
	{
		FrameIndex = FMath::FloorToInt((RelativeTime / DurationSeconds) * float(NumFramesMinusOne));
	}

	if ( CacheFrames[FrameIndex].SimulationAge > TimeSeconds)
	{
		while (FrameIndex > 0 && CacheFrames[FrameIndex].SimulationAge > TimeSeconds )
		{
			--FrameIndex;
		}
	}
	else
	{
		while (FrameIndex < NumFramesMinusOne && CacheFrames[FrameIndex + 1].SimulationAge <= TimeSeconds)
		{
			++FrameIndex;
		}
	}

	if (FrameIndex < NumFramesMinusOne)
	{
		const float SimulationAgeA = CacheFrames[FrameIndex].SimulationAge;
		const float SimulationAgeB = CacheFrames[FMath::Min(FrameIndex + 1, NumFramesMinusOne)].SimulationAge;
		const float AgeDiff = SimulationAgeB - SimulationAgeA;
		FrameFraction = FMath::IsNearlyEqual(AgeDiff, 0.0f) ? 0.0f : (TimeSeconds - SimulationAgeA) / AgeDiff;
		if (!CreateParameters.bAllowInterpolation && !CreateParameters.bAllowVelocityExtrapolation && FrameFraction > 0.5f)
		{
			FrameFraction = 0.0f;
			FrameIndex = FrameIndex + 1;
		}
	}

	return ReadFrame(FrameIndex, FrameFraction, SystemInstance);
}

bool UNiagaraSimCache::ReadFrame(int32 FrameIndex, float FrameFraction, FNiagaraSystemInstance* SystemInstance) const
{
	UE::TReadScopeLock RWGuard(RWLock);

	FNiagaraSimCacheHelper Helper(SystemInstance);
	if ( !Helper.HasValidSimulation() )
	{
		return false;
	}

	const FNiagaraSimCacheFrame& CacheFrame = CacheFrames[FrameIndex];
	const int32 NextFrameIndex = FMath::Min(FrameIndex + 1, CacheFrames.Num() - 1);

	FTransform RebaseATransform = FTransform::Identity;
	FTransform RebaseBTransform = FTransform::Identity;
	if ( USceneComponent* AttachComponent = SystemInstance->GetAttachComponent() )
	{
		// Note: We remove the scaling as how that is applied to uncached emitters can not be represented by applying a transform.
		//       If we desire to control this an option can be exposed on the cache.
		FTransform LocalToWorld = AttachComponent->GetComponentToWorld();
		LocalToWorld.AddToTranslation(FVector(SystemInstance->GetLWCTile()) * -FLargeWorldRenderScalar::GetTileSize());
		LocalToWorld.RemoveScaling();

		RebaseATransform = CacheFrame.LocalToWorld;
		RebaseATransform.RemoveScaling();
		RebaseBTransform = CacheFrames[NextFrameIndex].LocalToWorld;
		RebaseBTransform.RemoveScaling();

		RebaseATransform = RebaseATransform.Inverse() * LocalToWorld;
		RebaseBTransform = RebaseBTransform.Inverse() * LocalToWorld;
	}

	const float FrameDeltaSeconds = CacheFrames[NextFrameIndex].SimulationAge - CacheFrame.SimulationAge;
	const float SimDeltaSeconds = Helper.SystemInstance->CachedDeltaSeconds;

	Helper.SystemInstance->LocalBounds = CacheFrame.SystemData.LocalBounds;
	Helper.ReadDataBuffer(FrameFraction, FrameDeltaSeconds, SimDeltaSeconds, RebaseATransform, RebaseBTransform, CacheLayout.SystemLayout, CacheFrame.SystemData.SystemDataBuffers, CacheFrame.SystemData.SystemDataBuffers, Helper.GetSystemSimulationDataSet());

	const int32 NumEmitters = CacheLayout.EmitterLayouts.Num();
	for (int32 i=0; i < NumEmitters; ++i)
	{
		if (CacheLayout.EmitterLayouts[i].IsLayoutValid() == false)
		{
			continue;
		}

		const FNiagaraSimCacheEmitterFrame& CacheEmitterFrame = CacheFrame.EmitterData[i];
		FNiagaraEmitterInstance& EmitterInstance = Helper.SystemInstance->GetEmitters()[i].Get();
		EmitterInstance.CachedBounds = CacheEmitterFrame.LocalBounds;
		EmitterInstance.TotalSpawnedParticles = CacheEmitterFrame.TotalSpawnedParticles;

		const FNiagaraSimCacheEmitterFrame& CacheEmitterFrameB = CacheFrames[NextFrameIndex].EmitterData[i];
		if (CacheLayout.EmitterLayouts[i].SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			Helper.ReadDataBufferGPU(FrameFraction, FrameDeltaSeconds, SimDeltaSeconds, RebaseATransform, RebaseBTransform, EmitterInstance, CacheLayout.EmitterLayouts[i], CacheEmitterFrame.ParticleDataBuffers, CacheEmitterFrameB.ParticleDataBuffers, EmitterInstance.GetParticleData(), PendingCommandsInFlight);
		}
		else
		{
			Helper.ReadDataBuffer(FrameFraction, FrameDeltaSeconds, SimDeltaSeconds, RebaseATransform, RebaseBTransform, CacheLayout.EmitterLayouts[i], CacheEmitterFrame.ParticleDataBuffers, CacheEmitterFrameB.ParticleDataBuffers, EmitterInstance.GetParticleData());
		}
	}

	// Store data interface data
	//-OPT: We shouldn't need to search all the time here
	if (DataInterfaceStorage.IsEmpty() == false)
	{
		TSet<FNiagaraVariableBase> VisitedDataInterfaces;
		VisitedDataInterfaces.Reserve(DataInterfaceStorage.Num());

		FNiagaraSimCacheDataInterfaceReadContext DIReadContext;
		DIReadContext.SystemInstance		= Helper.SystemInstance;
		DIReadContext.FrameIndexA			= FrameIndex;
		DIReadContext.FrameIndexB			= NextFrameIndex;
		DIReadContext.FrameInterp			= FrameFraction;
		DIReadContext.RebaseTransformA		= RebaseATransform;
		DIReadContext.RebaseTransformB		= RebaseBTransform;
		DIReadContext.LWCTileA				= CacheFrames[FrameIndex].LWCTile;
		DIReadContext.LWCTileB				= CacheFrames[NextFrameIndex].LWCTile;

		bool bDataInterfacesSucess = true;

		FNiagaraDataInterfaceUtilities::ForEachDataInterface(
			Helper.SystemInstance,
			[&](const FNiagaraVariableBase& Variable, UNiagaraDataInterface* DataInterface)
			{
				if (UObject* StorageObject = DataInterfaceStorage.FindRef(Variable))
				{
					if (INiagaraSimCacheCustomStorageInterface* SimCacheCustomStorageInterface = Cast<INiagaraSimCacheCustomStorageInterface>(DataInterface))
					{
						if (VisitedDataInterfaces.Contains(Variable))
						{
							return true;
						}
						VisitedDataInterfaces.Add(Variable);

						DIReadContext.StorageObject				= StorageObject;
						DIReadContext.OptionalPerInstanceData	= Helper.SystemInstance->FindDataInterfaceInstanceData(DataInterface);

						// Determine the interpolation state for the data interface
						if (CreateParameters.bAllowInterpolation)
						{
							if (CreateParameters.InterpolationIncludeAttributes.Contains(Variable.GetName()))
							{
								DIReadContext.InterpolateData = true;
							}
							else if (CreateParameters.InterpolationExcludeAttributes.Contains(Variable.GetName()))
							{
								DIReadContext.InterpolateData = false;
							}
							else
							{
								DIReadContext.InterpolateData.Reset();
							}
						}
						else
						{
							DIReadContext.InterpolateData = false;
						}

						// Determine the rebasing state for the data interface
						if (CreateParameters.bAllowRebasing)
						{
							if (CreateParameters.RebaseIncludeAttributes.Contains(Variable.GetName()))
							{
								DIReadContext.RebaseData = true;
							}
							else if (CreateParameters.RebaseExcludeAttributes.Contains(Variable.GetName()))
							{
								DIReadContext.RebaseData = false;
							}
							else
							{
								DIReadContext.RebaseData.Reset();
							}
						}
						else
						{
							DIReadContext.RebaseData = false;
						}

						bDataInterfacesSucess &= SimCacheCustomStorageInterface->SimCacheReadFrame(DIReadContext);
					}
				}
				return true;
			}
		);


		if (bDataInterfacesSucess == false)
		{
			return false;
		}
	}

	//-TODO: This should loop over all DataInterfaces that register not just ones with instance data
	for (TPair<TWeakObjectPtr<UNiagaraDataInterface>, int32>& DataInterfacePair : SystemInstance->DataInterfaceInstanceDataOffsets)
	{
		if (INiagaraSimCacheCustomStorageInterface* SimCacheCustomStorageInterface = Cast<INiagaraSimCacheCustomStorageInterface>(DataInterfacePair.Key.Get()))
		{
			SimCacheCustomStorageInterface->SimCachePostReadFrame(&SystemInstance->DataInterfaceInstanceData[DataInterfacePair.Value], SystemInstance);
		}
	}
	return true;
}

int UNiagaraSimCache::GetEmitterIndex(FName EmitterName) const
{
	if ( IsCacheValid() )
	{
		for ( int i=0; i < CacheLayout.EmitterLayouts.Num(); ++i )
		{
			if ( CacheLayout.EmitterLayouts[i].LayoutName == EmitterName )
			{
				return i;
			}
		}
	}
	return INDEX_NONE;
}

UNiagaraSystem* UNiagaraSimCache::GetSystem(bool bLoadSynchronous)
{
	UNiagaraSystem* NiagaraSystem = SoftNiagaraSystem.Get();
	if (NiagaraSystem == nullptr && bLoadSynchronous)
	{
		NiagaraSystem = SoftNiagaraSystem.LoadSynchronous();
	}
	return NiagaraSystem;
}

TArray<FName> UNiagaraSimCache::GetEmitterNames() const
{
	TArray<FName> EmitterNames;
	if ( IsCacheValid() )
	{
		EmitterNames.Reserve(CacheLayout.EmitterLayouts.Num());
		for (const FNiagaraSimCacheDataBuffersLayout& EmitterLayout : CacheLayout.EmitterLayouts)
		{
			EmitterNames.Add(EmitterLayout.LayoutName);
		}
	}

	return EmitterNames;
}

TArray<FNiagaraVariableBase> UNiagaraSimCache::GetStoredDataInterfaces() const
{
	TArray<FNiagaraVariableBase> DataInterfaces;
	DataInterfaceStorage.GenerateKeyArray(DataInterfaces);
	return DataInterfaces;
}

const UObject* UNiagaraSimCache::GetDataInterfaceStorageObject(const FNiagaraVariableBase& DataInterface) const
{
	if (const TObjectPtr<UObject>* StoredObject = DataInterfaceStorage.Find(DataInterface))
	{
		return StoredObject->Get();
	}
	return nullptr;
}

#if WITH_EDITORONLY_DATA
void UNiagaraSimCache::RemoveDebugData()
{
	if (DebugData == nullptr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("RemoveDebugData", "Sim Cache Remove Debug Data"));

	Modify();

	OnCacheBeginWrite.Broadcast(this);

	DebugData = nullptr;

	OnCacheEndWrite.Broadcast(this);
}
#endif

int UNiagaraSimCache::GetEmitterNumInstances(int32 EmitterIndex, int32 FrameIndex) const
{
	if ( CacheFrames.IsValidIndex(FrameIndex) )
	{
		if ( EmitterIndex == INDEX_NONE )
		{
			return CacheFrames[FrameIndex].SystemData.SystemDataBuffers.NumInstances;
		}
		else if ( CacheFrames[FrameIndex].EmitterData.IsValidIndex(EmitterIndex) )
		{
			return CacheFrames[FrameIndex].EmitterData[EmitterIndex].ParticleDataBuffers.NumInstances;
		}
	}
	return 0;
}

void UNiagaraSimCache::ForEachEmitterAttribute(int32 EmitterIndex, TFunction<bool(const FNiagaraSimCacheVariable&)> Function) const
{
	if (EmitterIndex == INDEX_NONE)
	{
		for (const FNiagaraSimCacheVariable& CacheVariable : CacheLayout.SystemLayout.Variables)
		{
			if ( Function(CacheVariable) == false )
			{
				break;
			}
		}
	}
	else if (CacheLayout.EmitterLayouts.IsValidIndex(EmitterIndex))
	{
		for (const FNiagaraSimCacheVariable& CacheVariable : CacheLayout.EmitterLayouts[EmitterIndex].Variables)
		{
			if (Function(CacheVariable) == false)
			{
				break;
			}
		}
	}
}

void UNiagaraSimCache::ReadAttribute(TArray<float>& OutFloats, TArray<FFloat16>& OutHalfs, TArray<int32>& OutInts, FName AttributeName, FName EmitterName, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if ( AttributeReader.IsValid() )
	{
		const int32 OutFloatOffset = OutFloats.AddUninitialized(AttributeReader.Variable->FloatCount * AttributeReader.GetNumInstances());
		for ( int32 i=0; i < AttributeReader.Variable->FloatCount; ++i )
		{
			AttributeReader.CopyComponentFloats(i, &OutFloats[OutFloatOffset + (i * AttributeReader.GetNumInstances())]);
		}

		const int32 OutHalfOffset = OutHalfs.AddUninitialized(AttributeReader.Variable->HalfCount * AttributeReader.GetNumInstances());
		for (int32 i = 0; i < AttributeReader.Variable->HalfCount; ++i)
		{
			AttributeReader.CopyComponentHalfs(i, &OutHalfs[OutHalfOffset + (i * AttributeReader.GetNumInstances())]);
		}

		const int32 OutIntOffset = OutInts.AddUninitialized(AttributeReader.Variable->Int32Count * AttributeReader.GetNumInstances());
		for (int32 i = 0; i < AttributeReader.Variable->Int32Count; ++i)
		{
			AttributeReader.CopyComponentInts(i, &OutInts[OutIntOffset + (i * AttributeReader.GetNumInstances())]);
		}
	}
}

void UNiagaraSimCache::ReadIntAttribute(TArray<int32>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if (AttributeReader.IsValid() && AttributeReader.Variable->Variable.GetType() == FNiagaraTypeDefinition::GetIntDef())
	{
		const int32 OutValueOffset = OutValues.AddUninitialized(AttributeReader.GetNumInstances());
		for (int32 i = 0; i < AttributeReader.GetNumInstances(); ++i)
		{
			OutValues[OutValueOffset + i] = AttributeReader.ReadInt(i);
		}
	}
}

void UNiagaraSimCache::ReadFloatAttribute(TArray<float>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if (AttributeReader.IsValid() && AttributeReader.Variable->Variable.GetType() == FNiagaraTypeDefinition::GetFloatDef())
	{
		const int32 OutValueOffset = OutValues.AddUninitialized(AttributeReader.GetNumInstances());
		for (int32 i = 0; i < AttributeReader.GetNumInstances(); ++i)
		{
			OutValues[OutValueOffset + i] = AttributeReader.ReadFloat(i);
		}
	}
}

void UNiagaraSimCache::ReadVector2Attribute(TArray<FVector2D>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if (AttributeReader.IsValid() && AttributeReader.Variable->Variable.GetType() == FNiagaraTypeDefinition::GetVec2Def())
	{
		const int32 OutValueOffset = OutValues.AddUninitialized(AttributeReader.GetNumInstances());
		for (int32 i = 0; i < AttributeReader.GetNumInstances(); ++i)
		{
			OutValues[OutValueOffset + i] = FVector2D(AttributeReader.ReadFloat2f(i));
		}
	}
}

void UNiagaraSimCache::ReadVectorAttribute(TArray<FVector>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if (AttributeReader.IsValid() && AttributeReader.Variable->Variable.GetType() == FNiagaraTypeDefinition::GetVec3Def())
	{
		const int32 OutValueOffset = OutValues.AddUninitialized(AttributeReader.GetNumInstances());
		for (int32 i=0; i < AttributeReader.GetNumInstances(); ++i)
		{
			OutValues[OutValueOffset + i] = FVector(AttributeReader.ReadFloat3f(i));
		}
	}
}

void UNiagaraSimCache::ReadVector4Attribute(TArray<FVector4>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if (AttributeReader.IsValid() && AttributeReader.Variable->Variable.GetType() == FNiagaraTypeDefinition::GetVec4Def())
	{
		const int32 OutValueOffset = OutValues.AddUninitialized(AttributeReader.GetNumInstances());
		for (int32 i = 0; i < AttributeReader.GetNumInstances(); ++i)
		{
			OutValues[OutValueOffset + i] = FVector4(AttributeReader.ReadFloat4f(i));
		}
	}
}

void UNiagaraSimCache::ReadColorAttribute(TArray<FLinearColor>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if (AttributeReader.IsValid() && AttributeReader.Variable->Variable.GetType() == FNiagaraTypeDefinition::GetColorDef())
	{
		const int32 OutValueOffset = OutValues.AddUninitialized(AttributeReader.GetNumInstances());
		for (int32 i = 0; i < AttributeReader.GetNumInstances(); ++i)
		{
			OutValues[OutValueOffset + i] = AttributeReader.ReadColor(i);
		}
	}
}

void UNiagaraSimCache::ReadIDAttribute(TArray<FNiagaraID>& OutValues, FName AttributeName, FName EmitterName, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if (AttributeReader.IsValid() && AttributeReader.Variable->Variable.GetType() == FNiagaraTypeDefinition::GetIDDef())
	{
		const int32 OutValueOffset = OutValues.AddUninitialized(AttributeReader.GetNumInstances());
		for (int32 i = 0; i < AttributeReader.GetNumInstances(); ++i)
		{
			OutValues[OutValueOffset + i] = AttributeReader.ReadID(i);
		}
	}
}

void UNiagaraSimCache::ReadPositionAttribute(TArray<FVector>& OutValues, FName AttributeName, FName EmitterName, bool bLocalSpaceToWorld, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if (AttributeReader.IsValid() && AttributeReader.Variable->Variable.GetType() == FNiagaraTypeDefinition::GetPositionDef())
	{
		FTransform LocalToWorld;
		if ( bLocalSpaceToWorld && AttributeReader.DataBuffersLayout->bLocalSpace )
		{
			LocalToWorld = AttributeReader.CacheFrame->LocalToWorld;
		}
		else
		{
			LocalToWorld = FTransform::Identity;
			LocalToWorld.SetTranslation(FVector(AttributeReader.CacheFrame->LWCTile) * FLargeWorldRenderScalar::GetTileSize());
		}

		const int32 OutValueOffset = OutValues.AddUninitialized(AttributeReader.GetNumInstances());
		for (int32 i = 0; i < AttributeReader.GetNumInstances(); ++i)
		{
			const FVector Position = FVector(AttributeReader.ReadFloat3f(i));
			OutValues[OutValueOffset + i] = LocalToWorld.TransformPosition(Position);
		}
	}
}

void UNiagaraSimCache::ReadPositionAttributeWithRebase(TArray<FVector>& OutValues, FTransform Transform, FName AttributeName, FName EmitterName, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if (AttributeReader.IsValid() && AttributeReader.Variable->Variable.GetType() == FNiagaraTypeDefinition::GetPositionDef())
	{
		const FTransform LocalToWorld = AttributeReader.DataBuffersLayout->bLocalSpace ? Transform : Transform * AttributeReader.CacheFrame->LocalToWorld.Inverse();
		const int32 OutValueOffset = OutValues.AddUninitialized(AttributeReader.GetNumInstances());
		for (int32 i = 0; i < AttributeReader.GetNumInstances(); ++i)
		{
			const FVector Position = FVector(AttributeReader.ReadFloat3f(i));
			OutValues[OutValueOffset + i] = LocalToWorld.TransformPosition(Position);
		}
	}
}

void UNiagaraSimCache::ReadQuatAttribute(TArray<FQuat>& OutValues, FName AttributeName, FName EmitterName, bool bLocalSpaceToWorld, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if (AttributeReader.IsValid() && AttributeReader.Variable->Variable.GetType() == FNiagaraTypeDefinition::GetQuatDef())
	{
		const int32 OutValueOffset = OutValues.AddUninitialized(AttributeReader.GetNumInstances());
		if (bLocalSpaceToWorld && AttributeReader.DataBuffersLayout->bLocalSpace)
		{
			for (int32 i = 0; i < AttributeReader.GetNumInstances(); ++i)
			{
				const FQuat Quat = FQuat(AttributeReader.ReadQuat4f(i));
				OutValues[OutValueOffset + i] = Quat * AttributeReader.CacheFrame->LocalToWorld.GetRotation();
			}
		}
		else
		{
			for (int32 i = 0; i < AttributeReader.GetNumInstances(); ++i)
			{
				const FQuat Quat = FQuat(AttributeReader.ReadQuat4f(i));
				OutValues[OutValueOffset + i] = Quat;
			}
		}
	}
}

void UNiagaraSimCache::ReadQuatAttributeWithRebase(TArray<FQuat>& OutValues, FQuat Quat, FName AttributeName, FName EmitterName, int FrameIndex) const
{
	FNiagaraSimCacheAttributeReaderHelper AttributeReader(this, EmitterName, AttributeName, FrameIndex);
	if (AttributeReader.IsValid() && AttributeReader.Variable->Variable.GetType() == FNiagaraTypeDefinition::GetQuatDef())
	{
		const bool bRebase = AttributeReader.DataBuffersLayout->bLocalSpace == false && AttributeReader.DataBuffersLayout->RebaseVariableNames.Contains(AttributeName);

		const FQuat LocalToWorld = bRebase ? Quat * AttributeReader.CacheFrame->LocalToWorld.GetRotation().Inverse() : Quat;
		const int32 OutValueOffset = OutValues.AddUninitialized(AttributeReader.GetNumInstances());
		for (int32 i = 0; i < AttributeReader.GetNumInstances(); ++i)
		{
			const FQuat Rotation = FQuat(AttributeReader.ReadQuat4f(i));
			OutValues[OutValueOffset + i] = Rotation * LocalToWorld;
		}
	}
}

UObject* UNiagaraSimCache::ReadDataInterfaceAs(UClass* RequestedType, FName AttributeName, int FrameIndex) const
{
	for (auto DataInterfaceIt=DataInterfaceStorage.CreateConstIterator(); DataInterfaceIt; ++DataInterfaceIt)
	{
		const FNiagaraVariableBase& Variable = DataInterfaceIt.Key();
		if (Variable.GetName() != AttributeName)
		{
			continue;
		}

		UObject* StorageObject = DataInterfaceIt.Value();
		if (StorageObject->IsA(RequestedType))
		{
			return StorageObject;
		}
		//-TODO: Allow data interface to convert to the requested type
		return nullptr;
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
