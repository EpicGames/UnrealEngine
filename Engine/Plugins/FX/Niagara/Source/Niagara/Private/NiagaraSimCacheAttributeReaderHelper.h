// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Float16.h"
#include "NiagaraSimCache.h"

struct FNiagaraSimCacheAttributeReaderHelper
{
	FNiagaraSimCacheAttributeReaderHelper() = default;

	explicit FNiagaraSimCacheAttributeReaderHelper(const UNiagaraSimCache* SimCache, FName EmitterName, FName AttributeName, int FrameIndex)
	{
		Initialize(SimCache, EmitterName, AttributeName, FrameIndex);
	}

	void Initialize(const UNiagaraSimCache* SimCache, FName EmitterName, FName AttributeName, int FrameIndex)
	{
		CacheFrame = nullptr;
		DataBuffers = nullptr;
		DataBuffersLayout = nullptr;
		Variable = nullptr;

		if (SimCache->IsCacheValid() == false)
		{
			return;
		}

		if (SimCache->CacheFrames.IsValidIndex(FrameIndex) == false)
		{
			return;
		}
		CacheFrame = &SimCache->CacheFrames[FrameIndex];

		if ( EmitterName.IsNone() == false )
		{
			const int32 EmitterIndex = SimCache->CacheLayout.EmitterLayouts.IndexOfByPredicate([&EmitterName](const FNiagaraSimCacheDataBuffersLayout& FoundLayout) { return FoundLayout.LayoutName == EmitterName; });
			if (EmitterIndex == INDEX_NONE)
			{
				return;
			}

			DataBuffers = &CacheFrame->EmitterData[EmitterIndex].ParticleDataBuffers;
			DataBuffersLayout = &SimCache->CacheLayout.EmitterLayouts[EmitterIndex];
		}
		else
		{
			DataBuffers = &CacheFrame->SystemData.SystemDataBuffers;
			DataBuffersLayout = &SimCache->CacheLayout.SystemLayout;
		}

		if (DataBuffers->NumInstances == 0)
		{
			return;
		}

		const int32 VariableIndex = DataBuffersLayout->Variables.IndexOfByPredicate([&AttributeName](const FNiagaraSimCacheVariable& FoundVariable) { return FoundVariable.Variable.GetName() == AttributeName; });
		if (VariableIndex == INDEX_NONE)
		{
			return;
		}

		Variable = &DataBuffersLayout->Variables[VariableIndex];
	}

	void Initialize(const UNiagaraSimCache* SimCache, int EmitterIndex, int AttributeIndex, int FrameIndex)
	{
		CacheFrame = nullptr;
		DataBuffers = nullptr;
		DataBuffersLayout = nullptr;
		Variable = nullptr;

		if (SimCache->IsCacheValid() == false)
		{
			return;
		}

		if (SimCache->CacheFrames.IsValidIndex(FrameIndex) == false)
		{
			return;
		}
		CacheFrame = &SimCache->CacheFrames[FrameIndex];

		if ( EmitterIndex != INDEX_NONE )
		{
			if (SimCache->CacheLayout.EmitterLayouts.IsValidIndex(EmitterIndex) == false)
			{
				return;
			}

			DataBuffers = &CacheFrame->EmitterData[EmitterIndex].ParticleDataBuffers;
			DataBuffersLayout = &SimCache->CacheLayout.EmitterLayouts[EmitterIndex];
		}
		else
		{
			DataBuffers = &CacheFrame->SystemData.SystemDataBuffers;
			DataBuffersLayout = &SimCache->CacheLayout.SystemLayout;
		}

		if (DataBuffers->NumInstances == 0)
		{
			return;
		}

		if (DataBuffersLayout->Variables.IsValidIndex(AttributeIndex) == false)
		{
			return;
		}

		Variable = &DataBuffersLayout->Variables[AttributeIndex];
	}

	static int32 FindVariableIndex(const UNiagaraSimCache* SimCache, const FNiagaraVariableBase& Variable, int EmitterIndex)
	{
		if (SimCache->CacheLayout.EmitterLayouts.IsValidIndex(EmitterIndex))
		{
			const FNiagaraSimCacheDataBuffersLayout& EmitterLayout = SimCache->CacheLayout.EmitterLayouts[EmitterIndex];
			const int32 VariableIndex = EmitterLayout.Variables.IndexOfByPredicate([Variable](const FNiagaraSimCacheVariable& FoundVariable) { return FoundVariable.Variable == Variable; });
			return VariableIndex;
		}

		return INDEX_NONE;
	}

	static const FNiagaraSimCacheVariable* FindVariable(const UNiagaraSimCache* SimCache, const FNiagaraVariableBase& Variable, int EmitterIndex)
	{
		const int32 VariableIndex = FindVariableIndex(SimCache, Variable, EmitterIndex);
		if (VariableIndex != INDEX_NONE)
		{
			const FNiagaraSimCacheDataBuffersLayout& EmitterLayout = SimCache->CacheLayout.EmitterLayouts[EmitterIndex];
			return &EmitterLayout.Variables[VariableIndex];
		}

		return nullptr;
	}

	bool IsValid() const { return Variable != nullptr; }

	int32 GetNumInstances() const { check(IsValid()); return DataBuffers->NumInstances; }

	void CopyComponentFloats(int32 Component, float* OutBuffer) const
	{
		check(Component < Variable->FloatCount);
		FMemory::Memcpy(OutBuffer, &DataBuffers->FloatData[(Variable->FloatOffset + Component) * DataBuffers->NumInstances * sizeof(float)], sizeof(float) * DataBuffers->NumInstances);
	}

	void CopyComponentHalfs(int32 Component, FFloat16* OutBuffer) const
	{
		check(Component < Variable->HalfCount);
		FMemory::Memcpy(OutBuffer, &DataBuffers->HalfData[(Variable->HalfOffset + Component) * DataBuffers->NumInstances * sizeof(FFloat16)], sizeof(FFloat16) * DataBuffers->NumInstances);
	}

	void CopyComponentInts(int32 Component, int32* OutBuffer) const
	{
		check(Component < Variable->Int32Count);
		FMemory::Memcpy(OutBuffer, &DataBuffers->Int32Data[(Variable->Int32Offset + Component) * DataBuffers->NumInstances * sizeof(int32)], sizeof(int32) * DataBuffers->NumInstances);
	}

	int32 ReadInt(int32 Instance, int32 Offset = 0) const
	{
		check(IsValid());
		
		const int32 Int32Offset = Instance + ((Variable->Int32Offset + Offset) * DataBuffers->NumInstances);

		int32 Value;
		FMemory::Memcpy(&Value, &DataBuffers->Int32Data[Int32Offset * sizeof(int32)], sizeof(int32));
		return Value;
	}

	float InternalReadFloat(int32 Offset, int32 Instance) const
	{
		check(IsValid());
		
		const int32 FloatOffset = Instance + ((Variable->FloatOffset + Offset) * DataBuffers->NumInstances);

		float Value;
		FMemory::Memcpy(&Value, &DataBuffers->FloatData[FloatOffset * sizeof(float)], sizeof(float));
		return Value;
	}

	float ReadFloat(int32 Instance) const
	{
		check(Variable->FloatOffset != INDEX_NONE && Variable->FloatCount == 1);
		return InternalReadFloat(0, Instance);
	}

	FNiagaraID ReadID(int32 Instance) const
	{
		check(Variable->Int32Offset != INDEX_NONE && Variable->Int32Count == 2);
		return FNiagaraID(ReadInt(Instance, 0), ReadInt(Instance, 1));
	}

	FVector2f ReadFloat2f(int32 Instance) const
	{
		check(Variable->FloatOffset != INDEX_NONE && Variable->FloatCount == 2);
		return FVector2f(InternalReadFloat(0, Instance), InternalReadFloat(1, Instance));
	}

	FVector3f ReadFloat3f(int32 Instance) const
	{
		check(Variable->FloatOffset != INDEX_NONE && Variable->FloatCount == 3);
		return FVector3f(InternalReadFloat(0, Instance), InternalReadFloat(1, Instance), InternalReadFloat(2, Instance));
	}

	FVector4f ReadFloat4f(int32 Instance) const
	{
		check(Variable->FloatOffset != INDEX_NONE && Variable->FloatCount == 4);
		return FVector4f(InternalReadFloat(0, Instance), InternalReadFloat(1, Instance), InternalReadFloat(2, Instance), InternalReadFloat(3, Instance));
	}

	FQuat4f ReadQuat4f(int32 Instance) const
	{
		check(Variable->FloatOffset != INDEX_NONE && Variable->FloatCount == 4);
		return FQuat4f(InternalReadFloat(0, Instance), InternalReadFloat(1, Instance), InternalReadFloat(2, Instance), InternalReadFloat(3, Instance));
	}

	FLinearColor ReadColor(int32 Instance) const
	{
		check(Variable->FloatOffset != INDEX_NONE && Variable->FloatCount == 4);
		return FLinearColor(InternalReadFloat(0, Instance), InternalReadFloat(1, Instance), InternalReadFloat(2, Instance), InternalReadFloat(3, Instance));
	}

	const FNiagaraSimCacheFrame*				CacheFrame = nullptr;
	const FNiagaraSimCacheDataBuffers*			DataBuffers = nullptr;
	const FNiagaraSimCacheDataBuffersLayout*	DataBuffersLayout = nullptr;
	const FNiagaraSimCacheVariable*				Variable = nullptr;
};
