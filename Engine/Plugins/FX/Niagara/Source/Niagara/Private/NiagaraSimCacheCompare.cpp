// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimCacheCompare.h"
#include "NiagaraConstants.h"
#include "NiagaraModule.h"
#include "NiagaraSimCacheCustomStorageInterface.h"

namespace NiagaraSimCacheCompareInternal
{
	constexpr int32 MaxErrorStrLen = 8*1024;

	bool IsEqual(const float& Num1, const float& Num2, float Tolerance)
	{
		return FMath::IsNearlyEqual(Num1, Num2, Tolerance);
	}

	bool IsEqual(const int32& Num1, const int32& Num2, float)
	{
		return Num1 == Num2;
	}

	FString NumberToString(const float& Num)
	{
		return FString::SanitizeFloat(Num);
	}

	FString NumberToString(const int32& Num)
	{
		return FString::FromInt(Num);
	}

	bool MatchesWildcardArray(TConstArrayView<FString> Wildcards, FName Name)
	{
		if (Wildcards.Num() == 0)
		{
			return false;
		}

		const FString NameString = Name.ToString();
		for (const FString& Wildcard : Wildcards)
		{
			if (NameString.MatchesWildcard(Wildcard))
			{
				return true;
			}
		}
		return false;
	}

	TArray<FNiagaraSimCacheVariable> GatherEmitterVariables(const UNiagaraSimCache& SimCache, int32 EmitterIndex, const TSet<FNiagaraVariableBase>& ExcludeVariables, TConstArrayView<FString> WildcardMatches)
	{
		TArray<FNiagaraSimCacheVariable> OutVariables;
		SimCache.ForEachEmitterAttribute(
			EmitterIndex,
			[&OutVariables, &ExcludeVariables, &WildcardMatches](const FNiagaraSimCacheVariable& Variable) -> bool
			{
				if (!ExcludeVariables.Contains(Variable.Variable) && !MatchesWildcardArray(WildcardMatches, Variable.Variable.GetName()))
				{
					OutVariables.Add(Variable);
				}
				return true;
			}
		);
		return OutVariables;
	}

	template<typename TType, typename TValueString>
	bool CompareAttributeData(const TArray<int32>& InstanceMapping, const TArray<TType>& LhsData, const TArray<TType>& RhsData, int32 ComponentCount, TValueString& ValueDiffString, float ErrorTolerance)
	{
		const int32 NumInstances = InstanceMapping.Num();
		for (int32 i = 0; i < NumInstances; ++i)
		{
			const int32 LhsInstance = i;
			const int32 RhsInstance = InstanceMapping[i];

			for (int32 Component = 0; Component < ComponentCount; ++Component)
			{
				const int32 LhsComponent = LhsInstance + (Component * NumInstances);
				const int32 RhsComponent = RhsInstance + (Component * NumInstances);

				if (!LhsData.IsValidIndex(LhsComponent) || !RhsData.IsValidIndex(RhsComponent))
				{
					ValueDiffString.Append(TEXT("Unable to find source data for particle ID "));
					ValueDiffString.Append(FString::FromInt(LhsInstance));
					return false;
				}

				const TType LhsValue = LhsData[LhsComponent];
				const TType RhsValue = RhsData[RhsComponent];
				if (!IsEqual(LhsValue, RhsValue, ErrorTolerance))
				{
					ValueDiffString.Append(TEXT("Particle ID "));
					ValueDiffString.Append(FString::FromInt(LhsInstance));
					ValueDiffString.Append(TEXT(", Expected: "));
					ValueDiffString.Append(NumberToString(LhsValue));
					ValueDiffString.Append(TEXT(", Actual: "));
					ValueDiffString.Append(NumberToString(RhsValue));
					return false;
				}
			}
		}

		return true;
	}

	template<typename TValueString>
	bool CompareQuatAttributeData(const TArray<int32>& InstanceMapping, const TArray<float>& LhsData, const TArray<float>& RhsData, TValueString& ValueDiffString, float ErrorTolerance)
	{
		const int32 NumInstances = InstanceMapping.Num();
		for (int32 i = 0; i < NumInstances; ++i)
		{
			const int32 LhsInstance = i;
			const int32 RhsInstance = InstanceMapping[i];

			const FQuat4f LhsValue(LhsData[LhsInstance], LhsData[LhsInstance + (NumInstances * 1)], LhsData[LhsInstance + (NumInstances * 2)], LhsData[LhsInstance + (NumInstances * 3)]);
			const FQuat4f RhsValue(RhsData[RhsInstance], RhsData[RhsInstance + (NumInstances * 1)], RhsData[RhsInstance + (NumInstances * 2)], RhsData[RhsInstance + (NumInstances * 3)]);

			if (!LhsValue.Equals(RhsValue, ErrorTolerance))
			{
				ValueDiffString.Append(TEXT("Particle ID "));
				ValueDiffString.Append(FString::FromInt(LhsInstance));
				ValueDiffString.Append(TEXT(", Expected: "));
				ValueDiffString.Appendf(TEXT("(%s, %s, %s, %s)"), *NumberToString(LhsValue.X), *NumberToString(LhsValue.Y), *NumberToString(LhsValue.Z), *NumberToString(LhsValue.W));
				ValueDiffString.Append(TEXT(", Actual: "));
				ValueDiffString.Appendf(TEXT("(%s, %s, %s, %s)"), *NumberToString(RhsValue.X), *NumberToString(RhsValue.Y), *NumberToString(RhsValue.Z), *NumberToString(RhsValue.W));
				return false;
			}
		}

		return true;
	}

	void AddError(FString& OutErrors, const FString& Str)
	{
		if (OutErrors.Len() < MaxErrorStrLen)
		{
			OutErrors.Append(Str);
			if (OutErrors.Len() >= MaxErrorStrLen)
			{
				OutErrors.Append(TEXT("...\n"));
			}
			else
			{
				OutErrors.Append(TEXT("\n"));
			}
		}
	}

	void InternalSetToleranceByType(const UNiagaraSimCache& SimCache, const FNiagaraTypeDefinition& TypeDef, float Tolerance, int EmitterIndex, TMap<FNiagaraVariableBase, float>& VariableToFloatTolerances)
	{
		SimCache.ForEachEmitterAttribute(
			EmitterIndex,
			[&TypeDef, Tolerance, &VariableToFloatTolerances](const FNiagaraSimCacheVariable& CacheVariable) -> bool
			{
				if (CacheVariable.Variable.GetType() == TypeDef)
				{
					VariableToFloatTolerances.FindOrAdd(CacheVariable.Variable) = Tolerance;
				}
				return true;
			}
		);
	}

	void InternalSetToleranceByName(const UNiagaraSimCache& SimCache, const FString& NameWildcard, float Tolerance, int EmitterIndex, TMap<FNiagaraVariableBase, float>& VariableToFloatTolerances)
	{
		SimCache.ForEachEmitterAttribute(
			EmitterIndex,
			[&NameWildcard, Tolerance, &VariableToFloatTolerances](const FNiagaraSimCacheVariable& CacheVariable) -> bool
			{
				if (CacheVariable.Variable.GetName().ToString().MatchesWildcard(NameWildcard))
				{
					VariableToFloatTolerances.FindOrAdd(CacheVariable.Variable) = Tolerance;
				}
				return true;
			}
		);
	}
}

void FNiagaraSimCacheCompare::AddExcludeAttribute(FNiagaraVariableBase Variable)
{
	if (Variable.RemoveRootNamespace(FNiagaraConstants::ParticleAttributeNamespaceString))
	{
		AttributesToExclude_ParticleScript.Add(Variable);
	}
	else
	{
		AttributesToExclude_SystemScript.Add(Variable);
	}
}

void FNiagaraSimCacheCompare::AddExcludeAttributesByWildcard(FString Wildcard)
{
	AttributesToExclude_ByWildcard.AddUnique(Wildcard);
}

void FNiagaraSimCacheCompare::SetToleranceByType(const UNiagaraSimCache& SimCache, const FNiagaraTypeDefinition& TypeDef, float Tolerance)
{
	using namespace NiagaraSimCacheCompareInternal;

	InternalSetToleranceByType(SimCache, TypeDef, Tolerance, INDEX_NONE, VariableToFloatTolerances_SystemScript);
	for (int32 i=0; i < SimCache.GetNumEmitters(); ++i)
	{
		InternalSetToleranceByType(SimCache, TypeDef, Tolerance, i, VariableToFloatTolerances_ParticleScript);
	}
}

void FNiagaraSimCacheCompare::SetToleranceByName(const UNiagaraSimCache& SimCache, const FString& NameWildcard, float Tolerance)
{
	using namespace NiagaraSimCacheCompareInternal;

	InternalSetToleranceByName(SimCache, NameWildcard, Tolerance, INDEX_NONE, VariableToFloatTolerances_SystemScript);
	for (int32 i = 0; i < SimCache.GetNumEmitters(); ++i)
	{
		InternalSetToleranceByName(SimCache, NameWildcard, Tolerance, i, VariableToFloatTolerances_ParticleScript);
	}
}

bool FNiagaraSimCacheCompare::Compare(const UNiagaraSimCache& LhsCache, const UNiagaraSimCache& RhsCache, FString& OutDifferences)
{
	using namespace NiagaraSimCacheCompareInternal;

	// Check Frame Count Matches
	if (LhsCache.GetNumFrames() != RhsCache.GetNumFrames())
	{
		OutDifferences = FString::Printf(TEXT("Cache Frame count is different. Expected: %d, Actual: %d"), LhsCache.GetNumFrames(), RhsCache.GetNumFrames());
		return false;
	}

	// Check Emitter Count Matches
	if (LhsCache.GetNumEmitters() != RhsCache.GetNumEmitters())
	{
		OutDifferences = FString::Printf(TEXT("Cache Emitter count is different. Expected: %d, Actual: %d"), LhsCache.GetNumEmitters(), RhsCache.GetNumEmitters());
		return false;
	}

	// Check Emitter Names Match
	for (int iEmitter=0; iEmitter < LhsCache.GetNumEmitters(); ++iEmitter)
	{
		if (LhsCache.GetEmitterName(iEmitter) != RhsCache.GetEmitterName(iEmitter))
		{
			OutDifferences = FString::Printf(TEXT("Cache Emitter name is different. Expected: %s, Actual: %s"), *LhsCache.GetEmitterName(iEmitter).ToString(), *RhsCache.GetEmitterName(iEmitter).ToString());
			return false;
		}
	}

	// Add exclude variables
	if (bExcludeNoneDeterministicVariables)
	{
		for (FNiagaraVariableBase ParticleVariable : {SYS_PARAM_PARTICLES_ID, SYS_PARAM_PARTICLES_MATERIAL_RANDOM})
		{
			ParticleVariable.RemoveRootNamespace(FNiagaraConstants::ParticleAttributeNamespaceString);
			AttributesToExclude_ParticleScript.Add(ParticleVariable);
		}
	}

	// Check Frames Match
	OutDifferences.Reset(MaxErrorStrLen);

	bool bEqual = true;
	for (int FrameIndex=0; FrameIndex < LhsCache.GetNumFrames(); FrameIndex++)
	{
		const FNiagaraSimCacheFrame& LhsFrame = LhsCache.CacheFrames[FrameIndex];
		const FNiagaraSimCacheFrame& RhsFrame = RhsCache.CacheFrames[FrameIndex];

		if (!FMath::IsNearlyEqual(LhsFrame.SimulationAge, RhsFrame.SimulationAge, UE_KINDA_SMALL_NUMBER))
		{
			AddError(OutDifferences, FString::Printf(TEXT("Simulation age different - frame %d"), FrameIndex));
			bEqual = false;
		}

		bEqual &= CompareEmitter(LhsCache, RhsCache, INDEX_NONE, FrameIndex, OutDifferences);
		for (int EmitterIndex=0; EmitterIndex < LhsCache.GetNumEmitters(); EmitterIndex++)
		{
			bEqual &= CompareEmitter(LhsCache, RhsCache, EmitterIndex, FrameIndex, OutDifferences);
		}

		if (bEqual == false)
		{
			break;
		}
	}

	// Compare data interfaces
	if (bEqual && bIncludeDataInterfaces)
	{
		// Generate set of DIs to compare
		const TArray<FNiagaraVariableBase> DataInterfacesToCompare = LhsCache.GetStoredDataInterfaces();
		{
			TArray<FNiagaraVariableBase> ExpectedDataInterfaces = RhsCache.GetStoredDataInterfaces();
			for (const FNiagaraVariableBase& DIVariable : DataInterfacesToCompare)
			{
				if (ExpectedDataInterfaces.Remove(DIVariable) == 0)
				{
					AddError(OutDifferences, FString::Printf(TEXT("DataInterface(%s) was in Lhs and not Rhs cache."), *DIVariable.GetName().ToString()));
					bEqual = false;
				}
			}

			for (const FNiagaraVariableBase& DIVariable : ExpectedDataInterfaces)
			{
				AddError(OutDifferences, FString::Printf(TEXT("DataInterface(%s) was in Rhs and not Lhs cache."), *DIVariable.GetName().ToString()));
				bEqual = false;
			}
		}

		// Compare data interfaces
		if (bEqual)
		{
			for (const FNiagaraVariableBase& DIVariable : DataInterfacesToCompare)
			{
				if (AttributesToExclude_SystemScript.Contains(DIVariable))
				{
					continue;
				}
				
				if ( MatchesWildcardArray(AttributesToExclude_ByWildcard, DIVariable.GetName()) )
				{
					continue;
				}

				UClass* Class = DIVariable.GetType().GetClass();
				INiagaraSimCacheCustomStorageInterface* DataInterfaceCDO = Class ? Cast<INiagaraSimCacheCustomStorageInterface>(DIVariable.GetType().GetClass()->GetDefaultObject()) : nullptr;
				if (!DataInterfaceCDO)
				{
					AddError(OutDifferences, FString::Printf(TEXT("DataInterface(%s) could not find CDO for Class(%s)."), *DIVariable.GetName().ToString(), *GetNameSafe(Class)));
					bEqual = false;
					break;
				}

				TOptional<float> Tolerance;
				if (float* UserTolerance = VariableToFloatTolerances_SystemScript.Find(DIVariable))
				{
					Tolerance.Emplace(*UserTolerance);
				}

				const UObject* LhsStorageObject = LhsCache.GetDataInterfaceStorageObject(DIVariable);
				const UObject* RhsStorageObject = RhsCache.GetDataInterfaceStorageObject(DIVariable);
				if (LhsStorageObject && RhsStorageObject)
				{
					for (int FrameIndex=0; FrameIndex < LhsCache.GetNumFrames(); FrameIndex++)
					{
						FString DIErrors;
						if (!DataInterfaceCDO->SimCacheCompareFrame(LhsStorageObject, RhsStorageObject, FrameIndex, Tolerance, DIErrors))
						{
							AddError(OutDifferences, FString::Printf(TEXT("DataInterface(%s) frame (%d) is not equal - %s."), *DIVariable.GetName().ToString(), FrameIndex, *DIErrors));
							bEqual = false;
						}
					}
				}
				else
				{
					AddError(OutDifferences, FString::Printf(TEXT("DataInterface(%s) storage was not found in Rhs(%p) Lhs(%p) cache."), *DIVariable.GetName().ToString(), LhsStorageObject, RhsStorageObject));
					bEqual = false;
				}

				if (!bEqual)
				{
					break;
				}
			}
		}
	}


	return bEqual;
}

bool FNiagaraSimCacheCompare::CompareEmitter(const UNiagaraSimCache& LhsCache, const UNiagaraSimCache& RhsCache, int EmitterIndex, int FrameIndex, FString& OutDifferences)
{
	using namespace NiagaraSimCacheCompareInternal;

	const FName EmitterName = EmitterIndex == INDEX_NONE ? NAME_None : LhsCache.GetEmitterName(EmitterIndex);
	const FString EmitterNameString = EmitterIndex == INDEX_NONE ? FString(TEXT("SystemInstance")) : EmitterName.ToString();

	const TSet<FNiagaraVariableBase>& AttributesToExclude = EmitterIndex == INDEX_NONE ? AttributesToExclude_SystemScript : AttributesToExclude_ParticleScript;
	TMap<FNiagaraVariableBase, float>& VariableToFloatTolerances = EmitterIndex == INDEX_NONE ? VariableToFloatTolerances_SystemScript : VariableToFloatTolerances_ParticleScript;

	// Compare instance counts
	const int LhsNumInstances = LhsCache.GetEmitterNumInstances(EmitterIndex, FrameIndex);
	const int RhsNumInstances = RhsCache.GetEmitterNumInstances(EmitterIndex, FrameIndex);
	if (LhsNumInstances != RhsNumInstances)
	{
		AddError(OutDifferences, FString::Printf(TEXT("%s instance count different frame %d.  Expected: %d, Actual %d"), *EmitterNameString, FrameIndex, LhsNumInstances, RhsNumInstances));
		return false;
	}

	if (LhsNumInstances == 0)
	{
		return true;
	}

	// Gather all the attributes we want to compare
	const TArray<FNiagaraSimCacheVariable> CompareVariables = GatherEmitterVariables(LhsCache, EmitterIndex, AttributesToExclude, AttributesToExclude_ByWildcard);
	{
		bool bAttributesValid = true;
		const TArray<FNiagaraSimCacheVariable> RhsCompareVariables = GatherEmitterVariables(RhsCache, EmitterIndex, AttributesToExclude, AttributesToExclude_ByWildcard);
		if (CompareVariables.Num() != RhsCompareVariables.Num())
		{
			AddError(OutDifferences, FString::Printf(TEXT("%s attribute count different frame %d.  Expected: %d, Actual %d"), *EmitterNameString, FrameIndex, CompareVariables.Num(), RhsCompareVariables.Num()));
			bAttributesValid = false;
		}

		auto CompareVariableFunc = 
			[&OutDifferences, &EmitterNameString](const TArray<FNiagaraSimCacheVariable>& LhsVariables, const TArray<FNiagaraSimCacheVariable>& RhsVariables) -> bool
			{
				bool IsValid = true;
				for (const FNiagaraSimCacheVariable& Var : LhsVariables)
				{
					auto ComparePredicate = [&Var](const FNiagaraSimCacheVariable& Rhs) -> bool
					{
						return Var.Variable == Rhs.Variable;
					};

					if (!RhsVariables.ContainsByPredicate(ComparePredicate))
					{
						AddError(OutDifferences, FString::Printf(TEXT("%s attribute %s does not exist in both caches."), *EmitterNameString, *Var.Variable.GetName().ToString()));
						IsValid = false;
					}
				}
				return IsValid;
			};
		bAttributesValid &= CompareVariableFunc(CompareVariables, RhsCompareVariables);
		bAttributesValid &= CompareVariableFunc(RhsCompareVariables, CompareVariables);

		if (!bAttributesValid)
		{
			return false;
		}
	}

	// Attempt to build an instance mapping between the two caches as ordering might be inconsistent due to wave size on GPU, etc, otherwise we will have to assume consistent ordering
	TArray<int32> InstanceMapping;
	InstanceMapping.AddZeroed(LhsNumInstances);
	{
		const FName IDAttributeName("UniqueID");

		TArray<int32> LhsIDValues;
		TArray<int32> RhsIDValues;
		LhsCache.ReadIntAttribute(LhsIDValues, IDAttributeName, EmitterName, FrameIndex);
		RhsCache.ReadIntAttribute(RhsIDValues, IDAttributeName, EmitterName, FrameIndex);
		if (LhsIDValues.Num() > 0 || RhsIDValues.Num() > 0)
		{
			if (LhsIDValues.Num() != RhsIDValues.Num() || LhsIDValues.Num() != LhsNumInstances)
			{
				AddError(OutDifferences, FString::Printf(TEXT("%s has invalid particle IDs - frame %d."), *EmitterNameString, FrameIndex));
				return false;
			}

			bool bIDsValid = true;
			for (int32 i=0; i < LhsNumInstances; ++i)
			{
				const int32 LhsIDValue = LhsIDValues[i];
				const int32 RhsIndex = RhsIDValues.IndexOfByKey(LhsIDValue);
				if (RhsIndex == INDEX_NONE)
				{
					AddError(OutDifferences, FString::Printf(TEXT("%s has inconsistent particle IDs - frame %d.  ID %d not found."), *EmitterNameString, FrameIndex, LhsIDValue));
					bIDsValid = false;
				}
				InstanceMapping[i] = RhsIndex;
			}
			if (!bIDsValid)
			{
				return false;
			}
		}
		else
		{
			for (int32 i = 0; i < LhsNumInstances; ++i)
			{
				InstanceMapping[i] = i;
			}
		}
	}

	bool bAttributesEqual = true;
	for (const FNiagaraSimCacheVariable& CompareVariable : CompareVariables)
	{
		const FName AtributeName = CompareVariable.Variable.GetName();
		
		TArray<float> LhsFloats;
		TArray<float> RhsFloats;
		TArray<FFloat16> LhsHalfs;
		TArray<FFloat16> RhsHalfs;
		TArray<int32> LhsInts;
		TArray<int32> RhsInts;
		LhsCache.ReadAttribute(LhsFloats, LhsHalfs, LhsInts, AtributeName, EmitterName, FrameIndex);
		RhsCache.ReadAttribute(RhsFloats, RhsHalfs, RhsInts, AtributeName, EmitterName, FrameIndex);

		// Specialize comparison for quaternions
		TStringBuilder<1024> ValueDiff;
		bool bIsError = false;

		if (CompareVariable.Variable.GetType() == FNiagaraTypeDefinition::GetQuatDef())
		{
			const float ErrorTolerance = DefaultQuaternionTolerance.Get(UE_KINDA_SMALL_NUMBER);

			bIsError = !CompareQuatAttributeData(InstanceMapping, LhsFloats, RhsFloats, ValueDiff, ErrorTolerance);
		}
		else
		{
			const float ErrorTolerance = VariableToFloatTolerances.FindRef(CompareVariable.Variable, DefaultFloatTolerance);

			bIsError  = !CompareAttributeData(InstanceMapping, LhsFloats, RhsFloats, CompareVariable.FloatCount, ValueDiff, ErrorTolerance);
			bIsError |= !CompareAttributeData(InstanceMapping, LhsHalfs,  RhsHalfs,  CompareVariable.HalfCount,  ValueDiff, ErrorTolerance);
			bIsError |= !CompareAttributeData(InstanceMapping, LhsInts,   RhsInts,   CompareVariable.Int32Count, ValueDiff, ErrorTolerance);
		}

		if ( bIsError )
		{
			AddError(
				OutDifferences,
				FString::Printf(TEXT("%s different particle data - frame %d attribute %s - %s"), *EmitterNameString, FrameIndex, *AtributeName.ToString(), ValueDiff.ToString())
			);
			bAttributesEqual = false;
		}
	}

	return bAttributesEqual;
}
