// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimCacheDebugData.h"
#include "NiagaraSimCacheHelper.h"
#include "NiagaraEmitterInstanceImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSimCacheDebugData)

namespace NSCDebugDataPrivate
{
	template<typename TStringType>
	void AddParameterStore(UObject* Owner, FNiagaraSimCacheDebugDataFrame& FrameData, TStringType StoreName, const FNiagaraParameterStore& ParameterStore)
	{
		if (ParameterStore.Num() > 0)
		{
			FNiagaraParameterStore& DestStore = FrameData.DebugParameterStores.FindOrAdd(StoreName);
			DestStore.SetOwner(Owner);
			DestStore = ParameterStore;
		}
	}

	template<typename TStringType>
	void AddParameterStore(UObject* Owner, FNiagaraSimCacheDebugDataFrame& FrameData, TStringType StoreName, const FNiagaraScriptExecutionContextBase* ExecContext)
	{
		if (ExecContext && ExecContext->Parameters.ReadParameterVariables().Num() > 0)
		{
			FNiagaraParameterStore& DestStore = FrameData.DebugParameterStores.FindOrAdd(StoreName);
			DestStore.SetOwner(Owner);
			ExecContext->Parameters.CopyParametersTo(DestStore, false, FNiagaraParameterStore::EDataInterfaceCopyMethod::Value);
		}
	}

	template<typename TStringType>
	void AddParameterStore(UObject* Owner, FNiagaraSimCacheDebugDataFrame& FrameData, TStringType StoreName, const FNiagaraComputeExecutionContext* ExecContext)
	{
		if (ExecContext)
		{
			AddParameterStore(Owner, FrameData, StoreName, ExecContext->CombinedParamStore);
		}
	}

#if WITH_EDITORONLY_DATA
	template<typename TStringType>
	void AddStaticVariables(UObject* Owner, FNiagaraSimCacheDebugDataFrame& FrameData, TStringType StoreName, const UNiagaraScript* Script)
	{
		if (Script && Script->GetVMExecutableData().StaticVariablesWritten.Num() > 0)
		{
			FNiagaraParameterStore& DestStore = FrameData.DebugParameterStores.FindOrAdd(StoreName);

			for (const FNiagaraVariable& StaticVar : Script->GetVMExecutableData().StaticVariablesWritten)
			{
				DestStore.AddParameter(StaticVar, true, false);
			}
		}
	}
#endif
}

UNiagaraSimCacheDebugData::UNiagaraSimCacheDebugData(const FObjectInitializer& ObjectInitializer)
{
}

void UNiagaraSimCacheDebugData::CaptureFrame(FNiagaraSimCacheHelper& Helper, int FrameNumber)
{
	using namespace NSCDebugDataPrivate;

   	Frames.SetNum(FrameNumber + 1);

	FNiagaraSimCacheDebugDataFrame& FrameData = Frames[FrameNumber];

	// Add Override Parameters
	if (const FNiagaraParameterStore* OverrideParameterStore = Helper.SystemInstance->GetOverrideParameters())
	{
		AddParameterStore(this, FrameData, TEXT("OverrideParameters"), *OverrideParameterStore);
	}

	// Add Instance Parameters
	AddParameterStore(this, FrameData, TEXT("InstanceParameters"), Helper.SystemInstance->GetInstanceParameters());

	// Add System Script Parameters
	if (FNiagaraSystemSimulation* SystemSimulation = Helper.SystemInstance->GetSystemSimulation().Get())
	{
		AddParameterStore(this, FrameData, TEXT("System Spawn"), SystemSimulation->GetSpawnExecutionContext());
		AddParameterStore(this, FrameData, TEXT("System Update"), SystemSimulation->GetUpdateExecutionContext());

	#if WITH_EDITORONLY_DATA
		if (UNiagaraSystem* NiagaraSystem = SystemSimulation->GetSystem())
		{
			AddStaticVariables(this, FrameData, TEXT("Static Variables"), NiagaraSystem->GetSystemSpawnScript());
			AddStaticVariables(this, FrameData, TEXT("Static Variables"), NiagaraSystem->GetSystemUpdateScript());
		}
	#endif
	}

	// Add Emitter Parameters
	for (const FNiagaraEmitterInstanceRef& EmitterRef : Helper.SystemInstance->GetEmitters())
	{
		const FNiagaraEmitterHandle& EmitterHandle = EmitterRef->GetEmitterHandle();
		const FString EmitterName = EmitterHandle.GetName().ToString();

		AddParameterStore(this, FrameData, FString::Printf(TEXT("%s RendererBindings"), *EmitterName), EmitterRef->GetRendererBoundVariables());
		AddParameterStore(this, FrameData, FString::Printf(TEXT("%s GPUContext"), *EmitterName), EmitterRef->GetGPUContext());

		if (FNiagaraEmitterInstanceImpl* StatefulEmitter = EmitterRef->AsStateful())
		{
			AddParameterStore(this, FrameData, FString::Printf(TEXT("%s Spawn"), *EmitterName), &StatefulEmitter->GetSpawnExecutionContext());
			AddParameterStore(this, FrameData, FString::Printf(TEXT("%s Update"), *EmitterName), &StatefulEmitter->GetUpdateExecutionContext());
		}
	}
}
