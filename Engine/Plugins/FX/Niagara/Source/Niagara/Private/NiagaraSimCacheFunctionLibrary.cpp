// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimCacheFunctionLibrary.h"
#include "Engine/Engine.h"
#include "NiagaraComponent.h"
#include "NiagaraSimCacheCapture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSimCacheFunctionLibrary)

void UAsyncNiagaraCaptureSimCache::Activate()
{
	Super::Activate();

	bIsRunning = true;
	SimCacheCapture.OnCaptureComplete().AddUObject(this, &UAsyncNiagaraCaptureSimCache::CaptureFinished);
	SimCacheCapture.CaptureNiagaraSimCache(CaptureSimCache, CreateParameters, CaptureComponent, CaptureParameters);
}

void UAsyncNiagaraCaptureSimCache::Cancel()
{
	if (bIsRunning)
	{
		SimCacheCapture.FinishCapture();
	}

	Super::Cancel();
}

void UAsyncNiagaraCaptureSimCache::SetReadyToDestroy()
{
	Super::SetReadyToDestroy();

	if (bIsRunning)
	{
		bIsRunning = false;
		CaptureComplete.Broadcast(::IsValid(CaptureSimCache) ? CaptureSimCache->IsCacheValid() : false);
		CaptureComponent = nullptr;
	}
}

UAsyncNiagaraCaptureSimCache* UAsyncNiagaraCaptureSimCache::CaptureNiagaraSimCacheMultiFrame(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters InCreateParameters, UNiagaraComponent* NiagaraComponent, UNiagaraSimCache*& OutSimCache, int32 NumFrames, int32 CaptureRate, bool bAdvanceSimulation, float AdvanceDeltaTime)
{
	FNiagaraSimCacheCaptureParameters InCaptureParameters;
	InCaptureParameters.NumFrames = FMath::Max(1, NumFrames);
	InCaptureParameters.CaptureRate = FMath::Max(1, CaptureRate);
	InCaptureParameters.bCaptureAllFramesImmediatly = bAdvanceSimulation;
	InCaptureParameters.ImmediateCaptureDeltaTime = AdvanceDeltaTime;

	return CaptureNiagaraSimCacheImpl(SimCache, InCreateParameters, NiagaraComponent, InCaptureParameters, OutSimCache);
}

UAsyncNiagaraCaptureSimCache* UAsyncNiagaraCaptureSimCache::CaptureNiagaraSimCacheUntilComplete(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters InCreateParameters, UNiagaraComponent* NiagaraComponent, UNiagaraSimCache*& OutSimCache, int32 CaptureRate, bool bAdvanceSimulation, float AdvanceDeltaTime)
{
	FNiagaraSimCacheCaptureParameters InCaptureParameters;
	InCaptureParameters.NumFrames = 0;
	InCaptureParameters.CaptureRate = FMath::Max(1, CaptureRate);
	InCaptureParameters.bCaptureAllFramesImmediatly = bAdvanceSimulation;
	InCaptureParameters.ImmediateCaptureDeltaTime = AdvanceDeltaTime;

	return CaptureNiagaraSimCacheImpl(SimCache, InCreateParameters, NiagaraComponent, InCaptureParameters, OutSimCache);
}

UAsyncNiagaraCaptureSimCache* UAsyncNiagaraCaptureSimCache::CaptureNiagaraSimCache(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters InCreateParameters, UNiagaraComponent* NiagaraComponent, FNiagaraSimCacheCaptureParameters InCaptureParameters, UNiagaraSimCache*& OutSimCache)
{
	return CaptureNiagaraSimCacheImpl(SimCache, InCreateParameters, NiagaraComponent, InCaptureParameters, OutSimCache);
}

UAsyncNiagaraCaptureSimCache* UAsyncNiagaraCaptureSimCache::CaptureNiagaraSimCacheImpl(UNiagaraSimCache* SimCache, const FNiagaraSimCacheCreateParameters& InCreateParameters, UNiagaraComponent* NiagaraComponent, const FNiagaraSimCacheCaptureParameters& InCaptureParameters, UNiagaraSimCache*& OutSimCache)
{
	UAsyncNiagaraCaptureSimCache* CaptureAction = NewObject<UAsyncNiagaraCaptureSimCache>();
	CaptureAction->CaptureParameters = InCaptureParameters;
	CaptureAction->CreateParameters = InCreateParameters;
	CaptureAction->CaptureSimCache = SimCache;
	CaptureAction->CaptureComponent = NiagaraComponent;
	OutSimCache = SimCache;

	return CaptureAction;
}

void UAsyncNiagaraCaptureSimCache::CaptureFinished(UNiagaraSimCache* InCapturedSimCache)
{
	CaptureSimCache = InCapturedSimCache;
	SetReadyToDestroy();
}

UNiagaraSimCacheFunctionLibrary::UNiagaraSimCacheFunctionLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UNiagaraSimCacheFunctionLibrary::CaptureNiagaraSimCacheImmediate(UNiagaraSimCache* SimCache, FNiagaraSimCacheCreateParameters CreateParameters, UNiagaraComponent* NiagaraComponent, UNiagaraSimCache*& OutSimCache, bool bAdvanceSimulation, float AdvanceDeltaTime)
{
	FNiagaraSimCacheCapture SimCacheCapture;

	return SimCacheCapture.CaptureCurrentFrameImmediate(SimCache, CreateParameters, NiagaraComponent, OutSimCache, bAdvanceSimulation, AdvanceDeltaTime);
}

UNiagaraSimCache* UNiagaraSimCacheFunctionLibrary::CreateNiagaraSimCache(UObject* WorldContextObject)
{
	if ( UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) )
	{
		return NewObject<UNiagaraSimCache>(WorldContextObject);
	}

	return nullptr;
}
