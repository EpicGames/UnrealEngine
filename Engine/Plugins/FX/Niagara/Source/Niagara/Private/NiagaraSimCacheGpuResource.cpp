// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimCacheGpuResource.h"
#include "NiagaraSimCache.h"
#include "RenderingThread.h"

FNiagaraSimCacheGpuResource::FNiagaraSimCacheGpuResource(UNiagaraSimCache* SimCache)
	: WeakSimCache(SimCache)
{
	OnSimCacheChangedHandle = UNiagaraSimCache::OnCacheEndWrite.AddRaw(this, &FNiagaraSimCacheGpuResource::BuildResource);
	BuildResource(SimCache);
}

FNiagaraSimCacheGpuResource::~FNiagaraSimCacheGpuResource()
{
	check(IsInRenderingThread());
	if ( OnSimCacheChangedHandle.IsValid() )
	{
		UNiagaraSimCache::OnCacheEndWrite.Remove(OnSimCacheChangedHandle);
	}
}

FNiagaraSimCacheGpuResourcePtr FNiagaraSimCacheGpuResource::CreateResource(UNiagaraSimCache* SimCache)
{
	return MakeShareable(new FNiagaraSimCacheGpuResource(SimCache), FNiagaraRenderThreadDeletor<FNiagaraSimCacheGpuResource>());
}

void FNiagaraSimCacheGpuResource::BuildResource(UNiagaraSimCache* SimCache)
{
	check(SimCache != nullptr);

	NumFrames = SimCache->GetNumFrames();
	NumEmitters = SimCache->GetNumEmitters();

	const uint32 FrameDataSizeInInts = 4;	//-TODO: Match with shader, this is NumInstances, IntDataOffset, FloatDataOffset, HalfDataOffset
	uint32 CurrentFrameOffset = 0;
	uint32 CurrentDataOffset = NumFrames * NumEmitters * FrameDataSizeInInts;
	TArray<uint32> GpuCacheData;
	{
		uint32 RequiredElements = CurrentDataOffset;
		for (const FNiagaraSimCacheFrame& CacheFrame : SimCache->CacheFrames)
		{
			for (const FNiagaraSimCacheEmitterFrame& EmitterFrame : CacheFrame.EmitterData)
			{
				RequiredElements += FMath::DivideAndRoundUp<uint32>(EmitterFrame.ParticleDataBuffers.Int32Data.Num(), sizeof(uint32));
				RequiredElements += FMath::DivideAndRoundUp<uint32>(EmitterFrame.ParticleDataBuffers.FloatData.Num(), sizeof(uint32));
				RequiredElements += FMath::DivideAndRoundUp<uint32>(EmitterFrame.ParticleDataBuffers.HalfData.Num(), sizeof(uint32));
			}
		}

		GpuCacheData.AddUninitialized(RequiredElements);
	}

	for (const FNiagaraSimCacheFrame& CacheFrame : SimCache->CacheFrames)
	{
		for (const FNiagaraSimCacheEmitterFrame& EmitterFrame : CacheFrame.EmitterData)
		{
			GpuCacheData[CurrentFrameOffset++] = EmitterFrame.ParticleDataBuffers.NumInstances;

			GpuCacheData[CurrentFrameOffset++] = CurrentDataOffset;
			if (EmitterFrame.ParticleDataBuffers.Int32Data.Num() > 0)
			{
				FMemory::Memcpy(&GpuCacheData[CurrentDataOffset], EmitterFrame.ParticleDataBuffers.Int32Data.GetData(), EmitterFrame.ParticleDataBuffers.Int32Data.Num());
				CurrentDataOffset += FMath::DivideAndRoundUp<uint32>(EmitterFrame.ParticleDataBuffers.Int32Data.Num(), sizeof(uint32));
			}

			GpuCacheData[CurrentFrameOffset++] = CurrentDataOffset;
			if (EmitterFrame.ParticleDataBuffers.FloatData.Num() > 0)
			{
				FMemory::Memcpy(&GpuCacheData[CurrentDataOffset], EmitterFrame.ParticleDataBuffers.FloatData.GetData(), EmitterFrame.ParticleDataBuffers.FloatData.Num());
				CurrentDataOffset += FMath::DivideAndRoundUp<uint32>(EmitterFrame.ParticleDataBuffers.FloatData.Num(), sizeof(uint32));
			}

			GpuCacheData[CurrentFrameOffset++] = CurrentDataOffset;
			if (EmitterFrame.ParticleDataBuffers.HalfData.Num() > 0)
			{
				FMemory::Memcpy(&GpuCacheData[CurrentDataOffset], EmitterFrame.ParticleDataBuffers.HalfData.GetData(), EmitterFrame.ParticleDataBuffers.HalfData.Num());
				CurrentDataOffset += FMath::DivideAndRoundUp<uint32>(EmitterFrame.ParticleDataBuffers.HalfData.Num(), sizeof(uint32));
			}
		}
	}

	ENQUEUE_RENDER_COMMAND(FNiagaraSimCacheGpuResource_BuildResource)
	(
		[this, NumFrames_RT=NumFrames, NumEmitters_RT=NumEmitters, GpuCacheData_RT=MoveTemp(GpuCacheData)](FRHICommandList& RHICmdList)
		{
			NumFrames = NumFrames_RT;
			NumEmitters = NumEmitters_RT;
			SimCacheBuffer.Release();
			if (GpuCacheData_RT.Num() > 0)
			{
				SimCacheBuffer.Initialize(RHICmdList, TEXT("NiagaraSimCache"), sizeof(uint32), GpuCacheData_RT.Num(), EPixelFormat::PF_R32_UINT, BUF_Static);

				void* GpuMemory = RHICmdList.LockBuffer(SimCacheBuffer.Buffer, 0, SimCacheBuffer.NumBytes, RLM_WriteOnly);
				FMemory::Memcpy(GpuMemory, GpuCacheData_RT.GetData(), SimCacheBuffer.NumBytes);
				RHICmdList.UnlockBuffer(SimCacheBuffer.Buffer);
			}
		}
	);
}
