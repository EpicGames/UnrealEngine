// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "Templates/SharedPointer.h"

#include "NiagaraRenderThreadDeletor.h"
#include "NiagaraRenderer.h"

class UNiagaraSimCache;

struct FNiagaraSimCacheGpuResource
{
	UE_NONCOPYABLE(FNiagaraSimCacheGpuResource)
	friend FNiagaraRenderThreadDeletor<FNiagaraSimCacheGpuResource>;

private:
	FNiagaraSimCacheGpuResource(UNiagaraSimCache* SimCache);
	virtual ~FNiagaraSimCacheGpuResource();

public:
	static TSharedPtr<FNiagaraSimCacheGpuResource> CreateResource(UNiagaraSimCache* SimCache);

	const UNiagaraSimCache* GetSimCache() const { return WeakSimCache.Get(); }

	int32 GetNumFrames() const { return NumFrames; }
	int32 GetNumEmitters() const { return NumEmitters; }
	int32 GetEmitterIndex() const { return EmitterIndex; }
	FRHIShaderResourceView* GetBufferSRV() const { return SimCacheBuffer.SRV.IsValid() ? SimCacheBuffer.SRV.GetReference() : FNiagaraRenderer::GetDummyUIntBuffer(); }

private:
	void BuildResource(UNiagaraSimCache* SimCache);

private:
	TWeakObjectPtr<UNiagaraSimCache>	WeakSimCache;

	int32								NumFrames = 0;
	int32								NumEmitters = 0;
	int32								EmitterIndex = INDEX_NONE;
	FReadBuffer							SimCacheBuffer;

	FDelegateHandle						OnSimCacheChangedHandle;
};

typedef TSharedPtr<FNiagaraSimCacheGpuResource> FNiagaraSimCacheGpuResourcePtr;
