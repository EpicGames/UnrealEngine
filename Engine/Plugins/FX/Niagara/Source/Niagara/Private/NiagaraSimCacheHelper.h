// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "NiagaraClearCounts.h"
#include "NiagaraComponent.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraConstants.h"
#include "NiagaraDataSetReadback.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSimCache.h"
#include "NiagaraSystemInstanceController.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/NiagaraStatelessEmitterData.h"

struct FNiagaraSimCacheHelper
{
	/////////////////////////////////////////////////////////////////////////////////
	template<typename TBaseType, int32 NumComponents>
	struct FSoAReader
	{
		const TBaseType* Components[NumComponents];
		FSoAReader(const uint8* InData, const int32 InDataStride)
		{
			for (int32 i=0; i < NumComponents; ++i)
			{
				Components[i] = reinterpret_cast<const TBaseType*>(InData);
				InData += InDataStride;
			}
		}
	};

	struct FSoAVec3Reader : FSoAReader<float, 3>
	{
		FSoAVec3Reader(const uint8* InData, const int32 InDataStride) : FSoAReader<float, 3>(InData, InDataStride) {}
		FVector3f Get(int iInstance) const { return FVector3f(Components[0][iInstance], Components[1][iInstance], Components[2][iInstance]); }
	};

	struct FSoAQuatReader : FSoAReader<float, 4>
	{
		FSoAQuatReader(const uint8* InData, const int32 InDataStride) : FSoAReader<float, 4>(InData, InDataStride) {}
		FQuat4f Get(int iInstance) const { return FQuat4f(Components[0][iInstance], Components[1][iInstance], Components[2][iInstance], Components[3][iInstance]); }
	};

	/////////////////////////////////////////////////////////////////////////////////
	template<typename TBaseType, int32 NumComponents>
	struct FSoAWriter
	{
		TBaseType* Components[NumComponents];
		FSoAWriter(uint8* InData, const int32 InDataStride)
		{
			for (int32 i = 0; i < NumComponents; ++i)
			{
				Components[i] = reinterpret_cast<TBaseType*>(InData);
				InData += InDataStride;
			}
		}
	};

	struct FSoAVec3Writer : FSoAWriter<float, 3>
	{
		FSoAVec3Writer(uint8* InData, const int32 InDataStride) : FSoAWriter<float, 3>(InData, InDataStride) {}
		void Set(int iInstance, const FVector3f& Value) const { Components[0][iInstance] = Value.X; Components[1][iInstance] = Value.Y; Components[2][iInstance] = Value.Z; }
	};

	struct FSoAQuatWriter : FSoAWriter<float, 4>
	{
		FSoAQuatWriter(uint8* InData, const int32 InDataStride) : FSoAWriter<float, 4>(InData, InDataStride) {}
		void Set(int iInstance, const FQuat4f& Value) const { Components[0][iInstance] = Value.X; Components[1][iInstance] = Value.Y; Components[2][iInstance] = Value.Z; Components[3][iInstance] = Value.W; }
	};
	/////////////////////////////////////////////////////////////////////////////////

	explicit FNiagaraSimCacheHelper(FNiagaraSystemInstance* InSystemInstance)
	{
		SystemInstance = InSystemInstance;
		SystemSimulation = SystemInstance->GetSystemSimulation();
		check(SystemSimulation);
		SystemSimulationDataBuffer = SystemSimulation->MainDataSet.GetCurrentData();
		NiagaraSystem = SystemSimulation->GetSystem();
	}

	explicit FNiagaraSimCacheHelper(UNiagaraComponent* NiagaraComponent)
	{
		if ( NiagaraComponent == nullptr )
		{
			return;
		}

		NiagaraSystem = NiagaraComponent->GetAsset();
		if ( NiagaraSystem == nullptr )
		{
			return;
		}

		FNiagaraSystemInstanceControllerPtr SystemInstanceController = NiagaraComponent->GetSystemInstanceController();
		if (SystemInstanceController.IsValid() == false)
		{
			return;
		}

		SystemInstance = SystemInstanceController->GetSystemInstance_Unsafe();
		if (SystemInstance == nullptr)
		{
			return;
		}

		SystemSimulation = SystemInstance->GetSystemSimulation();
		if (SystemSimulation == nullptr)
		{
			return;
		}

		SystemSimulationDataBuffer = SystemSimulation->MainDataSet.GetCurrentData();
		if (SystemSimulationDataBuffer == nullptr)
		{
			return;
		}
	}

	FNiagaraDataSet& GetSystemSimulationDataSet() { return SystemSimulation->MainDataSet; }

	bool HasValidSimulation() const { return SystemSimulation != nullptr; }
	bool HasValidSimulationData() const { return SystemSimulationDataBuffer != nullptr; }

	void BuildCacheLayout(const FNiagaraSimCacheCreateParameters& CreateParameters, FNiagaraSimCacheDataBuffersLayout& CacheLayout, const FNiagaraDataSetCompiledData& CompiledData, FName LayoutName, TArray<FName> InRebaseVariableNames, TArray<FName> InInterpVariableNames, TConstArrayView<FName> ExplicitCaptureAttributes) const
	{
		CacheLayout.LayoutName = LayoutName;
		CacheLayout.SimTarget = CompiledData.SimTarget;
		CacheLayout.ComponentVelocity = INDEX_NONE;
		CacheLayout.CacheBufferWriteInfo.ComponentUniqueID = INDEX_NONE;

		// Determine the components to cache
		int32 TotalCacheComponents = 0;
		TArray<int32> CacheToDataSetVariables;
		bool bHasPositionAttribute = false;

		CacheToDataSetVariables.Reserve(CompiledData.Variables.Num());
		for (int32 i = 0; i < CompiledData.Variables.Num(); ++i)
		{
			const FNiagaraVariableLayoutInfo& DataSetVariableLayout = CompiledData.VariableLayouts[i];
			const FNiagaraVariableBase& DataSetVariable = CompiledData.Variables[i];
			if (CreateParameters.AttributeCaptureMode == ENiagaraSimCacheAttributeCaptureMode::All || ExplicitCaptureAttributes.Contains(DataSetVariable.GetName()))
			{
				CacheToDataSetVariables.Add(i);
				TotalCacheComponents += DataSetVariableLayout.GetNumFloatComponents() + DataSetVariableLayout.GetNumHalfComponents() + DataSetVariableLayout.GetNumInt32Components();
				bHasPositionAttribute |= DataSetVariable.GetType() == FNiagaraTypeDefinition::GetPositionDef();
			}
		}

		// We need to preserve the velocity attribute if we want to use velocity based extrapolation of positions
		// Note:  Only preserve the velocity attribute if it will be used (i.e. we have a position attribute we need to extrapolate)
		const FNiagaraVariableBase VelocityVariable(FNiagaraTypeDefinition::GetVec3Def(), "Velocity");
		if (CreateParameters.bAllowVelocityExtrapolation && bHasPositionAttribute)
		{
			const int32 VelocityAttributeIndex = CompiledData.Variables.IndexOfByKey(VelocityVariable);
			if (VelocityAttributeIndex != INDEX_NONE)
			{
				CacheLayout.bAllowVelocityExtrapolation = true;
				if (!CacheToDataSetVariables.Contains(VelocityAttributeIndex))
				{
					CacheToDataSetVariables.AddUnique(VelocityAttributeIndex);
					const FNiagaraVariableLayoutInfo& DataSetVariableLayout = CompiledData.VariableLayouts[VelocityAttributeIndex];
					TotalCacheComponents += DataSetVariableLayout.GetNumFloatComponents() + DataSetVariableLayout.GetNumHalfComponents() + DataSetVariableLayout.GetNumInt32Components();
				}
			}
		}

		const int32 NumCacheVariables = CacheToDataSetVariables.Num();

		CacheLayout.Variables.AddDefaulted(NumCacheVariables);

		CacheLayout.CacheBufferWriteInfo.ComponentMappingsFromDataBuffer.Empty(TotalCacheComponents);
		CacheLayout.CacheBufferWriteInfo.ComponentMappingsFromDataBuffer.AddDefaulted(TotalCacheComponents);
		CacheLayout.RebaseVariableNames = MoveTemp(InRebaseVariableNames);
		CacheLayout.InterpVariableNames = MoveTemp(InInterpVariableNames);

		for ( int32 iCacheVariable=0; iCacheVariable < NumCacheVariables; ++iCacheVariable)
		{
			const int32 iDataSetVariable = CacheToDataSetVariables[iCacheVariable];
			const FNiagaraVariableLayoutInfo& DataSetVariableLayout = CompiledData.VariableLayouts[iDataSetVariable];
			FNiagaraSimCacheVariable& CacheVariable = CacheLayout.Variables[iCacheVariable];

			CacheVariable.Variable = CompiledData.Variables[iDataSetVariable];
			CacheVariable.FloatOffset = DataSetVariableLayout.GetNumFloatComponents() > 0 ? CacheLayout.FloatCount : INDEX_NONE;
			CacheVariable.FloatCount = IntCastChecked<uint16>(DataSetVariableLayout.GetNumFloatComponents());
			CacheVariable.HalfOffset = DataSetVariableLayout.GetNumHalfComponents() > 0 ? CacheLayout.HalfCount : INDEX_NONE;
			CacheVariable.HalfCount = IntCastChecked<uint16>(DataSetVariableLayout.GetNumHalfComponents());
			CacheVariable.Int32Offset = DataSetVariableLayout.GetNumInt32Components() > 0 ? CacheLayout.Int32Count : INDEX_NONE;
			CacheVariable.Int32Count = IntCastChecked<uint16>(DataSetVariableLayout.GetNumInt32Components());

			CacheLayout.FloatCount += IntCastChecked<uint16>(DataSetVariableLayout.GetNumFloatComponents());
			CacheLayout.HalfCount += IntCastChecked<uint16>(DataSetVariableLayout.GetNumHalfComponents());
			CacheLayout.Int32Count += IntCastChecked<uint16>(DataSetVariableLayout.GetNumInt32Components());
		}

		if (CreateParameters.bAllowVelocityExtrapolation)
		{
			if (const FNiagaraSimCacheVariable* VelocityCacheVariable = CacheLayout.FindCacheVariable(VelocityVariable) )
			{
				CacheLayout.ComponentVelocity = IntCastChecked<uint16>(VelocityCacheVariable->FloatOffset);
			}
		}

		if (CreateParameters.bAllowInterpolation)
		{
			const FNiagaraVariableBase UniqueIDVariable(FNiagaraTypeDefinition::GetIntDef(), "UniqueID");
			const int32 IDVarIndex = CompiledData.Variables.IndexOfByKey(UniqueIDVariable);
			CacheLayout.CacheBufferWriteInfo.ComponentUniqueID = IDVarIndex == INDEX_NONE ? static_cast<uint16>(INDEX_NONE) : IntCastChecked<uint16>(IDVarIndex);
			if (CacheLayout.CacheBufferWriteInfo.ComponentUniqueID != static_cast<uint16>(INDEX_NONE))
			{
				const FNiagaraVariableLayoutInfo& DataSetVariableLayout = CompiledData.VariableLayouts[CacheLayout.CacheBufferWriteInfo.ComponentUniqueID];
				check(DataSetVariableLayout.GetNumInt32Components() == 1);
				CacheLayout.CacheBufferWriteInfo.ComponentUniqueID = IntCastChecked<uint16>(DataSetVariableLayout.GetInt32ComponentStart());
				CacheLayout.bAllowInterpolation = true;
			}
		}
		if (!CacheLayout.bAllowInterpolation)
		{
			CacheLayout.InterpVariableNames.Empty();
		}

		// Build write mappings we will build read mappings in a separate path
		int32 FloatOffset = 0;
		int32 HalfOffset = CacheLayout.FloatCount;
		int32 Int32Offset = HalfOffset + CacheLayout.HalfCount;
		for (int32 iCacheVariable = 0; iCacheVariable < NumCacheVariables; ++iCacheVariable)
		{
			const int32 iDataSetVariable = CacheToDataSetVariables[iCacheVariable];
			const FNiagaraVariableLayoutInfo& DataSetVariableLayout = CompiledData.VariableLayouts[iDataSetVariable];
			FNiagaraSimCacheVariable& CacheVariable = CacheLayout.Variables[iCacheVariable];

			for (int32 iComponent=0; iComponent < CacheVariable.FloatCount; ++iComponent)
			{
				CacheLayout.CacheBufferWriteInfo.ComponentMappingsFromDataBuffer[FloatOffset] = IntCastChecked<uint16>(DataSetVariableLayout.GetFloatComponentStart() + iComponent);
				++FloatOffset;
			}

			for (int32 iComponent=0; iComponent < CacheVariable.HalfCount; ++iComponent)
			{
				CacheLayout.CacheBufferWriteInfo.ComponentMappingsFromDataBuffer[HalfOffset] = IntCastChecked<uint16>(DataSetVariableLayout.GetHalfComponentStart() + iComponent);
				++HalfOffset;
			}

			for (int32 iComponent=0; iComponent < CacheVariable.Int32Count; ++iComponent)
			{
				CacheLayout.CacheBufferWriteInfo.ComponentMappingsFromDataBuffer[Int32Offset] = IntCastChecked<uint16>(DataSetVariableLayout.GetInt32ComponentStart() + iComponent);
				++Int32Offset;
			}
		}

		// Slightly inefficient but we can share the code between the paths
		BuildCacheReadMappings(CacheLayout, CompiledData);
	}

	void BuildCacheLayoutForSystem(const FNiagaraSimCacheCreateParameters& CreateParameters, FNiagaraSimCacheDataBuffersLayout& CacheLayout)
	{
		const FNiagaraDataSetCompiledData& SystemCompileData = NiagaraSystem->GetSystemCompiledData().DataSetCompiledData;

		TArray<FName> RebaseVariableNames;
		if ( CreateParameters.bAllowRebasing )
		{
			TArray<FString, TInlineAllocator<8>> LocalSpaceEmitters;
			for ( int32 i=0; i < NiagaraSystem->GetNumEmitters(); ++i )
			{
				const FNiagaraEmitterHandle& EmitterHandle = NiagaraSystem->GetEmitterHandle(i);
				if (EmitterHandle.GetIsEnabled() )
				{
					const FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetInstance().GetEmitterData();
					if (EmitterData && EmitterData->bLocalSpace)
					{
						LocalSpaceEmitters.Add(EmitterHandle.GetUniqueInstanceName());
					}
				}
			}

			for (const FNiagaraVariableBase& Variable : SystemCompileData.Variables)
			{
				if (Variable.GetType() == FNiagaraTypeDefinition::GetPositionDef())
				{
					// If this is an emitter variable we need to check if it's local space or not
					bool bIsLocalSpace = false;
					for ( const FString& LocalSpaceEmitter : LocalSpaceEmitters )
					{
						if ( Variable.IsInNameSpace(LocalSpaceEmitter) )
						{
							bIsLocalSpace = true;
							break;
						}
					}

					if ( bIsLocalSpace == false && CreateParameters.RebaseExcludeAttributes.Contains(Variable.GetName()) == false )
					{
						RebaseVariableNames.AddUnique(Variable.GetName());
					}
				}
				else if ( CanRebaseVariable(Variable) && CreateParameters.RebaseIncludeAttributes.Contains(Variable.GetName()) )
				{
					RebaseVariableNames.AddUnique(Variable.GetName());
				}
			}
		}

		TArray<FName> InterpVariableNames;
		if (CreateParameters.bAllowInterpolation)
		{
			for (const FNiagaraVariableBase& Variable : SystemCompileData.Variables)
			{
				if (!CanInterpolateVariable(CreateParameters.bInterpolateAllTypes, Variable) || CreateParameters.InterpolationExcludeAttributes.Contains(Variable.GetName()))
				{
					continue;
				}

				if (CreateParameters.InterpolationIncludeAttributes.Num() == 0 || CreateParameters.InterpolationIncludeAttributes.Contains(Variable.GetName()))
				{
					InterpVariableNames.Add(Variable.GetName());
				}
			}
		}

		BuildCacheLayout(CreateParameters, CacheLayout, SystemCompileData, NiagaraSystem->GetFName(), MoveTemp(RebaseVariableNames), MoveTemp(InterpVariableNames), CreateParameters.ExplicitCaptureAttributes);
	}

	void BuildCacheLayoutForEmitter(const FNiagaraSimCacheCreateParameters& CreateParameters, FNiagaraSimCacheDataBuffersLayout& CacheLayout, int EmitterIndex)
	{
		const FNiagaraEmitterHandle& EmitterHandle = NiagaraSystem->GetEmitterHandle(EmitterIndex);
		if (!EmitterHandle.GetIsEnabled())
		{
			return;
		}

		const FNiagaraDataSetCompiledData* ParticleDataSet = nullptr;
		const FVersionedNiagaraEmitterData* StatefulEmitterData = nullptr;

		if (EmitterHandle.GetEmitterMode() == ENiagaraEmitterMode::Standard)
		{
			const FNiagaraEmitterCompiledData& EmitterCompiledData = NiagaraSystem->GetEmitterCompiledData()[EmitterIndex].Get();
			StatefulEmitterData = EmitterHandle.GetInstance().GetEmitterData();
			ParticleDataSet = StatefulEmitterData ? &EmitterCompiledData.DataSetCompiledData : nullptr;
		}
		// Include stateless emitters for debugging only.  For none debugging there is no value in storing this data as they are stateless.
		else if ( CreateParameters.bIncludeDebugData )
		{
			const UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandle.GetStatelessEmitter();
			const FNiagaraStatelessEmitterDataPtr StatelessEmitterData = StatelessEmitter ? StatelessEmitter->GetEmitterData() : nullptr;
			ParticleDataSet = StatelessEmitterData ? StatelessEmitterData->ParticleDataSetCompiledData.Get() : nullptr;
		}

		if (ParticleDataSet == nullptr)
		{
			return;
		}

		// Find potential candidates for re-basing
		CacheLayout.bLocalSpace = StatefulEmitterData ? StatefulEmitterData->bLocalSpace : true;

		TArray<FName> RebaseVariableNames;
		if ( CreateParameters.bAllowRebasing && CacheLayout.bLocalSpace == false )
		{
			// Build list of include / exclude names
			TArray<FName> ForceIncludeNames;
			TArray<FName> ForceExcludeNames;
			if ( CreateParameters.RebaseIncludeAttributes.Num() > 0 || CreateParameters.RebaseExcludeAttributes.Num() > 0 )
			{
				const FString EmitterName = EmitterHandle.GetUniqueInstanceName();
				for (FName RebaseName : CreateParameters.RebaseIncludeAttributes)
				{
					FNiagaraVariableBase BaseVar(FNiagaraTypeDefinition::GetFloatDef(), RebaseName);
					if (BaseVar.RemoveRootNamespace(EmitterName))
					{
						ForceIncludeNames.Add(BaseVar.GetName());
					}
				}

				for (FName RebaseName : CreateParameters.RebaseExcludeAttributes)
				{
					FNiagaraVariableBase BaseVar(FNiagaraTypeDefinition::GetFloatDef(), RebaseName);
					if (BaseVar.RemoveRootNamespace(EmitterName))
					{
						ForceExcludeNames.Add(BaseVar.GetName());
					}
				}
			}

		#if WITH_EDITORONLY_DATA
			if (StatefulEmitterData)
			{
				// Look for renderer attributes bound to Quat / Matrix types are we will want to rebase those
				// We will add all Position types after this so no need to add them here
				StatefulEmitterData->ForEachEnabledRenderer(
					[&](UNiagaraRendererProperties* RenderProperties)
					{
						for (FNiagaraVariable BoundAttribute : RenderProperties->GetBoundAttributes())
						{
							if ( (BoundAttribute.GetType() == FNiagaraTypeDefinition::GetQuatDef()) ||
								 (BoundAttribute.GetType() == FNiagaraTypeDefinition::GetMatrix4Def()) )
							{
								if (BoundAttribute.RemoveRootNamespace(FNiagaraConstants::ParticleAttributeNamespaceString))
								{
									if (ParticleDataSet->Variables.Contains(BoundAttribute) && ForceExcludeNames.Contains(BoundAttribute.GetName()) == false )
									{
										RebaseVariableNames.AddUnique(BoundAttribute.GetName());
									}
								}
							}
						}
					}
				);
			}
		#endif

			// Look for regular attributes that we are forcing to rebase or can rebase like positions
			for (const FNiagaraVariableBase& Variable : ParticleDataSet->Variables)
			{
				if ( Variable.GetType() == FNiagaraTypeDefinition::GetPositionDef() )
				{
					if ( ForceExcludeNames.Contains(Variable.GetName()) == false )
					{
						RebaseVariableNames.AddUnique(Variable.GetName());
					}
				}
				else if ( ForceIncludeNames.Contains(Variable.GetName()) && CanRebaseVariable(Variable) )
				{
					RebaseVariableNames.AddUnique(Variable.GetName());
				}
			}
		}

		TArray<FName> InterpVariableNames;
		if (CreateParameters.bAllowInterpolation)
		{
			for (const FNiagaraVariableBase& Variable : ParticleDataSet->Variables)
			{
				if (!CanInterpolateVariable(CreateParameters.bInterpolateAllTypes, Variable) || CreateParameters.InterpolationExcludeAttributes.Contains(Variable.GetName()))
				{
					continue;
				}

				if (CreateParameters.InterpolationIncludeAttributes.Num() == 0 || CreateParameters.InterpolationIncludeAttributes.Contains(Variable.GetName()))
				{
					InterpVariableNames.Add(Variable.GetName());
				}
			}
		}

		TArray<FName> ExplicitCaptureAttributes;
		if ( CreateParameters.ExplicitCaptureAttributes.Num() > 0 )
		{
			const FString EmitterName = EmitterHandle.GetUniqueInstanceName();
			for ( FName AttributeName : CreateParameters.ExplicitCaptureAttributes)
			{
				FNiagaraVariableBase AttributeVar(FNiagaraTypeDefinition::GetFloatDef(), AttributeName);
				if (AttributeVar.RemoveRootNamespace(EmitterName))
				{
					if (AttributeVar.RemoveRootNamespace(FNiagaraConstants::ParticleAttributeNamespaceString))
					{
						ExplicitCaptureAttributes.Add(AttributeVar.GetName());
					}
				}
			}
		}

		BuildCacheLayout(CreateParameters, CacheLayout, *ParticleDataSet, EmitterHandle.GetName(), MoveTemp(RebaseVariableNames), MoveTemp(InterpVariableNames), ExplicitCaptureAttributes);
	}

	static bool BuildCacheReadMappings(FNiagaraSimCacheDataBuffersLayout& CacheLayout, const FNiagaraDataSetCompiledData& CompiledData)
	{
		const int32 CacheTotalComponents = CacheLayout.FloatCount + CacheLayout.HalfCount + CacheLayout.Int32Count;
		FNiagaraSimCacheDataBuffersLayout::FCacheBufferReadInfo& CacheBufferReadInfo = CacheLayout.CacheBufferReadInfo;
		CacheBufferReadInfo.ComponentMappingsToDataBuffer.Empty(CacheTotalComponents);
		CacheBufferReadInfo.ComponentMappingsToDataBuffer.AddDefaulted(CacheTotalComponents);
		CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Empty(0);
		CacheLayout.bNeedsCacheBufferReadInfoUpdateForRT = true;

		//-OPT: We don't have to do this all the time, figure out a better way
		TArray<FNiagaraVariableBase> InterpCurrAttributes;
		TArray<FNiagaraVariableBase> InterpPrevAttributes;
		if (CacheLayout.bAllowInterpolation)
		{
			for (const FNiagaraSimCacheVariable& PreviousVariable : CacheLayout.Variables)
			{
				// We only support previous interpolation for types that are used in velocity calculation
				const bool bInterpVariable =
					CacheLayout.InterpVariableNames.Contains(PreviousVariable.Variable.GetName()) &&
					(PreviousVariable.Variable.GetType() == FNiagaraTypeDefinition::GetPositionDef() || PreviousVariable.Variable.GetType() == FNiagaraTypeDefinition::GetQuatDef());
				if (!bInterpVariable)
				{
					continue;
				}

				FNiagaraVariableBase CurrentVariable = PreviousVariable.Variable;
				if (!CurrentVariable.RemoveRootNamespace(FNiagaraConstants::PreviousNamespaceString))
				{
					continue;
				}

				if (CompiledData.Variables.IndexOfByKey(PreviousVariable.Variable) == INDEX_NONE || CompiledData.Variables.IndexOfByKey(CurrentVariable) == INDEX_NONE)
				{
					continue;
				}

				InterpCurrAttributes.Emplace(CurrentVariable);
				InterpPrevAttributes.Emplace(PreviousVariable.Variable);
			}
		}

		int32 FloatOffset = 0;
		int32 HalfOffset = CacheLayout.FloatCount;
		int32 Int32Offset = HalfOffset + CacheLayout.HalfCount;
		for (const FNiagaraSimCacheVariable& SourceVariable : CacheLayout.Variables)
		{
			// Find variable, if it doesn't exist that's ok as the cache contains more data than is required
			const int32 DataSetVariableIndex = CompiledData.Variables.IndexOfByKey(SourceVariable.Variable);
			const FNiagaraVariableLayoutInfo* DestVariableLayout = nullptr;
			if (DataSetVariableIndex != INDEX_NONE)
			{
				DestVariableLayout = &CompiledData.VariableLayouts[DataSetVariableIndex];

				// If the variable exists but types not match the cache is invalid
				if ((DestVariableLayout->GetNumFloatComponents() != SourceVariable.FloatCount) ||
					(DestVariableLayout->GetNumHalfComponents() != SourceVariable.HalfCount) ||
					(DestVariableLayout->GetNumInt32Components() != SourceVariable.Int32Count))
				{
					return false;
				}
			}

			// We won't copy these attributes here as they will be part of a combined copy step to interpolate previous
			if (InterpPrevAttributes.Contains(SourceVariable.Variable))
			{
				DestVariableLayout = nullptr;
			}

			// Is this a type that requires conversion / re-basing?
			if (DestVariableLayout != nullptr)
			{
				const bool bInterpVariable = CacheLayout.bAllowInterpolation && CacheLayout.InterpVariableNames.Contains(SourceVariable.Variable.GetName());
				const bool bRebaseVariable = CacheLayout.RebaseVariableNames.Contains(SourceVariable.Variable.GetName());

				if (DestVariableLayout && (bInterpVariable || bRebaseVariable))
				{
					if (SourceVariable.Variable.GetType() == FNiagaraTypeDefinition::GetPositionDef())
					{
						check(SourceVariable.FloatCount == 3);
						if (bInterpVariable)
						{
							const int32 InterpVarIndex = InterpCurrAttributes.IndexOfByKey(SourceVariable.Variable);
							if (InterpVarIndex != INDEX_NONE)
							{
								const int32 PreviousDataSetVariableIndex = CompiledData.Variables.IndexOfByKey(InterpPrevAttributes[InterpVarIndex]);
								check(PreviousDataSetVariableIndex != INDEX_NONE);

								const FNiagaraSimCacheVariable* PrevSourceVariable = CacheLayout.FindCacheVariable(InterpPrevAttributes[InterpVarIndex]);
								check(PrevSourceVariable != nullptr);

								CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(
									SourceVariable.FloatOffset, PrevSourceVariable->FloatOffset,
									IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), IntCastChecked<uint16>(CompiledData.VariableLayouts[PreviousDataSetVariableIndex].GetFloatComponentStart()),
									bRebaseVariable ? &FNiagaraSimCacheHelper::InterpPositions<true, true> : &FNiagaraSimCacheHelper::InterpPositions<false, true>
									
								);
							}
							else
							{
								CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(SourceVariable.FloatOffset, IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), bRebaseVariable ? &FNiagaraSimCacheHelper::InterpPositions<true, false> : &FNiagaraSimCacheHelper::InterpPositions<false, false>);
							}
						}
						else if ( CacheLayout.bAllowVelocityExtrapolation )
						{
							CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(SourceVariable.FloatOffset, IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), bRebaseVariable ? &FNiagaraSimCacheHelper::ExtrapolatePositions<true> : &FNiagaraSimCacheHelper::ExtrapolatePositions<false>);
						}
						else
						{
							CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(SourceVariable.FloatOffset, IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), &FNiagaraSimCacheHelper::RebasePositions);
						}
						DestVariableLayout = nullptr;
					}
					else if (SourceVariable.Variable.GetType() == FNiagaraTypeDefinition::GetQuatDef())
					{
						check(SourceVariable.FloatCount == 4);
						if (bInterpVariable)
						{
							const int32 InterpVarIndex = InterpCurrAttributes.IndexOfByKey(SourceVariable.Variable);
							if (InterpVarIndex != INDEX_NONE)
							{
								const int32 PreviousDataSetVariableIndex = CompiledData.Variables.IndexOfByKey(InterpPrevAttributes[InterpVarIndex]);
								check(PreviousDataSetVariableIndex != INDEX_NONE);

								const FNiagaraSimCacheVariable* PrevSourceVariable = CacheLayout.FindCacheVariable(InterpPrevAttributes[InterpVarIndex]);
								check(PrevSourceVariable != nullptr);

								CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(
									SourceVariable.FloatOffset, PrevSourceVariable->FloatOffset,
									IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), IntCastChecked<uint16>(CompiledData.VariableLayouts[PreviousDataSetVariableIndex].GetFloatComponentStart()),
									bRebaseVariable ? &FNiagaraSimCacheHelper::InterpQuaternions<true, true> : &FNiagaraSimCacheHelper::InterpQuaternions<false, true>
									
								);
							}
							else
							{
								CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(SourceVariable.FloatOffset, IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), bInterpVariable  ? &FNiagaraSimCacheHelper::InterpQuaternions<true, false> : &FNiagaraSimCacheHelper::InterpQuaternions<false, false>);
							}
						}
						else
						{
							CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(SourceVariable.FloatOffset, IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), &FNiagaraSimCacheHelper::RebaseQuaternions);
						}
						DestVariableLayout = nullptr;
					}
					else if (SourceVariable.Variable.GetType() == FNiagaraTypeDefinition::GetMatrix4Def())
					{
						if (bRebaseVariable)
						{
							check(SourceVariable.FloatCount == 16);
							CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(SourceVariable.FloatOffset, IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), &FNiagaraSimCacheHelper::RebaseMatrices);
							DestVariableLayout = nullptr;
						}
					}
					else
					{
						if (SourceVariable.Variable.GetType() == FNiagaraTypeDefinition::GetFloatDef())
						{
							CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(SourceVariable.FloatOffset, IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), &FNiagaraSimCacheHelper::InterpFloats<1>);
							DestVariableLayout = nullptr;
						}
						else if (SourceVariable.Variable.GetType() == FNiagaraTypeDefinition::GetVec2Def())
						{
							CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(SourceVariable.FloatOffset, IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), &FNiagaraSimCacheHelper::InterpFloats<2>);
							DestVariableLayout = nullptr;
						}
						else if (SourceVariable.Variable.GetType() == FNiagaraTypeDefinition::GetVec3Def())
						{
							CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(SourceVariable.FloatOffset, IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), &FNiagaraSimCacheHelper::InterpFloats<3>);
							DestVariableLayout = nullptr;
						}
						else if (SourceVariable.Variable.GetType() == FNiagaraTypeDefinition::GetVec4Def() || SourceVariable.Variable.GetType() == FNiagaraTypeDefinition::GetColorDef())
						{
							CacheBufferReadInfo.VariableCopyMappingsToDataBuffer.Emplace(SourceVariable.FloatOffset, IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart()), &FNiagaraSimCacheHelper::InterpFloats<4>);
							DestVariableLayout = nullptr;
						}
					}
				}
			}

			for (int32 i = 0; i < SourceVariable.FloatCount; ++i)
			{
				CacheBufferReadInfo.ComponentMappingsToDataBuffer[FloatOffset++] = DestVariableLayout ? IntCastChecked<uint16>(DestVariableLayout->GetFloatComponentStart() + i) : uint16(INDEX_NONE);
			}

			for (int32 i = 0; i < SourceVariable.HalfCount; ++i)
			{
				CacheBufferReadInfo.ComponentMappingsToDataBuffer[HalfOffset++] = DestVariableLayout ? IntCastChecked<uint16>(DestVariableLayout->GetHalfComponentStart() + i) : uint16(INDEX_NONE);
			}

			for (int32 i = 0; i < SourceVariable.Int32Count; ++i)
			{
				CacheBufferReadInfo.ComponentMappingsToDataBuffer[Int32Offset++] = DestVariableLayout ? IntCastChecked<uint16>(DestVariableLayout->GetInt32ComponentStart() + i) : uint16(INDEX_NONE);
			}
		}

		return true;
	}

	static void CheckedMemcpy(TConstArrayView<uint8> DstArray, uint8* Dst, TConstArrayView<uint8> SrcArray, const uint8* Src, uint32 Size)
	{
		checkf(Src >= SrcArray.GetData() && Src + Size <= SrcArray.GetData() + SrcArray.Num(), TEXT("Source %p-%p is out of bounds, start %p end %p"), Src, Src + Size, SrcArray.GetData(), SrcArray.GetData() + SrcArray.Num());
		checkf(Dst >= DstArray.GetData() && Dst + Size <= DstArray.GetData() + DstArray.Num(), TEXT("Dest %p-%p is out of bounds, start %p end %p"), Dst, Dst + Size, DstArray.GetData(), DstArray.GetData() + DstArray.Num());
		FMemory::Memcpy(Dst, Src, Size);
	}

	void WriteDataBuffer(const FNiagaraDataBuffer& DataBuffer, const FNiagaraSimCacheDataBuffersLayout& CacheLayout, FNiagaraSimCacheDataBuffers& CacheBuffer, int32 FirstInstance, int32 NumInstances)
	{
		if ( NumInstances == 0 )
		{
			return;
		}

		CacheBuffer.NumInstances = NumInstances;
		CacheBuffer.IDToIndexTableElements = DataBuffer.GetIDTable().Num();
		CacheBuffer.SetupForWrite(CacheLayout);

		int32 iComponent = 0;

		// Copy Float
		for ( uint32 i=0; i < CacheLayout.FloatCount; ++i )
		{
			const uint32 Component = CacheLayout.CacheBufferWriteInfo.ComponentMappingsFromDataBuffer[iComponent++];
			const uint8* Source = DataBuffer.GetComponentPtrFloat(Component) + (FirstInstance * sizeof(float));
			uint8* Dest = CacheBuffer.FloatData.GetData() + (i * NumInstances * sizeof(float));
			CheckedMemcpy(CacheBuffer.FloatData, Dest, DataBuffer.GetFloatBuffer(), Source, sizeof(float) * NumInstances);
		}
		
		// Copy Half
		for (uint32 i = 0; i < CacheLayout.HalfCount; ++i)
		{
			const uint32 Component = CacheLayout.CacheBufferWriteInfo.ComponentMappingsFromDataBuffer[iComponent++];
			const uint8* Source = DataBuffer.GetComponentPtrHalf(Component) + (FirstInstance * sizeof(FFloat16));
			uint8* Dest = CacheBuffer.HalfData.GetData() + (i * NumInstances * sizeof(FFloat16));
			CheckedMemcpy(CacheBuffer.HalfData, Dest, DataBuffer.GetHalfBuffer(), Source, sizeof(FFloat16) * NumInstances);
		}

		// Copy Int32
		for (uint32 i = 0; i < CacheLayout.Int32Count; ++i)
		{
			const uint32 Component = CacheLayout.CacheBufferWriteInfo.ComponentMappingsFromDataBuffer[iComponent++];
			const uint8* Source = DataBuffer.GetComponentPtrInt32(Component) + (FirstInstance * sizeof(int32));
			uint8* Dest = CacheBuffer.Int32Data.GetData() + (i * NumInstances * sizeof(int32));
			CheckedMemcpy(CacheBuffer.Int32Data, Dest, DataBuffer.GetInt32Buffer(), Source, sizeof(int32) * NumInstances);
		}

		// Copy ID to Index Table
		if (CacheBuffer.IDToIndexTable.Num() > 0)
		{
			check(CacheBuffer.IDToIndexTable.Num() == DataBuffer.GetIDTable().Num());
			FMemory::Memcpy(CacheBuffer.IDToIndexTable.GetData(), DataBuffer.GetIDTable().GetData(), CacheBuffer.IDToIndexTable.Num() * sizeof(int32));
		}
		CacheBuffer.IDAcquireTag = DataBuffer.GetIDAcquireTag();

		// Generate a interp mapping (if we have enabled it)
		if (CacheLayout.bAllowInterpolation)
		{
			check(NumInstances == DataBuffer.GetNumInstances());
			const uint8* UniqueIDs = DataBuffer.GetComponentPtrInt32(CacheLayout.CacheBufferWriteInfo.ComponentUniqueID);
			FMemory::Memcpy(CacheBuffer.InterpMapping.GetData(), UniqueIDs, NumInstances * sizeof(int32));
		}
	}

	void WriteDataBufferGPU(FNiagaraEmitterInstance& EmitterInstance, const FNiagaraDataBuffer& DataBuffer, const FNiagaraSimCacheDataBuffersLayout& CacheLayout, FNiagaraSimCacheDataBuffers& CacheBuffer)
	{
		//-TODO: Make async
		TSharedRef<FNiagaraDataSetReadback> ReadbackRequest = MakeShared<FNiagaraDataSetReadback>();
		ReadbackRequest->ImmediateReadback(&EmitterInstance);
		if ( FNiagaraDataBuffer* CurrentData = ReadbackRequest->GetDataSet().GetCurrentData() )
		{
			WriteDataBuffer(*CurrentData, CacheLayout, CacheBuffer, 0, CurrentData->GetNumInstances());
		}
	}

	static void ReadFloatBuffers(uint32 NumInstances, int32& iComponent, uint32 FloatCount, TConstArrayView<uint16> ComponentMappingsToDataBuffer, const FNiagaraSimCacheDataBuffers& CacheBuffer, TArrayView<uint8> DestBuffer, uint32 DestStride)
	{
		const uint32 ComponentStride = CacheBuffer.NumInstances;
		for (uint32 i = 0; i < FloatCount; ++i)
		{
			const uint32 Component = ComponentMappingsToDataBuffer[iComponent++];
			if (Component == InvalidComponent)
			{
				continue;
			}
			const uint8* SourceFloats = CacheBuffer.FloatData.GetData() + (i * ComponentStride * sizeof(float));
			uint8* DestFloats = DestBuffer.GetData() + (DestStride * Component);
			CheckedMemcpy(DestBuffer, DestFloats, CacheBuffer.FloatData, SourceFloats, sizeof(float) * NumInstances);
		}
	}

	static void ReadHalfBuffers(uint32 NumInstances, int32& iComponent, uint32 HalfCount, TConstArrayView<uint16> ComponentMappingsToDataBuffer, const FNiagaraSimCacheDataBuffers& CacheBuffer, TArrayView<uint8> DestBuffer, uint32 DestStride)
	{
		const uint32 ComponentStride = CacheBuffer.NumInstances;
		for (uint32 i = 0; i < HalfCount; ++i)
		{
			const uint32 Component = ComponentMappingsToDataBuffer[iComponent++];
			if (Component == InvalidComponent)
			{
				continue;
			}
			const uint8* SourceHalfs = CacheBuffer.HalfData.GetData() + (i * ComponentStride * sizeof(FFloat16));
			uint8* DestHalfs = DestBuffer.GetData() + (DestStride * Component);
			CheckedMemcpy(DestBuffer, DestHalfs, CacheBuffer.HalfData, SourceHalfs, sizeof(FFloat16) * NumInstances);
		}
	}

	static void ReadInt32Buffers(uint32 NumInstances, int32& iComponent, uint32 Int32Count, TConstArrayView<uint16> ComponentMappingsToDataBuffer, const FNiagaraSimCacheDataBuffers& CacheBuffer, TArrayView<uint8> DestBuffer, uint32 DestStride)
	{
		const uint32 ComponentStride = CacheBuffer.NumInstances;
		for (uint32 i = 0; i < Int32Count; ++i)
		{
			const uint32 Component = ComponentMappingsToDataBuffer[iComponent++];
			if (Component == InvalidComponent)
			{
				continue;
			}
			const uint8* SourceInt32s = CacheBuffer.Int32Data.GetData() + (i * ComponentStride * sizeof(int32));
			uint8* DestInt32s = DestBuffer.GetData() + (DestStride * Component);
			CheckedMemcpy(DestBuffer, DestInt32s, CacheBuffer.Int32Data, SourceInt32s, sizeof(int32) * NumInstances);
		}
	}

	static void ReadCustomBuffers(uint32 NumInstances, float FrameFraction, float FrameDeltaSeconds, float SimDeltaSeconds, const FTransform& RebaseATransform, const FTransform& RebaseBTransform, uint16 ComponentVelocity, const TConstArrayView<FNiagaraSimCacheDataBuffersLayout::FVariableCopyMapping>& VariableCopyMappingsToDataBuffer, const FNiagaraSimCacheDataBuffers& CacheBufferA, const FNiagaraSimCacheDataBuffers& CacheBufferB, uint8* DestBuffer, uint32 DestStride)
	{
		if (VariableCopyMappingsToDataBuffer.Num() == 0)
		{
			return;
		}

		FNiagaraSimCacheDataBuffersLayout::FVariableCopyContext VariableCopyDataContext;
		VariableCopyDataContext.FrameFraction		= FrameFraction;
		VariableCopyDataContext.FrameDeltaSeconds	= FrameDeltaSeconds;
		VariableCopyDataContext.SimDeltaSeconds		= SimDeltaSeconds;
		VariableCopyDataContext.PrevFrameFraction	= 1.0f - FMath::Clamp(SimDeltaSeconds / (FrameDeltaSeconds * (1.0f + FrameFraction)), 0.0f, 1.0f);		//-TODO: We are assuming both frames have the same DT here
		VariableCopyDataContext.NumInstances		= NumInstances;
		VariableCopyDataContext.RebaseATransform	= RebaseATransform;
		VariableCopyDataContext.RebaseBTransform	= RebaseBTransform;
		VariableCopyDataContext.InterpMappings		= CacheBufferA.InterpMapping;
		VariableCopyDataContext.DestStride			= DestStride;
		VariableCopyDataContext.SourceAStride		= CacheBufferA.NumInstances * sizeof(float);
		VariableCopyDataContext.SourceBStride		= CacheBufferB.NumInstances * sizeof(float);
		if (ComponentVelocity != static_cast<uint16>(INDEX_NONE))
		{
			VariableCopyDataContext.Velocity = CacheBufferA.FloatData.GetData() + (ComponentVelocity * VariableCopyDataContext.SourceAStride);
		}
		else
		{
			VariableCopyDataContext.Velocity = nullptr;
		}

		for (const FNiagaraSimCacheDataBuffersLayout::FVariableCopyMapping& VariableCopyMapping : VariableCopyMappingsToDataBuffer)
		{
			VariableCopyDataContext.DestCurr	= DestBuffer + (IntCastChecked<uint32>(VariableCopyMapping.CurrComponentTo) * DestStride);
			VariableCopyDataContext.DestPrev	= DestBuffer + (IntCastChecked<uint32>(VariableCopyMapping.PrevComponentTo) * DestStride);
			VariableCopyDataContext.SourceACurr	= CacheBufferA.FloatData.GetData() + (VariableCopyMapping.CurrComponentFrom * VariableCopyDataContext.SourceAStride);
			VariableCopyDataContext.SourceAPrev = CacheBufferA.FloatData.GetData() + (VariableCopyMapping.PrevComponentFrom * VariableCopyDataContext.SourceAStride);
			VariableCopyDataContext.SourceBCurr	= CacheBufferB.FloatData.GetData() + (VariableCopyMapping.CurrComponentFrom * VariableCopyDataContext.SourceBStride);
			VariableCopyMapping.CopyFunc(VariableCopyDataContext);
		}
	}

	void ReadDataBuffer(float FrameFraction, float FrameDeltaSeconds, float SimDeltaSeconds, const FTransform& RebaseATransform, const FTransform& RebaseBTransform, const FNiagaraSimCacheDataBuffersLayout& CacheLayout, const FNiagaraSimCacheDataBuffers& CacheBufferA, const FNiagaraSimCacheDataBuffers& CacheBufferB, FNiagaraDataSet& DataSet)
	{
		const uint32 NumInstances = FMath::Min(CacheBufferA.NumInstances, DataSet.GetMaxAllocationCount());

		FNiagaraDataBuffer& DataBuffer = DataSet.BeginSimulate();
		DataBuffer.Allocate(NumInstances);
		DataBuffer.SetNumInstances(NumInstances);
		if (CacheBufferA.NumInstances > 0 )
		{
			int32 iComponent = 0;
			ReadFloatBuffers(NumInstances, iComponent, CacheLayout.FloatCount, CacheLayout.CacheBufferReadInfo.ComponentMappingsToDataBuffer, CacheBufferA, MakeArrayView(DataBuffer.GetComponentPtrFloat(0), DataBuffer.GetFloatBuffer().Num()), DataBuffer.GetFloatStride());
			ReadHalfBuffers(NumInstances, iComponent, CacheLayout.HalfCount, CacheLayout.CacheBufferReadInfo.ComponentMappingsToDataBuffer, CacheBufferA, MakeArrayView(DataBuffer.GetComponentPtrHalf(0), DataBuffer.GetHalfBuffer().Num()), DataBuffer.GetHalfStride());
			ReadInt32Buffers(NumInstances, iComponent, CacheLayout.Int32Count, CacheLayout.CacheBufferReadInfo.ComponentMappingsToDataBuffer, CacheBufferA, MakeArrayView(DataBuffer.GetComponentPtrInt32(0), DataBuffer.GetInt32Buffer().Num()), DataBuffer.GetInt32Stride());
			ReadCustomBuffers(NumInstances, FrameFraction, FrameDeltaSeconds, SimDeltaSeconds, RebaseATransform, RebaseBTransform, CacheLayout.ComponentVelocity, CacheLayout.CacheBufferReadInfo.VariableCopyMappingsToDataBuffer, CacheBufferA, CacheBufferB, DataBuffer.GetComponentPtrFloat(0), DataBuffer.GetFloatStride());
		}

		//-TODO:DestinationDataBuffer.SetIDTable(CacheBufferA.IDToIndexTable);
		DataBuffer.SetIDAcquireTag(CacheBufferA.IDAcquireTag);

		DataSet.EndSimulate();
	}

	void ReadDataBufferGPU(float InFrameFraction, float InFrameDeltaSeconds, float InSimDeltaSeconds, const FTransform& InRebaseATransform, const FTransform& InRebaseBTransform, FNiagaraEmitterInstance& EmitterInstance, const FNiagaraSimCacheDataBuffersLayout& InCacheLayout, const FNiagaraSimCacheDataBuffers& InCacheBufferA, const FNiagaraSimCacheDataBuffers& InCacheBufferB, FNiagaraDataSet& InDataSet, std::atomic<int32>& InPendingCommandsCounter)
	{
		if (EmitterInstance.IsDisabled())
		{
			return;
		}

		++InPendingCommandsCounter;

		check(EmitterInstance.GetGPUContext());

		//-OPT: Rather than a lazy update here we could do it on initial read
		if (InCacheLayout.bNeedsCacheBufferReadInfoUpdateForRT)
		{
			InCacheLayout.bNeedsCacheBufferReadInfoUpdateForRT = false;
			ENQUEUE_RENDER_COMMAND(NiagaraSimCacheGpuUpdateReadInfo)(
				[CacheLayout=&InCacheLayout, NewReadInfo=InCacheLayout.CacheBufferReadInfo](FRHICommandListImmediate&)
				{
					CacheLayout->CacheBufferReadInfo_RT = NewReadInfo;
				}
			);
		}

		FNiagaraGpuComputeDispatchInterface* DispathInterface = EmitterInstance.GetParentSystemInstance()->GetComputeDispatchInterface();
		FNiagaraComputeExecutionContext* GPUExecContext = EmitterInstance.GetGPUContext();
		GPUExecContext->ParticleCountReadFence.fetch_add(1, std::memory_order_relaxed);
		GPUExecContext->bResetPending_GT = false;

		ENQUEUE_RENDER_COMMAND(NiagaraSimCacheGpuReadFrame)(
			[DispathInterface, GPUExecContext, ParticleCountFence=GPUExecContext->ParticleCountReadFence.load(std::memory_order_relaxed), FrameFraction = InFrameFraction, FrameDeltaSeconds = InFrameDeltaSeconds, SimDeltaSeconds = InSimDeltaSeconds, RebaseATransform = InRebaseATransform, RebaseBTransform = InRebaseBTransform, CacheLayout = &InCacheLayout, CacheBufferA = &InCacheBufferA, CacheBufferB = &InCacheBufferB, DataSet = &InDataSet, PendingCommandsCounter = &InPendingCommandsCounter](FRHICommandListImmediate& RHICmdList)
			{
				const uint32 NumInstances = FMath::Min(CacheBufferA->NumInstances, DataSet->GetMaxAllocationCount());

				// Set Instance Count
				FNiagaraDataBuffer& DataBuffer = DataSet->GetCurrentDataChecked();
				{
					FNiagaraGPUInstanceCountManager& CountManager = DispathInterface->GetGPUInstanceCounterManager();
					if (GPUExecContext->CountOffset_RT == INDEX_NONE)
					{
						GPUExecContext->CountOffset_RT = CountManager.AcquireOrAllocateEntry(RHICmdList);
					}

					DataBuffer.SetGPUInstanceCountBufferOffset(GPUExecContext->CountOffset_RT);

					GPUExecContext->SetCurrentNumInstances(NumInstances);
					GPUExecContext->ParticleCountWriteFence.store(ParticleCountFence, std::memory_order_relaxed);

					const FRWBuffer& CountBuffer = CountManager.GetInstanceCountBuffer();
					const TPair<uint32, int32> DataToSet(GPUExecContext->CountOffset_RT, NumInstances);
					RHICmdList.Transition(FRHITransitionInfo(CountBuffer.UAV, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState, ERHIAccess::UAVCompute));
					NiagaraClearCounts::ClearCountsInt(RHICmdList, CountBuffer.UAV, MakeArrayView(&DataToSet, 1));
					RHICmdList.Transition(FRHITransitionInfo(CountBuffer.UAV, ERHIAccess::UAVCompute, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState));
				}

				// Copy instance counts
				DataBuffer.AllocateGPU(RHICmdList, NumInstances, DispathInterface->GetFeatureLevel(), TEXT("NiagaraSimCache"));
				DataBuffer.SetNumInstances(NumInstances);
				DataBuffer.SetGPUDataReadyStage(ENiagaraGpuComputeTickStage::PreInitViews);
				GPUExecContext->SetDataToRender(&DataBuffer);

				if (NumInstances > 0)
				{
					int32 iComponent = 0;

					// Copy Float
					if ( CacheLayout->FloatCount > 0 )
					{
						FRWBuffer& RWBuffer = DataBuffer.GetGPUBufferFloat();
						uint8* RWBufferMemory = reinterpret_cast<uint8*>(RHICmdList.LockBuffer(RWBuffer.Buffer, 0, RWBuffer.NumBytes, RLM_WriteOnly));
						ReadFloatBuffers(NumInstances, iComponent, CacheLayout->FloatCount, CacheLayout->CacheBufferReadInfo_RT.ComponentMappingsToDataBuffer, *CacheBufferA, MakeArrayView(RWBufferMemory, RWBuffer.NumBytes), DataBuffer.GetFloatStride());
						ReadCustomBuffers(NumInstances, FrameFraction, FrameDeltaSeconds, SimDeltaSeconds, RebaseATransform, RebaseBTransform, CacheLayout->ComponentVelocity, CacheLayout->CacheBufferReadInfo_RT.VariableCopyMappingsToDataBuffer, *CacheBufferA, *CacheBufferB, RWBufferMemory, DataBuffer.GetFloatStride());
						RHICmdList.UnlockBuffer(RWBuffer.Buffer);
					}

					// Copy Half
					if (CacheLayout->HalfCount > 0)
					{
						FRWBuffer& RWBuffer = DataBuffer.GetGPUBufferHalf();
						uint8* RWBufferMemory = reinterpret_cast<uint8*>(RHICmdList.LockBuffer(RWBuffer.Buffer, 0, RWBuffer.NumBytes, RLM_WriteOnly));
						ReadHalfBuffers(NumInstances, iComponent, CacheLayout->HalfCount, CacheLayout->CacheBufferReadInfo_RT.ComponentMappingsToDataBuffer, *CacheBufferA, MakeArrayView(RWBufferMemory, RWBuffer.NumBytes), DataBuffer.GetHalfStride());
						RHICmdList.UnlockBuffer(RWBuffer.Buffer);
					}

					// Copy Int32
					if (CacheLayout->Int32Count > 0)
					{
						FRWBuffer& RWBuffer = DataBuffer.GetGPUBufferInt();
						uint8* RWBufferMemory = reinterpret_cast<uint8*>(RHICmdList.LockBuffer(RWBuffer.Buffer, 0, RWBuffer.NumBytes, RLM_WriteOnly));
						ReadInt32Buffers(NumInstances, iComponent, CacheLayout->Int32Count, CacheLayout->CacheBufferReadInfo_RT.ComponentMappingsToDataBuffer, *CacheBufferA, MakeArrayView(RWBufferMemory, RWBuffer.NumBytes), DataBuffer.GetInt32Stride());
						RHICmdList.UnlockBuffer(RWBuffer.Buffer);
					}
				}

				//-TODO:DestinationDataBuffer.SetIDTable(CacheBuffer.IDToIndexTable);
				DataBuffer.SetIDAcquireTag(CacheBufferA->IDAcquireTag);

				// Ensure we decrement our counter so the GameThread knows the state of things
				--(*PendingCommandsCounter);
			}
		);
	}

	static bool CanInterpolateVariable(bool bInterpolateAllTypes, const FNiagaraVariableBase& Variable)
	{
		if (bInterpolateAllTypes)
		{
			if (
				Variable.GetType() == FNiagaraTypeDefinition::GetFloatDef() ||
				Variable.GetType() == FNiagaraTypeDefinition::GetVec2Def() ||
				Variable.GetType() == FNiagaraTypeDefinition::GetVec3Def() ||
				Variable.GetType() == FNiagaraTypeDefinition::GetVec4Def() ||
				Variable.GetType() == FNiagaraTypeDefinition::GetColorDef())
			{
				return true;
			}
		}

		return
			(Variable.GetType() == FNiagaraTypeDefinition::GetQuatDef()) ||
			(Variable.GetType() == FNiagaraTypeDefinition::GetPositionDef());
	}

	static bool CanRebaseVariable(const FNiagaraVariableBase& Variable)
	{
		return	
			(Variable.GetType() == FNiagaraTypeDefinition::GetQuatDef()) ||
			(Variable.GetType() == FNiagaraTypeDefinition::GetMatrix4Def()) ||
			(Variable.GetType() == FNiagaraTypeDefinition::GetPositionDef());
	}

	template<bool bWithRebase>
	static void ExtrapolatePositions(const FNiagaraSimCacheDataBuffersLayout::FVariableCopyContext& CopyDataContext)
	{
		FSoAVec3Reader SrcPositions(CopyDataContext.SourceACurr, CopyDataContext.SourceAStride);
		FSoAVec3Reader SrcVelocities(CopyDataContext.Velocity, CopyDataContext.SourceAStride);
		FSoAVec3Writer DstPositions(CopyDataContext.DestCurr, CopyDataContext.DestStride);

		for (uint32 i = 0; i < CopyDataContext.NumInstances; ++i)
		{
			const FVector3f Position = SrcPositions.Get(i);
			const FVector3f Velocity = SrcVelocities.Get(i);
			const FVector3f ExtrapolatedPosition = Position + (Velocity * CopyDataContext.FrameFraction * CopyDataContext.FrameDeltaSeconds);
			const FVector3f RebasedPosition = bWithRebase ? FVector3f(CopyDataContext.RebaseATransform.TransformPosition(FVector(ExtrapolatedPosition))) : ExtrapolatedPosition;
			DstPositions.Set(i, RebasedPosition);
		}
	}

	template<bool bWithRebase, bool bWithPrevious>
	static void InterpPositions(const FNiagaraSimCacheDataBuffersLayout::FVariableCopyContext& CopyDataContext)
	{
		FSoAVec3Reader SrcAPositions(CopyDataContext.SourceACurr, CopyDataContext.SourceAStride);
		FSoAVec3Reader SrcBPositions(CopyDataContext.SourceBCurr, CopyDataContext.SourceBStride);
		FSoAVec3Writer DstPositions(CopyDataContext.DestCurr, CopyDataContext.DestStride);

		if (bWithPrevious)
		{
			FSoAVec3Reader SrcAPrevPositions(CopyDataContext.SourceAPrev, CopyDataContext.SourceAStride);
			FSoAVec3Writer DstPrevPositions(CopyDataContext.DestPrev, CopyDataContext.DestStride);

			for (uint32 iInstanceA = 0; iInstanceA < CopyDataContext.NumInstances; ++iInstanceA)
			{
				const uint32 iInstanceB = CopyDataContext.InterpMappings[iInstanceA];
				FVector3f CurrPosA = SrcAPositions.Get(iInstanceA);
				FVector3f CurrPosB = iInstanceB == INDEX_NONE ? CurrPosA : SrcBPositions.Get(iInstanceB);
				FVector3f PrevPosA = SrcAPrevPositions.Get(iInstanceA);
				if ( bWithRebase )
				{
					CurrPosA = FVector3f(CopyDataContext.RebaseATransform.TransformPosition(FVector(CurrPosA)));
					CurrPosB = FVector3f(CopyDataContext.RebaseBTransform.TransformPosition(FVector(CurrPosB)));
					PrevPosA = FVector3f(CopyDataContext.RebaseATransform.TransformPosition(FVector(PrevPosA)));
				}


				const FVector3f CurrPos = FMath::Lerp(CurrPosA, CurrPosB, CopyDataContext.FrameFraction);
				const FVector3f PrevPos = FMath::Lerp(PrevPosA, CurrPos, CopyDataContext.PrevFrameFraction);
				DstPositions.Set(iInstanceA, CurrPos);
				DstPrevPositions.Set(iInstanceA, PrevPos);
			}
		}
		else
		{
			for (uint32 iInstanceA = 0; iInstanceA < CopyDataContext.NumInstances; ++iInstanceA)
			{
				const uint32 iInstanceB = CopyDataContext.InterpMappings[iInstanceA];
				FVector3f CurrPosA = SrcAPositions.Get(iInstanceA);
				FVector3f CurrPosB = iInstanceB == INDEX_NONE ? CurrPosA : SrcBPositions.Get(iInstanceB);
				if (bWithRebase)
				{
					CurrPosA = FVector3f(CopyDataContext.RebaseATransform.TransformPosition(FVector(CurrPosA)));
					CurrPosB = FVector3f(CopyDataContext.RebaseBTransform.TransformPosition(FVector(CurrPosB)));
				}
				const FVector3f CurrPos = FMath::Lerp(CurrPosA, CurrPosB, CopyDataContext.FrameFraction);
				DstPositions.Set(iInstanceA, CurrPos);
			}
		}
	}

	static void RebasePositions(const FNiagaraSimCacheDataBuffersLayout::FVariableCopyContext& CopyDataContext)
	{
		FSoAVec3Reader SrcPositions(CopyDataContext.SourceACurr, CopyDataContext.SourceAStride);
		FSoAVec3Writer DstPositions(CopyDataContext.DestCurr, CopyDataContext.DestStride);

		for (uint32 i = 0; i < CopyDataContext.NumInstances; ++i)
		{
			const FVector3f CachePosition = SrcPositions.Get(i);
			const FVector3f RebasedPosition = FVector3f(CopyDataContext.RebaseATransform.TransformPosition(FVector(CachePosition)));
			DstPositions.Set(i, RebasedPosition);
		}
	}

	////template<bool bWithRebase>
	////static void CopyVelocities(const FNiagaraSimCacheDataBuffersLayout::FVariableCopyContext& CopyDataContext)
	////{
	////	float* DstVelocity[3] = { reinterpret_cast<float*>(CopyDataContext.Dest + (CopyDataContext.DestStride * 0)), reinterpret_cast<float*>(CopyDataContext.Dest + (CopyDataContext.DestStride * 1)), reinterpret_cast<float*>(CopyDataContext.Dest + (CopyDataContext.DestStride * 2)) };
	////	const float* SrcVelocity[3] = { reinterpret_cast<const float*>(CopyDataContext.SourceAComponent + (CopyDataContext.SourceAStride * 0)), reinterpret_cast<const float*>(CopyDataContext.SourceAComponent + (CopyDataContext.SourceAStride * 1)), reinterpret_cast<const float*>(CopyDataContext.SourceAComponent + (CopyDataContext.SourceAStride * 2)) };

	////	for (uint32 i = 0; i < CopyDataContext.NumInstances; ++i)
	////	{
	////		const FVector3f CacheVelocity(SrcVelocity[0][i], SrcVelocity[1][i], SrcVelocity[2][i]);
	////		const FVector3f RebasedVelocity = bWithRebase ? FVector3f(CopyDataContext.RebaseTransform.TransformVector(FVector(CacheVelocity))) : CacheVelocity;
	////		DstVelocity[0][i] = RebasedVelocity.X * CopyDataContext.SimDeltaSeconds;
	////		DstVelocity[1][i] = RebasedVelocity.Y * CopyDataContext.SimDeltaSeconds;
	////		DstVelocity[2][i] = RebasedVelocity.Z * CopyDataContext.SimDeltaSeconds;
	////	}
	////}

	template<bool bWithRebase, bool bWithPrevious>
	static void InterpQuaternions(const FNiagaraSimCacheDataBuffersLayout::FVariableCopyContext& CopyDataContext)
	{
		FSoAQuatReader SrcAQuats(CopyDataContext.SourceACurr, CopyDataContext.SourceAStride);
		FSoAQuatReader SrcBQuats(CopyDataContext.SourceBCurr, CopyDataContext.SourceBStride);
		FSoAQuatWriter DstQuats(CopyDataContext.DestCurr, CopyDataContext.DestStride);

		const FQuat4f RebaseAQuat(CopyDataContext.RebaseATransform.GetRotation());
		const FQuat4f RebaseBQuat(CopyDataContext.RebaseBTransform.GetRotation());
		if (bWithPrevious)
		{
			FSoAQuatReader SrcAPrevQuats(CopyDataContext.SourceAPrev, CopyDataContext.SourceAStride);
			FSoAQuatWriter DstPrevQuats(CopyDataContext.DestPrev, CopyDataContext.DestStride);

			for (uint32 iInstanceA = 0; iInstanceA < CopyDataContext.NumInstances; ++iInstanceA)
			{
				const uint32 iInstanceB = CopyDataContext.InterpMappings[iInstanceA];
				FQuat4f CurrQuatA = SrcAQuats.Get(iInstanceA);
				FQuat4f CurrQuatB = iInstanceB == INDEX_NONE ? CurrQuatA : SrcBQuats.Get(iInstanceB);
				FQuat4f PrevQuatA = SrcAPrevQuats.Get(iInstanceA);
				if (bWithRebase)
				{
					CurrQuatA = CurrQuatA * RebaseAQuat;
					CurrQuatB = CurrQuatB * RebaseBQuat;
					PrevQuatA = PrevQuatA * RebaseAQuat;
				}
				const FQuat4f CurrQuat  = FQuat4f::Slerp(CurrQuatA, CurrQuatB, CopyDataContext.FrameFraction);
				const FQuat4f PrevQuat = FQuat4f::Slerp(PrevQuatA, CurrQuat, CopyDataContext.PrevFrameFraction);
				DstQuats.Set(iInstanceA, CurrQuat);
				DstPrevQuats.Set(iInstanceA, PrevQuat);
			}
		}
		else
		{
			for (uint32 iInstanceA=0; iInstanceA < CopyDataContext.NumInstances; ++iInstanceA)
			{
				const uint32 iInstanceB = CopyDataContext.InterpMappings[iInstanceA];
				FQuat4f CurrQuatA = SrcAQuats.Get(iInstanceA);
				FQuat4f CurrQuatB = iInstanceB == INDEX_NONE ? CurrQuatA : SrcBQuats.Get(iInstanceB);
				if (bWithRebase)
				{
					CurrQuatA = CurrQuatA * RebaseAQuat;
					CurrQuatB = CurrQuatB * RebaseBQuat;
				}
				const FQuat4f CurrQuat = FQuat4f::Slerp(CurrQuatA, CurrQuatB, CopyDataContext.FrameFraction);
				DstQuats.Set(iInstanceA, CurrQuat);
			}
		}
	}

	static void RebaseQuaternions(const FNiagaraSimCacheDataBuffersLayout::FVariableCopyContext& CopyDataContext)
	{
		FSoAQuatReader SrcQuats(CopyDataContext.SourceACurr, CopyDataContext.SourceAStride);
		FSoAQuatWriter DstQuats(CopyDataContext.DestCurr, CopyDataContext.DestStride);

		const FQuat4f RebaseQuat(CopyDataContext.RebaseATransform.GetRotation());
		for (uint32 i = 0; i < CopyDataContext.NumInstances; ++i)
		{
			const FQuat4f CacheRotation = SrcQuats.Get(i);
			const FQuat4f RebasedQuat = CacheRotation * RebaseQuat;
			DstQuats.Set(i, RebasedQuat);
		}
	}

	static void RebaseMatrices(const FNiagaraSimCacheDataBuffersLayout::FVariableCopyContext& CopyDataContext)
	{
		float* DstFloats = reinterpret_cast<float*>(CopyDataContext.DestCurr);
		const uint32 DstStride = CopyDataContext.DestStride >> 2;

		const float* SrcFloats = reinterpret_cast<const float*>(CopyDataContext.SourceACurr);
		const uint32 SrcStride = CopyDataContext.SourceAStride >> 2;

		const FMatrix44d RebaseMatrix = CopyDataContext.RebaseATransform.ToMatrixWithScale();
		for (uint32 i = 0; i < CopyDataContext.NumInstances; ++i)
		{
			FMatrix44d CacheMatrix;
			for (int32 j = 0; j < 16; ++j)
			{
				CacheMatrix.M[j >> 2][j & 0x3] = static_cast<double>(SrcFloats[i + (SrcStride * j)]);
			}

			CacheMatrix = CacheMatrix * RebaseMatrix;

			for (int32 j = 0; j < 16; ++j)
			{
				DstFloats[i + (DstStride * j)] = static_cast<float>(CacheMatrix.M[j >> 2][j & 0x3]);
			}
		}
	}

	template<uint32 NumFloats>
	static void InterpFloats(const FNiagaraSimCacheDataBuffersLayout::FVariableCopyContext& CopyDataContext)
	{
		for (uint32 iComponent = 0; iComponent < NumFloats; ++iComponent)
		{
			const float* SrcAFloats = reinterpret_cast<const float*>(CopyDataContext.SourceACurr + (CopyDataContext.SourceAStride * iComponent));
			const float* SrcBFloats = reinterpret_cast<const float*>(CopyDataContext.SourceBCurr + (CopyDataContext.SourceBStride * iComponent));
			float* DstFloats  = reinterpret_cast<float*>(CopyDataContext.DestCurr + (CopyDataContext.DestStride * iComponent));
			for (uint32 iInstanceA=0; iInstanceA < CopyDataContext.NumInstances; ++iInstanceA)
			{
				const uint32 iInstanceB = CopyDataContext.InterpMappings[iInstanceA];
				const float FloatA = SrcAFloats[iInstanceA];
				const float FloatB = iInstanceB == INDEX_NONE ? FloatA : SrcBFloats[iInstanceB];
				DstFloats[iInstanceA] = FMath::Lerp(FloatA, FloatB, CopyDataContext.FrameFraction);
			}
		}
	}

	UNiagaraSystem*						NiagaraSystem = nullptr;
	FNiagaraSystemInstance*				SystemInstance = nullptr;
	FNiagaraSystemSimulationPtr			SystemSimulation = nullptr;
	FNiagaraDataBuffer*					SystemSimulationDataBuffer = nullptr;

	static constexpr uint16				InvalidComponent = INDEX_NONE;
};
