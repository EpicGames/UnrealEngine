// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimCacheJson.h"

#include "NiagaraSimCacheCustomStorageInterface.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

DEFINE_LOG_CATEGORY_STATIC(LogNiagaraSimCacheJson, Log, All);

bool FNiagaraSimCacheJson::DumpToFile(const UNiagaraSimCache& SimCache, const FString& FullPath, EExportType ExportType)
{
	if (ExportType == EExportType::SingleJsonFile)
	{
		TSharedPtr<FJsonObject> JsonObject = ToJson(SimCache);
		if (JsonObject.IsValid())
		{
			UE_LOGF(LogNiagara, Warning, "Writing file to %ls", *FullPath);

			FString OutputString;
			TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
			FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);
			return FFileHelper::SaveStringToFile(OutputString, *FullPath);
		}
		return false;
	}
	else if (ExportType == EExportType::SeparateEachFrame)
	{
		return DumpFramesToFolder(SimCache, FullPath);
	}
	return false;
}

TSharedPtr<FJsonObject> FNiagaraSimCacheJson::ToJson(const UNiagaraSimCache& SimCache)
{
	if (!SimCache.IsCacheValid())
	{
		return nullptr;
	}

	TSharedPtr<FJsonObject> JsonCacheObject = SystemDataToJson(SimCache);

	// Write System Instance
	{
		TArray<TSharedPtr<FJsonValue>> JsonFrames;
		for (int32 iFrame=0; iFrame < SimCache.GetNumFrames(); ++iFrame)
		{
			TSharedPtr<FJsonObject> JsonFrame = EmitterFrameToJson(SimCache, INDEX_NONE, iFrame);
			JsonFrames.Add(TSharedPtr<FJsonValue>(new FJsonValueObject(JsonFrame)));
		}
		JsonCacheObject->SetArrayField(TEXT("SystemInstance"), JsonFrames);
	}

	// Write Emitter Instances
	for ( int32 iEmitter=0; iEmitter < SimCache.GetNumEmitters(); ++iEmitter )
	{
		TArray<TSharedPtr<FJsonValue>> JsonFrames;
		for (int32 iFrame = 0; iFrame < SimCache.GetNumFrames(); ++iFrame)
		{
			TSharedPtr<FJsonObject> JsonFrame = EmitterFrameToJson(SimCache, iEmitter, iFrame);
			JsonFrames.Add(TSharedPtr<FJsonValue>(new FJsonValueObject(JsonFrame)));
		}
		JsonCacheObject->SetArrayField(SimCache.GetEmitterName(iEmitter).ToString(), JsonFrames);
	}

	return JsonCacheObject;
}

TSharedPtr<FJsonObject> FNiagaraSimCacheJson::SystemDataToJson(const UNiagaraSimCache& SimCache)
{
	TSharedPtr<FJsonObject> JsonCacheObject = MakeShared<FJsonObject>();
	JsonCacheObject->SetStringField(TEXT("SystemAsset"), SimCache.GetSystemAsset().ToString());
	JsonCacheObject->SetStringField(TEXT("CacheGuid"), SimCache.GetCacheGuid().ToString());
	JsonCacheObject->SetNumberField(TEXT("StartSeconds"), SimCache.GetStartSeconds());
	JsonCacheObject->SetNumberField(TEXT("DurationSeconds"), SimCache.GetDurationSeconds());
	JsonCacheObject->SetNumberField(TEXT("NumFrames"), SimCache.GetNumFrames());
	JsonCacheObject->SetNumberField(TEXT("NumEmitters"), SimCache.GetNumEmitters());
	return JsonCacheObject;
}

TSharedPtr<FJsonObject> FNiagaraSimCacheJson::EmitterFrameToJson(const UNiagaraSimCache& SimCache, int EmitterIndex, int FrameIndex)
{
	const int NumInstances = SimCache.GetEmitterNumInstances(EmitterIndex, FrameIndex);

	TSharedPtr<FJsonObject> EmitterObject = MakeShared<FJsonObject>();
	EmitterObject->SetNumberField(TEXT("NumInstances"), NumInstances);
	if (NumInstances > 0)
	{
		const FName EmitterName = SimCache.GetEmitterName(EmitterIndex);

		TArray<FNiagaraVariableBase> Attributes;
		SimCache.ForEachEmitterAttribute(EmitterIndex, [&Attributes](const FNiagaraSimCacheVariable& CacheVariable) -> bool { Attributes.Emplace(CacheVariable.Variable); return true; });
		Algo::Sort(Attributes, [](const FNiagaraVariableBase& Lhs, const FNiagaraVariableBase& Rhs) { return FNameLexicalLess()(Lhs.GetName(), Rhs.GetName()); });

		TArray<TSharedPtr<FJsonValue>> AttributesObject;
		for (const FNiagaraVariableBase& Attribute : Attributes)
		{
			TSharedPtr<FJsonObject> AttributeObject = MakeShared<FJsonObject>();
			AttributeObject->SetStringField(TEXT("Name"), Attribute.GetName().ToString());
			AttributeObject->SetStringField(TEXT("Type"), Attribute.GetType().GetName());

			AttributesObject.Add(TSharedPtr<FJsonValue>(new FJsonValueObject(AttributeObject)));

			TArray<float>		Floats;
			TArray<FFloat16>	Halfs;
			TArray<int32>		Ints;
			SimCache.ReadAttribute(Floats, Halfs, Ints, Attribute.GetName(), EmitterName, FrameIndex);

			int32 TypeSize = Attribute.GetType().GetSize();

			// add float components
			if (Floats.Num() > 0)
			{
				TArray<TSharedPtr<FJsonValue>> Values;
				if (TypeSize > sizeof(float))
				{
					int32 ComponentNum = TypeSize / sizeof(float);
					for (int i = 0; i < Floats.Num(); i += ComponentNum)
					{
						TArray<TSharedPtr<FJsonValue>> SubValues;
						for (int Component = i; Floats.IsValidIndex(Component) && Component < i + ComponentNum; Component++)
						{
							SubValues.Emplace(MakeShared<FJsonValueNumber>(Floats[Component]));
						}
						Values.Emplace(MakeShared<FJsonValueArray>(SubValues));
					}
				}
				else
				{
					for ( float v : Floats )
					{
						Values.Emplace(MakeShared<FJsonValueNumber>(v));
					}
				}
				AttributeObject->SetArrayField(TEXT("Floats"), Values);
			}

			// add half components (converted to float)
			if (Halfs.Num() > 0)
			{
				TArray<TSharedPtr<FJsonValue>> Values;
				if (TypeSize > sizeof(uint16))
				{
					int32 ComponentNum = TypeSize / sizeof(uint16);
					for (int i = 0; i < Halfs.Num(); i += ComponentNum)
					{
						TArray<TSharedPtr<FJsonValue>> SubValues;
						for (int Component = i; Halfs.IsValidIndex(Component) && Component < i + ComponentNum; Component++)
						{
							SubValues.Emplace(MakeShared<FJsonValueNumber>(Halfs[Component].GetFloat()));
						}
						Values.Emplace(MakeShared<FJsonValueArray>(SubValues));
					}
				}
				else
				{
					for (FFloat16 v : Halfs)
					{
						Values.Emplace(MakeShared<FJsonValueNumber>(v.GetFloat()));
					}
				}
				AttributeObject->SetArrayField(TEXT("Halfs"), Values);
			}

			// add integer components
			if (Ints.Num() > 0)
			{
				TArray<TSharedPtr<FJsonValue>> Values;
				if (TypeSize > sizeof(int32))
				{
					int32 ComponentNum = TypeSize / sizeof(int32);
					for (int i = 0; i < Ints.Num(); i += ComponentNum)
					{
						TArray<TSharedPtr<FJsonValue>> SubValues;
						for (int Component = i; Ints.IsValidIndex(Component) && Component < i + ComponentNum; Component++)
						{
							SubValues.Emplace(MakeShared<FJsonValueNumber>(Ints[Component]));
						}
						Values.Emplace(MakeShared<FJsonValueArray>(SubValues));
					}
				}
				else
				{
					for (int32 v : Ints)
					{
						Values.Emplace(MakeShared<FJsonValueNumber>(v));
					}
				}
				AttributeObject->SetArrayField(TEXT("Ints"), Values);
			}
		}
		EmitterObject->SetArrayField(TEXT("Attributes"), AttributesObject);
	}
	return EmitterObject;
}

bool FNiagaraSimCacheJson::DumpFramesToFolder(const UNiagaraSimCache& SimCache, const FString& TargetFolder)
{
	IFileManager& FileManager = IFileManager::Get();
	if (!FileManager.DirectoryExists(*TargetFolder))
	{
		UE_LOGF(LogNiagaraSimCacheJson, Warning, "Unable to save to folder %ls", *TargetFolder);
		return false;
	}

	int32 EmitterCount = SimCache.GetNumEmitters();
	FScopedSlowTask SlowTask(SimCache.GetNumFrames(), NSLOCTEXT("SimCacheExport", "SlowTaskLabel","Exporting frames..."));
	SlowTask.MakeDialog(true);

	{
		// basic system data for the whole cache
		TSharedPtr<FJsonObject> SystemJson = SystemDataToJson(SimCache);
		if (!SystemJson.IsValid())
		{
			UE_LOGF(LogNiagaraSimCacheJson, Warning, "Unable to export system data for cache %ls", *SimCache.GetName());
			return false;
		}
		FString SystemFile = FPaths::Combine(TargetFolder, FPaths::MakeValidFileName(SimCache.GetName() + ".json", '_'));
		FString OutputString;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
		FJsonSerializer::Serialize(SystemJson.ToSharedRef(), Writer);
		FFileHelper::SaveStringToFile(OutputString, *SystemFile);
	}
	
	for (int FrameIndex = 0; FrameIndex < SimCache.GetNumFrames(); FrameIndex++)
	{
		SlowTask.EnterProgressFrame();
		if (SlowTask.ShouldCancel())
		{
			UE_LOGF(LogNiagaraSimCacheJson, Warning, "Export cancelled by user");
			return false;
		}
		
		FString FrameFolder = FPaths::Combine(TargetFolder, TEXT("Frame_") + FString::FormatAsNumber(FrameIndex));
		if (!FileManager.MakeDirectory(*FrameFolder))
		{
			return false;
		}
		
		{
			// system data for the frame
			TSharedPtr<FJsonObject> SystemJson = EmitterFrameToJson(SimCache, INDEX_NONE, FrameIndex);
			if (!SystemJson.IsValid())
			{
				UE_LOGF(LogNiagaraSimCacheJson, Warning, "Unable to export system data for cache %ls", *SimCache.GetName());
				return false;
			}
			FString SystemFile = FPaths::Combine(FrameFolder, "_SystemAttributes_.json");

			FString OutputString;
			TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
			FJsonSerializer::Serialize(SystemJson.ToSharedRef(), Writer);
			FFileHelper::SaveStringToFile(OutputString, *SystemFile);
			SlowTask.TickProgress();
		}

		for (int EmitterIndex = 0; EmitterIndex < EmitterCount; EmitterIndex++)
		{
			// emitter data	
			TSharedPtr<FJsonObject> EmitterJson = EmitterFrameToJson(SimCache, EmitterIndex, FrameIndex);
			if (!EmitterJson.IsValid())
			{
				UE_LOGF(LogNiagaraSimCacheJson, Warning, "Unable to export data for emitter %ls", *SimCache.GetEmitterName(EmitterIndex).ToString());
				return false;
			}
			FString EmitterFile = FPaths::Combine(FrameFolder, FPaths::MakeValidFileName(SimCache.GetEmitterName(EmitterIndex).ToString() + ".json", '_'));

			FString OutputString;
			TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
			FJsonSerializer::Serialize(EmitterJson.ToSharedRef(), Writer);
			FFileHelper::SaveStringToFile(OutputString, *EmitterFile);
			SlowTask.TickProgress();
		}

		for (const FNiagaraVariableBase& DataInterface : SimCache.GetStoredDataInterfaces())
		{
			// data interfaces
			if (const UObject* StorageObject = SimCache.GetDataInterfaceStorageObject(DataInterface))
			{
				UClass* Class = DataInterface.GetType().GetClass();
				INiagaraSimCacheCustomStorageInterface* DataInterfaceCDO = Class ? Cast<INiagaraSimCacheCustomStorageInterface>(Class->GetDefaultObject()) : nullptr;
				if (DataInterfaceCDO)
				{
					FString FilenamePrefix = FPaths::MakeValidFileName(DataInterface.GetName().ToString(), '_');
					FilenamePrefix.ReplaceCharInline('.', '-');
					TSharedPtr<FJsonObject> Json = DataInterfaceCDO->SimCacheToJson(StorageObject, FrameIndex, FrameFolder, FilenamePrefix);
					if (Json.IsValid())
					{
						FString OutputString;
						TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
						FJsonSerializer::Serialize(Json.ToSharedRef(), Writer);
						FFileHelper::SaveStringToFile(OutputString, *FPaths::Combine(FrameFolder, FilenamePrefix + ".json"));
					}
				}
			}
			SlowTask.TickProgress();
		}
	}

	return true;
}
