// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimStageExecutionData.h"
#include "NiagaraCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSimStageExecutionData)

#if WITH_EDITORONLY_DATA
bool FNiagaraSimStageExecutionData::Build(TConstArrayView<FSimulationStageMetaData> InSimStageMetaData, TConstArrayView<FNiagaraSimStageExecutionLoopEditorData> EditorExecutionLoops)
{
	ExecutionLoops.Empty();
	SimStageMetaData = InSimStageMetaData;

	// Nothing to do?
	if ( SimStageMetaData.Num() == 0 )
	{
		return true;
	}

	// Create a per stage loop index
	TArray<TOptional<int32>> PerStageLoopIndex;
	PerStageLoopIndex.AddDefaulted(SimStageMetaData.Num());

	bool bLoopsValid = true;
	for ( int32 LoopIndex=0; LoopIndex < EditorExecutionLoops.Num(); ++LoopIndex)
	{
		const FNiagaraSimStageExecutionLoopEditorData& EditorLoopData = EditorExecutionLoops[LoopIndex];
		if ( !EditorLoopData.bEnabled || EditorLoopData.StageNameStart.IsNone() || EditorLoopData.StageNameEnd.IsNone() )
		{
			continue;
		}

		const int32 IndexStart = SimStageMetaData.IndexOfByPredicate([StartName=EditorLoopData.StageNameStart](const FSimulationStageMetaData& Data) { return Data.SimulationStageName == StartName; });
		const int32 IndexEnd = SimStageMetaData.IndexOfByPredicate([EndName=EditorLoopData.StageNameEnd](const FSimulationStageMetaData& Data) { return Data.SimulationStageName == EndName; });
		if ( IndexStart == INDEX_NONE || IndexEnd == INDEX_NONE || IndexEnd < IndexStart )
		{
			UE_LOGF(LogNiagara, Error, "Sim Stage Loop has invalid start / end name, all loops will be ignored");
			bLoopsValid = false;
			break;
		}

		for ( int32 i=IndexStart; i <= IndexEnd; ++i )
		{
			if (PerStageLoopIndex[i].IsSet() )
			{
				UE_LOGF(LogNiagara, Error, "Sim Stage Loop has overlap, all loops will be ignored");
				bLoopsValid = false;
			}
			PerStageLoopIndex[i] = LoopIndex;
		}
	}

	// Invalid for some reason, just ignore all loops and run linear
	if ( !bLoopsValid )
	{
		PerStageLoopIndex.Reset(SimStageMetaData.Num());
		PerStageLoopIndex.AddDefaulted(SimStageMetaData.Num());
	}

	// Generate the loop data
	FNiagaraSimStageExecutionLoopData* CurrentLoop = &ExecutionLoops.AddDefaulted_GetRef();
	CurrentLoop->NumLoops = 1;

	constexpr int32 InvalidLoopIndex = INDEX_NONE;
	int32 CurrentLoopIndex = InvalidLoopIndex;
	for ( int32 i=1; i < SimStageMetaData.Num(); ++i )
	{
		const int32 LoopIndex = PerStageLoopIndex[i].Get(InvalidLoopIndex);
		if (LoopIndex != CurrentLoopIndex)
		{
			// Close current loop
			CurrentLoop->EndStageIndex		= i - 1;

			// Start new  loop
			CurrentLoop						= &ExecutionLoops.AddDefaulted_GetRef();
			CurrentLoop->NumLoopsBinding	= LoopIndex == InvalidLoopIndex ? NAME_None : EditorExecutionLoops[LoopIndex].NumLoopsBindingName;
			CurrentLoop->NumLoops			= LoopIndex == InvalidLoopIndex ? 1 : EditorExecutionLoops[LoopIndex].NumLoops;
			CurrentLoop->StartStageIndex	= i;
			CurrentLoopIndex				= LoopIndex;
		}
	}

	// Close the loop
	CurrentLoop->EndStageIndex = SimStageMetaData.Num() - 1;

	return bLoopsValid;
}
#endif //WITH_EDITORONLY_DATA
