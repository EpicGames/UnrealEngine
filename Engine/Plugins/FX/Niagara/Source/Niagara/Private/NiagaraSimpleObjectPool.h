// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"

/**
 * Very simple object pool that will hold weak references to objects that are returned, these can then be recycled rather than creating new objects.
 * The objects are not strongly referenced and will be lost when garbage collection is run, this is by design.
 */
class FNiagaraSimpleObjectPool
{
public:
	UObject* GetOrCreateObject(UClass* ObjectClass, UObject* Outer, bool& bIsExistingObject)
	{
		check(IsInGameThread());
		check(ObjectClass);

		const FName ClassName = ObjectClass->GetFName();
		if (TArray<TWeakObjectPtr<UObject>>* ObjectPool = ObjectPools.Find(ClassName))
		{
			while (ObjectPool->Num() > 0)
			{
				if (UObject* Obj = ObjectPool->Pop().Get())
				{
					bIsExistingObject = true;
					return Obj;
				}
			}
		}

		bIsExistingObject = false;
		return NewObject<UObject>(Outer, ObjectClass);
	}

	void ReturnObject(UObject* Obj)
	{
		check(IsInGameThread());
		check(::IsValid(Obj));

		const FName ClassName = Obj->GetClass()->GetFName();
		ObjectPools.FindOrAdd(ClassName).Add(Obj);
	}

	void PreGarbageCollect()
	{
		ObjectPools.Empty();
	}

private:
	TMap<FName, TArray<TWeakObjectPtr<UObject>>> ObjectPools;
};
