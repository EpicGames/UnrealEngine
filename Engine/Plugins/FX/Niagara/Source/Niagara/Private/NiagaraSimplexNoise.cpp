// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimplexNoise.h"
#include "Math/IntVector.h"

// This matches the HLSL code found in Engine/Shaders/Private/Random.ush, but is not
// fully unrolled and also doesn't expect everything to get 100% inlined.

static FIntVector Rand3DPCG16(FIntVector p)
{
	constexpr uint32 Mul = 1664525u;
	constexpr uint32 Add = 1013904223u;

	uint32 X = uint32(p.X) * Mul + Add;
	uint32 Y = uint32(p.Y) * Mul + Add;
	uint32 Z = uint32(p.Z) * Mul + Add;

	X += Y * Z;
	Y += Z * X;
	Z += X * Y;
	X += Y * Z;
	Y += Z * X;
	Z += X * Y;

	return FIntVector(X >> 16, Y >> 16, Z >> 16);
}

static FIntVector NiagaraVectorFloorToInt(FVector3f v) {
	return FIntVector(
		FGenericPlatformMath::FloorToInt(v.X),
		FGenericPlatformMath::FloorToInt(v.Y),
		FGenericPlatformMath::FloorToInt(v.Z)
	);
};

FVector3f SimplexNoiseOffsetFromSeed(uint32 Seed)
{
	FIntVector Rand = Rand3DPCG16(FIntVector(Seed, Seed, Seed));
	return FVector3f(Rand.X, Rand.Y, Rand.Z) * (1.0f / 100.0f);
}

static FVector3f SimplexGvec(int Rand)
{
	FVector3f MGradientScale(1.f / 0x4000, 1.f / 0x2000, 1.f / 0x1000);
	return FVector3f(float(Rand & 0x8000), float(Rand & 0x4000), float(Rand & 0x2000)) * MGradientScale - FVector3f(1.f);
}

FNiagaraMatrix3x4 JacobianSimplex_ALU(FVector3f V)
{
	// SimplexCorners in Random.ush
	float SkewAmount = (V.X + V.Y + V.Z) * (1.0f / 3.0f);
	FVector3f Tet {
		FGenericPlatformMath::FloorToFloat(V.X + SkewAmount),
		FGenericPlatformMath::FloorToFloat(V.Y + SkewAmount),
		FGenericPlatformMath::FloorToFloat(V.Z + SkewAmount)
	};
	FVector3f Base = Tet - FVector3f((Tet.X + Tet.Y + Tet.Z) * (1.0f / 6.0f));
	FVector3f F = V - Base;

	FVector3f G {
		F.X >= F.Y ? 1.0f : 0.0f,
		F.Y >= F.Z ? 1.0f : 0.0f,
		F.Z >= F.X ? 1.0f : 0.0f
	};
	FVector3f H { 1.0f - G.Z, 1.0f - G.X, 1.0f - G.Y };

	FVector3f A1 = G.ComponentMin(H) - FVector3f(1.f / 6.f);
	FVector3f A2 = G.ComponentMax(H) - FVector3f(1.f / 3.f);

	FVector3f TetCorners[4] {
		Base,
		Base + A1,
		Base + A2,
		Base + FVector3f(0.5f)
	};

	// Actual Jacobian calc
	FNiagaraMatrix3x4 Jacobian;

	for (int CornerIndex = 0; CornerIndex < 4; CornerIndex++)
	{
		FVector3f RelVec = V - TetCorners[CornerIndex];
		FIntVector Rand = Rand3DPCG16(NiagaraVectorFloorToInt(6 * TetCorners[CornerIndex] + 0.5));

		FVector3f GvecX = SimplexGvec(Rand.X);
		FVector3f GvecY = SimplexGvec(Rand.Y);
		FVector3f GvecZ = SimplexGvec(Rand.Z);

		float GradX = GvecX.Dot(RelVec);
		float GradY = GvecY.Dot(RelVec);
		float GradZ = GvecZ.Dot(RelVec);

		// Shared terms
		constexpr float Scale = 1024.f / 375.f;
		float SquaredDist = RelVec.Dot(RelVec);
		float S = FMath::Clamp(SquaredDist + SquaredDist, 0.0f, 1.0f);

		// sv / SimplexSmooth in Random.ush
		float SimplexValue = Scale * (1.0f + S * (-3.0f + S * (3.0f - S)));

		// ds / SimplexDSmooth in Random.ush
		float DerivScale = Scale * (-12.0f + S * (24.0f - S * 12.0f));

		Jacobian[0] += FVector4f(SimplexValue * GvecX + (DerivScale * GradX) * RelVec, SimplexValue * GradX);
		Jacobian[1] += FVector4f(SimplexValue * GvecY + (DerivScale * GradY) * RelVec, SimplexValue * GradY);
		Jacobian[2] += FVector4f(SimplexValue * GvecZ + (DerivScale * GradZ) * RelVec, SimplexValue * GradZ);
	}

	return Jacobian;
}

