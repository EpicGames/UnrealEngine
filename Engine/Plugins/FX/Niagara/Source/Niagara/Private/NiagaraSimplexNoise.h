// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "Math/Vector4.h"

/**
 * Jacobian as 3x4 matrix for JacobianSimplex_ALU
 */
struct FNiagaraMatrix3x4
{
	FVector4f row0;
	FVector4f row1;
	FVector4f row2;

	FNiagaraMatrix3x4() : row0(FVector4f::Zero()), row1(FVector4f::Zero()), row2(FVector4f::Zero()) {}
	FNiagaraMatrix3x4(const FVector4f& row0, const FVector4f& row1, const FVector4f& row2) : row0(row0), row1(row1), row2(row2) {}

	FVector4f& operator[](int row)
	{
		if (row == 0) return row0;
		if (row == 1) return row1;
		if (row == 2) return row2;
		check(false);
		return row0;
	}
	const FVector4f& operator[](int row) const
	{
		if (row == 0) return row0;
		if (row == 1) return row1;
		if (row == 2) return row2;
		check(false); 
		return row0; 
	}
};

/**
 * Turns a random seed into a 3D vector offset to use when sampling the 3D simplex noise field.
 */
FVector3f SimplexNoiseOffsetFromSeed(uint32 Seed);

/**
 * Jacobian of simplex noise at given position. Used for curl noise.
 */
FNiagaraMatrix3x4 JacobianSimplex_ALU(FVector3f v);

