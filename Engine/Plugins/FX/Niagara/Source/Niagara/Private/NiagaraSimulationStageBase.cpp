// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimulationStageBase.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraEmitter.h"
#include "NiagaraSimulationStageCompileData.h"
#include "NiagaraSystem.h"
#include "NiagaraScriptSourceBase.h"

#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSimulationStageBase)

const FName UNiagaraSimulationStageBase::ParticleSpawnUpdateName("ParticleSpawnUpdate");

namespace NiagaraSimulationStageLocal
{
	static FName NAME_Particles_StateIndex("Particles.StateIndex");
	static FNiagaraVariableBase GetDefaultEnabledBinding() { return FNiagaraVariableBase(FNiagaraTypeDefinition::GetBoolDef(), NAME_None); }
	static FNiagaraVariableBase GetDefaultElementCountBinding() { return FNiagaraVariableBase(FNiagaraTypeDefinition::GetIntDef(), NAME_None); }
	static FNiagaraVariableBase GetDefaultNumIterationsBinding() { return FNiagaraVariableBase(FNiagaraTypeDefinition::GetIntDef(), NAME_None); }
	static FNiagaraVariableBase GetDefaultParticleStateBinding() { return FNiagaraVariableBase(FNiagaraTypeDefinition::GetIntDef(), NAME_Particles_StateIndex); }
}

#if WITH_EDITORONLY_DATA
bool UNiagaraSimulationStageBase::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
#if WITH_EDITORONLY_DATA
	if (FNiagaraCompileHashVisitorDebugInfo* DebugInfo = InVisitor->AddDebugInfo())
	{
		DebugInfo->Object = FString::Printf(TEXT("Class: \"%s\"  Name: \"%s\""), *GetClass()->GetName(), *GetName());
	}
#endif
	InVisitor->UpdatePOD(TEXT("Enabled"), bEnabled ? 1 : 0);
	return true;
}
#endif

#if WITH_EDITOR
void UNiagaraSimulationStageBase::SetEnabled(bool bInEnabled)
{
	if (bEnabled != bInEnabled)
	{
		bEnabled = bInEnabled;
		RequestRecompile();
		OnChanged().Broadcast();
	}
}

void UNiagaraSimulationStageBase::RequestRecompile()
{
	if (!bEnabled)
	{
		Script->InvalidateCompileResults(TEXT("SimulationStage changed."));
	}

	FVersionedNiagaraEmitter OuterEmitter = GetOuterEmitter();
	if (FVersionedNiagaraEmitterData* EmitterData = OuterEmitter.GetEmitterData())
	{
		if (UNiagaraScriptSourceBase* GraphSource = EmitterData->UpdateScriptProps.Script->GetLatestSource())
		{
			GraphSource->MarkNotSynchronized(TEXT("SimulationStage changed."));
		}

		UNiagaraSystem::RequestCompileForEmitter(OuterEmitter);
	}
}

void UNiagaraSimulationStageBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName;
	if (PropertyChangedEvent.Property)
	{
		PropertyName = PropertyChangedEvent.Property->GetFName();
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageBase, bEnabled))
	{
		RequestRecompile();
	}
}
#endif

#if WITH_EDITORONLY_DATA
bool UNiagaraSimulationStageGeneric::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	Super::AppendCompileHash(InVisitor);

	InVisitor->UpdateString(TEXT("EnabledBinding"), EnabledBinding.GetDataSetBindableVariable().GetName().ToString());
	InVisitor->UpdateString(TEXT("ElementCountX"), ElementCountX.ToString());
	InVisitor->UpdateString(TEXT("ElementCountY"), ElementCountY.ToString());
	InVisitor->UpdateString(TEXT("ElementCountZ"), ElementCountZ.ToString());
	InVisitor->UpdateString(TEXT("NumIterations"), NumIterations.ToString());
	InVisitor->UpdatePOD(TEXT("IterationSource"), (int32)IterationSource);
	InVisitor->UpdatePOD(TEXT("ExecuteBehavior"), (int32)ExecuteBehavior);
	InVisitor->UpdatePOD(TEXT("bDisablePartialParticleUpdate"), bDisablePartialParticleUpdate ? 1 : 0);
	InVisitor->UpdateString(TEXT("DataInterface"), DataInterface.BoundVariable.GetName().ToString());
	InVisitor->UpdateString(TEXT("SimulationStageName"), SimulationStageName.ToString());
	InVisitor->UpdatePOD(TEXT("bParticleIterationStateEnabled"), bParticleIterationStateEnabled ? 1 : 0);
	InVisitor->UpdateString(TEXT("ParticleIterationStateBinding"), ParticleIterationStateBinding.GetDataSetBindableVariable().GetName().ToString());
	InVisitor->UpdateString(TEXT("ParticleIterationStateRange"), FString::Printf(TEXT("%d,%d"), ParticleIterationStateRange.X, ParticleIterationStateRange.Y));
	InVisitor->UpdatePOD(TEXT("bGpuDispatchForceLinear"), bGpuDispatchForceLinear ? 1 : 0);
	InVisitor->UpdatePOD(TEXT("DirectDispatchType"), (int32)DirectDispatchType);
	InVisitor->UpdatePOD(TEXT("DirectDispatchElementType"), int32(DirectDispatchElementType));
	InVisitor->UpdatePOD(TEXT("bOverrideGpuDispatchNumThreads"), bOverrideGpuDispatchNumThreads ? 1 : 0);
	InVisitor->UpdateString(TEXT("OverrideGpuDispatchNumThreadsX"), OverrideGpuDispatchNumThreadsX.ToString());
	InVisitor->UpdateString(TEXT("OverrideGpuDispatchNumThreadsY"), OverrideGpuDispatchNumThreadsY.ToString());
	InVisitor->UpdateString(TEXT("OverrideGpuDispatchNumThreadsZ"), OverrideGpuDispatchNumThreadsZ.ToString());

	return true;
}

bool UNiagaraSimulationStageGeneric::FillCompilationData(TArray<FNiagaraSimulationStageCompilationData>& CompilationSimStageData) const
{
	FNiagaraSimulationStageCompilationData& SimStageData = CompilationSimStageData.AddDefaulted_GetRef();
	SimStageData.StageGuid = Script->GetUsageId();
	SimStageData.StageName = SimulationStageName;
	SimStageData.EnabledBinding = EnabledBinding.GetName();
	SimStageData.ElementCount = FIntVector(ElementCountX.GetDefaultValue<int32>(), ElementCountY.GetDefaultValue<int32>(), ElementCountZ.GetDefaultValue<int32>());
	SimStageData.ElementCountXBinding = ElementCountX.ResolvedParameter.GetName();
	SimStageData.ElementCountYBinding = ElementCountY.ResolvedParameter.GetName();
	SimStageData.ElementCountZBinding = ElementCountZ.ResolvedParameter.GetName();
	SimStageData.NumIterations = NumIterations.GetDefaultValue<int32>();
	SimStageData.NumIterationsBinding = NumIterations.ResolvedParameter.GetName();
	SimStageData.IterationSourceType = IterationSource;
	SimStageData.IterationDataInterface = IterationSource == ENiagaraIterationSource::DataInterface ? DataInterface.BoundVariable.GetName() : FName();
	SimStageData.ExecuteBehavior = ExecuteBehavior;
	SimStageData.PartialParticleUpdate = bDisablePartialParticleUpdate == false;
	SimStageData.bParticleIterationStateEnabled = bParticleIterationStateEnabled;
	SimStageData.ParticleIterationStateRange = ParticleIterationStateRange;
	SimStageData.bGpuDispatchForceLinear = bGpuDispatchForceLinear;
	SimStageData.DirectDispatchType = DirectDispatchType;
	SimStageData.DirectDispatchElementType = DirectDispatchElementType;
	SimStageData.bOverrideGpuDispatchNumThreads = bOverrideGpuDispatchNumThreads;
	SimStageData.OverrideGpuDispatchNumThreadsXBinding = OverrideGpuDispatchNumThreadsX.AliasedParameter;
	SimStageData.OverrideGpuDispatchNumThreadsYBinding = OverrideGpuDispatchNumThreadsY.AliasedParameter;
	SimStageData.OverrideGpuDispatchNumThreadsZBinding = OverrideGpuDispatchNumThreadsZ.AliasedParameter;
	SimStageData.OverrideGpuDispatchNumThreads.X = OverrideGpuDispatchNumThreadsX.GetDefaultValue<int32>();
	SimStageData.OverrideGpuDispatchNumThreads.Y = OverrideGpuDispatchNumThreadsY.GetDefaultValue<int32>();
	SimStageData.OverrideGpuDispatchNumThreads.Z = OverrideGpuDispatchNumThreadsZ.GetDefaultValue<int32>();

	if (SimStageData.bParticleIterationStateEnabled)
	{
		FString AttributeName = ParticleIterationStateBinding.GetName().ToString();
		if (ensureMsgf(AttributeName.RemoveFromStart(TEXT("Particles.")), TEXT("Attribute '%s' is not in particles namespace"), *AttributeName))
		{
			SimStageData.ParticleIterationStateBinding = FName(AttributeName);
		}
	}
	return true;
}

void UNiagaraSimulationStageGeneric::RenameEmitter(const FName& InOldName, const UNiagaraEmitter* InRenamedEmitter)
{
	FNiagaraParameterBinding::ForEachRenameEmitter(this, InRenamedEmitter->GetUniqueEmitterName());
}

void UNiagaraSimulationStageGeneric::RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitter& InEmitter)
{
	FNiagaraParameterBinding::ForEachRenameVariable(this, OldVariable, NewVariable, InEmitter.Emitter->GetUniqueEmitterName());
}

void UNiagaraSimulationStageGeneric::RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitter& InEmitter)
{
	FNiagaraParameterBinding::ForEachRemoveVariable(this, OldVariable, InEmitter.Emitter->GetUniqueEmitterName());
}
#endif //WITH_EDITORONLY_DATA

void UNiagaraSimulationStageGeneric::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		using namespace NiagaraSimulationStageLocal;
		EnabledBinding.Setup(GetDefaultEnabledBinding(), GetDefaultEnabledBinding(), ENiagaraRendererSourceDataMode::Emitter);
#if WITH_EDITORONLY_DATA
		ElementCountBinding_DEPRECATED.Setup(GetDefaultElementCountBinding(), GetDefaultElementCountBinding(), ENiagaraRendererSourceDataMode::Emitter);
		NumIterationsBinding_DEPRECATED.Setup(GetDefaultNumIterationsBinding(), GetDefaultNumIterationsBinding(), ENiagaraRendererSourceDataMode::Emitter);
		ElementCountXBinding_DEPRECATED.Setup(GetDefaultElementCountBinding(), GetDefaultElementCountBinding(), ENiagaraRendererSourceDataMode::Emitter);
		ElementCountYBinding_DEPRECATED.Setup(GetDefaultElementCountBinding(), GetDefaultElementCountBinding(), ENiagaraRendererSourceDataMode::Emitter);
		ElementCountZBinding_DEPRECATED.Setup(GetDefaultElementCountBinding(), GetDefaultElementCountBinding(), ENiagaraRendererSourceDataMode::Emitter);
#endif
		ParticleIterationStateBinding.Setup(GetDefaultParticleStateBinding(), GetDefaultParticleStateBinding(), ENiagaraRendererSourceDataMode::Particles);

#if WITH_EDITORONLY_DATA
		ElementCountX.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		ElementCountX.SetAllowedTypeDefinitions({ FNiagaraTypeDefinition::GetIntDef() });
		ElementCountX.SetDefaultParameter(FNiagaraTypeDefinition::GetIntDef(), 0);

		ElementCountY.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		ElementCountY.SetAllowedTypeDefinitions({ FNiagaraTypeDefinition::GetIntDef() });
		ElementCountY.SetDefaultParameter(FNiagaraTypeDefinition::GetIntDef(), 0);

		ElementCountZ.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		ElementCountZ.SetAllowedTypeDefinitions({ FNiagaraTypeDefinition::GetIntDef() });
		ElementCountZ.SetDefaultParameter(FNiagaraTypeDefinition::GetIntDef(), 0);

		NumIterations.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		NumIterations.SetAllowedTypeDefinitions({FNiagaraTypeDefinition::GetIntDef()});
		NumIterations.SetDefaultParameter(FNiagaraTypeDefinition::GetIntDef(), 1);

		OverrideGpuDispatchNumThreadsX.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		OverrideGpuDispatchNumThreadsX.SetAllowedTypeDefinitions({ FNiagaraTypeDefinition::GetIntDef().ToStaticDef() });
		OverrideGpuDispatchNumThreadsX.SetDefaultParameter(FNiagaraTypeDefinition::GetIntDef().ToStaticDef(), 64);

		OverrideGpuDispatchNumThreadsY.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		OverrideGpuDispatchNumThreadsY.SetAllowedTypeDefinitions({ FNiagaraTypeDefinition::GetIntDef().ToStaticDef() });
		OverrideGpuDispatchNumThreadsY.SetDefaultParameter(FNiagaraTypeDefinition::GetIntDef().ToStaticDef(), 1);

		OverrideGpuDispatchNumThreadsZ.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		OverrideGpuDispatchNumThreadsZ.SetAllowedTypeDefinitions({ FNiagaraTypeDefinition::GetIntDef().ToStaticDef() });
		OverrideGpuDispatchNumThreadsZ.SetDefaultParameter(FNiagaraTypeDefinition::GetIntDef().ToStaticDef(), 1);
#endif
	}
}

void UNiagaraSimulationStageGeneric::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	const int32 UE5Version = GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID);
	const int32 NiagaraVersion = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);

	// Ensure data wasn't somehow saved incorrectly
	using namespace NiagaraSimulationStageLocal;
	if (EnabledBinding.GetType() != GetDefaultEnabledBinding().GetType())
	{
		EnabledBinding.Setup(GetDefaultEnabledBinding(), GetDefaultEnabledBinding(), ENiagaraRendererSourceDataMode::Emitter);
	}
	if (ElementCountXBinding_DEPRECATED.GetType() != GetDefaultElementCountBinding().GetType())
	{
		ElementCountXBinding_DEPRECATED.Setup(GetDefaultElementCountBinding(), GetDefaultElementCountBinding(), ENiagaraRendererSourceDataMode::Emitter);
	}
	if (ElementCountYBinding_DEPRECATED.GetType() != GetDefaultElementCountBinding().GetType())
	{
		ElementCountYBinding_DEPRECATED.Setup(GetDefaultElementCountBinding(), GetDefaultElementCountBinding(), ENiagaraRendererSourceDataMode::Emitter);
	}
	if (ElementCountZBinding_DEPRECATED.GetType() != GetDefaultElementCountBinding().GetType())
	{
		ElementCountZBinding_DEPRECATED.Setup(GetDefaultElementCountBinding(), GetDefaultElementCountBinding(), ENiagaraRendererSourceDataMode::Emitter);
	}
	if (NumIterationsBinding_DEPRECATED.GetType() != GetDefaultNumIterationsBinding().GetType())
	{
		NumIterationsBinding_DEPRECATED.Setup(GetDefaultNumIterationsBinding(), GetDefaultNumIterationsBinding(), ENiagaraRendererSourceDataMode::Emitter);
	}
	if (ParticleIterationStateBinding.GetType() != GetDefaultParticleStateBinding().GetType())
	{
		ParticleIterationStateBinding.Setup(GetDefaultParticleStateBinding(), GetDefaultParticleStateBinding(), ENiagaraRendererSourceDataMode::Particles);
	}

	// Make sure we post load all FNiagaraVariableAttributeBinding
	EnabledBinding.PostLoad(ENiagaraRendererSourceDataMode::Emitter);
	ParticleIterationStateBinding.PostLoad(ENiagaraRendererSourceDataMode::Particles);

	// Handle Deprecated Properties
	if (ElementCountBinding_DEPRECATED.IsValid())
	{
		bOverrideGpuDispatchType_DEPRECATED = true;
		OverrideGpuDispatchType_DEPRECATED = ENiagaraGpuDispatchType::OneD;
		ElementCountXBinding_DEPRECATED = ElementCountBinding_DEPRECATED;
	}

	if (bOverrideGpuDispatchType_DEPRECATED)
	{
		IterationSource = ENiagaraIterationSource::DirectSet;
		DirectDispatchType = OverrideGpuDispatchType_DEPRECATED;
	}

	if (UE5Version < FUE5MainStreamObjectVersion::NiagaraSimStageNumIterationsBindings)
	{
		NumIterations.SetDefaultParameter(NumIterationsBinding_DEPRECATED.GetName(), NumIterationsBinding_DEPRECATED.GetType(), Iterations_DEPRECATED);
		ElementCountX.SetDefaultParameter(ElementCountXBinding_DEPRECATED.GetName(), ElementCountXBinding_DEPRECATED.GetType(), 0);
		ElementCountY.SetDefaultParameter(ElementCountYBinding_DEPRECATED.GetName(), ElementCountYBinding_DEPRECATED.GetType(), 0);
		ElementCountZ.SetDefaultParameter(ElementCountZBinding_DEPRECATED.GetName(), ElementCountZBinding_DEPRECATED.GetType(), 0);

		OverrideGpuDispatchNumThreadsX.SetDefaultValueEditorOnly(OverrideGpuDispatchNumThreads_DEPRECATED.X);
		OverrideGpuDispatchNumThreadsY.SetDefaultValueEditorOnly(OverrideGpuDispatchNumThreads_DEPRECATED.Y);
		OverrideGpuDispatchNumThreadsZ.SetDefaultValueEditorOnly(OverrideGpuDispatchNumThreads_DEPRECATED.Z);
	}

	if (NiagaraVersion < FNiagaraCustomVersion::ParameterBindingWithValueRenameFixup)
	{
		if (UNiagaraEmitter* Emitter = GetTypedOuter<UNiagaraEmitter>())
		{
			const FString& EmitterName = Emitter->GetUniqueEmitterName();

			NumIterations.OnRenameEmitter(EmitterName);
			NumIterations.OnRenameEmitter(EmitterName);
			OverrideGpuDispatchNumThreadsX.OnRenameEmitter(EmitterName);
			OverrideGpuDispatchNumThreadsY.OnRenameEmitter(EmitterName);
			OverrideGpuDispatchNumThreadsZ.OnRenameEmitter(EmitterName);
			ElementCountX.OnRenameEmitter(EmitterName);
			ElementCountY.OnRenameEmitter(EmitterName);
			ElementCountZ.OnRenameEmitter(EmitterName);
		}
	}
#endif
}

#if WITH_EDITOR
void UNiagaraSimulationStageGeneric::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) 
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName;
	FName MemberPropertyName;
	if (PropertyChangedEvent.Property)
	{
		PropertyName = PropertyChangedEvent.Property->GetFName();
	}
	if (PropertyChangedEvent.MemberProperty)
	{
		MemberPropertyName = PropertyChangedEvent.MemberProperty->GetFName();
	}

	static const TSet<FName> ForceCompilePropertyNames =
	{
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, EnabledBinding),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ElementCountX),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ElementCountY),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ElementCountZ),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, NumIterations),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, IterationSource),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ExecuteBehavior),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, bDisablePartialParticleUpdate),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, DataInterface),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, SimulationStageName),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, bParticleIterationStateEnabled),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ParticleIterationStateBinding),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, bGpuDispatchForceLinear),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, DirectDispatchType),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, DirectDispatchElementType),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, bOverrideGpuDispatchNumThreads),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, OverrideGpuDispatchNumThreadsX),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, OverrideGpuDispatchNumThreadsY),
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, OverrideGpuDispatchNumThreadsZ),
	};

	static const TSet<FName> ForceCompileMemberPropertyNamess =
	{
		GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ParticleIterationStateRange),
	};

	const bool bNeedsRecompile = ForceCompilePropertyNames.Contains(PropertyName) || ForceCompileMemberPropertyNamess.Contains(MemberPropertyName);
	if (bNeedsRecompile)
	{
		RequestRecompile();
	}

	OnChanged().Broadcast();
}

FVersionedNiagaraEmitterData* UNiagaraSimulationStageBase::GetEmitterData() const
{
	if (UNiagaraEmitter* SrcEmitter = GetTypedOuter<UNiagaraEmitter>())
	{
		return SrcEmitter->GetEmitterData(OuterEmitterVersion);
	}
	return nullptr;
}

FVersionedNiagaraEmitter UNiagaraSimulationStageBase::GetOuterEmitter() const
{
	if (UNiagaraEmitter* SrcEmitter = GetTypedOuter<UNiagaraEmitter>())
	{
		return FVersionedNiagaraEmitter(SrcEmitter, OuterEmitterVersion);
	}
	return FVersionedNiagaraEmitter();
}

FVersionedNiagaraEmitterBase UNiagaraSimulationStageBase::GetOuterEmitterBase() const
{
	if (UNiagaraEmitterBase* SrcEmitter = GetTypedOuter<UNiagaraEmitterBase>())
	{
		return FVersionedNiagaraEmitterBase(SrcEmitter, OuterEmitterVersion);
	}
	return FVersionedNiagaraEmitterBase();
}

FName UNiagaraSimulationStageGeneric::GetStackContextReplacementName() const 
{
	switch (IterationSource)
	{
		case ENiagaraIterationSource::Particles:
		case ENiagaraIterationSource::DirectSet:
			return NAME_None;

		case ENiagaraIterationSource::DataInterface:
			return DataInterface.BoundVariable.GetName();

		default:
			ensureMsgf(false, TEXT("Should not get here! Need to handle unknown case!"));
			break;
	}
	return NAME_None;
}
#endif

