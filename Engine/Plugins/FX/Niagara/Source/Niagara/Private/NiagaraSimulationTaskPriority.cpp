// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimulationTaskPriority.h"
#include "NiagaraCommon.h"
#include "HAL/IConsoleManager.h"
#include "Particles/ParticlePerfStats.h"

namespace NiagaraSimulationTaskPriority
{
	// Adjusts the priorty when capturing performance tests to avoid tasks being preempted as much
	static int32 GPerfCapturePriority = 1;
	static FAutoConsoleVariableRef CVarPerfCapturePriority(
		TEXT("fx.Niagara.TaskPriorities.PerfCapturePriority"),
		GPerfCapturePriority,
		TEXT("Priority to use if performance captures are enabled.  Reduces the amount of context switching for Niagara to make performance measurements more reliable.")
		TEXT("Default is 1, set to -1 to not override the default priorities."),
		ECVF_Default
	);

	// Post actor tick priority for Niagara tasks
	static int32 GPostActorTickPriority = 1;
	static FAutoConsoleVariableRef CVarPostActorTickPriority(
		TEXT("fx.Niagara.TaskPriorities.PostActorTickPriority"),
		GPostActorTickPriority,
		TEXT("Any tasks we execute, such as spawning, in post actor tick will run at this priority."),
		ECVF_Default | ECVF_Scalability
	);

	// Priority for work which is not allowed to overlap tick groups
	static int32 GNoneOverlappingPriority = 1;
	static FAutoConsoleVariableRef CVarNoneOverlappingPriority(
		TEXT("fx.Niagara.TaskPriorities.NoneOverlappingPriority"),
		GNoneOverlappingPriority,
		TEXT("Priority to use when we are not allowed to overlap tick groups.")
		TEXT("Default is 1, set to -1 to not override the default priorities."),
		ECVF_Default
	);

	// Translation from a Niagara Priority into a Task Priority
	static FAutoConsoleTaskPriority GTaskPriorities[] =
	{
		//																														Thread Priority (w HiPri Thread)			Task Priority (w HiPri Thread)				Task Priority (w Normal Thread)
		FAutoConsoleTaskPriority(TEXT("fx.Niagara.TaskPriorities.High"),		TEXT("Task Priority When Set to High"),			ENamedThreads::HighThreadPriority,			ENamedThreads::HighTaskPriority,			ENamedThreads::HighTaskPriority),
		FAutoConsoleTaskPriority(TEXT("fx.Niagara.TaskPriorities.Normal"),		TEXT("Task Priority When Set to Normal"),		ENamedThreads::HighThreadPriority,			ENamedThreads::NormalTaskPriority,			ENamedThreads::NormalTaskPriority),
		FAutoConsoleTaskPriority(TEXT("fx.Niagara.TaskPriorities.Low"),			TEXT("Task Priority When Set to Low"),			ENamedThreads::NormalThreadPriority,		ENamedThreads::HighTaskPriority,			ENamedThreads::NormalTaskPriority),
		FAutoConsoleTaskPriority(TEXT("fx.Niagara.TaskPriorities.Background"),	TEXT("Task Priority When Set to Background"),	ENamedThreads::BackgroundThreadPriority,	ENamedThreads::NormalTaskPriority,			ENamedThreads::NormalTaskPriority),
	};

	static int32 GTickGroupPriorities[TG_MAX] = {};
	static FString GTickGroupPrioritiesString;
	static void UpdateTickGroupPriorities(IConsoleVariable*)
	{
		Initialize();

		TArray<FString> TGPairs;
		GTickGroupPrioritiesString.ParseIntoArray(TGPairs, TEXT(","));

		UEnum* TickingGroupEnum = StaticEnum<ETickingGroup>();
		check(TickingGroupEnum);
		for (const FString& TGPair : TGPairs)
		{
			int32 SepIndex;
			TGPair.FindChar(':', SepIndex);
			if (SepIndex == INDEX_NONE || TGPair.Len() < SepIndex + 1)
			{
				continue;
			}
			
			const FString TickGroupString = TGPair.Mid(0, SepIndex);
			const int32 TickGroup = TickingGroupEnum->GetIndexByNameString(TickGroupString);
			if (TickGroup >= 0 && TickGroup < TG_MAX)
			{
				int32 Priority = FCString::Atoi(*TGPair.Mid(SepIndex + 1));
				Priority = static_cast<int32>(FMath::Clamp(Priority, 0, UE_ARRAY_COUNT(GTaskPriorities) - 1));
				GTickGroupPriorities[TickGroup] = Priority;
			}
		}
	}

	static FAutoConsoleVariableRef CVarTickGroupPriorities(
		TEXT("fx.Niagara.TaskPriorities.TickGroupPriority"),
		GTickGroupPrioritiesString,
		TEXT("Set tick group priories for Niagara.")
		TEXT("For example, TG_PrePhysics:2,TG_DuringPhysics:2"),
		FConsoleVariableDelegate::CreateStatic(UpdateTickGroupPriorities),
		ECVF_Scalability | ECVF_Default
	);

	static FAutoConsoleCommand CCmdDumpPriorities(
		TEXT("fx.Niagara.TaskPriorities.Dump"),
		TEXT("Dump currently set priorities"),
		FConsoleCommandDelegate::CreateLambda(
			[]()
			{
				auto DumpPriority =
					[](int32 NiagaraPriority, const ENamedThreads::Type TaskThread, const TCHAR* TaskName)
					{
						UE_LOGF(LogNiagara, Log, "%ls = %d = Thread Priority(%d) Task Priority(%d)", TaskName, NiagaraPriority, ENamedThreads::GetThreadPriorityIndex(TaskThread), ENamedThreads::GetTaskPriority(TaskThread));
					};

				UE_LOGF(LogNiagara, Log, "=== Niagara Task Priorities");
				DumpPriority(GPostActorTickPriority, GetPostActorTickPriority(), TEXT("PostActorTickPriority"));

				UEnum* TickingGroupEnum = StaticEnum<ETickingGroup>();
				check(TickingGroupEnum);
				for (int32 i = 0; i < UE_ARRAY_COUNT(GTickGroupPriorities); ++i)
				{
					DumpPriority(
						GTickGroupPriorities[i],
						GetTickGroupPriority(ETickingGroup(i), true),
						*TickingGroupEnum->GetNameStringByValue(i)
					);
				}
			}
		)
	);

	static FAutoConsoleCommandWithWorldAndArgs CCmdRunTest(
		TEXT("fx.Niagara.TaskPriorities.RunTest"),
		TEXT("Run a test set of priorites"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
			[](const TArray<FString>& Args, UWorld* World)
			{
				const int32 TestSet = Args.Num() > 0 ? FCString::Atoi(*Args[0]) : 0;
				switch (TestSet)
				{
					// Current
					case 0:
						GPostActorTickPriority = 1;
						for ( int32& Pri : GTickGroupPriorities )
						{
							Pri = 2;
						}
						break;
					case 1:
						GPostActorTickPriority = 1;
						GTickGroupPriorities[TG_PrePhysics] = 2;
						GTickGroupPriorities[TG_StartPhysics] = 2;
						GTickGroupPriorities[TG_DuringPhysics] = 2;
						break;
					case 2:
						GPostActorTickPriority = 0;
						GTickGroupPriorities[TG_PrePhysics] = 2;
						GTickGroupPriorities[TG_StartPhysics] = 2;
						GTickGroupPriorities[TG_DuringPhysics] = 2;
						break;
					case 3:
						GPostActorTickPriority = 0;
						GTickGroupPriorities[TG_PrePhysics] = 2;
						GTickGroupPriorities[TG_StartPhysics] = 2;
						GTickGroupPriorities[TG_DuringPhysics] = 2;
						GTickGroupPriorities[TG_LastDemotable] = 0;
						break;
				}
			}
		)
	);

	bool GIsInitialized = false;
	void Initialize()
	{
		if ( GIsInitialized )
		{
			return;
		}
		GIsInitialized = true;

		for ( int32& Pri : GTickGroupPriorities)
		{
			Pri = 1;
		}
	}

	ENamedThreads::Type GetPostActorTickPriority()
	{
		const int32 Priority = FMath::Clamp(GPostActorTickPriority, 0, (int32)UE_ARRAY_COUNT(GTaskPriorities) - 1);
		return GTaskPriorities[Priority].Get();
	}

	ENamedThreads::Type GetTickGroupPriority(ETickingGroup TickGroup, bool bCanOverlapTickGroups)
	{
		int32 Priority = 0;
	#if WITH_PARTICLE_PERF_STATS
		// If we are profiling particle performance make sure we don't get context switched due to lower priority as that will confuse the results
		// Leave low pri if we're just gathering world stats but for per system or per component stats we should use high pri.
		if (GPerfCapturePriority >= 0 && (FParticlePerfStats::GetGatherSystemStats() || FParticlePerfStats::GetGatherComponentStats()))
		{
			Priority = GPerfCapturePriority;
		}
		else
	#endif
		{
			const int32 TickGroupIndex = FMath::Clamp(int32(Priority), 0, (int32)UE_ARRAY_COUNT(GTickGroupPriorities) - 1);
			Priority = GTickGroupPriorities[TickGroupIndex];
		}
		Priority = bCanOverlapTickGroups || GNoneOverlappingPriority < 0 ? Priority : GNoneOverlappingPriority;
		Priority = FMath::Clamp(Priority, 0, (int32)UE_ARRAY_COUNT(GTaskPriorities) - 1);
		return GTaskPriorities[Priority].Get();
	}
}
