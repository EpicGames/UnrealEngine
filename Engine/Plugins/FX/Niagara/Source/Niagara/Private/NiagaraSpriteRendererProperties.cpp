// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSpriteRendererProperties.h"
#include "Materials/Material.h"
#include "NiagaraRenderer.h"
#include "NiagaraConstants.h"
#include "NiagaraGPUSortInfo.h"
#include "NiagaraRendererSprites.h"
#include "NiagaraBoundsCalculatorHelper.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraEmitter.h"
#include "NiagaraSystem.h"

#include "Engine/Texture2D.h"
#include "Internationalization/Internationalization.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Modules/ModuleManager.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSpriteRendererProperties)

#if WITH_EDITORONLY_DATA
#include "DerivedDataCacheInterface.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "NiagaraModule.h"
#include "Widgets/Images/SImage.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/SWidget.h"
#include "AssetThumbnail.h"
#include "Widgets/Text/STextBlock.h"
#endif

#define LOCTEXT_NAMESPACE "UNiagaraSpriteRendererProperties"

TArray<TWeakObjectPtr<UNiagaraSpriteRendererProperties>> UNiagaraSpriteRendererProperties::SpriteRendererPropertiesToDeferredInit;

#if ENABLE_COOK_STATS
class NiagaraCutoutCookStats
{
public:
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats;
};

FCookStats::FDDCResourceUsageStats NiagaraCutoutCookStats::UsageStats;
FCookStatsManager::FAutoRegisterCallback NiagaraCutoutCookStats::RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
{
	UsageStats.LogStats(AddStat, TEXT("NiagaraCutout.Usage"), TEXT(""));
});
#endif // ENABLE_COOK_STATS

UNiagaraSpriteRendererProperties::UNiagaraSpriteRendererProperties()
	: Material(nullptr)
	, MaterialUserParamBinding(FNiagaraTypeDefinition(UMaterialInterface::StaticClass()))
	, bSubImageBlend(true)
	, bRemoveHMDRollInVR(false)
	, bSortOnlyWhenTranslucent(true)
#if WITH_EDITORONLY_DATA
	, BoundingMode(BVC_EightVertices)
	, AlphaThreshold(0.1f)
#endif // WITH_EDITORONLY_DATA
{
	AttributeBindings =
	{
		// NOTE: These bindings' indices have to align to their counterpart in ENiagaraSpriteVFLayout
		&PositionBinding,
		&ColorBinding,
		&VelocityBinding,
		&SpriteRotationBinding,
		&SpriteSizeBinding,
		&SpriteFacingBinding,
		&SpriteAlignmentBinding,
		&SubImageIndexBinding,
		&DynamicMaterialBinding,
		&DynamicMaterial1Binding,
		&DynamicMaterial2Binding,
		&DynamicMaterial3Binding,
		&CameraOffsetBinding,
		&UVScaleBinding,
		&PivotOffsetBinding,
		&MaterialRandomBinding,
		&CustomSortingBinding,
		&NormalizedAgeBinding,

		// These bindings are only actually used with accurate motion vectors (indices still need to align)
		&PrevPositionBinding,
		&PrevVelocityBinding,
		&PrevSpriteRotationBinding,
		&PrevSpriteSizeBinding,
		&PrevSpriteFacingBinding,
		&PrevSpriteAlignmentBinding,
		&PrevCameraOffsetBinding,
		&PrevPivotOffsetBinding,

		// The remaining bindings are not associated with attributes in the VF layout
		&RendererVisibilityTagBinding,
	};
}

FNiagaraRenderer* UNiagaraSpriteRendererProperties::CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
	FNiagaraRenderer* NewRenderer = new FNiagaraRendererSprites(FeatureLevel, this, Emitter);
	NewRenderer->Initialize(this, Emitter, InController);
	return NewRenderer;
}

FNiagaraBoundsCalculator* UNiagaraSpriteRendererProperties::CreateBoundsCalculator()
{
	if (GetCurrentSourceMode() == ENiagaraRendererSourceDataMode::Emitter)
	{
		return nullptr;
	}

	return new FNiagaraBoundsCalculatorHelper<true, false, false>();
}

void UNiagaraSpriteRendererProperties::GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const
{
	UMaterialInterface* MaterialInterface = nullptr;
	if (InEmitter != nullptr)
	{
		MaterialInterface = Cast<UMaterialInterface>(InEmitter->FindBinding(MaterialUserParamBinding.Parameter));
	}

#if WITH_EDITORONLY_DATA
	MaterialInterface = MaterialInterface ? MaterialInterface : ToRawPtr(MICMaterial);
#endif

	OutMaterials.Add(MaterialInterface ? MaterialInterface : ToRawPtr(Material));
}

void UNiagaraSpriteRendererProperties::CollectPSOPrecacheData(const FNiagaraEmitterInstance* InEmitter, FPSOPrecacheParamsList& OutParams) const
{
	const FVertexFactoryType* VFType = GetVertexFactoryType();
	UMaterialInterface* MaterialInterface = ToRawPtr(Material);

	if (MaterialInterface)
	{
		FPSOPrecacheParams& PSOPrecacheParams = OutParams.AddDefaulted_GetRef();
		PSOPrecacheParams.MaterialInterface = MaterialInterface;
		// Spite VF is the same for MVF and non-MVF cases
		PSOPrecacheParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(VFType));
	}
}

const FVertexFactoryType* UNiagaraSpriteRendererProperties::GetVertexFactoryType() const
{
	return &FNiagaraSpriteVertexFactory::StaticType;
}

void UNiagaraSpriteRendererProperties::PostLoad()
{
	Super::PostLoad();

	if ( Material )
	{
		Material->ConditionalPostLoad();
	}

#if WITH_EDITORONLY_DATA
	SourceMode = ValidateSourceMode(SourceMode);

	if (MaterialUserParamBinding.Parameter.GetType().GetClass() != UMaterialInterface::StaticClass())
	{
		FNiagaraTypeDefinition MaterialDef(UMaterialInterface::StaticClass());
		MaterialUserParamBinding.Parameter.SetType(MaterialDef);
	}

	if (!FPlatformProperties::RequiresCookedData())
	{
		if (CutoutTexture)
		{	// Here we don't call UpdateCutoutTexture() to avoid issue with the material postload.
			CutoutTexture->ConditionalPostLoad();
		}
		CacheDerivedData();
	}
	ChangeToPositionBinding(PositionBinding);
	ChangeToPositionBinding(PrevPositionBinding);
	PostLoadBindings(SourceMode);

	// Fix up these bindings from their loaded source bindings
	SetPreviousBindings(FVersionedNiagaraEmitterBase(), SourceMode);

	if (MaterialParameterBindings_DEPRECATED.Num() > 0)
	{
		MaterialParameters.AttributeBindings = MaterialParameterBindings_DEPRECATED;
		MaterialParameterBindings_DEPRECATED.Empty();
	}
#endif
	MaterialParameters.ConditionalPostLoad();
}

void UNiagaraSpriteRendererProperties::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		// We can end up hitting PostInitProperties before the Niagara Module has initialized bindings this needs, mark this object for deferred init and early out.
		if (FModuleManager::Get().IsModuleLoaded("Niagara") == false)
		{
			SpriteRendererPropertiesToDeferredInit.Add(this);
			return;
		}
		InitBindings();
	}
}

void UNiagaraSpriteRendererProperties::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& Ar = Record.GetUnderlyingArchive();
	Ar.UsingCustomVersion(FNiagaraCustomVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	const int32 NiagaraVersion = Ar.CustomVer(FNiagaraCustomVersion::GUID);
	const int32 UE5MainVersion = Ar.CustomVer(FUE5MainStreamObjectVersion::GUID);

	// Default Property Changes
	if (Ar.IsLoading())
	{
		if (NiagaraVersion < FNiagaraCustomVersion::DisableSortingByDefault)
		{
			SortMode = ENiagaraSortMode::ViewDistance;
		}
		if (UE5MainVersion < FUE5MainStreamObjectVersion::NiagaraSpriteRendererFacingAlignmentAutoDefault)
		{
			Alignment = ENiagaraSpriteAlignment::Unaligned;
			FacingMode = ENiagaraSpriteFacingMode::FaceCamera;
		}
		if (NiagaraVersion < FNiagaraCustomVersion::SubImageBlendEnabledByDefault)
		{
			bSubImageBlend = false;
		}
		if (NiagaraVersion < FNiagaraCustomVersion::CustomSortingBindingToAge)
		{
			CustomSortingBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_NORMALIZED_AGE);
		}
	}

	// MIC will replace the main material during serialize
	// Be careful if adding code that looks at the material to make sure you get the correct one
	{
	#if WITH_EDITORONLY_DATA
		TOptional<TGuardValue<TObjectPtr<UMaterialInterface>>> MICGuard;
		if (Ar.IsSaving() && Ar.IsCooking() && MICMaterial)
		{
			MICGuard.Emplace(Material, MICMaterial);
		}
	#endif

		Super::Serialize(Record);
	}

	bool bIsCookedForEditor = false;
#if WITH_EDITORONLY_DATA
	bIsCookedForEditor = ((Ar.GetPortFlags() & PPF_Duplicate) == 0) && GetPackage()->HasAnyPackageFlags(PKG_Cooked);
#endif // WITH_EDITORONLY_DATA

	if (Ar.IsCooking() || (FPlatformProperties::RequiresCookedData() && Ar.IsLoading()) || bIsCookedForEditor)
	{
		DerivedData.Serialize(Record.EnterField(TEXT("DerivedData")));
	}
}

void UNiagaraSpriteRendererProperties::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(RendererLayoutWithCustomSort.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(RendererLayoutWithoutCustomSort.GetAllocatedSize());
}

/** The bindings depend on variables that are created during the NiagaraModule startup. However, the CDO's are build prior to this being initialized, so we defer setting these values until later.*/
void UNiagaraSpriteRendererProperties::InitCDOPropertiesAfterModuleStartup()
{
	UNiagaraSpriteRendererProperties* CDO = CastChecked<UNiagaraSpriteRendererProperties>(UNiagaraSpriteRendererProperties::StaticClass()->GetDefaultObject());
	CDO->InitRendererBaseBindings();
	CDO->InitBindings();

	for (TWeakObjectPtr<UNiagaraSpriteRendererProperties>& WeakSpriteRendererProperties : SpriteRendererPropertiesToDeferredInit)
	{
		if (UNiagaraSpriteRendererProperties* RendererProperties = WeakSpriteRendererProperties.Get())
		{
			RendererProperties->InitRendererBaseBindings();
			RendererProperties->InitBindings();
		}
	}
}

void UNiagaraSpriteRendererProperties::InitBindings()
{
	if (PositionBinding.GetParamMapBindableVariable().GetName() == NAME_None)
	{
		PositionBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
		ColorBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_COLOR);
		VelocityBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VELOCITY);
		SpriteRotationBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SPRITE_ROTATION);
		SpriteSizeBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SPRITE_SIZE);
		SpriteFacingBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SPRITE_FACING);
		SpriteAlignmentBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SPRITE_ALIGNMENT);
		SubImageIndexBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX);
		DynamicMaterialBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM);
		DynamicMaterial1Binding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1);
		DynamicMaterial2Binding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2);
		DynamicMaterial3Binding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3);
		CameraOffsetBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_CAMERA_OFFSET);
		UVScaleBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_UV_SCALE);
		PivotOffsetBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_PIVOT_OFFSET);
		MaterialRandomBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_MATERIAL_RANDOM);
		NormalizedAgeBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_NORMALIZED_AGE);
		RendererVisibilityTagBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);

		//Default custom sorting to age
		CustomSortingBinding = FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_AGE);
	}

	SetPreviousBindings(FVersionedNiagaraEmitterBase(), SourceMode);
}

void UNiagaraSpriteRendererProperties::SetPreviousBindings(const FVersionedNiagaraEmitterBase& SrcEmitter, ENiagaraRendererSourceDataMode InSourceMode)
{
	PrevPositionBinding.SetAsPreviousValue(PositionBinding, SrcEmitter, InSourceMode);
	PrevVelocityBinding.SetAsPreviousValue(VelocityBinding, SrcEmitter, InSourceMode);
	PrevSpriteRotationBinding.SetAsPreviousValue(SpriteRotationBinding, SrcEmitter, InSourceMode);
	PrevSpriteSizeBinding.SetAsPreviousValue(SpriteSizeBinding, SrcEmitter, InSourceMode);
	PrevSpriteFacingBinding.SetAsPreviousValue(SpriteFacingBinding, SrcEmitter, InSourceMode);
	PrevSpriteAlignmentBinding.SetAsPreviousValue(SpriteAlignmentBinding, SrcEmitter, InSourceMode);
	PrevCameraOffsetBinding.SetAsPreviousValue(CameraOffsetBinding, SrcEmitter, InSourceMode);
	PrevPivotOffsetBinding.SetAsPreviousValue(PivotOffsetBinding, SrcEmitter, InSourceMode);
}

void UNiagaraSpriteRendererProperties::CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData)
{
	UpdateSourceModeDerivates(SourceMode);
	UpdateMICs();

	// Initialize layout
	const int32 NumLayoutVars = NeedsPreciseMotionVectors() ? ENiagaraSpriteVFLayout::Num_Max : ENiagaraSpriteVFLayout::Num_Default;
	RendererLayoutWithCustomSort.Initialize(NumLayoutVars);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, PositionBinding, ENiagaraSpriteVFLayout::Position);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, VelocityBinding, ENiagaraSpriteVFLayout::Velocity);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, ColorBinding, ENiagaraSpriteVFLayout::Color);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, SpriteRotationBinding, ENiagaraSpriteVFLayout::Rotation);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, SpriteSizeBinding, ENiagaraSpriteVFLayout::Size);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, SpriteFacingBinding, ENiagaraSpriteVFLayout::Facing);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, SpriteAlignmentBinding, ENiagaraSpriteVFLayout::Alignment);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, SubImageIndexBinding, ENiagaraSpriteVFLayout::SubImage);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, CameraOffsetBinding, ENiagaraSpriteVFLayout::CameraOffset);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, UVScaleBinding, ENiagaraSpriteVFLayout::UVScale);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, PivotOffsetBinding, ENiagaraSpriteVFLayout::PivotOffset);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, NormalizedAgeBinding, ENiagaraSpriteVFLayout::NormalizedAge);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, MaterialRandomBinding, ENiagaraSpriteVFLayout::MaterialRandom);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, CustomSortingBinding, ENiagaraSpriteVFLayout::CustomSorting);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, DynamicMaterialBinding, ENiagaraSpriteVFLayout::MaterialParam0);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, DynamicMaterial1Binding, ENiagaraSpriteVFLayout::MaterialParam1);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, DynamicMaterial2Binding, ENiagaraSpriteVFLayout::MaterialParam2);
	RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, DynamicMaterial3Binding, ENiagaraSpriteVFLayout::MaterialParam3);
	if (NeedsPreciseMotionVectors())
	{
		RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, PrevPositionBinding, ENiagaraSpriteVFLayout::PrevPosition);
		RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, PrevVelocityBinding, ENiagaraSpriteVFLayout::PrevVelocity);
		RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, PrevSpriteRotationBinding, ENiagaraSpriteVFLayout::PrevRotation);
		RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, PrevSpriteSizeBinding, ENiagaraSpriteVFLayout::PrevSize);
		RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, PrevSpriteFacingBinding, ENiagaraSpriteVFLayout::PrevFacing);
		RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, PrevSpriteAlignmentBinding, ENiagaraSpriteVFLayout::PrevAlignment);
		RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, PrevCameraOffsetBinding, ENiagaraSpriteVFLayout::PrevCameraOffset);
		RendererLayoutWithCustomSort.SetVariableFromBinding(CompiledData, PrevPivotOffsetBinding, ENiagaraSpriteVFLayout::PrevPivotOffset);
	}
	RendererLayoutWithCustomSort.Finalize();

	RendererLayoutWithoutCustomSort.Initialize(NumLayoutVars);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, PositionBinding, ENiagaraSpriteVFLayout::Position);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, VelocityBinding, ENiagaraSpriteVFLayout::Velocity);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, ColorBinding, ENiagaraSpriteVFLayout::Color);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, SpriteRotationBinding, ENiagaraSpriteVFLayout::Rotation);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, SpriteSizeBinding, ENiagaraSpriteVFLayout::Size);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, SpriteFacingBinding, ENiagaraSpriteVFLayout::Facing);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, SpriteAlignmentBinding, ENiagaraSpriteVFLayout::Alignment);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, SubImageIndexBinding, ENiagaraSpriteVFLayout::SubImage);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, CameraOffsetBinding, ENiagaraSpriteVFLayout::CameraOffset);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, UVScaleBinding, ENiagaraSpriteVFLayout::UVScale);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, PivotOffsetBinding, ENiagaraSpriteVFLayout::PivotOffset);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, NormalizedAgeBinding, ENiagaraSpriteVFLayout::NormalizedAge);
	RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, MaterialRandomBinding, ENiagaraSpriteVFLayout::MaterialRandom);
	const bool bDynamicParam0Valid = RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, DynamicMaterialBinding,  ENiagaraSpriteVFLayout::MaterialParam0);
	const bool bDynamicParam1Valid = RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, DynamicMaterial1Binding, ENiagaraSpriteVFLayout::MaterialParam1);
	const bool bDynamicParam2Valid = RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, DynamicMaterial2Binding, ENiagaraSpriteVFLayout::MaterialParam2);
	const bool bDynamicParam3Valid = RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, DynamicMaterial3Binding, ENiagaraSpriteVFLayout::MaterialParam3);
	if (NeedsPreciseMotionVectors())
	{
		RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, PrevPositionBinding, ENiagaraSpriteVFLayout::PrevPosition);
		RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, PrevVelocityBinding, ENiagaraSpriteVFLayout::PrevVelocity);
		RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, PrevSpriteRotationBinding, ENiagaraSpriteVFLayout::PrevRotation);
		RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, PrevSpriteSizeBinding, ENiagaraSpriteVFLayout::PrevSize);
		RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, PrevSpriteFacingBinding, ENiagaraSpriteVFLayout::PrevFacing);
		RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, PrevSpriteAlignmentBinding, ENiagaraSpriteVFLayout::PrevAlignment);
		RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, PrevCameraOffsetBinding, ENiagaraSpriteVFLayout::PrevCameraOffset);
		RendererLayoutWithoutCustomSort.SetVariableFromBinding(CompiledData, PrevPivotOffsetBinding, ENiagaraSpriteVFLayout::PrevPivotOffset);
	}
	RendererLayoutWithoutCustomSort.Finalize();

#if WITH_EDITORONLY_DATA
	// Build dynamic parameter mask
	// Serialize in cooked builds
	MaterialParamValidMask = GetDynamicParameterCombinedChannelMask(
		bDynamicParam0Valid ? DynamicMaterialBinding.GetName() : NAME_None,
		bDynamicParam1Valid ? DynamicMaterial1Binding.GetName() : NAME_None,
		bDynamicParam2Valid ? DynamicMaterial2Binding.GetName() : NAME_None,
		bDynamicParam3Valid ? DynamicMaterial3Binding.GetName() : NAME_None
	);
#endif
}

#if WITH_EDITORONLY_DATA
TArray<FNiagaraVariable> UNiagaraSpriteRendererProperties::GetBoundAttributes() const
{
	TArray<FNiagaraVariable> BoundAttributes = Super::GetBoundAttributes();
	BoundAttributes.Reserve(BoundAttributes.Num() + MaterialParameters.AttributeBindings.Num() + 2);

	for (const FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		BoundAttributes.AddUnique(MaterialParamBinding.GetParamMapBindableVariable());
	}

	if (MaterialUserParamBinding.Parameter.IsValid())
	{
		BoundAttributes.AddUnique(MaterialUserParamBinding.Parameter);
	}

	if (RendererEnabledBinding.IsValid())
	{
		BoundAttributes.AddUnique(RendererEnabledBinding.GetParamMapBindableVariable());
	}

	return BoundAttributes;
}
#endif

bool UNiagaraSpriteRendererProperties::PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore)
{
	bool bAnyAdded = Super::PopulateRequiredBindings(InParameterStore);

	for (const FNiagaraVariableAttributeBinding* Binding : AttributeBindings)
	{
		if (Binding && Binding->CanBindToHostParameterMap())
		{
			InParameterStore.AddParameter(Binding->GetParamMapBindableVariable(), false);
			bAnyAdded = true;
		}
	}

	for (FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		InParameterStore.AddParameter(MaterialParamBinding.GetParamMapBindableVariable(), false);
		bAnyAdded = true;
	}

	return bAnyAdded;
}

void UNiagaraSpriteRendererProperties::UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit)
{
	Super::UpdateSourceModeDerivates(InSourceMode, bFromPropertyEdit);
	
	UNiagaraEmitterBase* SrcEmitter = GetTypedOuter<UNiagaraEmitterBase>();
	if (SrcEmitter)
	{
		for (FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
		{
			MaterialParamBinding.CacheValues(SrcEmitter);
		}

		SetPreviousBindings(FVersionedNiagaraEmitterBase(), InSourceMode);
	}
}

void UNiagaraSpriteRendererProperties::UpdateMICs()
{
#if WITH_EDITORONLY_DATA
	UpdateMaterialParametersMIC(MaterialParameters, Material, MICMaterial);
#endif
}

#if WITH_EDITORONLY_DATA
void UNiagaraSpriteRendererProperties::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	SubImageSize.X = FMath::Max(SubImageSize.X, 1.0);
	SubImageSize.Y = FMath::Max(SubImageSize.Y, 1.0);

	// DerivedData.BoundingGeometry in case we cleared the CutoutTexture
	if (bUseMaterialCutoutTexture || CutoutTexture || DerivedData.BoundingGeometry.Num())
	{
		const bool bUpdateCutoutDDC =
			PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSpriteRendererProperties, bUseMaterialCutoutTexture) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSpriteRendererProperties, bCutoutIgnoreSubUV) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSpriteRendererProperties, CutoutTexture) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSpriteRendererProperties, BoundingMode) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSpriteRendererProperties, OpacitySourceMode) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSpriteRendererProperties, AlphaThreshold) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSpriteRendererProperties, SubImageSize) ||
			(bUseMaterialCutoutTexture && PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSpriteRendererProperties, Material));

		if (bUpdateCutoutDDC)
		{
			UpdateCutoutTexture();
			CacheDerivedData();
		}
	}

	// Update our MICs if we change material / material bindings
	//-OPT: Could narrow down further to only static materials
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSpriteRendererProperties, Material)) ||
		(MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSpriteRendererProperties, MaterialParameters)) )
	{
		UpdateMICs();
	}

	// If changing the source mode, we may need to update many of our values.
	if (PropertyChangedEvent.GetPropertyName() == TEXT("SourceMode"))
	{
		UpdateSourceModeDerivates(SourceMode, true);
	}
	else if (FStructProperty* StructProp = CastField<FStructProperty>(PropertyChangedEvent.Property))
	{
		if (StructProp->Struct == FNiagaraVariableAttributeBinding::StaticStruct())
		{
			UpdateSourceModeDerivates(SourceMode, true);
		}
	}
	else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(PropertyChangedEvent.Property))
	{
		if (ArrayProp->Inner)
		{
			FStructProperty* ChildStructProp = CastField<FStructProperty>(ArrayProp->Inner);
			if (ChildStructProp->Struct == FNiagaraMaterialAttributeBinding::StaticStruct())
			{
				UpdateSourceModeDerivates(SourceMode, true);
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UNiagaraSpriteRendererProperties::RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RenameVariable(OldVariable, NewVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MaterialParameters.RenameVariable(OldVariable, NewVariable, InEmitter, GetCurrentSourceMode());
#endif
}

void UNiagaraSpriteRendererProperties::RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RemoveVariable(OldVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MaterialParameters.RemoveVariable(OldVariable, InEmitter, GetCurrentSourceMode());
#endif
}

const TArray<FNiagaraVariable>& UNiagaraSpriteRendererProperties::GetOptionalAttributes()
{
	static TArray<FNiagaraVariable> Attrs;

	if (Attrs.Num() == 0)
	{
		Attrs.Add(SYS_PARAM_PARTICLES_POSITION);
		Attrs.Add(SYS_PARAM_PARTICLES_VELOCITY);
		Attrs.Add(SYS_PARAM_PARTICLES_COLOR);
		Attrs.Add(SYS_PARAM_PARTICLES_SPRITE_ROTATION);
		Attrs.Add(SYS_PARAM_PARTICLES_NORMALIZED_AGE);
		Attrs.Add(SYS_PARAM_PARTICLES_SPRITE_SIZE);
		Attrs.Add(SYS_PARAM_PARTICLES_SPRITE_FACING);
		Attrs.Add(SYS_PARAM_PARTICLES_SPRITE_ALIGNMENT);
		Attrs.Add(SYS_PARAM_PARTICLES_SUB_IMAGE_INDEX);
		Attrs.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM);
		Attrs.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_1);
		Attrs.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_2);
		Attrs.Add(SYS_PARAM_PARTICLES_DYNAMIC_MATERIAL_PARAM_3);
		Attrs.Add(SYS_PARAM_PARTICLES_CAMERA_OFFSET);
		Attrs.Add(SYS_PARAM_PARTICLES_UV_SCALE);
		Attrs.Add(SYS_PARAM_PARTICLES_PIVOT_OFFSET);
		Attrs.Add(SYS_PARAM_PARTICLES_MATERIAL_RANDOM);
	}

	return Attrs;
}

void UNiagaraSpriteRendererProperties::GetAdditionalVariables(TArray<FNiagaraVariableBase>& OutArray) const
{
	if (NeedsPreciseMotionVectors())
	{
		OutArray.Reserve(8);
		OutArray.Add(PrevPositionBinding.GetParamMapBindableVariable());
		OutArray.Add(PrevVelocityBinding.GetParamMapBindableVariable());
		OutArray.Add(PrevSpriteRotationBinding.GetParamMapBindableVariable());
		OutArray.Add(PrevSpriteSizeBinding.GetParamMapBindableVariable());
		OutArray.Add(PrevSpriteFacingBinding.GetParamMapBindableVariable());
		OutArray.Add(PrevSpriteAlignmentBinding.GetParamMapBindableVariable());
		OutArray.Add(PrevCameraOffsetBinding.GetParamMapBindableVariable());
		OutArray.Add(PrevPivotOffsetBinding.GetParamMapBindableVariable());		
	}
}

void UNiagaraSpriteRendererProperties::GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	TArray<UMaterialInterface*> Materials;
	GetUsedMaterials(InEmitter, Materials);

	CreateRendererWidgetsForAssets(Materials, InThumbnailPool, OutWidgets);
}

void UNiagaraSpriteRendererProperties::GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FText>& OutErrors, TArray<FText>& OutWarnings, TArray<FText>& OutInfo) const
{
	Super::GetRendererFeedback(InEmitter, OutErrors, OutWarnings, OutInfo);

	if (InEmitter.GetEmitterData()->SpawnScriptProps.Script->GetVMExecutableData().IsValid())
	{
		if (bUseMaterialCutoutTexture || CutoutTexture)
		{
			if (UVScaleBinding.DoesBindingExistOnSource())
			{
				OutInfo.Add(LOCTEXT("SpriteRendererUVScaleWithCutout", "Cutouts will not be sized dynamically with UVScale variable. If scaling above 1.0, geometry may clip."));
			}
		}
	}

	if (CutoutTexture)
	{
		DerivedData.GetFeedback(CutoutTexture, (int32)SubImageSize.X, (int32)SubImageSize.Y, BoundingMode, AlphaThreshold, OpacitySourceMode, OutErrors, OutWarnings, OutInfo);
	}
}

void UNiagaraSpriteRendererProperties::GetRendererFeedback(const FVersionedNiagaraEmitter & InEmitter, TArray<FNiagaraRendererFeedback>&OutErrors, TArray<FNiagaraRendererFeedback>&OutWarnings, TArray<FNiagaraRendererFeedback>&OutInfo) const
{
	Super::GetRendererFeedback(InEmitter, OutErrors, OutWarnings, OutInfo);

	if (MaterialParameters.HasAnyBindings())
	{
		TArray<UMaterialInterface*> Materials;
		GetUsedMaterials(nullptr, Materials);
		MaterialParameters.GetFeedback(Materials, OutWarnings);
	}

	// If sprite material is transluscent and subimage size is not (1,1) and cutout texture is not used, display note
	if (Material && Material->GetBlendMode() == BLEND_Translucent 
		&& (SubImageSize.X != 1.0f || SubImageSize.Y != 1.0f)
		&& (!bUseMaterialCutoutTexture && CutoutTexture == nullptr))
	{
		OutInfo.Emplace(
			LOCTEXT("MissingCutoutTexture", "Missing Cutout Texture for Sub UV Image, which might lead to overdraw and reduce performance.\nEnable Use Material Cutout Texture or set Cutout Texture"),
			LOCTEXT("RequiredCutoutTexture", "Cutout Texture is not set for Sub UV Image"),
			FText(),
			FNiagaraRendererFeedbackFix(),
			true
		);
	}
}

void UNiagaraSpriteRendererProperties::GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	TArray<UMaterialInterface*> Materials;
	GetUsedMaterials(InEmitter, Materials);
	if (Materials.Num() > 0)
	{
		GetRendererWidgets(InEmitter, OutWidgets, InThumbnailPool);
	}
	else
	{
		TSharedRef<SWidget> SpriteTooltip = SNew(STextBlock)
			.Text(LOCTEXT("SpriteRendererNoMat", "Sprite Renderer (No Material Set)"));
		OutWidgets.Add(SpriteTooltip);
	}
}

void UNiagaraSpriteRendererProperties::GetRendererMaterialUsageFeedback(TArray<FNiagaraRendererFeedback>& OutFeedback) const
{
	Super::GetRendererMaterialUsageFeedback(OutFeedback);
	GetMaterialUsageFeedback(MATUSAGE_NiagaraSprites, OutFeedback);
}

void UNiagaraSpriteRendererProperties::UpdateCutoutTexture()
{
	if (bUseMaterialCutoutTexture == false)
	{
		return;
	}

	CutoutTexture = nullptr;
	if (Material == nullptr || Material->GetMaterial() == nullptr)
	{
		return;
	}

	// The property should probably come from the blend mode, but keeping consistency with how this has always been
	for (const EMaterialProperty MaterialProperty : {MP_OpacityMask, MP_Opacity})
	{
		TArray<UMaterialExpression*> MaterialExpressions;
		Material->GetMaterial()->GetExpressionsInPropertyChain(MaterialProperty, MaterialExpressions, nullptr);
		for (UMaterialExpression* MaterialExpression : MaterialExpressions)
		{
			UMaterialExpressionTextureSample* TextureExpression = MaterialExpression ? Cast<UMaterialExpressionTextureSample>(MaterialExpression) : nullptr;
			if (TextureExpression == nullptr)
			{
				continue;
			}

			UTexture* ExpressionTexture = TextureExpression->Texture;
			if (UMaterialExpressionTextureSampleParameter* TextureParameterExpression = Cast<UMaterialExpressionTextureSampleParameter>(TextureExpression))
			{
				Material->GetTextureParameterValue(TextureExpression->GetParameterName(), ExpressionTexture);
			}

			// We bail on the first texture found, which isn't accurate but good enough
			CutoutTexture = Cast<UTexture2D>(ExpressionTexture);
			return;
		}
	}
	// Note we do not get here unless we failed to find a cutout texture
}

void UNiagaraSpriteRendererProperties::CacheDerivedData()
{
	if (CutoutTexture)
	{
		const FIntVector2 DerivedSubImageSize = bCutoutIgnoreSubUV ? FIntVector2(1, 1) : FIntVector2((int32)SubImageSize.X, (int32)SubImageSize.Y);

		const FString KeyString = FSubUVDerivedData::GetDDCKeyString(CutoutTexture->Source.GetId(), DerivedSubImageSize.X, DerivedSubImageSize.Y, (int32)BoundingMode, AlphaThreshold, (int32)OpacitySourceMode);
		TArray<uint8> Data;

		COOK_STAT(auto Timer = NiagaraCutoutCookStats::UsageStats.TimeSyncWork());
		if (GetDerivedDataCacheRef().GetSynchronous(*KeyString, Data, GetPathName()))
		{
			COOK_STAT(Timer.AddHit(Data.Num()));
			DerivedData.BoundingGeometry.Empty(Data.Num() / sizeof(FVector2f));
			DerivedData.BoundingGeometry.AddUninitialized(Data.Num() / sizeof(FVector2f));
			FPlatformMemory::Memcpy(DerivedData.BoundingGeometry.GetData(), Data.GetData(), Data.Num() * Data.GetTypeSize());
		}
		else
		{
			DerivedData.Build(CutoutTexture, DerivedSubImageSize.X, DerivedSubImageSize.Y, BoundingMode, AlphaThreshold, OpacitySourceMode);

			Data.Empty(DerivedData.BoundingGeometry.Num() * sizeof(FVector2f));
			Data.AddUninitialized(DerivedData.BoundingGeometry.Num() * sizeof(FVector2f));
			FPlatformMemory::Memcpy(Data.GetData(), DerivedData.BoundingGeometry.GetData(), DerivedData.BoundingGeometry.Num() * DerivedData.BoundingGeometry.GetTypeSize());
			GetDerivedDataCacheRef().Put(*KeyString, Data, GetPathName());
			COOK_STAT(Timer.AddMiss(Data.Num()));
		}

		if (bCutoutIgnoreSubUV)
		{
			const int32 NumFrames = (int32)SubImageSize.X * (int32)SubImageSize.Y;
			const int32 NumVerts = DerivedData.BoundingGeometry.Num();

			DerivedData.BoundingGeometry.SetNumUninitialized(NumVerts * NumFrames);
			for ( int32 i=1; i < NumFrames; ++i )
			{
				FMemory::Memcpy(&DerivedData.BoundingGeometry[NumVerts * i], &DerivedData.BoundingGeometry[0], sizeof(FVector2f) * NumVerts);
			}
		}
	}
	else
	{
		DerivedData.BoundingGeometry.Empty();
	}
}

FNiagaraVariable UNiagaraSpriteRendererProperties::GetBoundAttribute(const FNiagaraVariableAttributeBinding* Binding) const
{
	if (!NeedsPreciseMotionVectors())
	{
		if (Binding == &PrevPositionBinding
			|| Binding == &PrevVelocityBinding
			|| Binding == &PrevSpriteRotationBinding
			|| Binding == &PrevSpriteSizeBinding
			|| Binding == &PrevSpriteFacingBinding
			|| Binding == &PrevSpriteAlignmentBinding
			|| Binding == &PrevCameraOffsetBinding
			|| Binding == &PrevPivotOffsetBinding)
		{
			return FNiagaraVariable();
		}
	}

	return Super::GetBoundAttribute(Binding);
}

#endif // WITH_EDITORONLY_DATA


int32 UNiagaraSpriteRendererProperties::GetNumCutoutVertexPerSubimage() const
{
	if (DerivedData.BoundingGeometry.Num())
	{
		const int32 NumSubImages = FMath::Max<int32>(1, (int32)SubImageSize.X * (int32)SubImageSize.Y);
		const int32 NumCutoutVertexPerSubImage = DerivedData.BoundingGeometry.Num() / NumSubImages;

		// Based on BVC_FourVertices || BVC_EightVertices
		ensure(NumCutoutVertexPerSubImage == 4 || NumCutoutVertexPerSubImage == 8);

		return NumCutoutVertexPerSubImage;
	}
	else
	{
		return 0;
	}
}

uint32 UNiagaraSpriteRendererProperties::GetNumIndicesPerInstance() const
{
	// This is a based on cutout vertices making a triangle strip.
	if (GetNumCutoutVertexPerSubimage() == 8)
	{
		return 18;
	}
	else
	{
		return 6;
	}
}

#undef LOCTEXT_NAMESPACE

