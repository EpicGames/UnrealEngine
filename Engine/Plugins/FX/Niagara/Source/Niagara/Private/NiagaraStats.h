// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("Niagara"), STATGROUP_Niagara, STATCAT_Niagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("# CPU Particles"), STAT_NiagaraNumParticles, STATGROUP_Niagara);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Constant Setup [CNC]"), STAT_NiagaraConstants, STATGROUP_Niagara, EStatFlags::Verbose);
DECLARE_CYCLE_STAT(TEXT("Emitter Tick [CNC]"), STAT_NiagaraTick, STATGROUP_Niagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumSystems"), STAT_NiagaraNumSystems, STATGROUP_Niagara);
DECLARE_MEMORY_STAT(TEXT("Niagara particle data memory"), STAT_NiagaraParticleMemory, STATGROUP_Niagara);
DECLARE_MEMORY_STAT(TEXT("Niagara GPU particle data memory"), STAT_NiagaraGPUParticleMemory, STATGROUP_Niagara);
DECLARE_MEMORY_STAT(TEXT("Niagara GPU data interface memory"), STAT_NiagaraGPUDataInterfaceMemory, STATGROUP_Niagara);
DECLARE_MEMORY_STAT(TEXT("Niagara vertex buffer memory"), STAT_NiagaraVBMemory, STATGROUP_Niagara);
DECLARE_MEMORY_STAT(TEXT("Niagara RenderTarget memory"), STAT_NiagaraRenderTargetMemory, STATGROUP_Niagara);

DECLARE_STATS_GROUP(TEXT("NiagaraOverview"), STATGROUP_NiagaraOverview, STATCAT_Niagara);
DECLARE_CYCLE_STAT(TEXT("GT Total"), STAT_NiagaraOverview_GT, STATGROUP_NiagaraOverview);
DECLARE_CYCLE_STAT(TEXT("GT Concurrent Total"), STAT_NiagaraOverview_GT_CNC, STATGROUP_NiagaraOverview);
DECLARE_CYCLE_STAT(TEXT("RT Total"), STAT_NiagaraOverview_RT, STATGROUP_NiagaraOverview);
DECLARE_CYCLE_STAT(TEXT("RT Concurrent Total"), STAT_NiagaraOverview_RT_CNC, STATGROUP_NiagaraOverview);

DECLARE_STATS_GROUP(TEXT("Niagara Systems"), STATGROUP_NiagaraSystems, STATCAT_Niagara);
DECLARE_STATS_GROUP(TEXT("Niagara Emitters"), STATGROUP_NiagaraEmitters, STATCAT_Niagara);

DECLARE_STATS_GROUP(TEXT("Niagara System Counts"), STATGROUP_NiagaraSystemCounts, STATCAT_Niagara);

DECLARE_DWORD_COUNTER_STAT(TEXT("Total System Instances"), STAT_TotalNiagaraSystemInstances, STATGROUP_NiagaraSystemCounts);
DECLARE_DWORD_COUNTER_STAT(TEXT("Total Solo System Instances"), STAT_TotalNiagaraSystemInstancesSolo, STATGROUP_NiagaraSystemCounts);

DECLARE_CYCLE_STAT(TEXT("Collision"), STAT_NiagaraCollision, STATGROUP_Niagara);
