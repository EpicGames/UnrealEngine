// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemCollection.h"
#include "NiagaraSystem.h"

#include "Engine/StreamableManager.h"
#include "Engine/AssetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSystemCollection)

const TArray<TObjectPtr<UNiagaraSystem>>& FNiagaraSystemCollectionData::GetSystems()const
{
	check(IsInGameThread());
	if (AsyncLoadHandle)
	{
		AsyncLoadHandle->WaitUntilComplete();
		PostLoadSystems();
	}
	else
	{
		if (Systems.Num() != SystemsInternal.Num())
		{
			LoadSynchronous();
		}
	}

	return SystemsInternal;
}

void FNiagaraSystemCollectionData::InitFromArray(TConstArrayView<TSoftObjectPtr<UNiagaraSystem>> InSystems)
{
	check(Systems.Num() == 0);
	Systems.Append(InSystems);
}

void FNiagaraSystemCollectionData::LoadAsync()const
{
	if (!IsRunningDedicatedServer())
	{
		TArray<FSoftObjectPath> Requests;
		for (const TSoftObjectPtr<UNiagaraSystem>& SoftSys : Systems)
		{
			if (SoftSys.IsPending())
			{
				Requests.Add(SoftSys.ToSoftObjectPath());
			}
		}

		if (Requests.Num() > 0)
		{
			if (UAssetManager::IsInitialized())
			{
				AsyncLoadHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(Requests
					, FStreamableDelegate()
					, FStreamableManager::AsyncLoadHighPriority
					, false
					, false
					, TEXT("NiagaraDataChannelIsland_HandlerSystems"));
			}
			else
			{
				LoadSynchronous();
			}
		}
		else
		{
			PostLoadSystems();
		}
	}
}

void FNiagaraSystemCollectionData::LoadSynchronous()const
{
	if (!IsRunningDedicatedServer())
	{
		for (const TSoftObjectPtr<UNiagaraSystem>& SoftSys : Systems)
		{
			if (SoftSys.IsPending())
			{
				SoftSys.ToSoftObjectPath().TryLoad();
			}
		}
		PostLoadSystems();
	}
}

void FNiagaraSystemCollectionData::Release()
{
	AsyncLoadHandle->CancelHandle();
	AsyncLoadHandle.Reset();
	SystemsInternal.Empty();
}

void FNiagaraSystemCollectionData::PostLoadSystems()const
{
	check(IsInGameThread());
	SystemsInternal.Reset(Systems.Num());
	for (auto& SoftSys : Systems)
	{
		SystemsInternal.Add(SoftSys.Get());
	}

	AsyncLoadHandle.Reset();
}

//////////////////////////////////////////////////////////////////////////

void UNiagaraSystemCollection::PostLoad()
{
	Super::PostLoad();
	if(bLoadImmediately)
	{
		SystemCollection.LoadAsync();
	}
}
