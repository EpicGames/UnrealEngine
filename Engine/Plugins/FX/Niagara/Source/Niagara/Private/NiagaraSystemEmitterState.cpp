// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemEmitterState.h"
#include "NiagaraCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSystemEmitterState)

#if WITH_EDITORONLY_DATA
void FNiagaraEmitterStateData::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.CustomVer(FNiagaraCustomVersion::GUID) < FNiagaraCustomVersion::EmitterStateAddLoopDelayEnabled)
	{
		const FNiagaraStatelessRangeFloat LoopDelayRange = LoopDelay.CalculateRange();
		if ( LoopDelayRange.ParameterOffset != INDEX_NONE || !FMath::IsNearlyZero(LoopDelayRange.Min) || !FMath::IsNearlyZero(LoopDelayRange.Max) )
		{
			bLoopDelayEnabled = true;
		}
	}
}
#endif
