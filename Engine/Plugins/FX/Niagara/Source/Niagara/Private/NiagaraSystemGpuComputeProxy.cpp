// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemGpuComputeProxy.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGPUSystemTick.h"
#include "NiagaraRenderer.h"
#include "NiagaraSystemStaticBuffers.h"

FNiagaraSystemGpuComputeProxy::FNiagaraSystemGpuComputeProxy(FNiagaraSystemInstance* OwnerInstance)
	: DebugOwnerInstance(OwnerInstance)
	, SystemLWCTile(OwnerInstance->GetLWCTile())
	, SystemInstanceID(OwnerInstance->GetId())
{
	bRequiresGlobalDistanceField = OwnerInstance->RequiresGlobalDistanceField();
	bRequiresDepthBuffer = OwnerInstance->RequiresDepthBuffer();
	bRequiresEarlyViewData = OwnerInstance->RequiresEarlyViewData();
	bRequiresViewUniformBuffer = OwnerInstance->RequiresViewUniformBuffer();
	bRequiresRayTracingScene = OwnerInstance->RequiresRayTracingScene();
	bRequiresCurrentFrameNDC = OwnerInstance->RequiresCurrentFrameNDC();

	// Gather all emitter compute contexts
	for ( auto& Emitter : OwnerInstance->GetEmitters() )
	{
		if (FNiagaraComputeExecutionContext* ComputeContext = Emitter->GetGPUContext() )
		{
			ComputeContexts.Emplace(ComputeContext);
		}
		GpuCountBufferEstimate += Emitter->GetGpuCountBufferEstimate();
	}
	
	for (const auto& Pair : OwnerInstance->GPUDataInterfaces)
	{
		UNiagaraDataInterface* Interface = Pair.Key.Get();
		if (Interface == nullptr)
		{
			continue;
		}
		GpuCountBufferEstimate += Interface->GetGpuCountBufferEstimate();
	}

	// Calculate Tick Stage
	if (bRequiresGlobalDistanceField || bRequiresDepthBuffer || bRequiresRayTracingScene || bRequiresCurrentFrameNDC)
	{
		ComputeTickStage = ENiagaraGpuComputeTickStage::PostOpaqueRender;
	}
	else if (bRequiresEarlyViewData)
	{
		ComputeTickStage = ENiagaraGpuComputeTickStage::PostInitViews;
	}
	else if (bRequiresViewUniformBuffer)
	{
		ComputeTickStage = ENiagaraGpuComputeTickStage::PostOpaqueRender;
	}
	else
	{
		ComputeTickStage = ENiagaraGpuComputeTickStage::PreInitViews;
	}

	// Static buffers
	ENQUEUE_RENDER_COMMAND(SetStaticBuffers)(
		[RT_Context=this, RT_StaticBuffers=OwnerInstance->GetSystem()->GetStaticBuffers()](FRHICommandListImmediate& RHICmdList)
		{
			RT_Context->StaticFloatBuffer = RT_StaticBuffers->GetGpuFloatBuffer();
			if (!RT_Context->StaticFloatBuffer.IsValid())
			{
				RT_Context->StaticFloatBuffer = FNiagaraRenderer::GetDummyFloatBuffer();
			}
		}
	);
}

FNiagaraSystemGpuComputeProxy::~FNiagaraSystemGpuComputeProxy()
{
	check(IsInRenderingThread());
	check(DebugOwnerComputeDispatchInterface == nullptr);
}

void FNiagaraSystemGpuComputeProxy::AddToRenderThread(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface)
{
	check(IsInGameThread());
	check(DebugOwnerComputeDispatchInterface == nullptr);
	DebugOwnerComputeDispatchInterface = ComputeDispatchInterface;

	ENQUEUE_RENDER_COMMAND(AddProxyToComputeDispatchInterface)(
		[this, ComputeDispatchInterface](FRHICommandListImmediate& RHICmdList)
		{
			if (ComputeDispatchInterface)
			{
				ComputeDispatchInterface->AddGpuComputeProxy(this);
			}

			for (FNiagaraComputeExecutionContext* ComputeContext : ComputeContexts)
			{
				ComputeContext->bHasTickedThisFrame_RT = false;
				ComputeContext->SetCurrentNumInstances(0);
				ComputeContext->CurrentMaxInstances_RT = 0;

				ComputeContext->EmitterInstanceReadback.CPUCount = 0;
				ComputeContext->EmitterInstanceReadback.GPUCountOffset = INDEX_NONE;

				for (int i=0; i < UE_ARRAY_COUNT(ComputeContext->DataBuffers_RT); ++i)
				{
					check(ComputeContext->DataBuffers_RT[i] == nullptr);
					ComputeContext->DataBuffers_RT[i] = new FNiagaraDataBuffer(ComputeContext->MainDataSet);
				}
			}
		}
	);
}

void FNiagaraSystemGpuComputeProxy::RemoveFromRenderThread(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, bool bDeleteProxy)
{
	check(IsInGameThread());
	check(DebugOwnerComputeDispatchInterface == ComputeDispatchInterface);
	DebugOwnerComputeDispatchInterface = nullptr;

	ENQUEUE_RENDER_COMMAND(RemoveFromRenderThread)(
		[this, ComputeDispatchInterface, bDeleteProxy](FRHICommandListImmediate& RHICmdList)
		{
			if (ComputeDispatchInterface)
			{
				ComputeDispatchInterface->RemoveGpuComputeProxy(this);
			}
			ReleaseTicks(TNumericLimits<int32>::Max(), true);

			for (FNiagaraComputeExecutionContext* ComputeContext : ComputeContexts)
			{
				ComputeContext->ResetInternal(ComputeDispatchInterface);

				//-TODO: Can we move this inside the context???
				for (int i = 0; i < UE_ARRAY_COUNT(ComputeContext->DataBuffers_RT); ++i)
				{
					check(ComputeContext->DataBuffers_RT[i]);
					ComputeContext->DataBuffers_RT[i]->ReleaseGPU();
					ComputeContext->DataBuffers_RT[i]->Destroy();
					ComputeContext->DataBuffers_RT[i] = nullptr;
				}
			}

			if (bDeleteProxy)
			{
				delete this;
			}
		}
	);
}

void FNiagaraSystemGpuComputeProxy::ClearTicksFromRenderThread(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface)
{
	check(IsInGameThread());
	check(DebugOwnerComputeDispatchInterface == ComputeDispatchInterface);

	ENQUEUE_RENDER_COMMAND(ClearTicksFromProxy)(
		[this](FRHICommandListImmediate& RHICmdList)
		{
			ReleaseTicks(TNumericLimits<int32>::Max(), true);
		}
	);
}

void FNiagaraSystemGpuComputeProxy::QueueTick(FNiagaraGPUSystemTick& Tick)
{
	check(IsInRenderingThread());

	if(Tick.IsValid() == false)
	{
		Tick.Destroy();
		return;
	}

	//if ( !FNiagaraUtilities::AllowGPUParticles(GetShaderPlatform()) )
	//{
	//	return;
	//}

	//-OPT: Making a copy of the tick here, reduce this.
	PendingTicks.Add(Tick);

	// Consume DataInterface instance data
	//-TODO: This should be consumed as the command in executed rather than here, otherwise ticks are not processed correctly
	//       However an audit of all data interfaces is required to ensure they pass data safely, for example the skeletal mesh one does not
	if (Tick.DIInstanceData)
	{
		uint8* BasePointer = (uint8*)Tick.DIInstanceData->PerInstanceDataForRT;

		for (auto& Pair : Tick.DIInstanceData->InterfaceProxiesToOffsets)
		{
			FNiagaraDataInterfaceProxy* Proxy = Pair.Key;
			//if(Proxy->bConsumeCPUDataEarly)//TODO: Add path to optionally allow DIs to consume data in "Frames" rather than here when we consume the ticks.
			//{
				uint8* InstanceDataPtr = BasePointer + Pair.Value;
				Proxy->ConsumePerInstanceDataFromGameThread(InstanceDataPtr, Tick.SystemInstanceID);
			//}
		}
	}
}

void FNiagaraSystemGpuComputeProxy::ReleaseTicks(int32 NumTicksToRelease, bool bLastViewFamily)
{
	check(IsInRenderingThread());

	// Release all the ticks
	NumTicksToRelease = FMath::Min(NumTicksToRelease, PendingTicks.Num());
	for ( int32 iTick=0; iTick < NumTicksToRelease; ++iTick )
	{
		FNiagaraGPUSystemTick& Tick = PendingTicks[iTick];
		Tick.Destroy();
	}
	PendingTicks.RemoveAt(0, NumTicksToRelease, (NumTicksToRelease == PendingTicks.Num()) ? EAllowShrinking::Yes : EAllowShrinking::No);

	for (FNiagaraComputeExecutionContext* ComputeContext : ComputeContexts)
	{
		// Reset pending information as this will have the readback incorporated into it
		ComputeContext->bHasTickedThisFrame_RT = false;
		ComputeContext->CurrentMaxInstances_RT = 0;

		// This logic is deferred to the Niagara::MultiViewPreviousDataClear Pass for multi-view rendering
		if (bLastViewFamily)
		{
			// Clear counter offsets
			for (int i = 0; i < UE_ARRAY_COUNT(ComputeContext->DataBuffers_RT); ++i)
			{
				ComputeContext->DataBuffers_RT[i]->ClearGPUInstanceCount();
			}
		}
	}
}
