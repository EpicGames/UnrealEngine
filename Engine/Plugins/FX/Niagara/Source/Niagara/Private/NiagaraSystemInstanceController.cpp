// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemInstanceController.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/StaticMesh.h"
#include "Engine/TextureRenderTarget.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraSystemGpuComputeProxy.h"
#include "NiagaraWorldManager.h"

FNiagaraSystemInstanceController::FNiagaraSystemInstanceController()
	: bNeedsRendererRecache(false)
	, bNeedsOverrideParametersTicked(false)
	, bNeedsUpdateEmitterMaterials(false)
{
}

void FNiagaraSystemInstanceController::Initialize(UWorld& World, UNiagaraSystem& System, FNiagaraUserRedirectionParameterStore* InOverrideParameters,
	USceneComponent* AttachComponent, ENiagaraTickBehavior TickBehavior, bool bPooled, int32 RandomSeedOffset, bool bForceSolo, int32 InWarmupTickCount, float InWarmupTickDelta)
{
	OverrideParameters = InOverrideParameters;

	FNiagaraSystemInstance::AllocateSystemInstance(SystemInstance, World, System, OverrideParameters, AttachComponent, TickBehavior, bPooled);
	//UE_LOGF(LogNiagara, Log, "Create System: %p | %ls\n", SystemInstance.Get(), *GetAsset()->GetFullName());
	SystemInstance->SetRandomSeedOffset(RandomSeedOffset);
	SystemInstance->SetWarmupSettings(InWarmupTickCount, InWarmupTickDelta);
	SystemInstance->Init(bForceSolo);

	WorldManager = FNiagaraWorldManager::Get(&World);
	check(WorldManager);

	UpdateEmitterMaterials(); // On system reset we want to always reinit materials for now. Hopefully we can recycle the already created Mids.
}

void FNiagaraSystemInstanceController::Release()
{
	checkf(!SystemInstance.IsValid() || SystemInstance->IsComplete(), TEXT("NiagraSystemInstance must be complete when releasing the controller.  System(%s) Component(%s)"), *GetNameSafe(SystemInstance->GetSystem()), *GetFullNameSafe(SystemInstance->GetAttachComponent()));

	// Rather than setting the ptr to null here, we allow it to transition ownership to the system's deferred deletion queue. This allows us to safely
	// get rid of the system interface should we be doing this in response to a callback invoked during the system interface's lifetime completion cycle.
	FNiagaraSystemInstance::DeallocateSystemInstance(SystemInstance);
	WorldManager = nullptr;
	OverrideParameters = nullptr;
	OnMaterialsUpdatedDelegate.Unbind();
}

bool FNiagaraSystemInstanceController::HasValidSimulation() const
{
	if (SystemInstance.IsValid())
	{
		return SystemInstance->GetSystemSimulation().IsValid();
	}

	return false;
}

FNiagaraSystemRenderData* FNiagaraSystemInstanceController::CreateSystemRenderData(ERHIFeatureLevel::Type FeatureLevel) const
{
	if (auto SystemInst = SystemInstance.Get())
	{
		// We can't safely update emitter materials here because it can be called from non-game thread
		return new FNiagaraSystemRenderData(*this, *SystemInst, FeatureLevel);
	}
	return nullptr;
}

void FNiagaraSystemInstanceController::GenerateSetDynamicDataCommands(FNiagaraSystemRenderData::FSetDynamicDataCommandList& Commands, FNiagaraSystemRenderData& RenderData, const FNiagaraSceneProxy& SceneProxy)
{
	RenderData.GenerateSetDynamicDataCommands(Commands, SceneProxy, SystemInstance.Get(), EmitterMaterials);
}

void FNiagaraSystemInstanceController::PostTickRenderers(FNiagaraSystemRenderData& RenderData)
{
	if (auto SystemInst = SystemInstance.Get())
	{
		if (bNeedsOverrideParametersTicked)
		{
			bNeedsOverrideParametersTicked = false;
			OverrideParameters->ResolvePositions(SystemInst->GetLWCConverter());
			OverrideParameters->Tick();
		}
		if (bNeedsUpdateEmitterMaterials)
		{
			bNeedsUpdateEmitterMaterials = false;
			UpdateEmitterMaterials();
		}
		if (bNeedsRendererRecache)
		{
			RenderData.RecacheRenderers(*SystemInst, *this);
			bNeedsRendererRecache = false;
		}
		RenderData.PostTickRenderers(*SystemInst);
	}
}

void FNiagaraSystemInstanceController::NotifyRenderersComplete(FNiagaraSystemRenderData& RenderData)
{
	if (auto SystemInst = SystemInstance.Get())
	{
		RenderData.OnSystemComplete(*SystemInst);
	}
}

void FNiagaraSystemInstanceController::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& List) const
{
	if (!SystemInstance.IsValid())
	{
		return;
	}

	FMaterialInterfacePSOPrecacheParams NewEntry;
	NewEntry.PSOPrecacheParams.SetMobility(EComponentMobility::Movable);

	for (const FNiagaraEmitterInstanceRef& EmitterInst : SystemInstance->GetEmitters())
	{
		EmitterInst->ForEachEnabledRenderer(
			[&](const UNiagaraRendererProperties* Properties)
			{	
				Properties->CollectPSOPrecacheData(&EmitterInst.Get(), List);
			}
		);
	}
}

void FNiagaraSystemInstanceController::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials)
{
	if (!SystemInstance.IsValid())
	{
		return;
	}

	for (const FNiagaraEmitterInstanceRef& EmitterInst : SystemInstance->GetEmitters())
	{
		EmitterInst->ForEachEnabledRenderer(
			[&](const UNiagaraRendererProperties* Properties)
			{
				bool bCreateMidsForUsedMaterials = Properties->NeedsMIDsForMaterials();
				TArray<UMaterialInterface*> Mats;
				Properties->GetUsedMaterials(&EmitterInst.Get(), Mats);

				if (Properties->NeedsMIDsForMaterials())
				{
					for (const FMaterialOverride& Override : EmitterMaterials)
					{
						if (Override.EmitterRendererProperty == Properties && Mats.IsValidIndex(Override.MaterialSubIndex))
						{
							Mats[Override.MaterialSubIndex] = Override.Material;
						}
					}
				}

				OutMaterials.Append(Mats);
			}
		);
	}
}

void FNiagaraSystemInstanceController::GetMaterialStreamingInfo(FNiagaraMaterialAndScaleArray& OutMaterialAndScales) const
{
	if (!SystemInstance.IsValid())
	{
		return;
	}

	// If renderers read from parameters that might have changed in the parent we need to ensure it's flushed
	// Note that this can be called during AddPrimitive on concurrent threads
	if ( IsInGameThread() )
	{
		FNiagaraParameterStore& InstanceParameters = SystemInstance->GetInstanceParameters();
		const bool bInstancesDirty = InstanceParameters.GetInterfacesDirty() || InstanceParameters.GetUObjectsDirty();
		const bool bOverridesDirty = OverrideParameters && (OverrideParameters->GetInterfacesDirty() || OverrideParameters->GetUObjectsDirty());
		if (bInstancesDirty || bOverridesDirty)
		{
			SystemInstance->WaitForConcurrentTickDoNotFinalize();
			if (bOverridesDirty)
			{
				OverrideParameters->Tick();
			}
			if (bInstancesDirty)
			{
				InstanceParameters.Tick();
			}
		}
	}

	for (const FNiagaraEmitterInstanceRef& EmitterInst : SystemInstance->GetEmitters())
	{
		EmitterInst->GetRendererBoundVariables().Tick();

		EmitterInst->ForEachEnabledRenderer(
			[&](UNiagaraRendererProperties* Properties)
			{
				TArray<UMaterialInterface*> UsedMaterials;
				Properties->GetUsedMaterials(&EmitterInst.Get(), UsedMaterials);
				if (UsedMaterials.Num() == 0)
				{
					return;
				}

				if (Properties->NeedsMIDsForMaterials())
				{
					for (const FMaterialOverride& Override : EmitterMaterials)
					{
						if (Override.EmitterRendererProperty == Properties && UsedMaterials.IsValidIndex(Override.MaterialSubIndex))
						{
							UsedMaterials[Override.MaterialSubIndex] = Override.Material;
						}
					}
				}

				const float StreamingScale = Properties->GetMaterialStreamingScale();
				for (UMaterialInterface* UsedMaterial : UsedMaterials)
				{
					if (UsedMaterial == nullptr)
					{
						continue;
					}

					if (FNiagaraMaterialAndScale* Existing = OutMaterialAndScales.FindByPredicate([UsedMaterial](const FNiagaraMaterialAndScale& Existing) { return Existing.Material == UsedMaterial; }))
					{
						Existing->Scale = FMath::Max(Existing->Scale, StreamingScale);
					}
					else
					{
						OutMaterialAndScales.Emplace(UsedMaterial, StreamingScale);
					}
				}
			}
		);
	}
}

void FNiagaraSystemInstanceController::GetStreamingMeshInfo(const FBoxSphereBounds& OwnerBounds, FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	if (!SystemInstance.IsValid())
	{
		return;
	}

	for (const FNiagaraEmitterInstanceRef& EmitterInst : SystemInstance->GetEmitters())
	{
		EmitterInst->ForEachEnabledRenderer(
			[&](const UNiagaraRendererProperties* Properties)
			{
				Properties->GetStreamingMeshInfo(OwnerBounds, &EmitterInst.Get(), OutStreamingRenderAssets);
			}
		);
	}
}

UMaterialInterface* FNiagaraSystemInstanceController::GetMaterialOverride(const UNiagaraRendererProperties* InProps, int32 InMaterialSubIndex) const
{
	for (const FMaterialOverride& Override : EmitterMaterials)
	{
		if (Override.EmitterRendererProperty == InProps && InMaterialSubIndex == Override.MaterialSubIndex)
		{
			return Override.Material;
		}
	}

	return nullptr;
}

ENiagaraGpuComputeTickStage::Type FNiagaraSystemInstanceController::GetGpuComputeTickStage() const
{
	ENiagaraGpuComputeTickStage::Type TickStage = ENiagaraGpuComputeTickStage::PreInitViews;
	if (SystemInstance.IsValid())
	{
		if (FNiagaraSystemGpuComputeProxy* GpuComputeProxy = SystemInstance->GetSystemGpuComputeProxy())
		{
			return GpuComputeProxy->GetComputeTickStage();
		}
	}
	return TickStage;
}

void FNiagaraSystemInstanceController::DebugDump(bool bFullDump)
{
	if (SystemInstance.IsValid())
	{
		const UEnum* ExecutionStateEnum = StaticEnum<ENiagaraExecutionState>();
		UE_LOGF(LogNiagara, Log, "\tSystem ExecutionState(%ls) RequestedExecutionState(%ls)", *ExecutionStateEnum->GetNameStringByIndex((int32)SystemInstance->GetActualExecutionState()), *ExecutionStateEnum->GetNameStringByIndex((int32)SystemInstance->GetRequestedExecutionState()));

		if (!SystemInstance->IsComplete())
		{
			for (const FNiagaraEmitterInstanceRef& Emitter : SystemInstance->GetEmitters())
			{
				if ( Emitter->GetEmitter() != nullptr )
				{
					UE_LOGF(LogNiagara, Log, "\tEmitter '%ls' ExecutionState(%ls) NumParticles(%d)", *Emitter->GetEmitterHandle().GetUniqueInstanceName(), *ExecutionStateEnum->GetNameStringByIndex((int32)Emitter->GetExecutionState()), Emitter->GetNumParticles());
				}
			}
		}

		if (bFullDump)
		{
			UE_LOGF(LogNiagara, Log, "=========================== Begin Full Dump ===========================");
			SystemInstance->Dump();
			UE_LOGF(LogNiagara, Log, "=========================== End Full Dump ===========================");
		}
	}
}

SIZE_T FNiagaraSystemInstanceController::GetTotalBytesUsed() const
{
	SIZE_T Size = 0;
	if (IsValid())
	{
		for (const FNiagaraEmitterInstanceRef& Emitter : GetSystemInstance_Unsafe()->GetEmitters())
		{
			Size += Emitter->GetTotalBytesUsed();
		}
	}

	return Size;
}

void FNiagaraSystemInstanceController::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& Override : EmitterMaterials)
	{
		Collector.AddReferencedObject(Override.Material);

		// NOTE: Intentionally *not* adding a reference to EmitterRendererProperty because it's only used as an identifier
	}
}

void FNiagaraSystemInstanceController::UpdateEmitterMaterials()
{
	check(IsInRenderingThread() || IsInGameThread() || IsAsyncLoading() || UE::IsSavingPackage()); // Same restrictions as MIDs

	TArray<FMaterialOverride> NewEmitterMaterials;
	if (SystemInstance)
	{
		for (int32 i = 0; i < SystemInstance->GetEmitters().Num(); i++)
		{
			FNiagaraEmitterInstance* EmitterInst = &SystemInstance->GetEmitters()[i].Get();
			EmitterInst->ForEachEnabledRenderer(
				[&](const UNiagaraRendererProperties* Properties)
				{
					// Nothing to do if we don't create MIDs for this material
					if (!Properties->NeedsMIDsForMaterials())
					{
						return;
					}

					TArray<UMaterialInterface*> UsedMaterials;
					Properties->GetUsedMaterials(EmitterInst, UsedMaterials);

					uint32 MaterialIndex = 0;
					for (UMaterialInterface*& ExistingMaterial : UsedMaterials)
					{
						if (ExistingMaterial)
						{
							if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(ExistingMaterial))
							{
								if (EmitterMaterials.FindByPredicate([&](const FMaterialOverride& ExistingOverride) -> bool { return (ExistingOverride.Material == ExistingMaterial) && (ExistingOverride.EmitterRendererProperty == Properties) && (ExistingOverride.MaterialSubIndex == MaterialIndex); }))
								{
									// It's a MID we've previously created and are managing. Recreate it by grabbing the parent.
									// TODO: Are there cases where we don't always have to recreate it?
									ExistingMaterial = MID->Parent;
								}
								else
								{
									// If we get here this is an external MID so do not create a new one
									continue;
								}
							}

							// Create a new MID
							//UE_LOGF(LogNiagara, Log, "Create Dynamic Material for component %ls", *GetPathName());
							ExistingMaterial = UMaterialInstanceDynamic::Create(ExistingMaterial, nullptr);
							FMaterialOverride Override;
							Override.Material = ExistingMaterial;
							Override.EmitterRendererProperty = Properties;
							Override.MaterialSubIndex = MaterialIndex;

							NewEmitterMaterials.Add(Override);
						}
						++MaterialIndex;
					}
				}
			);
		}
	}
	EmitterMaterials = MoveTemp(NewEmitterMaterials);
	OnMaterialsUpdatedDelegate.ExecuteIfBound();
}

void FNiagaraSystemInstanceController::SetVariable_InternalUseOnly_Deferred(const FNiagaraVariableBase& Variable, const FNiagaraVariant& Variant)
{
	SetVariable_Deferred(Variable, Variant);
}

void FNiagaraSystemInstanceController::SetVariable(FNiagaraVariableBase Variable, FNiagaraVariant Variant)
{
	if (OverrideParameters && Variant.IsValid())
	{
		//TODO: Unify this modification code with the typed setters so it doesn't diverge/become a maintainance headache.
		bool bIsMaterial = Variable.GetType() == FNiagaraTypeDefinition::GetUMaterialDef();
		bool bIsStaticMesh = Variable.GetType() == FNiagaraTypeDefinition::GetUStaticMeshDef();
		bool bUpdateMaterialsIfModified = bIsMaterial || bIsStaticMesh;
		bool bMarkNeedsTickOnModified = bIsStaticMesh;
		bool bMarkNeedsRendererRecacheOnModified = bIsStaticMesh;
		bool bCheckModified = Variant.GetMode() == ENiagaraVariantMode::Object && (bUpdateMaterialsIfModified || bMarkNeedsRendererRecacheOnModified || bMarkNeedsTickOnModified);

		bool bUObjectModified = false;
		if (bCheckModified)
		{
			UObject* CurrValue = OverrideParameters->GetUObject(Variable);
			bUObjectModified = CurrValue != Variant.GetUObject();
		}

		OverrideParameters->SetParameter_InternalUseOnly(Variable, Variant, true);

		//Handle any modification updates for some uobject types.
		if (bUObjectModified)
		{
			if (bUpdateMaterialsIfModified)
			{
				bNeedsUpdateEmitterMaterials = true;
			}
			if (bMarkNeedsTickOnModified)
			{
				bNeedsOverrideParametersTicked = true;
			}
			if (bMarkNeedsRendererRecacheOnModified)
			{
				OnNeedsRendererRecache();
			}
		}
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, bool InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetBoolDef(), InVariableName);
		const FNiagaraBool BoolValue(InValue);
		OverrideParameters->SetParameterValue(BoolValue, VariableDesc, true);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, int32 InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetIntDef(), InVariableName);
		OverrideParameters->SetParameterValue(InValue, VariableDesc, true);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, float InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetFloatDef(), InVariableName);
		OverrideParameters->SetParameterValue(InValue, VariableDesc, true);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, FVector2f InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetVec2Def(), InVariableName);
		OverrideParameters->SetParameterValue(InValue, VariableDesc, true);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, FVector3f InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetVec3Def(), InVariableName);
		OverrideParameters->SetParameterValue(InValue, VariableDesc, true);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, FVector InValue)
{
	if (OverrideParameters)
	{
		OverrideParameters->SetPositionParameterValue(InValue, InVariableName, true);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, FVector4f InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetVec4Def(), InVariableName);
		OverrideParameters->SetParameterValue(InValue, VariableDesc, true);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, FLinearColor InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetColorDef(), InVariableName);
		OverrideParameters->SetParameterValue(InValue, VariableDesc, true);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, FQuat4f InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetQuatDef(), InVariableName);
		OverrideParameters->SetParameterValue(InValue, VariableDesc, true);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, const FMatrix44f& InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetMatrix4Def(), InVariableName);
		OverrideParameters->SetParameterValue(InValue, VariableDesc, true);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, TWeakObjectPtr<UObject> InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetUObjectDef(), InVariableName);
		OverrideParameters->SetUObject(InValue.Get(), VariableDesc);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, TWeakObjectPtr<UMaterialInterface> InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetUMaterialDef(), InVariableName);
		TObjectPtr<UObject> CurrentValue = OverrideParameters->GetUObject(VariableDesc);
		UMaterialInterface* NewValue = InValue.Get();
		OverrideParameters->SetUObject(NewValue, VariableDesc);
		if (CurrentValue != NewValue)
		{
			bNeedsUpdateEmitterMaterials = true; // Will need to update our internal tables. Maybe need a new MID.
		}
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, TWeakObjectPtr<UStaticMesh> InValue)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetUStaticMeshDef(), InVariableName);
		TObjectPtr<UObject> CurrentValue = OverrideParameters->GetUObject(VariableDesc);
		UStaticMesh* NewValue = InValue.Get();
		OverrideParameters->SetUObject(NewValue, VariableDesc);
		if (CurrentValue != NewValue)
		{
			bNeedsOverrideParametersTicked = true;
			bNeedsUpdateEmitterMaterials = true;
			OnNeedsRendererRecache();
		}
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, TWeakObjectPtr<UTexture> Texture)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetUTextureDef(), InVariableName);
		OverrideParameters->SetUObject(Texture.Get(), VariableDesc);
	}
}

void FNiagaraSystemInstanceController::SetVariable(FName InVariableName, TWeakObjectPtr<UTextureRenderTarget> TextureRenderTarget)
{
	if (OverrideParameters)
	{
		const FNiagaraVariable VariableDesc(FNiagaraTypeDefinition::GetUTextureRenderTargetDef(), InVariableName);
		OverrideParameters->SetUObject(TextureRenderTarget.Get(), VariableDesc);
	}
}
