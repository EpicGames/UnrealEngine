// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemMemReport.h"

#include "NiagaraDataInterface.h"
#include "NiagaraEmitter.h"
#include "Misc/StringBuilder.h"
#include "UObject/UObjectIterator.h"

#include "Serialization/ArchiveCountMem.h"

#include "NiagaraSystem.h"
#include "NiagaraSystemImpl.h"

namespace NiagaraMemReportInternal
{
	const FName NAME_ByteCode("ByteCode");
	const FName NAME_RapidIterationParameters("RapidIterationParameters");
	const FName NAME_ScriptExecutionParamStore("ScriptExecutionParamStore");

	static FAutoConsoleCommandWithWorldAndArgs CmdReportSystemMem(
		TEXT("NiagaraReportSystemMemory"),
		TEXT("Dumps some rough information about system memory breakdown"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
			[](const TArray<FString>& Args, UWorld*)
			{
				FNiagaraSystemMemReport::EReportType ReportType = FNiagaraSystemMemReport::EReportType::Basic;
				FString WildcardFilter;
	
				static const FString FilterPrefix("-filter=");
				static const FString TypePrefix("-type=");
				for (const FString& Arg : Args)
				{
					if (Arg.StartsWith(FilterPrefix))
					{
						WildcardFilter = Arg.Mid(FilterPrefix.Len());
					}
					else if (Arg.StartsWith(TypePrefix))
					{
						FString ValueString = Arg.Mid(TypePrefix.Len());
						ReportType = FNiagaraSystemMemReport::EReportType(FMath::Clamp(FCString::Atoi(*ValueString), 0, int32(FNiagaraSystemMemReport::EReportType::Max) - 1));
					}
				}
				
				UE_LOGF(LogNiagara, Log, "============ Niagara MemReport");
				for (TObjectIterator<UNiagaraSystem> SystemIt; SystemIt; ++SystemIt)
				{
					UNiagaraSystem* NiagaraSystem = *SystemIt;
					if (::IsValid(NiagaraSystem) == false)
					{
						continue;
					}

					if (!WildcardFilter.IsEmpty())
					{
						const FString PathName = NiagaraSystem->GetPathName();
						if ( !PathName.MatchesWildcard(WildcardFilter) )
						{
							continue;
						}
					}

					FNiagaraSystemMemReport MemReport;
					MemReport.GenerateReport(ReportType, NiagaraSystem);
					for ( const FNiagaraSystemMemReport::FNode& Node : MemReport.GetNodes() )
					{
						UE_LOGF(LogNiagara, Log, "%*ls = %ls %ls (%u, %u)", Node.Depth, TEXT(""), *Node.ObjectClass.ToString(), *Node.ObjectName.ToString(), uint32(Node.ExclusiveSizeBytes), uint32(Node.InclusiveSizeBytes));
					}
					UE_LOGF(LogNiagara, Log, "");
				}
			}
		)
	);

#if WITH_EDITORONLY_DATA
	void InternalHandleProperty(TSet<UObject*>& EditorOnlyObjects, TSet<UObject*>& RuntimeObjects, const FProperty* Property, const void* Container, bool bIsEditorOnly, int32 ArrayIndex = 0)
	{
		UObject* FoundObject = nullptr;
		if (const FWeakObjectProperty* WeakObjectProperty = CastField<const FWeakObjectProperty>(Property))
		{
			const FWeakObjectPtr& Value = WeakObjectProperty->GetPropertyValue_InContainer(Container, ArrayIndex);
			FoundObject = Value.Get();
		}
		else if (const FObjectProperty* ObjectProperty = CastField<const FObjectProperty>(Property))
		{
			const TObjectPtr<UObject>& Value = ObjectProperty->GetPropertyValue_InContainer(Container, ArrayIndex);
			FoundObject = Value.Get();
		}
		else if (const FSoftObjectProperty* SoftObjectProperty = CastField<const FSoftObjectProperty>(Property))
		{
			const FSoftObjectPtr& Value = SoftObjectProperty->GetPropertyValue_InContainer(Container, ArrayIndex);
			FoundObject = Value.Get();
		}

		if (FoundObject != nullptr)
		{
			if (bIsEditorOnly)
			{
				EditorOnlyObjects.Add(FoundObject);
			}
			else
			{
				RuntimeObjects.Add(FoundObject);
			}
		}
	}

	void InternalFindSubObjects(TSet<UObject*>& EditorOnlyObjects, TSet<UObject*>& RuntimeObjects, const UStruct* Struct, const void* Container, bool bIsParentEditorOnly)
	{
		for (TFieldIterator<const FProperty> PropertyIt(Struct); PropertyIt; ++PropertyIt)
		{
			const FProperty* Property = *PropertyIt;
			const bool bIsEditorOnly = Property->IsEditorOnlyProperty() || bIsParentEditorOnly;

			if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
			{
				if ( StructProperty->Struct )
				{
					InternalFindSubObjects(EditorOnlyObjects, RuntimeObjects, StructProperty->Struct, StructProperty->ContainerPtrToValuePtr<void>(Container, 0), bIsEditorOnly);
				}
			}
			else if (const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(Property))
			{
				const FProperty* ArrayInnerProp = ArrayProperty->Inner;

				if (ArrayInnerProp && (CastField<const FWeakObjectProperty>(ArrayInnerProp) || CastField<const FObjectProperty>(ArrayInnerProp) ||CastField<const FSoftObjectProperty>(ArrayInnerProp)))
				{
					FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Container, 0));
					for (int32 i=0; i < ArrayHelper.Num(); ++i)
					{
						const void* ArrayItemContainer = ArrayHelper.GetRawPtr(i);
						InternalHandleProperty(EditorOnlyObjects, RuntimeObjects, ArrayInnerProp, ArrayItemContainer, bIsEditorOnly);
					}
				}
			}
			else
			{
				InternalHandleProperty(EditorOnlyObjects, RuntimeObjects, Property, Container, bIsEditorOnly);
			}
		}
	}

	void RemoveEditorOnlyObjects(TArray<UObject*>& SubObjects, UObject* Object)
	{
		SubObjects.RemoveAll([](UObject* SubObject) { return SubObject->IsEditorOnly() || (!SubObject->NeedsLoadForClient() && !SubObject->NeedsLoadForServer()); });
		if (SubObjects.Num() == 0)
		{
			return;
		}

		TSet<UObject*> EditorOnlyObjects;
		TSet<UObject*> RuntimeObjects;
		InternalFindSubObjects(EditorOnlyObjects, RuntimeObjects, Object->GetClass(), Object, false);

		SubObjects.RemoveAll([&EditorOnlyObjects, &RuntimeObjects](UObject* SubObject) { return EditorOnlyObjects.Contains(SubObject) && !RuntimeObjects.Contains(SubObject); });
	}
#endif
}

FNiagaraSystemMemReport::FNode::FNode(UObject* Object, uint32 InDepth)
{
	FResourceSizeEx ResourceSize(EResourceSizeMode::Exclusive);
	ResourceSize.AddDedicatedSystemMemoryBytes(FArchiveCountMem(Object, true).GetMax());
	Object->GetResourceSizeEx(ResourceSize);

	ObjectName			= Object->GetFName();
	ObjectClass			= Object->GetClass()->GetFName();
	Depth				= InDepth;
	ExclusiveSizeBytes	= ResourceSize.GetTotalMemoryBytes();
	InclusiveSizeBytes	= ResourceSize.GetTotalMemoryBytes();

	if (UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(Object))
	{
		ObjectName = FName(*Emitter->GetUniqueEmitterName());
	}
}

FNiagaraSystemMemReport::FNode::FNode(FName CustomName, uint32 InDepth, uint32 ByteSize)
{
	ObjectName			= CustomName;
	ObjectClass			= NAME_None;
	Depth				= InDepth;
	ExclusiveSizeBytes	= ByteSize;
	InclusiveSizeBytes	= ByteSize;
}

void FNiagaraSystemMemReport::GenerateReport(EReportType ReportType, UNiagaraSystem* System)
{
	Nodes.Empty();
	DataInterfaceSizeBytes = 0;
	GatherResourceMemory(ReportType, System, System, 0);
}

uint64 FNiagaraSystemMemReport::GatherResourceMemory(EReportType ReportType, UNiagaraSystem* OwningSystem, UObject* Object, uint32 Depth)
{
	using namespace NiagaraMemReportInternal;

	const int32 NodeIndex = Nodes.Emplace(Object, Depth);

	if (ReportType == EReportType::Verbose)
	{
		if (UNiagaraScript* NiagaraScript = Cast<UNiagaraScript>(Object))
		{
			{
				const FNiagaraVMExecutableData& VMData = NiagaraScript->GetVMExecutableData();
				Nodes.Emplace(NAME_ByteCode, Depth + 1, VMData.ByteCode.GetLength());
				Nodes[NodeIndex].ExclusiveSizeBytes -= Nodes.Last().ExclusiveSizeBytes;
			}

			Nodes.Emplace(NAME_RapidIterationParameters, Depth + 1, NiagaraScript->RapidIterationParameters.GetResourceSize());
			Nodes[NodeIndex].ExclusiveSizeBytes -= Nodes.Last().ExclusiveSizeBytes;

			ENiagaraSimTarget SimTarget = ENiagaraSimTarget::CPUSim;
			FGuid EmitterHandleId;
			const FVersionedNiagaraEmitter VersionedNiagaraEmitter = NiagaraScript->GetOuterEmitter();
			if (const FVersionedNiagaraEmitterData* EmitterData = VersionedNiagaraEmitter.GetEmitterData())
			{
				SimTarget = EmitterData->SimTarget;
				const FNiagaraEmitterHandle* EmitterHandle = OwningSystem->GetEmitterHandles().FindByPredicate([&EmitterData]
					(const FNiagaraEmitterHandle& EmitterHandle) { return EmitterHandle.GetEmitterData() == EmitterData; });
				EmitterHandleId = EmitterHandle != nullptr ? EmitterHandle->GetId() : FGuid();
			}

			const FNiagaraScriptRuntimeData* ScriptRuntimeData = OwningSystem->GetScriptRuntimeData(EmitterHandleId, NiagaraScript->GetUsage(), NiagaraScript->GetUsageId()).Get();
			if (const FNiagaraScriptExecutionParameterStore* ExecParameterStore = ScriptRuntimeData != nullptr ? ScriptRuntimeData->GetExecutionReadyParameterStore() : nullptr)
			{
				Nodes.Emplace(NAME_ScriptExecutionParamStore, Depth + 1, ExecParameterStore->GetResourceSize());
				Nodes[NodeIndex].ExclusiveSizeBytes -= Nodes.Last().ExclusiveSizeBytes;
			}
		}
	}

	TArray<UObject*> SubObjects;
	GetObjectsWithOuter(Object, SubObjects, EGetObjectsFlags::None);
	if (SubObjects.Num() > 0)
	{
	#if WITH_EDITORONLY_DATA
		RemoveEditorOnlyObjects(SubObjects, Object);
	#endif

		for (UObject* SubObject : SubObjects)
		{
			Nodes[NodeIndex].InclusiveSizeBytes += GatherResourceMemory(ReportType, OwningSystem, SubObject, Depth + 1);
		}
	}

	// Keep track of data interface memory
	if (Object->IsA<UNiagaraDataInterface>())
	{
		DataInterfaceSizeBytes += Nodes[NodeIndex].InclusiveSizeBytes;
	}

	// If in basic mode remove all the nodes we added below us as we don't want them
	const uint64 InclusiveSizeBytes = Nodes[NodeIndex].InclusiveSizeBytes;
	if (ReportType == EReportType::Basic)
	{
		const int32 NumToRemove = Nodes.Num() - NodeIndex;
		if (NumToRemove > 0 && !Object->IsA<UNiagaraSystem>() && !Object->IsA<UNiagaraEmitter>())
		{
			Nodes.RemoveAt(NodeIndex, NumToRemove);
		}
	}

	return InclusiveSizeBytes;
}
