// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemRenderData.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraCrashReporterHandler.h"
#include "NiagaraSystem.h"
#include "PrimitiveViewRelevance.h"
#include "SceneView.h"
#include "Rendering/RenderCommandPipes.h"

DECLARE_RENDER_COMMAND_PIPE(NiagaraDynamicData, );

void FNiagaraSystemRenderData::ExecuteDynamicDataCommands_RenderThread(const FSetDynamicDataCommandList& Commands)
{
	for (auto& Command : Commands)
	{
		Command.Execute();
	}
}

FNiagaraSystemRenderData::FNiagaraSystemRenderData(const FNiagaraSystemInstanceController& SystemInstanceController, const FNiagaraSystemInstance& SystemInstance, ERHIFeatureLevel::Type InFeatureLevel)
{
	UNiagaraSystem* System = SystemInstance.GetSystem();
	check(System);
	FeatureLevel = InFeatureLevel;

	RendererDrawOrder = System->GetRendererDrawOrder();
	RecacheRenderers(SystemInstance, SystemInstanceController);
}

FNiagaraSystemRenderData::~FNiagaraSystemRenderData()
{
	if(EmitterRenderers_RT.Num() > 0)
	{
		Destroy_RenderThread();
	}
}

void FNiagaraSystemRenderData::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	for (auto Renderer : EmitterRenderers_RT)
	{
		if (Renderer)
		{
			Renderer->CreateRenderThreadResources(RHICmdList);
		}
	}
}

void FNiagaraSystemRenderData::ReleaseRenderThreadResources()
{
	for (auto Renderer : EmitterRenderers_RT)
	{
		if (Renderer)
		{
			Renderer->ReleaseRenderThreadResources();
		}
	}
}

void FNiagaraSystemRenderData::DestroyRenderState_Concurrent()
{
	// Give the renderers an opportunity to free resources that must be freed on the game thread (or task)
	for (auto Renderer : EmitterRenderers_GT)
	{
		if (Renderer)
		{
			Renderer->DestroyRenderState_Concurrent();
		}
	}

	// Don't delete the renderers on game/concurrent threads, they have to be deleted on the render thread
	EmitterRenderers_GT.Reset();
}

void FNiagaraSystemRenderData::Destroy_RenderThread()
{
	// Delete the renderers
	for (auto Renderer : EmitterRenderers_RT)
	{
		if (Renderer)
		{
			Renderer->ReleaseRenderThreadResources();
			delete Renderer;
		}
	}

	EmitterRenderers_RT.Reset();
}

FPrimitiveViewRelevance FNiagaraSystemRenderData::GetViewRelevance(const FSceneView& View, const FNiagaraSceneProxy& SceneProxy) const
{
	FPrimitiveViewRelevance Relevance;
	if (IsRenderingEnabled_RT())
	{
		for (const FNiagaraRenderer* Renderer : EmitterRenderers_RT)
		{
			if (Renderer)
			{
				Relevance |= Renderer->GetViewRelevance(&View, &SceneProxy);
			}
		}
	}

	return Relevance;
}

uint32 FNiagaraSystemRenderData::GetDynamicDataSize() const
{
	// only the render thread should use the RT data, the game thread and other parallel running tasks use the GT data
	const TArray<FNiagaraRenderer*>* LocalEmitterRenderers;
	if ( IsInParallelGameThread() || IsInGameThread() )
	{
		LocalEmitterRenderers = &EmitterRenderers_GT;
	}
	else
	{
		check(IsInParallelRenderingThread());
		LocalEmitterRenderers = &EmitterRenderers_RT;
	}

	uint32 Size = 0;
	for (const auto Renderer : *LocalEmitterRenderers)
	{
		if (Renderer)
		{
			Size += Renderer->GetDynamicDataSize();
		}
	}

	return Size;
}

void FNiagaraSystemRenderData::GenerateSetDynamicDataCommands(FSetDynamicDataCommandList& Commands, const FNiagaraSceneProxy& SceneProxy, const FNiagaraSystemInstance* SystemInstance, TConstArrayView<FMaterialOverride> MaterialOverrides)
{
	if (!SystemInstance)
	{
		for (auto Renderer : EmitterRenderers_GT)
		{
			if (Renderer)
			{
				Commands.Add(FSetDynamicDataCommand(Renderer, nullptr));
			}
		}

		return;
	}

	FNiagaraCrashReporterScope CRScope(SystemInstance);
	UNiagaraSystem* System = SystemInstance->GetSystem();
	check(System);

#if STATS
	TStatId SystemStatID = System ? System->GetStatID(true, true) : TStatId();
	FScopeCycleCounter SystemStatCounter(SystemStatID);
#endif

	const int32 NumEmitterRenderers = EmitterRenderers_GT.Num();
	if (NumEmitterRenderers == 0)
	{
		// Early out if we have no renderers
		return;
	}

	Commands.Reserve(NumEmitterRenderers);

	int32 RendererIndex = 0;
	for (int32 i = 0; i < SystemInstance->GetEmitters().Num(); i++)
	{
		FNiagaraEmitterInstance* EmitterInst = &SystemInstance->GetEmitters()[i].Get();
#if STATS
		TStatId EmitterStatID = EmitterInst->GetEmitterStatID(true, true);
		FScopeCycleCounter EmitterStatCounter(EmitterStatID);
#endif

		if (!EmitterInst->GetEmitterHandle().GetIsEnabled())
		{
			continue;
		}

		EmitterInst->ForEachEnabledRenderer(
			[&](const UNiagaraRendererProperties* Properties)
			{
			#if WITH_EDITOR
				if ( RendererIndex >= NumEmitterRenderers )
				{
					++RendererIndex;
					return;
				}
			#endif
				FNiagaraRenderer* Renderer = EmitterRenderers_GT[RendererIndex];
				FNiagaraDynamicDataBase* NewData = nullptr;

				if (Renderer && Properties->GetIsActive())
				{
					bool bRendererEditorEnabled = true;
				#if WITH_EDITORONLY_DATA
					bRendererEditorEnabled = EmitterInst->IsDisabledFromIsolation() == false;
				#endif
					if (bRendererEditorEnabled && !EmitterInst->IsComplete() && !SystemInstance->IsComplete())
					{
						NewData = Renderer->GenerateDynamicData(&SceneProxy, Properties, EmitterInst);

						if (NewData)
						{
							for (const FMaterialOverride& MaterialOverride : MaterialOverrides)
							{
								if (MaterialOverride.EmitterRendererProperty == Properties)
								{
									NewData->ApplyMaterialOverride(MaterialOverride.MaterialSubIndex, MaterialOverride.Material);
								}
							}
						}
					}

					Commands.Add(FSetDynamicDataCommand(Renderer, NewData));
				}

				++RendererIndex;
			}
		);
	}

#if WITH_EDITOR
	if (ensure(RendererIndex == NumEmitterRenderers) == false)
	{
		// This can happen in the editor when modifying the number or renderers while the system is running and the render thread is already processing the data.
		// in this case we just skip drawing this frame since the system will be reinitialized.
		if ( RendererIndex > NumEmitterRenderers )
		{
			Commands.SetNum(NumEmitterRenderers);
		}
	}
#endif
}

void FNiagaraSystemRenderData::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector, const FNiagaraSceneProxy& SceneProxy)
{
	for (int32 RendererIdx : RendererDrawOrder)
	{
		FNiagaraRenderer* Renderer = EmitterRenderers_RT[RendererIdx];
		if (Renderer && (Renderer->GetSimTarget() != ENiagaraSimTarget::GPUComputeSim || FNiagaraUtilities::AllowGPUParticles(ViewFamily.GetShaderPlatform())))
		{
			Renderer->GetDynamicMeshElements(Views, ViewFamily, VisibilityMap, Collector, &SceneProxy);
		}
	}
}

#if RHI_RAYTRACING
void FNiagaraSystemRenderData::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FNiagaraSceneProxy& SceneProxy)
{
	for (auto Renderer : EmitterRenderers_RT)
	{
		if (Renderer)
		{
			Renderer->GetDynamicRayTracingInstances(Collector, &SceneProxy);
		}
	}
}
#endif

void FNiagaraSystemRenderData::GatherSimpleLights(FSimpleLightArray& OutParticleLights) const
{
	for (auto Renderer : EmitterRenderers_RT)
	{
		if (Renderer && Renderer->HasLights())
		{
			Renderer->GatherSimpleLights(OutParticleLights);
		}
	}
}

void FNiagaraSystemRenderData::RecacheRenderers(const FNiagaraSystemInstance& SystemInstance, const FNiagaraSystemInstanceController& Controller)
{
	DestroyRenderState_Concurrent();

	EmitterRenderers_GT.Reserve(RendererDrawOrder.Num());

	bAnyMotionBlurEnabled = false;
	bAnyHeterogeneousVolumesEnabled = false;
	for (const FNiagaraEmitterInstanceRef& EmitterInst : SystemInstance.GetEmitters())
	{
		if (!EmitterInst->GetEmitterHandle().GetIsEnabled())
		{
			continue;
		}

		EmitterInst->ForEachEnabledRenderer(
			[&](UNiagaraRendererProperties* Properties)
			{
				//We can skip creation of the renderer if the current quality level doesn't support it. If the quality level changes all systems are fully reinitialized.
				FNiagaraRenderer* NewRenderer = nullptr;
				if (Properties->GetIsActive() && EmitterInst->GetParticleData().IsInitialized() && !EmitterInst->IsDisabled())
				{
					NewRenderer = Properties->CreateEmitterRenderer(FeatureLevel, &EmitterInst.Get(), Controller);
					if (Properties->MotionVectorSetting != ENiagaraRendererMotionVectorSetting::Disable)
					{
						bAnyMotionBlurEnabled = true;
					}
					if (Properties->UseHeterogeneousVolumes())
					{
						bAnyHeterogeneousVolumesEnabled = true;
					}
				}
				EmitterRenderers_GT.Add(NewRenderer);
			}
		);
	}

	// If we have renderers then the draw order on the system should match, when compiling the number of renderers can be zero
#if WITH_EDITOR
	// In editor builds we may have cases where modifications to the enabled state is inside undo / redo and we have no pre/post calls to execute an update context.
	// We can handle these invalid cases by clearing the renderers and logging an ensure, when we have a solid repro we can investigate fixing, or we can separate out editor vs runtime data.
	if (!ensureMsgf(EmitterRenderers_GT.Num() == 0|| (EmitterRenderers_GT.Num() == RendererDrawOrder.Num()), TEXT("EmitterRenderers Num %d does not match System DrawOrder %d"), EmitterRenderers_GT.Num(), RendererDrawOrder.Num()))
	{
		EmitterRenderers_GT.Empty();
	}
#else
	// In cooked builds the enabled state will not be mutated so any failures here are fatals
	checkf((EmitterRenderers_GT.Num() == 0) || (EmitterRenderers_GT.Num() == RendererDrawOrder.Num()), TEXT("EmitterRenderers Num %d does not match System DrawOrder %d"), EmitterRenderers_GT.Num(), RendererDrawOrder.Num());
#endif

	UE::Tasks::FTaskEvent TaskEvent{ UE_SOURCE_LOCATION };

	ENQUEUE_RENDER_COMMAND(SignalFence)(UE::RenderCommandPipe::NiagaraDynamicData, [TaskEvent] () mutable
	{
		TaskEvent.Trigger();
	});

	// NOTE: Since this object and its render thread resources may be concurrently accessed on the render thread, we have to create the renderers and pass them off to
	// replace the current ones on the render thread's time line
	ENQUEUE_RENDER_COMMAND(NiagaraRecacheRenderers)(
		[this, EmitterRenderers_Copy=EmitterRenderers_GT, TaskEvent = MoveTemp(TaskEvent)](FRHICommandListBase& RHICmdList) mutable
		{
			TaskEvent.Wait();

			// Release/destroy the current renderers
			Destroy_RenderThread();

			EmitterRenderers_RT = MoveTemp(EmitterRenderers_Copy);
		}
	);
}

void FNiagaraSystemRenderData::PostTickRenderers(const FNiagaraSystemInstance& SystemInstance)
{
	// Give renderers a chance to do some processing PostTick
	if (EmitterRenderers_GT.Num() > 0)
	{
		const UNiagaraSystem* System = SystemInstance.GetSystem();
		for (const FNiagaraRendererExecutionIndex& ExecIdx : System->GetRendererPostTickOrder())
		{
			if (EmitterRenderers_GT.IsValidIndex(ExecIdx.SystemRendererIndex))
			{
				const FNiagaraEmitterInstance& EmitterInst = SystemInstance.GetEmitters()[ExecIdx.EmitterIndex].Get();
				if ( !EmitterInst.IsComplete() )
				{
					if ( FNiagaraRenderer* EmitterRenderer = EmitterRenderers_GT[ExecIdx.SystemRendererIndex] )
					{
						const UNiagaraRendererProperties* RendererProperties = EmitterInst.GetRenderer(ExecIdx.EmitterRendererIndex);
						EmitterRenderer->PostSystemTick_GameThread(RendererProperties, &EmitterInst);
					}
				}
			}
		}
	}
}

void FNiagaraSystemRenderData::SetRenderingEnabled_GT(bool bInEnabled)
{
	check(IsInGameThread());

	if (bRenderingEnabled_GT != bInEnabled)
	{
		bRenderingEnabled_GT = bInEnabled;
		
		// Note we send immediately because there is no guarantee we would push new data this frame
		ENQUEUE_RENDER_COMMAND(FNiagaraSystemRenderData_SetRenderingEnabled)(UE::RenderCommandPipe::NiagaraDynamicData,
			[this]()
			{
				bRenderingEnabled_RT = bRenderingEnabled_GT;
			}
		);
	}
}

void FNiagaraSystemRenderData::OnSystemComplete(const FNiagaraSystemInstance& SystemInstance)
{
	if (EmitterRenderers_GT.Num() == 0)
	{
		return;
	}

	const UNiagaraSystem* System = SystemInstance.GetSystem();
	TConstArrayView<FNiagaraEmitterInstanceRef> EmitterInstances = SystemInstance.GetEmitters();
	if (EmitterInstances.Num() == 0)
	{
		return;
	}

	for (const FNiagaraRendererExecutionIndex& ExecIdx : System->GetRendererCompletionOrder())
	{
		if (EmitterRenderers_GT.IsValidIndex(ExecIdx.SystemRendererIndex) && ensure(EmitterInstances.IsValidIndex(ExecIdx.EmitterIndex)) )
		{
			const FNiagaraEmitterInstance& EmitterInstance = EmitterInstances[ExecIdx.EmitterIndex].Get();
			if (FNiagaraRenderer* EmitterRenderer = EmitterRenderers_GT[ExecIdx.SystemRendererIndex])
			{
				if (const UNiagaraRendererProperties* RendererProperties = EmitterInstance.GetRenderer(ExecIdx.EmitterRendererIndex))
				{
					EmitterRenderer->OnSystemComplete_GameThread(RendererProperties, &EmitterInstance);
				}
			}
		}
	}
}
