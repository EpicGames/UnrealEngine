// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemSimulation.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraModule.h"
#include "NiagaraTypes.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraParameterCollectionResolver.h"
#include "NiagaraSimCache.h"
#include "NiagaraSimulationTaskPriority.h"
#include "NiagaraSystemEmitterState.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemGpuComputeProxy.h"
#include "NiagaraConstants.h"
#include "NiagaraStats.h"
#include "NiagaraComponent.h"
#include "NiagaraWorldManager.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGPUSystemTick.h"
#include "NiagaraCrashReporterHandler.h"

#include "Misc/LargeWorldRenderPosition.h"

// Niagara Async Ticking Sequence
// - NiagaraSystemSimulation::Tick_GameThread
//  - Enqueue simulation concurrent tick (FNiagaraSystemSimulationTickConcurrentTask), tracks task event in instances
// - NiagaraSystemSimulation::Tick_Concurrent
//  - Enqueue instance concurrent ticks in batches (FNiagaraSystemInstanceTickConcurrentTask), tracks task inside instances
//  - Enqueue finalize tasks (FNiagaraSystemInstanceFinalizeTask), tracks finalize inside instances
//  - Appends all finalize tasks to a completion task (FNiagaraSystemSimulationAllWorkCompleteTask), when complete everything is done, used to track tick task completion
//
// Niagara Async Spawning Sequence
// - NiagaraSystemSimulation::Spawn_GameThread
//  - Enqueue simulation spawn concurrent FNiagaraSystemSimulationSpawnConcurrentTask, tracks task event in instances
// - NiagaraSystemSimulation::Spawn_Concurrent
//  - Enqueue instance concurrent ticks in batches (FNiagaraSystemInstanceTickConcurrentTask), they can not execute until Spawn_Concurrent is complete, tracks the task inside instances
//  - Enqueue finalize tasks (FNiagaraSystemInstanceFinalizeTask), tracks finalize inside instances
//  - Appends all finalize tasks to a completion task (FNiagaraSystemSimulationAllWorkCompleteTask), when complete everything is done

#define NIAGARA_SYSTEMSIMULATION_DEBUGGING 0

//High level stats for system sim tick.
DECLARE_CYCLE_STAT(TEXT("System Simulation Tick [GT]"), STAT_NiagaraSystemSim_TickGT, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("System Simulation Tick [CNC]"), STAT_NiagaraSystemSim_TickCNC, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("System Simulation SpawnNew [GT]"), STAT_NiagaraSystemSim_SpawnNewGT, STATGROUP_Niagara);
//Some more detailed stats for system sim tick
DECLARE_CYCLE_STAT(TEXT("System Prepare For Simulate [CNC]"), STAT_NiagaraSystemSim_PrepareForSimulateCNC, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("System Sim Update [CNC]"), STAT_NiagaraSystemSim_UpdateCNC, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("System Sim Spawn [CNC]"), STAT_NiagaraSystemSim_SpawnCNC, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("System Sim Transfer Results [CNC]"), STAT_NiagaraSystemSim_TransferResultsCNC, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("System Sim Init [GT]"), STAT_NiagaraSystemSim_Init, STATGROUP_Niagara);

DECLARE_CYCLE_STAT(TEXT("System Sim Init (DataSets) [GT]"), STAT_NiagaraSystemSim_Init_DataSets, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("System Sim Init (ExecContexts) [GT]"), STAT_NiagaraSystemSim_Init_ExecContexts, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("System Sim Init (BindParams) [GT]"), STAT_NiagaraSystemSim_Init_BindParams, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("System Sim Init (DatasetAccessors) [GT]"), STAT_NiagaraSystemSim_Init_DatasetAccessors, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("System Sim Init (DirectBindings) [GT]"), STAT_NiagaraSystemSim_Init_DirectBindings, STATGROUP_Niagara);

DECLARE_CYCLE_STAT(TEXT("ForcedWaitForAsync"), STAT_NiagaraSystemSim_ForceWaitForAsync, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("ForcedWait Fake Stall"), STAT_NiagaraSystemSim_ForceWaitFakeStall, STATGROUP_Niagara);

static int32 GbDumpSystemData = 0;
static FAutoConsoleVariableRef CVarNiagaraDumpSystemData(
	TEXT("fx.DumpSystemData"),
	GbDumpSystemData,
	TEXT("If > 0, results of system simulations will be dumped to the log. \n"),
	ECVF_Default
);

static int32 GNiagaraSystemSimulationUpdateOnSpawn = 1;
static FAutoConsoleVariableRef CVarSystemUpdateOnSpawn(
	TEXT("fx.Niagara.SystemSimulation.UpdateOnSpawn"),
	GNiagaraSystemSimulationUpdateOnSpawn,
	TEXT("If > 0, system simulations are given a small update after spawn. \n"),
	ECVF_Default
);

static int32 GNiagaraSystemSimulationAllowASync = 1;
static FAutoConsoleVariableRef CVarNiagaraSystemSimulationAllowASync(
	TEXT("fx.Niagara.SystemSimulation.AllowASync"),
	GNiagaraSystemSimulationAllowASync,
	TEXT("If > 0, system simulation tick is parallelized. \n"),
	ECVF_Default
);

static int32 GNiagaraSystemSimulationAllowASyncSimCache = 1;
static FAutoConsoleVariableRef CVarNiagaraSystemSimulationAllowASyncSimCache(
	TEXT("fx.Niagara.SystemSimulation.AllowASyncSimCache"),
	GNiagaraSystemSimulationAllowASyncSimCache,
	TEXT("If > 0, sim cache tick is parallelized. \n"),
	ECVF_Default
);

int32 GNiagaraSystemSimulationTaskStallTimeout = 0;
static FAutoConsoleVariableRef CVarNiagaraSystemSimulationTaskStallTimeout(
	TEXT("fx.Niagara.SystemSimulation.TaskStallTimeout"),
	GNiagaraSystemSimulationTaskStallTimeout,
	TEXT("Timeout in microseconds for Niagara simulation tasks to be considered stalled.\n")
	TEXT("When this is > 0 we busy wait as opposed to joining the TG so avoid using execpt for debugging."),
	ECVF_Default
);

static int32 GNiagaraSystemSimulationTickBatchSize = NiagaraSystemTickBatchSize;
static FAutoConsoleVariableRef CVarNiagaraSystemSimulationTickBatchSize(
	TEXT("fx.Niagara.SystemSimulation.TickBatchSize"),
	GNiagaraSystemSimulationTickBatchSize,
	TEXT("The number of system instances to process per async task. \n"),
	ECVF_Default
);

static int32 GNiagaraSystemSimulationTickBatchMode = 1;
static FAutoConsoleVariableRef CVarNiagaraSystemSimulationTickBatchMode(
	TEXT("fx.Niagara.SystemSimulation.TickBatchMode"),
	GNiagaraSystemSimulationTickBatchMode,
	TEXT("Controls what we do with each batch of instances in a simulation")
	TEXT("0 - Spawn tasks for all batches.")
	TEXT("1 - Spawn tasks for all but the final batch.")
	TEXT("2 - Run all instance inline with the main simulation task."),
	ECVF_Default
);

static int32 GNiagaraSystemSimulationConcurrentGPUTickInit = 1;
static FAutoConsoleVariableRef CVarNiagaraConcurrentGPUTickInit(
	TEXT("fx.Niagara.SystemSimulation.ConcurrentGPUTickInit"),
	GNiagaraSystemSimulationConcurrentGPUTickInit,
	TEXT("The if non zero we allow GPU Ticks to be initialized in the System's concurrent tick rather than on the game thread."),
	ECVF_Default
);

static int32 GNiagaraSystemSimulationBatchGPUTickSubmit = 1;
static FAutoConsoleVariableRef CVarNiagaraBatchGPUTickSubmit(
	TEXT("fx.Niagara.SystemSimulation.BatchGPUTickSubmit"),
	GNiagaraSystemSimulationBatchGPUTickSubmit,
	TEXT("The if non zero we allow GPU Ticks to be submitted to the Render Thread in batches."),
	ECVF_Default
);

static float GNiagaraSystemSimulationSkipTickDeltaSeconds = 0.0f;
static FAutoConsoleVariableRef CVarNiagaraSkipTickDeltaSeconds(
	TEXT("fx.Niagara.SystemSimulation.SkipTickDeltaSeconds"),
	GNiagaraSystemSimulationSkipTickDeltaSeconds,
	TEXT("When none zero we skip all ticks with a delta seconds less than equal to this number."),
	ECVF_Default
);

static int32 GNiagaraSystemSimulationMaxTickSubsteps = 100;
static FAutoConsoleVariableRef CVarNiagaraSystemSimulationMaxTickSubsteps(
	TEXT("fx.Niagara.SystemSimulation.MaxTickSubsteps"),
	GNiagaraSystemSimulationMaxTickSubsteps,
	TEXT("The max number of possible substeps per frame when a system uses a fixed tick delta."),
	ECVF_Default
);

namespace NiagaraSystemSimulationLocal
{
	static bool GTickTaskShouldWait = false;
	static FAutoConsoleVariableRef CVarGTickTaskShouldWait(
		TEXT("fx.Niagara.SystemSimulation.TickTaskShouldWait"),
		GTickTaskShouldWait,
		TEXT("When enabled the tick task will wait for concurrent work to complete, when disabled the task is complete once the GT tick is complete."),
		ECVF_Default
	);

	static int32 GTickTaskAllowFrameOverlap = false;
	static FAutoConsoleVariableRef CVarTickTaskAllowFrameOverlap(
		TEXT("fx.Niagara.SystemSimulation.TickTaskAllowFrameOverlap"),
		GTickTaskAllowFrameOverlap,
		TEXT("When enabled we allow ticks to overlap beyond PostActorTick until either EOF updates or the next tick."),
		ECVF_Default
	);

#if NIAGARA_SYSTEMSIMULATION_DEBUGGING
	static int GDebugKillAllOnSpawn = 0;
	static FAutoConsoleVariableRef CVarDebugKillAllOnSpawn(
		TEXT("fx.Niagara.SystemSimulation.DebugKillAllOnSpawn"),
		GDebugKillAllOnSpawn,
		TEXT("Will randomly kill all spawning instances based on probability of 0-100."),
		ECVF_Default
	);

	static int GDebugKillAllOnUpdate = 0;
	static FAutoConsoleVariableRef CVarDebugKillAllOnUpdate(
		TEXT("fx.Niagara.SystemSimulation.DebugKillAllOnUpdate"),
		GDebugKillAllOnUpdate,
		TEXT("Will randomly kill all updating instances based on probability of 0-100."),
		ECVF_Default
	);

	static int GDebugKillInstanceOnTick = 0;
	static FAutoConsoleVariableRef CVarDebugKillInstanceOnTick(
		TEXT("fx.Niagara.SystemSimulation.DebugKillInstanceOnTick"),
		GDebugKillInstanceOnTick,
		TEXT("Will randomly kill an instance on tick based on probability of 0-100."),
		ECVF_Default
	);

	static float GDebugForceDelayConcurrentTask = 0.0f;
	static FAutoConsoleVariableRef CVarDebugForceDelayConcurrentTask(
		TEXT("fx.Niagara.SystemSimulation.DebugForceDelayConcurrentTask"),
		GDebugForceDelayConcurrentTask,
		TEXT("Forces a time delay into the concurrent task."),
		ECVF_Default
	);

	static float GDebugForceDelayInstancesTask = 0.0f;
	static FAutoConsoleVariableRef CVarDebugForceDelayInstancesTask(
		TEXT("fx.Niagara.SystemSimulation.DebugForceDelayInstancesTask"),
		GDebugForceDelayInstancesTask,
		TEXT("Forces a time delay into the instances task."),
		ECVF_Default
	);

	bool DebugKillAllOnSpawn(FNiagaraSystemSimulationTickContext& Context)
	{
		if (GDebugKillAllOnSpawn > 0)
		{
			if ((FMath::Rand() % 100) < GDebugKillAllOnSpawn)
			{
				for (FNiagaraSystemInstance* SystemInst : Context.Instances)
				{
					SystemInst->SetActualExecutionState(ENiagaraExecutionState::Disabled);
				}
				Context.DataSet.EndSimulate();
				return true;
			}
		}
		return false;
	}

	bool DebugKillAllOnUpdate(FNiagaraSystemSimulationTickContext& Context)
	{
		if (GDebugKillAllOnUpdate > 0)
		{
			if ((FMath::Rand() % 100) < GDebugKillAllOnUpdate)
			{
				for (FNiagaraSystemInstance* SystemInst : Context.Instances)
				{
					SystemInst->SetActualExecutionState(ENiagaraExecutionState::Disabled);
				}
				Context.DataSet.EndSimulate();
				return true;
			}
		}
		return false;
	}

	void DebugKillInstanceOnTick(FNiagaraSystemInstance* Instance)
	{
		if (GDebugKillInstanceOnTick > 0)
		{
			if ((FMath::Rand() % 100) < GDebugKillInstanceOnTick)
			{
				Instance->Complete(true);
			}
		}
	}

	void DebugDelayConcurrentTask()
	{
		if (GDebugForceDelayConcurrentTask > 0.0f)
		{
			FPlatformProcess::Sleep(GDebugForceDelayConcurrentTask);
		}
	}

	void DebugDelayInstancesTask()
	{
		if (GDebugForceDelayInstancesTask > 0.0f)
		{
			FPlatformProcess::Sleep(GDebugForceDelayInstancesTask);
		}
	}
#endif
}

//////////////////////////////////////////////////////////////////////////

#if WITH_PER_COMPONENT_PARTICLE_PERF_STATS
FORCEINLINE FParticlePerfStats* GetInstancePerfStats(FNiagaraSystemInstance* Inst) { UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(Inst->GetAttachComponent()); return NiagaraComponent ? NiagaraComponent->ParticlePerfStats : nullptr; }
#else
FORCEINLINE FParticlePerfStats* GetInstancePerfStats(FNiagaraSystemInstance* Inst) { return nullptr; }
#endif

//////////////////////////////////////////////////////////////////////////
// Task used to determine when all work is complete, i.e. system simulation concurrent, system instance concurrent, finalize
// Do not wait on this in the GameThread as it may deadlock if we are within a task
struct FNiagaraSystemSimulationAllWorkCompleteTask
{
	FNiagaraSystemSimulationAllWorkCompleteTask(FGraphEventArray*& OutEventsToWaitFor)
	{
		OutEventsToWaitFor = &EventsToWaitFor;
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraSystemSimulationAllWorkCompleteTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::GameThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
		for (FGraphEventRef& Event : EventsToWaitFor)
		{
			MyCompletionGraphEvent->DontCompleteUntil(Event);
		}
		EventsToWaitFor.Empty();
	}

	FGraphEventArray EventsToWaitFor;
};
//////////////////////////////////////////////////////////////////////////
// Task to run FNiagaraSystemSimulation::Tick_Concurrent
struct FNiagaraSystemSimulationTickConcurrentTask
{
	FNiagaraSystemSimulationTickConcurrentTask(FNiagaraSystemSimulationTickContext InContext, FGraphEventRef& CompletionGraphEvent)
		: Context(InContext)
	{
		TaskThread = NiagaraSimulationTaskPriority::GetTickGroupPriority(Context.Owner->GetTickGroup(), Context.System->AsyncWorkCanOverlapTickGroups());
		CompletionTask = TGraphTask<FNiagaraSystemSimulationAllWorkCompleteTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndHold(Context.CompletionEvents);
		CompletionGraphEvent = CompletionTask->GetCompletionEvent();
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraSystemSimulationTickConcurrentTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return TaskThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(Context.World, Context.System), TickConcurrent);
#if NIAGARA_SYSTEMSIMULATION_DEBUGGING
		NiagaraSystemSimulationLocal::DebugDelayConcurrentTask();
#endif
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);

		Context.Owner->Tick_Concurrent(Context);
		CompletionTask->Unlock();
	}

	ENamedThreads::Type TaskThread;
	FNiagaraSystemSimulationTickContext Context;
	TGraphTask<FNiagaraSystemSimulationAllWorkCompleteTask>* CompletionTask = nullptr;
};

//////////////////////////////////////////////////////////////////////////
// Task to run FNiagaraSystemSimulation::Spawn_Concurrent
struct FNiagaraSystemSimulationSpawnConcurrentTask
{
	FNiagaraSystemSimulationSpawnConcurrentTask(FNiagaraSystemSimulationTickContext InContext, FGraphEventRef& CompletionGraphEvent)
		: Context(InContext)
	{
		TaskThread = NiagaraSimulationTaskPriority::GetPostActorTickPriority();
		CompletionTask = TGraphTask<FNiagaraSystemSimulationAllWorkCompleteTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndHold(Context.CompletionEvents);
		CompletionGraphEvent = CompletionTask->GetCompletionEvent();
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraSystemSimulationSpawnConcurrentTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return TaskThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(Context.World, Context.System), TickConcurrent);
#if NIAGARA_SYSTEMSIMULATION_DEBUGGING
		NiagaraSystemSimulationLocal::DebugDelayConcurrentTask();
#endif
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);

		Context.BeforeInstancesTickGraphEvents.Add(MyCompletionGraphEvent);
		Context.Owner->Spawn_Concurrent(Context);
		Context.BeforeInstancesTickGraphEvents.Empty();
		CompletionTask->Unlock();
	}

	ENamedThreads::Type TaskThread;
	FNiagaraSystemSimulationTickContext Context;
	TGraphTask<FNiagaraSystemSimulationAllWorkCompleteTask>* CompletionTask = nullptr;
};

//////////////////////////////////////////////////////////////////////////
// Task to run FNiagaraSystemInstance::Tick_Concurrent
struct FNiagaraSystemInstanceTickConcurrentTask
{
	FNiagaraSystemInstanceTickConcurrentTask(FNiagaraSystemSimulation* InSystemSimulation, FNiagaraSystemTickBatch& InBatch, UWorld* InWorldContext)
		: SystemSimulation(InSystemSimulation)
		, Batch(InBatch)
		, WorldContext(InWorldContext)
	{
		TaskThread = NiagaraSimulationTaskPriority::GetTickGroupPriority(SystemSimulation->GetTickGroup(), SystemSimulation->GetSystem()->AsyncWorkCanOverlapTickGroups());
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraSystemInstanceTickConcurrentTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return TaskThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(WorldContext, SystemSimulation->GetSystem()), TickConcurrent);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);

#if NIAGARA_SYSTEMSIMULATION_DEBUGGING
		NiagaraSystemSimulationLocal::DebugDelayInstancesTask();
#endif

		if (SystemSimulation->GetGPUTickHandlingMode() == ENiagaraGPUTickHandlingMode::ConcurrentBatched)
		{
			TArray<TPair<FNiagaraSystemGpuComputeProxy*, FNiagaraGPUSystemTick>, TInlineAllocator<NiagaraSystemTickBatchSize>> GPUTicks;
			GPUTicks.Reserve(Batch.Num());
			for (FNiagaraSystemInstance* Inst : Batch)
			{
				PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetInstancePerfStats(Inst)), TickConcurrent);
				Inst->Tick_Concurrent(false);
				if (Inst->NeedsGPUTick())
				{
					auto& Tick = GPUTicks.AddDefaulted_GetRef();
					Tick.Key = Inst->GetSystemGpuComputeProxy();
					Inst->InitGPUTick(Tick.Value);
				}
			}

			if (GPUTicks.Num() > 0)
			{
				ENQUEUE_RENDER_COMMAND(FNiagaraGiveSystemInstanceTickToRT)(
					[GPUTicks](FRHICommandListImmediate& RHICmdList) mutable
					{
						for (auto& GPUTick : GPUTicks)
						{
							GPUTick.Key->QueueTick(GPUTick.Value);
						}
					}
				);
			}
		}
		else
		{
			for (FNiagaraSystemInstance* Inst : Batch)
			{
				PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetInstancePerfStats(Inst)), TickConcurrent);
				Inst->Tick_Concurrent();
			}
		}
	}

	ENamedThreads::Type TaskThread;
	FNiagaraSystemSimulation* SystemSimulation = nullptr;
	FNiagaraSystemTickBatch Batch;
	UWorld* WorldContext = nullptr;
};

//////////////////////////////////////////////////////////////////////////
// Task to run FNiagaraSystemInstance::Finalize_GameThread
struct FNiagaraSystemInstanceFinalizeTask
{
	FNiagaraSystemInstanceFinalizeTask(FNiagaraSystemSimulation* InSystemSimulation, FNiagaraSystemTickBatch& InBatch)
		: SystemSimulation(InSystemSimulation)
		, Batch(InBatch)
		, TickHandlingMode(SystemSimulation->GetGPUTickHandlingMode())
	{
#if DO_CHECK
		DebugCounter = 0;
#endif
#if WITH_PARTICLE_PERF_STATS
		SystemPerfStatsContext = FParticlePerfStatsContext(InSystemSimulation->GetWorld(), InSystemSimulation->GetSystem());
#endif
		for ( int32 i=0; i < Batch.Num(); ++i )
		{
			FNiagaraSystemInstanceFinalizeRef FinalizeRef(&Batch[i]);
#if DO_CHECK
			FinalizeRef.SetDebugCounter(&DebugCounter);
#endif
			Batch[i]->SetPendingFinalize(FinalizeRef);
		}
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraSystemInstanceFinalizeTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::GameThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
#if WITH_PARTICLE_PERF_STATS
		PARTICLE_PERF_STAT_CYCLES_GT(SystemPerfStatsContext, Finalize);
#endif
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
		check(CurrentThread == ENamedThreads::GameThread);

		if ( TickHandlingMode == ENiagaraGPUTickHandlingMode::GameThreadBatched )
		{
			TArray<TPair<FNiagaraSystemGpuComputeProxy*, FNiagaraGPUSystemTick>, TInlineAllocator<NiagaraSystemTickBatchSize>> GPUTicks;
			GPUTicks.Reserve(Batch.Num());
			for (FNiagaraSystemInstance* Instance : Batch)
			{
				if (Instance == nullptr )
				{
					continue;
				}

				PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetInstancePerfStats(Instance)), Finalize);
				Instance->FinalizeTick_GameThread(false);
				if(Instance->NeedsGPUTick())
				{
					auto& Tick = GPUTicks.AddDefaulted_GetRef();
					Tick.Key = Instance->GetSystemGpuComputeProxy();
					Instance->InitGPUTick(Tick.Value);
				}
			}

			if(GPUTicks.Num() > 0)
			{
				ENQUEUE_RENDER_COMMAND(FNiagaraGiveSystemInstanceTickToRT)(
					[GPUTicks](FRHICommandListImmediate& RHICmdList) mutable
					{
						for(auto& GPUTick : GPUTicks)
						{
							GPUTick.Key->QueueTick(GPUTick.Value);
						}
					}
				);
			}
		}
		else
		{
			for (FNiagaraSystemInstance* Instance : Batch)
			{
				if (Instance == nullptr)
				{
					continue;
				}

				PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(GetInstancePerfStats(Instance)), Finalize);
				Instance->FinalizeTick_GameThread();
			}
		}
#if DO_CHECK
		ensureMsgf(DebugCounter.load() == 0, TEXT("Finalize batch is complete but counter is %d when it should be zero"), DebugCounter.load());
#endif
	}

	FNiagaraSystemSimulation* SystemSimulation = nullptr;
	FNiagaraSystemTickBatch Batch;
	ENiagaraGPUTickHandlingMode TickHandlingMode = ENiagaraGPUTickHandlingMode::None;
#if DO_CHECK
	std::atomic<int> DebugCounter;
#endif
#if WITH_PARTICLE_PERF_STATS
	FParticlePerfStatsContext SystemPerfStatsContext;
#endif
};

//////////////////////////////////////////////////////////////////////////

FNiagaraSystemSimulationTickContext::FNiagaraSystemSimulationTickContext(class FNiagaraSystemSimulation* InOwner, TArray<FNiagaraSystemInstance*>& InInstances, FNiagaraDataSet& InDataSet, float InDeltaSeconds, int32 InSpawnNum, bool bAllowAsync)
	: Owner(InOwner)
	, System(InOwner->GetSystem())
	, World(InOwner->GetWorld())
	, Instances(InInstances)
	, DataSet(InDataSet)
	, DeltaSeconds(InDeltaSeconds)
	, SpawnNum(InSpawnNum)
{
	static const auto EffectsQualityCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("sg.EffectsQuality"));
	check(EffectsQualityCVar != nullptr);
	EffectsQuality = EffectsQualityCVar->GetInt();

	bRunningAsync = bAllowAsync && GNiagaraSystemSimulationAllowASync && FApp::ShouldUseThreadingForPerformance();

#if NIAGARA_SYSTEM_CAPTURE
	if ( Owner->GetIsSolo() && Instances.Num() == 1 )
	{
		bRunningAsync &= Instances[0]->ShouldCaptureThisFrame() == false;
	}
#endif
}

//////////////////////////////////////////////////////////////////////////

FNiagaraSystemSimulation::FNiagaraSystemSimulation()
	: EffectType(nullptr)
	, SystemTickGroup(TG_MAX)
	, World(nullptr)
	, bCanExecute(false)
	, bBindingsInitialized(false)
	, bInSpawnPhase(false)
	, bIsSolo(false)
{

}

FNiagaraSystemSimulation::~FNiagaraSystemSimulation()
{
	Destroy();
}

bool FNiagaraSystemSimulation::Init(UNiagaraSystem* InSystem, UWorld* InWorld, bool bInIsSolo, ETickingGroup InTickGroup, const FNiagaraParameterCollectionResolver& ParameterCollectionResolver)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_Init);
	System = InSystem;

	EffectType = InSystem->GetEffectType();
	SystemTickGroup = InTickGroup;

	World = InWorld;

	bIsSolo = bInIsSolo;

	bBindingsInitialized = false;
	bInSpawnPhase = false;

	FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(InWorld);
	check(WorldMan);

	DispatchInterface = FNiagaraGpuComputeDispatchInterface::Get(World);

	bCanExecute = System->GetSystemSpawnScript()->GetVMExecutableData().IsValid() && System->GetSystemUpdateScript()->GetVMExecutableData().IsValid();

	{
		const FNiagaraSystemStateData& SystemState = System->GetSystemStateData();
		bRunSpawnScript		= SystemState.bRunSpawnScript;
		bRunUpdateScript	= SystemState.bRunUpdateScript;
	}

	MaxDeltaTime = System->GetMaxDeltaTime();

	if (bCanExecute)
	{
		{
			//SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_Init_DataSets);

			const FNiagaraSystemCompiledData& SystemCompiledData = System->GetSystemCompiledData();
			//Initialize the main simulation dataset.
			MainDataSet.Init(&SystemCompiledData.DataSetCompiledData);

			//Initialize the main simulation dataset.
			SpawningDataSet.Init(&SystemCompiledData.DataSetCompiledData);

			//Initialize the dataset for paused systems.
			PausedDataSet.Init(&SystemCompiledData.DataSetCompiledData);
			
			SpawnInstanceParameterDataSet.Init(&SystemCompiledData.SpawnInstanceParamsDataSetCompiledData);

			UpdateInstanceParameterDataSet.Init(&SystemCompiledData.UpdateInstanceParamsDataSetCompiledData);
		}

		UNiagaraScript* SpawnScript = System->GetSystemSpawnScript();
		UNiagaraScript* UpdateScript = System->GetSystemUpdateScript();

		{
			//SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_Init_ExecContexts);
			SpawnExecContext = MakeUnique<FNiagaraSystemScriptExecutionContext>(ENiagaraSystemSimulationScript::Spawn);
			UpdateExecContext = MakeUnique<FNiagaraSystemScriptExecutionContext>(ENiagaraSystemSimulationScript::Update);
			bCanExecute &= SpawnExecContext->Init(nullptr, SpawnScript, System->GetScriptRuntimeData(nullptr, *SpawnScript));
			bCanExecute &= UpdateExecContext->Init(nullptr, UpdateScript, System->GetScriptRuntimeData(nullptr, *UpdateScript));
		}

		{
			//SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_Init_BindParams);

			ParameterCollectionResolver.BindCollections(SpawnScript, SpawnExecContext->Parameters);
			ParameterCollectionResolver.BindCollections(UpdateScript, UpdateExecContext->Parameters);
			SpawnScript->RapidIterationParameters.Bind(&SpawnExecContext->Parameters);
			UpdateScript->RapidIterationParameters.Bind(&UpdateExecContext->Parameters);
		}

		{
			//SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_Init_DirectBindings);

			SpawnNumSystemInstancesParam.Init(SpawnExecContext->Parameters, SYS_PARAM_ENGINE_NUM_SYSTEM_INSTANCES);
			UpdateNumSystemInstancesParam.Init(UpdateExecContext->Parameters, SYS_PARAM_ENGINE_NUM_SYSTEM_INSTANCES);
			SpawnGlobalSpawnCountScaleParam.Init(SpawnExecContext->Parameters, SYS_PARAM_ENGINE_GLOBAL_SPAWN_COUNT_SCALE);
			UpdateGlobalSpawnCountScaleParam.Init(UpdateExecContext->Parameters, SYS_PARAM_ENGINE_GLOBAL_SPAWN_COUNT_SCALE);
			SpawnGlobalSystemCountScaleParam.Init(SpawnExecContext->Parameters, SYS_PARAM_ENGINE_GLOBAL_SYSTEM_COUNT_SCALE);
			UpdateGlobalSystemCountScaleParam.Init(UpdateExecContext->Parameters, SYS_PARAM_ENGINE_GLOBAL_SYSTEM_COUNT_SCALE);
		}
	}

	return true;
}

void FNiagaraSystemSimulation::Destroy()
{
	check(IsInGameThread());
	WaitForInstancesTickComplete();

	auto DeactivateInstances =
		[](TArray<FNiagaraSystemInstance*>& Instances)
		{
			while (Instances.Num() > 0)
			{
				FNiagaraSystemInstance* Instance = Instances.Last();
				Instance->Deactivate(true);		
				check(Instance->SystemInstanceState == ENiagaraSystemInstanceState::None);
			}
		};

	DeactivateInstances(GetSystemInstances(ENiagaraSystemInstanceState::Running));
	DeactivateInstances(GetSystemInstances(ENiagaraSystemInstanceState::Paused));
	DeactivateInstances(GetSystemInstances(ENiagaraSystemInstanceState::PendingSpawn));
	DeactivateInstances(GetSystemInstances(ENiagaraSystemInstanceState::PendingSpawnPaused));

	check(GetSystemInstances(ENiagaraSystemInstanceState::Spawning).Num() == 0);

	// Can be nullptr if bCanExecute is false
	if (SpawnExecContext)
	{
		SpawnExecContext->Parameters.UnbindFromSourceStores();
	}
	if (UpdateExecContext)
	{
		UpdateExecContext->Parameters.UnbindFromSourceStores();
	}

	World = nullptr;
}

void FNiagaraSystemSimulation::TransferInstance(FNiagaraSystemInstance* Instance)
{
	check(Instance);
	check(Instance->SystemSimulation.Get() != this);

	// Wait for all current instance ticks to complete we could complete the instance in question
	TSharedPtr<class FNiagaraSystemSimulation, ESPMode::ThreadSafe> SourceSimulation = Instance->SystemSimulation;
	WaitForInstancesTickComplete();
	SourceSimulation->WaitForInstancesTickComplete();

	switch ( Instance->SystemInstanceState)
	{
		// Nothing to do as nothing to transfer
		case ENiagaraSystemInstanceState::None:
			if (Instance->SystemSimulation.IsValid())
			{
				Instance->SystemSimulation = this->AsShared();
			}
			break;

		case ENiagaraSystemInstanceState::PendingSpawn:
			SourceSimulation->RemoveFromInstanceList(Instance);
			Instance->SystemSimulation = this->AsShared();
			AddToInstanceList(Instance, ENiagaraSystemInstanceState::PendingSpawn);
			break;

		case ENiagaraSystemInstanceState::PendingSpawnPaused:
			SourceSimulation->RemoveFromInstanceList(Instance);
			Instance->SystemSimulation = this->AsShared();
			AddToInstanceList(Instance, ENiagaraSystemInstanceState::PendingSpawnPaused);
			break;

		// Should never get here!
		case ENiagaraSystemInstanceState::Spawning:
			UE_LOGF(LogNiagara, Fatal, "Attempting to transfer an instance that is in the spawning state");
			break;

		case ENiagaraSystemInstanceState::Running:
		{
			const int32 NewDataSetIndex = MainDataSet.GetCurrentDataChecked().TransferInstance(SourceSimulation->MainDataSet.GetCurrentDataChecked(), Instance->SystemInstanceIndex);

			SourceSimulation->RemoveFromInstanceList(Instance);
			Instance->SystemSimulation = this->AsShared();
			AddToInstanceList(Instance, ENiagaraSystemInstanceState::Running);

			check(Instance->SystemInstanceIndex == NewDataSetIndex);
			break;
		}

		// If not pending spawn we need to transfer the data over
		case ENiagaraSystemInstanceState::Paused:
		{
			const int32 NewDataSetIndex = PausedDataSet.GetCurrentDataChecked().TransferInstance(SourceSimulation->PausedDataSet.GetCurrentDataChecked(), Instance->SystemInstanceIndex);

			SourceSimulation->RemoveFromInstanceList(Instance);
			Instance->SystemSimulation = this->AsShared();
			AddToInstanceList(Instance, ENiagaraSystemInstanceState::Paused);

			check(Instance->SystemInstanceIndex == NewDataSetIndex);
			break;
		}

		default:
			checkNoEntry();
	}

	if (!bBindingsInitialized)
	{
		InitParameterDataSetBindings(Instance);
	}
}

void FNiagaraSystemSimulation::DumpInstance(const FNiagaraSystemInstance* Inst)const
{
	ensure(!Inst->HasPendingFinalize());

	UE_LOGF(LogNiagara, Log, "==  %ls (%d) ========", *Inst->GetSystem()->GetFullName(), Inst->SystemInstanceIndex);
	UE_LOGF(LogNiagara, Log, ".................Spawn.................");
	SpawnExecContext->Parameters.DumpParameters(false);
	SpawnInstanceParameterDataSet.Dump(Inst->SystemInstanceIndex, 1, TEXT("Spawn Instance Parameters"));
	UE_LOGF(LogNiagara, Log, ".................Update.................");
	UpdateExecContext->Parameters.DumpParameters(false);
	UpdateInstanceParameterDataSet.Dump(Inst->SystemInstanceIndex, 1, TEXT("Update Instance Parameters"));
	UE_LOGF(LogNiagara, Log, "................. System Instance .................");
	MainDataSet.Dump(Inst->SystemInstanceIndex, 1, TEXT("System Data"));
}

void FNiagaraSystemSimulation::DumpTickInfo(FOutputDevice& Ar)
{
	check(IsInGameThread());

	UEnum* InstanceStateEnum = StaticEnum<ENiagaraSystemInstanceState>();
	check(InstanceStateEnum);

	for ( int i=0; i < int(ENiagaraSystemInstanceState::Num); ++i )
	{
		// None is invalid so don't use
		if ( i == int(ENiagaraSystemInstanceState::None) )
		{
			continue;
		}

		TArray<FNiagaraSystemInstance*> Instances = GetSystemInstances(ENiagaraSystemInstanceState(i));
		if (Instances.Num() > 0)
		{
			Ar.Logf(TEXT("\t\tInstances State(%s) Count(%d)"), *InstanceStateEnum->GetNameByIndex(i).ToString(), Instances.Num());
			for (FNiagaraSystemInstance* Instance : Instances)
			{
				Instance->DumpTickInfo(Ar);
			}
		}
	}
}

void FNiagaraSystemSimulation::AddTickGroupPromotion(FNiagaraSystemInstance* Instance)
{
	check(IsInGameThread());
	check(!PendingTickGroupPromotions.Contains(Instance));
	PendingTickGroupPromotions.Add(Instance);

	check(bIsSolo == false);

	if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World))
	{
		WorldManager->MarkSimulationForPostActorWork(this);
	}
}

void FNiagaraSystemSimulation::RemoveFromInstanceList(FNiagaraSystemInstance* Instance)
{
	const int32 InstanceIndex = Instance->SystemInstanceIndex;
	const int32 InstanceState = int32(Instance->SystemInstanceState);

	TArray<FNiagaraSystemInstance*>& SystemInstances = GetSystemInstances(Instance->SystemInstanceState);

	check(SystemInstances.IsValidIndex(InstanceIndex));
	check(SystemInstances[InstanceIndex] == Instance);

	SystemInstances.RemoveAtSwap(InstanceIndex);
	if (SystemInstances.IsValidIndex(InstanceIndex))
	{
		SystemInstances[InstanceIndex]->SystemInstanceIndex = InstanceIndex;
	}
	Instance->SystemInstanceIndex = INDEX_NONE;
	Instance->SystemInstanceState = ENiagaraSystemInstanceState::None;
}

void FNiagaraSystemSimulation::AddToInstanceList(FNiagaraSystemInstance* Instance, ENiagaraSystemInstanceState InstanceState)
{
	check(InstanceState != ENiagaraSystemInstanceState::None);
	check(Instance->SystemInstanceIndex == INDEX_NONE);
	check(Instance->SystemInstanceState == ENiagaraSystemInstanceState::None);
	Instance->SystemInstanceState = InstanceState;
	Instance->SystemInstanceIndex = GetSystemInstances(InstanceState).Add(Instance);
}

void FNiagaraSystemSimulation::SetInstanceState(FNiagaraSystemInstance* Instance, ENiagaraSystemInstanceState NewState)
{
	check(Instance != nullptr);

	switch ( Instance->SystemInstanceState )
	{
		// If we are not in a list the only possible place to go is pending spawn, anything else is invalid
		case ENiagaraSystemInstanceState::None:
		{
			check(NewState == ENiagaraSystemInstanceState::PendingSpawn);
			AddToInstanceList(Instance, NewState);
			break;
		}

		// If we are pending spawn we can only move to None, PendingSpawnPaused, Spawning or Running
		case ENiagaraSystemInstanceState::PendingSpawn:
		{
			check((NewState == ENiagaraSystemInstanceState::None) || (NewState == ENiagaraSystemInstanceState::PendingSpawnPaused) || (NewState == ENiagaraSystemInstanceState::Spawning) || (NewState == ENiagaraSystemInstanceState::Running));

			RemoveFromInstanceList(Instance);
			if (NewState != ENiagaraSystemInstanceState::None)
			{
				AddToInstanceList(Instance, NewState);
			}
			break;
		}

		// Can only move to None or PendingSpawn
		case ENiagaraSystemInstanceState::PendingSpawnPaused:
		{
			check((NewState == ENiagaraSystemInstanceState::None) || (NewState == ENiagaraSystemInstanceState::PendingSpawn));

			RemoveFromInstanceList(Instance);
			if (NewState != ENiagaraSystemInstanceState::None)
			{
				AddToInstanceList(Instance, NewState);
			}
			break;
		}

		// Can only move to None or Running
		case ENiagaraSystemInstanceState::Spawning:
		{
			check((NewState == ENiagaraSystemInstanceState::None) || (NewState == ENiagaraSystemInstanceState::Running));

			const int32 CurrDataSetIndex = Instance->SystemInstanceIndex;
			RemoveFromInstanceList(Instance);
			if (NewState == ENiagaraSystemInstanceState::None)
			{
				SpawningDataSet.GetCurrentDataChecked().KillInstance(CurrDataSetIndex);
			}
			else
			{
				const int32 NewDataSetIndex = MainDataSet.GetCurrentDataChecked().TransferInstance(SpawningDataSet.GetCurrentDataChecked(), CurrDataSetIndex);
				AddToInstanceList(Instance, NewState);
				check(Instance->SystemInstanceIndex == NewDataSetIndex);
			}
			break;
		}

		// Can only move to None or Paused
		case ENiagaraSystemInstanceState::Running:
		{
			check((NewState == ENiagaraSystemInstanceState::None) || (NewState == ENiagaraSystemInstanceState::Paused));

			const int32 CurrDataSetIndex = Instance->SystemInstanceIndex;
			RemoveFromInstanceList(Instance);
			if (NewState == ENiagaraSystemInstanceState::None)
			{
				MainDataSet.GetCurrentDataChecked().KillInstance(CurrDataSetIndex);
			}
			else
			{
				const int32 NewDataSetIndex = PausedDataSet.GetCurrentDataChecked().TransferInstance(MainDataSet.GetCurrentDataChecked(), CurrDataSetIndex);
				AddToInstanceList(Instance, NewState);
				check(Instance->SystemInstanceIndex == NewDataSetIndex);
			}
			break;
		}

		// Can only move to None or Running
		case ENiagaraSystemInstanceState::Paused:
		{
			check((NewState == ENiagaraSystemInstanceState::None) || (NewState == ENiagaraSystemInstanceState::Running));

			const int32 CurrDataSetIndex = Instance->SystemInstanceIndex;
			RemoveFromInstanceList(Instance);
			if (NewState == ENiagaraSystemInstanceState::None)
			{
				PausedDataSet.GetCurrentDataChecked().KillInstance(CurrDataSetIndex);
			}
			else
			{
				const int32 NewDataSetIndex = MainDataSet.GetCurrentDataChecked().TransferInstance(PausedDataSet.GetCurrentDataChecked(), CurrDataSetIndex);
				AddToInstanceList(Instance, NewState);
				check(Instance->SystemInstanceIndex == NewDataSetIndex);
			}
			break;
		}
	}

	// When we add an instance to pending spawn we must also flag for post actor work
	if ((NewState == ENiagaraSystemInstanceState::PendingSpawn) && !bIsSolo)
	{
		if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World))
		{
			WorldManager->MarkSimulationForPostActorWork(this);
		}
	}
}

void FNiagaraSystemSimulation::AddSystemToTickBatch(FNiagaraSystemSimulationTickContext& Context, FNiagaraSystemInstance* Instance, bool bIsLastInstance)
{
	Context.TickBatch.Add(Instance);
	if (bIsLastInstance || Context.TickBatch.Num() >= GNiagaraSystemSimulationTickBatchSize)
	{
		FlushTickBatch(Context, bIsLastInstance);
	}
}

void FNiagaraSystemSimulation::FlushTickBatch(FNiagaraSystemSimulationTickContext& Context, bool bIsLastInstance)
{
	if (Context.TickBatch.Num() == 0)
	{
		return;
	}

	// If we are running async create tasks to execute
	if ( Context.IsRunningAsync() )
	{
		// We execute the instance tick immediately if...
		// - We don't have a system script (stateless path)
		// - Mode is set to run last batch inlin
		// - Mode is set to run all in line
		const bool bExecuteInstanceTickImmediate =
			(bRunUpdateScript == false) ||
			((GNiagaraSystemSimulationTickBatchMode == 1) && bIsLastInstance) ||
			(GNiagaraSystemSimulationTickBatchMode == 2);

		FGraphEventArray FinalizePrereqArray;
		if (bExecuteInstanceTickImmediate)
		{
			for (FNiagaraSystemInstance* Inst : Context.TickBatch)
			{
				Inst->Tick_Concurrent();
			}
		}
		else
		{
			// Queue instance concurrent task and track information in the instance
			FGraphEventRef InstanceAsyncGraphEvent = TGraphTask<FNiagaraSystemInstanceTickConcurrentTask>::CreateTask(&Context.BeforeInstancesTickGraphEvents).ConstructAndDispatchWhenReady(this, Context.TickBatch, Context.World);

			for (FNiagaraSystemInstance* Inst : Context.TickBatch)
			{
				Inst->ConcurrentTickBatchGraphEvent = InstanceAsyncGraphEvent;
			}
			FinalizePrereqArray.Add(InstanceAsyncGraphEvent);

			// Ensure ConcurrentTickBatchGraphEvent is visible before we clear ConcurrentTickGraphEvent
			FPlatformMisc::MemoryBarrier();
		}

		// Queue finalize task which will run after the instances are complete, track with our all completion event
		FinalizePrereqArray.Add(Context.TickBatch[0]->ConcurrentTickGraphEvent);
		FGraphEventRef FinalizeTask = TGraphTask<FNiagaraSystemInstanceFinalizeTask>::CreateTask(&FinalizePrereqArray).ConstructAndDispatchWhenReady(this, Context.TickBatch);

		check(Context.CompletionEvents != nullptr);
		Context.CompletionEvents->Add(FinalizeTask);
	}
	// Execute immediately
	else
	{
		for (FNiagaraSystemInstance* Inst : Context.TickBatch)
		{
			Inst->Tick_Concurrent();
		}
	}

	Context.TickBatch.Reset();
}

/** First phase of system sim tick. Must run on GameThread. */
void FNiagaraSystemSimulation::Tick_GameThread(float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent)
{
	if (System->HasFixedTickDelta())
	{
		float FixedDelta = System->GetFixedTickDeltaTime();
		float Budget = FixedDelta > 0 ? FMath::Fmod(FixedDeltaTickAge, FixedDelta) + DeltaSeconds : 0;
		int32 Ticks = FixedDelta > 0 ? FMath::Min(FMath::FloorToInt(Budget / FixedDelta), GNiagaraSystemSimulationMaxTickSubsteps) : 0;

		TickInfo.UsesFixedTick = true;
		TickInfo.EngineTick = DeltaSeconds;
		TickInfo.SystemTick = FixedDelta;
		TickInfo.TickCount = Ticks;
		TickInfo.TickNumber = -1;
		TickInfo.TimeStepFraction = 0;
		
		for (int i = 0; i < Ticks; i++)
		{
			TickInfo.TickNumber = i;

			// Fraction of the total number of ticks we are evaluating
			// @note: I'd prefer if this were the fraction of the requested timestep, but
			// the implementation of budget doesn't readily allow it since time fraction could be
			// greater than 1
			TickInfo.TimeStepFraction = 1.0f * (i + 1) / Ticks;

			//Cannot do multiple tick off the game thread here without additional work. So we pass in null for the completion event which will force GT execution.
			Tick_GameThread_Internal(FixedDelta, Ticks == 1 ? MyCompletionGraphEvent : nullptr);
			Budget -= FixedDelta;
		}
		FixedDeltaTickAge += DeltaSeconds;
	}
	else
	{
		TickInfo.UsesFixedTick = false;
		TickInfo.EngineTick = DeltaSeconds;
		TickInfo.SystemTick = DeltaSeconds;
		TickInfo.TickCount = 1;
		TickInfo.TickNumber = 0;
		TickInfo.TimeStepFraction = 1;

		Tick_GameThread_Internal(DeltaSeconds, MyCompletionGraphEvent);
	}
}

bool FNiagaraSystemSimulation::MultiTick_GameThread(int32 TicksToProcess, float DeltaSecondsPerTick, float MaxSimTime, const FGraphEventRef& MyCompletionGraphEvent)
{
	if (DeltaSecondsPerTick <= 0 || TicksToProcess <= 0)
	{
		return false;
	}

	TickInfo.UsesFixedTick = true;
	TickInfo.EngineTick = TicksToProcess * DeltaSecondsPerTick;
	TickInfo.SystemTick = DeltaSecondsPerTick;
	TickInfo.TickCount = TicksToProcess;
	TickInfo.TickNumber = -1;
	TickInfo.TimeStepFraction = 0;
	
	const bool bUseMaxSimTime = MaxSimTime > 0.0f;
	const double EndMaxSimTime = bUseMaxSimTime ? FPlatformTime::Seconds() + MaxSimTime : 0.0f;
		
	for (int i = 0; i < TicksToProcess; i++)
	{
		TickInfo.TickNumber = i;
		TickInfo.TimeStepFraction = 1.0f * (i + 1) / TicksToProcess;

		//TODO: allow passing in MyCompletionGraphEvent to tick off the game thread here
		Tick_GameThread_Internal(DeltaSecondsPerTick, nullptr);

		if (System->HasFixedTickDelta())
		{
			FixedDeltaTickAge += DeltaSecondsPerTick;
		}

		// This limits the amount of time we will consume seeking forward
		if (bUseMaxSimTime)
		{
			const double CurrentTime = FPlatformTime::Seconds();
			if ( CurrentTime >= EndMaxSimTime )
			{
				break;
			}
		}
	}

	return TickInfo.TickNumber != (TicksToProcess - 1);
}

void FNiagaraSystemSimulation::Tick_GameThread_Internal(float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent)
{
	TArray<FNiagaraSystemInstance*>& SystemInstances = GetSystemInstances(ENiagaraSystemInstanceState::Running);
	TArray<FNiagaraSystemInstance*>& PendingSystemInstances = GetSystemInstances(ENiagaraSystemInstanceState::PendingSpawn);

	if ((PendingSystemInstances.Num() == 0 && SystemInstances.Num() == 0) || !bCanExecute)
	{
		return;
	}

	if ((GNiagaraSystemSimulationSkipTickDeltaSeconds > 0.0f) && (DeltaSeconds <= GNiagaraSystemSimulationSkipTickDeltaSeconds))
	{
		return;
	}

	check(IsInGameThread());
	check(bInSpawnPhase == false);

	FNiagaraCrashReporterScope CRScope(this);

	// Work may not be complete if we back to back tick the GameThread without sending EOF updates
	WaitForInstancesTickComplete();

	SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_TickGT);
	LLM_SCOPE(ELLMTag::Niagara);
	FScopeCycleCounterUObject AdditionalScope(GetSystem(), GET_STATID(STAT_NiagaraOverview_GT_CNC));

#if STATS
	FScopeCycleCounter SystemStatCounter(System->GetStatID(true, false));
#endif

	const int32 NumInstances = SystemInstances.Num();
	PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_GT(FParticlePerfStatsContext(GetWorld(), GetSystem()), TickGameThread, NumInstances);

	checkf(!ConcurrentTickGraphEvent.IsValid() || ConcurrentTickGraphEvent->IsComplete(), TEXT("NiagaraSystemSimulation System Concurrent has not completed when calling Tick_GameThread."));
	checkf(!AllWorkCompleteGraphEvent.IsValid() || AllWorkCompleteGraphEvent->IsComplete(), TEXT("NiagaraSystemSimulation Finalizes are not completed when calling Tick_GameThread"));
	ConcurrentTickGraphEvent = nullptr;
	AllWorkCompleteGraphEvent = nullptr;

	check(GetSystemInstances(ENiagaraSystemInstanceState::Running).Num() == MainDataSet.GetCurrentDataChecked().GetNumInstances());
	check(GetSystemInstances(ENiagaraSystemInstanceState::Spawning).Num() == SpawningDataSet.GetCurrentDataChecked().GetNumInstances());
	check(GetSystemInstances(ENiagaraSystemInstanceState::Paused).Num() == PausedDataSet.GetCurrentDataChecked().GetNumInstances());

	if (MaxDeltaTime.IsSet() && !System->HasFixedTickDelta())
	{
		DeltaSeconds = FMath::Clamp(DeltaSeconds, 0.0f, MaxDeltaTime.GetValue());

		TickInfo.SystemTick = DeltaSeconds;
		TickInfo.TimeStepFraction = DeltaSeconds / TickInfo.EngineTick;
	}

	UNiagaraScript* SystemSpawnScript = System->GetSystemSpawnScript();
	UNiagaraScript* SystemUpdateScript = System->GetSystemUpdateScript();
#if WITH_EDITOR
	SystemSpawnScript->RapidIterationParameters.Tick();
	SystemUpdateScript->RapidIterationParameters.Tick();
#endif

	const bool bUpdateTickGroups = !bIsSolo;

	// Update instances
	const bool bSupportsLargeWorldCoordinates = System->SupportsLargeWorldCoordinates();

	int32 SystemIndex = 0;
	FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World);
	check(WorldManager != nullptr);
	while (SystemIndex < SystemInstances.Num())
	{
		FNiagaraSystemInstance* Instance = SystemInstances[SystemIndex];

		// Update instance tick group, this can involve demoting the instance (i.e. removing from our list)
		if ( bUpdateTickGroups )
		{
			ETickingGroup DesiredTickGroup = Instance->CalculateTickGroup();
			if (DesiredTickGroup != SystemTickGroup )
			{
				// Tick demotion we need to do this now to ensure we complete in the correct group
				if (DesiredTickGroup > SystemTickGroup)
				{
					TSharedPtr<FNiagaraSystemSimulation, ESPMode::ThreadSafe> NewSim = WorldManager->GetSystemSimulation(DesiredTickGroup, System);
					NewSim->TransferInstance(Instance);
					continue;
				}
				// Tick promotions must be deferred as the tick group has already been processed
				//-OPT: We could tick in this group and add a task dependent on both groups to do the transform async
				else
				{
					AddTickGroupPromotion(Instance);
				}
			}
		}

		PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_GT(FParticlePerfStatsContext(GetInstancePerfStats(Instance)), TickGameThread, 1);

		// Perform instance tick
		Instance->Tick_GameThread(DeltaSeconds);

#if NIAGARA_SYSTEMSIMULATION_DEBUGGING
		NiagaraSystemSimulationLocal::DebugKillInstanceOnTick(Instance);
#endif

		// Has the actor position changed to the point where we need to reset the LWC tile
		if (bSupportsLargeWorldCoordinates)
		{
			if ( SystemInstances.IsValidIndex(SystemIndex) && (SystemInstances[SystemIndex] == Instance) )
			{
				if (USceneComponent* SceneComponent = Instance->GetAttachComponent())
				{
					if ( Instance->GetRequestedExecutionState() == ENiagaraExecutionState::Active && Instance->GetActualExecutionState() == ENiagaraExecutionState::Active)
					{
						const FVector ComponentLocation = SceneComponent->GetComponentLocation();
						if (UFXSystemComponent::RequiresLWCTileRecache(Instance->GetLWCTile(), ComponentLocation))
						{
							const FVector3f ActorTile = FLargeWorldRenderScalar::GetTileFor(ComponentLocation);
							Instance->UpdateLWCTile(ActorTile, MainDataSet);
						}
					}
				}
			}
		}

		// Ticking the instance can result in it being removed, completing + reactivating or transferring
		if (SystemInstances.IsValidIndex(SystemIndex) && (SystemInstances[SystemIndex] == Instance))
		{
			++SystemIndex;
		}
	}

	//Setup the few real constants like delta time.
	SetupParameters_GameThread(DeltaSeconds);

	// Somethings we don't want to happen during the spawn phase
	int32 SpawnNum = 0;
	if (PendingSystemInstances.Num() > 0)
	{
		SystemInstances.Reserve(SystemInstances.Num() + PendingSystemInstances.Num());

		while (PendingSystemInstances.Num() > 0)
		{
			FNiagaraSystemInstance* Instance = PendingSystemInstances[0];

			// Not solo, look to see if we should move tick group
			if (!bIsSolo)
			{
				const ETickingGroup DesiredTickGroup = Instance->CalculateTickGroup();
				if (DesiredTickGroup != SystemTickGroup)
				{
					TSharedPtr<FNiagaraSystemSimulation, ESPMode::ThreadSafe> DestSim = WorldManager->GetSystemSimulation(DesiredTickGroup, System);
					DestSim->TransferInstance(Instance);
					continue;
				}
			}

			// Execute instance tick
			PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_GT(FParticlePerfStatsContext(GetInstancePerfStats(Instance)), TickGameThread, 1);
			Instance->Tick_GameThread(DeltaSeconds);
#if NIAGARA_SYSTEMSIMULATION_DEBUGGING
			NiagaraSystemSimulationLocal::DebugKillInstanceOnTick(Instance);
#endif

			// Ensure we handle the instance being killed or potentially transfering system
			if ( PendingSystemInstances.IsValidIndex(0) && (PendingSystemInstances[0] == Instance) )
			{
				// When the first instance is added we need to initialize the parameter store to data set bindings.
				if (!bBindingsInitialized)
				{
					InitParameterDataSetBindings(Instance);
				}

				SetInstanceState(Instance, ENiagaraSystemInstanceState::Running);
				++SpawnNum;
			}
		}
	}

	//Solo systems add their counts in their component tick.
	if (GetIsSolo() == false)
	{
		System->AddToInstanceCountStat(SystemInstances.Num(), false);
		INC_DWORD_STAT_BY(STAT_TotalNiagaraSystemInstances, SystemInstances.Num());
	}

	//  Execute simulation async if allowed, otherwise everything will run on the game thread
	FNiagaraSystemSimulationTickContext Context(this, SystemInstances, MainDataSet, DeltaSeconds, SpawnNum, MyCompletionGraphEvent.IsValid());
	if ( Context.IsRunningAsync() )
	{
		auto ConcurrentTickTask = TGraphTask<FNiagaraSystemSimulationTickConcurrentTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndHold(Context, AllWorkCompleteGraphEvent);
		ConcurrentTickGraphEvent = ConcurrentTickTask->GetCompletionEvent();
		for (FNiagaraSystemInstance* Instance : Context.Instances)
		{
			Instance->ConcurrentTickGraphEvent = ConcurrentTickGraphEvent;
		}
		ConcurrentTickTask->Unlock();
		if (bIsSolo || NiagaraSystemSimulationLocal::GTickTaskShouldWait || !System->AsyncWorkCanOverlapTickGroups())
		{
			MyCompletionGraphEvent->DontCompleteUntil(AllWorkCompleteGraphEvent);
		}
		else
		{
			if ( System->AllDIsPostSimulateCanOverlapFrames() && NiagaraSystemSimulationLocal::GTickTaskAllowFrameOverlap )
			{
				WorldManager->MarkSimulationsForEndOfFrameWait(this);
			}
			else
			{
				WorldManager->MarkSimulationForPostActorWork(this);
			}
		}
	}
	else
	{
		Tick_Concurrent(Context);
	}
}

void FNiagaraSystemSimulation::UpdateTickGroups_GameThread()
{
	check(IsInGameThread());
	check(!bIsSolo);

	SCOPE_CYCLE_COUNTER(STAT_NiagaraOverview_GT);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	LLM_SCOPE(ELLMTag::Niagara);
	FScopeCycleCounterUObject AdditionalScope(GetSystem(), GET_STATID(STAT_NiagaraOverview_GT_CNC));

	FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World);
	check(WorldManager != nullptr);

	check(System != nullptr);

	// Transfer promoted instances to the new tick group
	//-OPT: This can be done async
	while (PendingTickGroupPromotions.Num() > 0)
	{
		FNiagaraSystemInstance* Instance = PendingTickGroupPromotions.Pop(EAllowShrinking::No);

		const ETickingGroup TickGroup = Instance->CalculateTickGroup();
		if (TickGroup != SystemTickGroup)
		{
			TSharedPtr<FNiagaraSystemSimulation, ESPMode::ThreadSafe> NewSim = WorldManager->GetSystemSimulation(TickGroup, System);
			NewSim->TransferInstance(Instance);
		}
	}
	PendingTickGroupPromotions.Reset();

	// Move pending system instances into new tick groups
	TArray<FNiagaraSystemInstance*>& PendingSystemInstances = GetSystemInstances(ENiagaraSystemInstanceState::PendingSpawn);
	int32 SystemIndex = 0;
	while ( SystemIndex < PendingSystemInstances.Num() )
	{
		FNiagaraSystemInstance* Instance = PendingSystemInstances[SystemIndex];
		const ETickingGroup DesiredTickGroup = Instance->CalculateTickGroup();
		if (DesiredTickGroup != SystemTickGroup)
		{
			TSharedPtr<FNiagaraSystemSimulation, ESPMode::ThreadSafe> DestSim = WorldManager->GetSystemSimulation(DesiredTickGroup, System);
			DestSim->TransferInstance(Instance);
			continue;
		}
		++SystemIndex;
	}
}

void FNiagaraSystemSimulation::Spawn_GameThread(float DeltaSeconds, bool bPostActorTick)
{
	// Early out, nothing to do
	TArray<FNiagaraSystemInstance*>& PendingSystemInstances = GetSystemInstances(ENiagaraSystemInstanceState::PendingSpawn);
	if (PendingSystemInstances.Num() == 0 || !bCanExecute)
	{
		return;
	}

	// Check to see if all work is complete already or not
	if (AllWorkCompleteGraphEvent.IsValid() && !AllWorkCompleteGraphEvent->IsComplete())
	{
		//-OPT: We should be able to chain this task off the AllWorkCompleteGraphEvent rather than not spawning
		if (bPostActorTick == false)
		{
			return;
		}

		// We should not wait here in post actor tick, but we will be safe and warn
		WaitForInstancesTickComplete(true);
	}
	ConcurrentTickGraphEvent = nullptr;
	AllWorkCompleteGraphEvent = nullptr;

	PARTICLE_PERF_STAT_CYCLES_GT(FParticlePerfStatsContext(World, System), TickGameThread);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_SpawnNewGT);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraOverview_GT);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	LLM_SCOPE(ELLMTag::Niagara);

	FScopeCycleCounterUObject AdditionalScope(System, GET_STATID(STAT_NiagaraOverview_GT_CNC));

	FNiagaraCrashReporterScope CRScope(this);

	bInSpawnPhase = true;

	if (MaxDeltaTime.IsSet())
	{
		DeltaSeconds = FMath::Clamp(DeltaSeconds, 0.0f, MaxDeltaTime.GetValue());
	}
	if (System->HasFixedTickDelta())
	{
		DeltaSeconds = System->GetFixedTickDeltaTime();
	}

#if WITH_EDITOR
	System->GetSystemSpawnScript()->RapidIterationParameters.Tick();
	System->GetSystemUpdateScript()->RapidIterationParameters.Tick();
#endif

	SetupParameters_GameThread(DeltaSeconds);

	// Spawn instances
	FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World);
	check(WorldManager != nullptr);

	TArray<FNiagaraSystemInstance*>& SpawningInstances = GetSystemInstances(ENiagaraSystemInstanceState::Spawning);
	SpawningInstances.Reserve(PendingSystemInstances.Num());

	int32 SystemIndex = 0;
	while (SystemIndex < PendingSystemInstances.Num())
	{
		FNiagaraSystemInstance* Instance = PendingSystemInstances[SystemIndex];

		// Do we need to move tick group, has something changed since we last checked?
		const ETickingGroup DesiredTickGroup = Instance->CalculateTickGroup();
		if (DesiredTickGroup != SystemTickGroup)
		{
			TSharedPtr<FNiagaraSystemSimulation, ESPMode::ThreadSafe> DestSim = WorldManager->GetSystemSimulation(DesiredTickGroup, System);
			DestSim->TransferInstance(Instance);
			continue;
		}

		// We are about to spawn execute game thread tick part
		Instance->Tick_GameThread(DeltaSeconds);
#if NIAGARA_SYSTEMSIMULATION_DEBUGGING
		NiagaraSystemSimulationLocal::DebugKillInstanceOnTick(Instance);
#endif

		// If we are still alive spawn
		if (Instance->SystemInstanceIndex != INDEX_NONE)
		{
			SetInstanceState(Instance, ENiagaraSystemInstanceState::Spawning);
		}
	}

	if ( SpawningInstances.Num() > 0 )
	{
		// When the first instance is added we need to initialize the parameter store to data set bindings.
		if (!bBindingsInitialized)
		{
			InitParameterDataSetBindings(SpawningInstances[0]);
		}

		// Execute spawning we only force solo onto the GameThread
		FNiagaraSystemSimulationTickContext Context(this, SpawningInstances, SpawningDataSet, DeltaSeconds, SpawningInstances.Num(), !bIsSolo);
		if ( Context.IsRunningAsync() )
		{
			auto ConcurrentTickTask = TGraphTask<FNiagaraSystemSimulationSpawnConcurrentTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndHold(Context, AllWorkCompleteGraphEvent);
			ConcurrentTickGraphEvent = ConcurrentTickTask->GetCompletionEvent();
			for (FNiagaraSystemInstance* Instance : Context.Instances)
			{
				Instance->ConcurrentTickGraphEvent = ConcurrentTickGraphEvent;
			}
			ConcurrentTickTask->Unlock();

			// Some simulations require us to complete inside PostActorTick and can not overlap to EOF updates / next frame, ensure we wait for them in the right location
			if ( System->AllDIsPostSimulateCanOverlapFrames() && NiagaraSystemSimulationLocal::GTickTaskAllowFrameOverlap )
			{
				WorldManager->MarkSimulationsForEndOfFrameWait(this);
			}
			else
			{
				WorldManager->MarkSimulationForPostActorWork(this);
			}
		}
		else
		{
			Spawn_Concurrent(Context);
		}
	}
	else
	{
		bInSpawnPhase = false;
	}
}

void FNiagaraSystemSimulation::Spawn_Concurrent(FNiagaraSystemSimulationTickContext& Context)
{
	check(bInSpawnPhase);
	Tick_Concurrent(Context);

	check(GetSystemInstances(ENiagaraSystemInstanceState::Running).Num() == MainDataSet.GetCurrentDataChecked().GetNumInstances());
	check(GetSystemInstances(ENiagaraSystemInstanceState::Spawning).Num() == SpawningDataSet.GetCurrentDataChecked().GetNumInstances());
	check(GetSystemInstances(ENiagaraSystemInstanceState::Paused).Num() == PausedDataSet.GetCurrentDataChecked().GetNumInstances());

	// Append spawned data to our active DataSet
	SpawningDataSet.CopyTo(MainDataSet, 0, INDEX_NONE, false);
	SpawningDataSet.ResetBuffers();

	// Move instances from spawning to active
	TArray<FNiagaraSystemInstance*>& SystemInstances = GetSystemInstances(ENiagaraSystemInstanceState::Running);
	TArray<FNiagaraSystemInstance*>& SpawningInstances = GetSystemInstances(ENiagaraSystemInstanceState::Spawning);

	SystemInstances.Reserve(SystemInstances.Num() + SpawningInstances.Num());
	for (FNiagaraSystemInstance* Instance : SpawningInstances)
	{
		Instance->SystemInstanceIndex = SystemInstances.Add(Instance);
		Instance->SystemInstanceState = ENiagaraSystemInstanceState::Running;
	}
	SpawningInstances.Reset();

	check(GetSystemInstances(ENiagaraSystemInstanceState::Running).Num() == MainDataSet.GetCurrentDataChecked().GetNumInstances());
	check(GetSystemInstances(ENiagaraSystemInstanceState::Spawning).Num() == SpawningDataSet.GetCurrentDataChecked().GetNumInstances());
	check(GetSystemInstances(ENiagaraSystemInstanceState::Paused).Num() == PausedDataSet.GetCurrentDataChecked().GetNumInstances());

	bInSpawnPhase = false;
}

void FNiagaraSystemSimulation::SimCacheTick_GameThread(UNiagaraSimCache* SimCache, float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent)
{
	check(System != nullptr);

	// Work may not be complete if we back to back tick the GameThread without sending EOF updates
	WaitForInstancesTickComplete();

	// Get pointer to instances
	TArray<FNiagaraSystemInstance*>& RunningSystemInstances = GetSystemInstances(ENiagaraSystemInstanceState::Running);
	TArray<FNiagaraSystemInstance*>& PendingSpawnSystemInstances = GetSystemInstances(ENiagaraSystemInstanceState::PendingSpawn);

	//-OPT: On cooked builds we do not need to do this every frame
	if (SimCache->CanRead(System) == false)
	{
		for (int32 i=RunningSystemInstances.Num() - 1; i >= 0; --i)
		{
			RunningSystemInstances[i]->Complete(false);
		}

		for (int32 i=PendingSpawnSystemInstances.Num() - 1; i >= 0; --i)
		{
			PendingSpawnSystemInstances[i]->Complete(false);
		}
		return;
	}

	// Should we run concurrently or not?
	// Data interfaces are not thread safe for reading, we will need to implement a way to do this....
	if (GNiagaraSystemSimulationAllowASyncSimCache && SimCache->CanRunAsync())
	{
		// Create finalize tick batch information to fit into how things work
		struct FFinalizeTickBatch
		{
			TArray<FNiagaraSystemInstance*, TInlineAllocator<4>> Instances;
		#if DO_CHECK
			std::atomic<int> DebugCounter = 0;
		#endif
		};
		TSharedPtr<FFinalizeTickBatch> FinalizeTickBatch(new FFinalizeTickBatch());

		// Create events to signal old task system
		ConcurrentTickGraphEvent = FGraphEvent::CreateGraphEvent();
		AllWorkCompleteGraphEvent = FGraphEvent::CreateGraphEvent();

		for (FNiagaraSystemInstance* SystemInstance : RunningSystemInstances)
		{
			SystemInstance->ConcurrentTickGraphEvent = ConcurrentTickGraphEvent;
		}

		for (FNiagaraSystemInstance* SystemInstance : PendingSpawnSystemInstances)
		{
			SystemInstance->ConcurrentTickGraphEvent = ConcurrentTickGraphEvent;
		}

		// Execute concurrent work
		UE::Tasks::FTask ConcurrentTask = UE::Tasks::Launch(
			TEXT("Niagara::SimCacheTick_Concurrent"),
			[this, SimCache, DeltaSeconds, FinalizeTickBatch]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Niagara::SimCacheTick_Concurrent);

				SimCacheTick_Concurrent(SimCache, DeltaSeconds);
				ConcurrentTickGraphEvent->DispatchSubsequents();

				TArray<FNiagaraSystemInstance*>& RunningSystemInstances = GetSystemInstances(ENiagaraSystemInstanceState::Running);
				FinalizeTickBatch->Instances.Reserve(RunningSystemInstances.Num());
				for (FNiagaraSystemInstance* SystemInstance : RunningSystemInstances)
				{
					FinalizeTickBatch->Instances.Add(SystemInstance);

					FNiagaraSystemInstanceFinalizeRef FinalizeRef(&FinalizeTickBatch->Instances.Last());
				#if DO_CHECK
					FinalizeRef.SetDebugCounter(&FinalizeTickBatch->DebugCounter);
				#endif
					SystemInstance->SetPendingFinalize(FinalizeRef);
				}
			}
		);

		// Execute finalize work
		UE::Tasks::Launch(
			TEXT("Niagara::SimCacheTick_Finalize"),
			[this, FinalizeTickBatch]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Niagara::SimCacheTick_Finalize);

				for (FNiagaraSystemInstance* Instance : FinalizeTickBatch->Instances)
				{
					if (Instance)
					{
						Instance->SimCacheTickFinalize_GameThread();
					}
				}
			#if DO_CHECK
				ensureMsgf(FinalizeTickBatch->DebugCounter.load() == 0, TEXT("Finalize batch is complete but counter is %d when it should be zero"), FinalizeTickBatch->DebugCounter.load());
			#endif
				AllWorkCompleteGraphEvent->DispatchSubsequents();
			},
			ConcurrentTask,
			UE::Tasks::ETaskPriority::Normal,
			UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri
		);
	}
	else
	{
		SimCacheTick_Concurrent(SimCache, DeltaSeconds);

		for ( int32 i=RunningSystemInstances.Num()-1; i >= 0; --i)
		{
			RunningSystemInstances[i]->SimCacheTickFinalize_GameThread();
		}
	}
}

void FNiagaraSystemSimulation::SimCacheTick_Concurrent(UNiagaraSimCache* SimCache, float DeltaSeconds)
{
	// Read running instances
	TArray<FNiagaraSystemInstance*>& SystemInstances = GetSystemInstances(ENiagaraSystemInstanceState::Running);
	for (FNiagaraSystemInstance* SystemInstance : SystemInstances)
	{
		if ( !SimCache->Read(SystemInstance->GetAge(), SystemInstance) )
		{
			SystemInstance->RequestedExecutionState = ENiagaraExecutionState::Complete;
		}
	}

	// Move pending spawn instances into running instances
	{
		TArray<FNiagaraSystemInstance*>& PendingInstances = GetSystemInstances(ENiagaraSystemInstanceState::PendingSpawn);
		for (int32 i=PendingInstances.Num()-1; i >= 0; --i)
		{
			FNiagaraSystemInstance* SystemInstance = PendingInstances[i];
			if (!SimCache->Read(SystemInstance->GetAge(), SystemInstance))
			{
				SystemInstance->RequestedExecutionState = ENiagaraExecutionState::Complete;
			}
			else
			{
				SetInstanceState(SystemInstance, ENiagaraSystemInstanceState::Running);
			}
		}
	}

	// Initialize bindings (if needed)
	if (!bBindingsInitialized && SystemInstances.Num() > 0)
	{
		InitParameterDataSetBindings(SystemInstances[0]);
	}

	// Copy over execution states push to stores, etc
	TConstArrayView<FNiagaraDataSetAccessor<ENiagaraExecutionState>> EmitterExecutionStateAccessors = System->GetEmitterExecutionStateAccessors();
	for ( int32 iSystemInstance=0; iSystemInstance < SystemInstances.Num(); ++iSystemInstance )
	{
		FNiagaraSystemInstance& SystemInstance = *SystemInstances[iSystemInstance];
		if (SystemInstance.RequestedExecutionState == ENiagaraExecutionState::Complete)
		{
			continue;
		}

		TArrayView<FNiagaraEmitterInstanceRef> Emitters = SystemInstance.GetEmitters();
		for (int32 iEmitter=0; iEmitter < Emitters.Num(); ++iEmitter)
		{
			FNiagaraEmitterInstance& EmitterInstance = Emitters[iEmitter].Get();
			if (EmitterInstance.IsComplete())
			{
				continue;
			}

			//DataSetToEmitterSpawnParameters[iEmitter].DataSetToParameterStore(EmitterInstance.GetSpawnExecutionContext().Parameters, MainDataSet, iSystemInstance);
			//DataSetToEmitterUpdateParameters[iEmitter].DataSetToParameterStore(EmitterInstance.GetUpdateExecutionContext().Parameters, MainDataSet, iSystemInstance);
			DataSetToEmitterRendererParameters[iEmitter].DataSetToParameterStore(EmitterInstance.GetRendererBoundVariables(), MainDataSet, iSystemInstance);

			//-TODO:Stateless: Support for SimCache?
			if ( FNiagaraEmitterInstanceImpl* StatefulEmitter = EmitterInstance.AsStateful() )
			{
				const ENiagaraExecutionState State = EmitterExecutionStateAccessors[iEmitter].GetReader(MainDataSet).GetSafe(iSystemInstance, ENiagaraExecutionState::Disabled);
				StatefulEmitter->SetExecutionState(State);
			}
		}
	}
}

void FNiagaraSystemSimulation::DumpStalledInfo()
{
	TStringBuilder<128> Builder;
	Builder.Appendf(TEXT("ConcurrentTickGraphEvent Complete (%d)\n"), ConcurrentTickGraphEvent ? ConcurrentTickGraphEvent->IsComplete() : true);
	Builder.Appendf(TEXT("AllWorkCompleteGraphEvent Complete (%d)\n"), AllWorkCompleteGraphEvent ? AllWorkCompleteGraphEvent->IsComplete() : true);
	Builder.Appendf(TEXT("SystemTickGroup (%d)\n"), SystemTickGroup);
	Builder.Appendf(TEXT("bInSpawnPhase (%d)\n"), bInSpawnPhase);
	Builder.Appendf(TEXT("bIsSolo (%d)\n"), bIsSolo);
	Builder.Appendf(TEXT("InstancesPendingSpawn (%d)\n"), GetSystemInstances(ENiagaraSystemInstanceState::PendingSpawn).Num());
	Builder.Appendf(TEXT("InstancesPendingSpawnPaused (%d)\n"), GetSystemInstances(ENiagaraSystemInstanceState::PendingSpawnPaused).Num());
	Builder.Appendf(TEXT("InstancesSpawning (%d)\n"), GetSystemInstances(ENiagaraSystemInstanceState::Spawning).Num());
	Builder.Appendf(TEXT("InstancesRunning (%d)\n"), GetSystemInstances(ENiagaraSystemInstanceState::Running).Num());
	Builder.Appendf(TEXT("InstancesPaused (%d)\n"), GetSystemInstances(ENiagaraSystemInstanceState::Paused).Num());

	UE_LOGF(LogNiagara, Fatal, "NiagaraSystemSimulation(%ls) is stalled.\n%ls", *GetNameSafe(GetSystem()), Builder.ToString());
}

void FNiagaraSystemSimulation::WaitForConcurrentTickComplete(bool bEnsureComplete)
{
	CSV_SCOPED_SET_WAIT_STAT(Effects);
	check(IsInGameThread());

	if (ConcurrentTickGraphEvent.IsValid() && !ConcurrentTickGraphEvent->IsComplete())
	{
		SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_ForceWaitForAsync);
		ensureAlwaysMsgf(!bEnsureComplete, TEXT("NiagaraSystemSimulation(%s) ConcurrentTickGraphEvent is not completed."), *GetSystem()->GetPathName());

		if (GNiagaraSystemSimulationTaskStallTimeout > 0)
		{
			if (WaitForAnyTaskCompleted({ ConcurrentTickGraphEvent }, FTimespan::FromMicroseconds(GNiagaraSystemSimulationTaskStallTimeout)) == INDEX_NONE)
			{
				DumpStalledInfo();
			}
		}
		else
		{
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(ConcurrentTickGraphEvent, ENamedThreads::GameThread_Local);
		}
	}
	ConcurrentTickGraphEvent = nullptr;
}

void FNiagaraSystemSimulation::WaitForInstancesTickComplete(bool bEnsureComplete)
{
	check(IsInGameThread());

	// If our AllWorkCompleteGraphEvent is not complete we need to wait on all instances to complete manually
	const bool bConcurrentInFlight = ConcurrentTickGraphEvent.IsValid() && !ConcurrentTickGraphEvent->IsComplete();
	const bool bInstancesInFlight = AllWorkCompleteGraphEvent.IsValid() && !AllWorkCompleteGraphEvent->IsComplete();
	if (bConcurrentInFlight || bInstancesInFlight)
	{
		SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_ForceWaitForAsync);
		ensureAlwaysMsgf(!bEnsureComplete, TEXT("NiagaraSystemSimulation(%s) AllWorkCompleteGraphEvent is not completed (ConcurrentInFlight=%d, InstancesInFlight=%d)."), *GetSystem()->GetPathName(), bConcurrentInFlight, bInstancesInFlight);
		WaitForConcurrentTickComplete(false);

		TArray<FNiagaraSystemInstance*>& SystemInstances = GetSystemInstances(ENiagaraSystemInstanceState::Running);
		int32 SystemInstanceIndex = 0;
		while ( SystemInstanceIndex < SystemInstances.Num() )
		{
			FNiagaraSystemInstance* Instance = SystemInstances[SystemInstanceIndex];
			Instance->WaitForConcurrentTickAndFinalize();

			// Finalize may result in the instance being removed or it being completed & reactivated
			if (SystemInstances.IsValidIndex(Instance->SystemInstanceIndex) && (SystemInstances[Instance->SystemInstanceIndex] == Instance) )
			{
				++SystemInstanceIndex;
			}
		}
	}

	ConcurrentTickGraphEvent = nullptr;
	AllWorkCompleteGraphEvent = nullptr;
}

void FNiagaraSystemSimulation::Tick_Concurrent(FNiagaraSystemSimulationTickContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_TickCNC);
	LLM_SCOPE(ELLMTag::Niagara);

	FScopeCycleCounterUObject AdditionalScope(Context.System, GET_STATID(STAT_NiagaraOverview_GT_CNC));
	if (!bCanExecute || !Context.Instances.Num())
	{
		return;
	}

	FNiagaraCrashReporterScope CRScope(this);

	if (GbDumpSystemData || Context.System->bDumpDebugSystemInfo)
	{
		UE_LOGF(LogNiagara, Log, "==========================================================");
		UE_LOGF(LogNiagara, Log, "Niagara System Sim Tick_Concurrent(): %ls", *Context.System->GetName());
		UE_LOGF(LogNiagara, Log, "==========================================================");
	}

#if STATS
	FScopeCycleCounter SystemStatCounter(Context.System->GetStatID(true, true));
#endif

	if (bRunUpdateScript == false)
	{
		const int32 NumInstances = Context.Instances.Num();
		const int32 FirstSpawnedInstance = NumInstances - Context.SpawnNum;
		if (bRunSpawnScript)// && Context.SpawnNum > 0)
		{
			for (int32 iSystemInstance = FirstSpawnedInstance; iSystemInstance < NumInstances; ++iSystemInstance)
			{
				FNiagaraSystemInstance* SystemInstance = Context.Instances[iSystemInstance];
				SystemInstance->TickInstanceParameters_Concurrent();
			}
			if (Context.SpawnNum > 0)
			{
				PrepareForSystemSimulate(Context);
				SpawnSystemInstances(Context);

				// Transfer results to parameter stores
				for (int32 iSystemInstance = FirstSpawnedInstance; iSystemInstance < NumInstances; ++iSystemInstance)
				{
					FNiagaraSystemInstance* SystemInstance = Context.Instances[iSystemInstance];

					TArrayView<FNiagaraEmitterInstanceRef> Emitters = SystemInstance->GetEmitters();
					for (int32 iEmitter = 0; iEmitter < Emitters.Num(); ++iEmitter)
					{
						FNiagaraEmitterInstance& EmitterInstance = Emitters[iEmitter].Get();
						if ( EmitterInstance.IsComplete() )
						{
							continue;
						}
						DataSetToEmitterRendererParameters[iEmitter].DataSetToParameterStore(EmitterInstance.GetRendererBoundVariables(), Context.DataSet, iSystemInstance);
					}
				}
			}
		}
		else
		{
			//-OPT: We should be able to avoid this but will require a lot of changes to the system simulation
			//      We might want to consider having different system simulation types
			{
				Context.DataSet.BeginSimulate();
				Context.DataSet.Allocate(Context.Instances.Num());
				Context.DataSet.GetDestinationDataChecked().SetNumInstances(Context.Instances.Num());
				Context.DataSet.EndSimulate();
			}
		}


		for ( int32 i=0; i < NumInstances; ++i )
		{
			FNiagaraSystemInstance* SystemInstance = Context.Instances[i];

			//-TODO: Stateless doesn't require a lot of this data
			if (i < FirstSpawnedInstance)
			{
				SystemInstance->TickInstanceParameters_Concurrent();
			}

			SystemInstance->TickSystemState();

			AddSystemToTickBatch(Context, SystemInstance, i == NumInstances - 1);
		}
	}
	else
	{
		for (FNiagaraSystemInstance* SystemInstance : Context.Instances)
		{
			SystemInstance->TickInstanceParameters_Concurrent();
		}

		PrepareForSystemSimulate(Context);

		if (Context.SpawnNum > 0)
		{
			SpawnSystemInstances(Context);
		}

		UpdateSystemInstances(Context);

		TransferSystemSimResults(Context);

		const int32 NumInstances = Context.Instances.Num();
		for (int32 i = 0; i < NumInstances; ++i)
		{
			FNiagaraSystemInstance* SystemInstance = Context.Instances[i];
			AddSystemToTickBatch(Context, SystemInstance, i == NumInstances - 1);
		}
	}

	// When not running async we can finalize straight away
	if ( !Context.IsRunningAsync() )
	{
		check(IsInGameThread());
		int32 InstanceIndex = 0;
		while (InstanceIndex < Context.Instances.Num())
		{
			FNiagaraSystemInstance* Instance = Context.Instances[InstanceIndex];
			Instance->FinalizeTick_GameThread();

			// Finalize can complete the instance and potentially reactivate
			if (Context.Instances.IsValidIndex(InstanceIndex) && (Context.Instances[InstanceIndex] == Instance))
			{
				++InstanceIndex;
			}

			check(Context.DataSet.GetCurrentDataChecked().GetNumInstances() == Context.Instances.Num());
		}
	}

	#if NIAGARA_SYSTEM_CAPTURE
	{
		FNiagaraSystemInstance* SoloSystemInstance = bIsSolo && Context.Instances.Num() == 1 ? Context.Instances[0] : nullptr;
		if (SoloSystemInstance)
		{
			SoloSystemInstance->FinishCapture();
		}
	}
	#endif

	INC_DWORD_STAT_BY(STAT_NiagaraNumSystems, Context.Instances.Num());
}

void FNiagaraSystemSimulation::SetupParameters_GameThread(float DeltaSeconds)
{
	check(IsInGameThread());

	const int32 NumInstances = GetSystemInstances(ENiagaraSystemInstanceState::Running).Num();
	SpawnNumSystemInstancesParam.SetValue(NumInstances);
	UpdateNumSystemInstancesParam.SetValue(NumInstances);
	SpawnGlobalSpawnCountScaleParam.SetValue(INiagaraModule::GetGlobalSpawnCountScale());
	UpdateGlobalSpawnCountScaleParam.SetValue(INiagaraModule::GetGlobalSpawnCountScale());
	SpawnGlobalSystemCountScaleParam.SetValue(INiagaraModule::GetGlobalSystemCountScale());
	UpdateGlobalSystemCountScaleParam.SetValue(INiagaraModule::GetGlobalSystemCountScale());
}

void FNiagaraSystemSimulation::PrepareForSystemSimulate(FNiagaraSystemSimulationTickContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_PrepareForSimulateCNC);

	const int32 NumInstances = Context.Instances.Num();
	if (NumInstances == 0)
	{
		return;
	}

	//Begin filling the state of the instance parameter datasets.
	SpawnInstanceParameterDataSet.BeginSimulate();
	UpdateInstanceParameterDataSet.BeginSimulate();

	SpawnInstanceParameterDataSet.Allocate(NumInstances);
	UpdateInstanceParameterDataSet.Allocate(NumInstances);

	check(System != nullptr);
	//Tick instance parameters and transfer any needed into the system simulation dataset.
	auto TransferInstanceParameters = [&](int32 SystemIndex)
	{
		FNiagaraSystemInstance* Inst = Context.Instances[SystemIndex];
		const FNiagaraParameterStore& InstParameters = Inst->GetInstanceParameters();

		if (InstParameters.GetParametersDirty() && bCanExecute)
		{
			SpawnInstanceParameterToDataSetBinding.ParameterStoreToDataSet(InstParameters, SpawnInstanceParameterDataSet, SystemIndex);
			UpdateInstanceParameterToDataSetBinding.ParameterStoreToDataSet(InstParameters, UpdateInstanceParameterDataSet, SystemIndex);
		}

		FNiagaraConstantBufferToDataSetBinding::CopyToDataSets(Context.System->GetSystemCompiledData(), *Inst, SpawnInstanceParameterDataSet, UpdateInstanceParameterDataSet, SystemIndex);

		//TODO: Find good way to check that we're not using any instance parameter data interfaces in the system scripts here.
		//In that case we need to solo and will never get here.
	};

	for (int32 i = 0; i < Context.Instances.Num(); ++i)
	{
		TransferInstanceParameters(i);
	}

	SpawnInstanceParameterDataSet.GetDestinationDataChecked().SetNumInstances(NumInstances);
	UpdateInstanceParameterDataSet.GetDestinationDataChecked().SetNumInstances(NumInstances);

	//We're done filling in the current state for the instance parameter datasets.
	SpawnInstanceParameterDataSet.EndSimulate();
	UpdateInstanceParameterDataSet.EndSimulate();
}

void FNiagaraSystemSimulation::SpawnSystemInstances(FNiagaraSystemSimulationTickContext& Context)
{
	//All instance spawning is done in a separate pass at the end of the frame so we can be sure we have all new spawns ready for processing.
	//We run the spawn and update scripts separately here as their own sim passes.

	SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_SpawnCNC);

	const int32 NumInstances = Context.Instances.Num();
	const int32 OrigNum = Context.Instances.Num() - Context.SpawnNum;
	const int32 SpawnNum = Context.SpawnNum;

	check(NumInstances >= Context.SpawnNum);

	FNiagaraSystemInstance* SoloSystemInstance = bIsSolo && Context.Instances.Num() == 1 ? Context.Instances[0] : nullptr;
	Context.DataSet.BeginSimulate();
	Context.DataSet.Allocate(NumInstances, true);
	Context.DataSet.GetDestinationDataChecked().SetNumInstances(NumInstances);

	// Run Spawn
	if (SpawnExecContext->Tick(SoloSystemInstance, ENiagaraSimTarget::CPUSim) == false)
	{
		for (FNiagaraSystemInstance* SystemInst : Context.Instances)
		{
			SystemInst->SetActualExecutionState(ENiagaraExecutionState::Disabled);
		}
		Context.DataSet.EndSimulate();
		return;
	}

#if NIAGARA_SYSTEMSIMULATION_DEBUGGING
	if ( NiagaraSystemSimulationLocal::DebugKillAllOnSpawn(Context) )
	{
		return;
	}
#endif

	SpawnExecContext->BindSystemInstances(Context.Instances);
	SpawnExecContext->BindData(0, Context.DataSet, OrigNum, false);
	SpawnExecContext->BindData(1, SpawnInstanceParameterDataSet, OrigNum, false);

	FScriptExecutionConstantBufferTable SpawnConstantBufferTable;
	BuildConstantBufferTable(Context.Instances[0]->GetGlobalParameters(), SpawnExecContext, SpawnConstantBufferTable);

	SpawnExecContext->Execute(nullptr, Context.DeltaSeconds, SpawnNum, SpawnConstantBufferTable);

	if (GbDumpSystemData || Context.System->bDumpDebugSystemInfo)
	{
		UE_LOGF(LogNiagara, Log, "=== Spwaned %d Systems ===", NumInstances);
		Context.DataSet.GetDestinationDataChecked().Dump(0, NumInstances, TEXT("System Dataset - Post Spawn"));
		SpawnInstanceParameterDataSet.GetCurrentDataChecked().Dump(0, NumInstances, TEXT("Spawn Instance Parameter Data"));
	}

	Context.DataSet.EndSimulate();

#if NIAGARA_SYSTEM_CAPTURE
	if (SoloSystemInstance && SoloSystemInstance->ShouldCaptureThisFrame())
	{
		TSharedPtr<struct FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> DebugInfo = SoloSystemInstance->GetActiveCaptureWrite(NAME_None, ENiagaraScriptUsage::SystemSpawnScript, FGuid());
		if (DebugInfo)
		{
			Context.DataSet.CopyTo(DebugInfo->Frame, OrigNum, SpawnNum);
			DebugInfo->Parameters = UpdateExecContext->Parameters;
			DebugInfo->bWritten = true;
		}
	}
#endif

	check(Context.DataSet.GetCurrentDataChecked().GetNumInstances() == Context.Instances.Num());
}

void FNiagaraSystemSimulation::UpdateSystemInstances(FNiagaraSystemSimulationTickContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_UpdateCNC);

	const int32 NumInstances = Context.Instances.Num();
	const int32 OrigNum = Context.Instances.Num() - Context.SpawnNum;
	const int32 SpawnNum = Context.SpawnNum;

	if (NumInstances > 0)
	{
		FNiagaraSystemInstance* SoloSystemInstance = bIsSolo && Context.Instances.Num() == 1 ? Context.Instances[0] : nullptr;

		FNiagaraDataBuffer& DestinationData = Context.DataSet.BeginSimulate();
		DestinationData.Allocate(NumInstances);
		DestinationData.SetNumInstances(NumInstances);

		// Tick UpdateExecContext, this can fail to bind VM functions if this happens we become invalid so mark all instances as disabled
		if (UpdateExecContext->Tick(Context.Instances[0], ENiagaraSimTarget::CPUSim) == false)
		{
			for (FNiagaraSystemInstance* SystemInst : Context.Instances)
			{
				SystemInst->SetActualExecutionState(ENiagaraExecutionState::Disabled);
			}
			Context.DataSet.EndSimulate();
			return;
		}

#if NIAGARA_SYSTEMSIMULATION_DEBUGGING
		if ( NiagaraSystemSimulationLocal::DebugKillAllOnUpdate(Context) )
		{
			return;
		}
#endif

		// Run update.
		if (OrigNum > 0)
		{
			UpdateExecContext->BindSystemInstances(Context.Instances);
			UpdateExecContext->BindData(0, Context.DataSet, 0, false);
			UpdateExecContext->BindData(1, UpdateInstanceParameterDataSet, 0, false);

			FScriptExecutionConstantBufferTable UpdateConstantBufferTable;
			BuildConstantBufferTable(Context.Instances[0]->GetGlobalParameters(), UpdateExecContext, UpdateConstantBufferTable);

			UpdateExecContext->Execute(nullptr, Context.DeltaSeconds, OrigNum, UpdateConstantBufferTable);
		}

		if (GbDumpSystemData || Context.System->bDumpDebugSystemInfo)
		{
			UE_LOGF(LogNiagara, Log, "=== Updated %d Systems ===", NumInstances);
			DestinationData.Dump(0, NumInstances, TEXT("System Data - Post Update"));
			UpdateInstanceParameterDataSet.GetCurrentDataChecked().Dump(0, NumInstances, TEXT("Update Instance Parameter Data"));
		}

		//Also run the update script on the newly spawned systems too.
		//TODO: JIRA - UE-60096 - Remove.
		//Ideally this should be compiled directly into the script similarly to interpolated particle spawning.
		if ( (SpawnNum > 0) && GNiagaraSystemSimulationUpdateOnSpawn)
		{
			UpdateExecContext->BindSystemInstances(Context.Instances);
			UpdateExecContext->BindData(0, Context.DataSet, OrigNum, false);
			UpdateExecContext->BindData(1, UpdateInstanceParameterDataSet, OrigNum, false);

			FNiagaraGlobalParameters UpdateOnSpawnParameters(Context.Instances[0]->GetGlobalParameters());
			if (GNiagaraSystemSimulationUpdateOnSpawn == 1)
			{
				UpdateOnSpawnParameters.EngineDeltaTime = 0.0001f;
				UpdateOnSpawnParameters.EngineInvDeltaTime = 10000.0f;
			}

			FScriptExecutionConstantBufferTable UpdateConstantBufferTable;
			BuildConstantBufferTable(UpdateOnSpawnParameters, UpdateExecContext, UpdateConstantBufferTable);

			UpdateExecContext->Execute(nullptr, Context.DeltaSeconds, SpawnNum, UpdateConstantBufferTable);

			if (GbDumpSystemData || Context.System->bDumpDebugSystemInfo)
			{
				UE_LOGF(LogNiagara, Log, "=== Spawn Updated %d Systems ===", SpawnNum);
				DestinationData.Dump(OrigNum, SpawnNum, TEXT("System Data - Post Update (new systems)"));
				UpdateInstanceParameterDataSet.GetCurrentDataChecked().Dump(OrigNum, SpawnNum, TEXT("Update Instance Parameter Data (new systems)"));
			}
		}

		Context.DataSet.EndSimulate();

#if NIAGARA_SYSTEM_CAPTURE
		if (SoloSystemInstance && SoloSystemInstance->ShouldCaptureThisFrame())
		{
			TSharedPtr<struct FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> DebugInfo = SoloSystemInstance->GetActiveCaptureWrite(NAME_None, ENiagaraScriptUsage::SystemUpdateScript, FGuid());
			if (DebugInfo)
			{
				Context.DataSet.CopyTo(DebugInfo->Frame);
				DebugInfo->Parameters = UpdateExecContext->Parameters;
				DebugInfo->bWritten = true;
			}
		}
#endif
	}

	check(Context.DataSet.GetCurrentDataChecked().GetNumInstances() == Context.Instances.Num());
}

void FNiagaraSystemSimulation::TransferSystemSimResults(FNiagaraSystemSimulationTickContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraSystemSim_TransferResultsCNC);

	if (Context.Instances.Num() == 0)
	{
		return;
	}

	check(System != nullptr);
#if STATS
	System->GetStatData().AddStatCapture(TTuple<uint64, ENiagaraScriptUsage>((uint64)this, ENiagaraScriptUsage::SystemSpawnScript), GetSpawnExecutionContext()->ReportStats());
	System->GetStatData().AddStatCapture(TTuple<uint64, ENiagaraScriptUsage>((uint64)this, ENiagaraScriptUsage::SystemUpdateScript), GetUpdateExecutionContext()->ReportStats());
#endif

	FNiagaraDataSetReaderInt32<ENiagaraExecutionState> SystemExecutionStateAccessor = System->GetSystemExecutionStateAccessor().GetReader(Context.DataSet);
	TConstArrayView<FNiagaraDataSetAccessor<ENiagaraExecutionState>> EmitterExecutionStateAccessors = System->GetEmitterExecutionStateAccessors();

	for (int32 SystemIndex = 0; SystemIndex < Context.Instances.Num(); ++SystemIndex)
	{
		FNiagaraSystemInstance* SystemInst = Context.Instances[SystemIndex];

		//Apply the systems requested execution state to it's actual execution state.
		ENiagaraExecutionState ExecutionState = SystemExecutionStateAccessor.GetSafe(SystemIndex, ENiagaraExecutionState::Disabled);
		SystemInst->SetActualExecutionState(ExecutionState);

		if (!SystemInst->IsDisabled())
		{
			//Now pull data out of the simulation and drive the emitters with it.
			TArrayView<FNiagaraEmitterInstanceRef> Emitters = SystemInst->GetEmitters();
			for (int32 EmitterIdx = 0; EmitterIdx < Emitters.Num(); ++EmitterIdx)
			{
				FNiagaraEmitterInstance& EmitterInstance = Emitters[EmitterIdx].Get();

				//Early exit before we set the state as if we're complete or disabled we should never let the emitter turn itself back. It needs to be reset/reinited manually.
				if (EmitterInstance.IsComplete())
				{
					continue;
				}

				check(Emitters.Num() > EmitterIdx);

				DataSetToEmitterRendererParameters[EmitterIdx].DataSetToParameterStore(EmitterInstance.GetRendererBoundVariables(), Context.DataSet, SystemIndex);

				if ( FNiagaraEmitterInstanceImpl* StatefulEmitter = EmitterInstance.AsStateful() )
				{
					ENiagaraExecutionState State = EmitterExecutionStateAccessors[EmitterIdx].GetReader(Context.DataSet).GetSafe(SystemIndex, ENiagaraExecutionState::Disabled);
					StatefulEmitter->SetExecutionState(State);

					TConstArrayView<FNiagaraDataSetAccessor<FNiagaraSpawnInfo>> EmitterSpawnInfoAccessors = System->GetEmitterSpawnInfoAccessors(EmitterIdx);
					TArray<FNiagaraSpawnInfo>& EmitterInstSpawnInfos = StatefulEmitter->GetSpawnInfo();
					for (int32 SpawnInfoIdx = 0; SpawnInfoIdx < EmitterSpawnInfoAccessors.Num(); ++SpawnInfoIdx)
					{
						if (SpawnInfoIdx < EmitterInstSpawnInfos.Num())
						{
							EmitterInstSpawnInfos[SpawnInfoIdx] = EmitterSpawnInfoAccessors[SpawnInfoIdx].GetReader(Context.DataSet).Get(SystemIndex);
						}
						else
						{
							ensure(SpawnInfoIdx < EmitterInstSpawnInfos.Num());
						}
					}

					//TODO: Any other fixed function stuff like this?

					FNiagaraScriptExecutionContext& SpawnContext = StatefulEmitter->GetSpawnExecutionContext();
					DataSetToEmitterSpawnParameters[EmitterIdx].DataSetToParameterStore(SpawnContext.Parameters, Context.DataSet, SystemIndex);

					FNiagaraScriptExecutionContext& UpdateContext = StatefulEmitter->GetUpdateExecutionContext();
					DataSetToEmitterUpdateParameters[EmitterIdx].DataSetToParameterStore(UpdateContext.Parameters, Context.DataSet, SystemIndex);

					FNiagaraComputeExecutionContext* GPUContext = StatefulEmitter->GetGPUContext();
					if (GPUContext)
					{
						DataSetToEmitterGPUParameters[EmitterIdx].DataSetToParameterStore(GPUContext->CombinedParamStore, Context.DataSet, SystemIndex);
					}

					TArrayView<FNiagaraScriptExecutionContext> EventContexts = StatefulEmitter->GetEventExecutionContexts();
					for (int32 EventIdx = 0; EventIdx < EventContexts.Num(); ++EventIdx)
					{
						FNiagaraScriptExecutionContext& EventContext = EventContexts[EventIdx];
						if (DataSetToEmitterEventParameters[EmitterIdx].Num() > EventIdx)
						{
							DataSetToEmitterEventParameters[EmitterIdx][EventIdx].DataSetToParameterStore(EventContext.Parameters, Context.DataSet, SystemIndex);
						}
						else
						{
							UE_LOGF(LogNiagara, Log, "Skipping DataSetToEmitterEventParameters because EventIdx is out-of-bounds. %d of %d", EventIdx, DataSetToEmitterEventParameters[EmitterIdx].Num());
						}
					}
				}
			}
		}
	}
}

void FNiagaraSystemSimulation::RemoveInstance(FNiagaraSystemInstance* Instance)
{
	if (Instance->SystemInstanceIndex == INDEX_NONE)
	{
		return;
	}

	check(IsInGameThread());
	if (UNiagaraEffectType* FXType = EffectType.Get())
	{
		--FXType->NumInstances;
	}

	// Remove from pending promotions list
	PendingTickGroupPromotions.RemoveSingleSwap(Instance);

	if(System)
	{
		System->UnregisterActiveInstance();
	}

	// Depending on the instance state we may need to wait for async work to complete in order to correctly remove the instance
	// as async work could impact if the instance is still alive or which group it remains in
	if ( (Instance->SystemInstanceState == ENiagaraSystemInstanceState::Running) || (Instance->SystemInstanceState == ENiagaraSystemInstanceState::Spawning) )
	{
		WaitForConcurrentTickComplete();
		Instance->WaitForConcurrentTickDoNotFinalize();
	}

	// We did not finalize and will not so clear the reference
	Instance->ConcurrentTickGraphEvent = nullptr;
	Instance->ConcurrentTickBatchGraphEvent = nullptr;
	Instance->FinalizeRef.ConditionalClear();

	// Remove the instance if it is still valid
	if (Instance->SystemInstanceState != ENiagaraSystemInstanceState::None)
	{
		SetInstanceState(Instance, ENiagaraSystemInstanceState::None);
	}

#if NIAGARA_NAN_CHECKING
	MainDataSet.CheckForNaNs();
#endif
}

void FNiagaraSystemSimulation::AddInstance(FNiagaraSystemInstance* Instance)
{
	check(IsInGameThread());
	check(Instance->SystemInstanceIndex == INDEX_NONE);

	WaitForConcurrentTickComplete();

	SetInstanceState(Instance, ENiagaraSystemInstanceState::PendingSpawn);
	
	if(System)
	{
		System->RegisterActiveInstance();
	}

	if (UNiagaraEffectType* FXType = EffectType.Get())
	{
		++FXType->NumInstances;
		FXType->bNewSystemsSinceLastScalabilityUpdate = true;
	}

	check(GetSystemInstances(ENiagaraSystemInstanceState::Running).Num() == MainDataSet.GetCurrentDataChecked().GetNumInstances());
	//Note: We can't check this here because AddInstance can be called while we are moving system from PendingSpawn -> Spawning
	//check(GetSystemInstances(ENiagaraSystemInstanceState::Spawning).Num() == SpawningDataSet.GetCurrentDataChecked().GetNumInstances());
	check(GetSystemInstances(ENiagaraSystemInstanceState::Paused).Num() == PausedDataSet.GetCurrentDataChecked().GetNumInstances());
}

void FNiagaraSystemSimulation::PauseInstance(FNiagaraSystemInstance* Instance)
{
	check(IsInGameThread());

	//-OPT: This can be more tightly scoped depending on the instance state
	WaitForInstancesTickComplete();

	if ( Instance->SystemInstanceState == ENiagaraSystemInstanceState::Running )
	{
		SetInstanceState(Instance, ENiagaraSystemInstanceState::Paused);
	}
	else if ( Instance->SystemInstanceState == ENiagaraSystemInstanceState::PendingSpawn )
	{
		SetInstanceState(Instance, ENiagaraSystemInstanceState::PendingSpawnPaused);
	}
	else
	{
		// Must be None as the instance has been killed
		check(Instance->SystemInstanceState == ENiagaraSystemInstanceState::None);
	}
}

void FNiagaraSystemSimulation::UnpauseInstance(FNiagaraSystemInstance* Instance)
{
	check(IsInGameThread());

	//-OPT: This can be more tightly scoped depending on the instance state
	WaitForInstancesTickComplete();

	if (Instance->SystemInstanceState == ENiagaraSystemInstanceState::Paused)
	{
		SetInstanceState(Instance, ENiagaraSystemInstanceState::Running);
	}
	else if (Instance->SystemInstanceState == ENiagaraSystemInstanceState::PendingSpawnPaused)
	{
		SetInstanceState(Instance, ENiagaraSystemInstanceState::PendingSpawn);
	}
	else
	{
		// Must be None as the instance has been killed
		check(Instance->SystemInstanceState == ENiagaraSystemInstanceState::None);
	}
}

void FNiagaraSystemSimulation::InitParameterDataSetBindings(FNiagaraSystemInstance* SystemInst)
{
	//Have to init here as we need an actual parameter store to pull the layout info from.
	//TODO: Pull the layout stuff out of each data set and store. So much duplicated data.
	//This assumes that all layouts for all emitters is the same. Which it should be.
	//Ideally we can store all this layout info in the systm/emitter assets so we can just generate this in Init()
	if (!bBindingsInitialized && SystemInst != nullptr)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Niagara_InitParameterDataSetBindings);

		bBindingsInitialized = true;

		SpawnInstanceParameterToDataSetBinding.Init(SpawnInstanceParameterDataSet, SystemInst->GetInstanceParameters());
		UpdateInstanceParameterToDataSetBinding.Init(UpdateInstanceParameterDataSet, SystemInst->GetInstanceParameters());

		TArrayView<FNiagaraEmitterInstanceRef> Emitters = SystemInst->GetEmitters();
		const int32 EmitterCount = Emitters.Num();

		DataSetToEmitterSpawnParameters.SetNum(EmitterCount);
		DataSetToEmitterUpdateParameters.SetNum(EmitterCount);
		DataSetToEmitterEventParameters.SetNum(EmitterCount);
		DataSetToEmitterGPUParameters.SetNum(EmitterCount);
		DataSetToEmitterRendererParameters.SetNum(EmitterCount);

		for (int32 EmitterIdx = 0; EmitterIdx < EmitterCount; ++EmitterIdx)
		{
			FNiagaraEmitterInstance& EmitterInstance = Emitters[EmitterIdx].Get();
			if (EmitterInstance.IsDisabled())
			{
				continue;
			}

			DataSetToEmitterRendererParameters[EmitterIdx].Init(MainDataSet, EmitterInstance.GetRendererBoundVariables());

			if (FNiagaraEmitterInstanceImpl* StatefulEmitter = EmitterInstance.AsStateful())
			{
				FNiagaraScriptExecutionContext& SpawnContext = StatefulEmitter->GetSpawnExecutionContext();
				DataSetToEmitterSpawnParameters[EmitterIdx].Init(MainDataSet, SpawnContext.Parameters);

				FNiagaraScriptExecutionContext& UpdateContext = StatefulEmitter->GetUpdateExecutionContext();
				DataSetToEmitterUpdateParameters[EmitterIdx].Init(MainDataSet, UpdateContext.Parameters);

				FNiagaraComputeExecutionContext* GPUContext = StatefulEmitter->GetGPUContext();
				if (GPUContext)
				{
					DataSetToEmitterGPUParameters[EmitterIdx].Init(MainDataSet, GPUContext->CombinedParamStore);
				}

				TArrayView<FNiagaraScriptExecutionContext> EventContexts = StatefulEmitter->GetEventExecutionContexts();
				const int32 EventCount = EventContexts.Num();
				DataSetToEmitterEventParameters[EmitterIdx].SetNum(EventCount);

				for (int32 EventIdx = 0; EventIdx < EventCount; ++EventIdx)
				{
					FNiagaraScriptExecutionContext& EventContext = EventContexts[EventIdx];
					DataSetToEmitterEventParameters[EmitterIdx][EventIdx].Init(MainDataSet, EventContext.Parameters);
				}
			}
		}
	}
}

const FString& FNiagaraSystemSimulation::GetCrashReporterTag()const
{
	if (CrashReporterTag.IsEmpty())
	{
		UNiagaraSystem* Sys = GetSystem();
		const FString& AssetName = Sys ? Sys->GetFullName() : TEXT("nullptr");

		CrashReporterTag = FString::Printf(TEXT("SystemSimulation | System: %s | bSolo: %s |"), *AssetName, bIsSolo ? TEXT("true") : TEXT("false"));
	}
	return CrashReporterTag;
}

void FNiagaraConstantBufferToDataSetBinding::CopyToDataSets(
	const FNiagaraSystemCompiledData& CompiledData,
	const FNiagaraSystemInstance& SystemInstance,
	FNiagaraDataSet& SpawnDataSet,
	FNiagaraDataSet& UpdateDataSet,
	int32 DataSestInstanceIndex)
{
	{
		const uint8* GlobalParameters = reinterpret_cast<const uint8*>(&SystemInstance.GetGlobalParameters());
		ApplyOffsets(CompiledData.SpawnInstanceGlobalBinding, GlobalParameters, SpawnDataSet, DataSestInstanceIndex);
		ApplyOffsets(CompiledData.UpdateInstanceGlobalBinding, GlobalParameters, UpdateDataSet, DataSestInstanceIndex);
	}

	{
		const uint8* SystemParameters = reinterpret_cast<const uint8*>(&SystemInstance.GetSystemParameters());
		ApplyOffsets(CompiledData.SpawnInstanceSystemBinding, SystemParameters, SpawnDataSet, DataSestInstanceIndex);
		ApplyOffsets(CompiledData.UpdateInstanceSystemBinding, SystemParameters, UpdateDataSet, DataSestInstanceIndex);
	}

	{
		const uint8* OwnerParameters = reinterpret_cast<const uint8*>(&SystemInstance.GetOwnerParameters());
		ApplyOffsets(CompiledData.SpawnInstanceOwnerBinding, OwnerParameters, SpawnDataSet, DataSestInstanceIndex);
		ApplyOffsets(CompiledData.UpdateInstanceOwnerBinding, OwnerParameters, UpdateDataSet, DataSestInstanceIndex);
	}

	TConstArrayView<FNiagaraEmitterInstanceRef> Emitters = SystemInstance.GetEmitters();
	const int32 EmitterCount = FMath::Min(Emitters.Num(), CompiledData.SpawnInstanceEmitterBindings.Num());

	for (int32 EmitterIdx = 0; EmitterIdx < EmitterCount; ++EmitterIdx)
	{
		const uint8* EmitterParameters = reinterpret_cast<const uint8*>(&SystemInstance.GetEmitterParameters(EmitterIdx));
		ApplyOffsets(CompiledData.SpawnInstanceEmitterBindings[EmitterIdx], EmitterParameters, SpawnDataSet, DataSestInstanceIndex);
		ApplyOffsets(CompiledData.UpdateInstanceEmitterBindings[EmitterIdx], EmitterParameters, UpdateDataSet, DataSestInstanceIndex);
	}
}

void FNiagaraConstantBufferToDataSetBinding::ApplyOffsets(
	const FNiagaraParameterDataSetBindingCollection& Offsets,
	const uint8* SourceData,
	FNiagaraDataSet& DataSet,
	int32 DataSetInstanceIndex)
{
	FNiagaraDataBuffer& CurrBuffer = DataSet.GetDestinationDataChecked();

	for (const auto& DataOffsets : Offsets.FloatOffsets)
	{
		float* ParamPtr = (float*)(SourceData + DataOffsets.ParameterOffset);
		float* DataSetPtr = CurrBuffer.GetInstancePtrFloat(DataOffsets.DataSetComponentOffset, DataSetInstanceIndex);
		*DataSetPtr = *ParamPtr;
	}
	for (const auto& DataOffsets : Offsets.Int32Offsets)
	{
		int32* ParamPtr = (int32*)(SourceData + DataOffsets.ParameterOffset);
		int32* DataSetPtr = CurrBuffer.GetInstancePtrInt32(DataOffsets.DataSetComponentOffset, DataSetInstanceIndex);
		*DataSetPtr = *ParamPtr;
	}
}


void FNiagaraSystemSimulation::BuildConstantBufferTable(
	const FNiagaraGlobalParameters& GlobalParameters,
	TUniquePtr<FNiagaraScriptExecutionContextBase>& ExecContext,
	FScriptExecutionConstantBufferTable& ConstantBufferTable) const
{
	const auto ScriptLiterals = ExecContext->GetScriptLiterals();

	check(!ExecContext->HasInterpolationParameters);

	const auto& ExternalParameterData = ExecContext->Parameters.GetParameterDataArray();
	uint8* ExternalParameterBuffer = const_cast<uint8*>(ExternalParameterData.GetData());
	const uint32 ExternalParameterSize = ExecContext->Parameters.GetExternalParameterSize();

	ConstantBufferTable.Reset(3);
	ConstantBufferTable.AddTypedBuffer(GlobalParameters);
	ConstantBufferTable.AddRawBuffer(ExternalParameterBuffer, ExternalParameterSize);
	ConstantBufferTable.AddRawBuffer(ScriptLiterals.GetData(), ScriptLiterals.Num());
}

ENiagaraGPUTickHandlingMode FNiagaraSystemSimulation::GetGPUTickHandlingMode()const
{
	if (DispatchInterface && FNiagaraUtilities::AllowGPUParticles(DispatchInterface->GetShaderPlatform()) && System && System->HasAnyGPUEmitters())
	{
		//TODO: Maybe some DI post ticks can even be done concurrent too which would also remove this restriction.
		bool bGT = System->HasDIsWithPostSimulateTick() || GNiagaraSystemSimulationConcurrentGPUTickInit == 0;
		bool bBatched = GNiagaraSystemSimulationBatchGPUTickSubmit && !bIsSolo;

		if (bGT)
		{
			return bBatched ? ENiagaraGPUTickHandlingMode::GameThreadBatched : ENiagaraGPUTickHandlingMode::GameThread;
		}
		else
		{
			return bBatched ? ENiagaraGPUTickHandlingMode::ConcurrentBatched : ENiagaraGPUTickHandlingMode::Concurrent;
		}
	}

	return ENiagaraGPUTickHandlingMode::None;
}
