// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemStaticBuffers.h"
#include "RenderingThread.h"

FNiagaraSystemStaticBuffers::~FNiagaraSystemStaticBuffers()
{
	check(IsInRenderingThread());
	GpuFloatBuffer.Release();
}

void FNiagaraSystemStaticBuffers::Finalize()
{
	check(bFinalized == false);
	bFinalized = true;

	if (GpuFloatResource.Num() > 0)
	{
		ENQUEUE_RENDER_COMMAND(InitBuffers)(
			[this](FRHICommandListImmediate& RHICmdList)
			{
				check(GpuFloatBuffer.NumBytes == 0);

				GpuFloatBuffer.Initialize(RHICmdList, TEXT("NiagaraSystemStaticBuffers"), sizeof(float), GpuFloatResource.Num(), EPixelFormat::PF_R32_FLOAT, BUF_Static, &GpuFloatResource);
			}
		);
	}
}

uint32 FNiagaraSystemStaticBuffers::AddCpuData(TConstArrayView<float> InFloatData)
{
	check(bFinalized == false);

	// Search for existing
	for (int32 i = 0; i <= CpuFloatBuffer.Num() - InFloatData.Num(); ++i)
	{
		if (FMemory::Memcmp(CpuFloatBuffer.GetData() + i, InFloatData.GetData(), InFloatData.Num() * sizeof(float)) == 0)
		{
			return i;
		}
	}

	// Add new
	const uint32 OutIndex = CpuFloatBuffer.AddUninitialized(InFloatData.Num());
	FMemory::Memcpy(CpuFloatBuffer.GetData() + OutIndex, InFloatData.GetData(), InFloatData.Num() * sizeof(float));
	return OutIndex;
}

uint32 FNiagaraSystemStaticBuffers::AddGpuData(TConstArrayView<float> InFloatData)
{
	check(bFinalized == false);

	// Search for existing
	for (int32 i = 0; i <= GpuFloatResource.Num() - InFloatData.Num(); ++i)
	{
		if (FMemory::Memcmp(GpuFloatResource.GetData() + i, InFloatData.GetData(), InFloatData.Num() * sizeof(float)) == 0)
		{
			return i;
		}
	}

	// Add new
	const uint32 OutIndex = GpuFloatResource.AddUninitialized(InFloatData.Num());
	FMemory::Memcpy(GpuFloatResource.GetData() + OutIndex, InFloatData.GetData(), InFloatData.Num() * sizeof(float));
	return OutIndex;
}
