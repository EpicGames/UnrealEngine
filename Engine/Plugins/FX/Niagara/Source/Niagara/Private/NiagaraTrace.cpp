// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraTrace.h"

#include "NiagaraComponent.h"
#include "NiagaraDataChannelPublishRequest.h"

#include "Particles/ParticlePerfStats.h"

UE_TRACE_CHANNEL_DEFINE(NiagaraChannel, "Traces Niagara particle system operations including compilation, emission, system state, emitter state, \
data channel operations, and particle data for debugging and performance analysis of VFX systems.");

#if WITH_NIAGARA_INSIGHTS
namespace NiagaraInsightsPrivate
{
	void BuildComponentName(FNameBuilder& Builder, const UNiagaraComponent* Component)
	{
		if (AActor* OwnerActor = Component->GetTypedOuter<AActor>())
		{
			FStringView ActorLabel = OwnerActor->GetActorLabelView();
			if (ActorLabel.IsEmpty())
			{
				OwnerActor->GetFName().AppendString(Builder);
			}
			else
			{
				Builder.Append(ActorLabel);
			}
			Builder.Append(TEXT("."));
		}
		Component->GetFName().AppendString(Builder);

		if (UNiagaraSystem* System = Component->GetAsset())
		{
			Builder.Append(TEXT(","));
			System->GetFName().AppendString(Builder);
		}
		else
		{
			Builder.Append(TEXT(",nullptr"));
		}
	}

	UE_TRACE_EVENT_BEGIN(NiagaraTrace, SystemPerformance_GT, NoSync)
		UE_TRACE_EVENT_FIELD(uint64,				EventTime)
		UE_TRACE_EVENT_FIELD(UE::Trace::WideString, SystemName)
		UE_TRACE_EVENT_FIELD(uint64,				NumInstances)
		UE_TRACE_EVENT_FIELD(uint64,				TickGameThreadCycles)
		UE_TRACE_EVENT_FIELD(uint64,				TickConcurrentCycles)		
		UE_TRACE_EVENT_FIELD(uint64,				FinalizeCycles)
		UE_TRACE_EVENT_FIELD(uint64,				EndOfFrameCycles)
		UE_TRACE_EVENT_FIELD(uint64,				ActivationCycles)
		UE_TRACE_EVENT_FIELD(uint64,				WaitCycles)
		UE_TRACE_EVENT_FIELD(uint64,				MemoryBytes)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(NiagaraTrace, SystemPerformance_RT, NoSync)
		UE_TRACE_EVENT_FIELD(uint64,				EventTime)
		UE_TRACE_EVENT_FIELD(UE::Trace::WideString, SystemName)
		UE_TRACE_EVENT_FIELD(uint64,				NumInstances)
		UE_TRACE_EVENT_FIELD(uint64,				RenderUpdateCycles)
		UE_TRACE_EVENT_FIELD(uint64,				GetDynamicMeshElementsCycles)
		UE_TRACE_EVENT_FIELD(uint64,				GpuNumInstances)
		UE_TRACE_EVENT_FIELD(uint64,				GpuTotalMicroseconds)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(NiagaraTrace, ComponentActivate, NoSync)
		UE_TRACE_EVENT_FIELD(uint64,				EventTime)
		UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ComponentName)
		UE_TRACE_EVENT_FIELD(bool,					bReset)
		UE_TRACE_EVENT_FIELD(bool,					bIsScalabilityCull)
		UE_TRACE_EVENT_FIELD(bool,					bAwaitingActivationDueToNotReady)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(NiagaraTrace, ComponentDeactivate, NoSync)
		UE_TRACE_EVENT_FIELD(uint64,				EventTime)
		UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ComponentName)
		UE_TRACE_EVENT_FIELD(bool,					bImmediate)
		UE_TRACE_EVENT_FIELD(bool,					bIsScalabilityCull)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(NiagaraTrace, ComponentComplete, NoSync)
		UE_TRACE_EVENT_FIELD(uint64,				EventTime)
		UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ComponentName)
		UE_TRACE_EVENT_FIELD(bool,					bExternalCompletion)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(NiagaraTrace, DataChannelPublish, NoSync)
		UE_TRACE_EVENT_FIELD(uint64,				EventTime)
		UE_TRACE_EVENT_FIELD(UE::Trace::WideString, SourceName)
		UE_TRACE_EVENT_FIELD(bool,					bGpuRequest)
		UE_TRACE_EVENT_FIELD(bool,					bVisibleToGame)
		UE_TRACE_EVENT_FIELD(bool,					bVisibleToCPUSims)
		UE_TRACE_EVENT_FIELD(bool,					bVisibleToGPUSims)
		UE_TRACE_EVENT_FIELD(uint32,				NumInstances)
		UE_TRACE_EVENT_FIELD(uint32,				NumInstanceAllocated)
	UE_TRACE_EVENT_END()

	UE_TRACE_EVENT_BEGIN(NiagaraTrace, DataChannelWrite, NoSync)
		UE_TRACE_EVENT_FIELD(uint64,				EventTime)
		UE_TRACE_EVENT_FIELD(UE::Trace::WideString, DataChannelName)
		UE_TRACE_EVENT_FIELD(UE::Trace::WideString, SourceName)
		UE_TRACE_EVENT_FIELD(int32,					NumInstances)
		UE_TRACE_EVENT_FIELD(bool,					bVisibleToGame)
		UE_TRACE_EVENT_FIELD(bool,					bVisibleToCPU)
		UE_TRACE_EVENT_FIELD(bool,					bVisibleToGPU)
	UE_TRACE_EVENT_END()

#if WITH_PARTICLE_PERF_STATS
	class FNiagaraInsightsStatsListener : public FParticlePerfStatsListener
	{
	public:
		virtual bool Tick() override
		{
			const uint64 CaptureTime = FPlatformTime::Cycles64();
			FParticlePerfStatsManager::ForAllSystemStats(
				[&](TWeakObjectPtr<const UFXSystemAsset>& WeakSystem, TUniquePtr<FParticlePerfStats>& Stats)
				{
					const FParticlePerfStats_GT& Stats_GT = Stats->GetGameThreadStats();
					if (Stats_GT.NumInstances > 0)
					{
						if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(WeakSystem.Get()))
						{
							FNameBuilder SystemNameBuilder;
							System->GetFName().ToString(SystemNameBuilder);

							UE_TRACE_LOG(NiagaraTrace, SystemPerformance_GT, NiagaraChannel)
								<< SystemPerformance_GT.EventTime(CaptureTime)
								<< SystemPerformance_GT.SystemName(SystemNameBuilder.ToString())
								<< SystemPerformance_GT.NumInstances(Stats_GT.NumInstances)
								<< SystemPerformance_GT.TickGameThreadCycles(Stats_GT.TickGameThreadCycles)
								<< SystemPerformance_GT.TickConcurrentCycles(Stats_GT.TickConcurrentCycles.Load(EMemoryOrder::Relaxed))
								<< SystemPerformance_GT.FinalizeCycles(Stats_GT.FinalizeCycles)
								<< SystemPerformance_GT.EndOfFrameCycles(Stats_GT.EndOfFrameCycles.Load(EMemoryOrder::Relaxed))
								<< SystemPerformance_GT.ActivationCycles(Stats_GT.ActivationCycles.Load(EMemoryOrder::Relaxed))
								<< SystemPerformance_GT.WaitCycles(Stats_GT.WaitCycles)
								<< SystemPerformance_GT.MemoryBytes(Stats_GT.MemoryBytes + Stats->MemoryKB_Asset.Get(0))
								;
						}
					}
				}
			);

			return true;
		}

		virtual void TickRT() override
		{
			const uint64 CaptureTime = FPlatformTime::Cycles64();
			FParticlePerfStatsManager::ForAllSystemStats(
				[&](TWeakObjectPtr<const UFXSystemAsset>& WeakSystem, TUniquePtr<FParticlePerfStats>& Stats)
				{
					const FParticlePerfStats_RT& Stats_RT = Stats->GetRenderThreadStats();
					if (Stats_RT.NumInstances > 0)
					{
						if (const UNiagaraSystem* System = Cast<UNiagaraSystem>(WeakSystem.Get()))
						{
							FNameBuilder SystemNameBuilder;
							System->GetFName().ToString(SystemNameBuilder);

							const FParticlePerfStats_GPU& GPUStats = Stats->GetGPUStats();

							UE_TRACE_LOG(NiagaraTrace, SystemPerformance_RT, NiagaraChannel)
								<< SystemPerformance_RT.EventTime(CaptureTime)
								<< SystemPerformance_RT.SystemName(SystemNameBuilder.ToString())
								<< SystemPerformance_RT.NumInstances(Stats_RT.NumInstances)
								<< SystemPerformance_RT.RenderUpdateCycles(Stats_RT.RenderUpdateCycles)
								<< SystemPerformance_RT.GetDynamicMeshElementsCycles(Stats_RT.GetDynamicMeshElementsCycles)
								<< SystemPerformance_RT.GpuNumInstances(GPUStats.NumInstances)
								<< SystemPerformance_RT.GpuTotalMicroseconds(GPUStats.TotalMicroseconds)
								;
						}
					}
				}
			);
		}

		virtual bool NeedsMemoryStats() const override { return true; }
		virtual bool NeedsWorldStats() const override { return false; }
		virtual bool NeedsSystemStats() const override { return true; }
		virtual bool NeedsComponentStats() const override { return false; }
	};

	TSharedPtr<FNiagaraInsightsStatsListener> GStatsListener;
#endif //WITH_PARTICLE_PERF_STATS
}

void FNiagaraInsights::Update()
{
#if WITH_PARTICLE_PERF_STATS
	if (IsChannelActive())
	{
		if (NiagaraInsightsPrivate::GStatsListener == nullptr)
		{
			NiagaraInsightsPrivate::GStatsListener = MakeShared<NiagaraInsightsPrivate::FNiagaraInsightsStatsListener>();
			FParticlePerfStatsManager::AddListener(NiagaraInsightsPrivate::GStatsListener);
		}
	}
	else
	{
		if (NiagaraInsightsPrivate::GStatsListener != nullptr)
		{
			FParticlePerfStatsManager::RemoveListener(NiagaraInsightsPrivate::GStatsListener);
			NiagaraInsightsPrivate::GStatsListener = nullptr;
		}
	}
#endif //WITH_PARTICLE_PERF_STATS
}

bool FNiagaraInsights::IsChannelActive()
{
	return UE_TRACE_CHANNELEXPR_IS_ENABLED(NiagaraChannel);
}

void FNiagaraInsights::NotifyComponentActivate(const UNiagaraComponent* Component, bool bReset, bool bIsScalabilityCull, bool bAwaitingActivationDueToNotReady)
{
	using namespace NiagaraInsightsPrivate;
	if (IsChannelActive())
	{
		FNameBuilder SystemNameBuilder;
		NiagaraInsightsPrivate::BuildComponentName(SystemNameBuilder, Component);

		UE_TRACE_LOG(NiagaraTrace, ComponentActivate, NiagaraChannel)
			<< ComponentActivate.EventTime(FPlatformTime::Cycles64())
			<< ComponentActivate.ComponentName(SystemNameBuilder.ToString())
			<< ComponentActivate.bReset(bReset)
			<< ComponentActivate.bIsScalabilityCull(bIsScalabilityCull)
			<< ComponentActivate.bAwaitingActivationDueToNotReady(bAwaitingActivationDueToNotReady)
			;
	}
}

void FNiagaraInsights::NotifyComponentDeactivate(const UNiagaraComponent* Component, bool bImmediate, bool bIsScalabilityCull)
{
	using namespace NiagaraInsightsPrivate;
	if (IsChannelActive())
	{
		FNameBuilder SystemNameBuilder;
		NiagaraInsightsPrivate::BuildComponentName(SystemNameBuilder, Component);

		UE_TRACE_LOG(NiagaraTrace, ComponentDeactivate, NiagaraChannel)
			<< ComponentDeactivate.EventTime(FPlatformTime::Cycles64())
			<< ComponentDeactivate.ComponentName(SystemNameBuilder.ToString())
			<< ComponentDeactivate.bImmediate(bImmediate)
			<< ComponentDeactivate.bIsScalabilityCull(bIsScalabilityCull)
			;
	}
}

void FNiagaraInsights::NotifyComponentComplete(const UNiagaraComponent* Component, bool bExternalCompletion)
{
	using namespace NiagaraInsightsPrivate;
	if (IsChannelActive())
	{
		FNameBuilder SystemNameBuilder;
		NiagaraInsightsPrivate::BuildComponentName(SystemNameBuilder, Component);

		UE_TRACE_LOG(NiagaraTrace, ComponentComplete, NiagaraChannel)
			<< ComponentComplete.EventTime(FPlatformTime::Cycles64())
			<< ComponentComplete.ComponentName(SystemNameBuilder.ToString())
			<< ComponentComplete.bExternalCompletion(bExternalCompletion)
			;
	}
}

void FNiagaraInsights::NotifyDataChannelPublish(const FNiagaraDataChannelPublishRequest& Request, bool bGpuRequest)
{
	using namespace NiagaraInsightsPrivate;
	if (IsChannelActive())
	{
		UE_TRACE_LOG(NiagaraTrace, DataChannelPublish, NiagaraChannel)
			<< DataChannelPublish.EventTime(FPlatformTime::Cycles64())
		#if WITH_NIAGARA_DEBUGGER
			<< DataChannelPublish.SourceName(*Request.DebugSource)
		#endif
			<< DataChannelPublish.bGpuRequest(bGpuRequest)
			<< DataChannelPublish.bVisibleToGame(Request.bVisibleToGame)
			<< DataChannelPublish.bVisibleToCPUSims(Request.bVisibleToCPUSims)
			<< DataChannelPublish.bVisibleToGPUSims(Request.bVisibleToGPUSims)
			<< DataChannelPublish.NumInstances(Request.Data.IsValid() ? Request.Data->GetNumInstances() : 0)
			<< DataChannelPublish.NumInstanceAllocated(Request.Data.IsValid() ? Request.Data->GetNumInstancesAllocated() : 0)
			;
	}
}

void FNiagaraInsights::NotifyDataChannelWrite(const UObject* DataChannel, const TCHAR* SourceName, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU)
{
	using namespace NiagaraInsightsPrivate;
	if (IsChannelActive())
	{
		FNameBuilder DataChannelName;
		if (DataChannel)
		{
			DataChannel->GetFName().ToString(DataChannelName);
		}

		UE_TRACE_LOG(NiagaraTrace, DataChannelWrite, NiagaraChannel)
			<< DataChannelWrite.EventTime(FPlatformTime::Cycles64())
			<< DataChannelWrite.DataChannelName(DataChannelName.ToString())
			<< DataChannelWrite.SourceName(SourceName ? SourceName : TEXT(""))
			<< DataChannelWrite.NumInstances(Count)
			<< DataChannelWrite.bVisibleToGame(bVisibleToGame)
			<< DataChannelWrite.bVisibleToCPU(bVisibleToCPU)
			<< DataChannelWrite.bVisibleToGPU(bVisibleToGPU)
			;
	}
}

#endif //WITH_NIAGARA_INSIGHTS
