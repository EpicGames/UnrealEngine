// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraTypeRegistry.h"

#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "HAL/CriticalSection.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/LazySingleton.h"
#include "NiagaraDataInterfaceBase.h"
#include "NiagaraDebugVis.h" // IWYU pragma: keep
#include "UObject/CoreRedirects.h"
#include "UObject/NameTypes.h"
#include <atomic>

// these two globals are intended to help ensure that a global variable is accessible to natvis (as defined in Niagara.natvis)
// while debugging.  GCoreTypeRegistrySingletonPtr is the pointer to the actual data stored within the TLazySingleton<FNiagaraTypeRegistry>
// while GTypeRegistrySingletonPtr can be declared in each module to ensure that it can be accessed while debugging any Niagara
// module.  See UE_VISUALIZERS_HELPERS.  Note that currently it seems like we can effectively debug NiagaraEditor/NiagaraShader without
// further declarations, which is nice, so will be leaving it as it is.
namespace NiagaraDebugVisHelper
{
	const FNiagaraTypeRegistry* GCoreTypeRegistrySingletonPtr = nullptr;
	UE_SELECT_ANY const FNiagaraTypeRegistry*& GTypeRegistrySingletonPtr = GCoreTypeRegistrySingletonPtr;
}

namespace NiagaraTypeRegistryLocal
{
	// TLS of the mapping between a TypeDefinition's TypeHash and the index in
	// the TypeRegistry's RegisteredTypes array.  Allows us to quickly access
	// the type definition from a handle without having to hit the lock
	static thread_local TMap<uint32, int32> RegisteredTypeIndexCacheTLS;
	static thread_local uint32 RegisteredTypeIndexCacheGeneration = INDEX_NONE;

	static thread_local uint32 RegistryProxyRefCount = 0;
	static thread_local TArray<FNiagaraTypeDefinition> RegistryProxyContents;

	static FDelegateHandle PluginUnmountedHandle;

	static std::atomic<int32> GSuspendPluginUnmountCounter = 0;
	static FCriticalSection GPluginUnmountSync;
	static TArray<FString> GPluginMountPointsToUnmount;

	void PluginUnmounted(IPlugin& Plugin)
	{
		const FString MountedPath = Plugin.GetMountedAssetPath();

		if (GSuspendPluginUnmountCounter.load(std::memory_order_acquire) > 0)
		{
			FScopeLock Guard(&NiagaraTypeRegistryLocal::GPluginUnmountSync);
			NiagaraTypeRegistryLocal::GPluginMountPointsToUnmount.Add(MountedPath);
		}
		else
		{
			// go through all of the registered types and see if any of them are from the plugin being unmounted.
			FNiagaraTypeRegistry::Get().InvalidateTypesByPath(MountedPath);
		}
	}
}

FNiagaraTypeRegistryTLSProxy::FNiagaraTypeRegistryTLSProxy()
	: CallingThreadId(FPlatformTLS::GetCurrentThreadId())
{
	++NiagaraTypeRegistryLocal::RegistryProxyRefCount;
}

FNiagaraTypeRegistryTLSProxy::~FNiagaraTypeRegistryTLSProxy()
{
	checkf(CallingThreadId == FPlatformTLS::GetCurrentThreadId(), TEXT("FNiagaraTypeRegistryTLSProxy destroyed from different thread (%x) from where it was constructed (%x)."), FPlatformTLS::GetCurrentThreadId(), CallingThreadId);
	if ((--NiagaraTypeRegistryLocal::RegistryProxyRefCount) == 0)
	{
		NiagaraTypeRegistryLocal::RegistryProxyContents.Reset();
	}
}

UNiagaraDataInterfaceBase* FNiagaraTypeRegistry::GetDefaultDataInterfaceByName(const FString& DIClassName)
{
	UClass* DIClass = nullptr;
	{
		FNiagaraTypeRegistry& Registry = Get();
		UE::TReadScopeLock Lock(Registry.RegisteredTypesLock);
		for (const TUniquePtr<FNiagaraTypeDefinition>& TypePtr : Registry.RegisteredTypes)
		{
			if (TypePtr.IsValid() && TypePtr->IsDataInterface())
			{
				UClass* FoundDIClass = TypePtr->GetClass();
				if (FoundDIClass && (FoundDIClass->GetName() == DIClassName || FoundDIClass->GetFullName() == DIClassName))
				{
					DIClass = FoundDIClass;
					break;
				}
			}
		}
	}

	// Consider the possibility of a redirector pointing to a new location..
	if (DIClass == nullptr)
	{
		FCoreRedirectObjectName OldObjName;
		OldObjName.ObjectName = *DIClassName;
		FCoreRedirectObjectName NewObjName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, OldObjName);
		if (NewObjName.IsValid() && OldObjName != NewObjName)
		{
			return GetDefaultDataInterfaceByName(NewObjName.ObjectName.ToString());
		}
	}

	if (DIClass)
	{
		return CastChecked<UNiagaraDataInterfaceBase>(DIClass->GetDefaultObject(false)); // We wouldn't be registered if the CDO had not already been created...
	}

	return nullptr;
}

// this will hold onto references to the classes/enums/structs that are used as a part of type definitions.  In the case that something is forcibly deleted
// the contents of the RegisteredTypes array will have it's ClassStructorEnum cleared, and it will no longer be treated as a valid type
// (FNiagaraTypeDefinitionHandle::Resolve will return an invalid Dummy reference).
// 
// It might be worth investigating changing RegisteredTypeIndexMap to be hashed based on the name of the class/enum/struct rather than the pointer.  That
// way if we reload a type old data, with their existing index, will still be treated as valid.  In this current implementation old FNiagaraVariables will
// have invalid types.  But, for now I think it would be an error to unload a type without having the variables that depended on it also getting deleted.
void FNiagaraTypeRegistry::AddReferencedObjects(FReferenceCollector& Collector)
{
	UE::TReadScopeLock Lock(RegisteredTypesLock);
	for (const TUniquePtr<FNiagaraTypeDefinition>& RegisteredType : RegisteredTypes)
	{
		if (RegisteredType.IsValid())
		{
			Collector.AddReferencedObject(RegisteredType->ClassStructOrEnum);
		}
	}
}

FString FNiagaraTypeRegistry::GetReferencerName() const
{
	return TEXT("FNiagaraTypeRegistry");
}

FNiagaraTypeRegistry::FNiagaraTypeRegistry()
{
	NiagaraDebugVisHelper::GCoreTypeRegistrySingletonPtr = this;

}

FNiagaraTypeRegistry::~FNiagaraTypeRegistry()
{
	NiagaraDebugVisHelper::GCoreTypeRegistrySingletonPtr = nullptr;
}

FNiagaraTypeRegistry& FNiagaraTypeRegistry::Get()
{
	return TLazySingleton<FNiagaraTypeRegistry>::Get();
}

void FNiagaraTypeRegistry::Init()
{
	IPluginManager& PluginManager = IPluginManager::Get();
	NiagaraTypeRegistryLocal::PluginUnmountedHandle = PluginManager.OnPluginUnmounted().AddStatic(NiagaraTypeRegistryLocal::PluginUnmounted);
}

void FNiagaraTypeRegistry::TearDown()
{
	IPluginManager& PluginManager = IPluginManager::Get();
	PluginManager.OnPluginUnmounted().Remove(NiagaraTypeRegistryLocal::PluginUnmountedHandle);

	TLazySingleton<FNiagaraTypeRegistry>::TearDown();
}

void FNiagaraTypeRegistry::SuspendPluginUnmount()
{
	NiagaraTypeRegistryLocal::GSuspendPluginUnmountCounter.fetch_add(1, std::memory_order_acq_rel);
}

void FNiagaraTypeRegistry::ResumePluginUnmount()
{
	if (NiagaraTypeRegistryLocal::GSuspendPluginUnmountCounter.fetch_sub(1, std::memory_order_acq_rel) == 1)
	{
		TArray<FString> LocalListOfPaths;
		{
			FScopeLock Guard(&NiagaraTypeRegistryLocal::GPluginUnmountSync);
			LocalListOfPaths = MoveTemp(NiagaraTypeRegistryLocal::GPluginMountPointsToUnmount);
		}
		InvalidateTypesByPaths(LocalListOfPaths);
	}
}

void FNiagaraTypeRegistry::InvalidateTypesByPath(const FString& AssetPath)
{
	InvalidateTypesByPaths({AssetPath});
}

void FNiagaraTypeRegistry::InvalidateTypesByPaths(TConstArrayView<FString> AssetPaths)
{
	if (AssetPaths.IsEmpty())
	{
		return;
	}

	FNiagaraTypeRegistry& Registry = Get();

	if (!Registry.bModuleInitialized)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraTypeRegistry_InvalidateTypesByPaths);

	TArray<int32> TypesToInvalidate;

	{
		FReadScopeLock Lock(Registry.RegistrationLock);

		FNameBuilder RegistryTypeNameStr;
		const int32 TypeCount = Registry.RegisteredSoftObjectPaths.Num();
		for (int32 TypeIt = 0; TypeIt < TypeCount; ++TypeIt)
		{
			Registry.RegisteredSoftObjectPaths[TypeIt].GetLongPackageFName().ToString(RegistryTypeNameStr);
			const FStringView RegistryTypeName = RegistryTypeNameStr.ToView();
			for (const FString& AssetPath : AssetPaths)
			{
				if (RegistryTypeName.StartsWith(AssetPath))
				{
					TypesToInvalidate.Add(TypeIt);
				}
			}
		}
	}

	if (!TypesToInvalidate.IsEmpty())
	{
		++Registry.RegisteredTypesGeneration;

		TArray<FNiagaraTypeDefinition> InvalidatedTypes;
		InvalidatedTypes.Reserve(TypesToInvalidate.Num());

		{
			UE::TWriteScopeLock Lock(Registry.RegisteredTypesLock);

			for (int32 TypeToInvalidate : TypesToInvalidate)
			{
				TUniquePtr<FNiagaraTypeDefinition>& TypePtr = Registry.RegisteredTypes[TypeToInvalidate];
				InvalidatedTypes.Add(*TypePtr);

				// we leave the type definition around, but invalidate it so that anyone holding onto a reference will still have
				// valid memory but we aren't going to be holding onto a class that has been invalidated
				TypePtr->Invalidate();
			}
		}

#if DO_ENSURE
		{
			FReadScopeLock Lock(Registry.RegistrationLock);

			for (const FNiagaraTypeDefinition& TypeToRemove : InvalidatedTypes)
			{
				// we only support unregistering implicitly added types which don't include support for the different
				// flag based categorizations
				ensure(!Registry.RegisteredUserVariableTypes.Contains(TypeToRemove));
				ensure(!Registry.RegisteredUserVariableTypes.Contains(TypeToRemove));
				ensure(!Registry.RegisteredSystemVariableTypes.Contains(TypeToRemove));
				ensure(!Registry.RegisteredEmitterVariableTypes.Contains(TypeToRemove));
				ensure(!Registry.RegisteredParticleVariableTypes.Contains(TypeToRemove));
				ensure(!Registry.RegisteredParamTypes.Contains(TypeToRemove));
				ensure(!Registry.RegisteredPayloadTypes.Contains(TypeToRemove));
				ensure(!Registry.RegisteredUserDefinedTypes.Contains(TypeToRemove));
				ensure(!Registry.RegisteredNumericTypes.Contains(TypeToRemove));
				ensure(!Registry.RegisteredIndexTypes.Contains(TypeToRemove));

				// we leave the following properties alone.  These could be considered orphaned, but in the current setup they
				// would not be repopulated if the type is re-registered (implicitly)
				//RegisteredStructConversionMap
				//RegisteredSoftObjectPaths
			}
		}
#endif
	}
}

int32 FNiagaraTypeRegistry::RegisterTypeInternal(const FNiagaraTypeDefinition& NewType)
{
	if (NiagaraTypeRegistryLocal::RegisteredTypeIndexCacheGeneration != RegisteredTypesGeneration)
	{
		NiagaraTypeRegistryLocal::RegisteredTypeIndexCacheGeneration = RegisteredTypesGeneration;
		NiagaraTypeRegistryLocal::RegisteredTypeIndexCacheTLS.Reset();
	}

	const uint32 TypeHash = GetTypeHash(NewType);
	if (NewType.RegisteredTypeDefIndex == INDEX_NONE)
	{
		// first see if we're in the TLS cache
		if (const int32* ExistingIndex = NiagaraTypeRegistryLocal::RegisteredTypeIndexCacheTLS.Find(TypeHash))
		{
			if (*ExistingIndex != INDEX_NONE)
			{
				// be sure to update the cached value on the type definition as well
				NewType.RegisteredTypeDefIndex = *ExistingIndex;

				return NewType.RegisteredTypeDefIndex;
			}
		}
	}

	// if we haven't managed to find the index in either of the above caches then we'll have to use a lock to 
	// check the primary array
	if (NewType.RegisteredTypeDefIndex == INDEX_NONE)
	{
		FReadScopeLock Lock(RegistrationLock);
		if (const int32* ExistingIndex = RegisteredTypeIndexMap.Find(TypeHash))
		{
			NewType.RegisteredTypeDefIndex = *ExistingIndex;
		}
	}

	// because types can be invalidated by plugins being unloaded we need to evaluate whether we need
	// to perform a soft re-registration, i.e. repopulating the type back into the RegisteredTypes array
	if (NewType.RegisteredTypeDefIndex != INDEX_NONE)
	{
		if (NewType.IsValid())
		{
			bool bNeedsRegistration = false;

			{
				UE::TReadScopeLock ReadLock(RegisteredTypesLock);
				if (ensure(RegisteredTypes.IsValidIndex(NewType.RegisteredTypeDefIndex)))
				{
					const FNiagaraTypeDefinition& OldType = *RegisteredTypes[NewType.RegisteredTypeDefIndex]; 
					bNeedsRegistration = !OldType.IsValid();
					checkf(bNeedsRegistration || (NewType == OldType),
						TEXT("bNeedsRegistration: %d, NewType: %s, OldType: %s NewTypeHash: %x OldTypeHash: %x"), 
						bNeedsRegistration, 
						*NewType.GetNameText().ToString(),
						*OldType.GetNameText().ToString(),
						TypeHash,
						GetTypeHash(OldType)
					);
				}
			}

			if (bNeedsRegistration)
			{
				UE::TWriteScopeLock Lock(RegisteredTypesLock);
				if (ensure(RegisteredTypes.IsValidIndex(NewType.RegisteredTypeDefIndex)))
				{
					*RegisteredTypes[NewType.RegisteredTypeDefIndex] = NewType;
				}
			}
		}
	}
	else
	{
		// We may need to create a new entry for a registered type
		// Between the read and the write it could have been added so we need to do an addition search to make sure we don't double register
		{
			UE::TWriteScopeLock Lock(RegisteredTypesLock);
			NewType.RegisteredTypeDefIndex = RegisteredTypes.IndexOfByPredicate([NewType](const TUniquePtr<FNiagaraTypeDefinition>& InDef) {return *InDef.Get() == NewType; });
			if (NewType.RegisteredTypeDefIndex == INDEX_NONE)
			{
				NewType.RegisteredTypeDefIndex = RegisteredTypes.Num();
				RegisteredTypes.Add(MakeUnique<FNiagaraTypeDefinition>(NewType));
			}
		}

		{
			FRWScopeLock Lock(RegistrationLock, SLT_Write);
			if (!RegisteredTypeIndexMap.Contains(TypeHash))
			{
				RegisteredSoftObjectPaths.Emplace(NewType.ClassStructOrEnum);
				RegisteredTypeIndexMap.Add(TypeHash, NewType.RegisteredTypeDefIndex);
			}
		}
	}

	NiagaraTypeRegistryLocal::RegisteredTypeIndexCacheTLS.Add(TypeHash, NewType.RegisteredTypeDefIndex);

	return NewType.RegisteredTypeDefIndex;
}

const FNiagaraTypeDefinition& FNiagaraTypeRegistry::GetRegisteredType(int32 TypeIndex) const
{
	if (TypeIndex != INDEX_NONE)
	{
		// check if we've got a RegistryProxy that can already provide the registered type
		if (NiagaraTypeRegistryLocal::RegistryProxyRefCount && NiagaraTypeRegistryLocal::RegistryProxyContents.IsValidIndex(TypeIndex))
		{
			const FNiagaraTypeDefinition& ProxyTypeDefinition = NiagaraTypeRegistryLocal::RegistryProxyContents[TypeIndex];
			if (ProxyTypeDefinition.IsValid())
			{
				return ProxyTypeDefinition;
			}
		}

		const FNiagaraTypeRegistry& Registry = FNiagaraTypeRegistry::Get();
		UE::TReadScopeLock Lock(Registry.RegisteredTypesLock);

		if (Registry.RegisteredTypes.IsValidIndex(TypeIndex))
		{
			const TUniquePtr<FNiagaraTypeDefinition>& RegisteredType = Registry.RegisteredTypes[TypeIndex];
			// If the type is invalid, then it's likely that it has been invalidated by GC because the underlying object has
			// been unloaded.
			if (RegisteredType.IsValid() && RegisteredType->IsValid())
			{
				if (NiagaraTypeRegistryLocal::RegistryProxyRefCount)
				{
					if (!NiagaraTypeRegistryLocal::RegistryProxyContents.IsValidIndex(TypeIndex))
					{
						NiagaraTypeRegistryLocal::RegistryProxyContents.SetNum(TypeIndex + 1);
					}
					NiagaraTypeRegistryLocal::RegistryProxyContents[TypeIndex] = *RegisteredType;
				}

				return *RegisteredType;
			}
		}
	}

	static FNiagaraTypeDefinition Dummy;
	return Dummy;
}

TArray<FNiagaraTypeDefinition> FNiagaraTypeRegistry::GetRegisteredTypes()
{
	FNiagaraTypeRegistry& Registry = Get();

	TArray<FNiagaraTypeDefinition> Types;

	{
		UE::TReadScopeLock Lock(Registry.RegisteredTypesLock);
		Types.Reserve(Registry.RegisteredTypes.Num());
		Algo::TransformIf(Registry.RegisteredTypes, Types,
			[](const TUniquePtr<FNiagaraTypeDefinition>& TypePtr) -> bool
			{
				return TypePtr.IsValid() && TypePtr->IsValid();
			},
			[](const TUniquePtr<FNiagaraTypeDefinition>& TypePtr) -> FNiagaraTypeDefinition
			{
				return *TypePtr;
			});
	}

	return Types;
}

TArray<FNiagaraTypeDefinition> FNiagaraTypeRegistry::GetRegisteredUserVariableTypes()
{
	FNiagaraTypeRegistry& Registry = Get();

	TArray<FNiagaraTypeDefinition> Types;

	{
		FReadScopeLock Lock(Registry.RegistrationLock);
		Types = Registry.RegisteredUserVariableTypes;
	}

	return Types;
}

TArray<FNiagaraTypeDefinition> FNiagaraTypeRegistry::GetRegisteredSystemVariableTypes()
{
	FNiagaraTypeRegistry& Registry = Get();

	TArray<FNiagaraTypeDefinition> Types;

	{
		FReadScopeLock Lock(Registry.RegistrationLock);
		Types = Registry.RegisteredSystemVariableTypes;
	}

	return Types;
}

TArray<FNiagaraTypeDefinition> FNiagaraTypeRegistry::GetRegisteredEmitterVariableTypes()
{
	FNiagaraTypeRegistry& Registry = Get();

	TArray<FNiagaraTypeDefinition> Types;

	{
		FReadScopeLock Lock(Registry.RegistrationLock);
		Types = Registry.RegisteredEmitterVariableTypes;
	}

	return Types;
}

TArray<FNiagaraTypeDefinition> FNiagaraTypeRegistry::GetRegisteredParticleVariableTypes()
{
	FNiagaraTypeRegistry& Registry = Get();

	TArray<FNiagaraTypeDefinition> Types;

	{
		FReadScopeLock Lock(Registry.RegistrationLock);
		Types = Registry.RegisteredParticleVariableTypes;
	}

	return Types;
}

TArray<FNiagaraTypeDefinition> FNiagaraTypeRegistry::GetRegisteredParameterTypes()
{
	FNiagaraTypeRegistry& Registry = Get();

	TArray<FNiagaraTypeDefinition> Types;

	{
		FReadScopeLock Lock(Registry.RegistrationLock);
		Types = Registry.RegisteredParamTypes;
	}

	return Types;
}

TArray<FNiagaraTypeDefinition> FNiagaraTypeRegistry::GetRegisteredPayloadTypes()
{
	FNiagaraTypeRegistry& Registry = Get();

	TArray<FNiagaraTypeDefinition> Types;

	{
		FReadScopeLock Lock(Registry.RegistrationLock);
		Types = Registry.RegisteredPayloadTypes;
	}

	return Types;
}

TArray<FNiagaraTypeDefinition> FNiagaraTypeRegistry::GetUserDefinedTypes()
{
	FNiagaraTypeRegistry& Registry = Get();

	TArray<FNiagaraTypeDefinition> Types;

	{
		FReadScopeLock Lock(Registry.RegistrationLock);
		Types = Registry.RegisteredUserDefinedTypes;
	}

	return Types;
}

TArray<FNiagaraTypeDefinition> FNiagaraTypeRegistry::GetNumericTypes()
{
	FNiagaraTypeRegistry& Registry = Get();

	TArray<FNiagaraTypeDefinition> Types;

	{
		FReadScopeLock Lock(Registry.RegistrationLock);
		Types = Registry.RegisteredNumericTypes;
	}

	return Types;
}

TArray<FNiagaraTypeDefinition> FNiagaraTypeRegistry::GetIndexTypes()
{
	FNiagaraTypeRegistry& Registry = Get();

	TArray<FNiagaraTypeDefinition> Types;

	{
		FReadScopeLock Lock(Registry.RegistrationLock);
		Types = Registry.RegisteredIndexTypes;
	}

	return Types;
}

TOptional<FNiagaraTypeDefinition> FNiagaraTypeRegistry::GetRegisteredTypeByName(FName TypeName)
{
	TOptional<FNiagaraTypeDefinition> Results;

	Get().ForAllSelectedRegisteredTypes(
		[TypeName](const FNiagaraTypeDefinition& TypeDef) -> bool
		{
			return (TypeDef.GetFName() == TypeName);
		},
		[&Results](const FNiagaraTypeDefinition& TypeDef) -> bool
		{
			Results = TypeDef;
			return false; // continue
		}
	);

	return Results;
}


void FNiagaraTypeRegistry::ClearUserDefinedRegistry()
{
	FNiagaraTypeRegistry& Registry = Get();

	FRWScopeLock Lock(Registry.RegistrationLock, SLT_Write);

	for (const FNiagaraTypeDefinition& Def : Registry.RegisteredUserDefinedTypes)
	{
		Registry.RegisteredPayloadTypes.Remove(Def);
		Registry.RegisteredParamTypes.Remove(Def);
		Registry.RegisteredNumericTypes.Remove(Def);
		Registry.RegisteredIndexTypes.Remove(Def);
	}

	Registry.RegisteredUserDefinedTypes.Empty();

	// note that we don't worry about cleaning up RegisteredTypes or RegisteredTypeIndexMap because we don't
	// want to invalidate any indexes that are already stored in FNiagaraTypeDefinitionHandle.  If re-registered
	// they will be given the same index, and if they are orphaned we don't want to have invalid indices on the handle.
}

void FNiagaraTypeRegistry::Register(const FNiagaraTypeDefinition& NewType, bool bCanBeParameter, bool bCanBePayload, bool bIsUserDefined)
{
	ENiagaraTypeRegistryFlags Flags =
		ENiagaraTypeRegistryFlags::AllowUserVariable |
		ENiagaraTypeRegistryFlags::AllowSystemVariable |
		ENiagaraTypeRegistryFlags::AllowEmitterVariable;
	if (bCanBeParameter)
	{
		Flags |= ENiagaraTypeRegistryFlags::AllowParameter;
	}
	if (bCanBePayload)
	{
		Flags |= ENiagaraTypeRegistryFlags::AllowPayload;
	}
	if (bIsUserDefined)
	{
		Flags |= ENiagaraTypeRegistryFlags::IsUserDefined;
	}

	Register(NewType, Flags);
}

void FNiagaraTypeRegistry::ProcessRegistryQueue()
{
	FNiagaraTypeRegistry& Registry = Get();
	{
		FRWScopeLock Lock(Registry.RegistrationLock, SLT_Write);
		Registry.bModuleInitialized = true;
	}
	FQueuedRegistryEntry Entry;
	while (Registry.RegistryQueue.Dequeue(Entry))
	{
		Register(Entry.NewType, Entry.Flags);
	}
}

void FNiagaraTypeRegistry::Register(const FNiagaraTypeDefinition& NewType, ENiagaraTypeRegistryFlags Flags)
{
	FNiagaraTypeRegistry& Registry = Get();
	{
		FReadScopeLock Lock(Registry.RegistrationLock);
		if (!Registry.bModuleInitialized)
		{
			// In a packaged game it can happen that CDOs are created before the Niagara module had a chance to be initialized.
			// This is problematic, as the swc struct builder tries to access other Niagara types, so we delay the registration until the module is properly initialized.
			Registry.RegistryQueue.Enqueue({ NewType, Flags });
			return;
		}
	}

	if (FNiagaraTypeHelper::IsLWCType(NewType))
	{
		// register the swc type as well if necessary
		FNiagaraTypeDefinition(FNiagaraTypeHelper::GetSWCStruct(NewType.GetScriptStruct()), FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Deny);
	}

	Registry.RegisterTypeInternal(NewType);

	{
		FRWScopeLock Lock(Registry.RegistrationLock, SLT_Write);
		if (EnumHasAnyFlags(Flags, ENiagaraTypeRegistryFlags::AllowUserVariable))
		{
			Registry.RegisteredUserVariableTypes.AddUnique(NewType);
		}

		if (EnumHasAnyFlags(Flags, ENiagaraTypeRegistryFlags::AllowSystemVariable))
		{
			Registry.RegisteredSystemVariableTypes.AddUnique(NewType);
		}

		if (EnumHasAnyFlags(Flags, ENiagaraTypeRegistryFlags::AllowEmitterVariable))
		{
			Registry.RegisteredEmitterVariableTypes.AddUnique(NewType);
		}

		if (EnumHasAnyFlags(Flags, ENiagaraTypeRegistryFlags::AllowParticleVariable))
		{
			Registry.RegisteredParticleVariableTypes.AddUnique(NewType);
		}

		if (EnumHasAnyFlags(Flags, ENiagaraTypeRegistryFlags::AllowParameter))
		{
			Registry.RegisteredParamTypes.AddUnique(NewType);
		}

		if (EnumHasAnyFlags(Flags, ENiagaraTypeRegistryFlags::AllowPayload))
		{
			Registry.RegisteredPayloadTypes.AddUnique(NewType);
		}

		if (EnumHasAnyFlags(Flags, ENiagaraTypeRegistryFlags::IsUserDefined))
		{
			Registry.RegisteredUserDefinedTypes.AddUnique(NewType);
		}

		if (FNiagaraTypeDefinition::IsValidNumericInput(NewType))
		{
			Registry.RegisteredNumericTypes.AddUnique(NewType);
		}

		if (NewType.IsIndexType())
		{
			Registry.RegisteredIndexTypes.AddUnique(NewType);
		}
	}
}

bool FNiagaraTypeRegistry::IsStaticPossible(const FNiagaraTypeDefinition& InSrc)
{
	if (InSrc.IsStatic())
	{
		return true;
	}

	FNiagaraTypeRegistry& Registry = Get();

	bool bStaticTypeFound = false;
	Registry.ForAllSelectedRegisteredTypes(
		[&InSrc](const FNiagaraTypeDefinition& TypeDef) -> bool
		{
			return InSrc.IsSameBaseDefinition(TypeDef) && TypeDef.IsStatic();
		},
		[&bStaticTypeFound](const FNiagaraTypeDefinition& TypeDef) -> bool
		{
			bStaticTypeFound = true;
			return false;
		});

	return bStaticTypeFound;
}

void FNiagaraTypeRegistry::RegisterStructConverter(const FNiagaraTypeDefinition& SourceType, const FNiagaraLwcStructConverter& StructConverter)
{
	FNiagaraTypeRegistry& Registry = Get();

	int32 TypeIndex = Registry.RegisterTypeInternal(SourceType);

	FRWScopeLock Lock(Registry.RegistrationLock, SLT_Write);
	Registry.RegisteredStructConversionMap.Add(TypeIndex, StructConverter);
}

FNiagaraLwcStructConverter FNiagaraTypeRegistry::GetStructConverter(const FNiagaraTypeDefinition& SourceType)
{
	FNiagaraTypeRegistry& Registry = Get();

	FReadScopeLock Lock(Registry.RegistrationLock);
	const uint32 TypeHash = GetTypeHash(SourceType);
	if (const int32* TypeIndex = Registry.RegisteredTypeIndexMap.Find(TypeHash))
	{
		if (FNiagaraLwcStructConverter* Converter = Registry.RegisteredStructConversionMap.Find(*TypeIndex))
		{
			return *Converter;
		}
	}

	return FNiagaraLwcStructConverter();
}

FNiagaraTypeDefinition FNiagaraTypeRegistry::GetTypeForStruct(UScriptStruct* InStruct)
{
	FNiagaraTypeRegistry& Registry = Get();

	FNiagaraTypeDefinition FoundType;

	Registry.ForAllSelectedRegisteredTypes(
		[InStruct](const FNiagaraTypeDefinition& TypeDef) -> bool
		{
			return TypeDef.GetStruct() && InStruct == TypeDef.GetStruct();
		},
		[&FoundType](const FNiagaraTypeDefinition& TypeDef) -> bool
		{
			FoundType = TypeDef;
			return false;
		});

	return FoundType.IsValid() ? FoundType : FNiagaraTypeDefinition(InStruct);
}
