// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraTypes.h"

#include "NiagaraConstants.h"
#include "NiagaraDataInterface.h"
#include "NiagaraEmitter.h"
#include "Misc/StringBuilder.h"
#include "UObject/Class.h"
#include "UObject/EnumProperty.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraTypes)

static FName NAME_NiagaraDouble(TEXT("NiagaraDouble"));
static FName NAME_NiagaraPosition(TEXT("NiagaraPosition"));

// Emitter stats makes some assumptions about how you can cast from a ENiagaraExecutionStateManagement to ENiagaraExecutionState and the enum values will match
static_assert(int(ENiagaraExecutionState::Num) == int(ENiagaraExecutionStateManagement::Num), "ENiagaraExecutionState & ENiagaraExecutionStateManagement must match 1:1 as scripts assume they can cast between");

void FNiagaraVariableBase::SetNamespacedName(const FString& InNamespace, FName InVariableName)
{
	TStringBuilder<128> NameBuilder;
	NameBuilder.Append(InNamespace);
	NameBuilder.AppendChar(TEXT('.'));
	InVariableName.AppendString(NameBuilder);
	Name = FName(NameBuilder);
}

bool FNiagaraVariableBase::RemoveRootNamespace(const FStringView& ExpectedNamespace)
{
	FNameBuilder NameString;
	Name.ToString(NameString);

	FStringView NameStringView = NameString.ToView();
	if ( (NameStringView.Len() > ExpectedNamespace.Len() + 1) && (NameStringView[ExpectedNamespace.Len()] == '.') && NameStringView.StartsWith(ExpectedNamespace) )
	{
		FNameBuilder NewNameString;
		NewNameString.Append(NameStringView.Mid(ExpectedNamespace.Len() + 1));
		Name = FName(FStringView(NewNameString));
		return true;
	}
	return false;
}

bool FNiagaraVariableBase::ReplaceRootNamespace(const FStringView& ExpectedNamespace, const FStringView& NewNamespace)
{
	FNameBuilder NameString;
	Name.ToString(NameString);

	FStringView NameStringView = NameString.ToView();
	if ( (NameStringView.Len() > ExpectedNamespace.Len() + 1) && (NameStringView[ExpectedNamespace.Len()] == '.') && NameStringView.StartsWith(ExpectedNamespace) )
	{
		FNameBuilder NewNameString;
		NewNameString.Append(NewNamespace);
		NewNameString.Append(NameStringView.Mid(ExpectedNamespace.Len()));
		Name = FName(FStringView(NewNameString));
		return true;
	}
	return false;
}

bool FNiagaraVariableBase::IsInExternalNamespace() const
{
	const FNameBuilder NameString(Name);
	const FStringView NameStringView(NameString);

	for (const FString& Namespace : { FNiagaraConstants::UserNamespaceString, FNiagaraConstants::ParameterCollectionNamespaceString })
	{
		if ((NameStringView.Len() > Namespace.Len() + 1) && (NameStringView[Namespace.Len()] == '.') && NameStringView.StartsWith(Namespace))
		{
			return true;
		}
	}
	return false;
}

bool FNiagaraVariableBase::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(FNiagaraVariable::StaticStruct()->GetFName()))
	{
		FNiagaraVariable Var;
		FNiagaraVariable::StaticStruct()->SerializeItem(Slot, &Var, nullptr);
		Name = Var.Name;
		TypeDefHandle = Var.TypeDefHandle;
		//ensureMsgf(Var.IsDataAllocated() == false, TEXT("NiagaraVariable has been demoted to Base but contains data, this is probably an error."));
		return true;
	}
	return false;
}

FVersionedNiagaraEmitterData* FVersionedNiagaraEmitter::GetEmitterData() const
{
	return Emitter ? Emitter->GetEmitterData(Version) : nullptr;
}

FVersionedNiagaraEmitterWeakPtr FVersionedNiagaraEmitter::ToWeakPtr() const
{
	return FVersionedNiagaraEmitterWeakPtr(Emitter, Version);
}

FVersionedNiagaraEmitterBase FVersionedNiagaraEmitter::ToBase() const
{
	return FVersionedNiagaraEmitterBase(Emitter, Version);
}

FVersionedNiagaraEmitterWeakPtr::FVersionedNiagaraEmitterWeakPtr(UNiagaraEmitter* InEmitter, const FGuid& InVersion)
{
	Emitter = InEmitter;
	Version = InVersion;
}

bool FVersionedNiagaraEmitterWeakPtr::IsValid() const
{
	return GetEmitterData() != nullptr;
}

FVersionedNiagaraEmitter FVersionedNiagaraEmitterWeakPtr::ResolveWeakPtr() const
{
	if (Emitter.IsValid())
	{
		return FVersionedNiagaraEmitter(Emitter.Get(), Version);
	}
	return FVersionedNiagaraEmitter();
}

FVersionedNiagaraEmitterData* FVersionedNiagaraEmitterWeakPtr::GetEmitterData() const
{
	return ResolveWeakPtr().GetEmitterData();
}

FVersionedNiagaraEmitterBaseWeakPtr FVersionedNiagaraEmitterWeakPtr::ToWeakBase() const
{
	return FVersionedNiagaraEmitterBaseWeakPtr(Emitter, Version);
}

FGuid FNiagaraAssetVersion::CreateStableVersionGuid(UObject* Object)
{
	if (Object)
	{
		uint32 HashBuffer[]{ 0, 0, 0, 0 };

		{
			TStringBuilder<256> ObjectPathString;
			Object->GetPathName(nullptr, ObjectPathString);

			FMD5 ObjectPathHash;
			ObjectPathHash.Update(reinterpret_cast<uint8*>(GetData(ObjectPathString)), ObjectPathString.Len() * sizeof(TCHAR));
			ObjectPathHash.Final(reinterpret_cast<uint8*>(&HashBuffer));
		}

		return FGuid(HashBuffer[0], HashBuffer[1], HashBuffer[2], HashBuffer[3]);
	}

	return FGuid();
}

FNiagaraLWCConverter::FNiagaraLWCConverter(FVector InSystemWorldPos)
{
	SystemWorldPos = InSystemWorldPos;
}

FVector3f FNiagaraLWCConverter::ConvertWorldToSimulationVector(FVector WorldPosition) const
{
	return FVector3f(WorldPosition - SystemWorldPos);
}

FNiagaraPosition FNiagaraLWCConverter::ConvertWorldToSimulationPosition(FVector WorldPosition) const
{
	return FNiagaraPosition(ConvertWorldToSimulationVector(WorldPosition));
}

FVector FNiagaraLWCConverter::ConvertSimulationPositionToWorld(FNiagaraPosition SimulationPosition) const
{
	return ConvertSimulationVectorToWorld(SimulationPosition);
}

FVector FNiagaraLWCConverter::ConvertSimulationVectorToWorld(FVector3f SimulationPosition) const
{
	return FVector(SimulationPosition) + SystemWorldPos;
}

FMatrix FNiagaraLWCConverter::ConvertWorldToSimulationMatrix(const FMatrix& Matrix) const
{
	return Matrix.ConcatTranslation(-SystemWorldPos);
}

FMatrix FNiagaraLWCConverter::ConvertSimulationToWorldMatrix(const FMatrix& Matrix) const
{
	return Matrix.ConcatTranslation(SystemWorldPos);
}

FTransform3f FNiagaraLWCConverter::ConvertWorldToSimulationTransform(const FTransform& Transform) const
{
	FTransform3f NewTransform;
	NewTransform.SetTranslation(FVector3f(Transform.GetTranslation() - SystemWorldPos));
	NewTransform.SetRotation(FQuat4f(Transform.GetRotation()));
	NewTransform.SetScale3D(FVector3f(Transform.GetScale3D()));
	return NewTransform;
}

FTransform FNiagaraLWCConverter::ConvertSimulationToWorldTransform(const FTransform3f& Transform) const
{
	FTransform NewTransform;
	NewTransform.SetTranslation(FVector(Transform.GetTranslation()) + SystemWorldPos);
	NewTransform.SetRotation(FQuat(Transform.GetRotation()));
	NewTransform.SetScale3D(FVector(Transform.GetScale3D()));
	return NewTransform;
}

FNiagaraStructConversionStep::FNiagaraStructConversionStep()
{
}

FNiagaraStructConversionStep::FNiagaraStructConversionStep(int32 InLWCBytes, int32 InLWCOffset, int32 InSimulationBytes, int32 InSimulationOffset, ENiagaraStructConversionType InConversionType)
	: LWCBytes(InLWCBytes), LWCOffset(InLWCOffset), SimulationBytes(InSimulationBytes), SimulationOffset(InSimulationOffset), ConversionType(InConversionType)
{
}

void FNiagaraStructConversionStep::CopyToSim(const uint8* LWCData, uint8* SimulationData, int32 Count, int32 LWCStride, int32 SimulationStride)const
{
	if (ConversionType == ENiagaraStructConversionType::DoubleToFloat)
	{
		checkf(LWCBytes == 8 && SimulationBytes == 4, TEXT("Wrong bytesizes for double->float conversion: Source %i, Simulation %i"), LWCBytes, SimulationBytes);
		CopyInternal<float, double>(SimulationData, SimulationStride, SimulationOffset, LWCData, LWCStride, LWCOffset, Count);
	}
	else if (ConversionType == ENiagaraStructConversionType::Vector2)
	{
		checkf(LWCBytes == 16 && SimulationBytes == 8, TEXT("Wrong bytesizes for Vector2 d->f conversion: Source %i, Simulation %i"), LWCBytes, SimulationBytes);
		CopyInternal<FVector2f, FVector2d>(SimulationData, SimulationStride, SimulationOffset, LWCData, LWCStride, LWCOffset, Count);
	}
	else if (ConversionType == ENiagaraStructConversionType::Vector3)
	{
		checkf(LWCBytes == 24 && SimulationBytes == 12, TEXT("Wrong bytesizes for Vector3 d->f conversion: Source %i, Simulation %i"), LWCBytes, SimulationBytes);
		CopyInternal<FVector3f, FVector>(SimulationData, SimulationStride, SimulationOffset, LWCData, LWCStride, LWCOffset, Count);
	}
	else if (ConversionType == ENiagaraStructConversionType::Vector4)
	{
		checkf(LWCBytes == 32 && SimulationBytes == 16, TEXT("Wrong bytesizes for Vector4 d->f conversion: Source %i, Simulation %i"), LWCBytes, SimulationBytes);
		CopyInternal<FVector4f, FVector4>(SimulationData, SimulationStride, SimulationOffset, LWCData, LWCStride, LWCOffset, Count);
	}
	else if (ConversionType == ENiagaraStructConversionType::Quat)
	{
		checkf(LWCBytes == 32 && SimulationBytes == 16, TEXT("Wrong bytesizes for Quat d->f conversion: Source %i, Simulation %i"), LWCBytes, SimulationBytes);
		CopyInternal<FQuat4f, FQuat4d>(SimulationData, SimulationStride, SimulationOffset, LWCData, LWCStride, LWCOffset, Count);
	}
	else
	{
		checkf(LWCBytes == SimulationBytes, TEXT("LWCBytes != TargetBytes: %i != %i"), LWCBytes, SimulationBytes);
		for (int32 i = 0; i < Count; ++i)
		{
			uint8* Dst = (SimulationData + i * SimulationStride) + SimulationOffset;
			const uint8* Src = (LWCData + i * LWCStride) + LWCOffset;
			FMemory::Memcpy(Dst, Src, LWCBytes);
		}
	}
}

void FNiagaraStructConversionStep::CopyFromSim(uint8* LWCData, const uint8* SimulationData, int32 Count, int32 LWCStride, int32 SimulationStride) const
{
	if (ConversionType == ENiagaraStructConversionType::DoubleToFloat)
	{
		checkf(LWCBytes == 8 && SimulationBytes == 4, TEXT("Wrong bytesizes for float->double conversion: Source %i, Simulation %i"), LWCBytes, SimulationBytes);
		CopyInternal<double, float>(LWCData, LWCStride, LWCOffset, SimulationData, SimulationStride, SimulationOffset, Count);
	}
	else if (ConversionType == ENiagaraStructConversionType::Vector2)
	{
		checkf(LWCBytes == 16 && SimulationBytes == 8, TEXT("Wrong bytesizes for Vector2 f->d conversion: Source %i, Simulation %i"), LWCBytes, SimulationBytes);
		CopyInternal<FVector2D, FVector2f>(LWCData, LWCStride, LWCOffset, SimulationData, SimulationStride, SimulationOffset, Count);
	}
	else if (ConversionType == ENiagaraStructConversionType::Vector3)
	{
		checkf(LWCBytes == 24 && SimulationBytes == 12, TEXT("Wrong bytesizes for Vector3 f->d conversion: Source %i, Simulation %i"), LWCBytes, SimulationBytes);
		CopyInternal<FVector, FVector3f>(LWCData, LWCStride, LWCOffset, SimulationData, SimulationStride, SimulationOffset, Count);
	}
	else if (ConversionType == ENiagaraStructConversionType::Vector4)
	{
		checkf(LWCBytes == 32 && SimulationBytes == 16, TEXT("Wrong bytesizes for Vector4 f->d conversion: Source %i, Simulation %i"), LWCBytes, SimulationBytes);
		CopyInternal<FVector4d, FVector4f>(LWCData, LWCStride, LWCOffset, SimulationData, SimulationStride, SimulationOffset, Count);
	}
	else if (ConversionType == ENiagaraStructConversionType::Quat)
	{
		checkf(LWCBytes == 32 && SimulationBytes == 16, TEXT("Wrong bytesizes for Quat f->d conversion: Source %i, Simulation %i"), LWCBytes, SimulationBytes);
		CopyInternal<FQuat4d, FQuat4f>(LWCData, LWCStride, LWCOffset, SimulationData, SimulationStride, SimulationOffset, Count);
	}
	else
	{
		checkf(LWCBytes == SimulationBytes, TEXT("LWCBytes != TargetBytes: %i != %i"), LWCBytes, SimulationBytes);
		for (int32 i = 0; i < Count; ++i)
		{
			uint8* Dst = (LWCData + i * LWCStride) + LWCOffset; 
			const uint8* Src = (SimulationData + i * SimulationStride) + SimulationOffset;
			FMemory::Memcpy(Dst, Src, LWCBytes);
		}
	}
}

void FNiagaraLwcStructConverter::ConvertDataToSimulation(uint8* DestinationData, const uint8* SourceData, int32 Count) const
{
	if (IsValid() == false || Count == 0)
	{
		return;
	}

	for (const FNiagaraStructConversionStep& ConversionStep : ConversionSteps)
	{
		ConversionStep.CopyToSim(SourceData, DestinationData, Count, LWCSize, SWCSize);
	}
}

void FNiagaraLwcStructConverter::ConvertDataFromSimulation(uint8* DestinationData, const uint8* SourceData, int32 Count) const
{
	if (IsValid() == false || Count == 0)
	{
		return;
	}

	for (const FNiagaraStructConversionStep& ConversionStep : ConversionSteps)
	{
		ConversionStep.CopyFromSim(DestinationData, SourceData, Count, LWCSize, SWCSize);
	}
}

void FNiagaraLwcStructConverter::AddConversionStep(int32 InSourceBytes, int32 InSourceOffset, int32 InSimulationBytes, int32 InSimulationOffset, ENiagaraStructConversionType ConversionType)
{
	check(InSourceBytes >= 0);
	check(InSourceOffset >= 0);
	check(InSimulationBytes >= 0);
	check(InSimulationOffset >= 0);
	// TODO (mg) as an optimization, if the last step was a copy step and this one is as well, they can be merged into one
	ConversionSteps.Emplace(InSourceBytes, InSourceOffset, InSimulationBytes, InSimulationOffset, ConversionType);

	// SWCSize and LWCSize are the sizes of the full structs.
	// We're updating them with each added step, as the full struct size should be (last property offset + size).
	SWCSize = InSimulationOffset + InSimulationBytes;
	LWCSize = InSourceOffset + InSourceBytes;
}

//////////////////////////////////////////////////////////////////////////

FNiagaraTypeDefinition FNiagaraTypeHelper::Vector2DDef;
FNiagaraTypeDefinition FNiagaraTypeHelper::VectorDef;
FNiagaraTypeDefinition FNiagaraTypeHelper::Vector4Def;
FNiagaraTypeDefinition FNiagaraTypeHelper::QuatDef;
FNiagaraTypeDefinition FNiagaraTypeHelper::DoubleDef;
FString FNiagaraTypeHelper::ConvertedSWCStructSuffix = TEXT("_SWC");

FRWLock FNiagaraTypeHelper::RemapTableLock;
TMap<TWeakObjectPtr<UScriptStruct>, FNiagaraTypeHelper::FRemapEntry> FNiagaraTypeHelper::RemapTable;
std::atomic<bool> FNiagaraTypeHelper::RemapTableDirty;

FString FNiagaraTypeHelper::ToString(const uint8* ValueData, const UObject* StructOrEnum)
{
	FString Ret;
	if (const UEnum* Enum = Cast<const UEnum>(StructOrEnum))
	{
		Ret = Enum->GetNameStringByValue(*reinterpret_cast<const int32*>(ValueData));
	}
	else if (const UScriptStruct* Struct = Cast<const UScriptStruct>(StructOrEnum))
	{
		if (Struct == FNiagaraTypeDefinition::GetFloatStruct())
		{
			Ret += FString::Printf(TEXT("%g "), *reinterpret_cast<const float*>(ValueData));
		}
		else if (Struct == FNiagaraTypeDefinition::GetIntStruct())
		{
			Ret += FString::Printf(TEXT("%d "), *reinterpret_cast<const int32*>(ValueData));
		}
		else if (Struct == FNiagaraTypeDefinition::GetBoolStruct())
		{
			int32 Val = *reinterpret_cast<const int32*>(ValueData);
			Ret += Val == 0xFFFFFFFF ? (TEXT("True")) : (Val == 0x0 ? TEXT("False") : TEXT("Invalid"));
		}
		else
		{
			for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
			{
				const FProperty* Property = *PropertyIt;
				const uint8* PropPtr = ValueData + PropertyIt->GetOffset_ForInternal();
				if (Property->IsA(FFloatProperty::StaticClass()))
				{
					Ret += FString::Printf(TEXT("%s: %g "), *Property->GetNameCPP(), *reinterpret_cast<const float*>(PropPtr));
				}
				else if (Property->IsA(FDoubleProperty::StaticClass()))
				{
					Ret += FString::Printf(TEXT("%s: %g "), *Property->GetNameCPP(), *reinterpret_cast<const double*>(PropPtr));
				}
				else if (Property->IsA(FUInt16Property::StaticClass()))
				{
					FFloat16 Val = *reinterpret_cast<const FFloat16*>(PropPtr);
					Ret += FString::Printf(TEXT("%s: %f "), *Property->GetNameCPP(), Val.GetFloat());
				}
				else if (Property->IsA(FIntProperty::StaticClass()))
				{
					Ret += FString::Printf(TEXT("%s: %d "), *Property->GetNameCPP(), *reinterpret_cast<const int32*>(PropPtr));
				}
				else if (Property->IsA(FBoolProperty::StaticClass()))
				{
					int32 Val = *reinterpret_cast<const int32*>(ValueData);
					FString BoolStr = Val == 0xFFFFFFFF ? (TEXT("True")) : (Val == 0x0 ? TEXT("False") : TEXT("Invalid"));
					Ret += FString::Printf(TEXT("%s: %s "), *Property->GetNameCPP(), *BoolStr);
				}
				else if (const FStructProperty* StructProp = CastFieldChecked<const FStructProperty>(Property))
				{
					Ret += FString::Printf(TEXT("%s: (%s) "), *Property->GetNameCPP(), *FNiagaraTypeHelper::ToString(PropPtr, StructProp->Struct));
				}
				else
				{
					check(false);
					Ret += TEXT("Unknown Type");
				}
			}
		}
	}
	return Ret;
}

FNiagaraLwcStructConverter BuildSWCStructure(UScriptStruct* NewStruct, UScriptStruct* InStruct)
{
	static UScriptStruct* Vector2fStruct = FNiagaraTypeDefinition::GetVec2Struct();
	static UScriptStruct* Vector3fStruct = FNiagaraTypeDefinition::GetVec3Struct();
	static UScriptStruct* Vector4fStruct = FNiagaraTypeDefinition::GetVec4Struct();
	static UScriptStruct* Quat4fStruct = FNiagaraTypeDefinition::GetQuatStruct();

	FNiagaraLwcStructConverter StructConverter;
	int32 AlignedOffset = 0;
	FProperty* LastProperty = nullptr;
	for (const FField* ChildProperty = InStruct->ChildProperties; ChildProperty; ChildProperty = ChildProperty->Next)
	{
		FProperty* NewProperty = nullptr;
		if (const FDoubleProperty* ChildAsDouble = CastField<const FDoubleProperty>(ChildProperty))
		{
			AlignedOffset = Align(AlignedOffset, ChildAsDouble->GetMinAlignment());
			NewProperty = new FFloatProperty(NewStruct, ChildProperty->GetFName());
			StructConverter.AddConversionStep(ChildAsDouble->GetSize(), ChildAsDouble->GetOffset_ForInternal(), NewProperty->GetSize(), AlignedOffset, ENiagaraStructConversionType::DoubleToFloat);
		}
		else if (const FStructProperty* ChildAsStruct = CastField<const FStructProperty>(ChildProperty))
		{
			FStructProperty* NewStructProperty = new FStructProperty(NewStruct, ChildProperty->GetFName());
			NewProperty = NewStructProperty;

			FName StructName = ChildAsStruct->Struct->GetFName();
			int32 OldPropertySize = ChildAsStruct->GetSize();
			int32 OldPropertyOffset = ChildAsStruct->GetOffset_ForInternal();
			if ((StructName == NAME_Vector2d) || (StructName == NAME_Vector2D))
			{
				NewStructProperty->Struct = Vector2fStruct;
				NewStructProperty->SetElementSize(Vector2fStruct->GetStructureSize());
				AlignedOffset = Align(AlignedOffset, Vector2fStruct->GetMinAlignment());
				StructConverter.AddConversionStep(OldPropertySize, OldPropertyOffset, NewStructProperty->GetSize(), AlignedOffset, ENiagaraStructConversionType::Vector2);
			}
			else if ((StructName == NAME_Vector3d) || (StructName == NAME_Vector))
			{
				NewStructProperty->Struct = Vector3fStruct;
				NewStructProperty->SetElementSize(Vector3fStruct->GetStructureSize());
				AlignedOffset = Align(AlignedOffset, Vector3fStruct->GetMinAlignment());
				StructConverter.AddConversionStep(OldPropertySize, OldPropertyOffset, NewStructProperty->GetSize(), AlignedOffset, ENiagaraStructConversionType::Vector3);
			}
			else if ((StructName == NAME_Vector4d) || (StructName == NAME_Vector4))
			{
				NewStructProperty->Struct = Vector4fStruct;
				NewStructProperty->SetElementSize(Vector4fStruct->GetStructureSize());
				AlignedOffset = Align(AlignedOffset, Vector4fStruct->GetMinAlignment());
				StructConverter.AddConversionStep(OldPropertySize, OldPropertyOffset, NewStructProperty->GetSize(), AlignedOffset, ENiagaraStructConversionType::Vector4);
			}
			else if ((StructName == NAME_Quat4d) || (StructName == NAME_Quat))
			{
				NewStructProperty->Struct = Quat4fStruct;
				NewStructProperty->SetElementSize(Quat4fStruct->GetStructureSize());
				AlignedOffset = Align(AlignedOffset, Quat4fStruct->GetMinAlignment());
				StructConverter.AddConversionStep(OldPropertySize, OldPropertyOffset, NewStructProperty->GetSize(), AlignedOffset, ENiagaraStructConversionType::Quat);
			}
			else
			{
				NewStructProperty->Struct = ChildAsStruct->Struct;
				NewStructProperty->SetElementSize(ChildAsStruct->Struct->GetStructureSize());
				AlignedOffset = Align(AlignedOffset, ChildAsStruct->Struct->GetMinAlignment());
				StructConverter.AddConversionStep(OldPropertySize, OldPropertyOffset, NewStructProperty->GetSize(), AlignedOffset, ENiagaraStructConversionType::CopyOnly);
			}
		}
		else if (
			ChildProperty->IsA(FFloatProperty::StaticClass()) ||
			ChildProperty->IsA(FIntProperty::StaticClass()) ||
			ChildProperty->IsA(FUInt32Property::StaticClass()) ||
			ChildProperty->IsA(FEnumProperty::StaticClass()) ||
			ChildProperty->IsA(FByteProperty::StaticClass()) ||
			ChildProperty->IsA(FBoolProperty::StaticClass()))
		{
			const FProperty* OldProperty = CastField<const FProperty>(ChildProperty);
			NewProperty = CastField<FProperty>(FField::Duplicate(ChildProperty, NewStruct, ChildProperty->GetFName()));
			AlignedOffset = Align(AlignedOffset, OldProperty->GetMinAlignment());
			StructConverter.AddConversionStep(NewProperty->GetSize(), OldProperty->GetOffset_ForInternal(), NewProperty->GetSize(), AlignedOffset, ENiagaraStructConversionType::CopyOnly);
		}

		if ( NewProperty == nullptr )
		{
			UE_LOGF(LogNiagara, Fatal, "Unsupported property '%ls' in custom struct '%ls'", *InStruct->GetName(), *ChildProperty->GetName());
		}

#if WITH_EDITORONLY_DATA
		FField::CopyMetaData(ChildProperty, NewProperty);
#endif
		if (LastProperty)
		{
			LastProperty->Next = NewProperty;
			LastProperty->PropertyLinkNext = NewProperty;
		}
		else
		{
			NewStruct->PropertyLink = NewProperty;
			NewStruct->ChildProperties = NewProperty;
		}
		LastProperty = NewProperty;
		AlignedOffset += NewProperty->GetSize();
	}

	return StructConverter;
}

bool FNiagaraTypeHelper::IsLWCStructure(const UStruct* InStruct)
{
	for (const FField* ChildProperty = InStruct->ChildProperties; ChildProperty; ChildProperty = ChildProperty->Next)
	{
		if (ChildProperty->IsA<FDoubleProperty>())
		{
			return true;
		}
		if ( const FStructProperty* ChildAsStruct = CastField<const FStructProperty>(ChildProperty) )
		{
			FName StructName = ChildAsStruct->Struct->GetFName();
			if ( (StructName == NAME_Vector2d) || (StructName == NAME_Vector2D) ||
				 (StructName == NAME_Vector3d) || (StructName == NAME_Vector) ||
				 (StructName == NAME_Vector4d) || (StructName == NAME_Vector4) ||
				 (StructName == NAME_Quat4d) || (StructName == NAME_Quat) )
			{
				return true;
			}
		}
	}
	return false;
}

bool FNiagaraTypeHelper::IsConvertedSWCStructure(const UStruct* InStruct)
{
	if (InStruct)
	{
		FNameBuilder StructName(InStruct->GetFName());
		if (!StructName.ToView().EndsWith(ConvertedSWCStructSuffix))
		{
			return false;
		}
	}

	{
		FReadScopeLock ReadLock(RemapTableLock);
		for (auto It = RemapTable.CreateConstIterator(); It; ++It)
		{
			if (It->Value.Struct.Get() == InStruct)
			{
				return true;
			}
		}
	}
	return false;
}

bool FNiagaraTypeHelper::IsLWCType(const FNiagaraTypeDefinition& InType)
{
	return InType.IsValid() && !InType.IsEnum() && !InType.IsUObject() && IsLWCStructure(InType.GetStruct());
}

void FNiagaraTypeHelper::TickTypeRemap()
{
	if (RemapTableDirty)
	{
		FWriteScopeLock WriteLock(RemapTableLock);

		for (auto it = RemapTable.CreateIterator(); it; ++it)
		{
			UScriptStruct* SrcScript = it->Key.Get();
			if (SrcScript == nullptr || it->Value.Get(SrcScript) == nullptr)
			{
				UScriptStruct* DstStruct = it->Value.Struct.Get();
				if (DstStruct && SrcScript != DstStruct)
				{
					DstStruct->RemoveFromRoot();
				}
				it.RemoveCurrent();
			}
		}

		RemapTableDirty = false;
	}
}

UScriptStruct* FNiagaraTypeHelper::GetSWCStruct(UScriptStruct* LWCStruct)
{
	// Attempt to find existing struct
	UScriptStruct* SWCStruct = nullptr;
	{
		FReadScopeLock ReadLock(RemapTableLock);
		if ( FRemapEntry* RemapEntry = RemapTable.Find(LWCStruct) )
		{
			SWCStruct = RemapEntry->Get(LWCStruct);
			RemapTableDirty = true;
		}
	}

	// No SWCStruct or it become invalid we need to create one
	if (SWCStruct == nullptr)
	{
		// Enable write lock and search again as between the two locks things may have changed
		FWriteScopeLock WriteLock(RemapTableLock);
		if (FRemapEntry* RemapEntry = RemapTable.Find(LWCStruct))
		{
			SWCStruct = RemapEntry->Get(LWCStruct);
			RemapTableDirty = true;
		}

		if (SWCStruct == nullptr)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Niagara_GetSWCStruct);

			// If this is a LWC structure we need to built a new structure now
			if (IsLWCStructure(LWCStruct))
			{
				//Return common SWC variant of common LWC Types.
				if (LWCStruct == FNiagaraDouble::StaticStruct())
				{
					SWCStruct = FNiagaraFloat::StaticStruct();
				}
				else if(LWCStruct == GetVector2DDef().GetStruct()) 
				{
					SWCStruct = FNiagaraTypeDefinition::GetVec2Struct();
				}
				else if (LWCStruct == GetVectorDef().GetStruct())
				{
					SWCStruct = FNiagaraTypeDefinition::GetVec3Struct();
				}
				else if (LWCStruct == GetVector4Def().GetStruct())
				{
					SWCStruct = FNiagaraTypeDefinition::GetVec4Struct();
				}
				else if (LWCStruct == GetQuatDef().GetStruct())
				{
					SWCStruct = FNiagaraTypeDefinition::GetQuatStruct();
				}
				//More?
				else
				{
					FName SWCName = FName(LWCStruct->GetName() + ConvertedSWCStructSuffix);

					//check if the remap table contains a previous entry for that struct. This might happen when the lwc struct was changed at runtime.
					// In that case we want to reuse the existing swc struct object, since it might already be referenced by compile results.
					for (auto Iter = RemapTable.CreateIterator(); Iter; ++Iter)
					{
						TWeakObjectPtr<UScriptStruct> ExistingStruct = Iter.Value().Struct;
						if (ExistingStruct.IsValid() && ExistingStruct->GetFName() == SWCName)
						{
							SWCStruct = ExistingStruct.Get();
							Iter.RemoveCurrent();
							RemapTableDirty = true;
							break;
						}
					}

					if (SWCStruct == nullptr)
					{
						SWCStruct = NewObject<UScriptStruct>(GetTransientPackage(), SWCName, RF_NoFlags);
						SWCStruct->AddToRoot();
					}
					FNiagaraLwcStructConverter StructConverter = BuildSWCStructure(SWCStruct, LWCStruct);
					FNiagaraTypeDefinition LwcTypeDefinition = FNiagaraTypeRegistry::GetTypeForStruct(LWCStruct);
					FNiagaraTypeRegistry::RegisterStructConverter(LwcTypeDefinition, StructConverter);
					SWCStruct->Bind();
					SWCStruct->PrepareCppStructOps();
					SWCStruct->StaticLink(true);
				}
			}
			else
			{
				SWCStruct = LWCStruct;
			}
			FRemapEntry& RemapEntry = RemapTable.Add(TWeakObjectPtr<UScriptStruct>(LWCStruct));
			RemapEntry.Struct = SWCStruct;
		#if WITH_EDITORONLY_DATA
			RemapEntry.SerialNumber = LWCStruct->FieldPathSerialNumber;
		#endif
		}
	}
	return SWCStruct;
}

UScriptStruct* FNiagaraTypeHelper::GetLWCStruct(UScriptStruct* SWCStruct)
{
	if (SWCStruct->GetOutermost() != GetTransientPackage())
	{
		return SWCStruct;
	}

	FReadScopeLock ReadLock(RemapTableLock);

	for (auto It = RemapTable.CreateConstIterator(); It; ++It)
	{
		const FRemapEntry& RemapEntry = It.Value();
		if (RemapEntry.Struct.Get() == SWCStruct)
		{
			return It.Key().Get();
		}
	}

	return nullptr;
}

FNiagaraTypeDefinition FNiagaraTypeHelper::GetSWCType(const FNiagaraTypeDefinition& InType)
{
	if (InType.IsEnum() || InType.IsUObject() || InType.IsDataInterface())
	{
		return InType;
	}

	if (IsLWCType(InType))
	{
		return FNiagaraTypeDefinition(FindNiagaraFriendlyTopLevelStruct(CastChecked<UScriptStruct>(InType.GetStruct()), ENiagaraStructConversion::Simulation));
	}
	return InType;
}

FNiagaraTypeDefinition FNiagaraTypeHelper::GetLWCType(const FNiagaraTypeDefinition& InType)
{
	if(InType.IsEnum() || InType.IsUObject() || InType.IsDataInterface())
	{
		return InType;
	}

	InitLWCTypes();
	if(InType == FNiagaraTypeDefinition::GetFloatDef())
	{
		return DoubleDef;
	}
	else if (InType == FNiagaraTypeDefinition::GetVec2Def())
	{
		return Vector2DDef;
	}
	else if (InType == FNiagaraTypeDefinition::GetVec3Def())
	{
		return VectorDef;
	}
	else if (InType == FNiagaraTypeDefinition::GetVec4Def())
	{
		return Vector4Def;
	}
	else if (InType == FNiagaraTypeDefinition::GetQuatDef())
	{
		return QuatDef;
	}

	if (IsLWCType(InType))
	{
		return InType;
	}

	if(UScriptStruct* LWCStruct = GetLWCStruct(InType.GetScriptStruct()))
	{
		return FNiagaraTypeDefinition(LWCStruct, FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Allow);
	}
	
	return InType;
}

void FNiagaraTypeHelper::InitLWCTypes()
{
	if(Vector2DDef.IsValid() == false)
	{
		UPackage* CoreUObjectPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/CoreUObject"));
		Vector2DDef = FNiagaraTypeDefinition(FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Vector2D")), FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Allow);
		VectorDef = FNiagaraTypeDefinition(FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Vector")), FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Allow);
		Vector4Def = FNiagaraTypeDefinition(FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Vector4")), FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Allow);
		QuatDef = FNiagaraTypeDefinition(FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Quat")), FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Allow);
		DoubleDef = FNiagaraTypeDefinition(FNiagaraDouble::StaticStruct(), FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Allow);

		FNiagaraTypeRegistry::Register(Vector2DDef, ENiagaraTypeRegistryFlags::None);
		FNiagaraTypeRegistry::Register(VectorDef, ENiagaraTypeRegistryFlags::None);
		FNiagaraTypeRegistry::Register(Vector4Def, ENiagaraTypeRegistryFlags::None);
		FNiagaraTypeRegistry::Register(QuatDef, ENiagaraTypeRegistryFlags::None);
		FNiagaraTypeRegistry::Register(DoubleDef, ENiagaraTypeRegistryFlags::None);
	}
}

UScriptStruct* FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(UScriptStruct* InStruct, ENiagaraStructConversion StructConversion)
{
	if (!InStruct)
	{
		return nullptr;
	}

	// Note: UE core types are converted to the float variant as Niagara only works with float types.
	if (InStruct->GetFName() == NAME_NiagaraDouble)
	{
		return FNiagaraTypeDefinition::GetFloatStruct();
	}

	if (InStruct->GetFName() == NAME_Vector2D || InStruct->GetFName() == NAME_Vector2d) // LWC support
	{
		return FNiagaraTypeDefinition::GetVec2Struct();
	}

	if (InStruct->GetFName() == NAME_Vector || InStruct->GetFName() == NAME_Vector3d) // LWC support
	{
		return FNiagaraTypeDefinition::GetVec3Struct();
	}

	if (InStruct->GetFName() == NAME_Vector4 || InStruct->GetFName() == NAME_Vector4d) // LWC support
	{
		return FNiagaraTypeDefinition::GetVec4Struct();
	}

	if (InStruct->GetFName() == NAME_Quat || InStruct->GetFName() == NAME_Quat4d) // LWC support
	{
		return FNiagaraTypeDefinition::GetQuatStruct();
	}

	if (InStruct->GetFName() == NAME_NiagaraPosition)
	{
		return FNiagaraTypeDefinition::GetPositionStruct();
	}

	if (InStruct->IsA<UNiagaraDataInterface>())
	{
		return InStruct;
	}

	if (StructConversion == ENiagaraStructConversion::UserFacing)
	{
		return InStruct;
	}
	check(StructConversion == ENiagaraStructConversion::Simulation);

	//-OPT: We could potentially add a list of known types here, it may or may not be faster
	return GetSWCStruct(InStruct);
}

bool FNiagaraTypeHelper::IsNiagaraFriendlyTopLevelStruct(UScriptStruct* InStruct, ENiagaraStructConversion StructConversion)
{
	return FindNiagaraFriendlyTopLevelStruct(InStruct, StructConversion) == InStruct;
}

/////////////////////////////////////////////////////////////////////////////

void FNiagaraTypeLayoutInfo::GenerateLayoutInfo(const UScriptStruct* Struct)
{
	int32 FloatCount = 0;
	int32 Int32Count = 0;
	int32 HalfCount = 0;
	GenerateLayoutInfoInternal(Struct, FloatCount, Int32Count, HalfCount, true);
	NumFloatComponents = IntCastChecked<uint16>(FloatCount);
	NumInt32Components = IntCastChecked<uint16>(Int32Count);
	NumHalfComponents = IntCastChecked<uint16>(HalfCount);

	FloatCount = 0;
	Int32Count = 0;
	HalfCount = 0;
	ComponentsOffsets.SetNum(GetTotalComponents() * 2);
	GenerateLayoutInfoInternal(Struct, FloatCount, Int32Count, HalfCount, false);
}

void FNiagaraTypeLayoutInfo::GenerateLayoutInfoInternal(const UScriptStruct* Struct, int32& FloatCount, int32& Int32Count, int32& HalfCount, bool bCountComponentsOnly, int32 BaseOffset)
{
	for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		const FProperty* Property = *PropertyIt;
		const int32 PropOffset = BaseOffset + Property->GetOffset_ForInternal();
		if (Property->IsA(FFloatProperty::StaticClass()))
		{
			if (bCountComponentsOnly == false)
			{
				ComponentsOffsets[GetFloatArrayByteOffset() + FloatCount] = PropOffset;
				ComponentsOffsets[GetFloatArrayRegisterOffset() + FloatCount] = FloatCount;
			}
			++FloatCount;
		}
		else if (Property->IsA(FUInt16Property::StaticClass()))
		{
			if (bCountComponentsOnly == false)
			{
				ComponentsOffsets[GetHalfArrayByteOffset() + HalfCount] = PropOffset;
				ComponentsOffsets[GetHalfArrayRegisterOffset() + HalfCount] = HalfCount;
			}
			++HalfCount;
		}
		else if (Property->IsA(FIntProperty::StaticClass()) || Property->IsA(FBoolProperty::StaticClass()))
		{
			if (bCountComponentsOnly == false)
			{
				ComponentsOffsets[GetInt32ArrayByteOffset() + Int32Count] = PropOffset;
				ComponentsOffsets[GetInt32ArrayRegisterOffset() + Int32Count] = Int32Count;
			}
			++Int32Count;
		}
		//Should be able to support double easily enough
		else if (const FStructProperty* StructProp = CastField<FStructProperty>(Property))
		{
			GenerateLayoutInfoInternal(FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProp->Struct, ENiagaraStructConversion::Simulation), FloatCount, Int32Count, HalfCount, bCountComponentsOnly, PropOffset);
		}
		else
		{
			checkf(false, TEXT("Property(%s) Class(%s) is not a supported type"), *Property->GetName(), *Property->GetClass()->GetName());
		}
	}
}
