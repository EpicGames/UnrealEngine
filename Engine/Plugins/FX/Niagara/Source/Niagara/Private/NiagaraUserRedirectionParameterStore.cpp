// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraUserRedirectionParameterStore.h"

#include "NiagaraConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraUserRedirectionParameterStore)

FNiagaraUserRedirectionParameterStore::FNiagaraUserRedirectionParameterStore() : FNiagaraParameterStore()
{

}

FNiagaraUserRedirectionParameterStore::FNiagaraUserRedirectionParameterStore(const FNiagaraParameterStore& Other)
{
	*this = Other;
}

FNiagaraUserRedirectionParameterStore& FNiagaraUserRedirectionParameterStore::operator=(const FNiagaraParameterStore& Other)
{
	Super::operator=(Other);	
	RecreateRedirections();
	return *this;
}

bool FNiagaraUserRedirectionParameterStore::IsUserParameter(const FNiagaraVariableBase& InVar) 
{
	FNameBuilder VarName(InVar.GetName());
	return VarName.ToView().StartsWith(TEXT("User."));
}

void FNiagaraUserRedirectionParameterStore::MakeUserVariable(FNiagaraVariableBase& InVar)
{
	if (IsUserParameter(InVar))
	{
		return;
	}
	FNameBuilder DisplayName;
	DisplayName.Append(TEXT("User."));
	InVar.GetName().AppendString(DisplayName);
	InVar.SetName(FName(DisplayName));
	return;
}


FNiagaraVariable FNiagaraUserRedirectionParameterStore::GetUserRedirection(const FNiagaraVariable & InVar) const
{
	if (!IsUserParameter(InVar))
	{
		return InVar;
	}
	FNiagaraVariable SimpleVar = InVar;
	FNameBuilder DisplayName;
	InVar.GetName().ToString(DisplayName);
	SimpleVar.SetName(FName(DisplayName.ToView().RightChop(5)));
	return SimpleVar;
}

void FNiagaraUserRedirectionParameterStore::RecreateRedirections()
{
	UserParameterRedirects.Reset();

	for (const FNiagaraVariable Var : ReadParameterVariables())
	{
		if (IsUserParameter(Var))
		{
			UserParameterRedirects.Add(GetUserRedirection(Var), Var);
		}
	}
}

bool FNiagaraUserRedirectionParameterStore::RedirectUserVariable(FNiagaraVariableBase& UserVar) const
{
	if (const FNiagaraVariable* RedirectedKey = UserParameterRedirects.Find(UserVar))
	{
		UserVar = FNiagaraVariableBase(*RedirectedKey);
		return true;
	}

	if (IsUserParameter(UserVar))
	{
		return true;
	}

	return false;
}

bool FNiagaraUserRedirectionParameterStore::AddParameter(const FNiagaraVariable& Param, bool bInitialize /*= true*/, bool bTriggerRebind /*= true*/, int32* OutOffset /*= nullptr*/)
{
	FNiagaraVariable AddParam;
	if (IsUserParameter(Param))
	{
		AddParam = Param;
	}
	else
	{
		FNameBuilder NameBuilder;
		FNiagaraConstants::UserNamespace.AppendString(NameBuilder);
		NameBuilder << TEXT(".");
		Param.GetName().AppendString(NameBuilder);
		AddParam = FNiagaraVariable(Param.GetType(), FName(NameBuilder.ToView()));
	}

	UserParameterRedirects.Add(GetUserRedirection(AddParam), AddParam);
	return Super::AddParameter(AddParam, bInitialize, bTriggerRebind, OutOffset);
}

void FNiagaraUserRedirectionParameterStore::SanityCheckData(bool bInitInterfaces)
{
	Super::SanityCheckData(bInitInterfaces);

	TArray<FNiagaraVariable> Vars;
	UserParameterRedirects.GenerateValueArray(Vars);
	for (const FNiagaraVariable& Var : Vars)
	{
		int32 Offset = IndexOf(Var);
		if (Offset == -1)
		{
			UE_LOGF(LogNiagara, Log, "User parameter redirect exists but no real value! Param: %ls Owner:%ls", *Var.GetName().ToString(), GetOwner() != nullptr ? *GetOwner()->GetPathName() : TEXT("Unknown owner"));
		}
	}
}

bool FNiagaraUserRedirectionParameterStore::RemoveParameter(const FNiagaraVariableBase& InVar)
{
	const FNiagaraVariable* Redirection = UserParameterRedirects.Find(InVar);
	const FNiagaraVariable& ToRemove = Redirection ? *Redirection : InVar;
	
	UserParameterRedirects.Remove(GetUserRedirection(ToRemove));
	bool Result = Super::RemoveParameter(ToRemove);
	return Result;
}

void FNiagaraUserRedirectionParameterStore::InitFromSource(const FNiagaraParameterStore* SrcStore, bool bNotifyAsDirty)
{
	Super::InitFromSource(SrcStore, bNotifyAsDirty);
	RecreateRedirections();
}

void FNiagaraUserRedirectionParameterStore::Empty(bool bClearBindings /*= true*/)
{
	Super::Empty(bClearBindings);
	UserParameterRedirects.Empty();
}

void FNiagaraUserRedirectionParameterStore::Reset(bool bClearBindings /*= true*/)
{
	Super::Reset(bClearBindings);
	UserParameterRedirects.Reset();
}

bool FNiagaraUserRedirectionParameterStore::SetPositionParameterValue(const FVector& InValue, const FName& ParamName, bool bAdd)
{
	FNiagaraVariable PositionVar = FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), ParamName);
	const FNiagaraVariable* Redirection = UserParameterRedirects.Find(PositionVar);
	return FNiagaraParameterStore::SetPositionParameterValue(InValue, Redirection ? Redirection->GetName() : ParamName, bAdd);
}

const FVector* FNiagaraUserRedirectionParameterStore::GetPositionParameterValue(const FName& ParamName) const
{
	FNiagaraVariable PositionVar = FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), ParamName);
	const FNiagaraVariable* Redirection = UserParameterRedirects.Find(PositionVar);
	return FNiagaraParameterStore::GetPositionParameterValue(Redirection ? Redirection->GetName() : ParamName);
}

#if WITH_EDITORONLY_DATA
void FNiagaraUserRedirectionParameterStore::ConvertParameterType(const FNiagaraVariable& ExistingParam,	const FNiagaraTypeDefinition& NewType)
{
	FNiagaraParameterStore::ConvertParameterType(ExistingParam, NewType);

	FNiagaraVariable OldVar = GetUserRedirection(ExistingParam);
	FNiagaraVariable OldRedirect = ExistingParam;
	if (UserParameterRedirects.RemoveAndCopyValue(OldVar, OldRedirect))
	{
		UserParameterRedirects.Add(FNiagaraVariable(NewType, OldVar.GetName()), FNiagaraVariable(NewType, OldRedirect.GetName()));
	}
}
#endif

bool FNiagaraUserRedirectionParameterStore::SerializeFromMismatchedTag(const FPropertyTag & Tag, FStructuredArchive::FSlot Slot)
{
	static FName StoreDataName("NiagaraParameterStore");
	if (Tag.GetType().IsStruct(StoreDataName))
	{
		FNiagaraParameterStore OldStore;
		FNiagaraParameterStore::StaticStruct()->SerializeItem(Slot, &OldStore, nullptr);
		// Call PostLoad() to convert the serialized ParameterOffsets to SortedParameterOffsets.
		OldStore.PostLoad(nullptr);
		*this = OldStore;
		return true;
	}

	return false;
}

