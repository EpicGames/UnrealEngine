// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraUvQuadTree.h"

static float GNiagaraUvQuadTreeDuplicateThreshold = 0.25f;


FNiagaraUvQuadTree::FNiagaraUvQuadTree(int32 InNodeCapacity, int32 InMaxDepth)
	: NodeCapacity(InNodeCapacity)
	, MaxDepth(InMaxDepth)
{
	AddDefaultSubTree();
}

void FNiagaraUvQuadTree::Insert(TriangleIndex Element, const FBox2D& Box)
{
	check(ChildTrees.Num());

	InsertElement(0, ChildNodes.Emplace(Box, Element), 0);
}

void FNiagaraUvQuadTree::Empty()
{
	ChildTrees.Empty();
	ChildNodes.Empty();

	AddDefaultSubTree();
}

void FNiagaraUvQuadTree::AddDefaultSubTree()
{
	check(ChildTrees.Num() == 0);
	ChildTrees.Emplace(FBox2D(FVector2D::ZeroVector, FVector2D::UnitVector));
}

void FNiagaraUvQuadTree::Freeze(FArchive& Ar) const
{
	check(ChildTrees.Num());
	ChildTrees[0].Freeze(*this, Ar);
}

FNiagaraUvQuadTree::FSubTree::FSubTree(const FBox2D& InCoverage)
	: Coverage(InCoverage)
	, bInternal(false)
{
	SubTreeIndices[0] = SubTreeIndices[1] = SubTreeIndices[2] = SubTreeIndices[3] = INDEX_NONE;
}

int32 FNiagaraUvQuadTree::FSubTree::GetQuads(const FBox2D& Box, FChildArray& Quads) const
{
	int32 QuadCount = 0;
	if (bInternal)
	{
		const FVector2D Position = Coverage.GetCenter();

		bool bNegX = Box.Min.X <= Position.X;
		bool bNegY = Box.Min.Y <= Position.Y;

		bool bPosX = Box.Max.X >= Position.X;
		bool bPosY = Box.Max.Y >= Position.Y;

		if (bNegX && bNegY)
		{
			Quads[QuadCount++] = SubTreeIndices[BottomLeft];
		}

		if (bPosX && bNegY)
		{
			Quads[QuadCount++] = SubTreeIndices[BottomRight];
		}

		if (bNegX && bPosY)
		{
			Quads[QuadCount++] = SubTreeIndices[TopLeft];
		}

		if (bPosX && bPosY)
		{
			Quads[QuadCount++] = SubTreeIndices[TopRight];
		}
	}

	return QuadCount;
}

int32 FNiagaraUvQuadTree::FSubTree::Freeze(const FNiagaraUvQuadTree & QuadTree, FArchive& Ar) const
{
	int64 StartOffset = Ar.Tell();
	int32 PlaceholderIndex = INDEX_NONE;

	for (int32 OffsetIt = 0; OffsetIt < SubTreeIndices.Num(); ++OffsetIt)
	{
		Ar << PlaceholderIndex;
	}

	int32 TriangleCount = ContentIndices.Num();
	Ar << TriangleCount;
	for (int32 TriangleIt = 0; TriangleIt < TriangleCount; ++TriangleIt)
	{
		int32 ElementIndex = QuadTree.ChildNodes[ContentIndices[TriangleIt]].ElementIndex;
		Ar << ElementIndex;
	}

	if (bInternal)
	{
		constexpr uint32 ChildCount = 4;
		int32 ChildOffsets[ChildCount];

		for (int32 ChildIt = 0; ChildIt < ChildCount; ++ChildIt)
		{
			ChildOffsets[ChildIt] = QuadTree.ChildTrees[SubTreeIndices[ChildIt]].Freeze(QuadTree, Ar);
		}

		const int64 EndOffset = Ar.Tell();
		Ar.Seek(StartOffset);

		for (int32 ChildIt = 0; ChildIt < ChildCount; ++ChildIt)
		{
			Ar << ChildOffsets[ChildIt];
		}

		Ar.Seek(EndOffset);
	}

	// we want to return the index into the buffer (not the byte offset)
	return StartOffset / sizeof(int32);
}

void FNiagaraUvQuadTree::Split(int32 SubTreeIndex)
{
	ChildTrees.Reserve(ChildTrees.Num() + 4);

	FSubTree& Parent = ChildTrees[SubTreeIndex];
	check(Parent.bInternal == false);

	const FVector2D Extent = Parent.Coverage.GetExtent();
	const FVector2D XExtent = FVector2D(Extent.X, 0.f);
	const FVector2D YExtent = FVector2D(0.f, Extent.Y);

	/************************************************************************
	 *  ___________max
	 * |     |     |
	 * |     |     |
	 * |-----c------
	 * |     |     |
	 * min___|_____|
	 *
	 * We create new quads by adding xExtent and yExtent
	 ************************************************************************/

	const FVector2D C = Parent.Coverage.GetCenter();
	const FVector2D TM = C + YExtent;
	const FVector2D ML = C - XExtent;
	const FVector2D MR = C + XExtent;
	const FVector2D BM = C - YExtent;
	const FVector2D BL = Parent.Coverage.Min;
	const FVector2D TR = Parent.Coverage.Max;

	Parent.SubTreeIndices[FSubTree::TopLeft] = ChildTrees.Emplace(FBox2D(ML, TM));
	Parent.SubTreeIndices[FSubTree::TopRight] = ChildTrees.Emplace(FBox2D(C, TR));
	Parent.SubTreeIndices[FSubTree::BottomLeft] = ChildTrees.Emplace(FBox2D(BL, C));
	Parent.SubTreeIndices[FSubTree::BottomRight] = ChildTrees.Emplace(FBox2D(BM, MR));

	//mark as no longer a leaf
	Parent.bInternal = true;

	// Place existing nodes and place them into the new subtrees that contain them
	// If a node overlaps multiple subtrees we'll either keep it in the parent node or pass the
	// index to multiple nodes depending on the coverage
	TArray<int32> LargeContentIndices;

	for (int32 ContentIndex : Parent.ContentIndices)
	{
		const FNode& ContentNode = ChildNodes[ContentIndex];

		FSubTree::FChildArray Quads;
		const int32 QuadCount = Parent.GetQuads(ContentNode.Coverage, Quads);
		if (QuadCount == 1)
		{
			ChildTrees[Quads[0]].ContentIndices.Add(ContentIndex);
		}
		// if the node we're trying to push down is larger than our threshold we keep it in the parent node
		else
		{
			const FVector2D OverlapMin = FVector2D::Max(ContentNode.Coverage.Min, Parent.Coverage.Min);
			const FVector2D OverlapMax = FVector2D::Min(ContentNode.Coverage.Max, Parent.Coverage.Max);
			FBox2D OverlapRegion(OverlapMin, FVector2D::Max(OverlapMin, OverlapMax));
			const double AreaRatio = OverlapRegion.GetArea() / Parent.Coverage.GetArea();

			if (AreaRatio > GNiagaraUvQuadTreeDuplicateThreshold)
			{
				LargeContentIndices.Add(ContentIndex);
			}
			else
			{
				for (int32 QuadIt = 0; QuadIt < QuadCount; ++QuadIt)
				{
					ChildTrees[Quads[QuadIt]].ContentIndices.Add(ContentIndex);
				}
			}
		}
	}

	Swap(LargeContentIndices, Parent.ContentIndices);
}

void FNiagaraUvQuadTree::InsertElement(int32 SubTreeIndex, int32 ContentIndex, int32 CurrentDepth)
{
	FSubTree& Parent = ChildTrees[SubTreeIndex];
	const FBox2D& Box = ChildNodes[ContentIndex].Coverage;

	FSubTree::FChildArray Quads;
	const int32 QuadCount = Parent.GetQuads(Box, Quads);
	if (QuadCount == 0)
	{
		check(!Parent.bInternal);

		// It's possible that all elements in the leaf are bigger than the leaf or that more elements than NodeCapacity exist outside the top level quad
		// In either case, we can get into an endless spiral of splitting
		const bool bCanSplitTree = CurrentDepth < MaxDepth;
		if (!bCanSplitTree || Parent.ContentIndices.Num() < NodeCapacity)
		{
			Parent.ContentIndices.Add(ContentIndex);
		}
		else
		{
			// This quad is at capacity, so split and try again.  ChildTrees will be resized, potentially invalidating any references to elements
			Split(SubTreeIndex);
			InsertElement(SubTreeIndex, ContentIndex, CurrentDepth);
		}
	}
	else if (QuadCount == 1)
	{
		check(Parent.bInternal);

		// Fully contained in a single subtree, so insert it there
		InsertElement(Quads[0], ContentIndex, CurrentDepth + 1);
	}
	else if ((Box.GetArea() / Parent.Coverage.GetArea()) > GNiagaraUvQuadTreeDuplicateThreshold)
	{
		// Overlaps multiple subtrees, store here
		check(Parent.bInternal);
		Parent.ContentIndices.Add(ContentIndex);
	}
	else
	{
		for (int32 QuadIt = 0; QuadIt < QuadCount; ++QuadIt)
		{
			InsertElement(Quads[QuadIt], ContentIndex, CurrentDepth + 1);
		}
	}
}