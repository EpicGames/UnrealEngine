// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/StaticArray.h"
#include "Math/Box2D.h"

// Largely pulled from TQuadTree (GenericQuadTree.h) but differs in having a way to freeze the data into a buffer suited for the GPU
// and also will duplicate entries in the tree to reduce the number of elements stuck at the root level (elements overlapping leaves
// will be duplicated unless their size is on par with the node itself).
class FNiagaraUvQuadTree
{
public:
	using TriangleIndex = int32;

	FNiagaraUvQuadTree() = delete;
	FNiagaraUvQuadTree(int32 InNodeCapacity, int32 InMaxDepth);

	/** Inserts an object of type ElementType with an associated 2D box of size Box (log n). Pass in a DebugContext so when an issue occurs the log can report what requested this insert. */
	void Insert(TriangleIndex Element, const FBox2D& Box);

	/** Given a 2D box, returns an array of elements within the box. There will not be any duplicates in the list. */
	template<typename ElementAllocatorType>
	void GetElements(const FBox2D& Box, TArray<TriangleIndex, ElementAllocatorType>& ElementsOut) const
	{
		ChildTrees[0].GetElements(*this, Box, ElementsOut);
	}

	template<typename TAction>
	void VisitElements(const FBox2D& Box, TAction Func) const
	{
		ChildTrees[0].VisitElements(*this, Box, Func);
	}

	/** Removes all elements of the tree */
	void Empty();

	// todo - reduce to uint16
	//struct FFrozenNode
	//{
	//	int32 ChildOffsets[4]
	//	int32 ElementCount
	//	int32 TriangleIndices[]
	//};
	void Freeze(FArchive& Ar) const;

	const int32 NodeCapacity;
	const int32 MaxDepth;

private:
	struct FNode
	{
		FNode() = delete;
		FNode(const FBox2D& InCoverage, TriangleIndex InElementIndex)
		: Coverage(InCoverage)
		, ElementIndex(InElementIndex)
		{}

		FBox2D Coverage;
		TriangleIndex ElementIndex;
	};

	struct FSubTree
	{
		enum QuadNames
		{
			TopLeft = 0,
			TopRight = 1,
			BottomLeft = 2,
			BottomRight = 3
		};

		FSubTree(const FBox2D& InCoverage);

		using FChildArray = TStaticArray<int32, 4>;

		const FBox2D Coverage;
		FChildArray SubTreeIndices;
		TArray<int32> ContentIndices;
		bool bInternal = false;

		/** Given a 2D box, return the subtrees that are touched. Returns 0 for leaves. */
		int32 GetQuads(const FBox2D& Box, FChildArray& Quads) const;

		template<typename ElementAllocatorType>
		void GetElements(const FNiagaraUvQuadTree& QuadTree, const FBox2D& Box, TArray<TriangleIndex, ElementAllocatorType>& ElementsOut) const
		{
			GetIntersectingElements(QuadTree, Box, ElementsOut);

			FChildArray Quads;
			const int32 QuadCount = GetQuads(Box, Quads);
			for (int32 QuadIt = 0; QuadIt < QuadCount; ++QuadIt)
			{
				QuadTree.ChildTrees[Quads[QuadIt]].GetElements(QuadTree, Box, ElementsOut);
			}
		}

		/** Given a list of nodes, return which ones actually intersect the box */
		template<typename ElementAllocatorType>
		void GetIntersectingElements(const FNiagaraUvQuadTree& QuadTree, const FBox2D& Box, TArray<TriangleIndex, ElementAllocatorType>& ElementsOut) const
		{
			ElementsOut.Reserve(ElementsOut.Num() + ContentIndices.Num());
			for (int32 ContentIndex : ContentIndices)
			{
				const FNode& Content = QuadTree.ChildNodes[ContentIndex];
				if (Box.Intersect(Content.Coverage))
				{
					ElementsOut.Add(Content.ElementIndex);
				}
			}
		}

		template<typename TAction>
		bool VisitElements(const FNiagaraUvQuadTree& QuadTree, const FBox2D& Box, TAction Func) const
		{
			for (int32 ContentIndex : ContentIndices)
			{
				const FNode& Content = QuadTree.ChildNodes[ContentIndex];
				if (Box.Intersect(Content.Coverage))
				{
					if (!Func(ContentIndex))
					{
						return false;
					}
				}
			}

			FChildArray Quads;
			const int32 QuadCount = GetQuads(Box, Quads);
			for (int32 QuadIt = 0; QuadIt < QuadCount; ++QuadIt)
			{
				if (!QuadTree.ChildTrees[Quads[QuadIt]].VisitElements(QuadTree, Box, Func))
				{
					return false;
				}
			}

			return true;
		}

		int32 Freeze(const FNiagaraUvQuadTree& QuadTree, FArchive& Ar) const;
	};

	TArray<FSubTree> ChildTrees;
	TArray<FNode> ChildNodes;

	void AddDefaultSubTree();
	void Split(int32 SubTreeIndex);
	void InsertElement(int32 SubTreeIndex, int32 ContentIndex, int32 CurrentDepth);
};


