// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraValidationRule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraValidationRule)

#if WITH_EDITOR
void UNiagaraValidationRule::CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& Results) const
{
}
#endif
