// Copyright Epic Games, Inc. All Rights Reserved.


#include "NiagaraVariableMetaData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraVariableMetaData)

#if WITH_EDITOR
#include "PropertyHandle.h"

FNiagaraInputParameterCustomization FNiagaraInputParameterCustomization::MakeFromProperty(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	FNiagaraInputParameterCustomization ParameterCustomization;
	if (PropertyHandle.IsValid())
	{
		static const FName ClampMinName("ClampMin");
		static const FName ClampMaxName("ClampMax");
		static const FName DeltaName("Delta");
		static const FName WidgetTypeName = GET_MEMBER_NAME_CHECKED(FNiagaraInputParameterCustomization, WidgetType);
		if (PropertyHandle->HasMetaData(WidgetTypeName))
		{
			FString TypeString = PropertyHandle->GetMetaData(WidgetTypeName);
			UEnum* TypeEnum = StaticEnum<ENiagaraInputWidgetType>();
			int32 Value = static_cast<int32>(TypeEnum->GetValueByNameString(TypeString));
			if (Value != INDEX_NONE)
			{
				ParameterCustomization.WidgetType = static_cast<ENiagaraInputWidgetType>(Value);
			}
		}
		if (PropertyHandle->HasMetaData(ClampMinName))
		{
			ParameterCustomization.bHasMinValue = true;
			ParameterCustomization.MinValue = PropertyHandle->GetFloatMetaData(ClampMinName);
		}
		if (PropertyHandle->HasMetaData(ClampMaxName))
		{
			ParameterCustomization.bHasMaxValue = true;
			ParameterCustomization.MaxValue = PropertyHandle->GetFloatMetaData(ClampMaxName);
		}
		if (PropertyHandle->HasMetaData(DeltaName))
		{
			ParameterCustomization.bHasStepWidth = true;
			ParameterCustomization.StepWidth = PropertyHandle->GetFloatMetaData(DeltaName);
		}
	}
	return ParameterCustomization;
}
#endif

void FNiagaraVariableMetaData::CopyUserEditableMetaData(const FNiagaraVariableMetaData& OtherMetaData)
{
	for (const FProperty* ChildProperty : TFieldRange<FProperty>(StaticStruct()))
	{
		if (ChildProperty->HasAnyPropertyFlags(CPF_Edit))
		{
			int32 PropertyOffset = ChildProperty->GetOffset_ForInternal();
			ChildProperty->CopyCompleteValue((uint8*)this + PropertyOffset, (uint8*)&OtherMetaData + PropertyOffset);
		};
	}
}
