// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraVariant.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraDataInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraVariant)

FNiagaraVariant::FNiagaraVariant()
{
	CurrentMode = ENiagaraVariantMode::None;
	Object = nullptr;
	DataInterface = nullptr;
}

FNiagaraVariant::FNiagaraVariant(TObjectPtr<UObject> InObject)
{
	CurrentMode = ENiagaraVariantMode::Object;
	Object = InObject;
	DataInterface = nullptr;
}

FNiagaraVariant::FNiagaraVariant(UNiagaraDataInterface* InDataInterface)
{
	CurrentMode = ENiagaraVariantMode::DataInterface;
	DataInterface = InDataInterface;
	Object = nullptr;
}

FNiagaraVariant::FNiagaraVariant(const TArray<uint8>& InBytes)
{
	CurrentMode = ENiagaraVariantMode::Bytes;
	Bytes = InBytes;
	DataInterface = nullptr;
	Object = nullptr;
}

FNiagaraVariant::FNiagaraVariant(const void* InBytes, int32 Size)
{
	CurrentMode = ENiagaraVariantMode::Bytes;
	Bytes.Append((const uint8*) InBytes, Size);
	DataInterface = nullptr;
	Object = nullptr;
}

bool FNiagaraVariant::IsValid(const FNiagaraTypeDefinition& Type)const
{
	if (Type.IsDataInterface())
	{
		return CurrentMode == ENiagaraVariantMode::DataInterface;
	}
	else if (Type.IsUObject())
	{
		return CurrentMode == ENiagaraVariantMode::Object;
	}
	else
	{
		return CurrentMode == ENiagaraVariantMode::Bytes && Bytes.Num() == Type.GetSize();
	}
}

void FNiagaraVariant::Reset()
{
	CurrentMode = ENiagaraVariantMode::None;
	Bytes.Reset();
	DataInterface = nullptr;
	Object = nullptr;
}

TObjectPtr<UObject> FNiagaraVariant::GetUObject() const
{
	ensure(CurrentMode == ENiagaraVariantMode::Object);
	return Object;
}

void FNiagaraVariant::SetUObject(TObjectPtr<UObject> InObject)
{
	ensure(CurrentMode == ENiagaraVariantMode::None || CurrentMode == ENiagaraVariantMode::Object);

	CurrentMode = ENiagaraVariantMode::Object;
	Object = InObject;
}

UNiagaraDataInterface* FNiagaraVariant::GetDataInterface() const 
{
	ensure(CurrentMode == ENiagaraVariantMode::DataInterface);
	return DataInterface;
}

void FNiagaraVariant::SetDataInterface(UNiagaraDataInterface* InDataInterface)
{
	ensure(CurrentMode == ENiagaraVariantMode::None || CurrentMode == ENiagaraVariantMode::DataInterface);
		
	CurrentMode = ENiagaraVariantMode::DataInterface;
	DataInterface = InDataInterface;
}

void FNiagaraVariant::AllocateBytes(int32 InCount)
{
	check(InCount > 0);
	ensure(CurrentMode == ENiagaraVariantMode::None || CurrentMode == ENiagaraVariantMode::Bytes);
	Bytes.SetNumZeroed(InCount);
	CurrentMode = ENiagaraVariantMode::Bytes;
}

void FNiagaraVariant::SetBytes(const uint8* InBytes, int32 InCount)
{
	check(InCount > 0);
	ensure(CurrentMode == ENiagaraVariantMode::None || CurrentMode == ENiagaraVariantMode::Bytes);

	CurrentMode = ENiagaraVariantMode::Bytes;
	Bytes.Reset(InCount);
	Bytes.Append(InBytes, InCount);
}

uint8* FNiagaraVariant::GetBytes() const
{
	ensure(CurrentMode == ENiagaraVariantMode::Bytes);
	return (uint8*) Bytes.GetData();
}

int32 FNiagaraVariant::GetNumBytes() const
{
	return Bytes.Num();
}

bool FNiagaraVariant::operator==(const FNiagaraVariant& Other) const
{
	if (CurrentMode == Other.CurrentMode)
	{
		switch (CurrentMode)
		{
			case ENiagaraVariantMode::Bytes:
				return Bytes == Other.Bytes;

			case ENiagaraVariantMode::Object:
				return Object == Other.Object;

			case ENiagaraVariantMode::DataInterface:
			{
				UNiagaraDataInterface* DI = GetDataInterface();
				if(ensure(DI))
				{
					return DI->Equals(Other.GetDataInterface());
				}
				return Other.GetDataInterface() == nullptr;
			}
			

			case ENiagaraVariantMode::None:
				return true;
		}
	}

	return false;
}

bool FNiagaraVariant::operator!=(const FNiagaraVariant& Other) const
{
	return !(operator==(Other));
}

bool FNiagaraVariant::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	return InVisitor->UpdateName(TEXT("NiagaraVariant.Object"), Object ? Object->GetFName() : NAME_None)
		&& InVisitor->UpdateName(TEXT("NiagaraVariant.DataInterface"), DataInterface ? DataInterface->GetFName() : NAME_None)
		&& InVisitor->UpdateArray(TEXT("NiagaraVariant.Bytes"), Bytes.GetData(), Bytes.Num())
		&& InVisitor->UpdatePOD<uint8>(TEXT("NiagaraVariant.CurrentMode"), static_cast<uint8>(CurrentMode));
}