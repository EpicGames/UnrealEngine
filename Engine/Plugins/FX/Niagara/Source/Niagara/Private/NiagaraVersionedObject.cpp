// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraVersionedObject.h"

#include "NiagaraVersionUpgradeBase.h"
#include "NiagaraTypes.h"
#include "Templates/SubclassOf.h"

//------------ dummy implementations -------------------------

namespace NiagaraVersionedObject
{
	struct DummyObject : FNiagaraVersionDataAccessor
	{
		virtual FNiagaraAssetVersion& GetObjectVersion() override { return Version; }
		virtual FText& GetVersionChangeDescription() override { return Desc; }
		virtual ENiagaraPythonUpdateScriptReference& GetUpdateScriptExecutionType() override {return Type; }
		virtual FString& GetPythonUpdateScript() override {return Script; }
		virtual FFilePath& GetScriptAsset() override {return Path; }
		virtual bool& IsDeprecated() override { return bDeprecated; }
		virtual TSubclassOf<UNiagaraVersionUpgradeBase>& GetUpdateUtilityClass() override { return UpdateClass; }

		FNiagaraAssetVersion Version;
		FText Desc;
		ENiagaraPythonUpdateScriptReference Type = ENiagaraPythonUpdateScriptReference::None;
		FString Script;
		FFilePath Path;
		bool bDeprecated;
		TSubclassOf<UNiagaraVersionUpgradeBase> UpdateClass;
	};

	static DummyObject Dummy;
}

FNiagaraAssetVersion& FNiagaraVersionDataAccessor::GetObjectVersion()
{
	unimplemented();
	return NiagaraVersionedObject::Dummy.GetObjectVersion();
}

FText& FNiagaraVersionDataAccessor::GetVersionChangeDescription()
{
	unimplemented();
	return NiagaraVersionedObject::Dummy.GetVersionChangeDescription();
}

bool& FNiagaraVersionDataAccessor::IsDeprecated()
{
	unimplemented();
	return NiagaraVersionedObject::Dummy.IsDeprecated();
}

FText& FNiagaraVersionDataAccessor::GetDeprecationMessage()
{
	unimplemented();
	return NiagaraVersionedObject::Dummy.GetVersionChangeDescription();
}

ENiagaraPythonUpdateScriptReference& FNiagaraVersionDataAccessor::GetUpdateScriptExecutionType()
{
	unimplemented();
	return NiagaraVersionedObject::Dummy.GetUpdateScriptExecutionType();
}

FString& FNiagaraVersionDataAccessor::GetPythonUpdateScript()
{
	unimplemented();
	return NiagaraVersionedObject::Dummy.GetPythonUpdateScript();
}

FFilePath& FNiagaraVersionDataAccessor::GetScriptAsset()
{
	unimplemented();
	return NiagaraVersionedObject::Dummy.GetScriptAsset();
}

TSubclassOf<UNiagaraVersionUpgradeBase>& FNiagaraVersionDataAccessor::GetUpdateUtilityClass()
{
	unimplemented();
	return NiagaraVersionedObject::Dummy.GetUpdateUtilityClass();
}

TArray<FNiagaraAssetVersion> FNiagaraVersionedObject::GetAllAvailableVersions() const
{
	unimplemented();
	return TArray<FNiagaraAssetVersion>();
}

#if WITH_EDITORONLY_DATA
TSharedPtr<FNiagaraVersionDataAccessor> FNiagaraVersionedObject::GetVersionDataAccessor(const FGuid& Version)
{
	unimplemented();
	return TSharedPtr<FNiagaraVersionDataAccessor>();
}

bool FNiagaraVersionedObject::IsVersioningEnabled() const
{
	unimplemented();
	return false;
}

FNiagaraAssetVersion FNiagaraVersionedObject::GetExposedVersion() const
{
	unimplemented();
	return FNiagaraAssetVersion();
}

FNiagaraAssetVersion const* FNiagaraVersionedObject::FindVersionData(const FGuid& VersionGuid) const
{
	unimplemented();
	return nullptr;
}

FGuid FNiagaraVersionedObject::AddNewVersion(int32 MajorVersion, int32 MinorVersion)
{
	unimplemented();
	return FGuid();
}

void FNiagaraVersionedObject::DeleteVersion(const FGuid& VersionGuid)
{
	unimplemented();
}

void FNiagaraVersionedObject::ExposeVersion(const FGuid& VersionGuid)
{
	unimplemented();
}

void FNiagaraVersionedObject::EnableVersioning()
{
	unimplemented();
}

void FNiagaraVersionedObject::DisableVersioning(const FGuid& VersionGuidToUse)
{
	unimplemented();
}
#endif //WITH_EDITORONLY_DATA
