// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraVolumeRendererProperties.h"
#include "NiagaraConstants.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraRenderer.h"
#include "NiagaraRendererVolumes.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraVolumeRendererProperties)

#if WITH_EDITOR
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SWidget.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateIconFinder.h"
#include "AssetThumbnail.h"
#endif

#define LOCTEXT_NAMESPACE "UNiagaraVolumeRendererProperties"

namespace NiagaraVolumeRendererPropertiesLocal
{
	TArray<TWeakObjectPtr<UNiagaraVolumeRendererProperties>> RendererPropertiesToDeferredInit;

	template<typename TValueType>
	static FNiagaraVariable MakeNiagaraVariableWithValue(const FNiagaraTypeDefinition& TypeDef, const FName& Name, const TValueType& Value)
	{
		FNiagaraVariable Variable(TypeDef, Name);
		Variable.SetValue(Value);
		return Variable;
	}

	static FNiagaraVariable GetVolumeRotationVariable()
	{
		static FNiagaraVariable Variable = MakeNiagaraVariableWithValue(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Particles.VolumeRotation"), UNiagaraVolumeRendererProperties::GetDefaultVolumeRotation());
		return Variable;
	}

	static FNiagaraVariable GetVolumeScaleVariable()
	{
		static FNiagaraVariable Variable = MakeNiagaraVariableWithValue(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particles.VolumeScale"), UNiagaraVolumeRendererProperties::GetDefaultVolumeScale());
		return Variable;
	}

	static FNiagaraVariable GetVolumeResolutionMaxAxisVariable()
	{
		static FNiagaraVariable Variable = MakeNiagaraVariableWithValue(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particles.VolumeResolutionMaxAxis"), UNiagaraVolumeRendererProperties::GetDefaultVolumeResolutionMaxAxis());
		return Variable;
	}

	static FNiagaraVariable GetVolumeWorldSpaceSizeVariable()
	{
		static FNiagaraVariable Variable = MakeNiagaraVariableWithValue(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particles.VolumeWorldSpaceSize"), UNiagaraVolumeRendererProperties::GetDefaultVolumeWorldSpaceSize());
		return Variable;
	}

	static void SetupBindings(UNiagaraVolumeRendererProperties* Props)
	{
		if (Props->PositionBinding.IsValid())
		{
			return;
		}
		Props->PositionBinding					= FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_POSITION);
		Props->RotationBinding.Setup(GetVolumeRotationVariable(), GetVolumeRotationVariable());
		Props->ScaleBinding.Setup(GetVolumeScaleVariable(), GetVolumeScaleVariable());
		Props->RendererVisibilityTagBinding		= FNiagaraConstants::GetAttributeDefaultBinding(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
		Props->VolumeResolutionMaxAxisBinding.Setup(GetVolumeResolutionMaxAxisVariable(), GetVolumeResolutionMaxAxisVariable());
		Props->VolumeWorldSpaceSizeBinding.Setup(GetVolumeWorldSpaceSizeVariable(), GetVolumeWorldSpaceSizeVariable());

	#if WITH_EDITORONLY_DATA
		Props->MaterialParameterBinding.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		Props->MaterialParameterBinding.SetAllowedObjects({ UMaterialInterface::StaticClass() });
	#endif
	}
}

UNiagaraVolumeRendererProperties::UNiagaraVolumeRendererProperties()
{
	AttributeBindings =
	{
		&PositionBinding,
		&RotationBinding,
		&ScaleBinding,
		&RendererVisibilityTagBinding,
		&VolumeResolutionMaxAxisBinding,
		&VolumeWorldSpaceSizeBinding,
	};
}

void UNiagaraVolumeRendererProperties::PostLoad()
{
	Super::PostLoad();
	
#if WITH_EDITORONLY_DATA
	ChangeToPositionBinding(PositionBinding);
#endif
	PostLoadBindings(ENiagaraRendererSourceDataMode::Particles);
	MaterialParameters.ConditionalPostLoad();
}

void UNiagaraVolumeRendererProperties::PostInitProperties()
{
	using namespace NiagaraVolumeRendererPropertiesLocal;

	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		// We can end up hitting PostInitProperties before the Niagara Module has initialized bindings this needs, mark this object for deferred init and early out.
		if (FModuleManager::Get().IsModuleLoaded("Niagara") == false)
		{
			NiagaraVolumeRendererPropertiesLocal::RendererPropertiesToDeferredInit.Add(this);
			return;
		}
		SetupBindings(this);
	}
}

void UNiagaraVolumeRendererProperties::Serialize(FArchive& Ar)
{
	// MIC will replace the main material during serialize
	// Be careful if adding code that looks at the material to make sure you get the correct one
	{
#if WITH_EDITORONLY_DATA
		TOptional<TGuardValue<TObjectPtr<UMaterialInterface>>> MICGuard;
		if (Ar.IsSaving() && Ar.IsCooking() && MICMaterial)
		{
			MICGuard.Emplace(Material, MICMaterial);
		}
#endif

		Super::Serialize(Ar);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraVolumeRendererProperties::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraVolumeRendererProperties, SourceMode))
	{
		UpdateSourceModeDerivates(SourceMode, true);
	}

	// Update our MICs if we change material / material bindings
	//-OPT: Could narrow down further to only static materials
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraVolumeRendererProperties, Material)) ||
		(MemberPropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraVolumeRendererProperties, MaterialParameters)))
	{
		UpdateMICs();
	}
}
#endif// WITH_EDITORONLY_DATA

void UNiagaraVolumeRendererProperties::InitCDOPropertiesAfterModuleStartup()
{
	using namespace NiagaraVolumeRendererPropertiesLocal;

	UNiagaraVolumeRendererProperties* CDO = CastChecked<UNiagaraVolumeRendererProperties>(UNiagaraVolumeRendererProperties::StaticClass()->GetDefaultObject());
	SetupBindings(CDO);

	for (TWeakObjectPtr<UNiagaraVolumeRendererProperties>& WeakProps : NiagaraVolumeRendererPropertiesLocal::RendererPropertiesToDeferredInit)
	{
		if (UNiagaraVolumeRendererProperties* Props = WeakProps.Get())
		{
			SetupBindings(Props);
		}
	}
}

FNiagaraRenderer* UNiagaraVolumeRendererProperties::CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController)
{
	FNiagaraRenderer* NewRenderer = new FNiagaraRendererVolumes(FeatureLevel, this, Emitter);
	NewRenderer->Initialize(this, Emitter, InController);
	return NewRenderer;
}

//class FNiagaraBoundsCalculatorVolumes : public FNiagaraBoundsCalculator
//{
//public:
//	FNiagaraBoundsCalculatorVolumes(const FNiagaraDataSetAccessor<FNiagaraPosition>& InPositionAccessor, const FNiagaraDataSetAccessor<FVector3f>& InVolumeWorldSpaceSizeAccessor)
//		: PositionAccessor(InPositionAccessor)
//		, VolumeWorldSpaceSizeAccessor(InVolumeWorldSpaceSizeAccessor)
//	{}
//
//	FNiagaraBoundsCalculatorVolumes() = delete;
//	virtual ~FNiagaraBoundsCalculatorVolumes() = default;
//
//	virtual void InitAccessors(const FNiagaraDataSetCompiledData* CompiledData) override {}
//	virtual FBox CalculateBounds(const FTransform& SystemTransform, const FNiagaraDataSet& DataSet, const int32 NumInstances) const override
//	{
//		if (!NumInstances || !PositionAccessor.IsValid())
//		{
//			return FBox(ForceInit);
//		}
//
//		FNiagaraPosition BoundsMin(ForceInitToZero);
//		FNiagaraPosition BoundsMax(ForceInitToZero);
//		PositionAccessor.GetReader(DataSet).GetMinMax(BoundsMin, BoundsMax);
//		FBox Bounds(BoundsMin, BoundsMax);
//
//		if (VolumeWorldSpaceSizeAccessor.IsValid())
//		{
//			Bounds = Bounds.ExpandBy(VolumeSizeAccessor.GetReader(DataSet).GetMax().GetAbsMax());
//		}
//		return Bounds;
//	}
//
//protected:
//	const FNiagaraDataSetAccessor<FNiagaraPosition>&	PositionAccessor;
//	const FNiagaraDataSetAccessor<FVector3f>&			VolumeWorldSpaceSizeAccessor;
//};

FNiagaraBoundsCalculator* UNiagaraVolumeRendererProperties::CreateBoundsCalculator()
{
	//-TODO: Dynamic Bounds Calculation
	//if (GetCurrentSourceMode() == ENiagaraRendererSourceDataMode::Emitter)
	//{
	//	return nullptr;
	//}

	//return new FNiagaraBoundsCalculatorVolumes(PositionDataSetAccessor, DecalSizeDataSetAccessor);
	return nullptr;
}

void UNiagaraVolumeRendererProperties::GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const
{
	UMaterialInterface* MaterialInterface = nullptr;
	if (InEmitter != nullptr)
	{
		MaterialInterface = Cast<UMaterialInterface>(InEmitter->FindBinding(MaterialParameterBinding.ResolvedParameter));
	}

#if WITH_EDITORONLY_DATA
	MaterialInterface = MaterialInterface ? MaterialInterface : ToRawPtr(MICMaterial);
#endif

	OutMaterials.Add(MaterialInterface ? MaterialInterface : ToRawPtr(Material));
}

void UNiagaraVolumeRendererProperties::CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData)
{
	UpdateSourceModeDerivates(SourceMode);
	UpdateMICs();

	PositionDataSetAccessor.Init(CompiledData, PositionBinding.GetDataSetBindableVariable().GetName());
	RotationDataSetAccessor.Init(CompiledData, RotationBinding.GetDataSetBindableVariable().GetName());
	ScaleDataSetAccessor.Init(CompiledData, ScaleBinding.GetDataSetBindableVariable().GetName());
	RendererVisibilityTagAccessor.Init(CompiledData, RendererVisibilityTagBinding.GetDataSetBindableVariable().GetName());
	VolumeResolutionMaxAxisAccessor.Init(CompiledData, VolumeResolutionMaxAxisBinding.GetDataSetBindableVariable().GetName());
	VolumeWorldSpaceSizeAccessor.Init(CompiledData, VolumeWorldSpaceSizeBinding.GetDataSetBindableVariable().GetName());
}

void UNiagaraVolumeRendererProperties::UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit)
{
	Super::UpdateSourceModeDerivates(InSourceMode, bFromPropertyEdit);

	if (UNiagaraEmitterBase* SrcEmitter = GetTypedOuter<UNiagaraEmitterBase>() )
	{
		for (FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
		{
			MaterialParamBinding.CacheValues(SrcEmitter);
		}
	}
}

bool UNiagaraVolumeRendererProperties::PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore)
{
	bool bAnyAdded = Super::PopulateRequiredBindings(InParameterStore);

	for (const FNiagaraVariableAttributeBinding* Binding : AttributeBindings)
	{
		if (Binding && Binding->CanBindToHostParameterMap())
		{
			InParameterStore.AddParameter(Binding->GetParamMapBindableVariable(), false);
			bAnyAdded = true;
		}
	}

	for (const FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		InParameterStore.AddParameter(MaterialParamBinding.GetParamMapBindableVariable(), false);
		bAnyAdded = true;
	}

	return bAnyAdded;
}

#if WITH_EDITORONLY_DATA
const TArray<FNiagaraVariable>& UNiagaraVolumeRendererProperties::GetOptionalAttributes()
{
	using namespace NiagaraVolumeRendererPropertiesLocal;

	static TArray<FNiagaraVariable> Attrs;
	if (Attrs.Num() == 0)
	{
		Attrs.Add(SYS_PARAM_PARTICLES_POSITION);
		//Attrs.Add(SYS_PARAM_PARTICLES_VISIBILITY_TAG);
	}
	return Attrs;
}

void UNiagaraVolumeRendererProperties::GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const
{
	OutWidgets.Add(SNew(STextBlock).Text(LOCTEXT("VolumeRenderer", "Volume Renderer")));
}

void UNiagaraVolumeRendererProperties::GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const
{
	Super::GetRendererFeedback(InEmitter, OutErrors, OutWarnings, OutInfo);

	if ( const FVersionedNiagaraEmitterData* EmitterData = InEmitter.GetEmitterData() )
	{
		if (SourceMode == ENiagaraRendererSourceDataMode::Particles && EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			OutErrors.Emplace(
				LOCTEXT("GPUNotSupportedDesc", "Volume renderer does not support reading from GPU particles."),
				LOCTEXT("GPUNotSupportedSummary", "GPU particles not support.")
				//LOCTEXT("GPUNotSupportedFix", "Switch emitter to CPU particles."),
				//FNiagaraRendererFeedbackFix::CreateLambda(
				//	[this]()
				//	{
				//		const_cast<UNiagaraRibbonRendererProperties*>(this)->MultiPlaneCount = FMath::Clamp(this->MultiPlaneCount - 1, 1, 16);
				//	}
				//),
			);
		}
	}

	if (MaterialParameters.HasAnyBindings())
	{
		TArray<UMaterialInterface*> Materials;
		GetUsedMaterials(nullptr, Materials);
		MaterialParameters.GetFeedback(Materials, OutWarnings);
	}
}

TArray<FNiagaraVariable> UNiagaraVolumeRendererProperties::GetBoundAttributes() const
{
	TArray<FNiagaraVariable> BoundAttributes = Super::GetBoundAttributes();
	BoundAttributes.Reserve(BoundAttributes.Num() + MaterialParameters.AttributeBindings.Num() + 1);

	for (const FNiagaraMaterialAttributeBinding& MaterialParamBinding : MaterialParameters.AttributeBindings)
	{
		BoundAttributes.AddUnique(MaterialParamBinding.GetParamMapBindableVariable());
	}

	if (MaterialParameterBinding.IsSetToDefault() == false)
	{
		BoundAttributes.AddUnique(MaterialParameterBinding.AliasedParameter);
	}

	return BoundAttributes;
}

void UNiagaraVolumeRendererProperties::RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RenameVariable(OldVariable, NewVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MaterialParameters.RenameVariable(OldVariable, NewVariable, InEmitter, GetCurrentSourceMode());
#endif
}

void UNiagaraVolumeRendererProperties::RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter)
{
	Super::RemoveVariable(OldVariable, InEmitter);
#if WITH_EDITORONLY_DATA
	MaterialParameters.RemoveVariable(OldVariable, InEmitter, GetCurrentSourceMode());
#endif
}
#endif // WITH_EDITORONLY_DATA

void UNiagaraVolumeRendererProperties::UpdateMICs()
{
#if WITH_EDITORONLY_DATA
	UpdateMaterialParametersMIC(MaterialParameters, Material, MICMaterial);
#endif
}

#undef LOCTEXT_NAMESPACE
