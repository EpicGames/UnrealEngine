// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraWorldManager.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "GameFramework/Pawn.h"
#include "NiagaraModule.h"
#include "NiagaraComponentSettings.h"
#include "NiagaraSystemSimulation.h"
#include "NiagaraTypes.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraParameterCollectionResolver.h"
#include "NiagaraSystemInstance.h"
#include "GameFramework/PlayerController.h"
#include "Engine/GameViewportClient.h"
#include "NiagaraStats.h"
#include "NiagaraComponentPool.h"
#include "NiagaraComponent.h"
#include "NiagaraDataChannelManager.h"
#include "NiagaraEffectType.h"
#include "NiagaraDebugHud.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSimpleObjectPool.h"
#include "NiagaraComponentLeakDetector.h"
#include "Particles/FXBudget.h"
#include "NiagaraCullProxyComponent.h"
#include "GameDelegates.h"
#include "Engine/LocalPlayer.h"
#include "SceneView.h"
#include "UObject/UObjectIterator.h"
#include "Logging/StructuredLog.h"
#include "NiagaraDataManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraWorldManager)

#if WITH_EDITORONLY_DATA
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Niagara Manager Update Scalability Managers [GT]"), STAT_UpdateScalabilityManagers, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Niagara Manager Tick [GT]"), STAT_NiagaraWorldManTick, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Niagara Manager Wait On Render [GT]"), STAT_NiagaraWorldManWaitOnRender, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Niagara Manager Wait Pre Garbage Collect [GT]"), STAT_NiagaraWorldManWaitPreGC, STATGROUP_Niagara); 
DECLARE_CYCLE_STAT(TEXT("Niagara Manager Refresh Owner Allows Scalability"), STAT_NiagaraWorldManRefreshOwnerAllowsScalability, STATGROUP_Niagara);
DECLARE_CYCLE_STAT(TEXT("Niagara Manager Tick Parameter Collections [GT]"), STAT_NiagaraWorldManTickParamCollections, STATGROUP_Niagara);

namespace NiagaraRollbackCVars
{
	//Rollback gate for culling more proxyless systems
	bool GAggressiveProxylessCulling = true;
	static FAutoConsoleVariableRef CVarRollback_AggressiveProxylessCulling(
		TEXT("fx.Niagara.Rollback.AggressiveProxylessCulling"),
		GAggressiveProxylessCulling,
		TEXT("When true components without proxies will be more aggresively culled."),
		ECVF_Default
	);

	//Rollback gate for new world cleanup flow.
	//Scheduled for removal in UE-347574
	bool NewWorldCleanup = true;
	static FAutoConsoleVariableRef CVarRollback_NewWorldCleanup(
		TEXT("fx.Niagara.Rollback.NewWorldCleanup"),
		NewWorldCleanup,
		TEXT("When true we'll use the new World Manager Cleanup flows. False will use old legacy code."),
		ECVF_Default
	);
	
	//Rollback gate for world initialization changes.
	//Scheduled for removal in UE-348784
	bool WorldManagerRespectsWorldCreateFXSystem = false;
	static FAutoConsoleVariableRef CVarRollback_WorldManagerRespectsWorldCreateFXSystem(
		TEXT("fx.Niagara.Rollback.WorldManagerRespectsWorldCreateFXSystem"),
		WorldManagerRespectsWorldCreateFXSystem,
		TEXT("When true, Niagara world managers will only be created for worlds with bCreateFXSystem==true in it's initialization options."),
		ECVF_Default
	);
	
	//Rollback gate for delaying NDC init. Avoids some cases where UWorlds are created but not "real" UWorlds. 
	//They don't need FX and having NDC managers created that themselves reference the world can cause some GC issues.
	//Scheduled for removal in UE-348786
	bool DelayNDCManagerInitialiation = true;
	static FAutoConsoleVariableRef CVarRollback_DelayNDCManagerInitialiation(
		TEXT("fx.Niagara.Rollback.DelayNDCManagerInitialiation"),
		DelayNDCManagerInitialiation,
		TEXT("When true, Niagara will not initialize it's NDC managers until the first tick of the world manager or first NDC use."),
		ECVF_Default
	);
}

bool GNiagaraWorldManagerUseTickPasses = false;
static FAutoConsoleVariableRef CVarNiagaraWorldManagerUseTickPasses(
	TEXT("fx.Niagara.WorldManager.UseTickPasses"),
	GNiagaraWorldManagerUseTickPasses,
	TEXT("When enabled we will use the tick passes to avoid blocking overlapping sims at tick boundaries."),
	ECVF_Default
);

static bool GNiagaraWorldManagerObjectPoolEnabled = 1;
static FAutoConsoleVariableRef CVarNiagaraWorldManagerObjectPoolEnabled(
	TEXT("fx.Niagara.WorldManager.ObjectPoolEnabled"),
	GNiagaraWorldManagerObjectPoolEnabled,
	TEXT("Should we pool objects, yay or nay."),
	ECVF_Default
);

static int GNiagaraWaitOnPreGC = 1;
static FAutoConsoleVariableRef CVarNiagaraWaitOnPreGC(
	TEXT("fx.Niagara.WaitOnPreGC"),
	GNiagaraWaitOnPreGC,
	TEXT("Toggles whether Niagara will wait for all async tasks to complete before any GC calls."),
	ECVF_Default
);

static int GNiagaraSpawnPerTickGroup = 1;
static FAutoConsoleVariableRef CVarNiagaraSpawnPerTickGroup(
	TEXT("fx.Niagara.WorldManager.SpawnPerTickGroup"),
	GNiagaraSpawnPerTickGroup,
	TEXT("Will attempt to spawn new systems earlier (default enabled)."),
	ECVF_Default
);

int GNigaraAllowPrimedPools = 1;
static FAutoConsoleVariableRef CVarNigaraAllowPrimedPools(
	TEXT("fx.Niagara.AllowPrimedPools"),
	GNigaraAllowPrimedPools,
	TEXT("Allow Niagara pools to be primed."),
	ECVF_Default
);

static int32 GbAllowVisibilityCullingForDynamicBounds = 1;
static FAutoConsoleVariableRef CVarAllowVisibilityCullingForDynamicBounds(
	TEXT("fx.Niagara.AllowVisibilityCullingForDynamicBounds"),
	GbAllowVisibilityCullingForDynamicBounds,
	TEXT("Allow async work to continue until the end of the frame, if false it will complete within the tick group it's started in."),
	ECVF_Default
);

FAutoConsoleCommandWithWorldAndArgs DumpNiagaraWorldManagerCommand(
	TEXT("DumpNiagaraWorldManager"),
	TEXT("Dump Information About the Niagara World Manager Contents"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args, UWorld* World)
		{
			FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World);
			if (WorldManager != nullptr && GLog != nullptr)
			{
				const bool bDetailed = Args.Num() > 0 ? FCString::Atoi(*Args[0]) != 0 : false;

				WorldManager->DumpDetails(*GLog, bDetailed);
			}
		}
	)
);

static int GEnableNiagaraVisCulling = 1;
static FAutoConsoleVariableRef CVarEnableNiagaraVisCulling(
	TEXT("fx.Niagara.Scalability.VisibilityCulling"),
	GEnableNiagaraVisCulling,
	TEXT("When non-zero, high level scalability culling based on visibility is enabled."),
	ECVF_Default
);

static int GEnableNiagaraDistanceCulling = 1;
static FAutoConsoleVariableRef CVarEnableNiagaraDistanceCulling(
	TEXT("fx.Niagara.Scalability.DistanceCulling"),
	GEnableNiagaraDistanceCulling,
	TEXT("When non-zero, high level scalability culling based on distance is enabled."),
	ECVF_Default
);

static int GEnableNiagaraInstanceCountCulling = 1;
static FAutoConsoleVariableRef CVarEnableNiagaraInstanceCountCulling(
	TEXT("fx.Niagara.Scalability.InstanceCountCulling"),
	GEnableNiagaraInstanceCountCulling,
	TEXT("When non-zero, high level scalability culling based on instance count is enabled."),
	ECVF_Default
);

int GEnableNiagaraGlobalBudgetCulling = 1;
static FAutoConsoleVariableRef CVarEnableNiagaraGlobalBudgetCulling(
	TEXT("fx.Niagara.Scalability.GlobalBudgetCulling"),
	GEnableNiagaraGlobalBudgetCulling,
	TEXT("When non-zero, high level scalability culling based on global time budget is enabled."),
	ECVF_Default
);

float GWorldLoopTime = 0.0f;
static FAutoConsoleVariableRef CVarWorldLoopTime(
	TEXT("fx.Niagara.Debug.GlobalLoopTime"),
	GWorldLoopTime,
	TEXT("If > 0 all Niagara FX will reset every N seconds. \n"),
	ECVF_Default
);

int GNiagaraAllowCullProxies = 1;
static FAutoConsoleVariableRef CVarAllowCullProxies(
	TEXT("fx.Niagara.AllowCullProxies"),
	GNiagaraAllowCullProxies,
	TEXT("Toggles whether Niagara will use Cull Proxy systems in place of systems culled by scalability."),
	ECVF_Default
);

float GNiagaraCSVSplitTime = 180.0f;
static FAutoConsoleVariableRef CVarCSVSplitTime(
	TEXT("fx.Niagara.CSVSplitTime"),
	GNiagaraCSVSplitTime,
	TEXT("Length of Niagara's split time events passed to the CSV profiler. There are used to give check more confined stat averages."),
	ECVF_Default
);

FAutoConsoleCommandWithWorldAndArgs GCmdNiagaraPlaybackMode(
	TEXT("fx.Niagara.Debug.PlaybackMode"),
	TEXT("Set playback mode\n")
	TEXT("0 - Play\n")
	TEXT("1 - Paused\n")
	TEXT("2 - Step\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args, UWorld* World)
		{
			if ( FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World) )
			{
				if (Args.Num() != 1)
				{
					UE_LOGF(LogNiagara, Log, "fx.Niagara.Debug.PlaybackMode %d", (int32)WorldManager->GetDebugPlaybackMode());
				}
				else
				{
					const ENiagaraDebugPlaybackMode PlaybackMode = FMath::Clamp((ENiagaraDebugPlaybackMode)FCString::Atoi(*Args[0]), ENiagaraDebugPlaybackMode::Play, ENiagaraDebugPlaybackMode::Step);
					WorldManager->SetDebugPlaybackMode(PlaybackMode);
				}
			}
		}
	)
);

FAutoConsoleCommandWithWorldAndArgs GCmdNiagaraPlaybackRate(
	TEXT("fx.Niagara.Debug.PlaybackRate"),
	TEXT("Set playback rate\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args, UWorld* World)
		{
			if ( FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World) )
			{
				if (Args.Num() != 1)
				{
					UE_LOGF(LogNiagara, Log, "fx.Niagara.Debug.PlaybackRate %5.2f", WorldManager->GetDebugPlaybackRate());
				}
				else
				{
					const float PlaybackRate = FCString::Atof(*Args[0]);
					WorldManager->SetDebugPlaybackRate(PlaybackRate);
				}
			}
		}
	)
);

FAutoConsoleCommandWithWorldAndArgs GCmdNiagaraScalabilityCullingMode(
	TEXT("fx.Niagara.Scalability.CullingMode"),
	TEXT("Set scalability culling mode\n")
	TEXT("0 - Enabled. Culling is enabled as normal.\n")
	TEXT("1 - Paused. No culling will occur but FX will still be tracked internally so culling can be resumed correctly later.\n")
	TEXT("2 - Disabled. No culling will occur and no FX will be tracked. Culling may not work correctly for some FX if enabled again after this.\n"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args, UWorld* World)
		{
			if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World))
			{
				if (Args.Num() != 1 || Args[0].IsNumeric() == false)
				{
					UE_LOGF(LogNiagara, Log, "fx.Niagara.ScalabililityCullingMode %d", (int32)WorldManager->GetScalabilityCullingMode());
					UE_LOGF(LogNiagara, Log, "Set scalability culling mode");
					UE_LOGF(LogNiagara, Log, "0 - Enabled. Culling is enabled as normal.");
					UE_LOGF(LogNiagara, Log, "1 - Paused. No culling will occur but FX will still be tracked internally so culling can be resumed correctly later.");
					UE_LOGF(LogNiagara, Log, "2 - Disabled. No culling will occur and no FX will be tracked. Culling may not work correctly for some FX if enabled again after this.");
				}
				else
				{
					const ENiagaraScalabilityCullingMode CullingMode = FMath::Clamp((ENiagaraScalabilityCullingMode)FCString::Atoi(*Args[0]), ENiagaraScalabilityCullingMode::Enabled, ENiagaraScalabilityCullingMode::Disabled);
					WorldManager->SetScalabilityCullingMode(CullingMode);
				}
			}
			else
			{
				UE_LOGF(LogNiagara, Warning, "Cannot set Niagara Scalability Culling Mode on a null world.");
			}
		}
	)
);

FDelegateHandle FNiagaraWorldManager::OnWorldInitHandle;
FDelegateHandle FNiagaraWorldManager::OnPostWorldInitHandle;
FDelegateHandle FNiagaraWorldManager::OnWorldCleanupHandle;
FDelegateHandle FNiagaraWorldManager::OnPostWorldCleanupHandle;
FDelegateHandle FNiagaraWorldManager::OnPreWorldFinishDestroyHandle;
FDelegateHandle FNiagaraWorldManager::OnWorldBeginTearDownHandle;
FDelegateHandle FNiagaraWorldManager::TickWorldStartHandle;
FDelegateHandle FNiagaraWorldManager::TickWorldEndHandle;
FDelegateHandle FNiagaraWorldManager::TickWorldHandle;
FDelegateHandle FNiagaraWorldManager::OnWorldPreActorTickHandle;
FDelegateHandle FNiagaraWorldManager::OnWorldPreSendAllEndOfFrameUpdatesHandle;
FDelegateHandle FNiagaraWorldManager::PreGCHandle;
FDelegateHandle FNiagaraWorldManager::PostReachabilityAnalysisHandle;
FDelegateHandle FNiagaraWorldManager::PostGCHandle;
FDelegateHandle FNiagaraWorldManager::PreGCBeginDestroyHandle;
FDelegateHandle FNiagaraWorldManager::ViewTargetChangedHandle;

std::atomic<bool> FNiagaraWorldManager::bInvalidateCachedSystemScalabilityData(false);
TMap<class UWorld*, class FNiagaraWorldManager*> FNiagaraWorldManager::WorldManagers;

namespace NiagaraWorldManagerInternal
{
	static int32 GFirstHighPriTickGroup = 0;
	static TQueue<TFunction<void()>, EQueueMode::Mpsc> GlobalDeferredCallbacks;

	void ExecuteGlobalDeferredCallbacks()
	{
		TFunction<void()> Callback;
		while (GlobalDeferredCallbacks.Dequeue(Callback))
		{
			Callback();
		}
	}

	int GetNiagaraTickGroup(ETickingGroup TickGroup)
	{
		const int ActualTickGroup = FMath::Clamp(TickGroup - NiagaraFirstTickGroup, 0, NiagaraNumTickGroups - 1);
		return ActualTickGroup;
	}

	ETickingGroup GetTickingGroupFromNiagara(int ActualTickGroup)
	{
		const int32 TickGroup = FMath::Clamp(ActualTickGroup + NiagaraFirstTickGroup, 0, TG_MAX - 1);
		return ETickingGroup(TickGroup);
	}

	bool GetNiagaraTickGroupPriority(int32 NiagaraTickGroup)
	{
		return NiagaraTickGroup >= GFirstHighPriTickGroup;
	}

	static FAutoConsoleVariableRef CVarFirstHighPriTickGroup(
		TEXT("fx.Niagara.WorldManager.FirstHighPriTickGroup"),
		GFirstHighPriTickGroup,
		TEXT("Defines which tick groups should be set to high priority for the world manager.\n")
		TEXT("0 - (Default) all tick groups will run high priority.\n")
		TEXT("1 - The first tick group will be normal priority, all others high, etc."),
		FConsoleVariableDelegate::CreateLambda(
			[](IConsoleVariable*)
			{
				FNiagaraWorldManager::ForAllWorldManagers(
					[](FNiagaraWorldManager& WorldManager)
					{
						for (int32 i=0; i < NiagaraNumTickGroups; ++i)
						{
							WorldManager.SetTickGroupPriority(i, GetNiagaraTickGroupPriority(i));
						}
					}
				);
			}
		),
		ECVF_Default
	); 

	static bool GTickParameterCollectionsOncePerFrame = true;
	static FAutoConsoleVariableRef CVarTickParameterCollectionsOncePerFrame(
		TEXT("fx.Niagara.WorldManager.TickParameterCollectionsOncePerFrame"),
		GTickParameterCollectionsOncePerFrame,
		TEXT("When true parameter collection updates will occur once per frame in the first tick group.\n")
		TEXT("When false updates will occur every tick group.\n")
		TEXT("Updating constantly can cost performance but can fix issues with NPC values updated then used from a system in a later tick group, this would be a frame out without updateing each tick group."),
		ECVF_Default | ECVF_Scalability
	);
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraWorldManagerTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	check(Owner);
	Owner->Tick(TickPass, TickGroup, DeltaTime, MyCompletionGraphEvent);
}

FString FNiagaraWorldManagerTickFunction::DiagnosticMessage()
{
	static const UEnum* EnumType = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Engine.ETickingGroup"));

	return TEXT("FNiagaraWorldManagerTickFunction::Tick(") + EnumType->GetNameStringByIndex(static_cast<uint32>(TickGroup)) + TEXT(")");
}

FName FNiagaraWorldManagerTickFunction::DiagnosticContext(bool bDetailed)
{
	return FName(TEXT("FNiagaraWorldManagerTickFunction"));
}

//////////////////////////////////////////////////////////////////////////

ENiagaraScalabilityCullingMode FNiagaraWorldManager::ScalabilityCullingMode = ENiagaraScalabilityCullingMode::Enabled;

void FNiagaraWorldManager::SetScalabilityCullingMode(ENiagaraScalabilityCullingMode NewMode) 
{
	//also reset fx budgets on state change.
	if (ScalabilityCullingMode != NewMode)
	{
		FFXBudget::Reset();
		ScalabilityCullingMode = NewMode;

		auto UpdateScalability = [](FNiagaraWorldManager& WorldMan)
		{
			WorldMan.UpdateScalabilityManagers(0.01f, false);
		};
		ForAllWorldManagers(UpdateScalability);
	}
}

FNiagaraWorldManager::FNiagaraWorldManager()
	: World(nullptr)
	, ActiveNiagaraTickGroup(-1)
	, bUseTickPasses(GNiagaraWorldManagerUseTickPasses)
	, bAppHasFocus(true)
	, DataChannelManager(new FNiagaraDataChannelManager(this))
{
}

void FNiagaraWorldManager::Init(UWorld* InWorld)
{
	// Reset variables that are not reset during OnWorldCleanup; Init/OnWorldCleanup can be called multiple times
	ComponentPool = nullptr; // Discard the existing ComponentPool
	bPoolIsPrimed = false;
	ActiveNiagaraTickGroup = -1;
	bAppHasFocus = true;
	bIsTearingDown = false;
	bIsCleaningUp = false;
	WorldLoopTime = 0.0f;
	RequestedDebugPlaybackMode = ENiagaraDebugPlaybackMode::Play;
	DebugPlaybackMode = ENiagaraDebugPlaybackMode::Play;
	DebugPlaybackRate = 1.0f;

	World = InWorld;

	LastRenderThreadUpdateTime = MakeShared<std::atomic<float>>();
	LastRenderThreadUpdateTime->store(InWorld->GetTimeSeconds());

	ComponentPool = NewObject<UNiagaraComponentPool>();

	for (int32 TickGroup = 0; TickGroup < NiagaraNumTickGroups; ++TickGroup)
	{
		for (int32 iTickPass=0; iTickPass < int32(ENiagaraWorldManagerTickPass::Num); ++iTickPass )
		{
			const ENiagaraWorldManagerTickPass TickPass = ENiagaraWorldManagerTickPass(iTickPass);

			FNiagaraWorldManagerTickFunction& TickFunc = TickFunctions[TickGroup][iTickPass];
			TickFunc.TickGroup						= ETickingGroup(NiagaraFirstTickGroup + TickGroup);
			TickFunc.EndTickGroup					= TickPass == ENiagaraWorldManagerTickPass::Overlapping ? TG_LastDemotable : (ETickingGroup)TickFunc.TickGroup;
			TickFunc.bCanEverTick					= true;
			TickFunc.bStartWithTickEnabled			= TickPass == ENiagaraWorldManagerTickPass::Overlapping;
			TickFunc.bAllowTickOnDedicatedServer	= false;
			TickFunc.bHighPriority					= NiagaraWorldManagerInternal::GetNiagaraTickGroupPriority(TickGroup);
			TickFunc.Owner							= this;
			TickFunc.TickPass						= TickPass;
			TickFunc.SetTickFunctionEnable(TickFunc.bStartWithTickEnabled);
			TickFunc.RegisterTickFunction(InWorld->PersistentLevel);
			if (iTickPass > 0)
			{
				TickFunc.AddPrerequisite(ComponentPool, TickFunctions[TickGroup][iTickPass - 1]);
			}
		}
	}

	//Ideally we'd do this here but it's too early in the init process and the world does not have a Scene yet.
	//Possibly a later hook we can use.
	//PrimePoolForAllSystems();

#if WITH_NIAGARA_DEBUGGER
	NiagaraDebugHud.Reset(new FNiagaraDebugHud(World));
#endif
#if WITH_NIAGARA_LEAK_DETECTOR
	if ( World->IsGameWorld() )
	{
		ComponentLeakDetector.Reset(new FNiagaraComponentLeakDetector());
	}
#endif

	// Make sure we update our component settings, this includes ban lists, etc
	FNiagaraComponentSettings::UpdateSettings();
}

void FNiagaraWorldManager::OnPostWorldInit()
{
	if(NiagaraRollbackCVars::DelayNDCManagerInitialiation == false)
	{
		DataChannelManager->Init();
	}
}

FNiagaraWorldManager::~FNiagaraWorldManager()
{
	OnWorldCleanup(true, true);
}

FNiagaraWorldManager* FNiagaraWorldManager::Get(const UWorld* World)
{
	FNiagaraWorldManager** OutWorld = WorldManagers.Find(World);
	if (OutWorld == nullptr)
	{
		UE_LOGF(LogNiagara, Warning, "Calling FNiagaraWorldManager::Get \"%ls\", but Niagara has never encountered this world before. "
			" This means that WorldInit never happened. This may happen in some edge cases in the editor, like saving invisible child levels, "
			"in which case the calling context needs to be safe against this returning nullptr.", World ? *World->GetName() : TEXT("nullptr"));
		return nullptr;
	}
	return *OutWorld;
}

void FNiagaraWorldManager::OnStartup()
{
	OnWorldInitHandle = FWorldDelegates::OnPreWorldInitialization.AddStatic(&FNiagaraWorldManager::OnWorldInit);
	OnPostWorldInitHandle = FWorldDelegates::OnPostWorldInitialization.AddStatic(&FNiagaraWorldManager::OnPostWorldInit);
	OnWorldCleanupHandle = FWorldDelegates::OnWorldCleanup.AddStatic(&FNiagaraWorldManager::OnWorldCleanup);
	OnPostWorldCleanupHandle = FWorldDelegates::OnPostWorldCleanup.AddStatic(&FNiagaraWorldManager::OnPostWorldCleanup);
	OnPreWorldFinishDestroyHandle = FWorldDelegates::OnPreWorldFinishDestroy.AddStatic(&FNiagaraWorldManager::OnPreWorldFinishDestroy);
	OnWorldBeginTearDownHandle = FWorldDelegates::OnWorldBeginTearDown.AddStatic(&FNiagaraWorldManager::OnWorldBeginTearDown);
	TickWorldStartHandle = FWorldDelegates::OnWorldTickStart.AddStatic(&FNiagaraWorldManager::TickWorldStart);
	TickWorldEndHandle = FWorldDelegates::OnWorldTickEnd.AddStatic(&FNiagaraWorldManager::TickWorldEnd);
	TickWorldHandle = FWorldDelegates::OnWorldPostActorTick.AddStatic(&FNiagaraWorldManager::TickWorld);
	OnWorldPreActorTickHandle = FWorldDelegates::OnWorldPreActorTick.AddStatic(&FNiagaraWorldManager::OnWorldPreActorTick);
	OnWorldPreSendAllEndOfFrameUpdatesHandle = FWorldDelegates::OnWorldPreSendAllEndOfFrameUpdates.AddLambda(
		[](UWorld* InWorld)
		{
			if ( FNiagaraWorldManager* FoundManager = WorldManagers.FindRef(InWorld) )
			{
				FoundManager->PreSendAllEndOfFrameUpdates();
			}
		}
	);

	PreGCHandle = FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddStatic(&FNiagaraWorldManager::OnPreGarbageCollect);
	PostReachabilityAnalysisHandle = FCoreUObjectDelegates::PostReachabilityAnalysis.AddStatic(&FNiagaraWorldManager::OnPostReachabilityAnalysis);
	PostGCHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddStatic(&FNiagaraWorldManager::OnPostGarbageCollect);
	PreGCBeginDestroyHandle = FCoreUObjectDelegates::PreGarbageCollectConditionalBeginDestroy.AddStatic(&FNiagaraWorldManager::OnPreGarbageCollectBeginDestroy);

	if ( FApp::CanEverRender() )
	{
		ViewTargetChangedHandle = FGameDelegates::Get().GetViewTargetChangedDelegate().AddLambda(
			[](APlayerController* PC, AActor* OldTarget, AActor* NewTarget)
			{
				OnRefreshOwnerAllowsScalability();
			}
		);
	}
}

void FNiagaraWorldManager::OnShutdown()
{
	FWorldDelegates::OnPreWorldInitialization.Remove(OnWorldInitHandle);
	FWorldDelegates::OnPostWorldInitialization.Remove(OnPostWorldInitHandle);
	FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupHandle);
	FWorldDelegates::OnPostWorldCleanup.Remove(OnPostWorldCleanupHandle);
	FWorldDelegates::OnPreWorldFinishDestroy.Remove(OnPreWorldFinishDestroyHandle);
	FWorldDelegates::OnWorldBeginTearDown.Remove(OnWorldBeginTearDownHandle);
	FWorldDelegates::OnWorldTickStart.Remove(TickWorldStartHandle);
	FWorldDelegates::OnWorldTickEnd.Remove(TickWorldEndHandle);
	FWorldDelegates::OnWorldPreActorTick.Remove(OnWorldPreActorTickHandle);
	FWorldDelegates::OnWorldPostActorTick.Remove(TickWorldHandle);
	FWorldDelegates::OnWorldPreSendAllEndOfFrameUpdates.Remove(OnWorldPreSendAllEndOfFrameUpdatesHandle);
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().Remove(PreGCHandle);
	FCoreUObjectDelegates::PostReachabilityAnalysis.Remove(PostReachabilityAnalysisHandle);
	FCoreUObjectDelegates::GetPostGarbageCollect().Remove(PostGCHandle);
	FCoreUObjectDelegates::PreGarbageCollectConditionalBeginDestroy.Remove(PreGCBeginDestroyHandle);
	
	if ( ViewTargetChangedHandle.IsValid() )
	{
		FGameDelegates::Get().GetViewTargetChangedDelegate().Remove(ViewTargetChangedHandle);
	}

	//Should have cleared up all world managers by now.
	if (!ensure(WorldManagers.Num() == 0))
	{
		for (TPair<UWorld*, FNiagaraWorldManager*> Pair : WorldManagers)
		{
			FNiagaraWorldManager* WorldMan = Pair.Value;
			if (ensure(WorldMan))
			{
				delete WorldMan;
			}
		
			Pair.Value = nullptr;
		}
		WorldManagers.Empty();
	}
}

void FNiagaraWorldManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	// World doesn't need to be added to the reference list. It will be handled via OnWorldInit & OnWorldCleanup & OnPreWorldFinishDestroy in INiagaraModule

	Collector.AddReferencedObjects(ParameterCollections);
	Collector.AddReferencedObject(ComponentPool);
	for (auto& Pair : ScalabilityManagers)
	{
		auto& EffectType = Pair.Key;
		FNiagaraScalabilityManager& ScalabilityMan = Pair.Value;

		Collector.AddReferencedObject(EffectType);
		ScalabilityMan.AddReferencedObjects(Collector);
	}

	Collector.AddReferencedObjects(CullProxyMap);

	Collector.AddReferencedObjects(ReferencedObjects);

	DataChannelManager->AddReferencedObjects(Collector);

	Collector.AddReferencedObjects(DataManagers);
}

FString FNiagaraWorldManager::GetReferencerName() const
{
	return TEXT("FNiagaraWorldManager");
}

UNiagaraCullProxyComponent* FNiagaraWorldManager::GetCullProxy(UNiagaraComponent* Component)
{
	UNiagaraSystem* System = Component->GetAsset();

	//Don't create or return the cull proxy if the world is cleaning up.
	bool bIsWorldCleaningUp = NiagaraRollbackCVars::NewWorldCleanup && (bIsTearingDown || bIsCleaningUp);
	if (ensure(System) && GNiagaraAllowCullProxies && bIsWorldCleaningUp == false)
	{
		auto& CullProxy = CullProxyMap.FindOrAdd(System);

		if (CullProxy == nullptr)
		{
			CullProxy = NewObject<UNiagaraCullProxyComponent>(GetWorld());
			CullProxy->SetAsset(System);
			CullProxy->SetAllowScalability(false);

			CullProxy->bAutoActivate = true;
			CullProxy->SetAutoDestroy(false);
			CullProxy->bAllowAnyoneToDestroyMe = true;
			CullProxy->RegisterComponentWithWorld(World);
			CullProxy->SetAbsolute(true, true, true);
			CullProxy->SetWorldLocationAndRotation(FVector::ZeroVector, FRotator::ZeroRotator);
			CullProxy->SetRelativeScale3D(FVector::OneVector);

			CullProxy->Activate(true);
			CullProxy->SetPaused(true);

			//Override the LODDistance to just below the max so the system is as cheap for simulation and rendering as possible. 
			float MaxDist = System->GetScalabilitySettings().MaxDistance;
			CullProxy->SetPreviewLODDistance(true, MaxDist * 0.999f, MaxDist);

			//Have to copy user parameters from first requesting component so we at least have some valid values.
			//Though this could be somewhat dangerous with DIs etc.
			Component->GetOverrideParameters().CopyParametersTo(CullProxy->GetOverrideParameters(), false, FNiagaraParameterStore::EDataInterfaceCopyMethod::Value);
		}
	
		return CullProxy;
	}
	return nullptr;
}

void FNiagaraWorldManager::TickParameterCollections()
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraWorldManTickParamCollections);
	CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_NiagaraTickParameterCollections);

	//-TODO: Do we need to do this per tick group?
	for (TPair<UNiagaraParameterCollection*, UNiagaraParameterCollectionInstance*> CollectionInstPair : ParameterCollections)
	{
		check(CollectionInstPair.Value);
		CollectionInstPair.Value->Tick(World);
	}
}

UNiagaraParameterCollectionInstance* FNiagaraWorldManager::GetParameterCollection(UNiagaraParameterCollection* Collection)
{
	if (!Collection || !HasActiveWorld())
	{
		return nullptr;
	}

	auto* OverrideInst = ParameterCollections.Find(Collection);
	if (!OverrideInst)
	{
		UNiagaraParameterCollectionInstance* DefaultInstance = Collection->GetDefaultInstance();
		OverrideInst = &ParameterCollections.Add(Collection);
		*OverrideInst = CastChecked<UNiagaraParameterCollectionInstance>(StaticDuplicateObject(DefaultInstance, World));
#if WITH_EDITOR
		//Bind to the default instance so that changes to the collection propagate through.
		DefaultInstance->GetParameterStore().Bind(&(*OverrideInst)->GetParameterStore());
#endif

		(*OverrideInst)->Bind(World);
	}

	check(OverrideInst && *OverrideInst);
	return *OverrideInst;
}

void FNiagaraWorldManager::CleanupParameterCollections()
{
	for (TPair<UNiagaraParameterCollection*, UNiagaraParameterCollectionInstance*> CollectionInstPair : ParameterCollections)
	{
		UNiagaraParameterCollection* Collection = CollectionInstPair.Key;
		UNiagaraParameterCollectionInstance* CollectionInst = CollectionInstPair.Value;

		CollectionInst->Unbind(World);

#if WITH_EDITOR
		//Ensure that the default instance is not bound to the override.
		UNiagaraParameterCollectionInstance* DefaultInst = Collection->GetDefaultInstance();
		DefaultInst->GetParameterStore().Unbind(&CollectionInst->GetParameterStore());
#endif
	}
	ParameterCollections.Empty();
}

FNiagaraSystemSimulationRef FNiagaraWorldManager::GetSystemSimulation(ETickingGroup TickGroup, UNiagaraSystem* System)
{
	LLM_SCOPE(ELLMTag::Niagara);

	int32 ActualTickGroup = NiagaraWorldManagerInternal::GetNiagaraTickGroup(TickGroup);
	if (ActiveNiagaraTickGroup == ActualTickGroup)
	{
		int32 DemotedTickGroup = NiagaraWorldManagerInternal::GetNiagaraTickGroup((ETickingGroup)(TickGroup + 1));
		ActualTickGroup = DemotedTickGroup == ActualTickGroup ? 0 : DemotedTickGroup;
		
		// Note we must change TickGroup to the new tick group in case we need to create a new simulation, otherwise it will have the wrong TickGroup assigned to it
		TickGroup = NiagaraWorldManagerInternal::GetTickingGroupFromNiagara(ActualTickGroup);
	}

	const bool bCanOverlapTickGroups = System->AsyncWorkCanOverlapTickGroups();
	const ENiagaraWorldManagerTickPass TickPass = bCanOverlapTickGroups || bUseTickPasses == false ? ENiagaraWorldManagerTickPass::Overlapping : ENiagaraWorldManagerTickPass::NoneOverlapping;
	FNiagaraSystemSimulationRef* SimPtr = SystemSimulations[ActualTickGroup][int(TickPass)].Find(System);
	if (SimPtr != nullptr)
	{
		return *SimPtr;
	}
	
	FNiagaraSystemSimulationRef Sim = MakeShared<FNiagaraSystemSimulation, ESPMode::ThreadSafe>();
	SystemSimulations[ActualTickGroup][int(TickPass)].Add(System, Sim);
	Sim->Init(System, World, false, TickGroup, FNiagaraParameterCollectionResolver(nullptr, System, this));

	// Old path which will block simulation at tick boundaries
	if (bUseTickPasses == false)
	{
		//If this system doesn't allow async work to overlap TGs then we must ensure our Tick task is performed in it's correct TG.
		//The async work for this system can then properly be chained to the tick task so that everything completes on time.
		//Most systems most of the time will no need this and we'll clear this requirement as soon as all systems no longer need it.
		//TODO: This should go away when we move over to the new task graph system and rejig our dependency tracking.
		if(!bCanOverlapTickGroups)
		{
			TickFunctions[ActualTickGroup][int(TickPass)].EndTickGroup = (ETickingGroup)ActualTickGroup;
		}
	}
	// New path which will only block none overlapping simulations are tick boundaries
	else
	{
		// Ensure tick function is running
		if (TickFunctions[ActualTickGroup][int(TickPass)].IsTickFunctionEnabled() == false )
		{
			TickFunctions[ActualTickGroup][int(TickPass)].SetTickFunctionEnable(true);
		}
	}

#if WITH_EDITOR
	if (!SystemPostChangeDelegateHandles.Contains(System))
	{
		SystemPostChangeDelegateHandles.Emplace(System, System->OnSystemPostEditChange().AddRaw(this, &FNiagaraWorldManager::OnSystemPostChange));
	}
#endif

	return Sim;
}

void FNiagaraWorldManager::DestroySystemSimulation(UNiagaraSystem* System)
{
	for ( int TG=0; TG < NiagaraNumTickGroups; ++TG )
	{
		for (int iTickPass=0; iTickPass < int(ENiagaraWorldManagerTickPass::Num); ++iTickPass)
		{
			FNiagaraSystemSimulationRef* Simulation = SystemSimulations[TG][iTickPass].Find(System);
			if (Simulation != nullptr)
			{
				SimulationsWithPostActorWork.Remove(*Simulation);
				SimulationsWithEndOfFrameWait.Remove(*Simulation);

				(*Simulation)->Destroy();
				SystemSimulations[TG][iTickPass].Remove(System);
			}
		}
	}
	if (ComponentPool)
	{
		ComponentPool->RemoveComponentsBySystem(System);
	}

#if WITH_EDITOR
	if (FDelegateHandle* Handle = SystemPostChangeDelegateHandles.Find(System))
	{
		System->OnSystemPostEditChange().Remove(*Handle);
		SystemPostChangeDelegateHandles.Remove(System);
	}
#endif
}

void FNiagaraWorldManager::DestroySystemInstance(FNiagaraSystemInstancePtr& InPtr)
{
	check(IsInGameThread());
	check(InPtr != nullptr);
	DeferredDeletionQueue.Emplace(MoveTemp(InPtr));
}

#if WITH_EDITOR
void FNiagaraWorldManager::OnSystemPostChange(UNiagaraSystem* System)
{
	if (UNiagaraEffectType* EffectType = System->GetEffectType())
	{
		if (FNiagaraScalabilityManager* ScalabilityMan = ScalabilityManagers.Find(EffectType))
		{
			ScalabilityMan->OnSystemPostChange(System);
		}
	}
}
#endif//WITH_EDITOR

void FNiagaraWorldManager::OnComputeDispatchInterfaceDestroyed_Internal(FNiagaraGpuComputeDispatchInterface* InComputeDispatchInterface)
{
	GetDataChannelManager().OnComputeDispatchInterfaceDestroyed(InComputeDispatchInterface);

	// Process the deferred deletion queue before deleting the ComputeDispatchInterface of this world.
	// This is required because the ComputeDispatchInterface is accessed in FNiagaraEmitterInstance::~FNiagaraEmitterInstance
	DeferredDeletionQueue.Empty();
}

void FNiagaraWorldManager::CleanupTransientObjectsAndManagers()
{
	//TODO: Move as much of cleanup as possible in here.
	//Some world cleanup flows don't call BeginTearDown or even OnWorldCleanup sometimes so anything that could be cleaned up immediately on teardown should go in here.
	if (ComponentPool)
	{
		ComponentPool->Cleanup(World);
	}
	DataChannelManager->Cleanup();

	for (auto it = CullProxyMap.CreateIterator(); it; ++it)
	{
		if (it->Value)
		{
			it->Value->DestroyComponent();
		}
	}
	CullProxyMap.Empty();

	ScalabilityManagers.Empty();

	DataManagers.Empty();
}

void FNiagaraWorldManager::OnWorldBeginTearDown()
{
	bIsTearingDown = true;
	if(NiagaraRollbackCVars::NewWorldCleanup)
	{
		CleanupTransientObjectsAndManagers();
	}
	else
	{
		LegacyOnWorldBeginTearDown();
	}
}

void FNiagaraWorldManager::OnWorldCleanup(bool bSessionEnded, bool bCleanupResources)
{
	bIsCleaningUp = true;
	if(NiagaraRollbackCVars::NewWorldCleanup)
	{
		DeferredMethods.ExecuteAndClear();
	
		CleanupTransientObjectsAndManagers();

		for (int TG = 0; TG < NiagaraNumTickGroups; ++TG)
		{
			for (int iTickPass = 0; iTickPass < int(ENiagaraWorldManagerTickPass::Num); ++iTickPass)
			{
				for (TPair<UNiagaraSystem*, FNiagaraSystemSimulationRef>& SimPair : SystemSimulations[TG][iTickPass])
				{
					SimPair.Value->Destroy();
				}
				SystemSimulations[TG][iTickPass].Empty();
			}
		}
		CleanupParameterCollections();

		SimulationsWithPostActorWork.Empty();
		SimulationsWithEndOfFrameWait.Empty();

		DeferredDeletionQueue.Empty();

		DIGeneratedData.Empty();

	#if WITH_EDITOR
		for (FSystemDelegateMap::TConstIterator DelegateIt(SystemPostChangeDelegateHandles); DelegateIt; ++DelegateIt)
		{
			if (UNiagaraSystem* System = DelegateIt.Key().Get())
			{
				System->OnSystemPostEditChange().Remove(DelegateIt.Value());
			}
		}
		SystemPostChangeDelegateHandles.Empty();
	#endif
	}
	else
	{
		LegacyOnWorldCleanup(bSessionEnded, bCleanupResources);
	}
}

void FNiagaraWorldManager::OnPostWorldCleanup(bool bSessionEnded, bool bCleanupResources)
{	
	if(NiagaraRollbackCVars::NewWorldCleanup)
	{
		//Ensure we've already cleaned everything up. In some cases OnWorldCleanup doesn't appear to be called by the world cleanup flows.
		OnWorldCleanup(bSessionEnded, bCleanupResources);
	}
	else
	{
		LegacyOnPostWorldCleanup(bSessionEnded, bCleanupResources);
	}
}

//////////////////////////////////////////////////////////////////////////
//Legacy cleanup code.
//Scheduled for removal in UE-347574
void FNiagaraWorldManager::LegacyOnWorldBeginTearDown()
{
	bIsTearingDown = true;
	if (ComponentPool)
	{
		ComponentPool->Cleanup(World);
	}
	DataChannelManager->Cleanup();
	
	DataManagers.Empty();
}

void FNiagaraWorldManager::LegacyOnWorldCleanup(bool bSessionEnded, bool bCleanupResources)
{
	bIsCleaningUp = true;

	DeferredMethods.ExecuteAndClear();
	if (ComponentPool)
	{
		ComponentPool->Cleanup(World);
	}

	DataChannelManager->Cleanup();
	
	DataManagers.Empty();

	for (int TG = 0; TG < NiagaraNumTickGroups; ++TG)
	{
		for (int iTickPass = 0; iTickPass < int(ENiagaraWorldManagerTickPass::Num); ++iTickPass)
		{
			for (TPair<UNiagaraSystem*, FNiagaraSystemSimulationRef>& SimPair : SystemSimulations[TG][iTickPass])
			{
				SimPair.Value->Destroy();
			}
			SystemSimulations[TG][iTickPass].Empty();
		}
	}
	CleanupParameterCollections();

	SimulationsWithPostActorWork.Empty();
	SimulationsWithEndOfFrameWait.Empty();

	DeferredDeletionQueue.Empty();

	ScalabilityManagers.Empty();

	CullProxyMap.Empty();
	DIGeneratedData.Empty();

#if WITH_EDITOR
	for (FSystemDelegateMap::TConstIterator DelegateIt(SystemPostChangeDelegateHandles); DelegateIt; ++DelegateIt)
	{
		if (UNiagaraSystem* System = DelegateIt.Key().Get())
		{
			System->OnSystemPostEditChange().Remove(DelegateIt.Value());
		}
	}
	SystemPostChangeDelegateHandles.Empty();
#endif
}

void FNiagaraWorldManager::LegacyOnPostWorldCleanup(bool bSessionEnded, bool bCleanupResources)
{
	if (ComponentPool)
	{
		ComponentPool->Cleanup(World);
	}

	DataChannelManager->Cleanup();
	
	DataManagers.Empty();
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraWorldManager::PreGarbageCollect()
{
	if (GNiagaraWaitOnPreGC)
	{
		SCOPE_CYCLE_COUNTER(STAT_NiagaraWorldManWaitPreGC);

		// We must wait for system simulation & instance async ticks to complete before garbage collection can start
		// The reason for this is that our async ticks could be referencing GC objects, i.e. skel meshes, etc, and we don't want them to become unreachable while we are potentially using them
		WaitForAsyncWork();
	}
	
	for (auto It = ScalabilityManagers.CreateIterator(); It; ++It)
	{
		UNiagaraEffectType* EffectType = It.Key();
		FNiagaraScalabilityManager& ScalabilityMan = It.Value();

		if(!IsValid(EffectType) || ScalabilityMan.NumRegisteredComponents() == 0)
		{
			It.RemoveCurrent();
		}
	}

	// Clear out the pooled objects
	if ( ObjectPool )
	{
		ObjectPool->PreGarbageCollect();
	}
}

void FNiagaraWorldManager::PostReachabilityAnalysis()
{
	// Post garbage collection we must mark parameter stores with UObjects dirty to ensure any objects which are nullptr are updated in the various parameter stores.
	// We must also mark the component render data as dirty to ensure any material references in renderers are updated as they could also have become nullptr.
	for (TObjectIterator<UNiagaraComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		UNiagaraComponent* Component = *ComponentIt;
		if (IsValid(Component) && !Component->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
		{
			FNiagaraUserRedirectionParameterStore& ParameterStore = Component->GetOverrideParameters();
			if (ParameterStore.GetUObjects().Num() > 0)
			{
				ParameterStore.MarkUObjectsDirty();
				Component->MarkRenderStateDirty();
			}
		}
	}
}


void FNiagaraWorldManager::PostGarbageCollect()
{
	//Clear out and scalability managers who's EffectTypes have been GCd.
	while (ScalabilityManagers.Remove(nullptr)) {}

	#if WITH_NIAGARA_DEBUGGER
	for (auto It = SystemScalabilityKillCounts.CreateIterator(); It; ++It)
	{
		if (It.Key().IsValid() == false)
		{
			It.RemoveCurrent();
		}
	}
	#endif

#if WITH_NIAGARA_LEAK_DETECTOR
	if (ComponentLeakDetector.IsValid() && HasActiveWorld())
	{
		ComponentLeakDetector->ReportLeaks(World);
	}
#endif
}

void FNiagaraWorldManager::PreGarbageCollectBeginDestroy()
{
	//Clear out and scalability managers who's EffectTypes have been GCd.
	while (ScalabilityManagers.Remove(nullptr)) {}

	//Also tell the scalability managers to clear out any references the GC has nulled.
	for (auto& Pair : ScalabilityManagers)
	{
		FNiagaraScalabilityManager& ScalabilityMan = Pair.Value;
		UNiagaraEffectType* EffectType = Pair.Key;
		ScalabilityMan.PreGarbageCollectBeginDestroy();
	}
}

void FNiagaraWorldManager::RefreshOwnerAllowsScalability()
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraWorldManRefreshOwnerAllowsScalability);

	//Refresh all component's owner allows scalability flags and register/unreister with the manager accordingly.
	for (TObjectIterator<UNiagaraComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		ComponentIt->ResolveOwnerAllowsScalability();
	}

	//Force a full refresh of scalability state next tick.
	for (auto& Pair : ScalabilityManagers)
	{
		UNiagaraEffectType* EffectType = Pair.Key;
		FNiagaraScalabilityManager& ScalabilityMan = Pair.Value;
		ScalabilityMan.OnRefreshOwnerAllowsScalability();
	}
}

void FNiagaraWorldManager::InitDataChannel(const UNiagaraDataChannel* Channel, bool bForce)
{
	WaitForAsyncWork();
	GetDataChannelManager().InitDataChannel(Channel, bForce);
}

void FNiagaraWorldManager::RemoveDataChannel(const UNiagaraDataChannel* Channel)
{
	WaitForAsyncWork();
	GetDataChannelManager().RemoveDataChannel(Channel);
}

UNiagaraDataChannelHandler* FNiagaraWorldManager::FindDataChannelHandler(const UNiagaraDataChannel* Channel)
{
	return GetDataChannelManager().FindDataChannelHandler(Channel);
}

void FNiagaraWorldManager::WaitForAsyncWork()
{
	ForAllSystemSimulations(
		[](FNiagaraSystemSimulation& SystemSimulation)
		{
			SystemSimulation.WaitForInstancesTickComplete();
		}
	);
}

void FNiagaraWorldManager::OnWorldInit(UWorld* World, const UWorld::InitializationValues IVS)
{
	if(IVS.bCreateFXSystem || !NiagaraRollbackCVars::WorldManagerRespectsWorldCreateFXSystem)
	{
		FNiagaraWorldManager*& NewManager = WorldManagers.FindOrAdd(World);
		if (!NewManager)
		{
			NewManager = new FNiagaraWorldManager();
		}
		NewManager->Init(World);
	}
}

void FNiagaraWorldManager::OnPostWorldInit(UWorld* World, const UWorld::InitializationValues IVS)
{
	FNiagaraWorldManager** Manager = WorldManagers.Find(World);
	if (Manager)
	{
		FNiagaraWorldManager* WorldMan = *Manager;
		if (ensure(WorldMan))
		{
			WorldMan->OnPostWorldInit();
		}
	}
}

void FNiagaraWorldManager::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	//Cleanup world manager contents but not the manager itself.
	FNiagaraWorldManager** Manager = WorldManagers.Find(World);
	if (Manager)
	{
		FNiagaraWorldManager* WorldMan = *Manager;
		if (ensure(WorldMan))
		{
			WorldMan->OnWorldCleanup(bSessionEnded, bCleanupResources);
		}
	}
}

void FNiagaraWorldManager::OnPostWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	//Cleanup world manager contents but not the manager itself.
	FNiagaraWorldManager** Manager = WorldManagers.Find(World);
	if (Manager)
	{
		FNiagaraWorldManager* WorldMan = *Manager;
		if (ensure(WorldMan))
		{
			WorldMan->OnPostWorldCleanup(bSessionEnded, bCleanupResources);
		}
	}
}

void FNiagaraWorldManager::OnPreWorldFinishDestroy(UWorld* World)
{
	FNiagaraWorldManager** Manager = WorldManagers.Find(World);
	if (Manager)
	{
		FNiagaraWorldManager* WorldMan = *Manager;
		if(ensure(WorldMan))
		{
			delete WorldMan;
		}
		WorldManagers.Remove(World);
	}
}

void FNiagaraWorldManager::OnWorldBeginTearDown(UWorld* World)
{
	FNiagaraWorldManager** Manager = WorldManagers.Find(World);
	if (Manager)
	{
		FNiagaraWorldManager* WorldMan = *Manager;
		if(ensure(WorldMan))
		{
			WorldMan->OnWorldBeginTearDown();
		}		
	}
}

void FNiagaraWorldManager::OnComputeDispatchInterfaceDestroyed(FNiagaraGpuComputeDispatchInterface* InComputeDispatchInterface)
{
	for (TPair<UWorld*, FNiagaraWorldManager*>& Pair : WorldManagers)
	{
		UWorld* World = Pair.Key;
		if (FNiagaraGpuComputeDispatchInterface::Get(World) == InComputeDispatchInterface)
		{
			Pair.Value->OnComputeDispatchInterfaceDestroyed_Internal(InComputeDispatchInterface);
		}
	}
}

void FNiagaraWorldManager::DestroyAllSystemSimulations(class UNiagaraSystem* System)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraWorldManager::DestroyAllSystemSimulations);

	for (TPair<UWorld*, FNiagaraWorldManager*>& Pair : WorldManagers)
	{
		Pair.Value->DestroySystemSimulation(System);
	}
}

void FNiagaraWorldManager::TickWorldStart(UWorld* World, ELevelTick TickType, float DeltaSeconds)
{
	if(FNiagaraWorldManager* WorldMan = Get(World))
	{
		WorldMan->TickStart(DeltaSeconds);
	}	
}

void FNiagaraWorldManager::TickWorldEnd(UWorld* World, ELevelTick TickType, float DeltaSeconds)
{
	if(FNiagaraWorldManager* WorldMan = Get(World))
	{
		WorldMan->TickEnd(DeltaSeconds);
	}
}

void FNiagaraWorldManager::TickWorld(UWorld* World, ELevelTick TickType, float DeltaSeconds)
{
	if(FNiagaraWorldManager* WorldMan = Get(World))
	{
		WorldMan->PostActorTick(DeltaSeconds);
	}
}

void FNiagaraWorldManager::OnWorldPreActorTick(UWorld* InWorld, ELevelTick InLevelTick, float InDeltaSeconds)
{
	if(FNiagaraWorldManager* WorldMan = Get(InWorld))
	{
		WorldMan->PreActorTick(InLevelTick, InDeltaSeconds);
	}
}

void FNiagaraWorldManager::OnPreGarbageCollect()
{
	for (TPair<UWorld*, FNiagaraWorldManager*>& Pair : WorldManagers)
	{
		Pair.Value->PreGarbageCollect();
	}
}

void FNiagaraWorldManager::OnPostReachabilityAnalysis()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraWorldManager::OnPostReachabilityAnalysis);
	for (TPair<UWorld*, FNiagaraWorldManager*>& Pair : WorldManagers)
	{
		Pair.Value->PostReachabilityAnalysis();
	}
}

void FNiagaraWorldManager::OnPostGarbageCollect()
{
	for (TPair<UWorld*, FNiagaraWorldManager*>& Pair : WorldManagers)
	{
		Pair.Value->PostGarbageCollect();
	}
}

void FNiagaraWorldManager::OnPreGarbageCollectBeginDestroy()
{
	for (TPair<UWorld*, FNiagaraWorldManager*>& Pair : WorldManagers)
	{
		Pair.Value->PreGarbageCollectBeginDestroy();
	}
}

void FNiagaraWorldManager::OnRefreshOwnerAllowsScalability()
{
	for (TPair<UWorld*, FNiagaraWorldManager*>& Pair : WorldManagers)
	{
		Pair.Value->RefreshOwnerAllowsScalability();
	}
}

void FNiagaraWorldManager::TickStart(float DeltaSeconds)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	LLM_SCOPE(ELLMTag::Niagara);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraWorldManTick);

	//Have the render thread signal the most recent world time it has consumed.
	//This will cause a several frame delay in render based visibility culling.
	//However it will make the culling less sensitive to variable RT times and hitches etc
	//that would otherwise cause systems to be culled in error.
	check(LastRenderThreadUpdateTime.IsValid());
	float WorldTime = GetWorld()->GetTimeSeconds();
	RenderThreadWorldTimeLag = WorldTime - LastRenderThreadUpdateTime->load(std::memory_order_relaxed);
	ENQUEUE_RENDER_COMMAND(FNiagaraWorldManagerRTUpdateTime)(
		[LRT = LastRenderThreadUpdateTime, WorldTime](FRHICommandListImmediate& RHICmdList)
		{
			LRT->store(WorldTime, std::memory_order_relaxed);
		}
	);

	DataChannelManager->BeginFrame(DeltaSeconds);

	for(auto& Pair : DataManagers)
	{
		UNiagaraDataManager* Manager = Pair.Value.Get();
		if(Manager)
		{
			Manager->BeginFrame();
		}
	}

#if WITH_NIAGARA_LEAK_DETECTOR
	if (ComponentLeakDetector.IsValid() && HasActiveWorld())
	{
		CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_NiagaraLeakDetector);
		ComponentLeakDetector->Tick(World);
	}
#endif
}

void FNiagaraWorldManager::TickEnd(float DeltaSeconds)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	LLM_SCOPE(ELLMTag::Niagara);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraWorldManTick);

	//Do any end of tick bookkeeping that must occur even when paused.	
	FNiagaraSharedObject::FlushDeletionList();
	NiagaraWorldManagerInternal::ExecuteGlobalDeferredCallbacks();

	// Delete any instances that were pending deletion
	//-TODO: This could be done after each system sim has run
	DeferredDeletionQueue.Empty();

#if WITH_NIAGARA_DEBUGGER
	// Tick debug HUD for the world
	if (NiagaraDebugHud != nullptr)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_NiagaraDebugHUD);
		CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_NiagaraDebugHUD);
		NiagaraDebugHud->GatherSystemInfo();
	}
#endif

	if (DebugPlaybackMode == ENiagaraDebugPlaybackMode::Step)
	{
		RequestedDebugPlaybackMode = ENiagaraDebugPlaybackMode::Paused;
		DebugPlaybackMode = ENiagaraDebugPlaybackMode::Paused;
	}

	HandleCSVStats(DeltaSeconds);

	DataChannelManager->EndFrame(DeltaSeconds);
	
	for(auto& Pair : DataManagers)
	{
		UNiagaraDataManager* Manager = Pair.Value.Get();
		if(Manager)
		{
			Manager->EndFrame();
		}
	}
}

void FNiagaraWorldManager::PreActorTick(ELevelTick InLevelTick, float InDeltaSeconds)
{	
	NiagaraWorldManagerInternal::ExecuteGlobalDeferredCallbacks();
}

void FNiagaraWorldManager::PostActorTick(float DeltaSeconds)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	LLM_SCOPE(ELLMTag::Niagara);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraWorldManTick);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraOverview_GT);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NiagaraPostActorTick_GT);

	NiagaraWorldManagerInternal::ExecuteGlobalDeferredCallbacks();

	DeltaSeconds *= DebugPlaybackRate;

	// Update any systems with post actor work
	// - Instances that need to move to a higher tick group
	// - Instances that are pending spawn
	// - Instances that were spawned and we need to ensure the async tick is complete
	if (SimulationsWithPostActorWork.Num() > 0)
	{
		// Tick Parameter Collections to ensure they are up to date
		if (NiagaraWorldManagerInternal::GTickParameterCollectionsOncePerFrame == false)
		{
			TickParameterCollections();
		}

		// Ensure completion of all systems
		for (int32 i = 0; i < SimulationsWithPostActorWork.Num(); ++i)
		{
			FNiagaraSystemSimulationRef& Simulation = SimulationsWithPostActorWork[i];
			if ( Simulation->IsValid() )
			{
				Simulation->WaitForInstancesTickComplete();
			}
		}

		// Update tick groups
		{
			CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_NiagaraPostActorUpdateTickGroups);
			for (int32 i = 0; i < SimulationsWithPostActorWork.Num(); ++i)
			{
				FNiagaraSystemSimulationRef& Simulation = SimulationsWithPostActorWork[i];
				if (Simulation->IsValid())
				{
					Simulation->UpdateTickGroups_GameThread();
				}
			}
		}

		// Execute spawning
		TArray<FNiagaraSystemSimulationRef> LocalSimulationsWithPostActorWork;
		Swap(SimulationsWithPostActorWork, LocalSimulationsWithPostActorWork);
		for (FNiagaraSystemSimulationRef& Simulation : LocalSimulationsWithPostActorWork)
		{
			if (Simulation->IsValid())
			{
				Simulation->Spawn_GameThread(DeltaSeconds, true);
			}
		}

		// Wait for any spawning that may be required to complete in PostActorTick
		for (int32 i = 0; i < SimulationsWithPostActorWork.Num(); ++i)
		{
			FNiagaraSystemSimulationRef& Simulation = SimulationsWithPostActorWork[i];
			if (Simulation->IsValid())
			{
				Simulation->WaitForInstancesTickComplete();
			}
		}
		SimulationsWithPostActorWork.Reset();
	}

	// Clear cached player view location list, it should never be used outside of the world tick
	CachedViewInfo.Reset();

	for (auto it = CullProxyMap.CreateIterator(); it; ++it)
	{
		UNiagaraSystem* System = it->Key;

		if (GNiagaraAllowCullProxies && it->Value && System->GetCullProxyMode() != ENiagaraCullProxyMode::None)
		{
			it->Value->TickCullProxy();
		}
		else
		{
			if (it->Value)
			{
				it->Value->DestroyComponent();
			}
			it.RemoveCurrent();
		}
	}
}

void FNiagaraWorldManager::HandleCSVStats(float DeltaSeconds)
{
	bool bCSVStatsEnabled = false;
#if WITH_PARTICLE_PERF_CSV_STATS
	if (FCsvProfiler* CSVProfiler = FCsvProfiler::Get())
	{
		if (CSVProfiler->IsCapturing() && FParticlePerfStats::GetCSVStatsEnabled())
		{
			bCSVStatsEnabled = true;
			//Record custom events marking split times at set intervals. Allows us to generate summary tables for averages over shorter bursts.
			if (GNiagaraCSVSplitTime > 0.0f)
			{
				const double WorldTime = World->GetTimeSeconds();
				const double PrevWorldTime = WorldTime - DeltaSeconds;

				int32 CurrentSplitIdx = (int32)(WorldTime / GNiagaraCSVSplitTime);
				int32 PrevSplitIdx = (int32)(PrevWorldTime / GNiagaraCSVSplitTime);
				if (CurrentSplitIdx > PrevSplitIdx)
				{
					CSVProfiler->RecordEvent(CSV_CATEGORY_INDEX(Particles), FString::Printf(TEXT("SplitTime%d"), CurrentSplitIdx));
				}
			}

			for (auto& Pair : ScalabilityManagers)
			{
				FNiagaraScalabilityManager& ScalabilityMan = Pair.Value;
				UNiagaraEffectType* EffectType = Pair.Key;
				check(EffectType);

				ScalabilityMan.CSVProfilerUpdate(CSVProfiler);
			}
			#if WITH_PER_FXTYPE_PARTICLE_PERF_STATS
			if (FXTypeCSVListener.IsValid() == false)
			{
				FXTypeCSVListener = MakeShared<FParticlePerfStatsListener_EffectType, ESPMode::ThreadSafe>();
				FParticlePerfStatsManager::AddListener(FXTypeCSVListener);
			}
			#endif
		}
	}
#endif 

#if WITH_PER_FXTYPE_PARTICLE_PERF_STATS
	if (bCSVStatsEnabled == false && FXTypeCSVListener.IsValid())
	{
		FParticlePerfStatsManager::RemoveListener(FXTypeCSVListener.Get());
		FXTypeCSVListener.Reset();
	}
#endif
}

void FNiagaraWorldManager::PreSendAllEndOfFrameUpdates()
{
	if ( SimulationsWithPostActorWork.Num() > 0 )
	{
		for (const auto& Simulation : SimulationsWithPostActorWork)
		{
			if (Simulation->IsValid())
			{
				Simulation->WaitForInstancesTickComplete();
			}
		}
		// Note: We do not clear the SimulationsWithPostActorWork array as we just want to safely wait for any async work,
		//       they still require additional processing, i.e. Tick Group Changes / Spawning
	}

	for (const auto& Simulation : SimulationsWithEndOfFrameWait)
	{
		if (Simulation->IsValid())
		{
			Simulation->WaitForInstancesTickComplete();
		}
	}
	SimulationsWithEndOfFrameWait.Reset();
}

void FNiagaraWorldManager::MarkSimulationForPostActorWork(FNiagaraSystemSimulation* SystemSimulation)
{
	check(SystemSimulation != nullptr);
	check(!SystemSimulation->GetIsSolo());
	if ( !SimulationsWithPostActorWork.ContainsByPredicate([&](const FNiagaraSystemSimulationRef& Existing) { return &Existing.Get() == SystemSimulation; }) )
	{
		SimulationsWithPostActorWork.Add(SystemSimulation->AsShared());
	}
}

void FNiagaraWorldManager::MarkSimulationsForEndOfFrameWait(FNiagaraSystemSimulation* SystemSimulation)
{
	check(SystemSimulation);
	check(!SystemSimulation->GetIsSolo());
	if (!SimulationsWithEndOfFrameWait.ContainsByPredicate([&](const FNiagaraSystemSimulationRef& Existing) { return &Existing.Get() == SystemSimulation; }))
	{
		SimulationsWithEndOfFrameWait.Add(SystemSimulation->AsShared());
	}
}

bool FNiagaraWorldManager::PrepareCachedViewInfo(const APlayerController* PlayerController, FNiagaraCachedViewInfo& OutViewInfo)
{
	const ULocalPlayer* LocalPlayer = PlayerController ? PlayerController->GetLocalPlayer() : nullptr;
	if (PlayerController && LocalPlayer && LocalPlayer->ViewportClient)
	{
		FSceneViewProjectionData ProjectionData;
		if (LocalPlayer->GetProjectionData(LocalPlayer->ViewportClient->Viewport, /*out*/ ProjectionData))
		{
			FVector POVLoc;
			FRotator POVRotation;
			PlayerController->GetPlayerViewPoint(POVLoc, POVRotation);
			FRotationTranslationMatrix ViewToWorld(POVRotation, POVLoc);

			FWorldCachedViewInfo WorldViewInfo;
			WorldViewInfo.ViewMatrix = FTranslationMatrix(-ProjectionData.ViewOrigin) * ProjectionData.ViewRotationMatrix;
			WorldViewInfo.ProjectionMatrix = ProjectionData.ProjectionMatrix;
			WorldViewInfo.ViewProjectionMatrix = ProjectionData.ComputeViewProjectionMatrix();
			WorldViewInfo.ViewToWorld = ViewToWorld;

			OutViewInfo.Init(WorldViewInfo);
			return true;
		}
	}

	return false;
}

void FNiagaraWorldManager::Tick(ENiagaraWorldManagerTickPass TickPass, ETickingGroup TickGroup, float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent)
{
	check(TickGroup >= NiagaraFirstTickGroup && TickGroup <= NiagaraLastTickGroup);

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	LLM_SCOPE(ELLMTag::Niagara);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraWorldManTick);
	SCOPE_CYCLE_COUNTER(STAT_NiagaraOverview_GT);

	// Modify playback rate
	DeltaSeconds *= DebugPlaybackRate;

	// Determine what work needs to run
	const int ActualTickGroup = NiagaraWorldManagerInternal::GetNiagaraTickGroup(TickGroup);
	const FNiagaraWorldManagerTickFunction& NonOverlappingTickFunction = TickFunctions[ActualTickGroup][int(ENiagaraWorldManagerTickPass::NoneOverlapping)];
	const bool bIsFirstPass = TickPass == ENiagaraWorldManagerTickPass::First || NonOverlappingTickFunction.IsTickFunctionEnabled() == false;
	const bool bIsLastPass = TickPass == ENiagaraWorldManagerTickPass::Last || NonOverlappingTickFunction.IsTickFunctionEnabled() == false;

	// Execute work at the start of each tick group
	if (bIsFirstPass)
	{
		ExecutePreTickWork(TickGroup, DeltaSeconds);
	}

	// If we are in paused don't do anything
	if (DebugPlaybackMode == ENiagaraDebugPlaybackMode::Paused)
	{
		return;
	}

	// Tick generated data (First tick pass only)
	if (bIsFirstPass)
	{
		CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_NiagaraDIGeneratedData);
		for (auto& GeneratedData : DIGeneratedData)
		{
			GeneratedData.Value->Tick(TickGroup, DeltaSeconds);
		}
	}

	// Execute simulations for this tick group
	ExecuteSimulations(TickPass, TickGroup, DeltaSeconds, MyCompletionGraphEvent);

	// Execute work at the end of each tick group
	if (bIsLastPass)
	{
		ExecutePostTickWork(TickGroup, DeltaSeconds);
	}
}

void FNiagaraWorldManager::ExecutePreTickWork(ETickingGroup TickGroup, float DeltaSeconds)
{
	// Execute any global and local deferred functions
	NiagaraWorldManagerInternal::ExecuteGlobalDeferredCallbacks();
	DeferredMethods.ExecuteAndClear();

	//Tick DataChannel Manager.
	DataChannelManager->Tick(DeltaSeconds, TickGroup);
	
	for(auto& Pair : DataManagers)
	{
		UNiagaraDataManager* Manager = Pair.Value.Get();
		if(Manager)
		{
			Manager->PreTick(TickGroup, DeltaSeconds);
		}
	}

	// We do book keeping in the first tick group
	if ( TickGroup == NiagaraFirstTickGroup )
	{
		// Update playback mode
		DebugPlaybackMode = RequestedDebugPlaybackMode;

		// Utility loop feature to trigger all systems to loop on a timer.
		if (GWorldLoopTime > 0.0f)
		{
			if (WorldLoopTime <= 0.0f)
			{
				WorldLoopTime = GWorldLoopTime;
				for (TObjectIterator<UNiagaraComponent> It; It; ++It)
				{
					if (UNiagaraComponent* Comp = *It)
					{
						if(Comp->GetWorld() == GetWorld())
						{
							Comp->ResetSystem();
						}
					}
				}
			}
			WorldLoopTime -= DeltaSeconds;
		}

		if (bInvalidateCachedSystemScalabilityData.exchange(false))
		{
			InvalidateCachedSystemScalabilityDataForAllWorlds();
		}

		//Ensure the pools have been primed.
		//WorldInit is too early.
		if(!bPoolIsPrimed)
		{
			PrimePoolForAllSystems();
			bPoolIsPrimed = true;
		}

#if WITH_EDITOR //PLATFORM_DESKTOP
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Niagara_IsThisApplicationForeground);
			bAppHasFocus = FApp::HasFocus();
		}
#else
		bAppHasFocus = true;
#endif

		// If we are in paused don't do anything
		if (DebugPlaybackMode == ENiagaraDebugPlaybackMode::Paused)
		{
			return;
		}

		{
			CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_NiagaraCacheViewInformation);
			bool bUseWorldCachedViews = !World->GetPlayerControllerIterator();
#if WITH_EDITOR
			if (GCurrentLevelEditingViewportClient && (GCurrentLevelEditingViewportClient->GetWorld() == World))
			{
				bUseWorldCachedViews = true;
			}
#endif
			// Cache player view info for all system instances to access
			//-TODO: Do we need to do this per tick group?
			if (bUseWorldCachedViews)
			{
				for (int32 i = 0; i < World->CachedViewInfoRenderedLastFrame.Num(); ++i)
				{
					FWorldCachedViewInfo& WorldViewInfo = World->CachedViewInfoRenderedLastFrame[i];

					FNiagaraCachedViewInfo& ViewInfo = CachedViewInfo.AddDefaulted_GetRef();
					ViewInfo.Init(WorldViewInfo);
				}
			}
			else
			{
				for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
				{
					APlayerController* PlayerController = Iterator->Get();
					if (PlayerController && PlayerController->IsLocalPlayerController())
					{
						FNiagaraCachedViewInfo& ViewInfo = CachedViewInfo.AddDefaulted_GetRef();
					
						const bool bIsValid = PrepareCachedViewInfo(PlayerController, ViewInfo);
						if (!bIsValid)
						{
							CachedViewInfo.RemoveAt(CachedViewInfo.Num() - 1);
						}
					}
				}
			}		
		}

		UpdateScalabilityManagers(DeltaSeconds, false);

		TickParameterCollections();
	}
	else
	{
		// Tick Parameter Collections to ensure they are up to date
		if (NiagaraWorldManagerInternal::GTickParameterCollectionsOncePerFrame == false)
		{
			TickParameterCollections();
		}
	}
}

void FNiagaraWorldManager::ExecuteSimulations(ENiagaraWorldManagerTickPass TickPass, ETickingGroup TickGroup, float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent)
{
	// Now tick all system instances. 
	const int ActualTickGroup = NiagaraWorldManagerInternal::GetNiagaraTickGroup(TickGroup);

	ActiveNiagaraTickGroup = ActualTickGroup;

	bool bHasSimulationsInTickPass = false;
	bool bCanOverlapTickGroups = true;
	for (auto SimIt=SystemSimulations[ActualTickGroup][int(TickPass)].CreateIterator(); SimIt; ++SimIt)
	{
		FNiagaraSystemSimulation* Sim = &SimIt.Value().Get();

		if (!SimIt.Value().IsUnique() && Sim->IsValid())
		{
			bHasSimulationsInTickPass = true;
			Sim->Tick_GameThread(DeltaSeconds, MyCompletionGraphEvent);
			bCanOverlapTickGroups &= Sim->GetSystem()->AsyncWorkCanOverlapTickGroups();
		}
		else
		{
			Sim->Destroy();
			SimIt.RemoveCurrent();
		}
	}

	// New path that uses different tick passes for overlapping / non overlapping work
	if (bUseTickPasses)
	{
		// Should we disable the none overlapping tick function from running?
		if (TickPass == ENiagaraWorldManagerTickPass::NoneOverlapping && !bHasSimulationsInTickPass)
		{
			FNiagaraWorldManagerTickFunction& NonOverlappingTickFunction = TickFunctions[ActualTickGroup][int(ENiagaraWorldManagerTickPass::NoneOverlapping)];
			NonOverlappingTickFunction.SetTickFunctionEnable(false);
		}
	}
	// Old tick path which modifies the end tick group which can cause stalls
	else
	{
		//If all systems in this TG can overlap their async work then we can allow the tick as a whole to run until last demotable.
		//When adding new system sims to this tick group we can change this.
		//Any systems needing their async work to complete must have this Tick function finish inside this TG so that we can properly complete our async work before the TG finishes.
		if ( bCanOverlapTickGroups )
		{
			//Set here but this wont take effect until next tick.
			TickFunctions[ActualTickGroup][int(TickPass)].EndTickGroup = TG_LastDemotable;
		}
		else
		{
			TickFunctions[ActualTickGroup][int(TickPass)].EndTickGroup = (ETickingGroup)ActualTickGroup;
		}
	}


	ActiveNiagaraTickGroup = -1;
}

void FNiagaraWorldManager::ExecutePostTickWork(ETickingGroup TickGroup, float DeltaSeconds)
{
	//We update scalability managers here so that any new systems can be culled or setup with other scalability based parameters correctly for their spawn.
	UpdateScalabilityManagers(DeltaSeconds, true);

	// Loop over all simulations that have been marked for post actor (i.e. ones whos TG is changing or have pending spawn systems)
	if (GNiagaraSpawnPerTickGroup && (SimulationsWithPostActorWork.Num() > 0))
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_NiagaraSpawnPerTickGroup_GT);
		for (int32 i = 0; i < SimulationsWithPostActorWork.Num(); ++i)
		{
			const auto& Simulation = SimulationsWithPostActorWork[i];
			if (Simulation->IsValid() && (Simulation->GetTickGroup() < TickGroup))
			{
				Simulation->Spawn_GameThread(DeltaSeconds, false);
			}
		}
	}
}

void FNiagaraWorldManager::DumpDetails(FOutputDevice& Ar, bool bDetailed)
{
	Ar.Logf(TEXT("=== FNiagaraWorldManager Dumping Information"));

	const UEnum* TickingGroupEnum = StaticEnum<ETickingGroup>();
	for ( int TG=0; TG < NiagaraNumTickGroups; ++TG )
	{
		TStringBuilder<8> TickStateString;
		int32 TotalSimulations = 0;
		for (int iTickPass=0; iTickPass < int(ENiagaraWorldManagerTickPass::Num); ++iTickPass)
		{
			TickStateString.Append(TickFunctions[TG][iTickPass].IsTickFunctionEnabled() ? TEXT("X") : TEXT("."));
			TotalSimulations += SystemSimulations[TG][iTickPass].Num();
		}

		Ar.Logf(
			TEXT("%-20s | TickState(%s) | %d Simulations"),
			*TickingGroupEnum->GetNameStringByIndex(TG + NiagaraFirstTickGroup),
			TickStateString.ToString(),
			TotalSimulations
		);

		if (bDetailed)
		{
			for (int iTickPass = 0; iTickPass < int(ENiagaraWorldManagerTickPass::Num); ++iTickPass)
			{
				for (TPair<UNiagaraSystem*, FNiagaraSystemSimulationRef>& SystemSim : SystemSimulations[TG][iTickPass])
				{
					FNiagaraSystemSimulation* Sim = &SystemSim.Value.Get();
					if (!Sim->IsValid())
					{
						continue;
					}

					Ar.Logf(TEXT("\tSimulation %s"), *Sim->GetSystem()->GetFullName());
					Sim->DumpTickInfo(Ar);
				}
			}
		}
	}
}

UWorld* FNiagaraWorldManager::GetWorld()
{
	return World;
}

//////////////////////////////////////////////////////////////////////////

FVector::FReal FNiagaraWorldManager::GetLODDistance(FVector Location)const
{
	FVector::FReal LODDistance = 0.0f;
	// If we are inside the WorldManager tick we will use the cache player view locations as we can be ticked on different threads
	if (GetCachedViewInfo().Num() > 0)
	{
		// We are being ticked inside the WorldManager and can safely use the list of cached player view locations
		FVector::FReal LODDistanceSqr = FMath::Square(WORLD_MAX);
		for (const FNiagaraCachedViewInfo& ViewInfo : GetCachedViewInfo())
		{
			const FVector::FReal DistanceToEffectSqr = FVector(ViewInfo.ViewToWorld.GetOrigin() - Location).SizeSquared();
			LODDistanceSqr = FMath::Min(LODDistanceSqr, DistanceToEffectSqr);
		}
		LODDistance = FMath::Sqrt(LODDistanceSqr);
	}
	else
	{
		// If we are not inside the WorldManager tick (solo tick) we must look over the player view locations manually
		ensureMsgf(IsInGameThread(), TEXT("FNiagaraSystemInstance::GetLODDistance called in potentially thread unsafe way"));

		TArray<FVector, TInlineAllocator<8> > PlayerViewLocations;
		if (World->GetPlayerControllerIterator())
		{
			for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
			{
				APlayerController* PlayerController = Iterator->Get();
				if (PlayerController && PlayerController->IsLocalPlayerController())
				{
					FVector* ViewLocation = new(PlayerViewLocations) FVector;
					FRotator ViewRotation;
					PlayerController->GetPlayerViewPoint(*ViewLocation, ViewRotation);
				}
			}
		}
		else
		{
			PlayerViewLocations = World->ViewLocationsRenderedLastFrame;
		}

		LODDistance = GetLODDistance(Location, PlayerViewLocations);
	}
	return LODDistance;
}

FVector::FReal FNiagaraWorldManager::GetLODDistance(FVector Location, TConstArrayView<FVector> ViewPoints) const
{
	FVector::FReal LODDistance = 0.0f;

	if (ViewPoints.Num() > 0)
	{
		FVector::FReal LODDistanceSqr = FMath::Square(WORLD_MAX);
		for (const FVector& ViewLocation : ViewPoints)
		{
			const FVector::FReal DistanceToEffectSqr = FVector(ViewLocation - Location).SizeSquared();
			LODDistanceSqr = FMath::Min(LODDistanceSqr, DistanceToEffectSqr);
		}
		LODDistance = FVector::FReal(FMath::Sqrt(LODDistanceSqr));
	}
	return LODDistance;
}

void FNiagaraWorldManager::UpdateScalabilityManagers(float DeltaSeconds, bool bNewSpawnsOnly)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateScalabilityManagers);
	CSV_SCOPED_TIMING_STAT(Particles, CoreSystems_NiagaraUpdateScalabilityManagers);

#if DO_CHECK
	TGuardValue<bool> GuardScalabilityManagersLocked(bScalabilityManagersLocked, true);
#endif

	for (auto& Pair : ScalabilityManagers)
	{
		FNiagaraScalabilityManager& ScalabilityMan = Pair.Value;
		UNiagaraEffectType* EffectType = Pair.Key;
		check(EffectType);

		if (bNewSpawnsOnly)
		{
			ScalabilityMan.Update(DeltaSeconds, true);
		}
		else
		{
			ScalabilityMan.Update(DeltaSeconds, false);
		}
	}
}

void FNiagaraWorldManager::RegisterWithScalabilityManager(UNiagaraComponent* Component, UNiagaraEffectType* EffectType)
{
	if (GetScalabilityCullingMode() == ENiagaraScalabilityCullingMode::Disabled)
	{
		return;
	}

	check(Component);
	if ( EffectType )
	{
		FNiagaraScalabilityManager* ScalabilityManager = ScalabilityManagers.Find(EffectType);

		if (!ScalabilityManager)
		{
		#if DO_CHECK
			ensureMsgf(bScalabilityManagersLocked == false, TEXT("ScalabilityManagers modified while iterating, by Component (%s) System (%s)"), *GetNameSafe(Component), *GetNameSafe(Component ? Component->GetAsset() : nullptr));
		#endif
			ScalabilityManager = &ScalabilityManagers.Emplace(EffectType, this);
			ScalabilityManager->EffectType = EffectType;
		}

		ScalabilityManager->Register(Component);
	}
}

void FNiagaraWorldManager::UnregisterWithScalabilityManager(UNiagaraComponent* Component, UNiagaraEffectType* EffectType)
{
	check(Component);
	if ( EffectType )
	{
		//Possibly the manager has been GCd.
		if (FNiagaraScalabilityManager* ScalabilityManager = ScalabilityManagers.Find(EffectType))
		{
			ScalabilityManager->Unregister(Component);
		}
		else
		{
			//The component does this itself in unregister.
			//Component->ScalabilityManagerHandle = INDEX_NONE;
		}
	}
}

bool FNiagaraWorldManager::ShouldPreCull(UNiagaraSystem* System, UNiagaraComponent* Component)
{
	if (GetScalabilityCullingMode() == ENiagaraScalabilityCullingMode::Enabled && System)
	{
		if (UNiagaraEffectType* EffectType = System->GetEffectType())
		{
			if (CanPreCull(EffectType))
			{
				FNiagaraScalabilityState State;
				const FNiagaraSystemScalabilitySettings& ScalabilitySettings = System->GetScalabilitySettings();
				CalculateScalabilityState(System, ScalabilitySettings, EffectType, Component, true, FFXBudget::GetWorstAdjustedUsage(), State);
				#if WITH_NIAGARA_DEBUGGER
				GetSystemScalabilityKillCounts(System).Add(State);				
				#endif
				return State.bCulled;
			}
		}
	}
	return false;
}

bool FNiagaraWorldManager::ShouldPreCull(UNiagaraSystem* System, FVector Location)
{
	if (GetScalabilityCullingMode() == ENiagaraScalabilityCullingMode::Enabled && System)
	{
		if (UNiagaraEffectType* EffectType = System->GetEffectType())
		{
			if (CanPreCull(EffectType))
			{
				FNiagaraScalabilityState State;
				const FNiagaraSystemScalabilitySettings& ScalabilitySettings = System->GetScalabilitySettings();
				CalculateScalabilityState(System, ScalabilitySettings, EffectType, Location, true, FFXBudget::GetWorstAdjustedUsage(), State);
				//TODO: Tell the debugger about recently PreCulled systems.
				return State.bCulled;
			}
		}
	}
	return false;
}

void FNiagaraWorldManager::CalculateScalabilityState(UNiagaraSystem* System, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, UNiagaraEffectType* EffectType, FVector Location, bool bIsPreCull, float WorstGlobalBudgetUse, FNiagaraScalabilityState& OutState)
{
	if (GetScalabilityCullingMode() == ENiagaraScalabilityCullingMode::Disabled)
	{
		return;
	}

	OutState.bCulled = false;

	DistanceCull(EffectType, ScalabilitySettings, Location, OutState);

	if (GEnableNiagaraVisCulling)
	{
		if (System->bFixedBounds)//We have no component in this path so we require fixed bounds for view based checks.
		{
			FBoxSphereBounds Bounds(System->GetFixedBounds());
			Bounds.Origin = Location;
			float TimeSinceRendered = 0.0f;
			ViewBasedCulling(EffectType, ScalabilitySettings, Bounds.GetSphere(), TimeSinceRendered, bIsPreCull, OutState);
		}
	}

	//If we have no significance handler there is no concept of relative significance for these systems so we can just pre cull if we go over the instance count.
	if (GEnableNiagaraInstanceCountCulling && bIsPreCull && EffectType->GetSignificanceHandler() == nullptr)
	{
		InstanceCountCull(EffectType, System, ScalabilitySettings, OutState);
	}

	//Cull if any of our budgets are exceeded.
	bool bEnabled = GEnableNiagaraGlobalBudgetCulling && FFXBudget::Enabled() && INiagaraModule::UseGlobalFXBudget();
 	if (!OutState.bCulled && bEnabled && ScalabilitySettings.BudgetScaling.bCullByGlobalBudget)
 	{
 		GlobalBudgetCull(ScalabilitySettings, WorstGlobalBudgetUse, OutState);
 	}

	//TODO: More progressive scalability options?
}

extern float GLastRenderTimeSafetyBias;
void FNiagaraWorldManager::CalculateScalabilityState(UNiagaraSystem* System, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, UNiagaraEffectType* EffectType, UNiagaraComponent* Component, bool bIsPreCull, float WorstGlobalBudgetUse, FNiagaraScalabilityState& OutState)
{
	if (GetScalabilityCullingMode() == ENiagaraScalabilityCullingMode::Disabled)
	{
		return;
	}

	OutState.bCulled = false;

	DistanceCull(EffectType, ScalabilitySettings, Component, OutState);

	if (GEnableNiagaraVisCulling)
	{
		FBoxSphereBounds Bounds = Component->CalcBounds(Component->GetComponentToWorld());		

		// If we don't have a scene proxy we can not cull against last render time
		// We also make sure to update the last render time so when the proxy becomes visible we don't get a potential invalid frame before we get data back from the render thread
		float TimeSinceRendered = 0.0f;
		if (Component->GetSceneProxy() || (NiagaraRollbackCVars::GAggressiveProxylessCulling && !Component->ShouldComponentAddToScene()))
		{
			TimeSinceRendered = float(FMath::Max(0.0f, GetWorld()->LastRenderTime - Component->GetLastRenderTime() - World->GetDeltaSeconds() - RenderThreadWorldTimeLag - GLastRenderTimeSafetyBias));
		}
		else
		{
			Component->SetLastRenderTime(GetWorld()->LastRenderTime);
		}

		if(bIsPreCull)
		{
			if (System->bFixedBounds)//We need valid bounds for view based culling so if we're preculling, restrict to systems with fixed bounds.
			{
				ViewBasedCulling(EffectType, ScalabilitySettings, Bounds.GetSphere(), TimeSinceRendered, bIsPreCull, OutState);
			}
		}
		else
		{
			if (GbAllowVisibilityCullingForDynamicBounds || System->bFixedBounds)
			{
				ViewBasedCulling(EffectType, ScalabilitySettings, Bounds.GetSphere(), TimeSinceRendered, bIsPreCull, OutState);
			}
		}
	}

	//Only apply hard instance count cull limit for precull if we have no significance handler.
	if (GEnableNiagaraInstanceCountCulling && bIsPreCull && EffectType->GetSignificanceHandler() == nullptr)
	{
		InstanceCountCull(EffectType, System, ScalabilitySettings, OutState);
	}

	bool bEnabled = GEnableNiagaraGlobalBudgetCulling && FFXBudget::Enabled() && INiagaraModule::UseGlobalFXBudget();
 	if (!OutState.bCulled && bEnabled && ScalabilitySettings.BudgetScaling.bCullByGlobalBudget)
	{
 		GlobalBudgetCull(ScalabilitySettings, WorstGlobalBudgetUse, OutState);
 	}

	//TODO: More progressive scalability options?
}

bool FNiagaraWorldManager::CanPreCull(UNiagaraEffectType* EffectType)
{
	checkSlow(EffectType);
	return EffectType->CullReaction == ENiagaraCullReaction::Deactivate || EffectType->CullReaction == ENiagaraCullReaction::DeactivateImmediate;
}

void FNiagaraWorldManager::SortedSignificanceCull(UNiagaraEffectType* EffectType, UNiagaraComponent* Component, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, float Significance, int32& EffectTypeInstCount, uint16& SystemInstCount, FNiagaraScalabilityState& OutState)
{
	//Cull all but the N most significance FX.
	bool bCull = false;
	
	if(GetScalabilityCullingMode() == ENiagaraScalabilityCullingMode::Enabled && GEnableNiagaraInstanceCountCulling)
	{
		UNiagaraSystem* System = Component->GetAsset();
		check(System);

		int32 SystemInstanceMax = 0;
		int32 EffectTypeInstanceMax = 0;
		System->GetMaxInstanceCounts(SystemInstanceMax, EffectTypeInstanceMax, false);

		bCull = ScalabilitySettings.bCullMaxInstanceCount && EffectTypeInstCount >= EffectTypeInstanceMax;
		bCull |= ScalabilitySettings.bCullPerSystemMaxInstanceCount && SystemInstCount >= SystemInstanceMax;

		bool bBudgetCullEnabled = GEnableNiagaraGlobalBudgetCulling && FFXBudget::Enabled() && INiagaraModule::UseGlobalFXBudget();
		if (bCull)
		{
#if DEBUG_SCALABILITY_STATE
			//Clear budget culled flag if we're culling for other reasons already. Help us determine the real impact of budget culling.
			OutState.bCulledByGlobalBudget = false;
#endif
		}
		else if (bBudgetCullEnabled)
		{
			//Grab Adjusted instance counts.
			System->GetMaxInstanceCounts(SystemInstanceMax, EffectTypeInstanceMax, bBudgetCullEnabled);

			bCull = ScalabilitySettings.bCullMaxInstanceCount && EffectTypeInstCount >= EffectTypeInstanceMax;
			bCull |= ScalabilitySettings.bCullPerSystemMaxInstanceCount && SystemInstCount >= SystemInstanceMax;

#if DEBUG_SCALABILITY_STATE
			OutState.bCulledByGlobalBudget |= bCull;
#endif
	  	}
	}

	OutState.bCulled |= bCull;

	//Only increment the instance counts if this is not culled. Including other causes of culling.
	//This considers only culled state and will only affect which components we *want* to turn on.
	//There is another check inside the apply that will limit instance counts to ensure they never exceed the max that are actually running (or winding down)
	if(OutState.bCulled == false)
	{
		++EffectTypeInstCount;
		++SystemInstCount;
	}

#if DEBUG_SCALABILITY_STATE
	OutState.bCulledByInstanceCount = bCull;
#endif
}

void FNiagaraWorldManager::ViewBasedCulling(UNiagaraEffectType* EffectType, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, FSphere BoundingSphere, float ComponentTimeSinceRendered, bool bIsPrecull, FNiagaraScalabilityState& OutState)
{
	if (GetScalabilityCullingMode() != ENiagaraScalabilityCullingMode::Enabled)
	{
		return;
	}

	bool bInsideAnyView = !ScalabilitySettings.VisibilityCulling.bCullByViewFrustum;

	//Iterator over all views to calculate and check the bounds against the view frustum.
	if (ScalabilitySettings.VisibilityCulling.bCullByViewFrustum)
	{
		for (FNiagaraCachedViewInfo& ViewInfo : CachedViewInfo)
		{
			//First check the view frustums to see if any part of the sphere is inside.
			bool bInsideThisView = true;
			if (bInsideAnyView == false)//If we already know we're in any view (or not doing frustum checks) then we can skip the rest.
			{
				for (FPlane& FrustumPlane : ViewInfo.FrutumPlanes)
				{
					if (FrustumPlane.IsValid())
					{
						//((dot(Plane, BSphere.xyz) - Plane.w) > BSphere.w);
						bool bInside = FrustumPlane.PlaneDot(BoundingSphere.Center) <= BoundingSphere.W;
						if (!bInside)
						{
							bInsideThisView = false;
							break;
						}
					}
				}
			}

			if (bInsideThisView)
			{
				bInsideAnyView = true;
			}
		}

		// If we have no CachedViewInfo this can mean one of two things, we have no views or we are outside of the main loop (i.e. reregister context) so start the sim anyway
		bInsideAnyView |= CachedViewInfo.Num() == 0;
	}

	float TimeSinceInsideView = 0.0f;
	if (bInsideAnyView)
	{
		OutState.LastVisibleTime = static_cast<float>(World->GetTimeSeconds());
	}
	else
	{
		TimeSinceInsideView = static_cast<float>(World->GetTimeSeconds() - OutState.LastVisibleTime);
	}

	bool bCullByOutsideViewFrustum = ScalabilitySettings.VisibilityCulling.bCullByViewFrustum &&
		(!bIsPrecull || ScalabilitySettings.VisibilityCulling.bAllowPreCullingByViewFrustum) &&
		TimeSinceInsideView > ScalabilitySettings.VisibilityCulling.MaxTimeOutsideViewFrustum;

	//Check for the component having been rendered recently. If the app doesn't have focus we skip this to avoid issues when alt-tabbing away from the game/editor.
	bool bCullByNotRendered =	bAppHasFocus && 
								ScalabilitySettings.VisibilityCulling.bCullWhenNotRendered && 
		ComponentTimeSinceRendered > ScalabilitySettings.VisibilityCulling.MaxTimeWithoutRender;

	//TODO: Pull screen size out into it's own debug flag in the scalability state.
	bool bCull = bCullByNotRendered || bCullByOutsideViewFrustum;

	OutState.bCulled |= bCull;
#if DEBUG_SCALABILITY_STATE
	OutState.bCulledByVisibility = bCull;
#endif
}

void FNiagaraWorldManager::InstanceCountCull(UNiagaraEffectType* EffectType, UNiagaraSystem* System, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, FNiagaraScalabilityState& OutState)
{
	if (GetScalabilityCullingMode() != ENiagaraScalabilityCullingMode::Enabled)
	{
		return;
	}

	int32 SystemInstanceMax = ScalabilitySettings.MaxSystemInstances;
	int32 EffectTypeInstanceMax = ScalabilitySettings.MaxInstances;

	bool bCull = ScalabilitySettings.bCullMaxInstanceCount && EffectType->NumInstances >= EffectTypeInstanceMax;
	bCull |= ScalabilitySettings.bCullPerSystemMaxInstanceCount && System->GetActiveInstancesCount() >= SystemInstanceMax;

	bool bBudgetCullEnabled = GEnableNiagaraGlobalBudgetCulling && FFXBudget::Enabled() && INiagaraModule::UseGlobalFXBudget();
	//Apply budget based adjustments separately so we can mark this cull as being due to budgetting or not.
	if (bCull)
	{
#if DEBUG_SCALABILITY_STATE
		//Clear budget culled flag if we're culling for other reasons already. Help us determine the real impact of budget culling.
		OutState.bCulledByGlobalBudget = false;
#endif
	}
	else if (bBudgetCullEnabled && (ScalabilitySettings.BudgetScaling.bScaleMaxInstanceCountByGlobalBudgetUse || ScalabilitySettings.BudgetScaling.bScaleSystemInstanceCountByGlobalBudgetUse))
	{
		float Usage = FFXBudget::GetWorstAdjustedUsage();

		if (ScalabilitySettings.BudgetScaling.bScaleMaxInstanceCountByGlobalBudgetUse)
		{
			const float Scale = ScalabilitySettings.BudgetScaling.MaxInstanceCountScaleByGlobalBudgetUse.Evaluate(Usage);
			EffectTypeInstanceMax = int32(float(EffectTypeInstanceMax) * Scale);
		}
		if (ScalabilitySettings.BudgetScaling.bScaleSystemInstanceCountByGlobalBudgetUse)
		{
			const float Scale = ScalabilitySettings.BudgetScaling.MaxSystemInstanceCountScaleByGlobalBudgetUse.Evaluate(Usage);
			SystemInstanceMax = int32(float(SystemInstanceMax) * Scale);
		}
		bCull = ScalabilitySettings.bCullMaxInstanceCount && EffectType->NumInstances >= EffectTypeInstanceMax;
		bCull |= ScalabilitySettings.bCullPerSystemMaxInstanceCount && System->GetActiveInstancesCount() >= SystemInstanceMax;
		OutState.bCulled |= bCull;
#if DEBUG_SCALABILITY_STATE
		OutState.bCulledByGlobalBudget |= bCull;
#endif
	}

	OutState.bCulled |= bCull;
#if DEBUG_SCALABILITY_STATE
	OutState.bCulledByInstanceCount = bCull;
#endif
}

void FNiagaraWorldManager::DistanceCull(UNiagaraEffectType* EffectType, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, UNiagaraComponent* Component, FNiagaraScalabilityState& OutState)
{
	float LODDistance = 0.0f;

	if (Component->bEnablePreviewLODDistance)
	{
		LODDistance = Component->PreviewLODDistance;
	}
	else if(GetCachedViewInfo().Num() > 0)
	{
		float ClosestDistSq = FLT_MAX;
		FVector Location = Component->GetComponentLocation();
		for (const FNiagaraCachedViewInfo& ViewInfo : GetCachedViewInfo())
		{
			ClosestDistSq = FMath::Min(ClosestDistSq, float(FVector::DistSquared(ViewInfo.ViewToWorld.GetOrigin(), Location)));
		}

		LODDistance = FMath::Sqrt(ClosestDistSq);
	}

	//Directly drive the system lod distance from here.
	float MaxDist = ScalabilitySettings.MaxDistance;
	Component->SetLODDistance(LODDistance, FMath::Max(MaxDist, 1.0f));

	if (GetScalabilityCullingMode() == ENiagaraScalabilityCullingMode::Enabled && GEnableNiagaraDistanceCulling && ScalabilitySettings.bCullByDistance)
	{
		bool bCull = LODDistance > MaxDist;
		OutState.bCulled |= bCull;

		bool bBudgetCullEnabled = GEnableNiagaraGlobalBudgetCulling && FFXBudget::Enabled() && INiagaraModule::UseGlobalFXBudget();
		//Check the budget adjusted range separately so we can tell what is down to budgets and what's distance.
		if (bCull)
		{
#if DEBUG_SCALABILITY_STATE
			//Clear budget culled flag if we're culling for other reasons already. Help us determine the real impact of budget culling.
			OutState.bCulledByGlobalBudget = false;
#endif
		}
		else if (bBudgetCullEnabled && ScalabilitySettings.BudgetScaling.bScaleMaxDistanceByGlobalBudgetUse)
		{
			float Usage = FFXBudget::GetWorstAdjustedUsage();
			float Scale = ScalabilitySettings.BudgetScaling.MaxDistanceScaleByGlobalBudgetUse.Evaluate(Usage);
			MaxDist *= Scale;

			bCull = LODDistance > MaxDist;

#if DEBUG_SCALABILITY_STATE
			OutState.bCulledByGlobalBudget |= bCull;
#endif
		}

#if DEBUG_SCALABILITY_STATE
		OutState.bCulledByDistance = bCull;
#endif
		OutState.bCulled |= bCull;
	}
}

void FNiagaraWorldManager::DistanceCull(UNiagaraEffectType* EffectType, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, FVector Location, FNiagaraScalabilityState& OutState)
{
	if (GetScalabilityCullingMode() == ENiagaraScalabilityCullingMode::Enabled && GetCachedViewInfo().Num() > 0)
	{
		float ClosestDistSq = FLT_MAX;
		for (const FNiagaraCachedViewInfo& ViewInfo : GetCachedViewInfo())
		{
			ClosestDistSq = FMath::Min(ClosestDistSq, float(FVector::DistSquared(ViewInfo.ViewToWorld.GetOrigin(), Location)));
		}

		if (GEnableNiagaraDistanceCulling && ScalabilitySettings.bCullByDistance)
		{
			float MaxDist = ScalabilitySettings.MaxDistance;
			float ClosestDist = FMath::Sqrt(ClosestDistSq);
			bool bCull = ClosestDist > MaxDist;

			bool bBudgetCullEnabled = GEnableNiagaraGlobalBudgetCulling && FFXBudget::Enabled() && INiagaraModule::UseGlobalFXBudget();
			//Check the budget adjusted range separately so we can tell what is down to budgets and what's distance.
			if (bCull)
			{
#if DEBUG_SCALABILITY_STATE
				//Clear budget culled flag if we're culling for other reasons already. Help us determine the real impact of budget culling.
				OutState.bCulledByGlobalBudget = false;
#endif
			}
			else if (bBudgetCullEnabled && ScalabilitySettings.BudgetScaling.bScaleMaxDistanceByGlobalBudgetUse)
			{
				float Usage = FFXBudget::GetWorstAdjustedUsage();
				float Scale = ScalabilitySettings.BudgetScaling.MaxDistanceScaleByGlobalBudgetUse.Evaluate(Usage);
				MaxDist *= Scale;

				bCull = ClosestDist > MaxDist;

#if DEBUG_SCALABILITY_STATE
				OutState.bCulledByGlobalBudget |= bCull;
#endif
			}

			OutState.bCulled |= bCull;
#if DEBUG_SCALABILITY_STATE
			OutState.bCulledByDistance = bCull;
#endif
		}
	}
}

void FNiagaraWorldManager::GlobalBudgetCull(const FNiagaraSystemScalabilitySettings& ScalabilitySettings, float WorstGlobalBudgetUse, FNiagaraScalabilityState& OutState)
{
 	bool bCull = GetScalabilityCullingMode() == ENiagaraScalabilityCullingMode::Enabled && WorstGlobalBudgetUse >= ScalabilitySettings.BudgetScaling.MaxGlobalBudgetUsage;
 	OutState.bCulled |= bCull;
#if DEBUG_SCALABILITY_STATE
	OutState.bCulledByGlobalBudget |= bCull;
#endif
}

bool FNiagaraWorldManager::GetScalabilityState(UNiagaraComponent* Component, FNiagaraScalabilityState& OutState) const
{
	if ( Component )
	{
		const int32 ScalabilityHandle = Component->GetScalabilityManagerHandle();
		if (ScalabilityHandle != INDEX_NONE)
		{
			if (UNiagaraSystem* System = Component->GetAsset())
			{
				if (UNiagaraEffectType* EffectType = System->GetEffectType())
				{
					if (const auto* ScalabilityManager = ScalabilityManagers.Find(EffectType))
					{
						OutState = ScalabilityManager->State[ScalabilityHandle];
						return true;
					}
				}
			}
		}
	}
	return false;
}

void FNiagaraWorldManager::RequestInvalidateCachedSystemScalabilityDataForAllWorlds()
{
	bInvalidateCachedSystemScalabilityData = true;
}

void FNiagaraWorldManager::InvalidateCachedSystemScalabilityDataForAllWorlds()
{
	for (auto& Pair : WorldManagers)
	{
		if (Pair.Value)
		{
			Pair.Value->InvalidateCachedSystemScalabilityData();
		}
	}
}

void FNiagaraWorldManager::InvalidateCachedSystemScalabilityData()
{
	for (auto& Pair : ScalabilityManagers)
	{
		FNiagaraScalabilityManager& ScalabilityMan = Pair.Value;
		ScalabilityMan.InvalidateCachedSystemData();
	}
}

bool FNiagaraWorldManager::HasActiveWorld() const
{
	return World && World->IsInitialized() && !World->bIsTearingDown && !bIsTearingDown && !bIsCleaningUp;
}

void FNiagaraWorldManager::PrimePoolForAllWorlds(UNiagaraSystem* System)
{
	if (GNigaraAllowPrimedPools)
	{
		for (auto& Pair : WorldManagers)
		{
			if (Pair.Value)
			{
				Pair.Value->PrimePool(System);
			}
		}
	}
}

void FNiagaraWorldManager::PrimePoolForAllSystems()
{
	if (GNigaraAllowPrimedPools && HasActiveWorld() && World->IsGameWorld() && ComponentPool)
	{
		//Prime the pool for all currently loaded systems.
		for (TObjectIterator<UNiagaraSystem> It; It; ++It)
		{
			UNiagaraSystem* Sys = *It;
			if (Sys)
			{
				ComponentPool->PrimePool(Sys, World);
			}
		}
	}
}

void FNiagaraWorldManager::PrimePool(UNiagaraSystem* System)
{
	if (GNigaraAllowPrimedPools && HasActiveWorld() && World->IsGameWorld() && ComponentPool)
	{
		ComponentPool->PrimePool(System, World);
	}
}

#if WITH_NIAGARA_DEBUGGER
void FNiagaraWorldManager::AddDebugHUDMessage(FName Key, const FNiagaraDebugMessage& Message)
{
	if(NiagaraDebugHud)
	{
		NiagaraDebugHud->AddMessage(Key, Message);
	}
}
#endif

void FNiagaraWorldManager::FlushComputeAndDeferredQueues(bool bWaitForGPU)
{
	check(IsInGameThread());

	// Flush the compute dispatch queue
	if ( FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = FNiagaraGpuComputeDispatchInterface::Get(World) )
	{
		if (bWaitForGPU)
		{
			ComputeDispatchInterface->FlushAndWait_GameThread();
		}
		else
		{
			ComputeDispatchInterface->FlushPendingTicks_GameThread();
		}
	}

	// Flush any deferred callbacks
	NiagaraWorldManagerInternal::ExecuteGlobalDeferredCallbacks();
}

void FNiagaraWorldManager::EnqueueGlobalDeferredCallback(TFunction<void()>&& Callback)
{
	using namespace NiagaraWorldManagerInternal;
	GlobalDeferredCallbacks.Enqueue(Callback);
}

UObject* FNiagaraWorldManager::ObjectPoolGetOrCreate(UClass* Class, bool& bIsExistingObject)
{
	check(IsInGameThread());
	check(Class != nullptr);

	bIsExistingObject = false;
	if (GNiagaraWorldManagerObjectPoolEnabled == false)
	{
		return NewObject<UObject>(World, Class);
	}

	if (ObjectPool == nullptr)
	{
		ObjectPool.Reset(new FNiagaraSimpleObjectPool());
	}

	return ObjectPool->GetOrCreateObject(Class, World, bIsExistingObject);
}

UObject* FNiagaraWorldManager::ObjectPoolGetOrCreate(UClass* Class)
{
	bool bIsExistingObject;
	return ObjectPoolGetOrCreate(Class, bIsExistingObject);
}

void FNiagaraWorldManager::ObjectPoolReturn(UObject* Obj)
{
	check(IsInGameThread());
	if (::IsValid(Obj) && GNiagaraWorldManagerObjectPoolEnabled)
	{
		if (ensure(ObjectPool) && ensure(Obj->GetOuter() == World))
		{
			ObjectPool->ReturnObject(Obj);
		}
	}
}

bool FNiagaraWorldManager::IsComponentLocalPlayerLinked(const USceneComponent* InComponent)
{
	check(InComponent);

	//Is our owner or instigator a locally viewed pawn?
	if (const AActor* Owner = InComponent->GetOwner())
	{
		if (const APawn* OwnerPawn = Cast<const APawn>(Owner))
		{
			if (OwnerPawn->IsLocallyViewed())
			{
				return true;
			}
		}

		if (const APawn* Instigator = Owner->GetInstigator())
		{
			if (Instigator->IsLocallyViewed())
			{
				return true;
			}
		}
	}

	//Walk the attachment hierarchy to check for locally viewed pawns.
	if (InComponent->GetAttachParent() && IsComponentLocalPlayerLinked(InComponent->GetAttachParent()))
	{
		return true;
	}

	return false;
}

void FNiagaraWorldManager::SetTickGroupPriority(const int32 NiagaraTickGroupIndex, const bool bHighPriority)
{
	check ((NiagaraTickGroupIndex >= 0) && (NiagaraTickGroupIndex < NiagaraNumTickGroups));
	for ( int iTickPass=0; iTickPass < int(ENiagaraWorldManagerTickPass::Num); ++iTickPass)
	{
		TickFunctions[NiagaraTickGroupIndex][iTickPass].SetPriorityIncludingPrerequisites(bHighPriority);
	}
}

#if DEBUG_SCALABILITY_STATE

void FNiagaraWorldManager::DumpScalabilityState()
{
	UE_LOGF(LogNiagara, Display, "========================================================================");
	UE_LOGF(LogNiagara, Display, "Niagara World Manager Scalability State. 0x%p - %ls", World, *World->GetPathName());
	UE_LOGF(LogNiagara, Display, "========================================================================");

	for (auto& Pair : ScalabilityManagers)
	{
		FNiagaraScalabilityManager& ScalabilityMan = Pair.Value;
		ScalabilityMan.Dump();
	}

	UE_LOGF(LogNiagara, Display, "========================================================================");
}


FAutoConsoleCommandWithWorld GDumpNiagaraScalabilityData(
	TEXT("fx.DumpNiagaraScalabilityState"),
	TEXT("Dumps state information for all Niagara Scalability Mangers."),
	FConsoleCommandWithWorldDelegate::CreateStatic(
		[](UWorld* World)
{
	FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World);
	WorldMan->DumpScalabilityState();
}));

#endif

void FNiagaraCachedViewInfo::Init(const FWorldCachedViewInfo& WorldViewInfo)
{
	ViewMat = WorldViewInfo.ViewMatrix;
	ProjectionMat = WorldViewInfo.ProjectionMatrix;
	ViewToWorld = WorldViewInfo.ViewToWorld;
	ViewProjMat = WorldViewInfo.ViewProjectionMatrix;

	FrutumPlanes.SetNumUninitialized(6);
	if (!ViewProjMat.GetFrustumNearPlane(FrutumPlanes[0]))
	{
		FrutumPlanes[0] = FPlane(0.0f, 0.0f, 0.0f, 0.0f);
	}
	if (!ViewProjMat.GetFrustumFarPlane(FrutumPlanes[1]))
	{
		FrutumPlanes[1] = FPlane(0.0f, 0.0f, 0.0f, 0.0f);
	}
	if (!ViewProjMat.GetFrustumTopPlane(FrutumPlanes[2]))
	{
		FrutumPlanes[2] = FPlane(0.0f, 0.0f, 0.0f, 0.0f);
	}
	if (!ViewProjMat.GetFrustumBottomPlane(FrutumPlanes[3]))
	{
		FrutumPlanes[3] = FPlane(0.0f, 0.0f, 0.0f, 0.0f);
	}
	if (!ViewProjMat.GetFrustumLeftPlane(FrutumPlanes[4]))
	{
		FrutumPlanes[4] = FPlane(0.0f, 0.0f, 0.0f, 0.0f);
	}
	if (!ViewProjMat.GetFrustumRightPlane(FrutumPlanes[5]))
	{
		FrutumPlanes[5] = FPlane(0.0f, 0.0f, 0.0f, 0.0f);
	}
}
