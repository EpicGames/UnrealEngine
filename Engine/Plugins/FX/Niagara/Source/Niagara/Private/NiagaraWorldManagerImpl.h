// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraWorldManager.h"
#include "NiagaraDataManager.h"

template<typename T>
T& FNiagaraWorldManager::GetOrCreateDataManager()
{
	FName ManagerName = T::StaticClass()->GetFName();
	T* Ret = nullptr;
	if(TObjectPtr<UNiagaraDataManager>* Found = DataManagers.Find(ManagerName))
	{
		Ret = Cast<T>(Found->Get());
	}

	if(Ret == nullptr)
	{
		TObjectPtr<UNiagaraDataManager>& NewMan = DataManagers.Add(ManagerName);
		NewMan = NewObject<T>(GetTransientPackageAsObject(), ManagerName);
		NewMan->Init(this);
		Ret = CastChecked<T>(NewMan.Get());
	}

	check(Ret);
	return *Ret;
}