// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Expressions/NiagaraStatelessExpressionColor.h"

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessSetShaderParameterContext.h"
#include "NiagaraParameterStore.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessExpressionColor)

FInstancedStruct FNiagaraStatelessExpressionColor::Build(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FInstancedStruct BuiltExpression;
	if (IsConstant())
	{
		FNiagaraStatelessExpressionColorConstant ConstantExpression;
		FNiagaraParameterStore EmptyParameterStore;
		ConstantExpression.A = EvaluateInternal(FEvaluateContext(EmptyParameterStore));
		return FInstancedStruct::Make(ConstantExpression);
	}
	return BuildInternal(BuildContext);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FNiagaraStatelessExpressionColorConstant::EvaluateInternal(const FEvaluateContext& Context) const
{
	return A;
}

bool FNiagaraStatelessExpressionColorConstant::IsConstant() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FInstancedStruct FNiagaraStatelessExpressionColorBinding::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionColorBinding BuiltExpression;
	BuiltExpression.A = A;
	BuiltExpression.ParameterOffset = BuildContext.AddRendererBinding(FNiagaraVariableBase(GetOutputTypeDef(), A));
	BuiltExpression.ParameterOffset *= sizeof(uint32);
	return FInstancedStruct::Make(BuiltExpression);
}

FLinearColor FNiagaraStatelessExpressionColorBinding::EvaluateInternal(const FEvaluateContext& Context) const
{
	return Context.ParameterStore.GetParameterValueFromOffset<FLinearColor>(ParameterOffset);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionColorAdd::FNiagaraStatelessExpressionColorAdd()
{
	A.InitializeAs<FNiagaraStatelessExpressionColorConstant>();
	B.InitializeAs<FNiagaraStatelessExpressionColorConstant>();
}

FInstancedStruct FNiagaraStatelessExpressionColorAdd::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionColorAdd BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FLinearColor FNiagaraStatelessExpressionColorAdd::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FLinearColor AValue = A.Get<FNiagaraStatelessExpressionColor>().EvaluateInternal(Context);
	const FLinearColor BValue = B.Get<FNiagaraStatelessExpressionColor>().EvaluateInternal(Context);
	return AValue + BValue;
}

bool FNiagaraStatelessExpressionColorAdd::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionColor>().IsConstant() && B.Get<FNiagaraStatelessExpressionColor>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionColorSubtract::FNiagaraStatelessExpressionColorSubtract()
{
	A.InitializeAs<FNiagaraStatelessExpressionColorConstant>();
	B.InitializeAs<FNiagaraStatelessExpressionColorConstant>();
}

FInstancedStruct FNiagaraStatelessExpressionColorSubtract::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionColorSubtract BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FLinearColor FNiagaraStatelessExpressionColorSubtract::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FLinearColor AValue = A.Get<FNiagaraStatelessExpressionColor>().EvaluateInternal(Context);
	const FLinearColor BValue = B.Get<FNiagaraStatelessExpressionColor>().EvaluateInternal(Context);
	return AValue - BValue;
}

bool FNiagaraStatelessExpressionColorSubtract::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionColor>().IsConstant() && B.Get<FNiagaraStatelessExpressionColor>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionColorMultiply::FNiagaraStatelessExpressionColorMultiply()
{
	A.InitializeAs<FNiagaraStatelessExpressionColorConstant>();
	B.InitializeAs<FNiagaraStatelessExpressionColorConstant>();
}

FInstancedStruct FNiagaraStatelessExpressionColorMultiply::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionColorMultiply BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FLinearColor FNiagaraStatelessExpressionColorMultiply::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FLinearColor AValue = A.Get<FNiagaraStatelessExpressionColor>().EvaluateInternal(Context);
	const FLinearColor BValue = B.Get<FNiagaraStatelessExpressionColor>().EvaluateInternal(Context);
	return AValue * BValue;
}

bool FNiagaraStatelessExpressionColorMultiply::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionColor>().IsConstant() && B.Get<FNiagaraStatelessExpressionColor>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionColorDivide::FNiagaraStatelessExpressionColorDivide()
{
	A.InitializeAs<FNiagaraStatelessExpressionColorConstant>();
	B.InitializeAs<FNiagaraStatelessExpressionColorConstant>();
}

FInstancedStruct FNiagaraStatelessExpressionColorDivide::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionColorDivide BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FLinearColor FNiagaraStatelessExpressionColorDivide::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FLinearColor AValue = A.Get<FNiagaraStatelessExpressionColor>().EvaluateInternal(Context);
	const FLinearColor BValue = B.Get<FNiagaraStatelessExpressionColor>().EvaluateInternal(Context);
	return FLinearColor(
		FMath::Abs(BValue.R) > UE_SMALL_NUMBER ? AValue.R / BValue.R : 0.0f,
		FMath::Abs(BValue.G) > UE_SMALL_NUMBER ? AValue.G / BValue.G : 0.0f,
		FMath::Abs(BValue.B) > UE_SMALL_NUMBER ? AValue.B / BValue.B : 0.0f,
		FMath::Abs(BValue.A) > UE_SMALL_NUMBER ? AValue.A / BValue.A : 0.0f
	);
}

bool FNiagaraStatelessExpressionColorDivide::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionColor>().IsConstant() && B.Get<FNiagaraStatelessExpressionColor>().IsConstant();
}
