// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Expressions/NiagaraStatelessExpressionFloat.h"

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessSetShaderParameterContext.h"
#include "NiagaraParameterStore.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessExpressionFloat)

FInstancedStruct FNiagaraStatelessExpressionFloat::Build(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FInstancedStruct BuiltExpression;
	if (IsConstant())
	{
		FNiagaraStatelessExpressionFloatConstant ConstantExpression;
		FNiagaraParameterStore EmptyParameterStore;
		ConstantExpression.A = EvaluateInternal(FEvaluateContext(EmptyParameterStore));
		return FInstancedStruct::Make(ConstantExpression);
	}
	return BuildInternal(BuildContext);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

float FNiagaraStatelessExpressionFloatConstant::EvaluateInternal(const FEvaluateContext& Context) const
{
	return A;
}

bool FNiagaraStatelessExpressionFloatConstant::IsConstant() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FInstancedStruct FNiagaraStatelessExpressionFloatBinding::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionFloatBinding BuiltExpression;
	BuiltExpression.A = A;
	BuiltExpression.ParameterOffset = BuildContext.AddRendererBinding(FNiagaraVariableBase(GetOutputTypeDef(), A));
	BuiltExpression.ParameterOffset *= sizeof(uint32);
	return FInstancedStruct::Make(BuiltExpression);
}

float FNiagaraStatelessExpressionFloatBinding::EvaluateInternal(const FEvaluateContext& Context) const
{
	return Context.ParameterStore.GetParameterValueFromOffset<float>(ParameterOffset);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionFloatAdd::FNiagaraStatelessExpressionFloatAdd()
{
	A.InitializeAs<FNiagaraStatelessExpressionFloatConstant>();
	B.InitializeAs<FNiagaraStatelessExpressionFloatConstant>();
}

FInstancedStruct FNiagaraStatelessExpressionFloatAdd::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionFloatAdd BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

float FNiagaraStatelessExpressionFloatAdd::EvaluateInternal(const FEvaluateContext& Context) const
{
	const float AValue = A.Get<FNiagaraStatelessExpressionFloat>().EvaluateInternal(Context);
	const float BValue = B.Get<FNiagaraStatelessExpressionFloat>().EvaluateInternal(Context);
	return AValue + BValue;
}

bool FNiagaraStatelessExpressionFloatAdd::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionFloat>().IsConstant() && B.Get<FNiagaraStatelessExpressionFloat>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionFloatSubtract::FNiagaraStatelessExpressionFloatSubtract()
{
	A.InitializeAs<FNiagaraStatelessExpressionFloatConstant>();
	B.InitializeAs<FNiagaraStatelessExpressionFloatConstant>();
}

FInstancedStruct FNiagaraStatelessExpressionFloatSubtract::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionFloatSubtract BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

float FNiagaraStatelessExpressionFloatSubtract::EvaluateInternal(const FEvaluateContext& Context) const
{
	const float AValue = A.Get<FNiagaraStatelessExpressionFloat>().EvaluateInternal(Context);
	const float BValue = B.Get<FNiagaraStatelessExpressionFloat>().EvaluateInternal(Context);
	return AValue - BValue;
}

bool FNiagaraStatelessExpressionFloatSubtract::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionFloat>().IsConstant() && B.Get<FNiagaraStatelessExpressionFloat>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionFloatMultiply::FNiagaraStatelessExpressionFloatMultiply()
{
	A.InitializeAs<FNiagaraStatelessExpressionFloatConstant>();
	B.InitializeAs<FNiagaraStatelessExpressionFloatConstant>();
}

FInstancedStruct FNiagaraStatelessExpressionFloatMultiply::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionFloatMultiply BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

float FNiagaraStatelessExpressionFloatMultiply::EvaluateInternal(const FEvaluateContext& Context) const
{
	const float AValue = A.Get<FNiagaraStatelessExpressionFloat>().EvaluateInternal(Context);
	const float BValue = B.Get<FNiagaraStatelessExpressionFloat>().EvaluateInternal(Context);
	return AValue * BValue;
}

bool FNiagaraStatelessExpressionFloatMultiply::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionFloat>().IsConstant() && B.Get<FNiagaraStatelessExpressionFloat>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionFloatDivide::FNiagaraStatelessExpressionFloatDivide()
{
	A.InitializeAs<FNiagaraStatelessExpressionFloatConstant>();
	B.InitializeAs<FNiagaraStatelessExpressionFloatConstant>();
}

FInstancedStruct FNiagaraStatelessExpressionFloatDivide::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionFloatDivide BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

float FNiagaraStatelessExpressionFloatDivide::EvaluateInternal(const FEvaluateContext& Context) const
{
	const float AValue = A.Get<FNiagaraStatelessExpressionFloat>().EvaluateInternal(Context);
	const float BValue = B.Get<FNiagaraStatelessExpressionFloat>().EvaluateInternal(Context);
	return FMath::Abs(BValue) > UE_SMALL_NUMBER ? AValue / BValue : 0.0f;
}

bool FNiagaraStatelessExpressionFloatDivide::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionFloat>().IsConstant() && B.Get<FNiagaraStatelessExpressionFloat>().IsConstant();
}
