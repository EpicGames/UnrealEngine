// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Expressions/NiagaraStatelessExpressionVec2.h"

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessSetShaderParameterContext.h"
#include "NiagaraParameterStore.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessExpressionVec2)

FInstancedStruct FNiagaraStatelessExpressionVec2::Build(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FInstancedStruct BuiltExpression;
	if (IsConstant())
	{
		FNiagaraStatelessExpressionVec2Constant ConstantExpression;
		FNiagaraParameterStore EmptyParameterStore;
		ConstantExpression.A = EvaluateInternal(FEvaluateContext(EmptyParameterStore));
		return FInstancedStruct::Make(ConstantExpression);
	}
	return BuildInternal(BuildContext);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FVector2f FNiagaraStatelessExpressionVec2Constant::EvaluateInternal(const FEvaluateContext& Context) const
{
	return A;
}

bool FNiagaraStatelessExpressionVec2Constant::IsConstant() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FInstancedStruct FNiagaraStatelessExpressionVec2Binding::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec2Binding BuiltExpression;
	BuiltExpression.A = A;
	BuiltExpression.ParameterOffset = BuildContext.AddRendererBinding(FNiagaraVariableBase(GetOutputTypeDef(), A));
	BuiltExpression.ParameterOffset *= sizeof(uint32);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector2f FNiagaraStatelessExpressionVec2Binding::EvaluateInternal(const FEvaluateContext& Context) const
{
	return Context.ParameterStore.GetParameterValueFromOffset<FVector2f>(ParameterOffset);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec2Add::FNiagaraStatelessExpressionVec2Add()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec2Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec2Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec2Add::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec2Add BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector2f FNiagaraStatelessExpressionVec2Add::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector2f AValue = A.Get<FNiagaraStatelessExpressionVec2>().EvaluateInternal(Context);
	const FVector2f BValue = B.Get<FNiagaraStatelessExpressionVec2>().EvaluateInternal(Context);
	return AValue + BValue;
}

bool FNiagaraStatelessExpressionVec2Add::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec2>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec2>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec2Subtract::FNiagaraStatelessExpressionVec2Subtract()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec2Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec2Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec2Subtract::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec2Subtract BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector2f FNiagaraStatelessExpressionVec2Subtract::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector2f AValue = A.Get<FNiagaraStatelessExpressionVec2>().EvaluateInternal(Context);
	const FVector2f BValue = B.Get<FNiagaraStatelessExpressionVec2>().EvaluateInternal(Context);
	return AValue - BValue;
}

bool FNiagaraStatelessExpressionVec2Subtract::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec2>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec2>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec2Multiply::FNiagaraStatelessExpressionVec2Multiply()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec2Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec2Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec2Multiply::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec2Multiply BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector2f FNiagaraStatelessExpressionVec2Multiply::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector2f AValue = A.Get<FNiagaraStatelessExpressionVec2>().EvaluateInternal(Context);
	const FVector2f BValue = B.Get<FNiagaraStatelessExpressionVec2>().EvaluateInternal(Context);
	return AValue * BValue;
}

bool FNiagaraStatelessExpressionVec2Multiply::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec2>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec2>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec2Divide::FNiagaraStatelessExpressionVec2Divide()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec2Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec2Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec2Divide::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec2Divide BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector2f FNiagaraStatelessExpressionVec2Divide::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector2f AValue = A.Get<FNiagaraStatelessExpressionVec2>().EvaluateInternal(Context);
	const FVector2f BValue = B.Get<FNiagaraStatelessExpressionVec2>().EvaluateInternal(Context);
	return FVector2f(
		FMath::Abs(BValue.X) > UE_SMALL_NUMBER ? AValue.X / BValue.X : 0.0f,
		FMath::Abs(BValue.Y) > UE_SMALL_NUMBER ? AValue.Y / BValue.Y : 0.0f
	);
}

bool FNiagaraStatelessExpressionVec2Divide::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec2>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec2>().IsConstant();
}
