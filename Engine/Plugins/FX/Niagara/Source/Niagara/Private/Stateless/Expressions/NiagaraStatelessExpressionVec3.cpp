// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Expressions/NiagaraStatelessExpressionVec3.h"

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessSetShaderParameterContext.h"
#include "NiagaraParameterStore.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessExpressionVec3)

FInstancedStruct FNiagaraStatelessExpressionVec3::Build(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FInstancedStruct BuiltExpression;
	if (IsConstant())
	{
		FNiagaraStatelessExpressionVec3Constant ConstantExpression;
		FNiagaraParameterStore EmptyParameterStore;
		ConstantExpression.A = EvaluateInternal(FEvaluateContext(EmptyParameterStore));
		return FInstancedStruct::Make(ConstantExpression);
	}
	return BuildInternal(BuildContext);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FVector3f FNiagaraStatelessExpressionVec3Constant::EvaluateInternal(const FEvaluateContext& Context) const
{
	return A;
}

bool FNiagaraStatelessExpressionVec3Constant::IsConstant() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FInstancedStruct FNiagaraStatelessExpressionVec3Binding::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec3Binding BuiltExpression;
	BuiltExpression.A = A;
	BuiltExpression.ParameterOffset = BuildContext.AddRendererBinding(FNiagaraVariableBase(GetOutputTypeDef(), A));
	BuiltExpression.ParameterOffset *= sizeof(uint32);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector3f FNiagaraStatelessExpressionVec3Binding::EvaluateInternal(const FEvaluateContext& Context) const
{
	return Context.ParameterStore.GetParameterValueFromOffset<FVector3f>(ParameterOffset);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec3Add::FNiagaraStatelessExpressionVec3Add()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec3Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec3Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec3Add::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec3Add BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector3f FNiagaraStatelessExpressionVec3Add::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector3f AValue = A.Get<FNiagaraStatelessExpressionVec3>().EvaluateInternal(Context);
	const FVector3f BValue = B.Get<FNiagaraStatelessExpressionVec3>().EvaluateInternal(Context);
	return AValue + BValue;
}

bool FNiagaraStatelessExpressionVec3Add::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec3>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec3>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec3Subtract::FNiagaraStatelessExpressionVec3Subtract()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec3Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec3Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec3Subtract::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec3Subtract BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector3f FNiagaraStatelessExpressionVec3Subtract::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector3f AValue = A.Get<FNiagaraStatelessExpressionVec3>().EvaluateInternal(Context);
	const FVector3f BValue = B.Get<FNiagaraStatelessExpressionVec3>().EvaluateInternal(Context);
	return AValue - BValue;
}

bool FNiagaraStatelessExpressionVec3Subtract::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec3>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec3>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec3Multiply::FNiagaraStatelessExpressionVec3Multiply()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec3Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec3Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec3Multiply::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec3Multiply BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector3f FNiagaraStatelessExpressionVec3Multiply::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector3f AValue = A.Get<FNiagaraStatelessExpressionVec3>().EvaluateInternal(Context);
	const FVector3f BValue = B.Get<FNiagaraStatelessExpressionVec3>().EvaluateInternal(Context);
	return AValue * BValue;
}

bool FNiagaraStatelessExpressionVec3Multiply::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec3>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec3>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec3Divide::FNiagaraStatelessExpressionVec3Divide()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec3Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec3Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec3Divide::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec3Divide BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector3f FNiagaraStatelessExpressionVec3Divide::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector3f AValue = A.Get<FNiagaraStatelessExpressionVec3>().EvaluateInternal(Context);
	const FVector3f BValue = B.Get<FNiagaraStatelessExpressionVec3>().EvaluateInternal(Context);
	return FVector3f(
		FMath::Abs(BValue.X) > UE_SMALL_NUMBER ? AValue.X / BValue.X : 0.0f,
		FMath::Abs(BValue.Y) > UE_SMALL_NUMBER ? AValue.Y / BValue.Y : 0.0f,
		FMath::Abs(BValue.Z) > UE_SMALL_NUMBER ? AValue.Z / BValue.Z : 0.0f
	);
}

bool FNiagaraStatelessExpressionVec3Divide::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec3>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec3>().IsConstant();
}
