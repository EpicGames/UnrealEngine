// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Expressions/NiagaraStatelessExpressionVec4.h"

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessSetShaderParameterContext.h"
#include "NiagaraParameterStore.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessExpressionVec4)

FInstancedStruct FNiagaraStatelessExpressionVec4::Build(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FInstancedStruct BuiltExpression;
	if (IsConstant())
	{
		FNiagaraStatelessExpressionVec4Constant ConstantExpression;
		FNiagaraParameterStore EmptyParameterStore;
		ConstantExpression.A = EvaluateInternal(FEvaluateContext(EmptyParameterStore));
		return FInstancedStruct::Make(ConstantExpression);
	}
	return BuildInternal(BuildContext);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FVector4f FNiagaraStatelessExpressionVec4Constant::EvaluateInternal(const FEvaluateContext& Context) const
{
	return A;
}

bool FNiagaraStatelessExpressionVec4Constant::IsConstant() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FInstancedStruct FNiagaraStatelessExpressionVec4Binding::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec4Binding BuiltExpression;
	BuiltExpression.A = A;
	BuiltExpression.ParameterOffset = BuildContext.AddRendererBinding(FNiagaraVariableBase(GetOutputTypeDef(), A));
	BuiltExpression.ParameterOffset *= sizeof(uint32);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector4f FNiagaraStatelessExpressionVec4Binding::EvaluateInternal(const FEvaluateContext& Context) const
{
	return Context.ParameterStore.GetParameterValueFromOffset<FVector4f>(ParameterOffset);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec4Add::FNiagaraStatelessExpressionVec4Add()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec4Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec4Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec4Add::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec4Add BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector4f FNiagaraStatelessExpressionVec4Add::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector4f AValue = A.Get<FNiagaraStatelessExpressionVec4>().EvaluateInternal(Context);
	const FVector4f BValue = B.Get<FNiagaraStatelessExpressionVec4>().EvaluateInternal(Context);
	return AValue + BValue;
}

bool FNiagaraStatelessExpressionVec4Add::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec4>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec4>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec4Subtract::FNiagaraStatelessExpressionVec4Subtract()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec4Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec4Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec4Subtract::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec4Subtract BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector4f FNiagaraStatelessExpressionVec4Subtract::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector4f AValue = A.Get<FNiagaraStatelessExpressionVec4>().EvaluateInternal(Context);
	const FVector4f BValue = B.Get<FNiagaraStatelessExpressionVec4>().EvaluateInternal(Context);
	return AValue - BValue;
}

bool FNiagaraStatelessExpressionVec4Subtract::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec4>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec4>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec4Multiply::FNiagaraStatelessExpressionVec4Multiply()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec4Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec4Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec4Multiply::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec4Multiply BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector4f FNiagaraStatelessExpressionVec4Multiply::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector4f AValue = A.Get<FNiagaraStatelessExpressionVec4>().EvaluateInternal(Context);
	const FVector4f BValue = B.Get<FNiagaraStatelessExpressionVec4>().EvaluateInternal(Context);
	return AValue * BValue;
}

bool FNiagaraStatelessExpressionVec4Multiply::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec4>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec4>().IsConstant();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNiagaraStatelessExpressionVec4Divide::FNiagaraStatelessExpressionVec4Divide()
{
	A.InitializeAs<FNiagaraStatelessExpressionVec4Constant>();
	B.InitializeAs<FNiagaraStatelessExpressionVec4Constant>();
}

FInstancedStruct FNiagaraStatelessExpressionVec4Divide::BuildInternal(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	FNiagaraStatelessExpressionVec4Divide BuiltExpression;
	BuiltExpression.A = A.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	BuiltExpression.B = B.Get<FNiagaraStatelessExpression>().Build(BuildContext);
	return FInstancedStruct::Make(BuiltExpression);
}

FVector4f FNiagaraStatelessExpressionVec4Divide::EvaluateInternal(const FEvaluateContext& Context) const
{
	const FVector4f AValue = A.Get<FNiagaraStatelessExpressionVec4>().EvaluateInternal(Context);
	const FVector4f BValue = B.Get<FNiagaraStatelessExpressionVec4>().EvaluateInternal(Context);
	return FVector4f(
		FMath::Abs(BValue.X) > UE_SMALL_NUMBER ? AValue.X / BValue.X : 0.0f,
		FMath::Abs(BValue.Y) > UE_SMALL_NUMBER ? AValue.Y / BValue.Y : 0.0f,
		FMath::Abs(BValue.Z) > UE_SMALL_NUMBER ? AValue.Z / BValue.Z : 0.0f,
		FMath::Abs(BValue.W) > UE_SMALL_NUMBER ? AValue.W / BValue.W : 0.0f
	);
}

bool FNiagaraStatelessExpressionVec4Divide::IsConstant() const
{
	return A.Get<FNiagaraStatelessExpressionVec4>().IsConstant() && B.Get<FNiagaraStatelessExpressionVec4>().IsConstant();
}
