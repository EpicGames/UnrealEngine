// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"

FName FNiagaraStatelessSystemScaleBuildData::GetName()
{
	return StaticStruct()->GetFName();
}

namespace NiagaraStateless
{
	FName FPhysicsBuildData::GetName()
	{
		return FName("FPhysicsBuildData");
	}

	FQuat4f DirectionToQuat(FVector3f Direction)
	{
		Direction = Direction.GetSafeNormal(UE_KINDA_SMALL_NUMBER, FVector3f::ZAxisVector);

		const FVector3f Forward = FVector3f::ZAxisVector;
		const float Dot = FVector3f::DotProduct(Forward, Direction);
		if (Dot < -0.99999f)
		{
			return FQuat4f(FVector3f::YAxisVector, PI);
		}

		const FVector3f Axis = FVector3f::CrossProduct(Forward, Direction);
		const float Angle = FMath::Acos(Dot);
		FQuat4f Quat(Axis, Angle);
		Quat.Normalize();
		return Quat;
	}
}
