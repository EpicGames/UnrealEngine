// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_AccelerationForce.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_AccelerationForce)

void UNiagaraStatelessModule_AccelerationForce::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessRangeVector3 AccelerationRange = AccelerationDistribution.CalculateRange(FVector3f::ZeroVector);

	NiagaraStateless::FPhysicsBuildData& PhysicsBuildData = BuildContext.GetTransientBuildData<NiagaraStateless::FPhysicsBuildData>();
	PhysicsBuildData.AccelerationCoordinateSpace = CoordinateSpace;
	PhysicsBuildData.AccelerationRange.Min += AccelerationRange.Min;
	PhysicsBuildData.AccelerationRange.Max += AccelerationRange.Max;
}
