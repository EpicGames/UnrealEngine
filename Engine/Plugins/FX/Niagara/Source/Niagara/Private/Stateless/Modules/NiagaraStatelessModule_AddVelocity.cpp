// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_AddVelocity.h"
#include "Stateless/NiagaraStatelessDrawDebugContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_AddVelocity)

void UNiagaraStatelessModule_AddVelocity::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	if (!IsModuleEnabled())
	{
		return;
	}

	NiagaraStateless::FPhysicsBuildData& PhysicsBuildData = BuildContext.GetTransientBuildData<NiagaraStateless::FPhysicsBuildData>();
	if (VelocityType == ENSM_VelocityType::Linear)
	{
		const FNiagaraStatelessRangeVector3 VelocityRange = BuildContext.ConvertDistributionToRange(LinearVelocityDistribution, FVector3f::ZeroVector);
		PhysicsBuildData.LinearVelocityCoordinateSpace = CoordinateSpace;
		PhysicsBuildData.LinearVelocityRange = BuildContext.ConvertDistributionToRange(LinearVelocityDistribution, FVector3f::ZeroVector);
		PhysicsBuildData.LinearVelocityScale = BuildContext.ConvertDistributionToRange(LinearVelocityScale, 1.0f);
	}
	else if (VelocityType == ENSM_VelocityType::FromPoint)
	{
		ensureMsgf(PhysicsBuildData.bPointVelocity == false, TEXT("Only a single point force is supported at the moment."));

		PhysicsBuildData.bPointVelocity = true;
		PhysicsBuildData.PointCoordinateSpace = CoordinateSpace;
		PhysicsBuildData.PointVelocityRange = BuildContext.ConvertDistributionToRange(PointVelocityDistribution, 0.0f);
		PhysicsBuildData.PointVelocityScale = BuildContext.ConvertDistributionToRange(PointVelocityScale, 1.0f);
		PhysicsBuildData.PointOrigin = PointOrigin;
	}
	else if (VelocityType == ENSM_VelocityType::InCone)
	{
		ensureMsgf(PhysicsBuildData.bConeVelocity == false, TEXT("Only a single cone force is supported at the moment."));

		PhysicsBuildData.bConeVelocity = true;
		PhysicsBuildData.ConeCoordinateSpace = CoordinateSpace;
		PhysicsBuildData.bUseConeConeRotator = ConeRotationType == ENSM_ConeRotationType::Rotation;
		PhysicsBuildData.ConeRotator = ConeRotationType == ENSM_ConeRotationType::Rotation ? BuildContext.ConvertDistributionToRange(ConeRotation, FRotator3f::ZeroRotator) : FNiagaraStatelessRangeRotator(FRotator3f::ZeroRotator);
		PhysicsBuildData.ConeDirection = ConeRotationType == ENSM_ConeRotationType::Direction ? BuildContext.ConvertDistributionToRange(ConeDirection, FVector3f::ZAxisVector) : FNiagaraStatelessRangeVector3(FVector3f::ZAxisVector);
		PhysicsBuildData.ConeVelocityRange = BuildContext.ConvertDistributionToRange(ConeVelocityDistribution, 0.0f);
		PhysicsBuildData.ConeVelocityScale = BuildContext.ConvertDistributionToRange(ConeVelocityScale, 1.0f);
		PhysicsBuildData.ConeOuterAngle = ConeAngle;
		PhysicsBuildData.ConeInnerAngle = InnerCone;
		PhysicsBuildData.ConeVelocityFalloff = bSpeedFalloffFromConeAxisEnabled ? FMath::Clamp(SpeedFalloffFromConeAxis, 0.0f, 1.0f) : 0.0f;
	}
}

#if WITH_EDITOR
void UNiagaraStatelessModule_AddVelocity::DrawDebug(const FNiagaraStatelessDrawDebugContext& DrawDebugContext) const
{
	const FVector WorldOrigin = DrawDebugContext.TransformPosition(FVector3f::ZeroVector);

	switch (VelocityType)
	{
	case ENSM_VelocityType::Linear:
	{
		const FNiagaraStatelessRangeVector3 LinearVelocityRange = LinearVelocityDistribution.CalculateRange(FVector3f::ZeroVector);
		const FNiagaraStatelessRangeFloat LinearVelocityScaleRange = LinearVelocityScale.CalculateRange();
		const FVector MinDir = FVector(LinearVelocityRange.Min * LinearVelocityScaleRange.Min * LinearVelocityRange.Min);
		const FVector MaxDir = FVector(LinearVelocityRange.Max * LinearVelocityScaleRange.Max * LinearVelocityRange.Max);
		DrawDebugContext.DrawArrow(FVector::ZeroVector, MinDir);

		if (!FMath::IsNearlyEqual(MinDir.Length(), MaxDir.Length()))
		{
			DrawDebugContext.DrawArrow(WorldOrigin, MaxDir);
		}
		break;
	}

	case ENSM_VelocityType::InCone:
	{
		FQuat4f LocalConeQuat;
		if (ConeRotationType == ENSM_ConeRotationType::Rotation)
		{
			if (ConeRotation.IsBinding())
			{
				return;
			}

			LocalConeQuat = ConeRotation.CalculateRange().Min.Quaternion();
		}
		else
		{
			if (ConeDirection.IsBinding())
			{
				return;
			}
			LocalConeQuat = NiagaraStateless::DirectionToQuat(ConeDirection.CalculateRange().Min);
		}

		const FQuat ConeQuat = DrawDebugContext.TransformRotation(LocalConeQuat);
		const float ConeHAngle = ConeAngle / 2.0f;

		TOptional<float> InnerConeHAngle;
		if (InnerCone > 0.0f && !FMath::IsNearlyEqual(ConeAngle, InnerCone))
		{
			InnerConeHAngle = InnerCone / 2.0f;
		}

		const FNiagaraStatelessRangeFloat ConeVelocityScaleRange = ConeVelocityScale.CalculateRange();
		FNiagaraStatelessRangeFloat ConeVelocityRange = ConeVelocityDistribution.CalculateRange(0.0f);
		ConeVelocityRange.Min *= ConeVelocityScaleRange.Min;
		ConeVelocityRange.Max *= ConeVelocityScaleRange.Max;

		DrawDebugContext.DrawCone(WorldOrigin, ConeQuat, ConeHAngle, ConeVelocityRange.Min);
		if (InnerConeHAngle.IsSet())
		{
			DrawDebugContext.DrawCone(WorldOrigin, ConeQuat, InnerConeHAngle.GetValue(), ConeVelocityRange.Min);
		}

		if (!FMath::IsNearlyEqual(ConeVelocityRange.Min, ConeVelocityRange.Max))
		{
			DrawDebugContext.DrawCone(WorldOrigin, ConeQuat, ConeHAngle, ConeVelocityRange.Max);
			if (InnerConeHAngle.IsSet())
			{
				DrawDebugContext.DrawCone(WorldOrigin, ConeQuat, InnerConeHAngle.GetValue(), ConeVelocityRange.Max);
			}
		}
		break;
	}

	case ENSM_VelocityType::FromPoint:
	{
		const FNiagaraStatelessRangeFloat PointVelocityScaleRange = PointVelocityScale.CalculateRange();
		FNiagaraStatelessRangeFloat PointVelocityRange = PointVelocityDistribution.CalculateRange(0.0f);
		PointVelocityRange.Min *= PointVelocityScaleRange.Min;
		PointVelocityRange.Max *= PointVelocityScaleRange.Max;

		if (!FMath::IsNearlyEqual(PointVelocityRange.Min, 0.0f))
			{
				DrawDebugContext.DrawSphere(DrawDebugContext.TransformPosition(PointOrigin), PointVelocityRange.Min);
			}
			if (!FMath::IsNearlyEqual(PointVelocityRange.Min, PointVelocityRange.Max))
			{
				DrawDebugContext.DrawSphere(DrawDebugContext.TransformPosition(PointOrigin), PointVelocityRange.Max);
			}
			break;
		}
	}
}
#endif
