// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_ApplyOwnerScaleToAttributes.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_ApplyOwnerScaleToAttributes)

void UNiagaraStatelessModule_ApplyOwnerScaleToAttributes::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	if (!IsModuleEnabled())
	{
		return;
	}

	const bool bAnyEnabled = 
		SystemScaleData.bScaleCameraOffset ||
		//SystemScaleData.bScaleDrag ||
		SystemScaleData.bScaleForces ||
		SystemScaleData.bScaleInitialVelocity ||
		SystemScaleData.bScaleInitialRibbonWidth ||
		//SystemScaleData.bScaleInitialMeshSize ||
		SystemScaleData.bScaleInitialSpriteSize ||
		SystemScaleData.bScaleInitialDecalSize ||
		SystemScaleData.bScaleInitialLightRadius;

	if (bAnyEnabled == false)
	{
		return;
	}

	FNiagaraStatelessSystemScaleBuildData* BuildData = BuildContext.AllocateSharedBuiltData<FNiagaraStatelessSystemScaleBuildData>();
	*BuildData = SystemScaleData;
}
