// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_CalculateAccurateVelocity.h"

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_CalculateAccurateVelocity)

namespace NSMCalculateAccurateVelocityPrivate
{
	struct FModuleBuiltData
	{
		int32 PositionVariableOffset			= INDEX_NONE;
		int32 PreviousPositionVariableOffset	= INDEX_NONE;
		int32 VelocityVariableOffset			= INDEX_NONE;
		int32 PreviousVelocityVariableOffset	= INDEX_NONE;
	};

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();

		const float InvDeltaTime = ParticleSimulationContext.GetInvDeltaTime();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const FVector3f Position			= ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PositionVariableOffset, i, FVector3f::ZeroVector);
			const FVector3f PreviousPosition	= ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousPositionVariableOffset, i, FVector3f::ZeroVector);
			const FVector3f Velocity			= (Position - PreviousPosition) * InvDeltaTime;

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->VelocityVariableOffset, i, Velocity);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousVelocityVariableOffset, i, Velocity);
		}
	}
}

void UNiagaraStatelessModule_CalculateAccurateVelocity::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMCalculateAccurateVelocityPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->PositionVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.PositionVariable);
	BuiltData->PreviousPositionVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousPositionVariable);
	BuiltData->VelocityVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.VelocityVariable);
	BuiltData->PreviousVelocityVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousVelocityVariable);
	if (BuiltData->VelocityVariableOffset == INDEX_NONE && BuiltData->PreviousVelocityVariableOffset == INDEX_NONE)
	{
		return;
	}

	BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
}

void UNiagaraStatelessModule_CalculateAccurateVelocity::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMCalculateAccurateVelocityPrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_CalculateAccurateVelocity::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_CalculateAccurateVelocity.ush");
}

void UNiagaraStatelessModule_CalculateAccurateVelocity::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.VelocityVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousVelocityVariable);
}
#endif
