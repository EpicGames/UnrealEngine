// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_CameraOffset.h"

#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_CameraOffset)

namespace NSMCameraOffsetPrivate
{
	struct FModuleBuiltData
	{
		FUintVector3	DistributionParameters = FUintVector3::ZeroValue;
		int32			CameraVariableOffset = INDEX_NONE;
		int32			PreviousCameraVariableOffset = INDEX_NONE;
	};

	using FParameters = NiagaraStateless::FCameraOffsetModule_ShaderParameters;

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const FParameters* Parameters = ParticleSimulationContext.ReadParameterNestedStruct<FParameters>();
		const float* NormalizedAgeData = ParticleSimulationContext.GetParticleNormalizedAge();
		const float* PreviousNormalizedAgeData = ParticleSimulationContext.GetParticlePreviousNormalizedAge();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			float CameraOffsets[2];
			ParticleSimulationContext.SampleDistributionValues(Parameters->CameraOffset_Distribution, i, 0, NormalizedAgeData[i], PreviousNormalizedAgeData[i], CameraOffsets[0], CameraOffsets[1]);
			CameraOffsets[0] *= Parameters->CameraOffset_Scale;
			CameraOffsets[1] *= Parameters->CameraOffset_Scale;

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->CameraVariableOffset, i, CameraOffsets[0]);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousCameraVariableOffset, i, CameraOffsets[1]);
		}
	}
}

void UNiagaraStatelessModule_CameraOffset::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMCameraOffsetPrivate;

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();

	FModuleBuiltData* BuiltData				= BuildContext.AllocateBuiltData<FModuleBuiltData>();
	BuiltData->CameraVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.CameraOffsetVariable);
	BuiltData->PreviousCameraVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousCameraOffsetVariable);

	const bool bAttributesUsed = (BuiltData->CameraVariableOffset != INDEX_NONE || BuiltData->PreviousCameraVariableOffset != INDEX_NONE);
	if (IsModuleEnabled() && bAttributesUsed)
	{
		BuiltData->DistributionParameters = BuildContext.AddDistribution(CameraOffsetDistribution);

		BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
	}
}

void UNiagaraStatelessModule_CameraOffset::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMCameraOffsetPrivate;

	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_CameraOffset::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMCameraOffsetPrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	Parameters->CameraOffset_Distribution	= ModuleBuiltData->DistributionParameters;
	Parameters->CameraOffset_Scale			= 1.0f;
	if (const FNiagaraStatelessSystemScaleBuildData* SystemScaleBuildData = SetShaderParameterContext.ReadSharedBuiltData<FNiagaraStatelessSystemScaleBuildData>())
	{
		if (SystemScaleBuildData->bScaleCameraOffset)
		{
			const FNiagaraStatelessSpaceTransforms& SpaceTransforms = SetShaderParameterContext.GetSpaceTransforms();
			const FVector3f ComponentScale = SpaceTransforms.GetLocalToWorld().GetScale3D();

			Parameters->CameraOffset_Scale = FMath::Lerp(1.0f, ComponentScale.GetMax(), SystemScaleBuildData->ScaleCameraOffsetAmount);
		}
	}
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_CameraOffset::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_CameraOffset.ush");
}

void UNiagaraStatelessModule_CameraOffset::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.CameraOffsetVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousCameraOffsetVariable);
}
#endif
