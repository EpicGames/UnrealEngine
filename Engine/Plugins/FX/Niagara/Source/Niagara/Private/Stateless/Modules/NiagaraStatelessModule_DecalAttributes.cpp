// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_DecalAttributes.h"

#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"
#include "Stateless/NiagaraStatelessCommon.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"
#include "NiagaraDecalRendererProperties.h"

#include "NiagaraConstants.h"
#include "NiagaraModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_DecalAttributes)

namespace NSMDecalAttributesPrivate
{
	enum class EModuleAttribute
	{
		Orientation,
		Size,
		Fade,
		Num
	};

	const FNiagaraVariableBase* GetAttributeVariable(EModuleAttribute Attribute, const UNiagaraStatelessModule_DecalAttributes* Module, UNiagaraStatelessModule::EVariableFilter Filter)
	{
		static const FNiagaraVariableBase DecalOrientationVariable = NiagaraStatelessCommon::ConvertParticleVariableToStateless(UNiagaraDecalRendererProperties::GetDecalOrientationVariable());
		static const FNiagaraVariableBase DecalSizeVariable = NiagaraStatelessCommon::ConvertParticleVariableToStateless(UNiagaraDecalRendererProperties::GetDecalSizeVariable());
		static const FNiagaraVariableBase DecalFadeVariable = NiagaraStatelessCommon::ConvertParticleVariableToStateless(UNiagaraDecalRendererProperties::GetDecalFadeVariable());

		const bool bIgnoreFilter = Filter == UNiagaraStatelessModule::EVariableFilter::None;
		switch (Attribute)
		{
			case EModuleAttribute::Orientation:		return bIgnoreFilter || Module->bApplyOrientation ? &DecalOrientationVariable : nullptr;
			case EModuleAttribute::Size:			return bIgnoreFilter || Module->bApplySize ? &DecalSizeVariable : nullptr;
			case EModuleAttribute::Fade:			return bIgnoreFilter || Module->bApplyFade ? &DecalFadeVariable : nullptr;
		}
		return nullptr;
	}

	struct FModuleBuiltData
	{
		FModuleBuiltData()
		{
			for (int i = 0; i < int(EModuleAttribute::Num); ++i)
			{
				AttributeDistributionParameters[i] = FUintVector3::ZeroValue;
				AttributeOffset[i] = INDEX_NONE;
			}
		}

		ENiagaraCoordinateSpace OrientationCoordinateSpace = ENiagaraCoordinateSpace::Local;
		FUintVector3			AttributeDistributionParameters[int(EModuleAttribute::Num)];
		int						AttributeOffset[int(EModuleAttribute::Num)];
	};

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();

		FVector3f SizeScale = FVector3f::OneVector;
		if (ModuleBuiltData->AttributeOffset[int(EModuleAttribute::Size)] != INDEX_NONE)
		{
			const FNiagaraStatelessSystemScaleBuildData* SystemScaleBuildData = ParticleSimulationContext.ReadSharedBuiltData<FNiagaraStatelessSystemScaleBuildData>();
			if (SystemScaleBuildData && SystemScaleBuildData->bScaleInitialDecalSize)
			{
				const FVector3f LocalToWorldScale = ParticleSimulationContext.GetLocalToWorldScale();
				SizeScale = FMath::Lerp(FVector3f::OneVector, LocalToWorldScale, SystemScaleBuildData->ScaleInitialDecalSizeScaleAmount);
			}
		}

		const float* NormalizedAgeData = ParticleSimulationContext.GetParticleNormalizedAge();

		const FQuat4f OrientationRotation = ParticleSimulationContext.GetToSimulationRotation(ModuleBuiltData->OrientationCoordinateSpace);

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			if (ModuleBuiltData->AttributeOffset[int(EModuleAttribute::Orientation)] != INDEX_NONE)
			{
				const FVector3f Rotation = ParticleSimulationContext.SampleDistributionValue<FVector3f>(ModuleBuiltData->AttributeDistributionParameters[int(EModuleAttribute::Orientation)], i, 0, NormalizedAgeData[i]);
				const FQuat4f SourceValue = FRotator3f(Rotation.X, Rotation.Y, Rotation.Z).Quaternion();
				const FQuat4f Value = OrientationRotation * SourceValue;
				ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->AttributeOffset[int(EModuleAttribute::Orientation)], i, Value);
			}
			if (ModuleBuiltData->AttributeOffset[int(EModuleAttribute::Size)] != INDEX_NONE)
			{
				const FVector3f Value = ParticleSimulationContext.SampleDistributionValue<FVector3f>(ModuleBuiltData->AttributeDistributionParameters[int(EModuleAttribute::Size)], i, 0, NormalizedAgeData[i]);
				ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->AttributeOffset[int(EModuleAttribute::Size)], i, Value * SizeScale);
			}
			if (ModuleBuiltData->AttributeOffset[int(EModuleAttribute::Fade)] != INDEX_NONE)
			{
				const float Value = ParticleSimulationContext.SampleDistributionValue<float>(ModuleBuiltData->AttributeDistributionParameters[int(EModuleAttribute::Fade)], i, 0, NormalizedAgeData[i]);
				ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->AttributeOffset[int(EModuleAttribute::Fade)], i, Value);
			}
		}
	}
}

ENiagaraStatelessFeatureMask UNiagaraStatelessModule_DecalAttributes::GetFeatureMask() const
{
	return ENiagaraStatelessFeatureMask::ExecuteCPU;
}

void UNiagaraStatelessModule_DecalAttributes::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMDecalAttributesPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	// Gather our attribute bindings
	{
		bool bAnyValidAttributes = false;
		for (int i = 0; i < int(EModuleAttribute::Num); ++i)
		{
			if (const FNiagaraVariableBase* Variable = GetAttributeVariable(EModuleAttribute(i), this, EVariableFilter::Used))
			{
				BuiltData->AttributeOffset[i] = BuildContext.FindParticleVariableIndex(*Variable);
				bAnyValidAttributes |= BuiltData->AttributeOffset[i] != INDEX_NONE;
			}
		}

		if ( !bAnyValidAttributes )
		{
			return;
		}
	}

	// Build distributions
	if (bApplyOrientation)
	{
		BuiltData->OrientationCoordinateSpace = OrientationCoordinateSpace;
		BuiltData->AttributeDistributionParameters[int(EModuleAttribute::Orientation)] = BuildContext.AddDistribution(Orientation);
	}
	if (bApplySize)
	{
		BuiltData->AttributeDistributionParameters[int(EModuleAttribute::Size)] = BuildContext.AddDistribution(Size);
	}
	if (bApplyFade)
	{
		BuiltData->AttributeDistributionParameters[int(EModuleAttribute::Fade)] = BuildContext.AddDistribution(Fade);
	}

	BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
}

void UNiagaraStatelessModule_DecalAttributes::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMDecalAttributesPrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();
}

#if WITH_EDITORONLY_DATA
void UNiagaraStatelessModule_DecalAttributes::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	using namespace NSMDecalAttributesPrivate;

	for (int i = 0; i < int(EModuleAttribute::Num); ++i)
	{
		if (const FNiagaraVariableBase* Variable = GetAttributeVariable(EModuleAttribute(i), this, Filter))
		{
			OutVariables.AddUnique(*Variable);
		}
	}
}
#endif
