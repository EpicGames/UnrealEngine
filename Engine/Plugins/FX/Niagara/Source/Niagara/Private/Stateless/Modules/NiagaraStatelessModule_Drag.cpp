// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_Drag.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_Drag)

void UNiagaraStatelessModule_Drag::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	if (!IsModuleEnabled())
	{
		return;
	}

	NiagaraStateless::FPhysicsBuildData& PhysicsBuildData = BuildContext.GetTransientBuildData<NiagaraStateless::FPhysicsBuildData>();
	PhysicsBuildData.DragRange = DragDistribution.CalculateRange(DefaultDrag);
}
