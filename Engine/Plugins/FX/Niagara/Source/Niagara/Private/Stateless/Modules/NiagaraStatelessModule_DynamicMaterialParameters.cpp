// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_DynamicMaterialParameters.h"

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_DynamicMaterialParameters)

namespace NSMDynamicMaterialParametersPrivate
{
	static constexpr int32 NumParameters = 4;
	static constexpr int32 NumChannelPerParameter = 4;

	using FParameters = NiagaraStateless::FDynamicMaterialParametersModule_ShaderParameters;

	struct FModuleBuiltData
	{
		FModuleBuiltData()
		{
			for (FUintVector3& v : ParameterDistributions)
			{
				v = FUintVector3::ZeroValue;
			}

			for (int32& ParameterVariableOffset : ParameterVariableOffsets)
			{
				ParameterVariableOffset = INDEX_NONE;
			}
		}

		uint32			ChannelMask = 0;
		FUintVector3	ParameterDistributions[NumParameters * NumChannelPerParameter];
		int32			ParameterVariableOffsets[NumParameters];
	};

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const float* NormalizedAgeData = ParticleSimulationContext.GetParticleNormalizedAge();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			for (int32 iParameter = 0; iParameter < NumParameters; ++iParameter)
			{
				const uint32 ChannelMask = (ModuleBuiltData->ChannelMask >> (iParameter * NumChannelPerParameter)) & 0xf;
				if (ChannelMask == 0)
				{
					continue;
				}

				const int32 FirstChannel = iParameter * NumChannelPerParameter;
				const float NormalizedAge = NormalizedAgeData[i];

				FVector4f DynamicParameter;
				for (int32 iChannel = 0; iChannel < NumChannelPerParameter; ++iChannel)
				{
					DynamicParameter[iChannel] = 0.0f;
					if ((ChannelMask & (1 << iChannel)) != 0 )
					{
						DynamicParameter[iChannel] = ParticleSimulationContext.SampleDistributionValue<float>(ModuleBuiltData->ParameterDistributions[FirstChannel + iChannel], i, FirstChannel + iChannel, NormalizedAge);
					}
				}

				ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->ParameterVariableOffsets[iParameter], i, DynamicParameter);
			}
		}
	}

}

const FNiagaraStatelessDynamicParameterSet& UNiagaraStatelessModule_DynamicMaterialParameters::GetParameterSet(int32 ParameterIndex) const
{
	switch (ParameterIndex)
	{
		case 0: return Parameter0;
		case 1: return Parameter1;
		case 2: return Parameter2;
		case 3: return Parameter3;
		default: checkNoEntry(); return Parameter0;
	}
}

uint32 UNiagaraStatelessModule_DynamicMaterialParameters::GetParameterChannelMask(int32 ParameterIndex) const
{
	uint32 Mask = 0;
	if ((ParameterIndex == 0 && bParameter0Enabled) ||
		(ParameterIndex == 1 && bParameter1Enabled) ||
		(ParameterIndex == 2 && bParameter2Enabled) ||
		(ParameterIndex == 3 && bParameter3Enabled))
	{
		const FNiagaraStatelessDynamicParameterSet& ParameterSet = GetParameterSet(ParameterIndex);
		Mask |= ParameterSet.bXChannelEnabled ? 1 << 0 : 0;
		Mask |= ParameterSet.bYChannelEnabled ? 1 << 1 : 0;
		Mask |= ParameterSet.bZChannelEnabled ? 1 << 2 : 0;
		Mask |= ParameterSet.bWChannelEnabled ? 1 << 3 : 0;
	}
	return Mask;
}

const FNiagaraDistributionFloat& UNiagaraStatelessModule_DynamicMaterialParameters::GetParameterChannelDistribution(int32 ParameterIndex, int32 ChannelIndex) const
{
	const FNiagaraStatelessDynamicParameterSet& ParameterSet = GetParameterSet(ParameterIndex);
	switch (ChannelIndex)
	{
		case 0: return ParameterSet.XChannelDistribution;
		case 1: return ParameterSet.YChannelDistribution;
		case 2: return ParameterSet.ZChannelDistribution;
		case 3: return ParameterSet.WChannelDistribution;
		default: checkNoEntry(); return ParameterSet.XChannelDistribution;
	}
}

const FNiagaraVariableBase& UNiagaraStatelessModule_DynamicMaterialParameters::GetParameterVariable(int32 ParameterIndex) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	switch (ParameterIndex)
	{
		default: checkNoEntry(); return StatelessGlobals.DynamicMaterialParameters0Variable;
		case 0: return StatelessGlobals.DynamicMaterialParameters0Variable;
		case 1: return StatelessGlobals.DynamicMaterialParameters1Variable;
		case 2: return StatelessGlobals.DynamicMaterialParameters2Variable;
		case 3: return StatelessGlobals.DynamicMaterialParameters3Variable;
	}
}


int32 UNiagaraStatelessModule_DynamicMaterialParameters::GetRendererChannelMask() const
{
	using namespace NSMDynamicMaterialParametersPrivate;

	uint32 ChannelMask = 0;
	if (IsModuleEnabled())
	{
		for (int32 iParameter = 0; iParameter < NumParameters; ++iParameter)
		{
			ChannelMask |= GetParameterChannelMask(iParameter) << (iParameter * NumChannelPerParameter);
		}
	}
	return ChannelMask;
}

void UNiagaraStatelessModule_DynamicMaterialParameters::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMDynamicMaterialParametersPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();

	if (IsModuleEnabled())
	{
		for (int32 iParameter=0; iParameter < NumParameters; ++iParameter)
		{
			const uint32 ParameterChannelMask = GetParameterChannelMask(iParameter);
			if (ParameterChannelMask == 0)
			{
				continue;
			}

			BuiltData->ParameterVariableOffsets[iParameter] = BuildContext.FindParticleVariableIndex(GetParameterVariable(iParameter));
			if (BuiltData->ParameterVariableOffsets[iParameter] == INDEX_NONE)
			{
				continue;
			}

			BuiltData->ChannelMask |= ParameterChannelMask << (iParameter * NumChannelPerParameter);
			for (int32 iChannel = 0; iChannel < NumChannelPerParameter; ++iChannel)
			{
				FUintVector3& BuiltDistribution = BuiltData->ParameterDistributions[(iParameter * NumChannelPerParameter) + iChannel];
				if ((ParameterChannelMask & (1 << iChannel)) != 0)
				{
					BuiltDistribution = BuildContext.AddDistribution(GetParameterChannelDistribution(iParameter, iChannel));
				}
				else
				{
					BuiltDistribution = BuildContext.AddDistribution(FNiagaraDistributionFloat(0.0f));
				}
			}
		}

		if (BuiltData->ChannelMask != 0)
		{
			BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
		}
	}
}

void UNiagaraStatelessModule_DynamicMaterialParameters::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMDynamicMaterialParametersPrivate;

	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_DynamicMaterialParameters::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMDynamicMaterialParametersPrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	Parameters->DynamicMaterialParameters_ChannelMask = ModuleBuiltData->ChannelMask;
	Parameters->DynamicMaterialParameters_Parameter0X = ModuleBuiltData->ParameterDistributions[0];
	Parameters->DynamicMaterialParameters_Parameter0Y = ModuleBuiltData->ParameterDistributions[1];
	Parameters->DynamicMaterialParameters_Parameter0Z = ModuleBuiltData->ParameterDistributions[2];
	Parameters->DynamicMaterialParameters_Parameter0W = ModuleBuiltData->ParameterDistributions[3];
	Parameters->DynamicMaterialParameters_Parameter1X = ModuleBuiltData->ParameterDistributions[4];
	Parameters->DynamicMaterialParameters_Parameter1Y = ModuleBuiltData->ParameterDistributions[5];
	Parameters->DynamicMaterialParameters_Parameter1Z = ModuleBuiltData->ParameterDistributions[6];
	Parameters->DynamicMaterialParameters_Parameter1W = ModuleBuiltData->ParameterDistributions[7];
	Parameters->DynamicMaterialParameters_Parameter2X = ModuleBuiltData->ParameterDistributions[8];
	Parameters->DynamicMaterialParameters_Parameter2Y = ModuleBuiltData->ParameterDistributions[9];
	Parameters->DynamicMaterialParameters_Parameter2Z = ModuleBuiltData->ParameterDistributions[10];
	Parameters->DynamicMaterialParameters_Parameter2W = ModuleBuiltData->ParameterDistributions[11];
	Parameters->DynamicMaterialParameters_Parameter3X = ModuleBuiltData->ParameterDistributions[12];
	Parameters->DynamicMaterialParameters_Parameter3Y = ModuleBuiltData->ParameterDistributions[13];
	Parameters->DynamicMaterialParameters_Parameter3Z = ModuleBuiltData->ParameterDistributions[14];
	Parameters->DynamicMaterialParameters_Parameter3W = ModuleBuiltData->ParameterDistributions[15];
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_DynamicMaterialParameters::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_DynamicMaterialParameters.ush");
}

void UNiagaraStatelessModule_DynamicMaterialParameters::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	using namespace NSMDynamicMaterialParametersPrivate;

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();

	for (int32 i = 0; i < NumParameters; ++i)
	{
		if (GetParameterChannelMask(i) != 0 || Filter == EVariableFilter::None)
		{
			OutVariables.AddUnique(GetParameterVariable(i));
		}
	}
}
#endif
