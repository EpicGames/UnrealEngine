// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_GravityForce.h"

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_GravityForce)

void UNiagaraStatelessModule_GravityForce::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	if (!IsModuleEnabled())
	{
		return;
	}
	const FNiagaraStatelessRangeVector3 GravityRange = GravityDistribution.CalculateRange(GetDefaultValue());

	NiagaraStateless::FPhysicsBuildData& PhysicsBuildData = BuildContext.GetTransientBuildData<NiagaraStateless::FPhysicsBuildData>();
	PhysicsBuildData.GravityRange.Min = GravityRange.Min;
	PhysicsBuildData.GravityRange.Max = GravityRange.Max;
}
