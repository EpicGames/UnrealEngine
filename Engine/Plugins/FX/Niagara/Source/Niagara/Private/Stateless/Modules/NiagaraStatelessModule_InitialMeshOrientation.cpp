// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_InitialMeshOrientation.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"
#include "NiagaraCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_InitialMeshOrientation)

namespace NSMInitialMeshOrientationPrivate
{
	struct FModuleBuiltData
	{
		ENSMInitialMeshOrientationMode	Mode = ENSMInitialMeshOrientationMode::None;
		FNiagaraStatelessRangeVector3	OrientationVector;
		FNiagaraStatelessRangeVector3	MeshAxisToOrient;
		FNiagaraStatelessRangeVector3	RotationRange;
		int32							MeshOrientationVariableOffset = INDEX_NONE;
		int32							PreviousMeshOrientationVariableOffset = INDEX_NONE;
	};

	template<typename TExecContext>
	static void ResolveRotationScaleBias(const TExecContext& ExecContext, const FModuleBuiltData* ModuleBuiltData, FVector3f& RotationScale, FVector3f& RotationBias)
	{
		RotationScale = FVector3f::ZeroVector;
		RotationBias = FVector3f::ZeroVector;
		ExecContext.ConvertRangeToScaleBias(ModuleBuiltData->RotationRange, RotationScale, RotationBias);

		if (ModuleBuiltData->Mode == ENSMInitialMeshOrientationMode::OrientToAxis)
		{
			FVector3f FromVector = FVector3f::XAxisVector;
			FVector3f ToVector = FVector3f::XAxisVector;
			FVector3f DummyVector;
			ExecContext.ConvertRangeToScaleBias(ModuleBuiltData->MeshAxisToOrient, DummyVector, FromVector);
			ExecContext.ConvertRangeToScaleBias(ModuleBuiltData->OrientationVector, DummyVector, ToVector);

			FromVector = FromVector.GetSafeNormal(UE_KINDA_SMALL_NUMBER, FVector3f::XAxisVector);
			ToVector = ToVector.GetSafeNormal(UE_KINDA_SMALL_NUMBER, FVector3f::XAxisVector);

			const FQuat4f QuatBetween = FQuat4f::FindBetweenVectors(FromVector, ToVector);
			const FRotator3f Rotator = QuatBetween.Rotator();
			RotationBias.X += Rotator.Roll;
			RotationBias.Y += Rotator.Pitch;
			RotationBias.Z += Rotator.Yaw;
		}
		RotationScale /= 360.0f;
		RotationBias /= 360.0f;
	}

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();

		FVector3f RotationScale;
		FVector3f RotationBias;
		ResolveRotationScaleBias(ParticleSimulationContext, ModuleBuiltData, RotationScale, RotationBias);

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const FVector3f	Rotation = ParticleSimulationContext.RandomScaleBiasFloat(i, 0, RotationScale, RotationBias);
			const FQuat4f Quat = ParticleSimulationContext.RotatorToQuat(Rotation);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->MeshOrientationVariableOffset, i, Quat);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousMeshOrientationVariableOffset, i, Quat);
		}
	}
}

void UNiagaraStatelessModule_InitialMeshOrientation::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMInitialMeshOrientationPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();

	const FNiagaraStatelessGlobals& StatelessGlobals	= FNiagaraStatelessGlobals::Get();
	BuiltData->MeshOrientationVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.MeshOrientationVariable);
	BuiltData->PreviousMeshOrientationVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousMeshOrientationVariable);

	if (IsModuleEnabled())
	{
		BuiltData->Mode = MeshOrientationMode;
		switch (MeshOrientationMode)
		{
			case ENSMInitialMeshOrientationMode::None:
				BuiltData->RotationRange = BuildContext.ConvertDistributionToRange(Rotation, FVector3f::ZeroVector);
				break;

			case ENSMInitialMeshOrientationMode::Random:
				BuiltData->RotationRange.Min = FVector3f::ZeroVector;
				BuiltData->RotationRange.Max = FVector3f(360.f, 360.0f, 360.0f);
				break;

			case ENSMInitialMeshOrientationMode::OrientToAxis:
			{
				BuiltData->RotationRange = BuildContext.ConvertDistributionToRange(Rotation, FVector3f::ZeroVector);
				BuiltData->OrientationVector = BuildContext.ConvertDistributionToRange(OrientationVector, FVector3f::ZeroVector);
				BuiltData->MeshAxisToOrient = BuildContext.ConvertDistributionToRange(MeshAxisToOrient, FVector3f::ZeroVector);
				break;
			}
		}
	}

	if (BuiltData->MeshOrientationVariableOffset != INDEX_NONE || BuiltData->PreviousMeshOrientationVariableOffset != INDEX_NONE)
	{
		BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
	}
}

void UNiagaraStatelessModule_InitialMeshOrientation::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_InitialMeshOrientation::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMInitialMeshOrientationPrivate;
	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	ResolveRotationScaleBias(SetShaderParameterContext, ModuleBuiltData, Parameters->InitialMeshOrientation_RotationScale, Parameters->InitialMeshOrientation_RotationBias);
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_InitialMeshOrientation::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_InitialMeshOrientation.ush");
}

void UNiagaraStatelessModule_InitialMeshOrientation::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.MeshOrientationVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousMeshOrientationVariable);
}

void UNiagaraStatelessModule_InitialMeshOrientation::PostLoad()
{
	Super::PostLoad();

	const int32 NiagaraVersion = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);

	if (NiagaraVersion < FNiagaraCustomVersion::StatelessInitialMeshOrientationV1)
	{
		Rotation.Max += RandomRotationRange_DEPRECATED;
	}
}
#endif
