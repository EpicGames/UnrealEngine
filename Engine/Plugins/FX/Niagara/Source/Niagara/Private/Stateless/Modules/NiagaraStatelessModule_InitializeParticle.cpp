// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_InitializeParticle.h"

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"
#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"

#include "NiagaraParameterBinding.h"
#include "NiagaraParameterStore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_InitializeParticle)

namespace NSMInitializeParticlePrivate
{
	using FParameters = NiagaraStateless::FInitializeParticleModule_ShaderParameters;

	static const uint32 EInitializeParticleModuleFlag_UniformSpriteSize = 1 << 0;
	static const uint32 EInitializeParticleModuleFlag_UniformMeshScale = 1 << 1;

	struct FModuleBuiltData
	{
		uint32							ModuleFlags = 0;
		FNiagaraStatelessRangeFloat		LifetimeRange;
		FUintVector3					InitialPosition = FUintVector3::ZeroValue;
		FUintVector3					InitialColor = FUintVector3::ZeroValue;
		FNiagaraStatelessRangeFloat		MassRange;
		FNiagaraStatelessRangeVector2	SpriteSizeRange;
		FNiagaraStatelessRangeFloat		SpriteRotationRange;
		FNiagaraStatelessRangeVector3	MeshScaleRange;
		FNiagaraStatelessRangeFloat		RibbonWidthRange;

		int32							PositionVariableOffset = INDEX_NONE;
		int32							ColorVariableOffset = INDEX_NONE;
		int32							RibbonWidthVariableOffset = INDEX_NONE;
		int32							SpriteSizeVariableOffset = INDEX_NONE;
		int32							SpriteRotationVariableOffset = INDEX_NONE;
		int32							ScaleVariableOffset = INDEX_NONE;

		int32							PreviousPositionVariableOffset = INDEX_NONE;
		int32							PreviousRibbonWidthVariableOffset = INDEX_NONE;
		int32							PreviousSpriteSizeVariableOffset = INDEX_NONE;
		int32							PreviousSpriteRotationVariableOffset = INDEX_NONE;
		int32							PreviousScaleVariableOffset = INDEX_NONE;
	};

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const FParameters* ShaderParameters = ParticleSimulationContext.ReadParameterNestedStruct<FParameters>();

		const bool bUniformSpriteSize = (ModuleBuiltData->ModuleFlags & EInitializeParticleModuleFlag_UniformSpriteSize) != 0;
		const bool bUniformMeshScale = (ModuleBuiltData->ModuleFlags & EInitializeParticleModuleFlag_UniformMeshScale) != 0;

		const int32* ParticleUniqueIDs = ParticleSimulationContext.GetParticleUniqueID();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const FVector3f		Position = ParticleSimulationContext.SampleDistributionValue<FVector3f>(ModuleBuiltData->InitialPosition, i, 0, float(ParticleUniqueIDs[i]));
			const FLinearColor	Color = ParticleSimulationContext.SampleDistributionValue<FLinearColor>(ModuleBuiltData->InitialColor, i, 1, float(ParticleUniqueIDs[i]));
			const float			RibbonWidth = ParticleSimulationContext.RandomScaleBiasFloat(i, 2, ShaderParameters->InitializeParticle_RibbonWidthScale, ShaderParameters->InitializeParticle_RibbonWidthBias);
			const FVector2f		SpriteSize = ParticleSimulationContext.RandomScaleBiasFloat(i, 3, ShaderParameters->InitializeParticle_SpriteSizeScale, ShaderParameters->InitializeParticle_SpriteSizeBias, bUniformSpriteSize);
			const float			SpriteRot = ParticleSimulationContext.RandomScaleBiasFloat(i, 4, ShaderParameters->InitializeParticle_SpriteRotationScale, ShaderParameters->InitializeParticle_SpriteRotationBias);
			const FVector3f		Scale = ParticleSimulationContext.RandomScaleBiasFloat(i, 5, ShaderParameters->InitializeParticle_MeshScaleScale, ShaderParameters->InitializeParticle_MeshScaleBias, bUniformMeshScale);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PositionVariableOffset, i, Position);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->ColorVariableOffset, i, Color);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->RibbonWidthVariableOffset, i, RibbonWidth);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->SpriteSizeVariableOffset, i, SpriteSize);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->SpriteRotationVariableOffset, i, SpriteRot);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->ScaleVariableOffset, i, Scale);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousPositionVariableOffset, i, Position);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousRibbonWidthVariableOffset, i, RibbonWidth);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousSpriteSizeVariableOffset, i, SpriteSize);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousSpriteRotationVariableOffset, i, SpriteRot);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousScaleVariableOffset, i, Scale);
		}
	}
}

void UNiagaraStatelessModule_InitializeParticle::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMInitializeParticlePrivate;

	BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);

	FModuleBuiltData* BuiltData			= BuildContext.AllocateBuiltData<FModuleBuiltData>();
	BuiltData->ModuleFlags				 = SpriteSizeDistribution.IsUniform() ? EInitializeParticleModuleFlag_UniformSpriteSize : 0;
	BuiltData->ModuleFlags				|= MeshScaleDistribution.IsUniform() ? EInitializeParticleModuleFlag_UniformMeshScale : 0;

	BuiltData->LifetimeRange			= LifetimeDistribution.CalculateRange(FNiagaraStatelessGlobals::GetDefaultLifetimeValue());
	BuiltData->InitialPosition			= BuildContext.AddDistribution(InitialPositionDistribution);
	BuiltData->InitialColor				= BuildContext.AddDistribution(ColorDistribution);
	BuiltData->MassRange				= BuildContext.ConvertDistributionToRange(MassDistribution, FNiagaraStatelessGlobals::GetDefaultMassValue());
	BuiltData->SpriteSizeRange			= BuildContext.ConvertDistributionToRange(SpriteSizeDistribution, FNiagaraStatelessGlobals::GetDefaultSpriteSizeValue());
	BuiltData->SpriteRotationRange		= BuildContext.ConvertDistributionToRange(SpriteRotationDistribution, FNiagaraStatelessGlobals::GetDefaultSpriteRotationValue());
	BuiltData->MeshScaleRange			= BuildContext.ConvertDistributionToRange(MeshScaleDistribution, FNiagaraStatelessGlobals::GetDefaultScaleValue());
	BuiltData->RibbonWidthRange			= BuildContext.ConvertDistributionToRange(RibbonWidthDistribution, FNiagaraStatelessGlobals::GetDefaultRibbonWidthValue());

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->PositionVariableOffset				= BuildContext.FindParticleVariableIndex(StatelessGlobals.PositionVariable);
	BuiltData->ColorVariableOffset					= BuildContext.FindParticleVariableIndex(StatelessGlobals.ColorVariable);
	BuiltData->RibbonWidthVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.RibbonWidthVariable);
	BuiltData->SpriteSizeVariableOffset				= BuildContext.FindParticleVariableIndex(StatelessGlobals.SpriteSizeVariable);
	BuiltData->SpriteRotationVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.SpriteRotationVariable);
	BuiltData->ScaleVariableOffset					= BuildContext.FindParticleVariableIndex(StatelessGlobals.ScaleVariable);
	BuiltData->PreviousPositionVariableOffset		= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousPositionVariable);
	BuiltData->PreviousRibbonWidthVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousRibbonWidthVariable);
	BuiltData->PreviousSpriteSizeVariableOffset		= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousSpriteSizeVariable);
	BuiltData->PreviousSpriteRotationVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousSpriteRotationVariable);
	BuiltData->PreviousScaleVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousScaleVariable);

	NiagaraStateless::FPhysicsBuildData& PhysicsBuildData = BuildContext.GetTransientBuildData<NiagaraStateless::FPhysicsBuildData>();
	PhysicsBuildData.MassRange			= MassDistribution.CalculateRange(FNiagaraStatelessGlobals::GetDefaultMassValue());
}

void UNiagaraStatelessModule_InitializeParticle::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMInitializeParticlePrivate;

	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_InitializeParticle::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMInitializeParticlePrivate;

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	Parameters->InitializeParticle_ModuleFlags			= ModuleBuiltData->ModuleFlags;
	Parameters->InitializeParticle_InitialPosition		= ModuleBuiltData->InitialPosition;
	Parameters->InitializeParticle_InitialColor			= ModuleBuiltData->InitialColor;
	SetShaderParameterContext.ConvertRangeToScaleBias(ModuleBuiltData->SpriteSizeRange,		Parameters->InitializeParticle_SpriteSizeScale, Parameters->InitializeParticle_SpriteSizeBias);
	SetShaderParameterContext.ConvertRangeToScaleBias(ModuleBuiltData->SpriteRotationRange, Parameters->InitializeParticle_SpriteRotationScale, Parameters->InitializeParticle_SpriteRotationBias);
	SetShaderParameterContext.ConvertRangeToScaleBias(ModuleBuiltData->MeshScaleRange,		Parameters->InitializeParticle_MeshScaleScale, Parameters->InitializeParticle_MeshScaleBias);
	SetShaderParameterContext.ConvertRangeToScaleBias(ModuleBuiltData->RibbonWidthRange,	Parameters->InitializeParticle_RibbonWidthScale, Parameters->InitializeParticle_RibbonWidthBias);

	if (const FNiagaraStatelessSystemScaleBuildData* SystemScaleBuildData = SetShaderParameterContext.ReadSharedBuiltData<FNiagaraStatelessSystemScaleBuildData>())
	{
		if (SystemScaleBuildData->bScaleInitialSpriteSize)
		{
			const FVector3f TransformScale = SetShaderParameterContext.GetSpaceTransforms().GetLocalToWorld().GetScale3D();
			float Scale = 0.0f;
			if (SystemScaleBuildData->bScaleInitialSpriteSizeBySingleAxis)
			{
				Scale = TransformScale.Component(int32(SystemScaleBuildData->ScaleInitialSpriteSizeAxis));
			}
			else
			{
				Scale = TransformScale.GetMax();
			}
			Scale = FMath::Lerp(1.0f, Scale, SystemScaleBuildData->ScaleInitialSpriteSizeScaleAmount);

			Parameters->InitializeParticle_SpriteSizeScale *= Scale;
			Parameters->InitializeParticle_SpriteSizeBias *= Scale;
		}

		//if (SystemScaleBuildData->bScaleInitialMeshSize)
		//{
		//	const FVector3f TransformScale = SetShaderParameterContext.GetSpaceTransforms().GetLocalToWorld().GetScale3D();
		//	const FVector3f MeshSpaceScale = SystemScaleBuildData->bScaleInitialMeshSizeAbsoluteScale ? TransformScale.GetAbs() : TransformScale;
		//	const FVector3f FinalScale = FMath::Lerp(FVector3f::OneVector, MeshSpaceScale, SystemScaleBuildData->ScaleInitialMeshSizeScaleAmount);
		//	Parameters->InitializeParticle_MeshScaleScale *= FinalScale;
		//	Parameters->InitializeParticle_MeshScaleBias *= FinalScale;
		//}

		if (SystemScaleBuildData->bScaleInitialRibbonWidth)
		{
			const FVector3f TransformScale = SetShaderParameterContext.GetSpaceTransforms().GetLocalToWorld().GetScale3D();
			float Scale = 0.0f;
			if (SystemScaleBuildData->bScaleInitialRibbonWidthBySingleAxis)
			{
				Scale = TransformScale.Component(int32(SystemScaleBuildData->ScaleInitialRibbonWidthScalingAxis));
			}
			else
			{
				Scale = TransformScale.GetMax();
			}
			Scale = FMath::Lerp(1.0f, Scale, SystemScaleBuildData->ScaleInitialRibbonWidthScaleAmount);

			Parameters->InitializeParticle_RibbonWidthScale *= Scale;
			Parameters->InitializeParticle_RibbonWidthBias *= Scale;
		}
	}
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_InitializeParticle::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_InitializeParticle.ush");
}

void UNiagaraStatelessModule_InitializeParticle::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	using namespace NSMInitializeParticlePrivate;

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.PositionVariable);
	OutVariables.AddUnique(StatelessGlobals.ColorVariable);
	OutVariables.AddUnique(StatelessGlobals.SpriteSizeVariable);
	OutVariables.AddUnique(StatelessGlobals.SpriteRotationVariable);
	OutVariables.AddUnique(StatelessGlobals.ScaleVariable);

	OutVariables.AddUnique(StatelessGlobals.PreviousPositionVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousSpriteSizeVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousSpriteRotationVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousScaleVariable);

	if (bWriteRibbonWidth || Filter == EVariableFilter::None)
	{
		OutVariables.AddUnique(StatelessGlobals.RibbonWidthVariable);
		OutVariables.AddUnique(StatelessGlobals.PreviousRibbonWidthVariable);
	}
}
#endif
