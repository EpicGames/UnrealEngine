// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_LightAttributes.h"

#include "Stateless/Modules/NiagaraStatelessModuleCommon.h"
#include "Stateless/NiagaraStatelessCommon.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include "NiagaraConstants.h"
#include "NiagaraModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_LightAttributes)

namespace NSMLightAttributesPrivate
{
	enum class EModuleAttribute
	{
		Radius,
		Falloff,
		DiffuseScale,
		SpecularScale,
		VolumetricScale,
		Num
	};

	const FNiagaraVariableBase* GetAttributeVariable(EModuleAttribute Attribute, const UNiagaraStatelessModule_LightAttributes* Module, UNiagaraStatelessModule::EVariableFilter Filter)
	{
		static const FNiagaraVariableBase LightRadiusVariable = NiagaraStatelessCommon::ConvertParticleVariableToStateless(SYS_PARAM_PARTICLES_LIGHT_RADIUS);
		static const FNiagaraVariableBase LightFalloffExponent = NiagaraStatelessCommon::ConvertParticleVariableToStateless(SYS_PARAM_PARTICLES_LIGHT_EXPONENT);
		static const FNiagaraVariableBase LightDiffuseScale = NiagaraStatelessCommon::ConvertParticleVariableToStateless(SYS_PARAM_PARTICLES_LIGHT_DIFFUSE_SCALE);
		static const FNiagaraVariableBase LightSpecularScale = NiagaraStatelessCommon::ConvertParticleVariableToStateless(SYS_PARAM_PARTICLES_LIGHT_SPECULAR_SCALE);
		static const FNiagaraVariableBase LightVolumetricScattering = NiagaraStatelessCommon::ConvertParticleVariableToStateless(SYS_PARAM_PARTICLES_LIGHT_VOLUMETRIC_SCATTERING);

		const bool bIgnoreFilter = Filter == UNiagaraStatelessModule::EVariableFilter::None;
		switch (Attribute)
		{
			case EModuleAttribute::Radius:			return bIgnoreFilter || Module->bApplyRadius ? &LightRadiusVariable : nullptr;
			case EModuleAttribute::Falloff:			return bIgnoreFilter || Module->bApplyFalloffExponent ? &LightFalloffExponent : nullptr;
			case EModuleAttribute::DiffuseScale:	return bIgnoreFilter || Module->bApplyDiffuseScale ? &LightDiffuseScale : nullptr;
			case EModuleAttribute::SpecularScale:	return bIgnoreFilter || Module->bApplySpecularScale ? &LightSpecularScale : nullptr;
			case EModuleAttribute::VolumetricScale:	return bIgnoreFilter || Module->bApplyVolumetricScattering ? &LightVolumetricScattering : nullptr;
		}
		return nullptr;
	}

	const FNiagaraDistributionFloat* GetAttributeDistribution(EModuleAttribute Attribute, const UNiagaraStatelessModule_LightAttributes* Module)
	{
		switch (Attribute)
		{
			case EModuleAttribute::Radius:			return Module->bApplyRadius ? &Module->Radius : nullptr;
			case EModuleAttribute::Falloff:			return Module->bApplyFalloffExponent ? &Module->FalloffExponent : nullptr;
			case EModuleAttribute::DiffuseScale:	return Module->bApplyDiffuseScale ? &Module->DiffuseScale : nullptr;
			case EModuleAttribute::SpecularScale:	return Module->bApplySpecularScale ? &Module->SpecularScale : nullptr;
			case EModuleAttribute::VolumetricScale:	return Module->bApplyVolumetricScattering ? &Module->VolumetricScattering : nullptr;
		}
		return nullptr;
	}

	struct FModuleBuiltData
	{
		FModuleBuiltData()
		{
			for (int i = 0; i < int(EModuleAttribute::Num); ++i)
			{
				AttributeDistributionParameters[i] = FUintVector3::ZeroValue;
				AttributeOffset[i] = INDEX_NONE;
			}
		}

		FUintVector3	AttributeDistributionParameters[int(EModuleAttribute::Num)];
		int				AttributeOffset[int(EModuleAttribute::Num)];
	};

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const float* NormalizedAgeData = ParticleSimulationContext.GetParticleNormalizedAge();

		float RadiusScale = 1.0f;
		if (const FNiagaraStatelessSystemScaleBuildData* SystemScaleBuildData = ParticleSimulationContext.ReadSharedBuiltData<FNiagaraStatelessSystemScaleBuildData>())
		{
			if (SystemScaleBuildData->bScaleInitialLightRadius)
			{
				const FVector3f LocalToWorldScale3D = ParticleSimulationContext.GetLocalToWorldScale();
				const float LocalToWorldScale = SystemScaleBuildData->bScaleInitialLightRadiusBySingleAxis ? LocalToWorldScale3D.Component(int32(SystemScaleBuildData->ScaleInitialLightRadiusScalingAxis)) : LocalToWorldScale3D.GetMax();
				RadiusScale = FMath::Lerp(1.0f, LocalToWorldScale, SystemScaleBuildData->ScaleInitialLightRadiusScaleAmount);
			}
		}

		for (uint32 i=0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			for ( int a=0; a < int(EModuleAttribute::Num); ++a )
			{
				if (ModuleBuiltData->AttributeOffset[a] != INDEX_NONE)
				{
					const float Scale = a == int(EModuleAttribute::Radius) ? RadiusScale : 1.0f;
					const float Value = ParticleSimulationContext.SampleDistributionValue<float>(ModuleBuiltData->AttributeDistributionParameters[a], i, 0, NormalizedAgeData[i]);
					ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->AttributeOffset[a], i, Value * Scale);
				}
			}
		}
	}
}

ENiagaraStatelessFeatureMask UNiagaraStatelessModule_LightAttributes::GetFeatureMask() const
{
	return ENiagaraStatelessFeatureMask::ExecuteCPU;
}

void UNiagaraStatelessModule_LightAttributes::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMLightAttributesPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	// Gather our attribute bindings
	{
		bool bAnyValidAttributes = false;
		for (int i = 0; i < int(EModuleAttribute::Num); ++i)
		{
			if (const FNiagaraVariableBase* Variable = GetAttributeVariable(EModuleAttribute(i), this, EVariableFilter::Used))
			{
				BuiltData->AttributeOffset[i] = BuildContext.FindParticleVariableIndex(*Variable);
				bAnyValidAttributes |= BuiltData->AttributeOffset[i] != INDEX_NONE;
			}
		}

		if ( !bAnyValidAttributes )
		{
			return;
		}
	}

	// Build distributions
	for (int i = 0; i < int(EModuleAttribute::Num); ++i)
	{
		const FNiagaraDistributionFloat* Distribution = BuiltData->AttributeOffset[i] != INDEX_NONE ? GetAttributeDistribution(EModuleAttribute(i), this) : nullptr;
		if (Distribution)
		{
			BuiltData->AttributeDistributionParameters[i] = BuildContext.AddDistribution(*Distribution);
		}
	}

	BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
}

void UNiagaraStatelessModule_LightAttributes::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMLightAttributesPrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();
}

#if WITH_EDITORONLY_DATA
void UNiagaraStatelessModule_LightAttributes::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	using namespace NSMLightAttributesPrivate;

	for (int i = 0; i < int(EModuleAttribute::Num); ++i)
	{
		if (const FNiagaraVariableBase* Variable = GetAttributeVariable(EModuleAttribute(i), this, Filter))
		{
			OutVariables.AddUnique(*Variable);
		}
	}
}
#endif
