// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_MeshIndex.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include "WeightedRandomSampler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_MeshIndex)

namespace NSMMeshIndexPrivate
{
	struct FMeshIndexWeightedSampler : public FWeightedRandomSampler
	{
		FMeshIndexWeightedSampler(int32 IndexRange, TConstArrayView<float> IndexWeights)
		{
			TotalWeight = 0.0f;
			PerIndexWeight.Reserve(IndexRange);
			for (int32 i = 0; i < IndexRange; ++i)
			{
				if (IndexWeights.Num() > 0)
				{
					PerIndexWeight.Add(IndexWeights.IsValidIndex(i) ? FMath::Max(IndexWeights[i], 0.0f) : 0.0f);
				}
				else
				{
					PerIndexWeight.Add(1.0f);
				}
				TotalWeight += PerIndexWeight.Last();
			}
		}

		virtual float GetWeights(TArray<float>& OutWeights) override
		{
			OutWeights = PerIndexWeight;
			return TotalWeight;
		}

		TArray<float> PerIndexWeight;
		float TotalWeight = 0.0f;
	};

	struct FModuleBuiltData
	{
		int	Index				= 0;
		int	TableOffset			= 0;
		int	TableNumElements	= 0;

		int MeshIndexOffset		= INDEX_NONE;
	};

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();

		const bool bIsParameter = (ModuleBuiltData->Index & 0x80000000) != 0;
		int32 MeshIndex = ModuleBuiltData->Index & ~0x80000000;;
		if (bIsParameter)
		{
			MeshIndex = ParticleSimulationContext.GetParameterBufferInt(MeshIndex, 0);
		}

		if (ModuleBuiltData->TableNumElements > 0)
		{
			for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
			{
				const FVector2f Rand = ParticleSimulationContext.RandomFloat2(i, 0);
				MeshIndex = FMath::RoundToInt(Rand.X * static_cast<float>(ModuleBuiltData->TableNumElements));
				const float Probability = ParticleSimulationContext.GetStaticFloat<float>(ModuleBuiltData->TableOffset, (MeshIndex * 2) + 0);
				if (Rand.Y > Probability)
				{
					MeshIndex = ParticleSimulationContext.GetStaticFloat<float>(ModuleBuiltData->TableOffset, (MeshIndex * 2) + 1);
				}
				ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->MeshIndexOffset, i, MeshIndex);
			}
		}
		else
		{
			for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
			{
				ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->MeshIndexOffset, i, MeshIndex);
			}
		}
	}
}

void UNiagaraStatelessModule_MeshIndex::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMMeshIndexPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->MeshIndexOffset = BuildContext.FindParticleVariableIndex(StatelessGlobals.MeshIndexVariable);
	if (BuiltData->MeshIndexOffset == INDEX_NONE)
	{
		return;
	}

	const FNiagaraStatelessRangeInt MeshIndexRange = BuildContext.ConvertDistributionToRange(MeshIndex, 0);
	if (MeshIndexRange.ParameterOffset != INDEX_NONE)
	{
		BuiltData->Index = MeshIndexRange.ParameterOffset | 0x80000000;
	}
	else
	{
		BuiltData->Index = MeshIndexRange.Min;
		if (MeshIndexRange.GetScale() > 0 && MeshIndexRange.GetScale() < 256)
		{
			FMeshIndexWeightedSampler Sampler(MeshIndexRange.GetScale() + 1, MeshIndexWeight);
			Sampler.Initialize();

			const int32 NumTableEntries = Sampler.GetNumEntries();
			if (NumTableEntries > 1)
			{
				BuiltData->TableNumElements = NumTableEntries - 1;

				TArray<float, TInlineAllocator<16>> StaticData;
				StaticData.AddUninitialized(NumTableEntries * 2);
				for (int32 i = 0; i < NumTableEntries; ++i)
				{
					StaticData[i * 2 + 0] = Sampler.GetProb()[i];
					StaticData[i * 2 + 1] = float(MeshIndexRange.Min + Sampler.GetAlias()[i]);
				}
				BuiltData->TableOffset = BuildContext.AddStaticData(StaticData);
			}
		}
	}

	BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
}

void UNiagaraStatelessModule_MeshIndex::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_MeshIndex::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMMeshIndexPrivate;

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();
	Parameters->MeshIndex_Index				= ModuleBuiltData->Index;
	Parameters->MeshIndex_TableOffset		= ModuleBuiltData->TableOffset;
	Parameters->MeshIndex_TableNumElements	= ModuleBuiltData->TableNumElements;
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_MeshIndex::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_MeshIndex.ush");
}

void UNiagaraStatelessModule_MeshIndex::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.MeshIndexVariable);
}
#endif
