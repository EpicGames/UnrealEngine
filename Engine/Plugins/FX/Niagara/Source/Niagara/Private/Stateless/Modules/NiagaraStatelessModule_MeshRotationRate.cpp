// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_MeshRotationRate.h"

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_MeshRotationRate)

namespace NSMMeshRotationRatePrivate
{
	using FParameters = NiagaraStateless::FMeshRotationRateModule_ShaderParameters;

	struct FModuleBuiltData
	{
		bool							ModuleEnabled = false;
		FNiagaraStatelessRangeVector3	RotationRange;
		FUintVector3					RateScaleParameters = FUintVector3::ZeroValue;
		int32							MeshOrientationVariableOffset = INDEX_NONE;
		int32							PreviousMeshOrientationVariableOffset = INDEX_NONE;
	};

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const FParameters* ShaderParameters = ParticleSimulationContext.ReadParameterNestedStruct<FParameters>();

		const float* LifetimeData = ParticleSimulationContext.GetParticleLifetime();
		const float* AgeData = ParticleSimulationContext.GetParticleNormalizedAge();
		const float* PreviousAgeData = ParticleSimulationContext.GetParticlePreviousNormalizedAge();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const FVector3f RotationRate = ParticleSimulationContext.RandomScaleBiasFloat(i, 0, ShaderParameters->MeshRotationRate_Scale, ShaderParameters->MeshRotationRate_Bias);
			const FVector3f MultiplyRate = ParticleSimulationContext.SampleCurve<FVector3f>(ShaderParameters->MeshRotationRate_RateScaleParameters, AgeData[i]);
			const FVector3f PreviousMultiplyRate = ParticleSimulationContext.SampleCurve<FVector3f>(ShaderParameters->MeshRotationRate_RateScaleParameters, PreviousAgeData[i]);

			FQuat4f MeshOrientation = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->MeshOrientationVariableOffset, i, FQuat4f::Identity);
			FQuat4f PreviousMeshOrientation = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousMeshOrientationVariableOffset, i, FQuat4f::Identity);

			MeshOrientation *= ParticleSimulationContext.RotatorToQuat(LifetimeData[i] * RotationRate * MultiplyRate);
			PreviousMeshOrientation *= ParticleSimulationContext.RotatorToQuat(LifetimeData[i] * RotationRate * PreviousMultiplyRate);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->MeshOrientationVariableOffset, i, MeshOrientation);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousMeshOrientationVariableOffset, i, PreviousMeshOrientation);
		}
	}
}

void UNiagaraStatelessModule_MeshRotationRate::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMMeshRotationRatePrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessGlobals& StatelessGlobals	= FNiagaraStatelessGlobals::Get();
	BuiltData->MeshOrientationVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.MeshOrientationVariable);
	BuiltData->PreviousMeshOrientationVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousMeshOrientationVariable);

	if (BuiltData->MeshOrientationVariableOffset != INDEX_NONE || BuiltData->PreviousMeshOrientationVariableOffset != INDEX_NONE)
	{
		BuiltData->ModuleEnabled = true;
		BuiltData->RotationRange = BuildContext.ConvertDistributionToRange(RotationRateDistribution, FVector3f::ZeroVector);
		BuiltData->RotationRange.Min *= 1.0f / 360.0f;
		BuiltData->RotationRange.Max *= 1.0f / 360.0f;
		if (bUseRateScale)
		{
			BuiltData->RateScaleParameters = BuildContext.AddDistributionAsCurve(RateScaleDistribution, FVector3f::OneVector);
		}
		else
		{
			BuiltData->RateScaleParameters = BuildContext.AddDistributionAsCurve(FNiagaraDistributionCurveVector3(ENiagaraDistributionCurveLUTMode::Accumulate), FVector3f::OneVector);
		}

		BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
	}
}

void UNiagaraStatelessModule_MeshRotationRate::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMMeshRotationRatePrivate;

	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_MeshRotationRate::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMMeshRotationRatePrivate;

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();
	Parameters->MeshRotationRate_ModuleEnabled = ModuleBuiltData->ModuleEnabled;
	SetShaderParameterContext.ConvertRangeToScaleBias(ModuleBuiltData->RotationRange, Parameters->MeshRotationRate_Scale, Parameters->MeshRotationRate_Bias);
	Parameters->MeshRotationRate_RateScaleParameters = ModuleBuiltData->RateScaleParameters;
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_MeshRotationRate::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_MeshRotationRate.ush");
}

void UNiagaraStatelessModule_MeshRotationRate::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	using namespace NSMMeshRotationRatePrivate;

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.MeshOrientationVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousMeshOrientationVariable);
}
#endif
