// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_RotateAroundPoint.h"

#include "Stateless/NiagaraStatelessDrawDebugContext.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_RotateAroundPoint)

namespace NSMRotateAroundPointPrivate
{
	struct FModuleBuiltData
	{
		FNiagaraStatelessRangeFloat		Rate = FNiagaraStatelessRangeFloat(0.0f);
		FNiagaraStatelessRangeFloat		Radius = FNiagaraStatelessRangeFloat(0.0f);
		FNiagaraStatelessRangeFloat		InitialPhase = FNiagaraStatelessRangeFloat(0.0f);
		ENiagaraCoordinateSpace			CenterCoordinateSpace = ENiagaraCoordinateSpace::Simulation;
		FNiagaraStatelessRangeVector3	Center = FNiagaraStatelessRangeVector3(FVector3f::ZeroVector);
		ENiagaraCoordinateSpace			RotationCoordinateSpace = ENiagaraCoordinateSpace::Simulation;
		FNiagaraStatelessRangeVector3	RotationAxis = FNiagaraStatelessRangeVector3(FVector3f::ZeroVector);

		int32							PositionVariableOffset = INDEX_NONE;
		int32							PreviousPositionVariableOffset = INDEX_NONE;
	};

	using FParameters = NiagaraStateless::FRotateAroundPointModule_ShaderParameters;

	static FVector3f CalculatePosition(float Age, float RotationRate, float RotationRadius, float InitialPhase, const FQuat4f& Rotation)
	{
		const float Time = InitialPhase + Age * RotationRate;
		const FVector3f Offset = FVector3f(FMath::Cos(Time), FMath::Sin(Time), 0.0f) * RotationRadius;
		return Rotation.RotateVector(Offset);
	}

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const FParameters* ShaderParameters = ParticleSimulationContext.ReadParameterNestedStruct<FParameters>();

		const float* ParticleAges = ParticleSimulationContext.GetParticleNormalizedAge();
		const float* ParticlePreviousAges = ParticleSimulationContext.GetParticlePreviousNormalizedAge();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const float RotationRate	= ParticleSimulationContext.RandomScaleBiasFloat(i, 0, ShaderParameters->RotateAroundPoint_RateScale, ShaderParameters->RotateAroundPoint_RateBias);
			const float RotationRadius	= ParticleSimulationContext.RandomScaleBiasFloat(i, 1, ShaderParameters->RotateAroundPoint_RadiusScale, ShaderParameters->RotateAroundPoint_RadiusBias);
			const float InitialPhase	= ParticleSimulationContext.RandomScaleBiasFloat(i, 2, ShaderParameters->RotateAroundPoint_InitialPhaseScale, ShaderParameters->RotateAroundPoint_InitialPhaseBias);
			const FVector3f Center		= ParticleSimulationContext.RandomScaleBiasFloat(i, 3, ShaderParameters->RotateAroundPoint_CenterScale, ShaderParameters->RotateAroundPoint_CenterBias);
			const FQuat4f Rotation		= ParticleSimulationContext.RotatorToQuat(ParticleSimulationContext.RandomScaleBiasFloat(i, 4, ShaderParameters->RotateAroundPoint_RotationAxisScale, ShaderParameters->RotateAroundPoint_RotationAxisBias));

			FVector3f Position = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PositionVariableOffset, i, FVector3f::ZeroVector);
			FVector3f PreviousPosition = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousPositionVariableOffset, i, FVector3f::ZeroVector);

			Position += Center + CalculatePosition(ParticleAges[i], RotationRate, RotationRadius, InitialPhase, Rotation);
			PreviousPosition += Center + CalculatePosition(ParticlePreviousAges[i], RotationRate, RotationRadius, InitialPhase, Rotation);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PositionVariableOffset, i, Position);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousPositionVariableOffset, i, PreviousPosition);
		}
	}
}

void UNiagaraStatelessModule_RotateAroundPoint::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMRotateAroundPointPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->PositionVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.PositionVariable);
	BuiltData->PreviousPositionVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousPositionVariable);
	if (BuiltData->PositionVariableOffset == INDEX_NONE && BuiltData->PreviousPositionVariableOffset == INDEX_NONE)
	{
		return;
	}

	BuiltData->Rate						= Rate.CalculateRange();
	BuiltData->Radius					= Radius.CalculateRange();
	BuiltData->InitialPhase				= InitialPhase.CalculateRange();
	BuiltData->CenterCoordinateSpace	= CenterCoordinateSpace;
	BuiltData->Center					= Center.CalculateRange();
	BuiltData->RotationCoordinateSpace	= RotationCoordinateSpace;
	BuiltData->RotationAxis				= RotationAxis.CalculateRange();

	BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
}

void UNiagaraStatelessModule_RotateAroundPoint::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMRotateAroundPointPrivate;

	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_RotateAroundPoint::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMRotateAroundPointPrivate;

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	const FModuleBuiltData* BuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	Parameters->RotateAroundPoint_RateScale			= FMath::DegreesToRadians(BuiltData->Rate.GetScale());
	Parameters->RotateAroundPoint_RateBias			= FMath::DegreesToRadians(BuiltData->Rate.Min);
	Parameters->RotateAroundPoint_RadiusScale		= BuiltData->Radius.GetScale();
	Parameters->RotateAroundPoint_RadiusBias		= BuiltData->Radius.Min;
	Parameters->RotateAroundPoint_InitialPhaseScale	= BuiltData->InitialPhase.GetScale();
	Parameters->RotateAroundPoint_InitialPhaseBias	= BuiltData->InitialPhase.Min;

	SetShaderParameterContext.TransformPositionRangeToScaleBias(
		BuiltData->Center,
		BuiltData->CenterCoordinateSpace,
		Parameters->RotateAroundPoint_CenterScale,
		Parameters->RotateAroundPoint_CenterBias
	);

	SetShaderParameterContext.TransformRotationRangeToScaleBias(
		BuiltData->RotationAxis,
		BuiltData->RotationCoordinateSpace,
		Parameters->RotateAroundPoint_RotationAxisScale,
		Parameters->RotateAroundPoint_RotationAxisBias
	);
	Parameters->RotateAroundPoint_RotationAxisScale /= 360.0f;
	Parameters->RotateAroundPoint_RotationAxisBias /= 360.0f;
}

#if WITH_EDITOR
void UNiagaraStatelessModule_RotateAroundPoint::DrawDebug(const FNiagaraStatelessDrawDebugContext& DrawDebugContext) const
{
	if (Center.IsBinding() || Radius.IsBinding() || RotationAxis.IsBinding())
	{
		return;
	}

	// Calculate rotation
	const FQuat DrawRotation = DrawDebugContext.TransformRotation(RotationCoordinateSpace, FQuat4f::MakeFromEuler(RotationAxis.Min));

	// Calculate center
	const FNiagaraStatelessRangeVector3 CenterRange = Center.CalculateRange();
	const FVector3f AverageCenter = (CenterRange.Min + CenterRange.Max) * 0.5f;
	const FVector DrawPosition = DrawDebugContext.TransformPosition(CenterCoordinateSpace, AverageCenter);

	// Calculate radius
	const FNiagaraStatelessRangeFloat RadiusRange = Radius.CalculateRange();
	DrawDebugContext.DrawAxis(DrawPosition, DrawRotation, RadiusRange.Min * 0.5f);
	DrawDebugContext.DrawRing(DrawPosition, DrawRotation, RadiusRange.Min, RadiusRange.Max, 0.0f);
}
#endif

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_RotateAroundPoint::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_RotateAroundPoint.ush");
}

void UNiagaraStatelessModule_RotateAroundPoint::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	using namespace NSMRotateAroundPointPrivate;

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.PositionVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousPositionVariable);
}
#endif
