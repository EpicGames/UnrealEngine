// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_ScaleColor.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_ScaleColor)

namespace NSMScaleColorPrivate
{
	struct FModuleBuiltData
	{
		FUintVector3	DistributionParameters = FUintVector3::ZeroValue;
		int32			ColorVariableOffset = INDEX_NONE;
	};

	using FParameters = NiagaraStateless::FScaleColorModule_ShaderParameters;

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const float* NormalizedAgeData = ParticleSimulationContext.GetParticleNormalizedAge();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			FLinearColor Color = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->ColorVariableOffset, i, FLinearColor::White);

			Color *= ParticleSimulationContext.SampleDistributionValue<FLinearColor>(ModuleBuiltData->DistributionParameters, i, 0, NormalizedAgeData[i]);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->ColorVariableOffset, i, Color);
		}
	}
}

void UNiagaraStatelessModule_ScaleColor::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMScaleColorPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->ColorVariableOffset = BuildContext.FindParticleVariableIndex(StatelessGlobals.ColorVariable);

	if (BuiltData->ColorVariableOffset != INDEX_NONE)
	{
		BuiltData->DistributionParameters = BuildContext.AddDistribution(ScaleDistribution);

		BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
	}
}

void UNiagaraStatelessModule_ScaleColor::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMScaleColorPrivate;

	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_ScaleColor::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMScaleColorPrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	Parameters->ScaleColor_Distribution = ModuleBuiltData->DistributionParameters;
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_ScaleColor::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleColor.ush");
}

void UNiagaraStatelessModule_ScaleColor::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.ColorVariable);
}
#endif
