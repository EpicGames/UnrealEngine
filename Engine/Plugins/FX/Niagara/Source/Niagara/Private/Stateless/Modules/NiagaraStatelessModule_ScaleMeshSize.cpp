// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_ScaleMeshSize.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_ScaleMeshSize)

namespace NSMScaleMeshSizePrivate
{
	struct FModuleBuiltData
	{
		FUintVector3	DistributionParameters = FUintVector3::ZeroValue;
		FVector3f		CurveScale = FVector3f::OneVector;
		int32			CurveScaleOffset = INDEX_NONE;

		int32			ScaleVariableOffset = INDEX_NONE;
		int32			PreviousScaleVariableOffset = INDEX_NONE;
	};

	using FParameters = NiagaraStateless::FScaleMeshSizeModule_ShaderParameters;

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const float* NormalizedAgeData = ParticleSimulationContext.GetParticleNormalizedAge();
		const float* PreviousNormalizedAgeData = ParticleSimulationContext.GetParticlePreviousNormalizedAge();

		const FVector3f CurveScale = ParticleSimulationContext.GetParameterBufferFloatDefaulted(ModuleBuiltData->CurveScaleOffset, ModuleBuiltData->CurveScale);

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const FVector3f MeshScale = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->ScaleVariableOffset, i, FVector3f::OneVector);
			const FVector3f PrevMeshScale = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousScaleVariableOffset, i, FVector3f::OneVector);

			FVector3f Scales[2];
			ParticleSimulationContext.SampleDistributionValues(ModuleBuiltData->DistributionParameters, i, 0, NormalizedAgeData[i], PreviousNormalizedAgeData[i], Scales[0], Scales[1]);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->ScaleVariableOffset, i, Scales[0] * CurveScale * MeshScale);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousScaleVariableOffset, i, Scales[1] * CurveScale * PrevMeshScale);
		}
	}
}

void UNiagaraStatelessModule_ScaleMeshSize::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		ScaleCurveRange.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		ScaleCurveRange.SetAllowedTypeDefinitions({ FNiagaraTypeDefinition::GetVec3Def() });
		ScaleCurveRange.SetDefaultParameter(FNiagaraTypeDefinition::GetVec3Def(), FVector3f::OneVector);
	}
#endif
}

void UNiagaraStatelessModule_ScaleMeshSize::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMScaleMeshSizePrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->ScaleVariableOffset = BuildContext.FindParticleVariableIndex(StatelessGlobals.ScaleVariable);
	BuiltData->PreviousScaleVariableOffset = BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousScaleVariable);

	if (BuiltData->ScaleVariableOffset == INDEX_NONE && BuiltData->PreviousScaleVariableOffset == INDEX_NONE)
	{
		return;
	}

	BuiltData->DistributionParameters = BuildContext.AddDistribution(ScaleDistribution);
	if (UseScaleCurveRange())
	{
		BuiltData->CurveScaleOffset = BuildContext.AddRendererBinding(ScaleCurveRange.ResolvedParameter);
		BuiltData->CurveScale = ScaleCurveRange.GetDefaultValue<FVector3f>();
	}
	BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
}

void UNiagaraStatelessModule_ScaleMeshSize::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMScaleMeshSizePrivate;

	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_ScaleMeshSize::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMScaleMeshSizePrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	FParameters* Parameters						= SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	Parameters->ScaleMeshSize_Distribution		= ModuleBuiltData->DistributionParameters;
	Parameters->ScaleMeshSize_CurveScale		= ModuleBuiltData->CurveScale;
	Parameters->ScaleMeshSize_CurveScaleOffset	= ModuleBuiltData->CurveScaleOffset;
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_ScaleMeshSize::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleMeshSize.ush");
}

void UNiagaraStatelessModule_ScaleMeshSize::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.ScaleVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousScaleVariable);
}
#endif
