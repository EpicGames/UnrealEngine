// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_ScaleMeshSizeBySpeed.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"
#include "Curves/RichCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_ScaleMeshSizeBySpeed)

namespace NSMScaleMeshSizeBySpeedPrivate
{
	struct FModuleBuiltData
	{
		FNiagaraStatelessRangeVector3	MinScaleFactor = FNiagaraStatelessRangeVector3(FVector3f::OneVector);
		FNiagaraStatelessRangeVector3	MaxScaleFactor = FNiagaraStatelessRangeVector3(FVector3f::OneVector);
		FNiagaraStatelessRangeFloat		VelocityNorm = FNiagaraStatelessRangeFloat(0.0f);
		FUintVector3					ScaleDistribution = FUintVector3::ZeroValue;

		int32							PositionVariableOffset = INDEX_NONE;
		int32							PreviousPositionVariableOffset = INDEX_NONE;
		int32							VelocityVariableOffset = INDEX_NONE;
		int32							ScaleVariableVariableOffset = INDEX_NONE;
		int32							PreviousScaleVariableVariableOffset = INDEX_NONE;
	};


	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const UNiagaraStatelessModule_ScaleMeshSizeBySpeed::FParameters* Parameters = ParticleSimulationContext.ReadParameterNestedStruct<UNiagaraStatelessModule_ScaleMeshSizeBySpeed::FParameters>();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const FVector3f Position = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PositionVariableOffset, i, FVector3f::ZeroVector);
			FVector3f Velocity;
			if (ModuleBuiltData->PreviousPositionVariableOffset != INDEX_NONE)
			{
				const FVector3f PreviousPosition = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousPositionVariableOffset, i, Position);
				Velocity = (Position - PreviousPosition) * ParticleSimulationContext.GetInvDeltaTime();
			}
			else
			{
				Velocity = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->VelocityVariableOffset, i, FVector3f::ZeroVector);
			}
			const float Speed = Velocity.SquaredLength();
			const float NormSpeed = FMath::Clamp(Speed * Parameters->ScaleMeshSizeBySpeed_VelocityNorm, 0.0f, 1.0f);
			const float Interp = ParticleSimulationContext.SampleDistributionValue<float>(Parameters->ScaleMeshSizeBySpeed_ScaleDistribution, i, 0, NormSpeed);
			const FVector3f	Scale = Parameters->ScaleMeshSizeBySpeed_ScaleFactorBias + (Parameters->ScaleMeshSizeBySpeed_ScaleFactorScale * FMath::Clamp(Interp, 0.0f, 1.0f));

			FVector3f ParticleScale = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->ScaleVariableVariableOffset, i, FVector3f::OneVector);
			FVector3f ParticlePreviousScale = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousScaleVariableVariableOffset, i, FVector3f::OneVector);

			ParticleScale *= Scale;
			ParticlePreviousScale *= Scale;

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->ScaleVariableVariableOffset, i, ParticleScale);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousScaleVariableVariableOffset, i, ParticlePreviousScale);
		}
	}
}

void UNiagaraStatelessModule_ScaleMeshSizeBySpeed::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMScaleMeshSizeBySpeedPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->PositionVariableOffset				= BuildContext.FindParticleVariableIndex(StatelessGlobals.PositionVariable);
	BuiltData->PreviousPositionVariableOffset		= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousPositionVariable);
	BuiltData->VelocityVariableOffset				= BuildContext.FindParticleVariableIndex(StatelessGlobals.VelocityVariable);
	BuiltData->ScaleVariableVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.ScaleVariable);
	BuiltData->PreviousScaleVariableVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousScaleVariable);

	if (BuiltData->ScaleVariableVariableOffset == INDEX_NONE && BuiltData->PreviousScaleVariableVariableOffset == INDEX_NONE)
	{
		return;
	}

	BuiltData->VelocityNorm = BuildContext.ConvertDistributionToRange(VelocityThreshold, DefaultVelocity);
	BuiltData->MinScaleFactor = BuildContext.ConvertDistributionToRange(MinScaleFactor, FVector3f::OneVector);
	BuiltData->MaxScaleFactor = BuildContext.ConvertDistributionToRange(MaxScaleFactor, FVector3f::OneVector);
	if ( bSampleScaleFactorByCurve )
	{
		BuiltData->ScaleDistribution = BuildContext.AddDistribution(SampleFactorCurve);
	}
	else
	{
		const FNiagaraDistributionFloat DefaultDistribution({ 0.0f, 1.0f });
		BuiltData->ScaleDistribution = BuildContext.AddDistribution(DefaultDistribution);
	}
	
	if (BuiltData->PreviousPositionVariableOffset != INDEX_NONE || BuiltData->VelocityVariableOffset != INDEX_NONE)
	{
		BuildContext.AddParticleSimulationExecSimulate(&NSMScaleMeshSizeBySpeedPrivate::ParticleSimulate);
	}
}

void UNiagaraStatelessModule_ScaleMeshSizeBySpeed::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_ScaleMeshSizeBySpeed::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMScaleMeshSizeBySpeedPrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	const float VelocityNorm	= SetShaderParameterContext.ConvertRangeToValue(ModuleBuiltData->VelocityNorm);
	const FVector3f MinScale	= SetShaderParameterContext.ConvertRangeToValue(ModuleBuiltData->MinScaleFactor);
	const FVector3f MaxScale	= SetShaderParameterContext.ConvertRangeToValue(ModuleBuiltData->MaxScaleFactor);

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	Parameters->ScaleMeshSizeBySpeed_ScaleFactorBias	= MinScale;
	Parameters->ScaleMeshSizeBySpeed_ScaleFactorScale	= MaxScale - MinScale;
	Parameters->ScaleMeshSizeBySpeed_VelocityNorm		= VelocityNorm > 0.0f ? 1.0f / (VelocityNorm * VelocityNorm) : 0.0f;
	Parameters->ScaleMeshSizeBySpeed_ScaleDistribution	= ModuleBuiltData->ScaleDistribution;
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_ScaleMeshSizeBySpeed::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleMeshSizeBySpeed.ush");
}

void UNiagaraStatelessModule_ScaleMeshSizeBySpeed::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.ScaleVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousScaleVariable);
}
#endif
