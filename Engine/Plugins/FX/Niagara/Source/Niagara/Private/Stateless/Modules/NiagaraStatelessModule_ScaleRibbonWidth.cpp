// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_ScaleRibbonWidth.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_ScaleRibbonWidth)

namespace NSMScaleRibbonWidthPrivate
{
	struct FModuleBuiltData
	{
		FUintVector3	DistributionParameters = FUintVector3::ZeroValue;
		float			CurveScale = 1.0f;
		int32			CurveScaleOffset = INDEX_NONE;

		int32			RibbonWidthVariableOffset = INDEX_NONE;
		int32			PreviousRibbonWidthVariableOffset = INDEX_NONE;
	};

	using FParameters = NiagaraStateless::FScaleRibbonWidthModule_ShaderParameters;

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();

		const float ScaleFactor = ParticleSimulationContext.GetParameterBufferFloatDefaulted(ModuleBuiltData->CurveScaleOffset, ModuleBuiltData->CurveScale);
		const float* NormalizedAgeData = ParticleSimulationContext.GetParticleNormalizedAge();
		const float* PreviousNormalizedAgeData = ParticleSimulationContext.GetParticlePreviousNormalizedAge();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const float RibbonWidth = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->RibbonWidthVariableOffset, i, 0.0f);
			const float PreviousRibbonWidth = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousRibbonWidthVariableOffset, i, 0.0f);

			float Scales[2];
			ParticleSimulationContext.SampleDistributionValues(ModuleBuiltData->DistributionParameters, i, 0, NormalizedAgeData[i], PreviousNormalizedAgeData[i], Scales[0], Scales[1]);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->RibbonWidthVariableOffset, i, RibbonWidth * ScaleFactor * Scales[0]);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousRibbonWidthVariableOffset, i, PreviousRibbonWidth * ScaleFactor * Scales[1]);
		}
	}
}

void UNiagaraStatelessModule_ScaleRibbonWidth::PostInitProperties()
{
	using namespace NSMScaleRibbonWidthPrivate;

	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		ScaleCurveRange.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		ScaleCurveRange.SetAllowedTypeDefinitions({ FNiagaraTypeDefinition::GetVec2Def() });
		ScaleCurveRange.SetDefaultParameter(FNiagaraTypeDefinition::GetFloatDef(), 1.0f);
	}
#endif
}

void UNiagaraStatelessModule_ScaleRibbonWidth::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMScaleRibbonWidthPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->RibbonWidthVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.RibbonWidthVariable);
	BuiltData->PreviousRibbonWidthVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousRibbonWidthVariable);

	const bool bAttributesUsed = (BuiltData->RibbonWidthVariableOffset != INDEX_NONE || BuiltData->PreviousRibbonWidthVariableOffset != INDEX_NONE);
	if (IsModuleEnabled() && bAttributesUsed)
	{
		BuiltData->DistributionParameters = BuildContext.AddDistribution(ScaleDistribution);
		if (UseScaleCurveRange())
		{
			BuiltData->CurveScaleOffset = BuildContext.AddRendererBinding(ScaleCurveRange.ResolvedParameter);
			BuiltData->CurveScale = ScaleCurveRange.GetDefaultValue<float>();
		}

		BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
	}
}

void UNiagaraStatelessModule_ScaleRibbonWidth::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMScaleRibbonWidthPrivate;

	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_ScaleRibbonWidth::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMScaleRibbonWidthPrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	FParameters* Parameters						= SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	Parameters->ScaleRibbonWidth_Distribution	= ModuleBuiltData->DistributionParameters;
	Parameters->ScaleRibbonWidth_CurveScale		= SetShaderParameterContext.GetRendererParameterValue(ModuleBuiltData->CurveScaleOffset, ModuleBuiltData->CurveScale);
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_ScaleRibbonWidth::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleRibbonWidth.ush");
}

void UNiagaraStatelessModule_ScaleRibbonWidth::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	using namespace NSMScaleRibbonWidthPrivate;

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.RibbonWidthVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousRibbonWidthVariable);
}
#endif
