// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_ScaleSpriteSize.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"
#include "NiagaraParameterBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_ScaleSpriteSize)

namespace NSMScaleSpriteSizePrivate
{
	struct FModuleBuiltData
	{
		FUintVector3	DistributionParameters = FUintVector3::ZeroValue;
		FVector2f		CurveScale = FVector2f::One();
		int32			CurveScaleOffset = INDEX_NONE;

		int32			SpriteSizeVariableOffset = INDEX_NONE;
		int32			PreviousSpriteSizeVariableOffset = INDEX_NONE;
	};

	using FParameters = NiagaraStateless::FScaleSpriteSizeModule_ShaderParameters;

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();

		const FVector2f ScaleFactor = ParticleSimulationContext.GetParameterBufferFloatDefaulted(ModuleBuiltData->CurveScaleOffset, ModuleBuiltData->CurveScale);
		const float* NormalizedAgeData = ParticleSimulationContext.GetParticleNormalizedAge();
		const float* PreviousNormalizedAgeData = ParticleSimulationContext.GetParticlePreviousNormalizedAge();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const FVector2f SpriteSize = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->SpriteSizeVariableOffset, i, FVector2f::ZeroVector);
			const FVector2f PreviousSpriteSize = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousSpriteSizeVariableOffset, i, FVector2f::ZeroVector);

			FVector2f Scales[2];
			ParticleSimulationContext.SampleDistributionValues(ModuleBuiltData->DistributionParameters, i, 0, NormalizedAgeData[i], PreviousNormalizedAgeData[i], Scales[0], Scales[1]);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->SpriteSizeVariableOffset, i, SpriteSize * Scales[0]);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousSpriteSizeVariableOffset, i, PreviousSpriteSize * Scales[1]);
		}
	}
}

void UNiagaraStatelessModule_ScaleSpriteSize::PostInitProperties()
{
	using namespace NSMScaleSpriteSizePrivate;

	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		ScaleCurveRange.SetUsage(ENiagaraParameterBindingUsage::NotParticle);
		ScaleCurveRange.SetAllowedTypeDefinitions({ FNiagaraTypeDefinition::GetVec2Def() });
		ScaleCurveRange.SetDefaultParameter(FNiagaraTypeDefinition::GetVec2Def(), FVector2f::One());
	}
#endif
}

void UNiagaraStatelessModule_ScaleSpriteSize::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMScaleSpriteSizePrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->SpriteSizeVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.SpriteSizeVariable);
	BuiltData->PreviousSpriteSizeVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousSpriteSizeVariable);

	const bool bAttributesUsed = (BuiltData->SpriteSizeVariableOffset != INDEX_NONE || BuiltData->PreviousSpriteSizeVariableOffset != INDEX_NONE);
	if (IsModuleEnabled() && bAttributesUsed)
	{
		BuiltData->DistributionParameters = BuildContext.AddDistribution(ScaleDistribution);
		if (UseScaleCurveRange())
		{
			BuiltData->CurveScaleOffset = BuildContext.AddRendererBinding(ScaleCurveRange.ResolvedParameter);
			BuiltData->CurveScale = ScaleCurveRange.GetDefaultValue<FVector2f>();
		}

		BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
	}
}

void UNiagaraStatelessModule_ScaleSpriteSize::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMScaleSpriteSizePrivate;

	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_ScaleSpriteSize::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMScaleSpriteSizePrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	FParameters* Parameters						= SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	Parameters->ScaleSpriteSize_Distribution	= ModuleBuiltData->DistributionParameters;
	Parameters->ScaleSpriteSize_CurveScale		= SetShaderParameterContext.GetRendererParameterValue(ModuleBuiltData->CurveScaleOffset, ModuleBuiltData->CurveScale);
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_ScaleSpriteSize::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleSpriteSize.ush");
}

void UNiagaraStatelessModule_ScaleSpriteSize::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	using namespace NSMScaleSpriteSizePrivate;

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.SpriteSizeVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousSpriteSizeVariable);
}
#endif
