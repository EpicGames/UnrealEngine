// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_ScaleSpriteSizeBySpeed.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_ScaleSpriteSizeBySpeed)

namespace NSMScaleSpriteSizeBySpeedPrivate
{
	struct FModuleBuiltData
	{
		FNiagaraStatelessRangeVector2	MinScaleFactor = FNiagaraStatelessRangeVector2(FVector2f::One());
		FNiagaraStatelessRangeVector2	MaxScaleFactor = FNiagaraStatelessRangeVector2(FVector2f::One());
		FNiagaraStatelessRangeFloat		VelocityNorm = FNiagaraStatelessRangeFloat(0.0f);
		FUintVector3					ScaleDistribution = FUintVector3::ZeroValue;

		int32							PositionVariableOffset = INDEX_NONE;
		int32							PreviousPositionVariableOffset = INDEX_NONE;
		int32							VelocityVariableOffset = INDEX_NONE;
		int32							SpriteSizeVariableOffset = INDEX_NONE;
		int32							PreviousSpriteSizeVariableOffset = INDEX_NONE;
	};

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const UNiagaraStatelessModule_ScaleSpriteSizeBySpeed::FParameters* Parameters = ParticleSimulationContext.ReadParameterNestedStruct<UNiagaraStatelessModule_ScaleSpriteSizeBySpeed::FParameters>();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const FVector3f Position			= ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PositionVariableOffset, i, FVector3f::ZeroVector);
			FVector3f Velocity;
			if (ModuleBuiltData->PreviousPositionVariableOffset != INDEX_NONE)
			{
				const FVector3f PreviousPosition = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousPositionVariableOffset, i, Position);
				Velocity = (Position - PreviousPosition) * ParticleSimulationContext.GetInvDeltaTime();
			}
			else
			{
				Velocity = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->VelocityVariableOffset, i, FVector3f::ZeroVector);
			}
			const float Speed					= Velocity.SquaredLength();
			const float NormSpeed				= FMath::Clamp(Speed * Parameters->ScaleSpriteSizeBySpeed_VelocityNorm, 0.0f, 1.0f);
			const float Interp					= ParticleSimulationContext.SampleDistributionValue<float>(Parameters->ScaleSpriteSizeBySpeed_ScaleDistribution, i, 0, NormSpeed);
			const FVector2f	Scale				= Parameters->ScaleSpriteSizeBySpeed_ScaleFactorBias + (Parameters->ScaleSpriteSizeBySpeed_ScaleFactorScale * FMath::Clamp(Interp, 0.0f, 1.0f));

			FVector2f SpriteSize			= ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->SpriteSizeVariableOffset, i, FVector2f::One());
			FVector2f PreviousSpriteSize	= ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousSpriteSizeVariableOffset, i, FVector2f::One());

			SpriteSize			*= Scale;
			PreviousSpriteSize	*= Scale;

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->SpriteSizeVariableOffset, i, SpriteSize);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousSpriteSizeVariableOffset, i, PreviousSpriteSize);
		}
	}
}

void UNiagaraStatelessModule_ScaleSpriteSizeBySpeed::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMScaleSpriteSizeBySpeedPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->PositionVariableOffset = BuildContext.FindParticleVariableIndex(StatelessGlobals.PositionVariable);
	BuiltData->PreviousPositionVariableOffset = BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousPositionVariable);
	BuiltData->VelocityVariableOffset = BuildContext.FindParticleVariableIndex(StatelessGlobals.VelocityVariable);
	BuiltData->SpriteSizeVariableOffset = BuildContext.FindParticleVariableIndex(StatelessGlobals.SpriteSizeVariable);
	BuiltData->PreviousSpriteSizeVariableOffset = BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousSpriteSizeVariable);

	if (BuiltData->SpriteSizeVariableOffset == INDEX_NONE && BuiltData->PreviousSpriteSizeVariableOffset == INDEX_NONE)
	{
		return;
	}

	BuiltData->VelocityNorm = BuildContext.ConvertDistributionToRange(VelocityThreshold, DefaultVelocity);
	BuiltData->MinScaleFactor = BuildContext.ConvertDistributionToRange(MinScaleFactor, FVector2f::One());
	BuiltData->MaxScaleFactor = BuildContext.ConvertDistributionToRange(MaxScaleFactor, FVector2f::One());
	if (bSampleScaleFactorByCurve)
	{
		BuiltData->ScaleDistribution = BuildContext.AddDistribution(SampleFactorCurve);
	}
	else
	{
		const FNiagaraDistributionFloat DefaultDistribution({ 0.0f, 1.0f });
		BuiltData->ScaleDistribution = BuildContext.AddDistribution(DefaultDistribution);
	}

	if (BuiltData->PreviousPositionVariableOffset != INDEX_NONE || BuiltData->VelocityVariableOffset != INDEX_NONE)
	{
		BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
	}
}

void UNiagaraStatelessModule_ScaleSpriteSizeBySpeed::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_ScaleSpriteSizeBySpeed::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMScaleSpriteSizeBySpeedPrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	const float VelocityNorm = SetShaderParameterContext.ConvertRangeToValue(ModuleBuiltData->VelocityNorm);
	const FVector2f MinScale = SetShaderParameterContext.ConvertRangeToValue(ModuleBuiltData->MinScaleFactor);
	const FVector2f MaxScale = SetShaderParameterContext.ConvertRangeToValue(ModuleBuiltData->MaxScaleFactor);

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	Parameters->ScaleSpriteSizeBySpeed_ScaleFactorBias = MinScale;
	Parameters->ScaleSpriteSizeBySpeed_ScaleFactorScale = MaxScale - MinScale;
	Parameters->ScaleSpriteSizeBySpeed_VelocityNorm = VelocityNorm > 0.0f ? 1.0f / (VelocityNorm * VelocityNorm) : 0.0f;
	Parameters->ScaleSpriteSizeBySpeed_ScaleDistribution = ModuleBuiltData->ScaleDistribution;
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_ScaleSpriteSizeBySpeed::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ScaleSpriteSizeBySpeed.ush");
}

void UNiagaraStatelessModule_ScaleSpriteSizeBySpeed::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.SpriteSizeVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousSpriteSizeVariable);
}
#endif
