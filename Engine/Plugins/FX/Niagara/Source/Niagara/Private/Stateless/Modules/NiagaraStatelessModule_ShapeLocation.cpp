// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_ShapeLocation.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"
#include "Stateless/NiagaraStatelessDrawDebugContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_ShapeLocation)

namespace NSMShapeLocationPrivate
{
	struct FModuleBuiltData
	{
		ENSM_ShapePrimitive				ShapePrimitive = ENSM_ShapePrimitive::Max;
		ENiagaraCoordinateSpace			CoordinateSpace = ENiagaraCoordinateSpace::Simulation;
		FNiagaraStatelessRangeRotator	ShapeRotation;
		FNiagaraStatelessRangeVector3	ShapeScale;
		FUintVector3					TransformOffset = FUintVector3::ZeroValue;
		FNiagaraStatelessRangeVector3	TransformOffsetScale;
		int32							PositionVariableOffset = INDEX_NONE;
		int32							PreviousPositionVariableOffset = INDEX_NONE;
	};

	struct FModuleBuiltDataBox
	{
		bool							bOnSurfaceOnly = false;
		FNiagaraStatelessRangeVector3	Size;
		ENSM_SurfaceExpansionMode		SurfaceExpansionMode = ENSM_SurfaceExpansionMode::Centered;
		FNiagaraStatelessRangeFloat		SurfaceThickness;
	};

	struct FModuleBuiltDataCylinder
	{
		FNiagaraStatelessRangeFloat	Height;
		FNiagaraStatelessRangeFloat	Radius;
		FNiagaraStatelessRangeFloat	MidPoint;
		bool						bDoSurfaceOnly = false;
		bool						bDoSurfaceOnlyIncludeEndCaps = true;
		ENSM_SurfaceExpansionMode	SurfaceExpansionMode = ENSM_SurfaceExpansionMode::Centered;
		FNiagaraStatelessRangeFloat	SurfaceThickness;
	};

	struct FModuleBuiltDataPlane
	{
		bool							bOnEdgesOnly = false;
		FNiagaraStatelessRangeVector2	Size;
		ENSM_SurfaceExpansionMode		EdgeExpansionMode = ENSM_SurfaceExpansionMode::Centered;
		FNiagaraStatelessRangeFloat		EdgeThickness;
	};

	struct FModuleBuiltDataRing
	{
		FNiagaraStatelessRangeFloat	Radius;
		FNiagaraStatelessRangeFloat	Coverage;
		FNiagaraStatelessRangeFloat	Distribution;
	};

	struct FModuleBuiltDataSphere
	{
		FNiagaraStatelessRangeFloat	Radius = FNiagaraStatelessRangeFloat(0.0f);
	};

	TTuple<float, float> CalculateThicknessParameters(ENSM_SurfaceExpansionMode ExpansionMode, float Thickness)
	{
		const float HThickness = Thickness * 0.5f;
		switch (ExpansionMode)
		{
			case ENSM_SurfaceExpansionMode::Inner:		return MakeTuple(-HThickness, Thickness);
			case ENSM_SurfaceExpansionMode::Centered:	return MakeTuple(0.0f, Thickness);
			case ENSM_SurfaceExpansionMode::Outside:	return MakeTuple(HThickness, Thickness);
		}
		checkNoEntry();
		return MakeTuple(0.0f, 0.0f);
	}

	TTuple<float, float> CalculateCylinderThicknessParameters(ENSM_SurfaceExpansionMode ExpansionMode, float Thickness)
	{
		const float HThickness = Thickness * 0.5f;
		switch (ExpansionMode)
		{
			case ENSM_SurfaceExpansionMode::Inner:		return MakeTuple(Thickness, -Thickness);
			case ENSM_SurfaceExpansionMode::Centered:	return MakeTuple(Thickness, -HThickness);
			case ENSM_SurfaceExpansionMode::Outside:	return MakeTuple(Thickness,	0.0f);
		}
		checkNoEntry();
		return MakeTuple(0.0f, 0.0f);
	}

	FVector3f ShapeLocation_GetLocation(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext, const UNiagaraStatelessModule_ShapeLocation::FParameters* ShaderParameters, uint32 iInstance)
	{
		// ENSM_ShapePrimitive::Box | ENSM_ShapePrimitive::Plane
		if ( ShaderParameters->ShapeLocation_Mode.X == 0 )
		{
			const FVector3f BoxSize = FVector3f(ShaderParameters->ShapeLocation_Parameters0);
			const FQuat4f Rotation = FQuat4f(ShaderParameters->ShapeLocation_Parameters2.X, ShaderParameters->ShapeLocation_Parameters2.Y, ShaderParameters->ShapeLocation_Parameters2.Z, ShaderParameters->ShapeLocation_Parameters2.W);
			const bool bOnSurface = ShaderParameters->ShapeLocation_Mode.Y == 1;

			const FVector3f P0 = ParticleSimulationContext.RandomFloat3(iInstance, 0) - 0.5f;
			if (bOnSurface)
			{
				static const FVector3f FaceDirA[] = { FVector3f(0.0f, 1.0f, 0.0f), FVector3f(0.0f, 1.0f, 0.0f),  FVector3f(0.0f, 0.0f, 1.0f), FVector3f(0.0f, 0.0f, 1.0f), FVector3f(1.0f, 0.0f, 0.0f), FVector3f(1.0f, 0.0f, 0.0f) };
				static const FVector3f FaceDirB[] = { FVector3f(0.0f, 0.0f, 1.0f), FVector3f(0.0f, 0.0f, 1.0f),  FVector3f(1.0f, 0.0f, 0.0f), FVector3f(1.0f, 0.0f, 0.0f), FVector3f(0.0f, 1.0f, 0.0f), FVector3f(0.0f, 1.0f, 0.0f) };
				static const FVector3f FaceDirC[] = { FVector3f(1.0f, 0.0f, 0.0f), FVector3f(-1.0f, 0.0f, 0.0f), FVector3f(0.0f, 1.0f, 0.0f), FVector3f(0.0f,-1.0f, 0.0f), FVector3f(0.0f, 0.0f, 1.0f), FVector3f(0.0f, 0.0f,-1.0f) };
				const uint32 NumFaces = ShaderParameters->ShapeLocation_Mode.Z;

				const FVector3f Thickness = FVector3f(ShaderParameters->ShapeLocation_Parameters1);
				const FVector3f LocalBoxSize = BoxSize + (Thickness * P0.Z);

				const uint32 iAxis = ParticleSimulationContext.RandomUInt(iInstance, 1) % NumFaces;
				FVector3f Location;
				Location  = LocalBoxSize * FaceDirA[iAxis] * P0.X;
				Location += LocalBoxSize * FaceDirB[iAxis] * P0.Y;
				Location += LocalBoxSize * FaceDirC[iAxis] * 0.5f;

				return Rotation.RotateVector(Location);
			}
			else
			{
				return Rotation.RotateVector(P0 * BoxSize);
			}
		}
	
		// ENSM_ShapePrimitive::Cylinder:
		if ( ShaderParameters->ShapeLocation_Mode.X == 1 )
		{
			const float HeightScale	= ShaderParameters->ShapeLocation_Parameters0.W;
			const float HeightBias	= ShaderParameters->ShapeLocation_Parameters1.W;
			const float Radius		= ShaderParameters->ShapeLocation_Parameters2.W;

			const FVector3f AxisX	= FVector3f(ShaderParameters->ShapeLocation_Parameters0.X, ShaderParameters->ShapeLocation_Parameters0.Y, ShaderParameters->ShapeLocation_Parameters0.Z);
			const FVector3f AxisY	= FVector3f(ShaderParameters->ShapeLocation_Parameters1.X, ShaderParameters->ShapeLocation_Parameters1.Y, ShaderParameters->ShapeLocation_Parameters1.Z);
			const FVector3f AxisZ	= FVector3f(ShaderParameters->ShapeLocation_Parameters2.X, ShaderParameters->ShapeLocation_Parameters2.Y, ShaderParameters->ShapeLocation_Parameters2.Z);

			const bool bOnSurface = (ShaderParameters->ShapeLocation_Mode.Y & 0x1) != 0;
			const FVector4f Random = ParticleSimulationContext.RandomFloat4(iInstance, 0);

			float XYOffset = Radius * Random.Z;
			float ZOffset = Random.W * HeightScale + HeightBias;
			if (bOnSurface)
			{
				const bool bOnSurfaceIncludeEndCaps = (ShaderParameters->ShapeLocation_Mode.Y & 0x2) != 0;
				const float ThicknessScale = reinterpret_cast<const float&>(ShaderParameters->ShapeLocation_Mode.Z);
				const float ThicknessBias = reinterpret_cast<const float&>(ShaderParameters->ShapeLocation_Mode.W);

				const uint32 Edge = (ParticleSimulationContext.RandomUInt(iInstance, 1) & 0x3) + (bOnSurfaceIncludeEndCaps ? 0 : 2);
				ZOffset = Edge == 0 ? HeightBias - ThicknessBias - (ThicknessScale * Random.W) : ZOffset;
				ZOffset = Edge == 1 ? HeightBias + HeightScale + ThicknessBias + (ThicknessScale * Random.W) : ZOffset;
				XYOffset = Edge >= 2 ? Radius + ThicknessBias + (ThicknessScale * Random.Z) : (Radius + ThicknessBias + ThicknessScale) * Random.Z;
			}
			const FVector2f XYLocation = ParticleSimulationContext.SafeNormalize(FVector2f(Random.X - 0.5f, Random.Y - 0.5f)) * XYOffset;

			FVector3f Location;
			Location  = AxisX * XYLocation.X;
			Location += AxisY * XYLocation.Y;
			Location += AxisZ * ZOffset;
			return Location;
		}
	
		// ENSM_ShapePrimitive::Ring:
		if ( ShaderParameters->ShapeLocation_Mode.X == 2 )
		{
			const float RadiusScale			= ShaderParameters->ShapeLocation_Parameters0.X;
			const float RadiusBias			= ShaderParameters->ShapeLocation_Parameters0.Y;
			const float UDistributionScale	= ShaderParameters->ShapeLocation_Parameters0.Z;
			const float UDistributionBias	= ShaderParameters->ShapeLocation_Parameters0.W;
			const FVector3f AxisX			= FVector3f(ShaderParameters->ShapeLocation_Parameters1.X, ShaderParameters->ShapeLocation_Parameters1.Y, ShaderParameters->ShapeLocation_Parameters1.Z);
			const FVector3f AxisY			= FVector3f(ShaderParameters->ShapeLocation_Parameters2.X, ShaderParameters->ShapeLocation_Parameters2.Y, ShaderParameters->ShapeLocation_Parameters2.Z);
			const float Radius				= ParticleSimulationContext.RandomScaleBiasFloat(iInstance, 0, RadiusScale, RadiusBias);
			const float U					= ParticleSimulationContext.RandomScaleBiasFloat(iInstance, 1, UDistributionScale, UDistributionBias);

			FVector3f Location;
			Location  = AxisX * FMath::Cos(U) * Radius;
			Location += AxisY * FMath::Sin(U) * Radius;
			return Location;
		}
	
		// ENSM_ShapePrimitive::Sphere:
		{
			const float SphereScale	= ShaderParameters->ShapeLocation_Parameters0.W;
			const float SphereBias	= ShaderParameters->ShapeLocation_Parameters1.W;
			const FVector3f AxisX = FVector3f(ShaderParameters->ShapeLocation_Parameters0.X, ShaderParameters->ShapeLocation_Parameters0.Y, ShaderParameters->ShapeLocation_Parameters0.Z);
			const FVector3f AxisY = FVector3f(ShaderParameters->ShapeLocation_Parameters1.X, ShaderParameters->ShapeLocation_Parameters1.Y, ShaderParameters->ShapeLocation_Parameters1.Z);
			const FVector3f AxisZ = FVector3f(ShaderParameters->ShapeLocation_Parameters2.X, ShaderParameters->ShapeLocation_Parameters2.Y, ShaderParameters->ShapeLocation_Parameters2.Z);

			const FVector3f Vector = ParticleSimulationContext.RandomUnitFloat3(iInstance, 0);
			const FVector3f Location = Vector * ParticleSimulationContext.RandomScaleBiasFloat(iInstance, 1, SphereScale, SphereBias);
			return (Location.X * AxisX) + (Location.Y * AxisY) + (Location.Z * AxisZ);
		}
	}

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const UNiagaraStatelessModule_ShapeLocation::FParameters* ShaderParameters = ParticleSimulationContext.ReadParameterNestedStruct<UNiagaraStatelessModule_ShapeLocation::FParameters>();

		const float* ParticlesAge = ParticleSimulationContext.GetParticleNormalizedAge();
		const float* ParticlesPreviousAge = ParticleSimulationContext.GetParticlePreviousNormalizedAge();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const FVector3f ShapeLocation = ShapeLocation_GetLocation(ParticleSimulationContext, ShaderParameters, i);
			const FVector3f Position = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PositionVariableOffset, i, FVector3f::ZeroVector);
			const FVector3f PreviousPosition = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousPositionVariableOffset, i, FVector3f::ZeroVector);

			FVector3f PositionOffsets[2];
			ParticleSimulationContext.SampleDistributionValues(ShaderParameters->ShapeLocation_TransformOffset, i, 1, ParticlesAge[i], ParticlesPreviousAge[i], PositionOffsets[0], PositionOffsets[1]);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PositionVariableOffset, i, Position + (PositionOffsets[0] * ShaderParameters->ShapeLocation_TransformOffsetScale) + ShapeLocation);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousPositionVariableOffset, i, PreviousPosition + (PositionOffsets[1] * ShaderParameters->ShapeLocation_TransformOffsetScale) + ShapeLocation);
		}
	}
}

void UNiagaraStatelessModule_ShapeLocation::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMShapeLocationPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->PositionVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.PositionVariable);
	BuiltData->PreviousPositionVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousPositionVariable);
	if (BuiltData->PositionVariableOffset == INDEX_NONE && BuiltData->PreviousPositionVariableOffset == INDEX_NONE)
	{
		return;
	}

	BuiltData->ShapePrimitive		= ShapePrimitive;
	BuiltData->CoordinateSpace		= CoordinateSpace;
	BuiltData->ShapeRotation		= BuildContext.ConvertDistributionToRange(ShapeRotation, FRotator3f::ZeroRotator);
	BuiltData->ShapeScale			= BuildContext.ConvertDistributionToRange(ShapeScale, FVector3f::OneVector);
	BuiltData->TransformOffset		= BuildContext.AddDistribution(TransformOffset);
	BuiltData->TransformOffsetScale	= BuildContext.ConvertDistributionToRange(TransformOffsetScale, FVector3f::OneVector);

	switch (ShapePrimitive)
	{
		case ENSM_ShapePrimitive::Box:
		{
			FModuleBuiltDataBox* BuiltDataBox	= BuildContext.AllocateBuiltData<FModuleBuiltDataBox>();
			BuiltDataBox->bOnSurfaceOnly		= bBoxSurfaceOnly;
			BuiltDataBox->Size					= BuildContext.ConvertDistributionToRange(BoxSize, FVector3f::ZeroVector);
			BuiltDataBox->SurfaceExpansionMode	= BoxSurfaceExpansion;
			BuiltDataBox->SurfaceThickness		= BuildContext.ConvertDistributionToRange(BoxSurfaceThickness, 0.0f);
			break;
		}
		case ENSM_ShapePrimitive::Cylinder:
		{
			FModuleBuiltDataCylinder* BuiltDataCylinder = BuildContext.AllocateBuiltData<FModuleBuiltDataCylinder>();
			BuiltDataCylinder->Height	= BuildContext.ConvertDistributionToRange(CylinderHeight, 0.0f);
			BuiltDataCylinder->Radius	= BuildContext.ConvertDistributionToRange(CylinderRadius, 0.0f);
			BuiltDataCylinder->MidPoint	= BuildContext.ConvertDistributionToRange(CylinderHeightMidpoint, 0.0f);

			BuiltDataCylinder->bDoSurfaceOnly				= bCylinderSurfaceOnly;
			BuiltDataCylinder->bDoSurfaceOnlyIncludeEndCaps	= bCylinderSurfaceOnlyIncludeEndCaps;
			BuiltDataCylinder->SurfaceExpansionMode			= CylinderSurfaceExpansion;
			BuiltDataCylinder->SurfaceThickness				= BuildContext.ConvertDistributionToRange(CylinderSurfaceThickness, 0.0f);
			break;
		}
		case ENSM_ShapePrimitive::Plane:
		{
			FModuleBuiltDataPlane* BuiltDataPlane = BuildContext.AllocateBuiltData<FModuleBuiltDataPlane>();
			BuiltDataPlane->bOnEdgesOnly		= bPlaneEdgesOnly;
			BuiltDataPlane->Size				= BuildContext.ConvertDistributionToRange(PlaneSize, FVector2f::ZeroVector);
			BuiltDataPlane->EdgeExpansionMode	= PlaneEdgeExpansion;
			BuiltDataPlane->EdgeThickness		= BuildContext.ConvertDistributionToRange(PlaneEdgeThickness, 0.0f);
			break;
		}
		case ENSM_ShapePrimitive::Ring:
		{
			FModuleBuiltDataRing* BuiltDataRing = BuildContext.AllocateBuiltData<FModuleBuiltDataRing>();
			BuiltDataRing->Radius		= BuildContext.ConvertDistributionToRange(RingRadius, 0.0f);
			BuiltDataRing->Coverage		= BuildContext.ConvertDistributionToRange(DiscCoverage, 0.0f);
			BuiltDataRing->Distribution	= BuildContext.ConvertDistributionToRange(RingUDistribution, 0.0f);
			break;
		}
		case ENSM_ShapePrimitive::Sphere:
		{
			FModuleBuiltDataSphere* BuiltDataSphere = BuildContext.AllocateBuiltData<FModuleBuiltDataSphere>();
			BuiltDataSphere->Radius	= BuildContext.ConvertDistributionToRange(SphereRadius, 0.0f);
			break;
		}
		default:
			checkNoEntry();
			return;
	}

	BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
}

void UNiagaraStatelessModule_ShapeLocation::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_ShapeLocation::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMShapeLocationPrivate;

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	const FQuat4f ResolvedShapeRotation = SetShaderParameterContext.GetRendererParameterValue(ModuleBuiltData->ShapeRotation.ParameterOffset, ModuleBuiltData->ShapeRotation.Min.Quaternion());
	const FQuat4f Rotation = SetShaderParameterContext.GetSpaceTransforms().TransformRotation(ModuleBuiltData->CoordinateSpace, ResolvedShapeRotation);

	const FVector3f Scale = SetShaderParameterContext.ConvertRangeToValue(ModuleBuiltData->ShapeScale);

	Parameters->ShapeLocation_Mode					= FUintVector4::ZeroValue;
	Parameters->ShapeLocation_Parameters0			= FVector4f::Zero();
	Parameters->ShapeLocation_Parameters1			= FVector4f::Zero();
	Parameters->ShapeLocation_Parameters2			= FVector4f::Zero();
	Parameters->ShapeLocation_TransformOffset		= ModuleBuiltData->TransformOffset;
	Parameters->ShapeLocation_TransformOffsetScale	= SetShaderParameterContext.ConvertRangeToValue(ModuleBuiltData->TransformOffsetScale);

	switch (ModuleBuiltData->ShapePrimitive)
	{
		case ENSM_ShapePrimitive::Box:
		{
			const FModuleBuiltDataBox* BuiltDataBox = SetShaderParameterContext.ReadBuiltData<FModuleBuiltDataBox>();
			const FVector3f Size	= SetShaderParameterContext.ConvertRangeToValue(BuiltDataBox->Size) * Scale;
			const float Thickness	= SetShaderParameterContext.ConvertRangeToValue(BuiltDataBox->SurfaceThickness);
			const TTuple<float, float> ThicknessParameters = CalculateThicknessParameters(BuiltDataBox->SurfaceExpansionMode, Thickness);

			Parameters->ShapeLocation_Mode.X		= 0;
			Parameters->ShapeLocation_Mode.Y		= BuiltDataBox->bOnSurfaceOnly ? 1 : 0;
			Parameters->ShapeLocation_Mode.Z		= 6;
			Parameters->ShapeLocation_Parameters0	= FVector4f(Size + ThicknessParameters.Key, 0.0f);
			Parameters->ShapeLocation_Parameters1	= FVector4f(ThicknessParameters.Value, ThicknessParameters.Value, ThicknessParameters.Value, 0.0f);
			Parameters->ShapeLocation_Parameters2	= FVector4f(Rotation.X, Rotation.Y, Rotation.Z, Rotation.W);

			break;
		}
		case ENSM_ShapePrimitive::Cylinder:
		{
			const FModuleBuiltDataCylinder* BuiltDataCylinder = SetShaderParameterContext.ReadBuiltData<FModuleBuiltDataCylinder>();
			const float Height		= SetShaderParameterContext.ConvertRangeToValue(BuiltDataCylinder->Height);
			const float Radius		= SetShaderParameterContext.ConvertRangeToValue(BuiltDataCylinder->Radius);
			const float MidPoint	= SetShaderParameterContext.ConvertRangeToValue(BuiltDataCylinder->MidPoint);
			const float Thickness	= SetShaderParameterContext.ConvertRangeToValue(BuiltDataCylinder->SurfaceThickness);
			const TTuple<float, float> ThicknessParameters = CalculateCylinderThicknessParameters(BuiltDataCylinder->SurfaceExpansionMode, Thickness);

			Parameters->ShapeLocation_Mode.X		= 1;
			Parameters->ShapeLocation_Mode.Y		= (BuiltDataCylinder->bDoSurfaceOnly ? 1 : 0) | (BuiltDataCylinder->bDoSurfaceOnlyIncludeEndCaps ? 2 : 0);
			reinterpret_cast<float&>(Parameters->ShapeLocation_Mode.Z) = ThicknessParameters.Key;
			reinterpret_cast<float&>(Parameters->ShapeLocation_Mode.W) = ThicknessParameters.Value;

			Parameters->ShapeLocation_Parameters0	= FVector4f(Rotation.GetAxisX() * Scale.X, Height);
			Parameters->ShapeLocation_Parameters1	= FVector4f(Rotation.GetAxisY() * Scale.Y, Height * -MidPoint);
			Parameters->ShapeLocation_Parameters2	= FVector4f(Rotation.GetAxisZ() * Scale.Z, Radius);
			break;
		}
		case ENSM_ShapePrimitive::Plane:
		{
			const FModuleBuiltDataPlane* BuiltDataPlane = SetShaderParameterContext.ReadBuiltData<FModuleBuiltDataPlane>();
			const FVector2f Size	= SetShaderParameterContext.ConvertRangeToValue(BuiltDataPlane->Size) * FVector2f(Scale.X, Scale.Y);
			const float Thickness	= SetShaderParameterContext.ConvertRangeToValue(BuiltDataPlane->EdgeThickness);
			const TTuple<float, float> ThicknessParameters = CalculateThicknessParameters(BuiltDataPlane->EdgeExpansionMode, Thickness);

			Parameters->ShapeLocation_Mode.X		= 0;
			Parameters->ShapeLocation_Mode.Y		= BuiltDataPlane->bOnEdgesOnly ? 1 : 0;
			Parameters->ShapeLocation_Mode.Z		= 4;
			Parameters->ShapeLocation_Parameters0	= FVector4f(Size.X + ThicknessParameters.Key, Size.Y + ThicknessParameters.Key, 0.0f, 0.0f);
			Parameters->ShapeLocation_Parameters1	= FVector4f(ThicknessParameters.Value, ThicknessParameters.Value, 0.0f, 0.0f);
			Parameters->ShapeLocation_Parameters2	= FVector4f(Rotation.X, Rotation.Y, Rotation.Z, Rotation.W);
			break;
		}
		case ENSM_ShapePrimitive::Ring:
		{
			const FModuleBuiltDataRing* BuiltDataRing = SetShaderParameterContext.ReadBuiltData<FModuleBuiltDataRing>();
			const float Radius			= SetShaderParameterContext.ConvertRangeToValue(BuiltDataRing->Radius);
			const float Coverage		= SetShaderParameterContext.ConvertRangeToValue(BuiltDataRing->Coverage);
			const float Distribution	= SetShaderParameterContext.ConvertRangeToValue(BuiltDataRing->Distribution);

			const float DC = FMath::Clamp(1.0f - Coverage, 0.0f, 1.0f);
			const float SDC = DC > 0.0f ? FMath::Sqrt(DC) : 0.0f;

			Parameters->ShapeLocation_Mode.X		= 2;
			Parameters->ShapeLocation_Parameters0.X = Radius * (1.0f - SDC);
			Parameters->ShapeLocation_Parameters0.Y = Radius * SDC;
			Parameters->ShapeLocation_Parameters0.Z = -UE_TWO_PI * (1.0f - Distribution);
			Parameters->ShapeLocation_Parameters0.W = 0.0f;
			Parameters->ShapeLocation_Parameters1 = FVector4f(Rotation.GetAxisX() * Scale.X);
			Parameters->ShapeLocation_Parameters2 = FVector4f(Rotation.GetAxisY() * Scale.Y);
			break;
		}
		case ENSM_ShapePrimitive::Sphere:
		{
			const FModuleBuiltDataSphere* BuiltDataSphere = SetShaderParameterContext.ReadBuiltData<FModuleBuiltDataSphere>();
			float SphereScale = 0.0f;
			float SphereBias = 0.0f;
			SetShaderParameterContext.ConvertRangeToScaleBias(BuiltDataSphere->Radius, SphereScale, SphereBias);

			Parameters->ShapeLocation_Mode.X		= 3;
			Parameters->ShapeLocation_Parameters0	= FVector4f(Rotation.GetAxisX() * Scale.X, SphereScale);
			Parameters->ShapeLocation_Parameters1	= FVector4f(Rotation.GetAxisY() * Scale.Y, SphereBias);
			Parameters->ShapeLocation_Parameters2	= FVector4f(Rotation.GetAxisZ() * Scale.Z, 0.0f);
			break;
		}
		default:
			break;
	}
}

#if WITH_EDITOR
void UNiagaraStatelessModule_ShapeLocation::DrawDebug(const FNiagaraStatelessDrawDebugContext& DrawDebugContext) const
{
	if (ShapeRotation.IsBinding() || ShapeScale.IsBinding())
	{
		return;
	}

	const FQuat4f ResolvedRotation = ShapeRotation.CalculateRange().Min.Quaternion();
	const FVector DrawPosition = DrawDebugContext.TransformPosition(FVector3f::ZeroVector);
	const FQuat DrawRotation = DrawDebugContext.TransformRotation(CoordinateSpace, ResolvedRotation);
	const FVector DrawScale = FVector(ShapeScale.CalculateRange().Min);

	switch (ShapePrimitive)
	{
		case ENSM_ShapePrimitive::Box:
		{
			if (BoxSize.IsBinding() == false)
			{
				DrawDebugContext.DrawBox(DrawPosition, DrawRotation, FVector(BoxSize.Min * 0.5f) * DrawScale);
			}
			break;
		}
		case ENSM_ShapePrimitive::Plane:
		{
			if (PlaneSize.IsBinding() == false)
			{
				DrawDebugContext.DrawBox(DrawPosition, DrawRotation, FVector(PlaneSize.Min.X * 0.5f, PlaneSize.Min.Y * 0.5f, 0.0f) * DrawScale);
			}
			break;
		}
		case ENSM_ShapePrimitive::Cylinder:
		{
			if (CylinderHeight.IsBinding() == false && CylinderRadius.IsBinding() == false && CylinderHeightMidpoint.IsBinding() == false)
			{
				DrawDebugContext.DrawCylinder(DrawPosition, DrawRotation, DrawScale, CylinderHeight.Min, CylinderRadius.Min, CylinderHeightMidpoint.Min);
			}
			break;
		}
		case ENSM_ShapePrimitive::Ring:
		{
			if (RingRadius.IsBinding() == false && DiscCoverage.IsBinding() == false && RingUDistribution.IsBinding() == false)
			{
				const float DC = FMath::Clamp(1.0f - DiscCoverage.Min, 0.0f, 1.0f);
				const float SDC = DC > 0.0f ? FMath::Sqrt(DC) : 0.0f;
				DrawDebugContext.DrawRing(DrawPosition, DrawRotation, DrawScale, RingRadius.Min * SDC, RingRadius.Min, RingUDistribution.Min);
			}
			break;
		}
		case ENSM_ShapePrimitive::Sphere:
		{
			if (SphereRadius.IsBinding() == false)
			{
				DrawDebugContext.DrawSphere(DrawPosition, DrawRotation, DrawScale, SphereRadius.Min);
				if (SphereRadius.IsRange())
				{
					DrawDebugContext.DrawSphere(DrawPosition, DrawRotation, DrawScale, SphereRadius.Max);
				}
			}
			break;
		}
	}
}
#endif

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_ShapeLocation::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_ShapeLocation.ush");
}

void UNiagaraStatelessModule_ShapeLocation::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.PositionVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousPositionVariable);
}

void UNiagaraStatelessModule_ShapeLocation::PostLoad()
{
	Super::PostLoad();

	if (!FMath::IsNearlyEqual(SphereMin_DEPRECATED, 0.0f) || !FMath::IsNearlyEqual(SphereMax_DEPRECATED, 100.0f))
	{
		SphereRadius = FNiagaraDistributionRangeFloat(SphereMin_DEPRECATED, SphereMax_DEPRECATED);
	}
}
#endif
