// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_SolveVelocitiesAndForces.h"
#include "Stateless/NiagaraStatelessNoiseLUT.h"

#include "VectorField/VectorField.h"
#include "VectorField/VectorFieldStatic.h"
#include "RHIStaticStates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_SolveVelocitiesAndForces)

namespace NiagaraStatelessModuleSolveVelocitiesAndForcesPrivate
{
	float GFrequencyNormalize = 50.0f;
	FAutoConsoleVariableRef CVarFrequencyNormalize(
		TEXT("fx.NiagaraStateless.SolveVelocities.Noise.FrequencyNormalize"),
		GFrequencyNormalize,
		TEXT("Translation from position -> lookup space."),
		ECVF_Default
	);

	float GFrequencyLUTNormalize = 0.1f;
	FAutoConsoleVariableRef CVarFrequencyLUTNormalize(
		TEXT("fx.NiagaraStateless.SolveVelocities.Noise.FrequencyLUTNormalize"),
		GFrequencyLUTNormalize,
		TEXT("Modifier applied when looking up from normalized total travel into the LUT."),
		ECVF_Default
	);

	struct FModuleBuiltData
	{
		NiagaraStateless::FPhysicsBuildData	PhysicsData;
		int32								PositionVariableOffset = INDEX_NONE;
		int32								VelocityVariableOffset = INDEX_NONE;
		int32								PreviousPositionVariableOffset = INDEX_NONE;
		int32								PreviousVelocityVariableOffset = INDEX_NONE;
	};

	static FVector3f IntegratePosition(float Age, float Drag, const FVector3f& Velocity, const FVector3f& Wind, const FVector3f& Acceleration)
	{
		if (Drag > 0.0001f)
		{
			const FVector3f TerminalVelocity = Acceleration / Drag + Wind;
			const FVector3f IntVelocity = Velocity - TerminalVelocity;
			const float LambdaAge = (1.0f - FMath::Exp(-(Age * Drag))) / Drag;
			return IntVelocity * LambdaAge + TerminalVelocity * Age;
		}

		// without drag, we can use the simpler formula for Newtonian motion v*t + 1/2*a*t²
		return Age * (Velocity + Wind) + 0.5f * Acceleration * Age * Age;
	}

	void EvaluateNoise(TConstArrayView<FVector3f> NoiseLUT, const UNiagaraStatelessModule_SolveVelocitiesAndForces::FParameters* Parameters, float Lifetime, float NormalizedAge, const FVector3f& NoiseFieldOffset, FVector3f& Position)
	{
		const FVector3f SamplePosition = (Position + NoiseFieldOffset) * Parameters->SolveVelocitiesAndForces_NoiseFrequency;

		const float fRow = FVector3f::DotProduct(SamplePosition.GetAbs(), FVector3f(1.0f / 8.0f, 2.0f / 8.0f, 5.0f / 8.0f));
		const uint32 RowA = uint32(FMath::FloorToInt(FMath::Abs(fRow))) & Parameters->SolveVelocitiesAndForces_NoiseLUTRowMask;
		const uint32 RowB = (RowA + 1) & Parameters->SolveVelocitiesAndForces_NoiseLUTRowMask;

		const float TotalDistance = Lifetime * Parameters->SolveVelocitiesAndForces_NoiseStrength;
		const float EndColumn = FMath::Clamp(TotalDistance * Parameters->SolveVelocitiesAndForces_NoiseLUTNormalize, 1.0f, float(Parameters->SolveVelocitiesAndForces_NoiseLUTRowWidth - 1));

		const float fColumn = NormalizedAge * EndColumn;
		const uint32 iColumnA = FMath::Min(uint32(FMath::FloorToInt(fColumn)), Parameters->SolveVelocitiesAndForces_NoiseLUTRowWidth - 1);
		const uint32 iColumnB = FMath::Min(iColumnA + 1, Parameters->SolveVelocitiesAndForces_NoiseLUTRowWidth - 1);

		const FVector3f ValueA0 = NoiseLUT[(Parameters->SolveVelocitiesAndForces_NoiseLUTRowWidth * RowA) + iColumnA];
		const FVector3f ValueA1 = NoiseLUT[(Parameters->SolveVelocitiesAndForces_NoiseLUTRowWidth * RowB) + iColumnA];
		const FVector3f ValueB0 = NoiseLUT[(Parameters->SolveVelocitiesAndForces_NoiseLUTRowWidth * RowA) + iColumnB];
		const FVector3f ValueB1 = NoiseLUT[(Parameters->SolveVelocitiesAndForces_NoiseLUTRowWidth * RowB) + iColumnB];
		const FVector3f ValueA = FMath::Lerp(ValueA0, ValueA1, FMath::Fractional(fRow));
		const FVector3f ValueB = FMath::Lerp(ValueB0, ValueB1, FMath::Fractional(fRow));

		Position += FMath::Lerp(ValueA, ValueB, FMath::Fractional(fColumn)) * TotalDistance / EndColumn;
	}

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const UNiagaraStatelessModule_SolveVelocitiesAndForces::FParameters* Parameters = ParticleSimulationContext.ReadParameterNestedStruct<UNiagaraStatelessModule_SolveVelocitiesAndForces::FParameters>();

		const float* LifetimeData				= ParticleSimulationContext.GetParticleLifetime();
		const float* AgeData					= ParticleSimulationContext.GetParticleAge();
		const float* PreviousAgeData			= ParticleSimulationContext.GetParticlePreviousAge();
		const float* NormalizedAgeData			= ParticleSimulationContext.GetParticleNormalizedAge();
		const float* PreviousNormalizedAgeData	= ParticleSimulationContext.GetParticlePreviousNormalizedAge();
		const int32* UniqueIDData				= ParticleSimulationContext.GetParticleUniqueID();
		const float InvDeltaTime				= ParticleSimulationContext.GetInvDeltaTime();

		TConstArrayView<FVector3f> NoiseLUT = NiagaraStateless::FNoiseLUT::GetGlobalLUT().GetCpuData();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const float Mass			= ParticleSimulationContext.RandomScaleBiasFloat(i, 0, Parameters->SolveVelocitiesAndForces_MassScale, Parameters->SolveVelocitiesAndForces_MassBias);
			float Drag					= ParticleSimulationContext.RandomScaleBiasFloat(i, 1, Parameters->SolveVelocitiesAndForces_DragScale, Parameters->SolveVelocitiesAndForces_DragBias);
			FVector3f InitialVelocity	= ParticleSimulationContext.RandomScaleBiasFloat(i, 2, Parameters->SolveVelocitiesAndForces_VelocityScale, Parameters->SolveVelocitiesAndForces_VelocityBias);
			FVector3f Wind				= ParticleSimulationContext.RandomScaleBiasFloat(i, 3, Parameters->SolveVelocitiesAndForces_WindScale, Parameters->SolveVelocitiesAndForces_WindBias);

			const float MassFactor	= 1 / (FMath::Max(Mass, UE_KINDA_SMALL_NUMBER)); // factor in mass for acceleration force
			FVector3f Acceleration	= MassFactor * ParticleSimulationContext.RandomScaleBiasFloat(i, 4, Parameters->SolveVelocitiesAndForces_AccelerationScale, Parameters->SolveVelocitiesAndForces_AccelerationBias);
			Acceleration			+= ParticleSimulationContext.RandomScaleBiasFloat(i, 41, Parameters->SolveVelocitiesAndForces_GravityScale, Parameters->SolveVelocitiesAndForces_GravityBias);

			FVector3f Position			= ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PositionVariableOffset, i, FVector3f::ZeroVector);
			FVector3f PreviousPosition	= ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousPositionVariableOffset, i, Position);

			if (Parameters->SolveVelocitiesAndForces_ConeVelocityEnabled != 0)
			{
				const float ConeAngle = ParticleSimulationContext.RandomScaleBiasFloat(i, 5, Parameters->SolveVelocitiesAndForces_ConeAngleScale, Parameters->SolveVelocitiesAndForces_ConeAngleBias);
				const float ConeRotation = ParticleSimulationContext.RandomFloat(i, 6) * UE_TWO_PI;
				const FVector2f scAng = FVector2f(FMath::Sin(ConeAngle), FMath::Cos(ConeAngle));
				const FVector2f scRot = FVector2f(FMath::Sin(ConeRotation), FMath::Cos(ConeRotation));
				const FVector3f Direction = FVector3f(scRot.X * scAng.X, scRot.Y * scAng.X, scAng.Y);

				float VelocityScale = ParticleSimulationContext.RandomScaleBiasFloat(i, 7, Parameters->SolveVelocitiesAndForces_ConeVelocityScale, Parameters->SolveVelocitiesAndForces_ConeVelocityBias);
				if (Parameters->SolveVelocitiesAndForces_ConeVelocityFalloff > 0.0f)
				{
					const float pf = FMath::Pow(FMath::Clamp(scAng.Y, 0.0f, 1.0f), Parameters->SolveVelocitiesAndForces_ConeVelocityFalloff * 10.0f);
					VelocityScale *= FMath::Lerp(1.0f, pf, Parameters->SolveVelocitiesAndForces_ConeVelocityFalloff);
				}

				InitialVelocity += Parameters->SolveVelocitiesAndForces_ConeQuat.RotateVector(Direction) * VelocityScale;
			}

			if (Parameters->SolveVelocitiesAndForces_PointVelocityEnabled != 0)
			{
				const FVector3f FallbackDir	= ParticleSimulationContext.RandomUnitFloat3(i, 8);
				const FVector3f Delta		= Position - Parameters->SolveVelocitiesAndForces_PointOrigin;
				const FVector3f Direction	= ParticleSimulationContext.SafeNormalize(Delta, FallbackDir);
				const float		VelocityScale = ParticleSimulationContext.RandomScaleBiasFloat(i, 9, Parameters->SolveVelocitiesAndForces_PointVelocityScale, Parameters->SolveVelocitiesAndForces_PointVelocityBias);

				InitialVelocity += Direction * VelocityScale;
			}

			if (Parameters->SolveVelocitiesAndForces_NoiseEnabled != 0)
			{
				const FVector3f NoiseFieldOffset = ParticleSimulationContext.RandomScaleBiasFloat(i, 10, Parameters->SolveVelocitiesAndForces_NoiseFieldOffsetScale, Parameters->SolveVelocitiesAndForces_NoiseFieldOffsetBias);
				EvaluateNoise(NoiseLUT, Parameters, LifetimeData[i], NormalizedAgeData[i], NoiseFieldOffset, Position);
				EvaluateNoise(NoiseLUT, Parameters, LifetimeData[i], PreviousNormalizedAgeData[i], NoiseFieldOffset, PreviousPosition);
			}

			// Apply owner scale before integration
			//Drag			*= Parameters->SolveVelocitiesAndForces_OwnerDragScale;
			InitialVelocity	*= Parameters->SolveVelocitiesAndForces_OwnerVelocityScale;
			Wind			*= Parameters->SolveVelocitiesAndForces_OwnerAccelerationScale;
			Acceleration	*= Parameters->SolveVelocitiesAndForces_OwnerAccelerationScale;

			Position += IntegratePosition(FMath::Max(AgeData[i], UE_KINDA_SMALL_NUMBER), Drag, InitialVelocity, Wind, Acceleration);
			PreviousPosition += IntegratePosition(PreviousAgeData[i], Drag, InitialVelocity, Wind, Acceleration);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PositionVariableOffset, i, Position);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousPositionVariableOffset, i, PreviousPosition);

			const FVector3f Velocity = (Position - PreviousPosition) * InvDeltaTime;
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->VelocityVariableOffset, i, Velocity);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousVelocityVariableOffset, i, Velocity);
		}
	}
}

void UNiagaraStatelessModule_SolveVelocitiesAndForces::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NiagaraStatelessModuleSolveVelocitiesAndForcesPrivate;

	NiagaraStateless::FPhysicsBuildData& PhysicsBuildData = BuildContext.GetTransientBuildData<NiagaraStateless::FPhysicsBuildData>();

	FModuleBuiltData* BuiltData				= BuildContext.AllocateBuiltData<FModuleBuiltData>();
	BuiltData->PhysicsData					= PhysicsBuildData;

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->PositionVariableOffset				= BuildContext.FindParticleVariableIndex(StatelessGlobals.PositionVariable);
	BuiltData->VelocityVariableOffset				= BuildContext.FindParticleVariableIndex(StatelessGlobals.VelocityVariable);
	BuiltData->PreviousPositionVariableOffset		= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousPositionVariable);
	BuiltData->PreviousVelocityVariableOffset		= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousVelocityVariable);
		
	const bool bAttributesUsed =
		BuiltData->PositionVariableOffset != INDEX_NONE ||
		BuiltData->VelocityVariableOffset != INDEX_NONE ||
		BuiltData->PreviousPositionVariableOffset != INDEX_NONE ||
		BuiltData->PreviousVelocityVariableOffset != INDEX_NONE;

	if (bAttributesUsed)
	{
		BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
	}
}

void UNiagaraStatelessModule_SolveVelocitiesAndForces::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_SolveVelocitiesAndForces::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NiagaraStatelessModuleSolveVelocitiesAndForcesPrivate;

	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();
	const NiagaraStateless::FPhysicsBuildData& PhysicsData = ModuleBuiltData->PhysicsData;

	const FNiagaraStatelessSpaceTransforms& SpaceTransforms = SetShaderParameterContext.GetSpaceTransforms();

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	Parameters->SolveVelocitiesAndForces_OwnerAccelerationScale	= FVector3f::OneVector;
	//Parameters->SolveVelocitiesAndForces_OwnerDragScale			= 1.0f;
	Parameters->SolveVelocitiesAndForces_OwnerVelocityScale		= FVector3f::OneVector;
	if (const FNiagaraStatelessSystemScaleBuildData* SystemScaleBuildData = SetShaderParameterContext.ReadSharedBuiltData<FNiagaraStatelessSystemScaleBuildData>())
	{
		if (SystemScaleBuildData->bScaleForces)
		{
			const FVector3f ComponentScale = SpaceTransforms.GetLocalToWorld().GetScale3D();
			const FVector3f U = SystemScaleBuildData->bScaleForcesInverseScaling ? NiagaraStateless::FParticleSimulationContext::SafeRcpSqrt(SystemScaleBuildData->ScaleForcesScaleAmount) : SystemScaleBuildData->ScaleForcesScaleAmount;
			Parameters->SolveVelocitiesAndForces_OwnerAccelerationScale = FMath::Lerp(FVector3f::OneVector, ComponentScale, U);
		}
		//if (SystemScaleBuildData->bScaleDrag)
		//{
		//	const float ComponentScale = SpaceTransforms.GetLocalToWorld().GetScale3D().GetMax();
		//	const float U = SystemScaleBuildData->bInverseDragScaling ? NiagaraStateless::FParticleSimulationContext::SafeRcpSqrt(SystemScaleBuildData->ScaleDragScaleAmount) : SystemScaleBuildData->ScaleDragScaleAmount;
		//	Parameters->SolveVelocitiesAndForces_OwnerDragScale = FMath::Lerp(1.0f, ComponentScale, U);
		//}
		if (SystemScaleBuildData->bScaleInitialVelocity)
		{
			const FVector3f ComponentScale = SpaceTransforms.GetLocalToWorld().GetScale3D();
			Parameters->SolveVelocitiesAndForces_OwnerVelocityScale = FMath::Lerp(FVector3f::OneVector, ComponentScale, SystemScaleBuildData->ScaleInitialVelocityScaleAmount);
		}
	}

	SetShaderParameterContext.ConvertRangeToScaleBias(PhysicsData.MassRange, Parameters->SolveVelocitiesAndForces_MassScale, Parameters->SolveVelocitiesAndForces_MassBias);
	SetShaderParameterContext.ConvertRangeToScaleBias(PhysicsData.DragRange, Parameters->SolveVelocitiesAndForces_DragScale, Parameters->SolveVelocitiesAndForces_DragBias);

	// Linear Velocity
	{
		SetShaderParameterContext.TransformVectorNoScaleRangeToScaleBias(PhysicsData.LinearVelocityRange, PhysicsData.LinearVelocityCoordinateSpace, Parameters->SolveVelocitiesAndForces_VelocityScale, Parameters->SolveVelocitiesAndForces_VelocityBias);
		const float LinearVelocityScale = SetShaderParameterContext.ConvertRangeToValue(PhysicsData.LinearVelocityScale);
		Parameters->SolveVelocitiesAndForces_VelocityScale *= LinearVelocityScale;
		Parameters->SolveVelocitiesAndForces_VelocityBias *= LinearVelocityScale;
	}

	// Wind / Acceleration
	SetShaderParameterContext.TransformVectorNoScaleRangeToScaleBias(PhysicsData.WindRange, PhysicsData.WindCoordinateSpace, Parameters->SolveVelocitiesAndForces_WindScale, Parameters->SolveVelocitiesAndForces_WindBias);
	{
		FVector3f AccelerationScale, AccelerationBias;
		FVector3f GravityScale, GravityBias;
		SetShaderParameterContext.ConvertRangeToScaleBias(PhysicsData.AccelerationRange, AccelerationScale, AccelerationBias);
		SetShaderParameterContext.ConvertRangeToScaleBias(PhysicsData.GravityRange, GravityScale, GravityBias);
		Parameters->SolveVelocitiesAndForces_AccelerationScale	= SpaceTransforms.TransformVectorNoScale(PhysicsData.AccelerationCoordinateSpace, AccelerationScale);
		Parameters->SolveVelocitiesAndForces_AccelerationBias	= SpaceTransforms.TransformVectorNoScale(PhysicsData.AccelerationCoordinateSpace, AccelerationBias);
		Parameters->SolveVelocitiesAndForces_GravityScale	= SpaceTransforms.TransformVectorNoScale(ENiagaraCoordinateSpace::World, GravityScale);
		Parameters->SolveVelocitiesAndForces_GravityBias	= SpaceTransforms.TransformVectorNoScale(ENiagaraCoordinateSpace::World, GravityBias);
	}

	// Cone Velocity
	Parameters->SolveVelocitiesAndForces_ConeVelocityEnabled = PhysicsData.bConeVelocity ? 1 : 0;
	{
		FQuat4f LocalConeQuat;
		if (PhysicsData.bUseConeConeRotator)
		{
			LocalConeQuat = SetShaderParameterContext.ConvertRangeToValue(PhysicsData.ConeRotator).Quaternion();
		}
		else
		{
			const FVector3f ConeDirection = SetShaderParameterContext.ConvertRangeToValue(PhysicsData.ConeDirection);
			LocalConeQuat = NiagaraStateless::DirectionToQuat(ConeDirection);
		}

		Parameters->SolveVelocitiesAndForces_ConeQuat				= SpaceTransforms.TransformRotation(PhysicsData.ConeCoordinateSpace, LocalConeQuat);
		SetShaderParameterContext.ConvertRangeToScaleBias(PhysicsData.ConeVelocityRange, Parameters->SolveVelocitiesAndForces_ConeVelocityScale, Parameters->SolveVelocitiesAndForces_ConeVelocityBias);
		Parameters->SolveVelocitiesAndForces_ConeAngleScale			= (PhysicsData.ConeOuterAngle - PhysicsData.ConeInnerAngle) * (UE_PI / 360.0f);
		Parameters->SolveVelocitiesAndForces_ConeAngleBias			= PhysicsData.ConeInnerAngle * (UE_PI / 360.0f);
		Parameters->SolveVelocitiesAndForces_ConeVelocityFalloff	= PhysicsData.ConeVelocityFalloff;

		const float ConeVelocityScale = SetShaderParameterContext.ConvertRangeToValue(PhysicsData.ConeVelocityScale);
		Parameters->SolveVelocitiesAndForces_ConeVelocityScale *= ConeVelocityScale;
		Parameters->SolveVelocitiesAndForces_ConeVelocityBias  *= ConeVelocityScale;
	}

	// Point Velocity
	Parameters->SolveVelocitiesAndForces_PointVelocityEnabled = PhysicsData.bPointVelocity ? 1 : 0;
	{
		SetShaderParameterContext.ConvertRangeToScaleBias(PhysicsData.PointVelocityRange, Parameters->SolveVelocitiesAndForces_PointVelocityScale, Parameters->SolveVelocitiesAndForces_PointVelocityBias);
		Parameters->SolveVelocitiesAndForces_PointOrigin = SpaceTransforms.TransformPosition(PhysicsData.PointCoordinateSpace, PhysicsData.PointOrigin);

		const float PointVelocityScale = SetShaderParameterContext.ConvertRangeToValue(PhysicsData.PointVelocityScale);
		Parameters->SolveVelocitiesAndForces_PointVelocityScale *= PointVelocityScale;
		Parameters->SolveVelocitiesAndForces_PointVelocityBias  *= PointVelocityScale;
	}

	{
		const NiagaraStateless::FNoiseLUT& NoiseLUT = NiagaraStateless::FNoiseLUT::GetGlobalLUT();
		checkf(FMath::IsPowerOfTwo(NoiseLUT.GetNumRows()), TEXT("Noise LUT must be PoT"));

		Parameters->SolveVelocitiesAndForces_NoiseEnabled			= PhysicsData.bNoiseEnabled ? 1 : 0;
		Parameters->SolveVelocitiesAndForces_NoiseStrength			= PhysicsData.NoiseStrength;
		Parameters->SolveVelocitiesAndForces_NoiseFrequency			= PhysicsData.NoiseFrequency / GFrequencyNormalize;
		SetShaderParameterContext.ConvertRangeToScaleBias(PhysicsData.NoiseFieldOffset, Parameters->SolveVelocitiesAndForces_NoiseFieldOffsetScale, Parameters->SolveVelocitiesAndForces_NoiseFieldOffsetBias);
		Parameters->SolveVelocitiesAndForces_NoiseLUTNormalize		= (PhysicsData.NoiseFrequency / GFrequencyNormalize) * GFrequencyLUTNormalize;
		Parameters->SolveVelocitiesAndForces_NoiseLUTRowMask		= NoiseLUT.GetNumRows() - 1;
		Parameters->SolveVelocitiesAndForces_NoiseLUTRowWidth		= NoiseLUT.GetRowWidth();
		Parameters->SolveVelocitiesAndForces_NoiseLUT				= NoiseLUT.GetGpuData();
	}
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_SolveVelocitiesAndForces::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_SolveVelocitiesAndForces.ush");
}

void UNiagaraStatelessModule_SolveVelocitiesAndForces::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.PositionVariable);
	OutVariables.AddUnique(StatelessGlobals.VelocityVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousPositionVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousVelocityVariable);
}
#endif
