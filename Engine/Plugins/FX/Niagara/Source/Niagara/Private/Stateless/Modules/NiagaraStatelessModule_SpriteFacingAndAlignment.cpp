// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_SpriteFacingAndAlignment.h"
#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_SpriteFacingAndAlignment)

namespace NSMSpriteFacingAndAlignmentPrivate
{
	struct FModuleBuiltData
	{
		FNiagaraStatelessRangeVector3	SpriteFacing	= FNiagaraStatelessRangeVector3(FVector3f::XAxisVector);
		FNiagaraStatelessRangeVector3	SpriteAlignment = FNiagaraStatelessRangeVector3(FVector3f::YAxisVector);

		int32		SpriteFacingVariableOffset = INDEX_NONE;
		int32		PreviousSpriteFacingVariableOffset = INDEX_NONE;
		int32		SpriteAlignmentVariable = INDEX_NONE;
		int32		PreviousSpriteAlignmentVariable = INDEX_NONE;
	};

	using FParameters = NiagaraStateless::FSpriteFacingAndAlignmentModule_ShaderParameters;

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const FParameters* Parameters = ParticleSimulationContext.ReadParameterNestedStruct<FParameters>();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->SpriteFacingVariableOffset, i, Parameters->SpriteFacingAndAlignment_SpriteFacing);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousSpriteFacingVariableOffset, i, Parameters->SpriteFacingAndAlignment_SpriteFacing);

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->SpriteAlignmentVariable, i, Parameters->SpriteFacingAndAlignment_SpriteAlignment);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousSpriteAlignmentVariable, i, Parameters->SpriteFacingAndAlignment_SpriteAlignment);
		}
	}
}

void UNiagaraStatelessModule_SpriteFacingAndAlignment::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMSpriteFacingAndAlignmentPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	if (bSpriteFacingEnabled)
	{
		BuiltData->SpriteFacingVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.SpriteFacingVariable);
		BuiltData->PreviousSpriteFacingVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousSpriteFacingVariable);
	}
	if (bSpriteAlignmentEnabled)
	{
		BuiltData->SpriteAlignmentVariable				= BuildContext.FindParticleVariableIndex(StatelessGlobals.SpriteAlignmentVariable);
		BuiltData->PreviousSpriteAlignmentVariable		= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousSpriteAlignmentVariable);
	}

	if ((BuiltData->SpriteFacingVariableOffset == INDEX_NONE) && (BuiltData->PreviousSpriteFacingVariableOffset == INDEX_NONE) &&
		(BuiltData->SpriteAlignmentVariable == INDEX_NONE) && (BuiltData->PreviousSpriteAlignmentVariable == INDEX_NONE))
	{
		return;
	}

	if (bSpriteFacingEnabled)
	{
		BuiltData->SpriteFacing = BuildContext.ConvertDistributionToRange(SpriteFacing, FVector3f::ZeroVector);
	}
	if (bSpriteAlignmentEnabled)
	{
		BuiltData->SpriteAlignment	= BuildContext.ConvertDistributionToRange(SpriteAlignment, FVector3f::ZeroVector);
	}

	BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
}

void UNiagaraStatelessModule_SpriteFacingAndAlignment::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMSpriteFacingAndAlignmentPrivate;

	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_SpriteFacingAndAlignment::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMSpriteFacingAndAlignmentPrivate;

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	Parameters->SpriteFacingAndAlignment_SpriteFacing		= SetShaderParameterContext.ConvertRangeToValue(ModuleBuiltData->SpriteFacing);
	Parameters->SpriteFacingAndAlignment_SpriteAlignment	= SetShaderParameterContext.ConvertRangeToValue(ModuleBuiltData->SpriteAlignment);
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_SpriteFacingAndAlignment::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_SpriteFacingAndAlignment.ush");
}

void UNiagaraStatelessModule_SpriteFacingAndAlignment::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	using namespace NSMSpriteFacingAndAlignmentPrivate;

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	if (bSpriteFacingEnabled || Filter == EVariableFilter::None)
	{
		OutVariables.AddUnique(StatelessGlobals.SpriteFacingVariable);
		OutVariables.AddUnique(StatelessGlobals.PreviousSpriteFacingVariable);
	}
	if (bSpriteAlignmentEnabled || Filter == EVariableFilter::None)
	{
		OutVariables.AddUnique(StatelessGlobals.SpriteAlignmentVariable);
		OutVariables.AddUnique(StatelessGlobals.PreviousSpriteAlignmentVariable);
	}
}
#endif
