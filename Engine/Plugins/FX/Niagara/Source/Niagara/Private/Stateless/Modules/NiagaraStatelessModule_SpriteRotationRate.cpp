// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_SpriteRotationRate.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_SpriteRotationRate)

namespace NSMSpriteRotationRatePrivate
{
	struct FModuleBuiltData
	{
		int32							ModuleEnabled = false;
		FNiagaraStatelessRangeFloat		RotationRange;
		FUintVector3					RateScaleParameters = FUintVector3::ZeroValue;
		int32							SpriteRotationVariableOffset = INDEX_NONE;
		int32							PreviousSpriteRotationVariableOffset = INDEX_NONE;
	};

	using FParameters = NiagaraStateless::FSpriteRotationRateModule_ShaderParameters;

	static void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const FParameters* ShaderParameters = ParticleSimulationContext.ReadParameterNestedStruct<FParameters>();

		const float* LifetimeData = ParticleSimulationContext.GetParticleLifetime();
		const float* AgeData = ParticleSimulationContext.GetParticleNormalizedAge();
		const float* PreviousAgeData = ParticleSimulationContext.GetParticlePreviousNormalizedAge();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const float RotationRate = ParticleSimulationContext.RandomScaleBiasFloat(i, 0, ShaderParameters->SpriteRotationRate_Scale, ShaderParameters->SpriteRotationRate_Bias);
			const float MultiplyRate = ParticleSimulationContext.SampleCurve<float>(ShaderParameters->SpriteRotationRate_RateScaleParameters, AgeData[i]);
			const float PreviousMultiplyRate = ParticleSimulationContext.SampleCurve<float>(ShaderParameters->SpriteRotationRate_RateScaleParameters, PreviousAgeData[i]);

			float SpriteRotation = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->SpriteRotationVariableOffset, i, 0.0f);
			float PreviousSpriteRotation = ParticleSimulationContext.ReadParticleVariable(ModuleBuiltData->PreviousSpriteRotationVariableOffset, i, 0.0f);

			SpriteRotation += LifetimeData[i] * RotationRate * MultiplyRate;
			PreviousSpriteRotation += LifetimeData[i] * RotationRate * PreviousMultiplyRate;

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->SpriteRotationVariableOffset, i, SpriteRotation);
			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->PreviousSpriteRotationVariableOffset, i, PreviousSpriteRotation);
		}
	}
}

void UNiagaraStatelessModule_SpriteRotationRate::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMSpriteRotationRatePrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->SpriteRotationVariableOffset			= BuildContext.FindParticleVariableIndex(StatelessGlobals.SpriteRotationVariable);
	BuiltData->PreviousSpriteRotationVariableOffset	= BuildContext.FindParticleVariableIndex(StatelessGlobals.PreviousSpriteRotationVariable);

	if (BuiltData->SpriteRotationVariableOffset == INDEX_NONE && BuiltData->PreviousSpriteRotationVariableOffset == INDEX_NONE)
	{
		return;
	}

	BuiltData->ModuleEnabled = true;
	BuiltData->RotationRange = BuildContext.ConvertDistributionToRange(RotationRateDistribution, 0.0f);
	if (bUseRateScale)
	{
		BuiltData->RateScaleParameters = BuildContext.AddDistributionAsCurve(RateScaleDistribution, 1.0f);
	}
	else
	{
		BuiltData->RateScaleParameters = BuildContext.AddDistributionAsCurve(FNiagaraDistributionCurveFloat(ENiagaraDistributionCurveLUTMode::Accumulate), 1.0f);
	}

	BuildContext.AddParticleSimulationExecSimulate(&ParticleSimulate);
}

void UNiagaraStatelessModule_SpriteRotationRate::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMSpriteRotationRatePrivate;

	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_SpriteRotationRate::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMSpriteRotationRatePrivate;

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	Parameters->SpriteRotationRate_Enabled = ModuleBuiltData->ModuleEnabled;
	SetShaderParameterContext.ConvertRangeToScaleBias(ModuleBuiltData->RotationRange, Parameters->SpriteRotationRate_Scale, Parameters->SpriteRotationRate_Bias);
	Parameters->SpriteRotationRate_RateScaleParameters = ModuleBuiltData->RateScaleParameters;
}

#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_SpriteRotationRate::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_SpriteRotationRate.ush");
}

void UNiagaraStatelessModule_SpriteRotationRate::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	using namespace NSMSpriteRotationRatePrivate;

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.SpriteRotationVariable);
	OutVariables.AddUnique(StatelessGlobals.PreviousSpriteRotationVariable);
}
#endif
