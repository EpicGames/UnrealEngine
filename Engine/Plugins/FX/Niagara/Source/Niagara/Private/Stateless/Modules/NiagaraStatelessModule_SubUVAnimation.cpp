// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/Modules/NiagaraStatelessModule_SubUVAnimation.h"

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModuleShaderParameters.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule_SubUVAnimation)

namespace NSMSubUVAnimationPrivate
{
	using FParameters = NiagaraStateless::FSubUVAnimationModule_ShaderParameters;

	struct FModuleBuiltData
	{
		int32						Mode = 0;
		int32						NumFrames = 0;
		FNiagaraStatelessRangeInt	InitialFrame;
		float						InitialFrameRateChange = 0.0f;
		int32						AnimFrameStart = 0;
		int32						AnimFrameRange = 0;
		float						RateScale = 0.0f;

		int32						SubImageIndexVariableOffset = INDEX_NONE;
	};

	static_assert(int(ENSMSubUVAnimation_Mode::InfiniteLoop) == 1 && int(ENSMSubUVAnimation_Mode::Linear) == 2, "Enum is out of sync with simulation");

	void ParticleSimulate(const NiagaraStateless::FParticleSimulationContext& ParticleSimulationContext)
	{
		using namespace NiagaraStateless;

		const FModuleBuiltData* ModuleBuiltData = ParticleSimulationContext.ReadBuiltData<FModuleBuiltData>();
		const FParameters* ShaderParameters = ParticleSimulationContext.ReadParameterNestedStruct<FParameters>();

		const float* ParticleAges = ParticleSimulationContext.GetParticleAge();
		const float* ParticleNormalizedAges = ParticleSimulationContext.GetParticleNormalizedAge();

		for (uint32 i = 0; i < ParticleSimulationContext.GetNumInstances(); ++i)
		{
			const float Particle_Age = ParticleAges[i];
			const float Particle_NormalizedAge = ParticleNormalizedAges[i];

			const uint32 SeedOffset = uint32(Particle_Age * ShaderParameters->SubUVAnimation_InitialFrameRateChange);
			float Frame = ParticleSimulationContext.RandomFloat(i, SeedOffset) * ShaderParameters->SubUVAnimation_InitialFrameScale + ShaderParameters->SubUVAnimation_InitialFrameBias;

			if (ShaderParameters->SubUVAnimation_Mode == 1)
			{
				const float Interp = Particle_Age * ShaderParameters->SubUVAnimation_RateScale;
				Frame = FMath::Fractional(Frame + ShaderParameters->SubUVAnimation_AnimFrameStart + (Interp * ShaderParameters->SubUVAnimation_AnimFrameRange));
			}
			else if (ShaderParameters->SubUVAnimation_Mode == 2)
			{
				const float Interp = Particle_NormalizedAge * ShaderParameters->SubUVAnimation_RateScale;
				Frame = FMath::Clamp(Frame + ShaderParameters->SubUVAnimation_AnimFrameStart + (Interp * ShaderParameters->SubUVAnimation_AnimFrameRange), 0.0f, 1.0f);
			}
			const float SubImageIndex = Frame * ShaderParameters->SubUVAnimation_NumFrames;

			ParticleSimulationContext.WriteParticleVariable(ModuleBuiltData->SubImageIndexVariableOffset, i, SubImageIndex);
		}
	}
}

void UNiagaraStatelessModule_SubUVAnimation::BuildEmitterData(const FNiagaraStatelessEmitterDataBuildContext& BuildContext) const
{
	using namespace NSMSubUVAnimationPrivate;

	FModuleBuiltData* BuiltData = BuildContext.AllocateBuiltData<FModuleBuiltData>();
	if (!IsModuleEnabled())
	{
		return;
	}

	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	BuiltData->SubImageIndexVariableOffset = BuildContext.FindParticleVariableIndex(StatelessGlobals.SubImageIndexVariable);
	if (BuiltData->SubImageIndexVariableOffset == INDEX_NONE)
	{
		return;
	}

	if (NumFrames <= 1)
	{
		return;
	}

	const int32 NumFramesMinusOne	= NumFrames - 1;
	const int32 FrameRangeStart		= bStartFrameRangeOverride_Enabled ? FMath::Clamp(StartFrameRangeOverride, 0, NumFramesMinusOne) : 0;
	const int32 FrameRangeEnd		= bEndFrameRangeOverride_Enabled ? FMath::Clamp(EndFrameRangeOverride, 0, NumFramesMinusOne) : NumFramesMinusOne;

	BuiltData->Mode			= int(AnimationMode);
	BuiltData->NumFrames	= float(NumFrames);
	switch (AnimationMode)
	{
		case ENSMSubUVAnimation_Mode::DirectSet:
			BuiltData->InitialFrame	 = BuildContext.ConvertDistributionToRange(FrameIndex, 0);
			break;

		case ENSMSubUVAnimation_Mode::InfiniteLoop:
			BuiltData->AnimFrameStart	= FrameRangeStart;
			BuiltData->AnimFrameRange	= FrameRangeEnd - FrameRangeStart;
			BuiltData->RateScale		= LoopsPerSecond;
			break;

		case ENSMSubUVAnimation_Mode::Linear:
			BuiltData->AnimFrameStart	= FrameRangeStart;
			BuiltData->AnimFrameRange	= FrameRangeEnd - FrameRangeStart;
			BuiltData->RateScale		= 1.0f;
			break;

		case ENSMSubUVAnimation_Mode::Random:
			BuiltData->InitialFrame				= FNiagaraStatelessRangeInt(FrameRangeStart, FrameRangeEnd);
			BuiltData->InitialFrameRateChange	= RandomChangeInterval > 0.0f ? 1.0f / RandomChangeInterval : 0.0f;
			break;
	}

	BuildContext.AddParticleSimulationExecSimulate(&NSMSubUVAnimationPrivate::ParticleSimulate);
}

void UNiagaraStatelessModule_SubUVAnimation::BuildShaderParameters(FNiagaraStatelessShaderParametersBuilder& ShaderParametersBuilder) const
{
	using namespace NSMSubUVAnimationPrivate;
	
	ShaderParametersBuilder.AddParameterNestedStruct<FParameters>();
}

void UNiagaraStatelessModule_SubUVAnimation::SetShaderParameters(const FNiagaraStatelessSetShaderParameterContext& SetShaderParameterContext) const
{
	using namespace NSMSubUVAnimationPrivate;

	FParameters* Parameters = SetShaderParameterContext.GetParameterNestedStruct<FParameters>();
	const FModuleBuiltData* ModuleBuiltData = SetShaderParameterContext.ReadBuiltData<FModuleBuiltData>();

	if (ModuleBuiltData->NumFrames > 1)
	{
		//const float fNumFramesMinusOne = ModuleBuiltData->NumFrames > 1 ? float(ModuleBuiltData->NumFrames - 1) : 1.0f;
		const float fNumFrames = float(ModuleBuiltData->NumFrames);

		int32 InitialFrameScale = 0;
		int32 InitialFrameBias = 0;
		SetShaderParameterContext.ConvertRangeToScaleBias(ModuleBuiltData->InitialFrame, InitialFrameScale, InitialFrameBias);

		Parameters->SubUVAnimation_Mode						= ModuleBuiltData->Mode;
		Parameters->SubUVAnimation_NumFrames				= fNumFrames;
		Parameters->SubUVAnimation_InitialFrameScale		= float(InitialFrameScale) / fNumFrames;
		Parameters->SubUVAnimation_InitialFrameBias			= float(InitialFrameBias) / fNumFrames;
		Parameters->SubUVAnimation_InitialFrameRateChange	= ModuleBuiltData->InitialFrameRateChange;
		Parameters->SubUVAnimation_AnimFrameStart			= ModuleBuiltData->AnimFrameStart / fNumFrames;
		Parameters->SubUVAnimation_AnimFrameRange			= ModuleBuiltData->AnimFrameRange / fNumFrames;
		Parameters->SubUVAnimation_RateScale				= ModuleBuiltData->RateScale;
	}
	else
	{
		Parameters->SubUVAnimation_Mode						= 0;
		Parameters->SubUVAnimation_NumFrames				= 0.0f;
		Parameters->SubUVAnimation_InitialFrameScale		= 0.0f;
		Parameters->SubUVAnimation_InitialFrameBias			= 0.0f;
		Parameters->SubUVAnimation_InitialFrameRateChange	= 0.0f;
		Parameters->SubUVAnimation_AnimFrameStart			= 0.0f;
		Parameters->SubUVAnimation_AnimFrameRange			= 0.0f;
		Parameters->SubUVAnimation_RateScale				= 0.0f;
	}
}


#if WITH_EDITORONLY_DATA
const TCHAR* UNiagaraStatelessModule_SubUVAnimation::GetShaderTemplatePath() const
{
	return TEXT("/Plugin/FX/Niagara/Private/Stateless/Modules/NiagaraStatelessModule_SubUVAnimation.ush");
}

void UNiagaraStatelessModule_SubUVAnimation::GetOutputVariables(TArray<FNiagaraVariableBase>& OutVariables, EVariableFilter Filter) const
{
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	OutVariables.AddUnique(StatelessGlobals.SubImageIndexVariable);
}
#endif
