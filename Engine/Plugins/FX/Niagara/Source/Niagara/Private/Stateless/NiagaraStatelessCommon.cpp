// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessCommon.h"

#include "Stateless/NiagaraStatelessEmitterTemplate.h"
#include "NiagaraConstants.h"
#include "NiagaraSystem.h"

#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessCommon)

static_assert(int(ENiagaraCoordinateSpace::Local) >= 0 && int(ENiagaraCoordinateSpace::Local) < 3, "Local space transform out of expected range");
static_assert(int(ENiagaraCoordinateSpace::World) >= 0 && int(ENiagaraCoordinateSpace::World) < 3, "World space transform out of expected range");
static_assert(int(ENiagaraCoordinateSpace::Simulation) >= 0 && int(ENiagaraCoordinateSpace::Simulation) < 3, "Simulation space transform out of expected range");

void FNiagaraStatelessSpaceTransforms::InitializeTransforms(bool bIsLocalSpace, const FTransform3f& LocalToWorld)
{
	TransformRemap[int(ENiagaraCoordinateSpace::Local)][int(ENiagaraCoordinateSpace::Local)]			= INDEX_NONE;
	TransformRemap[int(ENiagaraCoordinateSpace::Local)][int(ENiagaraCoordinateSpace::World)]			= LocalToWorldIndex;
	TransformRemap[int(ENiagaraCoordinateSpace::Local)][int(ENiagaraCoordinateSpace::Simulation)]		= bIsLocalSpace ? INDEX_NONE : LocalToWorldIndex;

	TransformRemap[int(ENiagaraCoordinateSpace::World)][int(ENiagaraCoordinateSpace::Local)]			= WorldToLocalIndex;
	TransformRemap[int(ENiagaraCoordinateSpace::World)][int(ENiagaraCoordinateSpace::World)]			= INDEX_NONE;
	TransformRemap[int(ENiagaraCoordinateSpace::World)][int(ENiagaraCoordinateSpace::Simulation)]		= bIsLocalSpace ? WorldToLocalIndex : INDEX_NONE;

	TransformRemap[int(ENiagaraCoordinateSpace::Simulation)][int(ENiagaraCoordinateSpace::Local)]		= bIsLocalSpace ? INDEX_NONE : WorldToLocalIndex;
	TransformRemap[int(ENiagaraCoordinateSpace::Simulation)][int(ENiagaraCoordinateSpace::World)]		= bIsLocalSpace ? LocalToWorldIndex : INDEX_NONE;
	TransformRemap[int(ENiagaraCoordinateSpace::Simulation)][int(ENiagaraCoordinateSpace::Simulation)]	= INDEX_NONE;

	Transforms[LocalToWorldIndex] = LocalToWorld;
	Transforms[WorldToLocalIndex] = LocalToWorld.Inverse();
}

bool FNiagaraStatelessSpaceTransforms::UpdateTransforms(const FTransform3f& LocalToWorld)
{
	if (LocalToWorld.Equals(Transforms[0]))
	{
		return false;
	}
	Transforms[LocalToWorldIndex] = LocalToWorld;
	Transforms[WorldToLocalIndex] = LocalToWorld.Inverse();
	return true;
}

const FTransform3f& FNiagaraStatelessSpaceTransforms::GetTransform(ENiagaraCoordinateSpace SourceSpace, ENiagaraCoordinateSpace DestinationSpace) const
{
	check(int(SourceSpace) >= 0 && int(SourceSpace) < TransformRemapSize && int(DestinationSpace) >= 0 && int(DestinationSpace) < TransformRemapSize);
	const int8 i = TransformRemap[int(SourceSpace)][int(DestinationSpace)];
	return i == INDEX_NONE ? FTransform3f::Identity : Transforms[i];
}

FQuat4f FNiagaraStatelessSpaceTransforms::TransformRotation(ENiagaraCoordinateSpace SourceSpace, ENiagaraCoordinateSpace DestinationSpace, FQuat4f Rotation) const
{
	check(int(SourceSpace) >= 0 && int(SourceSpace) < TransformRemapSize && int(DestinationSpace) >= 0 && int(DestinationSpace) < TransformRemapSize);
	const int8 i = TransformRemap[int(SourceSpace)][int(DestinationSpace)];
	return i == INDEX_NONE ? Rotation : Transforms[i].TransformRotation(Rotation);
}

FVector3f FNiagaraStatelessSpaceTransforms::TransformPosition(ENiagaraCoordinateSpace SourceSpace, ENiagaraCoordinateSpace DestinationSpace, FVector3f Position) const
{
	check(int(SourceSpace) >= 0 && int(SourceSpace) < TransformRemapSize && int(DestinationSpace) >= 0 && int(DestinationSpace) < TransformRemapSize);
	const int8 i = TransformRemap[int(SourceSpace)][int(DestinationSpace)];
	return i == INDEX_NONE ? Position : Transforms[i].TransformPosition(Position);
}

FVector3f FNiagaraStatelessSpaceTransforms::TransformVector(ENiagaraCoordinateSpace SourceSpace, ENiagaraCoordinateSpace DestinationSpace, FVector3f Vector) const
{
	check(int(SourceSpace) >= 0 && int(SourceSpace) < TransformRemapSize && int(DestinationSpace) >= 0 && int(DestinationSpace) < TransformRemapSize);
	const int8 i = TransformRemap[int(SourceSpace)][int(DestinationSpace)];
	return i == INDEX_NONE ? Vector : Transforms[i].TransformVector(Vector);
}

FVector3f FNiagaraStatelessSpaceTransforms::TransformVectorNoScale(ENiagaraCoordinateSpace SourceSpace, ENiagaraCoordinateSpace DestinationSpace, FVector3f Vector) const
{
	check(int(SourceSpace) >= 0 && int(SourceSpace) < TransformRemapSize && int(DestinationSpace) >= 0 && int(DestinationSpace) < TransformRemapSize);
	const int8 i = TransformRemap[int(SourceSpace)][int(DestinationSpace)];
	return i == INDEX_NONE ? Vector : Transforms[i].TransformVectorNoScale(Vector);
}

namespace NiagaraStatelessCommon
{
	static FNiagaraStatelessGlobals					GGlobals;
	static TOptional<ENiagaraStatelessFeatureMask>	GUpdatedFeatureMask;

	static void SetUpdateFeatureMask(ENiagaraStatelessFeatureMask Mask, bool bEnabled)
	{
		ENiagaraStatelessFeatureMask NewMask = GUpdatedFeatureMask.Get(GGlobals.FeatureMask);
		if (bEnabled)
		{
			EnumAddFlags(NewMask, Mask);
		}
		else
		{
			EnumRemoveFlags(NewMask, Mask);
		}
		GUpdatedFeatureMask = NewMask;
	}

	#define DEFINE_FEATURE_MASK(ENUM, DESC) \
		static bool bFeatureMask_##ENUM = true; \
		static FAutoConsoleVariableRef CVarFeatureMask_##ENUM( \
			TEXT("fx.NiagaraStateless.Feature."#ENUM), \
			bFeatureMask_##ENUM, \
			TEXT(DESC), \
			FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*) {SetUpdateFeatureMask(ENiagaraStatelessFeatureMask::ENUM, bFeatureMask_##ENUM);}), \
			ECVF_Scalability | ECVF_Default \
		)

		DEFINE_FEATURE_MASK(ExecuteGPU, "When enabled simulations are allowed to execute on the GPU");
		DEFINE_FEATURE_MASK(ExecuteCPU, "When enabled simulations are allowed to execute on the CPU");

	#undef DEFINE_FEATURE_MASK

	void Initialize()
	{
		GGlobals.CameraOffsetVariable				= FNiagaraVariableBase(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CameraOffset"));
		GGlobals.ColorVariable						= FNiagaraVariableBase(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color"));
		GGlobals.DynamicMaterialParameters0Variable = FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec4Def(), TEXT("DynamicMaterialParameter"));
		GGlobals.DynamicMaterialParameters1Variable = FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec4Def(), TEXT("DynamicMaterialParameter1"));
		GGlobals.DynamicMaterialParameters2Variable = FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec4Def(), TEXT("DynamicMaterialParameter2"));
		GGlobals.DynamicMaterialParameters3Variable = FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec4Def(), TEXT("DynamicMaterialParameter3"));
		GGlobals.MaterialRandomVariable				= FNiagaraVariableBase(FNiagaraTypeDefinition::GetFloatDef(), TEXT("MaterialRandom"));
		GGlobals.MeshIndexVariable					= FNiagaraVariableBase(FNiagaraTypeDefinition::GetIntDef(), TEXT("MeshIndex"));
		GGlobals.MeshOrientationVariable			= FNiagaraVariableBase(FNiagaraTypeDefinition::GetQuatDef(), TEXT("MeshOrientation"));
		GGlobals.PositionVariable					= FNiagaraVariableBase(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position"));
		GGlobals.RibbonWidthVariable				= FNiagaraVariableBase(FNiagaraTypeDefinition::GetFloatDef(), TEXT("RibbonWidth"));
		GGlobals.ScaleVariable						= FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale"));
		GGlobals.SpriteAlignmentVariable			= FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec3Def(), TEXT("SpriteAlignment"));
		GGlobals.SpriteFacingVariable				= FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec3Def(), TEXT("SpriteFacing"));
		GGlobals.SpriteSizeVariable					= FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec2Def(), TEXT("SpriteSize"));
		GGlobals.SpriteRotationVariable				= FNiagaraVariableBase(FNiagaraTypeDefinition::GetFloatDef(), TEXT("SpriteRotation"));
		GGlobals.SubImageIndexVariable				= FNiagaraVariableBase(FNiagaraTypeDefinition::GetFloatDef(), TEXT("SubImageIndex"));
		GGlobals.UniqueIDVariable					= FNiagaraVariableBase(FNiagaraTypeDefinition::GetIntDef(), TEXT("UniqueID"));
		GGlobals.VelocityVariable					= FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity"));

		GGlobals.PreviousCameraOffsetVariable		= FNiagaraVariableBase(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Previous.CameraOffset"));
		//GGlobals.PreviousColorVariable			= FNiagaraVariableBase(FNiagaraTypeDefinition::GetColorDef(), TEXT("Previous.Color"));
		//GGlobals.PreviousDynamicMaterialParameters0Variable		= FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec4Def(), TEXT("Previous.DynamicMaterialParameter"));
		GGlobals.PreviousMeshOrientationVariable	= FNiagaraVariableBase(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Previous.MeshOrientation"));
		GGlobals.PreviousPositionVariable			= FNiagaraVariableBase(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Previous.Position"));
		GGlobals.PreviousRibbonWidthVariable		= FNiagaraVariableBase(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Previous.RibbonWidth"));
		GGlobals.PreviousScaleVariable				= FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Previous.Scale"));
		GGlobals.PreviousSpriteAlignmentVariable	= FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Previous.SpriteAlignment"));
		GGlobals.PreviousSpriteFacingVariable		= FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Previous.SpriteFacing"));
		GGlobals.PreviousSpriteSizeVariable			= FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec2Def(), TEXT("Previous.SpriteSize"));
		GGlobals.PreviousSpriteRotationVariable		= FNiagaraVariableBase(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Previous.SpriteRotation"));
		GGlobals.PreviousVelocityVariable			= FNiagaraVariableBase(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Previous.Velocity"));

		UNiagaraStatelessEmitterTemplate::InitCDOPropertiesAfterModuleStartup();
	}

	void UpdateSettings()
	{
		if (!GUpdatedFeatureMask.IsSet())
		{
			return;
		}

		if (GUpdatedFeatureMask.GetValue() != GGlobals.FeatureMask)
		{
			GGlobals.FeatureMask = GUpdatedFeatureMask.GetValue();

			for (TObjectIterator<UNiagaraSystem> It; It; ++It)
			{
				if (UNiagaraSystem* System = *It)
				{
					System->UpdateScalability();
				}
			}
		}
		GUpdatedFeatureMask.Reset();
	}

	FNiagaraVariableBase ConvertParticleVariableToStateless(const FNiagaraVariableBase& InVariable)
	{
		FNiagaraVariableBase Variable = InVariable;
		Variable.RemoveRootNamespace(FNiagaraConstants::ParticleAttributeNamespaceString);
		return Variable;
	}
} //NiagaraStatelessCommon

const FNiagaraStatelessGlobals& FNiagaraStatelessGlobals::Get()
{
	return NiagaraStatelessCommon::GGlobals;
}
