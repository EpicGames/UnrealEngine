// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessComputeManager.h"
#include "Stateless/NiagaraStatelessEmitterData.h"
#include "Stateless/NiagaraStatelessEmitterInstance.h"
#include "Stateless/NiagaraStatelessSimulationShader.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraGPUInstanceCountManager.h"
#include "NiagaraRenderer.h"

#include "GPUSortManager.h"
#include "PrimitiveSceneProxy.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DECLARE_STATS_GROUP(TEXT("NiagaraStateless"), STATGROUP_NiagaraStateless, STATCAT_NiagaraStateless);

DECLARE_CYCLE_STAT(TEXT("GetDataBuffer"), STAT_NiagaraStateless_GetDataBuffer, STATGROUP_NiagaraStateless);
DECLARE_CYCLE_STAT(TEXT("GenerateGPUData"), STAT_NiagaraStateless_GenerateGPUData, STATGROUP_NiagaraStateless);
DECLARE_DWORD_COUNTER_STAT(TEXT("CPU Simulate"), STAT_NiagaraStateless_CPUSimulate, STATGROUP_NiagaraStateless);
DECLARE_DWORD_COUNTER_STAT(TEXT("GPU Simulate"), STAT_NiagaraStateless_GPUSimulate, STATGROUP_NiagaraStateless);
DECLARE_DWORD_COUNTER_STAT(TEXT("CPU Particle Count"), STAT_NiagaraStateless_CPUParticleCount, STATGROUP_NiagaraStateless);
DECLARE_DWORD_COUNTER_STAT(TEXT("GPU Particle Estimate"), STAT_NiagaraStateless_GPUParticleEstimate, STATGROUP_NiagaraStateless);
DECLARE_MEMORY_STAT(TEXT("Buffer Memory"), STAT_NiagaraStateless_BufferMemory, STATGROUP_NiagaraStateless);

DECLARE_GPU_STAT_NAMED(NiagaraStatelessGPUSimulation, TEXT("Niagara Stateless GPU Simulation"));

namespace NiagaraStatelessComputeManagerPrivate
{
	enum class EComputeExecutionPath
	{
		None,
		CPU,
		GPU,
	};

	bool GUseDataBufferCache = true;
	FAutoConsoleVariableRef CVarUseCache(
		TEXT("fx.NiagaraStateless.ComputeManager.UseCache"),
		GUseDataBufferCache,
		TEXT("When enabled we will attempt to reuse allocated buffers between frames."),
		ECVF_Default
	);

	int32 GParticleCountCPUThreshold = 0;
	FAutoConsoleVariableRef CVarCPUThreshold(
		TEXT("fx.NiagaraStateless.ComputeManager.CPUThreshold"),
		GParticleCountCPUThreshold,
		TEXT("When lower than this particle count prefer to use the CPU over dispatching a compute shader."),
		ECVF_Default
	);

	EComputeExecutionPath DetermineComputeExecutionPath(const FNiagaraStatelessEmitterData* EmitterData, uint32 ActiveParticlesEstimate, bool bAllowGPUGeneration)
	{
		const bool bAllowGPUExec = EnumHasAnyFlags(EmitterData->FeatureMask, ENiagaraStatelessFeatureMask::ExecuteGPU) && bAllowGPUGeneration;
		const bool bUseCPUExec = EnumHasAnyFlags(EmitterData->FeatureMask, ENiagaraStatelessFeatureMask::ExecuteCPU) && (!bAllowGPUExec || (ActiveParticlesEstimate <= uint32(GParticleCountCPUThreshold)));
		if (bUseCPUExec)
		{
			return EComputeExecutionPath::CPU;
		}
		if (bAllowGPUExec)
		{
			check(EmitterData->GetShader().IsValid());
			return EComputeExecutionPath::GPU;
		}
		return EComputeExecutionPath::None;
	}

	bool GenerateCPUDataForCPUSim(const NiagaraStateless::FEmitterInstance_RT* EmitterInstance, FNiagaraDataBuffer* DestinationBuffer)
	{
		using namespace NiagaraStateless;

		const FNiagaraStatelessEmitterData* EmitterData = EmitterInstance->EmitterData.Get();

		FParticleSimulationContext ParticleSimulation(EmitterData, EmitterInstance->ShaderParameters.Get(), EmitterInstance->BindingBufferData);
		ParticleSimulation.Simulate(EmitterInstance->RandomSeed, EmitterInstance->Age, EmitterInstance->DeltaTime, EmitterInstance->SpawnInfos, DestinationBuffer);

		INC_DWORD_STAT_BY(STAT_NiagaraStateless_CPUSimulate, 1);
		INC_DWORD_STAT_BY(STAT_NiagaraStateless_CPUParticleCount, ParticleSimulation.GetNumInstances());
		return ParticleSimulation.GetNumInstances() > 0;
	}

	bool GenerateCPUDataForGPUSim(FRHICommandListBase& RHICmdList, const NiagaraStateless::FEmitterInstance_RT* EmitterInstance, FNiagaraDataBuffer* DestinationBuffer)
	{
		using namespace NiagaraStateless;

		const FNiagaraStatelessEmitterData* EmitterData = EmitterInstance->EmitterData.Get();

		FParticleSimulationContext ParticleSimulation(EmitterData, EmitterInstance->ShaderParameters.Get(), EmitterInstance->BindingBufferData);
		ParticleSimulation.SimulateGPU(RHICmdList, EmitterInstance->RandomSeed, EmitterInstance->Age, EmitterInstance->DeltaTime, EmitterInstance->SpawnInfos, DestinationBuffer);

		INC_DWORD_STAT_BY(STAT_NiagaraStateless_CPUSimulate, 1);
		INC_DWORD_STAT_BY(STAT_NiagaraStateless_CPUParticleCount, ParticleSimulation.GetNumInstances());
		return ParticleSimulation.GetNumInstances() > 0;
	}

	void GenerateGPUData(FRHICommandList& RHICmdList, FNiagaraGpuComputeDispatchInterface* ComputeInterface, TConstArrayView<FNiagaraStatelessComputeManager::FStatelessDataGenerationRequest> GenerationRequests)
	{
		const int32 NumJobs = GenerationRequests.Num();
		INC_DWORD_STAT_BY(STAT_NiagaraStateless_GPUSimulate, NumJobs);
		SCOPE_CYCLE_COUNTER(STAT_NiagaraStateless_GenerateGPUData);

		// Get Count Buffer
		FNiagaraGPUInstanceCountManager& CountManager = ComputeInterface->GetGPUInstanceCounterManager();
		FRHIUnorderedAccessView* CountBufferUAV = CountManager.GetInstanceCountBuffer().UAV;

		// Build Transitions
		TArray<FRHITransitionInfo> TransitionsBefore;
		TArray<FRHITransitionInfo> TransitionsAfter;
		{
			TransitionsBefore.Reserve(1 + (NumJobs * 2));
			TransitionsAfter.Reserve(1 + (NumJobs * 2));

			TransitionsBefore.Emplace(CountManager.GetInstanceCountBuffer().Buffer, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState, ERHIAccess::UAVCompute);
			TransitionsAfter.Emplace(CountManager.GetInstanceCountBuffer().Buffer, ERHIAccess::UAVCompute, FNiagaraGPUInstanceCountManager::kCountBufferDefaultState);

			for (const FNiagaraStatelessComputeManager::FStatelessDataGenerationRequest& GenerationRequest : GenerationRequests)
			{
				FNiagaraDataBuffer* DestinationData = GenerationRequest.DestinationData;

				const FRWBuffer& FloatBuffer = DestinationData->GetGPUBufferFloat();
				if (FloatBuffer.NumBytes > 0)
				{
					TransitionsBefore.Emplace(FloatBuffer.Buffer, ERHIAccess::SRVMask, ERHIAccess::UAVCompute);
					TransitionsAfter.Emplace(FloatBuffer.Buffer, ERHIAccess::UAVCompute, ERHIAccess::SRVMask);
				}
				const FRWBuffer& IntBuffer = DestinationData->GetGPUBufferInt();
				if (IntBuffer.NumBytes > 0)
				{
					TransitionsBefore.Emplace(IntBuffer.Buffer, ERHIAccess::SRVMask, ERHIAccess::UAVCompute);
					TransitionsAfter.Emplace(IntBuffer.Buffer, ERHIAccess::UAVCompute, ERHIAccess::SRVMask);
				}

				// Flush any pending updates to the binding buffer as this can break parallel dispatches on the GPU
				const NiagaraStateless::FEmitterInstance_RT* EmitterInstance = GenerationRequest.EmitterInstance;
				if (EmitterInstance->bBindingBufferDirty)
				{
					EmitterInstance->bBindingBufferDirty = false;
					EmitterInstance->BindingBuffer.Release();

					if (EmitterInstance->BindingBufferData.Num())
					{
						EmitterInstance->BindingBuffer.Initialize(RHICmdList, TEXT("FNiagaraStatelessEmitterInstance::BindingBuffer"), EmitterInstance->BindingBufferData.Num(), EBufferUsageFlags::Static);
						void* LockedBuffer = RHICmdList.LockBuffer(EmitterInstance->BindingBuffer.Buffer, 0, EmitterInstance->BindingBuffer.NumBytes, RLM_WriteOnly);
						FMemory::Memcpy(LockedBuffer, EmitterInstance->BindingBufferData.GetData(), EmitterInstance->BindingBuffer.NumBytes);
						RHICmdList.UnlockBuffer(EmitterInstance->BindingBuffer.Buffer);
					}
				}
			}
		}

		FNiagaraEmptyUAVPoolScopedAccess UAVPoolAccessScope(ComputeInterface->GetEmptyUAVPool());
		FRHIUnorderedAccessView* EmptyFloatBufferUAV = ComputeInterface->GetEmptyUAVFromPool(RHICmdList, PF_R32_FLOAT, ENiagaraEmptyUAVType::Buffer);
		FRHIUnorderedAccessView* EmptyIntBufferUAV = ComputeInterface->GetEmptyUAVFromPool(RHICmdList, PF_R32_SINT, ENiagaraEmptyUAVType::Buffer);

		// Execute Simulations
		RHICmdList.Transition(TransitionsBefore);
		RHICmdList.BeginUAVOverlap(CountBufferUAV);
		for (const FNiagaraStatelessComputeManager::FStatelessDataGenerationRequest& GenerationRequest : GenerationRequests)
		{
			const NiagaraStateless::FEmitterInstance_RT* EmitterInstance = GenerationRequest.EmitterInstance;
			const FNiagaraStatelessEmitterData* EmitterData = EmitterInstance->EmitterData.Get();
			FNiagaraDataBuffer* DestinationData = GenerationRequest.DestinationData;

			// Update parameters for this compute invocation
			NiagaraStateless::FCommonShaderParameters* ShaderParameters = EmitterInstance->ShaderParameters.Get();
			ShaderParameters->Common_SimulationTime = EmitterInstance->Age;
			ShaderParameters->Common_SimulationDeltaTime = EmitterInstance->DeltaTime;
			ShaderParameters->Common_SimulationInvDeltaTime = EmitterInstance->DeltaTime > 0.0f ? (1.0f / EmitterInstance->DeltaTime) : 0.0f;
			ShaderParameters->Common_OutputBufferStride = DestinationData->GetFloatStride() / sizeof(float);
			ShaderParameters->Common_GPUCountBufferOffset = DestinationData->GetGPUInstanceCountBufferOffset();
			ShaderParameters->Common_FloatOutputBuffer = DestinationData->GetGPUBufferFloat().UAV.IsValid() ? DestinationData->GetGPUBufferFloat().UAV.GetReference() : EmptyFloatBufferUAV;
			//ShaderParameters->Common_HalfOutputBuffer		= DestinationData->GetGPUBufferHalf().UAV;
			ShaderParameters->Common_IntOutputBuffer = DestinationData->GetGPUBufferInt().UAV.IsValid() ? DestinationData->GetGPUBufferInt().UAV.GetReference() : EmptyIntBufferUAV;
			ShaderParameters->Common_GPUCountBuffer = CountBufferUAV;
			ShaderParameters->Common_StaticDataBuffer = EmitterData->StaticDataBufferGpu.SRV;
			ShaderParameters->Common_DynamicDataBuffer = FNiagaraRenderer::GetSrvOrDefaultByteAddress(EmitterInstance->BindingBuffer.SRV);

			// Execute the simulation
			TShaderRef<NiagaraStateless::FSimulationShader> ComputeShader = EmitterData->GetShader();
			FRHIComputeShader* ComputeShaderRHI = ComputeShader.GetComputeShader();
			const uint32 NumThreadGroups = FMath::DivideAndRoundUp<uint32>(GenerationRequest.ActiveParticles, NiagaraStateless::FSimulationShader::ThreadGroupSize);

			const FIntVector NumWrappedThreadGroups = FComputeShaderUtils::GetGroupCountWrapped(NumThreadGroups);
			FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, EmitterData->GetShaderParametersMetadata(), *ShaderParameters, NumWrappedThreadGroups);
		}
		RHICmdList.EndUAVOverlap(CountBufferUAV);

		RHICmdList.Transition(TransitionsAfter);
	}
}

FNiagaraStatelessComputeManager::FStatelessDataCache::~FStatelessDataCache()
{
	if (DataBuffer)
	{
		DataBuffer->Destroy();
		DataBuffer = nullptr;
	}
}

FNiagaraStatelessComputeManager::FNiagaraStatelessComputeManager(FNiagaraGpuComputeDispatchInterface* InOwnerInterface)
	: FNiagaraGpuComputeDataManager(InOwnerInterface)
{
	InOwnerInterface->GetOnPreInitViewsEvent().AddRaw(this, &FNiagaraStatelessComputeManager::OnPreInitViews);
	InOwnerInterface->GetOnPreRenderEvent().AddRaw(this, &FNiagaraStatelessComputeManager::OnPreRender);
	InOwnerInterface->GetOnPostRenderEvent().AddRaw(this, &FNiagaraStatelessComputeManager::OnPostPostRender);
}

FNiagaraStatelessComputeManager::~FNiagaraStatelessComputeManager()
{
#if NIAGARA_MEMORY_TRACKING && STATS
	for (const TUniquePtr<FStatelessDataCache>& CacheData : FreeData)
	{
		DEC_MEMORY_STAT_BY(STAT_NiagaraStateless_BufferMemory, CacheData->DataBuffer->GetAllocationSizeBytes());
	}
	ensure(UsedData.Num() == 0);
#endif
}

FNiagaraDataBuffer* FNiagaraStatelessComputeManager::GetDataBuffer(FRHICommandListBase& RHICmdList, uintptr_t EmitterKey, const NiagaraStateless::FEmitterInstance_RT* EmitterInstance)
{
	using namespace NiagaraStateless;
	using namespace NiagaraStatelessComputeManagerPrivate;

	SCOPE_CYCLE_COUNTER(STAT_NiagaraStateless_GetDataBuffer);

	// Is there any data to generate?
	if (EmitterInstance->ExecutionState == ENiagaraExecutionState::Complete || EmitterInstance->ExecutionState == ENiagaraExecutionState::Disabled)
	{
		return nullptr;
	}

	const FNiagaraStatelessEmitterData* EmitterData = EmitterInstance->EmitterData.Get();

	uint32 ActiveParticles = 0;
	{
		NiagaraStateless::FCommonShaderParameters* ShaderParameters = EmitterInstance->ShaderParameters.Get();
		ActiveParticles = EmitterData->CalculateActiveParticles(
			EmitterInstance->RandomSeed,
			EmitterInstance->SpawnInfos,
			EmitterInstance->Age,
			&ShaderParameters->SpawnParameters
		);
	}
	if (ActiveParticles == 0)
	{
		return nullptr;
	}

	FNiagaraGpuComputeDispatchInterface* ComputeInterface = GetOwnerInterface();

	// Until we add an extension to the render to notify about GDME start / end we can not allow GPU generation requests outside of PreInitViews / PreRender
	// The shadow rendering, for example, will call GDME outside of this and it will result in crashes
	// Therefore we fallback to CPU generation in these cases, if available
	const bool bAllowGPUGeneration = bAllowDeferredGeneration;
	const EComputeExecutionPath ComputeExecutionPath = DetermineComputeExecutionPath(EmitterData, ActiveParticles, bAllowGPUGeneration);
	uint32 CountOffset = INDEX_NONE;

	// Allocate / Reuse an existing buffer
	FStatelessDataCache* CacheData = nullptr;
	{
		const uint32 DataSetLayoutHash = EmitterInstance->EmitterData->ParticleDataSetCompiledData->GetLayoutHash();

		UE::TScopeLock ScopeLock(GetDataBufferGuard);
		if (TUniquePtr<FStatelessDataCache>* ExistingData = UsedData.Find(EmitterKey))
		{
			return (*ExistingData)->DataBuffer;
		}

		if (GUseDataBufferCache)
		{
			for (int32 i = 0; i < FreeData.Num(); ++i)
			{
				if (FreeData[i]->DataSetLayoutHash == DataSetLayoutHash)
				{
					CacheData = FreeData[i].Release();
					FreeData.RemoveAtSwap(i, EAllowShrinking::No);
					break;
				}
			}
		}

		if (CacheData == nullptr)
		{
			CacheData = new FStatelessDataCache();
			CacheData->DataSetLayoutHash = DataSetLayoutHash;
			CacheData->DataSetCompiledData = EmitterInstance->EmitterData->ParticleDataSetCompiledData;
			CacheData->DataSet.Init(CacheData->DataSetCompiledData.Get());
			CacheData->DataBuffer = new FNiagaraDataBuffer(&CacheData->DataSet);
		}

		// Note: We can not free the data when in paralell GDME mode as multiple lock / unlock operations could result in the wrong one providing the final data
		// The plus side to adding back into the cache data is that multiple calls to get the same emitter's data will resolve to no active particles.
		UsedData.Emplace(EmitterKey, CacheData);

		if (ComputeExecutionPath == EComputeExecutionPath::GPU)
		{
			FNiagaraGPUInstanceCountManager& CountManager = ComputeInterface->GetGPUInstanceCounterManager();
			CountOffset = CountManager.AcquireEntry();
			if (CountOffset != INDEX_NONE)
			{
				GPUGenerationRequests.Emplace(CacheData->DataBuffer, EmitterInstance, ActiveParticles);
				CountsToRelease.Add(CountOffset);
			}
		}
	}

#if NIAGARA_MEMORY_TRACKING && STATS
	DEC_MEMORY_STAT_BY(STAT_NiagaraStateless_BufferMemory, CacheData->DataBuffer->GetAllocationSizeBytes());
#endif

	CacheData->DataBuffer->AllocateGPU(RHICmdList, ActiveParticles, ComputeInterface->GetFeatureLevel(), TEXT("StatelessSimBuffer"));

#if NIAGARA_MEMORY_TRACKING && STATS
	INC_MEMORY_STAT_BY(STAT_NiagaraStateless_BufferMemory, CacheData->DataBuffer->GetAllocationSizeBytes());
#endif

	bool bDidGenerateData = false;

	switch (ComputeExecutionPath)
	{
		case EComputeExecutionPath::CPU:
		{
			bDidGenerateData = GenerateCPUDataForGPUSim(RHICmdList, EmitterInstance, CacheData->DataBuffer);
			break;
		}

		case EComputeExecutionPath::GPU:
		{
			if (CountOffset != INDEX_NONE)
			{
				CacheData->DataBuffer->SetNumInstances(ActiveParticles);
				CacheData->DataBuffer->SetGPUInstanceCountBufferOffset(CountOffset);
				bDidGenerateData = true;
			}
			// If we failed to alocate a count we will need to go through the CPU path (if available)
			// This should never happen as we reserve a count up front via the compute proxy
			// If it does occur this means some other system has used a count slot but not reserved one
			else
			{
				if (FNiagaraUtilities::LogVerboseWarnings())
				{
					ensureMsgf(false, TEXT("Count reserved for stateless was not available, attemping to generate on the CPU."));
				}

				const bool bAllowCPUExec = EnumHasAnyFlags(EmitterInstance->EmitterData->FeatureMask, ENiagaraStatelessFeatureMask::ExecuteCPU);
				bDidGenerateData = bAllowCPUExec && GenerateCPUDataForGPUSim(RHICmdList, EmitterInstance, CacheData->DataBuffer);
			}
			break;
		}

		default:
			ensureMsgf(false, TEXT("No execution path was found for stateless emitter, data will not be generated"));
			break;
	}

	return bDidGenerateData ? CacheData->DataBuffer : nullptr;
}

void FNiagaraStatelessComputeManager::GenerateDataBufferForDebugging(FRHICommandListImmediate& RHICmdList, FNiagaraDataBuffer* DataBuffer, const NiagaraStateless::FEmitterInstance_RT* EmitterInstance) const
{
	using namespace NiagaraStateless;
	using namespace NiagaraStatelessComputeManagerPrivate;

	check(IsInRenderingThread());

	const FNiagaraStatelessEmitterData* EmitterData = EmitterInstance->EmitterData.Get();
	const uint32 ActiveParticlesEstimate = EmitterData->CalculateActiveParticles(
		EmitterInstance->RandomSeed,
		EmitterInstance->SpawnInfos,
		EmitterInstance->Age,
		&EmitterInstance->ShaderParameters->SpawnParameters
	);

	if (ActiveParticlesEstimate == 0)
	{
		DataBuffer->SetNumInstances(0);
		return;
	}

	const bool bAllowGPUGeneration = true;
	const EComputeExecutionPath ComputeExecutionPath = DetermineComputeExecutionPath(EmitterData, ActiveParticlesEstimate, bAllowGPUGeneration);
	switch (ComputeExecutionPath)
	{
		case EComputeExecutionPath::CPU:
		{
			GenerateCPUDataForCPUSim(EmitterInstance, DataBuffer);
			break;
		}

		case EComputeExecutionPath::GPU:
		{
			FNiagaraGpuComputeDispatchInterface* ComputeInterface = GetOwnerInterface();
			FNiagaraGPUInstanceCountManager& CountManager = ComputeInterface->GetGPUInstanceCounterManager();

			// Allocate counter and destination data
			FNiagaraDataBufferRef GPUDataBuffer = new FNiagaraDataBuffer(DataBuffer->GetOwner());

			uint32 CountIndex = CountManager.AcquireOrAllocateEntry(RHICmdList);
			GPUDataBuffer->AllocateGPU(RHICmdList, ActiveParticlesEstimate, ComputeInterface->GetFeatureLevel(), TEXT("StatelessSimBuffer"));
			GPUDataBuffer->SetGPUInstanceCountBufferOffset(CountIndex);

			// Generate the data
			FStatelessDataGenerationRequest GenerationRequest(GPUDataBuffer, EmitterInstance, ActiveParticlesEstimate);

			RHICmdList.BeginUAVOverlap();
			GenerateGPUData(RHICmdList, ComputeInterface, MakeConstArrayView(&GenerationRequest, 1));
			RHICmdList.EndUAVOverlap();

			// Copy to CPU data
			//TransferGPUToCPU(RHICmdList, ComputeInterface, GPUDataBuffer, DataBuffer);
			GPUDataBuffer->TransferGPUToCPUImmediate(RHICmdList, ComputeInterface, DataBuffer);

			// Release the GPU buffer and count
			GPUDataBuffer->ReleaseGPU();
			GPUDataBuffer->SetGPUInstanceCountBufferOffset(INDEX_NONE);
			CountManager.FreeEntry(CountIndex);
			break;
		}

		default:
			break;
	}
}

void FNiagaraStatelessComputeManager::OnPreInitViews(FRDGBuilder& GraphBuilder)
{
	bAllowDeferredGeneration = true;
}

void FNiagaraStatelessComputeManager::OnPreRender(FRDGBuilder& GraphBuilder)
{
	bAllowDeferredGeneration = false;

	// Anything to process?
	if (GPUGenerationRequests.Num() == 0)
	{
		return;
	}

	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, NiagaraStateless);
	RDG_RHI_EVENT_SCOPE_STAT(GraphBuilder, NiagaraStatelessGPUSimulation, NiagaraStateless);
	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	// Execute dispatches
	AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FNiagaraStatelessComputeManager::OnPreRender"),
		[GPUGenerationRequests_RDG=MoveTemp(GPUGenerationRequests), ComputeInterface=GetOwnerInterface()](FRHICommandListImmediate& RHICmdList)
		{
			SCOPED_DRAW_EVENT(RHICmdList, FNiagaraStatelessComputeManager_OnPreRender);

			FNiagaraGPUInstanceCountManager& CountManager = ComputeInterface->GetGPUInstanceCounterManager();
			NiagaraStatelessComputeManagerPrivate::GenerateGPUData(RHICmdList, ComputeInterface, GPUGenerationRequests_RDG);
		}
	);
	GPUGenerationRequests.Empty();
}

void FNiagaraStatelessComputeManager::OnPostPostRender(FRDGBuilder& GraphBuilder)
{
	// Anything to process?
	if (UsedData.Num() + FreeData.Num() + CountsToRelease.Num() == 0)
	{
		return;
	}

	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, NiagaraStateless);
	RDG_RHI_EVENT_SCOPE_STAT(GraphBuilder, NiagaraStatelessGPUSimulation, NiagaraStateless);
	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FNiagaraStatelessComputeManager::OnPostPostRender"),
		[this](FRHICommandListImmediate& RHICmdList)
		{
		#if NIAGARA_MEMORY_TRACKING && STATS
			for (const TUniquePtr<FStatelessDataCache>& CacheData : FreeData)
			{
				DEC_MEMORY_STAT_BY(STAT_NiagaraStateless_BufferMemory, CacheData->DataBuffer->GetAllocationSizeBytes());
			}
		#endif

			FreeData.Empty(UsedData.Num());
			for (auto it=UsedData.CreateIterator(); it; ++it)
			{
				it.Value()->DataBuffer->SetGPUInstanceCountBufferOffset(INDEX_NONE);
				FreeData.Emplace(it.Value().Release());
			}
			UsedData.Empty();

			if (CountsToRelease.Num() > 0)
			{
				FNiagaraGpuComputeDispatchInterface* ComputeInterface = GetOwnerInterface();
				FNiagaraGPUInstanceCountManager& CountManager = ComputeInterface->GetGPUInstanceCounterManager();
				CountManager.FreeEntryArray(CountsToRelease);
				CountsToRelease.Reset();
			}
		}
	);
}
