// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraGpuComputeDataManager.h"
#include "NiagaraDataSet.h"
#include "NiagaraRendererProperties.h"

#include "RenderGraphFwd.h"

class FSceneViewFamily;
class FNiagaraDataBuffer;
class FNiagaraGpuComputeDispatchInterface;
namespace NiagaraStateless
{
	class FEmitterInstance_RT;
}

class FNiagaraStatelessComputeManager final : public FNiagaraGpuComputeDataManager
{
public:
	struct FStatelessDataCache
	{
		~FStatelessDataCache();

		uint32										DataSetLayoutHash = 0;
		TSharedPtr<FNiagaraDataSetCompiledData>		DataSetCompiledData;
		FNiagaraDataSet								DataSet;
		FNiagaraDataBufferRef						DataBuffer;
	};

	struct FStatelessDataGenerationRequest
	{
		FStatelessDataGenerationRequest() = default;
		explicit FStatelessDataGenerationRequest(FNiagaraDataBuffer* InDestinationData, const NiagaraStateless::FEmitterInstance_RT* InEmitterInstance, uint32 InActiveParticles)
			: DestinationData(InDestinationData)
			, EmitterInstance(InEmitterInstance)
			, ActiveParticles(InActiveParticles)
		{
		}

		FNiagaraDataBufferRef							DestinationData;
		const NiagaraStateless::FEmitterInstance_RT*	EmitterInstance = nullptr;
		uint32											ActiveParticles = 0;
	};

public:
	explicit FNiagaraStatelessComputeManager(FNiagaraGpuComputeDispatchInterface* InOwnerInterface);
	virtual ~FNiagaraStatelessComputeManager();

	static FName GetManagerName()
	{
		static FName ManagerName("FNiagaraStatelessComputeManager");
		return ManagerName;
	}

	FNiagaraDataBuffer* GetDataBuffer(FRHICommandListBase& RHICmdList, uintptr_t EmitterKey, const NiagaraStateless::FEmitterInstance_RT* EmitterInstance);

	// Used to execute the simulation immediately into a CPU side data buffer
	void GenerateDataBufferForDebugging(FRHICommandListImmediate& RHICmdList, FNiagaraDataBuffer* DataBuffer, const NiagaraStateless::FEmitterInstance_RT* EmitterInstance) const;

private:
	void OnPreInitViews(FRDGBuilder& GraphBuilder);
	void OnPreRender(FRDGBuilder& GraphBuilder);
	void OnPostPostRender(FRDGBuilder& GraphBuilder);

private:
	TMap<uintptr_t, TUniquePtr<FStatelessDataCache>>	UsedData;
	TArray<TUniquePtr<FStatelessDataCache>>				FreeData;
	TArray<uint32>										CountsToRelease;

	bool												bAllowDeferredGeneration = false;
	TArray<FStatelessDataGenerationRequest>				GPUGenerationRequests;

	UE::FMutex											GetDataBufferGuard;
};
