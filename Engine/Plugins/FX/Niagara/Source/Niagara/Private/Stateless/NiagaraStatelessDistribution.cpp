// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessDistribution.h"
#include "Stateless/NiagaraStatelessExpression.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessDistribution)

namespace NiagaraStatelessDistributionPrivate
{
#if WITH_EDITORONLY_DATA
	bool GOptimizeLUTs = true;
	FAutoConsoleVariableRef CVarNiagaraStatelessDistributionOptimizeLUTs(
		TEXT("fx.NiagaraStateless.Distribution.OptimizeLUTs"),
		GOptimizeLUTs,
		TEXT("When enabled we optimize the LUT generation."),
		ECVF_Default
	);

	bool GReduceLUTTimeRange = true;
	FAutoConsoleVariableRef CVarNiagaraStatelessDistributionReduceLUTTimeRange(
		TEXT("fx.NiagaraStateless.Distribution.ReduceLUTTimeRange"),
		GReduceLUTTimeRange,
		TEXT("When LUT optimization is enabled we will try to reduce the start / end time if we get duplicate keys, this can improve resolution in the area that matters."),
		ECVF_Default
	);

	TArray<float> CurvesToLUT(TArrayView<const FRichCurve> Curves, int32 NumSamples, const FVector2f& TimeRange, ENiagaraDistributionCurveLUTMode LUTMode)
	{
		const int32 NumChannels = Curves.Num();

		const float Duration = TimeRange.Y - TimeRange.X;
		const float SampleToDuration = Duration > 0.0f ? Duration / float(NumSamples - 1) : 0.0f;

		TArray<float> LUT;
		LUT.SetNumUninitialized(NumChannels * NumSamples);
		if (LUTMode == ENiagaraDistributionCurveLUTMode::Sample)
		{
			for (int32 iSample = 0; iSample < NumSamples; ++iSample)
			{
				const float Time = (float(iSample) * SampleToDuration) + TimeRange.X;
				for (int32 iChannel = 0; iChannel < NumChannels; ++iChannel)
				{
					LUT[(iSample * NumChannels) + iChannel] = Curves[iChannel].Eval(Time);
				}
			}
		}
		else //if (LUTMode == ENiagaraDistributionCurveLUTMode::Accumulate)
		{
			const float SampleDelta = NumSamples > 1 ? float(NumSamples - 1) : 1.0f;

			TArray<float> ChannelSum;
			ChannelSum.AddZeroed(Curves.Num());
			for (int32 iSample = 0; iSample < NumSamples; ++iSample)
			{
				const float Time = (float(iSample) * SampleToDuration) + TimeRange.X;
				for (int32 iChannel = 0; iChannel < NumChannels; ++iChannel)
				{
					LUT[(iSample * NumChannels) + iChannel] = ChannelSum[iChannel];
					ChannelSum[iChannel] += Curves[iChannel].Eval(Time) / SampleDelta;
				}
			}
		}
		return LUT;
	}

	FVector2f CurvesFindTimeRange(TArrayView<const FRichCurve> Curves, int32 NumSamples, ENiagaraDistributionCurveLUTMode LUTMode)
	{
		const int32 NumChannels = Curves.Num();
		check(NumChannels > 0);

		// Find time range based on keys
		FVector2f TimeRange = FVector2f(0.0f, 1.0f);
		Curves[0].GetTimeRange(TimeRange.X, TimeRange.Y);
		for (int32 iChannel = 1; iChannel < NumChannels; ++iChannel)
		{
			FVector2f ChannelTimeRange = TimeRange;
			Curves[iChannel].GetTimeRange(ChannelTimeRange.X, ChannelTimeRange.Y);
			TimeRange.X = FMath::Min(TimeRange.X, ChannelTimeRange.X);
			TimeRange.Y = FMath::Max(TimeRange.Y, ChannelTimeRange.Y);
		}

		// Reduce start / end times if the sampled results match
		if (GReduceLUTTimeRange)
		{
			TArray<float> LUT = CurvesToLUT(Curves, NumSamples, TimeRange, LUTMode);

			auto DoKeysMatch =
				[&LUT, &NumChannels](int iCurrKey, int iNextKey)
				{
					for (int32 iChannel = 0; iChannel < NumChannels; ++iChannel)
					{
						if (!FMath::IsNearlyEqual(LUT[(iCurrKey * NumChannels) + iChannel], LUT[(iNextKey * NumChannels) + iChannel]))
						{
							return false;
						}
					}
					return true;
				};

			int32 iFirstKey = 0;
			while (iFirstKey < NumSamples - 1 && DoKeysMatch(iFirstKey, iFirstKey + 1))
			{
				++iFirstKey;
			}

			int32 iLastKey = NumSamples - 1;
			while (iLastKey > 0 && DoKeysMatch(iLastKey, iLastKey - 1))
			{
				--iLastKey;
			}

			if (iFirstKey >= iLastKey)
			{
				TimeRange.Y = TimeRange.X;
			}
			else
			{
				const float StartTime = TimeRange.X;
				const float Duration = TimeRange.Y - TimeRange.X;
				const float SampleToDuration = Duration > 0.0f ? Duration / float(NumSamples - 1) : 0.0f;
				TimeRange.X = StartTime + (float(iFirstKey) * SampleToDuration);
				TimeRange.Y = StartTime + (float(iLastKey) * SampleToDuration);
			}
		}

		return TimeRange;
	}

	bool AreLUTsAlmostEqual(TArrayView<float> Lhs, TArrayView<float> Rhs, int32 NumChannels, float ErrorThreshold = 0.01f)
	{
		const int32 LhsNumSamples = Lhs.Num() / NumChannels;
		const int32 RhsNumSamples = Rhs.Num() / NumChannels;
		const int32 MaxSamples = FMath::Max(LhsNumSamples, RhsNumSamples);
		for ( int32 i=0; i < MaxSamples; ++i )
		{
			const float U = float(i) / float(MaxSamples - 1);

			const float LhsT = U * float(LhsNumSamples - 1);
			const float LhsU = FMath::Frac(LhsT);
			const int32 LhsA = FMath::Min(FMath::FloorToInt(LhsT), LhsNumSamples - 1);
			const int32 LhsB = FMath::Min(LhsA + 1, LhsNumSamples - 1);

			const float RhsT = U * float(RhsNumSamples - 1);
			const float RhsU = FMath::Frac(RhsT);
			const int32 RhsA = FMath::Min(FMath::FloorToInt(RhsT), RhsNumSamples - 1);
			const int32 RhsB = FMath::Min(RhsA + 1, RhsNumSamples - 1);

			for (int iChannel=0; iChannel < NumChannels; ++iChannel)
			{
				const float LhsValue = FMath::Lerp(Lhs[LhsA * NumChannels + iChannel], Lhs[LhsB * NumChannels + iChannel], LhsU);
				const float RhsValue = FMath::Lerp(Rhs[RhsA * NumChannels + iChannel], Rhs[RhsB * NumChannels + iChannel], RhsU);
				const float Error = FMath::Abs(LhsValue - RhsValue);
				if (Error > ErrorThreshold)
				{
					return false;
				}
			}
		}
		return true;
	}

	TArray<float> CurvesToOptimizedLUT(TArrayView<const FRichCurve> Curves, int32 MaxLutSampleCount, ENiagaraDistributionCurveLUTMode LUTMode, FVector2f& OutTimeRange)
	{
		if (LUTMode == ENiagaraDistributionCurveLUTMode::Sample)
		{
			OutTimeRange = CurvesFindTimeRange(Curves, MaxLutSampleCount, LUTMode);
			if (FMath::IsNearlyEqual(OutTimeRange.X, OutTimeRange.Y))
			{
				TArray<float> LUT = CurvesToLUT(Curves, 2, OutTimeRange, LUTMode);
				return LUT;
			}
		}

		TArray<float> LUT = CurvesToLUT(Curves, MaxLutSampleCount, OutTimeRange, LUTMode);
		if (GOptimizeLUTs)
		{
			for (int32 iSamples = 2; iSamples < MaxLutSampleCount; ++iSamples)
			{
				TArray<float> NewLUT = CurvesToLUT(Curves, iSamples, OutTimeRange, LUTMode);
				if (AreLUTsAlmostEqual(NewLUT, LUT, Curves.Num()))
				{
					return NewLUT;
				}
			}
		}
		return LUT;
	}

	void InternalCreateCurveFromValues(TConstArrayView<float> Values, TArray<FRichCurve>& ChannelCurves, TArray<float>& ChannelConstantsAndRanges, int NumChannels)
	{
		check((Values.Num() % NumChannels) == 0);

		ChannelCurves.Reset(NumChannels);
		ChannelCurves.AddDefaulted(NumChannels);

		ChannelConstantsAndRanges = Values;

		const int32 NumValues = Values.Num() / NumChannels;
		if (NumValues > 0)
		{
			const float UInc = NumValues > 1 ? 1.0f / float(NumValues - 1) : 0.0f;
			float U = 0.0f;

			for (int32 iValueIndex = 0; iValueIndex < NumValues; ++iValueIndex)
			{
				for (int32 iChannel = 0; iChannel < NumChannels; ++iChannel)
				{
					ChannelCurves[iChannel].AddKey(U, Values[(iValueIndex * NumChannels) + iChannel]);
				}

				U += UInc;
			}
		}
	}

	void CreateCurveFromValues(TConstArrayView<float> Values, TArray<FRichCurve>& ChannelCurves, TArray<float>& ChannelConstantsAndRanges)
	{
		InternalCreateCurveFromValues(Values, ChannelCurves, ChannelConstantsAndRanges, 1);
	}

	void CreateCurveFromValues(TConstArrayView<FVector3f> Values, TArray<FRichCurve>& ChannelCurves, TArray<float>& ChannelConstantsAndRanges)
	{
		InternalCreateCurveFromValues(MakeArrayView<const float>(reinterpret_cast<const float*>(Values.GetData()), Values.Num() * 3), ChannelCurves, ChannelConstantsAndRanges, 3);
	}

	template<typename FValueContainerSetNum, typename FValueAndChannelAccessor>
	void UpdateDistributionValues(ENiagaraDistributionMode InMode, const TArray<float>& InChannelConstantsAndRanges, const TArray<FRichCurve>& InChannelCurves, int32 InChannelCount, FVector2f& OutTimeRange, FValueContainerSetNum InContainerNum, FValueAndChannelAccessor InValueAndChannelAccessor, int32 MaxLutSampleCount)
	{
		const ENiagaraDistributionCurveLUTMode LUTMode = ENiagaraDistributionCurveLUTMode::Sample;
		switch(InMode)
		{
			case ENiagaraDistributionMode::Array:
				if ((InChannelConstantsAndRanges.Num() % InChannelCount) == 0)
				{
					const int32 NumEntries = InChannelConstantsAndRanges.Num() / InChannelCount;

					InContainerNum(NumEntries);
					OutTimeRange = FVector2f(0.0f, 1.0f);
					for (int32 iEntry=0; iEntry < NumEntries; ++iEntry)
					{
						for (int32 iChannel=0; iChannel < InChannelCount; ++iChannel)
						{
							InValueAndChannelAccessor(iEntry, iChannel) = InChannelConstantsAndRanges[(iEntry * InChannelCount) + iChannel];
						}
					}
				}
				break;
			case ENiagaraDistributionMode::UniformConstant:
				if (InChannelConstantsAndRanges.Num() >= 1)
				{
					// Note we set two values to simplify the GPU code
					InContainerNum(2);
					OutTimeRange = FVector2f(0.0f, 1.0f);
					for (int32 ChannelIndex = 0; ChannelIndex < InChannelCount; ChannelIndex++)
					{
						InValueAndChannelAccessor(0, ChannelIndex) = InChannelConstantsAndRanges[0];
						InValueAndChannelAccessor(1, ChannelIndex) = InChannelConstantsAndRanges[0];
					}
				}
				break;
			case ENiagaraDistributionMode::NonUniformConstant:
				if (InChannelConstantsAndRanges.Num() >= InChannelCount)
				{
					// Note we set two values to simplify the GPU code
					InContainerNum(2);
					OutTimeRange = FVector2f(0.0f, 1.0f);
					for (int32 ChannelIndex = 0; ChannelIndex < InChannelCount; ChannelIndex++)
					{
						InValueAndChannelAccessor(0, ChannelIndex) = InChannelConstantsAndRanges[ChannelIndex];
						InValueAndChannelAccessor(1, ChannelIndex) = InChannelConstantsAndRanges[ChannelIndex];
					}
				}
				break;
			case ENiagaraDistributionMode::UniformRange:
				if (InChannelConstantsAndRanges.Num() >= 2)
				{
					InContainerNum(2);
					OutTimeRange = FVector2f(0.0f, 1.0f);
					for (int32 ChannelIndex = 0; ChannelIndex < InChannelCount; ChannelIndex++)
					{
						InValueAndChannelAccessor(0, ChannelIndex) = InChannelConstantsAndRanges[0];
						InValueAndChannelAccessor(1, ChannelIndex) = InChannelConstantsAndRanges[1];
					}
				}
				break;
			case ENiagaraDistributionMode::NonUniformRange:
				if (InChannelConstantsAndRanges.Num() >= 2 * InChannelCount)
				{
					InContainerNum(2);
					OutTimeRange = FVector2f(0.0f, 1.0f);
					for (int32 ChannelIndex = 0; ChannelIndex < InChannelCount; ChannelIndex++)
					{
						InValueAndChannelAccessor(0, ChannelIndex) = InChannelConstantsAndRanges[ChannelIndex];
						InValueAndChannelAccessor(1, ChannelIndex) = InChannelConstantsAndRanges[InChannelCount + ChannelIndex];
					}
				}
				break;
			case ENiagaraDistributionMode::UniformCurve:
			case ENiagaraDistributionMode::NonUniformCurve:
			case ENiagaraDistributionMode::ColorGradient:
			{
				const int32 ExpectedChannels = InMode == ENiagaraDistributionMode::UniformCurve ? 1 : InChannelCount;
				if (InChannelCurves.Num() >= ExpectedChannels)
				{
					MaxLutSampleCount = FMath::Max(MaxLutSampleCount, 2);
					const TArray<float> LUT = CurvesToOptimizedLUT(MakeArrayView(InChannelCurves.GetData(), ExpectedChannels), MaxLutSampleCount, LUTMode, OutTimeRange);
					const int32 NumSamples = LUT.Num() / ExpectedChannels;

					InContainerNum(NumSamples);
					for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
					{
						for (int32 ChannelIndex=0; ChannelIndex < InChannelCount; ++ChannelIndex)
						{
							const int32 LUTChannelIndex = InMode == ENiagaraDistributionMode::UniformCurve ? 0 : ChannelIndex;
							InValueAndChannelAccessor(SampleIndex, ChannelIndex) = LUT[(SampleIndex * ExpectedChannels) + LUTChannelIndex];
						}
					}
				}
				break;
			}
		}
	}

	template<typename FValueContainerSetNum, typename FValueAndChannelAccessor>
	void UpdateDistributionCurveValues(ENiagaraDistributionMode InMode, const TArray<float>& InChannelConstantsAndRanges, const TArray<FRichCurve>& InChannelCurves, int32 InChannelCount, FVector2f& OutTimeRange, FValueContainerSetNum InContainerNum, FValueAndChannelAccessor InValueAndChannelAccessor, int32 MaxLutSampleCount, ENiagaraDistributionCurveLUTMode LUTMode)
	{
		check(InMode == ENiagaraDistributionMode::UniformCurve || InMode == ENiagaraDistributionMode::NonUniformCurve || InMode == ENiagaraDistributionMode::ColorGradient);

		const int32 ExpectedChannels = InMode == ENiagaraDistributionMode::UniformCurve ? 1 : InChannelCount;
		if (InChannelCurves.Num() >= ExpectedChannels)
		{
			MaxLutSampleCount = FMath::Max(MaxLutSampleCount, 2);
			const TArray<float> LUT = CurvesToOptimizedLUT(MakeArrayView(InChannelCurves.GetData(), ExpectedChannels), MaxLutSampleCount, LUTMode, OutTimeRange);
			const int32 NumSamples = LUT.Num() / ExpectedChannels;

			InContainerNum(NumSamples);
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				for (int32 ChannelIndex=0; ChannelIndex < InChannelCount; ++ChannelIndex)
				{
					const int32 LUTChannelIndex = InMode == ENiagaraDistributionMode::UniformCurve ? 0 : ChannelIndex;
					InValueAndChannelAccessor(SampleIndex, ChannelIndex) = LUT[(SampleIndex * ExpectedChannels) + LUTChannelIndex];
				}
			}
		}
	}

	void UpdateDistributionTimeRange(ENiagaraDistributionTimeRangeMode TimeRangeMode, const FVector2f& CustomTimeRange, int32 NumEntries, FVector2f& OutTimeRange)
	{
		switch (TimeRangeMode)
		{
			// The 'time' value passed in maps directly into the array
			// i.e. Index = (Time * 1) + 0
			case ENiagaraDistributionTimeRangeMode::Direct:
				OutTimeRange.X = 1.0f;
				OutTimeRange.Y = 0.0f;
				break;

			// The 'time' value passed in needs to expand to the array range
			// i.e. Index = (Time * (NumEntries - 1) + 0
			case ENiagaraDistributionTimeRangeMode::Normalized:
				OutTimeRange.X = float(NumEntries - 1);
				OutTimeRange.Y = 0.0f;
				break;

			// The 'time' value passed in covers a user defined range, copy directly
			case ENiagaraDistributionTimeRangeMode::Custom:
				OutTimeRange = CustomTimeRange;
				break;

			default:
				checkNoEntry();
				OutTimeRange = FVector2f::ZeroVector;
				break;
		}
	}

#endif
} //NiagaraStatelessCommon

void FNiagaraDistributionRangeInt::InitConstant(int32 Value)
{
	Mode = ENiagaraDistributionMode::UniformConstant;
	Min = Value;
	Max = Value;
}

FNiagaraStatelessRangeInt FNiagaraDistributionRangeInt::CalculateRange(const int32 Default) const
{
	return Mode == ENiagaraDistributionMode::UniformConstant ? FNiagaraStatelessRangeInt(Min, Min) : FNiagaraStatelessRangeInt(Min, Max);
}

void FNiagaraDistributionRangeFloat::InitConstant(float Value)
{
	Mode = ENiagaraDistributionMode::UniformConstant;
	Min = Value;
	Max = Value;
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ Value });
#endif
}

void FNiagaraDistributionRangeFloat::InitRange(float MinValue, float MaxValue)
{
	Mode = ENiagaraDistributionMode::UniformRange;
	Min = MinValue;
	Max = MaxValue;
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ MinValue, MaxValue });
#endif
}

FNiagaraStatelessRangeFloat FNiagaraDistributionRangeFloat::CalculateRange(const float Default) const
{
	return Mode == ENiagaraDistributionMode::UniformConstant ? FNiagaraStatelessRangeFloat(Min, Min) : FNiagaraStatelessRangeFloat(Min, Max);
}

void FNiagaraDistributionRangeVector2::InitConstant(const FVector2f& Value)
{
	Mode = ENiagaraDistributionMode::NonUniformConstant;
	Min = Value;
	Max = Value;
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ Value.X, Value.Y });
#endif
}

FNiagaraStatelessRangeVector2 FNiagaraDistributionRangeVector2::CalculateRange(const FVector2f& Default) const
{
	return Mode == ENiagaraDistributionMode::UniformConstant ? FNiagaraStatelessRangeVector2(Min, Min) : FNiagaraStatelessRangeVector2(Min, Max);
}

void FNiagaraDistributionRangeVector3::InitConstant(const FVector3f& Value)
{
	Mode = ENiagaraDistributionMode::NonUniformConstant;
	Min = Value;
	Max = Value;
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ Value.X, Value.Y, Value.Z });
#endif
}

FNiagaraStatelessRangeVector3 FNiagaraDistributionRangeVector3::CalculateRange(const FVector3f& Default) const
{
	return Mode == ENiagaraDistributionMode::UniformConstant ? FNiagaraStatelessRangeVector3(Min, Min) : FNiagaraStatelessRangeVector3(Min, Max);
}

void FNiagaraDistributionRangeColor::InitConstant(const FLinearColor& Value)
{
	Mode = ENiagaraDistributionMode::NonUniformConstant;
	Min = Value;
	Max = Value;
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ Value.R, Value.G, Value.B, Value.A });
#endif
}

FNiagaraStatelessRangeColor FNiagaraDistributionRangeColor::CalculateRange(const FLinearColor& Default) const
{
	return Mode == ENiagaraDistributionMode::UniformConstant ? FNiagaraStatelessRangeColor(Min, Min) : FNiagaraStatelessRangeColor(Min, Max);
}

void FNiagaraDistributionRangeRotator::InitConstant(const FRotator3f& Value)
{
	Mode = ENiagaraDistributionMode::NonUniformConstant;
	Min = Value;
	Max = Value;
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ Value.Roll, Value.Pitch, Value.Yaw });
#endif
}

FNiagaraStatelessRangeRotator FNiagaraDistributionRangeRotator::CalculateRange(const FRotator3f& Default) const
{
	return Mode == ENiagaraDistributionMode::UniformConstant ? FNiagaraStatelessRangeRotator(Min, Min) : FNiagaraStatelessRangeRotator(Min, Max);
}

void FNiagaraDistributionFloat::InitConstant(float Value)
{
	Mode = ENiagaraDistributionMode::UniformConstant;
	Values = TArray<float>({ Value, Value });
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ Value });
#endif
}

void FNiagaraDistributionFloat::InitCurve(std::initializer_list<float> CurvePoints)
{
	Mode = ENiagaraDistributionMode::UniformCurve;
	Values = CurvePoints;

#if WITH_EDITORONLY_DATA
	NiagaraStatelessDistributionPrivate::CreateCurveFromValues(Values, ChannelCurves, ChannelConstantsAndRanges);
#endif
}

#if WITH_EDITORONLY_DATA
void FNiagaraDistributionFloat::InitCurve(const TArray<FRichCurveKey>& CurveKeys)
{
	Mode = ENiagaraDistributionMode::UniformCurve;

	ChannelCurves.Reset(1);
	ChannelCurves.AddDefaulted_GetRef().SetKeys(CurveKeys);

	Values.SetNumUninitialized(CurveKeys.Num());
	ChannelConstantsAndRanges.SetNumUninitialized(CurveKeys.Num());

	for (int32 iKey=0; iKey < CurveKeys.Num(); ++iKey)
	{
		const float KeyValue = CurveKeys[iKey].Value;
		Values[iKey] = KeyValue;
		ChannelConstantsAndRanges[iKey] = KeyValue;
	}
}
#endif

FNiagaraStatelessRangeFloat FNiagaraDistributionFloat::CalculateRange(const float Default) const
{
	FNiagaraStatelessRangeFloat Range(Default, Default);
	if (Values.Num() > 0)
	{
		Range.Min = Values[0];
		Range.Max = Values[0];
		for (int i=1; i < Values.Num(); ++i)
		{
			Range.Min = FMath::Min(Range.Min, Values[i]);
			Range.Max = FMath::Max(Range.Max, Values[i]);
		}
	}
	return Range;
}

void FNiagaraDistributionVector2::InitConstant(const float Value)
{
	Mode = ENiagaraDistributionMode::UniformConstant;
	Values = TArray<FVector2f>({ FVector2f(Value, Value), FVector2f(Value, Value) });
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ Value, Value });
#endif
}

void FNiagaraDistributionVector2::InitConstant(const FVector2f& Value)
{
	Mode = ENiagaraDistributionMode::NonUniformConstant;
	Values = TArray<FVector2f>({ Value, Value });
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ Value.X, Value.Y });
#endif
}

FNiagaraStatelessRangeVector2 FNiagaraDistributionVector2::CalculateRange(const FVector2f& Default) const
{
	FNiagaraStatelessRangeVector2 Range(Default, Default);
	if (Values.Num() > 0)
	{
		Range.Min = Values[0];
		Range.Max = Values[0];
		for (int i = 1; i < Values.Num(); ++i)
		{
			Range.Min = FVector2f::Min(Range.Min, Values[i]);
			Range.Max = FVector2f::Max(Range.Max, Values[i]);
		}
	}
	return Range;
}

void FNiagaraDistributionVector3::InitConstant(const float Value)
{
	Mode = ENiagaraDistributionMode::UniformConstant;
	Values = TArray<FVector3f>({ FVector3f(Value, Value, Value), FVector3f(Value, Value, Value) });
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ Value, Value });
#endif
}

void FNiagaraDistributionVector3::InitConstant(const FVector3f& Value)
{
	Mode = ENiagaraDistributionMode::NonUniformConstant;
	Values = TArray<FVector3f>({ Value, Value });
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ Value.X, Value.Y, Value.Z });
#endif
}

void FNiagaraDistributionVector3::InitCurve(std::initializer_list<float> CurvePoints)
{
	const int32 NumValues = CurvePoints.size();

	Mode = ENiagaraDistributionMode::UniformCurve;
	Values.Reset(NumValues);
	for (float Value : CurvePoints)
	{
		Values.Emplace(Value);
	}

#if WITH_EDITORONLY_DATA
	NiagaraStatelessDistributionPrivate::CreateCurveFromValues(Values, ChannelCurves, ChannelConstantsAndRanges);
#endif
}

void FNiagaraDistributionVector3::InitCurve(std::initializer_list<FVector3f> CurvePoints)
{
	const int32 NumValues = CurvePoints.size();

	Mode = ENiagaraDistributionMode::NonUniformCurve;
	Values = TArray<FVector3f>(CurvePoints);

#if WITH_EDITORONLY_DATA
	NiagaraStatelessDistributionPrivate::CreateCurveFromValues(Values, ChannelCurves, ChannelConstantsAndRanges);
#endif
}

FNiagaraStatelessRangeVector3 FNiagaraDistributionVector3::CalculateRange(const FVector3f& Default) const
{
	FNiagaraStatelessRangeVector3 Range(Default, Default);
	if (Values.Num() > 0)
	{
		Range.Min = Values[0];
		Range.Max = Values[0];
		for (int i = 1; i < Values.Num(); ++i)
		{
			Range.Min = FVector3f::Min(Range.Min, Values[i]);
			Range.Max = FVector3f::Max(Range.Max, Values[i]);
		}
	}
	return Range;
}

void FNiagaraDistributionColor::InitConstant(const FLinearColor& Value)
{
	Mode = ENiagaraDistributionMode::NonUniformConstant;
	Values = TArray<FLinearColor>({ Value, Value });
#if WITH_EDITORONLY_DATA
	ChannelConstantsAndRanges = TArray<float>({ Value.R, Value.G, Value.B, Value.A });
#endif
}

FNiagaraStatelessRangeColor FNiagaraDistributionColor::CalculateRange(const FLinearColor& Default) const
{
	FNiagaraStatelessRangeColor Range(Default, Default);
	if (Values.Num() > 0)
	{
		Range.Min = Values[0];
		Range.Max = Values[0];
		for (int i = 1; i < Values.Num(); ++i)
		{
			Range.Min.R = FMath::Min(Range.Min.R, Values[i].R);
			Range.Min.G = FMath::Min(Range.Min.G, Values[i].G);
			Range.Min.B = FMath::Min(Range.Min.B, Values[i].B);
			Range.Min.A = FMath::Min(Range.Min.A, Values[i].A);

			Range.Max.R = FMath::Max(Range.Max.R, Values[i].R);
			Range.Max.G = FMath::Max(Range.Max.G, Values[i].G);
			Range.Max.B = FMath::Max(Range.Max.B, Values[i].B);
			Range.Max.A = FMath::Max(Range.Max.A, Values[i].A);
		}
	}
	return Range;
}

FNiagaraDistributionCurveFloat::FNiagaraDistributionCurveFloat()
 : FNiagaraDistributionCurveFloat(ENiagaraDistributionCurveLUTMode::Sample)
{
}

FNiagaraDistributionCurveFloat::FNiagaraDistributionCurveFloat(ENiagaraDistributionCurveLUTMode InLUTMode)
#if WITH_EDITORONLY_DATA
	: LUTMode(InLUTMode)
#endif
{
	Mode = ENiagaraDistributionMode::UniformCurve;
	if (InLUTMode == ENiagaraDistributionCurveLUTMode::Sample)
	{
		Values = { 1.0f, 1.0f };
	}
	else
	{
		Values = { 0.0f, 1.0f };
	}
#if WITH_EDITORONLY_DATA
	NiagaraStatelessDistributionPrivate::CreateCurveFromValues({ 1.0f, 1.0f }, ChannelCurves, ChannelConstantsAndRanges);
#endif
}

FNiagaraDistributionCurveFloat::FNiagaraDistributionCurveFloat(std::initializer_list<float> CurvePoints)
{
	Mode = ENiagaraDistributionMode::UniformCurve;
	Values = CurvePoints;

#if WITH_EDITORONLY_DATA
	NiagaraStatelessDistributionPrivate::CreateCurveFromValues(Values, ChannelCurves, ChannelConstantsAndRanges);
#endif
}

FNiagaraDistributionCurveVector3::FNiagaraDistributionCurveVector3()
	: FNiagaraDistributionCurveVector3(ENiagaraDistributionCurveLUTMode::Sample)
{
}

FNiagaraDistributionCurveVector3::FNiagaraDistributionCurveVector3(ENiagaraDistributionCurveLUTMode InLUTMode)
#if WITH_EDITORONLY_DATA
	: LUTMode(InLUTMode)
#endif
{
	Mode = ENiagaraDistributionMode::UniformCurve;
	if (InLUTMode == ENiagaraDistributionCurveLUTMode::Sample)
	{
		Values = { FVector3f::OneVector, FVector3f::OneVector };
	}
	else
	{
		Values = { FVector3f::ZeroVector, FVector3f::OneVector };
	}
#if WITH_EDITORONLY_DATA
	NiagaraStatelessDistributionPrivate::CreateCurveFromValues({ FVector3f::OneVector, FVector3f::OneVector }, ChannelCurves, ChannelConstantsAndRanges);
#endif
}

#if WITH_EDITORONLY_DATA
bool FNiagaraDistributionBase::operator==(const FNiagaraDistributionBase& Other) const
{
	return
		Mode == Other.Mode &&
		ParameterBinding == Other.ParameterBinding &&
		ParameterExpression.Identical(&Other.ParameterExpression, PPF_DeepComparison) &&
		ChannelConstantsAndRanges == Other.ChannelConstantsAndRanges &&
		ChannelCurves == Other.ChannelCurves &&
		MaxLutSampleCount == Other.MaxLutSampleCount;
}

void FNiagaraDistributionBase::ForEachParameterBinding(TFunction<void(const FNiagaraVariableBase&)> Delegate) const
{
	if (IsBinding())
	{
		Delegate(ParameterBinding);
	}
	else if (IsExpression())
	{
		FNiagaraStatelessExpression::ForEachBinding(ParameterExpression, Delegate);
	}
}

void FNiagaraDistributionBase::ForEachParameterBinding(const UStruct* ObjectStruct, const void* Object, TFunction<void(const FNiagaraVariableBase&)> Delegate)
{
	for (TFieldIterator<FStructProperty> PropIt(ObjectStruct); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = *PropIt)
		{
			if (StructProp->Struct && StructProp->Struct->IsChildOf(FNiagaraDistributionBase::StaticStruct()))
			{
				const FNiagaraDistributionBase* Distribution = StructProp->ContainerPtrToValuePtr<const FNiagaraDistributionBase>(Object);
				Distribution->ForEachParameterBinding(Delegate);
			}
			else if (StructProp->Struct && StructProp->Struct->IsChildOf(FNiagaraDistributionRangeInt::StaticStruct()))
			{
				const FNiagaraDistributionRangeInt* Distribution = StructProp->ContainerPtrToValuePtr<const FNiagaraDistributionRangeInt>(Object);
				if (Distribution->IsBinding())
				{
					Delegate(Distribution->ParameterBinding);
				}
				else if (Distribution->IsExpression())
				{
					FNiagaraStatelessExpression::ForEachBinding(Distribution->ParameterExpression, Delegate);
				}
			}
		}
	}
}

void FNiagaraDistributionBase::ForEachParameterBinding(const UObject* InObject, TFunction<void(const FNiagaraVariableBase&)> Delegate)
{
	ForEachParameterBinding(InObject->GetClass(), InObject, Delegate);
}

bool FNiagaraDistributionBase::AllowDistributionMode(ENiagaraDistributionMode InMode, const FProperty* Property) const
{
	static const FName DisableArrayDistributionName("DisableArrayDistribution");
	static const FName DisableBindingDistributionName("DisableBindingDistribution");
	static const FName DisableCurveDistributionName("DisableCurveDistribution");
	static const FName DisableUniformDistributionName("DisableUniformDistribution");
	static const FName DisableNonUniformDistributionName("DisableNonUniformDistribution");
	static const FName DisableRangeDistributionName("DisableRangeDistribution");
	static const FName DisableColorGradientDistributionName("DisableColorGradientDistribution");

	bool bAllowed = true;
	switch (InMode)
	{
		case ENiagaraDistributionMode::Array:
			bAllowed &= !Property || !Property->HasMetaData(DisableArrayDistributionName);
			break;

		case ENiagaraDistributionMode::Binding:
		case ENiagaraDistributionMode::Expression:
			bAllowed &= !Property || !Property->HasMetaData(DisableBindingDistributionName);
			bAllowed &= GetBindingTypeDef().IsValid();
			break;

		case ENiagaraDistributionMode::UniformConstant:
			bAllowed &= !Property || !Property->HasMetaData(DisableUniformDistributionName);
			break;

		case ENiagaraDistributionMode::NonUniformConstant:
			bAllowed &= !Property || !Property->HasMetaData(DisableNonUniformDistributionName);
			break;

		case ENiagaraDistributionMode::UniformRange:
			bAllowed &= !Property || (!Property->HasMetaData(DisableRangeDistributionName) && !Property->HasMetaData(DisableUniformDistributionName));
			break;

		case ENiagaraDistributionMode::NonUniformRange:
			bAllowed &= !Property || (!Property->HasMetaData(DisableRangeDistributionName) && !Property->HasMetaData(DisableNonUniformDistributionName));
			break;

		case ENiagaraDistributionMode::UniformCurve:
			bAllowed &= !Property || (!Property->HasMetaData(DisableCurveDistributionName) && !Property->HasMetaData(DisableUniformDistributionName));
			break;

		case ENiagaraDistributionMode::NonUniformCurve:
			bAllowed &= !Property || (!Property->HasMetaData(DisableCurveDistributionName) && !Property->HasMetaData(DisableNonUniformDistributionName));
			break;

		case ENiagaraDistributionMode::ColorGradient:
			bAllowed &= !Property || !Property->HasMetaData(DisableColorGradientDistributionName);
			break;

		default:
			break;
	}
	return bAllowed;
}

bool FNiagaraDistributionBase::InternalAllowRangeDistribution(ENiagaraDistributionMode InMode, const FProperty* Property) const
{
	switch (InMode)
	{
		case ENiagaraDistributionMode::Binding:
		case ENiagaraDistributionMode::Expression:
		case ENiagaraDistributionMode::UniformConstant:
		case ENiagaraDistributionMode::NonUniformConstant:
		case ENiagaraDistributionMode::UniformRange:
		case ENiagaraDistributionMode::NonUniformRange:
			return FNiagaraDistributionBase::AllowDistributionMode(InMode, Property);

		default:
			return false;
	}
}

bool FNiagaraDistributionBase::InternalAllowCurveDistribution(ENiagaraDistributionMode InMode, const FProperty* Property) const
{
	switch (InMode)
	{
		case ENiagaraDistributionMode::UniformCurve:
		case ENiagaraDistributionMode::NonUniformCurve:
		case ENiagaraDistributionMode::ColorGradient:
			return FNiagaraDistributionBase::AllowDistributionMode(InMode, Property);

		default:
			return false;
	}
}

bool FNiagaraDistributionBase::AllowAddressMode(const FProperty* Property) const
{
	static const FName DisableDistributionAddressModeName("DisableDistributionAddressMode");

	if (Mode != ENiagaraDistributionMode::Array)
	{
		return false;
	}

	if (Property && Property->HasMetaData(DisableDistributionAddressModeName))
	{
		return false;
	}

	return true;
}

bool FNiagaraDistributionBase::AllowInterpolationMode(const FProperty* Property) const
{
	static const FName DisableDistributionInterpolationModeName("DisableDistributionInterpolationMode");

	if (Mode != ENiagaraDistributionMode::Array)
	{
		return false;
	}

	if (Property && Property->HasMetaData(DisableDistributionInterpolationModeName))
	{
		return false;
	}

	return true;
}

bool FNiagaraDistributionBase::AllowLookupValueMode(const FProperty* Property) const
{
	static const FName DisableDistributionLookupValueModeName("DisableDistributionLookupValueMode");

	if (Mode != ENiagaraDistributionMode::Array && Mode != ENiagaraDistributionMode::ColorGradient && Mode != ENiagaraDistributionMode::NonUniformCurve && Mode != ENiagaraDistributionMode::UniformCurve)
	{
		return false;
	}

	if (Property && Property->HasMetaData(DisableDistributionLookupValueModeName))
	{
		return false;
	}

	return true;
}

const UEnum* FNiagaraDistributionBase::GetLookupValueModeEnum(const FProperty* Property)
{
	const UEnum* LookupEnum = nullptr;
	if (ensure(Property))
	{
		const FName NAME_DistributionLookupValueEnumPath("DistributionLookupValueEnumPath");
		const FString& EnumPath = Property->GetMetaData(NAME_DistributionLookupValueEnumPath);
		if (!EnumPath.IsEmpty())
		{
			LookupEnum = FindObject<const UEnum>(FTopLevelAssetPath(EnumPath));
			checkf(LookupEnum != nullptr, TEXT("Failed to find enum '%s' from property"), *EnumPath);
		}
	}
	return LookupEnum ? LookupEnum : StaticEnum<ENiagaraDistributionLookupValueMode>();
}

void FNiagaraDistributionBase::ForEachDistribution(const UStruct* ObjectStruct, const void* Object, TFunction<void(const FNiagaraDistributionBase&)> Delegate)
{
	for (TFieldIterator<FStructProperty> PropIt(ObjectStruct); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = *PropIt)
		{
			if (StructProp->Struct && StructProp->Struct->IsChildOf(FNiagaraDistributionBase::StaticStruct()))
			{
				const FNiagaraDistributionBase* Distribution = StructProp->ContainerPtrToValuePtr<const FNiagaraDistributionBase>(Object);
				Delegate(*Distribution);
			}
		}
	}
}

void FNiagaraDistributionBase::ForEachDistribution(const UStruct* ObjectStruct, void* Object, TFunction<void(FNiagaraDistributionBase&)> Delegate)
{
	for (TFieldIterator<FStructProperty> PropIt(ObjectStruct); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = *PropIt)
		{
			if (StructProp->Struct && StructProp->Struct->IsChildOf(FNiagaraDistributionBase::StaticStruct()))
			{
				FNiagaraDistributionBase* Distribution = StructProp->ContainerPtrToValuePtr<FNiagaraDistributionBase>(Object);
				Delegate(*Distribution);
			}
		}
	}
}

void FNiagaraDistributionBase::ForEachDistribution(const UObject* InObject, TFunction<void(const FNiagaraDistributionBase&)> Delegate)
{
	ForEachDistribution(InObject->GetClass(), InObject, Delegate);
}

void FNiagaraDistributionBase::ForEachDistribution(UObject* InObject, TFunction<void(FNiagaraDistributionBase&)> Delegate)
{
	ForEachDistribution(InObject->GetClass(), InObject, Delegate);
}

void FNiagaraDistributionBase::PostEditChangeProperty(UObject* OwnerObject, FPropertyChangedEvent& PropertyChangedEvent)
{
	FStructProperty* StructProperty = CastField<FStructProperty>(PropertyChangedEvent.Property);
	if (StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf(FNiagaraDistributionBase::StaticStruct()))
	{
		FProperty* ValueProperty = StructProperty;
		FNiagaraDistributionBase* ValuePtr = nullptr;
		if (PropertyChangedEvent.Property != PropertyChangedEvent.MemberProperty)
		{
			// Properties stored in a UStruct inside a UObject need to first offset from UObject -> UStruct then UStruct -> Property
			FStructProperty* MemberStructProperty = CastField<FStructProperty>(PropertyChangedEvent.MemberProperty);
			void* StructPtr = MemberStructProperty->ContainerPtrToValuePtr<void>(OwnerObject);
			ValueProperty = MemberStructProperty;
			ValuePtr = StructProperty->ContainerPtrToValuePtr<FNiagaraDistributionBase>(StructPtr);
		}
		else
		{
			ValuePtr = StructProperty->ContainerPtrToValuePtr<FNiagaraDistributionBase>(OwnerObject);
		}

		if (ValuePtr != nullptr)
		{
			ValuePtr->UpdateValuesFromDistribution();

			// Arrays can modify the time range packed inside the distribution, ensure we update that information here.
			// Some of this will be extended to curves in the future.
			if (ValuePtr->Mode == ENiagaraDistributionMode::Array)
			{
				ENiagaraDistributionTimeRangeMode TimeRangeMode = ENiagaraDistributionTimeRangeMode::Normalized;

				if (ValuePtr->AllowLookupValueMode(ValueProperty))
				{
					if (const UEnum* LookupEnum = FNiagaraDistributionBase::GetLookupValueModeEnum(ValueProperty))
					{
						const int32 ValueIndex = LookupEnum->GetIndexByValue(ValuePtr->GetLookupValueMode());
						const FString TimeRangeModeStr = ValueIndex != INDEX_NONE ? LookupEnum->GetMetaData(TEXT("TimeRangeMode"), ValueIndex) : FString();
						if (!TimeRangeModeStr.IsEmpty())
						{
							const int64 TimeRangeModeValue = StaticEnum<ENiagaraDistributionTimeRangeMode>()->GetValueByNameString(TimeRangeModeStr);
							if (ensureMsgf(TimeRangeModeValue != INDEX_NONE, TEXT("Time Range metadata(%s) on Enum(%s) is invalid."), *TimeRangeModeStr, *LookupEnum->GetName()))
							{
								TimeRangeMode = ENiagaraDistributionTimeRangeMode(TimeRangeModeValue);
							}
						}
					}
				}

				ValuePtr->UpdateTimeRangeFromDistribution(TimeRangeMode);
			}
			
			// Reset any values that are not supported in this mode
			if (!ValuePtr->AllowLookupValueMode(ValueProperty))
			{
				ValuePtr->SetLookupValueMode(uint8(ENiagaraDistributionLookupValueMode::ParticlesNormalizedAge));
			}
			if (!ValuePtr->AllowInterpolationMode(ValueProperty))
			{
				ValuePtr->SetInterpolationMode(ENiagaraDistributionInterpolationMode::Linear);
			}
			if (!ValuePtr->AllowAddressMode(ValueProperty))
			{
				ValuePtr->SetAddressMode(ENiagaraDistributionAddressMode::Clamp);
			}
		}
	}
}

void FNiagaraDistributionRangeFloat::UpdateValuesFromDistribution()
{
	FVector2f ValuesTimeRange;
	NiagaraStatelessDistributionPrivate::UpdateDistributionValues(
		Mode,
		ChannelConstantsAndRanges,
		ChannelCurves,
		1,
		ValuesTimeRange,
		[this](int32 Num) {},
		[this](int32 ValueIndex, int32 ChannelIndex) -> float& { return ValueIndex == 0 ? Min : Max; },
		MaxLutSampleCount
	);
}

void FNiagaraDistributionRangeVector2::UpdateValuesFromDistribution()
{
	FVector2f ValuesTimeRange;
	NiagaraStatelessDistributionPrivate::UpdateDistributionValues(
		Mode,
		ChannelConstantsAndRanges,
		ChannelCurves,
		2,
		ValuesTimeRange,
		[this](int32 Num) {},
		[this](int32 ValueIndex, int32 ChannelIndex) -> float& { return ValueIndex == 0 ? Min[ChannelIndex] : Max[ChannelIndex]; },
		MaxLutSampleCount
	);
}

void FNiagaraDistributionRangeVector3::UpdateValuesFromDistribution()
{
	FVector2f ValuesTimeRange;
	NiagaraStatelessDistributionPrivate::UpdateDistributionValues(
		Mode,
		ChannelConstantsAndRanges,
		ChannelCurves,
		3,
		ValuesTimeRange,
		[this](int32 Num) {},
		[this](int32 ValueIndex, int32 ChannelIndex) -> float& { return ValueIndex == 0 ? Min[ChannelIndex] : Max[ChannelIndex]; },
		MaxLutSampleCount
	);
}

void FNiagaraDistributionRangeColor::UpdateValuesFromDistribution()
{
	FVector2f ValuesTimeRange;
	NiagaraStatelessDistributionPrivate::UpdateDistributionValues(
		Mode,
		ChannelConstantsAndRanges,
		ChannelCurves,
		4,
		ValuesTimeRange,
		[this](int32 Num) {},
		[this](int32 ValueIndex, int32 ChannelIndex) -> float& { return ValueIndex == 0 ? Min.Component(ChannelIndex) : Max.Component(ChannelIndex); },
		MaxLutSampleCount
	);
}

void FNiagaraDistributionRangeRotator::UpdateValuesFromDistribution()
{
	FVector2f ValuesTimeRange;
	NiagaraStatelessDistributionPrivate::UpdateDistributionValues(
		Mode,
		ChannelConstantsAndRanges,
		ChannelCurves,
		3,
		ValuesTimeRange,
		[this](int32 Num) {},
		[this](int32 ValueIndex, int32 ChannelIndex) -> float&
		{
			FRotator3f& Target = ValueIndex == 0 ? Min : Max;
			switch (ChannelIndex)
			{
				default:	checkNoEntry(); // Intentional fall through
				case 0:		return Target.Roll;
				case 1:		return Target.Pitch;
				case 2:		return Target.Yaw;
			}
		},
		MaxLutSampleCount
	);
}

void FNiagaraDistributionFloat::UpdateValuesFromDistribution()
{
	NiagaraStatelessDistributionPrivate::UpdateDistributionValues(
		Mode, 
		ChannelConstantsAndRanges, 
		ChannelCurves,
		1,
		ValuesTimeRange,
		[this](int32 NumValues) { Values.SetNum(NumValues); },
		[this](int32 ValueIndex, int32 ChannelIndex) -> float& { return Values[ValueIndex]; },
		MaxLutSampleCount
	);
}

void FNiagaraDistributionFloat::UpdateTimeRangeFromDistribution(ENiagaraDistributionTimeRangeMode TimeRangeMode, const FVector2f& CustomTimeRange)
{
	NiagaraStatelessDistributionPrivate::UpdateDistributionTimeRange(
		TimeRangeMode,
		CustomTimeRange,
		Values.Num(),
		ValuesTimeRange
	);
}

void FNiagaraDistributionVector2::UpdateValuesFromDistribution()
{
	NiagaraStatelessDistributionPrivate::UpdateDistributionValues(
		Mode,
		ChannelConstantsAndRanges,
		ChannelCurves,
		2,
		ValuesTimeRange,
		[this](int32 NumValues) { Values.SetNum(NumValues); },
		[this](int32 ValueIndex, int32 ChannelIndex) -> float& { return Values[ValueIndex][ChannelIndex]; },
		MaxLutSampleCount
	);
}

void FNiagaraDistributionVector2::UpdateTimeRangeFromDistribution(ENiagaraDistributionTimeRangeMode TimeRangeMode, const FVector2f& CustomTimeRange)
{
	NiagaraStatelessDistributionPrivate::UpdateDistributionTimeRange(
		TimeRangeMode,
		CustomTimeRange,
		Values.Num(),
		ValuesTimeRange
	);
}

void FNiagaraDistributionVector3::UpdateValuesFromDistribution()
{
	NiagaraStatelessDistributionPrivate::UpdateDistributionValues(
		Mode,
		ChannelConstantsAndRanges,
		ChannelCurves,
		3,
		ValuesTimeRange,
		[this](int32 NumValues) { Values.SetNum(NumValues); },
		[this](int32 ValueIndex, int32 ChannelIndex) -> float& { return Values[ValueIndex][ChannelIndex]; },
		MaxLutSampleCount
	);
}

void FNiagaraDistributionVector3::UpdateTimeRangeFromDistribution(ENiagaraDistributionTimeRangeMode TimeRangeMode, const FVector2f& CustomTimeRange)
{
	NiagaraStatelessDistributionPrivate::UpdateDistributionTimeRange(
		TimeRangeMode,
		CustomTimeRange,
		Values.Num(),
		ValuesTimeRange
	);
}

void FNiagaraDistributionColor::UpdateValuesFromDistribution()
{
	NiagaraStatelessDistributionPrivate::UpdateDistributionValues(
		Mode,
		ChannelConstantsAndRanges,
		ChannelCurves,
		4,
		ValuesTimeRange,
		[this](int32 NumValues) { Values.SetNum(NumValues); },
		[this](int32 ValueIndex, int32 ChannelIndex) -> float& { return Values[ValueIndex].Component(ChannelIndex); },
		MaxLutSampleCount
	);
}

void FNiagaraDistributionColor::UpdateTimeRangeFromDistribution(ENiagaraDistributionTimeRangeMode TimeRangeMode, const FVector2f& CustomTimeRange)
{
	NiagaraStatelessDistributionPrivate::UpdateDistributionTimeRange(
		TimeRangeMode,
		CustomTimeRange,
		Values.Num(),
		ValuesTimeRange
	);
}

void FNiagaraDistributionCurveFloat::UpdateValuesFromDistribution()
{
	NiagaraStatelessDistributionPrivate::UpdateDistributionCurveValues(
		Mode,
		ChannelConstantsAndRanges,
		ChannelCurves,
		1,
		ValuesTimeRange,
		[this](int32 NumValues) { Values.SetNum(NumValues); },
		[this](int32 ValueIndex, int32 ChannelIndex) -> float& { return Values[ValueIndex]; },
		MaxLutSampleCount,
		LUTMode
	);
}

void FNiagaraDistributionCurveVector3::UpdateValuesFromDistribution()
{
	NiagaraStatelessDistributionPrivate::UpdateDistributionCurveValues(
		Mode,
		ChannelConstantsAndRanges,
		ChannelCurves,
		3,
		ValuesTimeRange,
		[this](int32 NumValues) { Values.SetNum(NumValues); },
		[this](int32 ValueIndex, int32 ChannelIndex) -> float& { return Values[ValueIndex].Component(ChannelIndex); },
		MaxLutSampleCount,
		LUTMode
	);
}
#endif

#if WITH_EDITORONLY_DATA
bool FNiagaraDistributionColor::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(FNiagaraDistributionRangeColor::StaticStruct()->GetFName()))
	{
		FNiagaraDistributionRangeColor RangeValue;
		FNiagaraDistributionRangeColor::StaticStruct()->SerializeItem(Slot, &RangeValue, nullptr);

		*static_cast<FNiagaraDistributionBase*>(this) = static_cast<FNiagaraDistributionBase&>(RangeValue);

		// Range would use UniformConstant / UniformRange when meaning non uniform so we need to convert to the correct mode here
		switch (Mode)
		{
			case ENiagaraDistributionMode::UniformConstant:
			case ENiagaraDistributionMode::NonUniformConstant:
				Mode = ENiagaraDistributionMode::NonUniformConstant;
				break;

			case ENiagaraDistributionMode::UniformRange:
			case ENiagaraDistributionMode::NonUniformRange:
				Mode = ENiagaraDistributionMode::NonUniformRange;
				break;
		}

		UpdateValuesFromDistribution();

		return true;
	}

	return false;
}
#endif

bool FNiagaraDistributionRangeFloat::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_FloatProperty)
	{
		float Value;
		Slot << Value;
		*this = FNiagaraDistributionRangeFloat(Value);
		return true;
	}

	return false;
}

bool FNiagaraDistributionRangeVector2::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Vector2f))
	{
		FVector2f Value;
		Slot << Value;
		*this = FNiagaraDistributionRangeVector2(Value);
		return true;
	}

	return false;
}

bool FNiagaraDistributionRangeVector3::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Vector3f))
	{
		FVector3f Value;
		Slot << Value;
		*this = FNiagaraDistributionRangeVector3(Value);
		return true;
	}

	return false;
}

bool FNiagaraDistributionRangeRotator::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Rotator3f))
	{
		FRotator3f Value;
		Slot << Value;
		*this = FNiagaraDistributionRangeRotator(Value);
		return true;
	}

	if (Tag.GetType().IsStruct(NAME_Rotator))
	{
		FRotator Value;
		Slot << Value;
		*this = FNiagaraDistributionRangeRotator(FRotator3f(Value));
		return true;
	}

	return false;
}

