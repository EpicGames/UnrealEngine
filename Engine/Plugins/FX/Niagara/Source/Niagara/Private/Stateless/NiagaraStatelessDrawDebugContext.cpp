// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessDrawDebugContext.h"
#include "DrawDebugHelpers.h"

#include "Components/LineBatchComponent.h"
#include "Engine/World.h"
#include "Misc/LargeWorldRenderPosition.h"

#if WITH_EDITOR
FNiagaraStatelessDrawDebugContext::FNiagaraStatelessDrawDebugContext(UWorld* InWorld, const FTransform& InLocalToWorld, bool bIsLocalSpace)
	: World(InWorld)
	, LocalToWorld(InLocalToWorld)
{
	bApplyLocalToWorld[int(ENiagaraCoordinateSpace::Simulation)]	= bIsLocalSpace;
	bApplyLocalToWorld[int(ENiagaraCoordinateSpace::World)]			= false;
	bApplyLocalToWorld[int(ENiagaraCoordinateSpace::Local)]			= true;

	LWCTileOffset = FVector(FLargeWorldRenderScalar::GetTileFor(LocalToWorld.GetTranslation())) * UE_LWC_RENDER_TILE_SIZE;
}

void FNiagaraStatelessDrawDebugContext::DrawAxis(const FVector& Origin, const FQuat& Rotation, float Scale) const
{
	DrawDebugCoordinateSystem(World, Origin, FRotator(Rotation), Scale);
}

void FNiagaraStatelessDrawDebugContext::DrawArrow(const FVector& Origin, const FVector& DirectionWithLength, const FColor& Color) const
{
	const float Len = static_cast<float>(DirectionWithLength.Length());
	if (Len > UE_SMALL_NUMBER)
	{
		DrawDebugDirectionalArrow(
			World,
			Origin,
			Origin + DirectionWithLength,
			Len,
			Color
		);
	}
}

void FNiagaraStatelessDrawDebugContext::DrawBox(const FVector& Center, const FQuat& Rotation, const FVector& Extent, const FColor& Color) const
{
	DrawDebugBox(
		World,
		Center,
		Extent,
		Rotation,
		Color
	);
}

void FNiagaraStatelessDrawDebugContext::DrawCone(const FVector& Origin, const FQuat& Rotation, float Angle, float Length, const FColor& Color) const
{
	DrawDebugCone(
		World,
		Origin,
		Rotation.GetAxisZ(),
		Length,
		FMath::DegreesToRadians(Angle),
		FMath::DegreesToRadians(Angle),
		16,
		Color
	);
}

void FNiagaraStatelessDrawDebugContext::DrawCylinder(const FVector& Center, const FQuat& Rotation, const FVector& Scale, float CylinderHeight, float CylinderRadius, float CylinderHeightMidpoint, const FColor& Color) const
{
	ULineBatchComponent* LineBatcher = World->GetLineBatcher(UWorld::ELineBatcherType::World);
	if (LineBatcher == nullptr)
	{
		return;
	}

	constexpr int NumSegments = 12;
	constexpr float AngleStep = 2.f * UE_PI / float(NumSegments);

	const FVector AxisX = Rotation.RotateVector(FVector::XAxisVector) * Scale.X * CylinderRadius;
	const FVector AxisY = Rotation.RotateVector(FVector::YAxisVector) * Scale.Y * CylinderRadius;
	const FVector AxisZ = Rotation.RotateVector(FVector::ZAxisVector) * Scale.Z;

	const float Offset = -CylinderHeightMidpoint * CylinderHeight;
	const FVector Start = Center + (AxisZ * Offset);
	const FVector End = Center + (AxisZ * (Offset + CylinderHeight));

	TArray<FBatchedLine> Lines;
	Lines.Empty(NumSegments * 3);

	float Angle = 0.f;
	for (int iSegment=0; iSegment < NumSegments; ++iSegment)
	{
		const FVector Offset1 = (AxisX * FMath::Cos(Angle) + AxisY * FMath::Sin(Angle));
		Angle += AngleStep;
		const FVector Offset2 = (AxisX * FMath::Cos(Angle) + AxisY * FMath::Sin(Angle));

		Lines.Emplace(Start + Offset1, Start + Offset2, Color, -1.0f, 0.0f, 0);
		Lines.Emplace(Start + Offset1, End + Offset1, Color, -1.0f, 0.0f, 0);
		Lines.Emplace(End + Offset1, End + Offset2, Color, -1.0f, 0.0f, 0);
	}
	LineBatcher->DrawLines(Lines);
}

void FNiagaraStatelessDrawDebugContext::DrawCircle(const FVector& Center, const FQuat& Rotation, const FVector& Scale, const float Radius, const FColor& Color) const
{
	ULineBatchComponent* LineBatcher = World->GetLineBatcher(UWorld::ELineBatcherType::World);
	if (LineBatcher == nullptr)
	{
		return;
	}

	int32 Segments = 16;
	const float AngleStep = 2.f * UE_PI / float(Segments);

	const FVector AxisX = Rotation.RotateVector(FVector::XAxisVector) * Scale.X;
	const FVector AxisY = Rotation.RotateVector(FVector::YAxisVector) * Scale.Y;

	TArray<FBatchedLine> Lines;
	Lines.Empty(Segments);

	float Angle = 0.f;
	while (Segments--)
	{
		const FVector Vertex1 = Center + Radius * (AxisX * FMath::Cos(Angle) + AxisY * FMath::Sin(Angle));
		Angle += AngleStep;
		const FVector Vertex2 = Center + Radius * (AxisX * FMath::Cos(Angle) + AxisY * FMath::Sin(Angle));
		Lines.Emplace(Vertex1, Vertex2, Color, -1.0f, 0.0f, 0);
	}
	LineBatcher->DrawLines(Lines);
}

void FNiagaraStatelessDrawDebugContext::DrawRing(const FVector& Center, const FQuat& Rotation, const FVector& Scale, const float InnerRadius, const float OuterRadius, float Distribution, const FColor& Color) const
{
	if (ULineBatchComponent* LineBatcher = World->GetLineBatcher(UWorld::ELineBatcherType::World))
	{
		const int32 NumSegments = 16;
		const float AngleStep = -2.f * UE_PI * FMath::Clamp(1.0f - Distribution, 0.0f, 1.0f) / float(NumSegments);

		const FVector AxisX = Rotation.RotateVector(FVector::XAxisVector) * Scale.X;
		const FVector AxisY = Rotation.RotateVector(FVector::YAxisVector) * Scale.Y;

		TArray<FBatchedLine> Lines;
		Lines.Empty(2 + (NumSegments * 2));

		const bool bDrawCaps = FMath::IsNearlyEqual(Distribution, 0.0f) == false;
		for (int32 i=0; i < NumSegments; ++i)
		{
			const FVector Curr = AxisX * FMath::Cos(AngleStep * float(i + 0)) + AxisY * FMath::Sin(AngleStep * float(i + 0));
			const FVector Next = AxisX * FMath::Cos(AngleStep * float(i + 1)) + AxisY * FMath::Sin(AngleStep * float(i + 1));
			const FVector CurrInner = Center + InnerRadius * Curr;
			const FVector NextInner = Center + InnerRadius * Next;
			const FVector CurrOuter = Center + OuterRadius * Curr;
			const FVector NextOuter = Center + OuterRadius * Next;

			if (bDrawCaps && i == 0)
			{
				Lines.Emplace(CurrInner, CurrOuter, Color, -1.0f, 0.0f, 0);
			}
			if (bDrawCaps && i == NumSegments - 1)
			{
				Lines.Emplace(NextInner, NextOuter, Color, -1.0f, 0.0f, 0);
			}

			Lines.Emplace(CurrInner, NextInner, Color, -1.0f, 0.0f, 0);
			Lines.Emplace(CurrOuter, NextOuter, Color, -1.0f, 0.0f, 0);
		}
		LineBatcher->DrawLines(Lines);
	}
}

void FNiagaraStatelessDrawDebugContext::DrawSphere(const FVector& Center, const FQuat& Rotation, const FVector& Scale, const float Radius, const FColor& Color) const
{
	ULineBatchComponent* LineBatcher = World->GetLineBatcher(UWorld::ELineBatcherType::World);
	if (LineBatcher == nullptr)
	{
		return;
	}

	constexpr int NumSegments = 16;

	const FVector AxisX = Rotation.RotateVector(FVector::XAxisVector) * Scale.X;
	const FVector AxisY = Rotation.RotateVector(FVector::YAxisVector) * Scale.Y;
	const FVector AxisZ = Rotation.RotateVector(FVector::ZAxisVector) * Scale.Z;

	const float AngleInc = 2.f * UE_PI / NumSegments;
	int32 NumSegmentsY = NumSegments;
	float Latitude = AngleInc;
	float SinY1 = 0.0f, CosY1 = 1.0f;

	TArray<FBatchedLine> Lines;
	Lines.Empty(NumSegments * NumSegments);

	while (NumSegmentsY--)
	{
		const float SinY2 = FMath::Sin(Latitude);
		const float CosY2 = FMath::Cos(Latitude);

		FVector Vertex1 = FVector(SinY1, 0.0f, CosY1) * Radius + Center;
		FVector Vertex3 = FVector(SinY2, 0.0f, CosY2) * Radius + Center;
		Vertex1 = (AxisX * Vertex1.X) + (AxisY * Vertex1.Y) + (AxisZ * Vertex1.Z);
		Vertex3 = (AxisX * Vertex3.X) + (AxisY * Vertex3.Y) + (AxisZ * Vertex3.Z);

		float Longitude = AngleInc;

		int32 NumSegmentsX = NumSegments;
		while (NumSegmentsX--)
		{
			const float SinX = FMath::Sin(Longitude);
			const float CosX = FMath::Cos(Longitude);

			FVector Vertex2 = FVector((CosX * SinY1), (SinX * SinY1), CosY1) * Radius + Center;
			FVector Vertex4 = FVector((CosX * SinY2), (SinX * SinY2), CosY2) * Radius + Center;
			Vertex2 = (AxisX * Vertex2.X) + (AxisY * Vertex2.Y) + (AxisZ * Vertex2.Z);
			Vertex4 = (AxisX * Vertex4.X) + (AxisY * Vertex4.Y) + (AxisZ * Vertex4.Z);

			Lines.Emplace(Vertex1, Vertex2, Color, -1.0f, 0.0f, 0);
			Lines.Emplace(Vertex1, Vertex3, Color, -1.0f, 0.0f, 0);

			Vertex1 = Vertex2;
			Vertex3 = Vertex4;
			Longitude += AngleInc;
		}
		SinY1 = SinY2;
		CosY1 = CosY2;
		Latitude += AngleInc;
	}

	LineBatcher->DrawLines(Lines);
}

FQuat FNiagaraStatelessDrawDebugContext::TransformRotation(ENiagaraCoordinateSpace SourceSpace, const FQuat4f& Rotation) const
{
	return bApplyLocalToWorld[int(SourceSpace)] ? LocalToWorld.TransformRotation(FQuat(Rotation)) : FQuat(Rotation);
}

FVector FNiagaraStatelessDrawDebugContext::TransformPosition(ENiagaraCoordinateSpace SourceSpace, const FVector3f& Position) const
{
	return bApplyLocalToWorld[int(SourceSpace)] ? LocalToWorld.TransformPosition(FVector(Position)) : FVector(Position) + LWCTileOffset;
}

FVector FNiagaraStatelessDrawDebugContext::TransformVector(ENiagaraCoordinateSpace SourceSpace, const FVector3f& Vector) const
{
	return bApplyLocalToWorld[int(SourceSpace)] ? LocalToWorld.TransformVector(FVector(Vector)) : FVector(Vector);
}

FVector FNiagaraStatelessDrawDebugContext::TransformVectorNoScale(ENiagaraCoordinateSpace SourceSpace, const FVector3f& Vector) const
{
	return bApplyLocalToWorld[int(SourceSpace)] ? LocalToWorld.TransformVectorNoScale(FVector(Vector)) : FVector(Vector);
}
#endif
