// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/NiagaraStatelessEmitterData.h"
#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessModule.h"
#include "Stateless/NiagaraStatelessDrawDebugContext.h"
#include "Stateless/NiagaraStatelessParticleSimExecData.h"
#include "Stateless/Modules/NiagaraStatelessModule_InitializeParticle.h"
#include "NiagaraConstants.h"
#include "NiagaraModule.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraRenderThreadDeletor.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"

#include "Algo/Copy.h"
#include "Interfaces/ITargetPlatform.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessEmitter)

namespace NiagaraStatelessInternal
{
	static int32 GetDataSetIntOffset(const FNiagaraDataSetCompiledData& CompiledData, const FNiagaraVariableBase& Variable)
	{
		if (const FNiagaraVariableLayoutInfo* Layout = CompiledData.FindVariableLayoutInfo(Variable))
		{
			if (Layout->GetNumInt32Components() > 0)
			{
				return Layout->GetInt32ComponentStart();
			}
		}
		return INDEX_NONE;
	}

	static int32 GetDataSetFloatOffset(const FNiagaraDataSetCompiledData& CompiledData, const FNiagaraVariableBase& Variable)
	{
		if (const FNiagaraVariableLayoutInfo* Layout = CompiledData.FindVariableLayoutInfo(Variable))
		{
			if (Layout->GetNumFloatComponents() > 0)
			{
				return Layout->GetFloatComponentStart();
			}
		}
		return INDEX_NONE;
	}

	bool IsValid(const FNiagaraStatelessEmitterData& EmitterData)
	{
		// No spawn info / no renderers == no point doing anything
		//-TODO: If we allow particle reads this might be different
		if (!EmitterData.SpawnInfos.Num() || !EmitterData.RendererProperties.Num())
		{
			return false;
		}

		// Validate we have a template assigned
		if (!EmitterData.EmitterTemplate)
		{
			return false;
		}

		// Validate we have output components if we don't there's no point simulating the emitter
		if (!EmitterData.ParticleDataSetCompiledData.Get() || !EmitterData.ParticleDataSetCompiledData->Variables.Num())
		{
			return false;
		}

		// Validate the shader is correct
		const FShaderParametersMetadata* ShaderParametersMetadata = EmitterData.GetShaderParametersMetadata();
		if (!ShaderParametersMetadata)
		{
			return false;
		}

		if (!ensureMsgf(ShaderParametersMetadata->GetLayout().UniformBuffers.Num() == 0, TEXT("UniformBuffers are not supported in stateless simulations currently")))
		{
			// We don't support this as it would require a pass to clear out the buffer pointers to avoid leaks
			return false;
		}

		// Test the the first member in the shader struct is the common parameter block otherwise the shader is invalid
		{
			const TArray<FShaderParametersMetadata::FMember> ShaderParameterMembers = ShaderParametersMetadata->GetMembers();
			if (ShaderParameterMembers.Num() == 0 ||
				ShaderParameterMembers[0].GetBaseType() != UBMT_INCLUDED_STRUCT ||
				ShaderParameterMembers[0].GetStructMetadata()->GetLayout() != NiagaraStateless::FCommonShaderParameters::FTypeInfo::GetStructMetadata()->GetLayout() )
			{
				ensureMsgf(false, TEXT("NiagaraStateless::FCommonShaderParameters must be included first in your shader parameters structure"));
				return false;
			}
		}

		return true;
	}
}

void UNiagaraStatelessEmitter::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	// When cooking we will remove any modules that are not enabled as we can replace with the CDO at runtime saving memory
	TOptional<TGuardValue<TArray<TObjectPtr<UNiagaraStatelessModule>>>> ModulesGuard;
	if (Ar.IsCooking() && Ar.CookingTarget()->RequiresCookedData())
	{
		TArray<TObjectPtr<UNiagaraStatelessModule>> TrimmedModules;
		TrimmedModules.Reserve(Modules.Num());
		for (UNiagaraStatelessModule* Module : Modules)
		{
			TrimmedModules.Add(Module->IsModuleEnabled() ? Module : nullptr);
		}
		ModulesGuard.Emplace(Modules, TrimmedModules);
	}
#endif

	Super::Serialize(Ar);

	// On cooked make sure any null modules are replaced with the CDO
	if (Ar.IsLoading() && FPlatformProperties::RequiresCookedData())
	{
		if (EmitterTemplate)
		{
			TConstArrayView<TObjectPtr<UClass>> ModuleClass = EmitterTemplate->GetModules();
			for (int32 i = 0; i < Modules.Num(); ++i)
			{
				if (Modules[i] == nullptr)
				{
					Modules[i] = CastChecked<UNiagaraStatelessModule>(ModuleClass[i]->GetDefaultObject());
				}
			}
		}
	}
}

void UNiagaraStatelessEmitter::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (EmitterTemplateClass_DEPRECATED)
	{
		EmitterTemplate = Cast<UNiagaraStatelessEmitterTemplate>(EmitterTemplateClass_DEPRECATED->GetDefaultObject());
	}
#endif

#if WITH_EDITOR
	// Ensure our module list is up to date
	PostTemplateChanged();
	//CacheParameterCollectionReferences();
#endif
}

bool UNiagaraStatelessEmitter::NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const
{
	// Don't load disabled emitters.
	// Awkwardly, this requires us to look for ourselves in the owning system.
	const UNiagaraSystem* OwnerSystem = GetTypedOuter<const UNiagaraSystem>();
	if (OwnerSystem)
	{
		for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
		{
			if (EmitterHandle.GetStatelessEmitter() == this)
			{
				if (!EmitterHandle.GetIsEnabled())
				{
					return false;
				}
				break;
			}
		}
	}

	if (!FNiagaraPlatformSet::ShouldPruneEmittersOnCook(TargetPlatform->IniPlatformName()))
	{
		return true;
	}

	if (OwnerSystem && !OwnerSystem->GetScalabilityPlatformSet().IsEnabledForPlatform(TargetPlatform->IniPlatformName()))
	{
		UE_LOGF(LogNiagara, Verbose, "Pruned emitter %ls for platform %ls from system scalability", *GetFullName(), *TargetPlatform->DisplayName().ToString());
		return false;
	}

	const bool bIsEnabled = Platforms.IsEnabledForPlatform(TargetPlatform->IniPlatformName());
	if (!bIsEnabled)
	{
		UE_LOGF(LogNiagara, Verbose, "Pruned emitter %ls for platform %ls", *GetFullName(), *TargetPlatform->DisplayName().ToString())
	}
	return bIsEnabled;
}

#if WITH_EDITOR
void UNiagaraStatelessEmitter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FNiagaraDistributionBase::PostEditChangeProperty(this, PropertyChangedEvent);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	//-TODO: This should be done differently
	if (UNiagaraSystem* NiagaraSystem = GetTypedOuter<UNiagaraSystem>())
	{
		FNiagaraSystemUpdateContext UpdateContext(NiagaraSystem, true);

		// Ensure our template is up to date, right not it's easy to remove things by accident and cause a crash
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UNiagaraStatelessEmitter, EmitterTemplate))
		{
			PostTemplateChanged();
		}
		CacheParameterCollectionReferences();
	}
}
#endif //WITH_EDITOR

bool UNiagaraStatelessEmitter::CanObtainParticleAttribute(const FNiagaraVariableBase& InVar, const FGuid& EmitterVersion, FNiagaraTypeDefinition& OutBoundType) const
{
	return false;
}

bool UNiagaraStatelessEmitter::CanObtainEmitterAttribute(const FNiagaraVariableBase& InVarWithUniqueNameNamespace, FNiagaraTypeDefinition& OutBoundType) const
{
	return false;
}

#if WITH_EDITOR
void UNiagaraStatelessEmitter::HandleVariableRenamed(const FNiagaraVariable& InOldVariable, const FNiagaraVariable& InNewVariable, bool bUpdateContexts, FGuid EmitterVersion)
{
	for (UNiagaraRendererProperties* Renderer : RendererProperties)
	{
		Renderer->Modify(false);
		Renderer->RenameVariable(InOldVariable, InNewVariable, FVersionedNiagaraEmitterBase(this, EmitterVersion));
	}

	//if (bUpdateContexts)
	//{
	//	FNiagaraSystemUpdateContext UpdateCtx(FVersionedNiagaraEmitter(this, EmitterVersion), true);
	//}
}

void UNiagaraStatelessEmitter::HandleVariableRemoved(const FNiagaraVariable& InOldVariable, bool bUpdateContexts, FGuid EmitterVersion)
{
	for (UNiagaraRendererProperties* Renderer : RendererProperties)
	{
		Renderer->Modify(false);
		Renderer->RemoveVariable(InOldVariable, FVersionedNiagaraEmitterBase(this, EmitterVersion));
	}

	//if (bUpdateContexts)
	//{
	//	FNiagaraSystemUpdateContext UpdateCtx(FVersionedNiagaraEmitter(this, EmitterVersion), true);
	//}
}

void UNiagaraStatelessEmitter::GatherSimulationAttributeReferences(const FNiagaraStatelessSpawnInfo& SpawnInfo, TFunction<void(FName, const FNiagaraVariableBase&, bool, bool)> ReferenceFound) const
{
	FNiagaraDistributionBase::ForEachParameterBinding(
		FNiagaraStatelessSpawnInfo::StaticStruct(),
		&SpawnInfo,
		[&ReferenceFound, &SpawnInfo](const FNiagaraVariableBase& Variable)
		{
			const UEnum* SpawnTypeEnum = StaticEnum<ENiagaraStatelessSpawnInfoType>();
			const FName SpawnTypeName(*SpawnTypeEnum->GetDisplayNameTextByValue(int64(SpawnInfo.Type)).ToString());
			ReferenceFound(SpawnTypeName, Variable, true, false);
		}
	);
}

void UNiagaraStatelessEmitter::GatherSimulationAttributeReferences(const UNiagaraStatelessModule* Module, TFunction<void(FName, const FNiagaraVariableBase&, bool, bool)> ReferenceFound) const
{
	if (StatelessEmitterData == nullptr || !StatelessEmitterData->ParticleDataSetCompiledData.IsValid() || !Module || !Module->IsModuleEnabled())
	{
		return;
	}

	FNiagaraDistributionBase::ForEachParameterBinding(
		Module,
		[this, Module, &ReferenceFound](const FNiagaraVariableBase& Variable)
		{
			const FText ModuleDisplayText = Module->GetClass()->GetDisplayNameText();
			ReferenceFound(FName(*ModuleDisplayText.ToString()), Variable, true, false);
		}
	);

	// Gather particle data read / writes
	TArray<FNiagaraVariableBase> ModuleOutputVariables;
	Module->GetOutputVariables(ModuleOutputVariables, UNiagaraStatelessModule::EVariableFilter::Used);

	for (const FNiagaraVariableBase& DataSetVariable : StatelessEmitterData->ParticleDataSetCompiledData->Variables)
	{
		if (!ModuleOutputVariables.Contains(DataSetVariable))
		{
			continue;
		}

		FNameBuilder VariableNameBuilder;
		FNiagaraConstants::ParticleAttributeNamespace.ToString(VariableNameBuilder);
		VariableNameBuilder.AppendChar('.');
		DataSetVariable.GetName().AppendString(VariableNameBuilder);

		const FNiagaraVariableBase ParticleVariable(DataSetVariable.GetType(), FName(VariableNameBuilder.ToString()));
		const FText ModuleDisplayText = Module->GetClass()->GetDisplayNameText();
		ReferenceFound(FName(*ModuleDisplayText.ToString()), ParticleVariable, false, true);
	}
}

void UNiagaraStatelessEmitter::GatherSimulationAttributeReferences(TFunction<void(FName, const FNiagaraVariableBase&, bool, bool)> ReferenceFound) const
{
	for (const FNiagaraStatelessSpawnInfo& SpawnInfo : SpawnInfos)
	{
		GatherSimulationAttributeReferences(SpawnInfo, ReferenceFound);
	}

	for (const UNiagaraStatelessModule* Module : Modules)
	{
		GatherSimulationAttributeReferences(Module, ReferenceFound);
	}
}
#endif

#if WITH_EDITOR
void UNiagaraStatelessEmitter::PostTemplateChanged()
{
	// No template nothing to do
	if (EmitterTemplate == nullptr)
	{
		Modules.Empty();
		OnTemplateChanged.Broadcast();
		return;
	}

	// Null modules exist if the module was removed by accident or the source object was removed
	Modules.RemoveAll([](UNiagaraStatelessModule* Module) { return Module == nullptr; });

	// Order modules according to the new template and switch any existing modules over directly.
	//-Note: We could store old customized modules as editor only data for easy switching between templates
	TArray<TObjectPtr<UNiagaraStatelessModule>> NewModules;
	NewModules.Reserve(EmitterTemplate->GetModules().Num());
	for (UClass* ModuleClass : EmitterTemplate->GetModules())
	{
		const int32 ExistingIndex = Modules.IndexOfByPredicate([ModuleClass](UNiagaraStatelessModule* Existing) { return Existing->GetClass() == ModuleClass; });
		if (ExistingIndex == INDEX_NONE)
		{
			UNiagaraStatelessModule* NewModule = NewObject<UNiagaraStatelessModule>(this, ModuleClass, NAME_None, RF_Transactional);
			if (NewModule->CanDisableModule())
			{
				NewModule->SetIsModuleEnabled(false);
			}
			NewModules.Add(NewModule);
		}
		else
		{
			NewModules.Add(Modules[ExistingIndex]);
			Modules.RemoveAtSwap(ExistingIndex, EAllowShrinking::No);
		}
	}
	Modules = MoveTemp(NewModules);
	OnTemplateChanged.Broadcast();
}

void UNiagaraStatelessEmitter::CacheParameterCollectionReferences()
{
	TArray<TObjectPtr<UNiagaraParameterCollection>> OriginalReferences;
	Swap(OriginalReferences, CachedParameterCollectionReferences);

	for (const UNiagaraStatelessModule* Module : Modules)
	{
		for (TFieldIterator<FStructProperty> PropIt(Module->GetClass()); PropIt; ++PropIt)
		{
			FStructProperty* StructProp = *PropIt;
			if (!StructProp || !StructProp->Struct || !StructProp->Struct->IsChildOf(FNiagaraDistributionBase::StaticStruct()))
			{
				continue;
			}

			const FNiagaraDistributionBase* DistributionBase = StructProp->ContainerPtrToValuePtr<const FNiagaraDistributionBase>(Module);
			DistributionBase->ForEachParameterBinding(
				[this](const FNiagaraVariableBase& Variable)
				{
					if (Variable.IsInNameSpace(FNiagaraConstants::ParameterCollectionNamespaceString))
					{
						for (TObjectIterator<UNiagaraParameterCollection> NPCIt; NPCIt; ++NPCIt)
						{
							UNiagaraParameterCollection* NPC = *NPCIt;
							if (NPC->GetParameters().Contains(Variable))
							{
								CachedParameterCollectionReferences.AddUnique(NPC);
								break;
							}
						}
					}
				}
			);
		}
	}
	
	if ( OriginalReferences != CachedParameterCollectionReferences )
	{
		Modify();
	}
}
#endif //WITH_EDITOR

bool UNiagaraStatelessEmitter::UsesCollection(const UNiagaraParameterCollection* Collection) const
{
	return CachedParameterCollectionReferences.Contains(Collection);
}

const UNiagaraStatelessEmitterTemplate* UNiagaraStatelessEmitter::GetEmitterTemplate() const
{
	return EmitterTemplate;
}

void UNiagaraStatelessEmitter::CacheFromCompiledData()
{
	StatelessEmitterData = MakeShareable(new FNiagaraStatelessEmitterData(), FNiagaraRenderThreadDeletor<FNiagaraStatelessEmitterData>());
	StatelessEmitterData->bCanEverExecute = true;

#if WITH_NIAGARA_DEBUG_EMITTER_NAME
	StatelessEmitterData->DebugSimulationName = GetPackage()->GetFName();
	StatelessEmitterData->DebugEmitterName = GetFName();
#endif
	StatelessEmitterData->EmitterTemplate = GetEmitterTemplate();

	// Determine feature mask (needed before we build the data set)
	// Determine our supported feature set mask
	// If we can not execute on any enabled units then the emitter is considered disabled
	StatelessEmitterData->FeatureMask = FNiagaraStatelessGlobals::Get().FeatureMask;
	StatelessEmitterData->FeatureMask &= ENiagaraStatelessFeatureMask(AllowedFeatureMask);

	for (const UNiagaraStatelessModule* Module : Modules)
	{
		if (Module->IsModuleEnabled())
		{
			StatelessEmitterData->FeatureMask &= Module->GetFeatureMask();
		}
	}

	if (!StatelessEmitterData->GetShader().IsValid())
	{
		// No GPU shader we can only execute on the CPU path
		StatelessEmitterData->FeatureMask &= ~ENiagaraStatelessFeatureMask::ExecuteGPU;
	}

	if (StatelessEmitterData->FeatureMask == ENiagaraStatelessFeatureMask::None)
	{
		StatelessEmitterData->bCanEverExecute = false;
		UE_LOGF(LogNiagara, Log, "Stateless Emitter (%ls) can not execute on any available path and will be disabled.", *GetFullName());
	}

	// Compute the simulation target
	// This must always be done to ensure we filter renderers correctly and that the data set uses the correct target
	StatelessEmitterData->SimTarget = ComputeSimTarget();

	// Build data set
	BuildCompiledDataSet();

	// Setup emitter state
	StatelessEmitterData->EmitterState = EmitterState;

	if (StatelessEmitterData->EmitterState.bEnableDistanceCulling)
	{
		StatelessEmitterData->EmitterState.bEnableDistanceCulling = StatelessEmitterData->EmitterState.bMinDistanceEnabled | StatelessEmitterData->EmitterState.bMaxDistanceEnabled;
	}

	StatelessEmitterData->EmitterState.LoopDuration.Min = FMath::Max(StatelessEmitterData->EmitterState.LoopDuration.Min, UE_KINDA_SMALL_NUMBER);
	StatelessEmitterData->EmitterState.LoopDuration.Max = FMath::Max(StatelessEmitterData->EmitterState.LoopDuration.Max, UE_KINDA_SMALL_NUMBER);

	// Fill in common data, removing any spawn infos that are invalid to the simulation
	StatelessEmitterData->bDeterministic = bDeterministic;
	StatelessEmitterData->RandomSeed = RandomSeed ^ 0xdefa081;
	StatelessEmitterData->FixedBounds = FixedBounds;

	StatelessEmitterData->SpawnInfos.Reserve(SpawnInfos.Num());
	Algo::CopyIf(
		SpawnInfos,
		StatelessEmitterData->SpawnInfos,
		[this](const FNiagaraStatelessSpawnInfo& SpawnInfo)
		{
			return SpawnInfo.IsValid(EmitterState.LoopDuration.Max);
		}
	);

	// Fill in renderers
	StatelessEmitterData->RendererProperties = RendererProperties;
	for (UNiagaraRendererProperties* Renderer : RendererProperties)
	{
		if (Renderer && Renderer->GetIsEnabled())
		{
			Renderer->CacheFromCompiledData(StatelessEmitterData->ParticleDataSetCompiledData.Get());
		}
	}

	StatelessEmitterData->bCanEverExecute &= NiagaraStatelessInternal::IsValid(*StatelessEmitterData);
	StatelessEmitterData->bCanEverExecute &= IsAllowedByScalability();

	// Resolve scalability settings
	ResolveScalabilitySettings();

	// Build buffers that are shared across all instances
	//-OPT: We should be able to build and serialize this data as part of the UNiagaraStatelessEmitter, potentially all of this data even since it's immutable and does not change at runtime
	if (StatelessEmitterData->bCanEverExecute)
	{
		if (EnumHasAnyFlags(StatelessEmitterData->FeatureMask, ENiagaraStatelessFeatureMask::ExecuteCPU))
		{
			StatelessEmitterData->ParticleSimExecData = new NiagaraStateless::FParticleSimulationExecData(*StatelessEmitterData->ParticleDataSetCompiledData.Get());
		}

		FNiagaraStatelessEmitterDataBuildContext EmitterBuildContext(*StatelessEmitterData);
		FNiagaraStatelessShaderParametersBuilder ShaderParametersBuilder;
		ShaderParametersBuilder.AddParameterNestedStruct<NiagaraStateless::FCommonShaderParameters>();

		for (const UNiagaraStatelessModule* Module : Modules)
		{
			EmitterBuildContext.PreModuleBuild(ShaderParametersBuilder.GetParametersStructSize());
			Module->BuildShaderParameters(ShaderParametersBuilder);
			Module->BuildEmitterData(EmitterBuildContext);
		}
		StatelessEmitterData->bModulesHaveRendererBindings = StatelessEmitterData->RendererBindings.Num() > 0;

		// Gather the parameter collections we need to bind to
	#if WITH_EDITOR
		// UNiagaraStatelessEmitter does not get a notification when bindings change.
		//-OPT: Shift this to be when distributions change mode in / out of binding + binding changed
		CacheParameterCollectionReferences();
	#endif
		StatelessEmitterData->BoundParameterCollections.Reset(CachedParameterCollectionReferences.Num());
		for (UNiagaraParameterCollection* ParameterCollection : CachedParameterCollectionReferences)
		{
			if (ParameterCollection)
			{
				StatelessEmitterData->BoundParameterCollections.Add(ParameterCollection);
			}
		}

		// Find lifetime values
		//-Note: We could abstact this out to be a more general modules implements Lifetime but this mirrors core Niagara where you always have Initialize Particle in 99% of cases
		UNiagaraStatelessModule_InitializeParticle* InitializeParticleModule = nullptr;
		if (StatelessEmitterData->EmitterTemplate)
		{
			Modules.FindItemByClass(&InitializeParticleModule);
			if (ensure(InitializeParticleModule))
			{
				StatelessEmitterData->LifetimeRange = EmitterBuildContext.ConvertDistributionToRange(InitializeParticleModule->LifetimeDistribution, 0.0f);

				if ((StatelessEmitterData->LifetimeRange.Min <= 0.0f) && (StatelessEmitterData->LifetimeRange.Max <= 0.0f) && StatelessEmitterData->LifetimeRange.ParameterOffset == INDEX_NONE)
				{
					StatelessEmitterData->bCanEverExecute = false;
				}
			}
		}

		// Populate any renderer bindings from the emitter state and spawn infos
		EmitterBuildContext.ConvertDistributionToRange(StatelessEmitterData->EmitterState.LoopDuration, 0.0f);
		EmitterBuildContext.ConvertDistributionToRange(StatelessEmitterData->EmitterState.LoopDelay, 0.0f);
		for ( const FNiagaraStatelessSpawnInfo& SpawnInfo : StatelessEmitterData->SpawnInfos )
		{
			if (SpawnInfo.Type == ENiagaraStatelessSpawnInfoType::Rate)
			{
				EmitterBuildContext.ConvertDistributionToRange(SpawnInfo.Rate, 0.0f);
			}
			else //if (SpawnInfo.Type == ENiagaraStatelessSpawnInfoType::Burst)
			{
				EmitterBuildContext.ConvertDistributionToRange(SpawnInfo.Amount, 0);
			}
			EmitterBuildContext.ConvertDistributionToRange(SpawnInfo.SpawnProbability, 0.0f);
			EmitterBuildContext.ConvertDistributionToRange(SpawnInfo.LoopCountLimit, 0);
		}

		// Prepare renderer bindings this avoids having to do this per instance spawned
		// Note: Order is important here we detect if we need to update shader parameters on binding changes above by looking to see if we had any renderer bindings so this must be below
		for (UNiagaraRendererProperties* Renderer : RendererProperties )
		{
			if (Renderer && Renderer->GetIsEnabled())
			{
				Renderer->PopulateRequiredBindings(StatelessEmitterData->RendererBindings);
			}
		}

		// If the simulation target was modified to CPU then we might not be able to execute
		if (StatelessEmitterData->SimTarget == ENiagaraSimTarget::CPUSim && !EnumHasAnyFlags(StatelessEmitterData->FeatureMask, ENiagaraStatelessFeatureMask::ExecuteCPU))
		{
			StatelessEmitterData->bCanEverExecute = false;
		}
	}
}

ENiagaraSimTarget UNiagaraStatelessEmitter::ComputeSimTarget() const
{
	for (UNiagaraRendererProperties* Renderer : RendererProperties)
	{
		if (Renderer && Renderer->GetIsEnabled())
		{
			if (Renderer->IsSimTargetSupported(ENiagaraSimTarget::GPUComputeSim) == false)
			{
				return ENiagaraSimTarget::CPUSim;
			}
		}
	}
	return ENiagaraSimTarget::GPUComputeSim;
}

void UNiagaraStatelessEmitter::BuildCompiledDataSet()
{
#if WITH_EDITORONLY_DATA
	ParticleDataSetCompiledData.Empty();
	ParticleDataSetCompiledData.SimTarget = StatelessEmitterData->SimTarget;

	if (EmitterTemplate)
	{
		// Gather a list of all the output variables from the modules, this can change based on what is enabled / disabled
		TArray<FNiagaraVariableBase> AvailableVariables;
		AvailableVariables.Append(EmitterTemplate->GetImplicitVariables());

		for (UNiagaraStatelessModule* Module : Modules)
		{
			if (Module->IsModuleEnabled())
			{
				Module->GetOutputVariables(AvailableVariables, UNiagaraStatelessModule::EVariableFilter::Used);
			}
		}

		// Validate and remove any variables that are not compatible with the GPU path (if enabled)
		TConstArrayView<FNiagaraVariableBase> ShaderOutputVariables = EmitterTemplate->GetShaderOutputVariables();
		if (EnumHasAnyFlags(StatelessEmitterData->FeatureMask, ENiagaraStatelessFeatureMask::ExecuteGPU))
		{
			for (auto it=AvailableVariables.CreateIterator(); it; ++it)
			{
				if (!ShaderOutputVariables.Contains(*it))
				{
					UE_LOGF(LogNiagara, Log, "Removed variable '%ls' for emitter '%ls' as it's not part of the output components", *it->GetName().ToString(), *GetFullName());
					it.RemoveCurrent();
				}
			}
		}

		// Force all the attributes in?
		if (bForceOutputAllAttributes)
		{
			for (const FNiagaraVariableBase& Variable : AvailableVariables)
			{
				ParticleDataSetCompiledData.Variables.Emplace(Variable);
			}
		}
		// Build data set from variables that are used by renderers
		else
		{
			ForEachEnabledRenderer(
				[this, &AvailableVariables](UNiagaraRendererProperties* RendererProps)
				{
					if (AvailableVariables.Num() == 0)
					{
						return;
					}

					for (FNiagaraVariableBase BoundAttribute : RendererProps->GetBoundAttributes())
					{
						// Edge condition with UniqueID which does not contain the Particle namespace from Ribbon Renderer
						BoundAttribute.RemoveRootNamespace(FNiagaraConstants::ParticleAttributeNamespaceString);

						const int32 Index = AvailableVariables.IndexOfByKey(BoundAttribute);
						if (Index != INDEX_NONE)
						{
							AvailableVariables.RemoveAtSwap(Index, EAllowShrinking::No);
							ParticleDataSetCompiledData.Variables.Emplace(BoundAttribute);
							if (AvailableVariables.Num() == 0)
							{
								return;
							}
						}
					}
				}
			);

			if (bForceOutputUniqueID)
			{
				if (AvailableVariables.Contains(FNiagaraStatelessGlobals::Get().UniqueIDVariable))
				{
					ParticleDataSetCompiledData.Variables.Emplace(FNiagaraStatelessGlobals::Get().UniqueIDVariable);
				}
			}
		}

		//-TODO: We can alias variables in the data set, for example PreviousSpriteFacing could be SpriteFacing in some cases
		ParticleDataSetCompiledData.BuildLayout();

		// Create mapping from output components to data set for the shader to output
		ShaderOutputVariableOffsets.Empty(ShaderOutputVariables.Num());
		for (const FNiagaraVariableBase& OutputVariable : ShaderOutputVariables)
		{
			if (OutputVariable.GetType() == FNiagaraTypeDefinition::GetIntDef())
			{
				ShaderOutputVariableOffsets.Add(NiagaraStatelessInternal::GetDataSetIntOffset(ParticleDataSetCompiledData, OutputVariable));
			}
			else
			{
				ShaderOutputVariableOffsets.Add(NiagaraStatelessInternal::GetDataSetFloatOffset(ParticleDataSetCompiledData, OutputVariable));
			}
		}
		ShaderOutputVariableOffsets.Shrink();
	}
#endif
	StatelessEmitterData->ParticleDataSetCompiledData = MakeShared<FNiagaraDataSetCompiledData>(ParticleDataSetCompiledData);
	StatelessEmitterData->ShaderOutputVariableOffsets = ShaderOutputVariableOffsets;
}

void UNiagaraStatelessEmitter::ResolveScalabilitySettings()
{
	const float DefaultSpawnCountScale = 1.0f;
	StatelessEmitterData->SpawnCountScale = DefaultSpawnCountScale;

	if (UNiagaraSystem* OwnerSystem = GetTypedOuter<UNiagaraSystem>())
	{
		if (UNiagaraEffectType* ActualEffectType = OwnerSystem->GetEffectType())
		{
			const FNiagaraEmitterScalabilitySettings& ScalabilitySettings = ActualEffectType->GetActiveEmitterScalabilitySettings();
			if (ScalabilitySettings.bScaleSpawnCount)
			{
				StatelessEmitterData->SpawnCountScale = ScalabilitySettings.SpawnCountScale;
			}
		}
	}

	for (FNiagaraEmitterScalabilityOverride& Override : ScalabilityOverrides.Overrides)
	{
		if (Override.Platforms.IsActive())
		{
			if (Override.bOverrideSpawnCountScale)
			{
				StatelessEmitterData->SpawnCountScale = Override.bScaleSpawnCount ? Override.SpawnCountScale : DefaultSpawnCountScale;
			}
		}
	}	
}

NiagaraStateless::FCommonShaderParameters* UNiagaraStatelessEmitter::AllocateShaderParameters(const FNiagaraStatelessSpaceTransforms& SpaceTransforms, const FNiagaraParameterStore& RendererBindings) const
{
	// Allocate parameters
	const FShaderParametersMetadata* ShaderParametersMetadata = StatelessEmitterData->GetShaderParametersMetadata();
	const int32 ShaderParametersSize = ShaderParametersMetadata->GetSize();
	void* UntypedShaderParameters = FMemory::Malloc(ShaderParametersSize, SHADER_PARAMETER_STRUCT_ALIGNMENT);
	FMemory::Memset(UntypedShaderParameters, 0, ShaderParametersSize);

	// Fill in all of the shader parameters
	FNiagaraStatelessSetShaderParameterContext SetShaderParametersContext(
		SpaceTransforms,
		RendererBindings.GetParameterDataArray(),
		StatelessEmitterData->BuiltData,
		StatelessEmitterData->SharedBuiltData,
		StatelessEmitterData->SharedBuiltDataOffsets,
		ShaderParametersMetadata,
		static_cast<uint8*>(UntypedShaderParameters)
	);

	NiagaraStateless::FCommonShaderParameters* CommonParameters = SetShaderParametersContext.GetParameterNestedStruct<NiagaraStateless::FCommonShaderParameters>();

	for (UNiagaraStatelessModule* Module : Modules)
	{
		Module->SetShaderParameters(SetShaderParametersContext);
	}

	//-TODO: Add a way to set them directly, we should know that the final struct is a series of ints in the order of the provided variables
	GetEmitterTemplate()->SetShaderParameters(static_cast<uint8*>(UntypedShaderParameters), StatelessEmitterData->ShaderOutputVariableOffsets);

	return CommonParameters;
}

bool UNiagaraStatelessEmitter::IsAllowedByScalability() const
{
	return Platforms.IsActive();
}

#if WITH_EDITOR
void UNiagaraStatelessEmitter::SetEmitterTemplate(UNiagaraStatelessEmitterTemplate* Template)
{
	FNiagaraSystemUpdateContext UpdateContext;
	UpdateContext.SetDestroyOnAdd(true);
	if (UNiagaraSystem* Owner = GetTypedOuter<UNiagaraSystem>())
	{
		UpdateContext.Add(Owner, true);
	}

	EmitterTemplate = Template;
	PostTemplateChanged();
}

void UNiagaraStatelessEmitter::AddRenderer(UNiagaraRendererProperties* Renderer, FGuid EmitterVersion)
{
	FNiagaraSystemUpdateContext UpdateContext;
	UpdateContext.SetDestroyOnAdd(true);
	if (UNiagaraSystem* Owner = GetTypedOuter<UNiagaraSystem>())
	{
		UpdateContext.Add(Owner, true);
	}

	Modify();
	Renderer->OuterEmitterVersion = EmitterVersion;
	RendererProperties.Add(Renderer);
#if WITH_EDITOR
	// LWE does not support Emitter namespace so reset any bindings that point to emitter parameters
	if (UNiagaraRendererProperties* RendererCDO = Renderer->GetClass()->GetDefaultObject<UNiagaraRendererProperties>())
	{
		for (TFieldIterator<FStructProperty> PropIt(Renderer->GetClass()); PropIt; ++PropIt)
		{
			FStructProperty* StructProp = *PropIt;
			if (!StructProp || !StructProp->Struct || !StructProp->Struct->IsChildOf(FNiagaraVariableAttributeBinding::StaticStruct()))
			{
				continue;
			}
			FNiagaraVariableAttributeBinding* ParameterBinding = StructProp->ContainerPtrToValuePtr<FNiagaraVariableAttributeBinding>(Renderer);
			if ( ParameterBinding->GetBindingSourceMode() == ENiagaraBindingSource::ExplicitEmitter )
			{
				FNiagaraVariableAttributeBinding* ParameterBindingDefault = StructProp->ContainerPtrToValuePtr<FNiagaraVariableAttributeBinding>(RendererCDO);
				*ParameterBinding = *ParameterBindingDefault;
			}
		}
	}

	OnRenderersChangedDelegate.Broadcast();
#endif
}

void UNiagaraStatelessEmitter::RemoveRenderer(UNiagaraRendererProperties* Renderer, FGuid EmitterVersion)
{
	FNiagaraSystemUpdateContext UpdateContext;
	UpdateContext.SetDestroyOnAdd(true);
	if (UNiagaraSystem* Owner = GetTypedOuter<UNiagaraSystem>())
	{
		UpdateContext.Add(Owner, true);
	}

	Modify();
//	FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(EmitterVersion);
	RendererProperties.Remove(Renderer);
#if WITH_EDITOR
//	Renderer->OnChanged().RemoveAll(this);
//	UpdateChangeId(TEXT("Renderer removed"));
	OnRenderersChangedDelegate.Broadcast();
#endif
//	EmitterData->RebuildRendererBindings(*this);
}

void UNiagaraStatelessEmitter::MoveRenderer(UNiagaraRendererProperties* Renderer, int32 NewIndex, FGuid EmitterVersion)
{
//	FVersionedNiagaraEmitterData* EmitterData = GetEmitterData(EmitterVersion);
	int32 CurrentIndex = RendererProperties.IndexOfByKey(Renderer);
	if (CurrentIndex == INDEX_NONE || CurrentIndex == NewIndex || !RendererProperties.IsValidIndex(NewIndex))
	{
		return;
	}

	FNiagaraSystemUpdateContext UpdateContext;
	UpdateContext.SetDestroyOnAdd(true);
	if (UNiagaraSystem* Owner = GetTypedOuter<UNiagaraSystem>())
	{
		UpdateContext.Add(Owner, true);
	}

	Modify();
	RendererProperties.RemoveAt(CurrentIndex);
	RendererProperties.Insert(Renderer, NewIndex);
#if WITH_EDITOR
//	UpdateChangeId(TEXT("Renderer moved"));
	OnRenderersChangedDelegate.Broadcast();
#endif
//	EmitterData->RebuildRendererBindings(*this);
}

FNiagaraStatelessSpawnInfo& UNiagaraStatelessEmitter::AddSpawnInfo()
{
	FNiagaraSystemUpdateContext UpdateContext;
	UpdateContext.SetDestroyOnAdd(true);
	if (UNiagaraSystem* Owner = GetTypedOuter<UNiagaraSystem>())
	{
		UpdateContext.Add(Owner, true);
	}

	return SpawnInfos.AddDefaulted_GetRef();
}

void UNiagaraStatelessEmitter::RemoveSpawnInfoBySourceId(FGuid& InSourceIdToRemove)
{
	FNiagaraSystemUpdateContext UpdateContext;
	UpdateContext.SetDestroyOnAdd(true);
	if (UNiagaraSystem* Owner = GetTypedOuter<UNiagaraSystem>())
	{
		UpdateContext.Add(Owner, true);
	}

	SpawnInfos.RemoveAll([InSourceIdToRemove](const FNiagaraStatelessSpawnInfo& SpawnInfo) { return SpawnInfo.SourceId == InSourceIdToRemove; });
}

int32 UNiagaraStatelessEmitter::IndexOfSpawnInfoBySourceId(const FGuid& InSourceId)
{
	return SpawnInfos.IndexOfByPredicate([InSourceId](const FNiagaraStatelessSpawnInfo& SpawnInfo) { return SpawnInfo.SourceId == InSourceId; });
}

FNiagaraStatelessSpawnInfo* UNiagaraStatelessEmitter::FindSpawnInfoBySourceId(const FGuid& InSourceId)
{
	return SpawnInfos.FindByPredicate([InSourceId](const FNiagaraStatelessSpawnInfo& SpawnInfo) { return SpawnInfo.SourceId == InSourceId; });
}

FNiagaraStatelessSpawnInfo* UNiagaraStatelessEmitter::GetSpawnInfoByIndex(int32 Index)
{
	FNiagaraStatelessSpawnInfo* SpawnInfo = nullptr;
	if (Index >= 0 && Index < SpawnInfos.Num())
	{
		SpawnInfo = &SpawnInfos[Index];
	}
	return SpawnInfo;
}

UNiagaraStatelessModule* UNiagaraStatelessEmitter::GetModule(UClass* Class) const
{
	for (UNiagaraStatelessModule* Module : Modules)
	{
		if (Module && Module->GetClass() == Class)
		{
			return Module;
		}
	}
	return nullptr;
}

UNiagaraStatelessEmitter* UNiagaraStatelessEmitter::CreateAsDuplicate(FName InDuplicateName, UNiagaraSystem& InDuplicateOwnerSystem) const
{
	UNiagaraStatelessEmitter* NewEmitter = Cast<UNiagaraStatelessEmitter>(StaticDuplicateObject(this, &InDuplicateOwnerSystem));
	NewEmitter->ClearFlags(RF_Standalone | RF_Public);
	NewEmitter->SetUniqueEmitterName(InDuplicateName.GetPlainNameString());

	return NewEmitter;
}

void UNiagaraStatelessEmitter::DrawModuleDebug(UWorld* World, const FTransform& LocalToWorld) const
{
	FNiagaraStatelessDrawDebugContext DrawDebugContext(World, LocalToWorld, true/*bLocalSpace*/);

	for (const UNiagaraStatelessModule* Module : Modules)
	{
		if (Module->IsDebugDrawEnabled())
		{
			Module->DrawDebug(DrawDebugContext);
		}
	}
}

#endif //WITH_EDITOR
