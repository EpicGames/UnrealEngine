// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessEmitterData.h"
#include "Stateless/NiagaraStatelessEmitterTemplate.h"
#include "Stateless/NiagaraStatelessParticleSimExecData.h"
#include "Stateless/NiagaraStatelessSimulationShader.h"

#include "Engine/Engine.h"

FNiagaraStatelessEmitterData::~FNiagaraStatelessEmitterData()
{
	check(IsInRenderingThread());
	StaticDataBufferGpu.Release();

	if (ParticleSimExecData)
	{
		delete ParticleSimExecData;
		ParticleSimExecData = nullptr;
	}
}

void FNiagaraStatelessEmitterData::InitRenderResources()
{
	ENQUEUE_RENDER_COMMAND(InitNiagaraStatelessEmitterData)
	(
		[this](FRHICommandListImmediate& RHICmdList)
		{
			StaticDataBufferGpu.Initialize(RHICmdList, TEXT("NiagaraStatelessEmitterData_StaticDataBuffer"), StaticDataBufferCpu.Num(), EBufferUsageFlags::Static);
			void* BufferData = RHICmdList.LockBuffer(StaticDataBufferGpu.Buffer, 0, StaticDataBufferGpu.NumBytes, RLM_WriteOnly);
			FMemory::Memcpy(BufferData, StaticDataBufferCpu.GetData(), StaticDataBufferGpu.NumBytes);
			RHICmdList.UnlockBuffer(StaticDataBufferGpu.Buffer);
		}
	);
}

TShaderRef<NiagaraStateless::FSimulationShader>	FNiagaraStatelessEmitterData::GetShader() const
{
	if (FApp::CanEverRender() && EmitterTemplate)
	{
		return EmitterTemplate->GetSimulationShader();
	}
	return TShaderRef<NiagaraStateless::FSimulationShader>();
}

const FShaderParametersMetadata* FNiagaraStatelessEmitterData::GetShaderParametersMetadata() const
{
	check(EmitterTemplate);
	return EmitterTemplate->GetShaderParametersMetadata();
}

//float FNiagaraStatelessEmitterData::CalculateCompletionAge(int32 InRandomSeed, TConstArrayView<FNiagaraStatelessRuntimeSpawnInfo> RuntimeSpawnInfos) const
//{
//	float CompletionAge = -1.0f;
//	//for (const FNiagaraStatelessSpawnInfo& SpawnInfo : SpawnInfos)
//	//{
//	//	CompletionAge = FMath::Max(CompletionAge, SpawnInfo.AgeEnd);
//	//}
//
//	return CompletionAge >= 0.0f ? CompletionAge + LifetimeRange.Y : 0.0f;
//}

uint32 FNiagaraStatelessEmitterData::CalculateActiveParticles(int32 InRandomSeed, TConstArrayView<FNiagaraStatelessRuntimeSpawnInfo> RuntimeSpawnInfos, TOptional<float> Age, NiagaraStateless::FSpawnInfoShaderParameters* SpawnParameters) const
{
	int32	GpuSpawnIndex = 0;
	uint32	TotalActiveParticles = 0;

#if WITH_EDITOR && WITH_NIAGARA_DEBUG_EMITTER_NAME
	uint32	AmountOverLimit = 0;
#endif

	for (const FNiagaraStatelessRuntimeSpawnInfo& SpawnInfo : RuntimeSpawnInfos)
	{
		uint32 SpawnInfoTotalParticles = 0;

		const bool bIsValidForAge = !Age.IsSet() || (Age.GetValue() >= SpawnInfo.SpawnTimeStart && Age.GetValue() < SpawnInfo.SpawnTimeEnd + SpawnInfo.LifetimeMax);
		if (bIsValidForAge == false)
		{
			continue;
		}

		uint32	NumActive		= SpawnInfo.Amount;
		uint32	ParticleOffset	= 0;
		float	SpawnRate		= 0.0f;
		float	SpawnTimeStart	= SpawnInfo.SpawnTimeStart;
		switch (SpawnInfo.Type)
		{
			case ENiagaraStatelessSpawnInfoType::Burst:
				break;

			case ENiagaraStatelessSpawnInfoType::Rate:
				// If the age is set we can make a narrowed number of active particles calculation
				if (Age.IsSet())
				{
					const uint32 MaxActive = SpawnInfo.Amount;
					ParticleOffset = FMath::FloorToInt(FMath::Max(Age.GetValue() - SpawnInfo.SpawnTimeStart - SpawnInfo.LifetimeMax, 0.0f) * SpawnInfo.Rate);
					ParticleOffset = FMath::Min(ParticleOffset, MaxActive);
					NumActive = FMath::FloorToInt(FMath::Max(Age.GetValue() - SpawnInfo.SpawnTimeStart, 0.0f) * SpawnInfo.Rate);
					NumActive = FMath::Min(NumActive, MaxActive);
					NumActive -= ParticleOffset;
				}
				SpawnRate = 1.0f / SpawnInfo.Rate;
				SpawnTimeStart += SpawnRate;
				break;

			default:
				checkNoEntry();
				break;
		}

		if (NumActive == 0)
		{
			continue;
		}

		if (GpuSpawnIndex >= NiagaraStateless::MaxGpuSpawnInfos)
		{
		#if WITH_EDITOR && WITH_NIAGARA_DEBUG_EMITTER_NAME
			++AmountOverLimit;
			continue;
		#else
			break;
		#endif
		}

		if (SpawnParameters)
		{
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_NumActive, GpuSpawnIndex)		= NumActive;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_ParticleOffset, GpuSpawnIndex)	= ParticleOffset;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_UniqueOffset, GpuSpawnIndex)	= SpawnInfo.UniqueOffset;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_Time, GpuSpawnIndex)			= SpawnTimeStart;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_Rate, GpuSpawnIndex)			= SpawnRate;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_Probability, GpuSpawnIndex)		= SpawnInfo.Probability;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_LifetimeScale, GpuSpawnIndex)	= SpawnInfo.LifetimeMax - SpawnInfo.LifetimeMin;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_LifetimeBias, GpuSpawnIndex)	= SpawnInfo.LifetimeMin;
			++GpuSpawnIndex;
		}
		TotalActiveParticles += NumActive;
	}

#if WITH_EDITOR && WITH_NIAGARA_DEBUG_EMITTER_NAME
	if (AmountOverLimit > 0)
	{
		GEngine->AddOnScreenDebugMessage(
			uint64(this),
			1.f,
			FColor::White,
			*FString::Printf(
				TEXT("Stateless Simulation (%s:%s) has overflowed spawn infos (%d of %d), this can result in flickering / missing particles.  Reduce the number of spawn infos / loop time / particle age / increase code limit to fix."),
				*DebugSimulationName.ToString(), *DebugEmitterName.ToString(),
				NiagaraStateless::MaxGpuSpawnInfos + AmountOverLimit,
				NiagaraStateless::MaxGpuSpawnInfos
			)
		);
	}
#endif

	if (SpawnParameters)
	{
		while (GpuSpawnIndex < NiagaraStateless::MaxGpuSpawnInfos)
		{
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_NumActive, GpuSpawnIndex)		= 0;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_ParticleOffset, GpuSpawnIndex)	= 0;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_UniqueOffset, GpuSpawnIndex)	= 0;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_Time, GpuSpawnIndex)			= 0.0f;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_Rate, GpuSpawnIndex)			= 0.0f;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_Probability, GpuSpawnIndex)		= 0.0f;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_LifetimeScale, GpuSpawnIndex)	= 0.0f;
			GET_SCALAR_ARRAY_ELEMENT(SpawnParameters->SpawnInfo_LifetimeBias, GpuSpawnIndex)	= 0.0f;
			++GpuSpawnIndex;
		}
	}
	return TotalActiveParticles;
}

