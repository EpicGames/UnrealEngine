// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessEmitterDataBuildContext.h"
#include "Stateless/NiagaraStatelessEmitterData.h"
#include "Stateless/NiagaraStatelessExpression.h"
#include "Stateless/NiagaraStatelessParticleSimExecData.h"

#include "NiagaraDataSetCompiledData.h"
#include "NiagaraParameterBinding.h"
#include "NiagaraParameterStore.h"

namespace NiagaraStatelessEmitterDataBuildContextPrivate
{
	uint32 AddStaticData(TArray<uint8>& Buffer, const void* Data, uint32 DataSize)
	{
		if (DataSize == 0)
		{
			return 0;
		}

		constexpr uint32 BlockSize = 4;
		check((DataSize % BlockSize) == 0);

		constexpr bool bDeduplicateData = true;
		if (bDeduplicateData)
		{
			const uint32 iEndOffset = Buffer.Num();
			for (uint32 iOffset = 0; iOffset + DataSize <= iEndOffset; iOffset += BlockSize)
			{
				if (FMemory::Memcmp(&Buffer[iOffset], Data, DataSize) == 0)
				{
					return iOffset / BlockSize;
				}
			}
		}

		const uint32 OutIndex = Buffer.AddUninitialized(DataSize);
		FMemory::Memcpy(&Buffer[OutIndex], Data, DataSize);
		return OutIndex / BlockSize;
	}
};


FNiagaraStatelessEmitterDataBuildContext::FNiagaraStatelessEmitterDataBuildContext(FNiagaraStatelessEmitterData& InEmitterData)
	: EmitterData(InEmitterData)
{
	check(EmitterData.ParticleDataSetCompiledData.IsValid());
}

FNiagaraStatelessEmitterDataBuildContext::~FNiagaraStatelessEmitterDataBuildContext()
{
	EmitterData.Expressions.Shrink();
	EmitterData.BuiltData.Shrink();
	EmitterData.SharedBuiltData.Shrink();
	//EmitterData.ParticleSimExecData.Shrink();

	// Make sure our static data buffer allows us to sample the max size in one chunk without having to worry about overflow
	constexpr int MinSafeBufferSize = 16;
	if (EmitterData.StaticDataBufferCpu.Num() < MinSafeBufferSize)
	{
		EmitterData.StaticDataBufferCpu.AddZeroed(MinSafeBufferSize - EmitterData.StaticDataBufferCpu.Num());
	}
	EmitterData.StaticDataBufferCpu.Shrink();

	// Initialize the GPU data
	EmitterData.InitRenderResources();
}

void FNiagaraStatelessEmitterDataBuildContext::PreModuleBuild(int32 InShaderParameterOffset)
{
	ModuleBuiltDataOffset = EmitterData.BuiltData.Num();
	ShaderParameterOffset = InShaderParameterOffset;
	++RandomSeedOffest;
}

uint32 FNiagaraStatelessEmitterDataBuildContext::AddStaticData(TConstArrayView<uint32> StaticData) const
{
	return NiagaraStatelessEmitterDataBuildContextPrivate::AddStaticData(EmitterData.StaticDataBufferCpu, StaticData.GetData(), StaticData.GetTypeSize() * StaticData.Num());
}

uint32 FNiagaraStatelessEmitterDataBuildContext::AddStaticData(TConstArrayView<int32> StaticData) const
{
	return NiagaraStatelessEmitterDataBuildContextPrivate::AddStaticData(EmitterData.StaticDataBufferCpu, StaticData.GetData(), StaticData.GetTypeSize() * StaticData.Num());
}

uint32 FNiagaraStatelessEmitterDataBuildContext::AddStaticData(TConstArrayView<float> StaticData) const
{
	return NiagaraStatelessEmitterDataBuildContextPrivate::AddStaticData(EmitterData.StaticDataBufferCpu, StaticData.GetData(), StaticData.GetTypeSize() * StaticData.Num());
}

uint32 FNiagaraStatelessEmitterDataBuildContext::AddStaticData(TConstArrayView<FVector2f> StaticData) const
{
	return NiagaraStatelessEmitterDataBuildContextPrivate::AddStaticData(EmitterData.StaticDataBufferCpu, StaticData.GetData(), StaticData.GetTypeSize() * StaticData.Num());
}

uint32 FNiagaraStatelessEmitterDataBuildContext::AddStaticData(TConstArrayView<FVector3f> StaticData) const
{
	return NiagaraStatelessEmitterDataBuildContextPrivate::AddStaticData(EmitterData.StaticDataBufferCpu, StaticData.GetData(), StaticData.GetTypeSize() * StaticData.Num());
}

uint32 FNiagaraStatelessEmitterDataBuildContext::AddStaticData(TConstArrayView<FVector4f> StaticData) const
{
	return NiagaraStatelessEmitterDataBuildContextPrivate::AddStaticData(EmitterData.StaticDataBufferCpu, StaticData.GetData(), StaticData.GetTypeSize() * StaticData.Num());
}

uint32 FNiagaraStatelessEmitterDataBuildContext::AddStaticData(TConstArrayView<FLinearColor> StaticData) const
{
	return NiagaraStatelessEmitterDataBuildContextPrivate::AddStaticData(EmitterData.StaticDataBufferCpu, StaticData.GetData(), StaticData.GetTypeSize() * StaticData.Num());
}

void* FNiagaraStatelessEmitterDataBuildContext::AllocateBuiltData(uint32 Size, uint32 Alingment) const
{
	const int32 Offset = Align(EmitterData.BuiltData.Num(), Alingment);
	EmitterData.BuiltData.AddZeroed(Offset + Size - EmitterData.BuiltData.Num());
	return EmitterData.BuiltData.GetData() + Offset;
}

void* FNiagaraStatelessEmitterDataBuildContext::AllocateSharedBuiltData(FName DataName, uint32 Size, uint32 Alingment) const
{
	check(EmitterData.SharedBuiltDataOffsets.ContainsByPredicate([DataName](const TPair<FName, uint32>& Pair) { return Pair.Key == DataName; }) == false);

	const int32 Offset = Align(EmitterData.SharedBuiltData.Num(), Alingment);
	EmitterData.SharedBuiltData.AddZeroed(Offset + Size - EmitterData.SharedBuiltData.Num());
	void* NewData = EmitterData.SharedBuiltData.GetData() + Offset;

	EmitterData.SharedBuiltDataOffsets.Emplace(DataName, reinterpret_cast<uintptr_t>(NewData) - reinterpret_cast<uintptr_t>(EmitterData.SharedBuiltData.GetData()));
	return NewData;
}

int32 FNiagaraStatelessEmitterDataBuildContext::AddRendererBinding(const FNiagaraVariableBase& Variable) const
{
	int32 DataOffset = INDEX_NONE;
	if (Variable.IsValid())
	{
		FNiagaraVariable Var(Variable);
		EmitterData.RendererBindings.AddParameter(Var, false, false, &DataOffset);
		DataOffset /= sizeof(uint32);
	}

	return DataOffset;
}

int32 FNiagaraStatelessEmitterDataBuildContext::AddRendererBinding(const FNiagaraParameterBinding& Binding) const
{
	int32 DataOffset = INDEX_NONE;
	if (Binding.ResolvedParameter.IsValid())
	{
		FNiagaraVariable Var(Binding.ResolvedParameter);
		EmitterData.RendererBindings.AddParameter(Var, false, false, &DataOffset);
		DataOffset /= sizeof(uint32);
	}

	return DataOffset;
}

int32 FNiagaraStatelessEmitterDataBuildContext::AddRendererBinding(const FNiagaraParameterBindingWithValue& Binding) const
{
	int32 DataOffset = INDEX_NONE;
	if (Binding.ResolvedParameter.IsValid())
	{
		FNiagaraVariable Var(Binding.ResolvedParameter);
		EmitterData.RendererBindings.AddParameter(Var, false, false, &DataOffset);

		TConstArrayView<uint8> DefaultValue = Binding.GetDefaultValueArray();
		if (DefaultValue.Num() > 0)
		{
			check(DataOffset != INDEX_NONE);
			check(DefaultValue.Num() == Var.GetSizeInBytes());
			EmitterData.RendererBindings.SetParameterData(DefaultValue.GetData(), DataOffset, DefaultValue.Num());
		}

		DataOffset /= sizeof(uint32);
	}

	return DataOffset;
}


int32 FNiagaraStatelessEmitterDataBuildContext::AddExpression(const FInstancedStruct& ExpressionStruct) const
{
	int32 DataOffset = INDEX_NONE;
	if (const FNiagaraStatelessExpression* Expression = ExpressionStruct.GetPtr<FNiagaraStatelessExpression>())
	{
		const FName ExpressionName("__StatelessInternal__.Expression", EmitterData.Expressions.Num());
		const FNiagaraVariable ExpressionVariable(Expression->GetOutputTypeDef(), ExpressionName);
		EmitterData.RendererBindings.AddParameter(ExpressionVariable, false, false, &DataOffset);

		EmitterData.Expressions.Emplace(DataOffset, Expression->Build(*this));
		DataOffset /= sizeof(uint32);
	}
	return DataOffset;
}

void FNiagaraStatelessEmitterDataBuildContext::AddParticleSimulationExecSimulate(TFunction<void(const NiagaraStateless::FParticleSimulationContext&)> Func) const
{
	if (!EmitterData.ParticleSimExecData)
	{
		return;
	}

	EmitterData.ParticleSimExecData->SimulateFunctions.Emplace(MoveTemp(Func), ModuleBuiltDataOffset, ShaderParameterOffset, RandomSeedOffest);
}

int32 FNiagaraStatelessEmitterDataBuildContext::FindParticleVariableIndex(const FNiagaraVariableBase& Variable) const
{
	return EmitterData.ParticleDataSetCompiledData->Variables.IndexOfByKey(Variable);
}
