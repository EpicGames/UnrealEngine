// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessEmitterInstance.h"
#include "Stateless/NiagaraStatelessComputeManager.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/NiagaraStatelessEmitterData.h"
#include "Stateless/NiagaraStatelessExpression.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraParameterCollectionResolver.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstance.h"

#include "Shader.h"

namespace NiagaraStatelessEmitterInstancePrivate
{
	static const float DefaultLoopDuration		= 0.001f;
	static const float DefaultLoopDelay			= 0.0f;
	static const float DefaultSpawnRate			= 0.0f;
	static const float DefaultSpawnProbability	= 0.0f;
	static const int32 DefaultLoopCountLimit	= 1;
	static const int32 DefaultSpawnAmount		= 0;

	// Other areas work on built distributions so this is a helper function to work with a raw distribution
	float EvaluateDistribution(const FNiagaraDistributionRangeFloat& Distribution, FRandomStream& RandomStream, const FNiagaraParameterStore& ParameterStore, const float DefaultValue)
	{
		if (Distribution.IsBinding())
		{
			return Distribution.ParameterBinding.IsValid() ? ParameterStore.GetParameterValueOrDefault(Distribution.ParameterBinding, DefaultValue) : DefaultValue;
		}
		else if (Distribution.IsExpression())
		{
			check(Distribution.ParameterExpression.Get<FNiagaraStatelessExpression>().GetOutputTypeDef() == FNiagaraTypeDefinition::GetFloatDef());
			float ExpressionValue = DefaultValue;
			Distribution.ParameterExpression.Get<FNiagaraStatelessExpression>().Evaluate(FNiagaraStatelessExpression::FEvaluateContext(ParameterStore), &ExpressionValue);
			return ExpressionValue;
		}
		const float Fraction = RandomStream.GetFraction();
		return ((Distribution.Max - Distribution.Min) * Fraction) + Distribution.Min;
	}

	int32 EvaluateDistribution(const FNiagaraDistributionRangeInt& Distribution, FRandomStream& RandomStream, const FNiagaraParameterStore& ParameterStore, const int32 DefaultValue)
	{
		if (Distribution.IsBinding())
		{
			return Distribution.ParameterBinding.IsValid() ? ParameterStore.GetParameterValueOrDefault(Distribution.ParameterBinding, DefaultValue) : DefaultValue;
		}
		else if (Distribution.IsExpression())
		{
			check(Distribution.ParameterExpression.Get<FNiagaraStatelessExpression>().GetOutputTypeDef() == FNiagaraTypeDefinition::GetIntDef());
			int32 ExpressionValue = DefaultValue;
			Distribution.ParameterExpression.Get<FNiagaraStatelessExpression>().Evaluate(FNiagaraStatelessExpression::FEvaluateContext(ParameterStore), &ExpressionValue);
			return ExpressionValue;
		}
		return RandomStream.RandRange(Distribution.Min, Distribution.Max);
	}

	bool EvaluateLifetime(const FNiagaraStatelessRangeFloat& Distribution, FRandomStream& RandomStream, const FNiagaraParameterStore& ParameterStore, float& OutMin, float& OutMax)
	{
		if (Distribution.ParameterOffset != INDEX_NONE)
		{
			OutMin = ParameterStore.GetParameterValueFromOffset<float>(Distribution.ParameterOffset * sizeof(int32));
			OutMax = OutMin;
		}
		else
		{
			OutMin = Distribution.Min;
			OutMax = Distribution.Max;
		}
		return (OutMin > 0.0f) || (OutMax > 0.0f);
	}

	void SetShaderParameterTransforms(const FNiagaraStatelessSpaceTransforms& Transforms, NiagaraStateless::FCommonShaderParameters* ShaderParameters)
	{
		ShaderParameters->Common_LocalToWorldScale = Transforms.GetLocalToWorld().GetScale3D();
		ShaderParameters->Common_ToSimulationRotations[int(ENiagaraCoordinateSpace::Simulation)] = Transforms.GetTransform(ENiagaraCoordinateSpace::Simulation, ENiagaraCoordinateSpace::Simulation).GetRotation();
		ShaderParameters->Common_ToSimulationRotations[int(ENiagaraCoordinateSpace::World)] = Transforms.GetTransform(ENiagaraCoordinateSpace::World, ENiagaraCoordinateSpace::Simulation).GetRotation();
		ShaderParameters->Common_ToSimulationRotations[int(ENiagaraCoordinateSpace::Local)] = Transforms.GetTransform(ENiagaraCoordinateSpace::Local, ENiagaraCoordinateSpace::Simulation).GetRotation();
	}
}

FNiagaraDataBuffer* NiagaraStateless::FEmitterInstance_RT::GetDataToRender(FRHICommandListBase& RHICmdList, bool bIsLowLatencyTranslucent) const
{
	return ComputeManager ? ComputeManager->GetDataBuffer(RHICmdList, uintptr_t(this), this) : nullptr;
}

FNiagaraStatelessEmitterInstance::FNiagaraStatelessEmitterInstance(FNiagaraSystemInstance* InParentSystemInstance)
	: Super(InParentSystemInstance)
{
	// Setup base properties
	bLocalSpace = true;
	SimTarget = ENiagaraSimTarget::GPUComputeSim;
	bNeedsPartialDepthTexture = false;
	ParticleDataSet = new FNiagaraDataSet();
}

FNiagaraStatelessEmitterInstance::~FNiagaraStatelessEmitterInstance()
{
	//-TODO: Should we move this into the base class?
	UnbindParameters(false);

	NiagaraStateless::FEmitterInstance_RT* RenderThreadData = RenderThreadDataPtr.Release();
	if (RenderThreadData || ParticleDataSet)
	{
		ENQUEUE_RENDER_COMMAND(FReleaseStatelessEmitter)(
			[RenderThreadData, ParticleDataSet=ParticleDataSet](FRHICommandListImmediate& RHICmdList)
			{
				if (RenderThreadData != nullptr)
				{
					delete RenderThreadData;
				}
				if (ParticleDataSet != nullptr )
				{
					delete ParticleDataSet;
				}
			}
		);
		ParticleDataSet = nullptr;
	}
}

void FNiagaraStatelessEmitterInstance::Init(int32 InEmitterIndex)
{
	Super::Init(InEmitterIndex);

	// Initialize the EmitterData ptr if this is invalid the emitter is not allowed to run
	InitEmitterData();
	if (!bCanEverExecute)
	{
		InternalExecutionState = ENiagaraExecutionState::Disabled;
		ExecutionState = InternalExecutionState;
		return;
	}

	// Pull out information
	RandomSeed = EmitterData->RandomSeed + ParentSystemInstance->GetRandomSeedOffset();
	if (!EmitterData->bDeterministic)
	{
		RandomSeed ^= FPlatformTime::Cycles();
	}
	RandomStream.Initialize(RandomSeed);
	//CompletionAge = EmitterData->CalculateCompletionAge(RandomSeed);

	// Initialize data set
	ParticleDataSet->Init(EmitterData->ParticleDataSetCompiledData.Get());

	// Prepare our parameters
	RendererBindings = EmitterData->RendererBindings;

	EmitterTransforms.InitializeTransforms(IsLocalSpace(), FTransform3f(ParentSystemInstance->GetWorldTransform()));

	// Allocate and fill shader parameters
	if (SimTarget == ENiagaraSimTarget::CPUSim)
	{
		ShaderParameters.Reset(WeakStatelessEmitter->AllocateShaderParameters(EmitterTransforms, RendererBindings));
		ShaderParameters->Common_RandomSeed = RandomSeed;
		NiagaraStatelessEmitterInstancePrivate::SetShaderParameterTransforms(EmitterTransforms, ShaderParameters.Get());
	}
	else
	{
		RenderThreadDataPtr.Reset(new NiagaraStateless::FEmitterInstance_RT());
		RenderThreadDataPtr->EmitterData = EmitterData;
		RenderThreadDataPtr->RandomSeed = RandomSeed;
		RenderThreadDataPtr->Age = 0.0f;
		RenderThreadDataPtr->DeltaTime = 0.0f;
		RenderThreadDataPtr->ExecutionState = ENiagaraExecutionState::Active;
		RenderThreadDataPtr->ShaderParameters.Reset(WeakStatelessEmitter->AllocateShaderParameters(EmitterTransforms, RendererBindings));
		RenderThreadDataPtr->ShaderParameters->Common_RandomSeed = RandomSeed;
		NiagaraStatelessEmitterInstancePrivate::SetShaderParameterTransforms(EmitterTransforms, RenderThreadDataPtr->ShaderParameters.Get());

		ENQUEUE_RENDER_COMMAND(FInitStatelessEmitter)(
			[RenderThreadData=RenderThreadDataPtr.Get(), ComputeInterface = ParentSystemInstance->GetComputeDispatchInterface()](FRHICommandListImmediate& RHICmdList)
			{
				RenderThreadData->ComputeManager = &ComputeInterface->GetOrCreateDataManager<FNiagaraStatelessComputeManager>();
			}
		);

		GPUDataBufferInterfaces = RenderThreadDataPtr.Get();
	}

	bNeedsEmitterStateInit = true;
}

void FNiagaraStatelessEmitterInstance::ResetSimulation(bool bKillExisting)
{
	if (!bCanEverExecute)
	{
		return;
	}

	if (bKillExisting)
	{
		SpawnInfos.Empty();

		UniqueIndexOffset = 0;
		if (!EmitterData->bDeterministic)
		{
			RandomSeed ^= FPlatformTime::Cycles();
		}
	}
	else
	{
		for (FNiagaraStatelessRuntimeSpawnInfo& SpawnInfo : SpawnInfos)
		{
			SpawnInfo.SpawnTimeStart -= Age;
			SpawnInfo.SpawnTimeEnd -= Age;
		}
	}
	ActiveSpawnRates.Empty();
	bSpawnInfosDirty = true;

	RandomStream.Initialize(RandomSeed);

	Age = 0.0f;
	bEmitterEnabled_CNC = bEmitterEnabled_GT;

	bNeedsEmitterStateInit = true;

	InternalExecutionState = ENiagaraExecutionState::Active;
	ExecutionState = InternalExecutionState;
	ScalabilityState = ENiagaraExecutionStateManagement::Awaken;

	if (SimTarget == ENiagaraSimTarget::CPUSim)
	{
		if (ShaderParameters.IsValid())
		{
			ShaderParameters->Common_RandomSeed = RandomSeed;
		}
	}
	else
	{
		if (NiagaraStateless::FEmitterInstance_RT* RenderThreadData = RenderThreadDataPtr.Get())
		{
			ENQUEUE_RENDER_COMMAND(UpdateStatelessAge)(
				[RenderThreadData, RandomSeed_RT=RandomSeed](FRHICommandListImmediate& RHICmdList)
				{
					RenderThreadData->Age = 0.0f;
					RenderThreadData->DeltaTime = 0.0f;
					RenderThreadData->ExecutionState = ENiagaraExecutionState::Active;
					RenderThreadData->RandomSeed = RandomSeed_RT;
					if (RenderThreadData->ShaderParameters.IsValid())
					{
						RenderThreadData->ShaderParameters->Common_RandomSeed = RandomSeed_RT;
					}
				}
			);
		}
	}
}

void FNiagaraStatelessEmitterInstance::SetEmitterEnable(bool bNewEnableState)
{
	bEmitterEnabled_GT = bNewEnableState;
}

bool FNiagaraStatelessEmitterInstance::HandleCompletion(bool bForce)
{
	bool bIsComplete = IsComplete();
	if (!bIsComplete && bForce)
	{
		InternalExecutionState = ENiagaraExecutionState::Complete;
		ExecutionState = InternalExecutionState;
		bIsComplete = true;

		if (NiagaraStateless::FEmitterInstance_RT* RenderThreadData = RenderThreadDataPtr.Get())
		{
			ENQUEUE_RENDER_COMMAND(CompleteStatelessEmitter)(
				[RenderThreadData](FRHICommandListImmediate& RHICmdList)
				{
					RenderThreadData->ExecutionState = ENiagaraExecutionState::Complete;
				}
			);
		}
	}

	return bIsComplete;
}

int32 FNiagaraStatelessEmitterInstance::GetNumParticles() const
{
	return bCanEverExecute && (SpawnInfos.Num() > 0) ? EmitterData->CalculateActiveParticles(RandomSeed, SpawnInfos, Age) : 0;
}

TConstArrayView<UNiagaraRendererProperties*> FNiagaraStatelessEmitterInstance::GetRenderers() const
{
	return EmitterData ? EmitterData->RendererProperties : TConstArrayView<UNiagaraRendererProperties*>();
}

void FNiagaraStatelessEmitterInstance::BindParameters(bool bExternalOnly)
{
	if (!RendererBindings.IsEmpty())
	{
		if (ParentSystemInstance)
		{
			ParentSystemInstance->BindToParameterStore(RendererBindings);

			const FNiagaraParameterCollectionResolver ParameterCollectionResolver = ParentSystemInstance->CreateParameterCollectionResolver();
			ParameterCollectionResolver.BindCollections(MakeArrayView(EmitterData->BoundParameterCollections), RendererBindings);
		}
	}
}

void FNiagaraStatelessEmitterInstance::UnbindParameters(bool bExternalOnly)
{
	if (!RendererBindings.IsEmpty())
	{
		if (ParentSystemInstance)
		{
			ParentSystemInstance->UnbindFromParameterStore(RendererBindings);

			const FNiagaraParameterCollectionResolver ParameterCollectionResolver = ParentSystemInstance->CreateParameterCollectionResolver();
			ParameterCollectionResolver.UnbindCollections(MakeArrayView(EmitterData->BoundParameterCollections), RendererBindings);
		}
	}
}

void FNiagaraStatelessEmitterInstance::RebindParameterCollection(UNiagaraParameterCollectionInstance* OldCollection, UNiagaraParameterCollectionInstance* NewCollection)
{
	if (RendererBindings.IsEmpty() || !ParentSystemInstance)
	{
		return;
	}

	if (!EmitterData->BoundParameterCollections.Contains(OldCollection->GetParent()))
	{
		return;
	}

	OldCollection->GetParameterStore().Unbind(&RendererBindings);
	NewCollection->GetParameterStore().Bind(&RendererBindings);
}

bool FNiagaraStatelessEmitterInstance::ShouldTick() const
{
	return InternalExecutionState <= ENiagaraExecutionState::Inactive;
}

void FNiagaraStatelessEmitterInstance::Tick(float DeltaSeconds)
{
	const bool bForceParametersUpdate = bNeedsEmitterStateInit;
	if ( bNeedsEmitterStateInit )
	{
		bNeedsEmitterStateInit = false;
		InitEmitterState();
		InitSpawnInfos(0.0f);
	}

	Age += DeltaSeconds;

	TickSpawnInfos();
	TickEmitterState();
	CalculateBounds();
	UpdateSimulationData(DeltaSeconds, bForceParametersUpdate);
}

void FNiagaraStatelessEmitterInstance::InitEmitterState()
{
	using namespace NiagaraStatelessEmitterInstancePrivate;

	const FNiagaraEmitterStateData& EmitterState = EmitterData->EmitterState;
	LoopCount			= 0;

	CurrentLoopDelay = 0.0f;
	if (EmitterState.bLoopDelayEnabled)
	{
		CurrentLoopDelay = EvaluateDistribution(EmitterState.LoopDelay, RandomStream, RendererBindings, DefaultLoopDelay);
		CurrentLoopDelay = FMath::Max(CurrentLoopDelay, 0.0f);
	}
	CurrentLoopAgeStart	= 0.0f;

	if (EmitterState.LoopBehavior == ENiagaraLoopBehavior::Once && EmitterState.LoopDurationMode == ENiagaraLoopDurationMode::Infinite)
	{
		// Infinite will loop after this time which works as expected, however we do start to run into numerical precision issues after running for around 1.5 days
		// We also can not use TNumericLimits<float>::Max() / FLT_MAX as it will introduce numerical precision issues resulting in nothing appearing to spawn for rate based spawning
		constexpr float LongDuration = 60.0f * 60.0f * 24.0f;
		CurrentLoopDuration	= LongDuration;
		CurrentLoopAgeEnd	= LongDuration;
	}
	else
	{
		CurrentLoopDuration = EvaluateDistribution(EmitterState.LoopDuration, RandomStream, RendererBindings, DefaultLoopDuration);
		CurrentLoopDuration = FMath::Max(CurrentLoopDuration, DefaultLoopDuration);
		CurrentLoopAgeEnd	= CurrentLoopAgeStart + CurrentLoopDelay + CurrentLoopDuration;
	}
}

void FNiagaraStatelessEmitterInstance::TickEmitterState()
{
	using namespace NiagaraStatelessEmitterInstancePrivate;

	// Update execution state based on the parent which be told to go inactive / complete
	{
		const ENiagaraExecutionState ParentExecutionState = ParentSystemInstance ? ParentSystemInstance->GetActualExecutionState() : ENiagaraExecutionState::Complete;
		if (ParentExecutionState > InternalExecutionState)
		{
			SetExecutionStateInternal(ParentExecutionState);
		}
	}

	// If we are going inactive and we hit zero particles we are now complete
	if (InternalExecutionState == ENiagaraExecutionState::Inactive)
	{
		if (GetNumParticles() == 0)
		{
			SetExecutionStateInternal(ENiagaraExecutionState::Complete);
		}
	}

	// If we are not active we don't need to evaluate loops / scalability anymore
	if (InternalExecutionState != ENiagaraExecutionState::Active )
	{
		return;
	}

	const FNiagaraEmitterStateData& EmitterState = EmitterData->EmitterState;

	// Evaluate scalability state
	{
		ENiagaraExecutionStateManagement RequestedScalabilityState = ENiagaraExecutionStateManagement::Awaken;
		if (EmitterState.bEnableVisibilityCulling)
		{
			const FNiagaraSystemParameters& SystemParameters = ParentSystemInstance->GetSystemParameters();
			if (SystemParameters.EngineTimeSinceRendered > EmitterState.VisibilityCullDelay)
			{
				RequestedScalabilityState = EmitterState.VisibilityCullReaction;
			}
		}

		if (EmitterState.bEnableDistanceCulling)
		{
			const float LODDistance = ParentSystemInstance->GetLODDistance();

			if (LODDistance > EmitterState.MaxDistance)
			{
				RequestedScalabilityState = EmitterState.MaxDistanceReaction;
			}
			else if (LODDistance < EmitterState.MinDistance)
			{
				RequestedScalabilityState = EmitterState.MinDistanceReaction;
			}
		}

		// We need to transition the state
		if (RequestedScalabilityState != ScalabilityState)
		{
			ExecutionState = InternalExecutionState;
			ScalabilityState = RequestedScalabilityState;
			switch (RequestedScalabilityState)
			{
				case ENiagaraExecutionStateManagement::Awaken:
					if (EmitterState.bResetAgeOnAwaken)
					{
						ResetSimulation(false);
					}
					break;

				case ENiagaraExecutionStateManagement::SleepAndLetParticlesFinish:
				case ENiagaraExecutionStateManagement::KillAfterParticlesFinish:
					ExecutionState = ENiagaraExecutionState::Inactive;
					CropSpawnInfos();
					break;

				case ENiagaraExecutionStateManagement::SleepAndClearParticles:
					ExecutionState = ENiagaraExecutionState::Inactive;
					KillSpawnInfos();
					break;

				case ENiagaraExecutionStateManagement::KillImmediately:
					SetExecutionStateInternal(ENiagaraExecutionState::Complete);
					return;
			}
		}

		// Perform any per frame operations for scalability state
		switch (ScalabilityState)
		{
			case ENiagaraExecutionStateManagement::KillAfterParticlesFinish:
				if ( GetNumParticles() == 0 )
				{
					SetExecutionStateInternal(ENiagaraExecutionState::Complete);
					return;
				}
				break;
		}
	}

	// Evaluate emitter state
	if ( Age >= CurrentLoopAgeEnd )
	{
		// Do we only execute a single loop?
		if (EmitterState.LoopBehavior == ENiagaraLoopBehavior::Once)
		{
			SetExecutionStateInternal(ENiagaraExecutionState::Inactive);
		}
		// Multi-loop inject our new spawn infos
		else
		{
			// Keep looping until we find out which loop we are in as a small loop age + large DT could result in crossing multiple loops
			do
			{
				++LoopCount;
				if (EmitterState.LoopBehavior == ENiagaraLoopBehavior::Multiple && LoopCount >= EmitterState.LoopCount)
				{
					SetExecutionStateInternal(ENiagaraExecutionState::Inactive);
					break;
				}

				if (EmitterState.bRecalculateDurationEachLoop)
				{
					CurrentLoopDuration = EvaluateDistribution(EmitterState.LoopDuration, RandomStream, RendererBindings, DefaultLoopDuration);
					CurrentLoopDuration = FMath::Max(CurrentLoopDuration, DefaultLoopDuration);
				}

				if (EmitterState.bLoopDelayEnabled)
				{
					if (EmitterState.bDelayFirstLoopOnly)
					{
						CurrentLoopDelay = 0.0f;
					}
					else if (EmitterState.bRecalculateDelayEachLoop)
					{
						CurrentLoopDelay = EvaluateDistribution(EmitterState.LoopDelay, RandomStream, RendererBindings, DefaultLoopDelay);
						CurrentLoopDelay = FMath::Max(CurrentLoopDelay, 0.0f);
					}
				}

				CurrentLoopAgeStart = CurrentLoopAgeEnd;
				CurrentLoopAgeEnd = CurrentLoopAgeStart + CurrentLoopDelay + CurrentLoopDuration;

				InitSpawnInfosForLoop(CurrentLoopAgeStart);
			} while (Age >= CurrentLoopAgeEnd);
		}
	}
}

void FNiagaraStatelessEmitterInstance::CalculateBounds()
{
	CachedBounds.Init();
	FRWScopeLock ScopeLock(FixedBoundsGuard, SLT_ReadOnly);
	if (FixedBounds.IsValid)
	{
		CachedBounds = FixedBounds;
	}
	else if (CachedSystemFixedBounds.IsValid)
	{
		CachedBounds = CachedSystemFixedBounds;
	}
	else
	{
		CachedBounds = EmitterData->FixedBounds;
	}
}

void FNiagaraStatelessEmitterInstance::UpdateSimulationData(float DeltaSeconds, bool bForceParametersUpdate)
{
	using namespace NiagaraStateless;

	// Update parameter data (if needed)
	bool bNeedsShaderParametersUpdate = false;
	bool bNeedsParametersUpdate = bForceParametersUpdate;
	if (RendererBindings.GetParametersDirty())
	{
		for (const TPair<int32, FInstancedStruct>& ExpressionMapping : EmitterData->Expressions)
		{
			const FNiagaraStatelessExpression& Expression = ExpressionMapping.Value.Get<FNiagaraStatelessExpression>();
			uint8* ValueDestination = RendererBindings.GetMutableParameterData(ExpressionMapping.Key, Expression.GetOutputTypeDef());
			Expression.Evaluate(FNiagaraStatelessExpression::FEvaluateContext(RendererBindings), ValueDestination);
		}
		RendererBindings.Tick();

		if (EmitterData->bModulesHaveRendererBindings)
		{
			bNeedsShaderParametersUpdate = true;
			bNeedsParametersUpdate = true;
		}
	}

	// Update transforms (if needed)
	const FTransform3f ParentTransform = FTransform3f(ParentSystemInstance->GetWorldTransform());
	bNeedsShaderParametersUpdate |= EmitterTransforms.UpdateTransforms(ParentTransform);

	// If CPU simulation execute immediately
	if (SimTarget == ENiagaraSimTarget::CPUSim)
	{
		if (bNeedsShaderParametersUpdate)
		{
			ShaderParameters.Reset(WeakStatelessEmitter->AllocateShaderParameters(EmitterTransforms, RendererBindings));
			ShaderParameters->Common_RandomSeed = RandomSeed;
			NiagaraStatelessEmitterInstancePrivate::SetShaderParameterTransforms(EmitterTransforms, ShaderParameters.Get());
		}

		FNiagaraDataBuffer& DataBuffer = ParticleDataSet->BeginSimulate();

		FParticleSimulationContext ParticleSimulation(EmitterData.Get(), ShaderParameters.Get(), RendererBindings.GetParameterDataArray());
		ParticleSimulation.Simulate(RandomSeed, Age, DeltaSeconds, SpawnInfos, &DataBuffer);

		ParticleDataSet->EndSimulate();
	}
	// If GPU simulation send data to the RT
	else if ( FEmitterInstance_RT* RenderThreadData = RenderThreadDataPtr.Get() )
	{
		struct FDataForRenderThread
		{
			float Age = 0.0f;
			ENiagaraExecutionState ExecutionState = ENiagaraExecutionState::Disabled;

			FCommonShaderParameters* ShaderParameters = nullptr;

			bool bHasBindingBufferData = false;
			TArray<uint8> BindingBufferData;

			bool bHasSpawnInfoData = false;
			TArray<FNiagaraStatelessRuntimeSpawnInfo>	SpawnInfos;
		};

		FDataForRenderThread DataForRenderThread;
		DataForRenderThread.Age				= Age;
		DataForRenderThread.ExecutionState	= InternalExecutionState;

		if (bNeedsParametersUpdate)
		{
			DataForRenderThread.bHasBindingBufferData = true;
			DataForRenderThread.BindingBufferData = RendererBindings.GetParameterDataArray();
			check((DataForRenderThread.BindingBufferData.Num() % sizeof(uint32)) == 0);
		}

		if (bNeedsShaderParametersUpdate)
		{
			DataForRenderThread.ShaderParameters = WeakStatelessEmitter->AllocateShaderParameters(EmitterTransforms, RendererBindings);
			DataForRenderThread.ShaderParameters->Common_RandomSeed = RandomSeed;
			NiagaraStatelessEmitterInstancePrivate::SetShaderParameterTransforms(EmitterTransforms, DataForRenderThread.ShaderParameters);
		}

		if (bSpawnInfosDirty)
		{
			DataForRenderThread.bHasSpawnInfoData = true;
			DataForRenderThread.SpawnInfos = SpawnInfos;
			bSpawnInfosDirty = false;
		}

		ENQUEUE_RENDER_COMMAND(UpdateStatelessAge)(
			[RenderThreadData, EmitterData=MoveTemp(DataForRenderThread)](FRHICommandListImmediate& RHICmdList) mutable
			{
				RenderThreadData->DeltaTime			= FMath::Max(EmitterData.Age - RenderThreadData->Age, 0.0f);
				RenderThreadData->Age				= EmitterData.Age;
				RenderThreadData->ExecutionState	= EmitterData.ExecutionState;

				if (EmitterData.ShaderParameters)
				{
					RenderThreadData->ShaderParameters.Reset(EmitterData.ShaderParameters);
				}

				if (EmitterData.bHasBindingBufferData)
				{
					RenderThreadData->bBindingBufferDirty = true;
					RenderThreadData->BindingBufferData = MoveTemp(EmitterData.BindingBufferData);
				}

				if (EmitterData.bHasSpawnInfoData)
				{
					RenderThreadData->SpawnInfos = MoveTemp(EmitterData.SpawnInfos);
				}
			}
		);
	}
}

void FNiagaraStatelessEmitterInstance::InitSpawnInfos(float InitializationAge)
{
	using namespace NiagaraStatelessEmitterInstancePrivate;

	// If we are not enabled, or not awake from scalability skip adding
	if (!bEmitterEnabled_GT || (ScalabilityState != ENiagaraExecutionStateManagement::Awaken))
	{
		return;
	}

	for (const FNiagaraStatelessSpawnInfo& SpawnInfo : EmitterData->SpawnInfos)
	{
		if (SpawnInfo.Type == ENiagaraStatelessSpawnInfoType::Rate)
		{
			FActiveSpawnRate& ActiveSpawnRate	= ActiveSpawnRates.AddDefaulted_GetRef();
			ActiveSpawnRate.SpawnRate			= SpawnInfo.Rate;
			if (SpawnInfo.bSpawnProbabilityEnabled)
			{
				ActiveSpawnRate.SpawnProbability = SpawnInfo.SpawnProbability;
			}
		}
	}

	InitSpawnInfosForLoop(InitializationAge);
}

void FNiagaraStatelessEmitterInstance::InitSpawnInfosForLoop(float InitializationAge)
{
	using namespace NiagaraStatelessEmitterInstancePrivate;

	// If we are not enabled, or not awake from scalability skip adding
	if (!bEmitterEnabled_GT || (ScalabilityState != ENiagaraExecutionStateManagement::Awaken))
	{
		return;
	}

	// Add the next chunk for any active spawn rates
	for (FActiveSpawnRate& SpawnInfo : ActiveSpawnRates)
	{
		// Unlike stateful emitters we evaluate the spawn probability & rate per loop
		float SpawnProbability = 1.0f;
		if (SpawnInfo.SpawnProbability.IsSet())
		{
			SpawnProbability = EvaluateDistribution(SpawnInfo.SpawnProbability.GetValue(), RandomStream, RendererBindings, DefaultSpawnProbability);
			SpawnProbability = FMath::Clamp(SpawnProbability, 0.0f, 1.0f);
			if (FMath::IsNearlyZero(SpawnProbability))
			{
				continue;
			}
		}

		const float SpawnRate = EvaluateDistribution(SpawnInfo.SpawnRate, RandomStream, RendererBindings, DefaultSpawnRate) * EmitterData->SpawnCountScale;
		if (SpawnRate <= 0.0f)
		{
			continue;
		}

		float LifetimeMin = 0.0f;
		float LifetimeMax = 0.0f;
		if ( EvaluateLifetime(EmitterData->LifetimeRange, RandomStream, RendererBindings, LifetimeMin, LifetimeMax) == false )
		{
			continue;
		}

		const float SpawnAgeStart	= FMath::Min(InitializationAge + CurrentLoopDelay - SpawnInfo.ResidualSpawnTime, CurrentLoopAgeEnd);
		const float ActiveDuration	= CurrentLoopAgeEnd - SpawnAgeStart;
		const int32 NumSpawned		= FMath::FloorToInt(ActiveDuration * SpawnRate);
		const float SpawnAgeEnd		= SpawnAgeStart + (float(NumSpawned) / SpawnRate);

		if ( NumSpawned > 0 )
		{
			// Try and append to the last info in the list if it's a rate type
			// We do this to reduce the number of spawn infos in the common case of having a single rate info
			bool bDidAppend = false;
			if ( SpawnInfos.Num() > 0 )
			{
				FNiagaraStatelessRuntimeSpawnInfo& ExistingInfo = SpawnInfos.Last();
				if ( (ExistingInfo.Type == ENiagaraStatelessSpawnInfoType::Rate) &&
					 (ExistingInfo.Rate == SpawnRate) &&
					 (ExistingInfo.SpawnTimeEnd == SpawnAgeStart) &&
					 FMath::IsNearlyEqual(ExistingInfo.Probability, SpawnProbability) &&
					 FMath::IsNearlyEqual(ExistingInfo.LifetimeMin, LifetimeMin) &&
					 FMath::IsNearlyEqual(ExistingInfo.LifetimeMax, LifetimeMax)
				)
				{
					if (ExistingInfo.UniqueOffset + ExistingInfo.Amount == UniqueIndexOffset)
					{
						ExistingInfo.SpawnTimeEnd = SpawnAgeEnd;
						ExistingInfo.Amount += NumSpawned;
						bDidAppend = true;
					}
				}
			}

			//-TODO: We need to add a spawn info as we have something to spawn
			if (!bDidAppend)
			{
				FNiagaraStatelessRuntimeSpawnInfo& NewSpawnInfo = SpawnInfos.AddDefaulted_GetRef();
				NewSpawnInfo.Type			= ENiagaraStatelessSpawnInfoType::Rate;
				NewSpawnInfo.UniqueOffset	= UniqueIndexOffset;
				NewSpawnInfo.SpawnTimeStart	= SpawnAgeStart;
				NewSpawnInfo.SpawnTimeEnd	= SpawnAgeEnd;
				NewSpawnInfo.Rate			= SpawnRate;
				NewSpawnInfo.Amount			= NumSpawned;
				NewSpawnInfo.Probability	= SpawnProbability;	
				NewSpawnInfo.LifetimeMin	= LifetimeMin;
				NewSpawnInfo.LifetimeMax	= LifetimeMax;
			}

			UniqueIndexOffset += NumSpawned;
			bSpawnInfosDirty = true;
		}

		SpawnInfo.ResidualSpawnTime = CurrentLoopAgeEnd - SpawnAgeEnd;
	}

	// Add bursts that fit within the loop duration (due to loop random they might not)
	for (const FNiagaraStatelessSpawnInfo& SpawnInfo : EmitterData->SpawnInfos)
	{
		if (SpawnInfo.Type == ENiagaraStatelessSpawnInfoType::Rate || !SpawnInfo.IsValid(CurrentLoopDuration))
		{
			continue;
		}

		if (SpawnInfo.bLoopCountLimitEnabled)
		{
			const int32 LoopCountLimit = EvaluateDistribution(SpawnInfo.LoopCountLimit, RandomStream, RendererBindings, DefaultLoopCountLimit);
			if (LoopCount >= LoopCountLimit)
			{
				continue;
			}
		}

		if (SpawnInfo.bSpawnProbabilityEnabled)
		{
			float SpawnProbability = EvaluateDistribution(SpawnInfo.SpawnProbability, RandomStream, RendererBindings, DefaultSpawnProbability);
			SpawnProbability = FMath::Clamp(SpawnProbability, 0.0f, 1.0f);
			if (SpawnProbability < RandomStream.FRand())
			{
				continue;
			}
		}

		const int32 UnscaledSpawnAmount = EvaluateDistribution(SpawnInfo.Amount, RandomStream, RendererBindings, DefaultSpawnAmount);
		const int32 SpawnAmount = UnscaledSpawnAmount > 0 ? FMath::Max(FMath::FloorToInt(float(UnscaledSpawnAmount) * EmitterData->SpawnCountScale), 1) : 0;
		if (SpawnAmount <= 0)
		{
			continue;
		}

		const float SpawnTime = CurrentLoopAgeStart + CurrentLoopDelay + SpawnInfo.SpawnTime;
		if (SpawnTime < InitializationAge)
		{
			continue;
		}

		float LifetimeMin = 0.0f;
		float LifetimeMax = 0.0f;
		if ( EvaluateLifetime(EmitterData->LifetimeRange, RandomStream, RendererBindings, LifetimeMin, LifetimeMax) == false )
		{
			continue;
		}

		FNiagaraStatelessRuntimeSpawnInfo& NewSpawnInfo = SpawnInfos.AddDefaulted_GetRef();
		NewSpawnInfo.Type			= ENiagaraStatelessSpawnInfoType::Burst;
		NewSpawnInfo.UniqueOffset	= UniqueIndexOffset;
		NewSpawnInfo.SpawnTimeStart	= SpawnTime;
		NewSpawnInfo.SpawnTimeEnd	= SpawnTime;
		NewSpawnInfo.Amount			= SpawnAmount;
		NewSpawnInfo.LifetimeMin	= LifetimeMin;
		NewSpawnInfo.LifetimeMax	= LifetimeMax;

		UniqueIndexOffset += SpawnAmount;
		bSpawnInfosDirty = true;
	}
}

void FNiagaraStatelessEmitterInstance::TickSpawnInfos()
{
	const bool bNewEmitterEnabled = bEmitterEnabled_GT && (ScalabilityState == ENiagaraExecutionStateManagement::Awaken);

	if (bEmitterEnabled_CNC != bNewEmitterEnabled)
	{
		bEmitterEnabled_CNC = bNewEmitterEnabled;

		if (bEmitterEnabled_CNC)
		{
			RestartSpawnInfos();
		}
		else
		{
			CropSpawnInfos();
		}
	}

	SpawnInfos.RemoveAll(
		[this](const FNiagaraStatelessRuntimeSpawnInfo& SpawnInfo)
		{
			return Age >= SpawnInfo.SpawnTimeEnd + SpawnInfo.LifetimeMax;
		}
	);
}

void FNiagaraStatelessEmitterInstance::CropSpawnInfos()
{
	if (!SpawnInfos.Num() && !ActiveSpawnRates.Num())
	{
		return;
	}

	ActiveSpawnRates.Empty(SpawnInfos.Num());

	for (auto it = SpawnInfos.CreateIterator(); it; ++it)
	{
		FNiagaraStatelessRuntimeSpawnInfo& SpawnInfo = *it;
		if (SpawnInfo.Type == ENiagaraStatelessSpawnInfoType::Rate)
		{
			SpawnInfo.SpawnTimeEnd = FMath::Min(SpawnInfo.SpawnTimeEnd, Age);
			SpawnInfo.Amount = FMath::FloorToInt((SpawnInfo.SpawnTimeEnd - SpawnInfo.SpawnTimeStart) * SpawnInfo.Rate);
		}
		if (Age < SpawnInfo.SpawnTimeStart || Age >= SpawnInfo.SpawnTimeEnd + SpawnInfo.LifetimeMax)
		{
			it.RemoveCurrent();
		}
	}

	bSpawnInfosDirty = true;
}

void FNiagaraStatelessEmitterInstance::KillSpawnInfos()
{
	if (!SpawnInfos.Num() && !ActiveSpawnRates.Num())
	{
		return;
	}

	SpawnInfos.Empty();
	ActiveSpawnRates.Empty();
	bSpawnInfosDirty = true;
}

void FNiagaraStatelessEmitterInstance::RestartSpawnInfos()
{
	InitSpawnInfos(Age);
}

void FNiagaraStatelessEmitterInstance::SetExecutionStateInternal(ENiagaraExecutionState RequestedExecutionState)
{
	if (InternalExecutionState == RequestedExecutionState)
	{
		return;
	}
	
	switch (RequestedExecutionState)
	{
		case ENiagaraExecutionState::Active:
			UE_LOGF(LogNiagara, Error, "Lightweight Emitter: Was requested to go Active and we do not supoprt that.");
			break;

		case ENiagaraExecutionState::Inactive:
			if (EmitterData->EmitterState.InactiveResponse == ENiagaraEmitterInactiveResponse::Kill)
			{
				KillSpawnInfos();
				InternalExecutionState = ENiagaraExecutionState::Complete;
				ExecutionState = InternalExecutionState;
			}
			else
			{
				CropSpawnInfos();
				InternalExecutionState = SpawnInfos.Num() > 0 ? ENiagaraExecutionState::Inactive : ENiagaraExecutionState::Complete;
				ExecutionState = InternalExecutionState;
			}
			break;

		case ENiagaraExecutionState::InactiveClear:
		case ENiagaraExecutionState::Complete:
			KillSpawnInfos();
			InternalExecutionState = ENiagaraExecutionState::Complete;
			ExecutionState = InternalExecutionState;
			break;
	}
}

void FNiagaraStatelessEmitterInstance::InitEmitterData()
{
	bCanEverExecute = false;
	EmitterData = nullptr;
	WeakStatelessEmitter = nullptr;

	const FNiagaraEmitterHandle& EmitterHandle = GetEmitterHandle();
	UNiagaraStatelessEmitter* StatelessEmitter = GetEmitterHandle().GetStatelessEmitter();
	WeakStatelessEmitter = StatelessEmitter;
	if (!StatelessEmitter)
	{
		return;
	}
	EmitterData = StatelessEmitter->GetEmitterData();

	if (EmitterData != nullptr)
	{
		// We need to extract the sim target here to ensure we are in sync when iterating available renderers
		SimTarget = EmitterData->SimTarget;
		bCanEverExecute = EmitterData->bCanEverExecute && EmitterHandle.GetIsEnabled();
	}
	else
	{
		bCanEverExecute = false;
	}
}

void FNiagaraStatelessEmitterInstance::EnqueueParticleDataBufferReadback(FNiagaraDataBuffer* DataBuffer, TFunction<void()> OnReadbackComplete) const
{
	check(IsInGameThread());
	check(DataBuffer);

	// Set instances to zero to handle any early outs
	DataBuffer->SetNumInstances(0);
	if (!bCanEverExecute || IsComplete())
	{
		OnReadbackComplete();
		return;
	}

	// If we target the CPU we can copy the data
	if (SimTarget == ENiagaraSimTarget::CPUSim)
	{
		if (FNiagaraDataBuffer* CurrentDataBuffer = ParticleDataSet->GetCurrentData())
		{
			CurrentDataBuffer->CopyTo(*DataBuffer, 0, 0, INDEX_NONE);
		}
		OnReadbackComplete();
	}
	// If we target the GPU then we need to generate the data
	else
	{
		ENQUEUE_RENDER_COMMAND(CaptureStatelessForDebugging)(
			[RenderThreadData=RenderThreadDataPtr.Get(), DataBuffer, OnReadbackComplete_RT=MoveTemp(OnReadbackComplete)](FRHICommandListImmediate& RHICmdList)
			{
				if ( RenderThreadData->ComputeManager )
				{
					RenderThreadData->ComputeManager->GenerateDataBufferForDebugging(RHICmdList, DataBuffer, RenderThreadData);
				}
				OnReadbackComplete_RT();
			}
		);
	}
}

void FNiagaraStatelessEmitterInstance::CaptureForDebugging(FNiagaraDataBuffer* DataBuffer) const
{
	bool bIsComplete = false;

	EnqueueParticleDataBufferReadback(DataBuffer, [&bIsComplete]() { bIsComplete = true; } );

	if (bIsComplete == false)
	{
		FlushRenderingCommands();
		check(bIsComplete == true);
	}
}
