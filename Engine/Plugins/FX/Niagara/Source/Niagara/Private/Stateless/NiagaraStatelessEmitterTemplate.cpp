// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessEmitterTemplate.h"
#include "Stateless/NiagaraStatelessCommon.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/NiagaraStatelessEmitterShaders.h"
#include "Stateless/NiagaraStatelessShaderParametersBuilder.h"
#include "NiagaraConstants.h"
#include "NiagaraModule.h"
#include "NiagaraSystem.h"

#include "Modules/ModuleManager.h"

#include "Stateless/Modules/NiagaraStatelessModule_AddVelocity.h"
#include "Stateless/Modules/NiagaraStatelessModule_AccelerationForce.h"
#include "Stateless/Modules/NiagaraStatelessModule_ApplyOwnerScaleToAttributes.h"
#include "Stateless/Modules/NiagaraStatelessModule_CalculateAccurateVelocity.h"
#include "Stateless/Modules/NiagaraStatelessModule_CameraOffset.h"
#include "Stateless/Modules/NiagaraStatelessModule_CurlNoiseForce.h"
#include "Stateless/Modules/NiagaraStatelessModule_DecalAttributes.h"
#include "Stateless/Modules/NiagaraStatelessModule_Drag.h"
#include "Stateless/Modules/NiagaraStatelessModule_GravityForce.h"
#include "Stateless/Modules/NiagaraStatelessModule_InitializeParticle.h"
#include "Stateless/Modules/NiagaraStatelessModule_InitialMeshOrientation.h"
#include "Stateless/Modules/NiagaraStatelessModule_LightAttributes.h"
#include "Stateless/Modules/NiagaraStatelessModule_MeshIndex.h"
#include "Stateless/Modules/NiagaraStatelessModule_MeshRotationRate.h"
#include "Stateless/Modules/NiagaraStatelessModule_RotateAroundPoint.h"
#include "Stateless/Modules/NiagaraStatelessModule_ScaleColor.h"
#include "Stateless/Modules/NiagaraStatelessModule_ScaleMeshSize.h"
#include "Stateless/Modules/NiagaraStatelessModule_ScaleMeshSizeBySpeed.h"
#include "Stateless/Modules/NiagaraStatelessModule_ScaleRibbonWidth.h"
#include "Stateless/Modules/NiagaraStatelessModule_ScaleSpriteSize.h"
#include "Stateless/Modules/NiagaraStatelessModule_ScaleSpriteSizeBySpeed.h"
#include "Stateless/Modules/NiagaraStatelessModule_ShapeLocation.h"
#include "Stateless/Modules/NiagaraStatelessModule_SolveVelocitiesAndForces.h"
#include "Stateless/Modules/NiagaraStatelessModule_SpriteFacingAndAlignment.h"
#include "Stateless/Modules/NiagaraStatelessModule_SpriteRotationRate.h"
#include "Stateless/Modules/NiagaraStatelessModule_SubUVAnimation.h"
#include "Stateless/Modules/NiagaraStatelessModule_DynamicMaterialParameters.h"

#include "UObject/AssetRegistryTagsContext.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetData.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Engine.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessEmitterTemplate)

namespace NiagaraStatelessEmitterTemplatePrivate
{
	static const FName NAME_ExposedToLibrary("bExposedToLibrary");

	TArray<TWeakObjectPtr<UNiagaraStatelessEmitterTemplate>> ObjectsToDeferredInit;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UNiagaraStatelessEmitterTemplate::PostInitProperties()
{
	Super::PostInitProperties();

	// We can end up hitting PostInitProperties before the Niagara Module has initialized bindings this needs, mark this object for deferred init and early out.
	if (GetClass() != UNiagaraStatelessEmitterTemplate::StaticClass())
	{
		if (FModuleManager::Get().IsModuleLoaded("Niagara") == false)
		{
			NiagaraStatelessEmitterTemplatePrivate::ObjectsToDeferredInit.Add(this);
		}
		else
		{
			InitModulesAndAttributes();
			checkf(Modules.Num() != 0, TEXT("StatelessTemplate(%s) has no modules, this is likely a code error"), *GetNameSafe(this));
		}
	}
}

void UNiagaraStatelessEmitterTemplate::PostLoad()
{
	Super::PostLoad();
	InitModulesAndAttributes();
}

void UNiagaraStatelessEmitterTemplate::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	using namespace NiagaraStatelessEmitterTemplatePrivate;

	Super::GetAssetRegistryTags(Context);
#if WITH_EDITORONLY_DATA
	Context.AddTag(FAssetRegistryTag(NAME_ExposedToLibrary, LexToString(bExposedToLibrary), FAssetRegistryTag::TT_Hidden));
#endif
}

void UNiagaraStatelessEmitterTemplate::InitCDOPropertiesAfterModuleStartup()
{
	for (TWeakObjectPtr<UNiagaraStatelessEmitterTemplate>& WeakObject : NiagaraStatelessEmitterTemplatePrivate::ObjectsToDeferredInit)
	{
		if (UNiagaraStatelessEmitterTemplate* EmitterObject = WeakObject.Get())
		{
			EmitterObject->InitModulesAndAttributes();
			checkf(EmitterObject->Modules.Num() != 0, TEXT("StatelessTemplate(%s) has no modules, this is likely a code error"), *GetNameSafe(EmitterObject));
		}
	}
	NiagaraStatelessEmitterTemplatePrivate::ObjectsToDeferredInit.Empty();
}

void UNiagaraStatelessEmitterTemplate::InitModulesAndAttributes()
{
	// Remove any invalid modules
	Modules.RemoveAll([](UClass* ModuleClass) { return ModuleClass == nullptr || !ModuleClass->IsChildOf(UNiagaraStatelessModule::StaticClass()); });

	BuildVariables();
	BuildShaderParameters();
}

#if WITH_EDITOR
void UNiagaraStatelessEmitterTemplate::ModifyTemplate(UNiagaraStatelessEmitterTemplate* FromTemplate)
{
	// Modify for capturing in transaction
	Modify();

	// Gather all the systems to update
	FNiagaraSystemUpdateContext UpdateContext;

	TArray<UNiagaraStatelessEmitter*> EmittersToUpdate;
	for (TObjectIterator<UNiagaraStatelessEmitter> It; It; ++It)
	{
		UNiagaraStatelessEmitter* Emitter = *It;
		if (Emitter && Emitter->GetEmitterTemplate() == this)
		{
			EmittersToUpdate.Add(Emitter);

			if (UNiagaraSystem* System = Emitter->GetTypedOuter<UNiagaraSystem>())
			{
				UpdateContext.Add(System, true);
			}
		}
	}

	// Copy template over
	UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyParams;
	CopyParams.bDoDelta = false;
	UEngine::CopyPropertiesForUnrelatedObjects(FromTemplate, this, CopyParams);

	// Initialize template
	InitModulesAndAttributes();

	for (UNiagaraStatelessEmitter* Emitter : EmittersToUpdate)
	{
		Emitter->PostTemplateChanged();
		Emitter->CacheFromCompiledData();
	}
}

bool UNiagaraStatelessEmitterTemplate::IsExposedToLibrary(const FAssetData& AssetData)
{
	using namespace NiagaraStatelessEmitterTemplatePrivate;

	bool bExposedToLibrary = false;
	AssetData.GetTagValue(NAME_ExposedToLibrary, bExposedToLibrary);
	return bExposedToLibrary;
}
#endif

void UNiagaraStatelessEmitterTemplate::SetShaderParameters(uint8* ShaderParametersBase, TConstArrayView<int32> ShaderOutputVariableOffsets) const
{
	//-TODO: We don't store component offsets right now
}

void UNiagaraStatelessEmitterTemplate::BuildVariables()
{
#if WITH_EDITORONLY_DATA
	// Clear Variable List
	ImplicitVariables.Empty();
	ModuleVariables.Empty();
	ShaderOutputVariables.Empty();

	// Implicit Variables
	const FNiagaraStatelessGlobals& StatelessGlobals = FNiagaraStatelessGlobals::Get();
	ImplicitVariables.Add(StatelessGlobals.UniqueIDVariable);
	ImplicitVariables.Add(StatelessGlobals.MaterialRandomVariable);

	ShaderOutputVariables = ImplicitVariables;

	// Extract variables from modules
	{
		TArray<FNiagaraVariableBase> TempVariables;
		for (UClass* ModuleClass : Modules)
		{
			UNiagaraStatelessModule* Module = CastChecked<UNiagaraStatelessModule>(ModuleClass->GetDefaultObject());
			Module->GetOutputVariables(TempVariables, UNiagaraStatelessModule::EVariableFilter::None);

			const bool bShaderOutput = EnumHasAnyFlags(Module->GetFeatureMask(), ENiagaraStatelessFeatureMask::ExecuteGPU);
			for (const FNiagaraVariableBase& Variable : TempVariables)
			{
				ModuleVariables.AddUnique(Variable);
				if (bShaderOutput)
				{
					ShaderOutputVariables.AddUnique(Variable);
				}
			}
			TempVariables.Reset();
		}
	}
#endif
}

void UNiagaraStatelessEmitterTemplate::BuildShaderParameters()
{
	FShaderParametersMetadataBuilder ShaderMetadataBuilder;
	FNiagaraStatelessShaderParametersBuilder ShaderParametersBuilder(&ShaderMetadataBuilder);

	ShaderParametersBuilder.AddParameterNestedStruct<NiagaraStateless::FCommonShaderParameters>();
	for (UClass* ModuleClass : Modules)
	{
		UNiagaraStatelessModule* Module = CastChecked<UNiagaraStatelessModule>(ModuleClass->GetDefaultObject());
		Module->BuildShaderParameters(ShaderParametersBuilder);
	}
	//-TODO: Reserve space for component outputs when compiling GPU ShaderParametersBuilder.AddParameterNestedStruct<NiagaraStateless::FCommonShaderParameters>();

	ShaderParametersMetadata = MakeShareable<FShaderParametersMetadata>(ShaderMetadataBuilder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("NiagaraStatelessEmitterTemplate")));
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UNiagaraStatelessEmitterDefault::InitModulesAndAttributes()
{
	Modules =
	{
		// Initializer Modules
		UNiagaraStatelessModule_InitializeParticle::StaticClass(),
		UNiagaraStatelessModule_InitialMeshOrientation::StaticClass(),
		UNiagaraStatelessModule_ShapeLocation::StaticClass(),
		// Before Solve
		UNiagaraStatelessModule_AddVelocity::StaticClass(),
		UNiagaraStatelessModule_AccelerationForce::StaticClass(),
		UNiagaraStatelessModule_CurlNoiseForce::StaticClass(),
		UNiagaraStatelessModule_Drag::StaticClass(),
		UNiagaraStatelessModule_GravityForce::StaticClass(),
		UNiagaraStatelessModule_ApplyOwnerScaleToAttributes::StaticClass(),
		UNiagaraStatelessModule_SolveVelocitiesAndForces::StaticClass(),
		// Post Solve
		UNiagaraStatelessModule_CameraOffset::StaticClass(),
		UNiagaraStatelessModule_DecalAttributes::StaticClass(),
		UNiagaraStatelessModule_DynamicMaterialParameters::StaticClass(),
		UNiagaraStatelessModule_LightAttributes::StaticClass(),
		UNiagaraStatelessModule_MeshIndex::StaticClass(),
		UNiagaraStatelessModule_MeshRotationRate::StaticClass(),
		UNiagaraStatelessModule_ScaleColor::StaticClass(),
		UNiagaraStatelessModule_ScaleRibbonWidth::StaticClass(),
		UNiagaraStatelessModule_ScaleSpriteSize::StaticClass(),
		UNiagaraStatelessModule_ScaleSpriteSizeBySpeed::StaticClass(),
		UNiagaraStatelessModule_ScaleMeshSize::StaticClass(),
		UNiagaraStatelessModule_ScaleMeshSizeBySpeed::StaticClass(),
		UNiagaraStatelessModule_SpriteFacingAndAlignment::StaticClass(),
		UNiagaraStatelessModule_SpriteRotationRate::StaticClass(),
		UNiagaraStatelessModule_SubUVAnimation::StaticClass(),
	};

	BuildVariables();
}

const FShaderParametersMetadata* UNiagaraStatelessEmitterDefault::GetShaderParametersMetadata() const
{
	using namespace NiagaraStateless;
	return FSimulationShaderDefaultCS::FParameters::FTypeInfo::GetStructMetadata();
}

TShaderRef<NiagaraStateless::FSimulationShader> UNiagaraStatelessEmitterDefault::GetSimulationShader() const
{
	using namespace NiagaraStateless;
	return TShaderMapRef<FSimulationShaderDefaultCS>(GetGlobalShaderMap(GMaxRHIFeatureLevel));
}

//-TODO: Add a way to set them directly, we should know that the final struct is a series of ints in the order of the provided variables
void UNiagaraStatelessEmitterDefault::SetShaderParameters(uint8* ShaderParametersBase, TConstArrayView<int32> ShaderOutputVariableOffsets) const
{
	using namespace NiagaraStateless;
	FSimulationShaderDefaultCS::FParameters* ShaderParameters	= reinterpret_cast<FSimulationShaderDefaultCS::FParameters*>(ShaderParametersBase);
	int iComponent = 0;
	ShaderParameters->Permutation_UniqueIDComponent					= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_MaterialRandomComponent			= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_PositionComponent					= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_ColorComponent					= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_SpriteSizeComponent				= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_SpriteRotationComponent			= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_ScaleComponent					= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_PreviousPositionComponent			= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_PreviousSpriteSizeComponent		= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_PreviousSpriteRotationComponent	= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_PreviousScaleComponent			= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_RibbonWidthComponent				= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_PreviousRibbonWidthComponent		= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_MeshOrientationComponent			= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_PreviousMeshOrientationComponent	= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_VelocityComponent					= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_PreviousVelocityComponent			= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_CameraOffsetComponent				= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_PreviousCameraOffsetComponent		= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_DynamicMaterialParameterComponent	= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_DynamicMaterialParameter1Component= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_DynamicMaterialParameter2Component= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_DynamicMaterialParameter3Component= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_MeshIndexComponent				= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_SpriteFacingComponent				= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_PreviousSpriteFacingComponent		= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_SpriteAlignmentComponent			= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_PreviousSpriteAlignmentComponent	= ShaderOutputVariableOffsets[iComponent++];
	ShaderParameters->Permutation_SubImageIndexComponent			= ShaderOutputVariableOffsets[iComponent++];
}
//-TODO: Add a way to set them directly, we should know that the final struct is a series of ints in the order of the provided variables

