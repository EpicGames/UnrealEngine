// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessExpression.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessExpression)

#if WITH_EDITORONLY_DATA
void FNiagaraStatelessExpression::ForEachBinding(const FInstancedStruct& ExpressionStruct, const TFunction<void(const FNiagaraVariableBase&)>& Delegate)
{
	const FNiagaraStatelessExpression* Expression = ExpressionStruct.GetPtr<const FNiagaraStatelessExpression>();
	if ( Expression == nullptr)
	{
		return;
	}

	// This isn't ideal, needs more thought
	static const FName NAME_BindingExpression("BindingExpression");
	const UScriptStruct* ExpressionScriptStruct = ExpressionStruct.GetScriptStruct();
	if (ExpressionScriptStruct->HasMetaData(NAME_BindingExpression))
	{
		FNameProperty* NameProperty = CastField<FNameProperty>(ExpressionScriptStruct->ChildProperties);
		if (ensure(NameProperty && NameProperty->Next == nullptr))
		{
			const FName BindingName = NameProperty->GetPropertyValue_InContainer(Expression);
			Delegate(FNiagaraVariableBase(Expression->GetOutputTypeDef(), BindingName));
		}
	}

	for (TFieldIterator<FProperty> PropertyIt(ExpressionStruct.GetScriptStruct()); PropertyIt; ++PropertyIt)
	{
		const FStructProperty* StructProperty = CastField<const FStructProperty>(*PropertyIt);
		if (StructProperty && StructProperty->Struct == FInstancedStruct::StaticStruct())
		{
			const FInstancedStruct* InnerExpressionStruct = StructProperty->ContainerPtrToValuePtr<const FInstancedStruct>(ExpressionStruct.GetMemory());
			ForEachBinding(*InnerExpressionStruct, Delegate);
		}
	}
}
#endif
