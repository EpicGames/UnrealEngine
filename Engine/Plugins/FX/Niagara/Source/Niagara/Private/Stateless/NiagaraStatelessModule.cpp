// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessModule)

#if WITH_EDITORONLY_DATA
const FName UNiagaraStatelessModule::PrivateMemberNames::bModuleEnabled = GET_MEMBER_NAME_CHECKED(UNiagaraStatelessModule, bModuleEnabled);
const FName UNiagaraStatelessModule::PrivateMemberNames::bDebugDrawEnabled = GET_MEMBER_NAME_CHECKED(UNiagaraStatelessModule, bDebugDrawEnabled);
#endif

bool UNiagaraStatelessModule::IsModuleEnabled() const
{
	return bModuleEnabled && !HasAnyFlags(RF_ClassDefaultObject);
}

#if WITH_EDITOR
bool UNiagaraStatelessModule::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty != nullptr)
	{
		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraStatelessModule, bModuleEnabled))
		{
			return CanDisableModule();
		}
		else if (CanDisableModule() && !IsModuleEnabled())
		{
			return false;
		}

		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraStatelessModule, bDebugDrawEnabled))
		{
			return CanDebugDraw();
		}
	}

	return true;
}

void UNiagaraStatelessModule::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	FNiagaraDistributionBase::PostEditChangeProperty(this, PropertyChangedEvent);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif //WITH_EDITOR
