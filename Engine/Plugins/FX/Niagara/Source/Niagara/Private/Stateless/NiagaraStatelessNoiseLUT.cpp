// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessNoiseLUT.h"
#include "NiagaraSimplexNoise.h"

namespace NiagaraStateless
{
	static TGlobalResource<NiagaraStateless::FNoiseLUT> GGlobalLUT(32, 64, 0.1f);

	FNoiseLUT::FNoiseLUT(uint32 Rows, uint32 RowWidth, float InFieldTravelSpeed)
		: LUTRows(Rows)
		, LUTRowWidth(RowWidth)
		, FieldTravelSpeed(InFieldTravelSpeed)
	{
		CpuData.AddUninitialized(LUTRows * LUTRowWidth);
		for (uint32 Row = 0; Row < LUTRows; ++Row)
		{
			const float fRow = float(Row) - (float(LUTRows) * 0.5f);
			FVector3f SamplePosition = FVector3f(fRow, fRow * 3.0f, fRow * 9.0f);
			FVector3f PositionOffset = FVector3f::ZeroVector;

			uint32 RowOutputOffset = Row * LUTRowWidth;
			CpuData[RowOutputOffset++] = PositionOffset;
			for (uint32 Column = 1; Column < LUTRowWidth; ++Column)
			{
				const FNiagaraMatrix3x4 J = JacobianSimplex_ALU(SamplePosition);
				const FVector3f Dir = FVector3f(J[1][2] - J[2][1], J[2][0] - J[0][2], J[0][1] - J[1][0]).GetSafeNormal();
				SamplePosition += Dir * FieldTravelSpeed;
				PositionOffset += Dir;
				CpuData[RowOutputOffset++] = PositionOffset;
			}
		}
	}

	void FNoiseLUT::InitRHI(FRHICommandListBase& RHICmdList)
	{
		const uint32 BufferByteSize = CpuData.Num() * CpuData.GetTypeSize();
		GpuData.Initialize(RHICmdList, TEXT("NiagaraStateless::FNoiseLUT"), BufferByteSize, EBufferUsageFlags::Static);

		void* UploadMemory = RHICmdList.LockBuffer(GpuData.Buffer, 0, BufferByteSize, RLM_WriteOnly);
		FMemory::Memcpy(UploadMemory, CpuData.GetData(), BufferByteSize);
		RHICmdList.UnlockBuffer(GpuData.Buffer);
	}

	void FNoiseLUT::ReleaseRHI()
	{
		GpuData.Release();
	}

	const FNoiseLUT& FNoiseLUT::GetGlobalLUT()
	{
		return GGlobalLUT;
	}
}
