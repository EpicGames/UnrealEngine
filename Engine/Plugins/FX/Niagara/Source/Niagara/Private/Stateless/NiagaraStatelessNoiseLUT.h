// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessCommon.h"
#include "RenderResource.h"
#include "RHIUtilities.h"

namespace NiagaraStateless
{
	class FNoiseLUT : public FRenderResource
	{
	public:
		explicit FNoiseLUT(uint32 Rows, uint32 RowWidth, float FieldTravelSpeed);

		virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
		virtual void ReleaseRHI() override;

		uint32 GetNumRows() const { return LUTRows; }
		uint32 GetRowWidth() const { return LUTRowWidth; }
		float GetFieldTravelSpeed() const { return FieldTravelSpeed; }
		TConstArrayView<FVector3f> GetCpuData() const { return CpuData; }
		FRHIShaderResourceView* GetGpuData() const { return GpuData.SRV; }

		static const FNoiseLUT& GetGlobalLUT();

	private:
		const uint32		LUTRows = 0;
		const uint32		LUTRowWidth = 0;
		const float			FieldTravelSpeed = 0.1f;

		TArray<FVector3f>	CpuData;
		FByteAddressBuffer	GpuData;
	};
}
