// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include "Stateless/NiagaraStatelessEmitterData.h"
#include "Stateless/NiagaraStatelessEmitterInstance.h"
#include "Stateless/NiagaraStatelessParticleSimExecData.h"

namespace NiagaraStateless
{
	FUintVector4 Rand4DPCG32(FUintVector4 v)
	{
		// Linear congruential step.
		v.X = v.X * 1664525u + 1013904223u;
		v.Y = v.Y * 1664525u + 1013904223u;
		v.Z = v.Z * 1664525u + 1013904223u;
		v.W = v.W * 1664525u + 1013904223u;

		// shuffle
		v.X += v.Y * v.W;
		v.Y += v.Z * v.X;
		v.Z += v.X * v.Y;
		v.W += v.Y * v.Z;

		// xoring high bits into low makes all 32 bits pretty good
		v.X ^= (v.X >> 16u);
		v.Y ^= (v.Y >> 16u);
		v.Z ^= (v.Z >> 16u);
		v.W ^= (v.W >> 16u);

		// final shuffle
		v.X += v.Y * v.W;
		v.Y += v.Z * v.X;
		v.Z += v.X * v.Y;
		v.W += v.Y * v.Z;

		return v;
	}

	FParticleSimulationContext::FParticleSimulationContext(const FNiagaraStatelessEmitterData* InEmitterData, const void* InShaderParametersData, TConstArrayView<uint8> InDynamicBufferData)
		: EmitterData(InEmitterData)
		, BuiltData(EmitterData->BuiltData)
		, SharedBuiltData(EmitterData->SharedBuiltData)
		, SharedBuiltDataOffsets(EmitterData->SharedBuiltDataOffsets)
		, ShaderParametersData(static_cast<const uint8*>(InShaderParametersData))
		, StaticFloatData(MakeConstArrayView(reinterpret_cast<const float*>(EmitterData->StaticDataBufferCpu.GetData()), EmitterData->StaticDataBufferCpu.Num() / sizeof(float)))
		, DynamicBufferData(InDynamicBufferData)
	{
		check(EmitterData->ParticleSimExecData);
	}

	TConstArrayView<FNiagaraVariableBase> FParticleSimulationContext::GetRequiredComponents()
	{
		static const FNiagaraVariableBase RequireComponents[int(EParticleComponent::Num)] =
		{
			{},														// uint32		- Alive
			{},														// float		- Lifetime
			{},														// float		- Age
			{},														// float		- NormalizedAge
			{},														// float		- PreviousAge
			{},														// float		- PreviousNormalizedAge
			FNiagaraStatelessGlobals::Get().UniqueIDVariable,		// int32		- UniqueID
			FNiagaraStatelessGlobals::Get().MaterialRandomVariable,	// float		- MaterialRandom
		};
		return MakeArrayView(RequireComponents);
	}

	void FParticleSimulationContext::Simulate(int32 InEmitterRandomSeed, float EmitterAge, float InDeltaTime, TConstArrayView<FNiagaraStatelessRuntimeSpawnInfo> SpawnInfos, FNiagaraDataBuffer* DestinationData)
	{
		NumInstances = 0;

		FSpawnInfoShaderParameters SpawnParameters;
		const uint32 ActiveParticles = EmitterData->CalculateActiveParticles(InEmitterRandomSeed, SpawnInfos, EmitterAge, &SpawnParameters);
		if (ActiveParticles > 0)
		{
			// Setup data buffer pointers
			DestinationData->Allocate(ActiveParticles);
			BufferStride	= DestinationData->GetFloatStride();
			BufferFloatData	= DestinationData->GetComponentPtrFloat(0);
			BufferInt32Data	= DestinationData->GetComponentPtrInt32(0);

			// Run Simulation
			SimulateInternal(InEmitterRandomSeed, EmitterAge, InDeltaTime, SpawnParameters, ActiveParticles);
		}

		// Set instance count
		DestinationData->SetNumInstances(NumInstances);
	}

	void FParticleSimulationContext::SimulateGPU(FRHICommandListBase& RHICmdList, int32 InEmitterRandomSeed, float EmitterAge, float InDeltaTime, TConstArrayView<FNiagaraStatelessRuntimeSpawnInfo> SpawnInfos, FNiagaraDataBuffer* DestinationData)
	{
		NumInstances = 0;

		FSpawnInfoShaderParameters SpawnParameters;
		const uint32 ActiveParticles = EmitterData->CalculateActiveParticles(InEmitterRandomSeed, SpawnInfos, EmitterAge, &SpawnParameters);
		if (ActiveParticles > 0)
		{
			check(DestinationData->GetNumInstancesAllocated() <= ActiveParticles);

			FRWBuffer& FloatBuffer = DestinationData->GetGPUBufferFloat();
			FRWBuffer& Int32Buffer = DestinationData->GetGPUBufferInt();

			// Setup data buffer pointers
			BufferStride	= DestinationData->GetFloatStride();
			BufferFloatData	= FloatBuffer.NumBytes > 0 ? reinterpret_cast<uint8*>(RHICmdList.LockBuffer(FloatBuffer.Buffer, 0, FloatBuffer.NumBytes, RLM_WriteOnly)) : nullptr;
			BufferInt32Data	= Int32Buffer.NumBytes > 0 ? reinterpret_cast<uint8*>(RHICmdList.LockBuffer(Int32Buffer.Buffer, 0, Int32Buffer.NumBytes, RLM_WriteOnly)) : nullptr;

			// Run Simulation
			SimulateInternal(InEmitterRandomSeed, EmitterAge, InDeltaTime, SpawnParameters, ActiveParticles);

			// Unlock buffers
			if (BufferFloatData)
			{
				RHICmdList.UnlockBuffer(FloatBuffer.Buffer);
			}
			if (BufferInt32Data)
			{
				RHICmdList.UnlockBuffer(Int32Buffer.Buffer);
			}
		}

		// Set instance count
		DestinationData->SetNumInstances(NumInstances);
	}

	void FParticleSimulationContext::SimulateInternal(int32 InEmitterRandomSeed, float EmitterAge, float InDeltaTime, FSpawnInfoShaderParameters& SpawnParameters, uint32 ActiveParticles)
	{
		// Setup simulation	
		NumInstances = 0;
		DeltaTime = InDeltaTime;
		InvDeltaTime = DeltaTime > 0.0f ? 1.0f / DeltaTime : 0.0f;
		EmitterRandomSeed = InEmitterRandomSeed;
		ModuleRandomSeed = 0;

		// Setup optional components
		const FParticleSimulationExecData* ExecData = EmitterData->ParticleSimExecData;
		VariableComponents.AddDefaulted(ExecData->VariableComponentOffsets.Num());
		for (int32 i = 0; i < ExecData->VariableComponentOffsets.Num(); ++i)
		{
			const uint32 VariableOffset = ExecData->VariableComponentOffsets[i].GetOffset();
			if (ExecData->VariableComponentOffsets[i].IsFloat())
			{
				VariableComponents[i] = BufferFloatData + (VariableOffset * BufferStride);
			}
			else //if (ExecData->VariableComponentOffsets.IsInt32())
			{
				VariableComponents[i] = BufferInt32Data + (VariableOffset * BufferStride);
			}
		}

		// Setup required components and temporary memory
		const uint32 TransientStorageSize = ExecData->RequiredComponentByteSize * BufferStride;
		uint8* TransientStorage = static_cast<uint8*>(FMemory::Malloc(TransientStorageSize));
		ON_SCOPE_EXIT { FMemory::Free(TransientStorage); };

		for (int32 i = 0; i < int32(EParticleComponent::Num); ++i)
		{
			if (ExecData->RequiredComponentOffsets[i].IsTransient())
			{
				RequiredComponents[i] = TransientStorage + (ExecData->RequiredComponentOffsets[i].GetOffset() * BufferStride);
			}
			else
			{					
				RequiredComponents[i] = VariableComponents[ExecData->RequiredComponentOffsets[i].GetOffset()];
			}
		}

		// Setup particles
		for (uint32 iParticle = 0; iParticle < ActiveParticles; ++iParticle)
		{
			uint32 Particle_UniqueID = 0;
			float Particle_Age = -1.0f;
			float Particle_Lifetime = 0.0f;
			{
				uint32 SpawnInfoIndex = iParticle;
				for (int32 GpuSpawnIndex = 0; GpuSpawnIndex < NiagaraStateless::MaxGpuSpawnInfos; ++GpuSpawnIndex)
				{
					const uint32 SpawnInfoNumActive			= GET_SCALAR_ARRAY_ELEMENT(SpawnParameters.SpawnInfo_NumActive, GpuSpawnIndex);
					const uint32 SpawnInfoParticleOffset	= GET_SCALAR_ARRAY_ELEMENT(SpawnParameters.SpawnInfo_ParticleOffset, GpuSpawnIndex);
					const uint32 SpawnInfoUniqueOffset		= GET_SCALAR_ARRAY_ELEMENT(SpawnParameters.SpawnInfo_UniqueOffset, GpuSpawnIndex);
					const float  SpawnInfoTime				= GET_SCALAR_ARRAY_ELEMENT(SpawnParameters.SpawnInfo_Time, GpuSpawnIndex);
					const float  SpawnInfoRate				= GET_SCALAR_ARRAY_ELEMENT(SpawnParameters.SpawnInfo_Rate, GpuSpawnIndex);
					const float  SpawnInfoProbability		= GET_SCALAR_ARRAY_ELEMENT(SpawnParameters.SpawnInfo_Probability, GpuSpawnIndex);
					const float  SpawnInfoLifetimeScale		= GET_SCALAR_ARRAY_ELEMENT(SpawnParameters.SpawnInfo_LifetimeScale, GpuSpawnIndex);
					const float  SpawnInfoLifetimeBias		= GET_SCALAR_ARRAY_ELEMENT(SpawnParameters.SpawnInfo_LifetimeBias, GpuSpawnIndex);
					if (SpawnInfoIndex < SpawnInfoNumActive)
					{
						const uint32 SpawnParticleIndex = SpawnInfoIndex + SpawnInfoParticleOffset;
						Particle_UniqueID	= SpawnInfoIndex + SpawnInfoUniqueOffset + SpawnInfoParticleOffset;
						Particle_Age		= EmitterAge - (SpawnInfoTime + float(SpawnParticleIndex) * SpawnInfoRate);

						// Write unique index as it's needed for the random operation
						GetParticleUniqueID()[NumInstances] = Particle_UniqueID;

						Particle_Lifetime = RandomScaleBiasFloat(NumInstances, 0, SpawnInfoLifetimeScale, SpawnInfoLifetimeBias);
						Particle_Lifetime = RandomFloat(NumInstances, 2) <= SpawnInfoProbability ? Particle_Lifetime : -1.0f;
						break;
					}

					SpawnInfoIndex -= SpawnInfoNumActive;
				}
				if (Particle_Age < 0.0f)
				{
					continue;
				}
			}

			if (Particle_Lifetime <= 0.0f || Particle_Age >= Particle_Lifetime)
			{
				continue;
			}

			const float PreviousAge = FMath::Max(Particle_Age - DeltaTime, 0.0f);

			// Initialize particle information
			GetParticleAlive()[NumInstances] = 1;
			GetParticleLifetime()[NumInstances] = Particle_Lifetime;
			GetParticleAge()[NumInstances] = Particle_Age;
			GetParticleNormalizedAge()[NumInstances] = Particle_Age / Particle_Lifetime;
			GetParticlePreviousAge()[NumInstances] = PreviousAge;
			GetParticlePreviousNormalizedAge()[NumInstances] = PreviousAge / Particle_Lifetime;
			GetParticleMaterialRandom()[NumInstances] = RandomFloat(NumInstances, 1);

			++NumInstances;
		}

		if (NumInstances == 0)
		{
			return;
		}

		// Execute the simulation
		for (const auto& Callback : ExecData->SimulateFunctions)
		{
			BuiltDataOffset = Callback.BuiltDataOffset;
			ShaderParameterOffset = Callback.ShaderParameterOffset;
			ModuleRandomSeed = Callback.RandomSeedOffset;
			Callback.Function(*this);
		}

		// Perform compaction is needed
		if (bNeedsParticleKillCompaction)
		{
			CompactDeadParticles();
		}
	}

	void FParticleSimulationContext::CompactDeadParticles()
	{
		const FParticleSimulationExecData* ExecData = EmitterData->ParticleSimExecData;

		int32* ParticleAlive = GetParticleAlive();
		for (int32 iParticle=int32(NumInstances-1); iParticle >= 0; --iParticle)
		{
			if (ParticleAlive[iParticle] != 0)
			{
				continue;
			}

			--NumInstances;
			if (iParticle >= int32(NumInstances))
			{
				continue;
			}

			for ( int32 iComponent=0; iComponent < int32(EParticleComponent::Num); ++iComponent )
			{
				uint32* ComponentData = static_cast<uint32*>(RequiredComponents[iComponent]);
				ComponentData[iParticle] = ComponentData[NumInstances];
			}
			for (int32 iVariable=0; iVariable < VariableComponents.Num(); ++iVariable)
			{
				uint8* VariableOutput = VariableComponents[iVariable];

				const uint32 NumComponents = ExecData->VariableComponentOffsets[iVariable].GetNum();
				for (uint32 iComponent=0; iComponent < NumComponents; ++iComponent)
				{
					uint32* ComponentData = reinterpret_cast<uint32*>(VariableOutput);
					ComponentData[iParticle] = ComponentData[NumInstances];
					VariableOutput += BufferStride;
				}
			}
		}
	}

	const FVector3f FParticleSimulationContext::GetLocalToWorldScale() const
	{
		using namespace NiagaraStateless;
		const FCommonShaderParameters* CommonShaderParameters = reinterpret_cast<const FCommonShaderParameters*>(ShaderParametersData);
		return CommonShaderParameters->Common_LocalToWorldScale;
	}

	const FQuat4f& FParticleSimulationContext::GetToSimulationRotation(ENiagaraCoordinateSpace SourceSpace) const
	{
		using namespace NiagaraStateless;
		const FCommonShaderParameters* CommonShaderParameters = reinterpret_cast<const FCommonShaderParameters*>(ShaderParametersData);
		return CommonShaderParameters->Common_ToSimulationRotations[int(SourceSpace)];
	}

	uint32 FParticleSimulationContext::RandomUInt(uint32 iInstance, uint32 RandomSeedOffset) const
	{
		const uint32 UniqueID = uint32(GetParticleUniqueID()[iInstance]);
		const FUintVector4 RandomSeed = FNiagaraStatelessDefinitions::MakeRandomSeed(EmitterRandomSeed, UniqueID, ModuleRandomSeed, RandomSeedOffset);
		const FUintVector4 RandomValue = Rand4DPCG32(RandomSeed);
		return RandomValue.X;
	}

	FUintVector2 FParticleSimulationContext::RandomUInt2(uint32 iInstance, uint32 RandomSeedOffset) const
	{
		const uint32 UniqueID = uint32(GetParticleUniqueID()[iInstance]);
		const FUintVector4 RandomSeed = FNiagaraStatelessDefinitions::MakeRandomSeed(EmitterRandomSeed, UniqueID, ModuleRandomSeed, RandomSeedOffset);
		const FUintVector4 RandomValue = Rand4DPCG32(RandomSeed);
		return FUintVector2(RandomValue.X, RandomValue.Y);
	}

	FUintVector3 FParticleSimulationContext::RandomUInt3(uint32 iInstance, uint32 RandomSeedOffset) const
	{
		const uint32 UniqueID = uint32(GetParticleUniqueID()[iInstance]);
		const FUintVector4 RandomSeed = FNiagaraStatelessDefinitions::MakeRandomSeed(EmitterRandomSeed, UniqueID, ModuleRandomSeed, RandomSeedOffset);
		const FUintVector4 RandomValue = Rand4DPCG32(RandomSeed);
		return FUintVector3(RandomValue.X, RandomValue.Y, RandomValue.Z);
	}

	FUintVector4 FParticleSimulationContext::RandomUInt4(uint32 iInstance, uint32 RandomSeedOffset) const
	{
		const uint32 UniqueID = uint32(GetParticleUniqueID()[iInstance]);
		const FUintVector4 RandomSeed = FNiagaraStatelessDefinitions::MakeRandomSeed(EmitterRandomSeed, UniqueID, ModuleRandomSeed, RandomSeedOffset);
		const FUintVector4 RandomValue = Rand4DPCG32(RandomSeed);
		return RandomValue;
	}

	float FParticleSimulationContext::RandomFloat(uint32 iInstance, uint32 RandomSeedOffset) const
	{
		const uint32 V = RandomUInt(iInstance, RandomSeedOffset);
		return float((V >> 8) & 0x00ffffff) / 16777216.0f;
	}

	FVector2f FParticleSimulationContext::RandomFloat2(uint32 iInstance, uint32 RandomSeedOffset) const
	{
		const FUintVector2 V = RandomUInt2(iInstance, RandomSeedOffset);
		return FVector2f(
			float((V.X >> 8) & 0x00ffffff) / 16777216.0f,
			float((V.Y >> 8) & 0x00ffffff) / 16777216.0f
		);
	}

	FVector3f FParticleSimulationContext::RandomFloat3(uint32 iInstance, uint32 RandomSeedOffset) const
	{
		const FUintVector3 V = RandomUInt3(iInstance, RandomSeedOffset);
		return FVector3f(
			float((V.X >> 8) & 0x00ffffff) / 16777216.0f,
			float((V.Y >> 8) & 0x00ffffff) / 16777216.0f,
			float((V.Z >> 8) & 0x00ffffff) / 16777216.0f
		);
	}

	FVector4f FParticleSimulationContext::RandomFloat4(uint32 iInstance, uint32 RandomSeedOffset) const
	{
		const FUintVector4 V = RandomUInt4(iInstance, RandomSeedOffset);
		return FVector4f(
			float((V.X >> 8) & 0x00ffffff) / 16777216.0f,
			float((V.Y >> 8) & 0x00ffffff) / 16777216.0f,
			float((V.Z >> 8) & 0x00ffffff) / 16777216.0f,
			float((V.W >> 8) & 0x00ffffff) / 16777216.0f
		);
	}
} //namespace NiagaraStateless
