// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessParticleSimExecData.h"
#include "Stateless/NiagaraStatelessParticleSimContext.h"

#include "NiagaraDataSetCompiledData.h"

namespace NiagaraStateless
{
	FParticleSimulationExecData::FParticleSimulationExecData(const FNiagaraDataSetCompiledData& ParticleDataSetCompiledData)
	{
		const int32 NumVariables = FMath::Min(ParticleDataSetCompiledData.Variables.Num(), ParticleDataSetCompiledData.VariableLayouts.Num());
		VariableComponentOffsets.AddDefaulted(NumVariables);

		for (int32 i = 0; i < NumVariables; ++i)
		{
			const FNiagaraVariableLayoutInfo& VariableLayout = ParticleDataSetCompiledData.VariableLayouts[i];
			const FNiagaraVariableBase& Variable = ParticleDataSetCompiledData.Variables[i];
			if (VariableLayout.GetNumFloatComponents() > 0)
			{
				check(VariableLayout.GetNumInt32Components() == 0 && VariableLayout.GetNumHalfComponents() == 0);
				VariableComponentOffsets[i].Type	= 0;
				VariableComponentOffsets[i].Offset	= VariableLayout.GetFloatComponentStart();
				VariableComponentOffsets[i].Num		= VariableLayout.GetNumFloatComponents();
			}
			else if (VariableLayout.GetNumInt32Components() > 0)
			{
				check(VariableLayout.GetNumFloatComponents() == 0 && VariableLayout.GetNumHalfComponents() == 0);
				VariableComponentOffsets[i].Type	= 1;
				VariableComponentOffsets[i].Offset	= VariableLayout.GetInt32ComponentStart();
				VariableComponentOffsets[i].Num		= VariableLayout.GetNumInt32Components();
			}
			else
			{
				// We don't support half components
				checkNoEntry();
			}
		}
		
		TConstArrayView<FNiagaraVariableBase> RequiredComponents = FParticleSimulationContext::GetRequiredComponents();
		RequiredComponentOffsets.Reserve(RequiredComponents.Num());
		for (const FNiagaraVariableBase& RequiredComponent : RequiredComponents)
		{
			const int32 OutputIndex = RequiredComponent.IsValid() ? ParticleDataSetCompiledData.Variables.IndexOfByKey(RequiredComponent) : INDEX_NONE;
			FRequiredComponentOffset& RequiredComponentOffset = RequiredComponentOffsets.AddDefaulted_GetRef();
			if (OutputIndex == INDEX_NONE)
			{
				RequiredComponentOffset.bTransient	= 1;
				RequiredComponentOffset.Offset		= RequiredComponentByteSize;
				RequiredComponentByteSize += sizeof(float);
			}
			else
			{
				RequiredComponentOffset.bTransient	= 0;
				RequiredComponentOffset.Offset		= OutputIndex;
			}
		}
	}

} //namespace NiagaraStateless
