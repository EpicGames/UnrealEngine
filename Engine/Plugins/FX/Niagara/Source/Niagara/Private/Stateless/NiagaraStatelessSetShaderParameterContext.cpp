// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessSetShaderParameterContext.h"

FNiagaraStatelessSetShaderParameterContext::FNiagaraStatelessSetShaderParameterContext(const FNiagaraStatelessSpaceTransforms& InSpaceTransforms, TConstArrayView<uint8> InRendererParameterData, TConstArrayView<uint8> InBuiltData, TConstArrayView<uint8> InSharedBuiltData, TConstArrayView<TPair<FName, uint32>> InSharedBuiltDataOffsets, const FShaderParametersMetadata* InShaderParametersMetadata, uint8* InShaderParameters)
	: SpaceTransforms(InSpaceTransforms)
	, RendererParameterData(InRendererParameterData)
	, BuiltData(InBuiltData)
	, SharedBuiltData(InSharedBuiltData)
	, SharedBuiltDataOffsets(InSharedBuiltDataOffsets)
	, ShaderParametersBase(InShaderParameters)
	, ParameterOffset(0)
	, ShaderParametersMetadata(InShaderParametersMetadata)
{
}

void FNiagaraStatelessSetShaderParameterContext::TransformVectorRangeToScaleBias(const FNiagaraStatelessRangeVector3& Range, ENiagaraCoordinateSpace SourceSpace, FVector3f& OutScale, FVector3f& OutBias) const
{
	if (Range.ParameterOffset == INDEX_NONE)
	{
		OutScale = SpaceTransforms.TransformVector(SourceSpace, Range.GetScale());
		OutBias  = SpaceTransforms.TransformVector(SourceSpace, Range.Min);
	}
	else
	{
		OutScale = FNiagaraStatelessRangeDefaultValue<FVector3f>::Zero();
		OutBias  = SpaceTransforms.TransformVector(SourceSpace, GetRendererParameterValue(Range.ParameterOffset, Range.Min));
	}
}

void FNiagaraStatelessSetShaderParameterContext::TransformVectorNoScaleRangeToScaleBias(const FNiagaraStatelessRangeVector3& Range, ENiagaraCoordinateSpace SourceSpace, FVector3f& OutScale, FVector3f& OutBias) const
{
	if (Range.ParameterOffset == INDEX_NONE)
	{
		OutScale = SpaceTransforms.TransformVectorNoScale(SourceSpace, Range.GetScale());
		OutBias = SpaceTransforms.TransformVectorNoScale(SourceSpace, Range.Min);
	}
	else
	{
		OutScale = FNiagaraStatelessRangeDefaultValue<FVector3f>::Zero();
		OutBias = SpaceTransforms.TransformVectorNoScale(SourceSpace, GetRendererParameterValue(Range.ParameterOffset, Range.Min));
	}
}

void FNiagaraStatelessSetShaderParameterContext::TransformPositionRangeToScaleBias(const FNiagaraStatelessRangeVector3& Range, ENiagaraCoordinateSpace SourceSpace, FVector3f& OutScale, FVector3f& OutBias) const
{
	if (Range.ParameterOffset == INDEX_NONE)
	{
		OutScale = SpaceTransforms.TransformVector(SourceSpace, Range.GetScale());
		OutBias = SpaceTransforms.TransformPosition(SourceSpace, Range.Min);
	}
	else
	{
		OutScale = FNiagaraStatelessRangeDefaultValue<FVector3f>::Zero();
		OutBias = SpaceTransforms.TransformPosition(SourceSpace, GetRendererParameterValue(Range.ParameterOffset, Range.Min));
	}
}

void FNiagaraStatelessSetShaderParameterContext::TransformRotationRangeToScaleBias(const FNiagaraStatelessRangeVector3& Range, ENiagaraCoordinateSpace SourceSpace, FVector3f& OutScale, FVector3f& OutBias) const
{
	if (Range.ParameterOffset == INDEX_NONE)
	{
		OutScale = Range.GetScale();
		OutBias = Range.Min;
	}
	else
	{
		OutScale = FNiagaraStatelessRangeDefaultValue<FVector3f>::Zero();
		OutBias = GetRendererParameterValue(Range.ParameterOffset, Range.Min);
	}
	const FQuat4f Rotation = SpaceTransforms.TransformRotation(SourceSpace, FQuat4f::MakeFromEuler(OutBias));
	OutBias = Rotation.Euler();
}

FVector3f FNiagaraStatelessSetShaderParameterContext::TransformPositionRangeToValue(const FNiagaraStatelessRangeVector3& Range, ENiagaraCoordinateSpace SourceSpace) const
{
	return SpaceTransforms.TransformPosition(SourceSpace, GetRendererParameterValue(Range.ParameterOffset, Range.Min));
}

#if DO_CHECK
void FNiagaraStatelessSetShaderParameterContext::ValidateIncludeStructType(uint32 StructOffset, const FShaderParametersMetadata* StructMetaData) const
{
	for (const FShaderParametersMetadata::FMember& Member : ShaderParametersMetadata->GetMembers())
	{
		if (Member.GetOffset() != StructOffset)
		{
			continue;
		}

		if (Member.GetBaseType() == UBMT_INCLUDED_STRUCT && Member.GetStructMetadata() && Member.GetStructMetadata()->GetLayout() == StructMetaData->GetLayout())
		{
			return;
		}

		const TCHAR* StructType = Member.GetStructMetadata() ? Member.GetStructMetadata()->GetStructTypeName() : TEXT("null");
		UE_LOGF(LogNiagara, Fatal, "Shader parameter struct member (%ls) at offset (%u) is not of type (%ls) struct type is (%ls)", Member.GetName(), StructOffset, StructMetaData->GetStructTypeName(), StructType);
		return;
	}

	UE_LOGF(LogNiagara, Fatal, "Failed to find shader parameter struct member type (%ls) at offset (%u)", StructMetaData->GetStructTypeName(), StructOffset);
}
#endif //DO_CHECK
