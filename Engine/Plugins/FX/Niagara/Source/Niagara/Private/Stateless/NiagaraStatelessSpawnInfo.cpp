// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessSpawnInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessSpawnInfo)

bool FNiagaraStatelessSpawnInfo::IsValid(TOptional<float> LoopDuration) const
{
	if (bEnabled)
	{
		switch (Type)
		{
			case ENiagaraStatelessSpawnInfoType::Burst:
				return (Amount.Min + Amount.Max) > 0 && SpawnTime >= 0.0f && SpawnTime < LoopDuration.Get(SpawnTime + UE_SMALL_NUMBER);

			case ENiagaraStatelessSpawnInfoType::Rate:
				return (Rate.Min + Rate.Max) > 0.0f;

			default:
				checkNoEntry();
				return false;
		}
	}
	return false;
}
