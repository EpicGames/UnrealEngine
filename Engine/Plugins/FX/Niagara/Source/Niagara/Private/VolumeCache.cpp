// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumeCache.cpp: UvolumeTexture implementation.
=============================================================================*/

#include "VolumeCache.h"
#include "NiagaraSettings.h"
#include "NiagaraUseOpenVDB.h"
#include "RHICommandList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VolumeCache)

#if UE_USE_OPENVDB
#include "NiagaraOpenVDB.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogVolumeCache, Log, All);


#if UE_USE_OPENVDB
class NIAGARA_API FOpenVDBCacheData : public FVolumeCacheData
{
public:
	FOpenVDBCacheData() : TotalMemoryUsage(0) {}

	virtual ~FOpenVDBCacheData()
	{
		OpenVDBGrids.Reset();
		DenseGridPtr = nullptr;
	}

	virtual void Init(FIntVector Resolution) override;
	virtual bool LoadFile(FString Path, int frame) override;
	virtual bool UnloadFile(int frame) override;
	virtual bool LoadRange(FString Path, int Start, int End) override;
	virtual void UnloadAll() override;
	virtual bool Fill3DTexture_RenderThread(int frame, FTextureRHIRef TextureToFill, FRHICommandListImmediate& RHICmdList) override;
	virtual bool Fill3DTexture(int frame, FTextureRHIRef TextureToFill) override;

private:
	TMap<int32, Vec4Grid::Ptr> OpenVDBGrids;
	TQueue<int32> CacheQueue;

	int64 TotalMemoryUsage;

	FCriticalSection DenseGridGuard;
	Vec4Dense::Ptr DenseGridPtr;
};
#endif


//*****************************************************************************
//***************************** UVolumeCache ********************************
//*****************************************************************************

UVolumeCache::UVolumeCache(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), CachedVolumeFiles(nullptr)
{	
}

void UVolumeCache::InitData()
{	
	if (CachedVolumeFiles != nullptr)
	{
		CachedVolumeFiles.Reset();
	}
	
	switch (CacheType)
	{
		case EVolumeCacheType::OpenVDB :
		{			
#if UE_USE_OPENVDB
			CachedVolumeFiles = TSharedPtr<FVolumeCacheData>(new FOpenVDBCacheData());
#else
			UE_LOGF(LogVolumeCache, Warning, "OpenVDB not supported on the platform");
#endif

			break;
		}
		default:
			UE_LOGF(LogVolumeCache, Warning, "Invalid Volume Cache type");
	}
		
	CachedVolumeFiles->Init(Resolution);	
}

bool UVolumeCache::LoadRange()
{
	if (CachedVolumeFiles == nullptr)
	{
		InitData();
	}

	return CachedVolumeFiles->LoadRange(FilePath, FrameRangeStart, FrameRangeEnd);
}

void UVolumeCache::UnloadAll()
{
	if (CachedVolumeFiles != nullptr)
	{
		CachedVolumeFiles->UnloadAll();
	}
}

FString UVolumeCache::GetAssetPath(int frame)
{
	check(CachedVolumeFiles != nullptr);

	return CachedVolumeFiles->GetAssetPath(FilePath, frame);
}

bool UVolumeCache::LoadFile(int frame)
{
	if (CachedVolumeFiles == nullptr)
	{
		InitData();
	}
	
	return CachedVolumeFiles->LoadFile(FilePath, frame);	
}

bool UVolumeCache::UnloadFile(int frame)
{
	if (CachedVolumeFiles == nullptr)
	{
		InitData();
	}

	return CachedVolumeFiles->UnloadFile(frame);
}

FString FVolumeCacheData::GetAssetPath(FString PathFormat, int32 FrameIndex) const
{
	const TMap<FString, FStringFormatArg> PathFormatArgs =
	{
		{TEXT("FrameIndex"),	FString::Printf(TEXT("%03d"), FrameIndex)},
	};
	FString AssetPath = FString::Format(*PathFormat, PathFormatArgs);
	AssetPath.ReplaceInline(TEXT("//"), TEXT("/"));
	return AssetPath;
}

#if UE_USE_OPENVDB
void FOpenVDBCacheData::Init(FIntVector Resolution)
{
	FScopeLock ScopeLock(&DenseGridGuard);
	
	DenseGridPtr.reset(new Vec4Dense(openvdb::CoordBBox(0, 0, 0, Resolution.X - 1, Resolution.Y - 1, Resolution.Z - 1), Vec4(0.0, 0.0, 0.0, 0.0)));

	if (OpenVDBGrids.Num() == 0)
	{
		TotalMemoryUsage = 0;
	}
}

bool FOpenVDBCacheData::LoadFile(FString Path, int frame)
{

	// if the file is loaded at the current frame, return true
	if (OpenVDBGrids.Contains(frame) && OpenVDBGrids[frame] != nullptr)
	{
		// if dense vdb buffer doesn't match current resolution, re initialize it				
		FScopeLock ScopeLock(&DenseGridGuard);
		if (DenseGridPtr == nullptr || (DenseGridPtr->bbox().dim() != openvdb::Coord(DenseResolution.X - 1, DenseResolution.Y - 1, DenseResolution.Z - 1)))
		{
			Init(DenseResolution);
		}

		openvdb::tools::CopyToDense<Vec4Tree, Vec4Dense> Copier(OpenVDBGrids[frame]->tree(), *DenseGridPtr);
		Copier.copy();

		return true;
	}
	else
	{
		FString FullPath = GetAssetPath(Path, frame);
		std::string FileNameStr(TCHAR_TO_ANSI(*FullPath));

		// Create a VDB file object.	
		openvdb::io::File file(FileNameStr);

		// Open the file.  This reads the file header, but not any grids.		
		try
		{
			file.open(false);
		}
		catch (openvdb::Exception e)
		{
			UE_LOGF(LogVolumeCache, Warning, "Cache File not Found: %ls", *FullPath);
			return false;
		}

		openvdb::MetaMap::Ptr Metadata = file.getMetadata();

		try
		{
			FIntVector Size;

			Size.X = Metadata->metaValue<int>("DenseResolutionX");
			Size.Y = Metadata->metaValue<int>("DenseResolutionY");
			Size.Z = Metadata->metaValue<int>("DenseResolutionZ");
			if (DenseResolution == FIntVector(-1, -1, -1))
			{
				DenseResolution = Size;
			}
			else if (DenseResolution != Size)
			{
				UE_LOGF(LogVolumeCache, Warning, "Grids are not the same size: %ls", *FullPath);
				return false;
			}

			bool UseFloatGrids = Metadata->metaValue<bool>("FloatGridStorage");

			if (UseFloatGrids)
			{
				UE_LOGF(LogVolumeCache, Warning, "Cache has float grids, which are not used for cache reading: %ls", *FullPath);
				return false;
			}
			else
			{
				Vec4Grid::Ptr ColorGrid;
				ColorGrid = openvdb::gridPtrCast<Vec4Grid>(file.readGrid(openvdb::Name("Color")));

				if (ColorGrid == nullptr)
				{
					UE_LOGF(LogVolumeCache, Warning, "Cache has invalid grids: %ls", *FullPath);
					return false;
				}

				OpenVDBGrids.Add(frame, ColorGrid);

				// add total memory usage reported in bytes
				TotalMemoryUsage += ColorGrid->memUsage() / 1000000;

				// enqueue 
				CacheQueue.Enqueue(frame);

				// if total memory usage is over the limit, dequeue oldest one
				int64 MaxMemory = GetDefault<UNiagaraSettings>()->SimCacheMaxCPUMemoryVolumetrics;
				if (TotalMemoryUsage > MaxMemory && !CacheQueue.IsEmpty())
				{
					int FrameToRemove = CacheQueue.Pop();
					UnloadFile(FrameToRemove);
				}

				// if dense vdb buffer doesn't match current resolution, re initialize it				
				FScopeLock ScopeLock(&DenseGridGuard);
				if (DenseGridPtr == nullptr || (DenseGridPtr->bbox().dim() != openvdb::Coord(DenseResolution.X - 1, DenseResolution.Y - 1, DenseResolution.Z - 1)))
				{
					Init(DenseResolution);
				}

				// load to dense buffer	
				openvdb::tools::CopyToDense<Vec4Tree, Vec4Dense> Copier(ColorGrid->tree(), *DenseGridPtr);
				Copier.copy();
			}
		}
		catch (openvdb::Exception e)
		{
			UE_LOGF(LogVolumeCache, Warning, "No cache metadata found: %ls", *FullPath);
			return false;
		}

		file.close();
	}
	return true;
}

bool FOpenVDBCacheData::UnloadFile(int frame)
{
	if (OpenVDBGrids.Contains(frame) && OpenVDBGrids[frame] != nullptr)
	{
		TotalMemoryUsage -= OpenVDBGrids[frame]->memUsage() / 100000;
		OpenVDBGrids[frame] = nullptr;
		OpenVDBGrids.Remove(frame);

		return true;
	}

	return false;
}

bool FOpenVDBCacheData::LoadRange(FString Path, int Start, int End)
{	
	for (int i = Start; i <= End; ++i)
	{
		if (!LoadFile(Path, i))
		{
			return false;
		}
	}

	return true;
}

void FOpenVDBCacheData::UnloadAll()
{
	OpenVDBGrids.Reset();
	CacheQueue.Empty();

	TotalMemoryUsage = 0;
}

bool FOpenVDBCacheData::Fill3DTexture_RenderThread(int frame, FTextureRHIRef TextureToFill, FRHICommandListImmediate& RHICmdList)
{
	if (OpenVDBGrids.Contains(frame) && OpenVDBGrids[frame] != nullptr)
	{
		FScopeLock ScopeLock(&DenseGridGuard);
		uint8* DataPtr = (uint8*)DenseGridPtr->data();

		const int32 FormatSize = GPixelFormats[TextureToFill->GetFormat()].BlockBytes;

		if (TextureToFill->GetSizeXYZ() != DenseResolution)
		{
			UE_LOGF(LogVolumeCache, Warning, "Target texture resolution %ls doesn't match OpenVDB grid resolution %ls.  Regenerate your cache", *TextureToFill->GetSizeXYZ().ToString(), *DenseResolution.ToString());
		}
		else
		{
			const FUpdateTextureRegion3D UpdateRegion(0, 0, 0, 0, 0, 0, DenseResolution.X, DenseResolution.Y, DenseResolution.Z);			
			FUpdateTexture3DData TheData = RHICmdList.BeginUpdateTexture3D(TextureToFill, 0, UpdateRegion);

			const SIZE_T RowSize = static_cast<SIZE_T>(UpdateRegion.Width) * static_cast<SIZE_T>(FormatSize);
			const SIZE_T SliceSize = static_cast<SIZE_T>(UpdateRegion.Height) * RowSize;
			if (TheData.RowPitch != RowSize || TheData.DepthPitch != SliceSize)
			{
				for (int32 z=0; z < DenseResolution.Z; ++z)
				{
					uint8* DestRow = TheData.Data + (SIZE_T(TheData.DepthPitch) * SIZE_T(z));
					for (int32 y=0; y < DenseResolution.Y; ++y)
					{
						FMemory::Memcpy(DestRow, DataPtr, RowSize);
						DestRow += TheData.RowPitch;
						DataPtr += RowSize;
					}
				}
			}
			else
			{
				const SIZE_T MemorySize = static_cast<SIZE_T>(UpdateRegion.Depth) * SliceSize;
				FMemory::Memcpy(TheData.Data, DataPtr, MemorySize);
			}
			RHICmdList.EndUpdateTexture3D(TheData);						
		}

		return true;
	}

	return false;
}

bool FOpenVDBCacheData::Fill3DTexture(int frame, FTextureRHIRef TextureToFill)
{
	if (OpenVDBGrids.Contains(frame) && OpenVDBGrids[frame] != nullptr)
	{
		ENQUEUE_RENDER_COMMAND(FUpdateData)(
			[this, frame, TextureToFill](FRHICommandListImmediate& RHICmdList)
			{
				Fill3DTexture_RenderThread(frame, TextureToFill, RHICmdList);
			}
		);
		return true;
	}
	return false;
}

bool OpenVDBTools::WriteImageDataToOpenVDBFile(FStringView FilePath, FIntVector ImageSize, TArrayView<FFloat16Color> ImageData, bool UseFloatGrids)
{
	openvdb::math::Transform::Ptr TransformPtr = openvdb::math::Transform::createLinearTransform(/*voxel size=*/1.0);

	openvdb::FloatGrid::Ptr RGrid;
	openvdb::FloatGrid::Ptr GGrid;
	openvdb::FloatGrid::Ptr BGrid;
	openvdb::FloatGrid::Ptr AGrid;

	Vec4Grid::Ptr ColorGrid;

	if (UseFloatGrids)
	{
		RGrid = openvdb::FloatGrid::create(0.0);
		GGrid = openvdb::FloatGrid::create(0.0);
		BGrid = openvdb::FloatGrid::create(0.0);
		AGrid = openvdb::FloatGrid::create(0.0);

		RGrid->setTransform(TransformPtr);
		GGrid->setTransform(TransformPtr);
		BGrid->setTransform(TransformPtr);
		AGrid->setTransform(TransformPtr);

		RGrid->setGridClass(openvdb::GRID_FOG_VOLUME);
		GGrid->setGridClass(openvdb::GRID_FOG_VOLUME);
		BGrid->setGridClass(openvdb::GRID_FOG_VOLUME);
		AGrid->setGridClass(openvdb::GRID_FOG_VOLUME);

		RGrid->setName("R");
		GGrid->setName("G");
		BGrid->setName("B");
		AGrid->setName("A");

		openvdb::FloatGrid::Accessor Raccessor = RGrid->getAccessor();
		openvdb::FloatGrid::Accessor Gaccessor = GGrid->getAccessor();
		openvdb::FloatGrid::Accessor Baccessor = BGrid->getAccessor();
		openvdb::FloatGrid::Accessor Aaccessor = AGrid->getAccessor();

		for (int z = 0; z < ImageSize.Z; ++z) {
			for (int y = 0; y < ImageSize.Y; ++y) {
				for (int x = 0; x < ImageSize.X; ++x) {
					uint32 LinearIndex = x + y * ImageSize.X + z * ImageSize.X * ImageSize.Y;
					FFloat16Color CurrValue = ImageData[LinearIndex];

					Raccessor.setValue(openvdb::Coord(x, y, z), CurrValue.R.GetFloat());
					Gaccessor.setValue(openvdb::Coord(x, y, z), CurrValue.G.GetFloat());
					Baccessor.setValue(openvdb::Coord(x, y, z), CurrValue.B.GetFloat());
					Aaccessor.setValue(openvdb::Coord(x, y, z), CurrValue.A.GetFloat());
				}	
			}
		}
	}
	else
	{
		ColorGrid = Vec4Grid::create(Vec4(0.0));
		ColorGrid->setTransform(TransformPtr);
		ColorGrid->setGridClass(openvdb::GRID_FOG_VOLUME);
		ColorGrid->setName("Color");
		Vec4Grid::Accessor ColorAccessor = ColorGrid->getAccessor();

		for (int z = 0; z < ImageSize.Z; ++z) {
			for (int y = 0; y < ImageSize.Y; ++y) {
				for (int x = 0; x < ImageSize.X; ++x) {
					uint32 LinearIndex = x + y * ImageSize.X + z * ImageSize.X * ImageSize.Y;
					FFloat16Color CurrValue = ImageData[LinearIndex];

					ColorAccessor.setValue(openvdb::Coord(x, y, z), Vec4(CurrValue.R.GetFloat(), CurrValue.G.GetFloat(), CurrValue.B.GetFloat(), CurrValue.A.GetFloat()));
				}
			}
		}
	}

	// Create a VDB file object.		
	FString Filename(FilePath);
	std::string FileNameStr(TCHAR_TO_ANSI(*Filename));

	openvdb::io::File file(FileNameStr);

	// Add the grid pointer to a container.
	openvdb::GridPtrVec grids;

	if (UseFloatGrids)
	{
		openvdb::tools::pruneInactive(RGrid->tree());
		openvdb::tools::pruneInactive(GGrid->tree());
		openvdb::tools::pruneInactive(BGrid->tree());
		openvdb::tools::pruneInactive(AGrid->tree());

		RGrid->pruneGrid(0);
		GGrid->pruneGrid(0);
		BGrid->pruneGrid(0);
		AGrid->pruneGrid(0);

		grids.push_back(RGrid);
		grids.push_back(GGrid);
		grids.push_back(BGrid);
		grids.push_back(AGrid);
	}	
	else
	{
		openvdb::tools::pruneInactive(ColorGrid->tree());
		ColorGrid->pruneGrid(0);
		grids.push_back(ColorGrid);
	}
	
	// Add file-level metadata.
	openvdb::MetaMap outMeta;
	outMeta.insertMeta("creator",
		openvdb::StringMetadata("Unreal Engine"));

	outMeta.insertMeta("FloatGridStorage", 
		openvdb::BoolMetadata(UseFloatGrids));

	outMeta.insertMeta("DenseResolutionX",
		openvdb::Int32Metadata(ImageSize.X));
	outMeta.insertMeta("DenseResolutionY",
		openvdb::Int32Metadata(ImageSize.Y));
	outMeta.insertMeta("DenseResolutionZ",
		openvdb::Int32Metadata(ImageSize.Z));


	uint32_t compressionFlags = openvdb::io::COMPRESS_BLOSC;
	file.setCompression(compressionFlags);

	file.write(grids, outMeta);
	file.close();

	return true;
}
#endif
