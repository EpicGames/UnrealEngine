// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class FMeshUvMappingBufferProxy;
struct FMeshUvMapping;
struct FSkeletalMeshUvMapping;
struct FStaticMeshUvMapping;

struct FMeshUvMappingUsage
{
	FMeshUvMappingUsage() = default;
	FMeshUvMappingUsage(bool InRequiresCpuAccess, bool InrequiresGpuAccess)
		: RequiresCpuAccess(InRequiresCpuAccess)
		, RequiresGpuAccess(InrequiresGpuAccess)
	{}

	bool IsValid() const { return RequiresCpuAccess || RequiresGpuAccess; }

	bool RequiresCpuAccess = false;
	bool RequiresGpuAccess = false;
};

struct FMeshUvMappingHandleBase
{
	FMeshUvMappingHandleBase() = default;
	FMeshUvMappingHandleBase(const FMeshUvMappingHandleBase& Other) = delete;
	NIAGARA_API ~FMeshUvMappingHandleBase();
	FMeshUvMappingHandleBase& operator=(const FMeshUvMappingHandleBase& Other) = delete;
	NIAGARA_API explicit operator bool() const;

	NIAGARA_API const FMeshUvMappingBufferProxy* GetQuadTreeProxy() const;
	NIAGARA_API int32 GetUvSetIndex() const;
	NIAGARA_API int32 GetLodIndex() const;
	NIAGARA_API void PinAndInvalidateHandle();

	FMeshUvMappingUsage Usage;

protected:
	NIAGARA_API FMeshUvMappingHandleBase(FMeshUvMappingUsage InUsage, const TSharedPtr<FMeshUvMapping>& InUvMappingData, bool bNeedsDataImmediately);
	NIAGARA_API FMeshUvMappingHandleBase(FMeshUvMappingHandleBase&& Other) noexcept;
	NIAGARA_API FMeshUvMappingHandleBase& operator=(FMeshUvMappingHandleBase&& Other) noexcept;

	TSharedPtr<FMeshUvMapping> UvMappingData;
};


template<typename MappingType>
struct TTypedMeshUvMappingHandle : public FMeshUvMappingHandleBase
{
	TTypedMeshUvMappingHandle() = default;
	TTypedMeshUvMappingHandle(FMeshUvMappingUsage InUsage, const TSharedPtr<MappingType>& InUvMappingData, bool bNeedsDataImmediately)
		: FMeshUvMappingHandleBase(InUsage, InUvMappingData, bNeedsDataImmediately) {}

	TTypedMeshUvMappingHandle(TTypedMeshUvMappingHandle<MappingType>&& Other) noexcept
		: FMeshUvMappingHandleBase(Other) {}

	TTypedMeshUvMappingHandle<MappingType>& operator=(TTypedMeshUvMappingHandle<MappingType>&& Other) noexcept
	{
		FMeshUvMappingHandleBase::operator=(MoveTemp(Other));
		return *this;
	}

	const MappingType* GetMappingData() const
	{
		if (UvMappingData)
		{
			return static_cast<const MappingType*>(UvMappingData.Get());
		}

		return nullptr;
	}
};

using FSkeletalMeshUvMappingHandle = TTypedMeshUvMappingHandle<FSkeletalMeshUvMapping>;
using FStaticMeshUvMappingHandle = TTypedMeshUvMappingHandle<FStaticMeshUvMapping>;
