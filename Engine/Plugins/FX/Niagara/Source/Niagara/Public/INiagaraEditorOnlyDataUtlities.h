// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

class UNiagaraDataInterface;
class UNiagaraEditorDataBase;
class UNiagaraEditorParametersAdapterBase;
class UNiagaraMessageDataBase;
class UNiagaraScriptSourceBase;
class UNiagaraSystem;
struct FNiagaraSystemStateData;

/** Defines utility methods for creating editor only data which is stored on runtime objects. */
class INiagaraEditorOnlyDataUtilities
{
public:
	virtual UNiagaraScriptSourceBase* CreateDefaultScriptSource(UObject* InOuter) const = 0;

	virtual UNiagaraEditorDataBase* CreateDefaultEditorData(UObject* InOuter) const = 0;

	virtual UNiagaraEditorParametersAdapterBase* CreateDefaultEditorParameters(UObject* InOuter) const = 0;

	virtual UObject::FAssetRegistryTag CreateClassUsageAssetRegistryTag(const UObject* SourceObject) const = 0;

	virtual UNiagaraMessageDataBase* CreateErrorMessage(UObject* InOuter, FText InMessageShort, FText InMessageLong, FName InTopicName, bool bInAllowDismissal = false) const = 0;

	virtual UNiagaraMessageDataBase* CreateWarningMessage(UObject* InOuter, FText InMessageShort, FText InMessageLong, FName InTopicName, bool bInAllowDismissal = false) const = 0;

	virtual bool IsEditorDataInterfaceInstance(const UNiagaraDataInterface* DataInterface) const = 0;

	virtual UNiagaraDataInterface* GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(const UNiagaraSystem& OwningSystem, UNiagaraDataInterface& EditorDataInterfaceInstance) const = 0;

	virtual TOptional<FNiagaraSystemStateData> GetSystemStateData(const UNiagaraSystem& System) const = 0;

	virtual bool GetResetSimulationOnParameterChange() const = 0;
	
	virtual void MoveUserAssetTagsToMetaData(UObject* NiagaraAsset) const = 0;
};
