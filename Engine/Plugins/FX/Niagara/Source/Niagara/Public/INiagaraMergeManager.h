// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

class FProperty;
class UNiagaraEmitter;
class UNiagaraMessageDataBase;
struct FVersionedNiagaraEmitter;

class INiagaraMergeManager
{
public:
	enum class EMergeEmitterResult
	{
		SucceededNoDifferences,
		SucceededDifferencesApplied,
		FailedToDiff,
		FailedToMerge,
		None
	};

	struct FMergeEmitterResults
	{
		FMergeEmitterResults()
			: MergeResult(EMergeEmitterResult::None)
			, MergeNiagaraMessage(nullptr)
			, bModifiedGraph(false)
			, MergedInstance(nullptr)
		{
		}

		EMergeEmitterResult MergeResult;
		TArray<FText> ErrorMessages;
		UNiagaraMessageDataBase* MergeNiagaraMessage;
		bool bModifiedGraph;
		UNiagaraEmitter* MergedInstance;

		FString GetErrorMessagesString() const
		{
			TArray<FString> ErrorMessageStrings;
			for (FText ErrorMessage : ErrorMessages)
			{
				ErrorMessageStrings.Add(ErrorMessage.ToString());
			}
			return FString::Join(ErrorMessageStrings, TEXT("\n"));
		}
	};

	virtual FMergeEmitterResults MergeEmitter(const FVersionedNiagaraEmitter& Parent, const FVersionedNiagaraEmitter& ParentAtLastMerge, const FVersionedNiagaraEmitter& Instance) const = 0;

	virtual void DiffEditableProperties(const void* BaseDataAddress, const void* OtherDataAddress, UStruct& Struct, TArray<FProperty*>& OutDifferentProperties) const = 0;

	virtual void CopyPropertiesToBase(void* BaseDataAddress, const void* OtherDataAddress, TArray<FProperty*> PropertiesToCopy) const = 0;

	virtual ~INiagaraMergeManager() = default;
};
