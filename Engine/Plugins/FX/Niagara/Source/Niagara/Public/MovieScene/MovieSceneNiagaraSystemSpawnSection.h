// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "MovieSceneSection.h"
#include "MovieSceneNiagaraSystemSpawnSection.generated.h"

/** Defines options for system life cycle for before the section is evaluating up to the first frame the section evaluates. */
UENUM(BlueprintType)
enum class ENiagaraSystemSpawnSectionStartBehavior : uint8
{
	/** When the time before the section evaluates the particle system's component will be deactivated and on the first frame of the section the
	 system's component will be activated. */
	Activate
};

/** Defines options for system life cycle for when the section is evaluating from the 2nd frame until the last frame of the section. */
UENUM(BlueprintType)
enum class ENiagaraSystemSpawnSectionEvaluateBehavior : uint8
{
	/** The system's component will be activated on any frame where it is inactive.  This is useful for continuous emitters, especially if the sequencer will start in the middle of the section. */
	ActivateIfInactive,
	/** There sill be no changes to the system life cycle while the section is evaluating. */
	None
};

/** Defines options for system life cycle for the time after the section. */
UENUM(BlueprintType)
enum class ENiagaraSystemSpawnSectionEndBehavior : uint8
{
	//** When the section ends the system is set to inactive which stops spawning but lets existing particles simulate until death.
	SetSystemInactive,
	//** When the section ends the system's component is deactivated which will kill all existing particles.
	Deactivate,
	//** Does nothing when the section ends and allows the system to continue to run as normal.
	None
};

UCLASS(MinimalAPI)
class UMovieSceneNiagaraSystemSpawnSection : public UMovieSceneSection
{
	GENERATED_BODY()

public:
	UMovieSceneNiagaraSystemSpawnSection();

	UFUNCTION(BlueprintPure, Category = "Niagara|Life Cycle")
	ENiagaraSystemSpawnSectionStartBehavior GetSectionStartBehavior() const;
	UFUNCTION(BlueprintCallable, Category = "Niagara|Life Cycle")
	NIAGARA_API void SetSectionStartBehavior(ENiagaraSystemSpawnSectionStartBehavior InBehavior);

	UFUNCTION(BlueprintPure, Category = "Niagara|Life Cycle")
	NIAGARA_API ENiagaraSystemSpawnSectionEvaluateBehavior GetSectionEvaluateBehavior() const;
	UFUNCTION(BlueprintCallable, Category = "Niagara|Life Cycle")
	NIAGARA_API void SetSectionEvaluateBehavior(ENiagaraSystemSpawnSectionEvaluateBehavior InBehavior);

	UFUNCTION(BlueprintPure, Category = "Niagara|Life Cycle")
	NIAGARA_API ENiagaraSystemSpawnSectionEndBehavior GetSectionEndBehavior() const;
	UFUNCTION(BlueprintCallable, Category = "Niagara|Life Cycle")
	NIAGARA_API void SetSectionEndBehavior(ENiagaraSystemSpawnSectionEndBehavior InBehavior);

	UFUNCTION(BlueprintPure, Category = "Niagara|Life Cycle")
	NIAGARA_API ENiagaraAgeUpdateMode GetAgeUpdateMode() const;
	UFUNCTION(BlueprintCallable, Category = "Niagara|Life Cycle")
	NIAGARA_API void SetAgeUpdateMode(ENiagaraAgeUpdateMode InMode);

	UFUNCTION(BlueprintPure, Category = "Niagara|Life Cycle")
	bool GetAllowScalability() const;
	UFUNCTION(BlueprintCallable, Category = "Niagara|Life Cycle")
	void SetAllowScalability(bool bInAllowScalability);

private:
	/** Specifies what should happen to the niagara system from before the section evaluates up until the first frame of the section. */
	UPROPERTY(EditAnywhere, Category = "Life Cycle")
	ENiagaraSystemSpawnSectionStartBehavior SectionStartBehavior;

	/** Specifies what should happen to the niagara system when section is evaluating from the 2nd frame until the last frame. */
	UPROPERTY(EditAnywhere, Category = "Life Cycle")
	ENiagaraSystemSpawnSectionEvaluateBehavior SectionEvaluateBehavior;

	/** Specifies what should happen to the niagara system when section evaluation finishes. */
	UPROPERTY(EditAnywhere, Category = "Life Cycle")
	ENiagaraSystemSpawnSectionEndBehavior SectionEndBehavior;

	/** Specifies how sequencer should update the age of the controlled niagara system. */
	UPROPERTY(EditAnywhere, Category = "Life Cycle")
	ENiagaraAgeUpdateMode AgeUpdateMode;
	
	UPROPERTY(EditAnywhere, Category = "Life Cycle")
	bool bAllowScalability;
};