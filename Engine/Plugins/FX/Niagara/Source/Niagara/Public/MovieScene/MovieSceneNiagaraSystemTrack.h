// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MovieSceneExecutionToken.h"
#include "MovieScene/MovieSceneNiagaraTrack.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "Evaluation/PersistentEvaluationData.h"
#include "MovieSceneNiagaraSystemTrack.generated.h"

struct FNiagaraSharedMarkerToken : IMovieSceneSharedExecutionToken
{
	virtual void Execute(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override {}

	TSet<FGuid> BoundObjectIDs;
};

UCLASS(MinimalAPI)
class UMovieSceneNiagaraSystemTrack : public UMovieSceneNiagaraTrack, public IMovieSceneTrackTemplateProducer
{
	GENERATED_BODY()

public:
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
	virtual void PostCompile(FMovieSceneEvaluationTrack& OutTrack, const FMovieSceneTrackCompilerArgs& Args) const override;
	virtual bool PopulateEvaluationTree(TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData) const override;

	NIAGARA_API static FMovieSceneSharedDataId SharedDataId;
};