// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneNameableTrack.h"
#include "MovieSceneNiagaraTrack.generated.h"

UCLASS(abstract, MinimalAPI)
class UMovieSceneNiagaraTrack : public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveAllAnimationData() override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;

protected:
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};