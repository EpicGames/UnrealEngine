// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene/Parameters/MovieSceneNiagaraParameterTrack.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "MovieSceneNiagaraColorParameterTrack.generated.h"

/** A track for animating color niagara parameters. */
UCLASS(MinimalAPI)
class UMovieSceneNiagaraColorParameterTrack : public UMovieSceneNiagaraParameterTrack, public IMovieSceneTrackTemplateProducer
{
	GENERATED_BODY()

public:
	/** UMovieSceneTrack interface. */
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual void SetSectionChannelDefaults(UMovieSceneSection* Section, const TArray<uint8>& DefaultValueData) const override;
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
};