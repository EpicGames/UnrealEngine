// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene/MovieSceneNiagaraTrack.h"
#include "NiagaraTypes.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneNiagaraParameterTrack.generated.h"

/** A base class for tracks that animate niagara parameters. */
UCLASS(abstract, MinimalAPI)
class UMovieSceneNiagaraParameterTrack : public UMovieSceneNiagaraTrack
{
	GENERATED_BODY()

public:
	/** Gets the parameter for this parameter track. */
	NIAGARA_API const FNiagaraVariable& GetParameter() const;

	/** Sets the parameter for this parameter track .*/
	NIAGARA_API void SetParameter(FNiagaraVariable InParameter);

	virtual void SetSectionChannelDefaults(UMovieSceneSection* Section, const TArray<uint8>& DefaultValueData) const { }

protected:
	template<class ChannelType>
	static ChannelType* GetEditableChannelFromProxy(FMovieSceneChannelProxy& ChannelProxy, const ChannelType& Channel)
	{
		int32 ChannelIndex = ChannelProxy.FindIndex(Channel.StaticStruct()->GetFName(), &Channel);
		if (ChannelIndex != INDEX_NONE)
		{
			return ChannelProxy.GetChannel<ChannelType>(ChannelIndex);
		}
		return nullptr;
	}

	template<class ChannelType, typename ValueType>
	static void SetChannelDefault(FMovieSceneChannelProxy& ChannelProxy, const ChannelType& TargetChannel, ValueType DefaultValue)
	{
		ChannelType* EditableTargetChannel = GetEditableChannelFromProxy(ChannelProxy, TargetChannel);
		if (EditableTargetChannel != nullptr)
		{
			EditableTargetChannel->SetDefault(DefaultValue);
		}
	}

private:
	/** The parameter for the parameter this track animates. */
	UPROPERTY()
	FNiagaraVariable Parameter;
};
