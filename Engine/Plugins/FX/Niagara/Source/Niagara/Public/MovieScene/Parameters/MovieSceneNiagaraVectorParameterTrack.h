// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene/Parameters/MovieSceneNiagaraParameterTrack.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "MovieSceneNiagaraVectorParameterTrack.generated.h"

/** A track for animating float niagara parameters. */
UCLASS(MinimalAPI)
class UMovieSceneNiagaraVectorParameterTrack : public UMovieSceneNiagaraParameterTrack, public IMovieSceneTrackTemplateProducer
{
	GENERATED_BODY()

public:
	/** UMovieSceneTrack interface. */
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual void SetSectionChannelDefaults(UMovieSceneSection* Section, const TArray<uint8>& DefaultValueData) const override;
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;

	NIAGARA_API int32 GetChannelsUsed() const;
	NIAGARA_API void SetChannelsUsed(int32 InChannelsUsed);

private:
	UPROPERTY()
	int32 ChannelsUsed;
};