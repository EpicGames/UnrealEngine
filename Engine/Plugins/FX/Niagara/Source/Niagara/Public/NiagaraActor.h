// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "NiagaraActor.generated.h"

/**
* Niagara Particle System Actor contains a Niagara Particle System Component which can be used to simulate and render Niagara Systems
*
* @see https://dev.epicgames.com/documentation/en-us/unreal-engine/BlueprintAPI/NiagaraActor
* @see UNiagaraComponent
*/
UCLASS(hideCategories = (Activation, "Components|Activation", Input, Collision, "Game|Damage"), ComponentWrapperClass, MinimalAPI, meta = (DisplayName = "Niagara Particle System Actor"))
class ANiagaraActor : public AActor
{
	GENERATED_BODY()
protected:

	NIAGARA_API ANiagaraActor(const FObjectInitializer& ObjectInitializer);
	
public:

	NIAGARA_API virtual void PostRegisterAllComponents() override;

	NIAGARA_API virtual void PostUnregisterAllComponents() override;

	/** Set true for this actor to self-destruct when the Niagara system finishes, false otherwise */
	UFUNCTION(BlueprintCallable, Category=NiagaraActor)
	NIAGARA_API void SetDestroyOnSystemFinish(bool bShouldDestroyOnSystemFinish);

	/** Returns true if the system will destroy on finish */
	UFUNCTION(BlueprintCallable, Category=NiagaraActor)
	bool GetDestroyOnSystemFinish() const { return bDestroyOnSystemFinish; }

private:
	/** Pointer to System component */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=NiagaraActor, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UNiagaraComponent> NiagaraComponent;

#if WITH_EDITORONLY_DATA
	// Reference to sprite visualization component
	UPROPERTY()
	TObjectPtr<class UBillboardComponent> SpriteComponent;

	// Reference to arrow visualization component
	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;

#endif

	/** True for this actor to self-destruct when the Niagara system finishes, false otherwise */
	UPROPERTY()
	uint32 bDestroyOnSystemFinish : 1;

	/** Callback when Niagara system finishes. */
	UFUNCTION(CallInEditor)
	NIAGARA_API void OnNiagaraSystemFinished(UNiagaraComponent* FinishedComponent);

public:
	/** Returns NiagaraComponent subobject **/
	class UNiagaraComponent* GetNiagaraComponent() const { return NiagaraComponent; }
#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	class UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
	/** Returns ArrowComponent subobject **/
	class UArrowComponent* GetArrowComponent() const { return ArrowComponent; }
#endif

#if WITH_EDITOR
	// AActor interface
	NIAGARA_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
	// End of AActor interface

	/** Reset this actor in the level.*/
	NIAGARA_API void ResetInLevel();
#endif // WITH_EDITOR

};
