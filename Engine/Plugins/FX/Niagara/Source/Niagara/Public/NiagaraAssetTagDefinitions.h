// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "NiagaraAssetTagDefinitions.generated.h"

#define UE_API NIAGARA_API

/** DEPRECATED - All classes and data in here are deprecated in favor of the generic User Asset Tags system. */

UENUM(meta=(Bitflags, UseEnumValuesAsMaskValuesInEditor=true))
enum class ENiagaraAssetLibraryAssetTypes : uint8
{
	Emitters = 1 << 0,
	Systems = 1 << 1,
	Scripts = 1 << 2
};
ENUM_CLASS_FLAGS(ENiagaraAssetLibraryAssetTypes)

UENUM()
enum class ENiagaraAssetTagDefinitionImportance : uint8
{
	Primary UMETA(ToolTip="This Asset Tag Definition is considered important and will get displayed in the primary filter UI in the Niagara Asset Browsers."),
	Secondary UMETA(ToolTip="This Asset Tag Definition is considered less important and will only get displayed in the additional drop down filter UI in the Niagara Asset Browsers."),
	Internal UMETA(ToolTip="This Asset Tag Definition is for internal use only, and will not be displayed in the primary tag view nor the secondary drop down filter UI.")
	//Both UMETA(ToolTip="This Asset Tag Definition will be displayed in both primary UI & the additional filter drop downs.")
};

/** An Asset Tag Definition defines a tag that can be added to various Niagara assets for sorting & filtering purposes. */
USTRUCT()
struct FNiagaraAssetTagDefinition
{
	GENERATED_BODY()

public:
	FNiagaraAssetTagDefinition() = default;

	FNiagaraAssetTagDefinition(FText InAssetTag, int32 InAssetFlags, FText InDescription, ENiagaraAssetTagDefinitionImportance InDisplayType, FLinearColor InColor, FGuid InTagGuid);
	
	/** The Display Name used for this tag. */
	UPROPERTY(EditAnywhere, Category="Properties")
	FText AssetTag;

	/** Select the asset types this tag can apply to. This is used to hide tags that can never apply for a given type. */
	UPROPERTY(EditAnywhere, Category="Properties", meta=(Bitmask, BitmaskEnum="/Script/Niagara.ENiagaraAssetLibraryAssetTypes"))
	int32 AssetFlags = 0;

	/** Further explanation of what this tag is about. */
	UPROPERTY(EditAnywhere, Category="Properties", meta=(MultiLine))
	FText Description;

	/** Whether this tag should be shown directly or in the drop down for additional filters. */
	UPROPERTY(EditAnywhere, Category="Properties")
	ENiagaraAssetTagDefinitionImportance DisplayType = ENiagaraAssetTagDefinitionImportance::Primary;

	/** The color used in UI to represent this tag. */
	UPROPERTY(EditAnywhere, Category="Properties")
	FLinearColor Color = FLinearColor::Black;

	/** The Tag Guid identifies this tag. This makes it possible to change the AssetTag name without it affecting functionality. */
	UPROPERTY(VisibleAnywhere, Category="Properties", meta=(IgnoreForMemberInitializationTest))
	FGuid TagGuid = FGuid::NewGuid();

	bool operator<(const FNiagaraAssetTagDefinition& Other) const
	{		
		return AssetTag.ToString() < Other.AssetTag.ToString();
	}

	bool operator==(const FNiagaraAssetTagDefinition& Other) const
	{
		return TagGuid == Other.TagGuid;
	}
public:
	NIAGARA_API bool IsValid() const;
	
	NIAGARA_API bool SupportsEmitters() const;
	NIAGARA_API bool SupportsSystems() const;
	NIAGARA_API bool SupportsScripts() const;
	NIAGARA_API TArray<UClass*> GetSupportedClasses() const;
	NIAGARA_API bool DoesAssetDataContainTag(const FAssetData& AssetData) const;

	NIAGARA_API FString GetGuidAsString() const;
};

inline uint32 GetTypeHash(const FNiagaraAssetTagDefinition& AssetTagDefinition)
{
	return GetTypeHash(AssetTagDefinition.TagGuid);
}

/** A Tag Definition Reference stores the guid of a Tag Definition. This is what assets should be storing. */
USTRUCT()
struct FNiagaraAssetTagDefinitionReference
{
	GENERATED_BODY()

public:
	FNiagaraAssetTagDefinitionReference() = default;
	UE_API FNiagaraAssetTagDefinitionReference(const FNiagaraAssetTagDefinition& InTagDefinition);
	
	UE_API FString GetGuidAsString() const;
	UE_API void AddTagToAssetRegistryTags(FAssetRegistryTagsContext& Context) const;
	
	void SetTagDefinitionReference(const FNiagaraAssetTagDefinition& InTagDefinition) { SetTagDefinitionReferenceGuid(InTagDefinition.TagGuid); }
	void SetTagDefinitionReferenceGuid(const FGuid& InTagDefinitionGuid) { AssetTagDefinitionGuid = InTagDefinitionGuid; }
	FGuid GetTagDefinitionReferenceGuid() const { return AssetTagDefinitionGuid; }

	bool operator==(const FNiagaraAssetTagDefinitionReference& Other) const
	{
		return AssetTagDefinitionGuid == Other.AssetTagDefinitionGuid;
	}
private:
	UPROPERTY(VisibleAnywhere, Category="Asset Tags")
	FGuid AssetTagDefinitionGuid;
};

/**
 * An Asset Tag Definition defines a tag that can be added to various Niagara assets for sorting & filtering purposes.
 * For example, custom tags will show up in the Create Niagara System dialog to filter available emitters.
 * They can also be used to filter assets in the content browser, when used with the custom filter option.
 *
 * You can modify asset tags in the content browser by right-clicking on a Niagara asset, then use the "Manage Tags" submenu to add or remove them.
 */
UCLASS(MinimalAPI, Deprecated)
class UDEPRECATED_NiagaraAssetTagDefinitions : public UObject
{
	GENERATED_BODY()
public:	
	const TArray<FNiagaraAssetTagDefinition>& GetAssetTagDefinitions() const { return TagDefinitions; }

	UE_API FText GetDisplayName() const;
	
	UE_API FText GetDescription() const;

	bool DisplayTagsAsFlatList() const { return bDisplayTagsAsFlatList; }
	
	UE_API bool DoesAssetDataContainAnyTag(const FAssetData& AssetData) const;

	int32 GetSortOrder() const { return SortOrder; }
private:
	/** The display name to use when listing this asset in the Niagara Asset Browser */
	UPROPERTY(EditAnywhere, Category="Properties")
	FText DisplayName;
	
	/** A description for this group of tags. Used for tooltips. */
	UPROPERTY(EditAnywhere, Category="Properties")
	FText Description;
	
	UPROPERTY(EditAnywhere, Category="Properties", meta=(TitleProperty="AssetTag"))
	TArray<FNiagaraAssetTagDefinition> TagDefinitions;
	
	/** If true, no 'parent' entry for this asset will be displayed in the Niagara Asset Browser. Instead a flat list of the contained tags will be added. */
	UPROPERTY(EditAnywhere, Category="Properties")
	bool bDisplayTagsAsFlatList = false;
	
	/** Tags are sorted by asset sort order first, then individually. That means tags of asset with sort order [0] come before tags of asset with sort order [1]. */
	UPROPERTY(EditAnywhere, Category="Properties")
	int32 SortOrder = INDEX_NONE;
};

#undef UE_API
