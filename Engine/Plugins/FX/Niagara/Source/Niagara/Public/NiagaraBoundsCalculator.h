// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "NiagaraModule.h"
#include "NiagaraDataSet.h"

class FNiagaraBoundsCalculator
{
public:
	virtual ~FNiagaraBoundsCalculator() = default;
	void InitAccessors(const FNiagaraDataSet& DataSet) { InitAccessors(&DataSet.GetCompiledData()); }
	virtual void InitAccessors(const FNiagaraDataSetCompiledData* CompiledData) = 0;
	virtual FBox CalculateBounds(const FTransform& SystemTransform, const FNiagaraDataSet& DataSet, const int32 NumInstances) const = 0;
};
