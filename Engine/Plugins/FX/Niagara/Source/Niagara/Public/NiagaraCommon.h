// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Particles/ParticlePerfStats.h"
#include "UObject/ObjectKey.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakFieldPtr.h"

#include "NiagaraComponentPoolMethodEnum.h"
#include "NiagaraDefines.h"
#include "NiagaraScalabilityState.h"
#include "NiagaraTickBehaviorEnum.h"
#include "NiagaraTypes.h"
#include "NiagaraTypeRegistry.h"
#include "NiagaraCore.h"
#include "Templates/RefCounting.h"

#include "NiagaraCommon.generated.h"

struct FVersionedNiagaraEmitter;
class UNiagaraComponent;
class UNiagaraEmitterBase;
class UNiagaraSystem;
class UNiagaraScript;
class UNiagaraDataInterface;
class UNiagaraEmitter;
struct FNiagaraEmitterHandle;
class FNiagaraSystemInstance;
class UNiagaraParameterCollection;
class UNiagaraParameterDefinitionsBase;
struct FNiagaraParameterStore;
enum class EPSCPoolMethod : uint8;
enum ETextureRenderTargetFormat : int;
enum class EPixelFormatCapabilities : uint32;

namespace ERHIFeatureLevel { enum Type : int; }
enum EShaderPlatform : uint16;
enum EPixelFormat : uint8;

class FNiagaraDataBuffer;
class FNiagaraShaderMap;

using FNiagaraDataBufferRef = TRefCountPtr<FNiagaraDataBuffer>;
using FNiagaraShaderMapRef = TRefCountPtr<FNiagaraShaderMap>;

#define NIAGARA_MEMORY_TRACKING	!UE_BUILD_SHIPPING

#define NIAGARA_SYSTEM_CAPTURE (!UE_BUILD_SHIPPING || WITH_EDITORONLY_DATA)

#define NIAGARA_VERBOSE_LOGGING_ENABLED (!NO_LOGGING && !UE_BUILD_SHIPPING)

#define NIAGARA_COMPUTEDEBUG_ENABLED_IN_EDITOR	1
#define NIAGARA_COMPUTEDEBUG_ENABLED_IN_COOKED	0
#define NIAGARA_COMPUTEDEBUG_ENABLED ((WITH_EDITOR && NIAGARA_COMPUTEDEBUG_ENABLED_IN_EDITOR) || (!WITH_EDITOR && NIAGARA_COMPUTEDEBUG_ENABLED_IN_COOKED))

#define WITH_NIAGARA_GPU_PROFILER_EDITOR (WITH_EDITOR && STATS)
#define WITH_NIAGARA_GPU_PROFILER ((WITH_PARTICLE_PERF_STATS && !UE_BUILD_SHIPPING) || WITH_NIAGARA_GPU_PROFILER_EDITOR)

/** When enabled allows builds to warn about system / emitter information such as hitting buffer caps.  Small performance overhead for copying the name around so RT has access. */
#define WITH_NIAGARA_DEBUG_EMITTER_NAME	1//!UE_BUILD_SHIPPING

#ifndef WITH_NIAGARA_LEAK_DETECTOR
	#define WITH_NIAGARA_LEAK_DETECTOR	1
#endif

#define INTERPOLATED_PARAMETER_PREFIX TEXT("PREV_")

/** Defines The maximum ThreadGroup size we allow in Niagara.  This is important for how memory is allocated as we always need to round this and the final instance is used to avoid overflowing the buffer. */
inline constexpr uint32 NiagaraComputeMaxThreadGroupSize = 64;

/** The maximum number of spawn infos we can run on the GPU, modifying this will require a version update as it is used in the shader compiler  */
inline constexpr uint32 NIAGARA_MAX_GPU_SPAWN_INFOS = 8;

/** TickGroup information for Niagara.  */
inline constexpr ETickingGroup NiagaraFirstTickGroup = TG_PrePhysics;
inline constexpr ETickingGroup NiagaraLastTickGroup = TG_LastDemotable;
inline constexpr int NiagaraNumTickGroups = NiagaraLastTickGroup - NiagaraFirstTickGroup + 1;

UENUM()
enum class ENiagaraBaseTypes : uint8
{
	Half,
	Float,
	Int32,
	Bool,
	Max UMETA(Hidden),
};

/** Niagara supported buffer formats on the GPU. */
UENUM()
enum class ENiagaraGpuBufferFormat : uint8
{
	/** 32-bit per channel floating point, range [-3.402823 x 10^38, 3.402823 x 10^38] */
	Float,
	/** 16-bit per channel floating point, range [-65504, 65504] */
	HalfFloat,
	/** 8-bit per channel fixed point, range [0, 1]. */
	UnsignedNormalizedByte,

	Max UMETA(Hidden),
};

UENUM()
enum class ENiagaraGpuSyncMode : uint8
{
	/** Data will not be automatically pushed and could diverge between Cpu & Gpu. */
	None,
	/** Cpu modifications will be pushed to the Gpu. */
	SyncCpuToGpu,
	/** Gpu will continuously push back to the Cpu, this will incur a performance penalty. */
	SyncGpuToCpu,
	/** Gpu will continuously push back to the Cpu and Cpu modifications will be pushed to the Gpu. */
	SyncBoth,
};

UENUM()
enum class ENiagaraMipMapGeneration : uint8
{
	/** Mips will not be created or automatically generated. */
	Disabled,
	/** Mips will be generated after each stage where the interfaces is written to. */
	PostStage,
	/** Mips will be generated after all stages have been run if the interface was written to. */
	PostSimulate,
};

// TODO: Custom will eventually mean that the default value or binding will be overridden by a subgraph default, i.e. expose it to a "Initialize variable" node. 
// TODO: Should we add an "Uninitialized" entry, or is that too much friction? 
UENUM()
enum class ENiagaraDefaultMode : uint8
{
	// Default initialize using a value widget in the Selected Details panel. 
	Value = 0, 
	// Default initialize using a dropdown widget in the Selected Details panel. 
	Binding,   
	// Default initialization is done using a sub-graph.
	Custom,    
	// Fail compilation if this value has not been set previously in the stack.
	FailIfPreviouslyNotSet
};

/** How to handle how Niagara rendered effects should generate motion vectors by default (can still be overridden on a case-by-case basis) */
UENUM()
enum class ENiagaraDefaultRendererMotionVectorSetting : uint8
{
	/**
	 * Motion vectors generated are precise (ideal for motion blur and temporal anti-aliasing).
	 * Requires relevant emitters to store more data per particle, and may affect vertex processing performance.
	 */
	Precise,
	/**
	 * Approximates the motion vectors from current velocity.
	 * Saves memory and performance, but can result in lower quality motion blur and/or anti-aliasing.
	 */
	Approximate
};

/** How a given Niagara renderer should handle motion vector generation. */
UENUM()
enum class ENiagaraRendererMotionVectorSetting : uint8
{
	/** Determines the best method to employ when generating motion vectors (accurate vs. approximate) based on project and renderer settings */
	AutoDetect,
	/** Force motion vectors to be precise for this renderer. */
	Precise,
	/** Force motion vectors to be approximate for this renderer (higher performance, lower particle memory usage). */
	Approximate,
	/** Do not generate motion vectors (i.e. render the object as though it is stationary). */
	Disable
};

UENUM()
enum class ENiagaraSimTarget : uint8
{
	CPUSim,
	GPUComputeSim
};


/** Defines modes for updating the component's age. */
UENUM()
enum class ENiagaraAgeUpdateMode : uint8
{
	/** Update the age using the delta time supplied to the component tick function. */
	TickDeltaTime,
	/** Update the age by seeking to the DesiredAge. To prevent major perf loss, we clamp to MaxClampTime*/
	DesiredAge,
	/** Update the age by tracking changes to the desired age, but when the desired age goes backwards in time,
	or jumps forwards in time by more than a few steps, the system is reset and simulated forward by a single step.
	This mode is useful for continuous effects controlled by sequencer. */
	DesiredAgeNoSeek
};

UENUM()
enum class ENiagaraStatEvaluationType : uint8
{
	Average,
    Maximum,
};

UENUM()
enum class ENiagaraStatDisplayMode : uint8
{
	Percent,
    Absolute,
};

UENUM()
enum class ENiagaraDataSetType : uint8
{
	ParticleData,
	Shared,
	Event,
};


UENUM()
enum class ENiagaraInputNodeUsage : uint8
{
	Undefined = 0 UMETA(Hidden),
	Parameter,
	Attribute,
	SystemConstant,
	TranslatorConstant,
	RapidIterationParameter
};

UENUM()
enum class ENiagaraInterpolatedSpawnMode : uint8
{
	/** This mode does no interpolation and skips the update script on the first frame for newly spawned particles. This means it has better performance, but can lead to visual pops when the update scripts needs to scale things like size or color. */
	NoInterpolation = 0,
	
	/** This mode runs both the spawn and update script, but without any value interpolation for particles. */
	RunUpdateScript,
	
	/** This mode runs both the spawn and update script and performs a partial update at spawn time. This adds some performance overhead, but will produce much smoother spawning for high spawn rates, erratic frame rates and fast moving emitters. */
	Interpolation
};

/**
* Enumerates states a Niagara script can be in.
*/
UENUM()
enum class ENiagaraScriptCompileStatus : uint8
{
	/** Niagara script is in an unknown state. */
	NCS_Unknown,
	/** Niagara script has been modified but not recompiled. */
	NCS_Dirty,
	/** Niagara script tried but failed to be compiled. */
	NCS_Error,
	/** Niagara script has been compiled since it was last modified. */
	NCS_UpToDate,
	/** Niagara script is in the process of being created for the first time. */
	NCS_BeingCreated,
	/** Niagara script has been compiled since it was last modified. There are warnings. */
	NCS_UpToDateWithWarnings,
	/** Niagara script has been compiled for compute since it was last modified. There are warnings. */
	NCS_ComputeUpToDateWithWarnings,
	NCS_MAX,
};

USTRUCT()
struct FNiagaraDataSetID
{
	GENERATED_USTRUCT_BODY()

	FNiagaraDataSetID()
	: Name(NAME_None)
	, Type(ENiagaraDataSetType::Event)
	{}

	FNiagaraDataSetID(FName InName, ENiagaraDataSetType InType)
		: Name(InName)
		, Type(InType)
	{}

	UPROPERTY(EditAnywhere, Category = "Data Set")
	FName Name;

	UPROPERTY()
	ENiagaraDataSetType Type;

	inline bool operator==(const FNiagaraDataSetID& Other)const
	{
		return Name == Other.Name && Type == Other.Type;
	}

	inline bool operator!=(const FNiagaraDataSetID& Other)const
	{
		return !(*this == Other);
	}
};


inline FArchive& operator<<(FArchive& Ar, FNiagaraDataSetID& VarInfo)
{
	Ar << VarInfo.Name << VarInfo.Type;
	return Ar;
}

inline uint32 GetTypeHash(const FNiagaraDataSetID& Var)
{
	return HashCombine(GetTypeHash(Var.Name), (uint32)Var.Type);
}

USTRUCT()
struct FNiagaraDataSetProperties
{
	GENERATED_BODY()
	
	UPROPERTY(VisibleAnywhere, Category = "Data Set")
	FNiagaraDataSetID ID;

	UPROPERTY()
	TArray<FNiagaraVariableBase> Variables;
};

/** Information about an input or output of a Niagara operation node. */
class FNiagaraOpInOutInfo
{
public:
	FName Name;
	FNiagaraTypeDefinition DataType;
	FText FriendlyName;
	FText Description;
	FString Default;
	FString HlslSnippet;

	FNiagaraOpInOutInfo(FName InName, FNiagaraTypeDefinition InType, FText InFriendlyName, FText InDescription, FString InDefault, FString InHlslSnippet = TEXT(""))
		: Name(InName)
		, DataType(InType)
		, FriendlyName(InFriendlyName)
		, Description(InDescription)
		, Default(InDefault)
		, HlslSnippet(InHlslSnippet)
	{

	}
};


/** Struct containing usage information about a script. Things such as whether it reads attribute data, reads or writes events data etc.*/
USTRUCT()
struct FNiagaraScriptDataUsageInfo
{
	GENERATED_BODY()

		FNiagaraScriptDataUsageInfo()
		: bReadsAttributeData(false)
	{}

	/** If true, this script reads attribute data. */
	UPROPERTY()
	bool bReadsAttributeData;
};

// automatically fills in source file and line for a FNiagaraFunctionSignature, so the editor can jump to it on double click
#define NIAGARA_ADD_FUNCTION_SOURCE_INFO(FunctionSignature) { FunctionSignature.SourceFile = __FILE__; FunctionSignature.SourceLine = __LINE__; }

USTRUCT()
struct FNiagaraFunctionSignature
{
	GENERATED_BODY()

	/** Name of the function. */
	UPROPERTY()
	FName Name;
	/** Input parameters to this function. The data stored in the variables is used for default values. */
	UPROPERTY()
	TArray<FNiagaraVariable> Inputs;
	/** Output parameters of this function. */
	UPROPERTY()
	TArray<FNiagaraVariableBase> Outputs;
	/** Id of the owner is this is a member function. */
	UPROPERTY()
	FName OwnerName;
	UPROPERTY()
	uint32 bRequiresContext : 1;
	/** Does this function need an exec pin for control flow because it has internal side effects that be seen by the script VM and could therefore be optimized out? If so, set to true. Default is false. */
	UPROPERTY()
	uint32 bRequiresExecPin : 1;
	/** True if this is the signature for a "member" function of a data interface. If this is true, the first input is the owner. */
	UPROPERTY()
	uint32 bMemberFunction : 1;
	/** Is this function experimental? */
	UPROPERTY()
	uint32 bExperimental : 1;

#if WITH_EDITORONLY_DATA
	/** The message to display when a function is marked experimental. */
	UPROPERTY(EditAnywhere, Category = Script, meta = (EditCondition = "bExperimental", MultiLine = true, SkipForCompileHash = true))
	FText ExperimentalMessage;

	/** Per function version, it is up to the discretion of the function as to what the version means. */
	UPROPERTY()
	uint32 FunctionVersion = 0;

	/**
	* All the inputs here must be wired to something in the graph and will produce a compile error if that's not the case.
	* This is useful if the default value for an input makes no sense and would lead to a failed function call at runtime.
	 */
	UPROPERTY()
	TArray<FNiagaraVariableBase> NoDefaultValueInputs;
#endif

	/** Support running on the CPU. */
	UPROPERTY()
	uint32 bSupportsCPU : 1;
	/** Support running on the GPU. */
	UPROPERTY()
	uint32 bSupportsGPU : 1;

	/** Writes data owned by the data interface.  Any stage using it will be marked as an Output stage. */
	UPROPERTY()
	uint32 bWriteFunction : 1;

	/** Reads data owned by the data interface.  Any stage using it will be marked as an Input stage. */
	UPROPERTY()
	uint32 bReadFunction : 1;

	/** Whether or not this function should show up in normal usage. */
	UPROPERTY()
	uint32 bSoftDeprecatedFunction : 1;

	/** Whether or not this function should be treated as a compile tag. */
	UPROPERTY()
	uint32 bIsCompileTagGenerator : 1;

	/** Hidden functions can not be placed but may be bound and used.  This is useful to hide functionality while developing. */
	UPROPERTY(transient)
	uint32 bHidden : 1;

	/** Bitmask for which scripts are supported for this function. Use UNiagaraScript::MakeSupportedUsageContextBitmask to make the bitmask. */
	UPROPERTY(meta = (Bitmask, BitmaskEnum = "/Script/Niagara.ENiagaraScriptUsage"))
	int32 ModuleUsageBitmask;

	/** Bit mask for misc usage. */
	UPROPERTY(meta = (Bitmask))
	uint16 MiscUsageBitMask;

	/** When using simulation stages and bRequiresContext is true this will be the index of the stage that is associated with the function. */
	UPROPERTY()
	int32 ContextStageIndex;

	/** Number of required inputs. Can be less than the actual number of inputs when using bVariadicInput. INDEX_NONE when not using bVariadicInput, denoting that all inputs are required. */
	UPROPERTY()
	int16 RequiredInputs;

	/** Number of required outputs. Can be less than the actual number of outputs when using bVariadicOutput. INDEX_NONE when not using bVariadicOutput, denoting that all outputs are required. */
	UPROPERTY()
	int16 RequiredOutputs;

	/** Function specifiers verified at bind time. */
	UPROPERTY()
	TMap<FName, FName> FunctionSpecifiers;

	/** Localized description of this node. Note that this is *not* used during the operator == below since it may vary from culture to culture.*/
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (SkipForCompileHash = true))
	FText Description;

	UPROPERTY(meta = (SkipForCompileHash = true))
	TMap<FNiagaraVariableBase, FText> InputDescriptions;

	UPROPERTY(meta = (SkipForCompileHash = true))
	TMap<FNiagaraVariableBase, FText> OutputDescriptions;

	UPROPERTY(meta = (SkipForCompileHash = true))
	FString SourceFile;

	UPROPERTY(meta = (SkipForCompileHash = true))
	int32 SourceLine = 0;
#endif

	FNiagaraFunctionSignature() 
		: bRequiresContext(false)
		, bRequiresExecPin(false)
		, bMemberFunction(false)
		, bExperimental(false)
		, bSupportsCPU(true)
		, bSupportsGPU(true)
		, bWriteFunction(false)
		, bReadFunction(false)
		, bSoftDeprecatedFunction(false)
		, bIsCompileTagGenerator(false)
		, bHidden(false)
		, ModuleUsageBitmask(0)
		, MiscUsageBitMask(0)
		, ContextStageIndex(INDEX_NONE)
		, RequiredInputs(INDEX_NONE)
		, RequiredOutputs(INDEX_NONE)
	{
	}

	FNiagaraFunctionSignature(FName InName, TConstArrayView<FNiagaraVariable> InInputs, TConstArrayView<FNiagaraVariableBase> InOutputs, bool bInRequiresContext, bool bInMemberFunction)
		: Name(InName)
		, Inputs(InInputs)
		, Outputs(InOutputs)
		, bRequiresContext(bInRequiresContext)
		, bRequiresExecPin(false)
		, bMemberFunction(bInMemberFunction)
		, bExperimental(false)
		, bSupportsCPU(true)
		, bSupportsGPU(true)
		, bWriteFunction(false)
		, bReadFunction(false)
		, bSoftDeprecatedFunction(false)
		, bIsCompileTagGenerator(false)
		, bHidden(false)
		, ModuleUsageBitmask(0)
		, MiscUsageBitMask(0)
		, ContextStageIndex(INDEX_NONE)
		, RequiredInputs(INDEX_NONE)
		, RequiredOutputs(INDEX_NONE)
	{
	}

	bool operator!=(const FNiagaraFunctionSignature& Other) const
	{
		return !(*this == Other);
	}

	bool operator==(const FNiagaraFunctionSignature& Other) const
	{
		bool bFunctionSpecifiersEqual = [&]()
		{
			if (Other.FunctionSpecifiers.Num() != FunctionSpecifiers.Num())
			{
				return false;
			}
			for (const TTuple<FName, FName>& Specifier : FunctionSpecifiers)
			{
				if (Other.FunctionSpecifiers.FindRef(Specifier.Key) != Specifier.Value)
				{
					return false;
				}
			}
			return true;
		}();

		bool bMatchingVariadics = true;
		if (VariadicInput() || VariadicOutput())
		{
			bMatchingVariadics &= Other.Inputs.Num() == Inputs.Num() && RequiredInputs == Other.RequiredInputs;
			bMatchingVariadics &= Other.Outputs.Num() == Outputs.Num() && RequiredOutputs == Other.RequiredOutputs;
			if (bMatchingVariadics && RequiredInputs != INDEX_NONE)
			{
				for (int32 i = RequiredInputs; i < Inputs.Num(); ++i)
				{
					bMatchingVariadics &= Inputs[i] == Other.Inputs[i];
				}
			}
			if (bMatchingVariadics && RequiredOutputs != INDEX_NONE)
			{
				for (int32 i = RequiredOutputs; i < Outputs.Num(); ++i)
				{
					bMatchingVariadics &= Outputs[i] == Other.Outputs[i];
				}
			}
		}
		return EqualsIgnoringSpecifiers(Other) && bFunctionSpecifiersEqual && bMatchingVariadics;
	}

	bool EqualsIgnoringSpecifiers(const FNiagaraFunctionSignature& Other) const
	{
		bool bMatches = Name.ToString().Equals(Other.Name.ToString());
		
		//Allow divergent inputs for variadic input functions.
		if (VariadicInput() && bMatches)
		{
			if (Other.Inputs.Num() >= RequiredInputs)
			{
				for (int32 i = 0; i < RequiredInputs; ++i)
				{
					bMatches &= Inputs[i] == Other.Inputs[i];
				}
			}
			else
			{
				bMatches = false;
			}
		}
		else
		{
			bMatches &= Inputs == Other.Inputs;
		}
		//Allow divergent outputs for variadic output functions.
		if (VariadicOutput() && bMatches)
		{
			if (Other.Inputs.Num() >= RequiredInputs)
			{
				for (int32 i = 0; i < RequiredOutputs; ++i)
				{
					bMatches &= Outputs[i] == Other.Outputs[i];
				}
			}
			else
			{
				bMatches = false;
			}
		}
		else
		{
			bMatches &= Outputs == Other.Outputs;
		}
		bMatches &= bRequiresContext == Other.bRequiresContext;
		bMatches &= bRequiresExecPin == Other.bRequiresExecPin;
		bMatches &= bMemberFunction == Other.bMemberFunction;
		bMatches &= OwnerName == Other.OwnerName;
		bMatches &= ContextStageIndex == Other.ContextStageIndex;
		bMatches &= bIsCompileTagGenerator == Other.bIsCompileTagGenerator;
		return bMatches;
	}

	FString GetNameString() const { return Name.ToString(); }

	void AddInput(const FNiagaraVariable& InputVar, const FText& Tooltip = FText())
	{
		Inputs.Add(InputVar);
	#if WITH_EDITORONLY_DATA
		if (!Tooltip.IsEmpty())
		{
			InputDescriptions.Add(InputVar, Tooltip);
		}
	#endif
	}
	
	template<typename T>
	void AddInputWithDefault(FNiagaraVariable InputVar, const T& Default, const FText& Tooltip = FText())
	{
		Inputs.Add_GetRef(InputVar).SetData((uint8*)&Default);
#if WITH_EDITORONLY_DATA
		if (!Tooltip.IsEmpty())
		{
			InputDescriptions.Add(InputVar, Tooltip);
		}
#endif
	}

	void AddInputWithoutDefault(const FNiagaraVariable& InputVar, const FText& Tooltip = FText())
	{
		Inputs.Add(InputVar);
#if WITH_EDITORONLY_DATA
		if (!Tooltip.IsEmpty())
		{
			InputDescriptions.Add(InputVar, Tooltip);
		}
		NoDefaultValueInputs.AddUnique(InputVar);
#endif
	}

	void AddOutput(const FNiagaraVariable& OutputVar, const FText& Tooltip = FText())
	{
		Outputs.Add(OutputVar);
	#if WITH_EDITORONLY_DATA
		if (!Tooltip.IsEmpty())
		{
			OutputDescriptions.Add(OutputVar, Tooltip);
		}
	#endif
	}

	void SetFunctionVersion(uint32 Version)
	{
	#if WITH_EDITORONLY_DATA
		FunctionVersion = Version;
	#endif
	}

	void SetInputDescription(const FNiagaraVariableBase& Variable, const FText& Desc)
	{
#if WITH_EDITORONLY_DATA
		InputDescriptions.FindOrAdd(Variable) = Desc;
#endif
	}

	void SetOutputDescription(const FNiagaraVariableBase& Variable, const FText& Desc)
	{
#if WITH_EDITORONLY_DATA
		OutputDescriptions.FindOrAdd(Variable) = Desc;
#endif
	}

	void SetDescription(const FText& Desc)
	{
	#if WITH_EDITORONLY_DATA
		Description = Desc;
	#endif
	}
	
	FText GetDescription() const
	{
	#if WITH_EDITORONLY_DATA
		return Description;
	#else
		return FText::FromName(Name);
	#endif
	}
	
	bool IsValid()const { return Name != NAME_None && (Inputs.Num() > 0 || Outputs.Num() > 0); }

	bool VariadicInput()const { return RequiredInputs != INDEX_NONE; }
	bool VariadicOutput()const { return RequiredOutputs != INDEX_NONE; }

	int32 VariadicInputStartIndex()const { return VariadicInput() ? NumRequiredInputs() + (bRequiresExecPin ? 1 : 0) : INDEX_NONE; }
	int32 VariadicOutputStartIndex()const { return VariadicOutput() ? NumRequiredOutputs() + (bRequiresExecPin ? 1 : 0) : INDEX_NONE; }

	int32 NumRequiredInputs()const { return RequiredInputs == INDEX_NONE ? Inputs.Num() : RequiredInputs; }
	int32 NumOptionalInputs()const { return Inputs.Num() - NumRequiredInputs(); }

	int32 NumRequiredOutputs()const { return RequiredOutputs == INDEX_NONE ? Outputs.Num() : RequiredOutputs; }
	int32 NumOptionalOutputs()const { return Outputs.Num() - NumRequiredOutputs(); }

#if WITH_EDITORONLY_DATA
	TArray<FNiagaraVariableBase> GetInputs() const
	{
		TArray<FNiagaraVariableBase> AllInputs;
		AllInputs.Append(Inputs);
		return AllInputs;
	}
	TConstArrayView<FNiagaraVariableBase> GetOutputs() const { return Outputs; }
#endif

	NIAGARA_API void GetVariadicInputs(TArray<FNiagaraVariableBase>& OutVariadicInputs)const;
	NIAGARA_API void GetVariadicOutputs(TArray<FNiagaraVariableBase>& OutVariadicOutputs)const;
};

USTRUCT()
struct FNiagaraScriptUObjectCompileInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FNiagaraVariableBase Variable;

	UPROPERTY()
	TObjectPtr<class UObject> Object;

	UPROPERTY()
	FSoftObjectPath ObjectPath;

	UPROPERTY()
	FName RegisteredParameterMapRead;

	UPROPERTY()
	TArray<FName> RegisteredParameterMapWrites;
};

USTRUCT()
struct FNiagaraResolvedUObjectInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FName ReadVariableName;

	UPROPERTY()
	FNiagaraVariableBase ResolvedVariable;

	UPROPERTY()
	TObjectPtr<class UObject> Object;

	//UPROPERTY()
	//int32 UserPtrIdx = INDEX_NONE;
};

USTRUCT()
struct FNiagaraExternalUObjectInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FNiagaraVariableBase Variable;

	UPROPERTY()
	FName ExternalName;

	[[nodiscard]] FNiagaraVariableBase GetExternalVariable() const { return FNiagaraVariableBase(Variable.GetType(), ExternalName); }
};

USTRUCT()
struct FNiagaraScriptDataInterfaceInfo
{
	GENERATED_USTRUCT_BODY()
public:
	FNiagaraScriptDataInterfaceInfo()
		: DataInterface(nullptr)
		, Name(NAME_None)
		, UserPtrIdx(INDEX_NONE)
	{

	}

	UPROPERTY()
	TObjectPtr<class UNiagaraDataInterface> DataInterface;
	
	UPROPERTY()
	FName Name;

	UPROPERTY()
	FName CompileName;

	/** Index of the user pointer for this data interface. */
	UPROPERTY()
	int32 UserPtrIdx;

	UPROPERTY()
	FNiagaraTypeDefinition Type;

	UPROPERTY()
	FName RegisteredParameterMapRead;

	UPROPERTY()
	FName RegisteredParameterMapWrite;

	UPROPERTY()
	FString SourceEmitterName;

	NIAGARA_API bool IsUserDataInterface() const;
};

USTRUCT()
struct FNiagaraResolvedUserDataInterfaceBinding
{
	GENERATED_BODY()

	UPROPERTY()
	int32 UserParameterStoreDataInterfaceIndex;

	UPROPERTY()
	int32 ScriptParameterStoreDataInterfaceIndex;

	FNiagaraResolvedUserDataInterfaceBinding()
		: UserParameterStoreDataInterfaceIndex(INDEX_NONE)
		, ScriptParameterStoreDataInterfaceIndex(INDEX_NONE)
	{
	}

	FNiagaraResolvedUserDataInterfaceBinding(int32 InUserParameterStoreDataInterfaceIndex, int32 InScriptParameterStoreDataInterfaceIndex)
		: UserParameterStoreDataInterfaceIndex(InUserParameterStoreDataInterfaceIndex)
		, ScriptParameterStoreDataInterfaceIndex(InScriptParameterStoreDataInterfaceIndex)
	{
	}
};

USTRUCT()
struct FNiagaraScriptResolvedDataInterfaceInfo
{
	GENERATED_USTRUCT_BODY();

	FNiagaraScriptResolvedDataInterfaceInfo()
		: bIsInternal(false)
		, ResolvedDataInterface(nullptr)
		, UserPtrIdx(-1)
	{
	}

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FName CompileName;

	UPROPERTY()
	FString ResolvedSourceEmitterName;

	UPROPERTY()
	FNiagaraVariableBase ResolvedVariable;

	UPROPERTY()
	FNiagaraVariableBase ParameterStoreVariable;

	UPROPERTY()
	bool bIsInternal;

	UPROPERTY()
	TObjectPtr<UNiagaraDataInterface> ResolvedDataInterface;

	UPROPERTY()
	int32 UserPtrIdx;

	NIAGARA_API bool NeedsPerInstanceBinding() const;
};

USTRUCT()
struct FNiagaraScriptDataInterfaceCompileInfo
{
	GENERATED_USTRUCT_BODY()
public:
	FNiagaraScriptDataInterfaceCompileInfo()
		: Name(NAME_None)
		, UserPtrIdx(INDEX_NONE)
		, bIsPlaceholder(false)
	{

	}

	UPROPERTY()
	FName Name;

	/** Index of the user pointer for this data interface. */
	UPROPERTY()
	int32 UserPtrIdx;

	UPROPERTY()
	FNiagaraTypeDefinition Type;

	// Removed from cooked builds, if we need to add this back the TMap<FName, FName> FunctionSpecifiers should be replaced with an array
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FNiagaraFunctionSignature> RegisteredFunctions;
#endif

	UPROPERTY()
	FName RegisteredParameterMapRead;

	UPROPERTY()
	FName RegisteredParameterMapWrite;

	UPROPERTY()
	bool bIsPlaceholder;

	UPROPERTY()
	FString SourceEmitterName;

	/** Would this data interface work on the target execution type? Only call this on the game thread.*/
	NIAGARA_API bool CanExecuteOnTarget(ENiagaraSimTarget SimTarget) const;

	/** Note that this is the CDO for this type of data interface, as we often cannot guarantee that the same instance of the data interface we compiled with is the one the user ultimately executes.  Only call this on the game thread.*/
	NIAGARA_API UNiagaraDataInterface* GetDefaultDataInterface() const;

	NIAGARA_API bool NeedsPerInstanceBinding() const;

	NIAGARA_API bool MatchesClass(const UClass* InClass) const;
};

USTRUCT()
struct FNiagaraStatScope
{
	GENERATED_USTRUCT_BODY();

	FNiagaraStatScope() {}
	FNiagaraStatScope(FName InFullName, FName InFriendlyName):FullName(InFullName), FriendlyName(InFriendlyName){}

	UPROPERTY()
	FName FullName;

	UPROPERTY()
	FName FriendlyName;

	bool operator==(const FNiagaraStatScope& Other) const { return FullName == Other.FullName; }
};

USTRUCT()
struct FVMFunctionSpecifier
{
	GENERATED_USTRUCT_BODY();

	FVMFunctionSpecifier() {}
	explicit FVMFunctionSpecifier(FName InKey, FName InValue) : Key(InKey), Value(InValue) {}

	UPROPERTY()
	FName Key;

	UPROPERTY()
	FName Value;
};

USTRUCT()
struct FVMExternalFunctionBindingInfo
{
	GENERATED_USTRUCT_BODY();

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FName OwnerName;

	UPROPERTY()
	TArray<bool> InputParamLocations;

	UPROPERTY()
	int32 NumOutputs = 0;

	UPROPERTY()
	TArray<FVMFunctionSpecifier> FunctionSpecifiers;

	UPROPERTY()
	TArray<FNiagaraVariableBase> VariadicInputs;

	UPROPERTY()
	TArray<FNiagaraVariableBase> VariadicOutputs;

	inline int32 GetNumInputs() const { return InputParamLocations.Num(); }
	inline int32 GetNumOutputs() const { return NumOutputs; }

	const FVMFunctionSpecifier* FindSpecifier(const FName& Key) const
	{
		return FunctionSpecifiers.FindByPredicate([&](const FVMFunctionSpecifier& v) -> bool { return v.Key == Key; });
	}

	bool Serialize(FArchive& Ar);

#if WITH_EDITORONLY_DATA
private:
	UPROPERTY()
	TMap<FName, FName> Specifiers_DEPRECATED;
#endif
};

template<>
struct TStructOpsTypeTraits<FVMExternalFunctionBindingInfo> : public TStructOpsTypeTraitsBase2<FVMExternalFunctionBindingInfo>
{
	enum
	{
		WithSerializer = true,
	};
};

/**
Helper for reseting/reinitializing Niagara systems currently active when they are being edited. 
Can be used inside a scope with Systems being reinitialized on destruction or you can store the context and use CommitUpdate() to trigger reinitialization.
For example, this can be split between PreEditChange and PostEditChange to ensure problematic data is not modified during execution of a system.
This can be made a UPROPERTY() to ensure safey in cases where a GC could be possible between Add() and CommitUpdate().
*/
USTRUCT()
struct FNiagaraSystemUpdateContext
{
	GENERATED_BODY()

	template <typename AddableType> // Must be handled by Add function
	FNiagaraSystemUpdateContext(AddableType&& Addable, bool bReInit,
		bool bInDestroyOnAdd = false, bool bInOnlyActive = false, bool bInDestroySystemSim = true)
		: bDestroyOnAdd(bInDestroyOnAdd), bOnlyActive(bInOnlyActive), bDestroySystemSim(bInDestroySystemSim)
	{
		Add(Forward<AddableType>(Addable), bReInit);
	}
	FNiagaraSystemUpdateContext() :bDestroyOnAdd(false), bOnlyActive(false), bDestroySystemSim(true) { }
	FNiagaraSystemUpdateContext(FNiagaraSystemUpdateContext& Other) = delete;

	NIAGARA_API ~FNiagaraSystemUpdateContext();

	void SetDestroyOnAdd(bool bInDestroyOnAdd) { bDestroyOnAdd = bInDestroyOnAdd; }
	void SetOnlyActive(bool bInOnlyActive) { bOnlyActive = bInOnlyActive; }
	void SetDestroySystemSim(bool bInDestroySystemSim) { bDestroySystemSim = bInDestroySystemSim; }

	NIAGARA_API void AddSoloComponent(UNiagaraComponent* Component, bool bReInit);
	NIAGARA_API void Add(const UNiagaraSystem* System, bool bReInit);
#if WITH_EDITORONLY_DATA
	NIAGARA_API void Add(const FVersionedNiagaraEmitterBase& Emitter, bool bReInit);
	NIAGARA_API void Add(const FVersionedNiagaraEmitter& Emitter, bool bReInit);
	NIAGARA_API void Add(const UNiagaraScript* Script, bool bReInit);
	NIAGARA_API void Add(const UNiagaraParameterCollection* Collection, bool bReInit);
#endif

	/** Adds all currently active systems.*/
	NIAGARA_API void AddAll(bool bReInit);
	
	/** Handles any pending reinits or resets of system instances in this update context. */
	NIAGARA_API void CommitUpdate();

	DECLARE_DELEGATE_OneParam(FCustomWorkDelegate, UNiagaraComponent*);
	FCustomWorkDelegate& GetPreWork(){ return PreWork; }
	FCustomWorkDelegate& GetPostWork() { return PostWork; }

private:
	template <typename SystemMatchesFunc>
	void AddComponentsWithMatchingSystem(bool bReinit, SystemMatchesFunc&& SystemMatches);
	void AddInternal(class UNiagaraComponent* Comp, bool bReInit, bool bAllowDestroySystemSim);

	UPROPERTY(transient)
	TArray<TObjectPtr<UNiagaraComponent>> ComponentsToReset;
	UPROPERTY(transient)
	TArray<TObjectPtr<UNiagaraComponent>> ComponentsToReInit;
	UPROPERTY(transient)
	TArray<TObjectPtr<UNiagaraComponent>> ComponentsToNotifySimDestroy;
	UPROPERTY(transient)
	TArray<TObjectPtr<UNiagaraComponent>> ComponentsToDestroyInstance;
	UPROPERTY(transient)
	TArray<TObjectPtr<UNiagaraSystem>> SystemSimsToDestroy;
	UPROPERTY(transient)
	TArray<TObjectPtr<UNiagaraSystem>> SystemSimsToRecache;

	bool bDestroyOnAdd;
	bool bOnlyActive;
	bool bDestroySystemSim;
	//TODO: When we allow component less systems we'll also want to find and reset those.

	/** Delegate called before a component is added to the context for update. */
	FCustomWorkDelegate PreWork;
	
	/** Delegate called before after a component has been updated. */
	FCustomWorkDelegate PostWork;
};

/** Defines different usages for a niagara script. */
UENUM()
enum class ENiagaraScriptUsage : uint8
{
	/** The script defines a function for use in modules. */
	Function,
	/** The script defines a module for use in particle, emitter, or system scripts. */
	Module,
	/** The script defines a dynamic input for use in particle, emitter, or system scripts. */
	DynamicInput,
	/** The script is called when spawning particles. */
	ParticleSpawnScript,
	/** Particle spawn script that handles intra-frame spawning and also pulls in the update script. */
	ParticleSpawnScriptInterpolated UMETA(Hidden),
	/** The script is called to update particles every frame. */
	ParticleUpdateScript,
	/** The script is called to update particles in response to an event. */
	ParticleEventScript ,
	/** The script is called as a particle simulation stage. */
	ParticleSimulationStageScript,
	/** The script is called to update particles on the GPU. */
	ParticleGPUComputeScript UMETA(Hidden),
	/** The script is called once when the emitter spawns. */
	EmitterSpawnScript,
	/** The script is called every frame to tick the emitter. */
	EmitterUpdateScript ,
	/** The script is called once when the system spawns. */
	SystemSpawnScript ,
	/** The script is called every frame to tick the system. */
	SystemUpdateScript,
};

/** Defines common bit masks for script usage */
namespace ENiagaraScriptUsageMask
{
	enum Type
	{
		System =
			(1 << int32(ENiagaraScriptUsage::Module)) |
			(1 << int32(ENiagaraScriptUsage::DynamicInput)) |
			(1 << int32(ENiagaraScriptUsage::Function)) |
			(1 << int32(ENiagaraScriptUsage::SystemSpawnScript)) |
			(1 << int32(ENiagaraScriptUsage::SystemUpdateScript)),
		Emitter =
			(1 << int32(ENiagaraScriptUsage::Module)) |
			(1 << int32(ENiagaraScriptUsage::DynamicInput)) |
			(1 << int32(ENiagaraScriptUsage::Function)) |
			(1 << int32(ENiagaraScriptUsage::EmitterSpawnScript)) |
			(1 << int32(ENiagaraScriptUsage::EmitterUpdateScript)),
		Particle =
			(1 << int32(ENiagaraScriptUsage::Module)) |
			(1 << int32(ENiagaraScriptUsage::DynamicInput)) |
			(1 << int32(ENiagaraScriptUsage::Function)) |
			(1 << int32(ENiagaraScriptUsage::ParticleSpawnScript)) |
			(1 << int32(ENiagaraScriptUsage::ParticleSpawnScriptInterpolated)) |
			(1 << int32(ENiagaraScriptUsage::ParticleUpdateScript)) |
			(1 << int32(ENiagaraScriptUsage::ParticleEventScript)) |
			(1 << int32(ENiagaraScriptUsage::ParticleSimulationStageScript)) |
			(1 << int32(ENiagaraScriptUsage::ParticleGPUComputeScript)),
	};
}

/** Misc usage flag for script to transmit informations. Limited to 16 bits, see MiscUsageBitMaskType. */
enum class ENiagaraScriptMiscUsageMask : uint16
{
	/** This Niagara script is sampling partial depth scene texture. So it should be rendered in the secondary depth pass. */
	UsesPartialDepthCollisionQuery = 1<<0,
};

/** Defines different usages for a niagara script. */
UENUM()
enum class ENiagaraCompileUsageStaticSwitch : uint8
{
	/** The script is called during the spawn phase. */
	Spawn,
	/** The script is called during the update phase. */
	Update,
	/** The script is called in an event context. */
	Event,
	/** The script is called as part of a simulation stage. */
	SimulationStage,
	/** The default value if the compiler cannot map the compilation context. */
	Default,
};

/** Defines different execution contexts for a niagara script. */
UENUM()
enum class ENiagaraScriptContextStaticSwitch : uint8
{
	/** The script is called in a system context. */
	System,
    /** The script is called in a emitter context. */
    Emitter,
	/** The script is called in a particle context. */
	Particle,
};

UENUM()
enum class ENiagaraScriptGroup : uint8
{
	Particle = 0,
	Emitter,
	System,
	Max
};

UENUM()
enum ENiagaraBindingSource : int
{
	ImplicitFromSource = 0,
	ExplicitParticles,
	ExplicitEmitter,
	ExplicitSystem,
	ExplicitUser,
	MaxBindingSource
};

USTRUCT()
struct FNiagaraScriptDataKey
{
	GENERATED_BODY();

	FNiagaraScriptDataKey()
		: ScriptUsage(ENiagaraScriptUsage::Function)
	{
	}

	FNiagaraScriptDataKey(const FGuid& InEmitterHandleId, ENiagaraScriptUsage InScriptUsage, const FGuid& InScriptUsageId)
		: EmitterHandleId(InEmitterHandleId)
		, ScriptUsage(InScriptUsage)
		, ScriptUsageId(InScriptUsageId)
	{
	}

	NIAGARA_API FNiagaraScriptDataKey(const FNiagaraEmitterHandle* EmitterHandle, const UNiagaraScript& Script);

	FORCEINLINE bool operator==(const FNiagaraScriptDataKey& Other) const
	{
		return EmitterHandleId == Other.EmitterHandleId && 
			ScriptUsage == Other.ScriptUsage &&
			ScriptUsageId == Other.ScriptUsageId;
	}

	friend uint32 GetTypeHash(const FNiagaraScriptDataKey& Ref)
	{
		uint32 Hash = GetTypeHash(Ref.EmitterHandleId);
		Hash = HashCombine(Hash, GetTypeHash(Ref.ScriptUsage));
		Hash = HashCombine(Hash, GetTypeHash(Ref.ScriptUsageId));
		return Hash;
	}

	UPROPERTY()
	FGuid EmitterHandleId;

	UPROPERTY()
	ENiagaraScriptUsage ScriptUsage;

	UPROPERTY()
	FGuid ScriptUsageId;
};

#if STATS

template<typename ElementType> class TSimpleRingBuffer;

struct FStatExecutionTimer
{
	FStatExecutionTimer();

	void AddTiming(float NewTiming);
	
	TArray<float> CapturedTimings;
	
private:
	int CurrentIndex = 0;
};

/** Combines all stat reporting and evaluation of niagara instances (emitter or system).
 * This is then used by the SNiagaraStackRowPerfWidget to display the data in the ui.
 */
struct FNiagaraStatDatabase
{	
	typedef TTuple<uint64, ENiagaraScriptUsage> FStatReportKey;

	/* Used by emitter and system instances to add the recorded data of a frame to this emitter's data store. */
	NIAGARA_API void AddStatCapture(FStatReportKey ReportKey, TMap<TStatIdData const*, float> CapturedData);

	/* Removes all captured stats. */
	NIAGARA_API void ClearStatCaptures();

	/* Returns the average runtime cost of a specific module call inside the script for the given usage. Returns 0 if no data was found. */
	NIAGARA_API float GetRuntimeStat(FName StatName, ENiagaraScriptUsage Usage, ENiagaraStatEvaluationType EvaluationType);

	/* Returns the average runtime cost of a script for the given usage. Returns 0 if no data was recorded for that usage. */
	NIAGARA_API float GetRuntimeStat(ENiagaraScriptUsage Usage, ENiagaraStatEvaluationType EvaluationType);

	/* Returns the names of all captures stat data points. Useful for debugging and to dump the stat data. */
	NIAGARA_API TMap<ENiagaraScriptUsage, TSet<FName>> GetAvailableStatNames();

	NIAGARA_API void Init();

private:
	/** The captured runtime stat data. The first key is a combination of reporter handle and script usage, the second key is the stat id which correlates to a single recorded scope. */
	TMap<FStatReportKey, TMap<TStatIdData const*, FStatExecutionTimer>> StatCaptures;

	FCriticalSection* GetCriticalSection() const;
	TSharedPtr<FCriticalSection> CriticalSection;
};
#endif

/** Defines all you need to know about a variable.*/
USTRUCT()
struct FNiagaraVariableInfo
{
	GENERATED_USTRUCT_BODY();

	FNiagaraVariableInfo() : DataInterface(nullptr) {}

	UPROPERTY()
	FNiagaraVariable Variable;

	UPROPERTY()
	FText Definition;

	UPROPERTY()
	TObjectPtr<UNiagaraDataInterface> DataInterface;
};

/** This enum decides how a renderer will attempt to process the incoming data from the stack.*/
UENUM()
enum class ENiagaraRendererSourceDataMode : uint8
{
	/** The renderer will draw particle data, but can potentially pull in data from the Emitter/User/or System namespaces when drawing each Particle.*/
	Particles = 0,
	/** The renderer will draw only one element per Emitter. It can only pull in data from Emitter/User/or System namespaces when drawing the single element. */
	Emitter
};

USTRUCT()
struct FNiagaraVariableAttributeBinding
{
	GENERATED_USTRUCT_BODY();

	FNiagaraVariableAttributeBinding() : BindingSourceMode(ENiagaraBindingSource::ImplicitFromSource), bBindingExistsOnSource(false), bIsCachedParticleValue(true) {}

	bool IsParticleBinding() const { return bIsCachedParticleValue; }
	bool DoesBindingExistOnSource() const { return bBindingExistsOnSource; }
	bool CanBindToHostParameterMap() const { return bBindingExistsOnSource && !bIsCachedParticleValue; }
	NIAGARA_API void SetValue(const FName& InValue, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	NIAGARA_API void SetAsPreviousValue(const FNiagaraVariableBase& Src, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	NIAGARA_API void SetAsPreviousValue(const FNiagaraVariableAttributeBinding& Src, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	NIAGARA_API void CacheValues(const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	NIAGARA_API bool RenameVariableIfMatching(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	NIAGARA_API bool Matches(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode) const;

	UE_DEPRECATED(5.7, "Please change to use FVersionedNiagaraEmitterBase from FVersionedNiagaraEmitter")
	NIAGARA_API void SetValue(const FName& InValue, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	UE_DEPRECATED(5.7, "Please change to use FVersionedNiagaraEmitterBase from FVersionedNiagaraEmitter")
	NIAGARA_API void SetAsPreviousValue(const FNiagaraVariableBase& Src, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	UE_DEPRECATED(5.7, "Please change to use FVersionedNiagaraEmitterBase from FVersionedNiagaraEmitter")
	NIAGARA_API void SetAsPreviousValue(const FNiagaraVariableAttributeBinding& Src, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	UE_DEPRECATED(5.7, "Please change to use FVersionedNiagaraEmitterBase from FVersionedNiagaraEmitter")
	NIAGARA_API void CacheValues(const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	UE_DEPRECATED(5.7, "Please change to use FVersionedNiagaraEmitterBase from FVersionedNiagaraEmitter")
	NIAGARA_API bool RenameVariableIfMatching(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	UE_DEPRECATED(5.7, "Please change to use FVersionedNiagaraEmitterBase from FVersionedNiagaraEmitter")
	NIAGARA_API bool Matches(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode) const;
	UE_DEPRECATED(5.7, "Please change to use FVersionedNiagaraEmitterBase from FVersionedNiagaraEmitter")
	NIAGARA_API void ResetToDefault(const FNiagaraVariableAttributeBinding& InOther, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);

#if WITH_EDITORONLY_DATA
	const FName& GetName() const { return CachedDisplayName; }

	[[nodiscard]] NIAGARA_API FString GetDefaultValueString() const;
#endif
	const FNiagaraVariableBase& GetParamMapBindableVariable() const { return ParamMapVariable; }
	[[nodiscard]] FNiagaraVariableBase GetDataSetBindableVariable() const { return FNiagaraVariableBase(ParamMapVariable.GetType(), DataSetName); }
	const FNiagaraTypeDefinition& GetType() const { return ParamMapVariable.GetType(); }
	ENiagaraBindingSource GetBindingSourceMode() const { return BindingSourceMode; }

	bool IsValid() const { return !DataSetName.IsNone() && ParamMapVariable.GetType().IsValid(); }
	
	template<typename T>
	UE_DEPRECATED(5.7, "To reduce memory usage of renderers the default value is no longer stored inside the attribute binding.  Please use FNiagaraConstants::GetAttributeWithDefaultValue to get the value.")
	T GetDefaultValue() const
	{
		return T{};
	}

	void NIAGARA_API Setup(const FNiagaraVariableBase& InRootVar, const FNiagaraVariable& InDefaultValue, ENiagaraRendererSourceDataMode InSourceMode = ENiagaraRendererSourceDataMode::Particles);

	void NIAGARA_API PostLoad(ENiagaraRendererSourceDataMode InSourceMode);
	void NIAGARA_API Dump() const;

	void NIAGARA_API ResetToDefault(const FNiagaraVariableAttributeBinding& InOther, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	bool NIAGARA_API MatchesDefault(const FNiagaraVariableAttributeBinding& InOther, ENiagaraRendererSourceDataMode InSourceMode) const;

private:
	[[nodiscard]] FNiagaraVariableBase GetRootVariable() const { return FNiagaraVariableBase(ParamMapVariable.GetType(), RootName); }

protected:
	/** The namespace and default value explicitly set by the user. If meant to be derived from the source mode, it will be without a namespace.*/
	UPROPERTY()
	FName RootName;

	/** The fully expressed namespace for the variable. If an emitter namespace, this will include the Emitter's unique name.*/
	UPROPERTY()
	FNiagaraVariableBase ParamMapVariable;

	/** The version of the namespace to be found in an attribute table lookup. I.e. without Particles or Emitter.*/
	UPROPERTY()
	FName DataSetName;

#if WITH_EDITORONLY_DATA
	/** Old variable brought in from previous setup. Generally ignored other than postload work.*/
	UPROPERTY()
	FNiagaraVariable BoundVariable;

	UPROPERTY(transient)
	FNiagaraVariable DefaultValue;

	UPROPERTY()
	FName CachedDisplayName;
#endif

	/** Captures the state of the namespace when the variable is set. Allows us to make later decisions about how to reconstititue the namespace.*/
	UPROPERTY()
	TEnumAsByte<ENiagaraBindingSource> BindingSourceMode;
	
	/** Determine if this varible is accessible by the associated emitter passed into CacheValues.*/
	UPROPERTY()
	uint8 bBindingExistsOnSource : 1; 

	/** When CacheValues is called, was this a particle attribute?*/
	UPROPERTY()
	uint8 bIsCachedParticleValue : 1;

#if WITH_EDITORONLY_DATA
	/** The version of the namespace to be found in an attribute table lookup. I.e. without Particles or Emitter.*/
	UPROPERTY()
	FNiagaraVariable DataSetVariable_DEPRECATED;

	UPROPERTY()
	FNiagaraVariable RootVariable_DEPRECATED;
#endif
};

USTRUCT()
struct FNiagaraMaterialAttributeBinding
{
	GENERATED_USTRUCT_BODY();

	UPROPERTY(EditAnywhere, Category = "Variable")
	FName MaterialParameterName;

	UPROPERTY(EditAnywhere, Category = "Variable")
	FNiagaraVariableBase NiagaraVariable;

	UPROPERTY()
	FNiagaraVariableBase ResolvedNiagaraVariable;

	UPROPERTY(EditAnywhere, Category = "Variable")
	FNiagaraVariableBase NiagaraChildVariable;

	NIAGARA_API void CacheValues(const UNiagaraEmitterBase* InEmitter);
	NIAGARA_API const FNiagaraVariableBase& GetParamMapBindableVariable() const;

	NIAGARA_API bool RenameVariableIfMatching(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const UNiagaraEmitterBase* InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	NIAGARA_API bool Matches(const FNiagaraVariableBase& OldVariable, const UNiagaraEmitterBase* InEmitter, ENiagaraRendererSourceDataMode InSourceMode);
};

USTRUCT()
struct FNiagaraVariableDataInterfaceBinding
{
	GENERATED_USTRUCT_BODY();

	FNiagaraVariableDataInterfaceBinding() {}
	FNiagaraVariableDataInterfaceBinding(const FNiagaraVariable& InVar) : BoundVariable(InVar)
	{
		ensure(InVar.IsDataInterface() == true);
	}

	UPROPERTY()
	FNiagaraVariable BoundVariable;

};

/** Primarily a wrapper around an FName to be used for customizations in the Selected Details panel 
    to select a default binding to initialize module inputs. The customization implementation
    is FNiagaraScriptVariableBindingCustomization. */
USTRUCT()
struct FNiagaraScriptVariableBinding
{
	GENERATED_USTRUCT_BODY();

	FNiagaraScriptVariableBinding() {}
	FNiagaraScriptVariableBinding(const FNiagaraVariable& InVar) : Name(InVar.GetName())
	{
		
	}
	FNiagaraScriptVariableBinding(const FName& InName) : Name(InName)
	{
		
	}

	UPROPERTY(EditAnywhere, Category = "Variable")
	FName Name;

	FName GetName() const { return Name; }
	void SetName(FName InName) { Name = InName; }
	bool IsValid() const { return Name != NAME_None; }
};

struct FNiagaraAliasContext
{
	/** Defines different modes which can be used to split rapid iteration parameter constant names. */
	enum class ERapidIterationParameterMode
	{
		/** Rapid iteration parameters will be from the system. */
		SystemScript,
		/** Rapid iteration parameters will be from an emitter or particle script. */
		EmitterOrParticleScript,
		/** Rapid iteration parameters will not be handled. NOTE: Using this mode will cause an assert if the context is used with a rapid iteration parameter constant name. */
		None,
		/** Special case of evaluating static variables which can include variables from both system scripts and particle/emitter scripts */
		StaticVariables,
	};

	FNiagaraAliasContext(ERapidIterationParameterMode InRapidIterationParameterMode = ERapidIterationParameterMode::None)
		: RapidIterationParameterMode(InRapidIterationParameterMode)
	{
	}

	FNiagaraAliasContext(ENiagaraScriptUsage InScriptUsage)
		: RapidIterationParameterMode(InScriptUsage == ENiagaraScriptUsage::SystemSpawnScript || InScriptUsage == ENiagaraScriptUsage::SystemUpdateScript ?
			ERapidIterationParameterMode::SystemScript : ERapidIterationParameterMode::EmitterOrParticleScript)
	{
	}

	/** Configures the context to replace the unaliased emitter namespace with an emitter name namespace in parameter map parameters and rapid iteration parameters. */
	NIAGARA_API FNiagaraAliasContext& ChangeEmitterToEmitterName(const FString& InEmitterName);

	/** Configures the context to replace an emitter name namespace with the unaliased emitter namespace in parameter map parameters and rapid iteration parameters. */
	NIAGARA_API FNiagaraAliasContext& ChangeEmitterNameToEmitter(const FString& InEmitterName);

	/** Configures the context to replace an old emitter name namespace with a new emitter name namespace in parameter map parameters and rapid iteration parameters. */
	NIAGARA_API FNiagaraAliasContext& ChangeEmitterName(const FString& InOldEmitterName, const FString& InNewEmitterName);

	/** Configures the context to replace the unaliased module namespace with a module name namespace in parameter map parameters and rapid iteration parameters. */
	NIAGARA_API FNiagaraAliasContext& ChangeModuleToModuleName(const FString& InModuleName);

	/** Configures the context to replace a module name namespace with the unaliased module namespace in parameter map parameters and rapid iteration parameters. */
	NIAGARA_API FNiagaraAliasContext& ChangeModuleNameToModule(const FString& InModuleName);

	/** Configures the context to replace an old module name namespace with a new module name namespace in parameter map parameters and rapid iteration parameters. */
	NIAGARA_API FNiagaraAliasContext& ChangeModuleName(const FString& InOldModuleName, const FString& InNewModuleName);

	/** Configures the context to replace the stack context namespace with the specified named stack context in parameter map parameters. */
	NIAGARA_API FNiagaraAliasContext& ChangeStackContext(const FString& InStackContextName);

	ERapidIterationParameterMode GetRapidIterationParameterMode() const { return RapidIterationParameterMode; }
	NIAGARA_API FNiagaraAliasContext& ChangeRapidIterationParameterMode(ERapidIterationParameterMode InMode);

	const TOptional<FString>& GetEmitterName() const { return EmitterName; }
	const TOptional<TPair<FString, FString>>& GetEmitterMapping() const { return EmitterMapping; }

	const TOptional<FString>& GetModuleName() const { return ModuleName; }
	const TOptional<TPair<FString, FString>>& GetModuleMapping() const { return ModuleMapping; }

	const TOptional<FString>& GetStackContextName() const { return StackContextName; }
	const TOptional<TPair<FString, FString>>& GetStackContextMapping() const { return StackContextMapping; }

private:
	ERapidIterationParameterMode RapidIterationParameterMode;

	TOptional<FString> EmitterName;
	TOptional<TPair<FString, FString>> EmitterMapping;

	TOptional<FString> ModuleName;
	TOptional<TPair<FString, FString>> ModuleMapping;

	TOptional<FString> StackContextName;
	TOptional<TPair<FString, FString>> StackContextMapping;
};

namespace FNiagaraUtilities
{
	/** Builds a unique name from a candidate name and a set of existing names.  The candidate name will be made unique
	if necessary by adding a 3 digit index to the end. */
	FName NIAGARA_API GetUniqueName(FName CandidateName, const TSet<FName>& ExistingNames);

	NIAGARA_API FString CreateRapidIterationConstantName(FName InVariableName, const TCHAR* InEmitterName, ENiagaraScriptUsage InUsage);
	FNiagaraVariable NIAGARA_API ConvertVariableToRapidIterationConstantName(FNiagaraVariable InVar, const TCHAR* InEmitterName, ENiagaraScriptUsage InUsage);

	inline bool SupportsNiagaraRendering(ERHIFeatureLevel::Type FeatureLevel)
	{
		return true;
	}

	inline bool SupportsNiagaraRendering(EShaderPlatform ShaderPlatform)
	{
		// Note:
		// IsFeatureLevelSupported does a FeatureLevel < MaxFeatureLevel(ShaderPlatform) so checking ES3.1 support will return true for SM5. I added it explicitly to be clear what we are doing.
		return true;
	}

	// When enabled log more information for the end user
#if NIAGARA_VERBOSE_LOGGING_ENABLED
	bool LogVerboseWarnings();
#else
	constexpr bool LogVerboseWarnings() { return false; }
#endif

	// Whether GPU particles are currently allowed. Could change depending on config and runtime switches.
	NIAGARA_API bool AllowGPUParticles();

	// Whether compute shaders are allowed. Could change depending on config and runtime switches.
	NIAGARA_API bool AllowComputeShaders();

	// Are we able to use the GPU for culling?
	NIAGARA_API bool AllowGPUCulling();

	// Are we able to use the GPU for sorting?
	NIAGARA_API bool AllowGPUSorting();

	//-TODO: Consider deprecating these methods if we don't need the ShaderPlatform
	inline bool AllowGPUParticles(EShaderPlatform ShaderPlatform) { return AllowGPUParticles(); }
	inline bool AllowComputeShaders(EShaderPlatform ShaderPlatform) { return AllowComputeShaders(); }
	inline bool AllowGPUCulling(EShaderPlatform ShaderPlatform) { return AllowGPUCulling(); }
	inline bool AllowGPUSorting(EShaderPlatform ShaderPlatform) { return AllowGPUSorting();  }

	// Helper function to detect if SRVs are always created for buffers or not
	bool AreBufferSRVsAlwaysCreated(EShaderPlatform ShaderPlatform);

	// Helper function to determine if we should sync data from CPU to GPU
	inline bool ShouldSyncCpuToGpu(ENiagaraGpuSyncMode SyncMode)
	{
		return SyncMode == ENiagaraGpuSyncMode::SyncBoth || SyncMode == ENiagaraGpuSyncMode::SyncCpuToGpu;
	}

	// Helper function to determine if we should sync data from GPU to CPU
	inline bool ShouldSyncGpuToCpu(ENiagaraGpuSyncMode SyncMode)
	{
		return SyncMode == ENiagaraGpuSyncMode::SyncBoth || SyncMode == ENiagaraGpuSyncMode::SyncGpuToCpu;
	}

	ENiagaraCompileUsageStaticSwitch NIAGARA_API ConvertScriptUsageToStaticSwitchUsage(ENiagaraScriptUsage ScriptUsage);
	ENiagaraScriptContextStaticSwitch NIAGARA_API ConvertScriptUsageToStaticSwitchContext(ENiagaraScriptUsage ScriptUsage);
	
#if WITH_EDITORONLY_DATA
	/**
	 * Prepares rapid iteration parameter stores for simulation by removing old parameters no longer used by functions, by initializing new parameters
	 * added to functions, and by copying parameters across parameter stores for interscript dependencies.
	 * @param Scripts The scripts who's rapid iteration parameter stores will be processed.
	 * @param ScriptDependencyMap A map of script dependencies where the key is the source script and the value is the script which depends on the source.  All scripts in this
	 * map must be contained in the Scripts array, both keys and values.
	 * @param ScriptToEmitterNameMap An array of scripts to the name of the emitter than owns them.  If this is a system script the name can be empty.  All scripts in the
	 * scripts array must have an entry in this map.
	 */
	void NIAGARA_API PrepareRapidIterationParameters(const TArray<UNiagaraScript*>& Scripts, const TMap<UNiagaraScript*, UNiagaraScript*>& ScriptDependencyMap, const TMap<UNiagaraScript*, FVersionedNiagaraEmitter>& ScriptToEmitterNameMap);

	bool NIAGARA_API AreTypesAssignable(const FNiagaraTypeDefinition& FromType, const FNiagaraTypeDefinition& ToType);
#endif

	void NIAGARA_API DumpHLSLText(const FString& SourceCode, const FString& DebugName);

	NIAGARA_API FString SystemInstanceIDToString(FNiagaraSystemInstanceID ID);

	/**
	Converts a Niagara format into a EPixelFormat, will look for fallback options if the requested format can not be satisfied based on the RequiredCapabilities.
	If the return value optional is invalid that means no format was found.
	*/
	NIAGARA_API TOptional<EPixelFormat> BufferFormatToPixelFormat(ENiagaraGpuBufferFormat NiagaraFormat, EPixelFormatCapabilities RequiredCapabilities, int NumberOfChannels, UObject* RequesterForDebugging);

	/**
	Converts a Niagara format into a ETextureRenderTargetFormat, will look for fallback options if the requested format can not be satisfied based on the RequiredCapabilities.
	If the return value optional is invalid that means no format was found.
	*/
	NIAGARA_API TOptional<ETextureRenderTargetFormat> BufferFormatToRenderTargetFormat(ENiagaraGpuBufferFormat NiagaraFormat, EPixelFormatCapabilities RequiredCapabilities, UObject* RequesterForDebugging);

	NIAGARA_API FString SanitizeNameForObjectsAndPackages(const FString& InName);

	NIAGARA_API FNiagaraVariable ResolveAliases(const FNiagaraVariable& InVar, const FNiagaraAliasContext& InContext);

	UE_DEPRECATED(5.8, "Please update your code to include the caller information.")
	NIAGARA_API TOptional<EPixelFormat> BufferFormatToPixelFormat(ENiagaraGpuBufferFormat NiagaraFormat, EPixelFormatCapabilities RequiredCapabilities, int NumberOfChannels = 1);
	UE_DEPRECATED(5.8, "Please update your code to include the caller information.")
	NIAGARA_API TOptional<ETextureRenderTargetFormat> BufferFormatToRenderTargetFormat(ENiagaraGpuBufferFormat NiagaraFormat, EPixelFormatCapabilities RequiredCapabilities);
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraUserParameterBinding
{
	GENERATED_USTRUCT_BODY()

	NIAGARA_API FNiagaraUserParameterBinding();

	NIAGARA_API FNiagaraUserParameterBinding(const FNiagaraTypeDefinition& InMaterialDef);

	/** Link to a User Parameter to control a value within a Niagara System. */
	UPROPERTY(EditAnywhere, Category = "User Parameter")
	FNiagaraVariable Parameter;

	inline bool operator==(const FNiagaraUserParameterBinding& Other)const
	{
		return Other.Parameter == Parameter;
	}
};

template<>
struct TStructOpsTypeTraits<FNiagaraUserParameterBinding> : public TStructOpsTypeTraitsBase2<FNiagaraUserParameterBinding>
{
	enum
	{
		WithIdenticalViaEquality = true
	};
};

//////////////////////////////////////////////////////////////////////////
// Legacy Anim Trail Support


/** 
Controls the way that the width scale property affects animation trails. 
Only used for Legacy Anim Trail support when converting from Cascade to Niagara.
*/
UENUM()
enum class ENiagaraLegacyTrailWidthMode : uint32
{
	FromCentre,
	FromFirst,
	FromSecond,
};


//////////////////////////////////////////////////////////////////////////
/// FNiagaraCompiledDataReference
//////////////////////////////////////////////////////////////////////////

// Simple struct intended to hide the details of passing a reference to compiled data.  In
// particular for editor builds it will actually make a copy of the data to help try to avoid the many
// edge cases of recompiling/editing, while regular builds can reap the wins of just referencing the data
// and saving memory.
template<typename T>
struct FNiagaraCompiledDataReference
{
public:
	void Init(const T* SourceValue)
	{
#if WITH_EDITOR
		OptionalStructValue = *SourceValue;
#else
		StructPtr = SourceValue;
#endif
	}

	const T* operator->() const
	{
		return Get();
	}

	const T* Get() const
	{
#if WITH_EDITOR
		if (OptionalStructValue.IsSet())
		{
			return &OptionalStructValue.GetValue();
		}
		return nullptr;
#else
		return StructPtr;
#endif
	}

	void Reset()
	{
#if WITH_EDITOR
		OptionalStructValue.Reset();
#else
		StructPtr = nullptr;
#endif
	}

private:
#if WITH_EDITOR
	TOptional<T> OptionalStructValue;
#else
	const T* StructPtr = nullptr;
#endif
};


/** Records necessary information to verify that this will link properly and trace where that linkage dependency exists. */
USTRUCT()
struct FNiagaraCompileDependency
{
	GENERATED_USTRUCT_BODY()
public:
	FNiagaraCompileDependency()
	{
		LinkerErrorMessage = FString();
		NodeGuid = FGuid();
		PinGuid = FGuid();
		StackGuids.Empty();
		bDependentVariableFromCustomIterationNamespace = false;
	};

	FNiagaraCompileDependency(
		const FNiagaraVariableBase& InVar
		, const FString& InLinkerErrorMessage
		, FGuid InNodeGuid = FGuid()
		, FGuid InPinGuid = FGuid()
		, const TArray<FGuid>& InCallstackGuids = TArray<FGuid>()
		, bool bInDependentVariableFromCustomIterationNamespace = false
	)
		: LinkerErrorMessage(InLinkerErrorMessage)
		, NodeGuid(InNodeGuid)
		, PinGuid(InPinGuid)
		, StackGuids(InCallstackGuids)
		, DependentVariable(InVar)
		, bDependentVariableFromCustomIterationNamespace(bInDependentVariableFromCustomIterationNamespace)
	{};

	/* The message itself*/
	UPROPERTY()
	FString LinkerErrorMessage;

	/** The node guid that generated the compile event*/
	UPROPERTY()
	FGuid NodeGuid;

	/** The pin persistent id that generated the compile event*/
	UPROPERTY()
	FGuid PinGuid;

	/** The compile stack frame of node id's*/
	UPROPERTY()
	TArray<FGuid> StackGuids;

	/** The variable we are dependent on.*/
	UPROPERTY()
	FNiagaraVariableBase DependentVariable;

	UPROPERTY()
	bool bDependentVariableFromCustomIterationNamespace;

	inline bool operator==(const FNiagaraCompileDependency& Other)const
	{
		return DependentVariable == Other.DependentVariable && NodeGuid == Other.NodeGuid && PinGuid == Other.PinGuid && StackGuids == Other.StackGuids;
	}

	inline bool operator!=(const FNiagaraCompileDependency& Other)const
	{
		return !(*this == Other);
	}
};


extern NIAGARA_API ENCPoolMethod ToNiagaraPooling(EPSCPoolMethod PoolingMethod);
extern NIAGARA_API EPSCPoolMethod ToPSCPoolMethod(ENCPoolMethod PoolingMethod);

UENUM()
enum class ENiagaraSystemInstanceState : uint8
{
	None,
	PendingSpawn,
	PendingSpawnPaused,
	Spawning,
	Running,
	Paused,
	Num
};

UENUM()
enum class ENiagaraFunctionDebugState : uint8
{
	NoDebug,
	Basic,
};

/** Args struct for INiagaraParameterDefinitionsSubscriberViewModel::SynchronizeWithParameterDefinitions(...). */
struct FSynchronizeWithParameterDefinitionsArgs
{
	NIAGARA_API FSynchronizeWithParameterDefinitionsArgs();

	/** If set, instead of gathering all available parameter libraries, only consider subscribed parameter definitions that have a matching Id. */
	TArray<FGuid> SpecificDefinitionsUniqueIds;

	/** If set, instead of synchronizing to all destination script variables (UNiagaraScriptVariable owned by the object the INiagaraParameterDefinitionsSubscriberViewModel is viewing,
	 *  only synchronize destination script variables that have a matching Id.
	 */
	TArray<FGuid> SpecificDestScriptVarIds;

	/** Default false; If true, instead of gathering available parameter definitions to synchronize via INiagaraParameterDefinitionsSubscriber::GetSubscribedParameterDefinitionsPendingSynchronization(),
	 *  ignore the pending synchronization flag and gather via INiagaraParameterDefinitionsSubscriber::GetSubscribedParameterDefinitions().
	 *  Note: If true, SpecificDefinitionsUniqueIds and SpecificDestScriptVarIds will still apply.
	 */
	bool bForceGatherDefinitions;

	/** Default false; If true, instead of comparing parameter change IDs to check if a parameter needs to be synchronized, ignore the change IDs.
	 *  Note: If true, SpecificDefinitionsUniqueIds and SpecificDestScriptVarIds will still apply.
	 */
	bool bForceSynchronizeParameters;

	/** Default false; If true, search for parameters with the same name and type as definitions, and if there is a match, subscribe to the definition. 
	 *  If the existing parameter default value does not match the definition default value, set the parameter to override the definition default value.
	 */
	bool bSubscribeAllNameMatchParameters;

	/** If set, the subscriber will also synchronize the additional parameter definitions in addition to those it is normally subscribed to. */
	TArray<UNiagaraParameterDefinitionsBase*> AdditionalParameterDefinitions;

	/** If set, the subscriber will rename assignment nodes and map get nodes in its underlying scripts for each Old to New name in addition to those it would during regular synchronization. */
	TArray<TTuple<FName, FName>> AdditionalOldToNewNames;
};

/** Enum used to track stage that GPU compute proxies will execute in. */
UENUM()
namespace ENiagaraGpuComputeTickStage
{
	enum Type : int
	{
		PreInitViews,
		PostInitViews,
		PostOpaqueRender,
		Max UMETA(hidden),
		First = PreInitViews UMETA(hidden),
		Last = PostOpaqueRender UMETA(hidden),
	};
}

enum class ENiagaraResourceAccess : uint8
{
	ReadOnly,
	WriteOnly,
	ReadWrite,
};

UENUM()
enum class ENiagaraConditionalOperator
{
	Equals,
	NotEqual,
	LessThan,
	LessThanOrEqual,
	GreaterThan,
	GreaterThanOrEqual,

	Max UMETA(Hidden),
};

template<typename T>
bool EvalConditional(ENiagaraConditionalOperator Op, const T& A, const T& B)
{
	switch (Op)
	{
	case ENiagaraConditionalOperator::Equals: return A == B;
	case ENiagaraConditionalOperator::NotEqual: return A != B;
	case ENiagaraConditionalOperator::LessThan: return A < B;
	case ENiagaraConditionalOperator::LessThanOrEqual: return A <= B;
	case ENiagaraConditionalOperator::GreaterThan: return A > B;
	case ENiagaraConditionalOperator::GreaterThanOrEqual: return A >= B;
	default:check(0);
	};

	return false;
};
