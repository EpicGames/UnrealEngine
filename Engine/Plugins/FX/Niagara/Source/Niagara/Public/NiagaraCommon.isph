// Copyright Epic Games, Inc. All Rights Reserved.

inline varying float LoadFloat(const uniform float InParam[], const varying int Offset, const uniform bool bConstant)
{
	return bConstant == true ? InParam[0] : InParam[Offset];
}

inline float TruncToFloat(float x)
{
	return (float)((int)(x));
}

inline float NormalizeTime(float T, const uniform float LUTMinTime, const uniform float LUTInvTimeRange)
{
	return (T - LUTMinTime) * LUTInvTimeRange;
}

inline float Lerp(const float A, const float B, const float Alpha)
{
	const float Delta = B - A;
	return Alpha * Delta + A;
}

