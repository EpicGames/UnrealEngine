// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"

#include "NiagaraScript.h"
#include "NiagaraTypes.h"

#include "NiagaraCompilationTypes.generated.h"

class UNiagaraSystem;
class ITargetPlatform;

using FNiagaraCompilationTaskHandle = int32;

struct FNiagaraScriptCompileMetrics
{
	float TaskWallTime = 0.0f;
	float DDCFetchTime = 0.0f;
	float CompilerWallTime = 0.0f;
	float CompilerWorkerTime = 0.0f;
	float CompilerPreprocessTime = 0.0f;
	float TranslateTime = 0.0f;
	float ByteCodeOptimizeTime = 0.0f;
};

struct FNiagaraSystemCompileMetrics
{
	float SystemCompileWallTime = 0.0f;
	float SystemPrecompileTime = 0.0f;
	float SystemCompilationCopyTime = 0.0f;
	TMap<TObjectKey<UNiagaraScript>, FNiagaraScriptCompileMetrics> ScriptMetrics;
};

struct FNiagaraCompiledShaderInfo
{
	FNiagaraShaderMapRef CompiledShader;
	TArray<FShaderCompilerError> CompilationErrors;
	const ITargetPlatform* TargetPlatform = nullptr;
	EShaderPlatform ShaderPlatform = SP_NumPlatforms;
	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::Num;
};

USTRUCT()
struct FNiagaraScriptAsyncCompileData
{
	GENERATED_USTRUCT_BODY()

	FNiagaraVMExecutableDataId CompileId;
	TSharedPtr<struct FNiagaraVMExecutableData> ExeData;
	TArray<FNiagaraCompiledShaderInfo> CompiledShaders;
	FString UniqueEmitterName;
	FNiagaraEmitterID EmitterID;
	FNiagaraScriptCompileMetrics CompileMetrics;

	bool bFromDerivedDataCache = false;

	UPROPERTY()
	TArray<FNiagaraVariable> RapidIterationParameters;

	UPROPERTY()
	TMap<FName, TObjectPtr<UNiagaraDataInterface>> NamedDataInterfaces;
};

USTRUCT()
struct FNiagaraSystemAsyncCompileResults
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<TObjectPtr<UObject>> RootObjects;

	FNiagaraCompilationTaskHandle CompilationTask;

	using FCompileResultMap = TMap<UNiagaraScript*, FNiagaraScriptAsyncCompileData>;
	FCompileResultMap CompileResultMap;

	UPROPERTY()
	TArray<FNiagaraVariable> ExposedVariables;

	bool bForced = false;

	float CombinedCompileTime = 0.0f;
	float StartTime = 0.0f;
};

struct FNiagaraCompilationOptions
{
	UNiagaraSystem* System = nullptr;
	const ITargetPlatform* TargetPlatform = nullptr;
	bool bForced = false;
};

struct FNiagaraQueryCompilationOptions
{
	FNiagaraQueryCompilationOptions();

	UNiagaraSystem* System = nullptr;
	double MaxWaitDuration = 0.125;
	bool bWait = false;
	bool bGenerateTimingsFile = false;
};

class FNiagaraActiveCompilation : public FGCObject
{
public:
	static TUniquePtr<FNiagaraActiveCompilation> CreateCompilation();

	virtual bool Launch(const FNiagaraCompilationOptions& Options) = 0;
	virtual void Abort() = 0;
	virtual bool QueryCompileComplete(const FNiagaraQueryCompilationOptions& Options) = 0;
	virtual bool ValidateConsistentResults(const FNiagaraQueryCompilationOptions& Options) const = 0;
	virtual void Apply(const FNiagaraQueryCompilationOptions& Options) = 0;
	virtual void ReportResults(const FNiagaraQueryCompilationOptions& Options) const = 0;
	virtual bool BlocksBeginCacheForCooked() const { return false; }
	virtual bool BlocksGarbageCollection() const { return true; }

	void Invalidate()
	{
		bShouldApply = false;
	}

	bool ShouldApply() const
	{
		return bShouldApply;
	}

	bool WasForced() const
	{
		return bForced;
	}

protected:
	void WriteTimingsEntry(const TCHAR* TimingsSource, const FNiagaraQueryCompilationOptions& Options, const FNiagaraSystemCompileMetrics& SystemMetrics) const;

	double TaskStartTime = 0;
	bool bForced = false;
	bool bShouldApply = true;
};
