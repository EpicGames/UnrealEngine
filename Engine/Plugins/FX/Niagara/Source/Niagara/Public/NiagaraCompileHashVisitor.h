// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShaderParameterMacros.h"
#include "NiagaraCompileHashVisitor.generated.h"

USTRUCT()
struct FNiagaraCompileHashVisitorDebugInfo
{
	GENERATED_USTRUCT_BODY()
public:
	UPROPERTY()
	FString Object;

	UPROPERTY()
	TArray<FString> PropertyKeys;

	UPROPERTY()
	TArray<FString> PropertyValues;
};

/**
Used to store the state of a graph when deciding if it has been dirtied for recompile.
*/
struct FNiagaraCompileHashVisitor
{
public:
	FNiagaraCompileHashVisitor(FSHA1& InHashState) : HashState(InHashState) {}

	FSHA1& HashState;
	TArray<const void*> ObjectList;

	static NIAGARA_API int LogCompileIdGeneration;

#if WITH_EDITORONLY_DATA
	FNiagaraCompileHashVisitorDebugInfo* AddDebugInfo()
	{
		if (LogCompileIdGeneration != 0)
		{
			return &Values.AddDefaulted_GetRef();
		}
		return nullptr;
	}

	// Debug data about the compilation hash, including key value pairs to detect differences.
	TArray<FNiagaraCompileHashVisitorDebugInfo> Values;

	template<typename T>
	void ToDebugString(const T* InData, uint64 InDataCount, FString& OutStr)
	{
		for (int32 i = 0; i < InDataCount; i++)
		{
			FString DataStr = LexToString(InData[i]);
			OutStr.Appendf(TEXT("%s "), *DataStr);
		}
	}
#endif
	/**
	Registers a pointer for later reference in the compile id in a deterministic manner.
	*/
	int32 RegisterReference(const void* InObject)
	{
		if (InObject == nullptr)
		{
			return -1;
		}

		int32 Index = ObjectList.Find(InObject);
		if (Index < 0)
		{
			Index = ObjectList.Add(InObject);
		}
		return Index;
	}

	/**
	We don't usually want to save GUID's or pointer values because they have nondeterministic values. Consider a PostLoad upgrade operation that creates a new node.
	Each pin and node gets a unique ID. If you close the editor and reopen, you'll get a different set of values. One of the characteristics we want for compilation
	behavior is that the same graph structure produces the same compile results, so we only want to embed information that is deterministic. This method is for use
	when registering a pointer to an object that is serialized within the compile hash.
	*/
	bool UpdateReference(const TCHAR* InDebugName, const void* InObject)
	{
		int32 Index = RegisterReference(InObject);
		return UpdatePOD(InDebugName, Index);
	}

	/**
	Adds an array of POD (plain old data) values to the hash.
	*/
	template<typename T>
	bool UpdateArray(const TCHAR* InDebugName, const T* InData, uint64 InDataCount = 1)
	{
		static_assert(TIsPODType<T>::Value, "UpdateArray does not support a constructor / destructor on the element class.");
		HashState.Update((const uint8 *)InData, sizeof(T)*InDataCount);
#if WITH_EDITORONLY_DATA
		if (LogCompileIdGeneration != 0)
		{
			FString ValuesStr = InDebugName;
			ValuesStr.Append(TEXT(" = "));
			ToDebugString(InData, InDataCount, ValuesStr);
			Values.Top().PropertyKeys.Push(InDebugName);
			Values.Top().PropertyValues.Push(ValuesStr);
		}
#endif
		return true;
	}

	/**
	 * Adds the string representation of an FName to the hash
	 */
	bool UpdateName(const TCHAR* InDebugName, FName InName)
	{
		FNameBuilder NameString(InName);
		return UpdateString(InDebugName, NameString.ToView());
	}

	/**
	Adds a single value of typed POD (plain old data) to the hash.
	*/
	template<typename T>
	bool UpdatePOD(const TCHAR* InDebugName, const T& InData)
	{
		static_assert(TIsPODType<T>::Value, "Update does not support a constructor / destructor on the element class.");
		static_assert(!TIsPointer<T>::Value, "Update does not support pointer types.");
		HashState.Update((const uint8 *)&InData, sizeof(T));
#if WITH_EDITORONLY_DATA
		if (LogCompileIdGeneration != 0)
		{
			FString ValuesStr;
			ToDebugString(&InData, 1, ValuesStr);
			Values.Top().PropertyKeys.Push(InDebugName);
			Values.Top().PropertyValues.Push(ValuesStr);
		}
#endif
		return true;
	}

	template<typename T>
	bool UpdateShaderParameters()
	{
		const FShaderParametersMetadata* ShaderParametersMetadata = TShaderParameterStructTypeInfo<T>::GetStructMetadata();
		return UpdatePOD(ShaderParametersMetadata->GetStructTypeName(), ShaderParametersMetadata->GetLayoutHash());
	}

	NIAGARA_API bool UpdateShaderFile(const TCHAR* ShaderFilePath);

	/**
	Adds an string value to the hash.
	*/
	NIAGARA_API bool UpdateString(const TCHAR* InDebugName, FStringView InData);
};
