// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraDefines.h"
#include "NiagaraScalabilityState.h"
#include "NiagaraTickBehaviorEnum.h"
#include "NiagaraUserRedirectionParameterStore.h"
#include "NiagaraVariant.h"
#include "PrimitiveViewRelevance.h"
#include "Particles/ParticlePerfStats.h"
#include "Particles/ParticleSystemComponent.h"


#include "NiagaraSettings.h"

#include "NiagaraComponent.generated.h"

class FMeshElementCollector;
class FNiagaraRenderer;
class UNiagaraEffectType;
class INiagaraSceneComponentUtils;
class UNiagaraSystem;
class UNiagaraParameterCollection;
class UNiagaraParameterCollectionInstance;
class FNiagaraSystemRenderData;
class FNiagaraSystemInstance;
class FNiagaraSystemInstanceController;
class FNiagaraSystemSimulation;
class FNiagaraGpuComputeDispatchInterface;
enum class ENCPoolMethod : uint8;
using FNiagaraSystemInstanceControllerPtr = TSharedPtr<FNiagaraSystemInstanceController, ESPMode::ThreadSafe>;
using FNiagaraSystemInstanceControllerConstPtr = TSharedPtr<const FNiagaraSystemInstanceController, ESPMode::ThreadSafe>;

UENUM()
enum class ENiagaraOcclusionQueryMode : uint8
{
	/** Allow Niagara to determine if we allow occlusion queries to be enabled or not. */
	Default,
	/** Occlusion queries will always be enabled for Niagara. */
	AlwaysEnabled,
	/** Occlusion queries will always be disabled for Niagara. */
	AlwaysDisabled,
};

// Called when the particle system is done
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnNiagaraSystemFinished, class UNiagaraComponent*, PSystem);

/**
* UNiagaraComponent is the primitive component for a Niagara System.
* @see ANiagaraActor
* @see UNiagaraSystem
*/
UCLASS(ClassGroup = (Rendering, Common), Blueprintable, hidecategories = Object, hidecategories = Physics, hidecategories = Collision, showcategories = Trigger, editinlinenew, meta = (BlueprintSpawnableComponent, DisplayName = "Niagara Particle System Component"), MinimalAPI)
class UNiagaraComponent : public UFXSystemComponent
{
	friend struct FNiagaraScalabilityManager;
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	DECLARE_MULTICAST_DELEGATE(FOnSystemInstanceChanged);
	DECLARE_MULTICAST_DELEGATE(FOnSynchronizedWithAssetParameters);
#endif

public:

	/********* UFXSystemComponent *********/
	NIAGARA_API virtual void SetBoolParameter(FName ParameterName, bool Param) override;
	NIAGARA_API virtual void SetIntParameter(FName ParameterName, int Param) override;
	NIAGARA_API virtual void SetFloatParameter(FName ParameterName, float Param) override;
	NIAGARA_API virtual void SetVectorParameter(FName ParameterName, FVector Param) override;
	NIAGARA_API virtual void SetColorParameter(FName ParameterName, FLinearColor Param) override;
	NIAGARA_API virtual void SetActorParameter(FName ParameterName, class AActor* Param) override;

	NIAGARA_API virtual UFXSystemAsset* GetFXSystemAsset() const override;
	NIAGARA_API virtual void SetEmitterEnable(FName EmitterName, bool bNewEnableState) override;
	NIAGARA_API virtual void ReleaseToPool() override;
	NIAGARA_API virtual uint32 GetApproxMemoryUsage() const override;
	NIAGARA_API virtual void ActivateSystem(bool bFlagAsJustAttached = false) override;
	/********* UFXSystemComponent *********/

	NIAGARA_API virtual FName GetFNameForStatID() const override;

	NIAGARA_API virtual bool RequiresGameThreadEndOfFrameRecreate() const override;

private:
	UPROPERTY(EditAnywhere, Category="Niagara", meta = (DisplayName = "Niagara System Asset"))
	TObjectPtr<UNiagaraSystem> Asset;

	/** Allows you to control how Niagara selects the tick group, changing this while an instance is active will result in not change as it is cached. */
	UPROPERTY(EditAnywhere, Category = "Niagara", meta = (DisplayName = "Niagara Tick Behavior"))
	ENiagaraTickBehavior TickBehavior = ENiagaraTickBehavior::UsePrereqs;

	/**
	 * Offsets the deterministic random seed of all emitters. Used to achieve variety between components, while still achieving determinism.
	 * WARNINGS:
	 * - If this value is set in a non-deterministic way, it has the potential to break determinism of the entire system.
	 * - This value is applied when emitters are activated/reset, and changing them while the emitter is active has no effect.
	 */
	UPROPERTY(EditAnywhere, Category = "Randomness")
	int32 RandomSeedOffset;

	UPROPERTY()
	FNiagaraUserRedirectionParameterStore OverrideParameters;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TMap<FName, bool> EditorOverridesValue_DEPRECATED;

	UPROPERTY(EditAnywhere, Category="Niagara")
	TMap<FNiagaraVariableBase, FNiagaraVariant> TemplateParameterOverrides;

	UPROPERTY(EditAnywhere, Category="Niagara")
	TMap<FNiagaraVariableBase, FNiagaraVariant> InstanceParameterOverrides;

	UPROPERTY(transient)
	TMap<FNiagaraVariableBase, FNiagaraVariant> TemplateParameterOverridesCache;

	UPROPERTY(transient)
	TMap<FNiagaraVariableBase, FNiagaraVariant> InstanceParameterOverridesCache;

	FOnSystemInstanceChanged OnSystemInstanceChangedDelegate;

	FOnSynchronizedWithAssetParameters OnSynchronizedWithAssetParametersDelegate;
#endif

	/**
	When true, this component's system will be force to update via a slower "solo" path rather than the more optimal batched path with other instances of the same system.
	*/
	uint32 bForceSolo : 1;

	/** When true the GPU simulation debug display will enabled, allowing information used during simulation to be visualized. */
	UPROPERTY(EditAnywhere, Category = Parameters)
	uint32 bEnableGpuComputeDebug : 1;

	/** When true then this instance will override the system's warmup settings. */
	UPROPERTY(EditAnywhere, Category = Warmup)
	uint32 bOverrideWarmupSettings : 1;
	
	/** Number of ticks to process for warmup of the system. Total warmup time is WarmupTickCount * WarmupTickDelta. */
	UPROPERTY(EditAnywhere, Category = Warmup, meta=(EditCondition="bOverrideWarmupSettings", ClampMin = "0"))
	int32 WarmupTickCount = 0;

	/** Delta time used when ticking the system in warmup mode. */
	UPROPERTY(EditAnywhere, Category = Warmup, meta = (EditCondition="bOverrideWarmupSettings", ForceUnits=s, UIMin = "0.01", UIMax = "1"))
	float WarmupTickDelta;

	FNiagaraSystemInstanceControllerPtr SystemInstanceController;

	/** Defines the mode use when updating the System age. */
	ENiagaraAgeUpdateMode AgeUpdateMode;

	/** The desired age of the System instance.  This is only relevant when using the DesiredAge age update mode. */
	float DesiredAge;

	/** The last desired age value that was handled by the tick function.  This is only relevant when using the DesiredAgeNoSeek age update mode. */
	float LastHandledDesiredAge;

	/** Whether or not the component can render while seeking to the desired age. */
	bool bCanRenderWhileSeeking;

	/** The delta time used when seeking to the desired age.  This is only relevant when using the DesiredAge age update mode. */
	float SeekDelta;

	bool bLockDesiredAgeDeltaTimeToSeekDelta;

	/** The maximum amount of time in seconds to spend seeking to the desired age in a single frame. */
	float MaxSimTime;

	/** Whether or not the component is currently seeking to the desired time. */
	bool bIsSeeking;

	UPROPERTY()
	uint32 bAutoDestroy : 1;

	UPROPERTY()
	uint32 bRenderingEnabled : 1;

	//~ Begin UActorComponent Interface.
protected:
	NIAGARA_API virtual void OnRegister() override;
	NIAGARA_API virtual void OnUnregister() override; 
	NIAGARA_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
	NIAGARA_API virtual void OnEndOfFrameUpdateDuringTick() override;
	NIAGARA_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	NIAGARA_API virtual void DestroyRenderState_Concurrent() override;
	NIAGARA_API virtual void SendRenderDynamicData_Concurrent() override;
	NIAGARA_API virtual void BeginDestroy() override;
	//virtual void OnAttachmentChanged() override;

public:
	/**
	* True if we should automatically attach to AutoAttachParent when activated, and detach from our parent when completed.
	* This overrides any current attachment that may be present at the time of activation (deferring initial attachment until activation, if AutoAttachParent is null).
	* When enabled, detachment occurs regardless of whether AutoAttachParent is assigned, and the relative transform from the time of activation is restored.
	* This also disables attachment on dedicated servers, where we don't actually activate even if bAutoActivate is true.
	* @see AutoAttachParent, AutoAttachSocketName, AutoAttachLocationType
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Attachment)
	uint32 bAutoManageAttachment : 1;

	/**
	 * Option for how we handle bWeldSimulatedBodies when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Attachment, meta=(EditCondition="bAutoManageAttachment"))
	uint32 bAutoAttachWeldSimulatedBodies : 1;

	/**
	 * Time between forced UpdateTransforms for systems that use dynamically calculated bounds,
	 * Which is effectively how often the bounds are shrunk.
	 */
	UPROPERTY()
	float MaxTimeBeforeForceUpdateTransform;

	/** How to handle pooling for this component instance. */
	ENCPoolMethod PoolingMethod;

protected:
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering)
	ENiagaraOcclusionQueryMode OcclusionQueryMode = ENiagaraOcclusionQueryMode::Default;

public:
	NIAGARA_API virtual void Activate(bool bReset = false) override;
	NIAGARA_API virtual void Deactivate() override;
	NIAGARA_API virtual void DeactivateImmediate() override;
	NIAGARA_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	NIAGARA_API ENiagaraExecutionState GetRequestedExecutionState() const;
	NIAGARA_API ENiagaraExecutionState GetExecutionState() const;

	NIAGARA_API bool IsComplete() const;

private:
	//Internal versions that can be called from the scalability code.
	//These will behave as expected but will keep the component registered with the scalability manager.
	NIAGARA_API void ActivateInternal(bool bReset, bool bIsScalabilityCull);
	NIAGARA_API void DeactivateInternal(bool bIsScalabilityCull);
	NIAGARA_API void DeactivateImmediateInternal(bool bIsScalabilityCull);
	NIAGARA_API void SetPausedInternal(bool bInPaused, bool bIsScalabilityCull);	


	NIAGARA_API bool ShouldPreCull();
	NIAGARA_API void RegisterWithScalabilityManager();
	NIAGARA_API void UnregisterWithScalabilityManager();

	NIAGARA_API void PostSystemTick_GameThread();
	NIAGARA_API void OnSystemComplete(bool bExternalCompletion);

public:

	NIAGARA_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	NIAGARA_API virtual const UObject* AdditionalStatObject() const override;
	NIAGARA_API virtual bool IsReadyForOwnerToAutoDestroy() const override;
	NIAGARA_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	NIAGARA_API virtual void OnComponentCreated() override;
	NIAGARA_API virtual void PostApplyToComponent() override;
	//~ End UActorComponent Interface.

	//~ Begin UPrimitiveComponent Interface
	NIAGARA_API virtual int32 GetNumMaterials() const override;
	NIAGARA_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	NIAGARA_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	NIAGARA_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	NIAGARA_API virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
	NIAGARA_API virtual void OnAttachmentChanged() override;
	NIAGARA_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;
	//~ End UPrimitiveComponent Interface

	//~ Begin USceneComponent Interface
	NIAGARA_API virtual void OnChildAttached(USceneComponent* ChildComponent) override;
	NIAGARA_API virtual void OnChildDetached(USceneComponent* ChildComponent) override;
	NIAGARA_API virtual bool IsVisible() const override;
	//~ Begin USceneComponent Interface

	UE_DEPRECATED(5.0, "This interface is no longer safe to access directly. Use the interface provided by GetSystemInstanceController instead.")
	NIAGARA_API TSharedPtr<FNiagaraSystemSimulation, ESPMode::ThreadSafe> GetSystemSimulation();

	NIAGARA_API bool InitializeSystem();
	NIAGARA_API void DestroyInstance();
	NIAGARA_API void DestroyInstanceNotComponent();

	NIAGARA_API void OnPooledReuse(UWorld* NewWorld);

	bool IsActiveForUpdateContext() const;

	/*
	Switch which asset the component is using.
	This requires Niagara to wait for concurrent execution and the override parameter store to be synchronized with the new asset.
	By default existing parameters are reset when we call SetAsset, modify bResetExistingOverrideParameters to leave existing parameter data as is.
	*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara System Asset"))
	NIAGARA_API void SetAsset(UNiagaraSystem* InAsset, bool bResetExistingOverrideParameters = true);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Niagara System Asset"))
	UNiagaraSystem* GetAsset() const { return Asset; }

	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API void SetOcclusionQueryMode(ENiagaraOcclusionQueryMode Mode);

	UFUNCTION(BlueprintCallable, Category = Niagara)
	ENiagaraOcclusionQueryMode GetOcclusionQueryMode() const { return OcclusionQueryMode; }

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Forced Solo Mode"))
	NIAGARA_API void SetForceSolo(bool bInForceSolo);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Is In Forced Solo Mode"))
	bool GetForceSolo() const { return bForceSolo; }

private:
	NIAGARA_API bool RequiresSoloMode() const;
	NIAGARA_API void UpdateInstanceSoloMode();


	const FNiagaraReadinessSettings& GetReadinessSettings() const;
	ENiagaraEmitterImportance GetHighestEmitterImportance() const;
	
	// returns true if the pso job is still running
	bool CheckEmitterPSOStillRunning() const;
	bool CheckPSOCacheIsReadyToRun();

public:
	NIAGARA_API virtual void PrecachePSOs() override;

	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API void SetGpuComputeDebug(bool bEnableDebug);

	/**
	Sets the custom time dilation value for the component.
	Note: This is only available on components that are in solo mode currently.
	*/
	UFUNCTION(BlueprintCallable, Category=Niagara)
	NIAGARA_API void SetCustomTimeDilation(float Dilation = 1.0f);

	UFUNCTION(BlueprintCallable, Category = Niagara)
	float GetCustomTimeDilation() const { return CustomTimeDilation; }

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Age Update Mode"))
	NIAGARA_API ENiagaraAgeUpdateMode GetAgeUpdateMode() const;

	/** Sets the age update mode for the System instance. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Age Update Mode"))
	NIAGARA_API void SetAgeUpdateMode(ENiagaraAgeUpdateMode InAgeUpdateMode);

	/** Gets the desired age of the System instance.  This is only relevant when using the DesiredAge age update mode. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Desired Age"))
	NIAGARA_API float GetDesiredAge() const;

	/** Sets the desired age of the System instance.  This is only relevant when using the DesiredAge age update mode. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Desired Age"))
	NIAGARA_API void SetDesiredAge(float InDesiredAge);

	/** Sets the desired age of the System instance and designates that this change is a seek.  When seeking to a desired age the
	    The component can optionally prevent rendering. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Seek to Desired Age"))
	NIAGARA_API void SeekToDesiredAge(float InDesiredAge);

	/** Sets whether or not the system can render while seeking. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Can Render While Seeking"))
	NIAGARA_API void SetCanRenderWhileSeeking(bool bInCanRenderWhileSeeking);

	/** Gets the delta value which is used when seeking from the current age, to the desired age.  This is only relevant
	when using the DesiredAge age update mode. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Desired Age Seek Delta"))
	NIAGARA_API float GetSeekDelta() const;

	/** Sets the delta value which is used when seeking from the current age, to the desired age.  This is only relevant
	when using the DesiredAge age update mode. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Desired Age Seek Delta"))
	NIAGARA_API void SetSeekDelta(float InSeekDelta);

	/** Gets whether or not the delta time used to tick the system instance when using desired age is locked to the seek delta.  When true, the system instance
	will only be ticked when the desired age has changed by more than the seek delta.  When false the system instance will be ticked by the change in desired
	age when not seeking. */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API bool GetLockDesiredAgeDeltaTimeToSeekDelta() const;

	/** Sets whether or not the delta time used to tick the system instance when using desired age is locked to the seek delta.  When true, the system instance
	will only be ticked when the desired age has changed by more than the seek delta.  When false the system instance will be ticked by the change in desired
	age when not seeking. */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API void SetLockDesiredAgeDeltaTimeToSeekDelta(bool bLock);

	/**
	Get the maximum CPU time in seconds we will simulate to the desired age, when we go beyond this limit ticks will be processed in the next frame.
	This is only relevant when using the DesiredAge age update mode.
	Note: The componet will not be visibile if we have ticks to process and SetCanRenderWhileSeeking is set to true
	*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Max Desired Age Tick Delta"))
	NIAGARA_API float GetMaxSimTime() const;

	/**
	Sets the maximum CPU time in seconds we will simulate to the desired age, when we go beyond this limit ticks will be processed in the next frame.
	This is only relevant when using the DesiredAge age update mode.
	Note: The componet will not be visibile if we have ticks to process and SetCanRenderWhileSeeking is set to true
	*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Max Desired Age Tick Delta"))
	NIAGARA_API void SetMaxSimTime(float InMaxTime);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Auto Destroy"))
	NIAGARA_API void SetAutoDestroy(bool bInAutoDestroy);

	UE_DEPRECATED(5.0, "This interface is no longer safe to access directly. Use the interface provided by GetSystemInstanceController instead.")
	NIAGARA_API FNiagaraSystemInstance* GetSystemInstance() const;

	FNiagaraSystemInstanceControllerPtr GetSystemInstanceController() { return SystemInstanceController; }
	FNiagaraSystemInstanceControllerConstPtr GetSystemInstanceController() const { return SystemInstanceController; }

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Tick Behavior"))
	NIAGARA_API void SetTickBehavior(ENiagaraTickBehavior NewTickBehavior);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Tick Behavior"))
	ENiagaraTickBehavior GetTickBehavior() const { return TickBehavior; }

	/**
	Sets the fixed bounds for the system instance, this overrides all other bounds.
	The box is expected to be in local space not world space.
	A default uninitialized box will clear the fixed bounds and revert back to system fixed / dynamic bounds.
	*/
	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API void SetSystemFixedBounds(FBox LocalBounds);

	/**
	Gets the fixed bounds for the system instance.
	This will return the user set fixed bounds if set, or the systems fixed bounds if set.
	Note: The returned box may be invalid if no fixed bounds exist
	*/
	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API FBox GetSystemFixedBounds() const;

	/** Clear any previously set fixed bounds for the system instance. */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API void ClearSystemFixedBounds();

	/**
	Sets the fixed bounds for an emitter instance, this overrides all other bounds.
	The box is expected to be in local space not world space.
	A default uninitialized box will clear the fixed bounds and revert back to emitter fixed / dynamic bounds.
	*/
	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API void SetEmitterFixedBounds(FName EmitterName, FBox LocalBounds);

	/**
	Gets the fixed bounds for an emitter instance.
	This will return the user set fixed bounds if set, or the emitters fixed bounds if set.
	Note: The returned box may be invalid if no fixed bounds exist
	*/
	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API FBox GetEmitterFixedBounds(FName EmitterName) const;

	/** Clear any previously set fixed bounds for the emitter instance. */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API void ClearEmitterFixedBounds(FName EmitterName);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Random Seed Offset"))
	NIAGARA_API void SetRandomSeedOffset(int32 NewRandomSeedOffset);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Random Seed Offset"))
	int32 GetRandomSeedOffset() const { return RandomSeedOffset; }

	/** Sets a Niagara FLinearColor parameter by name, overriding locally if necessary.*/
	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (LinearColor)", Keywords="user parameter variable color"))
	NIAGARA_API void SetNiagaraVariableLinearColor(const FString& InVariableName, const FLinearColor& InValue);

	/** Sets a Niagara FLinearColor parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (LinearColor)", Keywords="user parameter variable color"))
	NIAGARA_API void SetVariableLinearColor(FName InVariableName, const FLinearColor& InValue);

	/** Sets a Niagara Vector4 parameter by name, overriding locally if necessary.*/
	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (Vector4)", Keywords="user parameter variable vector"))
	NIAGARA_API void SetNiagaraVariableVec4(const FString& InVariableName, const FVector4& InValue);

	/** Sets a Niagara Vector4 parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Vector4)", Keywords="user parameter variable vector"))
	NIAGARA_API void SetVariableVec4(FName InVariableName, const FVector4& InValue);

	/** Sets a Niagara quaternion parameter by name, overriding locally if necessary.*/
	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (Quaternion)", Keywords="user parameter variable quaternion rotation"))
	NIAGARA_API void SetNiagaraVariableQuat(const FString& InVariableName, const FQuat& InValue);

	/** Sets a Niagara quaternion parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Quaternion)", Keywords="user parameter variable quaternion rotation"))
	NIAGARA_API void SetVariableQuat(FName InVariableName, const FQuat& InValue);

	/** Sets a Niagara matrix parameter by name, overriding locally if necessary.*/
	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (Matrix)", Keywords="user parameter variable matrix"))
	NIAGARA_API void SetNiagaraVariableMatrix(const FString& InVariableName, const FMatrix& InValue);

	/** Sets a Niagara matrix parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Matrix)", Keywords="user parameter variable matrix"))
	NIAGARA_API void SetVariableMatrix(FName InVariableName, const FMatrix& InValue);

	/** Sets a Niagara Vector3 parameter by name, overriding locally if necessary.*/
	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (Vector3)", Keywords="user parameter variable vector"))
	NIAGARA_API void SetNiagaraVariableVec3(const FString& InVariableName, FVector InValue);

	/** Sets a Niagara Vector3 parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Vector3)", Keywords="user parameter variable vector"))
	NIAGARA_API void SetVariableVec3(FName InVariableName, FVector InValue);

	/** Sets a Niagara Position parameter by name, overriding locally if necessary.*/
	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (Position)", Keywords="user parameter variable vector position lwc"))
	NIAGARA_API void SetNiagaraVariablePosition(const FString& InVariableName, FVector InValue);

	/** Sets a Niagara Position parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Position)", Keywords="user parameter variable vector position lwc"))
	NIAGARA_API void SetVariablePosition(FName InVariableName, FVector InValue);

	/** Sets a Niagara Vector2 parameter by name, overriding locally if necessary.*/
	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (Vector2)", Keywords="user parameter variable vector"))
	NIAGARA_API void SetNiagaraVariableVec2(const FString& InVariableName, FVector2D InValue);

	/** Sets a Niagara Vector2 parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Vector2)", Keywords="user parameter variable vector"))
	NIAGARA_API void SetVariableVec2(FName InVariableName, FVector2D InValue);

	/** Sets a Niagara float parameter by name, overriding locally if necessary.*/
	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (Float)", Keywords="user parameter variable float"))
	NIAGARA_API void SetNiagaraVariableFloat(const FString& InVariableName, float InValue);

	/** Sets a Niagara float parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Float)", Keywords="user parameter variable float"))
	NIAGARA_API void SetVariableFloat(FName InVariableName, float InValue);

	/** Sets a Niagara int parameter by name, overriding locally if necessary.*/
	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (Int32)", Keywords="user parameter variable int"))
	NIAGARA_API void SetNiagaraVariableInt(const FString& InVariableName, int32 InValue);

	/** Sets a Niagara int parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Int32)", Keywords="user parameter variable int"))
	NIAGARA_API void SetVariableInt(FName InVariableName, int32 InValue);

	/** Sets a Niagara bool parameter by name, overriding locally if necessary.*/
	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (Bool)", Keywords="user parameter variable bool"))
	NIAGARA_API void SetNiagaraVariableBool(const FString& InVariableName, bool InValue);

	/** Sets a Niagara bool parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Bool)", Keywords="user parameter variable bool"))
	NIAGARA_API void SetVariableBool(FName InVariableName, bool InValue);

	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (Actor)", Keywords="user parameter variable actor"))
	NIAGARA_API void SetNiagaraVariableActor(const FString& InVariableName, AActor* Actor);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Actor)", Keywords="user parameter variable actor"))
	NIAGARA_API void SetVariableActor(FName InVariableName, AActor* Actor);

	UE_DEPRECATED(5.3, "This method will be removed in a future release.  Please update to use the FName variant")
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable By String (Object)", Keywords="user parameter variable object"))
	NIAGARA_API void SetNiagaraVariableObject(const FString& InVariableName, UObject* Object);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Object)", Keywords="user parameter variable object"))
	NIAGARA_API void SetVariableObject(FName InVariableName, UObject* Object);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Material)", Keywords="user parameter variable material"))
	NIAGARA_API void SetVariableMaterial(FName InVariableName, UMaterialInterface* Object);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Static Mesh)", Keywords="user parameter variable mesh"))
	NIAGARA_API void SetVariableStaticMesh(FName InVariableName, UStaticMesh* InValue);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (Texture)", Keywords="user parameter variable texture"))
	NIAGARA_API void SetVariableTexture(FName InVariableName, class UTexture* Texture);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Variable (TextureRenderTarget)"))
	NIAGARA_API void SetVariableTextureRenderTarget(FName InVariableName, class UTextureRenderTarget* TextureRenderTarget);

	/** Gets a Niagara bool parameter by name. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Niagara Variable (Bool)", Keywords = "user parameter variable bool"))
	[[nodiscard]] NIAGARA_API bool GetVariableBool(FName InVariableName, bool& bIsValid) const;

	/** Gets a Niagara int parameter by name. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Niagara Variable (Int32)", Keywords = "user parameter variable int"))
	[[nodiscard]] NIAGARA_API int32 GetVariableInt(FName InVariableName, bool& bIsValid) const;

	/** Gets a Niagara float parameter by name. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Niagara Variable (Float)", Keywords = "user parameter variable float"))
	[[nodiscard]] NIAGARA_API float GetVariableFloat(FName InVariableName, bool& bIsValid) const;

	/** Gets a Niagara Vector2 parameter by name. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Niagara Variable (Vector2)", Keywords = "user parameter variable vector"))
	[[nodiscard]] NIAGARA_API FVector2D GetVariableVec2(FName InVariableName, bool& bIsValid) const;

	/** Gets a Niagara Vector3 parameter by name. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Niagara Variable (Vector3)", Keywords = "user parameter variable vector"))
	[[nodiscard]] NIAGARA_API FVector GetVariableVec3(FName InVariableName, bool& bIsValid) const;

	/** Gets a Niagara Vector4 parameter by name. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Niagara Variable (Vector4)", Keywords = "user parameter variable vector"))
	[[nodiscard]] NIAGARA_API FVector4 GetVariableVec4(FName InVariableName, bool& bIsValid) const;

	/** Gets a Niagara Position parameter by name. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Niagara Variable (Position)", Keywords = "user parameter variable vector position lwc"))
	[[nodiscard]] NIAGARA_API FVector GetVariablePosition(FName InVariableName, bool& bIsValid) const;

	/** Gets a Niagara Linear Color parameter by name. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Niagara Variable (LinearColor)", Keywords = "user parameter variable color"))
	[[nodiscard]] NIAGARA_API FLinearColor GetVariableColor(FName InVariableName, bool& bIsValid) const;

	/** Gets a Niagara quaternion parameter by name. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Niagara Variable (Quaternion)", Keywords = "user parameter variable quaternion rotation"))
	[[nodiscard]] NIAGARA_API FQuat GetVariableQuat(FName InVariableName, bool& bIsValid) const;

	/** Gets a Niagara matrix parameter by name. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Get Niagara Variable (Matrix)", Keywords = "user parameter variable matrix"))
	[[nodiscard]] NIAGARA_API FMatrix GetVariableMatrix(FName InVariableName, bool& bIsValid) const;

	/**For internal use only. Do not use.*/
	NIAGARA_API void SetVariable_InternalUseOnly(const FNiagaraVariableBase& Variable, const FNiagaraVariant& Variant);

	/**For internal use only. Do not use.*/
	NIAGARA_API void GetVariable_InternalUseOnly(const FNiagaraVariableBase& Variable, FNiagaraVariant& Variant)const;

	/**
	Override the parameter collection to use.
	The order for resolving collections is Component -> System -> World.
	Note that when using a collection override the simulation may be forced in solo mode which is less performant.
	*/
	UFUNCTION(Experimental, BlueprintCallable, Category = Niagara)
	NIAGARA_API void SetParameterCollectionOverride(UNiagaraParameterCollectionInstance* ParameterCollection);

	/** Remove the override collection, this will revert back to the default instance. */
	UFUNCTION(Experimental, BlueprintCallable, Category = Niagara)
	NIAGARA_API void RemoveParameterCollectionOverride(UNiagaraParameterCollectionInstance* OverrideCollection);

	/** Clear all parameter collection overrides. */
	UFUNCTION(Experimental, BlueprintCallable, Category = Niagara)
	NIAGARA_API void ClearParameterCollectionOverrides();

	/** Get the current parameter collection override if one exists. */
	UNiagaraParameterCollectionInstance* GetParameterCollectionOverride(UNiagaraParameterCollection* ParameterCollection) const;

	/** Resets the System to it's initial pre-simulated state. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Reset System"))
	NIAGARA_API void ResetSystem();

	/** Called on when an external object wishes to force this System to reinitialize itself from the System data.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Reinitialize System"))
	NIAGARA_API void ReinitializeSystem();

	/** Gets whether or not rendering is enabled for this component. */
	NIAGARA_API bool GetRenderingEnabled() const;

	/** Sets whether or not rendering is enabled for this component. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Rendering Enabled"))
	NIAGARA_API void SetRenderingEnabled(bool bInRenderingEnabled);

	/** Advances this system's simulation by the specified number of ticks and delta time. */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API void AdvanceSimulation(int32 TickCount, float TickDeltaSeconds);

	/** Advances this system's simulation by the specified time in seconds and delta time. Advancement is done in whole ticks of TickDeltaSeconds so actual simulated time will be the nearest lower multiple of TickDeltaSeconds. */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API void AdvanceSimulationByTime(float SimulateTime, float TickDeltaSeconds);

	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API void SetPaused(bool bInPaused);

	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API bool IsPaused() const;

	UE_DEPRECATED(4.27, "This method will be removed in a future release. Use the method provided by the Niagara Function Library instead.")
	UFUNCTION(BlueprintCallable, Category = Niagara)
	NIAGARA_API UNiagaraDataInterface* GetDataInterface(const FString &Name);

	/**
		The significant index for this component. i.e. this is the Nth most significant instance of it's system in the scene.
		Passed to the script to allow us to scale down internally for less significant systems instances.
	*/
	NIAGARA_API void SetSystemSignificanceIndex(int32 InIndex);

	//~ Begin UObject Interface.
	NIAGARA_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
	NIAGARA_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static NIAGARA_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

#if WITH_EDITOR
	NIAGARA_API virtual bool CanEditChange(const FProperty* InProperty) const override;

	NIAGARA_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	NIAGARA_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/**
	  * Find the value of an overridden parameter.  The value returned may not be the current value being used by the simulation but
	  * will reflect the last value which has been set through the editor on a component placed in a level, or on a component in the 
	  * blueprint editor.
	  * Returns null if the parameter isn't overridden by this component.
	  */
	NIAGARA_API FNiagaraVariant FindParameterOverride(const FNiagaraVariableBase& InKey) const;

	/** Gets the current value of a parameter which is being used by the simulation.  This value will reflect runtime changes such as
	  * those made by sequencer, or in PIE through code or blueprint. */
	NIAGARA_API FNiagaraVariant GetCurrentParameterValue(const FNiagaraVariableBase& InKey) const;

	NIAGARA_API bool HasParameterOverride(const FNiagaraVariableBase& InKey) const;
	NIAGARA_API void SetParameterOverride(const FNiagaraVariableBase& InKey, const FNiagaraVariant& InValue);

	/** Remove an override for a given parameter if one exists. */
	NIAGARA_API void RemoveParameterOverride(const FNiagaraVariableBase& InKey);

	NIAGARA_API void UpgradeDeprecatedParameterOverrides();
	NIAGARA_API void EnsureOverrideParametersConsistent() const;
#endif
	//~ End UObject Interface

	UFUNCTION(BlueprintCallable, Category = Preview, meta = (Keywords = "preview LOD Distance scalability"))
	NIAGARA_API void SetPreviewLODDistance(bool bEnablePreviewLODDistance, float PreviewLODDistance, float PreviewMaxDistance);

	UFUNCTION(BlueprintCallable, Category = Preview, meta = (Keywords = "preview LOD Distance scalability"))
	inline bool GetPreviewLODDistanceEnabled()const;

	UFUNCTION(BlueprintCallable, Category = Preview, meta = (Keywords = "preview LOD Distance scalability"))
	inline float GetPreviewLODDistance()const;

	/**
	Initializes this component for capturing a performance baseline.
	This will do things such as disabling distance culling and setting a LODDistance of 0 to ensure the effect is at it's maximum cost.
	*/
	UFUNCTION(BlueprintCallable, Category = Performance, meta = (Keywords = "Niagara Performance"))
	NIAGARA_API void InitForPerformanceBaseline();

	NIAGARA_API void SetLODDistance(float InLODDistance, float InMaxLODDistance);

#if WITH_EDITOR
	NIAGARA_API void PostLoadNormalizeOverrideNames();
	FOnSystemInstanceChanged& OnSystemInstanceChanged() { return OnSystemInstanceChangedDelegate; }

	FOnSynchronizedWithAssetParameters& OnSynchronizedWithAssetParameters() { return OnSynchronizedWithAssetParametersDelegate; }
#endif

	FNiagaraUserRedirectionParameterStore& GetOverrideParameters() { return OverrideParameters; }

	const FNiagaraParameterStore& GetOverrideParameters() const { return OverrideParameters; }

	NIAGARA_API bool IsWorldReadyToRun() const;

	//~ End UObject Interface.

	// Called when the particle system is done
	UPROPERTY(BlueprintAssignable, DuplicateTransient)
	FOnNiagaraSystemFinished OnSystemFinished;

	/** Removes all local overrides and replaces them with the values from the source System - note: this also removes the editor overrides from the component as it is used by the pooling mechanism to prevent values leaking between different instances. */
	NIAGARA_API void SetUserParametersToDefaultValues();

	/** Is this an effect on or linked to the local player. */
	NIAGARA_API bool IsLocalPlayerEffect()const;

private:
	/** Compare local overrides with the source System. Remove any that have mismatched types or no longer exist on the System.*/
	NIAGARA_API void SynchronizeWithSourceSystem();

	NIAGARA_API void FixInvalidUserParameterOverrideData();

	NIAGARA_API void AssetExposedParametersChanged();

	NIAGARA_API void CopyParametersFromAsset(bool bResetExistingOverrideParameters = true);

#if WITH_EDITOR
	void CacheAndClearEditorParameterOverrides();
	void CopyCachedEditorParameterOverrides();

	NIAGARA_API void SetOverrideParameterStoreValue(const FNiagaraVariableBase& InKey, const FNiagaraVariant& InValue);
	NIAGARA_API void ApplyOverridesToParameterStore();

	/**
	 * Ensures that the data interfaces in the template parameter overrides, instance parameter overrides, and in the override parameters
	 * parameter store have this component as their outer and have their archetype cleared.  This is required to hack around issues with
	 * subobject handling on components when they're spawned from templates which can cause invalid cross package references.
 	 */
	NIAGARA_API void FixDataInterfaceOuters();

	void FixDataInterfaceObjectFlags();
#endif

public:
	/**
	 * Component we automatically attach to when activated, if bAutoManageAttachment is true.
	 * If null during registration, we assign the existing AttachParent and defer attachment until we activate.
	 * @see bAutoManageAttachment
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category=Attachment, meta=(EditCondition="bAutoManageAttachment"))
	TWeakObjectPtr<USceneComponent> AutoAttachParent;

	/**
	 * Socket we automatically attach to on the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Attachment, meta=(EditCondition="bAutoManageAttachment"))
	FName AutoAttachSocketName;

	/**
	 * Options for how we handle our location when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment, EAttachmentRule
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attachment, meta = (EditCondition = "bAutoManageAttachment"))
	EAttachmentRule AutoAttachLocationRule;

	/**
	 * Options for how we handle our rotation when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment, EAttachmentRule
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attachment, meta = (EditCondition = "bAutoManageAttachment"))
	EAttachmentRule AutoAttachRotationRule;

	/**
	 * Options for how we handle our scale when we attach to the AutoAttachParent, if bAutoManageAttachment is true.
	 * @see bAutoManageAttachment, EAttachmentRule
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attachment, meta = (EditCondition = "bAutoManageAttachment"))
	EAttachmentRule AutoAttachScaleRule;


	/**
	 * Set AutoAttachParent, AutoAttachSocketName, AutoAttachLocationRule, AutoAttachRotationRule, AutoAttachScaleRule to the specified parameters. Does not change bAutoManageAttachment; that must be set separately.
	 * @param  Parent			Component to attach to.
	 * @param  SocketName		Socket on Parent to attach to.
	 * @param  LocationRule		Option for how we handle our location when we attach to Parent.
	 * @param  RotationRule		Option for how we handle our rotation when we attach to Parent.
	 * @param  ScaleRule		Option for how we handle our scale when we attach to Parent.
	 * @see bAutoManageAttachment, AutoAttachParent, AutoAttachSocketName, AutoAttachLocationRule, AutoAttachRotationRule, AutoAttachScaleRule
	 */
	NIAGARA_API void SetAutoAttachmentParameters(USceneComponent* Parent, FName SocketName, EAttachmentRule LocationRule, EAttachmentRule RotationRule, EAttachmentRule ScaleRule) override;

	virtual void SetUseAutoManageAttachment(bool bAutoManage) override { bAutoManageAttachment = bAutoManage; }

	float PreviewLODDistance;
	float PreviewMaxDistance;
	uint32 bEnablePreviewLODDistance : 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = Compilation)
	uint32 bWaitForCompilationOnActivate : 1;

	UPROPERTY(EditAnywhere, Category = "Niagara Utilities", meta = (UIMin="1", ClampMin="1", DisplayName="NumFramesToCapture"))
	TOptional<int32> SimCacheDebugNumFramesToCapture;
#endif

	/**
	Sets the simulation cache to use for the component.
	A null SimCache parameter will clear the active simulation cache.
	When clearing a simulation cache that has been running you may wish to reset or continue, this option is only
	valid when using a full simulation cache.  A renderer only cache will always reset as we can not continue the
	simulation due to missing simulation data.
	*/
	UFUNCTION(BlueprintCallable, Category = SimCache)
	NIAGARA_API void SetSimCache(UNiagaraSimCache* SimCache, bool bResetSystem = false);

	/**
	Get the active simulation cache, will return null if we do not have an active one.
	*/
	UFUNCTION(BlueprintCallable, Category = SimCache)
	NIAGARA_API UNiagaraSimCache* GetSimCache() const;

	/**
	Clear any active simulation cache.
	When clearing a simulation cache that has been running you may wish to reset or continue, this option is only
	valid when using a full simulation cache.  A renderer only cache will always reset as we can not continue the
	simulation due to missing simulation data.
	*/
	UFUNCTION(BlueprintCallable, Category = SimCache)
	void ClearSimCache(bool bResetSystem = false) { SetSimCache(nullptr, bResetSystem); }

	// DO NOT USE THIS IN EXTERNAL CODE as it is subject to change
	// Methods to handle interop between Actor/Entity/Desc
	UE_DEPRECATED(5.5, "This function is experimental and subject to change.")
	INiagaraSceneComponentUtils* CreateSceneComponentUtils();
	UE_DEPRECATED(5.5, "This function is experimental and subject to change.")
	void SetCreateSceneComponentUtilsFunction(TFunction<INiagaraSceneComponentUtils*()> Function) { CreateSceneComponentUtilsFunction = Function; }
	// DO NOT USE THIS IN EXTERNAL CODE as it is subject to change

	/** Set whether this component is allowed to perform scalability checks and potentially be culled etc. Occasionally it is useful to disable this for specific components. E.g. Effects on the local player. */
	UFUNCTION(BlueprintSetter, Category = Scalability, meta = (Keywords = "LOD scalability"))
	NIAGARA_API void SetAllowScalability(bool bAllow);
	
	UFUNCTION(BlueprintGetter)
	NIAGARA_API bool GetAllowScalability()const;
	
	UFUNCTION(BlueprintSetter, Category = Scalability, meta = (Keywords = "LOD scalability"))
	NIAGARA_API void SetForceLocalPlayerEffect(bool bIsPlayerEffect);
	
	UFUNCTION(BlueprintGetter)
	NIAGARA_API bool GetForceLocalPlayerEffect()const;

	inline bool IsRegisteredWithScalabilityManager()const { return ScalabilityManagerHandle != INDEX_NONE; }
	inline int32 GetScalabilityManagerHandle()const { return ScalabilityManagerHandle; }

	inline void BeginUpdateContextReset(){ bDuringUpdateContextReset = true; }
	inline void EndUpdateContextReset(){ bDuringUpdateContextReset = false; }

#if WITH_NIAGARA_DEBUGGER	
	//Cache our scalability state in the component so we have access to it easily and also after it has been removed from the scalability manager.
	FNiagaraScalabilityState DebugCachedScalabilityState;
#endif

	inline bool IsUsingCullProxy()const { return CullProxy != nullptr; }

	NIAGARA_API bool ResolveOwnerAllowsScalability(bool bRegister=true);

private:
	/** Did we try and activate but fail due to the asset being not yet ready. Keep looping.*/
	uint32 bAwaitingActivationDueToNotReady : 1;
	/** Should we try and reset when ready? */
	uint32 bActivateShouldResetWhenReady : 1;

	/** Did we auto attach during activation? Used to determine if we should restore the relative transform during detachment. */
	uint32 bDidAutoAttach : 1;

	/** Controls whether we allow scalability culling for this component. If enabled, this component's FX may be culled due to things such as distance, visibility, instance counts and performance. */
	UPROPERTY(EditAnywhere, Category=Niagara, BlueprintGetter=GetAllowScalability, BlueprintSetter=SetAllowScalability)
	uint32 bAllowScalability : 1;

	/** Flag allowing us to force this Effect to be considered a LocalPlayer Effect. This can prevent it being scalabiltiy culled in some situations etc. */
	//UPROPERTY(EditAnywhere, Category = Niagara, BlueprintGetter = GetForceLocalPlayerEffect, BlueprintSetter = SetForceLocalPlayerEffect)
	uint32 bForceLocalPlayerEffect : 1;

	/** Whether the owner of this component allows it to be scalability culled. We optionally do not allow scalability culling for components on or attached to the local player. */
	uint32 bOwnerAllowsScalabiltiy : 1;

	/** True if this component has been culled by the scalability manager. */
	uint32 bIsCulledByScalability : 1;

	/** True if this component has been fully completed. If false then we are still pending a final call to OnSystemComplete(). */
	uint32 bIsFullyComplete : 1;

	/** Flag to mark us as currently changing auto attachment as part of Activate/Deactivate so we don't reset in the OnAttachmentChanged() callback. */
	//uint32 bIsChangingAutoAttachment : 1;

	/** True if we're currently inside an update context reset. This will prevent us from doing some completion work such as releaseing to the pool or auto destroy etc during a reset. */
	uint32 bDuringUpdateContextReset : 1;

	/** Stores the current state for pause/unpause desired by the use. Allows us to pause/unpause correctly while also pausing/unpausing via scalability. */
	uint32 bDesiredPauseState : 1;

	/** Request recache the PSOs */
	uint32 bRecachePSOs : 1;

	/** Restore relative transform from auto attachment and optionally detach from parent (regardless of whether it was an auto attachment). */
	NIAGARA_API void CancelAutoAttachment(bool bDetachFromParent);

	/** Saved relative transform before auto attachment. Used during detachment to restore the transform if we had automatically attached. */
	FVector SavedAutoAttachRelativeLocation;
	FRotator SavedAutoAttachRelativeRotation;
	FVector SavedAutoAttachRelativeScale3D;

	FDelegateHandle AssetExposedParametersChangedHandle;

	UNiagaraEffectType* ScalabilityEffectType = nullptr;
	int32 ScalabilityManagerHandle;

	float ForceUpdateTransformTime;
	FBox CurrLocalBounds;

	FBox SystemFixedBounds;

	struct FEmitterOverrideInfo
	{
		explicit FEmitterOverrideInfo(FName InEmitterName) : EmitterName(InEmitterName) {}

		FName	EmitterName;
		bool	bEnabled = true;
		FBox	FixedBounds = FBox(EForceInit::ForceInit);
	};
	TArray<FEmitterOverrideInfo> EmitterOverrideInfos;
	const FEmitterOverrideInfo* FindEmitterOverrideInfo(FName EmitterName) const;
	FEmitterOverrideInfo* FindEmitterOverrideInfo(FName EmitterName);
	FEmitterOverrideInfo& FindOrAddEmitterOverrideInfo(FName EmitterName);

	float CustomTimeDilation = 1.0f;

	TFunction<INiagaraSceneComponentUtils*()> CreateSceneComponentUtilsFunction;

	UPROPERTY(transient)
	TObjectPtr<class UNiagaraSimCache> SimCache;

	UPROPERTY(transient)
	TObjectPtr<class UNiagaraCullProxyComponent> CullProxy;

	UPROPERTY(transient)
	TArray<TObjectPtr<UNiagaraParameterCollectionInstance>> ParameterCollectionOverrides;

	NIAGARA_API void CreateCullProxy(bool bForce = false);
	NIAGARA_API void DestroyCullProxy();

public:
	NIAGARA_API FParticlePerfStatsContext GetPerfStatsContext();
};

inline bool UNiagaraComponent::GetPreviewLODDistanceEnabled()const { return bEnablePreviewLODDistance; }
inline float UNiagaraComponent::GetPreviewLODDistance()const { return bEnablePreviewLODDistance ? PreviewLODDistance : 0.0f; }
