// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectKey.h"
#include "NiagaraCommon.h"
#include "NiagaraSystem.h"
#include "NiagaraComponent.h"
#include "NiagaraComponentPool.generated.h"

#define ENABLE_NC_POOL_DEBUGGING (!UE_BUILD_SHIPPING)


USTRUCT()
struct FNCPoolElement
{
	GENERATED_BODY()

	TStrongObjectPtr<UNiagaraComponent> Component;
	double LastUsedTime;

	FNCPoolElement()
		: Component(nullptr), LastUsedTime(0.0f)
	{

	}
	FNCPoolElement(UNiagaraComponent* InNC, double InLastUsedTime)
		: Component(InNC), LastUsedTime(InLastUsedTime)
	{

	}
};


USTRUCT()
struct FNCPool
{
	GENERATED_BODY()

	//Collection of all currently allocated, free items ready to be grabbed for use.
	//TODO: Change this to a FIFO queue to get better usage. May need to make this whole class behave similar to TCircularQueue.
	UPROPERTY(transient)
	TArray<FNCPoolElement> FreeElements;

public:

	void Cleanup();

	/** Gets a component from the pool ready for use. */
	UNiagaraComponent* Acquire(UWorld* World, UNiagaraSystem* Template, ENCPoolMethod PoolingMethod, bool bForceNew=false);

	/** Returns a component to the pool. */
	void Reclaim(UNiagaraComponent* NC, const double CurrentTimeSeconds);

	/** Forces us to remove a component, returns true if the component exists inside the pool. */
	bool RemoveComponent(UNiagaraComponent* Component);

	/** Kills any components that have not been used since the passed KillTime. */
	void KillUnusedComponents(double KillTime, UNiagaraSystem* Template);

	int32 NumComponents() { return FreeElements.Num(); }
};

UCLASS(Transient, MinimalAPI)
class UNiagaraComponentPool : public UObject
{
	GENERATED_UCLASS_BODY()

private:
	TMap<TObjectKey<UNiagaraSystem>, FNCPool> WorldParticleSystemPools;

	double LastParticleSytemPoolCleanTime;

public:
	static NIAGARA_API bool Enabled();

	NIAGARA_API virtual ~UNiagaraComponentPool() override;

	NIAGARA_API void Cleanup(UWorld* World);

	/** Clear all free entires of the specified system. */
	NIAGARA_API void ClearPool(UNiagaraSystem* System);

	NIAGARA_API void PrimePool(UNiagaraSystem* Template, UWorld* World);
	NIAGARA_API UNiagaraComponent* CreateWorldParticleSystem(UNiagaraSystem* Template, UWorld* World, ENCPoolMethod PoolingMethod);

	/** Called when an in-use particle component is finished and wishes to be returned to the pool. */
	NIAGARA_API void ReclaimWorldParticleSystem(UNiagaraComponent* Component);

	/** Notification that the component is being destroyed but has relevance to the component pool. */
	NIAGARA_API void PooledComponentDestroyed(UNiagaraComponent* Component);

	/** Remove any components that are using this system. */
	NIAGARA_API void RemoveComponentsBySystem(UNiagaraSystem* System);

	/** Dumps the current state of the pool to the log. */
	NIAGARA_API void Dump();

#if ENABLE_NC_POOL_DEBUGGING
	//Array of currently in flight components that will auto release.
	TArray<TWeakObjectPtr<UNiagaraComponent>> InUseComponents_Auto;

	//Array of currently in flight components that need manual release.
	TArray<TWeakObjectPtr<UNiagaraComponent>> InUseComponents_Manual;

	/** Keeping track of max in flight systems to help inform any future pre-population we do. */
	int32 MaxUsed = 0;
#endif
};
