// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraComponentPoolMethodEnum.generated.h"

UENUM()
enum class ENCPoolMethod : uint8
{
	/**
	* The component will be created fresh and not allocated from the pool.
	*/
	None,

	/**
	* The component is allocated from the pool and will be automatically released back to it.
	* User need not handle this any more that other NCs but interaction with the NC after the tick it's spawned in are unsafe.
	* This method is useful for one-shot fx that you don't need to keep a reference to and can fire and forget.
	*/
	AutoRelease,

	/**
	* NC is allocated from the pool but will NOT be automatically released back to it. User has ownership of the NC and must call ReleaseToPool when finished with it otherwise the NC will leak.
	* Interaction with the NC after it has been released are unsafe.
	* This method is useful for persistent FX that you need to modify parameters upon etc over it's lifetime.
	*/
	ManualRelease,

	/**
	Special entry allowing manual release NCs to be manually released but wait until completion to be returned to the pool.
	*/
	ManualRelease_OnComplete UMETA(Hidden),

	/**
	Special entry that marks a NC as having been returned to the pool. All NCs currently in the pool are marked this way.
	*/
	FreeInPool UMETA(Hidden),
};

