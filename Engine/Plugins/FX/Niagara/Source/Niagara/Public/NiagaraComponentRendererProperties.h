// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraEmitter.h"
#include "NiagaraRendererProperties.h"
#include "Components/PointLightComponent.h"
#include "NiagaraComponentRendererProperties.generated.h"

class FNiagaraEmitterInstance;
class SWidget;

USTRUCT()
struct FNiagaraComponentPropertyBinding
{
	GENERATED_USTRUCT_BODY();

	UPROPERTY()
	FNiagaraVariableAttributeBinding AttributeBinding;

	/** Actual name of the property we are bound to */
	UPROPERTY()
	FName PropertyName;

	/** Type of the target property (used for auto-conversion) */
	UPROPERTY()
	FNiagaraTypeDefinition PropertyType;

	/** (Optional) name of the property setter as defined in the metadata */
	UPROPERTY()
	FName MetadataSetterName;

	/** (Optional) If we have a setter with more than one parameter, this holds the default values of any optional function parameters */
	UPROPERTY()
	TMap<FString, FString> PropertySetterParameterDefaults;
};

struct FNiagaraPropertySetter
{
	UFunction* Function;
	bool bIgnoreConversion = false;

};

UCLASS(editinlinenew, MinimalAPI, meta = (DisplayName = "Component Renderer"))
class UNiagaraComponentRendererProperties : public UNiagaraRendererProperties
{
public:
	GENERATED_BODY()

	UNiagaraComponentRendererProperties();
	virtual ~UNiagaraComponentRendererProperties() override;

	//UObject Interface
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	//UObject Interface END

	static void InitCDOPropertiesAfterModuleStartup();

	//~ UNiagaraRendererProperties interface
	virtual FNiagaraRenderer* CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) override;
	virtual class FNiagaraBoundsCalculator* CreateBoundsCalculator() override { return nullptr; }
	virtual bool IsSimTargetSupported(ENiagaraSimTarget InSimTarget) const override { return (InSimTarget == ENiagaraSimTarget::CPUSim); };
	virtual void GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const override {};
#if WITH_EDITORONLY_DATA
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& e) override;

	virtual void GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const override;
	virtual const FSlateBrush* GetStackIcon() const override;
	virtual FText GetWidgetDisplayName() const override;

	virtual TArray<FNiagaraVariable> GetBoundAttributes() const override;
	virtual const TArray<FNiagaraVariable>& GetOptionalAttributes() override;
#endif // WITH_EDITORONLY_DATA

	/** The scene component class to instantiate */
	UPROPERTY(EditAnywhere, Category = "Component Rendering")
	TSubclassOf<USceneComponent> ComponentType;

	/** The max number of components that this emitter will spawn or update each frame. */
	UPROPERTY(EditAnywhere, Category = "Component Rendering", meta = (ClampMin = 1))
	uint32 ComponentCountLimit;

	/** Which attribute should we use to check if component rendering should be enabled for a particle? This can be used to control the spawn-rate on a per-particle basis. */
	UPROPERTY(EditAnywhere, Category = "Component Rendering")
	FNiagaraVariableAttributeBinding EnabledBinding;

	/** Which attribute should we use to check if component rendering should be enabled for a particle? This can be used to control the spawn-rate on a per-particle basis. */
	UPROPERTY(EditAnywhere, Category = "Component Rendering")
	FNiagaraVariableAttributeBinding RendererVisibilityTagBinding;

	/** If true then components will not be automatically assigned to the first particle available, but try to stick to the same particle based on its unique id.
	 * Disabling this option is faster, but a particle can get a different component each tick, which can lead to problems with for example motion blur. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Component Rendering")
	bool bAssignComponentsOnParticleID;

	/** If true then new components can only be created on newly spawned particles. If a particle is not able to create a component on it's first frame (e.g. because the component
	 * limit was reached) then it will be blocked from spawning a component on subsequent frames. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Component Rendering", DisplayName="Only Create Components on Particle Spawn", meta = (EditCondition = "bAssignComponentsOnParticleID"))
	bool bCreateComponentFirstParticleFrame = false;
	
	/** 
	If true then components will only be activated when newly acquired. e.g. on particle spawn or when the particle enables/disables the component.
	If false, components will be always kept active while they are used by an enabled particle.
	This can be useful for component types that can internally activate and deactivate during their lifetimes while still being used by an enabled particle.
	For example NiagaraComponents that deactivate/reactivate according to scalability settings.
	This setting is only valid when bAssignComponentsOnParticleID is true.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Component Rendering", meta = (EditCondition = "bAssignComponentsOnParticleID"))
	bool bOnlyActivateNewlyAquiredComponents;

#if WITH_EDITORONLY_DATA

	/** If true then the editor visualization is enabled for the component; has no effect in-game. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Component Rendering")
	bool bVisualizeComponents;

	UPROPERTY()
	bool bOnlyCreateComponentsOnParticleSpawn_DEPRECATED;

#endif

	/** If a render visibility tag is present, particles whose tag matches this value will be visible in this renderer. */
	UPROPERTY(EditAnywhere, Category = "Component Rendering")
	int32 RendererVisibility;

	/** The object template used to create new components at runtime. */
	UPROPERTY(Export, Instanced, EditAnywhere, Category = "Component Properties")
	TObjectPtr<USceneComponent> TemplateComponent;

	UPROPERTY()
	TArray<FNiagaraComponentPropertyBinding> PropertyBindings;

	TMap<FName, FNiagaraPropertySetter> SetterFunctionMapping;

	NIAGARA_API static bool IsConvertible(const FNiagaraTypeDefinition& SourceType, const FNiagaraTypeDefinition& TargetType);
	NIAGARA_API static FNiagaraTypeDefinition ToNiagaraType(FProperty* Property);
	static FNiagaraTypeDefinition GetFColorDef();
	static FNiagaraTypeDefinition GetFRotatorDef();
	static FNiagaraTypeDefinition GetFVector2DDef();
	static FNiagaraTypeDefinition GetFVectorDef();
	static FNiagaraTypeDefinition GetFVector4Def();
	static FNiagaraTypeDefinition GetFVector3fDef();
	static FNiagaraTypeDefinition GetFQuatDef();
	static FNiagaraTypeDefinition GetDoubleDef();

	virtual void CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData) override;

#if WITH_EDITORONLY_DATA
	virtual bool IsSupportedVariableForBinding(const FNiagaraVariableBase& InSourceForBinding, const FName& InTargetBindingName) const override;
#endif

	virtual bool NeedsSystemPostTick() const override { return true; }
	virtual bool NeedsSystemCompletion() const override { return true; }

protected:
	virtual void UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit = false) override;

private:
	static TArray<TWeakObjectPtr<UNiagaraComponentRendererProperties>> ComponentRendererPropertiesToDeferredInit;

	void CreateTemplateComponent();
	void UpdateTemplateCoordinateSpace();

	void UpdateSetterFunctions();

	bool HasPropertyBinding(FName PropertyName) const;

#if WITH_EDITOR
	/** Callback for whenever any blueprint components are reinstanced */
	void OnObjectsReplacedCallback(const TMap<UObject*, UObject*>& ReplacementsMap);
	void OnObjectsReplacedInternal(const TMap<UObject*, UObject*>& ReplacementsMap);

#endif

	std::atomic_bool IsSetterMappingDirty;
};
