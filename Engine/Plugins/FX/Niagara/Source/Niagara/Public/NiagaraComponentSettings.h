// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"

class UNiagaraEmitter;
class UNiagaraSystem;
struct FVersionedNiagaraEmitterData;

namespace FNiagaraComponentSettings
{
	extern void UpdateSettings();
	extern void RequestUpdateSettings(IConsoleVariable*);
	extern NIAGARA_API bool IsSystemAllowedToRun(const UNiagaraSystem* System);
	extern NIAGARA_API bool IsEmitterAllowedToRun(const FVersionedNiagaraEmitterData& EmitterData, const UNiagaraEmitter& NiagaraEmitter, const FGuid& EmitterHandleId);
};
