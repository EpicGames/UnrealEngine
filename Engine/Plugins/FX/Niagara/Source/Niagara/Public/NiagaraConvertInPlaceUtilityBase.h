// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Internationalization/Text.h"
#include "NiagaraConvertInPlaceUtilityBase.generated.h"

class UNiagaraScript;
class UNiagaraNodeFunctionCall;
class UNiagaraClipboardContent;
class UNiagaraStackScriptHierarchyRoot;

UCLASS(Abstract, MinimalAPI)
class UNiagaraConvertInPlaceUtilityBase : public UObject
{
	GENERATED_BODY()
public:
	UE_DEPRECATED(5.7, "Please implement ConvertInputs() instead and override UsesLegacyConversion().")
	virtual bool Convert(UNiagaraScript* InOldScript, UNiagaraClipboardContent* InOldClipboardContent, UNiagaraScript* InNewScript, UNiagaraStackScriptHierarchyRoot* InHierarchyRoot, UNiagaraClipboardContent* InNewClipboardContent, UNiagaraNodeFunctionCall* InCallingNode, FText& OutMessage) { return true; };

	// true if the old, deprecated conversion method should be called
	virtual bool UsesLegacyConversion() { return true; }

	// return true if the new clipboard was changed and should be pasted 
	virtual bool ConvertInputs(UNiagaraScript* InOldScript, UNiagaraClipboardContent* InOldClipboardContent, UNiagaraScript* InNewScript, UNiagaraClipboardContent* InNewClipboardContent, FText& OutMessage) { return false; };
};
