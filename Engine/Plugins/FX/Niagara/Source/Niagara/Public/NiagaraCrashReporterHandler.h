// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSingleton.h"
#include "HAL/CriticalSection.h"

class FNiagaraSystemInstance;
class FNiagaraSystemSimulation;
class UNiagaraSystem;

#ifndef WITH_NIAGARA_CRASHREPORTER
	#define WITH_NIAGARA_CRASHREPORTER 1
#endif

#if WITH_NIAGARA_CRASHREPORTER

/** Helper object allowing easy tracking of Niagara code in it's crash reporter integration.  */
class FNiagaraCrashReporterScope
{
private:
	bool bWasEnabled = false;
public:
	explicit FNiagaraCrashReporterScope(const FNiagaraSystemInstance* Inst);
	explicit FNiagaraCrashReporterScope(const FNiagaraSystemSimulation* Sim);
	explicit FNiagaraCrashReporterScope(const UNiagaraSystem* System);
	~FNiagaraCrashReporterScope();
};

#else //WITH_NIAGARA_CRASHREPORTER

class FNiagaraCrashReporterScope
{
public:
	explicit FNiagaraCrashReporterScope(const FNiagaraSystemInstance* Inst) {}
	explicit FNiagaraCrashReporterScope(const FNiagaraSystemSimulation* Sim) {}
	explicit FNiagaraCrashReporterScope(const UNiagaraSystem* System) {}
	~FNiagaraCrashReporterScope() = default;
};

#endif //WITH_NIAGARA_CRASHREPORTER
