// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraComponent.h"

#include "NiagaraCullProxyComponent.generated.h"


/** Info on a culled Niagara Component for use by it's cull proxy. */
USTRUCT()
struct FNiagaraCulledComponentInfo
{
	GENERATED_BODY()

	TWeakObjectPtr<UNiagaraComponent> WeakComponent;
};

/**
A specialization of UNiagaraComponent that can act as a proxy for many other NiagaraComponents that have been culled by scalability. 
*/
UCLASS(MinimalAPI, NotBlueprintable)
class UNiagaraCullProxyComponent : public UNiagaraComponent
{
	GENERATED_UCLASS_BODY()

	/** Array of additional instance transforms. This component will be rendered at it's own transform and additionally at each of these transforms. */
	UPROPERTY(EditAnywhere, Category = "Niagara")
	TArray<FNiagaraCulledComponentInfo> Instances;

#if WITH_PARTICLE_PERF_CSV_STATS
	FName CSVStat_NumCullProxies = NAME_None;
#endif

	static const FName TotalCullProxiesName;
public:

	//UActorComponent Interface
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//UActorComponent Interface END

	bool RegisterCulledComponent(UNiagaraComponent* Component, bool bForce);
	void UnregisterCulledComponent(UNiagaraComponent* Component);

	void TickCullProxy();
};