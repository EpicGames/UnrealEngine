// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraDataChannel.h: Code dealing with Niagara Data Channels and their management.

Niagara Data Channels are a system for communication between Niagara Systems and with Game code/BP.

Niagara Data Channels define a common payload and other settings for a particular named Data Channel.
Niagara Data Channel Handlers are the runtime handler class that will provide access to the data channel to it's users and manage it's internal data.

Niagara Systems can read from and write to Data Channels via data interfaces.
Blueprint and game code can also read from and write to Data Channels.
Each of these writes optionally being made visible to Game, CPU and/or GPU Systems.

At the "Game" level, all data is held in LWC compatible types in AoS format.
When making this data available to Niagara Systems it is converted to SWC, SoA layout that is compatible with Niagara simulation.

Some Current limitations:

Tick Ordering:
Niagara Systems can chose to read the current frame's data or the previous frame.
Reading from the current frame allows zero latency but introduces a frame dependency, i.e. you must ensure that the reader ticks after the writer.
This frame dependency needs work to be more robust and less error prone.
Reading the previous frames data introduces a frame of latency but removes the need to tick later than the writer. Also means you're sure to get a complete frame worth of data.

==============================================================================*/

#pragma once

#include "UObject/UObjectIterator.h"
#include "RenderCommandFence.h"

#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelAsset.h"
#include "NiagaraDataChannelVariable.h"
#include "NiagaraDataChannelAccessContext.h"

#include "NiagaraDataChannel.generated.h"

#define UE_API NIAGARA_API

DECLARE_STATS_GROUP(TEXT("Niagara Data Channels"), STATGROUP_NiagaraDataChannels, STATCAT_Niagara);

//////////////////////////////////////////////////////////////////////////

UCLASS(abstract, EditInlineNew, MinimalAPI, prioritizeCategories=("Data Channel"))
class UNiagaraDataChannel : public UObject
{
public:
	GENERATED_BODY()

	//UObject Interface
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API virtual void BeginDestroy() override;
	NIAGARA_API virtual bool IsReadyForFinishDestroy() override;
#if WITH_EDITOR
	NIAGARA_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	NIAGARA_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedDataChannel) override;
#endif
	//UObject Interface End.

	const UNiagaraDataChannelAsset* GetAsset() const { return CastChecked<UNiagaraDataChannelAsset>(GetOuter()); }
	TConstArrayView<FNiagaraDataChannelVariable> GetVariables() const { return ChannelVariables; }

	/** If true, we keep our previous frame's data. Some users will prefer a frame of latency to tick dependency. */
	bool KeepPreviousFrameData() const { return bKeepPreviousFrameData; }
	
	/** Returns the type for the Access Context used by this Data Channel. */
	[[nodiscard]] virtual TNDCAccessContextType GetAccessContextType()const { return TNDCAccessContextType(FNDCAccessContextLegacy::StaticStruct()); }
	
	/** Returns a pre allocated access context that you can use for accessing this data channel. Only safe to use from the game thread and it is not safe to hold onto this context after your access. */
	[[nodiscard]] inline FNDCAccessContextInst& GetTransientAccessContext()const;

	/** Create the appropriate handler object for this data channel. */
	virtual UNiagaraDataChannelHandler* CreateHandler(UWorld* OwningWorld) const PURE_VIRTUAL(UNiagaraDataChannel::CreateHandler, {return nullptr;} );
	
	NIAGARA_API const FNiagaraDataChannelLayoutInfoPtr GetLayoutInfo()const;

	//Creates a new GameData for this NDC.
	NIAGARA_API FNiagaraDataChannelGameDataPtr CreateGameData() const;

	NIAGARA_API bool IsValid() const;

	#if WITH_NIAGARA_DEBUGGER
	void SetVerboseLogging(bool bValue){ bVerboseLogging = bValue; }
	bool GetVerboseLogging()const { return bVerboseLogging; }
	#endif

	template<typename TFunc>
	static void ForEachDataChannel(TFunc Func);

	bool ShouldEnforceTickGroupReadWriteOrder() const {return bEnforceTickGroupReadWriteOrder;}
	
	/** If we are enforcing tick group read/write ordering the this returns the final tick group that this NDC can be written to. All reads must happen in Tick groups after this or next frame. */
	ETickingGroup GetFinalWriteTickGroup() const { return FinalWriteTickGroup; }

#if WITH_EDITORONLY_DATA
	/** Can be used to track structural changes that would need recompilation of downstream assets. */
	FGuid GetVersion() const { return VersionGuid; }
#endif
	
private:

	//TODO: add default values for editor previews
	
	/** The variables that define the data contained in this Data Channel. */
	UPROPERTY(EditAnywhere, Category = "Data Channel", meta=(EnforceUniqueNames = true))
	TArray<FNiagaraDataChannelVariable> ChannelVariables;

	/** If true, we keep our previous frame's data. This comes at a memory and performance cost but allows users to avoid tick order dependency by reading last frame's data. Some users will prefer a frame of latency to tick order dependency. */
	UPROPERTY(EditAnywhere, Category = "Data Channel")
	bool bKeepPreviousFrameData = true;

	/** If true we ensure that all writes happen in or before the Tick Group specified in EndWriteTickGroup and that all reads happen in tick groups after this. */
	UPROPERTY(EditAnywhere, Category = "Data Channel")
	bool bEnforceTickGroupReadWriteOrder = false;

	/** The final tick group that this data channel can be written to. */
	UPROPERTY(EditAnywhere, Category = "Data Channel", meta=(EditCondition="bEnforceTickGroupReadWriteOrder"))
	TEnumAsByte<ETickingGroup> FinalWriteTickGroup = ETickingGroup::TG_EndPhysics;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid VersionGuid;

	UPROPERTY(meta=(DeprecatedProperty))
	TArray<FNiagaraVariable> Variables_DEPRECATED;
#endif

	/**
	Data layout for payloads in Niagara datasets.
	*/
	mutable FNiagaraDataChannelLayoutInfoPtr LayoutInfo;
	
	#if WITH_NIAGARA_DEBUGGER
	mutable bool bVerboseLogging = false;
	#endif

	FRenderCommandFence RTFence;

	/** A transient access context instance that calling code can use for accessing this Data Channel. GT only*/
	mutable FNDCAccessContextInst TransientAccessContext;
};

FNDCAccessContextInst& UNiagaraDataChannel::GetTransientAccessContext()const
{
	check(IsInGameThread());
	TransientAccessContext.Init(GetAccessContextType());//We need to init (or reset) ready for use.
	return TransientAccessContext;
}

template<typename TAction>
void UNiagaraDataChannel::ForEachDataChannel(TAction Func)
{
	for(TObjectIterator<UNiagaraDataChannel> It; It; ++It)
	{
		UNiagaraDataChannel* NDC = *It;
		if (NDC && 
			NDC->HasAnyFlags(RF_ClassDefaultObject | RF_Transient) == false &&
			Cast<UNiagaraDataChannelAsset>(NDC->GetOuter()) != nullptr)
		{
			Func(*It);
		}
	}
}

#undef UE_API
