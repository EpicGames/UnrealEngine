// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"

#include "NiagaraDataChannelAccessContext.generated.h"

class UNiagaraComponent;

/**
Legacy search parameters to support legacy code.
*/
USTRUCT(BlueprintType)
struct FNiagaraDataChannelSearchParameters
{
	GENERATED_BODY()

	FNiagaraDataChannelSearchParameters() = default;

	FNiagaraDataChannelSearchParameters(USceneComponent* Owner)
		: OwningComponent(Owner)
		, Location(FVector::ZeroVector)
		, bOverrideLocation(false)
	{
	}

	FNiagaraDataChannelSearchParameters(USceneComponent* Owner, FVector LocationOverride)
		: OwningComponent(Owner)
		, Location(LocationOverride)
		, bOverrideLocation(true)
	{
	}

	FNiagaraDataChannelSearchParameters(FVector InLocation)
		: OwningComponent(nullptr)
		, Location(InLocation)
		, bOverrideLocation(true)
	{
	}

	[[nodiscard]] NIAGARA_API FVector GetLocation()const;
	[[nodiscard]] inline USceneComponent* GetOwner()const { return OwningComponent; }

	/** In cases where there is an owning component such as an object spawning from itself etc, then we pass that component in. Some handlers may only use it's location but others may make use of more data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	TObjectPtr<USceneComponent> OwningComponent = nullptr;

	/** In cases where there is no owning component for data being read or written to a data channel, we simply pass in a location. We can also use this when bOverrideLocaiton is set. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	FVector Location = FVector::ZeroVector;

	/** If true, even if an owning component is set, the data channel should use the Location value rather than the component location. If this is false, the NDC will get any location needed from the owning component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	uint32 bOverrideLocation : 1 = false;
};


/**
Base class for context used when retrieving a specific set of Data Channel Data to read or write.
Many Data Channel types will have multiple internal sets of data and these parameters control which the Channel should return to users for access.
An example of this would be the Islands Data Channel type which will subdivide the world and have a different set of data for each sub division.
It will return to users the correct data for their location based on these parameters.
NDC Types must specify which of these access context types they require.
The base struct contains only data needed for internal systems to function. 
Beyond this you are free to add anything else you need.
Some data can be intended as input to the search/internal routing or data creation. This should be marked with "NDCAccessContextInput".
Other data can be intended as output to store derived or transient state and pass back to the user. This should be marked with "NDCAccessContextOutput".
*/
USTRUCT(BlueprintType, meta=(Hidden))
struct FNDCAccessContextBase
{
	GENERATED_BODY()

public:

	FNDCAccessContextBase() = default;
	[[nodiscard]] explicit FNDCAccessContextBase(USceneComponent* Owner)
		: OwningComponent(Owner)
		, bIsAutoLinkingSystem(false)
	{
	}

	[[nodiscard]] inline USceneComponent* GetOwner()const { return OwningComponent; }

	UPROPERTY(BlueprintReadWrite, Transient, Category = "Input", meta=(NDCAccessContextInput))
	TObjectPtr<USceneComponent> OwningComponent = nullptr;
	
	/** 
	If true, this access is for a Niagara System that wishes to "Auto link" i.e. not do a proper lookup, just use the currently initializing map entry or the spawned component lookup.
	This is used by NDC Data Interfaces in systems spawned by NDCs to make linking back to the originating NDC entry easier.
	*/
	UPROPERTY(transient)
	uint32 bIsAutoLinkingSystem : 1 = false;
};

/** Utility class to wrap a weak pointer to a Niagara Component allowing FNDCAccessContext to hold an array of weak spawned systems. */
USTRUCT(BlueprintType)
struct FNDCSpawnedSystemRef
{
	GENERATED_BODY()
	
	FNDCSpawnedSystemRef() = default;
	[[nodiscard]] FNDCSpawnedSystemRef(UNiagaraComponent* Component);

	UPROPERTY(BlueprintReadWrite, Transient, Category = "Output")
	TWeakObjectPtr<UNiagaraComponent> SpawnedSystem;

	[[nodiscard]] inline UNiagaraComponent* Get()const{ return SpawnedSystem.Get(); }
};

/**
Basic AccessContext type providing a little more utility than the base.
If you require more data than this you can subclass this or replace entirely and subclass FNDCAccessContextBase.
*/
USTRUCT(BlueprintType)
struct FNDCAccessContext : public FNDCAccessContextBase
{
	GENERATED_BODY()

	FNDCAccessContext() = default;
	[[nodiscard]] explicit FNDCAccessContext(USceneComponent* Owner)
		: FNDCAccessContextBase(Owner)
	{
	}

	[[nodiscard]] explicit FNDCAccessContext(USceneComponent* Owner, FVector LocationOverride)
		: FNDCAccessContextBase(Owner)
		, Location(LocationOverride)
		, bOverrideLocation(true)
		, bOverrideSystemToSpawn(false)
	{
	}

	[[nodiscard]] explicit FNDCAccessContext(FVector InLocation)
		: FNDCAccessContextBase(nullptr)
		, Location(InLocation)
		, bOverrideLocation(true)
		, bOverrideSystemToSpawn(false)
	{
	}

	[[nodiscard]] NIAGARA_API FVector GetLocation()const;

	/** In cases where there is no owning component for data being read or written to a data channel, we simply pass in a location. We can also use this when bOverrideLocaiton is set. */
	UPROPERTY(BlueprintReadWrite, Transient, Category = "Input", meta=(NDCAccessContextInput))
	FVector Location = FVector::ZeroVector;

	/** If true, even if an owning component is set, the data channel should use the Location value rather than the component location. If this is false, the NDC will get any location needed from the owning component. */
	UPROPERTY(BlueprintReadWrite, Transient, Category = "Input", meta=(NDCAccessContextInput))
	uint32 bOverrideLocation : 1 = false;

	/** If true, we'll override the system to spawn for this NDC write. If false, we'll use the system defined in the NDC asset, if there is one. */
	UPROPERTY(BlueprintReadWrite, Category = "Input", meta = (InlineEditCondition, NDCAccessContextInput))
	uint32 bOverrideSystemToSpawn : 1 = false;
	
	/** If true, SpawnedSystems will be filled with both existing and newly spawned systems. If false, SpawnedSystems will contain only newly spawned systems. */
	UPROPERTY(BlueprintReadWrite, Category = "Input", meta = (NDCAccessContextInput))
	uint32 bReturnExistingSystems : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input", meta = (EditCondition="bOverrideSystemToSpawn", NDCAccessContextInput, AllowedClasses = "/Script/Niagara.NiagaraSystem,/Script/Niagara.NiagaraSystemCollection"))
	TObjectPtr<UObject> SystemToSpawn;

	/** 
	The Niagara System spawned in response to this access, if any. Allows callers to set parameters etc on this system. It is not safe to store and use this component reference later. It can be reclaimed by the system at any time.
	If bReturnExistingSystems is set, this will contain both new and existing systems matching this access context.
	*/
	UPROPERTY(BlueprintReadWrite, Transient, Category = "Output", meta = (NDCAccessContextOutput, ShowOnlyInnerProperties))
	TArray<FNDCSpawnedSystemRef> SpawnedSystems;
	
	template<typename TAction>
	void ForEachSpawnedSystem(TAction Func) const;

	template<typename TContainerType>
	void GetSpawnedSystems(TContainerType& Container) const;
};

/** Special legacy access context type. Used only to facilitate legacy NDCs to be usable by the new API. */
USTRUCT(BlueprintType)
struct FNDCAccessContextLegacy : public FNDCAccessContextBase
{
	GENERATED_BODY()

	FNDCAccessContextLegacy() = default;
	[[nodiscard]] explicit FNDCAccessContextLegacy(USceneComponent* Owner)
		: FNDCAccessContextBase(Owner)
	{
	}

	[[nodiscard]] explicit FNDCAccessContextLegacy(USceneComponent* Owner, FVector LocationOverride)
		: FNDCAccessContextBase(Owner)
		, Location(LocationOverride)
		, bOverrideLocation(true)
	{
	}

	[[nodiscard]] explicit FNDCAccessContextLegacy(FVector InLocation)
		: FNDCAccessContextBase(nullptr)
		, Location(InLocation)
		, bOverrideLocation(true)
	{
	}

	[[nodiscard]] NIAGARA_API FVector GetLocation()const;

	/** In cases where there is no owning component for data being read or written to a data channel, we simply pass in a location. We can also use this when bOverrideLocaiton is set. */
	UPROPERTY(BlueprintReadWrite, Transient, Category = "Input", meta = (NDCAccessContextInput))
	FVector Location = FVector::ZeroVector;

	/** If true, even if an owning component is set, the data channel should use the Location value rather than the component location. If this is false, the NDC will get any location needed from the owning component. */
	UPROPERTY(BlueprintReadWrite, Transient, Category = "Input", meta = (NDCAccessContextInput))
	uint32 bOverrideLocation : 1 = false;
};

//Describes a NDC Access Context Type.
//Wrapper around a UScriptStruct* that checks for a valid type.
struct TNDCAccessContextType
{
	TNDCAccessContextType() = default;
	[[nodiscard]] explicit TNDCAccessContextType(UScriptStruct* InStruct)
		: AccessContextStruct(InStruct && InStruct->IsChildOf(FNDCAccessContextBase::StaticStruct()) ? InStruct : nullptr)
	{
		check(AccessContextStruct);
	}

	[[nodiscard]] const UScriptStruct* Get()const { return AccessContextStruct; }

	//Returns the underlying struct as non-const. Only use to inter operate with code that stores structs without constness.
	[[nodiscard]] UScriptStruct* Get_MutableInteropOnly()const { return const_cast<UScriptStruct*>(AccessContextStruct); }

private:
	const UScriptStruct* AccessContextStruct = nullptr;
};

/**
Context object for Niagara and Game code to access a Niagara Data Channel.
Can be customized per Data Channel to provide specific control for each Data Channel type.
Can include input data from accessing code such as a location, game play tags etc that can influence internal data routing/partitions.
Can also include output data allowing the NDC to pass information back to the accessing code.
*/
USTRUCT(BlueprintType)
struct FNDCAccessContextInst
{
	GENERATED_BODY()

	FNDCAccessContextInst() = default;
	[[nodiscard]] FNDCAccessContextInst(TNDCAccessContextType AccessContextType)
	{
		AccessContext.InitializeAs(AccessContextType.Get());
	}

	inline bool Init(TNDCAccessContextType ParamsType)
	{
		AccessContext.InitializeAs(ParamsType.Get());
		return AccessContext.IsValid();
	}
	
	[[nodiscard]] bool operator==(const FNDCAccessContextInst&) const = default;

	inline void Reset()
	{
		AccessContext.Reset();
	}

	[[nodiscard]] inline bool IsValid()const { return AccessContext.IsValid(); }

	template<typename T = FNDCAccessContextBase>
	[[nodiscard]] inline T& GetChecked() { return AccessContext.GetMutable<T>(); }

	template<typename T = FNDCAccessContextBase>
	[[nodiscard]] inline T* Get() { return AccessContext.GetMutablePtr<T>();	}

	[[nodiscard]] inline const UScriptStruct* GetScriptStruct()const { return AccessContext.GetScriptStruct(); }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Access Context", meta = (ShowOnlyInnerProperties, BaseStruct = "/Script/Niagara.NDCAccessContextBase", ExcludeBaseStruct))
	FInstancedStruct AccessContext;
};