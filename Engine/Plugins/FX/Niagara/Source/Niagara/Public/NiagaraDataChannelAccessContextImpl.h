// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannelAccessContext.h"

template<typename TAction>
void FNDCAccessContext::ForEachSpawnedSystem(TAction Func) const
{
	for ( const FNDCSpawnedSystemRef& SysRef : SpawnedSystems )
	{
		if (UNiagaraComponent* Comp  = SysRef.Get())
		{
			Func(Comp);
		}
	}
}

template<typename TContainerType>
void FNDCAccessContext::GetSpawnedSystems(TContainerType& Container)const
{
	for ( const FNDCSpawnedSystemRef& SysRef : SpawnedSystems )
	{
		if (UNiagaraComponent* Comp  = SysRef.Get())
		{
			Container.Add(Comp);
		}
	}
}