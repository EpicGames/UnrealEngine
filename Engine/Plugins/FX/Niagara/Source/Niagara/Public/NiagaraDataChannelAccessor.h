// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelGameData.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelAccessor.generated.h"

#define UE_API NIAGARA_API

/** 
Initial simple API for reading and writing data in a data channel from game code / BP. 
Likely to be replaced in the near future with a custom BP node and a helper struct.
*/


UCLASS(BlueprintType, MinimalAPI)
class UNiagaraDataChannelReader : public UObject
{
	GENERATED_BODY()
private:
	friend UNiagaraDataChannelHandler;
	
	FNiagaraDataChannelDataPtr Data = nullptr;
	bool bReadingPreviousFrame = false;

	template<typename T>
	bool ReadData(const FNiagaraVariableBase& Var, int32 Index, T& OutData)const;

public:

	void Cleanup();

	UPROPERTY()
	TObjectPtr<UNiagaraDataChannelHandler> Owner;
	
	/** 
	Call before each access to the data channel to grab the correct data to read.
	NOTE: Init access is only to support legacy flows. Please use Begin Read.
	*/
	UFUNCTION(meta = (DisplayName="Init Access (Legacy)"))
	NIAGARA_API bool InitAccess(FNiagaraDataChannelSearchParameters SearchParams, bool bReadPrevFrameData);

	/** Call before each access to the data channel to grab the correct data to read. */
	NIAGARA_API bool BeginRead(FNDCAccessContextInst& AccessContext, bool bReadPrevFrameData);

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API int32 Num()const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API double ReadFloat(FName VarName, int32 Index, bool& IsValid)const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API FVector2D ReadVector2D(FName VarName, int32 Index, bool& IsValid)const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API FVector ReadVector(FName VarName, int32 Index, bool& IsValid)const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API FVector4 ReadVector4(FName VarName, int32 Index, bool& IsValid)const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API FQuat ReadQuat(FName VarName, int32 Index, bool& IsValid)const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API FLinearColor ReadLinearColor(FName VarName, int32 Index, bool& IsValid)const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API int32 ReadInt(FName VarName, int32 Index, bool& IsValid)const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API uint8 ReadEnum(FName VarName, int32 Index, bool& IsValid)const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API bool ReadBool(FName VarName, int32 Index, bool& IsValid)const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API FVector ReadPosition(FName VarName, int32 Index, bool& IsValid)const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API FNiagaraID ReadID(FName VarName, int32 Index, bool& IsValid)const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "Niagara DataChannel"))
	NIAGARA_API FNiagaraSpawnInfo ReadSpawnInfo(FName VarName, int32 Index, bool& IsValid)const;
};

UCLASS(BlueprintType, MinimalAPI)
class UNiagaraDataChannelWriter : public UObject
{
	GENERATED_BODY()
private:

	/** Local data buffers we're writing into. */
	FNiagaraDataChannelGameDataPtr Data = nullptr;

	//Starting index into our dest data. 
	int32 StartIndex = INDEX_NONE;

public:
	template<typename T>
	void WriteData(const FNiagaraVariableBase& Var, int32 Index, const T& InData)
	{
		if (ensure(Data.IsValid()))
		{
			if (FNiagaraDataChannelVariableBuffer* VarBuffer = Data->FindVariableBuffer(Var))
			{
				VarBuffer->Write<T>(Index, InData);
			}
		}
	}

	void Cleanup();

	UPROPERTY()
	TObjectPtr<UNiagaraDataChannelHandler> Owner;
	
	/** 
	Call before each batch of writes to allocate the data we'll be writing to.
	NOTE: Init Write is only to support legacy flows. Please use Begin Write.
	*/
	UFUNCTION(meta = (DisplayName="Init Write (Legacy)"))
	NIAGARA_API bool InitWrite(FNiagaraDataChannelSearchParameters SearchParams, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU, const FString& DebugSource);

	/** Call before each batch of writes to allocate the data we'll be writing to. */
	NIAGARA_API bool BeginWrite(FNDCAccessContextInst& AccessContext, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU, const FString& DebugSource);

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API int32 Num()const;

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WriteFloat(FName VarName, int32 Index, double InData);
	
	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WriteVector2D(FName VarName, int32 Index, FVector2D InData);

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WriteVector(FName VarName, int32 Index, FVector InData);

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WriteVector4(FName VarName, int32 Index, FVector4 InData);

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WriteQuat(FName VarName, int32 Index, FQuat InData);

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WriteLinearColor(FName VarName, int32 Index, FLinearColor InData);
	
	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WriteInt(FName VarName, int32 Index, int32 InData);

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WriteEnum(FName VarName, int32 Index, uint8 InData);

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WriteBool(FName VarName, int32 Index, bool InData);

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WriteSpawnInfo(FName VarName, int32 Index, FNiagaraSpawnInfo InData);

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WritePosition(FName VarName, int32 Index, FVector InData);

	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel"))
	NIAGARA_API void WriteID(FName VarName, int32 Index, FNiagaraID InData);


};



//////////////////////////////////////////////////////////////////////////

#define DEBUG_NDC_ACCESS !UE_BUILD_SHIPPING && !UE_BUILD_TEST

/*
* Below are several utility classes for reading from and writing to Niagara Data Channels from C++ code.
* Usage:
* Create a child of FNDCReaderBase or FNDCWriterBase in your own code.
* Add variable members to this class via the NDCVarWriter or NDCVarReader macros.
* The types and names used here should correspond to names and types of the data inside the NDC.
* Variables defined with FNDCVarWriter(...) will be expected and will trigger a warning if missing from the target NDC.
* Variables defined with FNDCVarWriterOptional(...) will be safely ignored with minimal overhead if missing from the NDC.
* Note: Take care to provide positions as FNiagaraPosition types rather than Vector so that they are interpreted by Niagara correctly as LWC positions.
* Note: Enum types can be used to target integer types inside the NDC.
* 
* 1.
* An examples Writer struct:
* struct FNDCExampleWriter : public FNDCWriterBase
* {
*		FNDCVarWriter(FNiagaraPosition, Position);
*		FNDCVarWriter(FVector, Velocity);
* 		FNDCVarWriter(bool, bSomeValue);* 
* };
* 
* 2.
* Make an instance of this class and initialize it with your desired Data Channel.
* It can be re-initialized with a new Data Channel in future if needed.
* Alternatively, the NDCWriter will (re)initialize itself if the Data Channel passed into BeginWrite is different (or has changed).
* class FYourUserClass
* {
*	...
*	FNDCExampleWriter ExampleWriter;
*	void Init(UNiagaraDataChannel* NDC)
*	{
*		ExampleWriter.Init(NDC);
*	}
*	...
* };
* 
* 3.
* Use this writer object to write data into the Data Channel.
* First you must call BeginWrite() to initialize the writer for the current access context.
* 
* void FYouUserClass::YourFunction()
* {
*	FNiagaraDataChannelSearchParameters SearchParameters;
*	SearchParameters.Location = MyLocation; //We find the correct NDC data to write into by using this position.
*	//SearchParameters.Owner = MyComponent; //Alternatively we can provide a component for either the location or some NDC types may pull other data from the component to find the correct NDC data.
* 
*	int32 NumItems = YourData.Num();
*	if(ExampleWriter.BeginWrite(MyWorld, DataChannel, SearchParameters, NumItems, bVisibleToGame, bVisibleToCPU, bVisibleToGPU)
*	{
*		for(int32 i=0; i < NumItems; ++i)
*		{	
* 			//The macro creates named write functions for you in your writer class.
			ExampleWriter.WritePosition(YourData[i].Position);
			ExampleWriter.WriteVelocity(YourData[i].Velocity);
			ExampleWriter.WritebSomeValue(YourData[i].bBoolValue);			
*		}
*	}
*	//Finally call end write to release and references the writer holds to it's destination data etc.
*	ExampleWriter.EndWrite();
* 
*	Alternatively you can use the scoped writer to use this much as above but without needing to remember to call EndWrite();
*	FNDCScopedWriter ScopedNDCWriter(ExampleWriter);
*	//We use the -> operator for the scoped writer to access it's internal NDCWriter.
*	if(ScopedNDCWriter->BeginWrite(MyWorld, DataChannel, SearchParameters, NumItems, bVisibleToGame, bVisibleToCPU, bVisibleToGPU)
*	{
*		for(int32 i=0; i < NumItems; ++i)
*		{
			ScopedNDCWriter->WritePosition(YourData[i].Position);
			ScopedNDCWriter->WriteVelocity(YourData[i].Velocity);
			ScopedNDCWriter->WritebSomeValue(YourData[i].bBoolValue);
*		}
*	}	
* }	//EndWrite is called automatically when we exit the scope.
* 
* Usage for Readers follows the same patterns as above.
*/

struct FNDCVarAccessorBase;

/*
Base class for NDC accessor utilities. Handles common bookkeeping for NDCWriter and NDCReader utility structs. 
See full description above.
*/
struct FNDCAccessorBase
{
friend struct FNDCVarAccessorBase;

protected:
	//Cached Layout for the NDC data. If the data channel changes layout then this will trigger a re-init of this accessor.
	FNiagaraDataChannelLayoutInfoPtr CachedLayout;

	//Array of our variable accessors, allow us to easily update all variable layout info.
	TArray<FNDCVarAccessorBase*> VariableAccessors;

	//Base index from which to access data in the found NDC buffer.
	int32 StartIndex = INDEX_NONE;

public:

	bool IsInitialized()const { return CachedLayout.IsValid() && !CachedLayout.IsUnique(); }
	bool IsInitialized(UNiagaraDataChannel* NDC)const { return CachedLayout.IsValid() && !CachedLayout.IsUnique() && NDC && CachedLayout == NDC->GetLayoutInfo(); }

	//Initialize the writer and update cached layout information.
	NIAGARA_API void Init(const UNiagaraDataChannel* DataChannel);
};


/*
Utility for accessing a specific NDC variable data from C++.
Caches offset for variable in NDC layout to speed up access.
Owning NDCReader/Writer will re-init if layout changes.
See full description above.
*/
struct FNDCVarAccessorBase
{
	friend FNDCAccessorBase;
protected:
	FNiagaraVariableBase Variable;

	int32 VarOffset:31;
	int32 bIsRequired:1;

public:

	NIAGARA_API FNDCVarAccessorBase(FNDCAccessorBase& Owner, FNiagaraVariableBase InVariable, bool bInIsRequired);
	NIAGARA_API void Init(const UNiagaraDataChannel* DataChannel);
	bool IsValid()const { return VarOffset != INDEX_NONE; }

#if DEBUG_NDC_ACCESS
	FNiagaraDataChannelLayoutInfoPtr DebugCachedLayout;
	TWeakObjectPtr<const UNiagaraDataChannel> WeakNDC;
	void ValidateLayout()const { checkf(WeakNDC->IsValid() && WeakNDC->GetLayoutInfo() == DebugCachedLayout, TEXT("NDC Variable being accessed with stale layout info.")); }
#else
	inline void ValidateLayout()const {}
#endif

	template<typename T>
	inline bool WriteInternal(const FNiagaraDataChannelGameDataPtr& Data, int32 Index, const T& InValue)
	{
		if (VarOffset != INDEX_NONE)
		{
			return Data->Write<T>(VarOffset, Index, InValue);
		}
		return false;
	}	
	
	template<typename T>
	inline bool ReadInternal(const FNiagaraDataChannelGameDataPtr& Data, int32 Index, T& OutValue, bool bPreviousFrame)
	{
		if (VarOffset != INDEX_NONE)
		{
			return Data->Read<T>(VarOffset, Index, OutValue, bPreviousFrame);
		}
		return false;
	}
};

/** Base class for NDC Reader Utility classes. See full description above. */
struct FNDCWriterBase : public FNDCAccessorBase
{
protected:
	//Current NDC target data we're accessing.
	FNiagaraDataChannelGameDataPtr Data;

	//StartIndex we're writing into. Can be non-zero in cases where we're writing into the middle of an existing buffer.
	int32 StartIndex = INDEX_NONE;

	//Number of items we're expecting/allowed to write.
	int32 Count = INDEX_NONE;

	template<typename TContext>
	inline bool BeginWrite_Internal(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, TContext& AccessContext, int32 InCount, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU);

public:

	//Finds the correct target NDC data using the current AccessContext.
	//Allocates enough space in the target NDC data.
	//Refreshes internal layout info if needed.
	UE_API bool BeginWrite(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, FNDCAccessContextInst& AccessContext, int32 InCount, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU);

	//Legacy version taking an old FNiagaraDataChannelSearchParameters struct. Works for legacy code but will not support newer NDC types or features.
	UE_API bool BeginWrite(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, FNiagaraDataChannelSearchParameters SearchParams, int32 InCount, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU);

	//Finalizes the write and free/reset any data.
	UE_API void EndWrite();

	// Debug string that can be used to track the source of data in a data channel 
	FString DebugSource;
};

/** Utility class for simplified writing of variables to a Niagara Data Channel. See full description above. */
template<typename TNDCWriterType> 
struct FNDCScopedWriter
{
private:
	TNDCWriterType& Writer;
public:
	FNDCScopedWriter(TNDCWriterType& InWriter) : Writer(InWriter){ }
	~FNDCScopedWriter()	{ Writer.EndWrite(); }
	TNDCWriterType* operator->(){ return &Writer; }
};

/** Base class for NDC Reader Utility classes. See full description above. */
struct FNDCReaderBase : public FNDCAccessorBase
{
protected:
	FNiagaraDataChannelGameDataPtr Data;
	bool bPreviousFrame = false;

	template<typename TContext>
	inline bool BeginRead_Internal(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, TContext& SearchParams, bool bInPreviousFrame);

public:

	//Finds the correct source NDC data using the current AccessContext.
	UE_API bool BeginRead(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, FNDCAccessContextInst& AccessContext, bool bInPreviousFrame);
	
	//Legacy version taking an old FNiagaraDataChannelSearchParameters struct. Works for legacy code but will not support newer NDC types or features.	
	UE_API bool BeginRead(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, FNiagaraDataChannelSearchParameters SearchParams, bool bInPreviousFrame);

	//Finalizes the read and free/reset any data.
	UE_API void EndRead();

	UE_API int32 Num()const;

};

/** Utility class for simplified reading of variables from a Niagara Data Channel. See full description above. */
template<typename TNDCReaderType>
struct FNDCScopedReader
{
private:
	TNDCReaderType& Reader;
public:
	FNDCScopedReader(TNDCReaderType& InReader) : Reader(InReader){ }
	~FNDCScopedReader() { Reader.EndRead();	}
	TNDCReaderType* operator->() { return &Reader; }
};


/** Utility class for writing variables to a Niagara Data Channel. See full description above. */
template<typename T>
struct FNDCVarWriter : public FNDCVarAccessorBase
{
public:
	FNDCVarWriter(FNDCWriterBase& Owner, FName VarName, bool bInIsRequired)
		: FNDCVarAccessorBase(Owner, FNiagaraVariableBase(FNiagaraTypeHelper::GetTypeDef<T>(), VarName), bInIsRequired)
	{}

	inline bool Write(const FNiagaraDataChannelGameDataPtr& Data, int32 Index, const typename TNiagaraTypeHelper<T>::TLWCType& InValue, int32 StartIndex, int32 Count)
	{
		if(Index >= 0 && Index < Count)
		{
			return WriteInternal<typename TNiagaraTypeHelper<T>::TLWCType>(Data, StartIndex + Index, InValue);
		}	
		return false;
	}
};

/** Utility class for reading variables from a Niagara Data Channel. See full description above. */
template<typename T>
struct FNDCVarReader : public FNDCVarAccessorBase
{
public:
	FNDCVarReader(FNDCReaderBase& Owner, FName VarName, bool bInIsRequired)
		: FNDCVarAccessorBase(Owner, FNiagaraVariableBase(FNiagaraTypeHelper::GetTypeDef<T>(), VarName), bInIsRequired)
	{}

	inline bool Read(const FNiagaraDataChannelGameDataPtr& Data, int32 Index, typename TNiagaraTypeHelper<T>::TLWCType& OutValue, bool bPreviousFrame)
	{
		return ReadInternal<typename TNiagaraTypeHelper<T>::TLWCType>(Data, Index, OutValue, bPreviousFrame);
	}
};


/** Defines an NDC Variable that we will write from C++. This variable is required and will generate errors if not present in the NDC. */
#define NDCVarWriter(VarType, VarName)\
FNDCVarWriter<VarType> VarName##Writer = FNDCVarWriter<VarType>(*this, #VarName, true);\
bool Write##VarName(int32 Index, const TNiagaraTypeHelper<VarType>::TLWCType& InData){ return VarName##Writer.Write(Data, Index, InData, StartIndex, Count); }\
const FNDCVarWriter<VarType>& Get##VarName##Writer()const{ return VarName##Writer; }

/** Defines an NDC Variable that we will write from C++. This variable is optional and will be ignored  if not present in the NDC. */
#define NDCVarWriterOptional(VarType, VarName)\
FNDCVarWriter<VarType> VarName##Writer = FNDCVarWriter<VarType>(*this, #VarName, false);\
bool Write##VarName(int32 Index, const TNiagaraTypeHelper<VarType>::TLWCType& InData){ return VarName##Writer.Write(Data, Index, InData, StartIndex, Count); }\
const FNDCVarWriter<VarType>& Get##VarName##Writer()const{ return VarName##Writer; }

/** Defines an NDC Variable that we will read from C++. This variable is required and will generate errors if not present in the NDC. */
#define NDCVarReader(VarType, VarName)\
FNDCVarReader<VarType> VarName##Reader = FNDCVarReader<VarType>(*this, #VarName, true);\
bool Read##VarName(int32 Index, TNiagaraTypeHelper<VarType>::TLWCType& OutData){ return VarName##Reader.Read(Data, Index, OutData, bPreviousFrame); }\
const FNDCVarReader<VarType>& Get##VarName##Reader()const { return VarName##Reader; }

/** Defines an NDC Variable that we will read from C++. This variable is optional and will be ignored  if not present in the NDC. */
#define NDCVarReaderOptional(VarType, VarName)\
FNDCVarReader<VarType> VarName##Reader = FNDCVarReader<VarType>(*this, #VarName, false);\
bool Read##VarName(int32 Index, TNiagaraTypeHelper<VarType>::TLWCType& OutData){ return VarName##Reader.Read(Data, Index, OutData, bPreviousFrame); }\
const FNDCVarReader<VarType>& Get##VarName##Reader()const { return VarName##Reader; }

struct FNDCPackedData
{
	int32 GetData() { return static_cast<int32>(PackedData); }//convert to int32 for passing into Niagara.

protected:
	uint32 PackedData = 0;

	template <typename T, uint32 Shift, uint32 NumBits>
	inline void Set(T Value)
	{
		static_assert(std::is_integral_v<T>, "Must use integral types for NDC packed data.");
		static_assert(!std::is_signed_v<T>, "Signed values not supported currently but could be.");
		static_assert(Shift < 32, "Shift cannot exceeded size of packed uint32.");
		static_assert(NumBits > 0, "Cannot pack zero bits.");
		static_assert(NumBits < 32, "Cannot pack more bits that the underlying uint32.");		
		static_assert(Shift + NumBits < 32, "Cannot pack more bits that the underlying uint32.");

		uint32 V;
		if constexpr(std::is_same_v<T,bool>)
		{
			static_assert(NumBits == 1, "Bools should be packed into one bit.");
			V = !!Value;
		}
		else
		{
			V = Value;
		}

		uint32 Mask = (1u << NumBits) - 1u;
		PackedData &= ~(Mask << Shift);
		PackedData |= (V & Mask) << Shift; 
	}

	template<typename T, uint32 Shift, uint32 NumBits>
	inline T Get()
	{
		static_assert(std::is_integral_v<T>, "Must use integral types for NDC packed data.");
		static_assert(!std::is_signed_v<T>, "Signed values not supported currently but could be.");
		static_assert(Shift < 32, "Shift cannot exceeded size of packed uint32.");
		static_assert(NumBits > 0, "Cannot pack zero bits.");
		static_assert(NumBits < 32, "Cannot pack more bits that the underlying uint32.");		
		static_assert(Shift + NumBits < 32, "Cannot pack more bits that the underlying uint32.");

		uint32 Mask = (1u << NumBits) - 1u;
		uint32 V = (PackedData >> Shift) & Mask;
		if constexpr(std::is_same_v<T,bool>)
		{
			static_assert(NumBits == 1, "Bools should be packed into one bit.");
			return V != 0;
		}
		else
		{
			return V;
		}
	}
};

#define NDCVarPackedValue(Name, Type, Shift, NumBits) \
void Set_##Name(Type Value){ Set<Type, Shift, NumBits>(Value); }\
Type Get_##Name(){ return Get<Type, Shift, NumBits>(); }

#undef UE_API
