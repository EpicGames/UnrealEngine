// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelAccessor.h"

#include "Engine/Engine.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelData.h"
#include "NiagaraDataChannelHandler.h"

#include "NiagaraWorldManager.h"

template<typename TContext>
bool FNDCWriterBase::BeginWrite_Internal(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, TContext& AccessContext, int32 InCount, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU)
{
	ensureMsgf(Data.IsValid() == false, TEXT("NDC Writer with non-null data on BeginWrite. Possible EndWrite was not called after previous write."));

	const FNiagaraDataChannelLayoutInfoPtr NDCLayout = DataChannel->GetLayoutInfo();

	if (CachedLayout != NDCLayout)
	{
		Init(DataChannel);
		CachedLayout = NDCLayout;
	}

#if DEBUG_NDC_ACCESS
	for (FNDCVarAccessorBase* Var : VariableAccessors)
	{
		Var->ValidateLayout();
	}
#endif

	check(IsInGameThread());
	UWorld* World = (WorldContextObject != nullptr) ? GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) : nullptr;
	if (World && InCount > 0 && CachedLayout.IsValid())
	{
		if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World))
		{
			if (UNiagaraDataChannelHandler* Handler = WorldMan->FindDataChannelHandler(DataChannel))
			{
				if (FNiagaraDataChannelDataPtr DestData = Handler->FindData(AccessContext, ENiagaraResourceAccess::WriteOnly))
				{
					//Attempt to use an existing cached dest data.
					Data = DestData->GetGameDataForWriteGT(InCount, bVisibleToGame, bVisibleToCPU, bVisibleToGPU, DebugSource);

					//We are potentially writing into the middle of an existing buffer here so track base index and count.
					StartIndex = Data->Num();
					Count = InCount;
					Data->SetNum(StartIndex + InCount);

					return true;
				}
			}
		}
	}
	return false;
}

template<typename TContext>
bool FNDCReaderBase::BeginRead_Internal(const UObject* WorldContextObject, const UNiagaraDataChannel* DataChannel, TContext& Context, bool bInPreviousFrame)
{
	ensureMsgf(Data.IsValid() == false, TEXT("NDC Reader with non-null data on BeginRead. Possible EndRead was not called after previous read."));

	const FNiagaraDataChannelLayoutInfoPtr NDCLayout = DataChannel->GetLayoutInfo();

	if (CachedLayout != NDCLayout)
	{
		Init(DataChannel);
		CachedLayout = NDCLayout;
	}

#if DEBUG_NDC_ACCESS
	for (FNDCVarAccessorBase* Var : VariableAccessors)
	{
		Var->ValidateLayout();
	}
#endif

	check(IsInGameThread());
	UWorld* World = (WorldContextObject != nullptr) ? GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) : nullptr;
	if (World && CachedLayout.IsValid())
	{
		if (FNiagaraWorldManager* WorldMan = FNiagaraWorldManager::Get(World))
		{
			if (UNiagaraDataChannelHandler* Handler = WorldMan->FindDataChannelHandler(DataChannel))
			{
				if (FNiagaraDataChannelDataPtr NDCData = Handler->FindData(Context, ENiagaraResourceAccess::ReadOnly))
				{
					Data = NDCData->GetGameData()->AsShared();
					bPreviousFrame = bInPreviousFrame;
					return true;
				}
			}
		}
	}
	return false;
}