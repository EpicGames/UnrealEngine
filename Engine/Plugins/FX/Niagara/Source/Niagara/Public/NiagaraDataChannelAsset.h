// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannelAsset.generated.h"

class UNiagaraDataChannel;

/** Niagara Data Channels are a system for communication between Niagara Systems and with game code/Blueprint.

Data channel assets define the payload as well as some transfer settings.
Niagara Systems can read from and write to data channels via data interfaces.
Blueprint and C++ code can also read from and write to data channels using its API functions.

 */
UCLASS(BlueprintType, DisplayName = "Niagara Data Channel", MinimalAPI)
class UNiagaraDataChannelAsset : public UObject
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = DataChannel, Instanced)
	TObjectPtr<UNiagaraDataChannel> DataChannel;
	
	#if WITH_EDITORONLY_DATA
	/** When changing data channel types we cache the old channel and attempt to copy over any common properties from one to the other. */
	UPROPERTY(Transient)
	TObjectPtr<UNiagaraDataChannel> CachedPreChangeDataChannel;
	#endif

public:

#if WITH_EDITOR
	NIAGARA_API virtual void PreEditChange(class FProperty* PropertyAboutToChange)override;
	NIAGARA_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)override;
#endif

	[[nodiscard]] UNiagaraDataChannel* Get() const { return DataChannel; }
};