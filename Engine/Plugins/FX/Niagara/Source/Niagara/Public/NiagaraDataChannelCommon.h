// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
* Common shared types, headers and fwd dcls for Niagara Data Channels.
*/

#include "Engine/EngineBaseTypes.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSet.h"

#include "NiagaraDataChannelCommon.generated.h"

class UWorld;
class UActorComponent;
class FNiagaraWorldManager;
class FNiagaraDataBuffer;
class FNiagaraSystemInstance;
struct FNiagaraSpawnInfo;
struct FNiagaraVariableBase;

class UNiagaraDataChannelAsset;
class UNiagaraDataChannel;
class UNiagaraDataChannelHandler;
class FNiagaraDataChannelManager;

class UNiagaraDataChannelWriter;
class UNiagaraDataChannelReader;
class UNiagaraDataInterfaceDataChannelWrite;
class UNiagaraDataInterfaceDataChannelRead;

struct FNiagaraDataChannelData;
struct FNiagaraDataChannelDataProxy;
struct FNiagaraDataChannelGameData;
struct FNiagaraDataChannelPublishRequest;

struct FNiagaraDataChannelLayoutInfo;
struct FNiagaraDataChannelGameDataLayout;

struct FNDCAccessContextInst;

using FNiagaraDataChannelDataPtr = TSharedPtr<FNiagaraDataChannelData>;
using FNiagaraDataChannelDataProxyPtr = TSharedPtr<FNiagaraDataChannelDataProxy>;
using FNiagaraDataChannelGameDataPtr = TSharedPtr<FNiagaraDataChannelGameData>;
using FNiagaraDataChannelLayoutInfoPtr = TSharedPtr<FNiagaraDataChannelLayoutInfo>;

USTRUCT(BlueprintType)
struct FNiagaraDataChannelUpdateContext
{
	GENERATED_BODY();

	UPROPERTY(BlueprintReadOnly, Category = "Data Channel")
	TObjectPtr<class UNiagaraDataChannelReader> Reader;

	UPROPERTY(BlueprintReadOnly, Category = "Data Channel")
	int32 FirstNewDataIndex = 0;

	UPROPERTY(BlueprintReadOnly, Category = "Data Channel")
	int32 LastNewDataIndex = 0;

	UPROPERTY(BlueprintReadOnly, Category = "Data Channel")
	int32 NewElementCount = 0;
};

DECLARE_DYNAMIC_DELEGATE_OneParam(FOnNewNiagaraDataChannelPublish, const FNiagaraDataChannelUpdateContext&, NewDataContext);