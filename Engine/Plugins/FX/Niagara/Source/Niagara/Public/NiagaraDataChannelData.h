// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelLayoutInfo.h"
#include "NiagaraDataSetCompiledData.h"
#include "NiagaraDataChannelPublishRequest.h"

class FNiagaraGpuComputeDispatchInterface;
struct FNiagaraDataSetCompiledData;
class FNiagaraGpuReadbackManager;
class FRHICommandListImmediate;
class FRDGBuilder;

struct FNDCGpuReadbackInfo
{
	FNiagaraDataBufferRef Buffer;
	bool bPublishToCPU = false;
	bool bPublishToGame = false;
	FVector3f LWCTile;
};

/** Render thread proxy of FNiagaraDataChannelData. */
struct FNiagaraDataChannelDataProxy : public TSharedFromThis<FNiagaraDataChannelDataProxy>
{
	~FNiagaraDataChannelDataProxy();

	TWeakPtr<FNiagaraDataChannelData> Owner;
	FNiagaraDataSet* GPUDataSet = nullptr;
	FNiagaraDataBufferRef CurrFrameData = nullptr;
	FNiagaraDataBufferRef PrevFrameData = nullptr;
	bool bNeedsPrevFrameData = false;
	bool bHasPendingCpuUpdate = false;

	//Keeping layout info ref to ensure lifetime for GPUDataSet.
	FNiagaraDataChannelLayoutInfoPtr LayoutInfo;

	//Buffers coming from the CPU that we're going to copy up for reading on the GPU
	TArray<FNiagaraDataBufferRef> PendingCPUBuffers;
	
	//The buffers received from the CPU most recently. We keep these around until we get another update from the Cpu.
	TArray<FNiagaraDataBufferRef> CurrentCPUBuffers;

	//Buffers written from the GPU that we must send back to the CPU.
	TArray<FNDCGpuReadbackInfo> PendingGPUReadbackBuffers;
	
	//Users that need space in this NDC Data add to this for each tick via AddGPUAllocationForNextTick().
	int32 PendingGPUAllocations = 0;

	//Allocation made for the most current frame for NDC writes from the GPU.
	int32 CurrentGPUAllocation = 0;

	//Track current read/write counts +ve for readers, -ve for writers. We cannot mix readers and writers in the same buffer in the same stage.
	int32 CurrBufferAccessCounts = 0;

	#if !UE_BUILD_SHIPPING
	bool bWarnedAboutSameStageRW = false;
	FNiagaraGpuComputeDispatchInterface* DispatchInterfaceForDebuggingOnly = nullptr;
	
	FString DebugName;
	const TCHAR* GetDebugName()const{return *DebugName;}
	#else
	const TCHAR* GetDebugName()const{return nullptr;}
	#endif

	void BeginFrame(FNiagaraGpuComputeDispatchInterface* DispatchInterface, FRHICommandListImmediate& RHICmdList);
	void EndFrame(FNiagaraGpuComputeDispatchInterface* DispatchInterface, FRHICommandListImmediate& RHICmdList);
	void Reset();

	FNiagaraDataBufferRef PrepareForWriteAccess(FRDGBuilder& GraphBuilder);
	void EndWriteAccess(FRDGBuilder& GraphBuilder);
	
	FNiagaraDataBufferRef PrepareForReadAccess(FRDGBuilder& GraphBuilder, bool bCurrentFrame);
	void EndReadAccess(FRDGBuilder& GraphBuilder, bool bCurrentFrame);


	FNiagaraDataBufferRef AllocateBufferForCPU(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, int32 AllocationSize, bool bPublishToGame, bool bPublishToCPU, FVector3f LWCTile);
	void AddBuffersFromCPU(const TArray<FNiagaraDataBufferRef>& BuffersFromCPU);	
	void AddGPUAllocationForNextTick(int32 AllocationCount);

	FNiagaraDataBufferRef GetCurrentData()const { return CurrFrameData; }
	FNiagaraDataBufferRef GetPrevFrameData()const { return PrevFrameData; }
	
	void AddTransition(FRDGBuilder& GraphBuilder, ERHIAccess AccessBefore, ERHIAccess AccessAfter, FNiagaraDataBuffer* Buffer);

	//Perform and bookkeeping required when we remove a proxy from a dispatcher.
	void OnAddedToDispatcher(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface);
	void OnRemovedFromDispatcher(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface);
};

//////////////////////////////////////////////////////////////////////////

/**
Underlying storage class for data channel data.
Some data channels will have many of these and can distribute them as needed to different accessing systems.
For example, some data channel handlers may subdivide the scene such that distant systems are not interacting.
In this case, each subdivision would have it's own FNiagaraDataChannelData and distribute these to the relevant NiagaraSystems.
*/
struct FNiagaraDataChannelData final : public TSharedFromThis<FNiagaraDataChannelData, ESPMode::ThreadSafe>
{
	UE_NONCOPYABLE(FNiagaraDataChannelData)
	NIAGARA_API explicit FNiagaraDataChannelData();
	NIAGARA_API ~FNiagaraDataChannelData();

	NIAGARA_API void Init(UNiagaraDataChannelHandler* Owner);
	NIAGARA_API void Reset();

	NIAGARA_API void BeginFrame(UNiagaraDataChannelHandler* Owner);
	NIAGARA_API void EndFrame(UNiagaraDataChannelHandler* Owner);
	NIAGARA_API int32 ConsumePublishRequests(UNiagaraDataChannelHandler* Owner, const ETickingGroup& TickGroup);

	NIAGARA_API FNiagaraDataChannelGameData* GetGameData();
	NIAGARA_API FNiagaraDataBufferRef GetCPUData(bool bPreviousFrame);
	FNiagaraDataChannelDataProxyPtr GetRTProxy(){ return RTProxy; }
	
	/** Adds a request to publish some data into the channel on the next tick. */
	NIAGARA_API void Publish(const FNiagaraDataChannelPublishRequest& Request);

	NIAGARA_API void PublishFromGPU(const FNiagaraDataChannelPublishRequest& Request);

	NIAGARA_API const FNiagaraDataSetCompiledData& GetCompiledData(ENiagaraSimTarget SimTarget);

	void SetLwcTile(FVector3f InLwcTile){ LwcTile = InLwcTile; }
	FVector3f GetLwcTile()const { return LwcTile; }

	//This will get a buffer from the CPU dataset intended to be written to on the CPU.
	FNiagaraDataBuffer* GetBufferForCPUWrite();

	void DestroyRenderThreadProxy(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface);

	void RegisterGPUSpawningReader() { ++NumGPUSpawningReaders; }
	void UnregisterGPUSpawningReader() { --NumGPUSpawningReaders; }
	int32 NumRegisteredGPUSpawningReaders()const{ return NumGPUSpawningReaders; }

	//Returns if this data is still valid. This can return false in cases where the owning data channel has been modified for example.
	bool IsLayoutValid(UNiagaraDataChannelHandler* Owner)const;

	//Return true if data has been written to this NDC data for the current frame.
	bool HasData()const;

	//Returns a game data buffer into which we can write Count valuse on the Game Thread.
	FNiagaraDataChannelGameDataPtr GetGameDataForWriteGT(int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU, const FString& DebugSource);

private:

	void CreateRenderThreadProxy(UNiagaraDataChannelHandler* Owner);

	void FlushPendingGameData(int32 Index);
	void FlushAllPendingGameData();

	/** DataChannel data accessible from Game/BP. AoS Layout. LWC types. */
	FNiagaraDataChannelGameDataPtr GameData;

	/** DataChannel data accessible to Niagara CPU sims. SoA layout. Non LWC types. */
	FNiagaraDataSet* CPUSimData = nullptr;

	// Cached off buffer with the previous frame's CPU Sim accessible data.
	// Some systems can choose to read this to avoid any current frame tick ordering issues.
	FNiagaraDataBufferRef PrevCPUSimData = nullptr;

	/** Dataset we use for staging game data for the consumption by RT/GPU sims. */
	FNiagaraDataSet* GameDataStaging = nullptr;

	/** Data buffers we'll be passing to the RT proxy for uploading to the GPU */
	TArray<FNiagaraDataChannelPublishRequest> PublishRequestsForGPU;

	/** Render thread proxy for this data. Owns all RT side data meant for GPU simulations. */
	FNiagaraDataChannelDataProxyPtr RTProxy;

	/** Pending requests to publish data into this data channel. These requests are consumed at tick tick group. */
	TArray<FNiagaraDataChannelPublishRequest> PublishRequests;

	/** Pending requests to publish data into this data channel from the GPU. To alleviate data race behavior with data coming back from the GPU, we always consume GPU requests at the start of the frame only. */
	TArray<FNiagaraDataChannelPublishRequest> PublishRequestsFromGPU;

	/** The world we were initialized with, used to get the compute interface. */
	TWeakObjectPtr<UWorld> WeakOwnerWorld;

	FVector3f LwcTile = FVector3f::ZeroVector;

	/** Critical section protecting shared state for multiple writers publishing from different threads. */
	FCriticalSection PublishCritSec;

	//Keep reference to the layout this data was built with.
	FNiagaraDataChannelLayoutInfoPtr LayoutInfo;

	/** 
	Track number of explicitly registered readers that spawn GPU particles from this data.
	If we're spawning GPU particles using the CPU data (Spawn Conditional etc) then we have to send all CPU data to the GPU every frame.
	Can possibly extend this to be a more automatic, registration based approach to shipping NDC data around rather than explicit flags on write.
	*/
	std::atomic<int32> NumGPUSpawningReaders;

	//We keep a set of incoming game data for all flag combinations and accumulate data into these rather than keeping all as separate
	TArray<FNiagaraDataChannelGameDataPtr, TInlineAllocator<8>> PendingDestGameData;
};
