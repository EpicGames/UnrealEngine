// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannelCommon.h"

#if WITH_NIAGARA_DEBUGGER

/** Hooks into internal NiagaraDataChannels code for debugging and testing purposes. */
class FNiagaraDataChannelDebugUtilities
{
public: 

	static NIAGARA_API void BeginFrame(FNiagaraWorldManager* WorldMan, float DeltaSeconds);
	static NIAGARA_API void EndFrame(FNiagaraWorldManager* WorldMan, float DeltaSeconds);
	static NIAGARA_API void Tick(FNiagaraWorldManager* WorldMan, float DeltaSeconds, ETickingGroup TickGroup);
	
	static NIAGARA_API UNiagaraDataChannelHandler* FindDataChannelHandler(FNiagaraWorldManager* WorldMan, UNiagaraDataChannel* DataChannel);

	static void LogWrite(const FNiagaraDataChannelPublishRequest& WriteRequest, const UNiagaraDataChannel* DataChannel, const ETickingGroup& TickGroup);
	static void DumpAllWritesToLog();

	static FNiagaraDataChannelDebugUtilities& Get();
	static void TearDown();

private:
	struct FChannelWriteRequest
	{
		TWeakObjectPtr<const UNiagaraDataChannel> Channel;
		TSharedPtr<FNiagaraDataChannelGameData> Data;
		bool bVisibleToGame = false;
		bool bVisibleToCPUSims = false;
		bool bVisibleToGPUSims = false;
		ETickingGroup TickGroup;
		FString DebugSource;
	};
	struct FFrameDebugData
	{
		uint64 FrameNumber;
		TArray<FChannelWriteRequest> WriteRequests;
	};
	
	static FString ToJson(FNiagaraDataChannelGameData* Data);
	static FString TickGroupToString(const ETickingGroup& TickGroup);
	TArray<FFrameDebugData> FrameData;
};


#endif//WITH_NIAGARA_DEBUGGER