// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "NiagaraDataChannelCommon.h"

#include "NiagaraDataChannelFunctionLibrary.generated.h"

#define UE_API NIAGARA_API

UENUM(BlueprintType)
enum class ENiagartaDataChannelReadResult : uint8
{
	Success,
	Failure
};

/**
* A C++ and Blueprint accessible library of utility functions for accessing Niagara DataChannel
*/
UCLASS(MinimalAPI)
class UNiagaraDataChannelLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	UFUNCTION(BlueprintInternalUseOnly, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API UNiagaraDataChannelHandler* GetNiagaraDataChannel(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel);

	
	//////////////////////////////////////////////////////////////////////////
	//These functions are soon to be deprecated in favor of new functions that take a more general AccessContext instead of Search Parameters.

	/**
	 * LEGACY FUNCTION: Please use non-legacy version.
	 * Initializes and returns the Niagara Data Channel writer to write N elements to the given data channel.
	 * This function is now legacy and and will soon be deprecated. Please use the new non-legacy function.
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to write to
	 * @param SearchParams			Parameters used when retrieving a specific set of Data Channel Data to read or write like the islands data channel type.
	 * @param Count					The number of elements to write 
	 * @param bVisibleToGame	If true, the data written to this data channel is visible to Blueprint and C++ logic reading from it
	 * @param bVisibleToCPU	If true, the data written to this data channel is visible to Niagara CPU emitters
	 * @param bVisibleToGPU	If true, the data written to this data channel is visible to Niagara GPU emitters
	 * @param DebugSource	Instigator for this write, used in the debug hud to track writes to the data channel from different sources
	 */
	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, DisplayName="Write To Niagara Data Channel (Batch) (Legacy)", meta = (AdvancedDisplay = "SearchParams, DebugSource", Keywords = "niagara DataChannel", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true", AutoCreateRefTerm="DebugSource"))
	static UE_API UNiagaraDataChannelWriter* WriteToNiagaraDataChannel(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNiagaraDataChannelSearchParameters SearchParams, int32 Count, UPARAM(DisplayName = "Visible to Blueprint") bool bVisibleToGame, UPARAM(DisplayName = "Visible to Niagara CPU") bool bVisibleToCPU, UPARAM(DisplayName = "Visible to Niagara GPU") bool bVisibleToGPU, const FString& DebugSource);

	/**
	 * LEGACY FUNCTION: Please use non-legacy version.
	 * Initializes and returns the Niagara Data Channel reader for the given data channel.
	 * This function is now legacy and and will soon be deprecated. Please use the new non-legacy function.
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to read from
	 * @param SearchParams			Parameters used when retrieving a specific set of Data Channel Data to read or write like the islands data channel type.
	 * @param bReadPreviousFrame	True if this reader will read the previous frame's data. If false, we read the current frame.
	 *								Reading the current frame allows for zero latency reads, but any data elements that are generated after this reader is used are missed.
	 *								Reading the previous frame's data introduces a frame of latency but ensures we never miss any data as we have access to the whole frame.
	 */
	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, DisplayName="Read From Niagara Data Channel (Batch) (Legacy)", meta = (AdvancedDisplay = "SearchParams", Keywords = "niagara DataChannel", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API UNiagaraDataChannelReader* ReadFromNiagaraDataChannel(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNiagaraDataChannelSearchParameters SearchParams, bool bReadPreviousFrame);

	/**
	 * LEGACY FUNCTION: Please use non-legacy version.
	 * Returns the number of readable elements in the given data channel
	 * This function is now legacy and and will soon be deprecated. Please use the new non-legacy function.
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to read from
	 * @param SearchParams			Parameters used when retrieving a specific set of Data Channel Data to read or write like the islands data channel type.
	 * @param bReadPreviousFrame	True if this reader will read the previous frame's data. If false, we read the current frame.
	 *								Reading the current frame allows for zero latency reads, but any data elements that are generated after this reader is used are missed.
	 *								Reading the previous frame's data introduces a frame of latency but ensures we never miss any data as we have access to the whole frame.
	 */
	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (bReadPreviousFrame="true", DisplayName="Get Data Channel Element Count (Legacy)", AdvancedDisplay = "SearchParams, bReadPreviousFrame", Keywords = "niagara DataChannel num size", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API int32 GetDataChannelElementCount(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNiagaraDataChannelSearchParameters SearchParams, bool bReadPreviousFrame);

	/**
	 * LEGACY FUNCTION: Please use non-legacy version.
	 * Subscribes to a single data channel and calls a delegate every times new data is written to the data channel.
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to subscribe to for updates
	 * @param SearchParams			Parameters used when retrieving a specific set of Data Channel Data to read - only used by some types of data channels, like the island types.
	 * @param UpdateDelegate		The delegate to be called when new data is available in the data channel. Can be called multiple times per tick.
	 * @param UnsubscribeToken		This token can be used to unsubscribe from the data channel.
	 */
	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (DisplayName="Subscribe To Niagara Data Channel (Legacy)", Keywords = "niagara DataChannel event reader subscription delegate listener updates", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API void SubscribeToNiagaraDataChannel(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNiagaraDataChannelSearchParameters SearchParams, const FOnNewNiagaraDataChannelPublish& UpdateDelegate, int32& UnsubscribeToken);

	/**
	 * Removes a prior registration from a data channel
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to unsubscribe from
	 * @param UnsubscribeToken		The token returned from the subscription call
	 */
	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel event reader subscription delegate listener updates", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API void UnsubscribeFromNiagaraDataChannel(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, const int32& UnsubscribeToken);
	
	/**
	 * LEGACY FUNCTION: Please use non-legacy version.
	 * Reads a single entry from the given data channel, if possible.
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to read from
	 * @param Index					The data index to read from
	 * @param SearchParams			Parameters used when retrieving a specific set of Data Channel Data to read or write like the islands data channel type.
	 * @param bReadPreviousFrame	True if this reader will read the previous frame's data. If false, we read the current frame.
	 *								Reading the current frame allows for zero latency reads, but any data elements that are generated after this reader is used are missed.
	 *								Reading the previous frame's data introduces a frame of latency but ensures we never miss any data as we have access to the whole frame.
	 * @param ReadResult			Used by Blueprint for the return value
	 */
	UFUNCTION(BlueprintInternalUseOnly, Category = NiagaraDataChannel, DisplayName="Read From Niagara Data Channel (Legacy)", meta = (bReadPreviousFrame="true", AdvancedDisplay = "SearchParams, bReadPreviousFrame", ExpandEnumAsExecs="ReadResult", Keywords = "niagara DataChannel", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API void ReadFromNiagaraDataChannelSingle(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, int32 Index, FNiagaraDataChannelSearchParameters SearchParams, bool bReadPreviousFrame, ENiagartaDataChannelReadResult& ReadResult);

	/**
	 * LEGACY FUNCTION: Please use non-legacy version.
	 * Writes a single element to a Niagara Data Channel. The element won't be immediately visible to readers, as it needs to be processed first. The earliest point it can be read is in the next tick group.
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to write to
	 * @param SearchParams			Parameters used when retrieving a specific set of Data Channel Data to read or write like the islands data channel type.
	 * @param bVisibleToBlueprint	If true, the data written to this data channel is visible to Blueprint and C++ logic reading from it
	 * @param bVisibleToNiagaraCPU	If true, the data written to this data channel is visible to Niagara CPU emitters
	 * @param bVisibleToNiagaraGPU	If true, the data written to this data channel is visible to Niagara GPU emitters
	 */
	UFUNCTION(BlueprintInternalUseOnly, Category = NiagaraDataChannel, DisplayName="Write To Niagara Data Channel (Legacy)", meta = (bVisibleToBlueprint="true", bVisibleToNiagaraCPU="true", bVisibleToNiagaraGPU="true", AdvancedDisplay = "bVisibleToBlueprint, bVisibleToNiagaraCPU, bVisibleToNiagaraGPU, SearchParams", Keywords = "niagara DataChannel event writer", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API void WriteToNiagaraDataChannelSingle(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, FNiagaraDataChannelSearchParameters SearchParams, bool bVisibleToBlueprint, bool bVisibleToNiagaraCPU, bool bVisibleToNiagaraGPU);

	//////////////////////////////////////////////////////////////////////////

	/**
	 * Initializes and returns the Niagara Data Channel reader for the given data channel.
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to read from
	 * @param AccessContext			Context allowing passing of NDC type specific information between the caller and internal Data Channel that can control internal routing and help feedback information to the caller.
	 * @param bReadPreviousFrame	True if this reader will read the previous frame's data. If false, we read the current frame.
	 *								Reading the current frame allows for zero latency reads, but any data elements that are generated after this reader is used are missed.
	 *								Reading the previous frame's data introduces a frame of latency but ensures we never miss any data as we have access to the whole frame.
	 */
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly, Category = NiagaraDataChannel, DisplayName="Read From Niagara Data Channel (Batch)", meta = (Keywords = "niagara DataChannel", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API UNiagaraDataChannelReader* ReadFromNiagaraDataChannel_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, UPARAM(ref) FNDCAccessContextInst& AccessContext, bool bReadPreviousFrame);
	
	/**
	 * Initializes and returns the Niagara Data Channel writer to write N elements to the given data channel.
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to write to
	 * @param AccessContext			Context allowing passing of NDC type specific information between the caller and internal Data Channel that can control internal routing and help feedback information to the caller.
	 * @param Count					The number of elements to write 
	 * @param bVisibleToGame	If true, the data written to this data channel is visible to Blueprint and C++ logic reading from it
	 * @param bVisibleToCPU	If true, the data written to this data channel is visible to Niagara CPU emitters
	 * @param bVisibleToGPU	If true, the data written to this data channel is visible to Niagara GPU emitters
	 * @param DebugSource	Instigator for this write, used in the debug hud to track writes to the data channel from different sources
	 */
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly, Category = NiagaraDataChannel, DisplayName = "Write To Niagara Data Channel (Batch)", meta = (AdvancedDisplay = "DebugSource", Keywords = "niagara DataChannel", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true", AutoCreateRefTerm = "DebugSource", bVisibleToBlueprint="true", bVisibleToNiagaraCPU="true", bVisibleToNiagaraGPU="true"))
	static UE_API UNiagaraDataChannelWriter* WriteToNiagaraDataChannel_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, UPARAM(ref) FNDCAccessContextInst& AccessContext, int32 Count, UPARAM(DisplayName = "Visible to Blueprint") bool bVisibleToBlueprint, UPARAM(DisplayName = "Visible to Niagara CPU") bool bVisibleToNiagaraCPU, UPARAM(DisplayName = "Visible to Niagara GPU") bool bVisibleToNiagaraGPU, const FString& DebugSource);
	
	/**
	 * Returns the number of readable elements in the given data channel
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to read from
	 * @param AccessContext			Context allowing passing of NDC type specific information between the caller and internal Data Channel that can control internal routing and help feedback information to the caller.
	 * @param bReadPreviousFrame	True if this reader will read the previous frame's data. If false, we read the current frame.
	 *								Reading the current frame allows for zero latency reads, but any data elements that are generated after this reader is used are missed.
	 *								Reading the previous frame's data introduces a frame of latency but ensures we never miss any data as we have access to the whole frame.
	 */
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly,Category = NiagaraDataChannel, meta = (bReadPreviousFrame="true", Keywords = "niagara DataChannel num size", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API int32 GetDataChannelElementCount_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, UPARAM(ref) FNDCAccessContextInst& AccessContext, bool bReadPreviousFrame);
	
	/**
	 * Subscribes to a single data channel and calls a delegate every times new data is written to the data channel.
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to subscribe to for updates
	 * @param AccessContext			Context allowing passing of NDC type specific information between the caller and internal Data Channel that can control internal routing and help feedback information to the caller.
	 * @param UpdateDelegate		The delegate to be called when new data is available in the data channel. Can be called multiple times per tick.
	 * @param UnsubscribeToken		This token can be used to unsubscribe from the data channel.
	 */
	UFUNCTION(BlueprintCallable, Category = NiagaraDataChannel, meta = (Keywords = "niagara DataChannel event reader subscription delegate listener updates", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API void SubscribeToNiagaraDataChannel_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, UPARAM(ref) FNDCAccessContextInst& AccessContext, const FOnNewNiagaraDataChannelPublish& UpdateDelegate, int32& UnsubscribeToken);
		
	/**
	 * Reads a single entry from the given data channel, if possible.
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to read from
	 * @param Index					The data index to read from
	 * @param AccessContext			Context allowing passing of NDC type specific information between the caller and internal Data Channel that can control internal routing and help feedback information to the caller.
	 * @param bReadPreviousFrame	True if this reader will read the previous frame's data. If false, we read the current frame.
	 *								Reading the current frame allows for zero latency reads, but any data elements that are generated after this reader is used are missed.
	 *								Reading the previous frame's data introduces a frame of latency but ensures we never miss any data as we have access to the whole frame.
	 * @param ReadResult			Used by Blueprint for the return value
	 */
	UFUNCTION(BlueprintInternalUseOnly, Category = NiagaraDataChannel, DisplayName="Read From Niagara Data Channel", meta = (bReadPreviousFrame="true", ExpandEnumAsExecs="ReadResult", Keywords = "niagara DataChannel", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API void ReadFromNiagaraDataChannelSingle_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, int32 Index, UPARAM(ref) FNDCAccessContextInst& AccessContext, bool bReadPreviousFrame, ENiagartaDataChannelReadResult& ReadResult);

	/**
	 * Writes a single element to a Niagara Data Channel. The element won't be immediately visible to readers, as it needs to be processed first. The earliest point it can be read is in the next tick group.
	 *
	 * @param WorldContextObject	World to execute in
	 * @param Channel				The channel to write to
	 * @param AccessContext			Context allowing passing of NDC type specific information between the caller and internal Data Channel that can control internal routing and help feedback information to the caller.
	 * @param bVisibleToBlueprint	If true, the data written to this data channel is visible to Blueprint and C++ logic reading from it
	 * @param bVisibleToNiagaraCPU	If true, the data written to this data channel is visible to Niagara CPU emitters
	 * @param bVisibleToNiagaraGPU	If true, the data written to this data channel is visible to Niagara GPU emitters
	 */
	UFUNCTION(BlueprintInternalUseOnly, Category = NiagaraDataChannel, DisplayName="Write To Niagara Data Channel", meta = (bVisibleToBlueprint="true", bVisibleToNiagaraCPU="true", bVisibleToNiagaraGPU="true", Keywords = "niagara DataChannel event writer", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static UE_API void WriteToNiagaraDataChannelSingle_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannelAsset* Channel, UPARAM(ref) FNDCAccessContextInst& AccessContext, bool bVisibleToBlueprint, bool bVisibleToNiagaraCPU, bool bVisibleToNiagaraGPU);

	static UE_API UNiagaraDataChannelHandler* FindDataChannelHandler(const UObject* WorldContextObject, const UNiagaraDataChannel* Channel);
	static UE_API UNiagaraDataChannelWriter* CreateDataChannelWriter(const UObject* WorldContextObject, const UNiagaraDataChannel* Channel, FNiagaraDataChannelSearchParameters SearchParams, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU, const FString& DebugSource);
	static UE_API UNiagaraDataChannelReader* CreateDataChannelReader(const UObject* WorldContextObject, const UNiagaraDataChannel* Channel, FNiagaraDataChannelSearchParameters SearchParams, bool bReadPreviousFrame);

	static UE_API UNiagaraDataChannelWriter* CreateDataChannelWriter_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannel* Channel, FNDCAccessContextInst& AccessContext, int32 Count, bool bVisibleToGame, bool bVisibleToCPU, bool bVisibleToGPU, const FString& DebugSource);
	static UE_API UNiagaraDataChannelReader* CreateDataChannelReader_WithContext(const UObject* WorldContextObject, const UNiagaraDataChannel* Channel, FNDCAccessContextInst& AccessContext, bool bReadPreviousFrame);


	/** Returns the access context for use when accessing a Niagara Data channel. */
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly, Category = "Niagara Data Channel")
	static UE_API FNDCAccessContextInst& GetUsableAccessContextFromNDC(const UNiagaraDataChannelAsset* DataChannel);

	/** Returns the access context for use when accessing a Niagara Data channel. */
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly, Category = "Niagara Data Channel")
	static UE_API FNDCAccessContextInst& GetUsableAccessContextFromNDCRef(const FNiagaraDataChannelReference& NDCRef);

	/** 
	Prepares an Access Context ready for accessing a Niagara Data Channel. 
	If the passed Data Channel Reference has a valid custom AccessContext then it's values will be applied. Otherwise the defaults will be used.
	Allows optional setting for the members of the Access Context. Transient properties are shown by default but properties to modify can be selected in the details panel.
	*/
	UFUNCTION(BlueprintInternalUseOnly, BlueprintCallable, Category = "Niagara Data Channel")
	static UE_API FNDCAccessContextInst& PrepareAccessContextFromNDCRef(UPARAM(Ref) FNiagaraDataChannelReference& NDCRef);

	/** Creates a new NDCAccessContext with a specified internal struct type. Struct must be a child of FNDCAccessContextBase. */
	UFUNCTION(BlueprintInternalUseOnly, BlueprintCallable, CustomThunk, Category = "Niagara Data Channel", meta = (HidePin = "ContextStruct"))
	static UE_API FNDCAccessContextInst MakeNDCAccessContextInstance(UScriptStruct* ContextStruct);

	/** 
	Gets the members from the internal Struct of an FNDCAccessContextIns. Expanded on compile into individual per property calls to GetSinglePropertyInNDCAccessContextInstance.
	*/
	UFUNCTION(BlueprintInternalUseOnly, BlueprintCallable, Category = "Niagara Data Channel", meta = (HidePin = "ContextStruct"))
	static UE_API void GetMembersInNDCAccessContextInstance(const FNDCAccessContextInst& AccessContext, UScriptStruct* ContextStruct);
	
	/** 
	Sets the members in the internal Struct of an FNDCAccessContextIns. Expanded on compile into individual per property calls to SetSinglePropertyInNDCAccessContextInstance.
	*/
	UFUNCTION(BlueprintInternalUseOnly, BlueprintCallable, Category = "Niagara Data Channel", meta = (HidePin = "ContextStruct"))
	static UE_API void SetMembersInNDCAccessContextInstance(UPARAM(Ref) FNDCAccessContextInst& AccessContext, UScriptStruct* ContextStruct);
	
	/** 
	Gets a specific property within the internal data of an FNDCAccessContextInst.
	*/
	UFUNCTION(BlueprintInternalUseOnly, BlueprintCallable, CustomThunk, Category = "Niagara Data Channel", meta = (CustomStructureParam = "Value", AutoCreateRefTerm="Value"))
	static UE_API void GetSinglePropertyInNDCAccessContextInstance(const FNDCAccessContextInst& AccessContext, UScriptStruct* ContextStruct, FName PropertyName, int32& Value);
	
	/** 
	Sets a specific property within the internal data of an FNDCAccessContextInst.
	*/
	UFUNCTION(BlueprintInternalUseOnly, BlueprintCallable, CustomThunk, Category = "Niagara Data Channel", meta = (CustomStructureParam = "Value", AutoCreateRefTerm="Value"))
	static UE_API void SetSinglePropertyInNDCAccessContextInstance(UPARAM(Ref) FNDCAccessContextInst& AccessContext, UScriptStruct* ContextStruct, FName PropertyName, const int32& Value);

private:
	DECLARE_FUNCTION(execMakeNDCAccessContextInstance);
	DECLARE_FUNCTION(execGetSinglePropertyInNDCAccessContextInstance);
	DECLARE_FUNCTION(execSetSinglePropertyInNDCAccessContextInstance);
};


#undef UE_API