// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelLayoutInfo.h"
#include "NiagaraDataChannelVariableBuffer.h"

/** Storage for game level DataChannels generated by BP / C++ */
struct FNiagaraDataChannelGameData : public TSharedFromThis<FNiagaraDataChannelGameData>
{
private:

	/** Per variable storage buffers. */
	TArray<FNiagaraDataChannelVariableBuffer> VariableData;

	int32 NumElements = 0;
	int32 MaxElements = 0;
	int32 PrevNumElements = 0;

	//Keep reference to the layout this data was built with.
	FNiagaraDataChannelLayoutInfoPtr LayoutInfo;

public:
	
	FNiagaraDataChannelGameData(const FNiagaraDataChannelLayoutInfoPtr& InLayoutInfo);
	NIAGARA_API void Init(const FNiagaraDataChannelLayoutInfoPtr& InLayoutInfo);
	NIAGARA_API void Empty();
	NIAGARA_API void Reset();
	NIAGARA_API void BeginFrame();
	NIAGARA_API void SetNum(int32 NewNum);
	NIAGARA_API void Reserve(int32 NewNum);
	int32 Num() const { return NumElements; }
	int32 Max() const { return MaxElements; }
	int32 PrevNum()const { return PrevNumElements; }

	int32 Add(int32 Count=1)
	{
		int32 Ret = NumElements;
		SetNum(Ret + Count);
		return Ret;
	}

	NIAGARA_API FNiagaraDataChannelVariableBuffer* FindVariableBuffer(const FNiagaraVariableBase& Var);

	NIAGARA_API void WriteToDataSet(FNiagaraDataBuffer* DestBuffer, int32 DestStartIdx, FVector3f SimulationLwcTile);

	NIAGARA_API void AppendFromGameData(const FNiagaraDataChannelGameData& GameData);

	NIAGARA_API void AppendFromDataSet(const FNiagaraDataBuffer* SrcBuffer, FVector3f SimulationLwcTile);

	const FNiagaraDataChannelLayoutInfoPtr& GetLayoutInfo()const { return LayoutInfo; }

	const TConstArrayView<FNiagaraDataChannelVariableBuffer> GetVariableBuffers()const { return VariableData; }
	void SetFromSimCache(const FNiagaraVariableBase& SourceVar, TConstArrayView<uint8> Data, int32 Size);

	template<typename T>
	bool Write(int32 VarIndex, int32 Index, const T& InValue)
	{
		if(VariableData.IsValidIndex(VarIndex))
		{
			FNiagaraDataChannelVariableBuffer& VarBuffer = VariableData[VarIndex];
			return VarBuffer.Write<T>(Index, InValue);
		}
		return false;
	}

	template<typename T>
	bool Read(int32 VarIndex, int32 Index, T& OutValue, bool bPreviousFrame)
	{
		if (VariableData.IsValidIndex(VarIndex))
		{
			FNiagaraDataChannelVariableBuffer& VarBuffer = VariableData[VarIndex];
			return VarBuffer.Read<T>(Index, OutValue, bPreviousFrame);
		}
		return false;
	}
};

using FNiagaraDataChannelGameDataPtr = TSharedPtr<FNiagaraDataChannelGameData>;