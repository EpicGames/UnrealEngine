// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
Base class for Niagara DataChannel Handlers.
Data Channel handlers are the runtime counterpart to Data Channels.
They control how data being written to the Data Channel is stored and how to expose data being read from the Data Channel.
For example, the simplest handler is the UNiagaraDataChannelHandler_Global which just keeps all data in a single large set that is used by all systems.
Some more complex handlers may want to divide up the scene in various different ways to better match particular use cases.

*/

#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelAccessContext.h"

#include "RenderCommandFence.h"
#include "NiagaraDataChannelHandler.generated.h"

class FNiagaraGpuComputeDispatchInterface;

UCLASS(abstract, BlueprintType, MinimalAPI)
class UNiagaraDataChannelHandler : public UObject
{
public:

	GENERATED_BODY()

	//UObject Interface
	NIAGARA_API virtual void BeginDestroy()override;
	NIAGARA_API virtual bool IsReadyForFinishDestroy()override;
	//UObject Interface END

	NIAGARA_API virtual void Init(const UNiagaraDataChannel* InChannel);

	NIAGARA_API virtual void Cleanup();

	NIAGARA_API virtual void BeginFrame(float DeltaTime);

	NIAGARA_API virtual void EndFrame(float DeltaTime);

	NIAGARA_API virtual void Tick(float DeltaTime, ETickingGroup TickGroup);

	/** 
	Legacy FindData that supports old FNiagaraDataChannelSearchParameters path.
	*/
	NIAGARA_API virtual FNiagaraDataChannelDataPtr FindData(FNiagaraDataChannelSearchParameters SearchParams, ENiagaraResourceAccess AccessType);

	/** 
	Finds the correct internal data for this data channel and the given access context. For example in some cases this may require a search of several elements of data that correspond to different spacial regions.
	This shared ptr provides access to Game level data, CPU simulation data and a render thread proxy that can be given to the RT and provides access to GPUSimulaiton data.
	*/
	NIAGARA_API virtual FNiagaraDataChannelDataPtr FindData(FNDCAccessContextInst& AccessContext, ENiagaraResourceAccess AccessType);

	const UNiagaraDataChannel* GetDataChannel()const
	{
		const UNiagaraDataChannel* Ret = DataChannel.Get();
		check(Ret);
		return Ret;
	}
	
	NIAGARA_API virtual void OnDataChannelLayoutChanged();

	UFUNCTION(BlueprintCallable, Category="Data Channel")
	NIAGARA_API UNiagaraDataChannelWriter* GetDataChannelWriter();

	UFUNCTION(BlueprintCallable, Category = "Data Channel")
	NIAGARA_API UNiagaraDataChannelReader* GetDataChannelReader();

	/** The provided delegate will be called whenever new entries are added to the relevant data channel. This means the delegate can be called multiple times per tick.
	 * This is only relevant for data published to the game thread, so no gpu data or data that's only visible to niagara systems. */
	UFUNCTION(BlueprintCallable, Category = "Data Channel", meta=(DisplayName="Subscribe To Data Channel Updates (Legacy)"))
	NIAGARA_API void SubscribeToDataChannelUpdates(FOnNewNiagaraDataChannelPublish UpdateDelegate, FNiagaraDataChannelSearchParameters SearchParams, int32& UnsubscribeToken);
	
	/** The provided delegate will be called whenever new entries are added to the relevant data channel. This means the delegate can be called multiple times per tick.
	 * This is only relevant for data published to the game thread, so no gpu data or data that's only visible to niagara systems. */
	UFUNCTION(BlueprintCallable, Category = "Data Channel")
	NIAGARA_API void SubscribeToDataChannelUpdates_WithContext(FOnNewNiagaraDataChannelPublish UpdateDelegate, FNDCAccessContextInst& AccessContext, int32& UnsubscribeToken);

	UFUNCTION(BlueprintCallable, Category = "Data Channel")
	NIAGARA_API void UnsubscribeFromDataChannelUpdates(const int32& UnsubscribeToken);

	template<typename T> 
	T* GetChannelTyped()const{ return Cast<T>(DataChannel); }

	NIAGARA_API FNiagaraDataChannelDataPtr CreateData();

	/** Returns the tick group we're currently being processed or most recently processed. */
	ETickingGroup GetCurrentTickGroup() { return CurrentTG; }

	void NotifySubscribers(FNiagaraDataChannelData* Source, int32 StartIndex, int32 NumNewElements);
	void OnComputeDispatchInterfaceDestroyed(FNiagaraGpuComputeDispatchInterface* InComputeDispatchInterface);

	template<typename TAction>
	void ForEachNDCData(TAction Func);

	/** Returns a pre allocated access context that you can use for accessing this data channel. Only safe to use from the game thread and it is not safe to hold onto this context after your access. */
	[[nodiscard]] inline FNDCAccessContextInst& GetTransientAccessContext()const;

	void SetOwningWorld(FNiagaraWorldManager* InOwner){ OwningWorld = InOwner; }
protected:

	FNiagaraWorldManager* OwningWorld = nullptr;

	NIAGARA_API void VerifyAccessContext(FNDCAccessContextInst& AccessContext);

	UPROPERTY()
	TWeakObjectPtr<const UNiagaraDataChannel> DataChannel;

	/** Helper object allowing BP to write data in this channel. */
	UPROPERTY()
	TObjectPtr<UNiagaraDataChannelWriter> Writer;

	/** Helper object allowing BP to read data in this channel. */
	UPROPERTY()
	TObjectPtr<UNiagaraDataChannelReader> Reader;

	ETickingGroup CurrentTG = ETickingGroup::TG_PrePhysics;

	//Weak refs to all NDC data created for this handler.
	//Allows us to perform book keeping and other operations on all data when needed.
	TArray<TWeakPtr<FNiagaraDataChannelData>> WeakDataArray;

	FRenderCommandFence RTFence;

	FNiagaraDataChannelLayoutInfoPtr CachedLayoutInfo;
private:

	struct FChannelSubscription
	{
		FOnNewNiagaraDataChannelPublish OnPublishDelegate;
		FNiagaraDataChannelSearchParameters SearchParams;
		FNDCAccessContextInst AccessContext;
		int32 SubscriptionToken = -1;
	};
	TArray<FChannelSubscription> ChannelSubscriptions;

	// used to generate the unsubscribe tokens that are stored in ChannelSubscriptions
	int32 SubscriberTokens = 0;

	/** A transient access context instance that calling code can use for accessing this Data Channel. GT only*/
	mutable FNDCAccessContextInst TransientAccessContext;
	TNDCAccessContextType AccessContextType;
};

FNDCAccessContextInst& UNiagaraDataChannelHandler::GetTransientAccessContext()const
{
	check(IsInGameThread());
	TransientAccessContext.Init(AccessContextType);//We need to init (or reset) ready for use.
	return TransientAccessContext;
}

template<typename TAction>
void UNiagaraDataChannelHandler::ForEachNDCData(TAction Func)
{
	check(IsInGameThread());
	for(auto& WeakData : WeakDataArray)
	{
		if(FNiagaraDataChannelDataPtr DataPtr = WeakData.Pin())
		{
			Func(DataPtr);
		}
	}
}