// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataSetCompiledData.h"
#include "NiagaraDataChannelVariable.h"
#include "NiagaraDataChannelLayoutInfo.generated.h"

class UNiagaraDataChannel;

USTRUCT()
struct FNiagaraDataChannelGameDataLayout
{
	GENERATED_BODY();

	/** Map of all variables contained in this DataChannel data and the indices into data arrays for game data. */
	UPROPERTY()
	TMap<FNiagaraVariableBase, int32> VariableIndices;

	/** Helpers for converting LWC types into Niagara simulation SWC types. */
	UPROPERTY()
	TArray<FNiagaraLwcStructConverter> LwcConverters;

	void Init(TConstArrayView<FNiagaraDataChannelVariable> Variables);
};

/** Data describing the layout of Niagara Data channel buffers that is used in multiple places and must live beyond it's owning Data Channel. */
struct FNiagaraDataChannelLayoutInfo : public TSharedFromThis<FNiagaraDataChannelLayoutInfo>
{
	FNiagaraDataChannelLayoutInfo(const UNiagaraDataChannel* DataChannel);
	~FNiagaraDataChannelLayoutInfo();

	const FNiagaraDataSetCompiledData& GetDataSetCompiledData()const{ return CompiledData; }
	const FNiagaraDataSetCompiledData& GetDataSetCompiledDataGPU()const { return CompiledDataGPU; }
	const FNiagaraDataChannelGameDataLayout& GetGameDataLayout()const { return GameDataLayout; }

	/** If true, we keep our previous frame's data. Some users will prefer a frame of latency to tick dependency. */
	bool KeepPreviousFrameData() const { return bKeepPreviousFrameData; }
private:

	/**
	Data layout for payloads in Niagara datasets.
	*/
	FNiagaraDataSetCompiledData CompiledData;

	FNiagaraDataSetCompiledData CompiledDataGPU;

	/** Layout information for any data stored at the "Game" level. i.e. From game code/BP. AoS layout and LWC types. */
	FNiagaraDataChannelGameDataLayout GameDataLayout;

	bool bKeepPreviousFrameData = false;
};