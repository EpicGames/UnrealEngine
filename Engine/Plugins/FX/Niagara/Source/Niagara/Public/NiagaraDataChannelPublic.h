// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
Types and headers required for users of Niagara Data Channels.
*/

#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelAsset.h"
#include "NiagaraDataChannelData.h"
#include "NiagaraDataChannelAccessContext.h"
#include "NiagaraDataChannelGameData.h"
#include "NiagaraDataChannelLayoutInfo.h"
#include "NiagaraDataSetCompiledData.h"
#include "NiagaraDataChannelFunctionLibrary.h"
#include "NiagaraDataChannelDebugUtilities.h"
#include "NiagaraDataChannelPublishRequest.h"
#include "NiagaraDataChannelReference.h"