// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraDefines.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSet.h"

struct FNiagaraDataChannelGameData;

/** A request to publish data into a Niagara Data Channel.  */
struct FNiagaraDataChannelPublishRequest
{
	/** The buffer containing the data to be published. This can come from a data channel DI or can be the direct contents on a Niagara simulation. */
	FNiagaraDataBufferRef Data;

	/** Game level data if this request comes from the game code. */
	TSharedPtr<FNiagaraDataChannelGameData> GameData = nullptr;

	/** If true, data in this request will be made visible to BP and C++ game code.*/
	bool bVisibleToGame = false;

	/** If true, data in this request will be made visible to Niagara CPU simulations. */
	bool bVisibleToCPUSims = false;

	/** If true, data in this request will be made visible to Niagara GPU simulations. */
	bool bVisibleToGPUSims = false;

	/** 
	LWC Tile for the originator system of this request.
	Allows us to convert from the Niagara Simulation space into LWC coordinates.
	*/
	FVector3f LwcTile = FVector3f::ZeroVector;

#if WITH_NIAGARA_DEBUGGER
	/** Instigator of this write, used for debug tracking */
	FString DebugSource;
#endif

	FNiagaraDataChannelPublishRequest() = default;
	explicit FNiagaraDataChannelPublishRequest(FNiagaraDataBufferRef InData)
		: Data(InData)
	{
	}

	explicit FNiagaraDataChannelPublishRequest(FNiagaraDataBufferRef InData, bool bInVisibleToGame, bool bInVisibleToCPUSims, bool bInVisibleToGPUSims, FVector3f InLwcTile)
	: Data(InData), bVisibleToGame(bInVisibleToGame), bVisibleToCPUSims(bInVisibleToCPUSims), bVisibleToGPUSims(bInVisibleToGPUSims), LwcTile(InLwcTile)
	{
	}
};