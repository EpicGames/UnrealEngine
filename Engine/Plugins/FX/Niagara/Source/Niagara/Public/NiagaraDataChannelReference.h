// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannelAccessContext.h"

#include "NiagaraDataChannelReference.generated.h"

/** A reference to a Niagara Data Channel with a pre-typed access context for convenience. */
USTRUCT(BlueprintType)
struct FNiagaraDataChannelReference
{
	GENERATED_BODY()

	[[nodiscard]] inline UNiagaraDataChannelAsset* GetDataChannel()const { return DataChannel; }
	/** Returns the internal access context instance. Can be invalid. */
	[[nodiscard]] NIAGARA_API const FNDCAccessContextInst& GetAccessContext()const;
	/** Returns a usable access context for accessing the referenced NDC. If the internal context is valid it will include it's input data. */
	[[nodiscard]] NIAGARA_API FNDCAccessContextInst& GetUsableAccessContext()const;
	NIAGARA_API void SetDataChannel(UNiagaraDataChannelAsset* InDataChannel);	
	NIAGARA_API void OnValueChanged();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DataChannel)
	TObjectPtr<UNiagaraDataChannelAsset> DataChannel;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DataChannel, meta = (EditCondition = "bCustomAccessContext" , ShowOnlyInnerProperties, ShowNDCAccessContextInput))
	mutable FNDCAccessContextInst AccessContext;

	/** If true this reference will provide a custom access context. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = DataChannel, meta = (InlineEditCondition))
	bool bCustomAccessContext = false;

private:

	void InitAccessContext(bool bTransferExisting = true)const;
};