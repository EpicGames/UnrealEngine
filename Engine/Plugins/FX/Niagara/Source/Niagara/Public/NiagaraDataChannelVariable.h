// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"
#include "NiagaraDataChannelVariable.generated.h"

USTRUCT(BlueprintType)
struct FNiagaraDataChannelVariable : public FNiagaraVariableBase
{
	GENERATED_USTRUCT_BODY()
	
	bool Serialize(FArchive& Ar);

#if WITH_EDITORONLY_DATA
	
	/** Can be used to track renamed data channel variables */
	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	FGuid Version = FGuid::NewGuid();

	NIAGARA_API static bool IsAllowedType(const FNiagaraTypeDefinition& Type);
#endif
	
	NIAGARA_API static FNiagaraTypeDefinition ToDataChannelType(const FNiagaraTypeDefinition& Type);

};

template<>
struct TStructOpsTypeTraits<FNiagaraDataChannelVariable> : public TStructOpsTypeTraitsBase2<FNiagaraDataChannelVariable>
{
	enum
	{
		WithSerializer = true,
	};
};