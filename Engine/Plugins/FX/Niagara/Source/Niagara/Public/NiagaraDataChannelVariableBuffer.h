// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraTypes.h"

/** Buffer containing a single FNiagaraVariable's data at the game level. AoS layout. LWC Types. */
struct FNiagaraDataChannelVariableBuffer
{
	TArray<uint8> Data;
	TArray<uint8> PrevData;
	int32 Size = 0;

	void Init(const FNiagaraVariableBase& Var)
	{
		//Position types are a special case where we have to store an LWC Vector in game level data and convert to a simulation friendly FVector3f as it enters Niagara.
		if (Var.GetType() == FNiagaraTypeDefinition::GetPositionDef())
		{
			Size = FNiagaraTypeHelper::GetVectorDef().GetSize();
		}
		else
		{
			Size = Var.GetSizeInBytes();
		}
	}

	void Empty() 
	{
		PrevData.Empty();
		Data.Empty(); 
	}

	void Reset()
	{
		PrevData.Reset();
		Data.Reset();
	}

	void BeginFrame(bool bKeepPrevious)
	{
		if (bKeepPrevious)
		{
			Swap(Data, PrevData);
		}
		Data.Reset();
	}

	template<typename T>
	bool Write(int32 Index, const T& InData)
	{
		check(sizeof(T) == Size);
		if (Index >= 0 && Index < Num())
		{
			T* Dest = reinterpret_cast<T*>(Data.GetData()) + Index;
			*Dest = InData;

			return true;
		}
		return false;
	}

	template<typename T>
	bool Read(int32 Index, T& OutData, bool bPreviousFrameData)const
	{
		check(sizeof(T) == Size);

		//Should we fallback to previous if we don't have current?
		//bPreviousFrameData |= Data.Num() == 0;

		int32 NumElems = bPreviousFrameData ? PrevNum() : Num();
		if (Index >= 0 && Index < NumElems)
		{
			const uint8* DataPtr = bPreviousFrameData ? PrevData.GetData() : Data.GetData();
			const T* Src = reinterpret_cast<const T*>(DataPtr) + Index;
			OutData = *Src;

			return true;
		}
		OutData = T();
		return false;
	}

	void SetNum(int32 Num)
	{
		Data.SetNumZeroed(Size * Num);
	}

	void Reserve(int32 Num)
	{
		Data.Reserve(Size * Num);
	}

	int32 Num()const { return Data.Num() / Size; }
	int32 PrevNum()const { return PrevData.Num() / Size; }

	int32 GetElementSize()const { return Size; }
};


