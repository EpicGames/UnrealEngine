// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannel_Map.h"
#include "GameplayTagContainer.h"
#include "NiagaraDataChannel_GameplayBurst.generated.h"

/** Access Context for Gameplay Burst NDC. */
USTRUCT(BlueprintType)
struct FNDCAccessContext_GameplayBurst : public FNDCAccessContext_MapBase
{
	GENERATED_BODY()
		
	/** Flag set during the access that indicates whether the spawned systems are attached to the owning component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output", meta=(NDCAccessContextOutput))
	uint8 bAttachedToOwningComponent : 1 = false;

	/** If true, we will attempt to attach to the owning component if one exists. Ignoring other attachment settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input", meta = (NDCAccessContextInput))
	uint8 bForceAttachToOwningComponent : 1 = false;

	/** 
	If true, we will override the default grid Cell Size from the Data Channel. 
	Caution: 
	Every unique cell size value will create a new data bucket and handler Niagara System. 
	Use with care. Overriding cell size provides more flexibility for users but can cause problems if not done correctly.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input", meta = (NDCAccessContextInput, InlineEditConditionToggle))
	uint8 bOverrideCellSize : 1 = false;
	
	/** If true the SystemBoundsPadding in the context will override the bounds padding defind in the NDC. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input", meta = (NDCAccessContextInput, InlineEditConditionToggle))
	uint8 bOverrideBoundsPadding : 1 = false;

	/** 
	An override for the Cell Size used for internal data routing.
	Caution:
	Every unique cell size value will create a new data bucket and handler Niagara System.
	Use with care. Overriding cell size provides more flexibility for users but can cause problems if not done correctly.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input", meta = (NDCAccessContextInput, EditCondition="bOverrideCellSize"))
	FVector CellSizeOverride = FVector(2500.0);

	/** 
	Padding applied to the bounds of spawned handler systems. 
	Should be large enough to contain any spawned FX near the edge of a grid cell. 
	When systems are attached to the owning component, bounds are the component bounds + SystemBoundsPadding.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input", meta = (NDCAccessContextInput, EditCondition="bOverrideBoundsPadding"))
	FVector SystemBoundsPadding = FVector(250.0);

	/** Data can optionally be routed by a given Gameplay Tag for additional variation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input", meta = (NDCAccessContextInput))
	FGameplayTag GameplayTag;
};

/** Settings controlling whether the NDC Gameplay Burst will use it's attached FX path and route data to a bucket targeted to a single owning component. */
USTRUCT()
struct FNDCGameplayBurstAttachmentSettings
{
	GENERATED_BODY()

	/** 
	Use attached path if the owning component is traveling >= this speed. Defaults to -ve to indicate we should not attach based on speed.
	*/
	UPROPERTY(EditAnywhere, Category = "Attachment")
	float SpeedThreshold = -1.0f;

	/** Use attached path if the owning component contains any of the following game play tags. */
	UPROPERTY(EditAnywhere, Category = "Attachment")
	FGameplayTagContainer GameplayTags;
	
	/** Use attached path if the owning component is one of these component types (including a subclass). */
	UPROPERTY(EditAnywhere, Category = "Attachment")
	TArray<TSubclassOf<USceneComponent>> ComponentTypes;

	/** Tests the given component against the attachment settings and returns true if spawned data and FX should be attached to the component. */
	bool UseAttachedPathForComponent(const USceneComponent* Component, bool bForce)const;
};


/**
Data Channel for Gameplay Burst FX.
Will bucket data primarily into world aligned grid cells.
Each bucket can optionally spawn a Niagara System to consume data in that bucket.
Can also route data to specific buckets and fx that are attached to the incoming OwnerComponent.
Optional additional routing by gameplay tag is available.
*/
UCLASS(MinimalAPI)
class UNiagaraDataChannel_GameplayBurst : public UNiagaraDataChannel_MapBase
{
	GENERATED_BODY()

public:

	virtual UNiagaraDataChannelHandler* CreateHandler(UWorld* OwningWorld)const override;

	[[nodiscard]] virtual TNDCAccessContextType GetAccessContextType()const override { return TNDCAccessContextType(FNDCAccessContext_GameplayBurst::StaticStruct()); }
	[[nodiscard]] const FNDCGameplayBurstAttachmentSettings& GetAttachmentSettings()const{ return AttachmentSettings; }
	[[nodiscard]] FVector GetGridCellSize()const { return CellSize; }
	[[nodiscard]] FVector GetSystemBoundsPadding()const { return SystemBoundsPadding; }

protected:
	
	/** 
	Default Cell Size. Can be overridden by users via the AccessContext.
	Internal data and their spawned handler FX are bucketed internally in a world aligned grid of cells.
	*/
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (NDCAccessContextInput))
	FVector CellSize = FVector(2500.0);

	/** 
	Default System Bounds Padding. Can be overridden by users via the AccessContext.
	Niagara Systems spawned by this NDC will have their bounds padded by this amount.
	For attached systems, the bounds will be the component bounds + padding.
	For non attached, the bounds will be the CellSize + padding.
	*/
	UPROPERTY(EditAnywhere, Category = "Grid", meta = (NDCAccessContextInput))
	FVector SystemBoundsPadding = FVector(250.0);

	/** 
	Settings controlling whether incoming data should route to the main world aligned spatial grids or use a special bucket attached to and owned by a specific component.
	Useful in cases where we want to attach Burst FX to dynamic moving objects etc for improved visuals.
	*/
	UPROPERTY(EditAnywhere, Category = "Attachment")
	FNDCGameplayBurstAttachmentSettings AttachmentSettings;
};

/** Map entry data stored for each unique Key generated by the Gameplay Burst NDC handler being accessed. */
USTRUCT()
struct FNDCMapEntry_GameplayBurst : public FNDCMapEntryBase
{
	GENERATED_BODY()
	friend class UNiagaraDataChannelHandler_GameplayBurst;

	virtual void Init(FNDCAccessContextInst& AccessContext, UNiagaraDataChannelHandler_MapBase* InOwner, const FNDCMapKey& Key)override;
	virtual void Reset(const FNDCMapKey& Key)override;
	virtual bool BeginFrame(float DeltaTime, FNiagaraWorldManager* OwningWorld, const FNDCMapKey& Key)override;

private:

	/** Non-null if this entry is attached to a specific component. */
	TWeakObjectPtr<USceneComponent> AttachComponent;
};

UCLASS(BlueprintType, MinimalAPI)
class UNiagaraDataChannelHandler_GameplayBurst : public UNiagaraDataChannelHandler_MapBase
{
	friend struct FNDCMapEntry_GameplayBurst;
	GENERATED_UCLASS_BODY()

	NIAGARA_API virtual void Init(const UNiagaraDataChannel* InChannel) override;

	/** Returns the per entry struct we'll store for each map key. */
	[[nodiscard]] virtual TNDCMapEntryType GetMapEntryType()const override { return TNDCMapEntryType(FNDCMapEntry_GameplayBurst::StaticStruct()); }

	/** Generates a map key for the given access context. Each unique key generated will store it's own map entry and spawned FX. */
	virtual void GenerateKey(FNDCAccessContextInst& AccessContext, FNDCMapKeyWriter& KeyWriter)const override;

protected:

	//cached values from NDC to avoid weak ptr dereference.

	UPROPERTY()
	FNDCGameplayBurstAttachmentSettings AttachmentSettings;

	FVector DefaultCellSize = FVector(2500.0);
	FVector DefaultSystemBoundsPadding = FVector(250.0);

	/** A tag we use to mark spawned attached components to skip some work re-checking attachment settings. */
	static const FName AttachedComponentTag;
};

//////////////////////////////////////////////////////////////////////////
