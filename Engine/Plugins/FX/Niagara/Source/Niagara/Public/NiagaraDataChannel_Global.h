// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelHandler.h"
#include "NiagaraDataChannel_Global.generated.h"

/**
Simple DataChannel handler that makes all data visible globally.
*/
UCLASS(MinimalAPI)
class UNiagaraDataChannel_Global : public UNiagaraDataChannel
{
	GENERATED_BODY()

	NIAGARA_API virtual UNiagaraDataChannelHandler* CreateHandler(UWorld* OwningWorld)const override;
};

/**
Basic DataChannel handler that makes all data visible globally.
*/
UCLASS(BlueprintType, MinimalAPI)
class UNiagaraDataChannelHandler_Global : public UNiagaraDataChannelHandler
{
	GENERATED_UCLASS_BODY()

	FNiagaraDataChannelDataPtr Data;

	NIAGARA_API virtual void Init(const UNiagaraDataChannel* InChannel) override;
	NIAGARA_API virtual void Cleanup() override;
	NIAGARA_API virtual void BeginFrame(float DeltaTime)override;
	NIAGARA_API virtual void EndFrame(float DeltaTime)override;
	NIAGARA_API virtual void Tick(float DeltaTime, ETickingGroup TickGroup) override;
	NIAGARA_API virtual FNiagaraDataChannelDataPtr FindData(FNiagaraDataChannelSearchParameters SearchParams, ENiagaraResourceAccess AccessType) override;
};

/**
DataChannel handler that separates DataChannel out by the hash of their position into spatially localized grid cells.
Useful for DataChannel that should only be visible to nearby consumers.
TODO: Likely unsuitable for LWC games as we need a defined min/max for the hash and storage. A sparse octree etc would be needed
*/
// UCLASS(EditInlineNew)
// class UNiagaraDataChannelChannelHandler_WorldGrid : public UObject
// {
// };

// UCLASS(EditInlineNew, abstract)
// class UNiagaraDataChannelChannelHandler_Octree : UObject
// {
// 
// };
