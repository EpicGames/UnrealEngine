// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelHandler.h"
#include "NiagaraDataChannel_Islands.generated.h"

struct FStreamableHandle;

UENUM()
enum class ENiagraDataChannel_IslandMode : uint8
{
	/** Islands are aligned to a grid and fixed to their MaxExtents. Avoids overlapping islands but can lead to larger than necessary islands. */
	AlignedStatic,
	/** Islands can exist a any location and will grow from their InitialExtents to their MaxExtents to fit data entering the channel data. Islands are as small as possible but can often overlap depending on the distribution of data entering the channel. */
	Dynamic,
};

/**
Data channel that will sub-divide the world into discreet islands based on distance.
New islands will be created as needed and existing islands discarded when no longer used.
Each Island can spawn one or more handler systems that will be in charge of reading and/or writing data for each channel.
The location and bounds of these handler systems will be modified to cover the whole island.
*/

/** Data for a single data channel island. */
USTRUCT()
struct FNDCIsland
{
	GENERATED_BODY()

	UE_NONCOPYABLE(FNDCIsland)
public:

	FNDCIsland();
	~FNDCIsland();

	void Init(UNiagaraDataChannelHandler_Islands* Owner);
	void BeginFrame();
	void EndFrame();
	void Tick(const ETickingGroup& TickGroup);

	bool Contains(FVector Point);
	double DistanceToPoint(FVector Point);
	bool Intersects(const FBoxSphereBounds& CheckBounds);
	bool IsHandlerSystem(USceneComponent* Component);

	/** 
	Attempt to grow this island such that it could contain the given point and element bounds. 
	Returns true if growing succeeded and false if this island cannot contain this point.
	*/
	bool TryGrow(FVector Point, FVector PerElementExtents, FVector MaxIslandExtents);

	void Grow(const FBoxSphereBounds& NewBounds);

	void OnAcquired(FVector Location);
	void OnReleased();
	bool IsBeingUsed()const;

	FNiagaraDataChannelDataPtr GetData()const { return Data; }

	void DebugDrawBounds();

private:

	/** The owning handler for this island. */
	UPROPERTY()
	TObjectPtr<UNiagaraDataChannelHandler_Islands> Owner = nullptr;

	/** Current bounds of this island. The bounds of any handler systems are modified to match these bounds. */
	UPROPERTY()
	FBoxSphereBounds Bounds = FBoxSphereBounds(EForceInit::ForceInit);

	/** Niagara components spawned for this island. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UNiagaraComponent>> SpawnedComponents;
	
	/** The underlying storage for this island. */
	FNiagaraDataChannelDataPtr Data = nullptr;
};

template<>
struct TStructOpsTypeTraits<FNDCIsland> : public TStructOpsTypeTraitsBase2<FNDCIsland>
{
	enum{ WithCopy = false };
};

USTRUCT()
struct FNDCIslandDebugDrawSettings
{
	GENERATED_BODY()
	
	FNDCIslandDebugDrawSettings()
	: bEnabled(false)
	, bShowIslandBounds(false)
	{}

	UPROPERTY(EditAnywhere, Category = "Debug Drawing")
	uint32 bEnabled : 1;

	UPROPERTY(EditAnywhere, Category = "Debug Drawing")
	uint32 bShowIslandBounds : 1;

	bool ShowBounds()const { return bEnabled && bShowIslandBounds; }
};

/**
Data channel that will automatically sub-divide the world into discreet "islands" based on location.
*/
UCLASS(MinimalAPI)
class UNiagaraDataChannel_Islands : public UNiagaraDataChannel
{
	GENERATED_BODY()

public:

	//UObject Interface
	NIAGARA_API virtual void PostLoad()override;
	//UObject Interface End

	NIAGARA_API virtual UNiagaraDataChannelHandler* CreateHandler(UWorld* OwningWorld)const override;

	void AsyncLoadSystems()const;

	ENiagraDataChannel_IslandMode GetMode()const { return Mode; }
	FVector GetInitialExtents()const { return InitialExtents; }
	FVector GetMaxExtents()const { return MaxExtents; }
	FVector GetPerElementExtents()const { return PerElementExtents; }

	NIAGARA_API TConstArrayView<TObjectPtr<UNiagaraSystem>> GetSystems()const;
	int32 GetIslandPoolSize()const { return IslandPoolSize; }
	const FNDCIslandDebugDrawSettings& GetDebugDrawSettings()const { return DebugDrawSettings; }

protected:

	/** Controls how islands are placed and sized. */
	UPROPERTY(EditAnywhere, Category = "Islands")
	ENiagraDataChannel_IslandMode Mode = ENiagraDataChannel_IslandMode::AlignedStatic;

	/** Starting extents of the island's bounds. */
	UPROPERTY(EditAnywhere, Category = "Islands", meta = (EditCondition = "Mode == ENiagraDataChannel_IslandMode::Dynamic"))
	FVector InitialExtents = FVector(1000.0 , 1000.0, 1000.0);

	/** The maximum total extents of each island. If a new element would grow the bounds beyond this size then a new island is created. */
	UPROPERTY(EditAnywhere, Category = "Islands")
	FVector MaxExtents = FVector(5000.0, 5000.0, 5000.0);

	/**
	The extents for every element entered into this data channel.
	We use this to pad the ends of islands to ensure that all data in an island will be covered.
	*/
	UPROPERTY(EditAnywhere, Category="Islands")
	FVector PerElementExtents = FVector(250.0, 250.0, 250.0);

	/** 
	One or more Niagara Systems to spawn that will consume the data in this island.
	Each island will have an instance of these systems created.
	These systems are intended to consume data for this whole island and generate effects that cover the whole island.
	The actual bounds of each of these system instances will be set to the current total bounds of the island.
	*/
	UPROPERTY(EditAnywhere, Category = "Islands")
	TArray<TSoftObjectPtr<UNiagaraSystem>> Systems;


	//TODO: Apply the same/similar scalability checks for things entering the data channel as there are for regular FX. Distance
	/** Variable from which to take the position of each element in the channel. */
	//UPROPERTY(EditAnywhere, Category = "Islands")
	//FNiagaraVariable PositionVariable;
	//UPROPERTY(EditAnywhere, Category="Data Channel")
	//FNiagaraSystemScalabilitySettings ScalabilitySettings;
	//TODO: Scalability

	/** How many pre-allocated islands to keep in the pool. Higher values will incur a larger standing memory cost but will reduce activation times for new islands. */
	UPROPERTY(EditAnywhere, Category = "Islands")
	int32 IslandPoolSize = 4;

	UPROPERTY(EditAnywhere, Category = "Debug Rendering")
	FNDCIslandDebugDrawSettings DebugDrawSettings;

	void PostLoadSystems()const;

	UPROPERTY(Transient)
	mutable TArray<TObjectPtr<UNiagaraSystem>> SystemsInternal;

	mutable TSharedPtr<FStreamableHandle> AsyncLoadHandle;
};

UCLASS(BlueprintType, MinimalAPI)
class UNiagaraDataChannelHandler_Islands : public UNiagaraDataChannelHandler
{
	GENERATED_UCLASS_BODY()

	NIAGARA_API virtual void Init(const UNiagaraDataChannel* InChannel) override;
	NIAGARA_API virtual void Cleanup() override;
	NIAGARA_API virtual void BeginFrame(float DeltaTime)override;
	NIAGARA_API virtual void EndFrame(float DeltaTime)override;
	NIAGARA_API virtual void Tick(float DeltaTime, ETickingGroup TickGroup) override;
	NIAGARA_API virtual FNiagaraDataChannelDataPtr FindData(FNiagaraDataChannelSearchParameters SearchParams, ENiagaraResourceAccess AccessType) override;

protected:

	/** All currently active Islands for this channel. */
	UPROPERTY()
	TArray<int32> ActiveIslands;

	/** All currently free Islands for this channel. */
	UPROPERTY()
	TArray<int32> FreeIslands;

	/** Pool of all islands. */
	UPROPERTY()
	TArray<FNDCIsland> IslandPool;

	/** Gets the correct island for the given location. */
	FNDCIsland* FindOrCreateIsland(const FNiagaraDataChannelSearchParameters& SearchParams, ENiagaraResourceAccess AccessType);

	/** Initializes and adds a new island to the ActiveIslands list. Either retrieving from the free pool of existing inactive islands or creating a new one. */
	int32 ActivateNewIsland(FVector Location);
};