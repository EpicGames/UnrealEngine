// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelHandler.h"
#include "NiagaraSystemCollection.h"
#include "NiagaraSystem.h"
#include "NiagaraWorldManager.h"
#include "NiagaraDataChannel_Map.generated.h"

class UNiagaraDataChannelHandler_MapBase;
struct FNDCMapKey;

#define UE_API NIAGARA_API

/*
Base class for Map entries for NDC Map. 
Child classes of the NDC Map type should implement their own entry type and implement any needed overrides. 
*/
USTRUCT()
struct FNDCMapEntryBase
{
	GENERATED_BODY()
	friend class UNiagaraDataChannelHandler_MapBase;

	virtual ~FNDCMapEntryBase() = default;


	/** 
	* Initializes a new map entry after creation or re-activation. 
	* 
	* @Param Access Context: Access context used to trigger creation/re-use of this entry.
	* @Param Key: The Key generated for this entry. Some Entry types may encode data in their key that can be used in other ways.
	*/
	virtual void Init(FNDCAccessContextInst& AccessContext, UNiagaraDataChannelHandler_MapBase* InOwner, const FNDCMapKey& Key);

	/** 
	* Called as the entry becomes inactive and is moved to the free pool. Performs bookkeeping like freeing large allocations etc but can cache any constant or expensive intermediate data.
	* @Param Key: The Key generated for this entry. Some Entry types may encode data in their key that can be used in other ways.
	*/
	virtual void Reset(const FNDCMapKey& Key);

	/** 
	* Cleans up a map entry just before it is destroyed.
	*/
	virtual void Cleanup();

	/**
	* Perform any bookkeeping needed at the beginning of each frame.
	*
	* @Param DeltaTime: The time in seconds from this world tick from the previous.
	* @Param OwningWorld: The world manager that owns this entry and it's owning handler.
	* @Param Key: The Key generated for this entry. Some Entry types may encode data in their key that can be used in other ways.
	* @Return True if the entry is still in use. False if this entry is unused and we can return it to the free entries pool.
	*/
	virtual bool BeginFrame(float DeltaTime, FNiagaraWorldManager* OwningWorld, const FNDCMapKey& Key);
	
	/**
	* Perform any bookkeeping needed at the end of each frame.
	*
	* @Param DeltaTime: The time in seconds from this world tick from the previous.
	* @Param OwningWorld: The world manager that owns this entry and it's owning handler.
	* @Param Key: The Key generated for this entry. Some Entry types may encode data in their key that can be used in other ways.
	*/
	virtual void EndFrame(float DeltaTime, FNiagaraWorldManager* OwningWorld, const FNDCMapKey& Key);

	/**
	* Perform per tick group processing. Called once per TickGroup.
	*
	* @Param DeltaTime: The time in seconds from this world tick from the previous.
	* @Param TickGroup: The current Tick Group being processed.
	* @Param OwningWorld: The world manager that owns this entry and it's owning handler.
	* @Param Key: The Key generated for this entry. Some Entry types may encode data in their key that can be used in other ways.
	*/
	virtual void Tick(float DeltaTime, ETickingGroup TickGroup, FNiagaraWorldManager* OwningWorld, const FNDCMapKey& Key);

	virtual UNiagaraComponent* SpawnSystem(FNDCAccessContext_MapBase& AccessContext, UObject* System, USceneComponent* AttacheParent, FVector Location, FVector BoundExtents, bool bActivate);

	template<typename T=UNiagaraDataChannelHandler_MapBase>
	[[nodiscard]] T* GetOwner()const{ return CastChecked<T>(Owner); }

	[[nodiscard]] const FNiagaraDataChannelDataPtr& GetData()const { return Data; }

	[[nodiscard]] float GetLastUsedTime()const{ return LastUsedTime; }

	const TArray<TObjectPtr<UNiagaraComponent>>& GetSpawnedSystems() { return SpawnedComponents; }
protected:

	/** The owning handler for this entry. */
	UPROPERTY()
	TObjectPtr<UNiagaraDataChannelHandler_MapBase> Owner = nullptr;

	/** Niagara Components spawned for this entry. */
	UPROPERTY()
	TArray<TObjectPtr<UNiagaraComponent>> SpawnedComponents;

	/** The underlying storage for this entry. */
	FNiagaraDataChannelDataPtr Data = nullptr;

	/** Times since this entry was last used. */
	UPROPERTY()
	float LastUsedTime = 0.0f;
};

//Describes a type that can be used for the NDC Map entries.
//Wrapper around a UScriptStruct* that checks for a valid type.
struct TNDCMapEntryType
{
	TNDCMapEntryType(UScriptStruct* InStruct)
	 :EntryStruct(InStruct->IsChildOf(FNDCMapEntryBase::StaticStruct()) ? InStruct : nullptr)
	{
		check(EntryStruct);
	}
	
	[[nodiscard]] inline const UScriptStruct* Get()const { return EntryStruct; }

private:
	UScriptStruct* EntryStruct = nullptr;
};

/** 
Entry type for the NDC internal map.
Wrapper around an FInstancedStruct that ensures a valid type is used.
*/
USTRUCT()
struct FNDCMapEntry
{
	GENERATED_BODY()
	FNDCMapEntry() = default;
	[[nodiscard]] explicit FNDCMapEntry(TNDCMapEntryType EntryType, int32 InIndex)
	: Index(InIndex)
	{
		EntryData.InitializeAs(EntryType.Get());
	}

	template<typename T=FNDCMapEntryBase>
	[[nodiscard]] T& Get(){ return EntryData.GetMutable<T>(); }

	template<typename T=FNDCMapEntryBase>
	[[nodiscard]] const T& Get()const { return EntryData.Get<T>(); }

	[[nodiscard]] int32 GetIndex()const { return Index; }

	[[nodiscard]] bool IsValid()const { return EntryData.IsValid(); }
private:

	UPROPERTY()
	FInstancedStruct EntryData;

	/** Index of this entry in the owning pool. */
	int32 Index = INDEX_NONE;
};


#define DEBUG_NDC_MAP_ACCESS 0

/** Key type for NDC access. Each handler can fill in key data as needed to route it's own AccessContext data to the correct internal Map Entry. */
USTRUCT()
struct FNDCMapKey
{
	GENERATED_BODY();

	[[nodiscard]] bool operator==(const FNDCMapKey&)const = default;

	TArray<uint8, TInlineAllocator<64>> KeyData;
};

inline uint32 GetTypeHash(const FNDCMapKey& MapKey)
{
	return GetArrayHash(MapKey.KeyData.GetData(), MapKey.KeyData.Num());
}

/**
A utility struct for Map NDCs to write an arbitrary map key based on their data/needs.
Key can be a limited subset of the source data but it must be ensured to be unique and to avoid conflicts.
*/
struct FNDCMapKeyWriter 
{
	FNDCMapKeyWriter(FNDCMapKey& InKey)
	: DestKey(InKey)
	{}

	inline void operator<<(const FName& N)
	{
		*this << N.GetComparisonIndex().ToUnstableInt();
	}

	inline void operator<<(const FNiagaraSystemCollectionData& SystemCollecton)
	{
		for(const TObjectPtr<UNiagaraSystem>& System : SystemCollecton.GetSystems())
		{
			*this << System->GetFName();
		}
	}
	
	template<typename T>
	inline void operator<<(const T& V)
	{
		static_assert(TIsPODType<T>::Value, "FNDCMapKeyWriter can only handle POD types.");
		int32 CurrNum = DestKey.KeyData.Num();
		DestKey.KeyData.AddUninitialized(sizeof(T));
		FMemory::Memcpy(DestKey.KeyData.GetData() + CurrNum, &V, sizeof(T));
	}
	
	FNDCMapKey& DestKey;
};

USTRUCT(BlueprintType, meta=(Hidden))
struct FNDCAccessContext_MapBase : public FNDCAccessContext
{
	GENERATED_BODY()
};

/**
Data channel that will sub divide it's internal data based on a customizable map key.
*/
UCLASS(MinimalAPI, abstract)
class UNiagaraDataChannel_MapBase : public UNiagaraDataChannel
{
	GENERATED_BODY()

public:

	[[nodiscard]] UObject* GetDefaultSystemToSpawn()const { return DefaultSystemToSpawn; }

protected:

	/** Niagara System (or System Collection) that we should spawn to listen for and handle NDC data. Can be overriden per via the SystemToSpawn property in the Access Context. */
	UPROPERTY(EditAnywhere, Category = "Data Channel", meta = (AllowedClasses = "/Script/Niagara.NiagaraSystem,/Script/Niagara.NiagaraSystemCollection"))
	TObjectPtr<UObject> DefaultSystemToSpawn;
};

/** Base Handler class for Data Channels that route and store subsets of internal data based in a map based on an arbitrary key they can generate in GenerateKey(). */
UCLASS(MinimalAPI, abstract)
class UNiagaraDataChannelHandler_MapBase : public UNiagaraDataChannelHandler
{
	GENERATED_UCLASS_BODY()

	virtual void Init(const UNiagaraDataChannel* InChannel) override;
	virtual void Cleanup() override;
	virtual void BeginFrame(float DeltaTime)override;
	virtual void EndFrame(float DeltaTime)override;
	virtual void Tick(float DeltaTime, ETickingGroup TickGroup) override;

	[[nodiscard]] virtual FNiagaraDataChannelDataPtr FindData(FNDCAccessContextInst& AccessContext, ENiagaraResourceAccess AccessType)override;

	[[nodiscard]] UWorld* GetWorld()const { return OwningWorld->GetWorld(); }

	[[nodiscard]] UObject* GetSystemToSpawn(FNDCAccessContextInst& AccessContext)const;
protected:

	/** Returns the map entry data type for this handler. Each child class should implement this and return their own subtype of FNDCMapEntryBase that contains their needed per entry data. */
	[[nodiscard]] virtual TNDCMapEntryType GetMapEntryType()const PURE_VIRTUAL(GetMapEntryType, return TNDCMapEntryType(FNDCMapEntryBase::StaticStruct()););

	/** Generates a map key for the given access context. */
	virtual void GenerateKey(FNDCAccessContextInst& AccessContext, FNDCMapKeyWriter& KeyWriter)const;

	[[nodiscard]] virtual FNDCMapEntry& FindOrAddEntry(FNDCAccessContextInst& AccessContext);
	
	float CleanFreeEntriesTimer = 0.0f;

	/** The index of the currently initializing entry to help handle re-entrant calls to FindData from spawned systems. */
	FNDCMapEntry* CurrentlyInitializingEntry = nullptr;

	UPROPERTY(transient)
	TArray<FNDCMapEntry> EntryPool;
	
	/** Currently Active Entries in EntryPool. Map of Entries by FNDCKey that we generate from an Access Context inside GenerateKey. Child classes can generate the key however they need and so we can easily subdivide the NDC internal data with this single map. */
	TMap<FNDCMapKey, int32> ActiveEntries;

	/** Currently unused entries in EntryPool that can be re-used next time a new Entry is needed. */
	TArray<int32> FreeEntries;

	/** Mapping from spawned handler systems back to their owning entry. Allows faster access when handler systems are accessing the NDC. */
	UPROPERTY(transient)
	TMap<TObjectPtr<UNiagaraComponent>, int32> SpawnedComponentsToActiveEntry;

	UPROPERTY()
	TObjectPtr<UObject> DefaultSystemToSpawn;
};

#undef UE_API
