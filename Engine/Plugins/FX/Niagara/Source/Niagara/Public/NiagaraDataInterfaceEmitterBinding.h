// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceEmitterBinding.generated.h"

class UNiagaraEmitter;
struct FNiagaraEmitterHandle;
class FNiagaraSystemInstance;
class FNiagaraEmitterInstance;

UENUM()
enum class ENiagaraDataInterfaceEmitterBindingMode
{
	/** Attempt to bind to the emitter the data interface is used with, this may not be possible in all situations (i.e. user parameter). */
	Self,
	/** Attempt to bind to an emitter using it's name. */
	Other
};

USTRUCT()
struct FNiagaraDataInterfaceEmitterBinding
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Binding")
	ENiagaraDataInterfaceEmitterBindingMode	BindingMode = ENiagaraDataInterfaceEmitterBindingMode::Other;
	
	UPROPERTY(EditAnywhere, Category = "Binding")
	FName EmitterName;

	/** Resolves the emitter binding, this can return nullptr if we failed to resolve */
	FNiagaraEmitterInstance* Resolve(const FNiagaraSystemInstance* SystemInstance, const UNiagaraDataInterface* DataInterface) const;

	/** Resolves the emitter binding for a data interface. */
	UNiagaraEmitter* Resolve(const UNiagaraDataInterface* DataInterface) const;

	/** Resolves the binding to an FNiagaraEmitterHandle or nullptr if it's invalid. */
	const FNiagaraEmitterHandle* ResolveHandle(const UNiagaraDataInterface* DataInterface) const;

	/** Resolves the binding to an FNiagaraEmitterHandle or nullptr if it's invalid. */
	const FNiagaraEmitterHandle* ResolveHandle(const UNiagaraSystem* OwnerSystem, const FNiagaraEmitterHandle* OwnerEmitter) const;

	/** Resolves the emitter name */
	FString ResolveUniqueName(const UNiagaraDataInterface* DataInterface) const;

	bool operator==(const FNiagaraDataInterfaceEmitterBinding& Other) const
	{
		return BindingMode == Other.BindingMode && EmitterName == Other.EmitterName;
	}
};
