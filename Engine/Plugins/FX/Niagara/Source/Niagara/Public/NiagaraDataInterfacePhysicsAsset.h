// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "UObject/Interface.h"
#include "RHIUtilities.h"

#include "NiagaraDataInterfacePhysicsAsset.generated.h"

#define PHYSICS_ASSET_MAX_PRIMITIVES 100
#define PHYSICS_ASSET_MAX_TRANSFORMS PHYSICS_ASSET_MAX_PRIMITIVES * 3

/** Element offsets in the array list */
struct FNDIPhysicsAssetElementOffset
{
	FNDIPhysicsAssetElementOffset(const uint32 InBoxOffset, const uint32 InSphereOffset, const uint32 InCapsuleOffset, const uint32 InNumElements) :
		BoxOffset(InBoxOffset), SphereOffset(InSphereOffset), CapsuleOffset(InCapsuleOffset), NumElements(InNumElements)
	{}

	FNDIPhysicsAssetElementOffset() :
		BoxOffset(0), SphereOffset(0), CapsuleOffset(0), NumElements(0)
	{}
	uint32 BoxOffset;
	uint32 SphereOffset;
	uint32 CapsuleOffset;
	uint32 NumElements;
};

/** Arrays in which the cpu datas will be str */
struct FNDIPhysicsAssetArrays
{
	FNDIPhysicsAssetElementOffset ElementOffsets;
	TStaticArray<FVector4f, 3 * PHYSICS_ASSET_MAX_TRANSFORMS> WorldTransform;
	TStaticArray<FVector4f, 3 * PHYSICS_ASSET_MAX_TRANSFORMS> InverseTransform;
	TStaticArray<FVector4f, PHYSICS_ASSET_MAX_TRANSFORMS> CurrentTransform;
	TStaticArray<FVector4f, PHYSICS_ASSET_MAX_TRANSFORMS> CurrentInverse;
	TStaticArray<FVector4f, PHYSICS_ASSET_MAX_TRANSFORMS> PreviousTransform;
	TStaticArray<FVector4f, PHYSICS_ASSET_MAX_TRANSFORMS> PreviousInverse;
	TStaticArray<FVector4f, PHYSICS_ASSET_MAX_TRANSFORMS> RestTransform;
	TStaticArray<FVector4f, PHYSICS_ASSET_MAX_TRANSFORMS> RestInverse;
	TStaticArray<FVector4f, PHYSICS_ASSET_MAX_PRIMITIVES> ElementExtent;
	TStaticArray<uint32, PHYSICS_ASSET_MAX_PRIMITIVES> PhysicsType;
};

/** Render buffers that will be used in hlsl functions */
struct FNDIPhysicsAssetBuffer : public FRenderResource
{
	/** Init the buffer */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Release the buffer */
	virtual void ReleaseRHI() override;

	/** Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FNDIPhysicsAssetBuffer"); }

	/** World transform buffer */
	FReadBuffer WorldTransformBuffer;

	/** Inverse transform buffer*/
	FReadBuffer InverseTransformBuffer;

	/** Element extent buffer */
	FReadBuffer ElementExtentBuffer;

	/** Physics type buffer */
	FReadBuffer PhysicsTypeBuffer;
};

/** Data stored per physics asset instance*/
struct FNDIPhysicsAssetData
{
	/** Initialize the cpu datas */
	void Init(class UNiagaraDataInterfacePhysicsAsset* Interface, FNiagaraSystemInstance* SystemInstance);

	/** Update the gpu datas */
	void Update(class UNiagaraDataInterfacePhysicsAsset* Interface, FNiagaraSystemInstance* SystemInstance);

	/** Release the buffers */
	void Release();

	/** Physics asset Gpu buffer */
	FNDIPhysicsAssetBuffer* AssetBuffer;

	/** Bounding box center */
	FVector BoxOrigin;

	/** Bounding box extent */
	FVector BoxExtent;

	/** Physics asset Cpu arrays */
	FNDIPhysicsAssetArrays AssetArrays;  

	/** The instance ticking group */
	ETickingGroup TickingGroup;
};

/** Data Interface for interacting with PhysicsAssets */
UCLASS(EditInlineNew, Category = "Physics", CollapseCategories, meta = (DisplayName = "Physics Asset"), MinimalAPI)
class UNiagaraDataInterfacePhysicsAsset : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	/** Skeletal Mesh from which the Physics Asset will be found. */
	UPROPERTY(EditAnywhere, Category = "Source")
	TObjectPtr<UPhysicsAsset> DefaultSource;

	/** The source actor from which to sample */
	UPROPERTY(EditAnywhere, Category = "Source", meta = (DisplayName = "Source Actor"))
	TSoftObjectPtr<AActor> SoftSourceActor;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<AActor> SourceActor_DEPRECATED;
#endif

	/** Reference to a user parameter if we're reading one. */
	UPROPERTY(EditAnywhere, Category = "Source")
	FNiagaraUserParameterBinding MeshUserParameter;

	/** The source component from which to sample */
	TArray<TWeakObjectPtr<class USkeletalMeshComponent>> SourceComponents;

	/** The source asset from which to sample */
	TArray<TWeakObjectPtr<class UPhysicsAsset>> PhysicsAssets;

	/** UObject Interface */
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;

	/** UNiagaraDataInterface Interface */
	NIAGARA_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target == ENiagaraSimTarget::GPUComputeSim; }
	NIAGARA_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	NIAGARA_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)override;
	NIAGARA_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual int32 PerInstanceDataSize()const override { return sizeof(FNDIPhysicsAssetData); }
	NIAGARA_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasTickGroupPrereqs() const override { return true; }
	NIAGARA_API virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const override;

	/** GPU simulation  functionality */
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	NIAGARA_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	NIAGARA_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	NIAGARA_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	NIAGARA_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	NIAGARA_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

	/** Extract the source component */
	NIAGARA_API void ExtractSourceComponent(FNiagaraSystemInstance* SystemInstance, FTransform& LocalTransform);

	/** Get the number of boxes*/
	NIAGARA_API void GetNumBoxes(FVectorVMExternalFunctionContext& Context);

	/** Get the number of spheres  */
	NIAGARA_API void GetNumSpheres(FVectorVMExternalFunctionContext& Context);

	/** Get the number of capsules */
	NIAGARA_API void GetNumCapsules(FVectorVMExternalFunctionContext& Context);

	/** Get the element point */
	NIAGARA_API void GetElementPoint(FVectorVMExternalFunctionContext& Context);

	/** Get the element distance */
	NIAGARA_API void GetElementDistance(FVectorVMExternalFunctionContext& Context);

	/** Get the closest element */
	NIAGARA_API void GetClosestElement(FVectorVMExternalFunctionContext& Context);

	/** Get the closest point */
	NIAGARA_API void GetClosestPoint(FVectorVMExternalFunctionContext& Context);

	/** Get the closest distance */
	NIAGARA_API void GetClosestDistance(FVectorVMExternalFunctionContext& Context);

	/** Get the closest texture point */
	NIAGARA_API void GetTexturePoint(FVectorVMExternalFunctionContext& Context);

	/** Get the projection point */
	NIAGARA_API void GetProjectionPoint(FVectorVMExternalFunctionContext& Context);

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	/** Copy one niagara DI to this */
	NIAGARA_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
};

/** Proxy to send data to gpu */
struct FNDIPhysicsAssetProxy : public FNiagaraDataInterfaceProxy
{
	/** Get the size of the data that will be passed to render*/
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FNDIPhysicsAssetData); }

	/** Get the data that will be passed to render*/
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override;

	/** Initialize the Proxy data buffer */
	void InitializePerInstanceData(const FNiagaraSystemInstanceID& SystemInstance);

	/** Destroy the proxy data if necessary */
	void DestroyPerInstanceData(const FNiagaraSystemInstanceID& SystemInstance);

	/** Launch all pre stage functions */
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override;

	/** List of proxy data for each system instances*/
	TMap<FNiagaraSystemInstanceID, FNDIPhysicsAssetData> SystemInstancesToProxyData;
};

UINTERFACE(MinimalAPI)
class UNiagaraPhysicsAssetDICollectorInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INiagaraPhysicsAssetDICollectorInterface
{
	GENERATED_IINTERFACE_BODY()

	virtual UPhysicsAsset* BuildAndCollect(
		FTransform& BoneTransform,
		TArray<TWeakObjectPtr<USkeletalMeshComponent>>& SourceComponents,
		TArray<TWeakObjectPtr<UPhysicsAsset>>& PhysicsAssets) const = 0;
};
