// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "NiagaraDataInterface.h"
#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "RHIUtilities.h"

#include "NiagaraDataInterfaceRigidMeshCollisionQuery.generated.h"

// Forward declaration
class AActor;

/** Element offsets in the array list */
struct FNDIRigidMeshCollisionElementOffset
{
	FNDIRigidMeshCollisionElementOffset(const uint32 InBoxOffset, const uint32 InSphereOffset, const uint32 InCapsuleOffset, const uint32 InNumElements) :
		BoxOffset(InBoxOffset), SphereOffset(InSphereOffset), CapsuleOffset(InCapsuleOffset), NumElements(InNumElements)
	{}

	FNDIRigidMeshCollisionElementOffset() :
		BoxOffset(0), SphereOffset(0), CapsuleOffset(0), NumElements(0)
	{}
	uint32 BoxOffset;
	uint32 SphereOffset;
	uint32 CapsuleOffset;
	uint32 NumElements;
};

/** Arrays in which the cpu datas will be str */
struct FNDIRigidMeshCollisionArrays
{
	FNDIRigidMeshCollisionElementOffset ElementOffsets;
	TArray<FVector4f> CurrentTransform;
	TArray<FVector4f> CurrentInverse;
	TArray<FVector4f> PreviousTransform;
	TArray<FVector4f> PreviousInverse;
	TArray<FVector4f> ElementExtent;
	TArray<FVector4f> MeshScale;
	TArray<uint32> PhysicsType;
	TArray<int32> ComponentIdIndex;
	TArray<FPrimitiveComponentId> UniqueCompnentId;

	FVector3f CombinedBBoxWorldMin;
	FVector3f CombinedBBoxWorldMax;

	FNDIRigidMeshCollisionArrays() = delete;
	FNDIRigidMeshCollisionArrays(uint32 Num)
		: MaxPrimitives(Num)
	{
		Reset();
	}

	void Reset()
	{
		ElementOffsets = FNDIRigidMeshCollisionElementOffset();
		CurrentTransform.Init(FVector4f(0, 0, 0, 0), 3 * MaxPrimitives);
		CurrentInverse.Init(FVector4f(0, 0, 0, 0), 3 * MaxPrimitives);
		PreviousTransform.Init(FVector4f(0, 0, 0, 0), 3 * MaxPrimitives);
		PreviousInverse.Init(FVector4f(0, 0, 0, 0), 3 * MaxPrimitives);
		ElementExtent.Init(FVector4f(0, 0, 0, 0), MaxPrimitives);
		MeshScale.Init(FVector4f(0, 0, 0, 0), MaxPrimitives);
		PhysicsType.Init(0, MaxPrimitives);
		ComponentIdIndex.Init(INDEX_NONE, MaxPrimitives);
		UniqueCompnentId.Reset();
	}

	const uint32 MaxPrimitives;
};

/** Render buffers that will be used in hlsl functions */
struct FNDIRigidMeshCollisionBuffer : public FRenderResource
{
	/** Init the buffer */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Release the buffer */
	virtual void ReleaseRHI() override;

	/** Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FNDIRigidMeshCollisionBuffer"); }

	/** World transform buffer */
	FReadBuffer WorldTransformBuffer;

	/** Inverse transform buffer*/
	FReadBuffer InverseTransformBuffer;

	/** Element extent buffer */
	FReadBuffer ElementExtentBuffer;

	/** Element extent buffer */
	FReadBuffer MeshScaleBuffer;

	/** Physics type buffer */
	FReadBuffer PhysicsTypeBuffer;

	/** Distance field index buffer */
	FReadBuffer DFIndexBuffer;

	/** Max number of primitives */
	uint32 MaxNumPrimitives;

	/** Max number of transforms (prev and next needed) */
	uint32 MaxNumTransforms;

	void SetMaxNumPrimitives(uint32 Num)
	{
		MaxNumPrimitives = Num;
		MaxNumTransforms = 2 * Num;
	}
};

/** Data stored per DI instance*/
struct FNDIRigidMeshCollisionData
{
	FNDIRigidMeshCollisionData(const FNiagaraSystemInstance* InSystemInstance, bool InHasScriptedFindActor)
		: SystemInstance(InSystemInstance)
		, bHasScriptedFindActor(InHasScriptedFindActor)
	{}

	/** Initialize the cpu datas */
	void Init(int32 MaxNumPrimitives);

	/** Update the gpu datas */
	void Update(UNiagaraDataInterfaceRigidMeshCollisionQuery* Interface);

	/** Release the buffers */
	void ReleaseBuffers();

	bool HasActors() const;
	bool ShouldRunGlobalSearch(UNiagaraDataInterfaceRigidMeshCollisionQuery* Interface) const;

	using FMergedActorArray = TArray<AActor*, TInlineAllocator<16>>;
	void MergeActors(FMergedActorArray& MergedActors) const;

	bool TrimMissingActors();

	/** Physics asset Gpu buffer */
	FNDIRigidMeshCollisionBuffer* AssetBuffer = nullptr;

	/** Physics asset Cpu arrays */
	TUniquePtr<FNDIRigidMeshCollisionArrays> AssetArrays = nullptr;

	/** Source actors **/
	TArray<TWeakObjectPtr<AActor>> ExplicitActors;

	struct FComponentBodyCount
	{
		uint32 ComponentHash = 0;
		uint32 BoxCount = 0;
		uint32 SphereCount = 0;
		uint32 CapsuleCount = 0;
	};

	TArray<FComponentBodyCount> MeshBodyCounts;

	/** Source actors **/
	TArray<TWeakObjectPtr<AActor>> FoundActors;

	const FNiagaraSystemInstance* SystemInstance = nullptr;

	/** Hash of the components that have been collected, used to see if we need to fully rebuild our array of transforms */
	uint32 ComponentCollectionHash = 0;

	/** Indicates that the instance is being used in a call to FindActors */
	const bool bHasScriptedFindActor;

	/** Indicates that a full update of the arrays is required */
	bool bRequiresFullUpdate = false;

	/** Indicates that we are currently in a failure state because we've found more components in the actors than we can support */
	bool bExceedingComponentLimits = false;
};

/** Data Interface used to collide against static meshes - whether it is the mesh distance field or a physics asset's collision primitive */
UCLASS(EditInlineNew, Category = "Collision", meta = (DisplayName = "Rigid Mesh Collision Query"))
class UNiagaraDataInterfaceRigidMeshCollisionQuery : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString Tag_DEPRECATED = TEXT("");
#endif

	/** Set of tags used to match against actors when searching for RigidBody providers.  Empty tags will be ignored, and only a single 
		tag is required for an actor to be matched. */
	UPROPERTY(EditAnywhere, Category = "Search")
	TArray<FName> ActorTags;

	/** Set of tags used to match against components when searching for RigidBody providers.  Empty tags will be ignored, and only a
		single tag is required for a component to be matched. */
	UPROPERTY(EditAnywhere, Category = "Source")
	TArray<FName> ComponentTags;

	/** Hardcoded references to actors that will be used as RigidBody providers. */
	UPROPERTY(EditAnywhere, Category = "Source")
	TArray<TSoftObjectPtr<AActor>> SourceActors;

	/** If enabled only actors that are considered moveable will be searched for RigidBodies. */
	UPROPERTY(EditAnywhere, Category = "Source")
	bool OnlyUseMoveable = true;

	/** If enabled, complex collisions will be searched for. */
	UPROPERTY(EditAnywhere, Category = "Source")
	bool UseComplexCollisions = false;

	/** If enabled, FindActors will use filtering based on ObjectType instead of Channel. */
	UPROPERTY(EditAnywhere, Category = "Source")
	bool bFilterByObjectType = false;

	/** If enabled the global search can be executed depending on GlobalSearchForced and GlobalSearchFallback_Unscripted */
	UE_DEPRECATED(5.2, "Global search will be deprecated in favor of using a spatial search through the FindActors function.")
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Source", meta = (DisplayName = "Global Search Allowed"))
	bool GlobalSearchAllowed = false;

	/** If enabled the global search will be performed only if there are no explicit actors specified */
	UE_DEPRECATED(5.2, "Global search will be deprecated in favor of using a spatial search through the FindActors function.")
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Source", meta = (DisplayName = "Global Search Forced", EditCondition = GlobalSearchAllowed))
	bool GlobalSearchForced = false;

	/** If enabled the global search will be performed only if there are no explicit actors specified */
	UE_DEPRECATED(5.2, "Global search will be deprecated in favor of using a spatial search through the FindActors function.")
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Source", meta = (DisplayName = "Global Search Only If Unscripted", EditCondition = GlobalSearchAllowed))
	bool GlobalSearchFallback_Unscripted = false;

	/** Maximum number of RigidBody represented by this DataInterface. */
	UPROPERTY(EditAnywhere, Category = "General")
	int MaxNumPrimitives = 100;

	/** UObject Interface */
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;

	/** UNiagaraDataInterface Interface */
	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target == ENiagaraSimTarget::GPUComputeSim; }
#if WITH_NIAGARA_DEBUGGER
	virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const override;
#endif
	virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)override;
	virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual int32 PerInstanceDataSize()const override { return sizeof(FNDIRigidMeshCollisionData); }
	virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool HasPostSimulateTick() const override { return true; }
	virtual bool HasTickGroupPrereqs() const override { return true; }
	virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const override;

	virtual bool RequiresEarlyViewData() const override { return true; }

	/** GPU simulation  functionality */
#if WITH_EDITORONLY_DATA
	virtual void GetCommonHLSL(FString& OutHLSL) override;
	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;

	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;

	virtual void ValidateFunction(const FNiagaraFunctionSignature& Function, TArray<FText>& OutValidationErrors) override;
#endif
	virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

	bool UpdateSourceActors(FNiagaraSystemInstance* SystemInstance, FNDIRigidMeshCollisionData& InstanceData) const;

	bool GetExplicitActors(FNDIRigidMeshCollisionData& InstanceData);
	bool FindActors(UWorld* World, FNDIRigidMeshCollisionData& InstanceData, ECollisionChannel Channel, const FVector& OverlapLocation, const FVector& OverlapExtent, const FQuat& OverlapRotation) const;
	
	UE_DEPRECATED(5.2, "Global search will be deprecated in favor of using a spatial search through the FindActors function.")
	bool GlobalFindActors(UWorld* World, FNDIRigidMeshCollisionData& InstanceData) const;

	void FindActorsCPU(FVectorVMExternalFunctionContext& Context);
	void GetNumElementsCPU(FVectorVMExternalFunctionContext& Context);
	void IsWorldPositionInsideCombinedBoundsCPU(FVectorVMExternalFunctionContext& Context);

#if WITH_EDITOR
	virtual void GetFeedback(UNiagaraSystem* InAsset, UNiagaraComponent* InComponent, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo) override;
#endif

protected:
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	/** Copy one niagara DI to this */
	virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	bool FilterComponent(const UPrimitiveComponent* Component) const;
	bool FilterActor(const AActor* Actor) const;
};

/**
* C++ and Blueprint library for accessing array types
*/
UCLASS(MinimalAPI)
class UNiagaraDIRigidMeshCollisionFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Niagara Set Source Actors"))
	static NIAGARA_API void SetSourceActors(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") FName OverrideName, const TArray<AActor*>& SourceActors);
};
