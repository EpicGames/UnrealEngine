// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCommon.h"
#include "NiagaraShared.h"

class UNiagaraComponent;
class UNiagaraDataInterface;
class UNiagaraSystem;
class FNiagaraSystemInstance;

namespace FNiagaraDataInterfaceUtilities
{
	struct FDataInterfaceUsageContext
	{
		const UObject*				OwnerObject = nullptr;
		FNiagaraVariableBase	Variable;
		UNiagaraDataInterface*	DataInterface = nullptr;
	};

	// Finds all VM function calls made using the runtime resolved data interface.
	// The same function call made be made multiple times across different scripts so you may see the same function multiple times
	// The action should return True to continue iteration or False to stop
	NIAGARA_API void ForEachVMFunction(UNiagaraDataInterface* ResolvedRuntimeDataInterface, const UNiagaraSystem* NiagaraSystem, TFunction<bool(const UNiagaraScript*, const FVMExternalFunctionBindingInfo&)> Action);
	NIAGARA_API void ForEachVMFunction(UNiagaraDataInterface* ResolvedRuntimeDataInterface, UNiagaraComponent* Component, TFunction<bool(const UNiagaraScript*, const FVMExternalFunctionBindingInfo&)> Action);
	NIAGARA_API void ForEachVMFunction(UNiagaraDataInterface* ResolvedRuntimeDataInterface, const FNiagaraSystemInstance* SystemInstance, TFunction<bool(const UNiagaraScript*, const FVMExternalFunctionBindingInfo&)> Action);

	// Finds all Gpu function calls made using the runtime resolved data interface.
	// The same function call made be made multiple times across different scripts so you may see the same function multiple times
	// The action should return True to continue iteration or False to stop
	NIAGARA_API void ForEachGpuFunction(UNiagaraDataInterface* ResolvedRuntimeDataInterface, const UNiagaraSystem* NiagaraSystem, TFunction<bool(const UNiagaraScript*, const FNiagaraDataInterfaceGeneratedFunction&)> Action);
	NIAGARA_API void ForEachGpuFunction(UNiagaraDataInterface* ResolvedRuntimeDataInterface, UNiagaraComponent* Component, TFunction<bool(const UNiagaraScript*, const FNiagaraDataInterfaceGeneratedFunction&)> Action);
	NIAGARA_API void ForEachGpuFunction(UNiagaraDataInterface* ResolvedRuntimeDataInterface, const FNiagaraSystemInstance* SystemInstance, TFunction<bool(const UNiagaraScript*, const FNiagaraDataInterfaceGeneratedFunction&)> Action);

	// Loops over all data interfaces inside the SystemInstance
	// The action should return True to continue iteration or False to stop
	NIAGARA_API void ForEachDataInterface(const FNiagaraSystemInstance* SystemInstance, TFunction<bool(const FNiagaraVariableBase Variable, UNiagaraDataInterface* DataInterface)> Action);
	// Loops over all data interfaces inside the SystemInstance
	// The action should return True to continue iteration or False to stop
	NIAGARA_API void ForEachDataInterface(const FNiagaraSystemInstance* SystemInstance, TFunction<bool(const FDataInterfaceUsageContext&)> Action);
	// Loops over all data interfaces inside the NiagaraSystem
	// The action should return True to continue iteration or False to stop
	NIAGARA_API void ForEachDataInterface(const UNiagaraSystem* NiagaraSystem, TFunction<bool(const FDataInterfaceUsageContext&)> Action);
}

#if WITH_EDITOR

/** Helper context object helping to facilitate data interfaces building their hlsl shader code for GPU simulations. */
struct FNiagaraDataInterfaceHlslGenerationContext
{
	DECLARE_DELEGATE_RetVal_OneParam(FString, FGetSanitizedFunctionParameters, const FNiagaraFunctionSignature& /*Signature*/)
	DECLARE_DELEGATE_RetVal_OneParam(FString, FGetFunctionSignatureSymbol, const FNiagaraFunctionSignature& /*Signature*/)
	DECLARE_DELEGATE_RetVal_OneParam(FString, FGetStructHlslTypeName, const FNiagaraTypeDefinition& /*Type*/)
	DECLARE_DELEGATE_RetVal_OneParam(FString, FGetPropertyHlslTypeName, const FProperty* /*Property*/)
	DECLARE_DELEGATE_RetVal_TwoParams(FString, FGetSanitizedSymbolName, FStringView /*SymbolName*/, bool /*bCollapsNamespaces*/)
	DECLARE_DELEGATE_RetVal_OneParam(FString, FGetHlslDefaultForType, const FNiagaraTypeDefinition& /*Type*/)

	FNiagaraDataInterfaceHlslGenerationContext(const FNiagaraDataInterfaceGPUParamInfo& InParameterInfo, TArrayView<const FNiagaraFunctionSignature> InSignatures)
		: ParameterInfo(InParameterInfo), Signatures(InSignatures)
	{
	}

	const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo;
	TArrayView<const FNiagaraFunctionSignature> Signatures;
	int32 FunctionInstanceIndex = INDEX_NONE;

	FGetSanitizedFunctionParameters GetSanitizedFunctionParametersDelegate;
	FGetFunctionSignatureSymbol GetFunctionSignatureSymbolDelegate;
	FGetStructHlslTypeName GetStructHlslTypeNameDelegate;
	FGetPropertyHlslTypeName GetPropertyHlslTypeNameDelegate;
	FGetSanitizedSymbolName GetSanitizedSymbolNameDelegate;
	FGetHlslDefaultForType GetHlslDefaultForTypeDelegate;

	const FNiagaraDataInterfaceGeneratedFunction& GetFunctionInfo() const { return ParameterInfo.GeneratedFunctions[FunctionInstanceIndex]; }
	const FNiagaraFunctionSignature& GetFunctionSignature() const { return Signatures[FunctionInstanceIndex]; }

	FString GetSanitizedFunctionParameters(const FNiagaraFunctionSignature& Signature) const { return GetSanitizedFunctionParametersDelegate.Execute(Signature); }
	FString GetFunctionSignatureSymbol(const FNiagaraFunctionSignature& Signature)const { return GetFunctionSignatureSymbolDelegate.Execute(Signature); }
	FString GetStructHlslTypeName(const FNiagaraTypeDefinition& Type)const { return GetStructHlslTypeNameDelegate.Execute(Type); }
	FString GetPropertyHlslTypeName(const FProperty* Property)const { return GetPropertyHlslTypeNameDelegate.Execute(Property); }
	FString GetSanitizedSymbolName(FStringView SymbolName, bool bCollapsNamespaces = false)const { return GetSanitizedSymbolNameDelegate.Execute(SymbolName, bCollapsNamespaces); }
	FString GetHlslDefaultForType(const FNiagaraTypeDefinition& Type)const { return GetHlslDefaultForTypeDelegate.Execute(Type); }
};

#endif