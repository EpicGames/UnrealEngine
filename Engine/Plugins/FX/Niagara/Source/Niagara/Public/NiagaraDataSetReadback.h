// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraDataSet.h"
#include "NiagaraParameterStore.h"

struct FNiagaraComputeExecutionContext;
class FNiagaraEmitterInstance;
class FNiagaraGpuComputeDispatchInterface;
class FNiagaraGpuReadbackManager;

class FNiagaraDataSetReadback : public TSharedFromThis<FNiagaraDataSetReadback, ESPMode::ThreadSafe>
{
public:
	DECLARE_DELEGATE_OneParam(FOnReadbackReady, const FNiagaraDataSetReadback&);

public:
	bool IsReady() const { return PendingReadbacks == 0; }
	void SetReadbackRead(FOnReadbackReady InOnReadbackReady);

	FName GetSourceName() const { return SourceName; }
	//ENiagaraScriptUsage GetSourceScriptUsage() const { return SourceScriptUsage; }
	const FNiagaraDataSet& GetDataSet() const { check(IsReady()); return DataSet; }
	const FNiagaraParameterStore& GetParameterStore() const { check(IsReady()); return ParameterStore; }

	void EnqueueReadback(FNiagaraEmitterInstance* EmitterInstance);
	void ImmediateReadback(FNiagaraEmitterInstance* EmitterInstance);

private:
	void ReadbackCompleteInternal();
	void GPUReadbackInternal(FRHICommandListImmediate& RHICmdList, FNiagaraGpuComputeDispatchInterface* DispatchInterface, FNiagaraComputeExecutionContext* GPUContext);

private:
	std::atomic<int>		PendingReadbacks;

	FName					SourceName;
	//ENiagaraScriptUsage		SourceScriptUsage;
	//FGuid					SourceUsageId;
	FNiagaraDataSet			DataSet;
	FNiagaraParameterStore	ParameterStore;

	FOnReadbackReady		OnReadbackReady;
};



class FNiagaraDataBufferReadback : public TSharedFromThis<FNiagaraDataBufferReadback, ESPMode::ThreadSafe>
{
public:
	DECLARE_DELEGATE_OneParam(FOnReadbackComplete, TSharedRef<FNiagaraDataBufferReadback>);

public:
	FOnReadbackComplete& GetOnReadbackComplete() { return OnReadbackComplete; }

	void EnqueueReadback(FRHICommandList& RHICmdList, FNiagaraDataBufferRef InDataBuffer, FNiagaraGpuReadbackManager* ReadbackManager, FNiagaraGPUInstanceCountManager& InstanceCountManager);

	void ReadResultsToDataBuffer(FNiagaraDataBuffer* DestBuffer)const;

private:
	void ReadbackCompleteInternal();
	void GPUReadbackInternal(FRHICommandList& RHICmdList, FNiagaraGpuReadbackManager* ReadbackManager, FNiagaraGPUInstanceCountManager& InstanceCountManager, FNiagaraDataBufferRef SrcDataBuffer);

	void GatherResults(int32 InstanceCount, const float* FloatData, const int32* Int32Data, const FFloat16* HalfData, int32 FloatStride, int32 Int32Stride, int32 HalfStride, int32 NumFloatComponents, int32 NumInt32Components, int32 NumHalfComponents);

	std::atomic<int>		PendingReadbacks;

	FOnReadbackComplete		OnReadbackComplete;

	TArray<float> ReadbackData_Float;
	TArray<int32> ReadbackData_Int32;
	TArray<FFloat16> ReadbackData_Half;
	int32 ReadbackData_Count = 0;
	int32 ReadbackData_FloatStride = 0;
	int32 ReadbackData_Int32Stride = 0;
	int32 ReadbackData_HalfStride = 0;
	int32 ReadbackData_NumFloatComponents = 0;
	int32 ReadbackData_NumInt32Components = 0;
	int32 ReadbackData_NumHalfComponents = 0;
};
