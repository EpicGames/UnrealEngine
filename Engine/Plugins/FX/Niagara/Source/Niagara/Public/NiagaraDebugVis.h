// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FNiagaraTypeRegistry;

#if !IS_MONOLITHIC
namespace NiagaraDebugVisHelper
{
	extern NIAGARA_API const FNiagaraTypeRegistry* GCoreTypeRegistrySingletonPtr;
}
#endif