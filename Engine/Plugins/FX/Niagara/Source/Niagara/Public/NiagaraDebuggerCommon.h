// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
All common code shared between the editor side debugger and debugger clients running in game.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/NotifyHook.h"
#include "NiagaraTypes.h"
#include "NiagaraCommon.h"
#include "NiagaraDebuggerCommon.generated.h"

#define UE_API NIAGARA_API

struct FNiagaraSystemSimCacheCaptureRequest;
class UNiagaraSimCache;
struct FNiagaraOutlinerData;

//////////////////////////////////////////////////////////////////////////
// Niagara Outliner.

USTRUCT()
struct FNiagaraOutlinerTimingData
{
	GENERATED_BODY()

	/** Game thread time, including concurrent tasks*/
	UPROPERTY(VisibleAnywhere, Category="Time")
	float GameThread = 0.0f;

	/** Render thread time. */
	UPROPERTY(VisibleAnywhere, Category="Time")
	float RenderThread = 0.0f;
};

USTRUCT()
struct FNiagaraOutlinerEmitterInstanceData
{
	GENERATED_BODY()

	//Name of this emitter.
	UPROPERTY(VisibleAnywhere, Category = "Emitter")
	FString EmitterName; //TODO: Move to shared asset representation.

	UPROPERTY(VisibleAnywhere, Category = "Emitter")
	ENiagaraSimTarget SimTarget = ENiagaraSimTarget::CPUSim; //TODO: Move to shared asset representation.

	UPROPERTY(VisibleAnywhere, Category = "Emitter")
	ENiagaraExecutionState ExecState = ENiagaraExecutionState::Num;
	
	UPROPERTY(VisibleAnywhere, Category = "Emitter")
	int32 NumParticles = 0;

	UPROPERTY(VisibleAnywhere, Category = "Emitter")
	uint32 bRequiresPersistentIDs : 1 = false;

	UPROPERTY(VisibleAnywhere, Category = "Emitter")
	uint32 bIsLightweight : 1 = false;

	//Mem Usage?
	//Scalability info?
};

/** Outliner information on a specific system instance. */
USTRUCT()
struct FNiagaraOutlinerSystemInstanceData
{
	GENERATED_BODY()

	/** Name of the component object for this instance, if there is one. */
	UPROPERTY(VisibleAnywhere, Category = "System")
	FString ComponentName;

	UPROPERTY(VisibleAnywhere, Category = "System")
	FVector3f LWCTile = FVector3f::Zero();

	UPROPERTY(VisibleAnywhere, Category = "System")
	TArray<FNiagaraOutlinerEmitterInstanceData> Emitters;
	
	UPROPERTY(VisibleAnywhere, Category = "State")
	ENiagaraExecutionState ActualExecutionState = ENiagaraExecutionState::Num;

	UPROPERTY(VisibleAnywhere, Category = "State")
	ENiagaraExecutionState RequestedExecutionState = ENiagaraExecutionState::Num;
	
	UPROPERTY(VisibleAnywhere, Category = "State")
	FNiagaraScalabilityState ScalabilityState;
	
	UPROPERTY(VisibleAnywhere, Category = "State")
	uint32 bPendingKill : 1;

	UPROPERTY(VisibleAnywhere, Category = "State")
	uint32 bUsingCullProxy : 1;
	
	UPROPERTY(VisibleAnywhere, Category = "State")
	ENCPoolMethod PoolMethod = ENCPoolMethod::None;

	UPROPERTY(VisibleAnywhere, Category = "Performance")
	FNiagaraOutlinerTimingData AverageTime;

	UPROPERTY(VisibleAnywhere, Category = "Performance")
	FNiagaraOutlinerTimingData MaxTime;

	UPROPERTY(VisibleAnywhere, Category = "Ticking")
	TEnumAsByte<ETickingGroup> TickGroup;

	UPROPERTY(VisibleAnywhere, Category = "Ticking")
	TEnumAsByte<ENiagaraGpuComputeTickStage::Type> GpuTickStage;

	UPROPERTY(VisibleAnywhere, Category = "Gpu")
	uint32 bIsSolo : 1;

	UPROPERTY(VisibleAnywhere, Category = "Gpu")
	uint32 bRequiresGlobalDistanceField : 1;

	UPROPERTY(VisibleAnywhere, Category = "Gpu")
	uint32 bRequiresDepthBuffer : 1;

	UPROPERTY(VisibleAnywhere, Category = "Gpu")
	uint32 bRequiresEarlyViewData : 1;

	UPROPERTY(VisibleAnywhere, Category = "Gpu")
	uint32 bRequiresViewUniformBuffer : 1;

	UPROPERTY(VisibleAnywhere, Category = "Gpu")
	uint32 bRequiresRayTracingScene : 1;

	UPROPERTY(VisibleAnywhere, Category = "Gpu")
	uint32 bRequiresCurrentFrameNDC : 1;

	FNiagaraOutlinerSystemInstanceData()
		: bPendingKill(false)
		, bUsingCullProxy(false)
		, TickGroup(0)
		, GpuTickStage(ENiagaraGpuComputeTickStage::First)
		, bIsSolo(false)
		, bRequiresGlobalDistanceField(false)
		, bRequiresDepthBuffer(false)
		, bRequiresEarlyViewData(false)
		, bRequiresViewUniformBuffer(false)
		, bRequiresRayTracingScene(false)
		, bRequiresCurrentFrameNDC(false)
	{}
};

/** Wrapper for array of system instance outliner data so that it can be placed in a map. */
USTRUCT()
struct FNiagaraOutlinerSystemData
{
	GENERATED_BODY()

	//TODO: Cache off any shared representation of the system and emitters here for the instances to reference. 

	UPROPERTY(VisibleAnywhere, Category = "System")
	FSoftObjectPath AssetPath;

	/** Map of System Instance data indexed by the UNiagaraSystem name. */
	UPROPERTY(VisibleAnywhere, Category = "System")
	TArray<FNiagaraOutlinerSystemInstanceData> SystemInstances;
	
	UPROPERTY(VisibleAnywhere, Category = "Performance")
	FNiagaraOutlinerTimingData AveragePerFrameTime;

	UPROPERTY(VisibleAnywhere, Category = "Performance")
	FNiagaraOutlinerTimingData MaxPerFrameTime;
	
	UPROPERTY(VisibleAnywhere, Category = "Performance")
	FNiagaraOutlinerTimingData AveragePerInstanceTime;

	UPROPERTY(VisibleAnywhere, Category = "Performance")
	FNiagaraOutlinerTimingData MaxPerInstanceTime;
};

/** All information about a specific world for the Niagara Outliner. */
USTRUCT()
struct FNiagaraOutlinerWorldData
{
	GENERATED_BODY()

	/** Map of System Instance data indexed by the UNiagaraSystem name. */
	UPROPERTY(VisibleAnywhere, Category = "World")
	TMap<FString, FNiagaraOutlinerSystemData> Systems;

	UPROPERTY(VisibleAnywhere, Category = "State")
	bool bHasBegunPlay = false;

	UPROPERTY(VisibleAnywhere, Category = "State")
	uint8 WorldType = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = "State")
	uint8 NetMode = INDEX_NONE;
	

	UPROPERTY(VisibleAnywhere, Category = "Performance")
	FNiagaraOutlinerTimingData AveragePerFrameTime;

	UPROPERTY(VisibleAnywhere, Category = "Performance")
	FNiagaraOutlinerTimingData MaxPerFrameTime;

	//Mem Usage?
};

USTRUCT()
struct FNiagaraOutlinerData
{
	GENERATED_BODY()

	/** Map all world data indexed by the world name. */
	UPROPERTY(VisibleAnywhere, Category = "Outliner")
	TMap<FString, FNiagaraOutlinerWorldData> WorldData;
};

// Outliner END
//////////////////////////////////////////////////////////////////////////

//////////////////////////////////////////////////////////////////////////
// Messages passed between the editor side debugger and the client. 

/** 
Messaged broadcast from debugger to request a connection to a particular session. 
If any matching client is found and it accepts, it will return a FNiagaraDebuggerAcceptConnection message to the sender. 
*/
USTRUCT()
struct FNiagaraDebuggerRequestConnection
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Message")
	FGuid SessionId;

	UPROPERTY(EditAnywhere, Category = "Message")
	FGuid InstanceId;

	FNiagaraDebuggerRequestConnection() { }

	FNiagaraDebuggerRequestConnection(const FGuid& InSessionId, const FGuid& InInstanceId)
		: SessionId(InSessionId)
		, InstanceId(InInstanceId)
	{}
};

/** Response message from the a debugger client accepting a connection requested by a FNiagaraDebuggerRequestConnection message. */
USTRUCT()
struct FNiagaraDebuggerAcceptConnection
{
	GENERATED_BODY()
		
	UPROPERTY(EditAnywhere, Category = "Message")
	FGuid SessionId;

	UPROPERTY(EditAnywhere, Category = "Message")
	FGuid InstanceId;

	FNiagaraDebuggerAcceptConnection() { }
	
	FNiagaraDebuggerAcceptConnection(const FGuid& InSessionId, const FGuid& InInstanceId)
		: SessionId(InSessionId)
		, InstanceId(InInstanceId)
	{}
};

/** Empty message informing a debugger client that the debugger is closing the connection. */
USTRUCT()
struct FNiagaraDebuggerConnectionClosed
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Message")
	FGuid SessionId;

	UPROPERTY(EditAnywhere, Category = "Message")
	FGuid InstanceId;

	FNiagaraDebuggerConnectionClosed() { }
	
	FNiagaraDebuggerConnectionClosed(const FGuid& InSessionId, const FGuid& InInstanceId)
		: SessionId(InSessionId)
		, InstanceId(InInstanceId)
	{}
};

/** Command that will execute a console command on the debugger client. */
USTRUCT() 
struct FNiagaraDebuggerExecuteConsoleCommand
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Message")
	FString Command;

	UPROPERTY()
	bool bRequiresWorld = false;

	FNiagaraDebuggerExecuteConsoleCommand() { }
	
	FNiagaraDebuggerExecuteConsoleCommand(FString InCommand, bool bInRequiresWorld)
		: Command(InCommand)
		, bRequiresWorld(bInRequiresWorld)
	{}
};

/** Message containing updated outliner information sent from the client to the debugger. */
USTRUCT()
struct FNiagaraDebuggerOutlinerUpdate
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Message")
	FNiagaraOutlinerData OutlinerData;

	FNiagaraDebuggerOutlinerUpdate() { }
};

// End of messages.
//////////////////////////////////////////////////////////////////////////

UENUM()
enum class ENiagaraDebugPlaybackMode : uint8
{
	Play = 0,
	Loop,
	Paused,
	Step,
};

UENUM()
enum class ENiagaraDebugHudHAlign : uint8
{
	Left,
	Center,
	Right
};

UENUM()
enum class ENiagaraDebugHudVAlign : uint8
{
	Top,
	Center,
	Bottom
};

UENUM()
enum class ENiagaraDebugHudFont
{
	Small = 0,
	Normal,
};

UENUM()
enum class ENiagaraDebugHudVerbosity
{
	None,
	Basic,
	Verbose,
};

USTRUCT()
struct FNiagaraDebugHudTextOptions
{
	GENERATED_BODY()
		
	UPROPERTY(Config, EditAnywhere, Category = "Text Options")
	ENiagaraDebugHudFont Font = ENiagaraDebugHudFont::Small;

	UPROPERTY(EditAnywhere, Category = "Text Options")
	ENiagaraDebugHudHAlign	HorizontalAlignment = ENiagaraDebugHudHAlign::Left;

	UPROPERTY(EditAnywhere, Category = "Text Options")
	ENiagaraDebugHudVAlign	VerticalAlignment = ENiagaraDebugHudVAlign::Top;

	UPROPERTY(EditAnywhere, Category = "Text Options")
	FVector2D ScreenOffset = FVector2D::ZeroVector;
};

USTRUCT()
struct FNiagaraDebugHUDVariable
{
	GENERATED_BODY()
		
	UPROPERTY(EditAnywhere, Category = "Attribute")
	bool bEnabled = true;

	/** Name of attributes to match, uses wildcard matching. */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	FString Name;

	static NIAGARA_API FString BuildVariableString(const TArray<FNiagaraDebugHUDVariable>& Variables);
	static NIAGARA_API void InitFromString(const FString& VariablesString, TArray<FNiagaraDebugHUDVariable>& OutVariables);
};

UENUM()
enum class ENiagaraDebugHUDOverviewMode
{
	Overview,
	Scalability,	
	Performance,
	PerformanceGraph,
	GpuComputePerformance,
}; 

UENUM()
enum class ENiagaraDebugHUDPerfGraphMode
{
	GameThread,
	RenderThread,
	GPU,
}; 

UENUM()
enum class ENiagaraDebugHUDPerfSampleMode
{
	FrameTotal,
	PerInstanceAverage,	
};

UENUM()
enum class ENiagaraDebugHUDPerfUnits
{
	Microseconds,
	Milliseconds,
};

UENUM()
enum class ENiagaraDebugHUDDOverviewSort
{
	/** Lexical sort on system name */
	Name,
	/** Sort by the number currently registered. */
	NumberRegistered,
	/** Sort by the number currently active. */
	NumberActive,
	/** Sort by the number currently managed by scalability. */
	NumberScalability,
	/** Sort by approximate memory usage. */
	MemoryUsage,
	/** Sort by when the system was recently visible. */
	RecentlyVisibilty,
};

/** Settings for Niagara debug HUD. Contained in it's own struct so that we can pass it whole in a message to the debugger client. */
USTRUCT()
struct FNiagaraDebugHUDSettingsData
{
	GENERATED_BODY()

	NIAGARA_API FNiagaraDebugHUDSettingsData();

	bool IsEnabled() const
	{
	#if WITH_EDITORONLY_DATA
		return bHudEnabled && (!GIsEditor || bWidgetEnabled);
	#else
		return bHudEnabled;
	#endif
	}

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	bool bWidgetEnabled = false;
#endif

	/** Primary control for all HUD features. */
	UPROPERTY()
	bool bHudEnabled = true;

	/** Primary control for HUD rendering. */
	UPROPERTY()
	bool bHudRenderingEnabled = true;

	/** Primary control for validation settings. */
	UPROPERTY()
	bool bValidationEnabled = false;

	/** When enabled the overview display will be enabled. */
	UPROPERTY()
	bool bOverviewEnabled = false;

	UPROPERTY(EditAnywhere, Category = "Debug Overview", meta = (DisplayName = "Debug Overview Mode"))
	ENiagaraDebugHUDOverviewMode OverviewMode = ENiagaraDebugHUDOverviewMode::Overview;

	// How to sort the displayed list of systems
	UPROPERTY(EditAnywhere, Category = "Debug Overview", meta = (DisplayName = "Debug Overview Sort Mode"))
	ENiagaraDebugHUDDOverviewSort OverviewSortMode = ENiagaraDebugHUDDOverviewSort::Name;

	/** When enabled the overview display will include Cascade FX. */
	UPROPERTY(EditAnywhere, Category = "Debug Overview", AdvancedDisplay)
	bool bIncludeCascade = true;

	/** When enabled the overview display will include Niagara FX. */
	UPROPERTY(EditAnywhere, Category = "Debug Overview", AdvancedDisplay)
	bool bIncludeNiagara = true;

	UPROPERTY(EditAnywhere, Category = "Debug Overview")
	bool bShowRegisteredComponents = false;
	
	/** When enabled the overview will only show the filter system information. */
	UPROPERTY()
	bool bOverviewShowFilteredSystemOnly = true;

	UPROPERTY(EditAnywhere, Category = "Debug Overview", AdvancedDisplay)
	bool bShowGlobalBudgetInfo = false;

	/** Master control for enabling and disabling filtering by asset / actor / component */
	UPROPERTY(EditAnywhere, Category = "Debug Filter")
	bool bFiltersEnabled = false;

	/**
	If enabled, then only systems matching the filters will be considered by the debug hud
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Filter", meta = (InlineEditConditionToggle))
	bool bSystemFilterEnabled = false;

	/**
	Wildcard filter for the systems to show more detailed information about.
	For example,. "NS_*" would match all systems starting with NS_.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Filter", meta = (DisplayName = "System Name Filter", EditCondition = "bSystemFilterEnabled"))
	FString SystemFilter;
	

	UPROPERTY(EditAnywhere, Category = "Debug Filter", meta = (InlineEditConditionToggle))
	bool bEmitterFilterEnabled = false;

	/**
	Wildcard filter used to match emitters when generating particle attribute view.
	For example,. "Fluid*" would match all emitters starting with Fluid and only particle attributes for those would be visible.
	Requires a valid SystemFilter to be set.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Filter", meta = (DisplayName = "Emitter Name Filter",EditCondition = "bEmitterFilterEnabled"))
	FString EmitterFilter;
	

	UPROPERTY(EditAnywhere, Category = "Debug Filter", meta = (InlineEditConditionToggle))
	bool bActorFilterEnabled = false;

	/**
	Wildcard filter which is compared against the Components Actor name to narrow down the detailed information.
	For example, "*Water*" would match all actors that contain the string "water".
	Requires a valid SystemFilter to be set.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Filter", meta = (DisplayName = "Actor Name Filter", EditCondition = "bActorFilterEnabled"))
	FString ActorFilter;
	

	UPROPERTY(EditAnywhere, Category = "Debug Filter", meta = (InlineEditConditionToggle))
	bool bComponentFilterEnabled = false;

	/**
	Wildcard filter for the components to show more detailed information about.
	For example, "*MyComp*" would match all components that contain MyComp.
	Requires a valid SystemFilter to be set.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Filter", meta = (DisplayName = "Component Name Filter", EditCondition = "bComponentFilterEnabled"))
	FString ComponentFilter;

	
	/**
	When enabled all Niagara systems that pass the filter will have the simulation data buffers validation.
	i.e. we will look for NaN or other invalidate data  inside it
	Note: This will have an impact on performance.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Validation", meta=(EditCondition = bValidationEnabled, EditConditionHides))
	bool bValidateSystemSimulationDataBuffers = true;

	/**
	When enabled all Niagara systems that pass the filter will have the particle data buffers validation.
	i.e. we will look for NaN or other invalidate data  inside it
	Note: This will have an impact on performance.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Validation", meta=(EditCondition = bValidationEnabled, EditConditionHides))
	bool bValidateParticleDataBuffers = true;

	/**
	When enabled all validation errors will be sent to the log as warnings.
	This can be useful to try and narrow down the exact source of an invalid value in the data buffers as often
	they will cascade from the first frame where one is generated into other attributes in the subsequent frames.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Validation", meta=(EditCondition = bValidationEnabled, EditConditionHides))
	bool bValidationLogErrors = false;

	/**
	When > 0 this is the maximum number of attributes we will display that contain a NaN,
	there could be more but the display will be truncated to this amount.  This is to reduce generating long strings.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Validation", meta=(EditCondition = bValidationEnabled, EditConditionHides))
	int32 ValidationAttributeDisplayTruncate = 3;

	/** When enabled system debug information will be displayed in world. */
	UPROPERTY(EditAnywhere, Category = "Debug System")
	ENiagaraDebugHudVerbosity SystemDebugVerbosity = ENiagaraDebugHudVerbosity::None;

	/** When enabled we show information about emitter / particle counts. */
	UPROPERTY(EditAnywhere, Category = "Debug System", meta = (EditCondition = "SystemDebugVerbosity != ENiagaraDebugHudVerbosity::None"))
	ENiagaraDebugHudVerbosity SystemEmitterVerbosity = ENiagaraDebugHudVerbosity::Basic;

	/** When enabled allows data interfaces to include additional debugging information. */
	UPROPERTY(EditAnywhere, Category = "Debug System", meta = (EditCondition = "SystemDebugVerbosity != ENiagaraDebugHudVerbosity::None"))
	ENiagaraDebugHudVerbosity DataInterfaceVerbosity = ENiagaraDebugHudVerbosity::None;

	/**
	List of attributes to show about the system, each entry uses wildcard matching.
	For example, "System.*" would match all system attributes.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug System", meta = (EditCondition = "SystemDebugVerbosity != ENiagaraDebugHudVerbosity::None", DisplayName="System Attributes"))
	TArray<FNiagaraDebugHUDVariable> SystemVariables;

	/** When disabled in world rendering will show systems deactivated by scalability. */
	UPROPERTY(EditAnywhere, Category = "Debug System", meta = (EditCondition = "SystemDebugVerbosity != ENiagaraDebugHudVerbosity::None"))
	bool bSystemShowActiveOnlyInWorld = true;

	/** When enabled will show particle attributes from the list. */
	UPROPERTY(EditAnywhere, Category = "Debug Particles", meta = (DisplayName="Show Particle Attributes"))
	bool bShowParticleVariables = false;

	/**
	List of attributes to show per particle, each entry uses wildcard matching.
	For example, "*Position" would match all attributes that end in Position.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Particles", meta = (EditCondition = "bShowParticleVariables", DisplayName="Particle Attributes"))
	TArray<FNiagaraDebugHUDVariable> ParticlesVariables;

	/**
	When enabled GPU particle data will be copied from the GPU to the CPU.
	Warning: This has an impact on performance & memory since we copy the whole buffer.
	The displayed data is latent since we are seeing what happened a few frames ago.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Particles", meta = (EditCondition = "bShowParticleVariables"))
	bool bEnableGpuParticleReadback = false;

	/**
	When enabled the particle index will be displayed along with any attributes.
	Note: This is the index in the particle data buffer and not the persistent ID index.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Particles", meta = (EditCondition = "bShowParticleVariables"))
	bool bShowParticleIndex = false;

	/**
	When enabled particle attributes will display with the system information
	rather than in world at the particle location. Only enabled if system is set up to display information as well.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Particles", meta = (EditCondition = "bShowParticleVariables && SystemDebugVerbosity != ENiagaraDebugHudVerbosity::None", DisplayName="Display With System Attributes"))
	bool bShowParticlesVariablesWithSystem = false;

	UPROPERTY(EditAnywhere, Category = "Debug Particles", meta = (EditCondition = "bShowParticleVariables", DisplayName = "Stack Attributes Vertical"))
	bool bShowParticleVariablesVertical = false;

	UPROPERTY(EditAnywhere, Category = "Debug Particles", meta = (EditCondition = "bShowParticleVariables"))
	bool bUseMaxParticlesToDisplay = true;

	/**
	When enabled, the maximum number of particles to show information about.
	When disabled all particles will show attributes, this can result in poor performance & potential OOM on some platforms.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Particles", meta = (EditCondition = "bUseMaxParticlesToDisplay && bShowParticleVariables", UIMin="1", ClampMin="1"))
	int32 MaxParticlesToDisplay = 32;

	/** When enabled we use the clip planes to narrow down which particles to display */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Debug Particles", meta = (EditCondition = "bShowParticleVariables"))
	bool bUseParticleDisplayClip = false;

	/** Clipping planes used to display particle attributes. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Debug Particles", meta = (EditCondition = "bShowParticleVariables", UIMin = "0", ClampMin = "0"))
	FVector2D ParticleDisplayClip = FVector2D(0.0f, 10000.0f);

	/** When enabled we use a radius from the display center to avoid showing too many particle attributes. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Debug Particles", meta = (EditCondition = "bShowParticleVariables"))
	bool bUseParticleDisplayCenterRadius = false;

	/** Radius from screen center where 0 is center to 1.0 is edge to avoid display too many particle attributes. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Debug Particles", meta = (EditCondition = "bShowParticleVariables", UIMin = "0", ClampMin = "0"))
	float ParticleDisplayCenterRadius = 1.0f;

	/** How many frames to capture in between updates to the max and average perf report values. */
	UPROPERTY(EditAnywhere, Category = "Performance")
	int32 PerfReportFrames = 60;

	UPROPERTY(EditAnywhere, Category = "Performance")
	ENiagaraDebugHUDPerfSampleMode PerfSampleMode = ENiagaraDebugHUDPerfSampleMode::FrameTotal;

	UPROPERTY(EditAnywhere, Category = "Performance")
	ENiagaraDebugHUDPerfUnits PerfUnits = ENiagaraDebugHUDPerfUnits::Microseconds;

	// Adds a column that displays only the GT cost to the perf overview. Only enabled when performance overview mode is active.
	UPROPERTY(EditAnywhere, Category = "Performance", meta = (DisplayName = "Show separate GT perf column", EditCondition = "OverviewMode == ENiagaraDebugHUDOverviewMode::Performance"))
	bool bShowPerfColumGameThreadOnly = false;

	/** Time range of the Y Axis of the perf graph */
	UPROPERTY(EditAnywhere, Category = "Performance Graph", meta = (EditCondition = "OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph", EditConditionHides))
	ENiagaraDebugHUDPerfGraphMode PerfGraphMode = ENiagaraDebugHUDPerfGraphMode::GameThread;

	/** How many frames of history to display in the perf graphs. */
	UPROPERTY(EditAnywhere, Category = "Performance Graph", meta = (EditCondition = "OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph", EditConditionHides))
	int32 PerfHistoryFrames = 600;

	/** Use the specified user range when enabled, otherwise we will auto detect the range to use. */
	UPROPERTY(EditAnywhere, Category = "Performance Graph", meta = (DisplayName = "Fixed Y Axis Range", EditCondition = "OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph", EditConditionHides))
	bool bUsePerfGraphTimeRange = false;

	/** Time range of the Y Axis of the perf graph */
	UPROPERTY(EditAnywhere, Category = "Performance Graph", meta = (DisplayName = "Y Axis Time Range", EditCondition = "OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph", EditConditionHides))
	float PerfGraphTimeRange = 500.0f;

	/** Pixel size of the perf graph. */
	UPROPERTY(EditAnywhere, Category = "Performance Graph", meta = (EditCondition = "OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph", EditConditionHides))
	FVector2D PerfGraphSize = FVector2D(500,500);

	UPROPERTY(EditAnywhere, Category = "Performance Graph", meta = (EditCondition = "OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph", EditConditionHides))
	FLinearColor PerfGraphAxisColor = FLinearColor::White;

	//Number of samples to use either size of a value when smoothing perf graphs.
	UPROPERTY(EditAnywhere, Category = "Performance Graph", meta = (EditCondition = "OverviewMode == ENiagaraDebugHUDOverviewMode::PerformanceGraph", EditConditionHides))
	int32 SmoothingWidth = 4;

	/** Overview display font to use. */
	UPROPERTY(EditAnywhere, Category = "Style", meta = (DisplayName = "Debug Overview Font", EditCondition = "bOverviewEnabled"))
	ENiagaraDebugHudFont OverviewFont = ENiagaraDebugHudFont::Normal;

	/** Overview display location. */
	UPROPERTY(EditAnywhere, Category = "Style", meta = (DisplayName = "Debug Overview Text Location", EditCondition = "bOverviewEnabled"))
	FVector2D OverviewLocation = FVector2D(30.0f, 150.0f);

	/** Sets display text options for system information. */
	UPROPERTY(EditAnywhere, Category = "Style", meta=(EditCondition="SystemDebugVerbosity != ENiagaraDebugHudVerbosity::None"))
	FNiagaraDebugHudTextOptions SystemTextOptions;

	/** Sets display text options for particle information. */
	UPROPERTY(EditAnywhere, Category = "Style", meta = (EditCondition = "bShowParticleVariables"))
	FNiagaraDebugHudTextOptions ParticleTextOptions;

	UPROPERTY()
	bool bDrawBoundsEnabled = false;

	UPROPERTY()
	bool bDrawBoundsWireframe = true;

	UPROPERTY()
	float DrawBoundsAlpha = 1.0f;

	// Default background color used generally for panels
	UPROPERTY(EditAnywhere, Category = "Style")
	FLinearColor DefaultBackgroundColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.5f);

	// Overview heading text color
	UPROPERTY(EditAnywhere, Category = "Style")
	FLinearColor OverviewHeadingColor = FLinearColor::Green;

	// Overview detail text color
	UPROPERTY(EditAnywhere, Category = "Style")
	FLinearColor OverviewDetailColor = FLinearColor::White;

	// Overview detail highlight text color
	UPROPERTY(EditAnywhere, Category = "Style")
	FLinearColor OverviewDetailHighlightColor = FLinearColor::Yellow;

	// In world text if an error is detected
	UPROPERTY(EditAnywhere, Category = "Style")
	FLinearColor InWorldErrorTextColor = FLinearColor(1.0f, 0.4f, 0.3f, 1.0f);
	// In world text color
	UPROPERTY(EditAnywhere, Category = "Style")
	FLinearColor InWorldTextColor = FLinearColor::White;

	// Message display text color
	UPROPERTY(EditAnywhere, Category = "Style")
	FLinearColor MessageInfoTextColor = FLinearColor::White;

	// Message display warning text color
	UPROPERTY(EditAnywhere, Category = "Style")
	FLinearColor MessageWarningTextColor = FLinearColor(0.9f, 0.7f, 0.0f, 1.0f);

	// Message display error text color
	UPROPERTY(EditAnywhere, Category = "Style")
	FLinearColor MessageErrorTextColor = FLinearColor(1.0f, 0.4f, 0.3f, 1.0f);
	
	/** Opacity of the system color background tile in overview table rows. */
	UPROPERTY(EditAnywhere, Category = "Style")
	float SystemColorTableOpacity = 0.2f;

	/** Additional seed value for random system colors. Useful if current colors of systems are too similar. */
	UPROPERTY(EditAnywhere, Category = "Style")
	uint32 SystemColorSeed = 0;

	/** Minimum HSV values for the random colors generated for each System. */
	UPROPERTY(EditAnywhere, Category = "Style")
	FVector SystemColorHSVMin = FVector(0, 200, 200);

	/** Maximum HSV values for the random colors generated for each System. */
	UPROPERTY(EditAnywhere, Category = "Style")
	FVector SystemColorHSVMax = FVector(255, 255, 255);

	UPROPERTY()
	ENiagaraDebugPlaybackMode PlaybackMode = ENiagaraDebugPlaybackMode::Play;

	UPROPERTY()
	bool bPlaybackRateEnabled = false;

	UPROPERTY()
	float PlaybackRate = 0.25f;

	UPROPERTY()
	bool bLoopTimeEnabled = false;

	UPROPERTY()
	float LoopTime = 1.0f;
};

/** Message passed from debugger to client when it needs updated simple client info. */
USTRUCT()
struct FNiagaraRequestSimpleClientInfoMessage
{
	GENERATED_BODY()
};

UCLASS(config = EditorPerProjectUserSettings, defaultconfig, MinimalAPI)
class UNiagaraDebugHUDSettings : public UObject, public FNotifyHook
{
	GENERATED_BODY()

public:
	virtual void PostInitProperties() override;

	DECLARE_MULTICAST_DELEGATE(FOnChanged);
	FOnChanged OnChangedDelegate;

	UPROPERTY(Config, EditAnywhere, Category = "Settings", meta=(ShowOnlyInnerProperties))
	FNiagaraDebugHUDSettingsData Data;

	NIAGARA_API void NotifyPropertyChanged();
	virtual void NotifyPreChange(FProperty*) {}
	virtual void NotifyPostChange(const FPropertyChangedEvent&, FProperty*) { NotifyPropertyChanged(); }
	virtual void NotifyPreChange(class FEditPropertyChain*) {}
	virtual void NotifyPostChange(const FPropertyChangedEvent&, class FEditPropertyChain*) { NotifyPropertyChanged(); }
};


USTRUCT()
struct FNiagaraOutlinerCaptureSettings
{
	GENERATED_BODY()

	/** Press to trigger a single capture of Niagara data from the connected debugger client. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bTriggerCapture = false;

	/** How many frames to delay capture. If gathering performance data, this is how many frames will be collected. */
	UPROPERTY(EditAnywhere, Config, Category="Settings")
	uint32 CaptureDelayFrames = 0;
	
	UPROPERTY(EditAnywhere, Config, Category = "Settings")
	bool bGatherPerfData = true;

	/** How many frames capture when capturing a sim cache. */
	UPROPERTY(EditAnywhere, Config, Category = "Settings")
	uint32 SimCacheCaptureFrames = 10;
};

/** Simple information on the connected client for use in continuous or immediate response UI elements. */
USTRUCT()
struct FNiagaraSimpleClientInfo
{
	GENERATED_BODY()

	/** List of all system names in the scene. */
	UPROPERTY(EditAnywhere, Category = "Info")
	TArray<FString> Systems;

	/** List of all actors with Niagara components. */
	UPROPERTY(EditAnywhere, Category = "Info")
	TArray<FString> Actors;
	
	/** List of all Niagara components. */
	UPROPERTY(EditAnywhere, Category = "Info")
	TArray<FString> Components;
	
	/** List of all Niagara emitters. */
	UPROPERTY(EditAnywhere, Category = "Info")
	TArray<FString> Emitters;
};

/** Message sent from the debugger to a client to request a sim cache capture for a particular component. */
USTRUCT()
struct FNiagaraSystemSimCacheCaptureRequest
{
	GENERATED_BODY()

	/** Name of the component we're going to capture. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName ComponentName;

	/** How many frames to delay capture. */
	UPROPERTY(EditAnywhere, Config, Category="Settings")
	uint32 CaptureDelayFrames = 0;

	/** How many frames to capture. */
	UPROPERTY(EditAnywhere, Config, Category = "Settings")
	uint32 CaptureFrames = 10;
};


/** Message sent from a debugger client to a connected debugger containing the results of a sim cache capture. */
USTRUCT()
struct FNiagaraSystemSimCacheCaptureReply
{
	GENERATED_BODY()

	/** Name of the captured component. */
	UPROPERTY()
	FName ComponentName;
	
	UPROPERTY()
	TArray<uint8> SimCacheData;
};

enum class ENiagaraDebugMessageType : uint8
{
	Info,
	Warning,
	Error
};

struct FNiagaraDebugMessage
{
	ENiagaraDebugMessageType Type;
	FString Message;
	float Lifetime;
	FNiagaraDebugMessage()
		: Type(ENiagaraDebugMessageType::Error)
		, Lifetime(0.0f)
	{}
	FNiagaraDebugMessage(ENiagaraDebugMessageType InType, const FString& InMessage, float InLifetime)
		: Type(InType)
		, Message(InMessage)
		, Lifetime(InLifetime)
	{}
};

//Temporary interface allowing direct access to the local debugger client until we can get a unified messaging system that works in all required cases. i.e. when WITH_UNREAL_TARGET_DEVELOPER_TOOLS = 0 and/or we don't have assess to the messaging or sessions systems.
DECLARE_DELEGATE_TwoParams(FOnNiagaraDebuggerClientSimCacheCapture, const FNiagaraSystemSimCacheCaptureRequest&, TObjectPtr<UNiagaraSimCache>);
DECLARE_DELEGATE_OneParam(FOnNiagaraDebuggerClientOutlinerCapture, const FNiagaraOutlinerData&);
class INiagaraDebuggerClient
{
public:

	static UE_API INiagaraDebuggerClient* Get();

	virtual void ExecConsoleCommand(const FNiagaraDebuggerExecuteConsoleCommand& Message) = 0;
	virtual void UpdateDebugHUDSettings(const FNiagaraDebugHUDSettingsData& Message) = 0;
	virtual void GetSimpleClientInfo(FNiagaraSimpleClientInfo& OutClientInfo) = 0;
	virtual void UpdateOutlinerSettings(const FNiagaraOutlinerCaptureSettings& Message, FOnNiagaraDebuggerClientOutlinerCapture OnCapture) = 0;
	virtual void SimCacheCaptureRequest(const FNiagaraSystemSimCacheCaptureRequest& Message, FOnNiagaraDebuggerClientSimCacheCapture OnCapture) = 0;
};

#undef UE_API
