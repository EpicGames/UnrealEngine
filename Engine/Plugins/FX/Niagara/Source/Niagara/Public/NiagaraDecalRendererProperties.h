// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSetAccessor.h"
#include "NiagaraParameterBinding.h"
#include "NiagaraDecalRendererProperties.generated.h"

class UMaterialInstanceConstant;
class UMaterialInterface;
class FNiagaraEmitterInstance;
class SWidget;

UCLASS(editinlinenew, MinimalAPI, meta = (DisplayName = "Decal Renderer", SupportsStateless))
class UNiagaraDecalRendererProperties : public UNiagaraRendererProperties
{
public:
	GENERATED_BODY()

	UNiagaraDecalRendererProperties();

	//UObject Interface
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITORONLY_DATA
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif// WITH_EDITORONLY_DATA
	//UObject Interface END

	static void InitCDOPropertiesAfterModuleStartup();

	//~ UNiagaraRendererProperties interface
	virtual FNiagaraRenderer* CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) override;
	virtual class FNiagaraBoundsCalculator* CreateBoundsCalculator() override;
	virtual void GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const override;
	virtual bool IsSimTargetSupported(ENiagaraSimTarget InSimTarget) const override { return InSimTarget == ENiagaraSimTarget::CPUSim || SourceMode == ENiagaraRendererSourceDataMode::Emitter; };
#if WITH_EDITORONLY_DATA
	virtual const TArray<FNiagaraVariable>& GetOptionalAttributes() override;
	virtual const FSlateBrush* GetStackIcon() const override;
	virtual void GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const override;
	virtual TArray<FNiagaraVariable> GetBoundAttributes() const override;
	virtual void RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
	virtual void RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
#endif // WITH_EDITORONLY_DATA
	virtual void CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData) override;
	virtual void UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit = false) override;
	virtual ENiagaraRendererSourceDataMode GetCurrentSourceMode() const override { return SourceMode; }
	virtual bool PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore) override;
	virtual bool NeedsMIDsForMaterials() const override { return MaterialParameters.HasAnyBindings(); }
#if WITH_NIAGARA_RENDERER_DEBUGDRAW
	virtual bool SupportsDebugDraw() const { return true; }
#endif
	//UNiagaraRendererProperties Interface END

	void UpdateMICs();

	UMaterialInterface* GetMaterial(const FNiagaraEmitterInstance* InEmitter) const;

	static const FQuat4f GetDefaultOrientation() { return FRotator3f(-90.0f, 0.0f, 90.0f).Quaternion(); }
	static const FVector3f GetDefaultDecalSize() { return FVector3f(50.0f, 50.0f, 50.0f); }
	static const float GetDefaultDecalFade() { return 1.0f; }
	static const int32 GetDefaultDecalSortOrder() { return 0; }
	static const FNiagaraBool GetDefaultDecalVisible() { return FNiagaraBool(true); }

	static FNiagaraVariable GetDecalOrientationVariable();
	static FNiagaraVariable GetDecalSizeVariable();
	static FNiagaraVariable GetDecalFadeVariable();
	static FNiagaraVariable GetDecalVisibleVariable();

	/** What material to use for the decal. */
	UPROPERTY(EditAnywhere, Category = "Decal Rendering")
	TObjectPtr<UMaterialInterface> Material;

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	TObjectPtr<UMaterialInstanceConstant> MICMaterial;
#endif

	/** Binding to material. */
	UPROPERTY(EditAnywhere, Category = "Decal Rendering")
	FNiagaraParameterBinding MaterialParameterBinding;

	/** Whether or not to draw a single element for the Emitter or to draw the particles.*/
	UPROPERTY(EditAnywhere, Category = "Decal Rendering")
	ENiagaraRendererSourceDataMode SourceMode = ENiagaraRendererSourceDataMode::Particles;

	/** If a render visibility tag is present, particles whose tag matches this value will be visible in this renderer. */
	UPROPERTY(EditAnywhere, Category = "Decal Rendering")
	int32 RendererVisibility = 0;

	/** When the decal is smaller than this screen size fade out the decal, can be used to reduce the amount of small decals drawn. */
	UPROPERTY(EditAnywhere, Category = "Decal Rendering")
	float DecalScreenSizeFade = 0.f;

	/** Position binding for the decals, should be center of the decal */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding PositionBinding;

	/** Orientation binding for the decal. */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DecalOrientationBinding;

	/** Size binding for the decal. */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DecalSizeBinding;

	/** Fade binding for the decal, value can be queried using the Decal Lifetime Opacity material node. */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DecalFadeBinding;

	/** Sort Order binding for the decal, used to ensure decals to not flicker when overlapping. */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DecalSortOrderBinding;

	/** Color binding for the decal, value can be queried using the Decal Color material node. */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DecalColorBinding;

	/** Should the decal be visibile or not, works in conjunction with RendererVisibilityTagBinding to determine visibility. */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DecalVisibleBinding;

	/** Visibility tag binding, when valid the returned values is compated with RendererVisibility. */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RendererVisibilityTagBinding;

	/** If this array has entries, we will create a MaterialInstanceDynamic per Emitter instance from Material and set the Material parameters using the Niagara simulation variables listed.*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraRendererMaterialParameters MaterialParameters;

	FNiagaraDataSetAccessor<FNiagaraPosition>	PositionDataSetAccessor;
	FNiagaraDataSetAccessor<FQuat4f>			DecalOrientationDataSetAccessor;
	FNiagaraDataSetAccessor<FVector3f>			DecalSizeDataSetAccessor;
	FNiagaraDataSetAccessor<float>				DecalFadeDataSetAccessor;
	FNiagaraDataSetAccessor<int32>				DecalSortOrderDataSetAccessor;
	FNiagaraDataSetAccessor<FLinearColor>		DecalColorDataSetAccessor;
	FNiagaraDataSetAccessor<FNiagaraBool>		DecalVisibleAccessor;
	FNiagaraDataSetAccessor<int32>				RendererVisibilityTagAccessor;
};
