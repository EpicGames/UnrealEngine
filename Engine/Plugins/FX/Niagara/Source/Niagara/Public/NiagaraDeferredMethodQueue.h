// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * This class is used to defer the execution of object methods that would otherwise be executing at an unsafe point, due to concurrent processing.
 */
class FNiagaraDeferredMethodQueue
{
public:
	DECLARE_DELEGATE(FMethod);

	void Enqueue(const FMethod& Func)
	{
		Queue.Add(Func);
	}

	void Enqueue(FMethod&& Func)
	{
		Queue.Add(MoveTemp(Func));
	}

	void ExecuteAndClear()
	{
		for (auto& Method : Queue)
		{
			Method.ExecuteIfBound();
		}
		Queue.Reset();
	}

private:
	TArray<FMethod> Queue;
};
