// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

//#define NIAGARA_NAN_CHECKING 1
#define NIAGARA_NAN_CHECKING 0


#ifndef WITH_NIAGARA_DEBUGGER
	#if (!UE_BUILD_SHIPPING || WITH_UNREAL_DEVELOPER_TOOLS || WITH_UNREAL_TARGET_DEVELOPER_TOOLS)
	#define WITH_NIAGARA_DEBUGGER 1
	#else
	#define WITH_NIAGARA_DEBUGGER 0
	#endif
#endif
