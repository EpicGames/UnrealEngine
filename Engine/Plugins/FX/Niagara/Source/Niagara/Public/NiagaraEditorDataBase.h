// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "NiagaraCommon.h"
#include "Types/SlateVector2.h"

#include "NiagaraEditorDataBase.generated.h"

class INiagaraParameterDefinitionsSubscriber;
class UNiagaraParameterDefinitionsBase;

USTRUCT()
struct FNiagaraGraphViewSettings
{
	GENERATED_BODY()
public:
	FNiagaraGraphViewSettings()
		: Location(FVector2f::ZeroVector)
		, Zoom(0.0f)
		, bIsValid(false)
	{
	}

	FNiagaraGraphViewSettings(const UE::Slate::FDeprecateVector2DParameter& InLocation, float InZoom)
		: Location(InLocation)
		, Zoom(InZoom)
		, bIsValid(true)
	{
	}

	const FDeprecateSlateVector2D& GetLocation() const { return Location; }
	float GetZoom() const { return Zoom; }
	bool IsValid() const { return bIsValid; }

private:
	UPROPERTY()
	FDeprecateSlateVector2D Location;

	UPROPERTY()
	float Zoom;

	UPROPERTY()
	bool bIsValid;
};

/** A base class for editor only data which supports post loading from the runtime owner object. */
UCLASS(abstract, MinimalAPI)
class UNiagaraEditorDataBase : public UObject
{
	GENERATED_BODY()
public:
#if WITH_EDITORONLY_DATA
	virtual void PostLoadFromOwner(UObject* InOwner) { }

	FSimpleMulticastDelegate& OnPersistentDataChanged() { return PersistentDataChangedDelegate; }

private:
	FSimpleMulticastDelegate PersistentDataChangedDelegate;
#endif
};

/** A base class for editor only data which owns UNiagaraScriptVariables and supports synchronizing them with definitions. */
UCLASS(abstract, MinimalAPI)
class UNiagaraEditorParametersAdapterBase : public UObject
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	/** Synchronize all source script variables that have been changed or removed from the parameter definitions to all eligible destination script variables owned by the editor data.
	 *
	 *  @param TargetDefinitions			The set of parameter definitions that will be synchronized with the editor only parameters.
	 *	@param AllDefinitions				All parameter definitions in the project. Used to add new subscriptions to definitions if specified in Args.
	 *  @param AllDefinitionsParameterIds	All unique Ids of all parameter definitions.
	 *	@param Subscriber					The INiagaraParameterDefinitionsSubscriber that owns the editor data. Used to add new subscriptions to definitions if specified in Args.
	 *	@param Args							Additional arguments that specify how to perform the synchronization.
	 * @return								Returns an array of name pairs representing old names of script vars that were synced and the new names they inherited, respectively.
	 */
	virtual TArray<TTuple<FName /*SyncedOldName*/, FName /*SyncedNewName*/>> SynchronizeParametersWithParameterDefinitions(
		const TArray<UNiagaraParameterDefinitionsBase*> TargetDefinitions,
		const TArray<UNiagaraParameterDefinitionsBase*> AllDefinitions,
		const TSet<FGuid>& AllDefinitionsParameterIds,
		INiagaraParameterDefinitionsSubscriber* Subscriber,
		FSynchronizeWithParameterDefinitionsArgs Args
	) {
		return TArray<TTuple<FName, FName>>();
	};
#endif
};
