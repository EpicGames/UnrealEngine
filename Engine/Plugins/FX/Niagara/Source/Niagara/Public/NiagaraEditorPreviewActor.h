// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "NiagaraEditorPreviewActor.generated.h"

UENUM()
enum class ENiagaraEditorPreviewActorPlaybackType
{
	/** Playback motion will stop when the duation is hit. */
	Once,
	/** Playback motion will loop when the duration is hit. */
	Looping,
	/** Playback motion will go back and forth. */
	PingPong,
};

UENUM()
enum class ENiagaraEditorPreviewActorShapeType
{
	Circle,
	Square,
	Triangle,
	Custom,
	Blueprint,
};

UENUM()
enum class ENiagaraEditorPreviewActorRotationMode
{
	// Do not apply any orientation changes to the component
	None,
	// Orient the component towards the direction of travel
	DirectionOfTravel,
	// Orient the component based on a blueprint callback
	Blueprint,
};

/**
* Niagara Particle System Actor for previewing in the editor will not exist in a cooked packagge or PIE
*/
UCLASS(ComponentWrapperClass, MinimalAPI, meta = (DisplayName = "Niagara Particle System Editor Preview Actor"))
class ANiagaraEditorPreviewActor final : public AActor
{
	GENERATED_BODY()

protected:
	NIAGARA_API ANiagaraEditorPreviewActor(const FObjectInitializer& ObjectInitializer);

public:
	NIAGARA_API virtual void PostRegisterAllComponents() override;
	NIAGARA_API virtual void PostUnregisterAllComponents() override;

	// AActor interface
	NIAGARA_API virtual void Tick(float DeltaSeconds) override;
	virtual bool ShouldTickIfViewportsOnly() const override { return true; }
#if WITH_EDITOR
	NIAGARA_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
#endif // WITH_EDITOR
	// End of AActor interface

	UFUNCTION(BlueprintImplementableEvent, BlueprintCallable, CallInEditor, Category = ActorMotion)
	void CalculateLocation(float MotionTime, FVector& OutLocation);

	UFUNCTION(BlueprintImplementableEvent, BlueprintCallable, CallInEditor, Category = ActorMotion)
	void CalculateRotation(float MotionTime, FQuat& OutRotation);

private:
	TOptional<FVector> InternalCalculateLocation(float MotionTime);

private:
	/** The time it takes for us to complete a cycle of the motion in seconds. */
	UPROPERTY(EditAnywhere, Category = ActorMotion)
	double MotionDuration = 5.0f;

	UPROPERTY(EditAnywhere, Category = ActorMotion, meta = (SegmentedDisplay))
	ENiagaraEditorPreviewActorPlaybackType PlaybackType = ENiagaraEditorPreviewActorPlaybackType::Looping;

	/** What motion type we want to preview view */
	UPROPERTY(EditAnywhere, Category = ActorMotion, meta = (SegmentedDisplay))
	ENiagaraEditorPreviewActorShapeType MotionType = ENiagaraEditorPreviewActorShapeType::Circle;

	UPROPERTY(EditAnywhere, Category = ActorMotion, meta = (EditCondition = "MotionType != ENiagaraEditorPreviewActorShapeType::Circle", EditConditionHides))
	double ShapeTension = 0.5;

	UPROPERTY(EditAnywhere, Category = ActorMotion)
	double ShapeScale = 1.0;

	UPROPERTY(EditAnywhere, Category = ActorMotion)
	FRotator ShapeRotation = FRotator::ZeroRotator;

	UPROPERTY(EditAnywhere, Category = ActorMotion, meta = (EditCondition = "MotionType == ENiagaraEditorPreviewActorShapeType::Custom", EditConditionHides, DisplayName = "Shape Points"))
	TArray<FVector> CustomShapePoints;

	UPROPERTY(EditAnywhere, Category = ActorMotion, meta = (EditCondition = "MotionType == ENiagaraEditorPreviewActorShapeType::Circle", EditConditionHides))
	double CircleRadius = 200.0;

	UPROPERTY(EditAnywhere, Category = ActorMotion, meta = (EditCondition = "MotionType == ENiagaraEditorPreviewActorShapeType::Circle", EditConditionHides))
	TOptional<double> CircleEndRadius;

	UPROPERTY(EditAnywhere, Category = ActorMotion, meta = (EditCondition = "MotionType == ENiagaraEditorPreviewActorShapeType::Circle", EditConditionHides))
	TOptional<double> CircleRotationRate;

	UPROPERTY(EditAnywhere, Category = ActorMotion, meta = (EditCondition = "MotionType == ENiagaraEditorPreviewActorShapeType::Square", EditConditionHides))
	FVector2D SquareSize = FVector2D(400.0, 400.0);

	UPROPERTY(EditAnywhere, Category = ActorMotion, meta = (EditCondition = "MotionType == ENiagaraEditorPreviewActorShapeType::Triangle", EditConditionHides))
	FVector2D TriangleSize = FVector2D(400.0, 400.0);

	UPROPERTY(EditAnywhere, Category = ActorMotion, meta = (SegmentedDisplay))
	ENiagaraEditorPreviewActorRotationMode RotationMode = ENiagaraEditorPreviewActorRotationMode::DirectionOfTravel;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = NiagaraActor, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UNiagaraComponent> NiagaraComponent;

	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;
};
