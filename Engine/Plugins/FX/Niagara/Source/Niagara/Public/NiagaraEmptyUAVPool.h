// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "RHI.h"
#include "RenderGraphFwd.h"

// Scoped access for UAVs this is not required when running inside the main dispatch loop
// but is required for external usage that is outside, i.e. if you are doing some custom dispatch setup
struct FNiagaraEmptyUAVPoolScopedAccess
{
	UE_NONCOPYABLE(FNiagaraEmptyUAVPoolScopedAccess)
public:
	explicit FNiagaraEmptyUAVPoolScopedAccess(class FNiagaraEmptyUAVPool* EmptyUAVPool);
	~FNiagaraEmptyUAVPoolScopedAccess();

private:
	class FNiagaraEmptyUAVPool* EmptyUAVPool;
};

// Scoped access for RDG UAVs this is not required when running inside the main dispatch loop
// but is required for external usage that is outside, i.e. if you are doing some custom dispatch setup
struct FNiagaraEmptyRDGUAVPoolScopedAccess
{
	UE_NONCOPYABLE(FNiagaraEmptyRDGUAVPoolScopedAccess)
public:
	explicit FNiagaraEmptyRDGUAVPoolScopedAccess(class FNiagaraEmptyUAVPool* EmptyUAVPool);
	~FNiagaraEmptyRDGUAVPoolScopedAccess();

private:
	class FNiagaraEmptyUAVPool* EmptyUAVPool;
};

// Type of empty UAV to get
enum class ENiagaraEmptyUAVType
{
	Buffer,
	Texture2D,
	Texture2DArray,
	Texture3D,
	TextureCube,
	TextureCubeArray,
	Num
};

// Empty UAV pool used for ensuring we bind a buffer when one does not exist
class FNiagaraEmptyUAVPool
{
	friend struct FNiagaraEmptyUAVPoolScopedAccess;
	friend struct FNiagaraEmptyRDGUAVPoolScopedAccess;

public:
	/** Must be called before we start to use the UAV pool for a given scene render. */
	void Tick();

	/**
	* Grab a temporary empty RW buffer from the pool.
	* Note: When doing this outside of Niagara you must be within a FNiagaraUAVPoolAccessScope.
	*/
	NIAGARA_API FRHIUnorderedAccessView* GetEmptyUAVFromPool(FRHICommandList& RHICmdList, EPixelFormat Format, ENiagaraEmptyUAVType Type);

	/**
	* Grab a temporary empty RDG Buffer UAV from the pool.
	* Note: When doing this outside of Niagara you must be within a FNiagaraUAVPoolAccessScope.
	*/
	NIAGARA_API FRDGBufferUAVRef GetEmptyRDGUAVFromPool(FRDGBuilder& GraphBuilder, EPixelFormat Format);
	/**
	* Grab a temporary empty RDG Texture UAV from the pool.
	* Note: When doing this outside of Niagara you must be within a FNiagaraUAVPoolAccessScope.
	*/
	NIAGARA_API FRDGTextureUAVRef GetEmptyRDGUAVFromPool(FRDGBuilder& GraphBuilder, EPixelFormat Format, ETextureDimension TextureDimension);

protected:
	/** Returns all used UAVs back to the pool. */
	void ResetEmptyUAVPools();

	/** Returns all the RDG UAVs back to the pool. */
	void ResetEmptyRDGUAVPools();

protected:
	struct FEmptyUAV
	{
		~FEmptyUAV();

		FBufferRHIRef Buffer;
		FTextureRHIRef Texture;
		FUnorderedAccessViewRHIRef UAV;
	};

	struct FEmptyUAVPool
	{
		~FEmptyUAVPool();

		int32 NextFreeIndex = 0;
		TArray<FEmptyUAV> UAVs;
	};

	uint32 UAVAccessCounter = 0;
	TMap<EPixelFormat, FEmptyUAVPool> UAVPools[(int)ENiagaraEmptyUAVType::Num];

	struct FBufferRDGUAVPool
	{
		int32 NextFreeIndex = 0;
		TArray<FRDGBufferUAVRef> UAVs;
	};
	struct FTextureRDGUAVPool
	{
		int32 NextFreeIndex = 0;
		TArray<FRDGTextureUAVRef> UAVs;
	};

	uint32 RDGUAVAccessCounter = 0;
	TMap<EPixelFormat, FBufferRDGUAVPool> BufferRDGUAVPool;
	TMap<TPair<EPixelFormat, ETextureDimension>, FTextureRDGUAVPool> TextureRDGUAVPool;
};
