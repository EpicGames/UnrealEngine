// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "NiagaraComponentPool.h"
#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "NiagaraSystem.h"
#include "Particles/ParticleSystemComponent.h"
#include "NiagaraFunctionLibrary.generated.h"

class UNiagaraComponent;
class USceneComponent;
class UVolumeTexture;
class UMaterialInterface;

USTRUCT(MinimalAPI, BlueprintType)
struct FNiagaraUserParameterInfo
{
	GENERATED_BODY()

	/** The name of the user parameter */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Niagara)
	FName ParameterName;

	// the underlying struct or uobject class of the parameter (set to integer in case of enums)
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Niagara)
	TObjectPtr<UStruct> ParameterType;

	// the name of the underlying type (e.g. Vector3f)
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Niagara)
	FName TypeName;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Niagara)
	int32 ParameterSizeBytes = 0;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Niagara)
	bool bIsEnumType = false;
};

USTRUCT(MinimalAPI, BlueprintType)
struct FNiagaraMinimalEmitterInfo
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Niagara)
	FName EmitterName;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Niagara)
	bool bIsEnabled = false;

	// set to true if this is a lightweight emitter
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Niagara)
	bool bIsLightweight = false;

	// A list of all materials referenced by renderers in this emitter. Could actually be less on the cooked target platform, depending on scalability settings.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Niagara)
	TArray<TObjectPtr<UMaterialInterface>> UsedRendererMaterials;

	// A list of all meshes referenced by renderers in this emitter. Could actually be less on the cooked target platform, depending on scalability settings.
	// This list does NOT include meshes from user parameters or data interfaces, which are resolved at runtime. Also, this covers only the meshes reported by
	// renderers, the emitter itself might use more meshes in other places, for example for skeletal mesh sampling.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Niagara)
	TArray<TObjectPtr<UObject>> UsedRendererMeshes;
};

/**
* A C++ and Blueprint accessible library of utility functions for accessing Niagara simulations
* All positions & orientations are returned in Unreal reference frame & units, assuming the Leap device is located at the origin.
*/
UCLASS(MinimalAPI)
class UNiagaraFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (Keywords = "niagara System", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static NIAGARA_API UNiagaraComponent* SpawnSystemAtLocationWithParams(const FFXSystemSpawnParameters& SpawnParams);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (Keywords = "niagara System", UnsafeDuringActorConstruction = "true"))
	static NIAGARA_API UNiagaraComponent* SpawnSystemAttachedWithParams(const FFXSystemSpawnParameters& SpawnParams);

	/**
	* Spawns a Niagara System at the specified world location/rotation
	* @return			The spawned UNiagaraComponent
	*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (Keywords = "niagara System", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static NIAGARA_API UNiagaraComponent* SpawnSystemAtLocation(const UObject* WorldContextObject, class UNiagaraSystem* SystemTemplate, FVector Location, FRotator Rotation = FRotator::ZeroRotator, FVector Scale = FVector(1.f), bool bAutoDestroy = true, bool bAutoActivate = true, ENCPoolMethod PoolingMethod = ENCPoolMethod::None, bool bPreCullCheck = true);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (Keywords = "niagara System", UnsafeDuringActorConstruction = "true"))
	static NIAGARA_API UNiagaraComponent* SpawnSystemAttached(UNiagaraSystem* SystemTemplate, USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, FRotator Rotation, EAttachLocation::Type LocationType, bool bAutoDestroy, bool bAutoActivate = true, ENCPoolMethod PoolingMethod = ENCPoolMethod::None, bool bPreCullCheck = true);

	static NIAGARA_API UNiagaraComponent* SpawnSystemAttached(UNiagaraSystem* SystemTemplate, USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, FRotator Rotation, FVector Scale, EAttachLocation::Type LocationType, bool bAutoDestroy, ENCPoolMethod PoolingMethod, bool bAutoActivate = true, bool bPreCullCheck = true);

	/** Sets a Niagara StaticMesh parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Static Mesh Component"))
	static NIAGARA_API void OverrideSystemUserVariableStaticMeshComponent(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") const FString& OverrideName, UStaticMeshComponent* StaticMeshComponent);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Static Mesh Directly"))
	static NIAGARA_API void OverrideSystemUserVariableStaticMesh(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") const FString& OverrideName, UStaticMesh* StaticMesh);

	/** Get the skeletal mesh data interface by name .*/
	static NIAGARA_API class UNiagaraDataInterfaceSkeletalMesh* GetSkeletalMeshDataInterface(UNiagaraComponent* NiagaraSystem, const FString& OverrideName);

	/** Sets a Niagara StaticMesh parameter by name, overriding locally if necessary.*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (DisplayName = "Set Niagara Skeletal Mesh Component"))
	static NIAGARA_API void OverrideSystemUserVariableSkeletalMeshComponent(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") const FString& OverrideName, USkeletalMeshComponent* SkeletalMeshComponent);

	/** Sets the SamplingRegion to use on the skeletal mesh data interface, this is destructive as it modifies the data interface. */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	static NIAGARA_API void SetSkeletalMeshDataInterfaceSamplingRegions(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") const FString& OverrideName, const TArray<FName>& SamplingRegions);

	/** Sets the Filtered Bones to use on the skeletal mesh data interface, this is destructive as it modifies the data interface. */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	static NIAGARA_API void SetSkeletalMeshDataInterfaceFilteredBones(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") const FString& OverrideName, const TArray<FName>& FilteredBones);

	/** Sets the Filtered Sockets to use on the skeletal mesh data interface, this is destructive as it modifies the data interface. */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	static NIAGARA_API void SetSkeletalMeshDataInterfaceFilteredSockets(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") const FString& OverrideName, const TArray<FName>& FilteredSockets);

	/** Sets managed mode parameters for the Scene capture 2D  data interface, this is destructive as it modifies the data interface. */
	UFUNCTION(BlueprintCallable, Category = Niagara)
	static NIAGARA_API void SetSceneCapture2DDataInterfaceManagedMode(UNiagaraComponent* NiagaraSystem, const FName& DIName,
		ESceneCaptureSource ManagedCaptureSource,
		FIntPoint ManagedTextureSize,
		ETextureRenderTargetFormat ManagedTextureFormat,
		ECameraProjectionMode::Type ManagedProjectionType,
		float ManagedFOVAngle,
		float ManagedOrthoWidth,
		bool bManagedCaptureEveryFrame,
		bool bManagedCaptureOnMovement,
		const TArray<AActor*> &ShowOnlyActors,
		float ManagedLODDistanceFactor = 1.0f);

	/** Overrides the Texture Object for a Niagara Texture Data Interface User Parameter.*/
	UFUNCTION(BlueprintCallable, Category = Niagara)
	static NIAGARA_API void SetTextureObject(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") const FString& OverrideName, UTexture* Texture);

	/** Overrides the 2D Array Texture for a Niagara 2D Array Texture Data Interface User Parameter.*/
	UFUNCTION(BlueprintCallable, Category = Niagara)
	static NIAGARA_API void SetTexture2DArrayObject(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") const FString& OverrideName, class UTexture2DArray* Texture);

	/** Overrides the Volume Texture for a Niagara Volume Texture Data Interface User Parameter.*/
	UFUNCTION(BlueprintCallable, Category = Niagara)
	static NIAGARA_API void SetVolumeTextureObject(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") const FString& OverrideName, UVolumeTexture* Texture);

	/** Returns basic info on all user parameters of a niagara system.*/
	UFUNCTION(BlueprintCallable, Category = Niagara)
	static NIAGARA_API TArray<FNiagaraUserParameterInfo> GetAllUserParameters(UNiagaraSystem* NiagaraSystem);

	/** Returns basic info on all emitters of a niagara system.*/
	UFUNCTION(BlueprintCallable, Category = Niagara)
	static NIAGARA_API TArray<FNiagaraMinimalEmitterInfo> GetAllEmitters(UNiagaraSystem* NiagaraSystem);
	
	/** Finds an array interface of the given class. */
	static NIAGARA_API UNiagaraDataInterface* GetDataInterface(UClass* DIClass, UNiagaraComponent* NiagaraSystem, FName OverrideName);

	/** Finds an array interface of the given class. */
	template<class TDIType>
	static TDIType* GetDataInterface(UNiagaraComponent* NiagaraSystem, FName OverrideName)
	{
		return (TDIType*)GetDataInterface(TDIType::StaticClass(), NiagaraSystem, OverrideName);
	}

	//This is gonna be totally reworked
// 	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (Keywords = "niagara System", UnsafeDuringActorConstruction = "true"))
// 	static void SetUpdateScriptConstant(UNiagaraComponent* Component, FName EmitterName, FName ConstantName, FVector Value);

	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (Keywords = "niagara parameter collection", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static NIAGARA_API UNiagaraParameterCollectionInstance* GetNiagaraParameterCollection(UObject* WorldContextObject, UNiagaraParameterCollection* Collection);

	/**
	Create a parameter collection instance for the provided parameter collection.
	This can return null if not in a game world or your passed in a null collection.
	*/
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (Keywords = "niagara parameter collection", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static NIAGARA_API UNiagaraParameterCollectionInstance* CreateNiagaraParameterCollectionInstance(UObject* WorldContextObject, UNiagaraParameterCollection* ParentCollection);

	static NIAGARA_API const TArray<FNiagaraFunctionSignature>& GetVectorVMFastPathOps(bool bIgnoreConsoleVariable = false);
	static NIAGARA_API bool DefineFunctionHLSL(const FNiagaraFunctionSignature& FunctionSignature, FString& HlslOutput);

	static NIAGARA_API bool GetVectorVMFastPathExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, FVMExternalFunction &OutFunc);


	//Functions providing access to HWRT collision specific features

	/** Sets the Niagara GPU ray traced collision group for the give primitive component. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (Keywords = "niagara collision ray tracing", WorldContext = "WorldContextObject"))
	static NIAGARA_API void SetComponentNiagaraGPURayTracedCollisionGroup(UObject* WorldContextObject, UPrimitiveComponent* Primitive, int32 CollisionGroup);
	
	/** Sets the Niagara GPU ray traced collision group for all primitive components on the given actor. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (Keywords = "niagara collision ray tracing", WorldContext = "WorldContextObject"))
	static NIAGARA_API void SetActorNiagaraGPURayTracedCollisionGroup(UObject* WorldContextObject, AActor* Actor, int32 CollisionGroup);

	/** Returns a free collision group for use in HWRT collision group filtering. Returns -1 on failure. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (Keywords = "niagara collision ray tracing", WorldContext = "WorldContextObject"))
	static NIAGARA_API int32 AcquireNiagaraGPURayTracedCollisionGroup(UObject* WorldContextObject);

	/** Releases a collision group back to the system for use by ohers. */
	UFUNCTION(BlueprintCallable, Category = Niagara, meta = (Keywords = "niagara collision ray tracing", WorldContext = "WorldContextObject"))
	static NIAGARA_API void ReleaseNiagaraGPURayTracedCollisionGroup(UObject* WorldContextObject, int32 CollisionGroup);
private:
	static NIAGARA_API void InitVectorVMFastPathOps();
	static NIAGARA_API TArray<FNiagaraFunctionSignature> VectorVMOps;
	static NIAGARA_API TArray<FString> VectorVMOpsHLSL;
};
