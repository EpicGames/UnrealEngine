// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"

class FNiagaraGpuComputeDispatchInterface;
class FRDGBuilder;

struct FNiagaraGpuComputeDataManagerTickContext
{
	FNiagaraGpuComputeDataManagerTickContext(FRDGBuilder& InGraphBuilder)
		: GraphBuilder(InGraphBuilder)
	{
	}

	FRDGBuilder& GetGraphBuilder() const { return GraphBuilder; }

private:
	FRDGBuilder& GraphBuilder;
};

// Abstract class for managing GPU data for Niagara
// Once the manager is created it will last for the lifetime of the owner dispatch interface
// The manager will be destroyed when the dispatch interface is also destroyed
class FNiagaraGpuComputeDataManager
{
	UE_NONCOPYABLE(FNiagaraGpuComputeDataManager);

public:
	FNiagaraGpuComputeDataManager(FNiagaraGpuComputeDispatchInterface* InOwnerInterface)
		: InternalOwnerInterface(InOwnerInterface)
	{
	}
	virtual ~FNiagaraGpuComputeDataManager() = default;

	FNiagaraGpuComputeDispatchInterface* GetOwnerInterface() const { return InternalOwnerInterface; }

	//static FName GetManagerName()
	//{
	//	static FName ManagerName("FNiagaraGpuComputeDataManager");
	//	return ManagerName;
	//}

	virtual void BeginFrame(FNiagaraGpuComputeDataManagerTickContext& Context){}
	virtual void EndFrame(FNiagaraGpuComputeDataManagerTickContext& Context){}
	virtual void PreTick(FNiagaraGpuComputeDataManagerTickContext& Context, ENiagaraGpuComputeTickStage::Type TickStage){}

private:
	FNiagaraGpuComputeDispatchInterface* InternalOwnerInterface = nullptr;
};
