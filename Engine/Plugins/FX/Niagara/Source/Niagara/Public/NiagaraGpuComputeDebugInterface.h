// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
Class used help realtime debug Gpu Compute simulations
==============================================================================*/
#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraRenderGraphUtils.h"
#include "RenderGraphDefinitions.h"

#if NIAGARA_COMPUTEDEBUG_ENABLED

class FNiagaraGpuComputeDebug;

class FNiagaraGpuComputeDebugInterface
{
	friend class FNiagaraGpuComputeDispatchInterface;

private:
	explicit FNiagaraGpuComputeDebugInterface(FNiagaraGpuComputeDebug* InComputeDebug)
		: ComputeDebug(InComputeDebug)
	{
	}

public:
	// Add a texture to visualize
	NIAGARA_API void AddTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FVector2D PreviewDisplayRange = FVector2D::ZeroVector);

	// Add a texture to visualize that contains a number of attributes and select which attributes to push into RGBA where -1 means ignore that channel
	// The first -1 in the attribute indices list will also limit the number of attributes we attempt to read.
	// NumTextureAttributes in this version is meant for a 2D atlas
	NIAGARA_API void AddAttributeTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FIntPoint NumTextureAttributes, FIntVector4 AttributeIndices, FVector2D PreviewDisplayRange = FVector2D::ZeroVector);

	// Add a texture to visualize that contains a number of attributes and select which attributes to push into RGBA where -1 means ignore that channel
	// The first -1 in the attribute indices list will also limit the number of attributes we attempt to read
	// NumTextureAttributes in this version is meant for a 3D atlas
	NIAGARA_API void AddAttributeTexture(FRDGBuilder& GraphBuilder, FNiagaraSystemInstanceID SystemInstanceID, FName SourceName, FRDGTextureRef Texture, FIntVector4 NumTextureAttributes, FIntVector4 AttributeIndices, FVector2D PreviewDisplayRange = FVector2D::ZeroVector);

private:
	FNiagaraGpuComputeDebug* ComputeDebug = nullptr;
};

#endif //NIAGARA_COMPUTEDEBUG_ENABLED
