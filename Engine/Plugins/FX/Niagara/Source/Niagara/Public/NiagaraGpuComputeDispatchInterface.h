// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StridedView.h"
#include "NiagaraCommon.h"
#include "NiagaraEmptyUAVPool.h"
#include "NiagaraGpuComputeDataManager.h"
#include "NiagaraGPUInstanceCountManager.h"
#include "FXSystem.h"

class FGlobalDistanceFieldParameterData;
class FRDGBuilder;
class FRDGExternalAccessQueue;
class FSceneView;

class FNiagaraAsyncGpuTraceHelper;
struct FNiagaraComputeExecutionContext;
class FNiagaraGpuComputeDebug;
class FNiagaraGpuComputeDebugInterface;
class FNiagaraGPUInstanceCountManager;
class FNiagaraGpuReadbackManager;
class FNiagaraRayTracingHelper;
struct FNiagaraScriptDebuggerInfo;
class FNiagaraSystemGpuComputeProxy;

using FNiagaraDataChannelDataProxyPtr = TSharedPtr<struct FNiagaraDataChannelDataProxy>;

// Public API for Niagara's Compute Dispatcher
// This is generally used with DataInterfaces or Custom Renderers
class FNiagaraGpuComputeDispatchInterface : public FFXSystemInterface
{
public:
	DECLARE_EVENT_OneParam(FNiagaraGpuComputeDispatchInterface, FOnPreInitViewsEvent, FRDGBuilder&);
	DECLARE_EVENT_OneParam(FNiagaraGpuComputeDispatchInterface, FOnPostPreRenderEvent, FRDGBuilder&);

	static NIAGARA_API FNiagaraGpuComputeDispatchInterface* Get(class UWorld* World);
	static NIAGARA_API FNiagaraGpuComputeDispatchInterface* Get(class FSceneInterface* Scene);
	static NIAGARA_API FNiagaraGpuComputeDispatchInterface* Get(class FFXSystemInterface* FXSceneInterface);

	NIAGARA_API explicit FNiagaraGpuComputeDispatchInterface(EShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel);
	NIAGARA_API virtual ~FNiagaraGpuComputeDispatchInterface();

	/** Get ShaderPlatform the batcher is bound to */
	EShaderPlatform GetShaderPlatform() const { return ShaderPlatform; }
	/** Get FeatureLevel the batcher is bound to */
	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel; }

	/** Add system instance proxy to the batcher for tracking. */
	virtual void AddGpuComputeProxy(FNiagaraSystemGpuComputeProxy* ComputeProxy) = 0;
	/** Remove system instance proxy from the batcher. */
	virtual void RemoveGpuComputeProxy(FNiagaraSystemGpuComputeProxy* ComputeProxy) = 0;

	/** Add NDC Data to the batcher for tracking */
	virtual void AddNDCDataProxy(FNiagaraDataChannelDataProxyPtr NDCDataProxy) = 0;
	/** Add NDC Data to the batcher for tracking */
	virtual void RemoveNDCDataProxy(FNiagaraDataChannelDataProxyPtr NDCDataProxy) = 0;

	/**
	 * Register work for GPU sorting (using the GPUSortManager).
	 * The constraints of the sort request are defined in SortInfo.SortFlags.
	 * The sort task bindings are set in SortInfo.AllocationInfo.
	 * The initial keys and values are generated in the GenerateSortKeys() callback.
	 *
	 * Return true if the work was registered, or false it GPU sorting is not available or impossible.
	 */
	virtual bool AddSortedGPUSimulation(FRHICommandListBase& RHICmdList, struct FNiagaraGPUSortInfo& SortInfo) = 0;

	UE_DEPRECATED(5.4, "AddSortedGPUSimulation requires an RHI command list")
	virtual bool AddSortedGPUSimulation(struct FNiagaraGPUSortInfo& SortInfo) final { return false; }

	template<typename TManager>
	const TManager* GetDataManager()const
	{
		check(IsInParallelRenderingThread());

		UE::TScopeLock ScopeLock(ComputeManagerGuard);
		const FName ManagerName = TManager::GetManagerName();
		for (auto& DataManager : GpuDataManagers)
		{
			if (DataManager.Key == ManagerName)
			{
				return static_cast<const TManager*>(DataManager.Value.Get());
			}
		}
		return nullptr;
	}

	/** Get or create the a data manager, must be done on the rendering thread only. */
	template<typename TManager>
	TManager& GetOrCreateDataManager()
	{
		check(IsInParallelRenderingThread());

		UE::TScopeLock ScopeLock(ComputeManagerGuard);
		const FName ManagerName = TManager::GetManagerName();
		for (auto& DataManager : GpuDataManagers)
		{
			if (DataManager.Key == ManagerName)
			{
				return *static_cast<TManager*>(DataManager.Value.Get());
			}
		}

		TManager* Manager = new TManager(this);
		GpuDataManagers.Emplace(ManagerName, Manager);
		return *Manager;
	}

	/**
	Get access to the Views the simulation is being rendered with.
	List is only valid during graph building (i.e. during ExecuteTicks) and for simulations in PostInitViews / PostRenderOpaque.
	*/
	TConstStridedView<FSceneView> GetSimulationSceneViews() const { return SimulationSceneViews; }

	/**
	* Get access to the global distance field data
	* This will return nullptr if you attempt to access at an invalid point (i.e. before GDF is prepared or the GDF is not available)
	*/
	virtual const FGlobalDistanceFieldParameterData* GetGlobalDistanceFieldData() const = 0;

	/** Get access to the instance count manager. */
	inline FNiagaraGPUInstanceCountManager& GetGPUInstanceCounterManager() { check(IsInParallelRenderingThread()); return GPUInstanceCounterManager; }
	inline const FNiagaraGPUInstanceCountManager& GetGPUInstanceCounterManager() const { check(IsInParallelRenderingThread()); return GPUInstanceCounterManager; }

#if NIAGARA_COMPUTEDEBUG_ENABLED
	/** Public interface to Niagara compute debugging. */
	NIAGARA_API FNiagaraGpuComputeDebugInterface GetGpuComputeDebugInterface() const;

	/** Get access to Niagara's GpuComputeDebug this is for internal use */
	FNiagaraGpuComputeDebug* GetGpuComputeDebugPrivate() const { return GpuComputeDebugPtr.Get(); }
#endif

#if WITH_NIAGARA_GPU_PROFILER
	/** Access to Niagara's GPU Profiler */
	virtual class FNiagaraGPUProfilerInterface* GetGPUProfiler() const = 0;
#endif

	/** Get access to Niagara's GpuReadbackManager. */
	FNiagaraGpuReadbackManager* GetGpuReadbackManager() const { return GpuReadbackManagerPtr.Get(); }

	/** Get access to Niagara's GpuReadbackManager. */
	//UE_DEPRECATED(5.1, "The UAV Pool will be removed, please update your code to support RenderGraph.")
	FNiagaraEmptyUAVPool* GetEmptyUAVPool() const { return EmptyUAVPoolPtr.Get(); }

	/** Convenience wrapper to get a UAV from the pool. */
	//UE_DEPRECATED(5.1, "The UAV Pool will be removed, please update your code to support RenderGraph.")
	FRHIUnorderedAccessView* GetEmptyUAVFromPool(FRHICommandList& RHICmdList, EPixelFormat Format, ENiagaraEmptyUAVType Type) const { return EmptyUAVPoolPtr->GetEmptyUAVFromPool(RHICmdList, Format, Type); }

	/** Helper function to return an RDG Texture where the texture contains 0 for all channels. */
	NIAGARA_API FRDGTextureRef GetBlackTexture(FRDGBuilder& GraphBuilder, ETextureDimension TextureDimension) const;

	/** Helper function to return a RDG Texture SRV where the texture contains 0 for all channels. */
	NIAGARA_API FRDGTextureSRVRef GetBlackTextureSRV(FRDGBuilder& GraphBuilder, ETextureDimension TextureDimension) const;

	/** Helper function to return a RDG Texture UAV you don't care about the contents of or the results, i.e. to use as a dummy binding. */
	NIAGARA_API FRDGTextureUAVRef GetEmptyTextureUAV(FRDGBuilder& GraphBuilder, EPixelFormat Format, ETextureDimension TextureDimension) const;

	/** Helper function to return a Buffer UAV you don't care about the contents of or the results, i.e. to use as a dummy binding. */
	NIAGARA_API FRDGBufferUAVRef GetEmptyBufferUAV(FRDGBuilder& GraphBuilder, EPixelFormat Format) const;

	/** Helper function to return a Buffer SRV which will contain 1 element of 0 value, i.e. to use as a dummy binding. */
	NIAGARA_API FRDGBufferSRVRef GetEmptyBufferSRV(FRDGBuilder& GraphBuilder, EPixelFormat Format) const;

	/**
	Call this to force all pending ticks to be flushed from the batcher.
	Doing so will execute them outside of a view context which may result in undesirable results.
	*/
	virtual void FlushPendingTicks_GameThread() = 0;

	/**
	This will flush all pending ticks & readbacks from the dispatcher.
	Note: This is a GameThread blocking call and will impact performance
	*/
	virtual void FlushAndWait_GameThread() = 0;

	/** Debug only function to readback data. */
	virtual void AddDebugReadback(FNiagaraSystemInstanceID InstanceID, TSharedPtr<FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> DebugInfo, FNiagaraComputeExecutionContext* Context) = 0;

	/** Processes all pending debug readbacks */
	virtual void ProcessDebugReadbacks(FRHICommandList& RHICmdList, bool bWaitCompletion) = 0;

	virtual FNiagaraAsyncGpuTraceHelper& GetAsyncGpuTraceHelper() const = 0;

	inline bool IsOutsideSceneRenderer() const { return bIsOutsideSceneRenderer; }

	inline bool IsFirstViewFamily() const { return bIsFirstViewFamily; }
	inline bool IsLastViewFamily() const { return bIsLastViewFamily; }

#if WITH_MGPU
	/**
	Notify that a GPU resource was modified that will impact MultiGPU rendering.
	bRequiredForSimulation	- When true we require this resource for simulation passes
	bRequiredForRendering	- When true we require this resource for rendering passes
	*/
	virtual void MultiGPUResourceModified(FRDGBuilder& GraphBuilder, FRHIBuffer* Buffer, bool bRequiredForSimulation, bool bRequiredForRendering) const = 0;
	virtual void MultiGPUResourceModified(FRDGBuilder& GraphBuilder, FRHITexture* Texture, bool bRequiredForSimulation, bool bRequiredForRendering) const = 0;

	UE_DEPRECATED(5.1, "ImmediateMode is deprecated for Niagara please migrate to using a FRDGBuilder")
	virtual void MultiGPUResourceModified(FRHICommandList& RHICmdList, FRHIBuffer* Buffer, bool bRequiredForSimulation, bool bRequiredForRendering) const = 0;
	UE_DEPRECATED(5.1, "ImmediateMode is deprecated for Niagara please migrate to using a FRDGBuilder")
	virtual void MultiGPUResourceModified(FRHICommandList& RHICmdList, FRHITexture* Texture, bool bRequiredForSimulation, bool bRequiredForRendering) const = 0;
#else
	inline void MultiGPUResourceModified(FRDGBuilder& GraphBuilder, FRHIBuffer* Buffer, bool bRequiredForSimulation, bool bRequiredForRendering) const {}
	inline void MultiGPUResourceModified(FRDGBuilder& GraphBuilder, FRHITexture* Texture, bool bRequiredForSimulation, bool bRequiredForRendering) const {}

	UE_DEPRECATED(5.1, "ImmediateMode is deprecated for Niagara please migrate to using a FRDGBuilder")
	inline void MultiGPUResourceModified(FRHICommandList& RHICmdList, FRHIBuffer* Buffer, bool bRequiredForSimulation, bool bRequiredForRendering) const {}
	UE_DEPRECATED(5.1, "ImmediateMode is deprecated for Niagara please migrate to using a FRDGBuilder")
	inline void MultiGPUResourceModified(FRHICommandList& RHICmdList, FRHITexture* Texture, bool bRequiredForSimulation, bool bRequiredForRendering) const {}
#endif

	/**
	Event that broadcast when we enter PreInitViews.
	*/
	FOnPreInitViewsEvent& GetOnPreInitViewsEvent() { check(IsInRenderingThread()); return OnPreInitViewsEvent; }
	/**
	Event that broadcast when we endter PreRender.
	This is called before we prepare any work or add passes for simulating.
	*/
	FOnPostPreRenderEvent& GetOnPreRenderEvent() { check(IsInRenderingThread()); return OnPreRenderEvent; }
	/**
	Event that broadcast at the end of PostRenderOpaque.
	This is called after all simulation passes have been added.
	*/
	FOnPostPreRenderEvent& GetOnPostRenderEvent() { check(IsInRenderingThread()); return OnPostRenderEvent; }

protected:
	EShaderPlatform							ShaderPlatform;
	ERHIFeatureLevel::Type					FeatureLevel;
#if NIAGARA_COMPUTEDEBUG_ENABLED
	TUniquePtr<FNiagaraGpuComputeDebug>		GpuComputeDebugPtr;
#endif
	TUniquePtr<FNiagaraGpuReadbackManager>	GpuReadbackManagerPtr;
	TUniquePtr<FNiagaraEmptyUAVPool>		EmptyUAVPoolPtr;

	// GPU emitter instance count buffer. Contains the actual particle / instance count generate in the GPU tick.
	FNiagaraGPUInstanceCountManager			GPUInstanceCounterManager;

	TArray<TPair<FName, TUniquePtr<FNiagaraGpuComputeDataManager>>> GpuDataManagers;

	TConstStridedView<FSceneView>			SimulationSceneViews;

	bool									bIsOutsideSceneRenderer = false;
	bool									bIsFirstViewFamily = true;
	bool									bIsLastViewFamily = true;

	FOnPreInitViewsEvent					OnPreInitViewsEvent;
	FOnPostPreRenderEvent					OnPreRenderEvent;
	FOnPostPreRenderEvent					OnPostRenderEvent;

	mutable UE::FMutex						ComputeManagerGuard;
};
