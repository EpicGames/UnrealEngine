// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "RenderGraphResources.h"

class FRDGBuilder;

class FNiagaraGpuReadbackManager
{
protected:
	typedef TFunction<void(TConstArrayView<TPair<void*, uint32>>)> FCompletionCallback;
		
	struct FPendingReadback
	{
		TArray<TPair<FStagingBufferRHIRef, uint32>, TInlineAllocator<1>>	StagingBuffers;
		FGPUFenceRHIRef														Fence;
		FCompletionCallback													Callback;
	};

public:
	struct FRDGBufferRequest
	{
		FRDGBufferRequest() {}
		FRDGBufferRequest(FRDGBufferRef InBuffer, uint32 InOffset, uint32 InSize) : Buffer(InBuffer), Offset(InOffset), Size(InSize) {}

		FRDGBufferRef	Buffer = nullptr;
		uint32			Offset = 0;
		uint32			Size = 0;
	};

	struct FBufferRequest
	{
		FBufferRequest() {}
		FBufferRequest(FRHIBuffer* InBuffer, uint32 InOffset, uint32 InSize) : Buffer(InBuffer), Offset(InOffset), Size(InSize) {}

		FRHIBuffer*	Buffer = nullptr;
		uint32		Offset = 0;
		uint32		Size = 0;
	};

public:
	NIAGARA_API FNiagaraGpuReadbackManager();
	NIAGARA_API ~FNiagaraGpuReadbackManager();

	// Tick call which polls for completed readbacks
	NIAGARA_API void Tick(FRHICommandListImmediate& RHICmdList);
	
private:
	// Internal tick impl
	void TickInternal(FRHICommandListImmediate& RHICmdList, bool bAssumeGpuIdle);

public:
	// Wait for all pending readbacks to complete
	NIAGARA_API void WaitCompletion(FRHICommandListImmediate& RHICmdList);

	// Enqueue a readback of a single buffer
	NIAGARA_API void EnqueueReadback(FRDGBuilder& GraphBuilder, FRDGBufferRef Buffer, FCompletionCallback Callback);
	NIAGARA_API void EnqueueReadback(FRDGBuilder& GraphBuilder, FRDGBufferRef Buffer, uint32 Offset, uint32 NumBytes, FCompletionCallback Callback);

	// Enqueue a readback of multiple buffers
	NIAGARA_API void EnqueueReadbacks(FRDGBuilder& GraphBuilder, TConstArrayView<FRDGBufferRef> Buffers, FCompletionCallback Callback);
	NIAGARA_API void EnqueueReadbacks(FRDGBuilder& GraphBuilder, TConstArrayView<FRDGBufferRequest> BufferRequest, FCompletionCallback Callback);

	//-TODO:RDG:Deprecate these calls
	//UE_DEPRECATED(5.1, "Support for legacy shaders bindings will be removed in a future release, please upgrade your data interface.")
	NIAGARA_API void EnqueueReadback(FRHICommandList& RHICmdList, FRHIBuffer* Buffer, FCompletionCallback Callback);
	//UE_DEPRECATED(5.1, "Support for legacy shaders bindings will be removed in a future release, please upgrade your data interface.")
	NIAGARA_API void EnqueueReadback(FRHICommandList& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 NumBytes, FCompletionCallback Callback);

	//-TODO:RDG:Deprecate these calls
	//UE_DEPRECATED(5.1, "Support for legacy shaders bindings will be removed in a future release, please upgrade your data interface.")
	NIAGARA_API void EnqueueReadbacks(FRHICommandList& RHICmdList, TConstArrayView<FRHIBuffer*> Buffers, FCompletionCallback Callback);
	//UE_DEPRECATED(5.1, "Support for legacy shaders bindings will be removed in a future release, please upgrade your data interface.")
	NIAGARA_API void EnqueueReadbacks(FRHICommandList& RHICmdList, TConstArrayView<FBufferRequest> BufferRequest, FCompletionCallback Callback);

private:
	TQueue<FPendingReadback> PendingReadbacks;

	static const FName FenceName;
};
