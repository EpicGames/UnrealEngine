// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSetAccessor.h"
#include "NiagaraLightRendererProperties.generated.h"

class FNiagaraEmitterInstance;
class SWidget;

UCLASS(editinlinenew, MinimalAPI, meta = (DisplayName = "Light Renderer", SupportsStateless))
class UNiagaraLightRendererProperties : public UNiagaraRendererProperties
{
public:
	GENERATED_BODY()

	UNiagaraLightRendererProperties();

	//UObject Interface
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
#if WITH_EDITORONLY_DATA
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif// WITH_EDITORONLY_DATA
	//UObject Interface END

	static void InitCDOPropertiesAfterModuleStartup();

	//~ UNiagaraRendererProperties interface
	virtual FNiagaraRenderer* CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) override;
	virtual class FNiagaraBoundsCalculator* CreateBoundsCalculator() override;
	virtual void GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const override;
	virtual bool IsSimTargetSupported(ENiagaraSimTarget InSimTarget) const override { return InSimTarget == ENiagaraSimTarget::CPUSim || SourceMode == ENiagaraRendererSourceDataMode::Emitter; };
#if WITH_EDITORONLY_DATA
	virtual TArray<FNiagaraVariable> GetBoundAttributes() const override;
	virtual const TArray<FNiagaraVariable>& GetOptionalAttributes() override;
	virtual void GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FText>& OutErrors, TArray<FText>& OutWarnings, TArray<FText>& OutInfo) const override;
#endif // WITH_EDITORONLY_DATA
	virtual void UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit = false) override;
	virtual ENiagaraRendererSourceDataMode GetCurrentSourceMode() const override { return SourceMode; }
	virtual bool PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore) override;
	virtual void CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData) override;
#if WITH_NIAGARA_RENDERER_DEBUGDRAW
	virtual bool SupportsDebugDraw() const { return true; }
#endif
	//UNiagaraRendererProperties Interface END

	static FLinearColor GetDefaultColor();
	static float GetDefaultRadius();
	static float GetDefaultVolumetricScattering();

	/** Whether or not to draw a single element for the Emitter or to draw the particles.*/
	UPROPERTY(EditAnywhere, Category = "Light Rendering")
	ENiagaraRendererSourceDataMode SourceMode = ENiagaraRendererSourceDataMode::Particles;

	/** Whether to use physically based inverse squared falloff from the light.  If unchecked, the value from the LightExponent binding will be used instead. */
	UPROPERTY(EditAnywhere, Category = "Light Rendering")
	uint32 bUseInverseSquaredFalloff : 1;

	/**
	 * Whether lights from this renderer should affect translucency.
	 * Use with caution - if enabled, create only a few particle lights at most, and the smaller they are, the less they will cost.
	 */
	UPROPERTY(EditAnywhere, Category = "Light Rendering")
	uint32 bAffectsTranslucency : 1;

	/**
	 * Whether lights from this renderer should be handled by MegaLights.
	 * Use with caution - if enabled, create only a few particle lights at most and keep them as small as possible. Look out for increased noise and sampling cost.
	 */
	UPROPERTY(EditAnywhere, Category = "Light Rendering")
	uint32 bAllowMegaLights : 1;

	/**
	 * Whether lights from this renderer can cast shadows when handled by MegaLights.
	 */
	UPROPERTY(EditAnywhere, Category = "Light Rendering", meta = (EditCondition = "bAllowMegaLights", EditConditionHides))
	uint32 bMegaLightsCastShadows : 1;

	/** When checked, will treat the alpha value of the particle's color as a multiplier of the light's brightness. */
	UPROPERTY(EditAnywhere, Category = "Light Rendering")
	uint32 bAlphaScalesBrightness : 1;

	/** When enabled we will override the project default setting with our local setting. */
	UPROPERTY(EditAnywhere, Category = "Light Rendering", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint32 bOverrideInverseExposureBlend : 1;

	/** A factor used to scale each particle light radius */
	UPROPERTY(EditAnywhere, Category = "Light Rendering", meta = (ClampMin = "0"))
	float RadiusScale;

	/** The exponent to use for all lights if no exponent binding was found */
	UPROPERTY(EditAnywhere, Category = "Light Rendering", meta = (ClampMin = "0", EditCondition = "!bUseInverseSquaredFalloff", EditConditionHides))
	float DefaultExponent;

	/** The specular scale to use for all lights if no binding was found */
	UPROPERTY(EditAnywhere, Category = "Light Rendering", meta = (ClampMin = "0"))
	float SpecularScale = 1.0f;

	/** The diffuse scale to use for all lights if no binding was found */
	UPROPERTY(EditAnywhere, Category = "Light Rendering", meta = (ClampMin = "0"))
	float DiffuseScale = 1.0f;

	/** A static color shift applied to each rendered light */
	UPROPERTY(EditAnywhere, Category = "Light Rendering")
	FVector3f ColorAdd;

	/**
	* Blend Factor used to blend between Intensity and Intensity/Exposure.
	* This is useful for gameplay lights that should have constant brighness on screen independent of current exposure.
	* This feature can cause issues with exposure particularly when used on the primary light on a scene, as such it's usage should be limited.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Light Rendering", AdvancedDisplay, meta = (UIMin = "0.0", UIMax = "1.0", EditCondition="bOverrideInverseExposureBlend"))
	float InverseExposureBlend = 0.0f;

	/** If a render visibility tag is present, particles whose tag matches this value will be visible in this renderer. */
	UPROPERTY(EditAnywhere, Category = "Light Rendering")
	int32 RendererVisibility;

	/** Which attribute should we use to check if light rendering should be enabled for a particle? This can be used to control the spawn-rate on a per-particle basis. */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding LightRenderingEnabledBinding;

	/** Which attribute should we use for the light's exponent when inverse squared falloff is disabled? */
	UPROPERTY(EditAnywhere, Category = "Bindings", meta = (EditCondition = "!bUseInverseSquaredFalloff", EditConditionHides))
	FNiagaraVariableAttributeBinding LightExponentBinding;

	/** Which attribute should we use for position when generating lights?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding PositionBinding;

	/** Which attribute should we use for light color when generating lights?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding ColorBinding;

	/** Which attribute should we use for light radius when generating lights?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RadiusBinding;

	/** Which attribute should we use for the intensity of the volumetric scattering from this light? This scales the light's intensity and color. */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding VolumetricScatteringBinding;

	/** Which attribute should we use for the renderer visibility tag? */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RendererVisibilityTagBinding;

	/** Which attribute should we use for the specular scale when generating lights?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding SpecularScaleBinding;

	/** Which attribute should we use for the diffuse scale when generating lights?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DiffuseScaleBinding;

	FNiagaraDataSetAccessor<FNiagaraPosition> PositionDataSetAccessor;
	FNiagaraDataSetAccessor<FLinearColor> ColorDataSetAccessor;
	FNiagaraDataSetAccessor<float> RadiusDataSetAccessor;
	FNiagaraDataSetAccessor<float> ExponentDataSetAccessor;
	FNiagaraDataSetAccessor<float> ScatteringDataSetAccessor;
	FNiagaraDataSetAccessor<FNiagaraBool> EnabledDataSetAccessor;
	FNiagaraDataSetAccessor<int32> RendererVisibilityTagAccessor;
	FNiagaraDataSetAccessor<int32> LightIdAccessor;
	FNiagaraDataSetAccessor<float> SpecularScaleAccessor;
	FNiagaraDataSetAccessor<float> DiffuseScaleAccessor;

private:
	static TArray<TWeakObjectPtr<UNiagaraLightRendererProperties>> LightRendererPropertiesToDeferredInit;
};
