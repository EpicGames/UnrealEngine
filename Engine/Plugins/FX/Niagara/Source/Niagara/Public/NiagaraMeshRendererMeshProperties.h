// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraRenderableMeshInterface.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraParameterBinding.h"
#include "NiagaraMeshRendererMeshProperties.generated.h"

class FNiagaraEmitterInstance;
class UStaticMesh;

UENUM()
enum class ENiagaraMeshPivotOffsetSpace : uint8
{
	/** The pivot offset is in the mesh's local space (default) */
	Mesh,
	/** The pivot offset is in the emitter's local space if the emitter is marked as local-space, or in world space otherwise */
	Simulation,
	/** The pivot offset is in world space */
	World,
	/** The pivot offset is in the emitter's local space */
	Local
};

UENUM()
enum class ENiagaraMeshLODMode : uint8
{
	/*
	Uses the provided LOD level to render all mesh particles.
	If the LOD is not streamed in or available on the platform the next available lower LOD level will be used.
	For example, LOD Level is set to 1 but the first available is LOD 3 then LOD 3 will be used.
	*/
	LODLevel,

	/**
	Takes the highest available LOD for the platform + LOD bias to render all mesh particles
	If the LOD is not streamed in or available on the platform the next available lower LOD level will be used.
	For example, LOD bias is set to 1, the current platform has Min LOD of 2 then 3 will be the used LOD.
	*/
	LODBias,

	/*
	The LOD level is calculated based on screen space size of the component bounds.
	All particles will be rendered with the same calculated LOD level.
	Increasing 'LOD calculation scale' will result in lower quality LODs being used, this is useful as component bounds generally are larger than the particle mesh bounds.
	*/
	ByComponentBounds,

	/*
	The LOD level will be calculated like we have a single particle at the component origin, i.e. it should match a static mesh with the exact same transform.
	All particles will be rendered with the same calculated LOD level.
	Increasing 'LOD calculation scale' will result in lower quality LODs being used
	*/
	ComponentOrigin,

	/*
	The LOD level is calcuated per particle using the particle position and mesh sphere bounds.
	This involves running a dispatch & draw per LOD level.
	Calculates and renders each particle with it's calcualted LOD level.
	Increasing 'LOD calculation scale' will result in lower quality LODs being used.
	*/
	PerParticle,
};

USTRUCT(BlueprintType)
struct FNiagaraMeshRendererMeshPropertiesBase
{
	GENERATED_BODY()

	/** The mesh to use when rendering this slot */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mesh")
	TObjectPtr<UStaticMesh> Mesh;

	/** Scale of the mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mesh")
	FVector Scale = FVector::OneVector;

	/** Rotation of the mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mesh")
	FRotator Rotation = FRotator::ZeroRotator;

	/** Offset of the mesh pivot */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mesh")
	FVector PivotOffset = FVector::ZeroVector;

	/** What space is the pivot offset in? */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mesh")
	ENiagaraMeshPivotOffsetSpace PivotOffsetSpace = ENiagaraMeshPivotOffsetSpace::Mesh;

	bool IsNearlyEqual(const FNiagaraMeshRendererMeshPropertiesBase& Rhs, float Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return
			Mesh == Rhs.Mesh &&
			Scale.Equals(Rhs.Scale, Tolerance) &&
			Rotation.Equals(Rhs.Rotation, Tolerance) &&
			PivotOffset.Equals(Rhs.PivotOffset, Tolerance) &&
			PivotOffsetSpace == Rhs.PivotOffsetSpace;
	}

	bool operator==(const FNiagaraMeshRendererMeshPropertiesBase& Rhs) const
	{
		return IsNearlyEqual(Rhs);
	}
};

USTRUCT()
struct FNiagaraMeshRendererMeshProperties : public FNiagaraMeshRendererMeshPropertiesBase
{
	GENERATED_BODY()

	NIAGARA_API FNiagaraMeshRendererMeshProperties();

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FNiagaraUserParameterBinding UserParamBinding_DEPRECATED;
#endif

	/** Binding to supported mesh types. */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	FNiagaraParameterBinding MeshParameterBinding;

	UPROPERTY(EditAnywhere, Category = "Mesh")
	ENiagaraMeshLODMode LODMode = ENiagaraMeshLODMode::LODLevel;

#if WITH_EDITORONLY_DATA
	/** Absolute LOD level to use */
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (UIMin = "0", DisplayName="LOD Level", EditCondition = "LODMode == ENiagaraMeshLODMode::LODLevel", EditConditionHides))
	FNiagaraParameterBindingWithValue LODLevelBinding;

	/* LOD bias to apply to the LOD calculation. */
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (UIMin = "0", DisplayName = "LOD Bias", EditCondition = "LODMode == ENiagaraMeshLODMode::LODBias", EditConditionHides))
	FNiagaraParameterBindingWithValue LODBiasBinding;
#endif

	UPROPERTY()
	int32 LODLevel = 0;

	UPROPERTY()
	int32 LODBias = 0;

	/** Used in LOD calculation to modify the distance, i.e. increasing the value will make lower poly LODs transition closer to the camera. */
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (UIMin = "0", DisplayName = "LOD Distance Factor", EditCondition = "LODMode == ENiagaraMeshLODMode::ByComponentBounds || LODMode == ENiagaraMeshLODMode::ComponentOrigin || LODMode == ENiagaraMeshLODMode::PerParticle", EditConditionHides))
	float LODDistanceFactor = 1.0f;

	/**
	When enabled you can restrict the LOD range we consider for LOD calculation.
	This can be useful to reduce the performance impact, as it reduces the number of draw calls required.
	*/
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (DisplayName = "Use LOD Range", EditCondition = "LODMode == ENiagaraMeshLODMode::PerParticle", EditConditionHides))
	bool bUseLODRange = false;

	/** Used to restrict the range of LODs we include when dynamically calculating the LOD level. */
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (DisplayName = "LOD Range", EditCondition = "bUseLODRange && LODMode == ENiagaraMeshLODMode::PerParticle", EditConditionHides))
	FIntVector2 LODRange;

	/** Resolve renderable mesh. */
	NIAGARA_API FNiagaraRenderableMeshPtr ResolveRenderableMesh(const FNiagaraEmitterInstance* EmitterInstance) const;

	/** Is the renderable mesh potentially valid or not. */
	NIAGARA_API bool HasValidRenderableMesh() const;
};
