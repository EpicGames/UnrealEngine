// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraGPUSortInfo.h"
#include "NiagaraMeshRendererMeshProperties.h"
#include "NiagaraRenderableMeshArrayInterface.h"
#include "NiagaraRenderableMeshInterface.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraParameterBinding.h"
#include "NiagaraMeshRendererProperties.generated.h"

class UMaterialInstanceConstant;
class FNiagaraEmitterInstance;
class FNiagaraSystemInstance;
class FAssetThumbnailPool;
class SWidget;
class UStaticMesh;

/** This enum decides how a mesh particle will orient its "facing" axis relative to camera. Must keep these in sync with NiagaraMeshVertexFactory.ush*/
UENUM()
enum class ENiagaraMeshFacingMode : uint8
{
	/** Ignores the camera altogether. The mesh aligns its local-space X-axis with the particles' local-space X-axis, after transforming by the Particles.Transform vector (if it exists).*/
	Default = 0,
	/** The mesh aligns it's local-space X-axis with the particle's Particles.Velocity vector.*/
	Velocity,
	/** Has the mesh local-space X-axis point towards the camera's position.*/
	CameraPosition,
	/** Has the mesh local-space X-axis point towards the closest point on the camera view plane.*/
	CameraPlane
};

UENUM()
enum class ENiagaraMeshLockedAxisSpace : uint8
{
	/** The locked axis is in the emitter's local space if the emitter is marked as local-space, or in world space otherwise */
	Simulation,
	/** The locked axis is in world space */
	World,
	/** The locked axis is in the emitter's local space */
	Local
};

USTRUCT()
struct FNiagaraMeshMICOverride
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMaterialInterface> OriginalMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceConstant> ReplacementMaterial;
};

USTRUCT()
struct FNiagaraMeshMaterialOverride
{
	GENERATED_USTRUCT_BODY()
public:
	NIAGARA_API FNiagaraMeshMaterialOverride();

	/** Used to upgrade a serialized FNiagaraParameterStore property to our own struct */
	NIAGARA_API bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	/** Use this UMaterialInterface if set to a valid value. This will be subordinate to UserParamBinding if it is set to a valid user variable.*/
	UPROPERTY(EditAnywhere, Category = "Material", meta = (EditCondition = "bOverrideMaterials"))
	TObjectPtr<UMaterialInterface> ExplicitMat;

	/** Use the UMaterialInterface bound to this user variable if it is set to a valid value. If this is bound to a valid value and ExplicitMat is also set, UserParamBinding wins.*/
	UPROPERTY(EditAnywhere, Category = "Material", meta = (EditCondition = "bOverrideMaterials"))
	FNiagaraUserParameterBinding UserParamBinding;

	bool operator==(const FNiagaraMeshMaterialOverride& Other)const
	{
		return UserParamBinding == Other.UserParamBinding && ExplicitMat == Other.ExplicitMat;
	}	
};

template<>
struct TStructOpsTypeTraits<FNiagaraMeshMaterialOverride> : public TStructOpsTypeTraitsBase2<FNiagaraMeshMaterialOverride>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
		WithIdenticalViaEquality = true
	};
};

namespace ENiagaraMeshVFLayout
{
	enum Type
	{
		Position,
		Velocity,
		Color,
		Scale,
		Rotation,
		MaterialRandom,
		NormalizedAge,
		CustomSorting,
		SubImage,
		DynamicParam0,
		DynamicParam1,
		DynamicParam2,
		DynamicParam3,
		CameraOffset,

		Num_Default,

		// The remaining layout params aren't needed unless accurate motion vectors are required
		PrevPosition = Num_Default,
		PrevScale,
		PrevRotation,
		PrevCameraOffset,
		PrevVelocity,

		Num_Max,
	};
};

UCLASS(editinlinenew, meta = (DisplayName = "Mesh Renderer", SupportsStateless), MinimalAPI)
class UNiagaraMeshRendererProperties : public UNiagaraRendererProperties, public INiagaraRenderableMeshArrayInterface
{
public:
	GENERATED_BODY()

	NIAGARA_API UNiagaraMeshRendererProperties();

	//UObject Interface
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void Serialize(FArchive& Ar) override;
	NIAGARA_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	NIAGARA_API virtual void RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
	NIAGARA_API virtual void RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
#endif// WITH_EDITORONLY_DATA
	//UObject Interface END

	static NIAGARA_API void InitCDOPropertiesAfterModuleStartup();

	//~ UNiagaraRendererProperties interface
	NIAGARA_API virtual FNiagaraRenderer* CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) override;
	NIAGARA_API virtual class FNiagaraBoundsCalculator* CreateBoundsCalculator() override;
	NIAGARA_API virtual void GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const override;
	NIAGARA_API virtual void GetUsedMeshes(const FNiagaraEmitterInstance* InEmitter, TArray<UObject*>& OutMeshes) const override;
	NIAGARA_API virtual void GetStreamingMeshInfo(const FBoxSphereBounds& OwnerBounds, const FNiagaraEmitterInstance* InEmitter, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
	NIAGARA_API virtual const FVertexFactoryType* GetVertexFactoryType() const override;
	virtual bool IsSimTargetSupported(ENiagaraSimTarget InSimTarget) const override { return true; };
	NIAGARA_API virtual bool PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore) override;
	NIAGARA_API virtual void CollectPSOPrecacheData(const FNiagaraEmitterInstance* InEmitter, FPSOPrecacheParamsList& OutParams) const override;

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual const TArray<FNiagaraVariable>& GetOptionalAttributes() override;
	NIAGARA_API virtual void GetAdditionalVariables(TArray<FNiagaraVariableBase>& OutArray) const override;
	NIAGARA_API virtual	void GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	NIAGARA_API virtual	void GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	NIAGARA_API virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const override;
	NIAGARA_API virtual void GetRendererMaterialUsageFeedback(TArray<FNiagaraRendererFeedback>& OutFeedback) const override;
	NIAGARA_API void CheckMaterialUsage();
	NIAGARA_API virtual TArray<FNiagaraVariable> GetBoundAttributes() const override;
#endif // WITH_EDITORONLY_DATA
	NIAGARA_API virtual void CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData) override;

#if WITH_EDITOR
	// Old function should not be called by user code but is in some instances
	NIAGARA_API void OnMeshChanged();
	// Internal API call for handling mesh changes (reimport / modification / etc)
	NIAGARA_API static void OnMeshChanged(UObject* MeshObject);
	// Internal API to remove all of the mesh changed handlers
	NIAGARA_API static void RemoveMeshChangedHandlers();
#endif

	NIAGARA_API void UpdateMICs();

	virtual ENiagaraRendererSourceDataMode GetCurrentSourceMode() const override { return SourceMode; }
	//UNiagaraRendererProperties Interface END

	NIAGARA_API void ApplyMaterialOverrides(const FNiagaraEmitterInstance* EmitterInstance, TArray<UMaterialInterface*>& InOutMaterials) const;

	//INiagaraRenderableMeshArrayInterface BEGIN
	virtual void ForEachMesh(FNiagaraSystemInstance* SystemInstance, TFunction<void(int32)> NumMeshesDelegate, TFunction<void(const FNiagaraMeshRendererMeshProperties&)> IterateDelegate) const override;
	//INiagaraRenderableMeshArrayInterface END

	const INiagaraRenderableMeshArrayInterface* GetMeshArrayInterface(const FNiagaraEmitterInstance* EmitterInstance) const;

	/**
	 * The static mesh(es) to be instanced when rendering mesh particles.
	 *
	 * NOTES:
	 * - If "Override Material" is not specified, the mesh's material is used. Override materials must have the Niagara Mesh Particles flag checked.
	 * - If "Enable Mesh Flipbook" is specified, this mesh is assumed to be the first frame of the flipbook.
	 */
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", meta = (EditCondition = "!bEnableMeshFlipbook", EditConditionHides))
	TArray<FNiagaraMeshRendererMeshProperties> Meshes;

	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", meta = (EditCondition = "!bEnableMeshFlipbook", EditConditionHides))
	FNiagaraParameterBinding MeshesBinding;

	/** Whether or not to draw a single element for the Emitter or to draw the particles.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering")
	ENiagaraRendererSourceDataMode SourceMode = ENiagaraRendererSourceDataMode::Particles;

	/** Determines how we sort the particles prior to rendering.*/
	UPROPERTY(EditAnywhere, Category = "Sorting")
	ENiagaraSortMode SortMode = ENiagaraSortMode::None;

	/** Sort precision to use when sorting is active. */
	UPROPERTY(EditAnywhere, Category = "Sorting", meta = (EditCondition = "SortMode != ENiagaraSortMode::None", EditConditionHides, DisplayAfter="bSortOnlyWhenTranslucent"))
	ENiagaraRendererSortPrecision SortPrecision = ENiagaraRendererSortPrecision::Default;

	/**
	Gpu simulations run at different points in the frame depending on what features are used, i.e. depth buffer, distance fields, etc.
	Opaque materials will run latent when these features are used.
	Translucent materials can choose if they want to use this frames or the previous frames data to match opaque draws.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Mesh Rendering")
	ENiagaraRendererGpuTranslucentLatency GpuTranslucentLatency = ENiagaraRendererGpuTranslucentLatency::ProjectDefault;

	/** Whether or not to use the OverrideMaterials array instead of the mesh's existing materials.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", DisplayName="Enable Material Overrides")
	uint32 bOverrideMaterials : 1;

	/** If true, the particles are only sorted when using a translucent material. */
	UPROPERTY(EditAnywhere, Category = "Sorting", meta = (EditCondition = "SortMode != ENiagaraSortMode::None", EditConditionHides))
	uint32 bSortOnlyWhenTranslucent : 1;

	/** If true, blends the sub-image UV lookup with its next adjacent member using the fractional part of the SubImageIndex float value as the linear interpolation factor.*/
	UPROPERTY(EditAnywhere, Category = "SubUV", meta = (DisplayName = "Sub UV Blending Enabled"))
	uint32 bSubImageBlend : 1;

	/**
	Enables frustum culling of individual mesh particles.
	GPU Scene will already perform per particle culling for meshes renderers, therefore enabling this may result in redundant work.
	*/
	UPROPERTY(EditAnywhere, Category = "Visibility")
	uint32 bEnableFrustumCulling : 1;

	/** Enables frustum culling of individual mesh particles */
	UPROPERTY(EditAnywhere, Category = "Visibility")
	uint32 bEnableCameraDistanceCulling : 1;

	/** When checked, will treat 'ParticleMesh' as the first frame of the flipbook, and will use the other mesh flipbook options to find the other frames */
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", meta = (DisplayAfter = "MeshBoundsScale"))
	uint32 bEnableMeshFlipbook : 1;

	/** If true and in a non-default facing mode, will lock facing direction to an arbitrary plane of rotation */
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering")
	uint32 bLockedAxisEnable : 1;

	/**
	When disabled the renderer will not cast shadows.
	The component controls if shadows are enabled, this flag allows you to disable the renderer casting shadows.
	*/
	UPROPERTY(EditAnywhere, Category = "Rendering")
	uint8 bCastShadows : 1 = 1;

#if WITH_EDITORONLY_DATA
	/**
	When disabled the mesh batches do not contribute to the hit proxy, i.e. they are not part of editor selection.
	*/
	UPROPERTY(EditAnywhere, Category = "Rendering")
	uint8 bIncludeInHitProxy : 1 = 1;
#endif

	/** The materials to be used instead of the StaticMesh's materials. Note that each material must have the Niagara Mesh Particles flag checked. If the ParticleMesh
	requires more materials than exist in this array or any entry in this array is set to None, we will use the ParticleMesh's existing Material instead.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", meta = (EditCondition = "bOverrideMaterials", EditConditionHides))
	TArray<FNiagaraMeshMaterialOverride> OverrideMaterials;

	UPROPERTY()
	TArray<FNiagaraMeshMICOverride> MICOverrideMaterials;

	/** When using SubImage lookups for particles, this variable contains the number of columns in X and the number of rows in Y.*/
	UPROPERTY(EditAnywhere, Category = "SubUV")
	FVector2D SubImageSize = FVector2D(1.0f, 1.0f);

	/** Arbitrary axis by which to lock facing rotations */
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", meta = (EditCondition = "bLockedAxisEnable", EditConditionHides, DisplayAfter="bLockedAxisEnable"))
	FVector LockedAxis = FVector(0.0f, 0.0f, 1.0f);

	/**
	Scale factor applied to all of the meshes bounds.
	This impacts distance based and per instance frustum culling.  Per instance frustum culling is enabled by default
	when GPU scene is enabled.  When using WPO with a material that may expand the mesh beyond the original bounds instances
	can be frustum culled incorrectly, this allows you to grow the bounds to avoid this issue.
	*/
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", AdvancedDisplay)
	FVector MeshBoundsScale = FVector::OneVector;

	/** Determines how the mesh orients itself relative to the camera. */
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering")
	ENiagaraMeshFacingMode FacingMode = ENiagaraMeshFacingMode::Default;

	/** Specifies what space the locked axis is in */
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", meta = (EditCondition = "bLockedAxisEnable", EditConditionHides, DisplayAfter = "LockedAxis"))
	ENiagaraMeshLockedAxisSpace LockedAxisSpace = ENiagaraMeshLockedAxisSpace::Simulation;

	UPROPERTY(EditAnywhere, Category = "Visibility", meta = (EditCondition = "bEnableCameraDistanceCulling", EditConditionHides, ClampMin = 0.0f))
	float MinCameraDistance;

	UPROPERTY(EditAnywhere, Category = "Visibility", meta = (EditCondition = "bEnableCameraDistanceCulling", EditConditionHides, ClampMin = 0.0f))
	float MaxCameraDistance = 1000.0f;

	/** If a render visibility tag is present, particles whose tag matches this value will be visible in this renderer. */
	UPROPERTY(EditAnywhere, Category = "Visibility")
	uint32 RendererVisibility = 0;

	/** Which attribute should we use for position when generating instanced meshes?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding PositionBinding;

	/** Which attribute should we use for color when generating instanced meshes?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding ColorBinding;

	/** Which attribute should we use for velocity when generating instanced meshes?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding VelocityBinding;

	/** Which attribute should we use for orienting meshes when generating instanced meshes?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding MeshOrientationBinding;

	/** Which attribute should we use for scale when generating instanced meshes?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding ScaleBinding;

	/** Which attribute should we use for sprite sub-image indexing when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding SubImageIndexBinding;

	/** Which attribute should we use for dynamic material parameters when generating instanced meshes?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterialBinding;

	/** Which attribute should we use for dynamic material parameters when generating instanced meshes?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterial1Binding;

	/** Which attribute should we use for dynamic material parameters when generating instanced meshes?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterial2Binding;

	/** Which attribute should we use for dynamic material parameters when generating instanced meshes?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterial3Binding;

	/** Which attribute should we use for material randoms when generating instanced meshes?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding MaterialRandomBinding;

	/** Which attribute should we use custom sorting of particles in this emitter. */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding CustomSortingBinding;

	/** Which attribute should we use for Normalized Age? */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding NormalizedAgeBinding;

	/** Which attribute should we use for camera offset when rendering meshes?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding CameraOffsetBinding;

	/** Which attribute should we use for the renderer visibility tag? */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RendererVisibilityTagBinding;

	/** Which attribute should we use to pick the element in the mesh array on the mesh renderer? */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding MeshIndexBinding;

	/** If this array has entries, we will create a MaterialInstanceDynamic per Emitter instance from Material and set the Material parameters using the Niagara simulation variables listed.*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraRendererMaterialParameters MaterialParameters;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FNiagaraMaterialAttributeBinding> MaterialParameterBindings_DEPRECATED;
#endif

	// The following bindings are not provided by the user, but are filled based on what other bindings are set to, and the value of bGenerateAccurateMotionVectors

	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevPositionBinding;
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevScaleBinding;
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevMeshOrientationBinding;
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevCameraOffsetBinding;
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevVelocityBinding;
	
#if WITH_EDITORONLY_DATA
	/**
	 * The static mesh to use for the first frame of the flipbook. Its name will also be used to find subsequent frames of a similar name.
	 * NOTE: The subsequent frames are expected to exist in the same content directory as the first frame of the flipbook, otherwise they
	 * will not be found or used.
	 */
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", meta = (EditCondition = "bEnableMeshFlipbook", EditConditionHides, DisplayAfter = "bEnableMeshFlipbook"))
	TObjectPtr<UStaticMesh> FirstFlipbookFrame;

	/**
	 * Provides the format of the suffix of the names of the static meshes when searching for flipbook frames. "{frame_number}" is used to mark
	 * where the frame number should appear in the suffix. If "Particle Mesh" contains this suffix, the number in its name will be treated as
	 * the starting frame index. Otherwise, it will assume "Particle Mesh" is frame number 0, and that subsequent frames follow this format,
	 * starting with frame number 1.
	 */
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", meta = (EditCondition = "bEnableMeshFlipbook && FirstFlipbookFrame != nullptr", EditConditionHides, DisplayAfter = "bEnableMeshFlipbook"))
	FString FlipbookSuffixFormat;

	/**
	* The number of digits to expect in the frame number of the flipbook page. A value of 1 will expect no leading zeros in the package names,
	* and can also be used for names with frame numbers that extend to 10 and beyond (Example: Frame_1, Frame_2, ..., Frame_10, Frame_11, etc.)
	*/
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", meta = (EditCondition = "bEnableMeshFlipbook && FirstFlipbookFrame != nullptr", EditConditionHides, ClampMin = 1, ClampMax = 10, NoSpinbox = true, DisplayAfter = "bEnableMeshFlipbook"))
	uint32 FlipbookSuffixNumDigits;

	/** The number of frames (static meshes) to be included in the flipbook. */
	UPROPERTY(EditAnywhere, Category = "Mesh Rendering", meta = (EditCondition = "bEnableMeshFlipbook && FirstFlipbookFrame != nullptr", EditConditionHides, ClampMin = 1, NoSpinbox = true, DisplayAfter = "bEnableMeshFlipbook"))
	uint32 NumFlipbookFrames;
#endif

	UPROPERTY()
	uint32 MaterialParamValidMask = 0;

	FNiagaraRendererLayout RendererLayoutWithCustomSorting;
	FNiagaraRendererLayout RendererLayoutWithoutCustomSorting;

protected:
	NIAGARA_API void InitBindings();
	NIAGARA_API void SetPreviousBindings(const FVersionedNiagaraEmitterBase& SrcEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	NIAGARA_API virtual void UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit = false) override;
	virtual bool NeedsMIDsForMaterials() const override { return MaterialParameters.HasAnyBindings(); }
#if WITH_EDITORONLY_DATA
	NIAGARA_API bool ChangeRequiresMeshListRebuild(const FProperty* Property);
	NIAGARA_API void RebuildMeshList();

	NIAGARA_API virtual FNiagaraVariable GetBoundAttribute(const FNiagaraVariableAttributeBinding* Binding) const override;
#endif

private:
	static NIAGARA_API TArray<TWeakObjectPtr<UNiagaraMeshRendererProperties>> MeshRendererPropertiesToDeferredInit;

#if WITH_EDITORONLY_DATA
	// These properties are deprecated and moved to FNiagaraMeshRendererMeshProperties
	UPROPERTY()
	TObjectPtr<UStaticMesh> ParticleMesh_DEPRECATED;

	UPROPERTY()
	FVector PivotOffset_DEPRECATED;

	UPROPERTY()
	ENiagaraMeshPivotOffsetSpace PivotOffsetSpace_DEPRECATED;
#endif
};
