// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "NiagaraMessageDataBase.generated.h"

UCLASS(MinimalAPI)
class UNiagaraMessageDataBase : public UObject
{
	GENERATED_BODY()

public:
	UNiagaraMessageDataBase() = default;

	virtual bool GetAllowDismissal() const { return false; }

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool Equals(const UNiagaraMessageDataBase* Other) const;
#endif
};
