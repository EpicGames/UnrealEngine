// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraMessageStore.generated.h"

class UNiagaraMessageDataBase;

USTRUCT()
struct FNiagaraMessageStore
{
	GENERATED_BODY();

#if WITH_EDITORONLY_DATA
	NIAGARA_API const TMap<FGuid, TObjectPtr<UNiagaraMessageDataBase>>& GetMessages() const;
	NIAGARA_API void SetMessages(const TMap<FGuid, TObjectPtr<UNiagaraMessageDataBase>>& InMessageKeyToMessageMap);
	NIAGARA_API void AddMessage(const FGuid& MessageKey, UNiagaraMessageDataBase* NewMessage);
	NIAGARA_API void RemoveMessage(const FGuid& MessageKey);
	NIAGARA_API void DismissMessage(const FGuid& MessageKey);
	NIAGARA_API bool IsMessageDismissed(const FGuid& MessageKey);
	NIAGARA_API bool HasDismissedMessages() const;
	NIAGARA_API void ClearDismissedMessages();

private:
	UPROPERTY()
	TMap<FGuid, TObjectPtr<UNiagaraMessageDataBase>> MessageKeyToMessageMap;

	UPROPERTY()
	TArray<FGuid> DismissedMessageKeys;
#endif
};

#if WITH_EDITORONLY_DATA

struct FNiagaraMessageSourceAndStore
{
public:
	NIAGARA_API FNiagaraMessageSourceAndStore(UObject& InSource, FNiagaraMessageStore& InStore);

	NIAGARA_API UObject* GetSource() const;

	NIAGARA_API FNiagaraMessageStore* GetStore() const;

private:
	TWeakObjectPtr<UObject> SourceWeak;
	FNiagaraMessageStore* Store;
};

#endif
