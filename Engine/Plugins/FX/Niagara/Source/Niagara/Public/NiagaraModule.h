// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "NiagaraTypes.h"
#include "Templates/SharedPointer.h"
#include "NiagaraCompilationTypes.h"
#include "NiagaraPerfBaseline.h"
#include "NiagaraDebuggerCommon.h"
#include "NiagaraScript.h"
#include "Templates/PimplPtr.h"

class FNiagaraWorldManager;
class UNiagaraEmitter;
struct FNiagaraVMExecutableData;
class UNiagaraScript;
class FNiagaraCompileOptions;
class FNiagaraCompileRequestDataBase;
class FNiagaraCompileRequestDuplicateDataBase;
class INiagaraMergeManager;
class INiagaraEditorOnlyDataUtilities;
class INiagaraVerseReflectionUtilities;
struct FNiagaraParameterStore;
class FCommonViewportClient;
class FNiagaraDebuggerClient;
struct FNiagaraSystemAsyncCompileResults;
struct FNiagaraAssetTagReference;
struct FNiagaraAssetTagDefinition;

extern NIAGARA_API int32 GEnableVerboseNiagaraChangeIdLogging;

/**
* Niagara module interface
*/
class INiagaraModule : public IModuleInterface
{
public:
#if WITH_EDITOR
	typedef TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> CompileRequestPtr;
	typedef TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> CompileRequestDuplicatePtr;
	typedef TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> GraphCachedDataPtr;

	DECLARE_DELEGATE_RetVal_ThreeParams(int32, FScriptCompiler, const FNiagaraCompileRequestDataBase*, const FNiagaraCompileRequestDuplicateDataBase*, const FNiagaraCompileOptions&);
	DECLARE_DELEGATE_RetVal_ThreeParams(TSharedPtr<FNiagaraVMExecutableData>, FCheckCompilationResult, int32, bool, FNiagaraScriptCompileMetrics&);
	DECLARE_DELEGATE_RetVal_TwoParams(CompileRequestPtr, FOnPrecompile, UObject*, FGuid);
	DECLARE_DELEGATE_RetVal_FiveParams(CompileRequestDuplicatePtr, FOnPrecompileDuplicate, const FNiagaraCompileRequestDataBase* /*OwningSystemRequestData*/, UNiagaraSystem* /*OwningSystem*/, UNiagaraEmitter* /*OwningEmitter*/, UNiagaraScript* /*TargetScript*/, FGuid /*Version*/);
	DECLARE_DELEGATE_RetVal_TwoParams(GraphCachedDataPtr, FOnCacheGraphTraversal, const UObject*, FGuid);

	DECLARE_DELEGATE_RetVal_ThreeParams(FNiagaraCompilationTaskHandle, FOnRequestCompileSystem, UNiagaraSystem*, bool, const ITargetPlatform*);
	DECLARE_DELEGATE_RetVal_FourParams(bool, FOnPollSystemCompile, FNiagaraCompilationTaskHandle, FNiagaraSystemAsyncCompileResults&, bool /*bWait*/, bool /*bPeek*/);
	DECLARE_DELEGATE_OneParam(FOnAbortSystemCompile, FNiagaraCompilationTaskHandle);

#endif
	DECLARE_DELEGATE_RetVal(void, FOnProcessQueue);

public:
	NIAGARA_API virtual void StartupModule()override;
	NIAGARA_API virtual void ShutdownModule()override;
	
	/** Get the instance of this module. */
	NIAGARA_API static INiagaraModule& Get();
	
	NIAGARA_API void ShutdownRenderingResources();
	
	NIAGARA_API void OnPostEngineInit();
	NIAGARA_API void OnPreExit();

	NIAGARA_API void OnWorldTickStart(UWorld* World, ELevelTick TickType, float DeltaSeconds);
	NIAGARA_API void OnBeginFrame();
	NIAGARA_API void OnPostGarbageCollect();
	NIAGARA_API void OnWorldBeginTearDown(UWorld* World);

	NIAGARA_API FDelegateHandle SetOnProcessShaderCompilationQueue(FOnProcessQueue InOnProcessQueue);
	NIAGARA_API void ResetOnProcessShaderCompilationQueue(FDelegateHandle DelegateHandle);
	NIAGARA_API void ProcessShaderCompilationQueue();

#if WITH_NIAGARA_DEBUGGER
	FNiagaraDebuggerClient* GetDebuggerClient() { return DebuggerClient.Get(); }
#endif

#if WITH_EDITOR
	NIAGARA_API const INiagaraMergeManager& GetMergeManager() const;

	NIAGARA_API void RegisterMergeManager(TSharedRef<INiagaraMergeManager> InMergeManager);

	NIAGARA_API void UnregisterMergeManager(TSharedRef<INiagaraMergeManager> InMergeManager);

	NIAGARA_API const INiagaraEditorOnlyDataUtilities& GetEditorOnlyDataUtilities() const;
	NIAGARA_API void RegisterEditorOnlyDataUtilities(TSharedRef<INiagaraEditorOnlyDataUtilities> InEditorOnlyDataUtilities);
	NIAGARA_API void UnregisterEditorOnlyDataUtilities(TSharedRef<INiagaraEditorOnlyDataUtilities> InEditorOnlyDataUtilities);

	NIAGARA_API const INiagaraVerseReflectionUtilities* GetVerseReflectionUtilities() const;
	NIAGARA_API void RegisterVerseReflectionUtilities(TSharedRef<INiagaraVerseReflectionUtilities> InUtilities);
	NIAGARA_API void UnregisterVerseReflectionUtilities(TSharedRef<INiagaraVerseReflectionUtilities> InUtilities);

	NIAGARA_API int32 StartScriptCompileJob(const FNiagaraCompileRequestDataBase* InCompileData, const FNiagaraCompileRequestDuplicateDataBase* InCompileDuplicateData, const FNiagaraCompileOptions& InCompileOptions);
	NIAGARA_API TSharedPtr<FNiagaraVMExecutableData> GetCompileJobResult(int32 JobID, bool bWait, FNiagaraScriptCompileMetrics& Metrics);

	NIAGARA_API FDelegateHandle RegisterScriptCompiler(FScriptCompiler ScriptCompiler);
	NIAGARA_API void UnregisterScriptCompiler(FDelegateHandle DelegateHandle);

	NIAGARA_API FDelegateHandle RegisterCompileResultDelegate(FCheckCompilationResult ResultDelegate);
	NIAGARA_API void UnregisterCompileResultDelegate(FDelegateHandle DelegateHandle);

	NIAGARA_API TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> Precompile(UObject* InObj, FGuid Version);
	NIAGARA_API TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> PrecompileDuplicate(
		const FNiagaraCompileRequestDataBase* OwningSystemRequestData,
		UNiagaraSystem* OwningSystem,
		UNiagaraEmitter* OwningEmitter,
		UNiagaraScript* TargetScript,
		FGuid TargetVersion);
	NIAGARA_API FDelegateHandle RegisterPrecompiler(FOnPrecompile PreCompiler);
	NIAGARA_API void UnregisterPrecompiler(FDelegateHandle DelegateHandle);
	NIAGARA_API FDelegateHandle RegisterPrecompileDuplicator(FOnPrecompileDuplicate PreCompileDuplicator);
	NIAGARA_API void UnregisterPrecompileDuplicator(FDelegateHandle DelegateHandle);

	NIAGARA_API TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CacheGraphTraversal(const UObject* InObj, FGuid Version);
	NIAGARA_API FDelegateHandle RegisterGraphTraversalCacher(FOnCacheGraphTraversal PreCompiler);
	NIAGARA_API void UnregisterGraphTraversalCacher(FDelegateHandle DelegateHandle);

	NIAGARA_API FNiagaraCompilationTaskHandle RequestCompileSystem(UNiagaraSystem* System, bool bForce, const ITargetPlatform* TargetPlatform);
	NIAGARA_API FDelegateHandle RegisterRequestCompileSystem(FOnRequestCompileSystem RequestCompileSystemCallback);
	NIAGARA_API void UnregisterRequestCompileSystem(FDelegateHandle DelegateHandle);

	NIAGARA_API bool PollSystemCompile(FNiagaraCompilationTaskHandle, FNiagaraSystemAsyncCompileResults&, bool /*bWait*/, bool /*bPeek*/);
	NIAGARA_API FDelegateHandle RegisterPollSystemCompile(FOnPollSystemCompile PollSystemCompileCallback);
	NIAGARA_API void UnregisterPollSystemCompile(FDelegateHandle DelegateHandle);

	NIAGARA_API void AbortSystemCompile(FNiagaraCompilationTaskHandle);
	NIAGARA_API FDelegateHandle RegisterAbortSystemCompile(FOnAbortSystemCompile AbortSystemCompileCallback);
	NIAGARA_API void UnregisterAbortSystemCompile(FDelegateHandle DelegateHandle);

	NIAGARA_API void OnAssetLoaded(UObject* Asset);
#endif

	static void RequestRefreshDataChannels() { bDataChannelRefreshRequested = true; }
	static NIAGARA_API void RefreshDataChannels();

	UE_DEPRECATED(5.7, "Niagara Asset Tags are deprecated in favor of the generic User Asset Tags. Please update your code as this function will be removed. ")
	NIAGARA_API const TArray<const FNiagaraAssetTagDefinition*>& GetInternalAssetTagDefinitions() const;
	UE_DEPRECATED(5.7, "Niagara Asset Tags are deprecated in favor of the generic User Asset Tags. Please update your code as this function will be removed. ")
	void RegisterInternalAssetTagDefinitions();
	
#if NIAGARA_PERF_BASELINES
	NIAGARA_API void GeneratePerfBaselines(TArray<UNiagaraEffectType*>& BaselinesToGenerate);

	NIAGARA_API bool ToggleStatPerfBaselines(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream = nullptr);
	NIAGARA_API int32 RenderStatPerfBaselines(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation = nullptr, const FRotator* ViewRotation = nullptr);
#endif

	inline static bool UseGlobalFXBudget() { return bUseGlobalFXBudget; }
	static NIAGARA_API void OnUseGlobalFXBudgetChanged(IConsoleVariable* Variable);

	inline static bool DataChannelsEnabled() { return bDataChannelsEnabled; }
	static NIAGARA_API void OnDataChannelsEnabledChanged(IConsoleVariable* Variable);

	inline static float GetGlobalSpawnCountScale() { return EngineGlobalSpawnCountScale; }
	inline static float GetGlobalSystemCountScale() { return EngineGlobalSystemCountScale; }

	static NIAGARA_API float EngineGlobalSpawnCountScale;
	static NIAGARA_API float EngineGlobalSystemCountScale;

	inline static const FNiagaraVariable&  GetVar_Engine_WorldDeltaTime() { return Engine_WorldDeltaTime; }
	inline static const FNiagaraVariable&  GetVar_Engine_DeltaTime() { return Engine_DeltaTime; }
	inline static const FNiagaraVariable&  GetVar_Engine_InvDeltaTime() { return Engine_InvDeltaTime; }
	inline static const FNiagaraVariable&  GetVar_Engine_Time() { return Engine_Time; }
	inline static const FNiagaraVariable&  GetVar_Engine_RealTime() { return Engine_RealTime; }
	inline static const FNiagaraVariable&  GetVar_Engine_QualityLevel() { return Engine_QualityLevel; }

	inline static const FNiagaraVariable&  GetVar_Engine_Owner_Position() { return Engine_Owner_Position; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_Velocity() { return Engine_Owner_Velocity; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_XAxis() { return Engine_Owner_XAxis; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_YAxis() { return Engine_Owner_YAxis; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_ZAxis() { return Engine_Owner_ZAxis; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_Scale() { return Engine_Owner_Scale; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_Rotation() { return Engine_Owner_Rotation; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_LWC_Tile() { return Engine_Owner_LWC_Tile; }
	inline static const FNiagaraVariable&  GetVar_Engine_ExecIndex() { return Engine_ExecIndex; }

	inline static const FNiagaraVariable&  GetVar_Engine_Owner_SystemLocalToWorld() { return Engine_Owner_SystemLocalToWorld; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_SystemWorldToLocal() { return Engine_Owner_SystemWorldToLocal; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_SystemLocalToWorldTransposed() { return Engine_Owner_SystemLocalToWorldTransposed; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_SystemWorldToLocalTransposed() { return Engine_Owner_SystemWorldToLocalTransposed; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_SystemLocalToWorldNoScale() { return Engine_Owner_SystemLocalToWorldNoScale; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_SystemWorldToLocalNoScale() { return Engine_Owner_SystemWorldToLocalNoScale; }

	inline static const FNiagaraVariable&  GetVar_Engine_Owner_TimeSinceRendered() { return Engine_Owner_TimeSinceRendered; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_LODDistance() { return Engine_Owner_LODDistance; }
	inline static const FNiagaraVariable&  GetVar_Engine_Owner_LODDistanceFraction() { return Engine_Owner_LODDistanceFraction; }

	inline static const FNiagaraVariable&  GetVar_Engine_Owner_ExecutionState() { return Engine_Owner_ExecutionState; }

	inline static const FNiagaraVariable&  GetVar_Engine_ExecutionCount() { return Engine_ExecutionCount; }
	inline static const FNiagaraVariable&  GetVar_Engine_Emitter_NumParticles() { return Engine_Emitter_NumParticles; }
	inline static const FNiagaraVariable&  GetVar_Engine_Emitter_SimulationPosition() { return Engine_Emitter_SimulationPosition; }
	inline static const FNiagaraVariable&  GetVar_Engine_Emitter_TotalSpawnedParticles() { return Engine_Emitter_TotalSpawnedParticles; }
	inline static const FNiagaraVariable&  GetVar_Engine_Emitter_SpawnCountScale() { return Engine_Emitter_SpawnCountScale; }
	inline static const FNiagaraVariable&  GetVar_Engine_Emitter_InstanceSeed() { return Engine_Emitter_InstanceSeed; }
	inline static const FNiagaraVariable&  GetVar_Engine_Emitter_ID() { return Engine_Emitter_ID; }
	inline static const FNiagaraVariable&  GetVar_Engine_System_TickCount() { return Engine_System_TickCount; }
	inline static const FNiagaraVariable&  GetVar_Engine_System_NumEmittersAlive() { return Engine_System_NumEmittersAlive; }
	inline static const FNiagaraVariable&  GetVar_Engine_System_SignificanceIndex() { return Engine_System_SignificanceIndex; }
	inline static const FNiagaraVariable&  GetVar_Engine_System_RandomSeed() { return Engine_System_RandomSeed; }

	inline static const FNiagaraVariable& GetVar_Engine_System_CurrentTimeStep() { return Engine_System_CurrentTimeStep; }
	inline static const FNiagaraVariable& GetVar_Engine_System_NumTimeSteps() { return Engine_System_NumTimeSteps; }
	inline static const FNiagaraVariable& GetVar_Engine_System_TimeStepFraction() { return Engine_System_TimeStepFraction; }
	inline static const FNiagaraVariable& GetVar_Engine_System_NumParticles() { return Engine_System_NumParticles; }

	inline static const FNiagaraVariable&  GetVar_Engine_System_NumEmitters() { return Engine_System_NumEmitters; }
	inline static const FNiagaraVariable&  GetVar_Engine_NumSystemInstances() { return Engine_NumSystemInstances; }

	inline static const FNiagaraVariable&  GetVar_Engine_GlobalSpawnCountScale() { return Engine_GlobalSpawnCountScale; }
	inline static const FNiagaraVariable&  GetVar_Engine_GlobalSystemScale() { return Engine_GlobalSystemScale; }

	inline static const FNiagaraVariable&  GetVar_Engine_System_Age() { return Engine_System_Age; }
	inline static const FNiagaraVariable&  GetVar_Emitter_Age() { return Emitter_Age; }
	inline static const FNiagaraVariable&  GetVar_Emitter_LocalSpace() { return Emitter_LocalSpace; }
	inline static const FNiagaraVariable&  GetVar_Emitter_Determinism() { return Emitter_Determinism; }
	inline static const FNiagaraVariable&  GetVar_Emitter_InterpolatedSpawn() { return Emitter_InterpolatedSpawn; }
	inline static const FNiagaraVariable&  GetVar_Emitter_OverrideGlobalSpawnCountScale() { return Emitter_OverrideGlobalSpawnCountScale; }
	inline static const FNiagaraVariable&  GetVar_Emitter_RandomSeed() { return Emitter_RandomSeed; }
	inline static const FNiagaraVariable&  GetVar_Emitter_SpawnRate() { return Emitter_SpawnRate; }
	inline static const FNiagaraVariable&  GetVar_Emitter_SpawnInterval() { return Emitter_SpawnInterval; }
	inline static const FNiagaraVariable&  GetVar_Emitter_SimulationTarget() { return Emitter_SimulationTarget; }
	inline static const FNiagaraVariable&  GetVar_ScriptUsage() { return ScriptUsage; }
	inline static const FNiagaraVariable&  GetVar_ScriptContext() { return ScriptContext; }
	inline static const FNiagaraVariable&  GetVar_FunctionDebugState() { return FunctionDebugState; }
	inline static const FNiagaraVariable&  GetVar_Emitter_InterpSpawnStartDt() { return Emitter_InterpSpawnStartDt; }
	inline static const FNiagaraVariable&  GetVar_Emitter_SpawnGroup() { return Emitter_SpawnGroup; }

	inline static const FNiagaraVariable&  GetVar_Particles_UniqueID() { return Particles_UniqueID; }
	inline static const FNiagaraVariable&  GetVar_Particles_ID() { return Particles_ID; }
	inline static const FNiagaraVariable&  GetVar_Particles_Position() { return Particles_Position; }
	inline static const FNiagaraVariable&  GetVar_Particles_Velocity() { return Particles_Velocity; }
	inline static const FNiagaraVariable&  GetVar_Particles_Color() { return Particles_Color; }
	inline static const FNiagaraVariable&  GetVar_Particles_SpriteRotation() { return Particles_SpriteRotation; }
	inline static const FNiagaraVariable&  GetVar_Particles_Age() { return Particles_Age; }
	inline static const FNiagaraVariable&  GetVar_Particles_NormalizedAge() { return Particles_NormalizedAge; }
	inline static const FNiagaraVariable&  GetVar_Particles_SpriteSize() { return Particles_SpriteSize; }
	inline static const FNiagaraVariable&  GetVar_Particles_SpriteFacing() { return Particles_SpriteFacing; }
	inline static const FNiagaraVariable&  GetVar_Particles_SpriteAlignment() { return Particles_SpriteAlignment; }
	inline static const FNiagaraVariable&  GetVar_Particles_SubImageIndex() { return Particles_SubImageIndex; }
	inline static const FNiagaraVariable&  GetVar_Particles_DynamicMaterialParameter() { return Particles_DynamicMaterialParameter; }
	inline static const FNiagaraVariable&  GetVar_Particles_DynamicMaterialParameter1() { return Particles_DynamicMaterialParameter1; }
	inline static const FNiagaraVariable&  GetVar_Particles_DynamicMaterialParameter2() { return Particles_DynamicMaterialParameter2; }
	inline static const FNiagaraVariable&  GetVar_Particles_DynamicMaterialParameter3() { return Particles_DynamicMaterialParameter3; }
	inline static const FNiagaraVariable&  GetVar_Particles_Scale() { return Particles_Scale; }
	inline static const FNiagaraVariable&  GetVar_Particles_Lifetime() { return Particles_Lifetime; }
	inline static const FNiagaraVariable&  GetVar_Particles_MeshOrientation() { return Particles_MeshOrientation; }
	inline static const FNiagaraVariable&  GetVar_Particles_UVScale() { return Particles_UVScale; }
	inline static const FNiagaraVariable&  GetVar_Particles_PivotOffset() { return Particles_PivotOffset; }
	inline static const FNiagaraVariable&  GetVar_Particles_CameraOffset() { return Particles_CameraOffset; }
	inline static const FNiagaraVariable&  GetVar_Particles_MaterialRandom() { return Particles_MaterialRandom; }
	inline static const FNiagaraVariable&  GetVar_Particles_LightRadius() { return Particles_LightRadius; }
	inline static const FNiagaraVariable&  GetVar_Particles_LightExponent() { return Particles_LightExponent; }
	inline static const FNiagaraVariable&  GetVar_Particles_LightEnabled() { return Particles_LightEnabled; }
	inline static const FNiagaraVariable&  GetVar_Particles_LightVolumetricScattering() { return Particles_LightVolumetricScattering; }
	inline static const FNiagaraVariable&  GetVar_Particles_LightSpecularScale() { return Particles_LightSpecularScale; }
	inline static const FNiagaraVariable&  GetVar_Particles_LightDiffuseScale() { return Particles_LightDiffuseScale; }
	inline static const FNiagaraVariable&  GetVar_Particles_RibbonID() { return Particles_RibbonID; }
	inline static const FNiagaraVariable&  GetVar_Particles_RibbonWidth() { return Particles_RibbonWidth; }
	inline static const FNiagaraVariable&  GetVar_Particles_RibbonTwist() { return Particles_RibbonTwist; }
	inline static const FNiagaraVariable&  GetVar_Particles_RibbonFacing() { return Particles_RibbonFacing; }
	inline static const FNiagaraVariable&  GetVar_Particles_RibbonLinkOrder() { return Particles_RibbonLinkOrder; }
	inline static const FNiagaraVariable&  GetVar_Particles_RibbonUVDistance() { return Particles_RibbonUVDistance; }
	inline static const FNiagaraVariable&  GetVar_Particles_RibbonU0Override() { return Particles_RibbonU0Override; }
	inline static const FNiagaraVariable&  GetVar_Particles_RibbonV0RangeOverride() { return Particles_RibbonV0RangeOverride; }
	inline static const FNiagaraVariable&  GetVar_Particles_RibbonU1Override() { return Particles_RibbonU1Override; }
	inline static const FNiagaraVariable&  GetVar_Particles_RibbonV1RangeOverride() { return Particles_RibbonV1RangeOverride; }
	inline static const FNiagaraVariable&  GetVar_Particles_VisibilityTag() { return Particles_VisibilityTag; }
	inline static const FNiagaraVariable&  GetVar_Particles_MeshIndex() { return Particles_MeshIndex; }
	inline static const FNiagaraVariable&  GetVar_Particles_ComponentsEnabled() { return Particles_ComponentsEnabled; }
	
	inline static const FNiagaraVariable&  GetVar_DataInstance_Alive() { return DataInstance_Alive; }
	inline static const FNiagaraVariable&  GetVar_BeginDefaults() { return Translator_BeginDefaults; }
	inline static const FNiagaraVariable&  GetVar_CallID() { return Translator_CallID; }
	
	FOnProcessQueue OnProcessQueue;
	
#if WITH_EDITORONLY_DATA
	TSharedPtr<INiagaraMergeManager> MergeManager;
	TSharedPtr<INiagaraEditorOnlyDataUtilities> EditorOnlyDataUtilities;
	TSharedPtr<INiagaraVerseReflectionUtilities> VerseReflectionUtilities;

	FScriptCompiler ScriptCompilerDelegate;
	FCheckCompilationResult CompilationResultDelegate;
	FOnPrecompile PrecompileDelegate;
	FOnPrecompileDuplicate PrecompileDuplicateDelegate;
	FOnCacheGraphTraversal GraphTraversalCacheDelegate;
	FOnRequestCompileSystem RequestCompileSystemDelegate;
	FOnPollSystemCompile PollSystemCompileDelegate;
	FOnAbortSystemCompile AbortSystemCompileDelegate;
#endif

	static NIAGARA_API int32 EngineEffectsQuality;

	static NIAGARA_API bool bUseGlobalFXBudget;
	static NIAGARA_API bool bDataChannelsEnabled;

	UE_DEPRECATED(5.7, "Niagara Asset Tags are deprecated in favor of the generic User Asset Tags. Access the new User Asset Tags using FMetaData functions. Please update your code as this property will be removed.")
	static const FNiagaraAssetTagDefinition LightweightTagDefinition;
	UE_DEPRECATED(5.7, "Niagara Asset Tags are deprecated in favor of the generic User Asset Tags. Access the new User Asset Tags using FMetaData functions. Please update your code as this property will be removed.")
	static const FNiagaraAssetTagDefinition TemplateTagDefinition;
	UE_DEPRECATED(5.7, "Niagara Asset Tags are deprecated in favor of the generic User Asset Tags. Access the new User Asset Tags using FMetaData functions. Please update your code as this property will be removed.")
	static const FNiagaraAssetTagDefinition LearningContentTagDefinition;
	UE_DEPRECATED(5.7, "Niagara Asset Tags are deprecated in favor of the generic User Asset Tags. Access the new User Asset Tags using FMetaData functions. Please update your code as this property will be removed.")
	static NIAGARA_API const FNiagaraAssetTagDefinition HiddenAssetTagDefinition;
	UE_DEPRECATED(5.7, "Niagara Asset Tags are deprecated in favor of the generic User Asset Tags. Access the new User Asset Tags using FMetaData functions. Please update your code as this property will be removed.")
	static NIAGARA_API const FNiagaraAssetTagDefinition DeprecatedTagDefinition;

	UE_DEPRECATED(5.7, "Niagara Asset Tags are deprecated in favor of the generic User Asset Tags. Access the new User Asset Tags using FMetaData functions. Please update your code as this property will be removed.")
	TArray<const FNiagaraAssetTagDefinition*> InternalAssetTagDefinitions;
private:
	static NIAGARA_API FNiagaraVariable Engine_WorldDeltaTime;
	static NIAGARA_API FNiagaraVariable Engine_DeltaTime;
	static NIAGARA_API FNiagaraVariable Engine_InvDeltaTime;
	static NIAGARA_API FNiagaraVariable Engine_Time; 
	static NIAGARA_API FNiagaraVariable Engine_RealTime; 
	static NIAGARA_API FNiagaraVariable Engine_QualityLevel; 

	static NIAGARA_API FNiagaraVariable Engine_Owner_Position;
	static NIAGARA_API FNiagaraVariable Engine_Owner_Velocity;
	static NIAGARA_API FNiagaraVariable Engine_Owner_XAxis;
	static NIAGARA_API FNiagaraVariable Engine_Owner_YAxis;
	static NIAGARA_API FNiagaraVariable Engine_Owner_ZAxis;
	static NIAGARA_API FNiagaraVariable Engine_Owner_Scale;
	static NIAGARA_API FNiagaraVariable Engine_Owner_Rotation;
	static NIAGARA_API FNiagaraVariable Engine_Owner_LWC_Tile;
	static NIAGARA_API FNiagaraVariable Engine_ExecIndex;

	static NIAGARA_API FNiagaraVariable Engine_Owner_SystemLocalToWorld;
	static NIAGARA_API FNiagaraVariable Engine_Owner_SystemWorldToLocal;
	static NIAGARA_API FNiagaraVariable Engine_Owner_SystemLocalToWorldTransposed;
	static NIAGARA_API FNiagaraVariable Engine_Owner_SystemWorldToLocalTransposed;
	static NIAGARA_API FNiagaraVariable Engine_Owner_SystemLocalToWorldNoScale;
	static NIAGARA_API FNiagaraVariable Engine_Owner_SystemWorldToLocalNoScale;

	static NIAGARA_API FNiagaraVariable Engine_Owner_TimeSinceRendered;
	static NIAGARA_API FNiagaraVariable Engine_Owner_LODDistance;
	static NIAGARA_API FNiagaraVariable Engine_Owner_LODDistanceFraction;
	
	static NIAGARA_API FNiagaraVariable Engine_Owner_ExecutionState;

	static NIAGARA_API FNiagaraVariable Engine_ExecutionCount;
	static NIAGARA_API FNiagaraVariable Engine_Emitter_NumParticles;
	static NIAGARA_API FNiagaraVariable Engine_Emitter_SimulationPosition;
	static NIAGARA_API FNiagaraVariable Engine_Emitter_TotalSpawnedParticles;
	static NIAGARA_API FNiagaraVariable Engine_Emitter_SpawnCountScale;
	static NIAGARA_API FNiagaraVariable Engine_System_TickCount;
	static NIAGARA_API FNiagaraVariable Engine_System_NumEmittersAlive;
	static NIAGARA_API FNiagaraVariable Engine_System_SignificanceIndex;
	static NIAGARA_API FNiagaraVariable Engine_System_RandomSeed;
	
	static NIAGARA_API FNiagaraVariable Engine_System_CurrentTimeStep;
	static NIAGARA_API FNiagaraVariable Engine_System_NumTimeSteps;
	static NIAGARA_API FNiagaraVariable Engine_System_TimeStepFraction;
	static NIAGARA_API FNiagaraVariable Engine_System_NumParticles;
	
	static NIAGARA_API FNiagaraVariable Engine_System_NumEmitters;
	static NIAGARA_API FNiagaraVariable Engine_NumSystemInstances;

	static NIAGARA_API FNiagaraVariable Engine_GlobalSpawnCountScale;
	static NIAGARA_API FNiagaraVariable Engine_GlobalSystemScale;

	static NIAGARA_API FNiagaraVariable Engine_System_Age;
	static NIAGARA_API FNiagaraVariable Emitter_Age;
	static NIAGARA_API FNiagaraVariable Emitter_LocalSpace;
	static NIAGARA_API FNiagaraVariable Emitter_Determinism;
	static NIAGARA_API FNiagaraVariable Emitter_InterpolatedSpawn;
	static NIAGARA_API FNiagaraVariable Emitter_OverrideGlobalSpawnCountScale;
	static NIAGARA_API FNiagaraVariable Emitter_SimulationTarget;
	static NIAGARA_API FNiagaraVariable Emitter_RandomSeed;
	static NIAGARA_API FNiagaraVariable Engine_Emitter_InstanceSeed;
	static NIAGARA_API FNiagaraVariable Engine_Emitter_ID;
	static NIAGARA_API FNiagaraVariable Emitter_SpawnRate;
	static NIAGARA_API FNiagaraVariable Emitter_SpawnInterval;
	static NIAGARA_API FNiagaraVariable Emitter_InterpSpawnStartDt;
	static NIAGARA_API FNiagaraVariable Emitter_SpawnGroup;

	static NIAGARA_API FNiagaraVariable Particles_UniqueID;
	static NIAGARA_API FNiagaraVariable Particles_ID;
	static NIAGARA_API FNiagaraVariable Particles_Position;
	static NIAGARA_API FNiagaraVariable Particles_Velocity;
	static NIAGARA_API FNiagaraVariable Particles_Color;
	static NIAGARA_API FNiagaraVariable Particles_SpriteRotation;
	static NIAGARA_API FNiagaraVariable Particles_Age;
	static NIAGARA_API FNiagaraVariable Particles_NormalizedAge;
	static NIAGARA_API FNiagaraVariable Particles_SpriteSize;
	static NIAGARA_API FNiagaraVariable Particles_SpriteFacing;
	static NIAGARA_API FNiagaraVariable Particles_SpriteAlignment;
	static NIAGARA_API FNiagaraVariable Particles_SubImageIndex;
	static NIAGARA_API FNiagaraVariable Particles_DynamicMaterialParameter;
	static NIAGARA_API FNiagaraVariable Particles_DynamicMaterialParameter1;
	static NIAGARA_API FNiagaraVariable Particles_DynamicMaterialParameter2;
	static NIAGARA_API FNiagaraVariable Particles_DynamicMaterialParameter3;
	static NIAGARA_API FNiagaraVariable Particles_Scale;
	static NIAGARA_API FNiagaraVariable Particles_Lifetime;
	static NIAGARA_API FNiagaraVariable Particles_MeshOrientation;
	static NIAGARA_API FNiagaraVariable Particles_VisibilityTag;
	static NIAGARA_API FNiagaraVariable Particles_MeshIndex;
	static NIAGARA_API FNiagaraVariable Particles_UVScale;
	static NIAGARA_API FNiagaraVariable Particles_PivotOffset;
	static NIAGARA_API FNiagaraVariable Particles_CameraOffset;
	static NIAGARA_API FNiagaraVariable Particles_MaterialRandom;
	static NIAGARA_API FNiagaraVariable Particles_LightRadius;
	static NIAGARA_API FNiagaraVariable Particles_LightExponent;
	static NIAGARA_API FNiagaraVariable Particles_LightEnabled;
	static NIAGARA_API FNiagaraVariable Particles_LightVolumetricScattering;
	static NIAGARA_API FNiagaraVariable Particles_LightSpecularScale;
	static NIAGARA_API FNiagaraVariable Particles_LightDiffuseScale;
	static NIAGARA_API FNiagaraVariable Particles_RibbonID;
	static NIAGARA_API FNiagaraVariable Particles_RibbonWidth;
	static NIAGARA_API FNiagaraVariable Particles_RibbonTwist;
	static NIAGARA_API FNiagaraVariable Particles_RibbonFacing;
	static NIAGARA_API FNiagaraVariable Particles_RibbonLinkOrder;
	static NIAGARA_API FNiagaraVariable Particles_ComponentsEnabled;
	static NIAGARA_API FNiagaraVariable Particles_RibbonUVDistance;
	static NIAGARA_API FNiagaraVariable Particles_RibbonU0Override;
	static NIAGARA_API FNiagaraVariable Particles_RibbonV0RangeOverride;
	static NIAGARA_API FNiagaraVariable Particles_RibbonU1Override;
	static NIAGARA_API FNiagaraVariable Particles_RibbonV1RangeOverride;

	static NIAGARA_API FNiagaraVariable ScriptUsage;
	static NIAGARA_API FNiagaraVariable ScriptContext;
	static NIAGARA_API FNiagaraVariable FunctionDebugState;
	static NIAGARA_API FNiagaraVariable DataInstance_Alive;
	static NIAGARA_API FNiagaraVariable Translator_BeginDefaults;
	static NIAGARA_API FNiagaraVariable Translator_CallID;

#if NIAGARA_PERF_BASELINES
	TUniquePtr<FNiagaraPerfBaselineHandler> BaselineHandler;
#endif

#if WITH_NIAGARA_DEBUGGER
	TPimplPtr<FNiagaraDebuggerClient> DebuggerClient;
#endif

	FDelegateHandle OnCVarUnregisteredHandle;

	static std::atomic<bool> bDataChannelRefreshRequested;
};

