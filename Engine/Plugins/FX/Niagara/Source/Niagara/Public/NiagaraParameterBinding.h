// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"

#include "NiagaraCommon.h"
#include "NiagaraParameterBinding.generated.h"

// Binding usage flags
// Note that Emitter / Particle bindings are only allowed if the binding variable is outered to a UNiagaraEmitter
enum class ENiagaraParameterBindingUsage
{
	User		= 1 << 0,		// Can we bind to user parameters
	System		= 1 << 1,		// Can we bind to system parameters
	Emitter		= 1 << 2,		// Can we bind to emitter parameters
	Particle	= 1 << 3,		// Can we bind to particle parameters
	StaticVariable = 1 << 4,	// Can we bind to static variables

	NotParticle	= StaticVariable | User | System | Emitter,
	All			= StaticVariable | User | System | Emitter | Particle,
};

ENUM_CLASS_FLAGS(ENiagaraParameterBindingUsage)

USTRUCT()
struct FNiagaraParameterBinding
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	virtual ~FNiagaraParameterBinding() = default;

	NIAGARA_API void OnRenameEmitter(FStringView EmitterName);
	NIAGARA_API void OnRenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, FStringView EmitterName);
	NIAGARA_API void OnRemoveVariable(const FNiagaraVariableBase& OldVariable, FStringView EmitterName);

	void SetUsage(ENiagaraParameterBindingUsage InUsage) { BindingUsage = InUsage; }
	void SetAllowedDataInterfaces(TArray<UClass*> InClasses) { AllowedDataInterfaces = InClasses; }
	void SetAllowedObjects(TArray<UClass*> InClasses) { AllowedObjects = InClasses; }
	void SetAllowedInterfaces(TArray<UClass*> InClasses) { AllowedInterfaces = InClasses; }
	void SetAllowedTypeDefinitions(TArray<FNiagaraTypeDefinition> InTypeDefs) { AllowedTypeDefinitions = InTypeDefs; }

	NIAGARA_API void SetDefaultParameter(const FNiagaraVariable& InVariable);
	void SetDefaultParameter(const FName& InName, const FNiagaraTypeDefinition& InTypeDef) { SetDefaultParameter(FNiagaraVariable(InTypeDef, InName)); }

	const FNiagaraVariable& GetDefaultAliasedParameter() { return DefaultAliasedParameter; }
	const FNiagaraVariable& GetDefaultResolvedParameter() { return DefaultResolvedParameter; }

	bool AllowUserParameters() const { return EnumHasAllFlags(BindingUsage, ENiagaraParameterBindingUsage::User); }
	bool AllowSystemParameters() const { return EnumHasAllFlags(BindingUsage, ENiagaraParameterBindingUsage::System); }
	bool AllowEmitterParameters() const { return EnumHasAllFlags(BindingUsage, ENiagaraParameterBindingUsage::Emitter); }
	bool AllowParticleParameters() const { return EnumHasAllFlags(BindingUsage, ENiagaraParameterBindingUsage::Particle); }
	bool AllowStaticVariables() const { return EnumHasAllFlags(BindingUsage, ENiagaraParameterBindingUsage::StaticVariable); }

	TConstArrayView<UClass*> GetAllowedDataInterfaces() const { return MakeArrayView(AllowedDataInterfaces); }
	TConstArrayView<UClass*> GetAllowedObjects() const { return MakeArrayView(AllowedObjects); }
	TConstArrayView<UClass*> GetAllowedInterfaces() const { return MakeArrayView(AllowedInterfaces); }
	TConstArrayView<FNiagaraTypeDefinition> GetAllowedTypeDefinitions() const { return MakeArrayView(AllowedTypeDefinitions); }

	NIAGARA_API bool IsSetToDefault() const;
	NIAGARA_API void SetToDefault();

	NIAGARA_API FString ToString() const;

	static NIAGARA_API void ForEachRenameEmitter(UObject* InObject, FStringView EmitterName);
	static NIAGARA_API void ForEachRenameVariable(UObject* InObject, const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, FStringView EmitterName);
	static NIAGARA_API void ForEachRemoveVariable(UObject* InObject, const FNiagaraVariableBase& OldVariable, FStringView EmitterName);

	virtual bool HasDefaultValueEditorOnly() const { return false; }
	virtual TConstArrayView<uint8> GetDefaultValueEditorOnly() const { checkNoEntry(); return MakeArrayView<uint8>(nullptr, 0); }
	virtual void SetDefaultValueEditorOnly(TConstArrayView<uint8> Memory) { checkNoEntry(); }
	virtual void SetDefaultValueEditorOnly(const uint8* Memory) { checkNoEntry(); }
#endif

	bool operator==(const FNiagaraParameterBinding& Other) const
	{
		return
		#if WITH_EDITORONLY_DATA
			AliasedParameter == Other.AliasedParameter &&
		#endif
			ResolvedParameter == Other.ResolvedParameter;
	}

	/** Parameter binding used by the runtime fully resolved, i.e. NamedEmitter.Parameter */
	UPROPERTY(EditAnywhere, Category = "Parameter Binding")
	FNiagaraVariableBase ResolvedParameter;

#if WITH_EDITORONLY_DATA
	/** Parameter binding used in the UI, i.e. Emitter.Parameter */
	UPROPERTY()
	FNiagaraVariableBase AliasedParameter;

protected:
	/** Default parameter for the binding, can also contain the default value. */
	FNiagaraVariable DefaultAliasedParameter;
	FNiagaraVariable DefaultResolvedParameter;

	/* Set the usage for the binding. */
	ENiagaraParameterBindingUsage BindingUsage = ENiagaraParameterBindingUsage::NotParticle;

	/** List of data interfaces we can bind to, matches with a valid Cast<>. */
	UPROPERTY(transient)
	TArray<TObjectPtr<UClass>> AllowedDataInterfaces;

	/** List of UObject types we can bind to, matches with a valid Cast<>. */
	UPROPERTY(transient)
	TArray<TObjectPtr<UClass>> AllowedObjects;

	/** List of Interfaces to look for on Object & DataInterfaces */
	UPROPERTY(transient)
	TArray<TObjectPtr<UClass>> AllowedInterfaces;

	/** List of explicit type definitions allowed, must be an exact match. */
	TArray<FNiagaraTypeDefinition>	AllowedTypeDefinitions;
#endif
};

USTRUCT()
struct FNiagaraParameterBindingWithValue : public FNiagaraParameterBinding
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	template<typename T> void SetDefaultParameter(const FNiagaraVariableBase& InVariable, const T& InDefaultValue) { FNiagaraVariable NewVariable(InVariable); NewVariable.SetValue<T>(InDefaultValue);  FNiagaraParameterBinding::SetDefaultParameter(NewVariable); }
	template<typename T> void SetDefaultParameter(const FNiagaraTypeDefinition& InTypeDef, const T& InDefaultValue) { SetDefaultParameter(FNiagaraVariableBase(InTypeDef, NAME_None), InDefaultValue); }
	template<typename T> void SetDefaultParameter(const FName& InName, const FNiagaraTypeDefinition& InTypeDef, const T& InDefaultValue) { SetDefaultParameter(FNiagaraVariableBase(InTypeDef, InName), InDefaultValue); }

	NIAGARA_API virtual bool HasDefaultValueEditorOnly() const override;
	NIAGARA_API virtual TConstArrayView<uint8> GetDefaultValueEditorOnly() const override;
	NIAGARA_API virtual void SetDefaultValueEditorOnly(TConstArrayView<uint8> Memory) override;
	NIAGARA_API virtual void SetDefaultValueEditorOnly(const uint8* Memory) override;
	template<typename T> void SetDefaultValueEditorOnly(const T& InDefaultValue) { check(sizeof(T) == DefaultAliasedParameter.GetType().GetSize()); SetDefaultValueEditorOnly(MakeArrayView(reinterpret_cast<const uint8*>(&InDefaultValue), sizeof(InDefaultValue))); }
#endif

	template<typename T> T GetDefaultValue() const
	{
		check(DefaultValue.Num() == sizeof(T));
		T Value;
		FMemory::Memcpy(&Value, DefaultValue.GetData(), sizeof(T));
		return Value;
	}

	TConstArrayView<uint8> GetDefaultValueArray() const { return DefaultValue; }

	bool operator==(const FNiagaraParameterBindingWithValue& Other) const
	{
		return
			FNiagaraParameterBinding::operator==(Other) &&
			DefaultValue == Other.DefaultValue;
	}

protected:
	/** Default value will only have contents if one is provided. */
	UPROPERTY()
	TArray<uint8> DefaultValue;
};
