// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_MULTICAST_DELEGATE(FOnSubscribedParameterDefinitionsChanged);
