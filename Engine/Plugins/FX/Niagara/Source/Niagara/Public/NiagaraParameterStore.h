// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"
#include "NiagaraParameterStore.generated.h"

struct FNiagaraVariant;

// When not cooked, sort by actual name to ensure deterministic cooked data
#define NIAGARA_VARIABLE_LEXICAL_SORTING WITH_EDITORONLY_DATA

class UNiagaraDataInterface;
struct FNiagaraParameterStore;

USTRUCT()
struct FNiagaraBoundParameter
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FNiagaraVariableBase Parameter;
	UPROPERTY()
	int32 SrcOffset = 0;
	UPROPERTY()
	int32 DestOffset = 0;

};

USTRUCT()
struct FNiagaraPositionSource
{
	GENERATED_USTRUCT_BODY()

	FNiagaraPositionSource() : Value(FVector::ZeroVector) {}
	FNiagaraPositionSource(FName InName, FVector InValue) : Name(InName), Value(InValue) {}
	
	UPROPERTY()
	FName Name;
	
	UPROPERTY()
	FVector Value;
};

typedef TArray<FNiagaraBoundParameter> FNiagaraBoundParameterArray;

//Binding from one parameter store to another.
//This does no tracking of lifetimes etc so the owner must ensure safe use and rebinding when needed etc.
struct FNiagaraParameterStoreBinding
{
	struct FParameterBinding
	{
		FParameterBinding(int32 InSrcOffset, int32 InDestOffset, int32 InSize)
			: SrcOffset((uint16)InSrcOffset), DestOffset((uint16)InDestOffset), Size((uint16)InSize)
		{
			//No way these offsets could ever be higher but check just in case.
			check(InSrcOffset < (int32)0xFFFF);
			check(InDestOffset < (int32)0xFFFF);
			check(InSize < (int32)0xFFFF);
		}
		inline bool operator==(const FParameterBinding& Other)const
		{
			return SrcOffset == Other.SrcOffset && DestOffset == Other.DestOffset && Size == Other.Size;
		}
		uint16 SrcOffset;
		uint16 DestOffset;
		uint16 Size;
	};
	/** Bindings of parameter data. Src offset, Dest offset and Size. */
	TArray<FParameterBinding> ParameterBindings;

	struct FInterfaceBinding
	{
		FInterfaceBinding(int32 InSrcOffset, int32 InDestOffset) 
			: SrcOffset((uint16)InSrcOffset), DestOffset((uint16)InDestOffset)
		{
			//No way these offsets could ever be higher but check just in case.
			check(InSrcOffset < (int32)0xFFFF);
			check(InDestOffset < (int32)0xFFFF);
		}
		inline bool operator==(const FInterfaceBinding& Other)const
		{
			return SrcOffset == Other.SrcOffset && DestOffset == Other.DestOffset;
		}
		uint16 SrcOffset;
		uint16 DestOffset;
	};
	/** Bindings of data interfaces. Src and Dest offsets.*/
	TArray<FInterfaceBinding> InterfaceBindings;

	struct FUObjectBinding
	{
		FUObjectBinding(int32 InSrcOffset, int32 InDestOffset)
			: SrcOffset((uint16)InSrcOffset), DestOffset((uint16)InDestOffset)
		{
			//No way these offsets could ever be higher but check just in case.
			check(InSrcOffset < (int32)0xFFFF);
			check(InDestOffset < (int32)0xFFFF);
		}
		inline bool operator==(const FUObjectBinding& Other)const
		{
			return SrcOffset == Other.SrcOffset && DestOffset == Other.DestOffset;
		}
		uint16 SrcOffset;
		uint16 DestOffset;
	};
	/** Bindings of UObject params. Src and Dest offsets.*/
	TArray<FUObjectBinding> UObjectBindings;

	inline void Empty(FNiagaraParameterStore* DestStore, FNiagaraParameterStore* SrcStore);
	inline bool Initialize(FNiagaraParameterStore* DestStore, FNiagaraParameterStore* SrcStore, const FNiagaraBoundParameterArray* BoundParameters = nullptr);
	NIAGARA_API bool VerifyBinding(const FNiagaraParameterStore* DestStore, const FNiagaraParameterStore* SrcStore)const;
	void CopyParameters(FNiagaraParameterStore* DestStore, const FNiagaraParameterStore* SrcStore) const;
	NIAGARA_API void Dump(const FNiagaraParameterStore* DestStore, const FNiagaraParameterStore* SrcStore)const;

	static void GetBindingData(FNiagaraParameterStore* DestStore, FNiagaraParameterStore* SrcStore, FNiagaraBoundParameterArray& OutBoundParameters);

private:
	
	template <typename TVisitor>
	inline static void MatchParameters(FNiagaraParameterStore* DestStore, FNiagaraParameterStore* SrcStore, TVisitor Visitor);

	bool BindParameters(FNiagaraParameterStore* DestStore, FNiagaraParameterStore* SrcStore, const FNiagaraBoundParameterArray* BoundParameters = nullptr);
};

USTRUCT()
struct FNiagaraVariableWithOffset : public FNiagaraVariableBase
{
	GENERATED_USTRUCT_BODY()

	// Those constructor enforce that there are no data allocated.
	inline FNiagaraVariableWithOffset() : Offset(INDEX_NONE) {}
	inline FNiagaraVariableWithOffset(const FNiagaraVariableBase& InVariable, int32 InOffset, const FNiagaraLwcStructConverter& InStructConverter) : FNiagaraVariableBase(InVariable.GetType(), InVariable.GetName()), Offset(InOffset), StructConverter(InStructConverter) {}

	bool Serialize(FArchive& Ar);
#if WITH_EDITORONLY_DATA
	void PostSerialize(const FArchive& Ar);
#endif

	UPROPERTY()
	int32 Offset;

	UPROPERTY()
	FNiagaraLwcStructConverter StructConverter;
};

template<>
struct TStructOpsTypeTraits<FNiagaraVariableWithOffset> : public TStructOpsTypeTraitsBase2<FNiagaraVariableWithOffset>
{
	enum
	{
		WithSerializer = true,
#if WITH_EDITORONLY_DATA
		WithPostSerialize = true,
#endif
	};
};

/** Base storage class for Niagara parameter values. */
USTRUCT()
struct FNiagaraParameterStore
{
#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnChanged);
	struct FScopedSuppressOnChanged : TGuardValue<bool>
	{
		FScopedSuppressOnChanged(FNiagaraParameterStore& TargetStore)
			: TGuardValue(TargetStore.bSuppressOnChanged, true)
		{
		}
	};
	
	DECLARE_MULTICAST_DELEGATE(FOnStructureChanged)
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnParameterRenamed, FNiagaraVariable OldVariable, FName NewName)
#endif

	GENERATED_USTRUCT_BODY()

	/** The View of the set of variables represented by this ParameterStore.  By default it will be the variables defined by the
	SortedParameterOffsets, but child classes can override it so that we can share the list of Variables across different
	instances as it's generally read only data in game */
	virtual TArrayView<const FNiagaraVariableWithOffset> ReadParameterVariables() const { return MakeArrayView(SortedParameterOffsets); }

private:
	/** Owner of this store. Used to provide an outer to data interfaces in this store. */
	UPROPERTY(Transient)
	TWeakObjectPtr<UObject> Owner;
	
#if WITH_EDITORONLY_DATA
	/** Map from parameter defs to their offset in the data table or the data interface. TODO: Separate out into a layout and instance class to reduce duplicated data for this?  */
	UPROPERTY(meta=(DeprecatedProperty))
	TMap<FNiagaraVariable, int32> ParameterOffsets;
#endif // WITH_EDITORONLY_DATA

	/** Storage for the set of variables that are represented by this ParameterStore.  Shouldn't be accessed directly, instead use
	ReadParameterVariables() */
	UPROPERTY()
	TArray<FNiagaraVariableWithOffset> SortedParameterOffsets;

	/** Buffer containing parameter data. Indexed using offsets in ParameterOffsets */
	UPROPERTY()
	TArray<uint8> ParameterData;
	
	/** Data interfaces for this script. Possibly overridden with externally owned interfaces. Also indexed by ParameterOffsets. */
	UPROPERTY()
	TArray<TObjectPtr<UNiagaraDataInterface>> DataInterfaces;

	/** UObjects referenced by this store. Also indexed by ParameterOffsets.*/
	UPROPERTY()
	TArray<TObjectPtr<UObject>> UObjects;

	/** Holds position type source data to be later converted to LWC format. We use an array here instead of a map to save some memory and because linear search is faster with the few elements in here. */
	UPROPERTY()
	TArray<FNiagaraPositionSource> OriginalPositionData;	
	
	/** Bindings between this parameter store and others we push data into when we tick. */
	typedef TPair<FNiagaraParameterStore*, FNiagaraParameterStoreBinding> BindingPair;
	TArray<BindingPair> Bindings;

	/** Parameter stores we've been bound to and are feeding data into us. */
	TArray<FNiagaraParameterStore*> SourceStores;

	/** Marks our parameters as dirty. They will be pushed to any bound stores on tick if true. */
	uint32 bParametersDirty : 1;
	/** Marks our interfaces as dirty. They will be pushed to any bound stores on tick if true. */
	uint32 bInterfacesDirty : 1;
	/** Marks our UObjects as dirty. They will be pushed to any bound stores on tick if true. */
	uint32 bUObjectsDirty : 1;

	uint32 bPositionDataDirty : 1; 

	/** Uniquely identifies the current layout of this parameter store for detecting layout changes. */
	uint32 LayoutVersion;

#if WITH_EDITOR
	FOnChanged OnChangedDelegate;
	bool bSuppressOnChanged = false;
	FOnStructureChanged OnStructureChangedDelegate;
	FOnParameterRenamed OnParameterRenamedDelegate;
#endif

	NIAGARA_API void SetPositionData(const FName& Name, const FVector& Position);
	NIAGARA_API bool HasPositionData(const FName& Name) const;
	NIAGARA_API const FVector* GetPositionData(const FName& Name) const;
	NIAGARA_API void RemovePositionData(const FName& Name);

public:
	NIAGARA_API FNiagaraParameterStore();
	NIAGARA_API FNiagaraParameterStore(const FNiagaraParameterStore& Other);
	NIAGARA_API FNiagaraParameterStore& operator=(const FNiagaraParameterStore& Other);

	NIAGARA_API virtual ~FNiagaraParameterStore();
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	FString DebugName;

	/** Guid data to remap rapid iteration parameters after a function input was renamed. */
	UPROPERTY()
	TMap<FNiagaraVariable, FGuid> ParameterGuidMapping;
#endif

	NIAGARA_API void SetOwner(UObject* InOwner);
	UObject* GetOwner() const { return Owner.Get(); }

	NIAGARA_API void Dump() const;
	NIAGARA_API void DumpParameters(bool bDumpBindings = false)const;

	SIZE_T GetResourceSize() const
	{
		SIZE_T ResourceSize = sizeof(FNiagaraParameterStore);
		ResourceSize += SortedParameterOffsets.GetAllocatedSize();
		ResourceSize += ParameterData.GetAllocatedSize();
		ResourceSize += DataInterfaces.GetAllocatedSize();
		ResourceSize += UObjects.GetAllocatedSize();
		ResourceSize += OriginalPositionData.GetAllocatedSize();
		ResourceSize += Bindings.GetAllocatedSize();
		ResourceSize += SourceStores.GetAllocatedSize();
		return ResourceSize;
	}

	inline bool GetParametersDirty() const { return bParametersDirty; }
	inline bool GetInterfacesDirty() const { return bInterfacesDirty; }
	inline bool GetUObjectsDirty() const { return bUObjectsDirty; }
	inline bool GetPositionDataDirty() const { return bPositionDataDirty; }

	inline void MarkParametersDirty() { bParametersDirty = true; }
	inline void MarkInterfacesDirty() { bInterfacesDirty = true; }
	inline void MarkUObjectsDirty() { bUObjectsDirty = true; }
	inline void MarkPositionDataDirty() { bPositionDataDirty = true; }

	inline uint32 GetLayoutVersion() const { return LayoutVersion; }

	/** Binds this parameter store to another, by default if we find no matching parameters we will not maintain a pointer to the store. */
	NIAGARA_API void Bind(FNiagaraParameterStore* DestStore, const FNiagaraBoundParameterArray* BoundParameters = nullptr);
	/** Unbinds this store form one it's bound to. */
	NIAGARA_API void Unbind(FNiagaraParameterStore* DestStore);
	/** Unbinds this store from all source and destination stores. */
	NIAGARA_API void UnbindAll();
	/** Recreates any bindings to reflect a layout change etc. */
	NIAGARA_API void Rebind();
	/** Recreates any bindings to reflect a layout change etc. */
	NIAGARA_API void TransferBindings(FNiagaraParameterStore& OtherStore);
	/** Handles any update such as pushing parameters to bound stores etc. */
	NIAGARA_API void Tick();
	/** Unbinds this store from all stores it's being driven by. */
	NIAGARA_API void UnbindFromSourceStores();
	
	NIAGARA_API bool VerifyBinding(const FNiagaraParameterStore* InDestStore) const;

	NIAGARA_API void CheckForNaNs() const;

	/**
	Adds the passed parameter to this store.
	Does nothing if this parameter is already present.
	Returns true if we added a new parameter.
	*/
	NIAGARA_API virtual bool AddParameter(const FNiagaraVariable& Param, bool bInitialize = true, bool bTriggerRebind = true, int32* OutOffset = nullptr);

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void ConvertParameterType(const FNiagaraVariable& ExistingParam, const FNiagaraTypeDefinition& NewType);
	
	template<typename BufferType>
	void AddConstantBuffer()
	{
		for (const FNiagaraVariable& BufferVariable : BufferType::GetVariables())
		{
			AddParameter(BufferVariable, true, false);
		}
	}
#endif

	/** Removes the passed parameter if it exists in the store. */
	NIAGARA_API virtual bool RemoveParameter(const FNiagaraVariableBase& Param);

	/** Renames the passed parameter. */
	NIAGARA_API virtual void RenameParameter(const FNiagaraVariableBase& Param, FName NewName);

	/** Changes the type of the passed parameter. */
	NIAGARA_API virtual void ChangeParameterType(const FNiagaraVariableBase& Param, const FNiagaraTypeDefinition& NewType);

	/** Removes all parameters from this store and releases any data. */
	NIAGARA_API virtual void Empty(bool bClearBindings = true);

	/** Removes all parameters from this store but doesn't change memory allocations. */
	NIAGARA_API virtual void Reset(bool bClearBindings = true);

	inline void GetParameters(TArray<FNiagaraVariable>& OutParameters) const
	{
		auto ParameterVariables = ReadParameterVariables();

		OutParameters.Reserve(ParameterVariables.Num());
		for (const FNiagaraVariableWithOffset& ParamWithOffset : ParameterVariables)
		{
			OutParameters.Add(ParamWithOffset);
		}
	}

	inline TArray<FNiagaraParameterStore*>& GetSourceParameterStores() { return SourceStores; }

	inline const TArray<TObjectPtr<UObject>>& GetUObjects()const { return UObjects; }
	inline const TArray<UNiagaraDataInterface*>& GetDataInterfaces()const { return DataInterfaces; }
	inline const TArray<uint8>& GetParameterDataArray()const { return ParameterData; }

	inline int32 Num() const {return SortedParameterOffsets.Num(); }
	inline bool IsEmpty() const { return SortedParameterOffsets.Num() == 0; }

	NIAGARA_API virtual void SanityCheckData(bool bInitInterfaces = true);

	// Called to initially set up the parameter store to *exactly* match the input store (other than any bindings and the internal name of it).
	NIAGARA_API virtual void InitFromSource(const FNiagaraParameterStore* SrcStore, bool bNotifyAsDirty);

	/** Gets the index of the passed parameter. If it is a data interface, this is an offset into the data interface table, otherwise a byte offset into the parameter data buffer. */
	inline int32 IndexOf(const FNiagaraVariableBase& Parameter) const
	{
		const int32* Off = FindParameterOffset(Parameter);
		return Off ? *Off : (int32)INDEX_NONE;
	}
	
	NIAGARA_API virtual const FNiagaraVariableWithOffset* FindParameterVariable(const FNiagaraVariable& Parameter, bool IgnoreType = false) const;

	template<typename T>
	T GetParameterValueFromOffset(int32 Offset) const
	{
		check((Offset >= 0) && (Offset + sizeof(T) <= ParameterData.Num()));

		T OutValue;
		FMemory::Memcpy(&OutValue, ParameterData.GetData() + Offset, sizeof(T));
		return OutValue;
	}

	/** Gets the typed parameter data. */
	template<typename T>
	inline void GetParameterValue(T& OutValue, const FNiagaraVariableBase& Parameter)const
	{
		check(Parameter.GetSizeInBytes() == sizeof(T));
		int32 Offset = IndexOf(Parameter);
		if (Offset != INDEX_NONE)
		{
			OutValue = GetParameterValueFromOffset<T>(Offset);
		}
	}

	template<typename T>
	inline T GetParameterValue(const FNiagaraVariableBase& Parameter)const
	{
		check(Parameter.GetSizeInBytes() == sizeof(T));
		int32 Offset = IndexOf(Parameter);
		if (Offset != INDEX_NONE)
		{
			return GetParameterValueFromOffset<T>(Offset);
		}
		return T();
	}

	template<typename T>
	inline TOptional<T> GetParameterOptionalValue(const FNiagaraVariableBase& Parameter)const
	{
		check(Parameter.GetSizeInBytes() == sizeof(T));
		TOptional<T> OptionalValue;
		const int32 Offset = IndexOf(Parameter);
		if (Offset != INDEX_NONE)
		{
			OptionalValue.Emplace(GetParameterValueFromOffset<T>(Offset));
		}
		return OptionalValue;
	}

	template<typename T>
	inline T GetParameterValueOrDefault(const FNiagaraVariableBase& Parameter, const T& DefaultValue) const
	{
		check(Parameter.GetSizeInBytes() == sizeof(T));
		int32 Offset = IndexOf(Parameter);
		return Offset != INDEX_NONE ? GetParameterValueFromOffset<T>(Offset) : DefaultValue;
	}

	UE_DEPRECATED(5.5, "GetParameterData without a type side is deprecated as it's unsafe")
	inline const uint8* GetParameterData(int32 Offset) const
	{
		return ParameterData.GetData() + Offset;
	}

	inline const uint8* GetParameterData(int32 Offset, int32 SizeInBytes) const
	{
		ensureMsgf(Offset >= 0 && (Offset + SizeInBytes) <= ParameterData.Num(), TEXT("OOB immutable access offset(%d) size(%d) store size(%d)"), Offset, SizeInBytes, ParameterData.Num());
		return ParameterData.GetData() + Offset;
	}

	inline const uint8* GetParameterData(int32 Offset, const FNiagaraTypeDefinition& TypeDef) const
	{
		ensureMsgf(Offset >= 0 && (Offset + TypeDef.GetSize()) <= ParameterData.Num(), TEXT("OOB immutable access offset(%d) size(%d) store size(%d)"), Offset, TypeDef.GetSize(), ParameterData.Num());
		return ParameterData.GetData() + Offset;
	}

	/** Returns a none const pointer to the parameter data, any modifications to the data will not be recorded and automatically pushed to other parameter stores, use with caution. */
	inline uint8* GetMutableParameterData(int32 Offset, const FNiagaraTypeDefinition& TypeDef)
	{
		ensureMsgf(Offset >= 0 && (Offset + TypeDef.GetSize()) <= ParameterData.Num(), TEXT("OOB mutable access offset(%d) size(%d) store size(%d)"), Offset, TypeDef.GetSize(), ParameterData.Num());
		return ParameterData.GetData() + Offset;
	}

	/** Returns the parameter data for the passed parameter if it exists in this store. Null if not. */
	inline const uint8* GetParameterData(const FNiagaraVariableBase& Parameter) const
	{
		int32 Offset = IndexOf(Parameter);
		if (Offset != INDEX_NONE)
		{
			return GetParameterData_Internal(Offset);
		}
		return nullptr;
	}

	inline const uint8* GetParameterData(const FNiagaraVariableWithOffset& Parameter) const
	{
		check(Parameter.Offset != INDEX_NONE);
		return GetParameterData_Internal(Parameter.Offset);
	}

	/** Copies the data stored for the given variable to the target pointer. This method automatically converts custom struct data back to lwc types.
	 * Just using the raw parameter store pointer via IndexOf() will be wrong if the target struct contains a lwc type like FVector.
	 * Returns true if the data was copied, false if the parameter could not be found in the store.
	 */
	NIAGARA_API bool CopyParameterData(const FNiagaraVariable& Parameter, uint8* DestinationData) const;

	/** Returns the data interface at the passed offset. */
	inline UNiagaraDataInterface* GetDataInterface(int32 Offset)const
	{
		if (DataInterfaces.IsValidIndex(Offset))
		{
			return DataInterfaces[Offset];
		}

		return nullptr;
	}

	/** Returns the data interface for the passed parameter if it exists in this store. */
	NIAGARA_API UNiagaraDataInterface* GetDataInterface(const FNiagaraVariable& Parameter) const;

	/** Returns a struct converter for the given variable, if the store contains the variable and it's a LWC type. */
	NIAGARA_API FNiagaraLwcStructConverter GetStructConverter(const FNiagaraVariable& Parameter) const;

	UE_DEPRECATED(5.4, "FindVariable has been replaced by FindVariableFromDataInterface.")
	const FNiagaraVariableBase* FindVariable(const UNiagaraDataInterface* Interface) const { return FindVariableFromDataInterface(Interface); }

	/** Returns the associated FNiagaraVariable for the passed data interface index if it exists in the store. Null if not.*/
	NIAGARA_API const FNiagaraVariableBase* FindVariableFromDataInterfaceIndex(int32 DataInterfaceIndex) const;
	/** Returns the associated FNiagaraVariable for the passed data interface if it exists in the store. Null if not.*/
	NIAGARA_API const FNiagaraVariableBase* FindVariableFromDataInterface(const UNiagaraDataInterface* Interface) const;

	NIAGARA_API virtual const int32* FindParameterOffset(const FNiagaraVariableBase& Parameter, bool IgnoreType = false) const;

	NIAGARA_API void PostLoad(UObject* InOwner);
	NIAGARA_API void SortParameters();

	/** Returns the UObject at the passed offset. */
	inline TObjectPtr<UObject> GetUObject(int32 Offset)const
	{
		if (UObjects.IsValidIndex(Offset))
		{
			return UObjects[Offset];
		}

		return nullptr;
	}

	inline TObjectPtr<UObject> GetUObject(const FNiagaraVariable& Parameter)const
	{
		int32 Offset = IndexOf(Parameter);
		TObjectPtr<UObject> Obj = GetUObject(Offset);
		checkSlow(!Obj || Obj.IsA(Parameter.GetType().GetClass()));
		return Obj;
	}

	/** Copies the passed parameter from this parameter store into another. */
	NIAGARA_API void CopyParameterData(FNiagaraParameterStore& DestStore, const FNiagaraVariable& Parameter) const;
	NIAGARA_API void CopyParameterData(FNiagaraParameterStore& DestStore, const FNiagaraVariable& SourceParameter, const FNiagaraVariable& TargetParameter) const;
	
	enum class EDataInterfaceCopyMethod
	{
		/** A new data interface will be created and it will be synchronized using the CopyTo method. */
		Value,
		/** A reference to the source data interface will be added to the destination. */
		Reference,
		/** Do not copy data interfaces.  This will cause an assert if there are data interfaces in the source
		  * store, and bOnlyAdd is false. */
		None
	};

	/** Copies all parameters from this parameter store into another.*/
	NIAGARA_API void CopyParametersTo(FNiagaraParameterStore& DestStore, bool bOnlyAdd, EDataInterfaceCopyMethod DataInterfaceCopyMethod = EDataInterfaceCopyMethod::None) const;

	/** Remove all parameters from this parameter store from another.*/
	NIAGARA_API void RemoveParameters(FNiagaraParameterStore& DestStore);

	NIAGARA_API FString ToString() const;

	NIAGARA_API virtual bool SetPositionParameterValue(const FVector& InValue, const FName& ParamName, bool bAdd=false);
	NIAGARA_API virtual const FVector* GetPositionParameterValue(const FName& ParamName) const;
	NIAGARA_API void ResolvePositions(FNiagaraLWCConverter LwcConverter);

	template<typename T>
	inline bool SetParameterValue(const T& InValue, const FNiagaraVariable& Param, bool bAdd=false)
	{
		check(Param.GetSizeInBytes() == sizeof(T));
#if WITH_EDITOR
		ensure(FNiagaraTypeHelper::IsLWCType(Param.GetType()) == false);
		if (Param.GetType() == FNiagaraTypeDefinition::GetPositionDef())
		{
			check(HasPositionData(Param.GetName()));
		}
#endif
		int32 Offset = IndexOf(Param);
		if (Offset != INDEX_NONE)
		{
			//Until we solve our alignment issues, temporarily just doing a memcpy here.
			FMemory::Memcpy(GetMutableParameterData_Internal(Offset), &InValue, sizeof(T));
			OnParameterChange();
			return true;
		}
		else
		{
			if (bAdd)
			{
				bool bInitInterfaces = false;
				bool bTriggerRebind = false;
				AddParameter(Param, bInitInterfaces, bTriggerRebind, &Offset);
				check(Offset != INDEX_NONE);
				//Until we solve our alignment issues, temporarily just doing a memcpy here.
				FMemory::Memcpy(GetMutableParameterData_Internal(Offset), &InValue, sizeof(T));
				OnLayoutChange();
				return true;
			}
		}
		return false;
	}

	inline void SetParameterData(const uint8* Data, int32 Offset, int32 Size)
	{
		check(Data != nullptr);
		check(Offset >= 0 && (Offset + Size) <= ParameterData.Num());
		uint8* Dest = GetMutableParameterData_Internal(Offset);
		if (Dest != Data)
		{
			FMemory::Memcpy(Dest, Data, Size);
		}
		OnParameterChange();
	}

	NIAGARA_API bool SetParameterData(const uint8* Data, FNiagaraVariable Param, bool bAdd = false);

	inline void SetDataInterface(UNiagaraDataInterface* InInterface, int32 Offset)
	{
		DataInterfaces[Offset] = InInterface;
		OnInterfaceChange();
	}

	inline void SetDataInterface(UNiagaraDataInterface* InInterface, const FNiagaraVariable& Parameter)
	{
		int32 Offset = IndexOf(Parameter);
		if (Offset != INDEX_NONE)
		{
			DataInterfaces[Offset] = InInterface;
			OnInterfaceChange();
		}
	}

	inline void SetUObject(TObjectPtr<UObject> InObject, int32 Offset)
	{
		UObjects[Offset] = InObject;
		OnUObjectChange();
	}

	inline void SetUObject(TObjectPtr<UObject> InObject, const FNiagaraVariable& Parameter)
	{
		int32 Offset = IndexOf(Parameter);
		if (Offset != INDEX_NONE)
		{
			UObjects[Offset] = InObject;
			OnUObjectChange();
		}
	}

	//Internal Use Only. Do not use.
	NIAGARA_API void SetParameter_InternalUseOnly(const FNiagaraVariableBase& Key, const FNiagaraVariant& Value, bool bAdd=false);
	NIAGARA_API void GetParameter_InternalUseOnly(const FNiagaraVariableBase& Key, FNiagaraVariant& Value)const;

	inline void OnParameterChange() 
	{ 
		bParametersDirty = true;
#if WITH_EDITOR
		if (bSuppressOnChanged == false)
		{
			OnChangedDelegate.Broadcast();
		}
#endif
	}

	inline void OnInterfaceChange() 
	{ 
		bInterfacesDirty = true;
#if WITH_EDITOR
		if (bSuppressOnChanged == false)
		{
			OnChangedDelegate.Broadcast();
		}
#endif
	}

	inline void OnUObjectChange()
	{
		bUObjectsDirty = true;
#if WITH_EDITOR
		if (bSuppressOnChanged == false)
		{
			OnChangedDelegate.Broadcast();
		}
#endif
	}

	inline void PostGenericEditChange()
	{
		bUObjectsDirty = true;
		bInterfacesDirty = true;
		bParametersDirty = true;
#if WITH_EDITOR
		if (bSuppressOnChanged == false)
		{
			OnChangedDelegate.Broadcast();
		}
#endif
	}

#if WITH_EDITOR
	NIAGARA_API FDelegateHandle AddOnChangedHandler(FOnChanged::FDelegate InOnChanged);
	NIAGARA_API void RemoveOnChangedHandler(FDelegateHandle DelegateHandle);
	NIAGARA_API void RemoveAllOnChangedHandlers(FDelegateUserObjectConst InUserObject);

	FOnStructureChanged& OnStructureChanged() { return OnStructureChangedDelegate; }
	FOnParameterRenamed& OnParameterRenamed() { return OnParameterRenamedDelegate; }
#endif

	void TriggerOnLayoutChanged() { OnLayoutChange(); }

protected:
	NIAGARA_API void TickBindings() const;
	NIAGARA_API void OnLayoutChange();
	NIAGARA_API void CopySortedParameterOffsets(TArrayView<const FNiagaraVariableWithOffset> Src);
	NIAGARA_API void AssignParameterData(TConstArrayView<uint8> SourceParameterData);
	static NIAGARA_API int32 PaddedParameterSize(int32 ParameterSize);

	const uint8* GetParameterData_Internal(int32 Offset) const { return ParameterData.GetData() + Offset; }
	uint8* GetMutableParameterData_Internal(int32 Offset) { return ParameterData.GetData() + Offset; }

	template<typename ParamType>
	void SetParameterByOffset(uint32 ParamOffset, const ParamType& Param)
	{
		ParamType* ParamPtr = (ParamType*)(GetParameterDataArray().GetData() + ParamOffset);
		*ParamPtr = Param;
		//SetParameterData((const uint8*)&Param, ParamOffset, sizeof(ParamType)); // TODO why aren't we using this path instead of SetParametersByOffset?
	}

	NIAGARA_API void SetParameterDataArray(const TArray<uint8>& InParameterDataArray, bool bNotifyAsDirty = true);
	NIAGARA_API void SetDataInterfaces(const TArray<UNiagaraDataInterface*>& InDataInterfaces, bool bNotifyAsDirty = true);
	NIAGARA_API void SetUObjects(const TArray<UObject*>& InUObjects, bool bNotifyAsDirty = true);
	NIAGARA_API void SetOriginalPositionData(const TArray<FNiagaraPositionSource>& InOriginalPositionData);

	friend struct FNiagaraParameterStoreToDataSetBinding;    // this should be the only class calling SetParameterByOffset
};

template<>
inline bool FNiagaraParameterStore::SetParameterValue(const FVector3d& InValue, const FNiagaraVariable& Param, bool bAdd)
{
	return SetParameterValue((FVector3f)InValue, Param, bAdd);
}

template<>
inline bool FNiagaraParameterStore::SetParameterValue(const FVector4d& InValue, const FNiagaraVariable& Param, bool bAdd)
{
	return SetParameterValue((FVector4f)InValue, Param, bAdd);
}

template<>
inline bool FNiagaraParameterStore::SetParameterValue(const FQuat4d& InValue, const FNiagaraVariable& Param, bool bAdd)
{
	return SetParameterValue((FQuat4f)InValue, Param, bAdd);
}

inline void FNiagaraParameterStoreBinding::Empty(FNiagaraParameterStore* DestStore, FNiagaraParameterStore* SrcStore)
{
	if (DestStore)
	{
		//UE_LOGF(LogNiagara, Log, "Remove Src Binding: Src: 0x%p - Dst: 0x%p", SrcStore, DestStore);
		DestStore->GetSourceParameterStores().RemoveSingleSwap(SrcStore, EAllowShrinking::No);
	}
	DestStore = nullptr;
	ParameterBindings.Reset();
	InterfaceBindings.Reset();
	UObjectBindings.Reset();
}

inline bool FNiagaraParameterStoreBinding::Initialize(FNiagaraParameterStore* DestStore, FNiagaraParameterStore* SrcStore, const FNiagaraBoundParameterArray* BoundParameters)
{
	checkSlow(DestStore);
	checkSlow(SrcStore);

	if (BindParameters(DestStore, SrcStore, BoundParameters))
	{
		//UE_LOGF(LogNiagara, Log, "Add Src Binding: Src: 0x%p - Dst: 0x%p", SrcStore, DestStore);
		DestStore->GetSourceParameterStores().AddUnique(SrcStore);
		return true;
	}
	else
	{
		return false;
	}
}

//////////////////////////////////////////////////////////////////////////

#define NIAGARA_VALIDATE_DIRECT_BINDINGS	DO_CHECK

/**
Direct binding to a parameter store to allow efficient gets/sets from code etc. 
Does no tracking of lifetimes etc so users are responsible for safety.
*/
template<typename T>
struct FNiagaraParameterDirectBinding
{
	static_assert(!TIsUECoreVariant<T, double>::Value, "Double core variant. Must be float type!");

	mutable T* ValuePtr;
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
	FNiagaraParameterStore* BoundStore;
	FNiagaraVariable BoundVariable;
	uint32 LayoutVersion;
#endif

	FNiagaraParameterDirectBinding()
		: ValuePtr(nullptr)
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		, BoundStore(nullptr)
#endif
	{}

	inline bool IsBound()const { return ValuePtr != nullptr; }

	T* Init(FNiagaraParameterStore& InStore, const FNiagaraVariable& DestVariable)
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		BoundStore = &InStore;
		BoundVariable = DestVariable;
		LayoutVersion = BoundStore->GetLayoutVersion();
#endif
		check(DestVariable.GetSizeInBytes() == sizeof(T));
		ValuePtr = (T*)InStore.GetParameterData(DestVariable);
		return ValuePtr;
	}

	inline void SetValue(const T& InValue)
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		checkSlow(BoundVariable.GetSizeInBytes() == sizeof(T));
		checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));
#endif
		if (ValuePtr)
		{
			*ValuePtr = InValue;
		}
	}

	inline T GetValue()const 
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		checkSlow(BoundVariable.GetSizeInBytes() == sizeof(T));
		checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));
#endif
		if (ValuePtr)
		{
			return *ValuePtr;
		}
		return T();
	}
};

template<>
struct FNiagaraParameterDirectBinding<FMatrix44f>
{
	mutable FMatrix44f* ValuePtr;
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
	FNiagaraParameterStore* BoundStore;
	FNiagaraVariable BoundVariable;
	uint32 LayoutVersion;
#endif

	FNiagaraParameterDirectBinding()
		: ValuePtr(nullptr)
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		, BoundStore(nullptr)
#endif
	{}

	inline bool IsBound()const { return ValuePtr != nullptr; }

	FMatrix44f* Init(FNiagaraParameterStore& InStore, const FNiagaraVariable& DestVariable)
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		BoundStore = &InStore;
		BoundVariable = DestVariable;
		LayoutVersion = BoundStore->GetLayoutVersion();
#endif
		check(DestVariable.GetSizeInBytes() == sizeof(FMatrix44f));
		ValuePtr = (FMatrix44f*)InStore.GetParameterData(DestVariable);
		return ValuePtr;
	}

	inline void SetValue(const FMatrix44f& InValue)
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		checkSlow(BoundVariable.GetSizeInBytes() == sizeof(FMatrix44f));
		checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));
#endif
		if (ValuePtr)
		{
			FMemory::Memcpy(ValuePtr, &InValue, sizeof(FMatrix44f));//Temp annoyance until we fix the alignment issues with parameter stores.
		}
	}

	inline FMatrix44f GetValue()const
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		checkSlow(BoundVariable.GetSizeInBytes() == sizeof(FMatrix44f));
		checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));
#endif
		FMatrix44f Ret = FMatrix44f::Identity;
		if (ValuePtr)
		{
			FMemory::Memcpy(&Ret, ValuePtr, sizeof(FMatrix44f));//Temp annoyance until we fix the alignment issues with parameter stores.
		}
		return Ret;
	}
};

template<>
struct FNiagaraParameterDirectBinding<FVector4f>
{
	mutable FVector4f* ValuePtr;
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
	FNiagaraParameterStore* BoundStore;
	FNiagaraVariable BoundVariable;
	uint32 LayoutVersion;
#endif

	FNiagaraParameterDirectBinding()
		: ValuePtr(nullptr)
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		, BoundStore(nullptr)
#endif
	{}

	inline bool IsBound()const { return ValuePtr != nullptr; }

	FVector4f* Init(FNiagaraParameterStore& InStore, const FNiagaraVariable& DestVariable)
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		BoundStore = &InStore;
		BoundVariable = DestVariable;
		LayoutVersion = BoundStore->GetLayoutVersion();
#endif
		check(DestVariable.GetSizeInBytes() == sizeof(FVector4f));
		ValuePtr = (FVector4f*)InStore.GetParameterData(DestVariable);
		return ValuePtr;
	}

	inline void SetValue(const FVector4f& InValue)
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		checkSlow(BoundVariable.GetSizeInBytes() == sizeof(FVector4f));
		checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));
#endif
		if (ValuePtr)
		{
			FMemory::Memcpy(ValuePtr, &InValue, sizeof(FVector4f));//Temp annoyance until we fix the alignment issues with parameter stores.
		}
	}

	inline FVector4f GetValue()const
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		check(BoundVariable.GetSizeInBytes() == sizeof(FVector4f));
		checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));
#endif
		FVector4f Ret;
		if (ValuePtr)
		{
			FMemory::Memcpy(&Ret, ValuePtr, sizeof(FVector4f));//Temp annoyance until we fix the alignment issues with parameter stores.
		}
		return Ret;
	}
};

template<>
struct FNiagaraParameterDirectBinding<FQuat4f>
{
	mutable FQuat4f* ValuePtr;
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
	FNiagaraParameterStore* BoundStore;
	FNiagaraVariable BoundVariable;
	uint32 LayoutVersion;
#endif

	FNiagaraParameterDirectBinding()
		: ValuePtr(nullptr)
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		, BoundStore(nullptr)
#endif
	{}

	inline bool IsBound()const { return ValuePtr != nullptr; }

	FQuat4f* Init(FNiagaraParameterStore& InStore, const FNiagaraVariable& DestVariable)
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		BoundStore = &InStore;
		BoundVariable = DestVariable;
		LayoutVersion = BoundStore->GetLayoutVersion();
#endif
		check(DestVariable.GetSizeInBytes() == sizeof(FQuat4f));
		ValuePtr = (FQuat4f*)InStore.GetParameterData(DestVariable);
		return ValuePtr;
	}

	inline void SetValue(const FQuat4f& InValue)
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		checkSlow(BoundVariable.GetSizeInBytes() == sizeof(FQuat4f));
		checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));
#endif
		if (ValuePtr)
		{
			FMemory::Memcpy(ValuePtr, &InValue, sizeof(FQuat4f));//Temp annoyance until we fix the alignment issues with parameter stores.
		}
	}

	inline FQuat4f GetValue()const
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		checkSlow(BoundVariable.GetSizeInBytes() == sizeof(FQuat4f));
		checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));
#endif
		FQuat4f Ret = FQuat4f::Identity;
		if (ValuePtr)
		{
			FMemory::Memcpy(&Ret, ValuePtr, sizeof(FQuat4f));//Temp annoyance until we fix the alignment issues with parameter stores.
		}
		return Ret;
	}
};

template<>
struct FNiagaraParameterDirectBinding<FNiagaraBool>
{
	mutable uint32* ValuePtr;
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
	FNiagaraParameterStore* BoundStore;
	FNiagaraVariable BoundVariable;
	uint32 LayoutVersion;
#endif

	FNiagaraParameterDirectBinding()
		: ValuePtr(nullptr)
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		, BoundStore(nullptr)
#endif
	{}

	inline bool IsBound()const { return ValuePtr != nullptr; }

	uint32* Init(FNiagaraParameterStore& InStore, const FNiagaraVariable& DestVariable)
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		BoundStore = &InStore;
		BoundVariable = DestVariable;
		LayoutVersion = BoundStore->GetLayoutVersion();
#endif
		check(DestVariable.GetSizeInBytes() == sizeof(FNiagaraBool));
		check(sizeof(uint32) == sizeof(FNiagaraBool));
		ValuePtr = (uint32*)InStore.GetParameterData(DestVariable);
		return ValuePtr;
	}

	inline void SetValue(const FNiagaraBool& InValue)
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		checkSlow(BoundVariable.GetSizeInBytes() == sizeof(FNiagaraBool));
		checkSlow(sizeof(uint32) == sizeof(FNiagaraBool));
		checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));
#endif
		if (ValuePtr)
		{
			FMemory::Memcpy(ValuePtr, &InValue, sizeof(FNiagaraBool));
		}
	}

	inline void SetValue(const bool& InValue)
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		checkSlow(BoundVariable.GetSizeInBytes() == sizeof(FNiagaraBool));
		checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));
#endif
		if (ValuePtr)
		{
			if (!InValue)
			{
				*ValuePtr = FNiagaraBool::False;
			}
			else
			{
				*ValuePtr = FNiagaraBool::True;
			}
		}
	}

	inline FNiagaraBool GetValue()const
	{
#if NIAGARA_VALIDATE_DIRECT_BINDINGS
		checkSlow(BoundVariable.GetSizeInBytes() == sizeof(FNiagaraBool));
		checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));
#endif
		FNiagaraBool Ret(false);
		if (ValuePtr)
		{
			if (FNiagaraBool::False == *ValuePtr)
			{
				Ret = FNiagaraBool(false);
			}
			else
			{
				Ret = FNiagaraBool(true);
			}
		}
		return Ret;
	}
};

template<>
struct FNiagaraParameterDirectBinding<UObject*>
{
	int32 UObjectOffset;
	FNiagaraParameterStore* BoundStore;
	FNiagaraVariable BoundVariable;
	uint32 LayoutVersion;

	FNiagaraParameterDirectBinding()
		: UObjectOffset(INDEX_NONE), BoundStore(nullptr)
	{}

	inline bool IsBound()const { return UObjectOffset != INDEX_NONE; }

	TObjectPtr<UObject> Init(FNiagaraParameterStore& InStore, const FNiagaraVariable& DestVariable)
	{
		if (DestVariable.IsValid())
		{
			BoundStore = &InStore;
			BoundVariable = DestVariable;
			LayoutVersion = BoundStore->GetLayoutVersion();

			check(BoundVariable.GetType().IsUObject());
			const FNiagaraVariableWithOffset* VariableWithOffset = BoundStore->FindParameterVariable(DestVariable, true);
			UObjectOffset = VariableWithOffset && VariableWithOffset->IsUObject() && !VariableWithOffset->IsDataInterface() ? VariableWithOffset->Offset : INDEX_NONE;
			TObjectPtr<UObject> Ret = BoundStore->GetUObject(UObjectOffset);
			return Ret;
		}
		return nullptr;
	}

	inline void SetValue(UObject* InValue)
	{
		if (UObjectOffset != INDEX_NONE)
		{
			checkSlow(BoundVariable.GetType().IsUObject());
			checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));

			BoundStore->SetUObject(InValue, UObjectOffset);
		}
	}

	inline TObjectPtr<UObject> GetValue() const
	{
		if (UObjectOffset != INDEX_NONE)
		{
			checkSlow(BoundVariable.GetType().IsUObject());
			checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));

			return BoundStore->GetUObject(UObjectOffset);
		}
		return nullptr;
	}

	template<class TObjectType>
	inline TObjectType* GetValueOrDefault(TObjectType* DefaultValue) const
	{
		if (UObjectOffset != INDEX_NONE)
		{
			checkSlow(BoundVariable.GetType().IsUObject());
			checkfSlow(LayoutVersion == BoundStore->GetLayoutVersion(), TEXT("This binding is invalid, its bound parameter store's layout was changed since it was created"));

			return Cast<TObjectType>(BoundStore->GetUObject(UObjectOffset));
		}
		return DefaultValue;
	}

	template<class TObjectType>
	inline TObjectType* GetValue() const
	{
		return GetValueOrDefault<TObjectType>(nullptr);
	}
};
