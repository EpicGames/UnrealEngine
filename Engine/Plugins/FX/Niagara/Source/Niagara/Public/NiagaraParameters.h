// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraCommon.h"
#include "NiagaraParameters.generated.h"


USTRUCT()
struct FNiagaraParameters
{
	GENERATED_USTRUCT_BODY()
	
public:
	//TODO: Sort the array so we can binary search, do not change to a TMap to avoid memory bloat!
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Uniform")
	TArray<FNiagaraVariable> Parameters;

#if WITH_EDITORONLY_DATA

	NIAGARA_API void Empty();

	/** 
	Fills only selected constants into the table. In the order they appear in the array of passed names not the order they appear in the set. 
	Checks the passed map for entries to supersede the default values in the set.
	*/
	void AppendToConstantsTable(uint8* ConstantsTable, const FNiagaraParameters& Externals)const;
	/** Appends the whole parameter set to a constant table with no external overrids. */
	void AppendToConstantsTable(uint8* ConstantsTable)const;

	NIAGARA_API FNiagaraVariable* SetOrAdd(const FNiagaraVariable& InParameter);

	/**	Returns the number of bytes these constants would use in a constants table. */
	int32 GetTableSize()const 
	{ 
		uint32 Size = 0;
		for (const FNiagaraVariable &Uni : Parameters)
		{
			Size += Uni.GetSizeInBytes();
		}
		return Size;
	}
	
	NIAGARA_API FNiagaraVariable* FindParameter(FNiagaraVariable InParam);
	NIAGARA_API const FNiagaraVariable* FindParameter(FNiagaraVariable InParam) const;
	
	NIAGARA_API FNiagaraVariable* FindParameter(FGuid InParamGuid);
	NIAGARA_API const FNiagaraVariable* FindParameter(FGuid InParamGuid) const;

	void Merge(FNiagaraParameters &InParameters)
	{
		for (FNiagaraVariable& C : InParameters.Parameters)
		{
			if (FNiagaraVariable* Param = FindParameter(C))
			{
				*Param = C;
			}
		}
	}

	//Allocates the data for any uniforms in their set missing allocated data.
	//Must do this before using the set in a simulation.
	void Allocate()
	{
		for (FNiagaraVariable& C : Parameters)
		{
			C.AllocateData();
		}
	}

	void Set(FNiagaraParameters &InParameters)
	{
		Empty();
		Parameters = InParameters.Parameters;
	}

	NIAGARA_API void DumpParameters();

#endif // WITH_EDITORONLY_DATA
};
