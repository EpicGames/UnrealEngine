// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraCommon.h"
#include "GameFramework/Actor.h"
#include "UObject/TextProperty.h"


#include "NiagaraPreviewGrid.generated.h"

//////////////////////////////////////////////////////////////////////////

/** Base actor for preview actors used in UNiagaraPreviewAxis. */
UCLASS(abstract, Blueprintable, Transient)
class ANiagaraPreviewBase : public AActor
{
	GENERATED_BODY()

public:
	//AActor Interface
	virtual bool ShouldTickIfViewportsOnly() const override final { return true; }
	//AActor Interface End

	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, CallInEditor, Category = "Niagara|Preview")
	void SetSystem(UNiagaraSystem* InSystem);

	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, CallInEditor, Category = "Niagara|Preview")
	void SetLabelText(const FText& InXAxisText, const FText& InYAxisText);
};

/**
Base class for all preview axis types.
NiagaraPreviewGrid uses these to control how many systems to spawn in each axis and how each system varies on that axis.
C++ Examples are show below. You can also create these as Blueprint classes as show in the Plugin content folder.
*/
UCLASS(abstract, EditInlineNew, BlueprintType, Blueprintable)
class UNiagaraPreviewAxis : public UObject
{
	GENERATED_BODY()

private:
	virtual int32 Num_Implementation() { return 1; }
	virtual void ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText) { }
public:

	//UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//UObject Interface END
	
	/** Returns the number of previews for this axis. */
	UFUNCTION(BlueprintNativeEvent)
	int32 Num();
	
	/** Applies this axis for the preview at PreviewIndex on this axis. */
	UFUNCTION(BlueprintNativeEvent)
	void ApplyToPreview(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText);
};

UCLASS(abstract, EditInlineNew, BlueprintType, Blueprintable)
class UNiagaraPreviewAxis_InterpParamBase : public UNiagaraPreviewAxis
{
	GENERATED_BODY()

protected:
	UPROPERTY(EditAnywhere, Category = Axis)
	FName Param;
	UPROPERTY(EditAnywhere, Category = Axis)
	int32 Count;

	virtual int32 Num_Implementation() override { return Count; }
};

UCLASS(EditInlineNew, BlueprintType, Blueprintable)
class UNiagaraPreviewAxis_InterpParamInt32 : public UNiagaraPreviewAxis_InterpParamBase
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Category = Axis)
	int32 Min;
	UPROPERTY(EditAnywhere, Category = Axis)
	int32 Max;

	//UNiagaraPreviewAxis Interface
	virtual void ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText) override;
	//UNiagaraPreviewAxis Interface END
};

UCLASS(EditInlineNew, BlueprintType, Blueprintable)
class UNiagaraPreviewAxis_InterpParamFloat : public UNiagaraPreviewAxis_InterpParamBase
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Category=Axis)
	float Min;	
	UPROPERTY(EditAnywhere, Category=Axis)
	float Max;

	//UNiagaraPreviewAxis Interface
	virtual void ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText) override;
	//UNiagaraPreviewAxis Interface END
};

UCLASS(EditInlineNew, BlueprintType, Blueprintable)
class UNiagaraPreviewAxis_InterpParamVector2D : public UNiagaraPreviewAxis_InterpParamBase
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Category = Axis)
	FVector2D Min;
	UPROPERTY(EditAnywhere, Category = Axis)
	FVector2D Max;

	//UNiagaraPreviewAxis Interface
	virtual void ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText) override;
	//UNiagaraPreviewAxis Interface END
};

UCLASS(EditInlineNew, BlueprintType, Blueprintable)
class UNiagaraPreviewAxis_InterpParamVector : public UNiagaraPreviewAxis_InterpParamBase
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Category = Axis)
	FVector Min;
	UPROPERTY(EditAnywhere, Category = Axis)
	FVector Max;

	//UNiagaraPreviewAxis Interface
	virtual void ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText) override;
	//UNiagaraPreviewAxis Interface END
};

UCLASS(EditInlineNew, BlueprintType, Blueprintable)
class UNiagaraPreviewAxis_InterpParamVector4 : public UNiagaraPreviewAxis_InterpParamBase
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Category = Axis)
	FVector4 Min;
	UPROPERTY(EditAnywhere, Category = Axis)
	FVector4 Max;

	//UNiagaraPreviewAxis Interface
	virtual void ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText) override;
	//UNiagaraPreviewAxis Interface END
};

UCLASS(EditInlineNew, BlueprintType, Blueprintable)
class UNiagaraPreviewAxis_InterpParamLinearColor : public UNiagaraPreviewAxis_InterpParamBase
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Category = Axis)
	FLinearColor Min;
	UPROPERTY(EditAnywhere, Category = Axis)
	FLinearColor Max;

	//UNiagaraPreviewAxis Interface
	virtual void ApplyToPreview_Implementation(UNiagaraComponent* PreviewComponent, int32 PreviewIndex, bool bIsXAxis, FString& OutLabelText) override;
	//UNiagaraPreviewAxis Interface END
};

UENUM()
enum class ENiagaraPreviewGridResetMode : uint8
{
	Never,/** Never resets the previews. */
	Individual,/** Resets each preview as it completes. */
	All,/** Resets all previews when all have completed. */
};

UCLASS()
class ANiagaraPreviewGrid : public AActor
{
	GENERATED_UCLASS_BODY()

public:
	//UObject Interface
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//UObject Interface End

	//AActor Interface
	virtual void TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction) override;
	virtual bool ShouldTickIfViewportsOnly() const override { return true; }
	//AActor Interface End

	UFUNCTION(BlueprintCallable, Category=Preview)
	void ActivatePreviews(bool bReset);

	UFUNCTION(BlueprintCallable, Category=Preview)
	void DeactivatePreviews();

	UFUNCTION(BlueprintCallable, Category = Preview)
	void SetPaused(bool bPaused);

	UFUNCTION(BlueprintCallable, Category = Preview)
	void GetPreviews(TArray<UNiagaraComponent*>& OutPreviews);

private:

	void DestroyPreviews();
	void GeneratePreviews();
	void TickPreviews();

	inline int32 PreviewIndex(int32 InX, int32 InY)const { return InX * NumY + InY; }

public:

	UPROPERTY(EditAnywhere, Category = Preview)
	TObjectPtr<UNiagaraSystem> System;
	
	UPROPERTY(EditAnywhere, Category = Preview)
	ENiagaraPreviewGridResetMode ResetMode;

	/** Object controlling behavior varying on the X axis. */
	UPROPERTY(EditAnywhere, Category = Preview, Instanced)
	TObjectPtr<UNiagaraPreviewAxis> PreviewAxisX;

	/** Object controlling behavior varying on the Y axis. */
	UPROPERTY(EditAnywhere, Category = Preview, Instanced)
	TObjectPtr<UNiagaraPreviewAxis> PreviewAxisY;

	/** Class used to for previews in this grid. */
	UPROPERTY(EditAnywhere, Category = Preview)
	TSubclassOf<ANiagaraPreviewBase> PreviewClass;

	//TODO: Have the preview actor/class define the size of the preview to make spacing eaiser.
	/** The default spacing between previews in X if the axis does not override it. */
	UPROPERTY(EditAnywhere, Category = Preview)
	float SpacingX;
	
	/** The default spacing between previews if the axis does not override it. */
	UPROPERTY(EditAnywhere, Category = Preview)
	float SpacingY;
	
private:

	UPROPERTY(transient)
	int32 NumX;
	
	UPROPERTY(transient)
	int32 NumY;

	UPROPERTY(transient)
	TArray<TObjectPtr<UChildActorComponent>> PreviewComponents;

#if WITH_EDITORONLY_DATA
	// Reference to sprite visualization component
	UPROPERTY()
	TObjectPtr<class UBillboardComponent> SpriteComponent;

	// Reference to arrow visualization component
	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;
#endif


	uint32 bPreviewDirty : 1;
	uint32 bPreviewActive : 1;
};


