// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RenderGraphResources.h"

//////////////////////////////////////////////////////////////////////////
// Helper for wrapping a persistent RW buffer
struct FNiagaraPooledRWBuffer
{
	~FNiagaraPooledRWBuffer() { Release(); }

	template<typename TResourceName>
	void Initialize(FRDGBuilder& GraphBuilder, const TResourceName& ResourceName, EPixelFormat InPixelFormat, const FRDGBufferDesc& BufferDesc)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<TResourceName, TIsCharEncodingCompatibleWithTCHAR>::Value, "Resource name must be a character array.");
		InitializeInternal(GraphBuilder, ResourceName, InPixelFormat, BufferDesc);
	}
	template<typename TResourceName>
	void Initialize(FRDGBuilder& GraphBuilder, const TResourceName& ResourceName, EPixelFormat InPixelFormat, const uint32 BytesPerElemenet, const uint32 NumElements, EBufferUsageFlags UsageFlags = EBufferUsageFlags::None)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<TResourceName, TIsCharEncodingCompatibleWithTCHAR>::Value, "Resource name must be a character array.");
		InitializeInternal(GraphBuilder, ResourceName, InPixelFormat, BytesPerElemenet, NumElements, UsageFlags);
	}

private:
	NIAGARA_API void InitializeInternal(FRDGBuilder& GraphBuilder, const TCHAR* ResourceName, EPixelFormat PixelFormat, const FRDGBufferDesc& BufferDesc);
	NIAGARA_API void InitializeInternal(FRDGBuilder& GraphBuilder, const TCHAR* ResourceName, EPixelFormat PixelFormat, const uint32 BytesPerElemenet, const uint32 NumElements, EBufferUsageFlags UsageFlags = EBufferUsageFlags::None);
public:
	NIAGARA_API void Release();

	bool IsValid() const { return Buffer.IsValid(); }


	const TRefCountPtr<FRDGPooledBuffer>& GetPooledBuffer() const { return Buffer; }
	NIAGARA_API FRDGBufferRef GetOrCreateBuffer(FRDGBuilder& GraphBuilder);
	NIAGARA_API FRDGBufferSRVRef GetOrCreateSRV(FRDGBuilder& GraphBuilder);
	NIAGARA_API FRDGBufferUAVRef GetOrCreateUAV(FRDGBuilder& GraphBuilder);

	NIAGARA_API void EndGraphUsage();

private:
	TRefCountPtr<FRDGPooledBuffer>	Buffer;
	EPixelFormat					PixelFormat = PF_Unknown;

	// Transient for the lifetime of the GraphBuilder, call PostSimulate to clear
	FRDGBufferRef					TransientRDGBuffer = nullptr;
	FRDGBufferSRVRef				TransientRDGSRV = nullptr;
	FRDGBufferUAVRef				TransientRDGUAV = nullptr;
#if STATS
	uint64							NumBytes = 0;
#endif
};

//////////////////////////////////////////////////////////////////////////
// Helper for wrapping a persistent RW texture
struct FNiagaraPooledRWTexture
{
	~FNiagaraPooledRWTexture() { Release(); }

	template<typename TResourceName>
	void Initialize(FRDGBuilder& GraphBuilder, const TResourceName& ResourceName, const FRDGTextureDesc& TextureDesc)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<TResourceName, TIsCharEncodingCompatibleWithTCHAR>::Value, "Resource name must be a character array.");
		InitializeInternal(GraphBuilder, ResourceName, TextureDesc);
	}
private:
	void InitializeInternal(FRDGBuilder& GraphBuilder, const TCHAR* ResourceName, const FRDGTextureDesc& TextureDesc);
public:
	NIAGARA_API void Release();

	bool IsValid() const { return Texture.IsValid(); }

	const TRefCountPtr<IPooledRenderTarget>& GetPooledTexture() const { return Texture; }
	NIAGARA_API FRDGTextureRef GetOrCreateTexture(FRDGBuilder& GraphBuilder);
	NIAGARA_API FRDGTextureSRVRef GetOrCreateSRV(FRDGBuilder& GraphBuilder);
	NIAGARA_API FRDGTextureUAVRef GetOrCreateUAV(FRDGBuilder& GraphBuilder);

	NIAGARA_API void CopyToTexture(FRDGBuilder& GraphBuilder, FRHITexture* DestinationTexture, const TCHAR* NameIfNotRegistered);

	NIAGARA_API void EndGraphUsage();

private:
	TRefCountPtr<IPooledRenderTarget>	Texture;

	// Transient for the lifetime of the GraphBuilder, call PostSimulate to clear
	FRDGTextureRef						TransientRDGTexture = nullptr;
	FRDGTextureSRVRef					TransientRDGSRV = nullptr;
	FRDGTextureUAVRef					TransientRDGUAV = nullptr;
#if STATS
	uint64								NumBytes = 0;
#endif
};
