// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCore.h"
#include "NiagaraMeshRendererMeshProperties.h"
#include "NiagaraRenderableMeshArrayInterface.generated.h"

class FNiagaraSystemInstance;

UINTERFACE()
class UNiagaraRenderableMeshArrayInterface : public UInterface
{
	GENERATED_BODY()
};

// Interface defintion for UObjects
class INiagaraRenderableMeshArrayInterface
{
	GENERATED_BODY()

public:
	virtual void ForEachMesh(FNiagaraSystemInstance* SystemInstance, TFunction<void(int32)> NumMeshesDelegate, TFunction<void(const FNiagaraMeshRendererMeshProperties&)> IterateDelegate) const = 0;
	void ForEachMesh(FNiagaraSystemInstance* SystemInstance, TFunction<void(const FNiagaraMeshRendererMeshProperties&)> IterateDelegate) const { ForEachMesh(SystemInstance, [](int32) {}, IterateDelegate); }
};
