// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "StaticMeshResources.h"

#include "NiagaraCore.h"
#include "NiagaraRenderableMeshInterface.generated.h"

class FIndexBuffer;
class FNiagaraEmitterInstance;
class FNiagaraMeshVertexFactory;
class UMaterialInterface;
class FRayTracingGeometry;
class FRHIUniformBuffer;
class FSceneView;
struct FStaticMeshSection;

// Abstact class that for a renderable mesh
// While Niagara holds a reference to one of these it is expected that the data will exist
class INiagaraRenderableMesh : public TSharedFromThis<INiagaraRenderableMesh, ESPMode::ThreadSafe>
{
public:
	struct FLODModelData
	{
		int32								LODIndex = INDEX_NONE;
		int32								NumVertices = 0;
		int32								NumIndices = 0;
		TConstArrayView<FStaticMeshSection>	Sections;
		const FIndexBuffer*					IndexBuffer = nullptr;
		FRHIUniformBuffer*					VertexFactoryUserData = nullptr;
		const FRayTracingGeometry*			RayTracingGeometry = nullptr;

		int32								WireframeNumIndices = 0;
		const FIndexBuffer*					WireframeIndexBuffer = nullptr;
	};

	virtual ~INiagaraRenderableMesh() = default;

	// Set the Min LOD Bias
	virtual void SetMinLODBias(int32 MinLODBias) {}
	// Get the local bounds for the mesh
	virtual FBox GetLocalBounds() const = 0;
	// Gather all the relevant mesh data to render the mesh
	virtual void GetLODModelData(FLODModelData& OutLODModelData, int32 LODLevel) const = 0;
	// Gather all the relevant mesh data to render the mesh in the raytracing scene
	virtual void GetRayTraceLODModelData(FLODModelData& OutLODModelData, int32 LODLevel) const { GetLODModelData(OutLODModelData, LODLevel); }
	// Setup the vertex factory for the mesh
	virtual void SetupVertexFactory(FRHICommandListBase& RHICmdList, class FNiagaraMeshVertexFactory& InVertexFactory, const FLODModelData& LODModelData) const = 0;
	// Setup the vertex factory for the mesh in the raytracing scene
	virtual void SetupRayTraceVertexFactory(FRHICommandListBase& RHICmdList, class FNiagaraMeshVertexFactory& InVertexFactory, const FLODModelData& LODModelData) const { SetupVertexFactory(RHICmdList, InVertexFactory, LODModelData); }

	UE_DEPRECATED(5.4, "SetupVertexFactory requires an RHI command list.")
	virtual void SetupVertexFactory(class FNiagaraMeshVertexFactory& InVertexFactory, const FLODModelData& LODModelData) const final {}

	// Gather a list of used materials
	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials) const = 0;

	// Get the available LOD range
	virtual FIntVector2 GetLODRange() const { return FIntVector2(0, 0); }
	// Return parameters used for LOD calculation, screen size min, screen size max & sphere size
	virtual FVector3f GetLODScreenSize(int32 LODLevel) const { return FVector3f(0.0f, 1.0f, 1.0f); }
	// Compute the LOD level for the sphere location
	virtual int32 ComputeLOD(const FVector& SphereOrigin, const float SphereRadius, const FSceneView& SceneView, float LODDistanceFactor) { return 0; }
};

using FNiagaraRenderableMeshPtr = TSharedPtr<INiagaraRenderableMesh, ESPMode::ThreadSafe>;

// Interface for UObjects to implement renderable mesh
UINTERFACE()
class UNiagaraRenderableMeshInterface : public UInterface
{
	GENERATED_BODY()
};

// Interface defintion for UObjects
class INiagaraRenderableMeshInterface
{
	GENERATED_BODY()

public:
	// Get renderable mesh pointer
	virtual FNiagaraRenderableMeshPtr GetRenderableMesh(FNiagaraSystemInstanceID SystemInstanceID) = 0;

	// Get used materials
	virtual void GetUsedMaterials(FNiagaraSystemInstanceID SystemInstanceID, TArray<UMaterialInterface*>& OutMaterials) const = 0;
};
