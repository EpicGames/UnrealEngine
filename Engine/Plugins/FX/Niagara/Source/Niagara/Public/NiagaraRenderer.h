// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraRenderer.h: Base class for Niagara render modules
==============================================================================*/
#pragma once

#include "CoreMinimal.h"
#include "DynamicBufferAllocator.h"
#include "NiagaraCommon.h"
#include "Materials/MaterialInterface.h"
#include "UniformBuffer.h"
#include "Materials/Material.h"
#include "PrimitiveViewRelevance.h"
#include "ParticleHelper.h"
#include "NiagaraRendererProperties.h"
#include "RenderingThread.h"
#include "NiagaraCutoutVertexBuffer.h"
#include "NiagaraBoundsCalculator.h"
#include "RayTracingGeometry.h"

struct INiagaraComputeDataBufferInterface;
class FNiagaraDataSet;
class FNiagaraSceneProxy;
class FNiagaraGPURendererCount;
class FNiagaraSystemInstanceController;
class FRayTracingInstanceCollector;
struct FRayTracingInstance;

/** Struct used to pass dynamic data from game thread to render thread */
struct FNiagaraDynamicDataBase
{
	NIAGARA_API explicit FNiagaraDynamicDataBase(const FNiagaraEmitterInstance* InEmitter);
	NIAGARA_API virtual ~FNiagaraDynamicDataBase();

	UE_NONCOPYABLE(FNiagaraDynamicDataBase);

	NIAGARA_API bool IsGpuLowLatencyTranslucencyEnabled() const;
	NIAGARA_API FNiagaraDataBuffer* GetParticleDataToRender(FRHICommandListBase& RHICmdList, bool bIsLowLatencyTranslucent = false) const;
	inline FMaterialRelevance GetMaterialRelevance() const { return MaterialRelevance; }

	inline void SetMaterialRelevance(FMaterialRelevance NewRelevance) { MaterialRelevance = NewRelevance; }

	inline FNiagaraSystemInstanceID GetSystemInstanceID() const { return SystemInstanceID; }

	virtual void ApplyMaterialOverride(int32 MaterialIndex, UMaterialInterface* MaterialOverride) {};

protected:
	FMaterialRelevance MaterialRelevance;
	FNiagaraSystemInstanceID SystemInstanceID;

	FNiagaraDataBufferRef CPUParticleData;
	INiagaraComputeDataBufferInterface* ComputeDataBufferInterface = nullptr;
};

//////////////////////////////////////////////////////////////////////////

struct FParticleRenderData
{
	FGlobalDynamicReadBuffer::FAllocation FloatData;
	FGlobalDynamicReadBuffer::FAllocation HalfData;
	FGlobalDynamicReadBuffer::FAllocation IntData;
	uint32 FloatStride = 0;
	uint32 HalfStride = 0;
	uint32 IntStride = 0;
};

/**
* Base class for Niagara System renderers.
*/
class FNiagaraRenderer
{
public:
	NIAGARA_API FNiagaraRenderer(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter);
	NIAGARA_API virtual ~FNiagaraRenderer();

	FNiagaraRenderer(const FNiagaraRenderer& Other) = delete;
	FNiagaraRenderer& operator=(const FNiagaraRenderer& Other) = delete;

	NIAGARA_API virtual void Initialize(const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InComponent);
	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) {}
	virtual void ReleaseRenderThreadResources() {}
	virtual void DestroyRenderState_Concurrent() {}

	NIAGARA_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View, const FNiagaraSceneProxy *SceneProxy)const;
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector, const FNiagaraSceneProxy *SceneProxy) const {}
	virtual FNiagaraDynamicDataBase* GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitteride) const { return nullptr; }

	virtual void GatherSimpleLights(FSimpleLightArray& OutParticleLights)const {}
	virtual int32 GetDynamicDataSize()const { return 0; }
	virtual bool IsMaterialValid(const UMaterialInterface* Mat)const { return Mat != nullptr; }

	// Determine if we are rendering into an opaque only view
	static NIAGARA_API bool IsViewRenderingOpaqueOnly(const FSceneView* View, bool bCanCastTranslucentShadows);
	static NIAGARA_API bool AreViewsRenderingOpaqueOnly(const TArray<const FSceneView*>& Views, int32 ViewVisibilityMask, bool bCanCastTranslucentShadows);

	UE_DEPRECATED(5.4, "SortIndices method should be replaced with SortAndCullIndices")
	static NIAGARA_API void SortIndices(const struct FNiagaraGPUSortInfo& SortInfo, const FNiagaraRendererVariableInfo& SortVariable, const FNiagaraDataBuffer& Buffer, FGlobalDynamicReadBuffer::FAllocation& OutIndices);
	static NIAGARA_API int32 SortAndCullIndices(const FNiagaraGPUSortInfo& SortInfo, const FNiagaraDataBuffer& Buffer, FGlobalDynamicReadBuffer::FAllocation& OutIndices);

	static NIAGARA_API FVector4f CalcMacroUVParameters(const FSceneView& View, FVector MacroUVPosition, float MacroUVRadius);

	NIAGARA_API void SetDynamicData_RenderThread(FNiagaraDynamicDataBase* NewDynamicData);
	inline FNiagaraDynamicDataBase *GetDynamicData() const { return DynamicDataRender; }
	inline bool HasDynamicData() const { return DynamicDataRender != nullptr; }
	inline bool HasLights() const { return bHasLights; }
	inline bool IsMotionBlurEnabled() const { return bMotionBlurEnabled; }

#if RHI_RAYTRACING
	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FNiagaraSceneProxy* Proxy) {}
#endif

	inline static FRHIShaderResourceView* GetSrvOrDefaultFloat(const FRWBuffer& RWBuffer) { return RWBuffer.SRV.IsValid() ? (FRHIShaderResourceView*)RWBuffer.SRV : GetDummyFloatBuffer(); }
	inline static FRHIShaderResourceView* GetSrvOrDefaultHalf(const FRWBuffer& RWBuffer) { return RWBuffer.SRV.IsValid() ? (FRHIShaderResourceView*)RWBuffer.SRV : GetDummyHalfBuffer(); }
	inline static FRHIShaderResourceView* GetSrvOrDefaultUInt(const FRWBuffer& RWBuffer) { return RWBuffer.SRV.IsValid() ? (FRHIShaderResourceView*)RWBuffer.SRV : GetDummyUIntBuffer(); }
	inline static FRHIShaderResourceView* GetSrvOrDefaultInt(const FRWBuffer& RWBuffer) { return RWBuffer.SRV.IsValid() ? (FRHIShaderResourceView*)RWBuffer.SRV : GetDummyIntBuffer(); }

	inline static FRHIShaderResourceView* GetSrvOrDefaultFloat(FGlobalDynamicReadBuffer::FAllocation& Allocation) { return Allocation.IsValid() ? (FRHIShaderResourceView*)Allocation.SRV : GetDummyFloatBuffer(); }
	inline static FRHIShaderResourceView* GetSrvOrDefaultHalf(FGlobalDynamicReadBuffer::FAllocation& Allocation) { return Allocation.IsValid() ? (FRHIShaderResourceView*)Allocation.SRV : GetDummyHalfBuffer(); }
	inline static FRHIShaderResourceView* GetSrvOrDefaultUInt(FGlobalDynamicReadBuffer::FAllocation& Allocation) { return Allocation.IsValid() ? (FRHIShaderResourceView*)Allocation.SRV : GetDummyUIntBuffer(); }
	inline static FRHIShaderResourceView* GetSrvOrDefaultInt(FGlobalDynamicReadBuffer::FAllocation& Allocation) { return Allocation.IsValid() ? (FRHIShaderResourceView*)Allocation.SRV : GetDummyIntBuffer(); }

	inline static FRHIShaderResourceView* GetSrvOrDefaultFloat(FRHIShaderResourceView* InSRV) { return InSRV ? InSRV : GetDummyFloatBuffer(); }
	inline static FRHIShaderResourceView* GetSrvOrDefaultFloat2(FRHIShaderResourceView* InSRV) { return InSRV ? InSRV : GetDummyFloat2Buffer(); }
	inline static FRHIShaderResourceView* GetSrvOrDefaultFloat4(FRHIShaderResourceView* InSRV) { return InSRV ? InSRV : GetDummyFloat4Buffer(); }
	inline static FRHIShaderResourceView* GetSrvOrDefaultUInt(FRHIShaderResourceView* InSRV) { return InSRV ? InSRV : GetDummyUIntBuffer(); }
	inline static FRHIShaderResourceView* GetSrvOrDefaultInt(FRHIShaderResourceView* InSRV) { return InSRV ? InSRV : GetDummyIntBuffer(); }
	inline static FRHIShaderResourceView* GetSrvOrDefaultByteAddress(FRHIShaderResourceView* InSRV) { return InSRV ? InSRV : GetDummyByteAddressBuffer(); }

	static NIAGARA_API FRHIShaderResourceView* GetDummyFloatBuffer();
	static NIAGARA_API FRHIShaderResourceView* GetDummyFloat2Buffer();
	static NIAGARA_API FRHIShaderResourceView* GetDummyFloat4Buffer();
	static NIAGARA_API FRHIShaderResourceView* GetDummyWhiteColorBuffer();
	static NIAGARA_API FRHIShaderResourceView* GetDummyIntBuffer();
	static NIAGARA_API FRHIShaderResourceView* GetDummyUIntBuffer();
	static NIAGARA_API FRHIShaderResourceView* GetDummyUInt2Buffer();
	static NIAGARA_API FRHIShaderResourceView* GetDummyUInt4Buffer();
	static NIAGARA_API FRHIShaderResourceView* GetDummyHalfBuffer();
	static NIAGARA_API FRHIShaderResourceView* GetDummyByteAddressBuffer();

	static inline int SelectVFComponent(const TConstArrayView<FNiagaraRendererVariableInfo>& Variables, int Variable, int Fallback)
	{
		const int GPUOffset = Variables[Variable].GetGPUOffset();
		return GPUOffset == INDEX_NONE ? Variables[Fallback].GetGPUOffset() : GPUOffset;
	}

	inline ENiagaraSimTarget GetSimTarget() const { return SimTarget; }

	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& UsedMaterials, bool bGetDebugMaterials) { UsedMaterials.Append(BaseMaterials_GT); }

	virtual void PostSystemTick_GameThread(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) { }
	virtual void OnSystemComplete_GameThread(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) { }

protected:

	NIAGARA_API virtual void ProcessMaterialParameterBindings(const FNiagaraRendererMaterialParameters& MaterialParameters, const FNiagaraEmitterInstance* InEmitter, TConstArrayView<UMaterialInterface*> InMaterials) const;

	NIAGARA_API bool IsRendererEnabled(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const;
	NIAGARA_API bool UseLocalSpace(const FNiagaraSceneProxy* Proxy) const;

	static NIAGARA_API bool ViewFamilySupportLowLatencyTranslucency(const FSceneViewFamily& ViewFamily);

	struct FNiagaraDynamicDataBase *DynamicDataRender;

#if RHI_RAYTRACING
	FRWBuffer RayTracingDynamicVertexBuffer;
	FRayTracingGeometry RayTracingGeometry;
#endif

	uint32 bLocalSpace : 1;
	uint32 bHasLights : 1;
	uint32 bMotionBlurEnabled : 1;
	uint32 bRendersInSecondaryDepthPass : 1;
	const ENiagaraSimTarget SimTarget;

	ERHIFeatureLevel::Type FeatureLevel;

#if STATS
	TStatId EmitterStatID;
#endif


	static NIAGARA_API FParticleRenderData TransferDataToGPU(FRHICommandListBase& RHICmdList, FGlobalDynamicReadBuffer& DynamicReadBuffer, const FNiagaraRendererLayout* RendererLayout, TConstArrayView<uint32> IntComponents, const FNiagaraDataBuffer* SrcData);

	/** Cached array of materials used from the properties data. Validated with usage flags etc. */
	TArray<UMaterialInterface*> BaseMaterials_GT;
	FMaterialRelevance BaseMaterialRelevance_GT;
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "SceneView.h"
#endif
