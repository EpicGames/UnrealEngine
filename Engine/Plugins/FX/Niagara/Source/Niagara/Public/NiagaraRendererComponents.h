// Copyright Epic Games, Inc. All Rights Reserved.

/*========================================================================================
NiagaraRendererComponents.h: Renderer for rendering Niagara particles as scene components.
=========================================================================================*/
#pragma once

#include "NiagaraRenderer.h"
#include "UObject/WeakFieldPtr.h"
#include "CoreMinimal.h"

class UNiagaraComponentRendererProperties;
class FNiagaraDataSet;

/**
* NiagaraRendererComponents renders an FNiagaraEmitterInstance as scene components
*/
class FNiagaraRendererComponents : public FNiagaraRenderer
{
	friend struct FNiagaraRendererComponentsOnObjectsReplacedHelper;
public:

	NIAGARA_API FNiagaraRendererComponents(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter);
	NIAGARA_API virtual ~FNiagaraRendererComponents();

	//FNiagaraRenderer interface
	NIAGARA_API virtual void DestroyRenderState_Concurrent() override;
	NIAGARA_API virtual void PostSystemTick_GameThread(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) override;
	NIAGARA_API virtual void OnSystemComplete_GameThread(const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) override;
	//FNiagaraRenderer interface END

private:

	struct FComponentPropertyAddress
	{
		TWeakFieldPtr<FProperty> Property;
		void* Address;

		FProperty* GetProperty() const
		{
			FProperty* PropertyPtr = Property.Get();
			if (PropertyPtr && Address)
			{
				return PropertyPtr;
			}
			return nullptr;
		}

		FComponentPropertyAddress() : Property(nullptr), Address(nullptr) {}
	};

	struct FComponentPoolEntry
	{
		TWeakObjectPtr<USceneComponent> Component;
		double LastActiveTime = 0.0;
		TMap<FName, FComponentPropertyAddress> PropertyAddressMapping;
		int32 LastAssignedToParticleID = -1;
	};

	NIAGARA_API void ResetComponentPool(bool bResetOwner);
#if WITH_EDITOR
	NIAGARA_API void OnObjectsReplacedCallback(const TMap<UObject*, UObject*>& ReplacementsMap);
#endif
	
	static NIAGARA_API void TickPropertyBindings(
		const UNiagaraComponentRendererProperties* Properties,
		USceneComponent* Component,
		const FNiagaraDataSet& Data,
		int32 ParticleIndex,
		FComponentPoolEntry& PoolEntry,
		const FNiagaraLWCConverter& LwcConverter);

	// These property accessor methods are largely copied over from MovieSceneCommonHelpers.h
	static NIAGARA_API FComponentPropertyAddress FindPropertyRecursive(void* BasePointer, UStruct* InStruct, const TArray<FName>& InPropertyNames, uint32 Index);
	static NIAGARA_API FComponentPropertyAddress FindProperty(const UObject& Object, const FString& InPropertyPath);

	// this key is used to check if the template object was changed, e.g. a blueprint compilation
	TObjectKey<USceneComponent> TemplateKey;

	// if the niagara component is not attached to an actor, we need to spawn and keep track of a temporary actor
	TWeakObjectPtr<AActor> SpawnedOwner;

	// all of the spawned components
	TArray<FComponentPoolEntry> ComponentPool;

#if WITH_EDITORONLY_DATA
	struct FNiagaraRendererComponentsOnObjectsReplacedHelper* OnObjectsReplacedHandler = nullptr;
#endif
};
