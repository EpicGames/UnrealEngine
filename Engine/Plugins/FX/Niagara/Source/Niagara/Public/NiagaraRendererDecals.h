// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraRenderer.h"

class FDeferredDecalProxy;
class USceneComponent;

class FNiagaraRendererDecals : public FNiagaraRenderer
{
public:
	NIAGARA_API explicit FNiagaraRendererDecals(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter);
	NIAGARA_API ~FNiagaraRendererDecals();

	NIAGARA_API void ReleaseAllDecals() const;

	//FNiagaraRenderer interface
	NIAGARA_API virtual void DestroyRenderState_Concurrent() override;
	NIAGARA_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View, const FNiagaraSceneProxy *SceneProxy)const override;
	NIAGARA_API virtual FNiagaraDynamicDataBase *GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const override;
	//FNiagaraRenderer interface END

	mutable TWeakObjectPtr<USceneComponent>		WeakOwnerComponent;
	mutable TWeakObjectPtr<UMaterialInterface>	WeakMaterial;
	mutable TArray<FDeferredDecalProxy*>		ActiveDecalProxies;
};
