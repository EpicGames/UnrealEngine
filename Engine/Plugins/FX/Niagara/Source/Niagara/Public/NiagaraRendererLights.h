// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraRendererLights.h: Renderer for rendering Niagara particles as Lights.
==============================================================================*/
#pragma once

#include "NiagaraRenderer.h"
#include "PrimitiveSceneProxy.h"

/**
* NiagaraRendererLights renders an FNiagaraEmitterInstance as simple lights
*/

class FNiagaraRendererLights : public FNiagaraRenderer
{
public:
	struct SimpleLightData
	{
		FSimpleLightEntry LightEntry;
		FSimpleLightPerViewEntry PerViewEntry;
	};

	NIAGARA_API FNiagaraRendererLights(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter);

	//FNiagaraRenderer interface
	NIAGARA_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View, const FNiagaraSceneProxy *SceneProxy)const override;
	NIAGARA_API virtual FNiagaraDynamicDataBase *GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const override;
	NIAGARA_API virtual void GatherSimpleLights(FSimpleLightArray& OutParticleLights)const override;
	//FNiagaraRenderer interface END
};


