// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraRenderer.h: Base class for Niagara render modules
==============================================================================*/
#pragma once

#include "NiagaraRenderer.h"
#include "NiagaraRenderableMeshInterface.h"
#include "NiagaraMeshRendererProperties.h"
#include "NiagaraMeshVertexFactory.h"
#include "NiagaraGPUSortInfo.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraGPUSceneUtils.h"
#include "InstanceUniformShaderParameters.h"

class FNiagaraDataSet;
struct FNiagaraDynamicDataMesh;

/**
* NiagaraRendererSprites renders an FNiagaraEmitterInstance as sprite particles
*/
class FNiagaraRendererMeshes : public FNiagaraRenderer
{
public:
	NIAGARA_API FNiagaraRendererMeshes(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter);
	NIAGARA_API virtual ~FNiagaraRendererMeshes() override;

	//FNiagaraRenderer Interface
	NIAGARA_API virtual void Initialize(const UNiagaraRendererProperties* InProps, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) override;
	NIAGARA_API virtual void ReleaseRenderThreadResources() override;

	NIAGARA_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector, const FNiagaraSceneProxy* SceneProxy) const override;
	NIAGARA_API virtual FNiagaraDynamicDataBase* GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const override;
	NIAGARA_API virtual int32 GetDynamicDataSize()const override;
	NIAGARA_API virtual bool IsMaterialValid(const UMaterialInterface* Mat)const override;
#if RHI_RAYTRACING
	NIAGARA_API virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FNiagaraSceneProxy* Proxy) final override;
#endif
	//FNiagaraRenderer Interface END

	bool HasValidMeshes() const { return Meshes.Num() > 0; }

protected:
	struct FParticleMeshRenderData
	{
		class FMeshElementCollector*	Collector = nullptr;
		const FNiagaraDynamicDataMesh*	DynamicDataMesh = nullptr;
		class FNiagaraDataBuffer*		SourceParticleData = nullptr;

		bool 							bUseGPUScene = false;
		bool							bHasTranslucentMaterials = false;
		bool							bAllTranslucentMaterials = false;
		bool							bCanCastTranslucentShadows = false;
		bool							bSortCullOnGpu = false;
		bool							bNeedsSort = false;
		bool							bNeedsCull = false;
		bool							bAllowPerParticleMeshLODs = false;
		bool							bAllowFirstMeshOnly = false;
		bool							bIsGpuLowLatencyTranslucency = false;

		const FNiagaraRendererLayout*	RendererLayout = nullptr;
		ENiagaraMeshVFLayout::Type		SortVariable = ENiagaraMeshVFLayout::Type(INDEX_NONE);

		FRHIShaderResourceView*			ParticleFloatSRV = nullptr;
		FRHIShaderResourceView*			ParticleHalfSRV = nullptr;
		FRHIShaderResourceView*			ParticleIntSRV = nullptr;
		uint32							ParticleFloatDataStride = 0;
		uint32							ParticleHalfDataStride = 0;
		uint32							ParticleIntDataStride = 0;
		FRHIShaderResourceView*			ParticleSortedIndicesSRV = nullptr;
		uint32							ParticleSortedIndicesOffset = 0xffffffff;

		FIntVector4						VertexFetch_Parameters = FIntVector4(INDEX_NONE);
		FRHIShaderResourceView*			TexCoordBufferSrv = nullptr;
		FRHIShaderResourceView*			PackedTangentsBufferSrv = nullptr;
		FRHIShaderResourceView*			ColorComponentsBufferSrv = nullptr;

		uint32							RendererVisTagOffset = INDEX_NONE;
		uint32							MeshIndexOffset = INDEX_NONE;
		TBitArray<TInlineAllocator<2>>	MeshUsed;

		FVector							WorldSpacePivotOffset = FVector::ZeroVector;
		FSphere							CullingSphere = FSphere(EForceInit::ForceInit);

		int32							ResolutionMaxAxis = 0;
		FVector3f						WorldSpaceSize = FVector3f::ZeroVector;
	};

	struct FMeshData
	{
		FNiagaraRenderableMeshPtr RenderableMesh;
		uint32 SourceMeshIndex = INDEX_NONE;
		ENiagaraMeshLODMode LODMode = ENiagaraMeshLODMode::LODLevel;
		int32 LODLevel = 0;
		float LODDistanceFactor = 1.0f;
		FVector3f LODScreenSize = FVector3f(0.0f, -1.0f, 1.0f);		// Defaults set to not use screen size culled, 0.0 is min size, -1.0f is max size which is tranlated to > 1.0
		FVector3f PivotOffset = FVector3f::ZeroVector;
		ENiagaraMeshPivotOffsetSpace PivotOffsetSpace = ENiagaraMeshPivotOffsetSpace::Mesh;
		FVector3f Scale = FVector3f(1.0f, 1.0f, 1.0f);
		FQuat4f Rotation = FQuat4f::Identity;
		TArray<uint32, TInlineAllocator<4>> MaterialRemapTable;
	};

	class FMeshCollectorResources : public FOneFrameResource
	{
	public:
		~FMeshCollectorResources() override { VertexFactory.ReleaseResource(); }

		FNiagaraMeshVertexFactory VertexFactory;
		FNiagaraMeshUniformBufferRef UniformBuffer;
	};

	struct FEmitterSourceInstanceData
	{
		FInstanceSceneData InstanceSceneData;
		FInstanceDynamicData InstanceDynamicData;
		float CustomData;
	};

	class FGPUSceneUpdateResource : public FOneFrameResource
	{
	public:
		ERHIFeatureLevel::Type FeatureLevel;
		bool bPreciseMotionVectors;
		FMeshBatchDynamicPrimitiveData DynamicPrimitiveData;
		FNiagaraGPUSceneUtils::FUpdateMeshParticleInstancesParams GPUWriteParams;
		FEmitterSourceInstanceData EmitterSourceData;

		FGPUSceneUpdateResource(ERHIFeatureLevel::Type InFeatureLevel, bool bInPreciseMotionVectors) : FeatureLevel(InFeatureLevel), bPreciseMotionVectors(bInPreciseMotionVectors) {}
		virtual ~FGPUSceneUpdateResource() = default;
	};

	NIAGARA_API void PrepareParticleMeshRenderData(FRHICommandListBase& RHICmdList, FParticleMeshRenderData& ParticleMeshRenderData, const FSceneViewFamily& ViewFamily, FMeshElementCollector& Collector, FNiagaraDynamicDataBase* InDynamicData, const FNiagaraSceneProxy* SceneProxy, bool bRayTracing, ENiagaraGpuComputeTickStage::Type GpuReadyTickStage) const;
	NIAGARA_API bool CalculateMeshUsed(FRHICommandListBase& RHICmdList, FParticleMeshRenderData& ParticleMeshRenderData) const;
	NIAGARA_API void PrepareParticleRenderBuffers(FRHICommandListBase& RHICmdList, FParticleMeshRenderData& ParticleMeshRenderData, FGlobalDynamicReadBuffer& DynamicReadBuffer) const;
	NIAGARA_API void InitializeSortInfo(const FParticleMeshRenderData& ParticleMeshRenderData, const FNiagaraSceneProxy& SceneProxy, const FSceneView& View, int32 ViewIndex, bool bIsInstancedStereo, FNiagaraGPUSortInfo& OutSortInfo) const;
	NIAGARA_API void PreparePerMeshData(FParticleMeshRenderData& ParticleMeshRenderData, const FNiagaraMeshVertexFactory& VertexFactory, const FNiagaraSceneProxy& SceneProxy, const FMeshData& MeshData) const;
	NIAGARA_API uint32 PerformSortAndCull(FRHICommandListBase& RHICmdList, FParticleMeshRenderData& ParticleMeshRenderData, FGlobalDynamicReadBuffer& ReadBuffer, FNiagaraGPUSortInfo& SortInfo, class FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, const FSceneView& View, const FMeshData& MeshData) const;
	NIAGARA_API FNiagaraMeshCommonParameters CreateCommonShaderParams(const FParticleMeshRenderData& ParticleMeshRenderData, const FSceneView& View, const FMeshData& MeshData, const FNiagaraSceneProxy& SceneProxy) const;
	NIAGARA_API FNiagaraMeshUniformBufferRef CreateVFUniformBuffer(const FParticleMeshRenderData& ParticleMeshRenderData, const FNiagaraMeshCommonParameters& CommonParams) const;

	static FVector4f GetShaderLODScreenSize(const FSceneView& View, const FMeshData& MeshData);

	NIAGARA_API void SetupElementForGPUScene(
		const FParticleMeshRenderData& ParticleMeshRenderData,
		const FNiagaraMeshCommonParameters& CommonParameters,
		const FNiagaraSceneProxy& SceneProxy,
		const FMeshData& MeshData,
		const FSceneView& View,
		uint32 NumInstances,
		bool bNeedsPrevTransform,
		FMeshBatchElement& OutMeshBatchElement
	) const;

	NIAGARA_API void CreateMeshBatchForSection(
		FRHICommandListBase& RHICmdList,
		const FParticleMeshRenderData& ParticleMeshRenderData,
		const FNiagaraMeshCommonParameters& CommonParams,
		FMeshBatch& MeshBatch,
		FVertexFactory& VertexFactory,
		FMaterialRenderProxy& MaterialProxy,
		const FNiagaraSceneProxy& SceneProxy,
		const FMeshData& MeshData,
		const INiagaraRenderableMesh::FLODModelData& LODModel,
		const FStaticMeshSection& Section,
		const FSceneView& View,
		int32 ViewIndex,
		uint32 NumInstances,
		uint32 GPUCountBufferOffset,
		bool bIsWireframe,
		bool bIsInstancedStereo,
		bool bNeedsPrevTransform
	) const;

private:
	TArray<FMeshData, TInlineAllocator<1>> Meshes;
	int32 MeshUsedMax = 0;

	ENiagaraRendererSourceDataMode SourceMode;
	ENiagaraSortMode SortMode;
	ENiagaraMeshFacingMode FacingMode;
	uint32 bOverrideMaterials : 1;
	uint32 bSortHighPrecision : 1;
	uint32 bSortOnlyWhenTranslucent : 1;
	uint32 bGpuLowLatencyTranslucency : 1;
	uint32 bLockedAxisEnable : 1;
	uint32 bEnableCulling : 1;
	uint32 bEnableFrustumCulling : 1;
	uint32 bEnableLODCulling : 1;
	uint32 bAccurateMotionVectors : 1;
	uint32 bCastShadows : 1;
#if WITH_EDITORONLY_DATA
	uint32 bIncludeInHitProxy : 1 = true;
#endif

	uint32 bSubImageBlend : 1;
	FVector2f SubImageSize = FVector2f::ZeroVector;

	FVector3f MeshBoundsScale = FVector3f::OneVector;

	FVector3f LockedAxis = FVector3f::ZeroVector;
	ENiagaraMeshLockedAxisSpace LockedAxisSpace;

	FVector2f DistanceCullRange = FVector2f::ZeroVector;
	FVector2f DistanceCullRangeSquared = FVector2f::ZeroVector;
	int32 ParticleRendererVisTagOffset = INDEX_NONE;
	int32 ParticleMeshIndexOffset = INDEX_NONE;
	int32 RendererVisibility = 0;
	int32 EmitterRendererVisTagOffset = INDEX_NONE;
	int32 EmitterMeshIndexOffset = INDEX_NONE;
	uint32 MaterialParamValidMask = 0;

	int32 VFBoundOffsetsInParamStore[ENiagaraMeshVFLayout::Type::Num_Max];
	uint32 bSetAnyBoundVars : 1;

	const FNiagaraRendererLayout* RendererLayoutWithCustomSorting;
	const FNiagaraRendererLayout* RendererLayoutWithoutCustomSorting;
};
