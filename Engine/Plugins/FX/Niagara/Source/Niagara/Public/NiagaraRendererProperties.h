// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "RHIDefinitions.h"
#include "NiagaraTypes.h"
#include "NiagaraCommon.h"
#include "NiagaraMergeable.h"
#include "NiagaraPlatformSet.h"
#include "PSOPrecache.h"
#include "NiagaraRendererProperties.generated.h"

struct FVersionedNiagaraEmitterData;
class FNiagaraRenderer;
class FNiagaraSystemInstanceController;
class FVertexFactoryType;
class UMaterial;
class UMaterialInterface;
class UMaterialInstanceConstant;
class FMaterialUpdateContext;
class UTexture;
class FNiagaraEmitterInstance;
class SWidget;
class FAssetThumbnailPool;
struct FNiagaraDataSetCompiledData;
struct FSlateBrush;
struct FStaticParameterSet;
struct FStreamingRenderAssetPrimitiveInfo;
enum EMaterialUsage : int;
enum EMaterialDomain : int;

#define WITH_NIAGARA_RENDERER_DEBUGDRAW	WITH_EDITORONLY_DATA

#if WITH_EDITOR
// Helper class for GUI error handling
DECLARE_DELEGATE(FNiagaraRendererFeedbackFix);
class FNiagaraRendererFeedback
{
public:
	FNiagaraRendererFeedback(FText InDescriptionText, FText InSummaryText, FText InFixDescription = FText(), FNiagaraRendererFeedbackFix InFix = FNiagaraRendererFeedbackFix(), bool InDismissable = false)
		: DescriptionText(InDescriptionText)
		  , SummaryText(InSummaryText)
		  , FixDescription(InFixDescription)
		  , Fix(InFix)
		  , Dismissable(InDismissable)
	{}

	FNiagaraRendererFeedback(FText InSummaryText)
        : DescriptionText(FText())
          , SummaryText(InSummaryText)
          , FixDescription(FText())
          , Fix(FNiagaraRendererFeedbackFix())
	{}

	FNiagaraRendererFeedback()
	{}

	/** Returns true if the problem can be fixed automatically. */
	bool IsFixable() const
	{
		return Fix.IsBound();
	}

	/** Applies the fix if a delegate is bound for it.*/
	void TryFix() const
	{
		if (Fix.IsBound())
		{
			Fix.Execute();
		}
	}

	/** Full description text */
	FText GetDescriptionText() const
	{
		return DescriptionText;
	}

	/** Shortened error description text*/
	FText GetSummaryText() const
	{
		return SummaryText;
	}

	/** Full description text */
	FText GetFixDescriptionText() const
	{
		return FixDescription;
	}

	bool IsDismissable() const
	{
		return Dismissable;
	}

private:
	FText DescriptionText;
	FText SummaryText;
	FText FixDescription;
	FNiagaraRendererFeedbackFix Fix;
	bool Dismissable = false;
};
#endif

/** Mapping between a variable in the source dataset and the location we place it in the GPU buffer passed to the VF. */
struct FNiagaraRendererVariableInfo
{
	friend struct FNiagaraRendererLayout;

	static constexpr uint16 kInvalidOffset = 0xffff;

	FNiagaraRendererVariableInfo() {}
	FNiagaraRendererVariableInfo(int32 InDataOffset, int32 InGPUBufferOffset, int32 InNumComponents, bool bInUpload, bool bInHalfType)
		: DatasetOffset(static_cast<uint16>(InDataOffset))
		, GPUBufferOffset(static_cast<uint16>(InGPUBufferOffset))
		, NumComponents(static_cast<uint16>(InNumComponents))
		, bUpload(bInUpload)
		, bHalfType(bInHalfType)
	{
		check(int32(InDataOffset) < TNumericLimits<uint16>::Max());
		check(int32(InGPUBufferOffset) < TNumericLimits<uint16>::Max());
		check(int32(InNumComponents) <= TNumericLimits<uint16>::Max());
	}
	NIAGARA_API FNiagaraRendererVariableInfo(const FNiagaraDataSetCompiledData* CompiledData, const FNiagaraVariableBase& Variable);

	inline int32 GetNumComponents() const { return NumComponents; }

	inline int32 GetRawGPUOffset() const { return GPUBufferOffset == kInvalidOffset ? INDEX_NONE : GPUBufferOffset; }
	inline int32 GetGPUOffset() const { return GetRawGPUOffset() | (bHalfType ? (1 << 31) : 0); }

	inline int32 GetRawDatasetOffset() const { return DatasetOffset == kInvalidOffset ? INDEX_NONE : DatasetOffset; }
	inline int32 GetEncodedDatasetOffset() const { return GetRawDatasetOffset() | (bHalfType ? (1 << 31) : 0); }

	inline bool ShouldUpload() const { return bUpload; }
	inline bool IsHalfType() const { return bHalfType; }

protected:
	uint16 DatasetOffset = kInvalidOffset;
	uint16 GPUBufferOffset = kInvalidOffset;
	uint16 NumComponents = 0;
	bool bUpload = false;
	bool bHalfType = false;
};

/** Used for building renderer layouts for vertex factories */
struct FNiagaraRendererLayout
{
	NIAGARA_API void Initialize(int32 NumVariables);
	NIAGARA_API bool SetVariable(const FNiagaraDataSetCompiledData* CompiledData, const FNiagaraVariableBase& Variable, int32 VFVarOffset);
	NIAGARA_API bool SetVariableFromBinding(const FNiagaraDataSetCompiledData* CompiledData, const FNiagaraVariableAttributeBinding& VariableBinding, int32 VFVarOffset);
	NIAGARA_API void Finalize();

	TConstArrayView<FNiagaraRendererVariableInfo> GetVFVariables_GameThread() const { check(IsInGameThread() || IsInParallelGameThread()); return MakeArrayView(VFVariables_GT); }
	TConstArrayView<FNiagaraRendererVariableInfo> GetVFVariables_RenderThread() const { check(IsInParallelRenderingThread()); return MakeArrayView(VFVariables_RT); }
	int32 GetTotalFloatComponents_RenderThread() const { check(IsInParallelRenderingThread()); return TotalFloatComponents_RT; }
	int32 GetTotalHalfComponents_RenderThread() const { check(IsInParallelRenderingThread()); return TotalHalfComponents_RT; }

	SIZE_T GetAllocatedSize() const { return VFVariables_GT.GetAllocatedSize() + VFVariables_RT.GetAllocatedSize(); }

private:
	TArray<FNiagaraRendererVariableInfo> VFVariables_GT;
	TArray<FNiagaraRendererVariableInfo> VFVariables_RT;

	uint16 TotalFloatComponents_GT = 0;
	uint16 TotalHalfComponents_GT = 0;

	uint16 TotalFloatComponents_RT = 0;
	uint16 TotalHalfComponents_RT = 0;
};

UENUM()
enum class ENiagaraRendererSortPrecision : uint8
{
	/** Uses the project settings value. */
	Default,
	/** Low precision sorting, half float (fp16) precision, faster and adequate for most cases. */
	Low,
	/** High precision sorting, float (fp32) precision, slower but may fix sorting artifacts. */
	High,
};

UENUM()
enum class ENiagaraRendererGpuTranslucentLatency : uint8
{
	/** Uses the project default value. */
	ProjectDefault,
	/** Gpu simulations will always read this frames data for translucent materials. */
	Immediate,
	/** Gpu simulations will read the previous frames data if the simulation has to run in PostRenderOpaque. */
	Latent,
};

USTRUCT()
struct FNiagaraRendererMaterialScalarParameter
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Material")
	FName MaterialParameterName;

	UPROPERTY(EditAnywhere, Category = "Material")
	float Value = 0.0f;
};

USTRUCT()
struct FNiagaraRendererMaterialVectorParameter
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Material")
	FName MaterialParameterName;

	UPROPERTY(EditAnywhere, Category = "Material")
	FLinearColor Value = FLinearColor::Black;
};

USTRUCT()
struct FNiagaraRendererMaterialTextureParameter
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Material")
	FName MaterialParameterName;

	UPROPERTY(EditAnywhere, Category = "Material")
	TObjectPtr<UTexture> Texture;
};

USTRUCT()
struct FNiagaraRendererMaterialStaticBoolParameter
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Material")
	FName MaterialParameterName;

	UPROPERTY(EditAnywhere, Category = "Material")
	FName StaticVariableName;

	UPROPERTY()
	TOptional<bool> StaticValue;
};


/**
* Parameters to apply to the material, these are both constant and dynamic bindings
* Having any bindings set will cause a MID to be generated
*/
USTRUCT()
struct FNiagaraRendererMaterialParameters
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Material")
	TArray<FNiagaraMaterialAttributeBinding> AttributeBindings;

	UPROPERTY(EditAnywhere, Category = "Material")
	TArray<FNiagaraRendererMaterialScalarParameter> ScalarParameters;

	UPROPERTY(EditAnywhere, Category = "Material")
	TArray<FNiagaraRendererMaterialVectorParameter> VectorParameters;

	UPROPERTY(EditAnywhere, Category = "Material")
	TArray<FNiagaraRendererMaterialTextureParameter> TextureParameters;

	UPROPERTY(EditAnywhere, Category = "Material")
	TArray<FNiagaraRendererMaterialStaticBoolParameter> StaticBoolParameters;

	NIAGARA_API void ConditionalPostLoad();
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "Please change to use FVersionedNiagaraEmitterBase from FVersionedNiagaraEmitter")
	NIAGARA_API void RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode SourceMode);
	UE_DEPRECATED(5.7, "Please change to use FVersionedNiagaraEmitterBase from FVersionedNiagaraEmitter")
	NIAGARA_API void RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitter& InEmitter, ENiagaraRendererSourceDataMode SourceMode);

	NIAGARA_API void RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode SourceMode);
	NIAGARA_API void RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter, ENiagaraRendererSourceDataMode SourceMode);
	NIAGARA_API void GetFeedback(TArrayView<UMaterialInterface*> Materials, TArray<FNiagaraRendererFeedback>& OutWarnings) const;
#endif

	bool HasAnyBindings() const
	{
		return AttributeBindings.Num() > 0 || ScalarParameters.Num() > 0 || VectorParameters.Num() > 0 || TextureParameters.Num() > 0;
	}
};

/**
* Emitter properties base class
* Each EmitterRenderer derives from this with its own class, and returns it in GetProperties; a copy
* of those specific properties is stored on UNiagaraEmitter (on the System) for serialization
* and handed back to the System renderer on load.
*/
UCLASS(ABSTRACT, MinimalAPI)
class UNiagaraRendererProperties : public UNiagaraMergeable
{
	GENERATED_BODY()
	
public:
#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnPropertiesChanged);
#endif
public:
	UNiagaraRendererProperties()		
		: bIsEnabled(true)
		, bAllowInCullProxies(true)
	{
	}

	//UObject Interface Begin
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
#if WITH_EDITOR
	NIAGARA_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif //WITH_EDITOR
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	//UObject Interface End
	
	virtual FNiagaraRenderer* CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) PURE_VIRTUAL ( UNiagaraRendererProperties::CreateEmitterRenderer, return nullptr;);
	virtual class FNiagaraBoundsCalculator* CreateBoundsCalculator() PURE_VIRTUAL(UNiagaraRendererProperties::CreateBoundsCalculator, return nullptr;);
	virtual void GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const PURE_VIRTUAL(UNiagaraRendererProperties::GetUsedMaterials,);

	/** Returns a list of meshes used by this renderer. What type of object meets the definition of a mesh is up to the renderer to decide, so this might return more than just static meshes (e.g. geometry caches or skel meshes).
	 * If the emitter instance is not given then the values returned might not be representative of the meshes used at runtime. */
	virtual void GetUsedMeshes(const FNiagaraEmitterInstance* InEmitter, TArray<UObject*>& OutMeshes) const {}
	virtual const FVertexFactoryType* GetVertexFactoryType() const { return nullptr; }
	virtual bool IsBackfaceCullingDisabled() const { return false; }

	virtual float GetMaterialStreamingScale() const { return 1.0f; }
	virtual void GetStreamingMeshInfo(const FBoxSphereBounds& OwnerBounds, const FNiagaraEmitterInstance* InEmitter, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const {}

	virtual bool IsSimTargetSupported(ENiagaraSimTarget InSimTarget) const { return false; };

	const TArray<const FNiagaraVariableAttributeBinding*>& GetAttributeBindings() const { return AttributeBindings; }
	NIAGARA_API uint32 ComputeMaxUsedComponents(const FNiagaraDataSetCompiledData* CompiledDataSetData) const;

	NIAGARA_API virtual bool NeedsLoadForTargetPlatform(const class ITargetPlatform* TargetPlatform) const override;

	/** Method to add asset tags that are specific to this renderer. By default we add in how many instances of this class exist in the list.*/
	NIAGARA_API virtual void GetAssetTagsForContext(const UObject* InAsset, FGuid AssetVersion, const TArray<const UNiagaraRendererProperties*>& InProperties, TMap<FName, uint32>& NumericKeys, TMap<FName, FString>& StringKeys) const;

	/** In the case that we need parameters bound in that aren't Particle variables, these should be set up here so that the data is appropriately populated after the simulation.*/
	NIAGARA_API virtual bool PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore);

	/** Collect PSO precache data using the optional emitter instance */
	NIAGARA_API void CollectPSOPrecacheData(FNiagaraEmitterInstance* EmitterInstance, FMaterialInterfacePSOPrecacheParamsList& MaterialInterfacePSOPrecacheParamsList) const;

	/**
	* Collect all the data required for PSO precaching 
	*/
	struct FPSOPrecacheParams
	{
		UMaterialInterface* MaterialInterface = nullptr;
		FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
	};
	typedef TArray<FPSOPrecacheParams, TInlineAllocator<2> > FPSOPrecacheParamsList;
	virtual void CollectPSOPrecacheData(const FNiagaraEmitterInstance* InEmitter, FPSOPrecacheParamsList& OutParams) const {};

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool IsSupportedVariableForBinding(const FNiagaraVariableBase& InSourceForBinding, const FName& InTargetBindingName) const;

	/** Internal handling of any emitter variable renames. Note that this doesn't modify the renderer, the caller will need to do that if it is desired.*/
	NIAGARA_API virtual void RenameEmitter(const FName& InOldName, const UNiagaraEmitter* InRenamedEmitter);
	NIAGARA_API virtual void RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter);
	NIAGARA_API virtual void RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter);
	virtual bool IsMaterialValidForRenderer(UMaterial* Material, FText& InvalidMessage) { return true; }

	UE_DEPRECATED(5.7, "Please change to use FVersionedNiagaraEmitterBase from FVersionedNiagaraEmitter")
	NIAGARA_API virtual void RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitter& InEmitter);
	UE_DEPRECATED(5.7, "Please change to use FVersionedNiagaraEmitterBase from FVersionedNiagaraEmitter")
	NIAGARA_API virtual void RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitter& InEmitter);

	virtual void FixMaterial(UMaterial* Material) { }

	NIAGARA_API virtual TArray<FNiagaraVariable> GetBoundAttributes() const;

	/** returns the variable associated with the supplied binding if it should be bound given the current settings of the RendererProperties. */
	NIAGARA_API virtual FNiagaraVariable GetBoundAttribute(const FNiagaraVariableAttributeBinding* Binding) const;

	virtual const TArray<FNiagaraVariable>& GetRequiredAttributes() { static TArray<FNiagaraVariable> Vars; return Vars; };
	virtual const TArray<FNiagaraVariable>& GetOptionalAttributes() { static TArray<FNiagaraVariable> Vars; return Vars; };

	virtual void GetAdditionalVariables(TArray<FNiagaraVariableBase>& OutArray) const { };

	UNiagaraRendererProperties* StaticDuplicateWithNewMergeId(UObject* InOuter) const
	{
		return CastChecked<UNiagaraRendererProperties>(Super::StaticDuplicateWithNewMergeIdInternal(InOuter));
	}

	NIAGARA_API void CreateDefaultRendererWidget(TArray<TSharedPtr<SWidget>>& OutWidgets) const;
	NIAGARA_API void CreateRendererWidgetsForAssets(TConstArrayView<UObject*> InAssets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool, TArray<TSharedPtr<SWidget>>& OutWidgets) const;
	NIAGARA_API void CreateRendererWidgetsForAssets(TConstArrayView<UMaterialInterface*> InMaterials, TSharedPtr<FAssetThumbnailPool> InThumbnailPool, TArray<TSharedPtr<SWidget>>& OutWidgets) const;

	NIAGARA_API virtual void GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const;
	virtual void GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const PURE_VIRTUAL(UNiagaraRendererProperties::GetRendererTooltipWidgets, );
	virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FText>& OutErrors, TArray<FText>& OutWarnings, TArray<FText>& OutInfo) const {}
	NIAGARA_API virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const;

	/**
	Provides feedback on any material usage issues, i.e. incompatable material assignment or missing usage flags
	Automaticaly called from GetRendererFeedback where OutFeedback results in Errors
	*/
	virtual void GetRendererMaterialUsageFeedback(TArray<FNiagaraRendererFeedback>& OutFeedback) const {}

	NIAGARA_API void GetMaterialUsageFeedback(EMaterialUsage Usage, TConstArrayView<EMaterialDomain> InvalidMaterialDomains, TArray<FNiagaraRendererFeedback>& OutFeedback) const;
	NIAGARA_API void GetMaterialUsageFeedback(EMaterialUsage Usage, TArray<FNiagaraRendererFeedback>& OutFeedback) const;

	// The icon to display in the niagara stack widget under the renderer section
	NIAGARA_API virtual const FSlateBrush* GetStackIcon() const;

	// The text to display in the niagara stack widget under the renderer section
	NIAGARA_API virtual FText GetWidgetDisplayName() const;

	NIAGARA_API bool SupportsEmitterMode() const;
	NIAGARA_API ENiagaraRendererSourceDataMode ValidateSourceMode(ENiagaraRendererSourceDataMode InSourceMode) const;
#endif // WITH_EDITORONLY_DATA

#if WITH_NIAGARA_RENDERER_DEBUGDRAW
	NIAGARA_API virtual bool SupportsDebugDraw() const;
	NIAGARA_API virtual TOptional<FText> GetDebugDrawTooltip() const;
	NIAGARA_API virtual bool IsDebugDrawEnabled() const;
	NIAGARA_API virtual void SetDebugDrawEnabled(bool bInEnabled);
#endif

	virtual ENiagaraRendererSourceDataMode GetCurrentSourceMode() const {	return ENiagaraRendererSourceDataMode::Particles;}

	NIAGARA_API virtual bool GetIsActive() const;
	virtual bool GetIsEnabled() const { return bIsEnabled; }
	NIAGARA_API virtual void SetIsEnabled(bool bInIsEnabled);

	virtual void CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData) {}

	virtual bool NeedsMIDsForMaterials() const { return false; }

	/** When true, the renderer will be opted in to get its PostSystemTick_GameThread overload called */
	virtual bool NeedsSystemPostTick() const { return false; }
	/** When true, the renderer will be opted in to get its OnSystemComplete_GameThread overload called */
	virtual bool NeedsSystemCompletion() const { return false; }

	NIAGARA_API bool NeedsPreciseMotionVectors() const;
	virtual bool UseHeterogeneousVolumes() const { return false; }

	static NIAGARA_API bool IsSortHighPrecision(ENiagaraRendererSortPrecision SortPrecision);

	/** Should the Gpu translucent data be this frame or not? */
	static NIAGARA_API bool ShouldGpuTranslucentThisFrame(ENiagaraRendererGpuTranslucentLatency Latency);
	/** Is the Gpu translucent data going to be this frame, this can be restricted by things like feature level. */
	static NIAGARA_API bool IsGpuTranslucentThisFrame(ERHIFeatureLevel::Type FeatureLevel, ENiagaraRendererGpuTranslucentLatency Latency);

	template<typename TAction>
	void ForEachPlatformSet(TAction Func);

	NIAGARA_API FVersionedNiagaraEmitterData* GetEmitterData() const;
	[[nodiscard]] NIAGARA_API FVersionedNiagaraEmitter GetOuterEmitter() const;
	[[nodiscard]] NIAGARA_API FVersionedNiagaraEmitterBase GetOuterEmitterBase() const;

#if WITH_EDITORONLY_DATA
	// Checks to see if th renderer is contained with it's owning emitters renderer array
	// If not this is likely an orphaned or invalid renderer
	[[nodiscard]] bool IsContainedInEmitterRendererArray() const;
#endif

	NIAGARA_API void InitRendererBaseBindings();

	/** Platforms on which this renderer is enabled. */
	UPROPERTY(EditAnywhere, Category = "Scalability", meta=(DisplayInScalabilityContext))
	FNiagaraPlatformSet Platforms;

	/** By default, emitters are drawn in the order that they are added to the system. This value will allow you to control the order in a more fine-grained manner.
	Materials of the same type (i.e. Transparent) will draw in order from lowest to highest within the system. The default value is 0.*/
	UPROPERTY(EditAnywhere, Category = "Rendering")
	int32 SortOrderHint = 0;

	/** Hint about how to generate motion (velocity) vectors for this renderer. */
	UPROPERTY(EditAnywhere, Category = "Rendering")
	ENiagaraRendererMotionVectorSetting MotionVectorSetting = ENiagaraRendererMotionVectorSetting::AutoDetect;

	UPROPERTY(EditAnywhere, Category = "Rendering")
	bool bIsEnabled;

	UPROPERTY(EditAnywhere, Category = "Scalability")
	bool bAllowInCullProxies;

	/**
	Binding to control if the renderer is enabled or disabled.
	When disabled the renderer does not generate or render any particle data.
	When disabled via a static bool the renderer will be removed in cooked content.
	*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RendererEnabledBinding;
	
	UPROPERTY()
	FGuid OuterEmitterVersion;

protected:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bMotionBlurEnabled_DEPRECATED = true; // This has been rolled into MotionVectorSetting
#endif

	TArray<const FNiagaraVariableAttributeBinding*> AttributeBindings;

	NIAGARA_API virtual void PostLoadBindings(ENiagaraRendererSourceDataMode InSourceMode);
	NIAGARA_API virtual void UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit = false);

#if WITH_EDITORONLY_DATA
	/** utility function that can be used to fix up old vec3 bindings into position bindings. */
	static NIAGARA_API void ChangeToPositionBinding(FNiagaraVariableAttributeBinding& Binding);

	/** Generates the static parameter set for the parent emitter */
	bool BuildMaterialStaticParameterSet(const FNiagaraRendererMaterialParameters& MaterialParameters, const UMaterialInterface* Material, FStaticParameterSet& StaticParameterSet) const;

	/** Update MIC Static Parameters. */
	UE_DEPRECATED(5.5, "This helper function has been deprecated.  Use BuildMaterialStaticParameterSet instead.")
	NIAGARA_API bool UpdateMaterialStaticParameters(const FNiagaraRendererMaterialParameters& MaterialParameters, UMaterialInstanceConstant* MIC);

	/**
	Utility function to updates MICs.
	If your renderer uses static draw lists and you are unsure if the data could still be inside the renderer make sure to pass a valid material update context.
	The default is to skip updating anything as Niagara system should be recreated as part of the caller to update the MICs.
	*/
	NIAGARA_API void UpdateMaterialParametersMIC(const FNiagaraRendererMaterialParameters& MaterialParameters, TObjectPtr<UMaterialInterface>& InOutMaterial, TObjectPtr<UMaterialInstanceConstant>& InOutMIC, FMaterialUpdateContext* OptionalMaterialUpdateContext = nullptr);
	NIAGARA_API void UpdateMaterialParametersMIC(const FNiagaraRendererMaterialParameters& MaterialParameters, TArrayView<UMaterialInterface*> Materials, TArray<TObjectPtr<UMaterialInstanceConstant>>& InOutMICs, FMaterialUpdateContext* OptionalMaterialUpdateContext = nullptr);

	NIAGARA_API int32 GetDynamicParameterChannelMask(const FVersionedNiagaraEmitterData* EmitterData, FName BindingName, int32 DefaultChannelMask) const;
	NIAGARA_API int32 GetDynamicParameterCombinedChannelMask(FName Parameter0Name, FName Parameter1Name, FName Parameter2Name, FName Parameter3Name) const;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bDebugDrawEnabled = false;
#endif
#if WITH_EDITOR
	FOnPropertiesChanged OnPropertiesChangedDelegate;
#endif

	template<typename TAccessorType>
	void InitParticleDataSetAccessor(TAccessorType& Accessor, const FNiagaraDataSetCompiledData* CompiledData, const FNiagaraVariableAttributeBinding& Binding)
	{
		Accessor.Init(Binding.IsParticleBinding() ? CompiledData : nullptr, Binding.GetDataSetBindableVariable().GetName());
	}
};

template<typename TAction>
void UNiagaraRendererProperties::ForEachPlatformSet(TAction Func)
{
	Func(Platforms);
}
