// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraRenderer.h: Base class for Niagara render modules
==============================================================================*/
#pragma once

#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraRibbonVertexFactory.h"
#include "NiagaraRibbonCompute.h"
#include "NiagaraRenderer.h"
#include "NiagaraRibbonRendererProperties.h"
#include "Chaos/AABB.h"
#include "Chaos/AABB.h"
#include "Chaos/AABB.h"
#include "Chaos/AABB.h"
#include "Chaos/AABB.h"
#include "Chaos/AABB.h"

class FNiagaraDataSet;

struct FNiagaraDynamicDataRibbon;
class FNiagaraGpuRibbonsDataManager;
struct FNiagaraRibbonRenderingFrameResources;
struct FNiagaraRibbonRenderingFrameViewResources;
struct FNiagaraRibbonGPUInitParameters;
struct FNiagaraRibbonGPUInitComputeBuffers;
struct FNiagaraGenerationInputDataCPUAccessors;
struct FMeshBatch;

struct FNiagaraIndexGenerationInput
{
	float ViewDistance = 0.0f;
	float LODDistanceFactor = 0.0f;
	
	uint32 MaxSegmentCount = 0;
	uint32 SubSegmentCount = 0;

	uint32 SegmentBitShift = 0;
	uint32 SegmentBitMask = 0;

	uint32 SubSegmentBitShift = 0;
	uint32 SubSegmentBitMask = 0;

	uint32 ShapeBitMask = 0;
	
	uint32 TotalBitCount = 0;
	uint32 TotalNumIndices = 0;

	uint32 CPUTriangleCount = 0;
};

struct FNiagaraRibbonGenerationConfig
{
public:
	FNiagaraRibbonGenerationConfig(const UNiagaraRibbonRendererProperties* Properties)
		: MaterialParamValidMask(Properties->MaterialParamValidMask)
		, CurveTension(Properties->CurveTension)
		, MaxNumRibbons(Properties->MaxNumRibbons)
		, bHasFullRibbonIDs(Properties->RibbonFullIDDataSetAccessor.IsValid())
		, bHasSimpleRibbonIDs(Properties->RibbonIdDataSetAccessor.IsValid())
		, bHasTwist(Properties->TwistDataSetAccessor.IsValid() && Properties->SizeDataSetAccessor.IsValid())
		, bHasCustomU0Data(Properties->UV0Settings.bEnablePerParticleUOverride && Properties->U0OverrideIsBound)
		, bHasCustomU1Data(Properties->UV1Settings.bEnablePerParticleUOverride && Properties->U1OverrideIsBound)
		, bWantsConstantTessellation(Properties->TessellationMode == ENiagaraRibbonTessellationMode::Custom && Properties->bUseConstantFactor)
		, bWantsAutomaticTessellation(Properties->TessellationMode != ENiagaraRibbonTessellationMode::Disabled && !bWantsConstantTessellation)
		, bNeedsPreciseMotionVectors(Properties->NeedsPreciseMotionVectors())
	{
		
	}
	
	uint32 GetMaterialParamValidMask() const { return MaterialParamValidMask; }
	float GetCurveTension() const { return FMath::Min(CurveTension, 0.99f); }
	int32 GetMaxNumRibbons() const { return MaxNumRibbons; }
	
	bool HasFullRibbonIDs() const { return bHasFullRibbonIDs; }
	bool HasSimpleRibbonIDs() const { return bHasSimpleRibbonIDs; }
	bool HasRibbonIDs() const { return HasFullRibbonIDs() || HasSimpleRibbonIDs(); };
	
	bool HasTwist() const { return bHasTwist; }
	
	bool HasCustomU0Data() const { return bHasCustomU0Data; }
	bool HasCustomU1Data() const { return bHasCustomU1Data; }
	
	bool WantsConstantTessellation () const { return bWantsConstantTessellation; }
	bool WantsAutomaticTessellation() const { return bWantsAutomaticTessellation; }
	
	bool NeedsPreciseMotionVectors() const { return bNeedsPreciseMotionVectors; }

private:
	const uint32 MaterialParamValidMask;
	const float CurveTension;
	const int32 MaxNumRibbons;
	
	const uint32 bHasFullRibbonIDs : 1;
	const uint32 bHasSimpleRibbonIDs : 1;
	const uint32 bHasTwist : 1;
	
	const uint32 bHasCustomU0Data : 1;
	const uint32 bHasCustomU1Data : 1;
	const uint32 bWantsConstantTessellation : 1;
	const uint32 bWantsAutomaticTessellation : 1;	
	const uint32 bNeedsPreciseMotionVectors : 1;
};

struct FRibbonMultiRibbonInfoBufferEntry
{
	static constexpr int32 NumElements = 8;
	
	float U0Scale = 1.0;
	float U0Offset = 0.0;
	float U0DistributionScaler = 1.0;
	float U1Scale = 1.0;
	float U1Offset = 0.0;
	float U1DistributionScaler = 1.0;
	int32 FirstParticleId = INDEX_NONE;
	int32 LastParticleId = INDEX_NONE;
};

struct FRibbonMultiRibbonInfo
{	
	/** start and end world space position of the ribbon, to figure out draw direction */
	FVector StartPos;
	FVector EndPos;
	int32 BaseSegmentDataIndex = 0;
	int32 NumSegmentDataIndices = 0;

	FRibbonMultiRibbonInfoBufferEntry BufferEntry;
	

	inline bool UseInvertOrder(const FVector& ViewDirection, const FVector& ViewOriginForDistanceCulling, ENiagaraRibbonDrawDirection DrawDirection) const
	{
		const double StartDist = FVector::DotProduct(ViewDirection, StartPos - ViewOriginForDistanceCulling);
		const double EndDist = FVector::DotProduct(ViewDirection, EndPos - ViewOriginForDistanceCulling);
		return ((StartDist >= EndDist) && DrawDirection == ENiagaraRibbonDrawDirection::BackToFront)
			|| ((StartDist < EndDist) && DrawDirection == ENiagaraRibbonDrawDirection::FrontToBack);
	}

	void PackElementsToLookupTableBuffer(TArray<uint32>& OutputData) const
	{
		const int32 Index = OutputData.Num();
		OutputData.AddUninitialized(FRibbonMultiRibbonInfoBufferEntry::NumElements);		
		*reinterpret_cast<FRibbonMultiRibbonInfoBufferEntry*>(&OutputData[Index]) = BufferEntry;
	}
};


struct FNiagaraRibbonCPUGeneratedVertexData
{	
	// The list of all segments, each one connecting SortedIndices[SegmentId] to SortedIndices[SegmentId + 1].
	// We use this format because the final index buffer gets generated based on view sorting and InterpCount.
	TArray<uint32> SegmentData;

	/** The list of all particle (instance) indices. Converts raw indices to particles indices. Ordered along each ribbons, from head to tail. */
	TArray<uint32> SortedIndices;
	
	/** The tangent and distance between segments, for each raw index (raw VS particle indices). */
	TArray<FVector4f> TangentAndDistances;
	
	/** The multi ribbon index, for each raw index. (raw VS particle indices). */
	TArray<uint32> MultiRibbonIndices;
	
	/** Ribbon perperties required for sorting. */
	TArray<FRibbonMultiRibbonInfo, TInlineAllocator<1>> RibbonInfoLookup;

	float TotalSegmentLength = 0.0f;
	float AverageSegmentLength = 0.0f;
	float AverageSegmentAngle = 0.0f;
	float AverageTwistAngle = 0.0f;
	float AverageWidth = 0.0f;

	int32 GetAllocatedSize()const
	{
		int32 Size = 0;
		Size += int32(SegmentData.GetAllocatedSize());
		Size += int32(SortedIndices.GetAllocatedSize());
		Size += int32(TangentAndDistances.GetAllocatedSize());
		Size += int32(MultiRibbonIndices.GetAllocatedSize());
		Size += int32(RibbonInfoLookup.GetAllocatedSize());

		return Size;
	}
};

struct FNiagaraRibbonTessellationConfig
{
	ENiagaraRibbonTessellationMode TessellationMode;
	int32 CustomTessellationFactor;
	bool bCustomUseConstantFactor;
	float CustomTessellationMinAngle;
	bool bCustomUseScreenSpace;
};

struct FNiagaraRibbonTessellationSmoothingData
{
	// Average curvature of the segments.
	float TessellationAngle = 0;
	// Average curvature of the segments (computed from the segment angle in radian).
	float TessellationCurvature = 0;
	// Average twist of the segments.
	float TessellationTwistAngle = 0;
	// Average twist curvature of the segments.
	float TessellationTwistCurvature = 0;
	// Average twist curvature of the segments.
	float TessellationTotalSegmentLength = 0;
};

struct FNiagaraRibbonGpuBuffer
{
	FNiagaraRibbonGpuBuffer(const TCHAR* InDebugName, EPixelFormat InPixelFormat, uint32 InElementBytes)
		: DebugName(InDebugName)
		, PixelFormat(InPixelFormat)
		, ElementBytes(InElementBytes)
	{
	}

	~FNiagaraRibbonGpuBuffer()
	{
		Release();
	}

	bool Allocate(FRHICommandListBase& RHICmdList, uint32 NumElements, uint32 MaxElements, ERHIAccess InResourceState, bool bGpuReadOnly, EBufferUsageFlags AdditionalBufferUsage = EBufferUsageFlags::None);
	void Release();

	const TCHAR*				DebugName = nullptr;
	const EPixelFormat			PixelFormat = PF_Unknown;
	const uint32				ElementBytes = 0;

	uint32						NumBytes = 0;
	FBufferRHIRef				Buffer;
	FUnorderedAccessViewRHIRef	UAV;
	FShaderResourceViewRHIRef	SRV;
};

struct FNiagaraRibbonVertexBuffers
{
	FNiagaraRibbonVertexBuffers();

	FNiagaraRibbonGpuBuffer SortedIndicesBuffer;
	FNiagaraRibbonGpuBuffer TangentsAndDistancesBuffer;
	FNiagaraRibbonGpuBuffer MultiRibbonIndicesBuffer;
	FNiagaraRibbonGpuBuffer RibbonLookupTableBuffer;
	FNiagaraRibbonGpuBuffer SegmentsBuffer;
	FNiagaraRibbonGpuBuffer GPUComputeCommandBuffer;
	bool bJustCreatedCommandBuffer = false;

	void InitializeOrUpdateBuffers(FRHICommandListBase& RHICmdList, const FNiagaraRibbonGenerationConfig& GenerationConfig, const TSharedPtr<FNiagaraRibbonCPUGeneratedVertexData>& GeneratedGeometryData, const FNiagaraDataBuffer* SourceParticleData, int32 MaxAllocationCount, bool bIsUsingGPUInit);

	void Release()
	{
		SortedIndicesBuffer.Release();
		TangentsAndDistancesBuffer.Release();
		MultiRibbonIndicesBuffer.Release();
		RibbonLookupTableBuffer.Release();
		SegmentsBuffer.Release();
		GPUComputeCommandBuffer.Release();
	}
};

struct FNiagaraRibbonShapeGeometryData
{
	struct FVertex
	{
		static constexpr int32 NumElements = 5;
		
		FVector2f Position;
		FVector2f Normal;
		float TextureV;

		FVertex(const FVector2f& InPosition, const FVector2f& InNormal, float InTextureV)
			: Position(InPosition), Normal(InNormal), TextureV(InTextureV) { }
	};		
	static_assert(sizeof(FVertex) == (sizeof(float) * FVertex::NumElements));
	
	// This sets up the first and next vertex for each pair of triangles in the slice.
	// For a plane this will just be a linear set
	// For a multiplane it will be multiple separate linear sets
	// For a tube it will be a linear set that wraps back around to itself,
	// Same with the custom vertices.
	TArray<uint32, TInlineAllocator<32>> SliceTriangleToVertexIds;
	FReadBuffer SliceTriangleToVertexIdsBuffer;
	
	TArray<FVertex> SliceVertexData;		
	FReadBuffer SliceVertexDataBuffer;

	ENiagaraRibbonShapeMode Shape;
	
	int32 TrianglesPerSegment;
	int32 NumVerticesInSlice;
	int32 BitsNeededForShape;
	int32 BitMaskForShape;
	bool bDisableBackfaceCulling;
	bool bShouldFlipNormalToView;
};


/**
* NiagaraRendererRibbons renders an FNiagaraEmitterInstance as a ribbon connecting all particles
* in order by particle age.
*/
class FNiagaraRendererRibbons : public FNiagaraRenderer
{
public:
	NIAGARA_API FNiagaraRendererRibbons(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter);	// FNiagaraRenderer Interface 
	NIAGARA_API ~FNiagaraRendererRibbons();

	// FNiagaraRenderer Interface 
	NIAGARA_API virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	NIAGARA_API virtual void ReleaseRenderThreadResources() override;

	NIAGARA_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector, const FNiagaraSceneProxy *SceneProxy) const override;
	NIAGARA_API virtual FNiagaraDynamicDataBase *GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitteride) const override;
	NIAGARA_API virtual int32 GetDynamicDataSize()const override;
	NIAGARA_API virtual bool IsMaterialValid(const UMaterialInterface* Mat)const override;
#if RHI_RAYTRACING
	NIAGARA_API virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FNiagaraSceneProxy* Proxy) final override;
#endif

protected:
	
	static NIAGARA_API void GenerateShapeStateMultiPlane(FNiagaraRibbonShapeGeometryData& State, int32 MultiPlaneCount, int32 WidthSegmentationCount, bool bEnableAccurateGeometry, bool bUseMaterialBackfaceCulling);
	static NIAGARA_API void GenerateShapeStateTube(FNiagaraRibbonShapeGeometryData& State, int32 TubeSubdivisions, bool bUseMaterialBackfaceCulling);
	static NIAGARA_API void GenerateShapeStateCustom(FNiagaraRibbonShapeGeometryData& State, const TArray<FNiagaraRibbonShapeCustomVertex>& CustomVertices, bool bUseMaterialBackfaceCulling);
	static NIAGARA_API void GenerateShapeStatePlane(FNiagaraRibbonShapeGeometryData& State, int32 WidthSegmentationCount, bool bUseMaterialBackfaceCulling);
	NIAGARA_API void InitializeShape(const UNiagaraRibbonRendererProperties* Properties);
	
	NIAGARA_API void InitializeTessellation(const UNiagaraRibbonRendererProperties* Properties);
	
	template<typename IntType>
	static void CalculateUVScaleAndOffsets(const FNiagaraRibbonUVSettings& UVSettings, const TArray<IntType>& RibbonIndices, const TArray<FVector4f>& RibbonTangentsAndDistances, const FNiagaraDataSetReaderFloat<float>& NormalizedAgeReader,
		int32 StartIndex, int32 EndIndex, int32 NumSegments, float TotalLength, float& OutUScale, float& OutUOffset, float& OutUDistributionScaler);

	template<bool bWantsTessellation, bool bHasTwist, bool bWantsMultiRibbon>
	void GenerateVertexBufferForRibbonPart(const FNiagaraGenerationInputDataCPUAccessors& CPUData, TConstArrayView<uint32> RibbonIndices, uint32 RibbonIndex, FNiagaraRibbonCPUGeneratedVertexData& OutputData) const;

	template<typename IDType, typename ReaderType, bool bWantsTessellation, bool bHasTwist>
	void GenerateVertexBufferForMultiRibbonInternal(const FNiagaraGenerationInputDataCPUAccessors& CPUData, const ReaderType& IDReader, FNiagaraRibbonCPUGeneratedVertexData& OutputData) const;
	
	template<typename IDType, typename ReaderType>
	void GenerateVertexBufferForMultiRibbon(const FNiagaraGenerationInputDataCPUAccessors& CPUData, const ReaderType& IDReader, FNiagaraRibbonCPUGeneratedVertexData& OutputData) const;
	
	void GenerateVertexBufferCPU(const FNiagaraGenerationInputDataCPUAccessors& CPUData, FNiagaraRibbonCPUGeneratedVertexData& OutputData) const;

	int32 CalculateTessellationFactor(const FNiagaraSceneProxy* SceneProxy, const FSceneView* View, const FVector& ViewOriginForDistanceCulling) const;
	FNiagaraIndexGenerationInput CalculateIndexBufferConfiguration(const TSharedPtr<FNiagaraRibbonCPUGeneratedVertexData>& GeneratedVertices, const FNiagaraDataBuffer* SourceParticleData,
		const FNiagaraSceneProxy* SceneProxy, const FSceneView* View, const FVector& ViewOriginForDistanceCulling, bool bShouldUseGPUInitIndices, bool bIsGPUSim) const;
	
	void GenerateIndexBufferForView(
		FRHICommandListBase& RHICmdList,
		FNiagaraGpuRibbonsDataManager& GpuRibbonsDataManager, FMeshElementCollector& Collector,
		FNiagaraIndexGenerationInput& GeneratedData, FNiagaraDynamicDataRibbon* DynamicDataRibbon,
		FNiagaraRibbonRenderingFrameViewResources* RenderingViewResources, const FSceneView* View, const FVector& ViewOriginForDistanceCulling
	) const;
	
	template <typename TValue>
	static void GenerateIndexBufferCPU(
		FNiagaraIndexGenerationInput& GeneratedData, FNiagaraDynamicDataRibbon* DynamicDataRibbon, const FNiagaraRibbonShapeGeometryData& ShapeState,
		TValue* StartIndexBuffer, const FSceneView* View, const FVector& ViewOriginForDistanceCulling, ERHIFeatureLevel::Type FeatureLevel, ENiagaraRibbonDrawDirection DrawDirection
	);

	template <typename TValue>
	static TValue* AppendToIndexBufferCPU(TValue* OutIndices, const FNiagaraIndexGenerationInput& GeneratedData, const FNiagaraRibbonShapeGeometryData& ShapeState, const TArrayView<uint32>& SegmentData, bool bInvertOrder);
	
	void SetupPerViewUniformBuffer(FNiagaraIndexGenerationInput& GeneratedData,
	                               const FSceneView* View, const FSceneViewFamily& ViewFamily, const FNiagaraSceneProxy* SceneProxy, FNiagaraRibbonUniformBufferRef& OutUniformBuffer) const;

	void SetupMeshBatchAndCollectorResourceForView(FRHICommandListBase& RHICmdList, const FNiagaraIndexGenerationInput& GeneratedData, FNiagaraDynamicDataRibbon* DynamicDataRibbon,
	                                               const FNiagaraDataBuffer* SourceParticleData, const FSceneView* View, const FSceneViewFamily& ViewFamily, const FNiagaraSceneProxy* SceneProxy,
	                                               const TSharedPtr<FNiagaraRibbonRenderingFrameResources>& RenderingResources, FNiagaraRibbonRenderingFrameViewResources* RenderingViewResources, FMeshBatch& OutMeshBatch, bool bShouldUseGPUInitIndices) const;


	void InitializeViewIndexBuffersGPU(FRHICommandListImmediate& RHICmdList, FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, const FNiagaraRibbonGPUInitParameters& GpuInitParameters,
		const FNiagaraRibbonRenderingFrameViewResources* RenderingViewResources) const;

	void InitializeVertexBuffersResources(FRHICommandListBase& RHICmdList, const FNiagaraDynamicDataRibbon* DynamicDataRibbon, FNiagaraDataBuffer* SourceParticleData,
	                                      FGlobalDynamicReadBuffer& DynamicReadBuffer, const TSharedPtr<FNiagaraRibbonRenderingFrameResources>& RenderingResources, bool bShouldUseGPUInit) const;
	
	void InitializeVertexBuffersGPU(FRHICommandListImmediate& RHICmdList, FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, const FNiagaraRibbonGPUInitParameters& GpuInitParameters,
		FNiagaraRibbonGPUInitComputeBuffers& TempBuffers, const TSharedPtr<FNiagaraRibbonRenderingFrameResources>& RenderingResources) const;

	FRibbonComputeUniformParameters SetupComputeVertexGenParams(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, const TSharedPtr<FNiagaraRibbonRenderingFrameResources>& RenderingResources, const FNiagaraRibbonGPUInitParameters& GpuInitParameters) const;

	FNiagaraRibbonGenerationConfig GenerationConfig;
	
	FNiagaraRibbonUVSettings UV0Settings;
	FNiagaraRibbonUVSettings UV1Settings;
	FNiagaraRibbonShapeGeometryData ShapeState;
	FNiagaraRibbonTessellationConfig TessellationConfig;
	
	ENiagaraRibbonFacingMode FacingMode;
	ENiagaraRibbonDrawDirection DrawDirection;
	
	const FNiagaraRendererLayout* RendererLayout;
	uint32 bCastShadows : 1 = true;
#if WITH_EDITORONLY_DATA
	uint32 bIncludeInHitProxy : 1 = true;
#endif
	uint32 bUseGeometryNormals : 1 = false;
	uint32 bGpuRibbonLinkIsFloat : 1 = false;
	uint32 GpuRibbonLinkOrderOffset = INDEX_NONE;

	mutable UE::FMutex VertexBuffersGuard;
	mutable FNiagaraRibbonVertexBuffers VertexBuffers;
	
	mutable FNiagaraRibbonTessellationSmoothingData TessellationSmoothingData;
	int32 RibbonIDParamDataSetOffset;

	friend FNiagaraGpuRibbonsDataManager;
};
