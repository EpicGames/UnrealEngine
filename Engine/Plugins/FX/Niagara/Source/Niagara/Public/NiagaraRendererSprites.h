// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
NiagaraRendererSprites.h: Renderer for rendering Niagara particles as sprites.
==============================================================================*/

#pragma once

#include "Engine/EngineTypes.h"
#include "NiagaraRenderer.h"
#include "NiagaraSpriteRendererProperties.h"
#include "NiagaraSpriteVertexFactory.h"
#include "SceneManagement.h"

struct FNiagaraDynamicDataSprites;

/**
* FNiagaraRendererSprites renders an FNiagaraEmitterInstance as sprite particles
*/
class FNiagaraRendererSprites : public FNiagaraRenderer
{
public:
	NIAGARA_API FNiagaraRendererSprites(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter);
	NIAGARA_API ~FNiagaraRendererSprites();

	//FNiagaraRenderer interface
	NIAGARA_API virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	NIAGARA_API virtual void ReleaseRenderThreadResources() override;

	NIAGARA_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector, const FNiagaraSceneProxy *SceneProxy) const override;
	NIAGARA_API virtual FNiagaraDynamicDataBase* GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const override;
	NIAGARA_API virtual int GetDynamicDataSize()const override;
	NIAGARA_API virtual bool IsMaterialValid(const UMaterialInterface* Mat)const override;

#if RHI_RAYTRACING
		NIAGARA_API virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FNiagaraSceneProxy* Proxy) final override;
#endif
	//FNiagaraRenderer interface END

private:
	struct FParticleSpriteRenderData
	{
		const FNiagaraDynamicDataSprites*	DynamicDataSprites = nullptr;
		class FNiagaraDataBuffer*			SourceParticleData = nullptr;

		EBlendMode							BlendMode = BLEND_Opaque;
		bool								bHasTranslucentMaterials = false;
		bool								bCanCastTranslucentShadows = false;
		bool								bSortCullOnGpu = false;
		bool								bNeedsSort = false;
		bool								bNeedsCull = false;

		const FNiagaraRendererLayout*		RendererLayout = nullptr;
		ENiagaraSpriteVFLayout::Type		SortVariable = ENiagaraSpriteVFLayout::Type(INDEX_NONE);

		FRHIShaderResourceView*				ParticleFloatSRV = nullptr;
		FRHIShaderResourceView*				ParticleHalfSRV = nullptr;
		FRHIShaderResourceView*				ParticleIntSRV = nullptr;
		uint32								ParticleFloatDataStride = 0;
		uint32								ParticleHalfDataStride = 0;
		uint32								ParticleIntDataStride = 0;

		uint32								RendererVisTagOffset = INDEX_NONE;
	};

	/* Mesh collector classes */
	class FMeshCollectorResources : public FOneFrameResource
	{
	public:
		~FMeshCollectorResources() override { VertexFactory.ReleaseResource(); }

		FNiagaraSpriteVertexFactory		VertexFactory;
		FNiagaraSpriteUniformBufferRef	UniformBuffer;
	};

	void PrepareParticleSpriteRenderData(FRHICommandListBase& RHICmdList, FParticleSpriteRenderData& ParticleSpriteRenderData, const FSceneViewFamily& ViewFamily, FNiagaraDynamicDataBase* InDynamicData, const FNiagaraSceneProxy* SceneProxy, ENiagaraGpuComputeTickStage::Type GpuReadyTickStage) const;
	void PrepareParticleRenderBuffers(FRHICommandListBase& RHICmdList, FParticleSpriteRenderData& ParticleSpriteRenderData, FGlobalDynamicReadBuffer& DynamicReadBuffer) const;
	void InitializeSortInfo(FParticleSpriteRenderData& ParticleSpriteRenderData, const FNiagaraSceneProxy& SceneProxy, const FSceneView& View, int32 ViewIndex, FNiagaraGPUSortInfo& OutSortInfo) const;
	void SetupVertexFactory(FRHICommandListBase& RHICmdList, FParticleSpriteRenderData& ParticleSpriteRenderData, FNiagaraSpriteVertexFactory& VertexFactory) const;
	FNiagaraSpriteUniformBufferRef CreateViewUniformBuffer(FParticleSpriteRenderData& ParticleSpriteRenderData, const FSceneView& View, const FSceneViewFamily& ViewFamily, const FNiagaraSceneProxy& SceneProxy, FNiagaraSpriteVertexFactory& VertexFactory) const;

	void CreateMeshBatchForView(
		FRHICommandListBase& RHICmdList,
		FParticleSpriteRenderData& ParticleSpriteRenderData,
		FMeshBatch& MeshBatch,
		const FSceneView& View,
		const FNiagaraSceneProxy& SceneProxy,
		FNiagaraSpriteVertexFactory& VertexFactory,
		uint32 NumInstances,
		uint32 GPUCountBufferOffset,
		bool bDoGPUCulling
	) const;

	//Cached data from the properties struct.
	ENiagaraRendererSourceDataMode SourceMode;
	ENiagaraSpriteAlignment Alignment;
	ENiagaraSpriteFacingMode FacingMode;
	ENiagaraSortMode SortMode;
	FVector2f PivotInUVSpace;
	float MacroUVRadius;
	FVector2f SubImageSize;

	uint32 NumIndicesPerInstance;

	uint32 bSubImageBlend : 1;
	uint32 bRemoveHMDRollInVR : 1;
	uint32 bSortHighPrecision : 1;
	uint32 bSortOnlyWhenTranslucent : 1;
	uint32 bGpuLowLatencyTranslucency : 1;
	uint32 bEnableCulling : 1;
	uint32 bEnableDistanceCulling : 1;
	uint32 bAccurateMotionVectors : 1;
	uint32 bCastShadows : 1;
#if WITH_EDITORONLY_DATA
	uint32 bIncludeInHitProxy : 1 = true;
#endif
	uint32 bSetAnyBoundVars : 1;
	uint32 bVisTagInParamStore : 1;

	ENiagaraRendererPixelCoverageMode PixelCoverageMode = ENiagaraRendererPixelCoverageMode::Automatic;
	float PixelCoverageBlend = 0.0f;

	float MinFacingCameraBlendDistance;
	float MaxFacingCameraBlendDistance;
	FVector2f DistanceCullRange;
	FNiagaraCutoutVertexBuffer CutoutVertexBuffer;
	int32 NumCutoutVertexPerSubImage = 0;
	uint32 MaterialParamValidMask = 0;

	int32 RendererVisTagOffset;
	int32 RendererVisibility;

	int32 VFBoundOffsetsInParamStore[ENiagaraSpriteVFLayout::Type::Num_Max];

	const FNiagaraRendererLayout* RendererLayoutWithCustomSort;
	const FNiagaraRendererLayout* RendererLayoutWithoutCustomSort;
};
