// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraRenderer.h"
#include "NiagaraVolumeRendererProperties.h"

#include "LocalVertexFactory.h"

class USceneComponent;

class FNiagaraRendererVolumes : public FNiagaraRenderer
{
public:
	NIAGARA_API explicit FNiagaraRendererVolumes(ERHIFeatureLevel::Type FeatureLevel, const UNiagaraRendererProperties *InProps, const FNiagaraEmitterInstance* Emitter);
	NIAGARA_API ~FNiagaraRendererVolumes();

	//FNiagaraRenderer interface
	NIAGARA_API virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	NIAGARA_API virtual void ReleaseRenderThreadResources() override;

	NIAGARA_API virtual bool IsMaterialValid(const UMaterialInterface* Material) const override;

	NIAGARA_API virtual FNiagaraDynamicDataBase *GenerateDynamicData(const FNiagaraSceneProxy* Proxy, const UNiagaraRendererProperties* InProperties, const FNiagaraEmitterInstance* Emitter) const override;
	NIAGARA_API virtual int GetDynamicDataSize() const override;

	NIAGARA_API void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector, const FNiagaraSceneProxy* SceneProxy) const;
#if RHI_RAYTRACING
	NIAGARA_API void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FNiagaraSceneProxy* SceneProxy);
#endif //RHI_RAYTRACING
	//FNiagaraRenderer interface END


protected:
	ENiagaraRendererSourceDataMode	SourceMode = ENiagaraRendererSourceDataMode::Emitter;
	int32							RendererVisibilityTag = 0;

	FNiagaraDataSetAccessor<FNiagaraPosition>	PositionDataSetAccessor;
	FNiagaraDataSetAccessor<FQuat4f>			RotationDataSetAccessor;
	FNiagaraDataSetAccessor<FVector3f>			ScaleDataSetAccessor;
	FNiagaraDataSetAccessor<int32>				RendererVisibilityTagAccessor;
	FNiagaraDataSetAccessor<int32>				VolumeResolutionMaxAxisAccessor;
	FNiagaraDataSetAccessor<FVector3f>			VolumeWorldSpaceSizeAccessor;

	bool							bAnyVFBoundOffsets = false;
	int32							VFBoundOffsetsInParamStore[int32(ENiagaraVolumeVFLayout::Num)];

	FLocalVertexFactory				VertexFactory;

};
