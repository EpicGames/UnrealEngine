// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSetAccessor.h"
#include "NiagaraRibbonRendererProperties.generated.h"

class UMaterialInstanceConstant;
class FNiagaraEmitterInstance;
class FAssetThumbnailPool;
class FVertexFactoryType;
class SWidget;

UENUM()
enum class ENiagaraRibbonFacingMode : uint8
{
	/** Have the ribbon face the screen. */
	Screen = 0,

	/** Use Particles.RibbonFacing as the facing vector. */
	Custom,

	/** Use Particles.RibbonFacing as the side vector, and calculate the facing vector from that.
	 *  Using ribbon twist with this mode is NOT supported.
	 */
	CustomSideVector
};

/** Defines different modes for offsetting UVs by age when ordering ribbon particles using normalized age. */
UENUM()
enum class ENiagaraRibbonAgeOffsetMode : uint8
{
	/** Offset the UVs by age for smooth texture movement, but scale the 0-1 UV range to the current normalized age range of the particles. */
	Scale,
	/** Offset the UVs by age for smooth texture movement, but use the normalized age range directly as the UV range which will clip the texture for normalized age ranges that don't go from 0-1. */
	Clip
};

/** This enum decides in which order the ribbon segments will be rendered */
UENUM()
enum class ENiagaraRibbonDrawDirection : uint8
{
	FrontToBack,
	BackToFront
};

UENUM()
enum class ENiagaraRibbonShapeMode : uint8
{
	/** Default shape, flat plane facing the camera. */
	Plane,
	/** Multiple Planes evenly rotated around the axis to 180 degrees. */
	MultiPlane,
	/** 3D Tube shape, from triangular to cylindrical depending on vertex count. */
	Tube,
	/** Custom shape, defined by cross section. */
	Custom
};

USTRUCT()
struct FNiagaraRibbonShapeCustomVertex
{
	GENERATED_BODY();

	FNiagaraRibbonShapeCustomVertex();

	UPROPERTY(EditAnywhere, Category = "Ribbon Rendering")
	FVector2f Position;

	UPROPERTY(EditAnywhere, Category = "Ribbon Rendering")
	FVector2f Normal;

	UPROPERTY(EditAnywhere, Category = "Ribbon Rendering")
	float TextureV;
};

UENUM()
enum class ENiagaraRibbonTessellationMode : uint8
{
	/** Default tessellation parameters. */
	Automatic,
	/** Custom tessellation parameters. */
	Custom,
	/** Disable tessellation entirely. */
	Disabled
};

/** Specifies options for handling UVs at the leading and trailing edges of ribbons. */
UENUM()
enum class ENiagaraRibbonUVEdgeMode : uint8
{
	/** The UV value at the edge will smoothly transition across the segment using normalized age.
	This will result in	UV values which are outside of the standard 0-1 range and works best with
	clamped textures. */
	SmoothTransition,
	/** The UV value at the edge will be locked to 0 at the leading edge, or locked to 1 at the
	Trailing edge. */
	Locked,
};

/** Specifies options for distributing UV values across ribbon segments. */
UENUM()
enum class ENiagaraRibbonUVDistributionMode : uint8
{
	/** Ribbon UVs will stretch the length of the ribbon, without repeating, but distributed by segment, so can be uneven with unequal length segments. */
	ScaledUniformly UMETA(DisplayName = "Uniform Scale (By Segment)"),
	/** Ribbon UVs will stretch the length of the ribbon, without repeating, but account for segment length to make an even distribution the entire length of the ribbon. */
	ScaledUsingRibbonSegmentLength UMETA(DisplayName = "Non-Uniform Scale (By Total Length)"),
	/** Ribbon UVs will be tiled along the length of the ribbon evenly, based on TilingLength setting. */
	TiledOverRibbonLength UMETA(DisplayName = "Tiled (By Segment Length)"),
	/** Ribbon UVs will be tiled along the length of the ribbon evenly, based on RibbonUVDistance parameter and the TilingLength scale value, to create 'traintrack' style UVs. NOTE: Dependent on Particle Attribute RibbonUVDistance */
	TiledFromStartOverRibbonLength UMETA(DisplayName = "Tiled By Distance (By Particles.RibbonUVDistance)")
};



/** Defines settings for UV behavior for a UV channel on ribbons. */
USTRUCT()
struct FNiagaraRibbonUVSettings
{
	GENERATED_BODY();

	FNiagaraRibbonUVSettings();


	/** Specifies how ribbon UVs are distributed along the length of a ribbon. */
	UPROPERTY(EditAnywhere, Category = UVs, meta = (DisplayName="UV Mode", EditCondition = "!bEnablePerParticleUOverride", EditConditionHides))
	ENiagaraRibbonUVDistributionMode DistributionMode;

	/** Specifies how UVs transition into life at the leading edge of the ribbon. */
	UPROPERTY(EditAnywhere, Category = UVs, meta = (DisplayName="Leading Edge Transition", EditCondition = "!bEnablePerParticleUOverride && DistributionMode != ENiagaraRibbonUVDistributionMode::TiledFromStartOverRibbonLength", EditConditionHides))
	ENiagaraRibbonUVEdgeMode LeadingEdgeMode;

	/** Specifies how UVs transition out of life at the trailing edge of the ribbon. */
	UPROPERTY(EditAnywhere, Category = UVs, meta = (DisplayName="Trailing Edge Transition", EditCondition = "!bEnablePerParticleUOverride && DistributionMode != ENiagaraRibbonUVDistributionMode::TiledOverRibbonLength && DistributionMode != ENiagaraRibbonUVDistributionMode::TiledFromStartOverRibbonLength", EditConditionHides))
	ENiagaraRibbonUVEdgeMode TrailingEdgeMode;

	/** Enables overriding the U component with values read from the particles. When enabled, edge behavior and distribution are ignored. */
	UPROPERTY(EditAnywhere, Category = UVs)
	uint8 bEnablePerParticleUOverride : 1;

	/** Enables overriding the range of the V component with values read from the particles. */
	UPROPERTY(EditAnywhere, Category = UVs)
	uint8 bEnablePerParticleVRangeOverride : 1;

	/** Specifies the length in world units to use when tiling UVs across the ribbon when using one of the tiled distribution modes. */
	UPROPERTY(EditAnywhere, Category = UVs, meta = (EditCondition="!bEnablePerParticleUOverride && DistributionMode == ENiagaraRibbonUVDistributionMode::TiledOverRibbonLength || DistributionMode == ENiagaraRibbonUVDistributionMode::TiledFromStartOverRibbonLength", EditConditionHides))
	float TilingLength;

	/** Specifies an additional offset which is applied to the UV range */
	UPROPERTY(EditAnywhere, Category = UVs)
	FVector2D Offset;

	/** Specifies an additional scaler which is applied to the UV range. */
	UPROPERTY(EditAnywhere, Category = UVs)
	FVector2D Scale;
};

namespace ENiagaraRibbonVFLayout
{
	enum Type
	{
		Position,
		Velocity,
		Color,
		Width,
		Twist,
		Facing,
		NormalizedAge,
		MaterialRandom,
		MaterialParam0,
		MaterialParam1,
		MaterialParam2,
		MaterialParam3,
		DistanceFromStart,
		U0Override,
		V0RangeOverride,
		U1Override,
		V1RangeOverride,
		PrevPosition,
		PrevRibbonWidth,
		PrevRibbonFacing,
		PrevRibbonTwist,
		Num,
	};
};

UCLASS(editinlinenew, meta = (DisplayName = "Ribbon Renderer", SupportsStateless), MinimalAPI)
class UNiagaraRibbonRendererProperties : public UNiagaraRendererProperties
{
public:
	GENERATED_BODY()

	NIAGARA_API UNiagaraRibbonRendererProperties();

	//UObject Interface
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
	NIAGARA_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
#if WITH_EDITOR
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	NIAGARA_API virtual void RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
	NIAGARA_API virtual void RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
#endif
	//UObject Interface END

	static NIAGARA_API void InitCDOPropertiesAfterModuleStartup();

	//UNiagaraRendererProperties Interface
	NIAGARA_API virtual FNiagaraRenderer* CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) override;
	NIAGARA_API virtual class FNiagaraBoundsCalculator* CreateBoundsCalculator() override;
	NIAGARA_API virtual void GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const override;
	NIAGARA_API virtual const FVertexFactoryType* GetVertexFactoryType() const override;
	NIAGARA_API virtual bool IsBackfaceCullingDisabled() const override;
	virtual bool IsSimTargetSupported(ENiagaraSimTarget InSimTarget) const override { return true; };
	NIAGARA_API virtual bool PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore) override;
	NIAGARA_API virtual void CollectPSOPrecacheData(const FNiagaraEmitterInstance* InEmitter, FPSOPrecacheParamsList& OutParams) const override;

#if WITH_EDITOR
	NIAGARA_API virtual const TArray<FNiagaraVariable>& GetOptionalAttributes() override;
	NIAGARA_API virtual void GetAdditionalVariables(TArray<FNiagaraVariableBase>& OutArray) const override;
	NIAGARA_API virtual FNiagaraVariable GetBoundAttribute(const FNiagaraVariableAttributeBinding* Binding) const override;
	NIAGARA_API virtual void GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	NIAGARA_API virtual void GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	NIAGARA_API virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const override;
	NIAGARA_API virtual void GetRendererMaterialUsageFeedback(TArray<FNiagaraRendererFeedback>& OutFeedback) const override;

	NIAGARA_API virtual TArray<FNiagaraVariable> GetBoundAttributes() const override; 
#endif
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool IsSupportedVariableForBinding(const FNiagaraVariableBase& InSourceForBinding, const FName& InTargetBindingName) const override;
#endif
	NIAGARA_API virtual void CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData) override;
	//UNiagaraRendererProperties Interface END

	UPROPERTY(EditAnywhere, Category = "Ribbon Rendering")
	TObjectPtr<UMaterialInterface> Material;

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	TObjectPtr<UMaterialInstanceConstant> MICMaterial;
#endif

	int32 GetWidthSegmentationCount() const { return FMath::Clamp(WidthSegmentationCount, 1, 16); }

	/** Use the UMaterialInterface bound to this user variable if it is set to a valid value. If this is bound to a valid value and Material is also set, UserParamBinding wins.*/
	UPROPERTY(EditAnywhere, Category = "Ribbon Rendering")
	FNiagaraUserParameterBinding MaterialUserParamBinding;

	UPROPERTY(EditAnywhere, Category = "Ribbon Rendering", meta=(DisplayName="UV0 Settings"))
	FNiagaraRibbonUVSettings UV0Settings;

	UPROPERTY(EditAnywhere, Category = "Ribbon Rendering", meta=(DisplayName="UV1 Settings"))
	FNiagaraRibbonUVSettings UV1Settings;

	UPROPERTY(EditAnywhere, Category = "Ribbon Rendering", meta=(DisplayAfter="MaterialUserParamBinding"))
	ENiagaraRibbonFacingMode FacingMode = ENiagaraRibbonFacingMode::Screen;

#if WITH_EDITORONLY_DATA
private:
	/** Tiles UV0 based on the distance traversed by the ribbon. Disables offsetting UVs by age. */
	UPROPERTY()
	float UV0TilingDistance_DEPRECATED;
	UPROPERTY()
	FVector2D UV0Scale_DEPRECATED;
	UPROPERTY()
	FVector2D UV0Offset_DEPRECATED;

	/** Defines the mode to use when offsetting UV channel 0 by age which enables smooth texture movement when particles are added and removed at the end of the ribbon.  Not used when the RibbonLinkOrder binding is in use or when tiling distance is in use. */
	UPROPERTY()
	ENiagaraRibbonAgeOffsetMode UV0AgeOffsetMode_DEPRECATED;

	/** Tiles UV1 based on the distance traversed by the ribbon. Disables offsetting UVs by age. */
	UPROPERTY()
	float UV1TilingDistance_DEPRECATED;
	UPROPERTY()
	FVector2D UV1Scale_DEPRECATED;
	UPROPERTY()
	FVector2D UV1Offset_DEPRECATED;

	/** Defines the mode to use when offsetting UV channel 1 by age which enables smooth texture movement when particles are added and removed at the end of the ribbon.  Not used when the RibbonLinkOrder binding is in use or when tiling distance is in use. */
	UPROPERTY()
	ENiagaraRibbonAgeOffsetMode UV1AgeOffsetMode_DEPRECATED;
#endif

public:
	UPROPERTY(EditAnywhere, Category = "Ribbon Rendering")	
	int32 MaxNumRibbons;
	
	/** Controls the order the ribbon segments will be rendered. */
	UPROPERTY(EditAnywhere, Category = "Sorting")
	ENiagaraRibbonDrawDirection DrawDirection;

	/** Shape of the ribbon, from flat plane, multiplane, 3d tube, and custom shapes. */
	UPROPERTY(EditAnywhere, Category = "Ribbon Shape")
	ENiagaraRibbonShapeMode Shape;

	/** Disables two-sided forced rendering (Will still respect material settings)
	  * MultiPlane will double geometry count to have triangles facing both sides. With this off MultiPlane will switch normal direction to face view.
	  * 3D Ribbons will render like normal meshes with backface culling enabled.
	  */
	UPROPERTY(EditAnywhere, Category = "Ribbon Shape", meta = (EditCondition = "Shape != ENiagaraRibbonShapeMode::Plane", EditConditionHides))
	uint8 bEnableAccurateGeometry : 1;

	/** When enabled the ribbons renderer will not override how backface culling works depending on shape type, but instad use the material culling mode */
	UPROPERTY(EditAnywhere, Category = "Ribbon Shape")
	uint8 bUseMaterialBackfaceCulling : 1;

	/** When enabled the ribbons normals will follow the shape of the geometry rather than being aligned to screen / custom facing. */
	UPROPERTY(EditAnywhere, Category = "Ribbon Shape", meta = (EditCondition = "Shape == ENiagaraRibbonShapeMode::Plane", EditConditionHides))
	uint8 bUseGeometryNormals : 1;

	/**
	*	Whether we use the CPU or GPU to generate ribbon geometry for CPU systems.
	*	GPU systems will always use a fully GPU initialization pipeline,
	*	Will fall back to CPU init when GPU init isn't available.
	*/ 
	UPROPERTY(EditAnywhere, Category = "Ribbon Rendering")	
	uint8 bUseGPUInit : 1;

	/** If checked, use the above constant factor. Otherwise, adaptively select the tessellation factor based on the below parameters. */
	UPROPERTY(EditAnywhere, Category = "Ribbon Tessellation", meta = (EditCondition = "TessellationMode == ENiagaraRibbonTessellationMode::Custom", EditConditionHides, DisplayAfter="TessellationFactor"))
	uint8 bUseConstantFactor : 1;

	/** If checked, use the ribbon's screen space percentage to adaptively adjust the tessellation factor. */
	UPROPERTY(EditAnywhere, Category = "Ribbon Tessellation", meta = (DisplayName = "Screen Space", EditCondition = "TessellationMode == ENiagaraRibbonTessellationMode::Custom && !bUseConstantFactor", EditConditionHides, DisplayAfter="TessellationAngle"))
	uint8 bScreenSpaceTessellation : 1;

	UPROPERTY()
	uint8 bLinkOrderUseUniqueID : 1;

	/**
	When disabled the renderer will not cast shadows.
	The component controls if shadows are enabled, this flag allows you to disable the renderer casting shadows.
	*/
	UPROPERTY(EditAnywhere, Category = "Rendering")
	uint8 bCastShadows : 1 = 1;

#if WITH_EDITORONLY_DATA
	/**
	When disabled the mesh batches do not contribute to the hit proxy, i.e. they are not part of editor selection.
	*/
	UPROPERTY(EditAnywhere, Category = "Rendering")
	uint8 bIncludeInHitProxy : 1 = 1;
#endif

	/** Tessellation factor to apply to the width of the ribbon.
	* Ranges from 1 to 16. Greater values increase amount of tessellation.
	*/
	UPROPERTY(EditAnywhere, Category = "Ribbon Shape", meta = (EditCondition = "Shape == ENiagaraRibbonShapeMode::Plane || Shape == ENiagaraRibbonShapeMode::MultiPlane", ClampMin = "1", ClampMax = "16", EditConditionHides))
	int32 WidthSegmentationCount;

	/** Number of planes in multiplane shape. Evenly distributed from 0-90 or 0-180 degrees off camera facing depending on setting */
	UPROPERTY(EditAnywhere, Category = "Ribbon Shape", meta = (EditCondition = "Shape == ENiagaraRibbonShapeMode::MultiPlane", ClampMin = "2", ClampMax = "8", EditConditionHides))
	int32 MultiPlaneCount;

	/** Number of vertices/faces in a tube.  */
	UPROPERTY(EditAnywhere, Category = "Ribbon Shape", meta = (EditCondition = "Shape == ENiagaraRibbonShapeMode::Tube", ClampMin = "3", ClampMax = "16", EditConditionHides))
	int32 TubeSubdivisions;

	/** Vertices for a cross section of the ribbon in custom shape mode. */
	UPROPERTY(EditAnywhere, Category = "Ribbon Shape", meta = (EditCondition = "Shape == ENiagaraRibbonShapeMode::Custom", EditConditionHides))
	TArray<FNiagaraRibbonShapeCustomVertex> CustomVertices;

	/** Defines the tessellation mode allowing custom tessellation parameters or disabling tessellation entirely. */
	UPROPERTY(EditAnywhere, Category = "Ribbon Tessellation", meta = (DisplayName = "Mode"))
	ENiagaraRibbonTessellationMode TessellationMode;

	/** Defines the curve tension, or how long the curve's tangents are.
	  * Ranges from 0 to 1. The higher the value, the sharper the curve becomes.
	  */
	UPROPERTY(EditAnywhere, Category = "Ribbon Tessellation", meta = (EditCondition = "TessellationMode != ENiagaraRibbonTessellationMode::Disabled", EditConditionHides, ClampMin = "0", ClampMax = "0.99"))
	float CurveTension;

	/** Custom tessellation factor.
	  * Ranges from 1 to 16. Greater values increase amount of tessellation.
	  */
	UPROPERTY(EditAnywhere, Category = "Ribbon Tessellation", meta = (DisplayName = "Max Tessellation Factor", EditCondition = "TessellationMode == ENiagaraRibbonTessellationMode::Custom", EditConditionHides, ClampMin = "1", ClampMax = "16"))
	int32 TessellationFactor;

	/** Defines the angle in degrees at which tessellation occurs.
	  * Ranges from 1 to 180. Smaller values increase amount of tessellation.
	  * If set to 0, use the maximum tessellation set above.
	  */
	UPROPERTY(EditAnywhere, Category = "Ribbon Tessellation", meta = (EditCondition = "TessellationMode == ENiagaraRibbonTessellationMode::Custom && !bUseConstantFactor", ClampMin = "0", ClampMax = "180", UIMin = "1", UIMax = "180", EditConditionHides))
	float TessellationAngle;

	/** Which attribute should we use for position when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding PositionBinding;

	/** Which attribute should we use for color when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding ColorBinding;

	/** Which attribute should we use for velocity when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding VelocityBinding;

	/** Which attribute should we use for normalized age when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding NormalizedAgeBinding;

	/** Which attribute should we use for ribbon twist when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RibbonTwistBinding;

	/** Which attribute should we use for ribbon width when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RibbonWidthBinding;

	/** Which attribute should we use for ribbon facing when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RibbonFacingBinding;

	/** Which attribute should we use for ribbon id when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RibbonIdBinding;

	/** Which attribute should we use for RibbonLinkOrder when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RibbonLinkOrderBinding;

	/** Which attribute should we use for MaterialRandom when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding MaterialRandomBinding;

	/** Which attribute should we use for dynamic material parameters when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterialBinding;

	/** Which attribute should we use for dynamic material parameters when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterial1Binding;

	/** Which attribute should we use for dynamic material parameters when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterial2Binding;

	/** Which attribute should we use for dynamic material parameters when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterial3Binding;

	/** Which attribute should we use for ribbon distance traveled for use in UV operations when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RibbonUVDistance;

	/** Which attribute should we use for UV0 U when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding U0OverrideBinding;

	/** Which attribute should we use for UV0 V when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding V0RangeOverrideBinding;

	/** Which attribute should we use for UV1 U when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding U1OverrideBinding;

	/** Which attribute should we use for UV1 V when generating ribbons?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding V1RangeOverrideBinding;

	/** If this array has entries, we will create a MaterialInstanceDynamic per Emitter instance from Material and set the Material parameters using the Niagara simulation variables listed.*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraRendererMaterialParameters MaterialParameters;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FNiagaraMaterialAttributeBinding> MaterialParameterBindings_DEPRECATED;
#endif

	/** Implicit binding for previous position */
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevPositionBinding;

	/** Implicit binding for previous ribbon width */
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevRibbonWidthBinding;

	/** Implicit binding for previous ribbon facing */
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevRibbonFacingBinding;

	/** Implicit binding for previous ribbon twist */
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevRibbonTwistBinding;

	UPROPERTY()
	uint32 MaterialParamValidMask = 0;

	FNiagaraDataSetAccessor<float>		RibbonLinkOrderFloatAccessor;
	FNiagaraDataSetAccessor<int32>		RibbonLinkOrderInt32Accessor;
	FNiagaraDataSetAccessor<FNiagaraPosition>	PositionDataSetAccessor;
	FNiagaraDataSetAccessor<float>		NormalizedAgeAccessor;
	FNiagaraDataSetAccessor<float>		SizeDataSetAccessor;
	FNiagaraDataSetAccessor<float>		TwistDataSetAccessor;
	FNiagaraDataSetAccessor<FVector3f>	FacingDataSetAccessor;
	FNiagaraDataSetAccessor<FVector4f>	MaterialParam0DataSetAccessor;
	FNiagaraDataSetAccessor<FVector4f>	MaterialParam1DataSetAccessor;
	FNiagaraDataSetAccessor<FVector4f>	MaterialParam2DataSetAccessor;
	FNiagaraDataSetAccessor<FVector4f>	MaterialParam3DataSetAccessor;
	bool								DistanceFromStartIsBound;
	bool								U0OverrideIsBound;
	bool								U1OverrideIsBound;

	FNiagaraDataSetAccessor<int32>		RibbonIdDataSetAccessor;
	FNiagaraDataSetAccessor<FNiagaraID>	RibbonFullIDDataSetAccessor;
	
	bool								bGpuRibbonLinkIsFloat = false;
	uint32								GpuRibbonLinkOrderOffset = INDEX_NONE;

	FNiagaraRendererLayout RendererLayout;

protected:
	NIAGARA_API void InitBindings();
	NIAGARA_API void SetPreviousBindings(const FVersionedNiagaraEmitterBase& SrcEmitter);

	NIAGARA_API void UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit);

	NIAGARA_API void UpdateMICs();

	virtual bool NeedsMIDsForMaterials() const { return MaterialParameters.HasAnyBindings(); }
};
