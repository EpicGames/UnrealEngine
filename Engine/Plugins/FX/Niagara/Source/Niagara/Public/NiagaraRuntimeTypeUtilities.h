// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraTypes.h"

namespace FNiagaraRuntimeTypeUtilities
{
	struct FTypeConverter
	{
		using FCopyFunc = void(*)(const void*, void*);

		FTypeConverter() = default;
		FTypeConverter(FNiagaraTypeDefinition InTypeDef, FCopyFunc InToNiagara, FCopyFunc InFromNiagara)
			: TypeDef(InTypeDef)
			, ToNiagara(InToNiagara)
			, FromNiagara(InFromNiagara)
		{
		}

		FNiagaraTypeDefinition	TypeDef;
		FCopyFunc				ToNiagara = nullptr;
		FCopyFunc				FromNiagara = nullptr;
	};

	void Initialize();

	NIAGARA_API void RegisterStructTypeConverter(const UScriptStruct* Struct, FTypeConverter Converter);
	NIAGARA_API void UnregisterStructTypeConverter(const UScriptStruct* Struct);

	template<typename TType>
	void RegisterStructTypeConverter(FTypeConverter Converter)
	{
		RegisterStructTypeConverter(StaticStruct<TType>(), Converter);
	}

	template<typename TType>
	void UnregisterStructTypeConverter()
	{
		UnregisterStructTypeConverter(StaticStruct<TType>());
	}

	NIAGARA_API const FTypeConverter* FindTypeConverterForProperty(const FProperty* Property);
}
