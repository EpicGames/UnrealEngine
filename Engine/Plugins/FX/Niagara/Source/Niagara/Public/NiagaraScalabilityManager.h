// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEffectType.h"
#include "NiagaraCommon.h"

#include "NiagaraScalabilityManager.generated.h"

class FNiagaraWorldManager;
class UNiagaraSystem;
class UNiagaraComponent;
class FReferenceCollector;

struct FComponentIterationContext
{
	TArray<int32> SignificanceIndices;
	TBitArray<> ComponentRequiresUpdate;

	int32 MaxUpdateCount = 0;
	float WorstGlobalBudgetUse = 0.0f;

	bool bNewOnly = false;
	bool bProcessAllComponents = false;
	bool bHasDirtyState = false;
	bool bRequiresGlobalSignificancePass = false;
};

/** Working data and cached scalability relevant state for UNiagaraSystems. */
struct FNiagaraScalabilitySystemData
{
	FNiagaraScalabilitySystemData()
	: InstanceCount(0)
	, CullProxyCount(0)
	, bNeedsSignificanceForActiveOrDirty(0)
	, bNeedsSignificanceForCulled(0)
	{}
	uint16 InstanceCount = 0;
	uint16 CullProxyCount = 0;
	
	/** True if we need significance data for active instances of this system. */
	uint16 bNeedsSignificanceForActiveOrDirty : 1;
	/** True if we need significance data for culled instances of this system. */
	uint16 bNeedsSignificanceForCulled : 1;
};

USTRUCT()
struct FNiagaraScalabilityManager
{
	GENERATED_USTRUCT_BODY()

	FNiagaraWorldManager* WorldMan = nullptr;

	UPROPERTY(transient)
	TObjectPtr<UNiagaraEffectType> EffectType;

	UPROPERTY(transient)
	TArray<TObjectPtr<UNiagaraComponent>>  ManagedComponents;

	TArray<FNiagaraScalabilityState> State;

	TMap<UNiagaraSystem*, int32> SystemDataIndexMap;
	TArray<FNiagaraScalabilitySystemData> SystemData;

	double LastUpdateTime;

	bool bRefreshOwnerAllowsScalability = false;

	FNiagaraScalabilityManager(FNiagaraWorldManager* InOwner);
	FNiagaraScalabilityManager();
	~FNiagaraScalabilityManager();
	void Update(float DeltaSeconds, bool bNewOnly);
	void Register(UNiagaraComponent* Component);
	void Unregister(UNiagaraComponent* Component);

	void AddReferencedObjects(FReferenceCollector& Collector);
	void PreGarbageCollectBeginDestroy();

	void OnRefreshOwnerAllowsScalability();

	int32 NumRegisteredComponents()const { return ManagedComponents.Num(); }

#if DEBUG_SCALABILITY_STATE
	void Dump();
#endif

#if WITH_PARTICLE_PERF_CSV_STATS
	void CSVProfilerUpdate(FCsvProfiler* CSVProfiler);
#endif

#if WITH_EDITOR
	void OnSystemPostChange(UNiagaraSystem* System);
#endif//WITH_EDITOR

	void InvalidateCachedSystemData();

private: 
	void UnregisterAt(int32 IndexToRemove);
	bool HasPendingUpdates() const { return DefaultContext.ComponentRequiresUpdate.Num() > 0; }

	void UpdateInternal(FComponentIterationContext& Context);
	bool EvaluateCullState(FComponentIterationContext& Context, int32 ComponentIndex, int32& UpdateCounter);
	void ProcessSignificance(UNiagaraSignificanceHandler* SignificanceHandler, FComponentIterationContext& Context);
	bool ApplyScalabilityState(int32 ComponentIndex, ENiagaraCullReaction CullReaction, bool bNewOnly);

	FNiagaraScalabilitySystemData& GetSystemData(int32 ComponentIndex, bool bForceRefresh=false);

	FComponentIterationContext DefaultContext;

	bool bRefreshCachedSystemData;
};
