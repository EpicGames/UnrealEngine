// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "NiagaraPlatformSet.h"

#include "NativeGameplayTags.h"
#include "GameplayTagContainer.h"
#include "Engine/DataAsset.h"

#include "NiagaraScalabilitySettings.generated.h"

enum class EPSOPrecachePriority : uint8;

NIAGARA_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(TAG_Niagara_ScalabilityProfile_Critical);
NIAGARA_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(TAG_Niagara_ScalabilityProfile_High);
NIAGARA_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(TAG_Niagara_ScalabilityProfile_Medium);
NIAGARA_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(TAG_Niagara_ScalabilityProfile_Low);

UENUM()
enum class ENiagaraSystemPSOPrecachingDelayMode : uint8
{
	/** The Niagara System will delay its rendering until the PSO is ready. Simulation will still occur but will not be rendered until the PSO is ready. */
	DelayRendering,
	/** The Niagara System will delay its activation until the PSO is ready. No simulation or rendering will occur until the PSO is ready. */
	DelayActivation,
};

UENUM()
enum class ENiagaraEmitterImportance : uint8
{
	/** The emitter is critical for the system and blocks both simulation and rendering if it is not fully loaded. */
	Critical = 0,
	/** The emitter has default importance and uses the readiness settings of the system/effect type for rendering and simulation delay. */
	Normal = 1,
	/** The system is allowed to simulate and render even if this emitter is not yet ready. Can cause pop-ins if a renderer material has finished loading. */
	Low = 2,
};

UENUM()
enum class ENiagaraSystemPSOPrecachingPriorityBoost : uint8
{
	/** Niagara will boost the relevant PSO requests to highest priority */
	Highest,
	/** Niagara will boost the relevant PSO requests to high priority */
	High,
};

UENUM()
enum class ENiagaraPSOPrecachingTime : uint8
{
	/** Do not precache PSOs for this Niagara Component/Emitter. */
	None,
	/** Niagara will precache their PSOs when the Component/Emitter is initialized. */
	OnInitialize,
	/** Niagara will precache their PSOs when the Component/Emitter is activated. This is implied if the delay mode is set to DelayActivation. */
	OnActivated,
	/** Niagara will precache their PSOs when the Component/Emitter render state is created. */
	OnRenderStateCreation,
};

USTRUCT()
struct FNiagaraSettingsProfile
{
	GENERATED_BODY()
};

/** A base asset class for Niagara Settings Profile Sets. */
UCLASS()
class UNiagaraSettingsProfileSet : public UDataAsset
{
	GENERATED_BODY()
public:

	/** Handle any required changes to cached data after a scalability settings update. */
	virtual void OnScalabilityUpdate() {}
};


/** Settings controlling how Niagara Systems can be delayed based on their PSO readiness. */
USTRUCT()
struct FNiagaraSystemPSOPrecacheSettings
{
	GENERATED_BODY()

	/** How should this Niagara System be delayed to wait for its PSO, if at all. */
	UPROPERTY(EditAnywhere, Category = "PSO Precache Settings")
	ENiagaraSystemPSOPrecachingDelayMode DelayMode = ENiagaraSystemPSOPrecachingDelayMode::DelayRendering;

	/** The PSO pre caching priority boost when the FX are used and potentially delayed. */
	UPROPERTY(EditAnywhere, Category = "PSO Precache Settings")
	ENiagaraSystemPSOPrecachingPriorityBoost PreCachePriorityBoost = ENiagaraSystemPSOPrecachingPriorityBoost::Highest;

	/** Determines when a Niagara FX should precache its PSOs. */
	UPROPERTY(EditAnywhere, Category = "PSO Precache Settings")
	ENiagaraPSOPrecachingTime PrecachingTime = ENiagaraPSOPrecachingTime::OnInitialize;

	/** If true, Niagara Systems will precache their PSOs on load. */
	UPROPERTY(EditAnywhere, Category = "PSO Precache Settings")
	uint8 bCacheOnLoad : 1 = false;

	EPSOPrecachePriority GetPriorityBoost() const;
};

/** A settings profile containing Readiness settings. */
USTRUCT()
struct FNiagaraReadinessSettings : public FNiagaraSettingsProfile
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Scalabiltiy")
	FNiagaraPlatformSet PlatformSet;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FNiagaraSystemPSOPrecacheSettings PSOPrecacheSettings;
};

/** Base class for settings objects that can be customized per Platform Set. */
USTRUCT()
struct FNiagaraScalabiltiySettingsProfile
{
	GENERATED_BODY()

private:
	mutable int32 CachedPlatformSetIndex = INDEX_NONE;

public:
	void OnScalabilityUpdate()
	{
		//Invalidate our cached index so we'll regenerate on next access.
		CachedPlatformSetIndex = INDEX_NONE;
	}

protected:
	template <typename T>
	const T& GetInternal(const TArray<T>& Settings)const
	{		
		if (Settings.IsValidIndex(CachedPlatformSetIndex))
		{
			return Settings[CachedPlatformSetIndex];
		}

		CachedPlatformSetIndex = INDEX_NONE;
		for (int32 i = 0; i < Settings.Num(); ++i)
		{
			const T& PlatformSettings = Settings[i];
			if (PlatformSettings.PlatformSet.IsActive())
			{
				CachedPlatformSetIndex = i;
				return PlatformSettings;
			}
		}
		static T Fallback;
		return Fallback;
	}
};

/** Profile containing readiness setings. Customizable by platform. */
USTRUCT()
struct FNiagaraReadinessSettingsProfile : public FNiagaraScalabiltiySettingsProfile
{
	GENERATED_BODY()

public:
	
	/** Get the Readiness settings for the current platform. */
	const FNiagaraReadinessSettings& Get()const { return GetInternal(Settings); }

protected:
	/** Readiness separated by Platform Set. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (NiagaraOrthogonalPlatformSets))
	TArray<FNiagaraReadinessSettings> Settings;
};


/** Profiles defining how Niagara Systems can be delayed when waiting for their prerequisite data to be ready. Also controls priorities for loading/building such data. */
UCLASS(BlueprintType)
class UNiagaraSystemReadinessSettings : public UNiagaraSettingsProfileSet
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	// UObject interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif

	const FNiagaraReadinessSettingsProfile* Find(FGameplayTag Profile)const;

	/** Handle any required changes to cached data after a scalability settings update. */
	virtual void OnScalabilityUpdate()override;

protected:
	
	/** Parent profile settings. If a user cannot find a requested profile in these settings, we will look up into the parent. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TObjectPtr<UNiagaraSystemReadinessSettings> Parent;

	/** Set of named profiles for settings controlling Readiness. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (Categories = "Niagara.ScalabilityProfile"))
	TMap<FGameplayTag, FNiagaraReadinessSettingsProfile> Profiles;
};

