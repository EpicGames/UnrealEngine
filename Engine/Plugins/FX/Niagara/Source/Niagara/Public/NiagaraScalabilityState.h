// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDefines.h"

#include "NiagaraScalabilityState.generated.h"

USTRUCT()
struct FNiagaraScalabilityState
{
	GENERATED_BODY()

	FNiagaraScalabilityState()
		: Significance(1.0f)
		, LastVisibleTime(0.0f)
		, SystemDataIndex(INDEX_NONE)
		, bNewlyRegistered(1)
		, bNewlyRegisteredDirty(0)
		, bCulled(0)
		, bPreviousCulled(0)
		, bCulledByDistance(0)
		, bCulledByInstanceCount(0)
		, bCulledByVisibility(0)
		, bCulledByGlobalBudget(0)
	{
	}

	FNiagaraScalabilityState(float InSignificance, bool InCulled, bool InPreviousCulled)
		: Significance(InSignificance)
		, LastVisibleTime(0.0f)
		, SystemDataIndex(INDEX_NONE)
		, bNewlyRegistered(1)
		, bNewlyRegisteredDirty(0)
		, bCulled(InCulled)
		, bPreviousCulled(InPreviousCulled)
		, bCulledByDistance(0)
		, bCulledByInstanceCount(0)
		, bCulledByVisibility(0)
		, bCulledByGlobalBudget(0)
	{
	}

	bool IsDirty() const { return bCulled != bPreviousCulled; }
	void Apply() { bPreviousCulled = bCulled; }

	UPROPERTY(VisibleAnywhere, Category="Scalability")
	float Significance;

	UPROPERTY(VisibleAnywhere, Category = "Scalability")
	float LastVisibleTime;

	int16 SystemDataIndex;

	UPROPERTY(VisibleAnywhere, Category = "Scalability")
	uint8 bNewlyRegistered : 1;

	UPROPERTY(VisibleAnywhere, Category = "Scalability")
	uint8 bNewlyRegisteredDirty : 1;

	UPROPERTY(VisibleAnywhere, Category = "Scalability")
	uint8 bCulled : 1;

	UPROPERTY(VisibleAnywhere, Category="Scalability")
	uint8 bPreviousCulled : 1;

	UPROPERTY(VisibleAnywhere, Category="Scalability")
	uint8 bCulledByDistance : 1;

	UPROPERTY(VisibleAnywhere, Category = "Scalability")
	uint8 bCulledByInstanceCount : 1;

	UPROPERTY(VisibleAnywhere, Category = "Scalability")
	uint8 bCulledByVisibility : 1;
	
	UPROPERTY(VisibleAnywhere, Category = "Scalability")
	uint8 bCulledByGlobalBudget : 1;
};

#if WITH_NIAGARA_DEBUGGER
/** 
Debug tracking for scalability culling that kills systems rather than having them sleep.
Displayed in the debug hud per system to help track down culling issues.
*/
struct FNiagaraSystemScalabilityKillCounts
{
	int32 Total = 0;
	int32 Distance = 0;
	int32 InstanceCounts = 0;
	int32 Visibility = 0;
	int32 GlobalBudget = 0;

	inline void Reset()
	{
		Total = 0;
		Distance = 0;
		InstanceCounts = 0;
		Visibility = 0;
		GlobalBudget = 0;
	}

	inline void Add(const FNiagaraScalabilityState& State)
	{
		if (State.bCulled)
		{
			++Total;
			Distance += State.bCulledByDistance ? 1 : 0;
			InstanceCounts += State.bCulledByInstanceCount ? 1 : 0;
			Visibility += State.bCulledByVisibility ? 1 : 0;
			GlobalBudget += State.bCulledByGlobalBudget ? 1 : 0;
		}
	}

	inline void Add(const FNiagaraSystemScalabilityKillCounts& InCounts)
	{
		Total += InCounts.Total;
		Distance += InCounts.Distance;
		InstanceCounts += InCounts.InstanceCounts;
		Visibility += InCounts.Visibility;
		GlobalBudget += InCounts.GlobalBudget;
	}
};
#endif