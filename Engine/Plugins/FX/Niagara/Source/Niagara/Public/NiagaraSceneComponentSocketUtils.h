// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "UObject/Object.h"
#include "Engine/World.h"

#include "NiagaraComponent.h"
#include "NiagaraSystemInstance.h"

#include "Components/SceneComponent.h"
#include "Engine/Canvas.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSocket.h"

// Utilities to help reading sockets
// DO NOT USE THIS IN EXTERNAL CODE as it is subject to change
class FNiagaraSceneComponentSocketUtils
{
public:
	FNiagaraSceneComponentSocketUtils(TWeakObjectPtr<UObject> InWeakOwnerComponent)
		: WeakOwnerComponent(InWeakOwnerComponent)
	{
	}
	virtual ~FNiagaraSceneComponentSocketUtils() = default;

	// Enable resolving via a parameter binding
	void SetAllowParameterBinding(FNiagaraSystemInstance* SystemInstance, const FNiagaraVariable& BoundParameter)
	{
		ParameterBinding.Init(SystemInstance->GetInstanceParameters(), BoundParameter);
		bTryResolveBinding = ParameterBinding.IsBound();
	}

	// Enable resolving via attach parent
	void SetAllowAttachParent(UClass* ComponentClass, FName ComponentTag)
	{
		bTryResolveAttachParent = true;
		WeakAttachComponentClass = ComponentClass;
		AttachComponentTag = ComponentTag;
	}

	// Enable resolving via a source object
	void SetAllowSource(TLazyObjectPtr<UObject> InSourceActor, UObject* InSourceAsset)
	{
		SourceActor = InSourceActor;
		SourceAsset = InSourceAsset;
		bTryResolveSource = true;
	}

#if WITH_EDITOR
	// Enable resolving via an editor only asset
	void SetEditorPreviewAsset(TSoftObjectPtr<UObject>	InEditorPreviewAsset)
	{
		EditorPreviewAsset = InEditorPreviewAsset;
	}
#endif

	// Are we bound to something valid?
	bool IsValid() const
	{
		return WeakResolvedObject.Get() != nullptr;
	}

	// Resolve the object we are bound to
	// Returns true if the object changed, otherwise false
	bool ResolveObject()
	{
		UObject* ResolvedObject = nullptr;

		// Resolve using the parameter binding?
		if (bTryResolveBinding)
		{
			ResolvedObject = ParameterBinding.GetValue();
			if (ResolvedObject)
			{
				if (AActor* AsActor = Cast<AActor>(ResolvedObject))
				{
					ResolvedObject = AsActor->GetRootComponent();
				}
			}
		}

		// Resolve using attach parent?
		if (bTryResolveAttachParent && !ResolvedObject)
		{
			ResolvedObject = ResolveAttachParentInternal();
		}

		// Resolve Source?
		if (bTryResolveSource && !ResolvedObject)
		{
			ResolvedObject = ResolveSourceInternal();
		}

#if WITH_EDITOR
		// In editor fallback to the editor preview asset
		if (!ResolvedObject)
		{
			USceneComponent* OwnerComponent = Cast<USceneComponent>(WeakOwnerComponent.Get());
			UWorld* World = OwnerComponent ? OwnerComponent->GetWorld() : nullptr;
			if (World && !World->IsGameWorld())
			{
				ResolvedObject = EditorPreviewAsset.LoadSynchronous();
			}
		}
#endif

		if (WeakResolvedObject.Get() == ResolvedObject)
		{
#if WITH_EDITOR
			// When in the editor the socket counts can change so we might need to recache the socket list
			TArray<FName> PrevSocketNames;
			Swap(PrevSocketNames, SocketNames);
			CacheSocketNames(ResolvedObject);

			if (PrevSocketNames != SocketNames)
			{
				return true;
			}
#endif

			return false;
		}

		WeakResolvedObject = ResolvedObject;
		CacheSocketNames(ResolvedObject);

		return true;
	}

	// Get the number of sockets available
	int32 GetNumSockets() const
	{
		return SocketNames.Num();
	}

	// Get all of the available socket names
	virtual TConstArrayView<FName> GetSocketNames() const
	{
		return MakeArrayView(SocketNames);
	}

	// Get socket transforms
	// Optionally reading from only SocketIndices
	// Returns true if we read any data, otherwise false
	bool GetSocketTransforms(TArray<FTransform3f>& OutSocketTransforms, TOptional<TConstArrayView<int32>> SocketIndices = {}) const
	{
		UObject* ResolvedObject = WeakResolvedObject.Get();

		if (GetSocketTransformsInternal(ResolvedObject, OutSocketTransforms, SocketIndices))
		{
			return true;
		}

		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(ResolvedObject))
		{
			GetSocketTransformsStaticMesh(StaticMesh, OutSocketTransforms, SocketIndices);
			return true;
		}

		if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(ResolvedObject))
		{
			GetSocketTransformsSkeletalMesh(SkeletalMesh, OutSocketTransforms, SocketIndices);
			return true;
		}

		return false;
	}

	// Get the current local to world for the socket owner
	virtual FTransform GetLocalToWorldTransform() const = 0;

	// Get the resolve object
	// Note: Do not make any assumptions about where this comes from
	UObject* GetResolvedObject() const
	{
		return WeakResolvedObject.Get();
	}

protected:
	void CacheSocketNames(UObject* ResolvedObject)
	{
		SocketNames.Reset();
		if (CacheSocketNamesInternal(ResolvedObject))
		{
			return;
		}
		
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(ResolvedObject))
		{
			CacheSocketNamesStaticMesh(StaticMesh);
			return;
		}
		
		if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(ResolvedObject))
		{
			CacheSocketNamesSkeletalMesh(SkeletalMesh);
			return;
		}
		SocketNames.Empty();
	}

	void CacheSocketNamesStaticMesh(UStaticMesh* StaticMesh)
	{
		const int32 NumSockets = StaticMesh->Sockets.Num();
		SocketNames.Reserve(StaticMesh->Sockets.Num());
		for (UStaticMeshSocket* Socket : StaticMesh->Sockets)
		{
			check(Socket);
			SocketNames.Add(Socket->SocketName);
		}
	}

	void CacheSocketNamesSkeletalMesh(USkeletalMesh* SkeletalMesh)
	{
		const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
		const TArray<FMeshBoneInfo>& RefBoneInfo = RefSkeleton.GetRefBoneInfo();
		const int32 NumBones = RefBoneInfo.Num();
		const int32 NumSockets = SkeletalMesh->NumSockets();

		SocketNames.Reserve(NumBones + NumSockets);
		for (int i = 0; i < NumBones; ++i)
		{
			SocketNames.Add(RefBoneInfo[i].Name);
		}

		for (int i = 0; i < NumSockets; ++i)
		{
			USkeletalMeshSocket* Socket = SkeletalMesh->GetSocketByIndex(i);
			check(Socket);
			SocketNames.Add(Socket->SocketName);
		}
	}

	void GetSocketTransformsStaticMesh(UStaticMesh* StaticMesh, TArray<FTransform3f>& OutSocketTransforms, TOptional<TConstArrayView<int32>> SocketIndices) const
	{
		const bool bIndirect = SocketIndices.IsSet();
		const int32 NumIndicies = bIndirect ? SocketIndices.GetValue().Num() : SocketNames.Num();
		for (int iIndex = 0; iIndex < NumIndicies; ++iIndex)
		{
			const int32 iSocketIndex = bIndirect ? SocketIndices.GetValue()[iIndex] : iIndex;
			if (SocketNames.IsValidIndex(iSocketIndex) && StaticMesh->Sockets.IsValidIndex(iSocketIndex))
			{
				UStaticMeshSocket* Socket = StaticMesh->Sockets[iSocketIndex];
				OutSocketTransforms[iSocketIndex].SetTranslation(FVector3f(Socket->RelativeLocation));
				OutSocketTransforms[iSocketIndex].SetRotation(FQuat4f(Socket->RelativeRotation.Quaternion()));
				OutSocketTransforms[iSocketIndex].SetScale3D(FVector3f(Socket->RelativeScale));
			}
		}
	}
	
	void GetSocketTransformsSkeletalMesh(USkeletalMesh* SkeletalMesh, TArray<FTransform3f>& OutSocketTransforms, TOptional<TConstArrayView<int32>> SocketIndices) const
	{
		// For SKM we read everything all the time, as there is a hierarchy to the data that needs to be resolved
		const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
		const TArray<FMeshBoneInfo>& RefBoneInfo = RefSkeleton.GetRefBoneInfo();
		const TArray<FTransform>& RefBonePose = RefSkeleton.GetRefBonePose();
		const int32 NumBones = RefBoneInfo.Num();
		const int32 NumSockets = SkeletalMesh->NumSockets();

		for (int i = 0; i < NumBones; ++i)
		{
			FTransform3f SocketTransform = FTransform3f(RefBonePose[i]);
			const int32 ParentIndex = RefBoneInfo[i].ParentIndex;
			if (ParentIndex >= 0 && ParentIndex < i)
			{
				SocketTransform = SocketTransform * OutSocketTransforms[ParentIndex];
			}
			OutSocketTransforms[i] = FTransform3f(SocketTransform);
		}

		for (int i = 0; i < NumSockets; ++i)
		{
			USkeletalMeshSocket* Socket = SkeletalMesh->GetSocketByIndex(i);
			check(Socket);

			FTransform3f SocketTransform(FTransform(Socket->RelativeRotation.Quaternion(), Socket->RelativeLocation, Socket->RelativeScale));
			const int32 ParentIndex = RefSkeleton.FindBoneIndex(Socket->BoneName);
			if (ParentIndex >= 0)
			{
				SocketTransform = SocketTransform * OutSocketTransforms[ParentIndex];
			}
			OutSocketTransforms[i + NumBones] = FTransform3f(SocketTransform);
		}
	}

	// Resolve from attach parent Actor / Component / Entity
	virtual UObject* ResolveAttachParentInternal() const = 0;
	// Resolve from source Actor / Component / Entity
	virtual UObject* ResolveSourceInternal() const = 0;

	// Cache socket names from Actor / Component / Entity if valid
	virtual bool CacheSocketNamesInternal(UObject* ResolvedObject) = 0;
	// Read socket transforms from Actor / Component / Entity if valid
	virtual bool GetSocketTransformsInternal(UObject* ResolvedObject, TArray<FTransform3f>& OutSocketTransforms, TOptional<TConstArrayView<int32>> SocketIndices) const = 0;

protected:
	bool										bTryResolveBinding = false;
	bool										bTryResolveAttachParent = false;
	bool										bTryResolveSource = false;

	TWeakObjectPtr<UObject>						WeakOwnerComponent;
	TWeakObjectPtr<UObject>						WeakResolvedObject;

	FNiagaraParameterDirectBinding<UObject*>	ParameterBinding;

	TWeakObjectPtr<UClass>						WeakAttachComponentClass;
	FName										AttachComponentTag;

	TLazyObjectPtr<UObject>						SourceActor;
	TWeakObjectPtr<UObject>						SourceAsset;

#if WITH_EDITOR
	TSoftObjectPtr<UObject>						EditorPreviewAsset;
#endif

	TArray<FName>								SocketNames;
};
