// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Components/SceneComponent.h"
#include "UObject/Object.h"
#include "PrimitiveComponentId.h"

class FColorVertexBuffer;
class UNiagaraComponent;
class UStaticMesh;

class FNiagaraSceneComponentSocketUtils;

// Helper class to abstract how we need to search the scene for components in various data interfaces
// This is temporary until scene graph is folded into the core engine, or we have more official APIs to abstract Actors/Entity/Desc
// DO NOT USE THIS IN EXTERNAL CODE as it is subject to change
class INiagaraSceneComponentUtils
{
public:
	virtual ~INiagaraSceneComponentUtils() = default;

	// Gets the underlying owner object
	virtual UObject* GetOwner() const = 0;
	// Finds a property interface for the provided source object
	virtual UClass* FindPropertyInterface(UObject* SourceObject, FName InterfaceFullPath) const = 0;

	// Resolve the static mesh from what the interface's owner object
	virtual void ResolveStaticMesh(bool bRecurseParents, UObject*& OutComponent, UStaticMesh*& OutStaticMesh) const = 0;
	// Resolve the static mesh from the provided object which could be a static mesh / component / actor / entity / etc
	virtual void ResolveStaticMesh(UObject* ObjectFrom, bool bRecurseParents, UObject*& OutComponent, UStaticMesh*& OutStaticMesh) const = 0;
	// Get the component transform and any ISM instance transforms
	virtual bool GetStaticMeshTransforms(UObject* Component, FTransform& OutComponentTransform, TArray<FTransform>& OutInstanceTransforms) const = 0;
	// Get the static mesh override vertex colors, if any
	virtual FColorVertexBuffer* GetStaticMeshOverrideColors(UObject* Component, int32 LODIndex) const = 0;

	// Get the FPrimitiveComponentId for the provided component
	virtual FPrimitiveComponentId GetPrimitiveSceneId(UObject* Component) const = 0;
	// Get the physics lineary velociy for the provided component
	virtual FVector GetPhysicsLinearVelocity(UObject* Component) const = 0;

	// Create a socket utils
	virtual FNiagaraSceneComponentSocketUtils* CreateSocketUtils() const = 0;
};

// Implementation for AActor & UActorComponents
class FNiagaraActorSceneComponentUtils final : public INiagaraSceneComponentUtils
{
public:
	explicit FNiagaraActorSceneComponentUtils(UNiagaraComponent* OwnerComponent);

	virtual UObject* GetOwner() const override;
	virtual UClass* FindPropertyInterface(UObject* SourceObject, FName InterfaceFullPath) const override;

	virtual void ResolveStaticMesh(bool bRecurseParents, UObject*& OutComponent, UStaticMesh*& OutStaticMesh) const override;
	virtual void ResolveStaticMesh(UObject* ObjectFrom, bool bRecurseParents, UObject*& OutComponent, UStaticMesh*& OutStaticMesh) const override;
	virtual bool GetStaticMeshTransforms(UObject* Component, FTransform& OutComponentTransform, TArray<FTransform>& OutInstanceTransforms) const override;
	virtual FColorVertexBuffer* GetStaticMeshOverrideColors(UObject* Component, int32 LODIndex) const override;

	virtual FPrimitiveComponentId GetPrimitiveSceneId(UObject* Component) const override;
	virtual FVector GetPhysicsLinearVelocity(UObject* Component) const override;

	virtual FNiagaraSceneComponentSocketUtils* CreateSocketUtils() const override;

private:
	TWeakObjectPtr<USceneComponent> WeakOwnerComponent;
};
